// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using UnrealBuildTool;

public class CoreUObject : ModuleRules
{
	public CoreUObject(ReadOnlyTargetRules Target) : base(Target)
	{
		// Autogenerate headers for our bytecode.
		GenerateHeaderFuncs.Add(("VerseVMBytecode", VerseVMBytecodeGenerator.Generate));
		// Include fewer headers in .generated.h files to reduce circular dependency issues within CoreUObject
		bMinimizeGeneratedIncludes = true;

		PrivatePCHHeaderFile = "Private/CoreUObjectPrivatePCH.h";

		SharedPCHHeaderFile = "Public/CoreUObjectSharedPCH.h";

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
				"TargetPlatform",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"TraceLog",
				"CorePreciseFP",
				"Json",
			}
		);

		SetupVerse();

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"AutoRTFM",
				"Projects",
			}
		);

		if (Target.GlobalDefinitions.Contains("UE_WITH_REMOTE_OBJECT_HANDLE=1"))
		{
			// todo this is a bit hacky, we should have something like Target.bDstmEnabled
			PrivateDependencyModuleNames.Add("libpas");
		}

		if (Target.bBuildWithEditorOnlyData || Target.Type == TargetType.Server)
		{
			PublicDefinitions.Add("WITH_VERSE_COMPILER=1");
		}
		else
		{
			PublicDefinitions.Add("WITH_VERSE_COMPILER=0");
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		if (Target.bBuildWithEditorOnlyData && Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop))
		{
			PrivateDependencyModuleNames.Add("DerivedDataCache");
		}

		if (Target.bCompileAgainstApplicationCore)
		{
			PrivateIncludePathModuleNames.Add("ApplicationCore");
		}

		PrivateDefinitions.Add("UE_DEFINE_LEGACY_MATH_CONSTANT_MACRO_NAMES=0");
	}
}
