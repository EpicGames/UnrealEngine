// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Serialization/FileRegions.h"
#include "Serialization/LargeMemoryWriter.h"

template<typename ArchiveType>
class TFileRegionArchive
	: public ArchiveType
{
public:
	using ArchiveType::ArchiveType;
	using ArchiveType::Tell;

	const TArray<FFileRegion>& GetFileRegions() const
	{
		return FileRegions;
	}

	virtual void PushFileRegionType(EFileRegionType Type) override final
	{
		check(CurrentRegion.Offset == 0);
		CurrentRegion.Offset = Tell();
		CurrentRegion.Type = Type;
	}

	virtual void PopFileRegionType() override final
	{
		CurrentRegion.Length = Tell() - CurrentRegion.Offset;
		FileRegions.Add(CurrentRegion);
		CurrentRegion = FFileRegion();
	}

private:
	TArray<FFileRegion> FileRegions;
	FFileRegion CurrentRegion;
};

using FFileRegionMemoryWriter = TFileRegionArchive<FLargeMemoryWriter>;
