// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/PackageStore.h"

class FPackageStoreInternal
{
public:
	static void IterateBackends(FPackageStore& PackageStore, FVisitPackageStoreBackend&& Visit)
	{
		PackageStore.IterateBackends(MoveTemp(Visit));
	}

	static void IterateBackends(FVisitPackageStoreBackend&& Visit)
	{
		IterateBackends(FPackageStore::Get(), MoveTemp(Visit));
	}
};
