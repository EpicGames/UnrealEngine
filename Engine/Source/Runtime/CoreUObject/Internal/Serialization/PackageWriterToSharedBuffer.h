// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Memory/SharedBuffer.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/BasePackageWriter.h"
#include "Templates/UniquePtr.h"

class FPackageWriterRecords
{
public:
	struct FPackage;

	COREUOBJECT_API void BeginPackage(FPackage* Record, const IPackageWriter::FBeginPackageInfo& Info);
	COREUOBJECT_API void WritePackageData(const IPackageWriter::FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive,
		const TArray<FFileRegion>& FileRegions);
	COREUOBJECT_API void WriteBulkData(const IPackageWriter::FBulkDataInfo& Info, const FIoBuffer& BulkData,
		const TArray<FFileRegion>& FileRegions);
	COREUOBJECT_API void WriteAdditionalFile(const IPackageWriter::FAdditionalFileInfo& Info,
		const FIoBuffer& FileData);
	COREUOBJECT_API void WriteLinkerAdditionalData(const IPackageWriter::FLinkerAdditionalDataInfo& Info,
		const FIoBuffer& Data, const TArray<FFileRegion>& FileRegions);
	COREUOBJECT_API void WritePackageTrailer(const IPackageWriter::FPackageTrailerInfo& Info, const FIoBuffer& Data);

	struct FWritePackage
	{
		IPackageWriter::FPackageInfo Info;
		FSharedBuffer Buffer;
		TArray<FFileRegion> Regions;
	};
	struct FBulkData
	{
		IPackageWriter::FBulkDataInfo Info;
		FSharedBuffer Buffer;
		TArray<FFileRegion> Regions;
	};
	struct FAdditionalFile
	{
		IPackageWriter::FAdditionalFileInfo Info;
		FSharedBuffer Buffer;
	};
	struct FLinkerAdditionalData
	{
		IPackageWriter::FLinkerAdditionalDataInfo Info;
		FSharedBuffer Buffer;
		TArray<FFileRegion> Regions;
	};
	struct FPackageTrailer
	{
		IPackageWriter::FPackageTrailerInfo Info;
		FSharedBuffer Buffer;
	};
	struct FPackage
	{
		/** TPackageWriterToSharedBuffer can allocate FPackage subclasses, so the base needs a virtual destructor. */
		virtual ~FPackage() = default;

		/** Always valid during CommitPackageInternal */
		IPackageWriter::FBeginPackageInfo Begin;
		/** Always valid during CommitPackageInternal if Info.Status == Success */
		TArray<FWritePackage> Packages;
		TArray<FBulkData> BulkDatas;
		TArray<FAdditionalFile> AdditionalFiles;
		TArray<FLinkerAdditionalData> LinkerAdditionalDatas;
		TArray<FPackageTrailer> PackageTrailers;
	};

	/** Get the Record created by BeginPackage for the given PackageName; assert that it is valid */
	COREUOBJECT_API FPackage& FindRecordChecked(FName InPackageName) const;
	/** Get the Record created by BeginPackage and remove it; assert that it is valid */
	COREUOBJECT_API TUniquePtr<FPackage> FindAndRemoveRecordChecked(FName InPackageName);
	/** Verify records from all Write functions are valid, and the required ones are present */
	COREUOBJECT_API void ValidateCommit(FPackage& Record, const IPackageWriter::FCommitPackageInfo& Info) const;
	
private:
	mutable FRWLock MapLock;
	TMap<FName, TUniquePtr<FPackage>> Map;
};

/**
 * A base class for IPackageWriter subclasses that writes to records that are read in CommitPackage.
 * To avoid diamond inheritance, this class specifies the base class for the interface it is implementing (either
 * FBasePackageWriter for IPackageWriter or FBaseCookedPackageWriter for ICookedPackageWriter) by template. Subclasses
 * should derive from one of
 *     TPackageWriterToSharedBuffer<FBasePackageWriter>
 *     TPackageWriterToSharedBuffer<FBaseCookedPackageWriter>.
 */
template <typename BaseClass>
class TPackageWriterToSharedBuffer : public BaseClass
{
public:
	using FWritePackageRecord = FPackageWriterRecords::FWritePackage;
	using FBulkDataRecord = FPackageWriterRecords::FBulkData;
	using FAdditionalFileRecord = FPackageWriterRecords::FAdditionalFile;
	using FLinkerAdditionalDataRecord = FPackageWriterRecords::FLinkerAdditionalData;
	using FPackageTrailerRecord = FPackageWriterRecords::FPackageTrailer;
	using FPackageRecord = FPackageWriterRecords::FPackage;

	virtual void BeginPackage(const IPackageWriter::FBeginPackageInfo& Info) override
	{
		Records.BeginPackage(ConstructRecord(), Info);
	}
	virtual void WritePackageData(const IPackageWriter::FPackageInfo& Info, FLargeMemoryWriter& ExportsArchive,
		const TArray<FFileRegion>& FileRegions) override
	{
		Records.WritePackageData(Info, ExportsArchive, FileRegions);
	}
	virtual void WriteBulkData(const IPackageWriter::FBulkDataInfo& Info, const FIoBuffer& BulkData,
		const TArray<FFileRegion>& FileRegions) override
	{
		Records.WriteBulkData(Info, BulkData, FileRegions);
	}
	virtual void WriteAdditionalFile(const IPackageWriter::FAdditionalFileInfo& Info,
		const FIoBuffer& FileData) override
	{
		Records.WriteAdditionalFile(Info, FileData);
	}
	virtual void WriteLinkerAdditionalData(const IPackageWriter::FLinkerAdditionalDataInfo& Info,
		const FIoBuffer& Data, const TArray<FFileRegion>& FileRegions) override
	{
		Records.WriteLinkerAdditionalData(Info, Data, FileRegions);
	}
	virtual void WritePackageTrailer(const IPackageWriter::FPackageTrailerInfo& Info, const FIoBuffer& Data) override
	{
		Records.WritePackageTrailer(Info, Data);
	}
	virtual void CommitPackage(IPackageWriter::FCommitPackageInfo&& Info) override
	{
		TUniquePtr<FPackageRecord> Record = Records.FindAndRemoveRecordChecked(Info.PackageName);
		ValidateCommit(*Record, Info);
		CommitPackageInternal(MoveTemp(*Record), Info);
	}

protected:
	virtual void CommitPackageInternal(FPackageRecord&& Record,
		const IPackageWriter::FCommitPackageInfo& Info) = 0;

protected:
	/** Construct a record for the package, called during BeginPackage. Can be subclass of FPackage. */
	virtual FPackageWriterRecords::FPackage* ConstructRecord()
	{
		return new FPackageWriterRecords::FPackage();
	}

	/** Get the Record created by BeginPackage for the given PackageName; assert that it is valid */
	FPackageRecord& FindRecordChecked(FName InPackageName)
	{
		return Records.FindRecordChecked(InPackageName);
	}
	/** Verify records from all Write functions are valid, and the required ones are present */
	void ValidateCommit(FPackageRecord& Record, const IPackageWriter::FCommitPackageInfo& Info)
	{
		Records.ValidateCommit(Record, Info);
	}

	FPackageWriterRecords Records;
};

/** A PackageWriter that saves the portions of the SavePackage output to sharedbuffers on a record structure. */
class FPackageWriterToRecord : public TPackageWriterToSharedBuffer<FBasePackageWriter>
{
public:
	FPackageWriterRecords::FPackage SavedRecord;

protected:
	virtual void CommitPackageInternal(FPackageWriterRecords::FPackage&& Record,
		const FCommitPackageInfo& Info) override
	{
		SavedRecord = MoveTemp(Record);
	}
};
