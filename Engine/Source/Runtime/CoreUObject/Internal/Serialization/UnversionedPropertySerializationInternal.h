// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class ITargetPlatform;

// Check if unversioned property serialization is configured to be used on target platform
COREUOBJECT_API bool CanUseUnversionedPropertySerialization(const ITargetPlatform* Target);
