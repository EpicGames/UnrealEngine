// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "IO/PackageId.h"
#include "Memory/MemoryFwd.h"
#include "Serialization/AsyncLoading2.h"
#include "Serialization/MappedName.h"
#include "UObject/NameTypes.h"

struct FZenPackageHeader
{
	uint32 ExportCount = 0; // Need to keep this count around after ExportMap is cleared
	TOptional<FZenPackageVersioningInfo> VersioningInfo;
	FNameMap NameMap;
	FName PackageName;

	// Backed by IoBuffer
	const FZenPackageSummary* PackageSummary = nullptr;
	TArrayView<const uint64> ImportedPublicExportHashes;
	TArrayView<const FPackageObjectIndex> ImportMap;
	TArrayView<const FExportMapEntry> ExportMap;
	TArrayView<const FPackageObjectIndex> CellImportMap;
	TArrayView<const FCellExportMapEntry> CellExportMap;
	TArrayView<const FBulkDataMapEntry> BulkDataMap;
	TArrayView<const FExportBundleEntry> ExportBundleEntries;
	TArrayView<const FDependencyBundleHeader> DependencyBundleHeaders;
	TArrayView<const FDependencyBundleEntry> DependencyBundleEntries;

	TArray<FName> ImportedPackageNames;

	COREUOBJECT_API static FZenPackageHeader MakeView(FMemoryView Memory);
	COREUOBJECT_API static FZenPackageHeader MakeView(FMemoryView Memory, FString& OutError);
	COREUOBJECT_API void Reset();
};
