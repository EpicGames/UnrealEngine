// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "RedirectionSummary.h"
#include "UObject/CoreRedirects.h"

#define UE_API COREUOBJECT_API

/**
* Stores state required for FCoreRedirects API. The context encapsulates the necessary state for 
* the FCoreRedirects API to function. Changine the context can be used to change the set of
* CoreRedirects for the current thread without affecting other threads. It is up to the user to ensure
* contexts are shared or not between threads. By default a single global context is shared amoung all
* threads until a thread local context is set.
*/
struct FCoreRedirectsContext
{
	enum class EFlags : uint32
	{
		None						= 0,
		DebugMode					= 1 << 0,	// Enables extra validation and logging
		ValidateAddedRedirects		= 1 << 1,	// Validates CoreRedirects are well-formed before addition
		UseRedirectionSummary		= 1 << 2,   // New redirects will be appended to the RedirectionSummary

		Default = ValidateAddedRedirects | UseRedirectionSummary
	};

	/*
	* Returns the global context used by FCoreRedirects when no new context has been provided as an override.
	*/
	static UE_API FCoreRedirectsContext& GetGlobalContext();

	/*
	* Returns the current context used by FCoreRedirects for the current thread. If no new context has been applied,
	* a global context is used for all threads.
	*/
	static UE_API FCoreRedirectsContext& GetThreadContext();

	/*
	* Changes the context for FCoreRedirects calls to use for the current thread. The passed in context must remain
	* valid until PopThreadContext is called.
	*/
	static UE_API void SetThreadContext(FCoreRedirectsContext& NewContext);

	/**
	 * Run initialization steps that are needed before any data can be stored from FCoreRedirects calls.
	 */
	UE_API void InitializeContext();

	/** Creates a context with a debug context name */
	UE_API FCoreRedirectsContext();

	/**
	* Creates a  context with a debug context name, and inherits the state from OtherContext.
	* The state from OtherContext is copied into the new context
	*/
	UE_API FCoreRedirectsContext(const FCoreRedirectsContext& OtherContext);

	/** Get whether this has been initialized */
	const bool IsInitialized() const;

	/** True if we are in debug mode that does extra validation */
	const bool IsInDebugMode() const;

	/** Gets the flags for the context */
	EFlags GetFlags() const;

	/** Sets the flags for the context */
	void SetFlags(EFlags Flags);

	/** True if we have done our initial validation. After initial validation, each change to redirects will validate independently */
	const bool HasValidated() const;

	/** Map from config name to flag */
	TMap<FName, ECoreRedirectFlags>& GetConfigKeyMap();

	/** Map from name of thing being mapped to full list. List must be filtered further */
	FCoreRedirects::FRedirectTypeMap& GetRedirectTypeMap();

#if WITH_EDITOR
	FRedirectionSummary& GetRedirectionSummary();
#endif

	UE_API FCoreRedirectsContext& operator=(const FCoreRedirectsContext& Other);

private:
	friend void FCoreRedirects::ValidateAllRedirects();
	/** Sets that we have done our initial validation. After initial validation, each change to redirects will validate independently */
	void SetHasValidated();

	/** This lock allows exclusive locking (WriteLock) and shared locking (ReadLock)
	 *  Additionally, it permits limited types of recursion. It is possible to ReadLock()
	 *  or WriteLock() while locked for write. It is NOT possible to Read or WriteLock() while
	 *  locked for read. I.e., if the lock is held exclusively, it re-acquiring it is always permitted.
	 *  If the lock is held shared, re-acquiring it is never permitted.
	 */
	struct FRWLockWithExclusiveRecursion
	{
		void ReadLock();

		void WriteLock();

		void WriteUnlock();

		void ReadUnlock();

	private:
		uint32 LoadWriteLockOwnerThreadIdRelaxed() const;
		void StoreWriteLockOwnerThreadId(uint32 ThreadId, std::memory_order Order = std::memory_order_seq_cst);

		FTransactionallySafeRWLock InternalLock;
		std::atomic<uint32> WriteLockOwnerThreadId = 0;
		int32 RecursionCount = 0;
	};
	friend struct FScopeCoreRedirectsReadLockedContext;
	friend struct FScopeCoreRedirectsWriteLockedContext;

private:
	std::atomic<bool> bInitialized;
	std::atomic<EFlags> Flags;
	bool bValidatedOnce;
	TMap<FName, ECoreRedirectFlags> ConfigKeyMap;
	FCoreRedirects::FRedirectTypeMap RedirectTypeMap;
	/** Lock to protect multithreaded access to the CoreRedirect system */
	FRWLockWithExclusiveRecursion RWLock;

#if WITH_EDITOR
	FRedirectionSummary RedirectionSummary;
#endif
};
ENUM_CLASS_FLAGS(FCoreRedirectsContext::EFlags);

/**
* RAII type for swapping the current thread's FCoreRedirects context to a new context. Can optionally
* copy the existing state from another context when created.
*/
struct FScopeCoreRedirectsContext
{
	UE_API FScopeCoreRedirectsContext();
	UE_API FScopeCoreRedirectsContext(FCoreRedirectsContext& ContextToCopyFrom);
	UE_API ~FScopeCoreRedirectsContext();

	FCoreRedirectsContext ScopeContext;
	FCoreRedirectsContext& ContextToRestore;
};


/**
* RAII type for locking a context for reading. Supports recursively entering the lock.
*/
struct FScopeCoreRedirectsReadLockedContext
{
public:
	explicit FScopeCoreRedirectsReadLockedContext(FCoreRedirectsContext& InContext)
		: Context(InContext)
		, NeedsUnlock(true)
	{
		Context.RWLock.ReadLock();
	}

	~FScopeCoreRedirectsReadLockedContext()
	{
		if (NeedsUnlock)
		{
			Context.RWLock.ReadUnlock();
		}
	}

	FCoreRedirectsContext& Get() { return Context; }

protected:
	enum class EInitFlag
	{
		InitFlag
	};

	FScopeCoreRedirectsReadLockedContext(FCoreRedirectsContext& InContext, EInitFlag Unused)
		: Context(InContext)
		, NeedsUnlock(false)
	{}

	FCoreRedirectsContext& Context;
	bool NeedsUnlock;
};

/**
* RAII type for locking a context for writing. Supports recursively entering the lock.
*/
struct FScopeCoreRedirectsWriteLockedContext : public FScopeCoreRedirectsReadLockedContext
{
public:
	explicit FScopeCoreRedirectsWriteLockedContext(FCoreRedirectsContext& InContext)
		: FScopeCoreRedirectsReadLockedContext(InContext, EInitFlag::InitFlag)
	{
		Context.RWLock.WriteLock();
	}

	~FScopeCoreRedirectsWriteLockedContext()
	{
		Context.RWLock.WriteUnlock();
	}
};

///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

/** Get whether this has been initialized */
inline const bool FCoreRedirectsContext::IsInitialized() const
{
	// Use an atomic variable rather than take a read lock because we might already be under a read lock
	return bInitialized.load(std::memory_order_acquire);
}

/** True if we are in debug mode that does extra validation */
inline const bool FCoreRedirectsContext::IsInDebugMode() const
{
	return Flags.load(std::memory_order_relaxed) == EFlags::DebugMode;
}

/** Gets the flags for the context */
inline FCoreRedirectsContext::EFlags FCoreRedirectsContext::GetFlags() const
{
	// Use an atomic variable rather than take a read lock because we might already be under a read lock
	return Flags.load(std::memory_order_relaxed);
}

/** Sets the flags for the context */
inline void FCoreRedirectsContext::SetFlags(FCoreRedirectsContext::EFlags NewFlags)
{
	return Flags.store(NewFlags, std::memory_order_relaxed);
}

/** True if we have done our initial validation. After initial validation, each change to redirects will validate independently */
inline const bool FCoreRedirectsContext::HasValidated() const 
{ 
	return bValidatedOnce; 
}

/** Sets that we have done our initial validation. After initial validation, each change to redirects will validate independently */
inline void FCoreRedirectsContext::SetHasValidated() 
{ 
	bValidatedOnce = true;
}

/** Map from config name to flag */
inline TMap<FName, ECoreRedirectFlags>& FCoreRedirectsContext::GetConfigKeyMap()
{ 
	return ConfigKeyMap;
}

/** Map from name of thing being mapped to full list. List must be filtered further */
inline FCoreRedirects::FRedirectTypeMap& FCoreRedirectsContext::GetRedirectTypeMap() 
{ 
	return RedirectTypeMap;
}

#if WITH_EDITOR
inline FRedirectionSummary& FCoreRedirectsContext::GetRedirectionSummary() 
{ 
	return RedirectionSummary;
}
#endif

#undef UE_API
