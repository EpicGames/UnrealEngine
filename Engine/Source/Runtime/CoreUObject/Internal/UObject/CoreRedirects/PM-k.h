// ================================================================================
// Software Name: pm4-bitap.c
// Version: V1.0
// URL: https://www.genivia.com/files/BSD-3.txt
// ===========================================================================================
//  BSD 3-Clause License
// 
// Copyright (c) 2023, Robert van Engelen, Genivia Inc.
// All rights reserved.
// 
// Redistribution and use in source and binary forms, with or without
// modification, are permitted provided that the following conditions are met:
// 
// 1. Redistributions of source code must retain the above copyright notice, this
//    list of conditions and the following disclaimer.
// 
// 2. Redistributions in binary form must reproduce the above copyright notice,
//    this list of conditions and the following disclaimer in the documentation
//    and/or other materials provided with the distribution.
// 
// 3. Neither the name of the copyright holder nor the names of its
//    contributors may be used to endorse or promote products derived from
//    this software without specific prior written permission.
// 
// THIS SOFTWARE IS PROVIDED BY THE COPYRIGHT HOLDERS AND CONTRIBUTORS "AS IS"
// AND ANY EXPRESS OR IMPLIED WARRANTIES, INCLUDING, BUT NOT LIMITED TO, THE
// IMPLIED WARRANTIES OF MERCHANTABILITY AND FITNESS FOR A PARTICULAR PURPOSE ARE
// DISCLAIMED. IN NO EVENT SHALL THE COPYRIGHT HOLDER OR CONTRIBUTORS BE LIABLE
// FOR ANY DIRECT, INDIRECT, INCIDENTAL, SPECIAL, EXEMPLARY, OR CONSEQUENTIAL
// DAMAGES (INCLUDING, BUT NOT LIMITED TO, PROCUREMENT OF SUBSTITUTE GOODS OR
// SERVICES; LOSS OF USE, DATA, OR PROFITS; OR BUSINESS INTERRUPTION) HOWEVER
// CAUSED AND ON ANY THEORY OF LIABILITY, WHETHER IN CONTRACT, STRICT LIABILITY,
// OR TORT (INCLUDING NEGLIGENCE OR OTHERWISE) ARISING IN ANY WAY OUT OF THE USE
// OF THIS SOFTWARE, EVEN IF ADVISED OF THE POSSIBILITY OF SUCH DAMAGE.

#pragma once
#include "CoreTypes.h"
#include "HAL/UnrealMemory.h"

/**
	Predict Match (pm-k) is a fuzzy matching algorithm that enables looking for a potential match across
	multiple substrings at the same time. We take two approaches, one is Bitap (Bit Approximation) which uses a
	sliding window of the size of the smallest substring (small substrings will create more false positives). Bitap
	is very quick for scanning potential matches. We then use pm-8, a sliding window of a fixed size
	(in our case 8 bytes), to further look for potential matches. If we find a hit, we must defer to slower substring
	matching approaches to confirm the match. PredictMatch relies on hashing to distribute the mapping of characters
	in substrings at specific offsets. This hashing helps reduce overlaps between similar substrings which reduces false
	positives, however this does mean that if your hash is poor, or the table size is too small, you may have more false
	positives than preferred. 
	
	This implementation is written by Epic but based on the licensed pm4-bitap.c implementation found at https://www.genivia.com/ugrep.html
*/
struct alignas(PLATFORM_CACHE_LINE_SIZE) FPredictMatch8
{
	static constexpr uint16 TABLE_SIZE = 256;
	static constexpr uint16 ALPHABET_SIZE = 256;

	FPredictMatch8()
	{
		Reset();
	}

	uint16 GetMinimumWordLength() const
	{
		return MinimumWordLength;
	}

	void AddPredictionWord(const uint8* Data, const uint16 DataLen)
	{
		uint8 Bytes[8] = {};

		check(DataLen);
		switch (DataLen)
		{
		default:
			Bytes[7] = Data[7];
		case 7:
			Bytes[6] = Data[6];
		case 6:
			Bytes[5] = Data[5];
		case 5:
			Bytes[4] = Data[4];
		case 4:
			Bytes[3] = Data[3];
		case 3:
			Bytes[2] = Data[2];
		case 2:
			Bytes[1] = Data[1];
		case 1:
			Bytes[0] = Data[0];
		}

		const uint16 Hash1 = HashFn(Bytes[0], Bytes[1]);
		const uint16 Hash2 = HashFn(Hash1, Bytes[2]);
		const uint16 Hash3 = HashFn(Hash2, Bytes[3]);
		const uint16 Hash4 = HashFn(Hash3, Bytes[4]);
		const uint16 Hash5 = HashFn(Hash4, Bytes[5]);
		const uint16 Hash6 = HashFn(Hash5, Bytes[6]);
		const uint16 Hash7 = HashFn(Hash6, Bytes[7]);

		// For each character, store a pair of bits indicating a match (even value, b10)
		// or an accept (00, even value < 2) into the table at increasing offsets into out 16-bit matching window, with each
		// character stored into a different bucket (due to hashing). The last character for our added word is always
		// 00 by definition.
		// 
		// e.g Adding 5-letter prediction word "apple"
		//                character pos
		//			      0  1  2  3  4  5  6  7
		// bucket['a']    10 bb bb bb bb bb bb bb
		// bucket[h('p')] bb 10 bb bb bb bb bb bb
		// bucket[h('p')] bb bb 10 bb bb bb bb bb
		// bucket[h('l')] bb bb bb 10 bb bb bb bb
		// bucket[h('e')] bb bb bb bb 00 bb bb bb  <-- last character is match and accept

		PredictMatchTable[Bytes[0]] &=				  (0x3FFF | ((DataLen > 1) << 15));
		PredictMatchTable[Hash1]	&= (DataLen > 1 ? (0xCFFF | ((DataLen > 2) << 13)) : 0xFFFF);
		PredictMatchTable[Hash2]	&= (DataLen > 2 ? (0xF3FF | ((DataLen > 3) << 11)) : 0xFFFF);
		PredictMatchTable[Hash3]	&= (DataLen > 3 ? (0xFCFF | ((DataLen > 4) << 9))  : 0xFFFF);
		PredictMatchTable[Hash4]	&= (DataLen > 4 ? (0xFF3F | ((DataLen > 5) << 7))  : 0xFFFF);
		PredictMatchTable[Hash5]	&= (DataLen > 5 ? (0xFFCF | ((DataLen > 6) << 5))  : 0xFFFF);
		PredictMatchTable[Hash6]	&= (DataLen > 6 ? (0xFFF3 | ((DataLen > 7) << 3))  : 0xFFFF);
		PredictMatchTable[Hash7]	&= (DataLen > 7 ? (0xFFFC)                         : 0xFFFF);

		MinimumWordLength = MinimumWordLength < DataLen ? MinimumWordLength : DataLen;
		for (int i = 0; i < MinimumWordLength; ++i)
		{
			const uint8 Byte = Data[i];
			BitApproxTable[Byte] &= ~(1 << i);
		}
	}

	bool MatchApproximate(const uint8* Data, const uint32 DataLen) const
	{
		check(DataLen);

		// Note this mask is not what is normally expected. 
		// This mask is used to check if our sliding window of bits contains 
		// a 0 in bit position MinimumWordLength which would indicate a potential match
		const uint16 BitApproxMask = (uint16) (1 << (MinimumWordLength - 1));
		const uint8* const DataEnd = Data + DataLen;

		// Start with no matching bits (all 1)
		uint16 Bits = ~0;
		for (;;)
		{
			// Continue scanning until we either run out of bytes or we find an approximate bit match
			// Shift left and OR our sliding window of potential matches. BitApproxTable[*Data] OR'd
			// with Bits will either keep sliding a 0 bit left indicating we have a fuzzy match, or as we 
			// OR values from BitApproxTable[*Data], we will stomp over the sliding 0 with a 1 indicating 
			// no match and the matching process starts over by sliding in a 0 with the next left shift
			while ((Data < DataEnd) && (((Bits = (uint16)(Bits << 1) | BitApproxTable[*Data]) & BitApproxMask) != 0))
			{
				Data++;
			}

			if (Data >= DataEnd)
			{
				return false;
			}

			// The Bitap scanning above has indicated we have a potential match, but now defer 
			// to PredictMatch to further refine our prediction, since Bitap operates window sizes of 
			// the smallest substring which might be quite small compared to the substrings we are searching for.
			const uint8* PredictionStart = Data - MinimumWordLength + 1;
			if (PredictMatch(PredictMatchTable, PredictionStart, (uint32)(DataEnd - PredictionStart)))
			{
				return true;
			}
			++Data;
		}
	}

	void Reset()
	{
		// Can be no greater than the number of bits in a BitApproxTable element. This value will shrink if we are given a smaller substring
		MinimumWordLength = sizeof(BitApproxTable[0]) * 8;

		FMemory::Memset(BitApproxTable, 0xFF, sizeof(BitApproxTable));
		FMemory::Memset(PredictMatchTable, 0xFF, sizeof(PredictMatchTable));
	}

private:
	static uint16 HashFn(const uint16 A, const uint8 B)
	{
		return (uint16)((((uint16)(A) << 3) ^ (uint16)(B)) & (TABLE_SIZE - 1));
	}

	static bool PredictMatch(const uint16* RESTRICT PredictMatchTable, const uint8* RESTRICT Data, const uint32 DataLen)
	{
		uint8 Bytes[8] = {};

		check(DataLen);
		switch (DataLen)
		{
		default:
			Bytes[7] = Data[7];
		case 7:
			Bytes[6] = Data[6];
		case 6:
			Bytes[5] = Data[5];
		case 5:
			Bytes[4] = Data[4];
		case 4:
			Bytes[3] = Data[3];
		case 3:
			Bytes[2] = Data[2];
		case 2:
			Bytes[1] = Data[1];
		case 1:
			Bytes[0] = Data[0];
		}

		/*
		* Branchless implementation of the following logic:
		* 
		*	if Accept(Bytes[0], 0) then return TRUE			// If we have a substring that ends with Bytes[0] at position 0
		*   if Match(Bytes[0], 0) then						// Otherwise, If we have a substring that has Bytes[0] at position 0
		*       if Accept(Bytes[1], 1) then return TRUE
		*       if Match(Bytes[1], 1) then
		* 		...
		*           if Accept(Bytes[6], 6) then return TRUE
		*           if Match(Bytes[6], 6) then
		*               // It's the last character in the window so no need to check Accept(Bytes[7], 7)
		*               if Matchbit(Bytes[7], 7) then return TRUE
		*/
		const uint16 Hash1 = HashFn(Bytes[0],	Bytes[1]);
		const uint16 Hash2 = HashFn(Hash1,		Bytes[2]);
		const uint16 Hash3 = HashFn(Hash2,		Bytes[3]);
		const uint16 Hash4 = HashFn(Hash3,		Bytes[4]);
		const uint16 Hash5 = HashFn(Hash4,		Bytes[5]);
		const uint16 Hash6 = HashFn(Hash5,		Bytes[6]);
		const uint16 Hash7 = HashFn(Hash6,		Bytes[7]);

		const uint16 AcceptBit0 = PredictMatchTable[Bytes[0]];
		const uint16 AcceptBit1 = PredictMatchTable[Hash1];
		const uint16 AcceptBit2 = PredictMatchTable[Hash2];
		const uint16 AcceptBit3 = PredictMatchTable[Hash3];
		const uint16 AcceptBit4 = PredictMatchTable[Hash4];
		const uint16 AcceptBit5 = PredictMatchTable[Hash5];
		const uint16 AcceptBit6 = PredictMatchTable[Hash6];
		const uint16 AcceptBit7 = PredictMatchTable[Hash7];

		const uint16 Bits =
			(AcceptBit0 & 0xC000) | (AcceptBit1 & 0x3000) | (AcceptBit2 & 0x0C00) | (AcceptBit3 & 0x0300) |
			(AcceptBit4 & 0x00C0) | (AcceptBit5 & 0x0030) | (AcceptBit6 & 0x000C) | (AcceptBit7 & 0x0003);
		const uint16 MatchBits = ((((((((((((((Bits >> 2) | Bits) >> 2) | Bits) >> 2) | Bits) >> 2) | Bits) >> 2) | Bits) >> 2) | Bits) >> 1) | Bits);

		// all two-bit pairs are set meaning no matches (no even valued pairs)
		return MatchBits != 0xFFFF;
	}

	uint16	MinimumWordLength;

	// Table describing for character 'x', BitApproxTable[x] returns a value (in this case 16 bits)
	// where a 0 in the nth bit implies you may have a match if 'n' consecutive  possible matches have 
	// been seen when scanning a string of characters.
	uint16	BitApproxTable[ALPHABET_SIZE];

	// Table encoding a 'match'and 'accept' value using two bits for each character in a window size of 8
	// PredictMatch relies on two pieces of information:
	// Match(x,n) == 1, meaning any of our prediction words has x as their nth character
	// Accept(x, n) == 1, meaning any of our prediction words ends at n with x
	// The PredictionMatchTable[x] returns 16-bits for an 8-character window of characters where 
	// for each two bit pair, we provide the answer to Match(x, n) == 1 (true if the bit pair is even), 
	// and  Accept(x, n) == 1 (true if the bit pair is < 2).
	// Since hashing is used to distribute match and accept data for prediction words, the values stored 
	// at PredictionMatchTable[x] might not represent the window for character 'x' across all prediction words
	// so that predictions can ideally produce fewer false positives.
	// For more information it's highly recommended reading the reference link above
	uint16	PredictMatchTable[TABLE_SIZE];
};
