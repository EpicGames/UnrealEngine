// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "HAL/CriticalSection.h"
#include "Hash/Blake3.h"
#include "UObject/CoreRedirects.h"
#include "UObject/NameTypes.h"

/**
 * Container for FCoreRedirects that can affect a package. Used by class FCoreRedirects to implement
 * GetHashAffectingPackages.
 */
class FRedirectionSummary
{
public:
	FRedirectionSummary() = default;
	FRedirectionSummary(FRedirectionSummary&& Other);
	FRedirectionSummary& operator=(FRedirectionSummary&& Other);

	void Add(const FCoreRedirect& CoreRedirect);
	void Remove(const FCoreRedirect& CoreRedirect);

	void AddObjectRedirector(const FCoreRedirect& ObjectRedirect);
	void RemoveObjectRedirector(const FCoreRedirect& ObjectRedirect);

	void GetObjectRedirectHashAffectingPackage(const FName& PackageName, FBlake3& Hasher);
	void GetHashAffectingPackages(const TConstArrayView<FName>& PackageNames, TArray<FBlake3Hash>& HashArray);
	void AppendHashGlobal(FBlake3& Hasher);

	FCoreRedirects::FGlobalCoreRedirectAddedEvent& GetOnGlobalCoreRedirectAdded();

	void DebugPrintGlobalCoreRedirects() const;

private:
	struct FCompareRedirect
	{
		bool operator()(const FCoreRedirect& A, const FCoreRedirect& B);
	};
	struct FRedirectContainer
	{
	public:
		void Add(FCoreRedirect&& Redirect);
		void Remove(const FCoreRedirect& Redirect);
		bool IsEmpty() const;
		void Empty();
		bool TryAppendHashInReadLock(FBlake3& Hasher) const;
		void AppendHashInWriteLock(FBlake3& Hasher);
		void DebugPrint() const;
	private:
		void CalculateHash();
		void AppendHashWithoutDirtyCheck(FBlake3& Hasher) const;
	private:
		TMap<FCoreRedirect, bool> Redirects;
		FBlake3Hash Hash;
		bool bHashDirty = false;
	};

private:
	static TArray<FName, TInlineAllocator<2>> GetAffectedPackages(const FCoreRedirect& Redirect);

private:
	/** Core Redirects specific to a package.*/
	TMap<FName, FRedirectContainer> RedirectsForPackage;

	/** Global core redirects. Those can affect any package.*/
	FRedirectContainer GlobalRedirects;

	/** Prefixed core redirects that can affect multiple packages.*/
	TSortedMap<FCoreRedirect, bool, FDefaultAllocator, FCompareRedirect> PrefixedRedirects;
	
	/** Object redirectors. The key is the source package name. Redirectors in this map are not in RedirectsForPackage. */
	TMap<FName, FRedirectContainer> ObjectRedirectsForPackage;

	/**
	 * CoreRedirects are written when the engine is single threaded, but we do not write the hashes for the
	 * global and per-package containers until they are requested on demand. The requests on demand can
	 * occur on multiple threads, so we need a lock to ensure that the first thread that requests them and finds
	 * them dirty does not cause a data race with the second thread requests them.
	 */
	FRWLock Lock;

	FCoreRedirects::FGlobalCoreRedirectAddedEvent OnGlobalCoreRedirectAdded;
};

#endif
