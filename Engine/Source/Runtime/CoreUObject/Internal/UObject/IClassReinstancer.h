// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/UniquePtr.h"
#include "Containers/Map.h"

namespace UE::Private
{
	struct IClassReinstancer
	{
		virtual ~IClassReinstancer() = default;
		virtual void DoReinstance(const TMap<UClass*, UClass*>& OldClassToNewClass) = 0;
	};

	COREUOBJECT_API TUniquePtr<IClassReinstancer> CreateClassReinstancer(TArray<UClass*>& InOutClasses);
	
	typedef TUniquePtr<IClassReinstancer> (*FReinstanceClassesPrepassFPtr)(TArray<UClass*>&);
	extern COREUOBJECT_API FReinstanceClassesPrepassFPtr GCreateClassReinstancerFPtr;
}