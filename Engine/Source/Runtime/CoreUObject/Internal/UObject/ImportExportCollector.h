// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if WITH_EDITORONLY_DATA
#include "Containers/Map.h"
#include "Containers/RingBuffer.h"
#include "Containers/Set.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/Package.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

/** An Archive that records all of the imported packages from a tree of exports. */
class FImportExportCollector : public FArchiveUObject
{
public:
	COREUOBJECT_API explicit FImportExportCollector(UPackage* InRootPackage);

	/**
	 * Mark that a given export (e.g. the export that is doing the collecting) should not be explored
	 * if encountered again. Prevents infinite recursion when the collector is constructed and called during
	 * Serialize.
	 */
	COREUOBJECT_API void AddExportToIgnore(UObject* Export);
	/**
	 * Serialize the given object, following its object references to find other imports and exports,
	 * and recursively serialize any new exports that it references.
	 */
	COREUOBJECT_API void SerializeObjectAndReferencedExports(UObject* RootObject);
	/** Restore the collector to empty. */
	COREUOBJECT_API void Reset();
	TArray<UObject*> GetExports() const;
	const TMap<FSoftObjectPath, ESoftObjectPathCollectType>& GetImports() const;
	const TMap<FName, ESoftObjectPathCollectType>& GetImportedPackages() const;
	/**
	 * By default, when this->IsFilterEditorOnly, exports that are EditorOnly exports are not serialized
	 * to look for imports or other exports, and EditorOnly imports are not recorded. But the caller can
	 * override this and allow some of them to be serialized.
	 */
	void SetCallbackIsEditorOnlyObjectAllowed(TFunction<bool(const UObject*)> InCallback);

	COREUOBJECT_API virtual FArchive& operator<<(UObject*& Obj) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;

private:
	void AddImport(const FSoftObjectPath& Path, ESoftObjectPathCollectType CollectType);
	ESoftObjectPathCollectType Union(ESoftObjectPathCollectType A, ESoftObjectPathCollectType B);
	bool CachedIsEditorOnlyObject(const UObject* Object);

	enum class EVisitResult
	{
		Uninitialized,
		Excluded,
		Import,
		Export,
	};
	TMap<UObject*, EVisitResult> Visited;
	TRingBuffer<UObject*> ExportsExploreQueue;
	TMap<FSoftObjectPath, ESoftObjectPathCollectType> Imports;
	TMap<FName, ESoftObjectPathCollectType> ImportedPackages;
	TMap<const UObject*, UE::SavePackageUtilities::EEditorOnlyObjectResult> EditorOnlyObjectCache;
	TFunction<bool(const UObject*)> CallbackIsEditorOnlyObjectAllowed;
	UPackage* RootPackage;
	FName RootPackageName;
};

///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline TArray<UObject*> FImportExportCollector::GetExports() const
{
	TArray<UObject*> Exports;
	for (const TPair<UObject*, EVisitResult>& Pair : Visited)
	{
		if (Pair.Value == EVisitResult::Export)
		{
			Exports.Add(Pair.Key);
		}
	}
	return Exports;
}

inline const TMap<FSoftObjectPath, ESoftObjectPathCollectType>& FImportExportCollector::GetImports() const
{
	return Imports;
}

inline const TMap<FName, ESoftObjectPathCollectType>& FImportExportCollector::GetImportedPackages() const
{
	return ImportedPackages;
}

inline void FImportExportCollector::SetCallbackIsEditorOnlyObjectAllowed(TFunction<bool(const UObject*)> InCallback)
{
	CallbackIsEditorOnlyObjectAllowed = MoveTemp(InCallback);
}

#endif // WITH_EDITORONLY_DATA
