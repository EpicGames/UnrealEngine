// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/NotNull.h"

#if WITH_EDITORONLY_DATA

#define UE_API COREUOBJECT_API

class FFieldVariant;
class FProperty;
class FStructuredArchiveRecord;
class FArchive;
class UClass;
class UObject;
class UStruct;

namespace UE { class FPropertyPathNameTree; }
namespace UE { class FUnknownEnumNames; }
namespace UE::Serialization::Private { class FImportTypeHierarchy; }
namespace UE::Serialization::Private { struct FTypeResource; }

namespace UE
{

/** Query if InstanceDataObject support is available generally. */
UE_API bool IsInstanceDataObjectSupportEnabled();
/** Query if InstanceDataObject support is enabled for a specific object. */
UE_API bool IsInstanceDataObjectSupportEnabled(const UObject* Object);
/** Query if InstanceDataObject support is enabled for garbage collection for a specific class. */
UE_API bool IsInstanceDataObjectSupportEnabledForGC(const UClass* Class);
/** Query if uninitialized alert UI is enabled **/
UE_API bool IsUninitializedAlertUIEnabled();
/** Query if InstanceDataObjects should be saved instead of instances. */
UE_API bool IsInstanceDataObjectImpersonationEnabledOnSave(const UObject* Object);
/** Query if InstanceDataObjects should be constructed using an archetype chain. */
UE_API bool IsInstanceDataObjectArchetypeChainEnabled();
/** Query if InstanceDataObjects should impose placeholder objects for missing types. */
UE_API bool IsInstanceDataObjectPlaceholderObjectSupportEnabled();
/** Query if InstanceDataObjects of subobjects should be outered to the parent object's IDO */
UE_API bool IsInstanceDataObjectOuterChainEnabled();

UE_API bool StructContainsLooseProperties(const UStruct* Struct);
UE_API bool StructIsInstanceDataObjectStruct(const UStruct* Struct);

/** @return Whether the property's has the "LooseProperty" MetaData */
UE_API bool IsPropertyLoose(TNotNull<const FProperty*> Property);
/** @return Whether the provided property's or any properties in the archive serialization chain has the "LooseProperty" MetaData */
UE_API bool IsSerializingUnknownData(const FArchive& Archive, const FProperty* InProperty = nullptr);

/** Query if placeholder support is enabled for a specific import class type. */
bool CanCreatePropertyBagPlaceholderTypeForImportClass(const UClass* ClassKind, const UE::Serialization::Private::FImportTypeHierarchy* ImportTypeHierarchy);
/** Query if placeholder support is enabled for a specific type. */
bool CanCreatePropertyBagPlaceholderForType(const UClass* ClassKind, const UStruct* Type);

/** Query if a given type can serve as a super-type for a placeholder type. */
bool CanCreatePropertyBagPlaceholderSuperType(const UE::Serialization::Private::FTypeResource& SuperType);
UE_API bool CanCreatePropertyBagPlaceholderSuperType(const UStruct* SuperType);

/** Helper to check if a class is an IDO class. */
UE_API bool IsClassOfInstanceDataObjectClass(const UStruct* Class);

/** When Archetype chains are enabled, find or construct an object of a matching InstanceDataObjectClass with the data in InstanceArchetype */
UE_API UObject* GetTemplateForInstanceDataObject(const UObject* Instance, const UObject* InstanceArchetype, const UClass* InstanceDataObjectClass);

/** Generate a UClass that contains the union of the properties of PropertyTree and OwnerClass. */
UClass* CreateInstanceDataObjectClass(const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames, UClass* OwnerClass, UObject* Outer);

/** Returns whether the object is an IDO. */
UE_API bool IsInstanceDataObject(const UObject* Object);

/**
 * Creates an IDO for Owner and serializes it from Archive.
 *
 * USE THE OVERLOAD TAKING AN ARCHIVE IF UNKNOWN PROPERTY VALUES ARE REQUIRED!
 *
 * Creates an IDO class containing the union of the fields in Owner and any associated unknown property tree.
 *
 * @param Owner   Object from which to construct an InstanceDataObject.
 * @return The InstanceDataObject for Owner.
 */
UE_API UObject* CreateInstanceDataObject(UObject* Owner);

/**
 * Creates an IDO for Owner and serializes it from Archive.
 *
 * Creates an IDO class containing the union of the fields in Owner and any associated unknown property tree.
 *
 * @param Owner   Object from which to construct an InstanceDataObject.
 * @param Archive   Archive from which to serialize script properties for the object.
 * @param StartOffset   Offset to seek Archive to before serializing script properties for the object.
 * @param EndOffset   Offset to expect Archive to be at after serializing script properties for the object.
 * @return The InstanceDataObject for Owner.
 */
UE_API UObject* CreateInstanceDataObject(UObject* Owner, FArchive& Ar, int64 StartOffset, int64 EndOffset);

/** Returns the IDO for the object if it has one, otherwise returns the object. */
UE_API UObject* ResolveInstanceDataObject(UObject* Object);

} // UE

#undef UE_API

#endif // WITH_EDITORONLY_DATA
