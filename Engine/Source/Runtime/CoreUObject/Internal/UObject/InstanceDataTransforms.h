// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Misc/DateTime.h"
#include "Misc/NotNull.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Object.h"

#if !UE_BUILD_SHIPPING_WITH_EDITOR && WITH_EDITORONLY_DATA 

class FJsonObject;
struct FTopLevelAssetPath;
namespace UE { class FPropertyPathName; }
namespace UE::Private { struct FRegesterInstanceDataTransformHelper; }

namespace UE::Private::IDT
{
	using FTransformParams = TSharedPtr<FJsonObject>;
	enum ETransformResult
	{
		Break,
		Continue
	};
	using FTransformFunction = TFunction<ETransformResult(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner, const FPropertyPathName& BasePath, const FTransformParams&)>;
}

namespace UE
{
	// public facing list of operations that can be applied to an object or saved to disk
	class FInstanceDataTransformSet
	{
	public:
		using FTransformParams = UE::Private::IDT::FTransformParams;
		using ETransformResult = UE::Private::IDT::ETransformResult;
		struct FOperation
		{
			FString OpCode; // ie RemoveProperty, RedirectProperty, etc
			FTransformParams Params;
			ETransformResult operator()(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner, const FPropertyPathName& BasePath) const;
			ETransformResult operator()(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner) const;
		};

		FTopLevelAssetPath StructPath;
		TArray<FOperation> Operations;
	};

	class FInstanceDataTransforms
	{
	public:
		using FTransformParams = UE::Private::IDT::FTransformParams;
		using ETransformResult = UE::Private::IDT::ETransformResult;
		using FTransformFunction = UE::Private::IDT::FTransformFunction;

		COREUOBJECT_API static FInstanceDataTransforms& Get();

		COREUOBJECT_API TObjectPtr<UObject> PatchInstanceDataObject(TNotNull<UObject*> IDO, TNotNull<UObject*> Owner);

		COREUOBJECT_API FInstanceDataTransformSet GetTransformSet(UStruct& ClassOrStruct);
		COREUOBJECT_API TObjectPtr<UObject> ApplyTransformSet(const FInstanceDataTransformSet& TransformSet, TNotNull<UObject*> IDO, TNotNull<UObject*> Owner);
		COREUOBJECT_API void SaveTransformSet(const FInstanceDataTransformSet& TransformSet);

		COREUOBJECT_API bool IsSerializationEnabled();

	private:

		const FInstanceDataTransformSet* FindTransformsInternal(const FTopLevelAssetPath& StructPath);
		const FInstanceDataTransformSet* FindTransformsInternal(const TNotNull<UStruct*> Struct);

		void ReadConfigFile(const FString& ConfigFilepath);

		friend Private::FRegesterInstanceDataTransformHelper;
		void RegisterOperation(FStringView Name, FTransformFunction Function);

		void PatchClass(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner);
		void PatchStructs(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner);

		friend struct FInstanceDataTransformSet::FOperation;

		TMap<FString, FDateTime> ConfigFileLastReadTime;
		TMap<FTopLevelAssetPath, FInstanceDataTransformSet> StructPathToIDTs;
		TMap<FString, FTransformFunction> RegisteredOperations;
	};
}

#endif // !UE_BUILD_SHIPPING_WITH_EDITOR && WITH_EDITORONLY_DATA 