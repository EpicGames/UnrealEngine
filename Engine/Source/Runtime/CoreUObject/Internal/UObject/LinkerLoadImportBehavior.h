// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/ObjectHandle.h"

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

class FLinkerLoad;
struct FObjectImport;
class IAssetRegistryInterface;

namespace UE::LinkerLoad
{
enum class EImportBehavior : uint8
{
	Eager = 0,
	// @TODO: OBJPTR: we want to permit lazy background loading in the future
	//LazyBackground,
	LazyOnDemand,
};


EImportBehavior GetPropertyImportLoadBehavior(const FObjectImport& Import, const FLinkerLoad& LinkerLoad);

/// @brief tries to lazy load an imported FObjectPtr
/// 
/// @param AssetRegistry to see if import exists
/// @param Import to try to lazy load
/// @param LinkerLoad for the Import
/// @param ObjectPtr out parameter
/// @return true if import was lazy loaded otherwise false
bool TryLazyImport(const IAssetRegistryInterface& AssetRegistry, const FObjectImport& Import, const FLinkerLoad& LinkerLoad, FObjectPtr& ObjectPtr);

/// @brief determines if an import can be lazy loaded
/// 
/// @param AssetRegistry to see if import exists
/// @param Import to try to lazy load
/// @param LinkerLoad for the Import
/// @return true if import can be lazy loaded otherwise false
bool CanLazyImport(const IAssetRegistryInterface& AssetRegistry, const FObjectImport& Import, const FLinkerLoad& LinkerLoad);

/// @brief tries to resolve an object path to a lazy loaded TObjectPtr
///
/// @param Class type of the object that will be resolved
/// @param ObjectPath object path of to resolve
/// @param OutObjectPtr out parameter
/// @return true if was lazy loaded otherwise false
bool TryLazyLoad(const UClass& Class, const FSoftObjectPath& ObjectPath, TObjectPtr<UObject>& OutObjectPtr);

/// @brief Finds LoadBehavior meta data recursively
/// @return Eager by default in not found
EImportBehavior FindLoadBehavior(const UClass& Class);

/// @brief returns true if lazy load imports is enabled via command line or config
bool IsImportLazyLoadEnabled();


}

#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
