// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"

namespace UE::CoreUObject
{
	enum class EAllowSetNullOnNonNullableBehavior : uint8
	{
		Disabled,
		Enabled
	};
	
#if WITH_EDITORONLY_DATA
	class FScopedSetNullOnNonNullable
	{
	public:
		// If set, InBehavior will adjust later code in stack to adopt the behavior
		// If not set, prevailing behavior is preserved
		COREUOBJECT_API explicit FScopedSetNullOnNonNullable(TOptional<EAllowSetNullOnNonNullableBehavior> InBehavior);
		COREUOBJECT_API ~FScopedSetNullOnNonNullable();
	
		EAllowSetNullOnNonNullableBehavior GetBehavior() const;		
	
	private:
		FScopedSetNullOnNonNullable(const FScopedSetNullOnNonNullable&) = delete;
		FScopedSetNullOnNonNullable(FScopedSetNullOnNonNullable&&) = delete;
		FScopedSetNullOnNonNullable& operator=(const FScopedSetNullOnNonNullable&) = delete;
		FScopedSetNullOnNonNullable& operator=(FScopedSetNullOnNonNullable&&) = delete;
		
		FScopedSetNullOnNonNullable* Previous;
		EAllowSetNullOnNonNullableBehavior Behavior;
		bool bIsBehaviorSet;
	};

	// Thread local, set with FScopedSetNullOnNonNullable
	// If true, will allow setting null on a NonNullable property
	// Some cases in editor deserialization requires null to be set on NonNull properties
	// Example: deserializing a previously null value to NonNullable property from the transaction buffer for Undo/Redo
	EAllowSetNullOnNonNullableBehavior AllowSetNullOnNonNullableBehavior();
#else
	inline EAllowSetNullOnNonNullableBehavior AllowSetNullOnNonNullableBehavior()
	{
		return  EAllowSetNullOnNonNullableBehavior::Disabled;
	}
#endif
}
