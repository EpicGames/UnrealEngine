// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Logging/LogMacros.h"
#include "UObject/UnrealNames.h"

struct FObjectImport;
struct FPackageFileSummary;

// Category to use for the log related to the relocation
COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogPackageRelocation, Log, All);

/**
 * Object relocation is a process done on load to make the references in ImportMap and SoftReferencePathList that point to
 * packages in the same mount point behave as if they were relative paths.
 * This namespace contains the utilities to calculate relocations and is called from code that loads packages.
 * 
 * The API from this namespace is experimental and will be changed without deprecation
 */
namespace UE::Package::Relocation::Private
{
	/** A struct that contains the most used arguments for the relocations functions */
	struct FPackageRelocationContext
	{
		FStringView OriginalPackagePath;
		FStringView CurrentPackagePath;
		FStringView OriginalPackageMount;
	};


	/**
	 * Determine if we should relocate the package or not.
	 * Populate the context for the relocation functions if the package should be relocated
	 * 
	 * Note: Keep the package summary and the source of LoadedPackageName StringView in memory as the OutPackageRelocationContext use a view on the data of these arguments
	 */
	COREUOBJECT_API bool ShouldApplyRelocation(const FPackageFileSummary& PackageSummary, FStringView LoadedPackageName, FPackageRelocationContext& OutPackageRelocationContext);

	/**
	 * Try to relocate a package name with the same relative pathing from the current package path as from the original package path
	 * i.e.
	 * OriginalPackagePath: /Game/Folder/
	 * CurrentPackagePath: /Game/OtherFolder/SubFolder/
	 * Example 1:
	 * InPackageNameToRelocate: /Game/Folder/Sub/Asset
	 * OutNewLocation: /Game/OtherFolder/SubFolder/Sub/Asset
	 * Example 2:
	 * InPackageNameToRelocate: /Game/OtherAsset
	 * OutNewLocation: /Game/OtherFolder/OtherAsset
	 * @param InPackageRelocationContext The context used for the relocation of the loaded asset
	 * @param InPackageNameToRelocate The package name we want to remap relative to the current location
	 * @param OutNewLocation The relocated package name.
	 * @return True If the package need to be relocated.
	 */
	COREUOBJECT_API bool TryRelocateReference(const FPackageRelocationContext& InPackageRelocationContext, FStringView InPackageNameToRelocate, FStringBuilderBase& OutNewLocation);

	COREUOBJECT_API void ApplyRelocationToObjectImportMap(const FPackageRelocationContext& InPackageRelocationContext, TArrayView<FObjectImport> ImportMapView);

	COREUOBJECT_API void ApplyRelocationToSoftObjectArray(const FPackageRelocationContext& InPackageRelocationContext, TArrayView<FSoftObjectPath> SoftObjectPaths);

	COREUOBJECT_API void ApplyRelocationToNameArray(const FPackageRelocationContext& InPackageRelocationContext, TArrayView<FName> PackageNames);
}