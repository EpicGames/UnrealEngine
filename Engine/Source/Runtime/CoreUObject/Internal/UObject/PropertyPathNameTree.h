// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Map.h"
#include "Templates/TypeHash.h"
#include "UObject/PropertyPathName.h"
#include "UObject/PropertyTag.h"
#include "UObject/PropertyTypeName.h"

#define UE_API COREUOBJECT_API

class FBlake3;

namespace UE
{

/**
 * A tree of property path names and their associated types.
 *
 * A union of the paths that are added, ignoring any container index.
 */
class FPropertyPathNameTree
{
	/** Key corresponding to a FPropertyPathNameSegment with no index. */
	struct FKey
	{
		FName Name;
		FPropertyTypeName Type;

		[[nodiscard]] friend inline bool operator==(const FKey& Lhs, const FKey& Rhs)
		{
			return Lhs.Name == Rhs.Name && Lhs.Type == Rhs.Type;
		}

		[[nodiscard]] friend inline bool operator<(const FKey& Lhs, const FKey& Rhs)
		{
			if (int32 Compare = Lhs.Name.Compare(Rhs.Name))
			{
				return Compare < 0;
			}
			return Lhs.Type < Rhs.Type;
		}

		[[nodiscard]] friend inline uint32 GetTypeHash(const FKey& Key)
		{
			return HashCombineFast(GetTypeHash(Key.Name), GetTypeHash(Key.Type));
		}

		friend inline void AppendHash(FBlake3& Builder, const FKey& Key)
		{
			DispatchAppendHash(Builder, Key.Name);
			DispatchAppendHash(Builder, Key.Type);
		}
	};

	struct FValue
	{
		TUniquePtr<FPropertyPathNameTree> SubTree;
		FPropertyTag Tag;
	};

public:
	struct FConstNode
	{
		const FValue* Value = nullptr;
		inline explicit operator bool() const { return !!Value; }
		inline const FPropertyPathNameTree* GetSubTree() const { return Value ? Value->SubTree.Get() : nullptr; }
		inline const FPropertyTag* GetTag() const { return Value && !Value->Tag.Name.IsNone() ? &Value->Tag : nullptr; }
	};

	struct FNode : FConstNode
	{
		using FConstNode::GetSubTree;
		using FConstNode::GetTag;
		inline FPropertyPathNameTree* GetSubTree() { return const_cast<FPropertyPathNameTree*>(FConstNode::GetSubTree()); }
		inline FPropertyTag* GetTag() { return const_cast<FPropertyTag*>(FConstNode::GetTag()); }
		UE_API void SetTag(const FPropertyTag& Tag);
	};

	FPropertyPathNameTree() = default;
	FPropertyPathNameTree(FPropertyPathNameTree&&) = default;
	FPropertyPathNameTree(const FPropertyPathNameTree&) = delete;
	FPropertyPathNameTree& operator=(FPropertyPathNameTree&&) = default;
	FPropertyPathNameTree& operator=(const FPropertyPathNameTree&) = delete;

	/** True if the tree contains no property path names. */
	inline bool IsEmpty() const { return Nodes.IsEmpty(); }

	/** Remove every property path name from the tree. */
	UE_API void Empty();

	/** Adds the path to the tree. Keeps any existing nodes that match both name and type. */
	UE_API FNode Add(const FPropertyPathName& Path, int32 StartIndex = 0);

	/** Removes the path from the tree. This may create a parent with no children. If that parent also has no tag, remove it too */
	UE_API bool Remove(const FPropertyPathName& Path, int32 StartIndex = 0);

	/**
	 * Finds the path within the tree.
	 *
	 * @param OutSubTree   If non-null, this is set to the maybe-null sub-tree at the path if the path is found.
	 * @param Path         Path to search within the tree.
	 * @param StartIndex   Index of the segment within Path at which to start searching.
	 * @return An accessor for the node at the path if it exists.
	 */
	[[nodiscard]] UE_API FNode Find(const FPropertyPathName& Path, int32 StartIndex = 0);
	[[nodiscard]] inline FConstNode Find(const FPropertyPathName& Path, int32 StartIndex = 0) const
	{
		return {const_cast<FPropertyPathNameTree*>(this)->Find(Path, StartIndex).Value};
	}

	class FConstIterator
	{
	public:
		inline explicit FConstIterator(const TMap<FKey, FValue>::TConstIterator& InNodeIt)
			: NodeIt(InNodeIt)
		{
		}

		inline FConstIterator& operator++() { ++NodeIt; return *this; }

		inline explicit operator bool() const { return (bool)NodeIt; }

		inline bool operator==(const FConstIterator& Rhs) const { return NodeIt == Rhs.NodeIt; }
		inline bool operator!=(const FConstIterator& Rhs) const { return NodeIt != Rhs.NodeIt; }

		inline FName GetName() const { return NodeIt.Key().Name; }
		inline FPropertyTypeName GetType() const { return NodeIt.Key().Type; }
		inline FConstNode GetNode() const { return {&NodeIt.Value()}; }

	private:
		TMap<FKey, FValue>::TConstIterator NodeIt;
	};

	[[nodiscard]] inline FConstIterator CreateConstIterator() const { return FConstIterator(Nodes.CreateConstIterator()); }

private:
	UE_API friend void AppendHash(FBlake3& Builder, const FPropertyPathNameTree& Tree);

	TMap<FKey, FValue> Nodes;
};

} // UE

#undef UE_API
