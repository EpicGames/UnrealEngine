// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#if WITH_EDITORONLY_DATA

#include "Containers/ContainersFwd.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

class FBlake3;
class FName;
class FStructuredArchiveRecord;
class UEnum;
class UObject;
class UStruct;
namespace UE { class FPropertyPathNameTree; }
namespace UE { class FPropertyTypeName; }

#define UE_API COREUOBJECT_API

namespace UE
{

/**
 * An accessor for a specific property value flag.
 *
 * All functions that take FProperty require the property to be owned directly by the struct/class
 * that this is constructed with, or a super struct/class of that type.
 */
template <EPropertyValueFlags Flag>
class TPropertyValueState
{
public:
	/** Construct an accessor for the property value flags of instance of InStruct pointed to by InData. */
	inline explicit TPropertyValueState(const UStruct* InStruct, void* InData)
		: Struct(InStruct)
		, Data(InData)
	{
	}

	/** Construct an accessor for the property value flags of the object. */
	inline explicit TPropertyValueState(UObject* Object)
		: Struct(Object->GetClass())
		, Data(Object)
	{
	}

	/** Try to activate tracking of the property value flag. Returns true if tracking is active, otherwise false. */
	inline bool ActivateTracking()
	{
		return Struct->ActivateTrackingPropertyValueFlag(Flag, Data);
	}

	/** Query whether property value flag is being tracked. */
	inline bool IsTracking() const
	{
		return Struct->IsTrackingPropertyValueFlag(Flag, Data);
	}

	/** Query whether the property has the property value flag set. */
	bool IsSet(const FProperty* Property, int32 ArrayIndex = 0) const
	{
		bool bIsSet = Struct->HasPropertyValueFlag(Flag, Data, Property, ArrayIndex);

		// This is a temporary workaround to handle the case of an object reference being
		// garbage collected and set to NULL (see FGarbageEliminationScopeGuard). This if
		// should be removed once the Editor handles the clearing of deleted object references.
		if (bIsSet && (Flag == EPropertyValueFlags::Initialized) && Property->HasAnyPropertyFlags(CPF_NonNullable))
		{
			if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
			{
				const UObject* PropertyValue = ObjectProperty->LoadObjectPropertyValue_InContainer(Data);
				if (!PropertyValue)
				{
					bIsSet = false;
				}
			}
		}
		
		return bIsSet;
	}

	/** Set the property value flag for the property to bValue. */
	inline void SetTo(bool bValue, const FProperty* Property, int32 ArrayIndex = 0)
	{
		Struct->SetPropertyValueFlag(Flag, bValue, Data, Property, ArrayIndex);
	}

	/** Set the property value flag for the property. */
	inline void Set(const FProperty* Property, int32 ArrayIndex = 0)
	{
		SetTo(/*bValue*/ true, Property, ArrayIndex);
	}

	/** Clear the property value flag for the property. */
	inline void Clear(const FProperty* Property, int32 ArrayIndex = 0)
	{
		SetTo(/*bValue*/ false, Property, ArrayIndex);
	}

	/** Reset the property value flag for every property in the type. */
	inline void Reset()
	{
		Struct->ResetPropertyValueFlags(Flag, Data);
	}

	/** Serialize the property value flags for every property in the type. */
	inline void Serialize(FStructuredArchiveRecord Record)
	{
		Struct->SerializePropertyValueFlags(Flag, Data, Record, GetArchiveFieldName());
	}

private:
	inline static FArchiveFieldName GetArchiveFieldName()
	{
		static_assert(Flag == EPropertyValueFlags::Initialized || Flag == EPropertyValueFlags::Serialized);
		if constexpr (Flag == EPropertyValueFlags::Initialized)
		{
			return TEXT("PropertyValueInitializedFlags");
		}
		else
		{
			return TEXT("PropertyValueSerializedFlags");
		}
	}

	const UStruct* Struct = nullptr;
	void* Data = nullptr;
};

// A property is initialized if it has a value set on it.
// A property may be uninitialized if it has no default and has never been set.
// A property may be uninitialized if it is an unknown property in an InstanceDataObject (IDO)
// and has no value on a particular instance or has had its value unset while being fixed up.
using FInitializedPropertyValueState = TPropertyValueState<EPropertyValueFlags::Initialized>;

// A property is serialized if a value was loaded into it during serialization.
// A property may be initialized without having been serialized because a template object provided a default.
// A property may be serialized without being initialized if its value was removed following serialization.
using FSerializedPropertyValueState = TPropertyValueState<EPropertyValueFlags::Serialized>;

/**
 * A tree of unknown properties found during serialization. Used with bTrackUnknownProperties.
 */
class FUnknownPropertyTree
{
public:
	/** Constructs an accessor for the unknown property tree of the owner. */
	explicit FUnknownPropertyTree(const UObject* Owner);

	/** Finds the existing unknown property path name tree for the owner. */
	TSharedPtr<FPropertyPathNameTree> Find() const;

	/** Finds the existing unknown property path name tree for the owner or creates one. */
	TSharedPtr<FPropertyPathNameTree> FindOrCreate();

	/** Destroys the unknown property path name tree for the owner if it has one. */
	void Destroy();

private:
	const UObject* Owner = nullptr;
};

/**
 * A record of unknown enumerator names found during serialization. Used with bTrackUnknownEnumNames.
 */
class FUnknownEnumNames
{
public:
	/** Constructs an accessor for the unknown enum names of the owner. */
	explicit FUnknownEnumNames(const UObject* Owner);

	/**
	 * Adds an unknown enumerator name within an enum type.
	 *
	 * @param Enum The enum associated with the unknown enumerator. May be null or the wrong type.
	 * @param EnumTypeName The type name of the enum associated with the unknown enumerator.
	 * @param EnumValueName The name of the unknown enumerator.
	 * @note At least one of Enum and EnumTypeName must be valid.
	 */
	void Add(const UEnum* Enum, FPropertyTypeName EnumTypeName, FName EnumValueName);

	/**
	 * Finds unknown enumerator names associated with an enum type.
	 *
	 * @param EnumTypeName The type name of the enum containing the unknown names to find.
	 * @param OutNames Array to assign the unknown names to. Empty on return if no names are found.
	 * @param bOutHasFlags Assigned to true if the enum is known to have flags, otherwise false.
	 */
	void Find(FPropertyTypeName EnumTypeName, TArray<FName>& OutNames, bool& bOutHasFlags) const;

	/** True if there are no unknown enumerator names tracked for the owner. */
	bool IsEmpty() const;

	/** Destroys the unknown property enum names for the owner if it has any. */
	void Destroy();

private:
	friend void AppendHash(FBlake3& Builder, const FUnknownEnumNames& EnumNames);

	const UObject* Owner = nullptr;
};

} // UE

#undef UE_API

#endif // WITH_EDITORONLY_DATA
