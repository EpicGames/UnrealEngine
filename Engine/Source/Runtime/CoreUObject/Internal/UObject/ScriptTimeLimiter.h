// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE
{

class FScriptTimeLimiter
{
public:
	FScriptTimeLimiter() = default;

	/** @return Reference to the thread-local singleton ScriptTimeLimiter */
	COREUOBJECT_API static FScriptTimeLimiter& Get();

	/** 
	 * The timer is started when the first (outermost) call to StartTimer occurs.
	 * Nesting calls is allowed, but will not grant additional computation time past the limit.
	 */
	COREUOBJECT_API void StartTimer();

	/** 
	 * The timer is stopped when every call to StartTimer has had a matching StopTimer call.
	 * At this point, the time limit is reset.
	 */
	COREUOBJECT_API void StopTimer();

	/**
	 * Adds a time duration to the accumulated overhead time, so it can be excluded from the total script execution time.
	 */
	COREUOBJECT_API void AddOverheadTime(double Duration);

	/** 
	 * Returns true if the timer has been running longer than the Verse computation limit.
	 * False is always returned if the timer is not running.
	 */
	COREUOBJECT_API bool HasExceededTimeLimit();

	/** 
	 * Returns true if the timer has been running longer than the passed-in time limit.
	 * False is always returned if the timer is not running.
	 */
	COREUOBJECT_API bool HasExceededTimeLimit(double TimeLimit);

private:
	// This tracks the nesting depth of calls to StartTimer/StopTimer.
	int32 NestingDepth = 0;

	// This tracks the starting time (in FPlatformTime::Seconds) of the outermost StartTimer.
	double StartingTime = 0.0;

	// This tracks accumulated overhead time within the active timer scope.
	double TotalOverheadTime = 0.0;
};

}
