// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "Misc/NotNull.h"
#include "UObject/Object.h"
#include "UObject/UObjectAnnotation.h"

class FObjectArchetypeHelper
{
public:
	class IObjectArchetypePolicy
	{
#if WITH_EDITOR
	public:
		virtual UObject* GetArchetype(const UObject* InObject) const = 0;
#endif
	};

	/** Returns archetype for an object, possibly using a runtime cache */
	COREUOBJECT_API static UObject* GetArchetype(const UObject* InObject, const IObjectArchetypePolicy* InPolicy);

	/** Fills in archetype cache for specific object if enabled */
	COREUOBJECT_API static void CacheArchetypeForObject(UObject* Object, UObject* Archetype);

	/** Removes all cached archetypes to avoid doing it in static exit where it may cause crashes */
	COREUOBJECT_API static void CleanupCachedArchetypes();

	/** Invalidate archetype cache for specific object after a rename */
	COREUOBJECT_API static void InvalidateArchetypeCache(const UObject* Object);
};

#if WITH_EDITOR

struct FCacheArchetype
{
	bool IsDefault() const
	{
		return CachedArchetype == nullptr;
	}

	/** Archetype is cache when it is or its archetype is about to be replaced **/
	TObjectPtr<UObject> CachedArchetype = nullptr;
};

class FEditorCacheArchetypeManager
{
public:
	/**
	 * @return the static instance managing the reinstantiation */
	COREUOBJECT_API static FEditorCacheArchetypeManager& Get();

	/**
	 * Caches the archetype for this object during reinstantiation has we rename object in that phase.
	 * This is only a utility method to ensure the archetype is not lost during reinstantiation
	 * as it renames and changes outers which makes impossible to retrieve it during the process.
	 * @param Object top cache its archetype
	 * @param Archetype to use as a cache, if null it will use GetArchetype on the owner of the specified object */
	void CacheArchetype(TNotNull<const UObject*> Object, UObject* Archetype = nullptr)
	{
		FCacheArchetype Annotation = ObjectCachedArchetypeAnnotations.GetAnnotation(Object);
		if (Annotation.CachedArchetype == nullptr)
		{
			Annotation.CachedArchetype = Archetype ? Archetype : Object->GetArchetype();
			ObjectCachedArchetypeAnnotations.AddAnnotation(Object, Annotation);
		}
	}

	void ResetCacheArchetype(TNotNull<const UObject*> Object)
	{
		ObjectCachedArchetypeAnnotations.RemoveAnnotation(Object);
	}

	/**
	 * Gets the archetype that was cached when the CacheArchetype method was called
	 * @return the cached Archetype if any */
	UObject* GetCachedArchetype(TNotNull<const UObject*> Object)
	{
		return ObjectCachedArchetypeAnnotations.GetAnnotation(Object).CachedArchetype;
	}

private:
	FUObjectAnnotationSparse<FCacheArchetype, true/*bAutoRemove*/> ObjectCachedArchetypeAnnotations;
};

#endif // WITH_EDITOR
