// Copyright Epic Games, Inc. All Rights Reserved.

// UObjectGlobalsInternal.h - Global UObject functions for use by other engine modules but not by game modules.

#pragma once

#include "CoreMinimal.h"

/**
 * Global CoreUObject delegates for use by other engine modules
 */
struct FCoreUObjectInternalDelegates
{
	/** Called before GC verification code rename a package containing a World to try and prevent such errors blocking testing and development. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPackageRename, UPackage*);
	static COREUOBJECT_API FPackageRename& GetOnLeakedPackageRenameDelegate();
};

/**
 * Trash a `UObject`. Should be used with extreme caution. Used by systems that
 * need to force reload objects from disk and thus need to not find the original
 * object alive and well.
 * 
 * It is illegal to trash an object that is in the process of being loaded.
 */
void COREUOBJECT_API TrashObject(TNotNull<UObject* const> Object);
