// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UObject/RemoteExecutor.h"
#include "UObject/RemoteObjectTypes.h"

#define UE_API COREUOBJECT_API

/** Which side of the migration are we on? */
enum class EObjectMigrationSide : uint8
{
	/** We are receiving (migrating-in) an object */
	Receive,

	/** We are sending (migrating-out) an object */
	Send
};

/**
 * Possible types of migration that have happened when you have received an Object
 */
enum class EObjectMigrationRecvType : uint8
{
	Invalid,

	/** Received an object without changing ownership (implies it will be sent back to the server that owns it eventually) */
	Borrowed,

	/** We are receiving an Object we have previously loaned-out (we already own it) */
	ReturnedLoan,

	/** We received objects and must take ownership of the them (we have no choice) */
	AssignedOwnership
};
UE_API const TCHAR* ToString(EObjectMigrationRecvType Value);

/**
 * Possible types of migration that are happening when you are sending an Object
 */
enum class EObjectMigrationSendType : uint8
{
	Invalid,

	/** Loaned-out an object while retaining ownership (implies we want this back, we still own it) */
	Loan,

	/** This borrowed object is being returned (sent back) to the server that loaned it out */
	ReturnBorrowed,

	/** This borrowed object is being forwarded to another server which requested it (it should be returned to the owner, not us) */
	ReassignBorrowed,

	/** Reassign objects to relinquish ownership and assign them to the destination server */
	ReassignOwnership
};
UE_API const TCHAR* ToString(EObjectMigrationSendType Value);

/** Structure that holds context for what we're intending to do during migration */
struct FUObjectMigrationContext final
{
	struct FMigratedObjectStatus
	{
		FRemoteServerId OwningServerId;
		EResidence Residence = EResidence::Local;
	};

	/** The Object that is being migrated, it may not be the top-level UObject */
	FRemoteObjectId ObjectId;

	/** The ServerId on the other side of the migration */
	FRemoteServerId RemoteServerId;

	/** The (New) Owner of the ObjectId as a result of this migration */
	FRemoteServerId OwnerServerId;

	/** Which side of the migration does this context apply to? Sending or Receiving? */
	EObjectMigrationSide MigrationSide = EObjectMigrationSide::Receive;

	/** If valid, the request id of the multi-server commit */
	FRemoteTransactionId MultiServerCommitRequestId;

	/** Maps remote object Ids to the ownership status on the current server (true means the object was owned by the MigrationSide at the moment the object's data was migrated) */
	mutable TMap<FRemoteObjectId, FMigratedObjectStatus> CachedObjectStatus;

	/** Are the values of this context valid? */
	UE_API bool IsValid() const;

	UE_API void CacheObjectStatus(const struct FRemoteObjectData& ObjectData);
	UE_API const FMigratedObjectStatus& CacheObjectStatus(FRemoteObjectId InObjectId) const;

	inline bool IsOwned(FRemoteObjectId InObjectId) const
	{
		return CachedObjectStatus.FindChecked(InObjectId).OwningServerId.IsLocal();
	}

	inline EResidence GetResidence(FRemoteObjectId InObjectId) const
	{
		return CachedObjectStatus.FindChecked(InObjectId).Residence;
	}

	inline FRemoteServerId TryGetOwningServerId(FRemoteObjectId InObjectId) const
	{
		if (FMigratedObjectStatus* Status = CachedObjectStatus.Find(InObjectId))
		{
			return Status->OwningServerId;
		}
		return FRemoteServerId();
	}

	/**
	* During migration, figure out the role of the object being received
	*/
	UE_API EObjectMigrationRecvType GetObjectMigrationRecvType(const UObjectBase* Object) const;
	UE_API EObjectMigrationRecvType GetObjectMigrationRecvType(FRemoteObjectId ObjectId) const;

	/**
	* During migration, figure out the role of the object being sent
	*/
	UE_API EObjectMigrationSendType GetObjectMigrationSendType(const UObjectBase* Object) const;
	UE_API EObjectMigrationSendType GetObjectMigrationSendType(FRemoteObjectId ObjectId) const;

	/** Gets the current Migration Context if it exists (only exists during actual object migrations) */
	static const FUObjectMigrationContext* const GetCurrentMigrationContext()
	{
		checkf(IsInGameThread(), TEXT("%hs: We expect migrations to only occur on the GameThread."), __func__);
		return CurrentMigrationContext;
	}

	
private: // Interface used for FScopedObjectMigrationContext
	friend struct FScopedObjectMigrationContext;

	inline static const FUObjectMigrationContext* CurrentMigrationContext = nullptr;
	static void SetCurrentMigrationContext(const FUObjectMigrationContext* NewMigrationContext)
	{
		checkf(IsInGameThread(), TEXT("%hs: We expect migrations to only occur on the GameThread."), __func__);
		CurrentMigrationContext = NewMigrationContext;
	}
};

/**
 * Helper class to be used whenever we create an FUObjectMigrationContext to ensure all calls within the scope
 * have their FUObjectMigrationContext::GetCurrentMigrationContext set to the desired value.  It will correctly pop
 * and restore the previous value on destruction, obeying the stack of migrations (such as needing to create a new
 * MigrationContext when we're loading from the Database).
 */
struct FScopedObjectMigrationContext
{
	FScopedObjectMigrationContext(const FUObjectMigrationContext& NewMigrationContext)
		: PreviousMigrationContext(FUObjectMigrationContext::CurrentMigrationContext)
	{
		FUObjectMigrationContext::SetCurrentMigrationContext(&NewMigrationContext);
	}

	~FScopedObjectMigrationContext()
	{
		FUObjectMigrationContext::SetCurrentMigrationContext(PreviousMigrationContext);
	}

private:
	const FUObjectMigrationContext* PreviousMigrationContext;
};

#undef UE_API
