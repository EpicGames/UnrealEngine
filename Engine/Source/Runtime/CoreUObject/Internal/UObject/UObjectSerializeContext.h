// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/PropertyPathName.h"

#define UE_API COREUOBJECT_API

class FArchive;
class UObject;

namespace UE
{

/**
* An object that configures the FUObjectSerializeContext for serialization of the object with the archive.
*
* Construct this on the stack within the scope that the object will be serialized.
* Objects that support creation of an InstanceDataObject (IDO) on load will create the IDO when this is destructed.
*/
struct FScopedObjectSerializeContext
{
	UE_API FScopedObjectSerializeContext(UObject* Object, FArchive& Archive);
	UE_API ~FScopedObjectSerializeContext();

	// specialized context specifically for serializing individual properties during IDTs
	FScopedObjectSerializeContext(UObject* Object, FArchive& Archive, const FPropertyPathName& PathBeingSerialized);

	FScopedObjectSerializeContext(const FScopedObjectSerializeContext&) = delete;
	FScopedObjectSerializeContext& operator=(const FScopedObjectSerializeContext&) = delete;

private:
#if WITH_EDITORONLY_DATA
	FArchive& Archive;
	UObject* const Object;
#endif
	UObject* SavedSerializedObject;
#if WITH_EDITORONLY_DATA
	int64 SavedSerializedObjectScriptStartOffset;
	int64 SavedSerializedObjectScriptEndOffset;
	bool bSavedTrackSerializedPropertyPath;
	bool bSavedTrackInitializedProperties;
	bool bSavedTrackSerializedProperties;
	bool bSavedTrackUnknownProperties;
	bool bSavedTrackUnknownEnumNames;
	bool bSavedImpersonateProperties;
	bool bCreateInstanceDataObject;
	FPropertyPathName SavedSerializedPropertyPath;
#endif
};

} // UE

#undef UE_API
