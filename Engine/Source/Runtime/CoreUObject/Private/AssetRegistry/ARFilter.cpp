// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/ARFilter.h"

#include "Algo/Compare.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/UnrealTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ARFilter)

namespace UE::AssetRegistry
{
	constexpr int32 FARCompactBinaryVersion = 1;
}

namespace Algo
{
	// TODO: Think of a name for this Compare function that returns -1,0,1 and put it in Compare.h
	template <typename InAT, typename InBT, typename LessThanType>
	constexpr int CompareAsInt(InAT&& InputA, InBT&& InputB, LessThanType LessThan)
	{
		const SIZE_T SizeA = GetNum(InputA);
		const SIZE_T SizeB = GetNum(InputB);

		if (SizeA != SizeB)
		{
			return SizeA < SizeB ? -1 : 1;
		}

		auto* A = GetData(InputA);
		auto* B = GetData(InputB);

		for (SIZE_T Count = SizeA; Count; --Count)
		{
			if (LessThan(*A, *B))
			{
				return -1;
			}
			if (LessThan(*A, *B))
			{
				return 1;
			}
			A++;
			B++;
		}
		return 0;
	}

	// TODO: Write this without copying to an array and put it in Compare.h
	template <typename MapType, typename KeyLessThanType, typename ValueLessThanType>
	int CompareMultiMap(const MapType& A, const MapType& B, KeyLessThanType KeyLessThan, ValueLessThanType ValueLessThan)
	{
		// TODO: Add KeyType and ValueType to TMultiMap
		using KeyType = decltype(DeclVal<typename MapType::ElementType>().Key);
		using ValueType = decltype(DeclVal<typename MapType::ElementType>().Value);

		if (A.Num() != B.Num())
		{
			return A.Num() < B.Num() ? -1 : 1;
		}
		if (A.Num() == 0)
		{
			return 0;
		}

		auto PairLessThan = [&KeyLessThan, &ValueLessThan]
		(const TPair<KeyType, ValueType>& PairA, const TPair<KeyType, ValueType>& PairB)
			{
				if (KeyLessThan(PairA.Key, PairB.Key))
				{
					return true;
				}
				if (KeyLessThan(PairB.Key, PairA.Key))
				{
					return false;
				}
				if (ValueLessThan(PairA.Value, PairB.Value))
				{
					return true;
				}
				if (ValueLessThan(PairB.Value, PairA.Value))
				{
					return false;
				}
				return false; // equal
			};
		TArray<TPair<KeyType, ValueType>> AVals = A.Array();
		TArray<TPair<KeyType, ValueType>> BVals = B.Array();
		Algo::Sort(AVals, PairLessThan);
		Algo::Sort(BVals, PairLessThan);
		return CompareAsInt(AVals, BVals, PairLessThan);
	}
}

// CompactBinary for FSoftObjectPath; we can move these into public if necessary
static FCbWriter& operator<<(FCbWriter& Writer, const FSoftObjectPath& SoftObjectPath)
{
	Writer.BeginArray();
	Writer << SoftObjectPath.GetAssetPath();
	Writer << SoftObjectPath.GetSubPathString();
	Writer.EndArray();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FSoftObjectPath& SoftObjectPath)
{
	FTopLevelAssetPath AssetPath;
	FString SubPath;
	FCbFieldViewIterator Iter = Field.CreateViewIterator();
	bool bOk = true;
	bOk = LoadFromCompactBinary(Iter++, AssetPath) & bOk;
	bOk = LoadFromCompactBinary(Iter++, SubPath) & bOk;
	SoftObjectPath = FSoftObjectPath(MoveTemp(AssetPath), MoveTemp(SubPath));
	return bOk;
}

// Special case CompactBinary that could be replaced with general versions
static FCbWriter& operator<<(FCbWriter& Writer, const TPair<FName, TOptional<FString>>& Value)
{
	bool bHasValue = Value.Value.IsSet();
	Writer.BeginArray();
	Writer << Value.Key;
	Writer << bHasValue;
	if (bHasValue)
	{
		Writer << *Value.Value;
	}
	Writer.EndArray();
	return Writer;
}

static bool LoadFromCompactBinary(FCbFieldView Field, TPair<FName, TOptional<FString>>& Value)
{
	FCbFieldViewIterator Iter = Field.CreateViewIterator();
	bool bOk = LoadFromCompactBinary(*Iter++, Value.Key);
	bool bHasValue;
	if (LoadFromCompactBinary(*Iter++, bHasValue))
	{
		if (bHasValue)
		{
			Value.Value.Emplace();
			bOk = LoadFromCompactBinary(*Iter++, *Value.Value);
		}
	}
	else
	{
		bOk = false;
	}
	return bOk;
}

struct FOptionalFStringIgnoreCaseLess
{
	bool operator()(const TOptional<FString>& A, const TOptional<FString>& B) const
	{
		if (A.IsSet() != B.IsSet())
		{
			return !A.IsSet();
		}
		if (!A.IsSet())
		{
			return false; // Both not set, equal
		}
		return *A < *B;
	}
};

void FARFilter::SortForSaving()
{
	Algo::Sort(PackageNames, FNameLexicalLess());
	Algo::Sort(PackagePaths, FNameLexicalLess());
	Algo::Sort(SoftObjectPaths, FSoftObjectPathLexicalLess());
	Algo::Sort(ClassPaths, FTopLevelAssetPathLexicalLess());
	TagsAndValues.KeySort(FNameLexicalLess());
	TagsAndValues.ValueSort(FOptionalFStringIgnoreCaseLess());
	RecursiveClassPathsExclusionSet.Sort(FTopLevelAssetPathLexicalLess());
}

bool FARFilter::operator==(const FARFilter& Other) const
{
	// Arrays are compared without sorting; caller is responsible for calling Sort
	int32 Compare;
	Compare = Algo::CompareAsInt(PackageNames, Other.PackageNames, FNameLexicalLess());
	if (Compare != 0)
	{
		return false;
	}
	Compare = Algo::CompareAsInt(PackagePaths, Other.PackagePaths, FNameLexicalLess());
	if (Compare != 0)
	{
		return false;
	}
	Compare = Algo::CompareAsInt(SoftObjectPaths, Other.SoftObjectPaths, FSoftObjectPathLexicalLess());
	if (Compare != 0)
	{
		return false;
	}
	Compare = Algo::CompareAsInt(ClassPaths, Other.ClassPaths, FTopLevelAssetPathLexicalLess());
	if (Compare != 0)
	{
		return false;
	}
	TArray<TPair<FName, TOptional<FString>>> A;
	Compare = Algo::CompareMultiMap(TagsAndValues, Other.TagsAndValues, FNameLexicalLess(),
		FOptionalFStringIgnoreCaseLess());
	if (Compare != 0)
	{
		return false;
	}
	Compare = Algo::CompareSet(RecursiveClassPathsExclusionSet, Other.RecursiveClassPathsExclusionSet,
		FTopLevelAssetPathLexicalLess());
	if (Compare != 0)
	{
		return false;
	}
	if (bRecursivePaths != Other.bRecursivePaths)
	{
		return false;
	}
	if (bRecursiveClasses != Other.bRecursiveClasses)
	{
		return false;
	}
	if (bIncludeOnlyOnDiskAssets != Other.bIncludeOnlyOnDiskAssets)
	{
		return false;
	}
	if (WithoutPackageFlags != Other.WithoutPackageFlags)
	{
		return false;
	}
	if (WithPackageFlags != Other.WithPackageFlags)
	{
		return false;
	}
	return true;
}

bool FARFilter::operator<(const FARFilter& Other) const
{
	int32 Compare;
	Compare = Algo::CompareAsInt(PackageNames, Other.PackageNames, FNameLexicalLess());
	if (Compare != 0)
	{
		return Compare < 0;
	}
	Compare = Algo::CompareAsInt(PackagePaths, Other.PackagePaths, FNameLexicalLess());
	if (Compare != 0)
	{
		return Compare < 0;
	}
	Compare = Algo::CompareAsInt(SoftObjectPaths, Other.SoftObjectPaths, FSoftObjectPathLexicalLess());
	if (Compare != 0)
	{
		return Compare < 0;
	}
	Compare = Algo::CompareAsInt(ClassPaths, Other.ClassPaths, FTopLevelAssetPathLexicalLess());
	if (Compare != 0)
	{
		return Compare < 0;
	}
	Compare = Algo::CompareMultiMap(TagsAndValues, Other.TagsAndValues, FNameLexicalLess(),
		FOptionalFStringIgnoreCaseLess());
	if (Compare != 0)
	{
		return Compare < 0;
	}
	Compare = Algo::CompareSet(RecursiveClassPathsExclusionSet, Other.RecursiveClassPathsExclusionSet,
		FTopLevelAssetPathLexicalLess());
	if (Compare != 0)
	{
		return Compare < 0;
	}
	if (bRecursivePaths != Other.bRecursivePaths)
	{
		return !bRecursivePaths;
	}
	if (bRecursiveClasses != Other.bRecursiveClasses)
	{
		return !bRecursiveClasses;
	}
	if (bIncludeOnlyOnDiskAssets != Other.bIncludeOnlyOnDiskAssets)
	{
		return !bIncludeOnlyOnDiskAssets;
	}
	if (WithoutPackageFlags != Other.WithoutPackageFlags)
	{
		return WithoutPackageFlags < Other.WithoutPackageFlags;
	}
	if (WithPackageFlags != Other.WithPackageFlags)
	{
		return WithPackageFlags < Other.WithPackageFlags;
	}
	return false;
}

void FARFilter::Save(FCbWriter& Writer) const
{
	Writer.BeginObject();
	Writer << "Version" << UE::AssetRegistry::FARCompactBinaryVersion;
	Writer << "PackageNames" << PackageNames;
	Writer << "PackagePaths" << PackagePaths;
	Writer << "SoftObjectPaths" << SoftObjectPaths;
	Writer << "ClassPaths" << ClassPaths;
	TArray<TPair<FName, TOptional<FString>>> TagsAndValuesArray = TagsAndValues.Array();
	Writer << "TagsAndValues" << TagsAndValuesArray;
	TArray<FTopLevelAssetPath> RecursiveClassPathsExclusionSetArray = RecursiveClassPathsExclusionSet.Array();
	Writer << "RecursiveClassPathsExclusionSet" << RecursiveClassPathsExclusionSetArray;
	Writer << "bRecursivePaths" << bRecursivePaths;
	Writer << "bRecursiveClasses" << bRecursiveClasses;
	Writer << "bIncludeOnlyOnDiskAssets" << bIncludeOnlyOnDiskAssets;
	Writer << "WithoutPackageFlags" << WithoutPackageFlags;
	Writer << "WithPackageFlags" << WithPackageFlags;
	Writer.EndObject();
}

bool FARFilter::TryLoad(const FCbFieldView& Field)
{
	int32 Version = -1;
	for (FCbFieldViewIterator ElementView(Field.CreateViewIterator()); ElementView; )
	{
		const FCbFieldViewIterator Last = ElementView;
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("Version")))
		{
			if (!LoadFromCompactBinary(ElementView++, Version))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("PackageNames")))
		{
			if (!LoadFromCompactBinary(ElementView++, PackageNames))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("PackagePaths")))
		{
			if (!LoadFromCompactBinary(ElementView++, PackagePaths))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("SoftObjectPaths")))
		{
			if (!LoadFromCompactBinary(ElementView++, SoftObjectPaths))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("ClassPaths")))
		{
			if (!LoadFromCompactBinary(ElementView++, ClassPaths))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("TagsAndValues")))
		{
			TArray<TPair<FName, TOptional<FString>>> TagsAndValuesArray;
			if (!LoadFromCompactBinary(ElementView++, TagsAndValuesArray))
			{
				return false;
			}
			TagsAndValues.Empty(TagsAndValuesArray.Num());
			for (TPair<FName, TOptional<FString>>& Pair : TagsAndValuesArray)
			{
				TagsAndValues.Add(Pair.Key, Pair.Value);
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("RecursiveClassPathsExclusionSet")))
		{
			TArray<FTopLevelAssetPath> RecursiveClassPathsExclusionSetArray;
			if (!LoadFromCompactBinary(ElementView++, RecursiveClassPathsExclusionSetArray))
			{
				return false;
			}
			RecursiveClassPathsExclusionSet.Empty(RecursiveClassPathsExclusionSetArray.Num());
			RecursiveClassPathsExclusionSet.Append(RecursiveClassPathsExclusionSetArray);
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("bRecursivePaths")))
		{
			if (!LoadFromCompactBinary(ElementView++, bRecursivePaths))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("bRecursiveClasses")))
		{
			if (!LoadFromCompactBinary(ElementView++, bRecursiveClasses))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("bIncludeOnlyOnDiskAssets")))
		{
			if (!LoadFromCompactBinary(ElementView++, bIncludeOnlyOnDiskAssets))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("WithoutPackageFlags")))
		{
			if (!LoadFromCompactBinary(ElementView++, WithoutPackageFlags))
			{
				return false;
			}
		}
		if (ElementView.GetName().Equals(UTF8TEXTVIEW("WithPackageFlags")))
		{
			if (!LoadFromCompactBinary(ElementView++, WithPackageFlags))
			{
				return false;
			}
		}

		if (ElementView == Last)
		{
			++ElementView;
		}
	}
	if (Version != UE::AssetRegistry::FARCompactBinaryVersion)
	{
		return false;
	}
	return true;
}

namespace UE::AssetRegistry
{
	uint32 GetTypeHash(const FARFilter& Filter)
	{
		uint32 Hash = 0;
		Hash = HashCombineFast(Hash, GetTypeHash(Filter.PackageNames));
		Hash = HashCombineFast(Hash, GetTypeHash(Filter.PackagePaths));
		Hash = HashCombineFast(Hash, GetTypeHash(Filter.SoftObjectPaths));
		Hash = HashCombineFast(Hash, GetTypeHash(Filter.ClassPaths));

		for (TMultiMap<FName, TOptional<FString>>::TConstIterator It = Filter.TagsAndValues.CreateConstIterator(); It; ++It)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(It.Key()));

			const TOptional<FString>& Value = It.Value();
			if (Value.IsSet())
			{
				Hash = HashCombineFast(Hash, GetTypeHash(Value.GetValue()));
			}
		}

		for (const FTopLevelAssetPath& AssetPath : Filter.RecursiveClassPathsExclusionSet)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(AssetPath));
		}

		Hash = HashCombineFast(Hash, ::GetTypeHash(Filter.bRecursivePaths));
		Hash = HashCombineFast(Hash, ::GetTypeHash(Filter.bRecursiveClasses));
		Hash = HashCombineFast(Hash, ::GetTypeHash(Filter.bIncludeOnlyOnDiskAssets));
		Hash = HashCombineFast(Hash, ::GetTypeHash(Filter.WithoutPackageFlags));
		Hash = HashCombineFast(Hash, ::GetTypeHash(Filter.WithPackageFlags));

		return Hash;
	}

	FARFilter ConvertToNonCompiledFilter(const FARCompiledFilter& CompiledFilter)
	{
		FARFilter OutFilter;
		OutFilter.PackageNames = CompiledFilter.PackageNames.Array();
		OutFilter.PackagePaths = CompiledFilter.PackagePaths.Array();

		OutFilter.SoftObjectPaths = CompiledFilter.SoftObjectPaths.Array();

		OutFilter.ClassPaths = CompiledFilter.ClassPaths.Array();
		OutFilter.TagsAndValues = CompiledFilter.TagsAndValues;
		OutFilter.bIncludeOnlyOnDiskAssets = CompiledFilter.bIncludeOnlyOnDiskAssets;
		OutFilter.WithoutPackageFlags = CompiledFilter.WithoutPackageFlags;
		OutFilter.WithPackageFlags = CompiledFilter.WithPackageFlags;

		OutFilter.bRecursivePaths = false;
		OutFilter.bRecursiveClasses = false;

		return OutFilter;
	}
}
