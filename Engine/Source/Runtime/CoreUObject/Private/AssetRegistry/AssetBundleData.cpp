// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetBundleData.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/OutputDeviceNull.h"
#include "UObject/PropertyPortFlags.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetBundleData)

UE_IMPLEMENT_STRUCT("/Script/CoreUObject", AssetBundleData);


namespace UE::AssetBundleEntry::Private
{
	const FStringView BundleNamePrefix(TEXT("(BundleName=\""));
	const FStringView BundleAssetsPrefix(TEXT(",BundleAssets=("));
	const FStringView AssetPathsPrefix(TEXT(",AssetPaths=("));
	const FStringView EmptyAssetPathsPrefix(TEXT(",AssetPaths="));
	const FStringView AssetPathsSuffix(TEXT("))"));
	const FStringView BundlesPrefix(TEXT("(Bundles=("));
	const FStringView BundlesSuffix(TEXT("))"));
	const FStringView EmptyBundles(TEXT("(Bundles=)"));
	const FStringView StructOrArraySuffix(TEXT(")"));

	static bool SkipPrefix(const TCHAR*& InOutIt, FStringView Prefix)
	{
		const bool bOk = FStringView(InOutIt, Prefix.Len()) == Prefix;
		if (bOk)
		{
			InOutIt += Prefix.Len();
		}
		return bOk;
	}
}

bool FAssetBundleEntry::ExportTextItem(FString& ValueStr, const FAssetBundleEntry& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{	
	// If the DefaultValue points to this, that is the import/export system's way of specifying that there are no
	// available defaults and the entire struct should be exported. If the DefautlValue is an empty version of the
	// struct, then we also will export the entire struct. Otherwise there are actual defaults and we're supposed to
	// export a delta. This path does not handle that delta, fall back to normal export path
	if (&DefaultValue != this && DefaultValue.IsValid())
	{
		return false;
	}

	using namespace UE::AssetBundleEntry::Private;

	const uint32 OriginalLen = ValueStr.Len();

	ValueStr += BundleNamePrefix;
	BundleName.AppendString(ValueStr);
	ValueStr += '\"';

	if (AssetPaths.Num())
	{
		ValueStr += AssetPathsPrefix;
		const FSoftObjectPath EmptyAssetPath;
		for (const FTopLevelAssetPath& Path : AssetPaths)
		{
			FSoftObjectPath ObjectPath(Path, {});
			if (!ObjectPath.ExportTextItem(ValueStr, EmptyAssetPath, Parent, PortFlags, ExportRootScope))
			{
				ValueStr.LeftInline(OriginalLen);
				return false;
			}
			
			ValueStr.AppendChar(TEXT(','));
		}

		// Remove last comma
		ValueStr.LeftChopInline(1, EAllowShrinking::No);

		ValueStr += AssetPathsSuffix;
	}
	else
	{

		ValueStr += EmptyAssetPathsPrefix;
		ValueStr += StructOrArraySuffix;
	}

	return true;
}

bool FAssetBundleEntry::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	using namespace UE::AssetBundleEntry::Private;

	const TCHAR* BufferIt = Buffer;
	FName LocalBundleName;
	TArray<FSoftObjectPath> LocalObjectPaths;

	if (SkipPrefix(/* in-out */ BufferIt, BundleNamePrefix))
	{
		if (const TCHAR* BundleNameEnd = FCString::Strchr(BufferIt, TCHAR('"')))
		{
			LocalBundleName = FName(UE_PTRDIFF_TO_INT32(BundleNameEnd - BufferIt), BufferIt);
			BufferIt = BundleNameEnd + 1;
		}
		else
		{
			ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - Unterminated BundleName string. When parsing import text '%s'."), Buffer);
			return false;
		}
	}
	else
	{
		ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - Text does not start with expected prefix '%.*s'. When parsing import text '%s'."), 
			BundleNamePrefix.Len(), BundleNamePrefix.GetData(), Buffer);
		return false;
	}

	FOutputDeviceNull SilenceErrors;
	bool bBundleAssetsSeen = false;
	bool bAssetPathsSeen = false;
	while (true)
	{
		// Parse deprecated format 
		if (SkipPrefix(/* in-out */ BufferIt, BundleAssetsPrefix))
		{
			if (bBundleAssetsSeen)
			{
				// Parse error 
				ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - Encountered key BundleAssets twice. When parsing import text '%s'."), Buffer);
				return false;
			}
			bBundleAssetsSeen = true;

			if (SkipPrefix(/* in-out */ BufferIt, StructOrArraySuffix))
			{
				// End of array
				continue;
			}

			while (LocalObjectPaths.Emplace_GetRef().ImportTextItem(/* in-out */ BufferIt, PortFlags, Parent, ErrorText))
			{
				if(!LocalObjectPaths.Last().GetSubPathString().IsEmpty())
				{
					ErrorText->Logf(ELogVerbosity::Warning, TEXT("FAssetBundleEntry::ImportTextItem - Asset bundle entries should all be top level objects, but found a subobject path '%s'. When parsing import text '%s'."), 
						*LocalObjectPaths.Last().ToString(), Buffer);
				}

				if (*BufferIt == ',')
				{
					++BufferIt; // We expect another path
				}
				else if (SkipPrefix(/* in-out */ BufferIt, StructOrArraySuffix))
				{
					// End of array
					break;
				}
				else
				{
					// Parse error 
					ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - Unterminated or ill formed BundleAssets list while importing asset bundle entry. When parsing import text '%s'."), Buffer);
					return false;
				}
			}
		}
		else if (SkipPrefix(/* in-out */ BufferIt, AssetPathsPrefix))
		{
			if (bAssetPathsSeen)
			{
				// Parse error 
				ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - Encountered key AssetPaths twice. When parsing import text '%s'."), Buffer);
				return false;
			}
			bAssetPathsSeen = true;

			if (SkipPrefix(/* in-out */ BufferIt, StructOrArraySuffix))
			{
				// End of array
				continue;
			}

			// Try multiple formats to parse our list of FTopLevelAssetPaths
			//     - the native export converts FTopLevelAssetPaths to FSoftObjectPaths and exports the FSoftObjectPaths as strings
			//     - the default export path (activated when defaults is not null) delegates to FTopLevelAssetPath
			FSoftObjectPath TempObjectPath;
			FTopLevelAssetPath TempTopLevelPath;
			while (true)
			{
				TempObjectPath.Reset();
				TempTopLevelPath.Reset();
				// FSoftObjectPath format 
				if (TempObjectPath.ImportTextItem(BufferIt, PortFlags, Parent, &SilenceErrors))
				{
					if (!TempObjectPath.GetSubPathString().IsEmpty())
					{
						// Parse error, these should only have top-level paths
						ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - Asset bundle entries should all be top level objects, but found a subobject path '%s'. When parsing import text '%s'."), 
							*TempObjectPath.ToString(), Buffer);
						return false;
					}
					LocalObjectPaths.Emplace(MoveTemp(TempObjectPath));
				}
				// FTopLevelAssetPath's native ExportText will export a string that can also be parsed by FSoftObjectPath so this won't be called unless some stale data is in structured text form
				else if(TempTopLevelPath.ImportTextItem(BufferIt, PortFlags, Parent, &SilenceErrors))
				{
					LocalObjectPaths.Emplace(TempTopLevelPath);
				}
				else
				{					
					ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - unterminated or ill formed AssetPaths list while importing asset bundle entry: '%s'. When parsing import text '%s'."), BufferIt, Buffer);
					return false;
				}

				if (*BufferIt == ',')
				{
					++BufferIt; // We expect another path
				}
				else if (SkipPrefix(/* in-out */ BufferIt, StructOrArraySuffix))
				{
					// End of array
					break;
				}
				else
				{
					// Parse error 
					ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - unterminated or ill formed AssetPaths list while importing asset bundle entry: '%s'. When parsing import text '%s'."), BufferIt, Buffer);
					return false;
				}
			}
		}
		else if (SkipPrefix(/* in-out */ BufferIt, EmptyAssetPathsPrefix))
		{
			// Empty array 
			continue;
		}
		else if (SkipPrefix(/* in-out */ BufferIt, StructOrArraySuffix))
		{
			// End
			break;
		}
		else
		{
			// Parse error 
			ErrorText->Logf(ELogVerbosity::Error, TEXT("FAssetBundleEntry::ImportTextItem - Ill-formed asset bundle entry, unexpected: '%s'. When parsing import text '%s'."), BufferIt, Buffer);
			return false;
		}
	}

	BundleName = LocalBundleName;
	AssetPaths.Reset(LocalObjectPaths.Num());
	for (const FSoftObjectPath& Path : LocalObjectPaths)
	{
		AssetPaths.Add(Path.GetAssetPath());
	}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	BundleAssets.Reset(AssetPaths.Num());
	for (const FTopLevelAssetPath& Path : AssetPaths)
	{
		BundleAssets.Add(FSoftObjectPath(Path, {}));
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	// Success, update how much we read for the caller
	Buffer = BufferIt;
	return true;
}

FAssetBundleEntry* FAssetBundleData::FindEntry(FName SearchName)
{
	for (FAssetBundleEntry& Entry : Bundles)
	{
		if (Entry.BundleName == SearchName)
		{
			return &Entry;
		}
	}
	return nullptr;
}


void FAssetBundleData::AddBundleAsset(FName BundleName, const FTopLevelAssetPath& AssetPath)
{
	if (!AssetPath.IsValid())
	{
		return;
	}

	FAssetBundleEntry* FoundEntry = FindEntry(BundleName);

	if (!FoundEntry)
	{
		FoundEntry = &Bundles.Emplace_GetRef(BundleName);
	}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FoundEntry->BundleAssets.AddUnique(FSoftObjectPath(AssetPath, {}));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	FoundEntry->AssetPaths.AddUnique(AssetPath);
}

void FAssetBundleData::AddBundleAssets(FName BundleName, const TArray<FTopLevelAssetPath>& AssetPaths)
{
	FAssetBundleEntry* FoundEntry = FindEntry(BundleName);

	for (const FTopLevelAssetPath& Path : AssetPaths)
	{
		if (Path.IsValid())
		{
			// Only create if required
			if (!FoundEntry)
			{
				FoundEntry = &Bundles.Emplace_GetRef(BundleName);
			}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FoundEntry->BundleAssets.AddUnique(FSoftObjectPath(Path, {}));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
			FoundEntry->AssetPaths.AddUnique(Path);
		}
	}
}

void FAssetBundleData::SetBundleAssets(FName BundleName, TArray<FTopLevelAssetPath>&& AssetPaths)
{
	FAssetBundleEntry* FoundEntry = FindEntry(BundleName);

	if (!FoundEntry)
	{
		FoundEntry = &Bundles.Emplace_GetRef(BundleName);
	}

	FoundEntry->AssetPaths = MoveTemp(AssetPaths);
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FoundEntry->BundleAssets.Reset();
	Algo::Transform(FoundEntry->AssetPaths, FoundEntry->BundleAssets, [](FTopLevelAssetPath Path) { return FSoftObjectPath(Path, {}); });
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

void FAssetBundleData::AddBundleAssetTruncated(FName BundleName, const FSoftObjectPath& AssetPath)
{
	if (!AssetPath.IsValid())
	{
		return;
	}

	FAssetBundleEntry* FoundEntry = FindEntry(BundleName);

	if (!FoundEntry)
	{
		FoundEntry = &Bundles.Emplace_GetRef(BundleName);
	}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FoundEntry->BundleAssets.AddUnique(AssetPath.GetWithoutSubPath());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	FoundEntry->AssetPaths.AddUnique(AssetPath.GetAssetPath());
}

void FAssetBundleData::AddBundleAssetsTruncated(FName BundleName, const TArray<FSoftObjectPath>& AssetPaths)
{
	FAssetBundleEntry* FoundEntry = FindEntry(BundleName);

	for (const FSoftObjectPath& Path : AssetPaths)
	{
		if (Path.IsValid())
		{
			// Only create if required
			if (!FoundEntry)
			{
				FoundEntry = &Bundles.Emplace_GetRef(BundleName);
			}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FoundEntry->BundleAssets.AddUnique(Path.GetWithoutSubPath());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
			FoundEntry->AssetPaths.AddUnique(Path.GetAssetPath());
		}
	}
}

void FAssetBundleData::SetBundleAssetsTruncated(FName BundleName, const TArray<FSoftObjectPath>& AssetPaths)
{
	FAssetBundleEntry* FoundEntry = FindEntry(BundleName);

	if (!FoundEntry)
	{
		FoundEntry = &Bundles.Emplace_GetRef(BundleName);
	}

	FoundEntry->AssetPaths.Reset(AssetPaths.Num());
	for (const FSoftObjectPath& Path : AssetPaths)
	{	
		FoundEntry->AssetPaths.Add(Path.GetAssetPath());
	}

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FoundEntry->BundleAssets.Reset();
	Algo::Transform(FoundEntry->AssetPaths, FoundEntry->BundleAssets, [](FTopLevelAssetPath Path) { return FSoftObjectPath(Path); });
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif 
}

void FAssetBundleData::Reset()
{
	Bundles.Reset();
}

bool FAssetBundleData::ExportTextItem(FString& ValueStr, FAssetBundleData const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (Bundles.Num() == 0)
	{
		// Empty, don't write anything to avoid it cluttering the asset registry tags
		return true;
	}
	// If the DefaultValue points to this, that is the import/export system's way of specifying that there are no
	// available defaults and the entire struct should be exported. If the DefautlValue is an empty version of the
	// struct, then we also will export the entire struct. Otherwise there are actual defaults and we're supposed to
	// export a delta. This path does not handle that delta, fall back to normal export path
	else if (&DefaultValue != this && DefaultValue.Bundles.Num() != 0)
	{
		return false;
	}
	
	using namespace UE::AssetBundleEntry::Private;

	const uint32 OriginalLen = ValueStr.Len();

	ValueStr += BundlesPrefix;

	const FAssetBundleEntry EmptyEntry;
	for (const FAssetBundleEntry& Entry : Bundles)
	{
		if (!Entry.ExportTextItem(ValueStr, EmptyEntry, Parent, PortFlags, ExportRootScope))
		{
			ValueStr.LeftInline(OriginalLen);
			return false;
		}
		ValueStr.AppendChar(TEXT(','));
	}

	// Remove last comma
	ValueStr.LeftChopInline(1, EAllowShrinking::No);

	ValueStr += BundlesSuffix;

	return true;
}

bool FAssetBundleData::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	if (*Buffer != TEXT('('))
	{
		// Empty, don't read/write anything
		return true;
	}

	using namespace UE::AssetBundleEntry::Private;
	
	const TCHAR* It = Buffer;
	if (SkipPrefix(/* in-out */ It, BundlesPrefix))
	{
		TArray<FAssetBundleEntry> Entries;
		while (Entries.Emplace_GetRef().ImportTextItem(/* in-out */ It, PortFlags, Parent, ErrorText))
		{
			if (*It == ',')
			{
				++It;
			}
			else if (SkipPrefix(/* in-out */ It, BundlesSuffix))
			{
				Bundles = MoveTemp(Entries);
				Buffer = It;
				
				return true;
			}
			else
			{
				return false;
			}
		}
	}

	return SkipPrefix(/* in-out */ Buffer, EmptyBundles);
}

FString FAssetBundleData::ToDebugString() const
{
	TStringBuilder<220> Result;

	bool bFirstLine = true;
	for (const FAssetBundleEntry& Data : Bundles)
	{
		if (!bFirstLine)
		{
			Result.Append(TEXT("\n"));
		}

		Result.Appendf(TEXT("%s -> (%s)"),
			*Data.BundleName.ToString(),
			*FString::JoinBy(Data.AssetPaths, TEXT(", "), [](const FTopLevelAssetPath& LoadBundle) { return LoadBundle.ToString(); })
		);

		bFirstLine = false;
	}

	return Result.ToString();
}


#if WITH_DEV_AUTOMATION_TESTS 

#include "Misc/AutomationTest.h"

// Combine import/export tests

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetBundleEntryImportExportTextTest, "System.AssetRegistry.AssetBundleEntry.ImportExportText", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAssetBundleEntryImportExportTextTest::RunTest(const FString& Parameters)
{
	FAssetBundleEntry DefaultEntry;

	FAssetBundleEntry EmptyEntry;
	EmptyEntry.BundleName = "Empty";
	FString EmptyEntryString = "(BundleName=\"Empty\",AssetPaths=)";

	FAssetBundleEntry SingleAssetEntry;
	SingleAssetEntry.BundleName = "Single";
	SingleAssetEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Characters/Steve.Steve")));
	FString SingleAssetEntryString = "(BundleName=\"Single\",AssetPaths=(\"/Game/Characters/Steve.Steve\"))";

	FAssetBundleEntry MultiAssetEntry;
	MultiAssetEntry.BundleName = "Multi";
	MultiAssetEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Meshes/ConcreteWall.ConcreteWall")));
	MultiAssetEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Meshes/GarbageCan.GarbageCan")));
	MultiAssetEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Blueprints/Widget.Widget_C")));
	FString MultiAssetEntryString = "(BundleName=\"Multi\",AssetPaths=(\"/Game/Meshes/ConcreteWall.ConcreteWall\",\"/Game/Meshes/GarbageCan.GarbageCan\",\"/Game/Blueprints/Widget.Widget_C\"))";

	FString EmptyExported;
	TestTrue(TEXT("Empty asset bundle exports to text"), EmptyEntry.ExportTextItem(EmptyExported, FAssetBundleEntry{}, nullptr, PPF_Delimited, nullptr));
	TestEqual(TEXT("Empty asset bundle exports to correct string"), EmptyExported, EmptyEntryString);

	FString SingleExported;
	TestTrue(TEXT("Single-asset asset bundle exports to text"), SingleAssetEntry.ExportTextItem(SingleExported, FAssetBundleEntry{}, nullptr, PPF_Delimited, nullptr));
	TestEqual(TEXT("Single-asset asset bundle exports to correct string"), SingleExported, SingleAssetEntryString);

	FString MultiExported;
	TestTrue(TEXT("Multi-asset asset bundle exports to text"), MultiAssetEntry.ExportTextItem(MultiExported, FAssetBundleEntry{}, nullptr, PPF_Delimited, nullptr));
	TestEqual(TEXT("Multi-asset asset bundle exports to correct string"), MultiExported, MultiAssetEntryString);

	const TCHAR* EmptyBuffer = *EmptyEntryString;
	FAssetBundleEntry EmptyImported;
	TestTrue(TEXT("Empty asset bundle imports from text"), EmptyImported.ImportTextItem(EmptyBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Empty asset bundle imports correctly"), EmptyImported, EmptyEntry);

	const TCHAR* SingleBuffer = *SingleAssetEntryString;
	FAssetBundleEntry SingleImported;
	TestTrue(TEXT("Single-asset asset bundle imports from text"), SingleImported.ImportTextItem(SingleBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Single-asset asset bundle imports correctly"), SingleImported, SingleAssetEntry);

	const TCHAR* MultiBuffer = *MultiAssetEntryString;
	FAssetBundleEntry MultiImported;
	TestTrue(TEXT("Multi-asset asset bundle imports from text"), MultiImported.ImportTextItem(MultiBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Multi-asset asset bundle imports correctly"), MultiImported, MultiAssetEntry);

	AddExpectedError(TEXT("FAssetBundleEntry::ImportTextItem - Asset bundle entries should all be top level objects"));
	FString SubobjectEntryString = "(BundleName=\"Subobject\",AssetPaths=(\"/Game/Characters/Steve.Steve:Hat\"))";
	const TCHAR* SubobjectBuffer = *SubobjectEntryString;
	FAssetBundleEntry SubobjectImported;
	TestFalse(TEXT("Subobject bundle does not import from text"), SubobjectImported.ImportTextItem(SubobjectBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Subobject bundle does not import from text"), SubobjectImported, DefaultEntry);

	AddExpectedError(TEXT("FAssetBundleEntry::ImportTextItem - Unterminated BundleName string"));
	FString UnterminatedBundleNameString = "(BundleName=\"TestUnterminated,BundleAssets=))";
	const TCHAR* UnterminatedBundleNameBuffer = *UnterminatedBundleNameString;
	FAssetBundleEntry UnterminatedEntry;
	TestFalse(TEXT("Unterminated asset bundle name fails import"), UnterminatedEntry.ImportTextItem(UnterminatedBundleNameBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Unterminated asset bundle name fails import"), UnterminatedEntry, DefaultEntry);

	AddExpectedError(TEXT("AssetBundleEntry::ImportTextItem - Ill-formed asset bundle entry"));
	FString MismatchedQuotesString = "(BundleName=\"TestUnterminated,BundleAssets=(\"/Game/Characters/Steve.Steve\")))";
	const TCHAR* MismatchedQuotesBuffer = *MismatchedQuotesString;
	FAssetBundleEntry MismatchedQuotesEntry;
	TestFalse(TEXT("Mismatched quotes asset bundle name fails import"), MismatchedQuotesEntry.ImportTextItem(MismatchedQuotesBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Mismatched quotes asset bundle name fails import"), MismatchedQuotesEntry, DefaultEntry);

	AddExpectedError(TEXT("FAssetBundleEntry::ImportTextItem - Text does not start with expected prefix"));
	FString UnsupportedOrderString = "(BundleAssets=(\"/Game/Characters/Steve.Steve\"), BundleName=\"UnsupportedOrder\")";
	const TCHAR* UnsupportedOrderBuffer = *UnsupportedOrderString;
	FAssetBundleEntry UnsupportedOrderEntry;
	TestFalse(TEXT("Unsupported order fails import"), UnsupportedOrderEntry.ImportTextItem(UnsupportedOrderBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Unsupported order fails import"), UnsupportedOrderEntry, DefaultEntry);

	AddExpectedError(TEXT("FAssetBundleEntry::ImportTextItem - Encountered key BundleAssets twice"));
	FString DuplicateKeysString = "(BundleName=\"DuplicateKeys\",BundleAssets=(\"/Game/Characters/Steve.Steve\"),BundleAssets=(\"/Game/Characters/Will.Will\"))";
	const TCHAR* DuplicateKeysBuffer = *DuplicateKeysString;
	FAssetBundleEntry DuplicateKeysEntry;
	TestFalse(TEXT("Duplicate keys fails import"), DuplicateKeysEntry.ImportTextItem(DuplicateKeysBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Duplicate keys fails import"), DuplicateKeysEntry, DefaultEntry);

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLegacyAssetBundleEntryTest, "System.AssetRegistry.AssetBundleEntry.LegacyAssetBundleEntry", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
// Test ImportText for old asset bundles with BundleAssets field
bool FLegacyAssetBundleEntryTest::RunTest(const FString& Parameters)
{
	// Test subobject path truncation
	FAssetBundleEntry DefaultEntry;

	FAssetBundleEntry EmptyEntry;
	EmptyEntry.BundleName = "Empty";
	FString EmptyEntryString = "(BundleName=\"Empty\",AssetPaths=())";
	FString EmptyEntryString2 = "(BundleName=\"Empty\",AssetPaths=)";

	FAssetBundleEntry SingleAssetEntry;
	SingleAssetEntry.BundleName = "Single";
	SingleAssetEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Characters/Steve.Steve")));
	FString SingleAssetEntryString = "(BundleName=\"Single\",BundleAssets=(\"/Game/Characters/Steve.Steve\"))";

	FAssetBundleEntry MultiAssetEntry;
	MultiAssetEntry.BundleName = "Multi";
	MultiAssetEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Meshes/ConcreteWall.ConcreteWall")));
	MultiAssetEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Meshes/GarbageCan.GarbageCan")));
	MultiAssetEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Blueprints/Widget.Widget_C")));
	FString MultiAssetEntryString = "(BundleName=\"Multi\",BundleAssets=(\"/Game/Meshes/ConcreteWall.ConcreteWall\",\"/Game/Meshes/GarbageCan.GarbageCan\",\"/Game/Blueprints/Widget.Widget_C\"))";

	FAssetBundleEntry SubobjectEntry;
	SubobjectEntry.BundleName = "Subobject";
	SubobjectEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Characters/Steve.Steve")));
	FString SubobjectEntryString = "(BundleName=\"Subobject\",BundleAssets=(\"/Game/Characters/Steve.Steve:Hat\"))";

	const TCHAR* EmptyBuffer = *EmptyEntryString;
	FAssetBundleEntry EmptyImported;
	TestTrue(TEXT("Empty asset bundle imports from text"), EmptyImported.ImportTextItem(EmptyBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Empty asset bundle imports correctly"), EmptyImported, EmptyEntry);

	const TCHAR* EmptyBuffer2 = *EmptyEntryString2;
	FAssetBundleEntry EmptyImported2;
	TestTrue(TEXT("Empty asset bundle without inner parens imports from text"), EmptyImported2.ImportTextItem(EmptyBuffer2, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Empty asset bundle without inner parens imports correctly"), EmptyImported2, EmptyEntry);

	const TCHAR* SingleBuffer = *SingleAssetEntryString;
	FAssetBundleEntry SingleImported;
	TestTrue(TEXT("Single-asset asset bundle imports from text"), SingleImported.ImportTextItem(SingleBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Single-asset asset bundle imports correctly"), SingleImported, SingleAssetEntry);

	const TCHAR* MultiBuffer = *MultiAssetEntryString;
	FAssetBundleEntry MultiImported;
	TestTrue(TEXT("Multi-asset asset bundle imports from text"), MultiImported.ImportTextItem(MultiBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Multi-asset asset bundle imports correctly"), MultiImported, MultiAssetEntry);

	const TCHAR* SubobjectBuffer = *SubobjectEntryString;
	FAssetBundleEntry SubobjectImported;
	TestTrue(TEXT("Subobject asset bundle imports from text"), SubobjectImported.ImportTextItem(SubobjectBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Subobject asset bundle imports correctly"), SubobjectImported, SubobjectEntry);

	FAssetBundleEntry MixedFormatEntry;
	MixedFormatEntry.BundleName = "MixedFormat";
	MixedFormatEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Meshes/ConcreteWall.ConcreteWall")));
	MixedFormatEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Meshes/GarbageCan.GarbageCan")));
	MixedFormatEntry.AssetPaths.Add(FTopLevelAssetPath(TEXT("/Game/Blueprints/Widget.Widget_C")));
	FString MixedFormatString = "(BundleName=\"MixedFormat\",AssetPaths=(\"/Game/Meshes/ConcreteWall.ConcreteWall\"),BundleAssets=(\"/Game/Meshes/GarbageCan.GarbageCan\",\"/Game/Blueprints/Widget.Widget_C\"))";
	const TCHAR* MixedFormatBuffer = *MixedFormatString;
	FAssetBundleEntry MixedFormatImported;
	TestTrue(TEXT("Mixed format asset bundle imports from text"), MixedFormatImported.ImportTextItem(MixedFormatBuffer, PPF_Delimited, nullptr, GLog->Get()));
	TestEqual(TEXT("Mixed format asset bundle imports correctly"), MixedFormatImported, MixedFormatEntry);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetBundlDataImportExportTextTest, "System.AssetRegistry.AssetBundleData.ImportExportText", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAssetBundlDataImportExportTextTest::RunTest(const FString& Parameters)
{
	UScriptStruct* Struct = TBaseStructure<FAssetBundleData>::Get();

	FAssetBundleData AssetBundles;
	FAssetBundleEntry& Entry = AssetBundles.Bundles.Add_GetRef(FAssetBundleEntry{});
	Entry.BundleName = "TestBundle";
	Entry.AssetPaths.Add(FTopLevelAssetPath("/Game/Characters/Steve.Steve"));

	constexpr const TCHAR* NullTChar = (const TCHAR*)nullptr;

	// Round trip a data with a single asset bundle with a single asset
	FString Exported;
	Struct->ExportText(Exported, &AssetBundles, nullptr, nullptr, PPF_None, nullptr);
	if (TestFalse("Single-asset bundle: Export with no defaults: Exported text not empty", Exported.IsEmpty()))
	{
		FAssetBundleData Imported1;
		if (TestNotEqual("Single-asset bundle: Export with no defaults: Asset bundle data re-imports successfully",
						 Struct->ImportText(*Exported, &Imported1, nullptr, PPF_None, GLog->Get(), Struct->GetName()), NullTChar))
		{
			TestEqual("Single-asset bundle: Export with no defaults: Re-imported asset bundle data matches", Imported1, AssetBundles);
		}
	}

	Exported.Reset();
	Struct->ExportText(Exported, &AssetBundles, &AssetBundles, nullptr, PPF_None, nullptr); // Export with reference to defaults as the same pointer
	if (TestFalse("Single-asset bundle: Export with defaults: Exported text not empty", Exported.IsEmpty()))
	{
		FAssetBundleData Imported2;
		if (TestNotEqual("Single-asset bundle: Export with defaults: Asset bundle data re-imports successfully",
						 Struct->ImportText(*Exported, &Imported2, nullptr, PPF_None, GLog->Get(), Struct->GetName()), NullTChar))
		{
			TestEqual("Single-asset bundle: Export with defaults: Re-imported asset bundle data matches", Imported2, AssetBundles);
		}
	}

	// Test with multiple asset paths
	Entry.BundleName = "TestBundle";
	Entry.AssetPaths.Add(FTopLevelAssetPath("/Game/Characters/Geoff.Geoff"));
	Entry.AssetPaths.Add(FTopLevelAssetPath("/Game/Characters/Mary.Mary"));

	Exported.Reset();
	Struct->ExportText(Exported, &AssetBundles, nullptr, nullptr, PPF_None, nullptr);
	if (TestFalse("Multi-asset bundle: Export with no defaults: Exported text not empty", Exported.IsEmpty()))
	{
		FAssetBundleData Imported3;
		if (TestNotEqual("Multi-asset bundle: Export with no defaults: Asset bundle data re-imports successfully",
						 Struct->ImportText(*Exported, &Imported3, nullptr, PPF_None, GLog->Get(), Struct->GetName()), NullTChar))
		{
			TestEqual("Multi-asset bundle: Export with no defaults: Re-imported asset bundle data matches", Imported3, AssetBundles);
		}
	}

	Exported.Reset();
	Struct->ExportText(Exported, &AssetBundles, &AssetBundles, nullptr, PPF_None, nullptr); // Export with reference to defaults as the same pointer
	if (TestFalse("Multi-asset bundle: Export with defaults: Exported text not empty", Exported.IsEmpty()))
	{
		FAssetBundleData Imported4;
		if (TestNotEqual("Multi-asset bundle: Export with defaults: Asset bundle data re-imports successfully",
						 Struct->ImportText(*Exported, &Imported4, nullptr, PPF_None, GLog->Get(), Struct->GetName()), NullTChar))
		{
			TestEqual("Multi-asset bundle: Export with defaults: Re-imported asset bundle data matches", Imported4, AssetBundles);
		}
	}
	return true;
}
#endif // WITH_DEV_AUTOMATION_TESTS
