// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetData.h"

#include "Algo/IsSorted.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetRegistry/ARFilter.h"
#include "Containers/Set.h"
#include "Containers/VersePath.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformMath.h"
#include "Misc/AsciiSet.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/CustomVersion.h"
#include "String/Find.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectHash.h"
#include "VerseVM/VVMVerseClass.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetData)

DEFINE_LOG_CATEGORY(LogAssetData);

UE_IMPLEMENT_STRUCT("/Script/CoreUObject", ARFilter);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", AssetData);

// Register Asset Registry version
const FGuid FAssetRegistryVersion::GUID(0x717F9EE7, 0xE9B0493A, 0x88B39132, 0x1B388107);
FCustomVersionRegistration GRegisterAssetRegistryVersion(FAssetRegistryVersion::GUID, FAssetRegistryVersion::LatestVersion, TEXT("AssetRegistry"));

void FAssetIdentifier::WriteCompactBinary(FCbWriter& Writer) const
{
	Writer.BeginArray();
	FName PrimaryAssetTypeName = (FName)PrimaryAssetType;
	Writer << PrimaryAssetTypeName;
	Writer << PackageName;
	if (!ObjectName.IsNone())
	{
		Writer << ObjectName;
	}
	Writer.EndArray();
}

bool LoadFromCompactBinary(FCbFieldView Field, FAssetIdentifier& Identifier)
{
	FCbArrayView ArrayView = Field.AsArrayView();
	if (ArrayView.Num() < 2)
	{
		Identifier = FAssetIdentifier();
		return false;
	}
	FCbFieldViewIterator Iter = ArrayView.CreateViewIterator();
	FName PrimaryAssetTypeName;
	if (LoadFromCompactBinary(Iter++, PrimaryAssetTypeName))
	{
		Identifier.PrimaryAssetType = PrimaryAssetTypeName;
	}
	else
	{
		Identifier = FAssetIdentifier();
		return false;
	}
	if (!LoadFromCompactBinary(Iter++, Identifier.PackageName))
	{
		return false;
	}
	if (ArrayView.Num() >= 3)
	{
		if (!LoadFromCompactBinary(Iter++, Identifier.ObjectName))
		{
			Identifier = FAssetIdentifier();
			return false;
		}
	}
	return true;
}

void SerializeForLog(FCbWriter& Writer, const FAssetIdentifier& Value)
{
	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("AssetIdentifier"));
	TStringBuilder<256> Text;
	Value.AppendString(Text);
	Writer.AddString(ANSITEXTVIEW("$text"), Text);
	Writer.AddString(ANSITEXTVIEW("PackageName"), WriteToUtf8String<256>(Value.PackageName));
	Writer.AddString(ANSITEXTVIEW("PrimaryAssetType"), WriteToUtf8String<256>(Value.PrimaryAssetType.GetName()));
	Writer.AddString(ANSITEXTVIEW("ObjectName"), WriteToUtf8String<256>(Value.ObjectName));
	Writer.AddString(ANSITEXTVIEW("ValueName"), WriteToUtf8String<256>(Value.ValueName));
	Writer.EndObject();
}

namespace UE::AssetRegistry::Private
{
	FAssetPathParts SplitIntoOuterPathAndAssetName(FStringView InObjectPath)
	{
		constexpr FAsciiSet Delimiters(SUBOBJECT_DELIMITER ".");
		FStringView OuterPathPlusDelimiter = FAsciiSet::TrimSuffixWithout(InObjectPath, Delimiters);

		return FAssetPathParts{
			OuterPathPlusDelimiter.LeftChop(1),
			InObjectPath.RightChop(OuterPathPlusDelimiter.Len())
		};
	}

	void ConcatenateOuterPathAndObjectName(FStringBuilderBase& Builder, FName OuterPath, FName ObjectName)
	{
		// We assume that OuterPath was correctly constructed with a subobject delimiter if it needed one
		// So we only need to decide if OuterPath and ObjectName should be separated by '.' or ':'
		// See UObjectBaseUtility::GetPathName.
		// We don't have access to type information here so the best we can do is rely on the fact that we don't have
		// UPackage anywhere but top-level and ensure that the second delimiter in any path string is a ':'

		int32 StartingLen = Builder.Len();
		Builder << OuterPath;

		TCHAR Delimiter = '.';

		FStringView OuterPathView = Builder.ToView().Mid(StartingLen, Builder.Len() - StartingLen);
		int32 DotIndex = 0;
		if (OuterPathView.FindChar('.', DotIndex))
		{
			// Contains a dot delimiter, so we may need to use SUBOBJECT_DELIMITER_CHAR
			int32 SubobjectDelimIndex = 0;
			if (!OuterPathView.RightChop(DotIndex + 1).FindChar(SUBOBJECT_DELIMITER_CHAR, SubobjectDelimIndex))
			{
				// No delimiter, OuterPath must be of the form 'A.B' so we need SUBOBJECT_DELIMITER_CHAR to produce full path 'A.B:C'
				Delimiter = SUBOBJECT_DELIMITER_CHAR;
			}
		}

		Builder << Delimiter << ObjectName;
	}

	class FVersePathHelper
	{
	public:
		FVersePathHelper() = default;
		FVersePathHelper(const FVersePathHelper&) = delete;

		FVersePathHelper& operator=(const FVersePathHelper&) = delete;

		bool IsGeneratedBlueprintClass(const FTopLevelAssetPath& AssetClassPath)
		{
			// The Asset Registry already has a threadsafe list of blueprint generated classes internally,
			// we'd like to expose that through the IAssetRegistryInterface so we can query it.
			check(IsInGameThread());

			const uint64 ExpectedVersionNumber = GetRegisteredNativeClassesVersionNumber();
			if (!VersionNumber.IsSet() || VersionNumber.GetValue() != ExpectedVersionNumber)
			{
				VersionNumber = ExpectedVersionNumber;
				GeneratedBlueprintClassPaths.Reset();

				const UClass* BlueprintGeneratedClass = FindObject<UClass>(GetClassPathBlueprintGeneratedClass());
				if (BlueprintGeneratedClass != nullptr)
				{
					TArray<UClass*> DerivedClasses;
					GetDerivedClasses(BlueprintGeneratedClass, DerivedClasses, true);

					GeneratedBlueprintClassPaths.Reserve(1 + DerivedClasses.Num());
					if (!BlueprintGeneratedClass->HasAnyClassFlags(CLASS_Abstract))
					{
						GeneratedBlueprintClassPaths.Add(BlueprintGeneratedClass->GetClassPathName());
					}
					for (const UClass* DerivedClass : DerivedClasses)
					{
						if (!DerivedClass->HasAnyClassFlags(CLASS_Abstract))
						{
							GeneratedBlueprintClassPaths.Add(DerivedClass->GetClassPathName());
						}
					}
				}
			}

			return GeneratedBlueprintClassPaths.Contains(AssetClassPath);
		}

	private:
		TOptional<uint64> VersionNumber;
		TSet<FTopLevelAssetPath> GeneratedBlueprintClassPaths;
	} GVersePathHelper;
}

const FName GAssetBundleDataName("AssetBundleData");

namespace UE { namespace AssetData { namespace Private {

static TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe> ParseAssetBundles(const TCHAR* Text, const FAssetData& Context)
{
	// Register that the SoftObjectPaths we read in the FAssetBundleEntry::BundleAssets are non-package data and don't need to be tracked
	FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None, ESoftObjectPathCollectType::NonPackage, ESoftObjectPathSerializeType::AlwaysSerialize);

	FAssetBundleData Temp;
	if (!Temp.ImportTextItem(Text, PPF_None, nullptr, (FOutputDevice*)GWarn))
	{
		// Native UScriptStruct isn't available during early cooked asset registry preloading.
		// Preloading should not require this fallback.
		
		UScriptStruct& Struct = *TBaseStructure<FAssetBundleData>::Get();
		Struct.ImportText(Text, &Temp, nullptr, PPF_None, (FOutputDevice*)GWarn, 
							[&]() { return Context.AssetName.ToString(); });
	}
	
	if (Temp.Bundles.Num() > 0)
	{
		return TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>(new FAssetBundleData(MoveTemp(Temp)));
	}

	return TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>();
}

}}} // end namespace UE::AssetData::Private

namespace UE::AssetRegistry
{

class FChunkArrayRegistryEntry
{
public:
	FChunkArrayRegistryEntry(FAssetData::FChunkArray&& InChunkArray)
		: ChunkArray(MoveTemp(InChunkArray))
	{
	}

	bool operator==(const FChunkArrayRegistryEntry& OtherEntry) const
	{
		// Arrays are guaranteed to be sorted/unique when an entry is created
		return ChunkArray == OtherEntry.ChunkArray;
	}

	FAssetData::FChunkArrayView GetChunkIDs() const
	{
		return ChunkArray;
	}

	friend uint32 GetTypeHash(const FChunkArrayRegistryEntry& InEntry)
	{
		uint32 Hash = 0;
		for (int32 ChunkID : InEntry.ChunkArray)
		{
			Hash = ::HashCombineFast(Hash, ::GetTypeHash(ChunkID));
		}
		return Hash;
	}

	SIZE_T GetAllocatedSize() const
	{
		return ChunkArray.GetAllocatedSize();
	}

private:
	/* Array of chunk IDs that's guaranteed to be unique and sorted in ascending order. */
	FAssetData::FChunkArray ChunkArray;
};

class FChunkArrayRegistry
{
public:
	FChunkArrayRegistryHandle FindOrAdd(const FAssetData::FChunkArrayView& InChunkIDs)
	{
		// Make a copy so we can sort it inside; these are very small arrays generally
		FAssetData::FChunkArray ChunkArray = FAssetData::FChunkArray(InChunkIDs);
		return FindOrAdd(MoveTemp(ChunkArray));
	}

	FChunkArrayRegistryHandle FindOrAdd(FAssetData::FChunkArray&& InChunkIDs)
	{
		// Sort and remove duplicates before inserting
		Algo::Sort(InChunkIDs);
		return FindOrAddSorted(MoveTemp(InChunkIDs));
	}

	FChunkArrayRegistryHandle FindOrAddSorted(FAssetData::FChunkArray&& InChunkIDs)
	{
		// Make sure we have no duplicates on top of being sorted
		InChunkIDs.SetNum(Algo::Unique(InChunkIDs));

		FChunkArrayRegistryHandle Index;
		if (!InChunkIDs.IsEmpty())
		{
			FChunkArrayRegistryEntry Entry(MoveTemp(InChunkIDs));
			const uint32 Hash = GetTypeHash(Entry);
			{
				const UE::TReadScopeLock _(Lock);
				Index = ChunkArrays.FindIdByHash(Hash, Entry);
			}

			if (!Index.IsValidId())
			{
				// Two threads may hit this point simultaneously and race to insert, and if we just emplace, the second thread will
				// replace the array inserted by the first. We never want this to happen since that would invalidate array views produced
				// since, so we have to do the look-up again. Insertions should be rare in general, so the cost shouldn't be noticeable.
				const UE::TWriteScopeLock _(Lock);
				Index = ChunkArrays.FindIdByHash(Hash, Entry);
				if (!Index.IsValidId())
				{
					Index = ChunkArrays.EmplaceByHash(Hash, MoveTemp(Entry));
				}
			}
		}
		return Index;
	}

	FAssetData::FChunkArrayView GetChunkIDs(FChunkArrayRegistryHandle ChunkArrayRegistryHandle) const
	{
		FAssetData::FChunkArrayView ChunkIDs;
		if (ChunkArrayRegistryHandle.IsValidId())
		{
			const UE::TReadScopeLock _(Lock);
			ChunkIDs = ChunkArrays[ChunkArrayRegistryHandle].GetChunkIDs();
		}
		return ChunkIDs;
	}

	SIZE_T GetAllocatedSize() const
	{
		SIZE_T AllocatedSize = sizeof(*this);
		AllocatedSize += ChunkArrays.GetAllocatedSize();
		for (const FChunkArrayRegistryEntry& Entry : ChunkArrays)
		{
			AllocatedSize += Entry.GetAllocatedSize();
		}
		return AllocatedSize;
	}

private:
	// We will only ever add elements to this set, so we can return persistent indices
	TSet<FChunkArrayRegistryEntry> ChunkArrays;
	mutable FTransactionallySafeRWLock Lock;
} GChunkArrayRegistry;

} // namespace UE::AssetRegistry

FAssetData::FAssetData(FName InPackageName, FName InPackagePath, FName InAssetName, FTopLevelAssetPath InAssetClassPathName, FAssetDataTagMap InTags, TArrayView<const int32> InChunkIDs, uint32 InPackageFlags)
	: PackageName(InPackageName)
	, PackagePath(InPackagePath)
	, AssetName(InAssetName)
	, AssetClassPath(InAssetClassPathName)
	, PackageFlags(InPackageFlags)
{
	SetTagsAndAssetBundles(MoveTemp(InTags));

	FNameBuilder ObjectPathStr(PackageName);
	ObjectPathStr << TEXT('.');
	AssetName.AppendString(ObjectPathStr);
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ObjectPath = FName(FStringView(ObjectPathStr));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	SetChunkIDs(InChunkIDs);
}

FAssetData::FAssetData(FStringView InLongPackageName, FStringView InObjectPath, FTopLevelAssetPath InAssetClassPathName, FAssetDataTagMap InTags, TArrayView<const int32> InChunkIDs, uint32 InPackageFlags)
	: PackageName(InLongPackageName)
	, AssetClassPath(InAssetClassPathName)
	, PackageFlags(InPackageFlags)
{
	using namespace UE::AssetRegistry::Private;

#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ObjectPath = FName(InObjectPath);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	SetTagsAndAssetBundles(MoveTemp(InTags));

	// Note: This emulates the logic of FPackageName::GetLongPackagePath, which does not have a FStringView overload, 
	//       but which returns InLongPackageName if the given value has no path separator, vs FPathViews::GetPath 
	//       which returns an empty string in that case
	PackagePath = FName(FPathViews::GetPath(InLongPackageName));
	if (PackagePath.IsNone())
	{
		if (int32 Unused = INDEX_NONE;
			!InLongPackageName.FindChar(TEXT('/'), Unused))
		{
			PackagePath = PackageName;
		}
	}

	const FAssetPathParts Parts = SplitIntoOuterPathAndAssetName(InObjectPath);
	AssetName = FName(Parts.InnermostName);

#if WITH_EDITORONLY_DATA
	if (!Parts.OuterPath.Equals(InLongPackageName, ESearchCase::IgnoreCase))
	{
		OptionalOuterPath = FName(Parts.OuterPath);
	}
#endif

	SetChunkIDs(InChunkIDs);
}

FAssetData::FAssetData(const UObject* InAsset, FAssetData::ECreationFlags InCreationFlags)
	: FAssetData(InAsset, InCreationFlags, EAssetRegistryTagsCaller::Uncategorized)
{
}

FAssetData::FAssetData(const UObject* InAsset, FAssetData::ECreationFlags InCreationFlags, EAssetRegistryTagsCaller Caller)
{
	if (InAsset != nullptr)
	{
#if WITH_EDITORONLY_DATA
		// ClassGeneratedBy TODO: This may be wrong in cooked builds
		const UClass* InClass = Cast<UClass>(InAsset);
		if (InClass && InClass->ClassGeneratedBy && !EnumHasAnyFlags(InCreationFlags, FAssetData::ECreationFlags::AllowBlueprintClass))
		{
			// For Blueprints, the AssetData refers to the UBlueprint and not the UBlueprintGeneratedClass
			InAsset = InClass->ClassGeneratedBy;
		}
#endif

		const UPackage* Package = InAsset->GetPackage();

		PackageName = Package->GetFName();
		PackagePath = FName(*FPackageName::GetLongPackagePath(Package->GetName()));
		AssetName = InAsset->GetFName();
		AssetClassPath = InAsset->GetClass()->GetPathName();
#if WITH_EDITORONLY_DATA
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ObjectPath = FName(*InAsset->GetPathName());
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (InAsset->GetOuter() != Package)
		{
			OptionalOuterPath = *InAsset->GetOuter()->GetPathName();
		}
#endif

		if (!EnumHasAnyFlags(InCreationFlags, FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering))
		{
			FAssetRegistryTagsContextData Context(InAsset, Caller);
			InAsset->GetAssetRegistryTags(Context, *this);
		}

		PackageFlags = Package->GetPackageFlags();
		SetChunkIDs(Package->GetChunkIDs());
	}
}

FSoftObjectPath FAssetData::GetSoftObjectPath() const
{
	if (IsTopLevelAsset())
	{
		return FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(PackageName, AssetName));
	}
	else
	{
		TStringBuilder<FName::StringBufferSize> Builder;
		AppendObjectPath(Builder);
		return FSoftObjectPath(Builder.ToView());
	}
}

bool FAssetData::IsUAsset(UObject* InAsset)
{
	if (InAsset == nullptr)
	{
		return false;
	}

#if WITH_EDITORONLY_DATA
	if (InAsset->IsPackageExternal())
	{
		return false;
	}
#endif //WITH_EDITORONLY_DATA

	const UPackage* Package = InAsset->GetPackage();

	TStringBuilder<FName::StringBufferSize> AssetNameStrBuilder;
	InAsset->GetPathName(Package, AssetNameStrBuilder);

	TStringBuilder<FName::StringBufferSize> PackageNameStrBuilder;
	Package->GetFName().AppendString(PackageNameStrBuilder);

	return DetectIsUAssetByNames(PackageNameStrBuilder, AssetNameStrBuilder);
}

bool FAssetData::IsTopLevelAsset() const
{
#if WITH_EDITORONLY_DATA
	if (OptionalOuterPath.IsNone())
	{
		// If no outer path, then path is PackageName.AssetName so we must be top level
		return true;
	}

	TStringBuilder<FName::StringBufferSize> Builder;
	AppendObjectPath(Builder);

	int32 SubObjectIndex;
	FStringView(Builder).FindChar(SUBOBJECT_DELIMITER_CHAR, SubObjectIndex);
	return SubObjectIndex == INDEX_NONE;
#else
	// Non-top-level assets only appear in the editor
	return true;
#endif
}

bool FAssetData::IsTopLevelAsset(UObject* Object)
{
	if (!Object)
	{
		return false;
	}
	UObject* Outer = Object->GetOuter();
	if (!Outer)
	{
		return false;
	}
	return Outer->IsA<UPackage>();
}

UE::Core::FVersePath FAssetData::GetVersePath() const
{
	if (!IsValid() || !IsTopLevelAsset())
	{
		return {};
	}

	if (AssetClassPath == UVerseClass::StaticClass()->GetClassPathName())
	{
		FName PackageVersePath;
		FName PackageRelativeVersePath;
		if (GetTagValue(UVerseClass::PackageVersePathTagName, PackageVersePath) && GetTagValue(UVerseClass::PackageRelativeVersePathTagName, PackageRelativeVersePath))
		{
			FNameBuilder PackageVersePathBuilder;
			PackageVersePath.ToString(PackageVersePathBuilder);

			FNameBuilder PackageRelativeVersePathBuilder;
			PackageRelativeVersePath.ToString(PackageRelativeVersePathBuilder);

			UE::Core::FVersePath Result;
			if (UE::Core::FVersePath::TryMake(Result, FPaths::Combine(PackageVersePathBuilder, PackageRelativeVersePathBuilder)))
			{
				return Result;
			}
		}

		return {};
	}

	FName VerseAssetName = AssetName;

	if (UE::AssetRegistry::Private::GVersePathHelper.IsGeneratedBlueprintClass(AssetClassPath))
	{
		FNameBuilder AssetNameBuilder(AssetName);
		FStringView AssetNameView(AssetNameBuilder);
		if (AssetNameView.EndsWith(TEXT("_C"), ESearchCase::CaseSensitive))
		{
			AssetNameView.LeftChopInline(2);

			VerseAssetName = FName(AssetNameView);
		}
	}

	return FPackageName::GetVersePath(FTopLevelAssetPath(PackageName, VerseAssetName));
}

UClass* FAssetData::GetClass(EResolveClass ResolveClass) const
{
	if ( !IsValid() )
	{
		// Dont even try to find the class if the objectpath isn't set
		return nullptr;
	}

	UClass* FoundClass = FindObject<UClass>(AssetClassPath);
	if (!FoundClass)
	{
		// Look for class redirectors
		FString NewPath = FLinkerLoad::FindNewPathNameForClass(AssetClassPath.ToString(), false);
		if (!NewPath.IsEmpty())
		{
			FoundClass = FindObject<UClass>(nullptr, *NewPath);
		}
	}

	// If they decided to load the class if unresolved, then lets load it.
	if (!FoundClass && ResolveClass == EResolveClass::Yes)
	{
		FoundClass = LoadObject<UClass>(nullptr, *AssetClassPath.ToString());
	}
	
	return FoundClass;
}


FAssetData::FChunkArrayView FAssetData::GetChunkIDs() const
{
	return UE::AssetRegistry::GChunkArrayRegistry.GetChunkIDs(ChunkArrayRegistryHandle);
}

void FAssetData::SetChunkIDs(FChunkArray&& InChunkIDs)
{
	ChunkArrayRegistryHandle = UE::AssetRegistry::GChunkArrayRegistry.FindOrAdd(MoveTemp(InChunkIDs));
}

void FAssetData::SetChunkIDs(const FChunkArrayView& InChunkIDs)
{
	ChunkArrayRegistryHandle = UE::AssetRegistry::GChunkArrayRegistry.FindOrAdd(InChunkIDs);
}

void FAssetData::AddChunkID(int32 ChunkID)
{
	// Chunk arrays are guaranteed to be sorted/unique when coming back from registry, so maintain that here
	const FChunkArrayView CurrentChunkIDs = GetChunkIDs();
	const int32 NumChunkIDs = CurrentChunkIDs.Num();
	const int32 InsertIndex = Algo::LowerBound(CurrentChunkIDs, ChunkID);
	if (CurrentChunkIDs.IsValidIndex(InsertIndex) && (CurrentChunkIDs[InsertIndex] == ChunkID))
	{
		return;
	}

	// Build the new array in parts to save having to shift/reallocate unnecessarily
	const FChunkArrayView BeforeInserted = CurrentChunkIDs.Left(InsertIndex);
	const FChunkArrayView AfterInserted = CurrentChunkIDs.RightChop(InsertIndex);
	FChunkArray NewChunkIDs;
	NewChunkIDs.Reserve(NumChunkIDs + 1);
	NewChunkIDs.Append(BeforeInserted.GetData(), BeforeInserted.Num());
	NewChunkIDs.Add(ChunkID);
	NewChunkIDs.Append(AfterInserted.GetData(), AfterInserted.Num());
	ChunkArrayRegistryHandle = UE::AssetRegistry::GChunkArrayRegistry.FindOrAddSorted(MoveTemp(NewChunkIDs));
}

void FAssetData::ClearChunkIDs()
{
	SetChunkIDs(FChunkArray());
}

bool FAssetData::HasSameChunkIDs(const FAssetData& OtherAssetData) const
{
	return ChunkArrayRegistryHandle == OtherAssetData.ChunkArrayRegistryHandle;
}

SIZE_T FAssetData::GetChunkArrayRegistryAllocatedSize()
{
	return UE::AssetRegistry::GChunkArrayRegistry.GetAllocatedSize();
}

void FAssetData::SetTagsAndAssetBundles(FAssetDataTagMap&& Tags)
{
	using namespace UE::AssetData::Private;

	for (FAssetDataTagMap::TIterator Iter(Tags); Iter; ++Iter)
	{
		if (Iter->Key.IsNone())
		{
			ensureMsgf(!Iter->Key.IsNone(),
				TEXT("FAssetData::SetTagsAndAssetBundles called on %s with empty key name. Empty key names are invalid. The Tag will be removed."),
				*this->GetFullName());
			Iter.RemoveCurrent();
			continue;
		}
		if (Iter->Value.IsEmpty())
		{
			ensureMsgf(!Iter->Value.IsEmpty(),
				TEXT("FAssetData::SetTagsAndAssetBundles called on %s with empty value for tag %s. Empty values are invalid. The Tag will be removed."),
				*this->GetFullName(), *Iter->Key.ToString());
			Iter.RemoveCurrent();
			continue;
		}
	}

	FString AssetBundles;
	if (Tags.RemoveAndCopyValue(GAssetBundleDataName, AssetBundles))
	{
		TaggedAssetBundles = ParseAssetBundles(*AssetBundles, *this);
	}
	else
	{
		TaggedAssetBundles.Reset();
	}

	TagsAndValues = Tags.Num() > 0 ? FAssetDataTagMapSharedView(MoveTemp(Tags)) : FAssetDataTagMapSharedView();
}

FPrimaryAssetId FAssetData::GetPrimaryAssetId() const
{
	FName PrimaryAssetType = GetTagValueRef<FName>(FPrimaryAssetId::PrimaryAssetTypeTag);
	FName PrimaryAssetName = GetTagValueRef<FName>(FPrimaryAssetId::PrimaryAssetNameTag);

	if (!PrimaryAssetType.IsNone() && !PrimaryAssetName.IsNone())
	{
		return FPrimaryAssetId(PrimaryAssetType, PrimaryAssetName);
	}

	return FPrimaryAssetId();
}

void FAssetData::SerializeForCacheInternal(FArchive& Ar, FAssetRegistryVersion::Type Version, void (*SerializeTagsAndBundles)(FArchive& , FAssetData&, FAssetRegistryVersion::Type))
{
	// Serialize out the asset info
	// Only needed if Version < FAssetRegistryVersion::RemoveAssetPathFNames but we need to reference it later to 
	// rebuild OptionalOuterPath for assets which are stored in a different package to their outer (e.g. external actors)
	FName OldObjectPath; 
	if (Version < FAssetRegistryVersion::RemoveAssetPathFNames)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Ar << OldObjectPath;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	Ar << PackagePath;

	// Serialize the asset class.
	if (Version >= FAssetRegistryVersion::ClassPaths)
	{
		Ar << AssetClassPath;
	}
	else
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Ar << AssetClass;
		AssetClassPath = FAssetData::TryConvertShortClassNameToPathName(AssetClass, ELogVerbosity::NoLogging);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	Ar << PackageName;
	Ar << AssetName;

#if WITH_EDITORONLY_DATA
	if (Version >= FAssetRegistryVersion::RemoveAssetPathFNames)
	{
		if (!Ar.IsFilterEditorOnly())
		{
			Ar << OptionalOuterPath;
		}
		else if (Ar.IsLoading())
		{
			OptionalOuterPath = NAME_None;
		}
	}
	else
	{
		check(Ar.IsLoading());
		check(!OldObjectPath.IsNone());

		using namespace UE::AssetRegistry::Private;

		OptionalOuterPath = NAME_None;
		TStringBuilder<FName::StringBufferSize> Builder;
		Builder << PackageName << '.' << AssetName;
		if (OldObjectPath != *Builder)
		{
			Builder.Reset();
			Builder << OldObjectPath;
			FAssetPathParts Parts = SplitIntoOuterPathAndAssetName(Builder.ToView());
			if (PackageName.ToString() != Parts.OuterPath)
			{
				OptionalOuterPath = FName(Parts.OuterPath);
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	SerializeTagsAndBundles(Ar, *this, Version);

	FAssetData::FChunkArray SerializedChunkIDs;
	if (Ar.IsSaving())
	{
		SerializedChunkIDs = GetChunkIDs();
		Ar << SerializedChunkIDs;
	}
	else
	{
		Ar << SerializedChunkIDs;
		check(Algo::IsSorted(SerializedChunkIDs));
		ChunkArrayRegistryHandle = UE::AssetRegistry::GChunkArrayRegistry.FindOrAddSorted(MoveTemp(SerializedChunkIDs));
	}

	Ar << PackageFlags;

#if WITH_EDITORONLY_DATA
	// Rebuild deprecated ObjectPath field 
	TStringBuilder<FName::StringBufferSize> Builder;
	AppendObjectPath(Builder);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ObjectPath = *Builder;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
}

void FAssetData::NetworkWrite(FCbWriter& Writer, bool bWritePackageName) const
{
	// Note we use single-letter field names to reduce network bandwidth
	Writer.BeginObject();
	if (bWritePackageName)
	{
		Writer << "P" << PackagePath;
		Writer << "Q" << PackageName;
#if WITH_EDITORONLY_DATA
		Writer << "OO" << OptionalOuterPath;
#endif
	}
	Writer << "N" << AssetName;
	Writer << "C" << AssetClassPath.ToString();
	if (TagsAndValues.Num() != 0 || TaggedAssetBundles)
	{
		Writer.BeginArray("T");
		TagsAndValues.ForEach([&Writer](const TPair<FName, FAssetTagValueRef>& Pair)
			{
				Writer.BeginObject();
				Writer << "K" << Pair.Key;
				Writer << "V" << Pair.Value.GetStorageString();
				Writer.EndObject();
			});
		if (TaggedAssetBundles)
		{
			FString ValueText;
			TaggedAssetBundles->ExportTextItem(ValueText, FAssetBundleData(), nullptr, PPF_None, nullptr);

			Writer.BeginObject();
			Writer << "K" << GAssetBundleDataName;
			Writer << "V" << ValueText;
			Writer.EndObject();
		}
		Writer.EndArray();
	}

	FChunkArrayView CurrentChunkIDs = GetChunkIDs();
	if (!CurrentChunkIDs.IsEmpty())
	{
		FChunkArray SerializedChunkIDs(CurrentChunkIDs);
		Writer << "I" << SerializedChunkIDs;
	}
	Writer.EndObject();
}

bool FAssetData::TryNetworkRead(FCbFieldView Field, bool bReadPackageName, FName InPackageName)
{
	bool bOk = true;
	FCbObjectView Object = Field.AsObjectView();
	bOk &= !Field.HasError();

	bool bHasAssetName = LoadFromCompactBinary(Object["N"], AssetName);
	bOk &= bHasAssetName;
	if (bReadPackageName)
	{
		bOk = LoadFromCompactBinary(Object["P"], PackagePath) & bOk;
		bOk = LoadFromCompactBinary(Object["Q"], PackageName) & bOk;
#if WITH_EDITORONLY_DATA
		bOk = LoadFromCompactBinary(Object["OO"], OptionalOuterPath) & bOk;
#endif
	}
	else
	{
		if (bHasAssetName)
		{
			PackagePath = FName(FPathViews::GetPath(WriteToString<256>(InPackageName)));
		}
		else
		{
			PackagePath = NAME_None;
		}
		PackageName = InPackageName;
	}
	FString ClassPath;
	if (LoadFromCompactBinary(Object["C"], ClassPath))
	{
		bOk = AssetClassPath.TrySetPath(ClassPath) & bOk;
	}
	else
	{
		AssetClassPath.Reset();
		bOk = false;
	}

	FCbFieldView TagsField = Object["T"];
	FCbArrayView TagsArray = TagsField.AsArrayView();
	if (!TagsField.HasError()) // Ok if it does not exist
	{
		FAssetDataTagMap Tags;
		Tags.Reserve(IntCastChecked<int32>(TagsArray.Num()));
		for (FCbFieldView TagField : TagsArray)
		{
			FName TagName;
			bOk = LoadFromCompactBinary(TagField["K"], TagName) & bOk;
			FString& TagValue = Tags.FindOrAdd(TagName);
			bOk = LoadFromCompactBinary(TagField["V"], TagValue) & bOk;
		}
		SetTagsAndAssetBundles(MoveTemp(Tags));
	}
	else
	{
		SetTagsAndAssetBundles(FAssetDataTagMap());
	}

	FChunkArray SerializedChunkIDs;
	LoadFromCompactBinary(Object["I"], SerializedChunkIDs); // Ok if it does not exist
	check(Algo::IsSorted(SerializedChunkIDs));
	ChunkArrayRegistryHandle = UE::AssetRegistry::GChunkArrayRegistry.FindOrAddSorted(MoveTemp(SerializedChunkIDs));

#if WITH_EDITORONLY_DATA
	// Rebuild deprecated ObjectPath field 
	TStringBuilder<FName::StringBufferSize> Builder;
	AppendObjectPath(Builder);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ObjectPath = *Builder;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	return bOk;
}

void FAssetData::SerializeForCacheWithTagsAndBundles(FArchive& Ar, void (*SerializeTagsAndBundles)(FArchive&, FAssetData&, FAssetRegistryVersion::Type))
{
	SerializeForCacheInternal(Ar, FAssetRegistryVersion::LatestVersion, SerializeTagsAndBundles);
}
void FAssetData::SerializeForCacheOldVersionWithTagsAndBundles(FArchive& Ar, FAssetRegistryVersion::Type Version, void (*SerializeTagsAndBundles)(FArchive&, FAssetData&, FAssetRegistryVersion::Type Version))
{
	SerializeForCacheInternal(Ar, Version, SerializeTagsAndBundles);
}

bool FAssetData::IsRedirectorClassName(FTopLevelAssetPath ClassPathName)
{
	return ClassPathName == UE::AssetRegistry::GetClassPathObjectRedirector();
}

FTopLevelAssetPath FAssetData::TryConvertShortClassNameToPathName(FName InClassName, ELogVerbosity::Type FailureMessageVerbosity /*= ELogVerbosity::Warning*/)
{
	FTopLevelAssetPath ClassPath;
	if (!InClassName.IsNone())
	{
		FString ClassNameString(InClassName.ToString());
		ELogVerbosity::Type AmbiguousMessageVerbosity = (FailureMessageVerbosity == ELogVerbosity::NoLogging || FailureMessageVerbosity > ELogVerbosity::Warning) ?
			FailureMessageVerbosity : ELogVerbosity::Warning;
		ClassPath = UClass::TryConvertShortTypeNameToPathName<UStruct>(ClassNameString, AmbiguousMessageVerbosity, TEXT("AssetRegistry trying to convert short name to path name"));
		if (ClassPath.IsNull())
		{
			// In some cases the class name stored in asset registry tags have been redirected with ini class redirects
			FString RedirectedName = FLinkerLoad::FindNewPathNameForClass(ClassNameString, false);
			if (!FPackageName::IsShortPackageName(RedirectedName))
			{
				ClassPath = FTopLevelAssetPath(RedirectedName);
			}
			else
			{
				ClassPath = UClass::TryConvertShortTypeNameToPathName<UStruct>(RedirectedName, AmbiguousMessageVerbosity, TEXT("AssetRegistry trying to convert redirected short name to path name"));
			}

			if (ClassPath.IsNull())
			{
				// Fallback to a fake name but at least the class name will be preserved
				ClassPath = FTopLevelAssetPath(TEXT("/Unknown"), InClassName);
#if !NO_LOGGING
				if (FailureMessageVerbosity != ELogVerbosity::NoLogging)
				{
					FMsg::Logf(__FILE__, __LINE__, LogAssetData.GetCategoryName(), FailureMessageVerbosity, TEXT("Failed to convert deprecated short class name \"%s\" to path name. Using \"%s\""), *InClassName.ToString(), *ClassPath.ToString());
				}
#endif
			}
		}
	}
	return ClassPath;
}

bool FAssetRegistryVersion::SerializeVersion(FArchive& Ar, FAssetRegistryVersion::Type& Version)
{
	FGuid Guid = FAssetRegistryVersion::GUID;

	if (Ar.IsLoading())
	{
		Version = FAssetRegistryVersion::PreVersioning;
	}

	Ar << Guid;

	if (Ar.IsError())
	{
		return false;
	}

	if (Guid == FAssetRegistryVersion::GUID)
	{
		int32 VersionInt = Version;
		Ar << VersionInt;
		Version = (FAssetRegistryVersion::Type)VersionInt;

		Ar.SetCustomVersion(Guid, VersionInt, TEXT("AssetRegistry"));
	}
	else
	{
		return false;
	}

	return !Ar.IsError();
}

namespace UE::AssetRegistry
{

FCbWriter& FPackageCustomVersion::Write(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << Key << Version;
	Writer.EndArray();
	return Writer;
}

bool FPackageCustomVersion::TryRead(const FCbFieldView& Field)
{
	FCbFieldViewIterator Iter = Field.CreateViewIterator();
	bool bOk = LoadFromCompactBinary(*Iter++, Key);
	bOk = LoadFromCompactBinary(*Iter++, Version) & bOk;
	return bOk;
}

}

void FAssetPackageData::SerializeForCacheInternal(FArchive& Ar, FAssetPackageData& PackageData, FAssetRegistryVersion::Type Version)
{
	Ar << PackageData.DiskSize;

	// The serialization format for FAssetPackageData is shared between editor and runtime, so that PackageDatas
	// written by one can be read by the other. But the PackageSavedHash is excluded from the runtime data and API to
	// save memory. Whenever serializing an AssetPackageData at runtime, we read a placeholder and write zeroes.
	// Caveat: this means a round trip from editor -> runtime -> editor will lose the PackageSavedHash data and replace
	// it with zeroes.
	if (Version < FAssetRegistryVersion::PackageSavedHash)
	{
		FGuid LegacyGuid;
		Ar << LegacyGuid;
#if WITH_EDITORONLY_DATA
		if (Ar.IsLoading())
		{
			FMemory::Memcpy(&PackageData.PackageSavedHash, &LegacyGuid,
				FMath::Min(sizeof(decltype(PackageData.PackageSavedHash.GetBytes())), sizeof(LegacyGuid)));
		}
#endif
	}
	else
	{
#if WITH_EDITORONLY_DATA
		Ar << PackageData.PackageSavedHash;
#else
		FIoHash PlaceholderSavedHash;
		Ar << PlaceholderSavedHash;
#endif
	}
	if (Version >= FAssetRegistryVersion::AddedCookedMD5Hash)
	{
		Ar << PackageData.CookedHash;
	}
	if (Version >= FAssetRegistryVersion::AddedChunkHashes)
	{
		Ar << PackageData.ChunkHashes;
	}
	if (Version >= FAssetRegistryVersion::WorkspaceDomain)
	{
		if (Version >= FAssetRegistryVersion::PackageFileSummaryVersionChange)
		{
			Ar << PackageData.FileVersionUE;
		}
		else
		{
			int32 UE4Version;
			Ar << UE4Version;

			PackageData.FileVersionUE = FPackageFileVersion::CreateUE4Version(UE4Version);
		}

		Ar << PackageData.FileVersionLicenseeUE;
		Ar << PackageData.Flags;
		Ar << PackageData.CustomVersions;
	}
	if (Version >= FAssetRegistryVersion::PackageImportedClasses)
	{
		if (Ar.IsSaving() && !Algo::IsSorted(PackageData.ImportedClasses, FNameLexicalLess()))
		{
			Algo::Sort(PackageData.ImportedClasses, FNameLexicalLess());
		}
		Ar << PackageData.ImportedClasses;
	}
	if (Version >= FAssetRegistryVersion::AssetPackageDataHasExtension)
	{
		FString ExtensionText;
		if (Ar.IsLoading())
		{
			Ar << ExtensionText;
			PackageData.Extension = FPackagePath::ParseExtension(ExtensionText);
		}
		else
		{
			ExtensionText = LexToString(PackageData.Extension);
			Ar << ExtensionText;
		}
	}
	else if (Ar.IsLoading())
	{
		Extension = EPackageExtension::Unspecified;
	}
}

void FAssetPackageData::SerializeForCache(FArchive& Ar)
{
	// Calling with hard-coded version and using force-inline on SerializeForCacheInternal eliminates the cost of its if-statements
	SerializeForCacheInternal(Ar, *this, FAssetRegistryVersion::LatestVersion);
}

void FAssetPackageData::SerializeForCacheOldVersion(FArchive& Ar, FAssetRegistryVersion::Type Version)
{
	SerializeForCacheInternal(Ar, *this, Version);
}

#if WITH_EDITORONLY_DATA
FIoHash FAssetPackageData::GetPackageSavedHash() const
{
	return PackageSavedHash;
}

void FAssetPackageData::SetPackageSavedHash(const FIoHash& InHash)
{
	PackageSavedHash = InHash;
}
#endif

inline FCbWriter& operator<<(FCbWriter& Writer, const TPair<FIoChunkId, FIoHash>& Value)
{
	Writer.BeginArray();
	Writer << Value.Key << Value.Value;
	Writer.EndArray();
	return Writer;
}

inline bool LoadFromCompactBinary(FCbFieldView Field, TPair<FIoChunkId, FIoHash>& Value)
{
	FCbFieldViewIterator Iter = Field.CreateViewIterator();
	bool bOk = LoadFromCompactBinary(*Iter++, Value.Key);
	bOk = LoadFromCompactBinary(*Iter++, Value.Value) & bOk;
	return bOk;
}

void FAssetPackageData::NetworkWrite(FCbWriter& Writer) const
{
	Writer.BeginArray();
	bool bCookedHash = CookedHash.IsValid();
	Writer << bCookedHash;
	if (bCookedHash)
	{
		Writer << CookedHash;
	}

	// The network serialization format for FAssetPackageData is shared between editor and runtime for robustness
	// just like the persistent format; see the comment in SerializeForCacheInternal for notes.
#if WITH_EDITORONLY_DATA
	bool bPackageSavedHash = !PackageSavedHash.IsZero();
	Writer << bPackageSavedHash;
	if (bPackageSavedHash)
	{
		Writer << PackageSavedHash;
	}
#else
	bool bPackageSavedHash = false;
	Writer << bPackageSavedHash;
#endif
	Writer << ChunkHashes.Array();
	Writer << ImportedClasses;
	Writer << DiskSize;
	Writer << FileVersionUE;
	Writer << FileVersionLicenseeUE;
	TArray<UE::AssetRegistry::FPackageCustomVersion> LocalCustomVersions;
	LocalCustomVersions.Append(GetCustomVersions());
	Writer << LocalCustomVersions;
	Writer << Flags;
	Writer << static_cast<uint8>(Extension);
	Writer.EndArray();
}

bool FAssetPackageData::TryNetworkRead(FCbFieldView Field)
{
	FCbFieldViewIterator Iter = Field.CreateViewIterator();
	bool bCookedHash = false;
	bool bOk = LoadFromCompactBinary(*Iter++, bCookedHash);
	if (bCookedHash)
	{
		bOk = LoadFromCompactBinary(*Iter++, CookedHash) & bOk;
	}
	else
	{
		CookedHash = FMD5Hash();
	}
	bool bPackageSavedHash = false;
	bOk = LoadFromCompactBinary(*Iter++, bPackageSavedHash) & bOk;
#if WITH_EDITORONLY_DATA
	if (bPackageSavedHash)
	{
		bOk = LoadFromCompactBinary(*Iter++, PackageSavedHash) & bOk;
	}
	else
	{
		PackageSavedHash = FIoHash::Zero;
	}
#else
	if (bPackageSavedHash)
	{
		FIoHash UnusedPackageSavedHash;
		bOk = LoadFromCompactBinary(*Iter++, UnusedPackageSavedHash) & bOk;
	}
#endif
	TArray<TPair<FIoChunkId, FIoHash>> ChunkHashesArray;
	if (LoadFromCompactBinary(*Iter++, ChunkHashesArray))
	{
		ChunkHashes.Empty(ChunkHashesArray.Num());
		for (TPair<FIoChunkId, FIoHash>& Pair : ChunkHashesArray)
		{
			ChunkHashes.Add(Pair.Key, Pair.Value);
		}
	}
	else
	{
		bOk = false;
	}
	bOk = LoadFromCompactBinary(*Iter++, ImportedClasses) & bOk;
	bOk = LoadFromCompactBinary(*Iter++, DiskSize) & bOk;
	bOk = LoadFromCompactBinary(*Iter++, FileVersionUE) & bOk;
	bOk = LoadFromCompactBinary(*Iter++, FileVersionLicenseeUE) & bOk;
	TArray<UE::AssetRegistry::FPackageCustomVersion> LocalCustomVersions;
	if (LoadFromCompactBinary(*Iter++, LocalCustomVersions))
	{
		SetCustomVersions(LocalCustomVersions);
	}
	bOk = LoadFromCompactBinary(*Iter++, Flags) & bOk;
	uint8 ExtensionInt = 0;
	if (LoadFromCompactBinary(*Iter++, ExtensionInt) && ExtensionInt < static_cast<uint8>(EPackageExtension::Count))
	{
		Extension = static_cast<EPackageExtension>(ExtensionInt);
	}
	else
	{
		bOk = false;
	}
	return bOk;
}

void FARFilter::PostSerialize(const FArchive& Ar)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if WITH_EDITORONLY_DATA

	auto ConvertShortClassNameToPathName = [](FName ShortClassFName)
	{
		FTopLevelAssetPath ClassPathName;
		if (ShortClassFName != NAME_None)
		{
			FString ShortClassName = ShortClassFName.ToString();
			ClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(*ShortClassName, ELogVerbosity::Warning, TEXT("FARFilter::PostSerialize"));
			UE_CLOGF(ClassPathName.IsNull(), LogAssetData, Error, "Failed to convert short class name %ls to class path name.", *ShortClassName);
		}
		return ClassPathName;
	};

	for (FName ClassFName : ClassNames)
	{
		FTopLevelAssetPath ClassPathName = ConvertShortClassNameToPathName(ClassFName);
		ClassPaths.Add(ClassPathName);
	}
	for (FName ClassFName : RecursiveClassesExclusionSet)
	{
		FTopLevelAssetPath ClassPathName = ConvertShortClassNameToPathName(ClassFName);
		RecursiveClassPathsExclusionSet.Add(ClassPathName);
	}

	ClassNames.Empty();
	RecursiveClassPathsExclusionSet.Empty();

#endif // WITH_EDITORONLY_DATA
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

namespace UE
{
namespace AssetRegistry
{

uint32 GetTypeHash(const TArray<FPackageCustomVersion>& Versions)
{
	constexpr uint32 HashPrime = 23;
	uint32 Hash = 0;
	for (const FPackageCustomVersion& Version : Versions)
	{
		Hash = Hash * HashPrime + GetTypeHash(Version.Key);
		Hash = Hash * HashPrime + Version.Version;
	}
	return Hash;
}

class FPackageCustomVersionRegistry
{
public:
	FPackageCustomVersionsHandle FindOrAdd(TArray<FPackageCustomVersion>&& InVersions)
	{
		FPackageCustomVersionsHandle Result;
		Algo::Sort(InVersions);
		uint32 Hash = GetTypeHash(InVersions);
		{
			const UE::TReadScopeLock _(Lock);
			TArray<FPackageCustomVersion>* Existing = RegisteredValues.FindByHash(Hash, InVersions);
			if (Existing)
			{
				// We return a TArrayView with a pointer to the allocation managed by the element in the Set
				// The element in the set may be destroyed and a moved copy recreated when the set changes size,
				// but since TSet uses move constructors during the resize, the allocation will be unchanged,
				// so we can safely refer to it from external handles.
				Result.Ptr = TConstArrayView<FPackageCustomVersion>(*Existing);
				return Result;
			}
		}
		{
			const UE::TWriteScopeLock _(Lock);
			TArray<FPackageCustomVersion>& Existing = RegisteredValues.FindOrAddByHash(Hash, MoveTemp(InVersions));
			Result.Ptr = TConstArrayView<FPackageCustomVersion>(Existing);
			return Result;
		}
	}

private:
	TSet<TArray<FPackageCustomVersion>> RegisteredValues;
	FTransactionallySafeRWLock Lock;
} GFPackageCustomVersionRegistry;

FPackageCustomVersionsHandle FPackageCustomVersionsHandle::FindOrAdd(TConstArrayView<FCustomVersion> InVersions)
{
	TArray<FPackageCustomVersion> PackageFormat;
	PackageFormat.Reserve(InVersions.Num());
	for (const FCustomVersion& Version : InVersions)
	{
		PackageFormat.Emplace(Version.Key, Version.Version);
	}
	return GFPackageCustomVersionRegistry.FindOrAdd(MoveTemp(PackageFormat));
}

FPackageCustomVersionsHandle FPackageCustomVersionsHandle::FindOrAdd(TConstArrayView<FPackageCustomVersion> InVersions)
{
	return GFPackageCustomVersionRegistry.FindOrAdd(TArray<FPackageCustomVersion>(InVersions));
}

FPackageCustomVersionsHandle FPackageCustomVersionsHandle::FindOrAdd(TArray<FPackageCustomVersion>&& InVersions)
{
	return GFPackageCustomVersionRegistry.FindOrAdd(MoveTemp(InVersions));
}

FArchive& operator<<(FArchive& Ar, UE::AssetRegistry::FPackageCustomVersionsHandle& Handle)
{
	using namespace UE::AssetRegistry;

	if (Ar.IsLoading())
	{
		int32 NumCustomVersions;
		Ar << NumCustomVersions;
		TArray<UE::AssetRegistry::FPackageCustomVersion> CustomVersions;
		CustomVersions.SetNum(NumCustomVersions);
		for (UE::AssetRegistry::FPackageCustomVersion& CustomVersion : CustomVersions)
		{
			Ar << CustomVersion;
		}
		Handle = FPackageCustomVersionsHandle::FindOrAdd(MoveTemp(CustomVersions));
	}
	else
	{
		TConstArrayView<UE::AssetRegistry::FPackageCustomVersion> CustomVersions = Handle.Get();
		int32 NumCustomVersions = CustomVersions.Num();
		Ar << NumCustomVersions;
		for (UE::AssetRegistry::FPackageCustomVersion CustomVersion : CustomVersions)
		{
			Ar << CustomVersion;
		}
	}
	return Ar;
}

}
}

FAssetIdentifier::FAssetIdentifier(UObject* SourceObject, FName InValueName)
{
	if (SourceObject)
	{
		UPackage* Package = SourceObject->GetOutermost();
		PackageName = Package->GetFName();
		ObjectName = SourceObject->GetFName();
		ValueName = InValueName;
	}
}

#if WITH_DEV_AUTOMATION_TESTS 

#include "Misc/AutomationTest.h"
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetDataTests, "System.CoreUObject.AssetData", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FAssetDataTests::RunTest(const FString& Parameters)
{
	FAssetData EmptyAssetData;

	TestEqual(TEXT("Empty Asset Data: Object path string is empty"), EmptyAssetData.GetObjectPathString(), FString());

	return true;
}


#endif // WITH_DEV_AUTOMATION_TESTS
