// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetDataTagMap.h"

#include "Algo/Sort.h"
#include "AssetRegistry/AssetDataTagMapSerializationDetails.h"
#include "AssetRegistry/AssetRegistrySerializationDetails.h"
#include "AssetRegistry/AssetRegistryStateSerialization.h"
#include "AutoRTFM.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/PackageName.h"
#include "Misc/Optional.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Misc/ScopeLock.h"
#include "Serialization/ArrayWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

#include <limits>

DEFINE_LOG_CATEGORY_STATIC(LogAssetDataTags, Log, All);

//////////////////////////////////////////////////////////////////////////

void UE::AssetRegistry::AlignPosInArchive(FArchive& Archive)
{
	// see if current offset is aligned
	int64 Pos = Archive.Tell();
	int64 AlignedPos = Align(Pos, AlignmentForMemoryMappedArchive);
	if (AlignedPos != Pos)
	{
		int64 DataToSkip = AlignedPos - Pos;

		if (Archive.IsSaving())
		{
			char Dunnage[AlignmentForMemoryMappedArchive] = { 0 };

#if 0 // for easier debugging
			for (SIZE_T Idx = 0; Idx < sizeof(Dunnage); ++Idx)
			{
				Dunnage[Idx] = static_cast<char>(Idx);
			}
#endif
			Archive.Serialize(Dunnage, DataToSkip);
		}
		else
		{
			Archive.Seek(AlignedPos);
		}

		checkf(Archive.Tell() == AlignedPos, TEXT("We were not able to align the position in the archive to the desired boundary."));
	}
}

//////////////////////////////////////////////////////////////////////////

template<class T>
FAssetRegistryExportPath ParseExportPath(T ExportPath)
{
	FAssetRegistryExportPath Out;

	int32 Dummy;
	T  ObjectPath;
	if (ExportPath.FindChar('\'', Dummy))
	{
		T ClassName;
		verify(FPackageName::ParseExportTextPath(ExportPath, &ClassName, &ObjectPath));
		Out.ClassPath = FTopLevelAssetPath(ClassName);
	}
	else
	{
		ObjectPath = ExportPath;
	}

	T PackageName = FPackageName::ObjectPathToPackageName(ObjectPath);
	if (PackageName != ObjectPath)
	{
		Out.Object = FName(ObjectPath.Mid(PackageName.Len() + 1));
	}

	Out.Package = FName(PackageName);

	return Out;
}

FAssetRegistryExportPath::FAssetRegistryExportPath(FWideStringView ExportPath)
	: FAssetRegistryExportPath(ParseExportPath(ExportPath))
{}

FAssetRegistryExportPath::FAssetRegistryExportPath(FAnsiStringView ExportPath)
	: FAssetRegistryExportPath(ParseExportPath(ExportPath))
{}

FString FAssetRegistryExportPath::ToString() const
{
	TStringBuilder<256> Path;
	ToString(Path);
	return FString(Path);
}

FName FAssetRegistryExportPath::ToName() const
{
	if (ClassPath.IsNull() && Object.IsNone())
	{
		return Package;
	}

	TStringBuilder<256> Path;
	ToString(Path);
	return FName(Path);
}

void FAssetRegistryExportPath::ToString(FStringBuilderBase& Out) const
{
	if (!ClassPath.IsNull())
	{
		Out << ClassPath << '\'';
	}
	Out << Package;
	if (!Object.IsNone())
	{
		Out << '.' << Object;
	}
	if (!ClassPath.IsNull())
	{
		Out << '\'';
	}
}

FString FAssetRegistryExportPath::ToPath() const
{
	TStringBuilder<256> Path;
	ToPath(Path);
	return FString(Path);
}

void FAssetRegistryExportPath::ToPath(FStringBuilderBase& Out) const
{
	Out << Package;
	if (!Object.IsNone())
	{
		Out << '.' << Object;
	}
}

static FAssetRegistryExportPath MakeNumberedPath(const FNameEntry& NameEntry)
{
	TStringBuilder<NAME_SIZE> Buffer;
	NameEntry.AppendNameToString(Buffer);
	return FAssetRegistryExportPath(FStringView(Buffer));
}
	
static FAssetRegistryExportPath MakeNumberedPath(FName Name)
{
	TCHAR Buffer[FName::StringBufferSize];
	Name.ToString(Buffer);
	return FAssetRegistryExportPath(FStringView(Buffer));
}

static FArchive& operator<<(FArchive& Ar, FAssetRegistryExportPath& Path)
{
	return Ar << Path.ClassPath << Path.Object << Path.Package;
}

bool operator==(const FAssetRegistryExportPath& A, const FAssetRegistryExportPath& B)
{
	return A.ClassPath == B.ClassPath & A.Package == B.Package & A.Object == B.Object;
}

uint32 GetTypeHash(const FAssetRegistryExportPath& Path)
{
	return FixedTagPrivate::HashCombineQuick(GetTypeHash(Path.ClassPath), GetTypeHash(Path.Package), GetTypeHash(Path.Object));
}

//////////////////////////////////////////////////////////////////////////

static FString ToComplexString(const FText& In)
{
	FString Out;
	FTextStringHelper::WriteToBuffer(Out, In, /*bRequireQuotes*/false, /*bStripPackageNamespace*/true);
	return Out;
}

static bool FromComplexString(const FString& In, FText& Out)
{
	return FTextStringHelper::IsComplexText(*In) && FTextStringHelper::ReadFromBuffer(*In, Out);
}

static FString LocalizeIfComplexString(const FString& Value)
{
	if (FTextStringHelper::IsComplexText(*Value))
	{
		FText Text;
		if (FTextStringHelper::ReadFromBuffer(*Value, Text))
		{
			return Text.ToString();
		}
	}

	return Value;
}

//////////////////////////////////////////////////////////////////////////

namespace FixedTagPrivate
{
	FMarshalledText::FMarshalledText(const FUtf8String& InComplexString)
		: String(InComplexString)
	{
	}
	FMarshalledText::FMarshalledText(const FString& InComplexString)
		: String(InComplexString)
	{
	}
	FMarshalledText::FMarshalledText(FUtf8String&& InComplexString)
		: String(MoveTemp(InComplexString))
	{
	}
	FMarshalledText::FMarshalledText(const FText& InText)
		: String(ToComplexString(InText))
	{
	}
	FMarshalledText::FMarshalledText(FText&& InText)
		: String(ToComplexString(InText))
	{
	}

	const FUtf8String& FMarshalledText::GetAsComplexString() const
	{
		return String;
	}

	FText FMarshalledText::GetAsText() const
	{
		return FTextStringHelper::CreateFromBuffer(StringCast<TCHAR>(*String).Get());
	}

	int64 FMarshalledText::GetResourceSize() const
	{
		return String.GetAllocatedSize();
	}

	int32 FMarshalledText::CompareToCaseIgnored(const FMarshalledText& Other) const
	{
		return String.Compare(Other.String);
	}

	FArchive& operator<<(FArchive& Ar, FLegacyAssetRegistryExportPath& Path)
	{
		return Ar << Path.Class << Path.Object << Path.Package;
	}

	uint32 HashCaseSensitive(const TCHAR* Str, int32 Len)		
	{
		return CityHash32(reinterpret_cast<const char*>(Str), sizeof(TCHAR) * Len);
	}

	uint32 HashCaseSensitive(const UTF8CHAR* Str, int32 Len)
	{
		return CityHash32(reinterpret_cast<const char*>(Str), sizeof(UTF8CHAR) * Len);
	}

	uint32 HashCombineQuick(uint32 A, uint32 B)
	{
		return A ^ (B + 0x9e3779b9 + (A << 6) + (A >> 2));
	}

	uint32 HashCombineQuick(uint32 A, uint32 B, uint32 C)
	{
		return HashCombineQuick(HashCombineQuick(A, B), C);
	}

	static bool EqualsInsensitive(FStringView A, const ANSICHAR* B)
	{
		return FPlatformString::Strnicmp(A.GetData(), B, A.Len()) == 0 && B[A.Len()] == '\0';
	}

	static bool EqualsInsensitive(FStringView A, const WIDECHAR* B)
	{
		return FPlatformString::Strnicmp(A.GetData(), B, A.Len()) == 0 && B[A.Len()] == '\0';
	}

	static bool EqualsInsensitive(FStringView A, const UTF8CHAR* B)
	{
		auto BStringCast = StringCast<TCHAR>(B);
		return A.Equals(FStringView(BStringCast.Get(), BStringCast.Length()), ESearchCase::IgnoreCase);
	}

	static bool EqualsInsensitive(const ANSICHAR* A, const ANSICHAR* B)
	{
		return FPlatformString::Stricmp(A, B) == 0;
	}

	static bool EqualsInsensitive(const WIDECHAR* A, const WIDECHAR* B)
	{
		return FPlatformString::Stricmp(A, B) == 0;
	}

	static bool EqualsInsensitive(const UTF8CHAR* A, const UTF8CHAR* B)
	{
		return FPlatformString::Stricmp(A, B) == 0;
	}

	static bool EqualsInsensitive(const FMarshalledText& A, const FMarshalledText& B)
	{
		return A.CompareToCaseIgnored(B) == 0;
	}
	
	static bool EqualsInsensitive(FName A, FName B)
	{
		return A == B;
	}

	static bool EqualsInsensitive(FNameEntryId A, FNameEntryId B)
	{
		return A == B || FName::GetComparisonIdFromDisplayId(A) == FName::GetComparisonIdFromDisplayId(B);
	}
	
	static bool EqualsInsensitive(const FNumberlessExportPath& A, const FNumberlessExportPath& B)
	{
		return	EqualsInsensitive(A.ClassPackage, B.ClassPackage) &		//-V792
				EqualsInsensitive(A.ClassObject, B.ClassObject) &		//-V792
				EqualsInsensitive(A.Package, B.Package) &	//-V792
				EqualsInsensitive(A.Object, B.Object);
	}

	static bool EqualsInsensitive(const FAssetRegistryExportPath& A, const FAssetRegistryExportPath& B)
	{
		return A == B;
	}

	static bool IsNumberless(FName Name)
	{
		return Name.GetNumber() == NAME_NO_NUMBER_INTERNAL;
	}
	
	static FName MakeNumberedName(FNameEntryId EntryId)
	{
		return FName::CreateFromDisplayId(EntryId, NAME_NO_NUMBER_INTERNAL);
	}

	static FName MakeNumberedName(FDisplayNameEntryId EntryId)
	{
		return EntryId.ToName(NAME_NO_NUMBER_INTERNAL);
	}

	static bool EqualsInsensitive(FDisplayNameEntryId A, FDisplayNameEntryId B)
	{
		return MakeNumberedName(A) == MakeNumberedName(B);
	}
	
	static FNameEntryId MakeNumberlessDisplayName(FName Name)
	{
		check(Name.GetNumber() == NAME_NO_NUMBER_INTERNAL);
		return Name.GetDisplayIndex();
	}

	static bool IsNumberless(FAssetRegistryExportPath Path)
	{
		return IsNumberless(Path.ClassPath.GetAssetName()) & IsNumberless(Path.Object) & IsNumberless(Path.Package); //-V792
	}

	static FAssetRegistryExportPath MakeNumberedPath(const FNumberlessExportPath& Path)
	{
		FAssetRegistryExportPath Out;
		Out.ClassPath = FTopLevelAssetPath(MakeNumberedName(Path.ClassPackage), MakeNumberedName(Path.ClassObject));
		Out.Object	  = MakeNumberedName(Path.Object);
		Out.Package	  = MakeNumberedName(Path.Package);
		return Out;
	}
	
	static FNumberlessExportPath MakeNumberlessPath(const FAssetRegistryExportPath& Path)
	{
		FNumberlessExportPath Out;
		Out.ClassPackage = MakeNumberlessDisplayName(Path.ClassPath.GetPackageName());
		Out.ClassObject  = MakeNumberlessDisplayName(Path.ClassPath.GetAssetName());
		Out.Object	     = MakeNumberlessDisplayName(Path.Object);
		Out.Package	     = MakeNumberlessDisplayName(Path.Package);
		return Out;
	}
	
	static FAssetRegistryExportPath ConvertLegacyPath(const FLegacyAssetRegistryExportPath& Path)
	{
		FAssetRegistryExportPath Out;
		Out.ClassPath = FAssetData::TryConvertShortClassNameToPathName(Path.Class, ELogVerbosity::NoLogging);
		Out.Object = Path.Object;
		Out.Package = Path.Package;
		return Out;
	}

	FString FNumberlessExportPath::ToString() const
	{
		return MakeNumberedPath(*this).ToString();
	}
	
	FName FNumberlessExportPath::ToName() const
	{
		return MakeNumberedPath(*this).ToName();
	}
	
	void FNumberlessExportPath::ToString(FStringBuilderBase& Out) const
	{
		MakeNumberedPath(*this).ToString(Out);
	}
	
	//////////////////////////////////////////////////////////////////////////

	enum class EValueType : uint32
	{
		AnsiString,
		WideString,
		NumberlessName,
		Name,
		NumberlessExportPath,
		ExportPath,
		LocalizedText
	};
	
	static constexpr uint32 NumValueTypes = static_cast<uint32>(EValueType::LocalizedText) + 1;
	static_assert((1u << FValueId::TypeBits) >= NumValueTypes, "");

	//////////////////////////////////////////////////////////////////////////

	class FStoreManager
	{
	public:
		static constexpr uint32 Capacity = 1u << FMapHandle::StoreIndexBits;

		FStore& operator[](uint32 Index) const
		{
			check(Index < Capacity && Data[Index] != nullptr);
			return *Data[Index];
		}

		FStore* CreateAndRegister()
		{
			UE::TScopeLock Lock(Mutex);

			static constexpr uint32 Mask = Capacity - 1; 
			for (uint32 RawIndex = ProbeIndex, RawEnd = ProbeIndex + Capacity; RawIndex < RawEnd + Capacity; ++RawIndex)
			{
				uint32 Index = RawIndex & Mask;
				if (Data[Index] == nullptr)
				{
					Data[Index] = new FStore(Index); 
					ProbeIndex = (Index + 1) & Mask;
					return Data[Index];
				}
			}

			checkf(false, TEXT("Failed to allocated asset registry fixed tag store. "
				"Might be caused by a FAssetDataTagMapSharedView or FAssetRegistryState leak."));

			return nullptr;
		}

		/**
		 * Create a placeholder FStore that is used to trigger SkipSerialize functions on TSerializer
		 * but is otherwise unused and is deleted before serialization returns.
		 */
		FStore* CreateSkipBytesStore()
		{
			return new FStore(Capacity);
		}

		void Unregister(FStore& Store)
		{
			if (Store.Index == Capacity)
			{
				// A placeholder store used in SkipBytes calls. These are never registered or unregistered.
				return;
			}
			check(Store.Index < Capacity);

			UE::TScopeLock Lock(Mutex);
			check(Data[Store.Index] == &Store);

			Data[Store.Index] = nullptr;
		}

#if !UE_BUILD_SHIPPING
		UE::CoreUObject::Private::FStoreBase* const* GetDataForVisualizer() const
		{
			return reinterpret_cast<UE::CoreUObject::Private::FStoreBase* const*>(Data);
		}
#endif

	private:		
		FTransactionallySafeCriticalSection Mutex;
		FStore* Data[Capacity] = {};
		uint32 ProbeIndex = 0;
	}
	GStores;

#if !UE_BUILD_SHIPPING
	// Assign the core global so natvis can resolve FMapHandle pairs from any DLL.
	// GCoreFixedTagStoresForDebugVisualizers is declared in CoreGlobals.h and replicated per-DLL
	// via GFixedTagStoresForDebugVisualizers in ModuleBoilerplate.h.
	static struct FInitFixedTagStoresForVisualizer
	{
		FInitFixedTagStoresForVisualizer()
		{
			GCoreFixedTagStoresForDebugVisualizers = GStores.GetDataForVisualizer();
		}
	} GInitFixedTagStoresForVisualizer;
#endif

	//////////////////////////////////////////////////////////////////////////

	template <bool bForStorage>
	FString	FValueHandle::AsString() const
	{
		FStore& Store = GStores[StoreIndex];
		uint32 Index = Id.Index;

		switch (Id.Type)
		{
		case EValueType::AnsiString:			return FString(Store.GetAnsiString(Index));
		case EValueType::WideString:			return FString(Store.GetWideString(Index));
		case EValueType::NumberlessName:		return MakeNumberedName(Store.NumberlessNames[Index]).GetPlainNameString();
		case EValueType::Name:					return Store.Names[Index].ToString();
		case EValueType::NumberlessExportPath:	return Store.NumberlessExportPaths[Index].ToString();
		case EValueType::ExportPath:			return Store.ExportPaths[Index].ToString();
		case EValueType::LocalizedText:
			return bForStorage ? StringCast<TCHAR>(*Store.Texts[Index].GetAsComplexString()).Get()
				: Store.Texts[Index].GetAsText().ToString();
		}

		check(false);
		return FString();
	}

	int64 FValueHandle::GetResourceSize() const
	{
		FStore& Store = GStores[StoreIndex];
		uint32 Index = Id.Index;

		switch (Id.Type)
		{
		case EValueType::AnsiString:			return TCString<ANSICHAR>::Strlen(Store.GetAnsiString(Index))*sizeof(ANSICHAR);
		case EValueType::WideString:			return TCString<WIDECHAR>::Strlen(Store.GetWideString(Index))*sizeof(WIDECHAR);
		case EValueType::NumberlessName:		return sizeof(FDisplayNameEntryId);
		case EValueType::Name:					return sizeof(FName);
		case EValueType::NumberlessExportPath:	return sizeof(FNumberlessExportPath);
		case EValueType::ExportPath:			return sizeof(FAssetRegistryExportPath);
		case EValueType::LocalizedText:			return sizeof(Store.Texts[Index]) + Store.Texts[Index].GetResourceSize();
		}

		check(false);
		return 0;
	}


	FString	FValueHandle::AsDisplayString() const
	{
		return AsString<false /* bForStorage */>();
	}

	FString	FValueHandle::AsStorageString() const
	{
		return AsString<true /* bForStorage */>();
	}

	FName FValueHandle::AsName() const
	{
		FStore& Store = GStores[StoreIndex];
		uint32 Index = Id.Index;

		ensureMsgf(Id.Type != EValueType::LocalizedText, TEXT("Localized strings should never be converted to FName"));

		switch (Id.Type)
		{
		case EValueType::AnsiString:			return FName(Store.GetAnsiString(Index));
		case EValueType::WideString:			return FName(Store.GetWideString(Index));
		case EValueType::NumberlessName:		return MakeNumberedName(Store.NumberlessNames[Index]);
		case EValueType::Name:					return Store.Names[Index];
		case EValueType::NumberlessExportPath:	return Store.NumberlessExportPaths[Index].ToName();
		case EValueType::ExportPath:			return Store.ExportPaths[Index].ToName();
		case EValueType::LocalizedText:			return FName(Store.Texts[Index].GetAsText().ToString());
		}

		check(false);
		return FName();
	}

	FAssetRegistryExportPath FValueHandle::AsExportPath() const
	{
		FStore& Store = GStores[StoreIndex];
		uint32 Index = Id.Index;

		ensureMsgf(Id.Type != EValueType::LocalizedText, TEXT("Localized strings should never be converted to FAssetRegistryExportPath"));

		switch (Id.Type)
		{
		case EValueType::AnsiString:			return FAssetRegistryExportPath(Store.GetAnsiString(Index));
		case EValueType::WideString:			return FAssetRegistryExportPath(Store.GetWideString(Index));
		case EValueType::NumberlessName:		return MakeNumberedPath(MakeNumberedName(Store.NumberlessNames[Index]));
		case EValueType::Name:					return MakeNumberedPath(Store.Names[Index]);
		case EValueType::NumberlessExportPath:	return MakeNumberedPath(Store.NumberlessExportPaths[Index]);
		case EValueType::ExportPath:			return Store.ExportPaths[Index];
		case EValueType::LocalizedText:			return FAssetRegistryExportPath(Store.Texts[Index].GetAsText().ToString());
		}

		check(false);
		return FAssetRegistryExportPath();
	}

	bool FValueHandle::AsText(FText& Out) const
	{
		if (Id.Type == EValueType::LocalizedText)
		{
			Out = GStores[StoreIndex].Texts[Id.Index].GetAsText();
			return true;
		}

		return false;
	}

	bool FValueHandle::AsMarshalledText(FMarshalledText& Out) const
	{
		if (Id.Type == EValueType::LocalizedText)
		{
			Out = GStores[StoreIndex].Texts[Id.Index];
			return true;
		}
		return false;
	}

	static bool EqualsInsensitive(const FStringView& Str, const FAssetRegistryExportPath& Path)
	{
		TStringBuilder<256> Temp;
		Path.ToString(Temp);
		return Str.Equals(FStringView(Temp), ESearchCase::IgnoreCase);
	}

	static bool EqualsInsensitive(const FStringView& A, FName B)
	{
		TCHAR Buffer[FName::StringBufferSize];
		uint32 Len = B.ToString(Buffer);
		return Len == A.Len() && FPlatformString::Strnicmp(A.GetData(), Buffer, Len) == 0;
	}

	static bool EqualsInsensitive(const FStringView& Str, const FNumberlessExportPath& Path)
	{
		return EqualsInsensitive(Str, MakeNumberedPath(Path));
	}

	bool FValueHandle::Equals(FStringView Str) const
	{
		FStore& Store = GStores[StoreIndex];
		uint32 Index = Id.Index;

		switch (Id.Type)
		{
		case EValueType::AnsiString:			return EqualsInsensitive(Str, Store.GetAnsiString(Index));
		case EValueType::WideString:			return EqualsInsensitive(Str, Store.GetWideString(Index));
		case EValueType::NumberlessName:		return EqualsInsensitive(Str, MakeNumberedName(Store.NumberlessNames[Index]));
		case EValueType::Name:					return EqualsInsensitive(Str, Store.Names[Index]);
		case EValueType::NumberlessExportPath:	return EqualsInsensitive(Str, Store.NumberlessExportPaths[Index]);
		case EValueType::ExportPath:			return EqualsInsensitive(Str, Store.ExportPaths[Index]);
		case EValueType::LocalizedText:			return EqualsInsensitive(Str, *Store.Texts[Index].GetAsComplexString());
		}

		check(false);
		return false;
	}
	
	static bool EqualsInsensitive(FValueId A, FValueId B, const FStore& StoreA, const FStore& StoreB)
	{
		if (A.Type != B.Type)
		{
			// This assumes both stores were indexed with the same FOptions
			return false;
		}

		switch (A.Type)
		{
		case EValueType::AnsiString:			return EqualsInsensitive(StoreA.GetAnsiString(A.Index),			StoreB.GetAnsiString(B.Index));
		case EValueType::WideString:			return EqualsInsensitive(StoreA.GetWideString(A.Index),			StoreB.GetWideString(B.Index));
		case EValueType::NumberlessName:		return EqualsInsensitive(StoreA.NumberlessNames[A.Index],		StoreB.NumberlessNames[B.Index]);
		case EValueType::Name:					return EqualsInsensitive(StoreA.Names[A.Index],					StoreB.Names[B.Index]);
		case EValueType::NumberlessExportPath:	return EqualsInsensitive(StoreA.NumberlessExportPaths[A.Index], StoreB.NumberlessExportPaths[B.Index]);
		case EValueType::ExportPath:			return EqualsInsensitive(StoreA.ExportPaths[A.Index],			StoreB.ExportPaths[B.Index]);
		case EValueType::LocalizedText:			return EqualsInsensitive(StoreA.Texts[A.Index],					StoreB.Texts[B.Index]);
		}

		check(false);
		return false;
	}

	//////////////////////////////////////////////////////////////////////////

	template<typename T>
	TArrayView<T> Slice(TArrayView<T> FullView, int32 SliceIndex, int32 SliceNum)
	{
		check(SliceIndex + SliceNum <= FullView.Num());
		return TArrayView<T>(FullView.GetData() + SliceIndex, SliceNum);
	}

	static_assert(sizeof(FMapHandle) == 8, "");

	TArrayView<const FNumberedPair> FMapHandle::GetNumberedView() const
	{
		check(HasNumberlessKeys == 0);
		return Slice(GStores[StoreIndex].Pairs, PairBegin, Num);
	}

	TArrayView<const FNumberlessPair> FMapHandle::GetNumberlessView() const
	{
		check(HasNumberlessKeys == 1);
		return Slice(GStores[StoreIndex].NumberlessPairs, PairBegin, Num);
	}

	const FValueId* FMapHandle::FindValue(FName Key) const
	{
		if (HasNumberlessKeys == 0)
		{
			for (const FNumberedPair& Pair : GetNumberedView())
			{
				if (Key == Pair.Key)
				{
					return &Pair.Value;
				}
			}
		}
		else if (Key.GetNumber() == NAME_NO_NUMBER_INTERNAL)
		{
			for (const FNumberlessPair& Pair : GetNumberlessView())
			{
				if (Key.GetComparisonIndex() == MakeNumberedName(Pair.Key).GetComparisonIndex())
				{
					return &Pair.Value;
				}
			}
		}
			
		return nullptr;
	}
	
	static FNumberedPair MakeNumberedPair(FNumberlessPair Pair)
	{
		return {MakeNumberedName(Pair.Key), Pair.Value };
	}

	static FNumberlessPair MakeNumberlessPair(FNumberedPair Pair)
	{
		check(Pair.Key.GetNumber() == NAME_NO_NUMBER_INTERNAL);
		return {FDisplayNameEntryId(Pair.Key), Pair.Value};
	}

	FNumberedPair FMapHandle::At(uint32 Index) const
	{
		check(Index < Num);

		if (HasNumberlessKeys == 1)
		{
			return MakeNumberedPair(GStores[StoreIndex].NumberlessPairs[PairBegin + Index]);
		}
		else
		{
			return GStores[StoreIndex].Pairs[PairBegin + Index];
		}
	}

	template<typename PairType>
	static bool EqualsInsensitive(TArrayView<PairType> A, TArrayView<PairType> B, const FStore& AStore, const FStore& BStore)
	{
		check(A.Num() == B.Num());

		for (int32 Idx = 0; Idx < A.Num(); ++Idx)
		{
			if (!EqualsInsensitive(A[Idx].Key,		B[Idx].Key) ||
				!EqualsInsensitive(A[Idx].Value,	B[Idx].Value, AStore, BStore))
			{
				return false;
			}
		}

		return true;
	}

	bool operator==(FMapHandle A, FMapHandle B)
	{
		if (reinterpret_cast<uint64&>(A) == reinterpret_cast<uint64&>(B))
		{
			return true;
		}

		check(A.IsValid & B.IsValid);

		if ((A.Num == B.Num) & (A.HasNumberlessKeys == B.HasNumberlessKeys))
		{
			const FStore& StoreA = GStores[A.StoreIndex];
			const FStore& StoreB = GStores[B.StoreIndex];

			return A.HasNumberlessKeys	? EqualsInsensitive(A.GetNumberlessView(),	B.GetNumberlessView(),	StoreA, StoreB)
										: EqualsInsensitive(A.GetNumberedView(),	B.GetNumberedView(),	StoreA, StoreB);
		}

		return false;
	}

	//////////////////////////////////////////////////////////////////////////

	uint64 FPartialMapHandle::ToInt() const
	{
		return (uint64(bHasNumberlessKeys) << 63) |  (uint64(Num) << 32)  | PairBegin;
	}

	FPartialMapHandle FPartialMapHandle::FromInt(uint64 Int)
	{
		FPartialMapHandle Out;
		Out.bHasNumberlessKeys = !!(Int >> 63);
		Out.Num = static_cast<uint16>(Int >> 32);
		Out.PairBegin = static_cast<uint32>(Int);
		return Out;
	}

	FMapHandle FPartialMapHandle::MakeFullHandle(uint32 StoreIndex) const
	{
		check(StoreIndex < FStoreManager::Capacity);

		FMapHandle Out;
		Out.IsValid = 1;
		Out.HasNumberlessKeys = bHasNumberlessKeys;
		Out.StoreIndex = static_cast<uint16>(StoreIndex);
		Out.Num = Num;
		Out.PairBegin = PairBegin;

		return Out;
	}

	//////////////////////////////////////////////////////////////////////////

	FPartialMapHandle FStoreBuilder::AddTagMap(const FAssetDataTagMapSharedView& Map)
	{
		check(!bFinalized);

		if (Map.Num() == 0)
		{
			// Return PairBegin 0 for empty maps
			return FPartialMapHandle();
		}

		bool bHasNumberlessKeys = true;
		for (TPair<FName, FAssetTagValueRef> Pair : Map)
		{
			bHasNumberlessKeys &= IsNumberless(Pair.Key);
		}
		TArray<FNumberedPair>& Pairs = bHasNumberlessKeys ? NumberlessPairs : NumberedPairs; 
		
		FPartialMapHandle Out;
		Out.bHasNumberlessKeys = bHasNumberlessKeys;
		check(Map.Num() <= TNumericLimits<decltype(FPartialMapHandle::Num)>::Max());
		Out.Num = static_cast<uint16>(Map.Num());
		Out.PairBegin = Pairs.Num();

		TArray<TPair<FName, FAssetTagValueRef>> SortedMap;
		SortedMap.Reserve(Map.Num());
		for (TPair<FName, FAssetTagValueRef> Pair : Map)
		{
			SortedMap.Add(Pair);
		}
		Algo::Sort(SortedMap, [](TPair<FName, FAssetTagValueRef>& A, TPair<FName, FAssetTagValueRef>& B) { return A.Key.LexicalLess(B.Key); });
		for (TPair<FName, FAssetTagValueRef> Pair : SortedMap)
		{
			FValueId Value = IndexValue(Pair.Key, Pair.Value);
			Pairs.Add({Pair.Key, Value});
		}

		return Out;
	}

	
	template<class CharType>
	void CopyString(CharType* Dst, const TCHAR* Src, uint32 Num)
	{
		for (uint32 Idx = 0; Idx < Num; ++Idx)
		{
			check(Src[Idx] <= std::numeric_limits<CharType>::max());
			Dst[Idx] = static_cast<CharType>(Src[Idx]);
		}
	}

	template<class CharType, class MapType>
	TArray<CharType> FlattenAndConcatenateAs(uint32 NumCharacters, const MapType& StringIndices, const TArray<uint32>& Offsets)
	{
		check(StringIndices.Num() == Offsets.Num());

		TArray<CharType> Out;
		Out.SetNumUninitialized(NumCharacters);
		for (const TPair<FString, uint32>& Pair : StringIndices)
		{
			CopyString(&Out[Offsets[Pair.Value]], *Pair.Key, Pair.Key.Len() + 1);
		}

		return Out;
	}

	TArray<ANSICHAR> FStoreBuilder::FStringIndexer::FlattenAsAnsi() const
	{
		return FlattenAndConcatenateAs<ANSICHAR>(NumCharacters, StringIndices, Offsets); 
	}

	TArray<WIDECHAR> FStoreBuilder::FStringIndexer::FlattenAsWide() const
	{
		return FlattenAndConcatenateAs<WIDECHAR>(NumCharacters, StringIndices, Offsets); 
	}

	template<typename T, class KeyFuncs>
	TArray<T> Flatten(const TMap<T, uint32, FDefaultSetAllocator, KeyFuncs>& Index)
	{
		TArray<T> Out;
		Out.SetNum(Index.Num());
		for (const TPair<T, uint32>& Pair : Index)
		{
			Out[Pair.Value] = Pair.Key;
		}

		return Out;
	}

	static TArray<FNumberlessPair> MakeNumberlessPairs(const TArray<FNumberedPair>& In)
	{
		TArray<FNumberlessPair> Out;
		Out.Reserve(In.Num());
		for (FNumberedPair Pair : In)
		{
			Out.Add(MakeNumberlessPair(Pair));
		}
		return Out;
	}

	FStoreData FStoreBuilder::Finalize()
	{
		check(!bFinalized);
		bFinalized = true;

		FStoreData Out;
		Out.Pairs = NumberedPairs;
		Out.NumberlessPairs = MakeNumberlessPairs(NumberlessPairs);
		Out.AnsiStringOffsets = AnsiStrings.Offsets;
		Out.WideStringOffsets = WideStrings.Offsets;
		Out.NumberlessNames = Flatten(NumberlessNameIndices);
		Out.Names = Flatten(NameIndices);
		Out.NumberlessExportPaths = Flatten(NumberlessExportPathIndices);
		Out.ExportPaths = Flatten(ExportPathIndices);
		Out.Texts = Flatten(TextIndices);
		Out.AnsiStrings = AnsiStrings.FlattenAsAnsi();
		Out.WideStrings = WideStrings.FlattenAsWide();

		return Out;
	}
	
	static constexpr uint32 MaxValuesPerType = 1 << FValueId::IndexBits;

	template<class MapType, class ValueType>
	static uint32 Index(MapType& OutIndices, ValueType&& Value)
	{
		uint32 Index = OutIndices.FindOrAdd(MoveTemp(Value), OutIndices.Num());
		check(Index < MaxValuesPerType);
		return Index;
	}

	uint32 FStoreBuilder::FStringIndexer::Index(FString&& String)
	{
		uint32 Len = String.Len();
		uint32 Idx = FixedTagPrivate::Index(StringIndices, MoveTemp(String));
		
		if (Offsets.Num() < StringIndices.Num())
		{
			Offsets.Add(NumCharacters);
			NumCharacters += Len + 1;

			check(Offsets.Num() == StringIndices.Num());
			checkf(NumCharacters > Offsets.Last(), TEXT("Overflow"));
		}

		return Idx;
	}

	FValueId FStoreBuilder::IndexValue(FName Key, FAssetTagValueRef Value)
	{
		FMarshalledText MarshalledText; 
		if (Value.TryGetAsMarshalledText(MarshalledText))
		{
			return FValueId{EValueType::LocalizedText,			Index(TextIndices, MoveTemp(MarshalledText))};
		}
		else if (Options.StoreAsName.Contains(Key))
		{
			FName Name = Value.AsName();
			return IsNumberless(Name)
				? FValueId{EValueType::NumberlessName,			Index(NumberlessNameIndices, FDisplayNameEntryId(Name))}
				: FValueId{EValueType::Name,					Index(NameIndices, Name)};
		}
		else if (Options.StoreAsPath.Contains(Key))
		{
			FAssetRegistryExportPath Path = Value.AsExportPath();
			return IsNumberless(Path)
				? FValueId{EValueType::NumberlessExportPath,	Index(NumberlessExportPathIndices, MakeNumberlessPath(Path))}
				: FValueId{EValueType::ExportPath,				Index(ExportPathIndices, MoveTemp(Path))};
		}
		else
		{	
			FString String = Value.AsString();
			// Zero-length Tag Values are invalid. They are supposed to be prevented by early exits in
			// UE::AssetRegistry::ReadPackageDataMain and FAssetData::SetTagsAndAssetBundles.
			checkf(String.Len() > 0, TEXT("Invalid Zero-length tag: Key: \"%s\", Value: \"%s\" IsFixed: %s."),
				*Key.ToString(), *String, *LexToString(Value.IsFixed()));
			if (FCString::IsPureAnsi(*String, String.Len()))
			{
				return FValueId{EValueType::AnsiString,	 AnsiStrings.Index(MoveTemp(String))};
			}
			else
			{
				return FValueId{EValueType::WideString,	WideStrings.Index(MoveTemp(String))};
			}
		}
	}

	//////////////////////////////////////////////////////////////////////////

	enum EOrder { Member, TextFirst, SkipText };

	template<EOrder Order = Member, typename StoreType, typename VisitorType>
	void VisitViews(StoreType&& Store, VisitorType&& Visitor)
	{
		// This order determines serialization order and the binary format.
		// Serializing string offsets is redundant, they can be recreated from null terminators,
		// but I've chosen to include them for code simplicity. --jtorp

		if (Order == EOrder::TextFirst)
		{
			Visitor(Store.Texts);
		}

		Visitor(Store.NumberlessNames);
		Visitor(Store.Names);
		Visitor(Store.NumberlessExportPaths);
		Visitor(Store.ExportPaths);

		if (Order == EOrder::Member)
		{
			Visitor(Store.Texts);
		}
		
		Visitor(Store.AnsiStringOffsets);
		Visitor(Store.WideStringOffsets);
		Visitor(Store.AnsiStrings);
		Visitor(Store.WideStrings);

		Visitor(Store.NumberlessPairs);
		Visitor(Store.Pairs);
	}

	template<typename T>
	void DestroyElements(TArrayView<T> View)
	{
		for (T& Item : View)
		{
			Item.~T();
		}
	}


	void* FStore::AllocateMemoryForView(SIZE_T NumBytes, SIZE_T AlignmentRequired)
	{
		void* Memory = FMemory::Malloc(NumBytes, static_cast<uint32>(AlignmentRequired));
		if (LIKELY(Memory))
		{
			FScopeLock Lock(&ViewPointersCS);
			ViewPointers.Add(Memory);
		}
		return Memory;
	}

	FStore::FStore(uint32 InIndex) 
		: Index(InIndex)
	{
		checkf(!AutoRTFM::IsClosed() || !AutoRTFM::IsOnCurrentTransactionStack(this), TEXT("Not allowed to construct a stack local within a transaction."));
		RefCountObject = new FStoreRefCount(this);
	}

	FStore::~FStore()
	{
		GStores.Unregister(*this);

		// not locking the pointers since no other thread should be using the store at this point
		VisitViews(*this, [this](auto& View)
			{
				// If the view is not memory-mmapped, destroy its elements. Memory-mapped views have no-op destructors
				if (ViewPointers.Contains(View.GetData()))
				{
					DestroyElements(View);
				}
			}
		);

		for (void* Ptr : ViewPointers)
		{
			FMemory::Free(Ptr);
		}
	}

	FStoreRefCount::FStoreRefCount(FStore* InOwner)
		: Owner(InOwner)
	{
		WeakAddRef();
	}

	void FStoreRefCount::StrongRelease()
	{
		UE_AUTORTFM_ONCOMMIT(this)
		{
			if (StrongRefCount.fetch_add(-1, std::memory_order_acq_rel) == 1)
			{
				delete Owner;
				Owner = nullptr;
				// The call to WeakRelease might delete this; do not access this after this call to WeakRelease.
				// Note that it will not delete this if external code still holds a weak pointer.
				WeakRelease();
			}
		};
	}

	void FStoreRefCount::WeakAddRef()
	{
		UE_AUTORTFM_OPEN
		{
			WeakRefCount.fetch_add(1, std::memory_order_relaxed);
		};

		UE_AUTORTFM_ONABORT(this)
		{
			WeakRefCount.fetch_add(-1, std::memory_order_relaxed);
		};
	}

	void FStoreRefCount::WeakRelease()
	{
		UE_AUTORTFM_ONCOMMIT(this)
		{
			if (WeakRefCount.fetch_add(-1, std::memory_order_acq_rel) == 1)
			{
				delete this;
			}
		};
	}

	TRefCountPtr<const FStore> FStoreRefCount::Pin()
	{
		bool bSucceeded = false;

		// This call to UE_AUTORTFM_OPEN is matched by a second UE_AUTORTFM_OPEN before we leave the function
		// that undoes the memory change, so we do not need a matching UE_AUTORTFM_ONABORT.
		UE_AUTORTFM_OPEN 
		{
			// See SharedPointerInternals::TReferenceControllerBase<Mode>::ConditionallyAddSharedReference
			// for commenting about the std::atomic implementation details of this loop.
			int32 OriginalCount = StrongRefCount.load(std::memory_order_relaxed);
			for (;;)
			{
				if (OriginalCount == 0)
				{
					// Never add a shared reference if the pointer has already expired
					break;
				}
				if (StrongRefCount.compare_exchange_weak(OriginalCount, OriginalCount + 1, std::memory_order_relaxed))
				{
					bSucceeded = true;
					break;
				}
			}
		};

		if (bSucceeded)
		{
			// Now that we have a ref count, we can safely access the object. Create a TRefCountPtr based on it,
			// which increments the refcount again, and then drop the refcount we added; we only needed it to
			// be able to thread-safely access our Owner pointer.
			TRefCountPtr<const FStore> Result(Owner);

			UE_AUTORTFM_OPEN
			{
				StrongRefCount.fetch_add(-1, std::memory_order_relaxed);
			};
			return Result;
		}
		else
		{
			return TRefCountPtr<const FStore>();
		}
	}

	FWeakStorePtr::FWeakStorePtr(const TRefCountPtr<const FStore>& Store)
	{
		if (Store)
		{
			RefCountObject = Store->RefCountObject;
			RefCountObject->WeakAddRef();
		}
	}

	FWeakStorePtr::FWeakStorePtr(const FWeakStorePtr& Other)
	{
		if (Other.RefCountObject)
		{
			RefCountObject = Other.RefCountObject;
			RefCountObject->WeakAddRef();
		}
	}

	FWeakStorePtr::FWeakStorePtr(FWeakStorePtr&& Other)
	{
		if (Other.RefCountObject)
		{
			RefCountObject = Other.RefCountObject;
			Other.RefCountObject = nullptr;
		}
	}

	FWeakStorePtr& FWeakStorePtr::operator=(const FWeakStorePtr& Other)
	{
		if (&Other == this)
		{
			return *this;
		}
		FStoreRefCount* OldObject = RefCountObject;
		RefCountObject = Other.RefCountObject;
		if (RefCountObject)
		{
			RefCountObject->WeakAddRef();
		}
		if (OldObject)
		{
			OldObject->WeakRelease();
		}
		return *this;
	}

	FWeakStorePtr& FWeakStorePtr::operator=(FWeakStorePtr&& Other)
	{
		if (&Other == this)
		{
			return *this;
		}
		FStoreRefCount* OldObject = RefCountObject;
		RefCountObject = Other.RefCountObject;
		if (RefCountObject)
		{
			Other.RefCountObject = nullptr;
		}
		if (OldObject)
		{
			OldObject->WeakRelease();
		}
		return *this;
	}

	FWeakStorePtr::~FWeakStorePtr()
	{
		if (RefCountObject)
		{
			RefCountObject->WeakRelease();
		}
	}

	TRefCountPtr<const FStore> FWeakStorePtr::Pin() const
	{
		if (RefCountObject)
		{
			TRefCountPtr<const FStore> StrongPtr = RefCountObject->Pin();
			if (!StrongPtr)
			{
				RefCountObject->WeakRelease();
				RefCountObject = nullptr;
			}
			return StrongPtr;
		}
		else
		{
			return TRefCountPtr<const FStore>();
		}
	}

	//////////////////////////////////////////////////////////////////////////
	
	template<class T>
	uint64 GetBytes(TArrayView<T> View)
	{
		return sizeof(T) * View.Num();
	}

	template<class T>
	void SetNum(TArrayView<T>& View, int32 Num)
	{
		View = MakeArrayView(View.GetData(), Num);
	}


	template<class T>
	void SetUntypedDataPtr(TArrayView<T>& View, void* Data)
	{
		check(IsAligned(Data, alignof(T)));
		View = MakeArrayView(reinterpret_cast<T*>(Data), View.Num());
	}

	template <FAssetRegistryVersion::Type Version>
	class TSerializer
	{
		template<class T>
		void SaveItem(T Item)
		{
			Ar << Item;
		}

		template<class T>
		T LoadItem()
		{
			T Item;
			Ar << Item;
			return Item;
		}

		template<class T>
		void SkipItem()
		{
			(void) LoadItem<T>();
		}

		void SaveItem(FDisplayNameEntryId NameId)
		{
			SaveItem(MakeNumberedName(NameId));
		}

		template<> FDisplayNameEntryId LoadItem<FDisplayNameEntryId>()
		{
			return FDisplayNameEntryId(LoadItem<FName>());
		}

		template<> void SkipItem<FDisplayNameEntryId>()
		{
			SkipItem<FName>();
		}

		void SaveItem(FNumberlessExportPath Path)
		{
			SaveItem(MakeNumberedPath(Path));
		}

		template<> FAssetRegistryExportPath LoadItem()
		{
			if constexpr (Version >= FAssetRegistryVersion::ClassPaths)
			{
				FAssetRegistryExportPath ExportPath;
				Ar << ExportPath;
				return ExportPath;
			}
			else
			{
				return ConvertLegacyPath(LoadItem<FLegacyAssetRegistryExportPath>());
			}
		}

		template<> FNumberlessExportPath LoadItem()
		{
			return MakeNumberlessPath(LoadItem<FAssetRegistryExportPath>());
		}

		template<> void SkipItem<FNumberlessExportPath>()
		{
			SkipItem<FAssetRegistryExportPath>();
		}

		void SaveItem(FValueId Value)
		{
			SaveItem(Value.ToInt());
		}

		template<> FValueId LoadItem<FValueId>()
		{
			return FValueId::FromInt(LoadItem<uint32>());
		}
		
		void SaveItem(FNumberlessPair Pair)
		{
			SaveItem(Pair.Key);
			SaveItem(Pair.Value);
		}
		
		template<> FNumberlessPair LoadItem<>()
		{
			// Note: Don't change to constructor - only braced initialization evaluation order is guaranteed until C++17
			return { LoadItem<FDisplayNameEntryId>(), LoadItem<FValueId>() };
		}

		void SaveItem(FNumberedPair Pair)
		{
			SaveItem(Pair.Key);
			SaveItem(Pair.Value);
		}

		template<> FNumberedPair LoadItem<>()
		{
			// Note: Don't change to constructor - only braced initialization evaluation order is defined until C++17
			return { LoadItem<FName>(), LoadItem<FValueId>() };
		}
		
		void SaveItem(const FMarshalledText& Text)
		{
			Ar << const_cast<FUtf8String&>(Text.GetAsComplexString());
		}

		template<> FMarshalledText LoadItem()
		{
			if constexpr (Version >= FAssetRegistryVersion::MarshalledTextAsUTF8String)
			{
				FUtf8String ComplexString;
				Ar << ComplexString;
				return FMarshalledText(MoveTemp(ComplexString));
			}
			else
			{
				FString ComplexString;
				Ar << ComplexString;
				return FMarshalledText(ComplexString);
			}
		}

		template<> void SkipItem<FMarshalledText>()
		{
			if constexpr (Version >= FAssetRegistryVersion::MarshalledTextAsUTF8String)
			{
				FUtf8String ComplexString;
				Ar << ComplexString;
			}
			else
			{
				FString ComplexString;
				Ar << ComplexString;
			}
		}

		template<typename T>
		void SaveViewData(TArrayView<T> Items)
		{
			for (T& Item : Items)
			{
				SaveItem(Item);
			}
		}

		template<class T>
		void MemZeroViewDataIfNeeded(TArrayView<T> Items)
		{
			// Handle LoadViewData() assignment and early destruction in case of archive error
			// for non-trivial types like FText
			if (!std::is_trivially_move_assignable_v<T> || !std::is_trivially_destructible_v<T>)
			{
				FMemory::Memzero(Items.GetData(), Items.Num() * sizeof(T));
			}
		}

		template<typename T>
		void LoadViewData(TArrayView<T> Items)
		{
			if (!Ar.IsError())
			{
				for (T& Item : Items)
				{
					Item = LoadItem<T>();
				}
			}
		}
		template <typename T>
		void SkipViewData(TArrayView<T> Items)
		{
			// If this function is too slow, we can change the FStore serialized format to include byte lengths for
			// each view.
			if (!Ar.IsError())
			{
				int32 NumItems = Items.Num();
				for (int32 Index = 0; Index < NumItems; ++Index)
				{
					SkipItem<T>();
				}
			}
		}

		template<typename T>
		void BulkSerialize(TArrayView<T> View)
		{
			Ar.Serialize(View.GetData(), View.Num() * View.GetTypeSize());
		}
		
		void SaveViewData(TArrayView<ANSICHAR> View) { BulkSerialize(View); }
		void SaveViewData(TArrayView<WIDECHAR> View) { BulkSerialize(View); }
		void LoadViewData(TArrayView<ANSICHAR> View) { BulkSerialize(View); }
		void LoadViewData(TArrayView<WIDECHAR> View) { BulkSerialize(View); }
		static_assert(PLATFORM_LITTLE_ENDIAN, "Byte-swapping WIDECHAR on load needed to load on big-endian platforms");
		void SkipViewData(TArrayView<WIDECHAR> View) { Ar.Seek(Ar.Tell() + GetBytes(View)); }
		void SkipViewData(TArrayView<ANSICHAR> View) { Ar.Seek(Ar.Tell() + GetBytes(View)); }
		void SkipViewData(TArrayView<uint32> View) { Ar.Seek(Ar.Tell() + GetBytes(View)); };

		template<class T>
		inline void LoadViewFromMemoryWithoutAttemptingToMap(FStore& Store, TArrayView<T>& View)
		{
			uint64 Bytes = GetBytes(View);
			if (Bytes)
			{
				SetUntypedDataPtr(View, Store.AllocateMemoryForView(Bytes, alignof(T)));
				MemZeroViewDataIfNeeded(View);
				LoadViewData(View);
			}
		}

		// by default, assume the type cannot be memory mapped
		template<class T>
		void LoadViewFromMemoryMappedArchive(FStore& Store, TArrayView<T>& View)
		{
			LoadViewFromMemoryWithoutAttemptingToMap(Store, View);
		}

		void LoadViewFromMemoryMappedArchive(FStore& Store, TArrayView<uint32>& View)
		{
#if WITH_EDITOR
			if (Store.BackingFile.IsValid())
			{
				if (int64 Size = GetBytes(View))
				{
					int64 Offset = Ar.Tell();
					FMemoryView Part = Store.BackingFile->GetView(Offset, Size);
					if (void* DataPtr = const_cast<void*>(Part.GetData()))
					{
						SetUntypedDataPtr(View, DataPtr);
						Ar.Seek(Offset + Size);
					}
					else
					{
						// in an unlikely case we could not map, switch to allocation
						LoadViewFromMemoryWithoutAttemptingToMap(Store, View);
					}
				}
			}
			else
#endif
			{
				LoadViewFromMemoryWithoutAttemptingToMap(Store, View);
			}
		}

		// ANSI and WIDE chars can be memory mapped
		void LoadViewFromMemoryMappedArchive(FStore& Store, TArrayView<ANSICHAR>& View)
		{
#if WITH_EDITOR
			if (Store.BackingFile.IsValid())
			{
				if (int64 Size = GetBytes(View))
				{
					int64 Offset = Ar.Tell();
					FMemoryView Part = Store.BackingFile->GetView(Offset, Size);
					if (void* DataPtr = const_cast<void*>(Part.GetData()))
					{
						SetUntypedDataPtr(View, DataPtr);
						Ar.Seek(Offset + Size);
					}
					else
					{
						// in an unlikely case we could not map, switch to allocation
						LoadViewFromMemoryWithoutAttemptingToMap(Store, View);
					}
				}
			}
			else
#endif
			{
				LoadViewFromMemoryWithoutAttemptingToMap(Store, View);
			}
		}
		void LoadViewFromMemoryMappedArchive(FStore& Store, TArrayView<WIDECHAR>& View)
		{
#if WITH_EDITOR
			if (Store.BackingFile.IsValid())
			{
				if (int64 Size = GetBytes(View))
				{
					int64 Offset = Ar.Tell();
					FMemoryView Part = Store.BackingFile->GetView(Offset, Size);
					if (void* DataPtr = const_cast<void*>(Part.GetData()))
					{
						SetUntypedDataPtr(View, DataPtr);
						Ar.Seek(Offset + Size);
					}
					else
					{
						// in an unlikely case we could not map, switch to allocation
						LoadViewFromMemoryWithoutAttemptingToMap(Store, View);
					}
				}
			}
			else
#endif
			{
				LoadViewFromMemoryWithoutAttemptingToMap(Store, View);
			}
		}


		FArchive& Ar;
		FString Scratch;

		static constexpr uint32 OldBeginMagic	= 0x12345678;
		static constexpr uint32 BeginMagic		= 0x12345679;
		static constexpr uint32 EndMagic		= 0x87654321;
		static constexpr uint32 MaxViewAlignment = 16;
	public:
		TSerializer(FArchive& InAr)
			: Ar(InAr)
		{
		}

		void SaveTextData(TArrayView<const FMarshalledText> Texts)
		{
			FArrayWriter Data;
			TSerializer<Version>(Data).SaveViewData(Texts);
			SaveItem(Data.Num());
			AlignPosInArchive();
			Ar.Serialize(Data.GetData(), Data.Num());
		}

		void Save(const FStoreData& Store)
		{
			// Store is expected to be saved at an aligned position
			check(UE::AssetRegistry::IsPosInArchiveAligned(Ar));
			SaveItem(BeginMagic);
			VisitViews(Store, [&](auto& Array) { SaveItem(Array.Num()); });
			SaveTextData(MakeArrayView(Store.Texts));
			// while not all data will be mmaped, give all of it a possibility to be so by aligning everything
			VisitViews<EOrder::SkipText>(Store, [&] (auto& Array) { AlignPosInArchive(); SaveViewData(MakeArrayView(Array)); });
			SaveItem(EndMagic);
		}

		static TOptional<ELoadOrder> GetLoadOrder(uint32 InitialMagic)
		{
			switch (InitialMagic)
			{
				case OldBeginMagic:	return ELoadOrder::Member;
				case BeginMagic:	return ELoadOrder::TextFirst;
				default:			return TOptional<ELoadOrder>();
			}
		}

		inline void AlignPosInArchive()
		{
			if constexpr (Version >= FAssetRegistryVersion::MemoryMappedTagDataStore)
			{
				UE::AssetRegistry::AlignPosInArchive(Ar);
			}
		}

		TOptional<ELoadOrder> LoadHeader(FStore& Store)
		{
			// store is expected to be saved at an aligned position, so it can align its data
			AlignPosInArchive();
			uint32 InitialMagic = LoadItem<uint32>();
			TOptional<ELoadOrder> Order = GetLoadOrder(InitialMagic);

			if (!Order)
			{
				UE_LOGF(LogAssetDataTags, Warning, "Bad init magic, archive '%ls' is corrupt", *Ar.GetArchiveName());
				Ar.SetError();
				return Order;
			}

			// Load view sizes
			VisitViews(Store, [&] (auto& View) { SetNum(View, LoadItem<int32>()); });

			return Order;
		}
		
		void Load(FStore& Store)
		{
			if (TOptional<ELoadOrder> Order = LoadHeader(/* Out */ Store))
			{
				// Load view data
				if (Order.GetValue() == ELoadOrder::TextFirst)
				{
					uint32 TextDataBytes = LoadItem<uint32>();

					VisitViews<EOrder::TextFirst>(Store, [&](auto& View) { AlignPosInArchive(); LoadViewFromMemoryMappedArchive(Store, View); });
				}
				else
				{
					VisitViews(Store, [&] (auto View) { LoadViewData(View); });
				}

				if (LoadItem<uint32>() != EndMagic)
				{
					UE_LOGF(LogAssetDataTags, Warning, "Bad end magic, archive '%ls' is corrupt", *Ar.GetArchiveName());
					Ar.SetError();
				}
			}
		}

		void SkipBytes()
		{
			TRefCountPtr<FStore> PlaceholderStore = GStores.CreateSkipBytesStore();
			if (TOptional<ELoadOrder> Order = LoadHeader(/* Out */ *PlaceholderStore))
			{
				// Load view data
				checkf(Order.GetValue() == ELoadOrder::TextFirst, TEXT("SkipBytes of old versions is not supported."));
				uint32 TextDataBytes = LoadItem<uint32>();

				VisitViews<EOrder::TextFirst>(*PlaceholderStore, [&](auto& View)
					{
						AlignPosInArchive();
						SkipViewData(View);
					});

				if (LoadItem<uint32>() != EndMagic)
				{
					UE_LOGF(LogAssetDataTags, Warning, "Bad end magic, archive '%ls' is corrupt", *Ar.GetArchiveName());
					Ar.SetError();
				}
			}
		}

		TArray<uint8> ReadTextData()
		{
			uint32 TextDataBytes = LoadItem<uint32>();
			TArray<uint8> Out;
			Out.SetNumUninitialized(TextDataBytes);
			AlignPosInArchive();
			Ar.Serialize(Out.GetData(), Out.Num());
			return Out;
		}

		void LoadTextData(FStore& Store)
		{
			LoadViewFromMemoryMappedArchive(Store, Store.Texts);
		}

		void LoadFinalData(FStore& Store, ELoadOrder Order)
		{
			checkf(Order == ELoadOrder::TextFirst, TEXT("Loading old versions of cache is no longer supported"));
			VisitViews<EOrder::SkipText>(Store, [&] (auto& View) { AlignPosInArchive(); LoadViewFromMemoryMappedArchive(Store, View); });
			
			if (LoadItem<uint32>() != EndMagic)
			{
				UE_LOGF(LogAssetDataTags, Warning, "Bad end magic, archive '%ls' is corrupt", *Ar.GetArchiveName());
				Ar.SetError();
			}
		}
	};

	void SaveStore(const FStoreData& Store, FArchive& Ar)
	{
		// Saving always uses the latest FAssetRegistryVersion
		TSerializer<FAssetRegistryVersion::LatestVersion>(Ar).Save(Store);
	}

	TRefCountPtr<const FStore> LoadStore(FArchive& Ar, FAssetRegistryVersion::Type Version /*= FAssetRegistryVersion::LatestVersion*/, TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> InBackingFile /*= nullptr*/)
	{
		if (Ar.IsError())
		{
			return nullptr;
		}

		FStore* Store = GStores.CreateAndRegister();
#if WITH_EDITOR
		Store->BackingFile = InBackingFile;
#endif

		// Versioning support inside of TSerializer was added with FAssetRegistryVersion::ClassPaths so everything before can just be
		// serialized with FAssetRegistryVersion::ClassPaths - 1 = FAssetRegistryVersion::AddedChunkHashes.
		// The main purpose of hardcoding version value and passing it as template argument is to eliminate
		// branches in performance critical serialization code.
		// Note that we branch in this function only because it's the only function used to load legacy 
		// asset registries in DiffAssetRegistries commandlet hence it's editor only code too
		if (Version < FAssetRegistryVersion::ClassPaths)
		{
			TSerializer<FAssetRegistryVersion::AddedChunkHashes>(Ar).Load(*Store);
		}
		else if (Version <= FAssetRegistryVersion::AssetPackageDataHasPackageLocation)
		{
			TSerializer<FAssetRegistryVersion::AssetPackageDataHasPackageLocation>(Ar).Load(*Store);
		}
		else if (Version <= FAssetRegistryVersion::TagSetMapSupports64BitStringNum)
		{
			TSerializer<FAssetRegistryVersion::TagSetMapSupports64BitStringNum>(Ar).Load(*Store);
		}
		else // Add more branches should serialization format change
		{
			TSerializer<FAssetRegistryVersion::LatestVersion>(Ar).Load(*Store);
		}

		return TRefCountPtr<const FStore>(Store);
	}

	void SkipBytesStore(FArchive& Ar, FAssetRegistryVersion::Type Version /*= FAssetRegistryVersion::LatestVersion*/)
	{
		if (Ar.IsError())
		{
			return;
		}

		// We only support SkipBytes for the latest version.
		TSerializer<FAssetRegistryVersion::LatestVersion>(Ar).SkipBytes();
	}

	FAsyncStoreLoader::FAsyncStoreLoader()
		: Store(GStores.CreateAndRegister())
	{}

	TFuture<void> FAsyncStoreLoader::ReadInitialDataAndKickLoad(FArchive& Ar, uint32 MaxWorkerTasks, FAssetRegistryVersion::Type HeaderVersion)
	{
		// should only load latest version of data
		check(HeaderVersion == FAssetRegistryVersion::LatestVersion);
		Order = TSerializer<FAssetRegistryVersion::LatestVersion>(Ar).LoadHeader(*Store);

		checkf(Order == ELoadOrder::TextFirst, TEXT("Reading old formats isn't supported."));

		UE::AssetRegistry::AlignPosInArchive(Ar);
		TArray<uint8> TextData = TSerializer<FAssetRegistryVersion::LatestVersion>(Ar).ReadTextData();

		if (TextData.Num())
		{
			return Async(EAsyncExecution::TaskGraph, [Data = MoveTemp(TextData), OutStore = Store]()
			{
				FMemoryReader Reader(Data);
				TSerializer<FAssetRegistryVersion::LatestVersion>(Reader).LoadTextData(*OutStore);
			});
		}
		return TFuture<void>();
	}

	TRefCountPtr<const FStore> FAsyncStoreLoader::LoadFinalData(FArchive& Ar, FAssetRegistryVersion::Type HeaderVersion)
	{
		if (Order.IsSet())
		{
			// should only load latest version of data
			check(HeaderVersion == FAssetRegistryVersion::LatestVersion);
			TSerializer<FAssetRegistryVersion::LatestVersion>(Ar).LoadFinalData(/* Out */ *Store, Order.GetValue());

			return TRefCountPtr<const FStore>(Store);
		}

		return TRefCountPtr<const FStore>();
	}

	TRefCountPtr<const FStore> InitializeFromBuilder(FStoreBuilder& InBuilder)
	{
		TArray64<uint8> Data;
		FMemoryWriter64 MemoryWriter(Data, true);
		FStoreData StoreData = InBuilder.Finalize();

		SaveStore(StoreData, MemoryWriter);

		FMemoryView DataView(Data);
		FMemoryReaderView MemoryReader(DataView, true);
		return LoadStore(MemoryReader);
	}

} // end namespace FixedTagPrivate

////////////////////////////////////////////////////////////////////////////

FixedTagPrivate::FValueHandle FAssetTagValueRef::AsFixed() const
{
	checkSlow(IsFixed());
	return { Fixed.GetStoreIndex(), FixedTagPrivate::FValueId::FromInt(Fixed.GetValueId()) }; 
}

const FString& FAssetTagValueRef::AsLoose() const
{
	checkSlow(!IsFixed());
	check(IsSet());
	return *Loose;
}

FString FAssetTagValueRef::AsString() const
{
	return IsFixed() ? AsFixed().AsDisplayString() : LocalizeIfComplexString(AsLoose());
}

FName FAssetTagValueRef::AsName() const
{
	return IsFixed() ? AsFixed().AsName() : FName(AsLoose());
}

FAssetRegistryExportPath FAssetTagValueRef::AsExportPath() const
{
	return IsFixed() ? AsFixed().AsExportPath() : FAssetRegistryExportPath(AsLoose());
}

bool FAssetTagValueRef::TryGetAsText(FText& Out) const
{
	return IsFixed() ? AsFixed().AsText(Out) : FromComplexString(AsLoose(), Out);
}

FText FAssetTagValueRef::AsText() const
{
	FText Tmp;
	return TryGetAsText(Tmp) ? Tmp : FText::FromString(IsFixed() ? AsFixed().AsStorageString() : AsLoose());
}

FString FAssetTagValueRef::ToLoose() const
{
	return IsFixed() ? AsFixed().AsStorageString() : AsLoose();
}

int64 FAssetTagValueRef::GetResourceSize() const
{
	return IsFixed() ? AsFixed().GetResourceSize() : AsLoose().GetAllocatedSize();
}

bool FAssetTagValueRef::TryGetAsMarshalledText(FixedTagPrivate::FMarshalledText& Out) const
{
	if (IsFixed())
	{
		return AsFixed().AsMarshalledText(Out);
	}
	else
	{
		const FString& String = AsLoose();
		if (FTextStringHelper::IsComplexText(*String))
		{
			Out = FixedTagPrivate::FMarshalledText(String);
			return true;
		}
		return false;
	}
}

bool FAssetTagValueRef::Equals(FStringView Str) const
{
	if (IsSet())
	{
		return IsFixed() ? AsFixed().Equals(Str) : FStringView(AsLoose()) == Str; 
	}

	return Str.IsEmpty();
}

////////////////////////////////////////////////////////////////////////////

FAssetDataTagMapSharedView::FAssetDataTagMapSharedView(const FAssetDataTagMapSharedView& O)
	: Bits(O.Bits)
{
	if (IsFixed())
	{
		FixedTagPrivate::FStore* Store = &FixedTagPrivate::GStores[Fixed.StoreIndex];
		UE_AUTORTFM_OPEN
		{
			Store->AddRef();
		};
		UE_AUTORTFM_ONABORT(Store)
		{
			Store->Release();
		};
	}
	else if (IsLoose())
	{
		UE_AUTORTFM_OPEN
		{
			Loose->RefCount.Increment(); //-V614
		};
		UE_AUTORTFM_ONABORT(Loose = this->Loose)
		{
			if (Loose->RefCount.Decrement() == 0)
			{
				delete Loose;
			}
		};
	}
}

FAssetDataTagMapSharedView::FAssetDataTagMapSharedView(FAssetDataTagMapSharedView&& O)
	: Bits(O.Bits)
{
	O.Bits = 0;
}

FAssetDataTagMapSharedView::FAssetDataTagMapSharedView(FixedTagPrivate::FMapHandle InFixed)
	: Fixed(InFixed)
{
	FixedTagPrivate::FStore* Store = &FixedTagPrivate::GStores[Fixed.StoreIndex];
	UE_AUTORTFM_OPEN
	{
		Store->AddRef();
	};
	UE_AUTORTFM_ONABORT(Store)
	{
		Store->Release();
	};
}

FAssetDataTagMapSharedView::FAssetDataTagMapSharedView(FAssetDataTagMap&& InLoose)
{
	if (InLoose.Num())
	{
		Loose = new FAssetDataTagMap(MoveTemp(InLoose));
		UE_AUTORTFM_OPEN
		{
			Loose->RefCount.Increment();
		};
	}
}

FAssetDataTagMapSharedView& FAssetDataTagMapSharedView::operator=(const FAssetDataTagMapSharedView& O)
{
	FAssetDataTagMapSharedView Tmp(O);
	Swap(Bits, Tmp.Bits);
	return *this;
}

FAssetDataTagMapSharedView& FAssetDataTagMapSharedView::operator=(FAssetDataTagMapSharedView&& O)
{
	FAssetDataTagMapSharedView Tmp(MoveTemp(O));
	Swap(Bits, Tmp.Bits);
	return *this;
}

FAssetDataTagMapSharedView::~FAssetDataTagMapSharedView()
{
	if (IsFixed())
	{
		FixedTagPrivate::FStore* Store = &FixedTagPrivate::GStores[Fixed.StoreIndex];
		UE_AUTORTFM_ONCOMMIT(Store)
		{
			Store->Release();
		};
	}
	else if (IsLoose())
	{
		UE_AUTORTFM_ONCOMMIT(Loose = this->Loose)
		{
			if (Loose->RefCount.Decrement() == 0)
			{
				delete Loose;
			}
		};
	}
}

FAssetDataTagMap FAssetDataTagMapSharedView::CopyMap() const
{
	if (IsFixed())
	{
		FAssetDataTagMap Out;
		Out.Reserve(Num());
		ForEach([&](TPair<FName, FAssetTagValueRef> Pair){ Out.Add(Pair.Key, Pair.Value.AsFixed().AsStorageString()); });
		return Out;
	}
	else
	{
		return Loose != nullptr ? *Loose : FAssetDataTagMap();
	}
}

void FAssetDataTagMapSharedView::Shrink()
{
	if (IsLoose())
	{
		Loose->Shrink();
	}
}

namespace FixedTagPrivate
{
	static bool MapEqualsHelper(FixedTagPrivate::FMapHandle Fixed, const FAssetDataTagMap* Loose)
	{
		checkSlow(Fixed.Num == Loose->Num());
		// Since Num is the same and the maps are unique, test only whether all keys in Fixed exist with equal value
		// in Loose. Once we've done that, we don't have to test whether any keys in Loose are missing from Fixed.
		if (Fixed.HasNumberlessKeys != 0)
		{
			for (FNumberlessPair Pair : Fixed.GetNumberlessView())
			{
				const FString* LooseValue = Loose->Find(MakeNumberedName(Pair.Key));
				if (!LooseValue || FAssetTagValueRef(Fixed.StoreIndex, Pair.Value) != *LooseValue)
				{
					return false;
				}
			}
		}
		else
		{
			for (FNumberedPair Pair : Fixed.GetNumberedView())
			{
				const FString* LooseValue = Loose->Find(Pair.Key);
				if (!LooseValue || FAssetTagValueRef(Fixed.StoreIndex, Pair.Value) != *LooseValue)
				{
					return false;
				}
			}
		}
		return true;
	}
}

bool operator==(const FAssetDataTagMapSharedView& A, const FAssetDataTagMap& B)
{
	if (A.Num() != B.Num())
	{
		return false;
	}
	else if (A.IsFixed())
	{
		return FixedTagPrivate::MapEqualsHelper(A.Fixed, &B);
	}

	return A.Num() == 0 || *A.Loose == B;
}

bool operator==(const FAssetDataTagMapSharedView& A, const FAssetDataTagMapSharedView& B)
{
	if (A.Num() != B.Num())
	{
		return false;
	}
	else if (A.Num() == 0)
	{
		return true;
	}
	else if (A.IsFixed() != B.IsFixed())
	{
		return A.IsFixed() ? FixedTagPrivate::MapEqualsHelper(A.Fixed, B.Loose) :
			FixedTagPrivate::MapEqualsHelper(B.Fixed, A.Loose);
	}
	else if (A.IsFixed())
	{
		return A.Fixed == B.Fixed;
	}
	else
	{
		return *A.Loose == *B.Loose;
	}
}

static SIZE_T GetLooseMapMemoryUsage(const FAssetDataTagMap& Map)
{
	SIZE_T Out = sizeof(FAssetDataTagMap) + Map.GetAllocatedSize();
	for (const TPair<FName, FString> & Pair : Map)
	{
		Out += Pair.Value.GetAllocatedSize();
	}

	return Out;
}

void FAssetDataTagMapSharedView::FMemoryCounter::Include(const FAssetDataTagMapSharedView& Tags)
{
	if (Tags.IsFixed())
	{
		FixedStoreIndices.Add(Tags.Fixed.StoreIndex);
	}
	else if (Tags.IsLoose())
	{
		LooseBytes += GetLooseMapMemoryUsage(*Tags.Loose);
	}
}

SIZE_T FAssetDataTagMapSharedView::FMemoryCounter::GetFixedSize() const
{
	SIZE_T Out = 0;
	for (uint32 StoreIndex : FixedStoreIndices)
	{
		Out += sizeof(FixedTagPrivate::FStore);
		VisitViews(FixedTagPrivate::GStores[StoreIndex], [&Out] (auto& View) { Out += View.Num() * View.GetTypeSize(); });
	}

	return Out;
}

void FAssetDataTagMapSharedView::FMemoryCounter::ReportFixedStoreBreakdown() const
{
	for (uint32 StoreIndex : FixedStoreIndices)
	{
		uint32 Idx = 0;
		VisitViews(FixedTagPrivate::GStores[StoreIndex], [StoreIndex, &Idx] (auto& View) {
			UE_LOG(LogAssetDataTags, Log, TEXT("Store[%d]:View[%d]: Num=%d Size=%" SIZE_T_FMT "k"), StoreIndex, Idx++, View.Num(), View.Num() * View.GetTypeSize() / 1024);
		});
	}
}

namespace UE::AssetRegistry
{
	FMemoryMappedFile::FMemoryMappedFile(const TCHAR* Path)
	{
		FOpenMappedResult Result = FPlatformFileManager::Get().GetPlatformFile().OpenMappedEx(Path, IPlatformFile::EOpenReadFlags::AllowDelete);
		if (!Result.HasError())
		{
			Handle = Result.StealValue();
		}
	}

	FMemoryView FMemoryMappedFile::GetView(uint64 Start, uint64 Length) const
	{
		FMappedSubRegion New;
		New.Start = Start;
		New.Length = Length;

		if (!Handle)
		{
			return FMemoryView();
		}

		New.Region.Reset(Handle->MapRegion(Start, Length));
		if (!New.Region)
		{
			return FMemoryView();
		}

		FMemoryView Result = FMemoryView(New.Region->GetMappedPtr(), New.Region->GetMappedSize());
		Regions.Add(MoveTemp(New));
		return Result;
	}

} // namespace UE::AssetRegistry

////////////////////////////////////////////////////////////////////////////

#if WITH_DEV_AUTOMATION_TESTS 

#include "Misc/AutomationTest.h"
#include "Serialization/MemoryReader.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAssetRegistryExportPathTest, "System.AssetRegistry.ExportPath", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FAssetRegistryExportPathTest::RunTest(const FString& Parameters)
{
	TestEqual("Full numbered path",	FAssetRegistryExportPath("/S/P.C_1\'P_2.O_3\'").ToString(),	"/S/P.C_1\'P_2.O_3\'");
	TestEqual("Full path",			FAssetRegistryExportPath("/S/P.C\'P.O\'").ToString(),		"/S/P.C\'P.O\'");
	TestEqual("Package path",		FAssetRegistryExportPath("P.O").ToString(),					"P.O");
	TestEqual("Object only path",	FAssetRegistryExportPath("O").ToString(),					"O");
	TestEqual("Class parse",		FAssetRegistryExportPath("/S/P.C\'P.O\'").ClassPath.ToString(),	"/S/P.C");
	TestEqual("Package parse",		FAssetRegistryExportPath("/S/P.C\'P.O\'").Package,			FName("P"));
	TestEqual("Object parse",		FAssetRegistryExportPath("/S/P.C\'P.O\'").Object,			FName("O"));

	return true;
}

namespace FixedTagPrivate
{

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCompactExportPathTest, "System.AssetRegistry.FixedTag.NumberlessExportPath", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FCompactExportPathTest::RunTest(const FString& Parameters)
{
	const FAssetRegistryExportPath FullPath = FAssetRegistryExportPath("/S/P.C\'P.O\'");

	TestTrue("Roundtrip",	FullPath == MakeNumberedPath(MakeNumberlessPath(FullPath)));

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FStoreTest, "System.AssetRegistry.FixedTag.Store", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

static TRefCountPtr<const FStore> MakeTestStore(FStoreData& Data)
{
	FStore* Out = GStores.CreateAndRegister();

	Out->NumberlessPairs		= MakeArrayView(Data.NumberlessPairs);
	Out->Pairs					= MakeArrayView(Data.Pairs);

	Out->AnsiStringOffsets		= MakeArrayView(Data.AnsiStringOffsets);
	Out->WideStringOffsets		= MakeArrayView(Data.WideStringOffsets);
	Out->NumberlessNames		= MakeArrayView(Data.NumberlessNames);
	Out->Names					= MakeArrayView(Data.Names);
	Out->NumberlessExportPaths	= MakeArrayView(Data.NumberlessExportPaths);
	Out->ExportPaths			= MakeArrayView(Data.ExportPaths);
	Out->Texts					= MakeArrayView(Data.Texts);

	Out->AnsiStrings			= MakeArrayView(Data.AnsiStrings);
	Out->WideStrings			= MakeArrayView(Data.WideStrings);

	return Out;
}

FAssetDataTagMapSharedView MakeLooseMap(std::initializer_list<TPairInitializer<const char*, FString>> Pairs)
{
	FAssetDataTagMap Out;
	Out.Reserve(Pairs.size());
	for (TPair<const char*, FString> Pair : Pairs)
	{
		Out.Add(FName(Pair.Key), Pair.Value);
	}
	return FAssetDataTagMapSharedView(MoveTemp(Out));
}


static FAnsiStringView Scan(FAnsiStringView String, FAnsiStringView Substring)
{
	check(Substring.Len() > 0);

	for (FAnsiStringView It = String; It.Len() >= Substring.Len(); It.RightChopInline(1))
	{
		if (It.StartsWith(Substring, ESearchCase::CaseSensitive))
		{
			return It;
		}
	}

	return FAnsiStringView(); 
}

static uint32 CountOccurences(const TArray<ANSICHAR>& Characters, FAnsiStringView Substring)
{
	FAnsiStringView It = FAnsiStringView(Characters.GetData(), Characters.Num());
	for (uint32 Out = 0; true; ++Out)
	{
		It = Scan(It, Substring);

		if (It.IsEmpty())
		{
			return Out;
		}

		It = It.RightChop(Substring.Len());
	}
}

bool FStoreTest::RunTest(const FString& Parameters)
{
	TArray<FAssetDataTagMapSharedView> LooseMaps;
	LooseMaps.Add(FAssetDataTagMapSharedView());
	LooseMaps.Add(MakeLooseMap({{"Key",			"StringValue"}, 
								{"Key_0",		"StringValue_0"}}));
	LooseMaps.Add(MakeLooseMap({{"Name",		"NameValue"}, 
								{"Name_0",		"NameValue_0"}}));
	LooseMaps.Add(MakeLooseMap({{"FullPath",	"/S/P.C\'P.O\'"}, 
								{"PkgPath",		"P.O"},
								{"ObjPath",		"O"}}));
	LooseMaps.Add(MakeLooseMap({{"NumPath_0",	"/S/P.C\'P.O_0\'"}, 
								{"NumPath_1",	"/S/P.C\'P_0.O\'"},
								{"NumPath_2",	"/S/P.C_0\'P.O\'"},
								{"NumPath_3",	"/S/P.C\'P_0.O_0\'"},
								{"NumPath_4",	"/S/P.C_0\'P_0.O\'"},
								{"NumPath_5",	"/S/P.C_0\'P.O_0\'"},
								{"NumPath_6",	"/S/P.C_0\'P_0.O_0\'"}}));
	LooseMaps.Add(MakeLooseMap({{"SameSame",	"SameSame"}, 
								{"AlsoSame",	"SameSame"}}));
	LooseMaps.Add(MakeLooseMap({{"FilterKey1",	"FilterValue1"}, 
								{"FilterKey2",	"FilterValue2"}}));
	LooseMaps.Add(MakeLooseMap({{"Localized",	"NSLOCTEXT(\"\", \"5F8411BA4D1A349F6E2C56BB04A1A810\", \"Content Browser Walkthrough\")"}}));
	LooseMaps.Add(MakeLooseMap({{"Wide",		TEXT("Wide\x00DF")}}));
	LooseMaps.Add(MakeLooseMap({{"LowerCase",	"stringvalue"}}));

	FOptions Options;
	Options.StoreAsName = {	"Name", "Name_0"};
	Options.StoreAsPath = {	"FullPath", "PkgPath", "ObjPath",
							"NumPath_0", "NumPath_1", "NumPath_2",
							"NumPath_3", "NumPath_4", "NumPath_5", "NumPath_6"};


	auto FixLooseMaps =
		[this, &LooseMaps](const FOptions& Options)
		{
			FStoreBuilder Builder(Options);
			TArray<FPartialMapHandle> PartialFixedMaps;
			for (FAssetDataTagMapSharedView LooseMap : LooseMaps)
			{
				PartialFixedMaps.Add(Builder.AddTagMap(LooseMap));
			}

			// FixedData must outlive FixedStore
			FStoreData FixedData = Builder.Finalize();
			TRefCountPtr<const FStore> FixedStore = MakeTestStore(FixedData);

			TArray<FAssetDataTagMapSharedView> FixedMaps;
			for (FPartialMapHandle PartialMap : PartialFixedMaps)
			{
				FMapHandle FixedMapHandle = PartialMap.MakeFullHandle(FixedStore->Index);
				FixedMaps.Add(FAssetDataTagMapSharedView(FixedMapHandle));
			}

			TestTrue("StoreBuilder round-trip", FixedMaps == LooseMaps);

			return FixedData;
		};

	// Test values are stored with type configured in options
	{
		FStoreData Data = FixLooseMaps(Options);

		TestEqual("String values contains",		CountOccurences(Data.AnsiStrings, "StringValue_0"), 1);
		TestEqual("String values contains",		CountOccurences(Data.AnsiStrings, "StringValue"), 2);
		TestEqual("String values lower case",	CountOccurences(Data.AnsiStrings, "stringvalue"), 1);
		TestEqual("String values excludes",		CountOccurences(Data.AnsiStrings, "NameValue"), 0);
		TestEqual("String value deduplication",	CountOccurences(Data.AnsiStrings, "SameSame"), 1);
		TestEqual("Wide characters",			FStringView(Data.WideStrings.GetData(), Data.WideStrings.Num() - 1), FStringView(TEXT("Wide\x00DF")));
		TestEqual("Numberless name values",		Data.NumberlessNames, {FDisplayNameEntryId("NameValue")});
		TestEqual("Numbered name values",		Data.Names, {FName("NameValue_0")});
		TestEqual("Numberless path values",		Data.NumberlessExportPaths.Num(), 3); // C\'P.O\', P.O, O  
		TestEqual("Numbered path values",		Data.ExportPaths.Num(), 7); // NumPath[0-7] values
		TestEqual("Localized values",			Data.Texts.Num(), 1);
		TestEqual("Numberless keys",			Data.NumberlessPairs.Num(), 10);
		TestEqual("Numbered keys",				Data.Pairs.Num(), 11);
	}

	// Test all values are stored as strings with default options
	{
		FStoreData Data = FixLooseMaps(FOptions{});

		TestEqual("Numberless name values",		Data.NumberlessNames.Num(), 0);
		TestEqual("Numbered name values",		Data.Names.Num(), 0);
		TestEqual("Numberless path values",		Data.NumberlessExportPaths.Num(), 0);
		TestEqual("Numbered path values",		Data.ExportPaths.Num(), 0);
		TestEqual("Localized values",			Data.Texts.Num(), 1);
		TestEqual("Numberless keys",			Data.NumberlessPairs.Num(), 10);
		TestEqual("Numbered keys",				Data.Pairs.Num(), 11);
	}
	
	return true;
}

}

#endif //WITH_DEV_AUTOMATION_TESTS
