// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistry/AssetDataToken.h" 

#define LOCTEXT_NAMESPACE "AssetRegistry"

FOnMessageTokenActivated FAssetDataToken::DefaultMessageTokenActivated;
FAssetDataToken::FOnGetDisplayName FAssetDataToken::DefaultGetAssetDisplayName;

TSharedRef<FAssetDataToken> FAssetDataToken::Create(const FAssetData& InAssetData, const FText& InLabelOverride)
{
    return MakeShared<FAssetDataToken>(FPrivateToken{}, InAssetData, InLabelOverride);
}

FAssetDataToken::FAssetDataToken(FPrivateToken, const FAssetData& InAssetData, const FText& InLabelOverride)
    : AssetData(InAssetData)
{
	if ( !InLabelOverride.IsEmpty() )
	{
		CachedText = InLabelOverride;
	}
	else
	{
		if ( DefaultGetAssetDisplayName.IsBound() )
		{
			CachedText = DefaultGetAssetDisplayName.Execute(InAssetData, false);
		}
#if WITH_EDITORONLY_DATA
		else if (InAssetData.GetOptionalOuterPathName() != FName{})
		{
			// Provide both path and stored package
			CachedText = FText::Format(LOCTEXT("AssetDataToken_ExternalPackageObject", "{0} ({1})"), 
				FText::FromString(InAssetData.GetExportTextName()),
				FText::FromStringView(WriteToString<256>(InAssetData.PackageName).ToView())
			);
		}
#endif
		else 
		{
			CachedText = FText::FromString(InAssetData.GetExportTextName());
		}
	}
}

const FOnMessageTokenActivated& FAssetDataToken::GetOnMessageTokenActivated() const
{
	if(MessageTokenActivated.IsBound())
	{
		return MessageTokenActivated;
	}
	else
	{
		return DefaultMessageTokenActivated;
	}
}

#undef LOCTEXT_NAMESPACE