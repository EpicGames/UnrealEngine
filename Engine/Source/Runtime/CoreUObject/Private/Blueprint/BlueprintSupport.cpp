// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprint/BlueprintSupport.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectHash.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "Serialization/DuplicatedDataWriter.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageName.h"
#include "Async/UniqueLock.h"
#include "UObject/ObjectResource.h"
#include "UObject/GCObject.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerLoadImportBehavior.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/LinkerPlaceholderFunction.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/StructScriptLoader.h"
#include "UObject/UObjectThreadContext.h"

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
#include "UObject/UObjectIterator.h"
#endif

#if WITH_EDITORONLY_DATA
#include "UObject/PropertyStateTracking.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintSupport, Log, All);

const FName FBlueprintTags::GeneratedClassPath(TEXT("GeneratedClass"));
const FName FBlueprintTags::ParentClassPath(TEXT("ParentClass"));
const FName FBlueprintTags::NativeParentClassPath(TEXT("NativeParentClass"));
const FName FBlueprintTags::ClassFlags(TEXT("ClassFlags"));
const FName FBlueprintTags::BlueprintType(TEXT("BlueprintType"));
const FName FBlueprintTags::BlueprintDescription(TEXT("BlueprintDescription"));
const FName FBlueprintTags::BlueprintDisplayName(TEXT("BlueprintDisplayName"));
const FName FBlueprintTags::BlueprintCategory(TEXT("BlueprintCategory"));
const FName FBlueprintTags::IsDataOnly(TEXT("IsDataOnly"));
const FName FBlueprintTags::ImplementedInterfaces(TEXT("ImplementedInterfaces"));
const FName FBlueprintTags::FindInBlueprintsData(TEXT("FiBData"));
const FName FBlueprintTags::UnversionedFindInBlueprintsData(TEXT("FiB"));
const FName FBlueprintTags::NumReplicatedProperties(TEXT("NumReplicatedProperties"));
const FName FBlueprintTags::NumNativeComponents(TEXT("NativeComponents"));
const FName FBlueprintTags::NumBlueprintComponents(TEXT("BlueprintComponents"));
const FName FBlueprintTags::BlueprintPathWithinPackage(TEXT("BlueprintPath"));

static TAutoConsoleVariable<bool> CVarEnableFullBlueprintPreloading(
	TEXT("linker.EnableFullBlueprintPreloading"),
	true,
	TEXT("If true, Blueprint class regeneration will perform a complete preload of all dependencies.")
);

/**
 * Defined in BlueprintSupport.cpp
 * Duplicates all fields of a class in depth-first order. It makes sure that everything contained
 * in a class is duplicated before the class itself, as well as all function parameters before the
 * function itself.
 *
 * @param	StructToDuplicate			Instance of the struct that is about to be duplicated
 * @param	Writer						duplicate writer instance to write the duplicated data to
 */
void FBlueprintSupport::DuplicateAllFields(UStruct* StructToDuplicate, FDuplicateDataWriter& Writer)
{
	// This is a very simple fake topological-sort to make sure everything contained in the class
	// is processed before the class itself is, and each function parameter is processed before the function
	if (StructToDuplicate)
	{
		// Make sure each field gets allocated into the array
		for (TFieldIterator<UField> FieldIt(StructToDuplicate, EFieldIteratorFlags::ExcludeSuper); FieldIt; ++FieldIt)
		{
			UField* Field = *FieldIt;

			// Make sure functions also do their parameters and children first
			if (UFunction* Function = dynamic_cast<UFunction*>(Field))
			{
				for (TFieldIterator<UField> FunctionFieldIt(Function, EFieldIteratorFlags::ExcludeSuper); FunctionFieldIt; ++FunctionFieldIt)
				{
					UField* InnerField = *FunctionFieldIt;
					Writer.GetDuplicatedObject(InnerField);
				}
			}

			Writer.GetDuplicatedObject(Field);
		}
	}
}

bool FBlueprintSupport::UseDeferredDependencyLoading()
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	static const FBoolConfigValueHelper DeferDependencyLoads(TEXT("Kismet"), TEXT("bDeferDependencyLoads"), GEngineIni);
	bool bUseDeferredDependencyLoading = DeferDependencyLoads;

	if (FPlatformProperties::RequiresCookedData())
	{
		static const FBoolConfigValueHelper DisableCookedBuildDefering(TEXT("Kismet"), TEXT("bForceDisableCookedDependencyDeferring"), GEngineIni);
		bUseDeferredDependencyLoading &= !((bool)DisableCookedBuildDefering);
	}
	return bUseDeferredDependencyLoading;
#else
	return false;
#endif
}

bool FBlueprintSupport::IsDeferredExportCreationDisabled()
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	static const FBoolConfigValueHelper NoDeferredExports(TEXT("Kismet"), TEXT("bForceDisableDeferredExportCreation"), GEngineIni);
	return !UseDeferredDependencyLoading() || NoDeferredExports;
#else
	return false;
#endif
}

bool FBlueprintSupport::IsDeferredCDOInitializationDisabled()
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	static const FBoolConfigValueHelper NoDeferredCDOInit(TEXT("Kismet"), TEXT("bForceDisableDeferredCDOInitialization"), GEngineIni);
	return !UseDeferredDependencyLoading() || NoDeferredCDOInit;
#else
	return false;
#endif
}

static FFlushReinstancingQueueFPtr FlushReinstancingQueueFPtr = nullptr;
static FFlushCompilationQueueFPtr FlushCompilationQueueFPtr = nullptr;
static FClassReparentingFPtr ClassReparentingFPtr = nullptr;

void FBlueprintSupport::FlushReinstancingQueue()
{
	if(FlushReinstancingQueueFPtr)
	{
		(*FlushReinstancingQueueFPtr)();
	}
}

void FBlueprintSupport::FlushCompilationQueue(FUObjectSerializeContext* InLoadContext)
{
	if (FlushCompilationQueueFPtr)
	{
		(*FlushCompilationQueueFPtr)(InLoadContext);
	}
}

void FBlueprintSupport::ReparentHierarchies(const TMap<UClass*, UClass*>& OldClassToNewClass)
{
	if (ClassReparentingFPtr)
	{
		(*ClassReparentingFPtr)(OldClassToNewClass);
	}
}

void FBlueprintSupport::SetFlushReinstancingQueueFPtr(FFlushReinstancingQueueFPtr Ptr)
{
	FlushReinstancingQueueFPtr = Ptr;
}

void FBlueprintSupport::SetFlushCompilationQueueFPtr(FFlushCompilationQueueFPtr Ptr)
{
	FlushCompilationQueueFPtr = Ptr;
}

void FBlueprintSupport::SetClassReparentingFPtr(FClassReparentingFPtr Ptr)
{
	ClassReparentingFPtr = Ptr;
}

bool FBlueprintSupport::IsDeferredDependencyPlaceholder(const UObject* LoadedObj)
{
	return LoadedObj && ( LoadedObj->IsA<ULinkerPlaceholderClass>() ||
		LoadedObj->IsA<ULinkerPlaceholderFunction>() ||
		LoadedObj->IsA<ULinkerPlaceholderExportObject>() );
}

#if WITH_EDITOR
void FBlueprintSupport::RegisterDeferredDependenciesInStruct(const UStruct* Struct, void* StructData)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	for (TPropertyValueIterator<const FObjectProperty> It(Struct, StructData); It; ++It)
	{
		const FObjectProperty* Property = It.Key();
		void* PropertyValue = (void*)It.Value();
		TObjectPtr<UObject> ObjectValue = Property->GetObjectPtrPropertyValue(PropertyValue);

		ULinkerPlaceholderExportObject* PlaceholderVal = Cast<ULinkerPlaceholderExportObject>(ObjectValue);
		ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(ObjectValue);

		if (PlaceholderVal == nullptr && PlaceholderClass == nullptr)
		{
			continue;
		}

		// Create a stack of property trackers to deal with any outer Struct Properties
		TArray<const FProperty*> PropertyChain;
		It.GetPropertyChain(PropertyChain);
		TIndirectArray<FScopedPlaceholderPropertyTracker> PlaceholderStack;

		// Iterate property chain in reverse order as we need to start with parent
		for (int32 PropertyIndex = PropertyChain.Num() - 1; PropertyIndex >= 0; PropertyIndex--)
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(PropertyChain[PropertyIndex]))
			{
				PlaceholderStack.Add(new FScopedPlaceholderPropertyTracker(StructProperty));
			}
		}
		
		if (PlaceholderVal)
		{
			PlaceholderVal->AddReferencingPropertyValue(Property, PropertyValue);
		}
		else 
		{
			PlaceholderClass->AddReferencingPropertyValue(Property, PropertyValue);
		}

		// Specifically destroy entries in reverse order they were added, to simulate unrolling a code stack
		for (int32 StackIndex = PlaceholderStack.Num() - 1; StackIndex >= 0; StackIndex--)
		{
			PlaceholderStack.RemoveAt(StackIndex);
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}
#endif

void FBlueprintSupport::RepairDeferredDependenciesInObject(UObject* Object)
{
	// Go through each property's value on the object and check for placeholders. 
	//   Try to replace them with the real imported object, if it exists.
	// This function was created to catch any cases where a deferred dependency fails to resolve during load

	for (TPropertyValueIterator<const FObjectProperty> It(Object->GetClass(), Object); It; ++It)
	{
		const FObjectProperty* Property = It.Key();
		const TObjectPtr<UObject>& PropertyValue = Property->GetPropertyValue(It.Value());
		if (!PropertyValue.IsResolved())
		{
			continue;
		}

		UObject* PropertyValueAsObj = PropertyValue.Get();

		FLinkerPlaceholderBase* Placeholder = nullptr;

		if (ULinkerPlaceholderExportObject* ValueAsPlaceholderObj = Cast<ULinkerPlaceholderExportObject>(PropertyValueAsObj))
		{
			Placeholder = (FLinkerPlaceholderBase*)ValueAsPlaceholderObj;
		}
		else if (ULinkerPlaceholderClass* ValueAsPlaceholderClass = Cast<ULinkerPlaceholderClass>(PropertyValueAsObj))
		{
			Placeholder = (FLinkerPlaceholderBase*)ValueAsPlaceholderClass;
		}

		if (Placeholder)
		{
			UE_LOGF(LogBlueprintSupport, Warning, "Object %ls still has a %ls '%ls' in property %ls. This indicates a failure to resolve every deferred/circular dependency in blueprints.", *Object->GetName(), *PropertyValueAsObj->GetClass()->GetName(), *PropertyValueAsObj->GetName(), *Property->GetName());

			bool bDidRepairStalePlaceholder = false;

			if (!Placeholder->PackageIndex.IsNull())
			{
				if (const UPackage* PlaceholderPackage = PropertyValueAsObj->GetPackage())
				{
					if (FLinkerLoad* PlaceholderLinker = PlaceholderPackage->GetLinker())
					{
						int32 const ImportIndex = Placeholder->PackageIndex.ToImport();
						FObjectImport& Import = PlaceholderLinker->ImportMap[ImportIndex];
						if ((Import.XObject != nullptr) && (Import.XObject != PropertyValueAsObj))
						{
							Property->SetObjectPropertyValue(PropertyValue, Import.XObject);
							bDidRepairStalePlaceholder = true;

							UE_LOGF(LogBlueprintSupport, Display, "Repaired deferred dependency on object %ls: replaced '%ls' with '%ls'", *Object->GetName(), *PropertyValueAsObj->GetName(), *Import.XObject->GetName());
						}
					}
				}
			}

			if (!bDidRepairStalePlaceholder)
			{
				UE_LOGF(LogBlueprintSupport, Error, "Failed to repair deferred dependency on object %ls (%ls). This may lead to blueprint execution problems.", *Object->GetName(), *PropertyValueAsObj->GetName());
			}
		}
	}

}


bool FBlueprintSupport::IsInBlueprintPackage(UObject* LoadedObj)
{
	bool bHasBlueprintClass = false;

	UPackage* Pkg = LoadedObj->GetOutermost();
	if (Pkg && !Pkg->HasAnyPackageFlags(PKG_CompiledIn))
	{	
		ForEachObjectWithOuterBreakable(Pkg, [&bHasBlueprintClass](UObject* PkgObj)
		{
			if (UClass* PkgClass = Cast<UClass>(PkgObj))
			{
				bHasBlueprintClass = PkgClass && PkgClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint);
				return false; // break
			}

			return true;
		}, EGetObjectsFlags::None);
		
	}
	return bHasBlueprintClass;
}

static TArray<FBlueprintWarningDeclaration> BlueprintWarnings;
static TSet<FName> BlueprintWarningsToTreatAsError;
static TSet<FName> BlueprintWarningsToSuppress;

void FBlueprintSupport::RegisterBlueprintWarning(const FBlueprintWarningDeclaration& Warning)
{
	BlueprintWarnings.Add(Warning);
}

const TArray<FBlueprintWarningDeclaration>& FBlueprintSupport::GetBlueprintWarnings()
{
	return BlueprintWarnings;
}

void FBlueprintSupport::UpdateWarningBehavior(const TArray<FName>& WarningIdentifiersToTreatAsError, const TArray<FName>& WarningIdentifiersToSuppress)
{
	BlueprintWarningsToTreatAsError = TSet<FName>(WarningIdentifiersToTreatAsError);
	BlueprintWarningsToSuppress = TSet<FName>(WarningIdentifiersToSuppress);
}

bool FBlueprintSupport::ShouldTreatWarningAsError(FName WarningIdentifier)
{
	return BlueprintWarningsToTreatAsError.Find(WarningIdentifier) != nullptr;
}

bool FBlueprintSupport::ShouldSuppressWarning(FName WarningIdentifier)
{
	return BlueprintWarningsToSuppress.Find(WarningIdentifier) != nullptr;
}

bool FBlueprintSupport::IsClassPlaceholder(const UClass* Class)
{
	while (Class)
	{
		if (Cast<const ULinkerPlaceholderClass>(Class))
		{
			return true;
		}

		Class = Class->GetSuperClass();
	}

	return false;
}

#if WITH_EDITOR
void FBlueprintSupport::ValidateNoRefsToOutOfDateClasses()
{
	// ensure no TRASH/REINST types remain:
	TArray<UObject*> OutOfDateClasses;
	GetObjectsOfClass(UClass::StaticClass(), OutOfDateClasses);
	OutOfDateClasses.RemoveAllSwap( 
		[](UObject* Obj)
		{ 
			UClass* AsClass = CastChecked<UClass>(Obj);
			return (!AsClass->HasAnyClassFlags(CLASS_NewerVersionExists)) || !AsClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint); 
		} 
	);

	for(UObject* Obj : OutOfDateClasses)
	{
		FReferenceChainSearch RefChainSearch(Obj, EReferenceChainSearchMode::Shortest);
		if( RefChainSearch.GetReferenceChains().Num() != 0 )
		{
			RefChainSearch.PrintResults();
			ensureAlwaysMsgf(false, TEXT("Found and output bad class references"));
		}
	}
}

void FBlueprintSupport::ValidateNoExternalRefsToSkeletons()
{
	// bit of a hack to find the skel class, because UBlueprint is not visible here,
	// but it's very useful to be able to validate BP assumptions in low level code:
	auto IsSkeleton = [](UClass* InClass)
	{
		return InClass->ClassGeneratedBy && InClass->GetName().StartsWith(TEXT("SKEL_"));
	};

	auto IsOuteredToSkeleton = [IsSkeleton](UObject* Object)
	{
		UObject* Iter = Object->GetOuter();
		while(Iter)
		{
			if(UClass* AsClass = Cast<UClass>(Iter))
			{
				if(IsSkeleton(AsClass))
				{
					return true;
				}
			}
			Iter = Iter->GetOuter();
		}
		return false;
	};

	TArray<UObject*> SkeletonClasses;
	GetObjectsOfClass(UClass::StaticClass(), SkeletonClasses);
	SkeletonClasses.RemoveAllSwap( 
		[IsSkeleton](UObject* Obj)
		{ 
			UClass* AsClass = CastChecked<UClass>(Obj);
			return !IsSkeleton(AsClass);
		} 
	);

	for(UObject* SkeletonClass : SkeletonClasses)
	{
		FReferenceChainSearch RefChainSearch(SkeletonClass, EReferenceChainSearchMode::Shortest | EReferenceChainSearchMode::ExternalOnly);
		bool bBadRefs = false;
		for(const FReferenceChainSearch::FReferenceChain* Chain : RefChainSearch.GetReferenceChains())
		{
			UObject* ChainRootObject = Chain->GetRootNode()->ObjectInfo->TryResolveObject();
			checkf(ChainRootObject, TEXT("Unable to resolve reference chain root object %s"), *Chain->GetRootNode()->ObjectInfo->GetPathName());
			if(ChainRootObject->GetOutermost() != SkeletonClass->GetOutermost())
			{
				bBadRefs = true;
				for (int32 NodeIndex = 1; bBadRefs && NodeIndex < Chain->Num(); ++NodeIndex)
				{
					// if there's a skeleton class (or an object outered to a skeleton class) somewhere in the chain, then it's fine:
					UObject* ObjectReferencingSkeletonClass = Chain->GetNode(NodeIndex)->ObjectInfo->TryResolveObject();
					checkf(ChainRootObject, TEXT("Unable to resolve reference object referencing skeleton class %s"), *Chain->GetNode(NodeIndex)->ObjectInfo->GetPathName());
					if (UClass* AsClass = Cast<UClass>(ObjectReferencingSkeletonClass))
					{
						if (IsSkeleton(AsClass))
						{
							bBadRefs = false;
						}
					}
					else if (IsOuteredToSkeleton(ObjectReferencingSkeletonClass))
					{
						bBadRefs = false;
					}
				}
			}
		}

		if(bBadRefs)
		{
			RefChainSearch.PrintResults();
			ensureAlwaysMsgf(false, TEXT("Found and output bad references to skeleton classes"));
		}
	}
}
#endif // WITH_EDITOR

/*******************************************************************************
 * FScopedClassDependencyGather
 ******************************************************************************/

#if WITH_EDITOR
thread_local UClass* FScopedClassDependencyGather::BatchAuthorityClass = nullptr;
thread_local TArray<UClass*> FScopedClassDependencyGather::BatchClassDependencies;
thread_local FScopedClassDependencyGather* FScopedClassDependencyGather::AuthoritativeScope = nullptr;

FScopedClassDependencyGather::FScopedClassDependencyGather(UClass* ClassToGather, FUObjectSerializeContext* InLoadContext)
	: bAuthoritativeClass(false)
	, LoadContext(InLoadContext)
{
	// Do NOT track duplication dependencies, as these are intermediate products that we don't care about
	if( !GIsDuplicatingClassForReinstancing )
	{
		if( BatchAuthorityClass == nullptr )
		{
			// If there is no current dependency authority, register this class as the authority, and reset the array
			BatchAuthorityClass = ClassToGather;
			BatchClassDependencies.Empty();
			bAuthoritativeClass = true;

			AuthoritativeScope = this;
		}
		else
		{
			// This class was instantiated while another class was gathering dependencies, so record it as a dependency
			BatchClassDependencies.AddUnique(ClassToGather);
		}
	}
}

FScopedClassDependencyGather::~FScopedClassDependencyGather()
{
	// If this gatherer was the initial gatherer for the current scope, process 
	// dependencies (unless compiling on load is explicitly disabled)
	if( bAuthoritativeClass )
	{
		BatchAuthorityClass->ConditionalRecompileClass(LoadContext);

		BatchAuthorityClass = nullptr;
	}

	if (AuthoritativeScope == this)
	{
		AuthoritativeScope = nullptr;

		OnScopeExitDelegate.Broadcast();
	}
}

FScopedClassDependencyGather* FScopedClassDependencyGather::GetAuthoritativeScope()
{
	return AuthoritativeScope;
}

TArray<UClass*> const& FScopedClassDependencyGather::GetCachedDependencies()
{
	return BatchClassDependencies;
}
#endif //WITH_EDITOR



/*******************************************************************************
 * FLinkerLoad
 ******************************************************************************/

// rather than littering the code with USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
// checks, let's just define DEFERRED_DEPENDENCY_CHECK for the file
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	#define DEFERRED_DEPENDENCY_CHECK(CheckExpr) ensure(CheckExpr)
#else  // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	#define DEFERRED_DEPENDENCY_CHECK(CheckExpr)
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
 
struct FPreloadMembersHelper
{
	static void PreloadMembers(UObject* InObject)
	{
		// Collect a list of all things this element owns
		TArray<UObject*> BPMemberReferences;
		FReferenceFinder ComponentCollector(BPMemberReferences, InObject, false, true, true, true);
		ComponentCollector.FindReferences(InObject);

		// Iterate over the list, and preload everything so it is valid for refreshing
		for (TArray<UObject*>::TIterator it(BPMemberReferences); it; ++it)
		{
			UObject* CurrentObject = *it;
			if (!CurrentObject->HasAnyFlags(RF_LoadCompleted))
			{
				if (FLinkerLoad* Linker = CurrentObject->GetLinker())
				{
					UE::TUniqueLock LinkerLock(Linker->Mutex);
					CurrentObject->SetFlags(RF_NeedLoad);
					Linker->Preload(CurrentObject);
					PreloadMembers(CurrentObject);
				}
			}
		}
	}

	static void PreloadExternalNativeDependencies(UObject* InObject)
	{
		TArray<UObject*> MemberReferences;
		FReferenceFinder ComponentCollector(MemberReferences, nullptr, false, true, true, true);
		ComponentCollector.FindReferences(InObject);

		for (UObject* CurrentObject : MemberReferences)
		{
			check(CurrentObject);

			const bool bIsValidNativeDependency =
				CurrentObject->HasAnyFlags(RF_NeedLoad) &&
				!CurrentObject->IsA<UClass>() &&
				!CurrentObject->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint)
			;

			if (bIsValidNativeDependency)
			{
				if (FLinkerLoad* Linker = CurrentObject->GetLinker())
				{
					Linker->Preload(CurrentObject);
					PreloadExternalNativeDependencies(CurrentObject);
				}
			}
		}
	}

	static void PreloadObject(UObject* InObject)
	{
		if (InObject && !InObject->HasAnyFlags(RF_LoadCompleted))
		{
			if (FLinkerLoad* Linker = InObject->GetLinker())
			{
				UE::TUniqueLock LinkerLock(Linker->Mutex);
				InObject->SetFlags(RF_NeedLoad);
				Linker->Preload(InObject);
			}
		}
	}
};

/** 
 * A helper utility for tracking exports whose classes we're currently running
 * through ForceRegenerateClass(). This is primarily relied upon to help prevent
 * infinite recursion since ForceRegenerateClass() doesn't do anything to 
 * progress the state of the linker.
 */
struct FResolvingExportTracker : TThreadSingleton<FResolvingExportTracker>
{
public:
	/**  */
	void FlagLinkerExportAsResolving(FLinkerLoad* Linker, int32 ExportIndex)
	{
		ResolvingExports.FindOrAdd(Linker).Add(ExportIndex);
	}

	/**  */
	bool IsLinkerExportBeingResolved(FLinkerLoad* Linker, int32 ExportIndex) const
	{
		if (const TSet<int32>* ExportIndices = ResolvingExports.Find(Linker))
		{
			return ExportIndices->Contains(ExportIndex);
		}
		return false;
	}

	/**  */
	void FlagExportClassAsFullyResolved(FLinkerLoad* Linker, int32 ExportIndex)
	{
		if (TSet<int32>* ExportIndices = ResolvingExports.Find(Linker))
		{
			ExportIndices->Remove(ExportIndex);
			if (ExportIndices->Num() == 0)
			{
				ResolvingExports.Remove(Linker);
			}
		}
	}

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	void FlagFullExportResolvePassComplete(FLinkerLoad* Linker)
	{
		FullyResolvedLinkers.Add(Linker);
	}

	bool HasPerformedFullExportResolvePass(FLinkerLoad* Linker)
	{
		return FullyResolvedLinkers.Contains(Linker);
	}
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	void Reset(FLinkerLoad* Linker)
	{
		ResolvingExports.Remove(Linker);
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		FullyResolvedLinkers.Remove(Linker);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

		// ClassToPlaceholderMap may have entries because instances of placeholder classes (which 
		// will be resolved in ResolveDeferredExports()), will never have had ResolvePlaceholders
		// for their class called. These entries are harmless and we can discard them here:
		ClassToPlaceholderMap.Reset();
	}

	void AddLinkerPlaceholderObject(UClass* ClassWaitingFor, ULinkerPlaceholderExportObject* Placeholder)
	{
		ClassToPlaceholderMap.FindOrAdd(ClassWaitingFor).Add(Placeholder);
	}

	void ResolvePlaceholders(UClass* ForClass)
	{
		TArray<ULinkerPlaceholderExportObject*>* PlaceholdersPtr = ClassToPlaceholderMap.Find(ForClass);
		if (PlaceholdersPtr)
		{
			// Resolving placeholders below may incur additional loads that can, in turn, add
			// new elements to ClassToPlaceholderMap. This could trigger a reallocation of the
			// elements and invalidate the value ptr that was obtained above, which could lead
			// to an invalid memory access. Thus, we copy the array value here before iterating.
			TArray<ULinkerPlaceholderExportObject*> Placeholders(*PlaceholdersPtr);
			for (ULinkerPlaceholderExportObject* Placeholder : Placeholders)
			{
				if(!Placeholder->IsMarkedResolved())
				{
					FLinkerLoad* Linker = Placeholder->GetLinker();
					if(ensure(Linker))
					{
						Linker->ResolvePlaceholder( Placeholder );
					}
				}
			}
			// Remove from map as we could get GCd later
			ClassToPlaceholderMap.Remove(ForClass);
		}
	}

private:
	/**  */
	TMap< FLinkerLoad*, TSet<int32> > ResolvingExports;

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	TSet<FLinkerLoad*> FullyResolvedLinkers;
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	TMap< UClass*, TArray<ULinkerPlaceholderExportObject*> > ClassToPlaceholderMap;
};

bool FLinkerLoad::RegenerateBlueprintClass(UClass* LoadClass, UObject* ClassDefaultObject)
{
	check(LoadClass);
	ensure(ClassDefaultObject && ClassDefaultObject->HasAnyFlags(RF_ClassDefaultObject));

	auto GetClassSourceObjectLambda = [](UClass* ForClass) -> UObject*
	{
#if WITH_EDITORONLY_DATA
		return ForClass->ClassGeneratedBy ? ForClass->ClassGeneratedBy : ForClass;
#else
		return ForClass;
#endif
	};

	UObject* ClassSourceObject = GetClassSourceObjectLambda(LoadClass);
	check(ClassSourceObject);

	if (CVarEnableFullBlueprintPreloading.GetValueOnAnyThread())
	{
		// "Re-preload" cyclic dependencies.
		// 
		// Some known objects, specifically FMetaData and UBlueprint, have a cyclic relationship with the current class.
		// When these objects preload a UClass, they may not have finished preloading their remaining fields.
		// In these cases, we need to effectively "re-preload" them to ensure that they actually complete the preload step.
		// 
		// For example, serializing UBlueprintCore::GeneratedClass could lead us to this point, but the memory
		// for GeneratedClass hasn't been resolved yet. Performing a preload again will correctly fix GeneratedClass,
		// since the class object is in memory during the second preload.
		auto ForcePreloadObject = [this](UObject& Object)
		{
			// The absence of the RF_LoadCompleted flag implies that they're currently in the preload stage.
			if (!Object.HasAnyFlags(RF_LoadCompleted))
			{
				Object.SetFlags(RF_NeedLoad);
				Preload(&Object);
			}
		};

		// Preload order is important here.
		// Metdata exports can serialize all sorts of various exports, and has an implicit requirement that the
		// Blueprint has been completely preloaded.
		ForcePreloadObject(*ClassSourceObject);

#if WITH_METADATA
		// We likely don't need to load meta data here at all! but we have been doing so 
		// since 2080292 - subtly the code from 2080292 wouldn't assert about missing metadata
		// but only because UPackage::GetMetaData creates a dummy UMetaData object that is
		// tagged as RF_LoadCompleted. Consider removing the forced metadata creation.
		int32 MetadataIndex = LoadMetaDataFromExportMap(true);

		// Older content may not have a metadata object in its package.
		if (MetadataIndex != INDEX_NONE)
		{
			const FObjectExport& MetadataExport = Exp(FPackageIndex::FromExport(MetadataIndex));
			if(MetadataExport.Object) // metadata not loaded in -game, has not been at least since UE4
			{
				ForcePreloadObject(*MetadataExport.Object);
			}
		}
#endif // WITH_METADATA

		// Flush (ie: create and preload) all remaining exports in the package.
		//
		// A Blueprint and its generated class often reference other exports in the same package.
		// These need to be preloaded prior Blueprint compilation.
		// This technique is a bit heavy-handed since we might be preloading exports that aren't
		// used by either the Blueprint or its generated class.
		// However, this is still preferable because
		// 1) These exports are going to be preloaded anyway. We're just doing it now.
		// 2) In most cases, the majority of exports are referenced by the Blueprint and its generated class.
		// 3) This is more efficient (and less error prone) than using FReferenceFinder to find specific dependencies.
		for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
		{
			// If there was an earlier load error, we might not have a valid class.
			if (UClass* Class = GetExportLoadClass(ExportIndex))
			{
				const bool bForcePreload = true;
				CreateExportAndPreload(ExportIndex, bForcePreload);
			}
		}

		// The CDO may reference default values that live in other packages, which need to be preloaded.
		FPreloadMembersHelper::PreloadExternalNativeDependencies(ClassDefaultObject);

		{
#if WITH_EDITOR
			// RegenerateClass largely performs redundant work since we already preloaded the remaining exports.
			// However, in some circumstances, a re-preload might occur on an export that doesn't have RF_LoadCompleted set due to a cycle.
			// This is largely dependent on how the derived Blueprint (eg: Widget BPs, Animation BPs, etc.) decides to handle this.
			ClassSourceObject->SetFlags(RF_BeingRegenerated);
			ClassSourceObject->RegenerateClass(LoadClass, ClassDefaultObject);
			ClassSourceObject->ClearFlags(RF_BeingRegenerated);

			// A regenerated class won't be post-loaded, so we can clear these flags.
			LoadClass->ClearFlags(RF_NeedPostLoad | RF_NeedPostLoadSubobjects);
#endif
		}

		// After regeneration, we can now notify placeholders that it's safe to create instances of our class.
		FResolvingExportTracker::Get().ResolvePlaceholders(LoadClass);

		return true;
	}

	// determine if somewhere further down the callstack, we're already in this
	// function for this class
	const bool bAlreadyRegenerating = ClassSourceObject->HasAnyFlags(RF_BeingRegenerated);
	// Flag the class source object, so we know we're already in the process of compiling this class
	ClassSourceObject->SetFlags(RF_BeingRegenerated);

	// Cache off the current CDO, and specify the CDO for the load class 
	// manually... do this before we Preload() any children members so that if 
	// one of those preloads subsequently ends up back here for this class, 
	// then the ExportObject is carried along and used in the eventual RegenerateClass() call
	check(!bAlreadyRegenerating || (LoadClass->GetDefaultObject(false) == ClassDefaultObject));
	LoadClass->SetDefaultObject(ClassDefaultObject);

	// Finish loading the class here, so we have all the appropriate data to copy over to the new CDO
	TArray<UObject*> AllChildMembers;
	GetObjectsWithOuter(LoadClass, /*out*/ AllChildMembers);
	for (int32 Index = 0; Index < AllChildMembers.Num(); ++Index)
	{
		UObject* Member = AllChildMembers[Index];
		Preload(Member);
	}

	// if this was subsequently regenerated from one of the above preloads, then 
	// we don't have to finish this off, it was already done
	const bool bWasSubsequentlyRegenerated = !ClassSourceObject->HasAnyFlags(RF_BeingRegenerated);
	// @TODO: find some other condition to block this if we've already  
	//        regenerated the class (not just if we've regenerated the class 
	//        from an above Preload(Member))... UBlueprint::RegenerateClass() 
	//        has an internal conditional to block getting into it again, but we
	//        can't check UBlueprint members from this module
	if (!bWasSubsequentlyRegenerated)
	{
		Preload(LoadClass);

		LoadClass->StaticLink(true);
		Preload(ClassDefaultObject);

		// CDO preloaded - we can now resolve placeholders:
		FResolvingExportTracker::Get().ResolvePlaceholders(LoadClass);

		// Make sure that we regenerate any parent classes first before attempting to build a child
		TArray<UClass*> ClassChainOrdered;
		{
			// Just ordering the class hierarchy from root to leafs:
			UClass* ClassChain = LoadClass->GetSuperClass();
			while (ClassChain && ClassChain->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
			{
				// O(n) insert, but n is tiny because this is a class hierarchy...
				ClassChainOrdered.Insert(ClassChain, 0);
				ClassChain = ClassChain->GetSuperClass();
			}
		}
		for (UClass* SuperClass : ClassChainOrdered)
		{
			UObject* SuperClassSourceObject = GetClassSourceObjectLambda(SuperClass);
			if (SuperClassSourceObject && SuperClassSourceObject->HasAnyFlags(RF_BeingRegenerated))
			{
				// This code appears to be completely unused:

				// Always load the parent blueprint here in case there is a circular dependency. This will
				// ensure that the blueprint is fully serialized before attempting to regenerate the class.
				FPreloadMembersHelper::PreloadObject(SuperClassSourceObject);

				FPreloadMembersHelper::PreloadMembers(SuperClassSourceObject);
				// recurse into this function for this parent class; 
				// 'ClassDefaultObject' should be the class's original ExportObject
				RegenerateBlueprintClass(SuperClass, SuperClass->GetDefaultObject(false));
			}
		}

		{
#if WITH_EDITOR
			ClassSourceObject = GetClassSourceObjectLambda(LoadClass);

			// Preload the blueprint to make sure it has all the data the class needs for regeneration
			FPreloadMembersHelper::PreloadObject(ClassSourceObject);

			UClass* RegeneratedClass = ClassSourceObject->RegenerateClass(LoadClass, ClassDefaultObject);
			if (RegeneratedClass)
			{
				ClassSourceObject->ClearFlags(RF_BeingRegenerated);
				// Fix up the linker so that the RegeneratedClass is used
				LoadClass->ClearFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects);
			}

			// Ensure that the class source object is marked standalone so it doesn't get GC'd in the editor.
			// In particular, this is needed for a BPGC asset in a cooked package.
			if (LoadClass->bCooked)
			{
				ClassSourceObject->SetFlags(RF_Standalone);
			}
#endif //if WITH_EDITOR
		}
	}

	const bool bSuccessfulRegeneration = !ClassSourceObject->HasAnyFlags(RF_BeingRegenerated);
	// if this wasn't already flagged as regenerating when we first entered this 
	// function, the clear it ourselves.
	if (!bAlreadyRegenerating)
	{
		ClassSourceObject->ClearFlags(RF_BeingRegenerated);
	}

	return bSuccessfulRegeneration;
}

/** 
 * Frivolous helper functions, to provide unique identifying names for our different placeholder types.
 */
template<class PlaceholderType>
static FString GetPlaceholderPrefix()                      { return TEXT("PLACEHOLDER_"); }
template<>
FString GetPlaceholderPrefix<ULinkerPlaceholderFunction>() { return TEXT("PLACEHOLDER-FUNCTION_"); }
template<>
FString GetPlaceholderPrefix<ULinkerPlaceholderClass>()    { return TEXT("PLACEHOLDER-CLASS_"); }

/** Internal utility function for spawning various type of placeholder objects. */
template<class PlaceholderType>
static PlaceholderType* MakeImportPlaceholder(UObject* Outer, const UClass* TargetObjType, const TCHAR* TargetObjName, int32 ImportIndex = INDEX_NONE)
{
	PlaceholderType* PlaceholderObj = nullptr;

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	FName PlaceholderName(*FString::Printf(TEXT("%s_%s"), *GetPlaceholderPrefix<PlaceholderType>(), TargetObjName));
	PlaceholderName = MakeUniqueObjectName(Outer, PlaceholderType::StaticClass(), PlaceholderName);

	PlaceholderObj = NewObject<PlaceholderType>(Outer, PlaceholderType::StaticClass(), PlaceholderName, RF_Public | RF_Transient);

	if (ImportIndex != INDEX_NONE)
	{
		PlaceholderObj->PackageIndex = FPackageIndex::FromImport(ImportIndex);
	}
	// else, this is probably coming from something like an ImportText() call, 
	// and isn't referenced by the ImportMap... instead, this should be stored 
	// in the FLinkerLoad's ImportPlaceholders map

	// Record the type of object that's being deferred
	PlaceholderObj->DeferredObjectType = TargetObjType;

	// make sure the class is fully formed (has its 
	// CppClassStaticFunctions/ClassConstructor members set)
	PlaceholderObj->Bind();
	PlaceholderObj->StaticLink(/*bRelinkExistingProperties =*/true);

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	if (ULinkerPlaceholderClass* OuterAsPlaceholder = dynamic_cast<ULinkerPlaceholderClass*>(Outer))
	{
		OuterAsPlaceholder->AddChildObject(PlaceholderObj);
	}
#endif //USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
#endif //USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	return PlaceholderObj;
}

/** Recursive utility function, set up to find a specific import that has already been created (emulates a block from FLinkerLoad::CreateImport)*/
static UObject* FindExistingImportObject(const int32 Index, const TArray<FObjectImport>& ImportMap)
{
	const FObjectImport& Import = ImportMap[Index];

	UObject* FindOuter = nullptr;
	if (Import.OuterIndex.IsImport())
	{
		int32 OuterIndex = Import.OuterIndex.ToImport();
		const FObjectImport& OuterImport = ImportMap[OuterIndex];

		if (OuterImport.XObject != nullptr)
		{
			FindOuter = OuterImport.XObject;
		}
		else
		{
			FindOuter = FindExistingImportObject(OuterIndex, ImportMap);
		}
	}

	UObject* FoundObject = nullptr;
	if (FindOuter != nullptr || Import.OuterIndex.IsNull())
	{
		if (UObject* ClassPackage = FindObject<UPackage>(/*Outer =*/nullptr, *Import.ClassPackage.ToString()))
		{
			if (UClass* ImportClass = FindObject<UClass>(ClassPackage, *Import.ClassName.ToString()))
			{
				// This function is set up to emulate a block towards the top of 
				// FLinkerLoad::CreateImport(). However, since this is used in 
				// deferred dependency loading we need to be careful not to invoke
				// subsequent loads. The block in CreateImport() calls Preload() 
				// and GetDefaultObject() which are not suitable here, so to 
				// emulate/keep the contract that that block provides, we'll only 
				// lookup the object if its class is loaded, and has a CDO (this
				// is just to mitigate risk from this change)
				if (!ImportClass->HasAnyFlags(RF_NeedLoad) && ImportClass->GetDefaultObject(false))
				{
					FoundObject = StaticFindObjectFast(ImportClass, FindOuter, Import.ObjectName);
				}				
			}
		}
	}
	return FoundObject;
}

/**
 * This utility struct helps track blueprint structs/linkers that are currently 
 * in the middle of a call to ResolveDeferredDependencies(). This can be used  
 * to know if a dependency's resolve needs to be finished (to avoid unwanted 
 * placeholder references ending up in script-code).
 */
struct FUnresolvedStructTracker
{
public:
	/** Marks the specified struct (and its linker) as "resolving" for the lifetime of this instance */
	FUnresolvedStructTracker(UStruct* LoadStruct)
		: TrackedStruct(LoadStruct)
	{
		DEFERRED_DEPENDENCY_CHECK((LoadStruct != nullptr) && (LoadStruct->GetLinker() != nullptr));
		FScopeLock UnresolvedStructsLock(&UnresolvedStructsCritical);
		UnresolvedStructs.Add(LoadStruct);
	}

	/** Removes the wrapped struct from the "resolving" set (it has been fully "resolved") */
	~FUnresolvedStructTracker()
	{
		// even if another FUnresolvedStructTracker added this struct earlier,  
		// we want the most nested one removing it from the set (because this 
		// means the struct is fully resolved, even if we're still in the middle  
		// of a ResolveDeferredDependencies() call further up the stack)
		FScopeLock UnresolvedStructsLock(&UnresolvedStructsCritical);
		UnresolvedStructs.Remove(TrackedStruct);
	}

	/**
	 * Checks to see if the specified import object is a blueprint class/struct 
	 * that is currently in the midst of resolving (and hasn't completed that  
	 * resolve elsewhere in some nested call).
	 * 
	 * @param  ImportObject    The object you wish to check.
	 * @return True if the specified object is a class/struct that hasn't been fully resolved yet (otherwise false).
	 */
	static bool IsImportStructUnresolved(UObject* ImportObject)
	{
		FScopeLock UnresolvedStructsLock(&UnresolvedStructsCritical);
		return UnresolvedStructs.Contains(ImportObject);
	}

	/**
	 * Checks to see if the specified linker is associated with any of the 
	 * unresolved structs that this is currently tracking.
	 *
	 * NOTE: This could return false, even if the linker is in a 
	 *       ResolveDeferredDependencies() call futher up the callstack... in 
	 *       that scenario, the associated struct was fully resolved by a 
	 *       subsequent call to the same function (for the same linker/struct).
	 * 
	 * @param  Linker	The linker you want to check.
	 * @return True if the specified linker is in the midst of an unfinished ResolveDeferredDependencies() call (otherwise false).
	 */
	static bool IsAssociatedStructUnresolved(const FLinkerLoad* Linker)
	{
		FScopeLock UnresolvedStructsLock(&UnresolvedStructsCritical);
		for (UObject* UnresolvedObj : UnresolvedStructs)
		{
			// each unresolved struct should have a linker set on it, because 
			// they would have had to go through Preload()
			if (UnresolvedObj->GetLinker() == Linker)
			{
				return true;
			}
		}
		return false;
	}

	/**  */
	static void Reset(const FLinkerLoad* Linker)
	{
		FScopeLock UnresolvedStructsLock(&UnresolvedStructsCritical);
		TArray<UObject*> ToRemove;
		for (UObject* UnresolvedObj : UnresolvedStructs)
		{
			if (UnresolvedObj->GetLinker() == Linker)
			{
				ToRemove.Add(UnresolvedObj);
			}
		}
		for (UObject* ResetingObj : ToRemove)
		{
			UnresolvedStructs.Remove(ResetingObj);
		}
	}

private:
	/** The struct that is currently being "resolved" */
	UStruct* TrackedStruct;

	/** 
	 * A set of blueprint structs & classes that are currently being "resolved"  
	 * by ResolveDeferredDependencies() (using UObject* instead of UStruct, so
	 * we don't have to cast import objects before checking for their presence).
	 */
	static TSet<UObject*> UnresolvedStructs;
	static FCriticalSection UnresolvedStructsCritical;
};
/** A global set that tracks structs currently being ran through (and unfinished by) FLinkerLoad::ResolveDeferredDependencies() */
TSet<UObject*> FUnresolvedStructTracker::UnresolvedStructs;
FCriticalSection FUnresolvedStructTracker::UnresolvedStructsCritical;

bool FLinkerLoad::DeferPotentialCircularImport(const int32 Index)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (!FBlueprintSupport::UseDeferredDependencyLoading())
	{
		return false;
	}

	//--------------------------------------
	// Phase 1: Stub in Dependencies
	//--------------------------------------

	FObjectImport& Import = ImportMap[Index];

	if (Import.XObject != nullptr)
	{
		FLinkerPlaceholderBase* ImportPlaceholder = nullptr;
		if (ULinkerPlaceholderClass* AsPlaceholderClass = Cast<ULinkerPlaceholderClass>(Import.XObject))
		{
			ImportPlaceholder = AsPlaceholderClass;
		}
		else if (ULinkerPlaceholderFunction* AsPlaceholderFunc = Cast<ULinkerPlaceholderFunction>(Import.XObject))
		{
			ImportPlaceholder = AsPlaceholderFunc;
		}

		const bool bIsResolvingPlaceholders = ImportPlaceholder && (LoadFlags & LOAD_DeferDependencyLoads) == LOAD_None;
		// if this import already had a placeholder spawned for it, but the package 
		// has passed the need for placeholders (it's in the midst of ResolveDeferredDependencies)
		if (bIsResolvingPlaceholders)
		{
			// this is to validate our assumption that this package is in ResolveDeferredDependencies() earlier up the stack
			DEFERRED_DEPENDENCY_CHECK(FUnresolvedStructTracker::IsAssociatedStructUnresolved(this));
		
			UClass* LoadClass = nullptr;
			// Get the LoadClass that is currently in the midst of being resolved (needed to pass to ResolveDependencyPlaceholder)
			{
				// if DeferredCDOIndex is not set, then this is presumably a struct package (it should always be  
				// set at this point for class BP packages - see Preload() where DeferredCDOIndex is assigned)
				if (DeferredCDOIndex != INDEX_NONE)
				{
					FPackageIndex ClassIndex = ExportMap[DeferredCDOIndex].ClassIndex;
					DEFERRED_DEPENDENCY_CHECK(ClassIndex.IsExport());

					if (ClassIndex.IsExport())
					{
						FObjectExport ClassExport = ExportMap[ClassIndex.ToExport()];
						LoadClass = Cast<UClass>(ClassExport.Object);
					}

					DEFERRED_DEPENDENCY_CHECK(LoadClass != nullptr);
				}
			}

			// go ahead and resolve the placeholder here (since someone's requesting it and we're already in the 
			// midst of resolving placeholders earlier in the stack) - the idea is that the resolve, already in progress, will 
			// eventually get to this placeholder, it just hasn't looped there yet
			//
			// this will prevent other, needless placeholders from being created (export templates that are relying on this class, etc.)
			ResolveDependencyPlaceholder(ImportPlaceholder, LoadClass);

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			const bool bIsStillPlaceholder = Import.XObject && (Import.XObject->IsA<ULinkerPlaceholderClass>() || Import.XObject->IsA<ULinkerPlaceholderFunction>());
			DEFERRED_DEPENDENCY_CHECK(!bIsStillPlaceholder);
			return bIsStillPlaceholder;
#else 
			// presume that ResolveDependencyPlaceholder() worked and the import is no longer a placeholder
			return false;
#endif 

		}
		return (ImportPlaceholder != nullptr);
	}

	if ((LoadFlags & LOAD_DeferDependencyLoads) && !IsImportNative(Index))
	{
		// emulate the block in CreateImport(), that attempts to find an existing
		// object in memory first... this is to account for async loading, which
		// can clear Import.XObject (via FLinkerManager::DissociateImportsAndForcedExports)
		// at inopportune times (after it's already been set) - in this case
		// we shouldn't need a placeholder, because the object already exists; we
		// just need to keep from serializing it any further (which is why we've
		// emulated it here, to cut out on a Preload() call)
		if (!GIsEditor && !IsRunningCommandlet())
		{
			Import.XObject = FindExistingImportObject(Index, ImportMap);
			if (Import.XObject)
			{
				return true;
			}
		}

		if (UObject* ClassPackage = FindObject<UPackage>(/*Outer =*/nullptr, *Import.ClassPackage.ToString()))
		{
			if (const UClass* ImportClass = FindObject<UClass>(ClassPackage, *Import.ClassName.ToString()))
			{
				if (ImportClass->HasAnyClassFlags(CLASS_NeedsDeferredDependencyLoading))
				{
					Import.XObject = MakeImportPlaceholder<ULinkerPlaceholderClass>(LinkerRoot, ImportClass, *Import.ObjectName.ToString(), Index);
				}
				else if (ImportClass->IsChildOf<UFunction>() && Import.OuterIndex.IsImport())
				{
					const int32 OuterImportIndex = Import.OuterIndex.ToImport();
					// @TODO: if the sole reason why we have ULinkerPlaceholderFunction 
					//        is that it's outer is a placeholder, then we 
					//        could instead log it (with the placeholder) as 
					//        a referencer, and then move the function later
					if (DeferPotentialCircularImport(OuterImportIndex))
					{
						UObject* FuncOuter = ImportMap[OuterImportIndex].XObject;
						// This is an ugly check to make sure we don't make a placeholder function for a missing native instance.
						// We likely also need to avoid making placeholders for anything that's not outered to a ULinkerPlaceholderClass,
						// but the DEFERRED_DEPENDENCY_CHECK may be out of date...
						if(Cast<UClass>(FuncOuter))
						{
							Import.XObject = MakeImportPlaceholder<ULinkerPlaceholderFunction>(FuncOuter, ImportClass, *Import.ObjectName.ToString(), Index);
							DEFERRED_DEPENDENCY_CHECK(dynamic_cast<ULinkerPlaceholderClass*>(FuncOuter) != nullptr);
						}
					}
				}
			}
		}

		// not the best way to check this (but we don't have ObjectFlags on an 
		// import), but we don't want non-native (blueprint) CDO refs slipping 
		// through... we've only seen these needed when serializing a class's 
		// bytecode, and we resolved that by deferring script serialization
		DEFERRED_DEPENDENCY_CHECK(!Import.ObjectName.ToString().StartsWith("Default__"));
	}
	return (Import.XObject != nullptr);
#else // !USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return false;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

#if WITH_EDITOR
/** Helper function find the actual class object given import class and package namme */
static UClass* FindImportClass(FName ClassPackageName, FName ClassName)
{
	UClass* Class = nullptr;
	UPackage* ClassPackage = Cast<UPackage>(StaticFindObjectFast(UPackage::StaticClass(), nullptr, ClassPackageName));
	if (ClassPackage)
	{
		Class = Cast<UClass>(StaticFindObjectFast(UClass::StaticClass(), ClassPackage, ClassName));
	}
	return Class;
}
bool FLinkerLoad::IsSuppressableBlueprintImportError(int32 ImportIndex) const
{
	// We want to suppress any import errors that target a BlueprintGeneratedClass
	// since these issues can occur when an externally referenced Blueprint is saved 
	// without compiling. This should not be a problem because all Blueprints are
	// compiled-on-load.
	static const FName NAME_BlueprintGeneratedClass("BlueprintGeneratedClass");
	static const FName NAME_EnginePackage("/Script/Engine");
	UClass* BlueprintGeneratedClass = FindImportClass(NAME_EnginePackage, NAME_BlueprintGeneratedClass);
	check(BlueprintGeneratedClass);
	// We will look at each outer of the Import to see if any of them are a BPGC
	while (ImportMap.IsValidIndex(ImportIndex))
	{
		const FObjectImport& TestImport = ImportMap[ImportIndex];
		UClass* ImportClass = FindImportClass(TestImport.ClassPackage, TestImport.ClassName);
		if (ImportClass && ImportClass->IsChildOf(BlueprintGeneratedClass))
		{
			// The import is a BPGC, suppress errors
			return true;
		}

		// Check if this is a BP CDO, if so our class will be in the import table
		for (const FObjectImport& PotentialBPClass : ImportMap)
		{
			if (PotentialBPClass.ObjectName == TestImport.ClassName)
			{
				UClass* PotentialBPClassClass = FindImportClass(PotentialBPClass.ClassPackage, PotentialBPClass.ClassName);
				if (PotentialBPClassClass && PotentialBPClassClass->IsChildOf(BlueprintGeneratedClass))
				{
					return true;
				}
			}
		}

		if (!TestImport.OuterIndex.IsNull() && TestImport.OuterIndex.IsImport())
		{
			ImportIndex = TestImport.OuterIndex.ToImport();
		}
		else
		{
			// It's not an import, we are done
			break;
		}
	}

	return false;
}
#endif // WITH_EDITOR

/** 
 * A helper struct that adds and removes its linker/export combo from the 
 * thread's FResolvingExportTracker (based off the scope it was declared within).
 */
struct FScopedResolvingExportTracker
{
public: 
	FScopedResolvingExportTracker(FLinkerLoad* Linker, int32 ExportIndex)
		: TrackedLinker(Linker), TrackedExport(ExportIndex)
	{
		FResolvingExportTracker::Get().FlagLinkerExportAsResolving(Linker, ExportIndex);
	}

	~FScopedResolvingExportTracker()
	{
		FResolvingExportTracker::Get().FlagExportClassAsFullyResolved(TrackedLinker, TrackedExport);
	}

private:
	FLinkerLoad* TrackedLinker;
	int32        TrackedExport;
};

bool FLinkerLoad::DeferExportCreation(const int32 Index, UObject* Outer)
{
	FObjectExport& Export = ExportMap[Index];

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (!FBlueprintSupport::UseDeferredDependencyLoading() || FBlueprintSupport::IsDeferredExportCreationDisabled())
	{
		return false;
	}

	if ((Export.Object != nullptr))
	{
		return false;
	}
	
	UClass* LoadClass = GetExportLoadClass(Index);
	
	if (LoadClass == nullptr)
	{
		return false;
	}

	if(ULinkerPlaceholderExportObject* OuterPlaceholder = Cast<ULinkerPlaceholderExportObject>(Outer))
	{
		// we deferred the outer, so its constructor has not had a chance
		// to create and initialize native subobjects. We must defer this subobject:
		FString ClassName = LoadClass->GetName();
		FName PlaceholderName(*FString::Printf(TEXT("PLACEHOLDER-INST_of_%s"), *ClassName));
		UClass*   PlaceholderType  = ULinkerPlaceholderExportObject::StaticClass();
		PlaceholderName = MakeUniqueObjectName(Outer, PlaceholderType, PlaceholderName);

		ULinkerPlaceholderExportObject* Placeholder = NewObject<ULinkerPlaceholderExportObject>(Outer, PlaceholderType, PlaceholderName, RF_Public | RF_Transient);
		Placeholder->SetLinker(this, Index, false);
		Placeholder->PackageIndex = FPackageIndex::FromExport(Index);
		
		Export.Object = Placeholder;

		// the subobject placeholder must be resolved after its outer has been resolved:
		OuterPlaceholder->SetupPlaceholderSubobject(Placeholder);

		return true;
	}

	if (LoadClass->HasAnyClassFlags(CLASS_Native))
	{
		return false;
	}

	const bool bIsCDOExport = (Export.ObjectFlags & RF_ClassDefaultObject) != 0;
	if (bIsCDOExport)
	{
		// Check for any load dependencies that may have been deferred.
		bool bHasDeferredDependencies = false;
		TArray<UObject*> CDOPreloadDependencies;
		LoadClass->GetDefaultObjectPreloadDependencies(CDOPreloadDependencies);
		for (const UObject* PreloadDependency : CDOPreloadDependencies)
		{
			if (FBlueprintSupport::IsDeferredDependencyPlaceholder(PreloadDependency))
			{
				bHasDeferredDependencies = true;
			}
		}

		// Defer the CDO export only if we're preloading its class and it has a deferred dependency. For
		// example, we may need to resolve a non-native subobject type override before we can construct
		// the actual CDO and execute its native ctor/initializer.
		if (((LoadFlags & LOAD_DeferDependencyLoads) != 0) && bHasDeferredDependencies)
		{
			// This will cause IsBlueprintFinalizationPending() to return true (which is what we want).
			// We'll then fall through and create a placeholder object for the CDO in order to defer its
			// actual construction (along with serialization) until after we've resolved its dependencies.
			DEFERRED_DEPENDENCY_CHECK(DeferredCDOIndex == INDEX_NONE);
			DeferredCDOIndex = Index;
		}
		else
		{
			return false;
		}
	}

	ULinkerPlaceholderClass* AsPlaceholderClass = Cast<ULinkerPlaceholderClass>(LoadClass);
	const bool bIsPlaceholderClass = (AsPlaceholderClass != nullptr);

	FLinkerLoad* ClassLinker = LoadClass->GetLinker();
	const UObject* ClassDefaultObject = LoadClass->GetDefaultObject(false);
	if (!bIsPlaceholderClass
		&& ((ClassLinker == nullptr) || !ClassLinker->IsBlueprintFinalizationPending())
		&& (!ClassDefaultObject || ClassDefaultObject->HasAnyFlags(RF_LoadCompleted) || !ClassDefaultObject->HasAnyFlags(RF_WasLoaded)))
	{
		return false;
	}

	const bool bIsLoadingExportClass = (LoadFlags & LOAD_DeferDependencyLoads) ||
		IsBlueprintFinalizationPending();
	// if we're not in the process of "loading/finalizing" this package's 
	// Blueprint class, then we're either running this before the linker has got 
	// to that class, or we're finished and in the midst of regenerating that 
	// class... either way, we don't have to defer the export (as long as we 
	// make sure the export's class is fully regenerated... presumably it is in 
	// the midst of doing so somewhere up the callstack)
	if (!bIsLoadingExportClass || (LoadFlags & LOAD_ResolvingDeferredExports) != 0)
	{
		bool bAlreadyResolvingExport = IsExportBeingResolved(Index);
		FScopedResolvingExportTracker ReentranceGuard(this, Index);

		// we want to be very careful, since we haven't filled in the export yet,
		// we could get stuck in a recursive loop here (force-finalizing the 
		// class here ends us back 
		if (!bAlreadyResolvingExport || GetLoaderType() != ELoaderType::ZenLoader)
		{
			ForceRegenerateClass(LoadClass);
		}

		return false;
	}
	
	UPackage* PlaceholderOuter = LinkerRoot;
	UClass*   PlaceholderType  = ULinkerPlaceholderExportObject::StaticClass();	

	FString ClassName = LoadClass->GetName();
	//ClassName.RemoveFromEnd("_C");	
	FName PlaceholderName(*FString::Printf(TEXT("PLACEHOLDER-INST_of_%s"), *ClassName));
	PlaceholderName = MakeUniqueObjectName(PlaceholderOuter, PlaceholderType, PlaceholderName);

	ULinkerPlaceholderExportObject* Placeholder = NewObject<ULinkerPlaceholderExportObject>(PlaceholderOuter, PlaceholderType, PlaceholderName, RF_Public | RF_Transient);
	Placeholder->PackageIndex = FPackageIndex::FromExport(Index);
	Placeholder->SetLinker(this, Index, false);
	FResolvingExportTracker::Get().AddLinkerPlaceholderObject(LoadClass, Placeholder);

	Export.Object = Placeholder;

	if (bIsCDOExport)
	{
		DEFERRED_DEPENDENCY_CHECK(LoadClass->GetDefaultObject(false) == nullptr);
		LoadClass->SetDefaultObject(Placeholder);
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	return true;
}

int32 FLinkerLoad::FindCDOExportIndex(UClass* LoadClass)
{
	DEFERRED_DEPENDENCY_CHECK(LoadClass->GetLinker() == this);
	int32 const ClassExportIndex = LoadClass->GetLinkerIndex();

	// @TODO: the cdo SHOULD be listed after the class in the ExportMap, so we 
	//        could start with ClassExportIndex to save on some cycles
	for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
	{
		FObjectExport& Export = ExportMap[ExportIndex];
		if ((Export.ObjectFlags & RF_ClassDefaultObject) && Export.ClassIndex.IsExport() && (Export.ClassIndex.ToExport() == ClassExportIndex))
		{
			return ExportIndex;
		}
	}
	return INDEX_NONE;
}

UPackage* LoadPackageInternal(UPackage* InOuter, const FPackagePath& PackagePath, uint32 LoadFlags, FLinkerLoad* ImportLinker, FArchive* InReaderOverride, const FLinkerInstancingContext* InstancingContext, const FPackagePath* DiffPackagePath);

bool ShouldSkipResolveDeferredDependencies(UStruct* LoadStruct)
{
	if (IsInActualLoadingThread() || IsInParallelLoadingThread())
	{
		if (!LoadStruct->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1))
		{
			// If we're in the loader, it would be a race hazard to try to recurse into
			// packages and linkers that have finished loading, so we skip unless the
			// package we target is also owned by the loader.
			// We know that a struct can't contain deferred dependencies anymore once it
			// leaves its phase 1 and is ready for postload.
			return true;
		}
	}

	return false;
}

void FLinkerLoad::ResolveDeferredDependencies(UStruct* LoadStruct)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	UE::TUniqueLock ScopeLock(Mutex);
	//--------------------------------------
	// Phase 2: Resolve Dependency Stubs
	//--------------------------------------
	TGuardValue<uint32> LoadFlagsGuard(LoadFlags, (LoadFlags & ~LOAD_DeferDependencyLoads));

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	static int32 RecursiveDepth = 0;
	int32 const MeasuredDepth = RecursiveDepth;
	TGuardValue<int32> RecursiveDepthGuard(RecursiveDepth, RecursiveDepth + 1);

	DEFERRED_DEPENDENCY_CHECK( (LoadStruct != nullptr) && (LoadStruct->GetLinker() == this) );
	DEFERRED_DEPENDENCY_CHECK( LoadStruct->HasAnyFlags(RF_LoadCompleted) );
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	// scoped block to manage the lifetime of ScopedResolveTracker, so that 
	// this resolve is only tracked for the duration of resolving all its 
	// placeholder classes, all member struct's placholders, and its parent's
	{
		FUnresolvedStructTracker ScopedResolveTracker(LoadStruct);
		
		UClass* const LoadClass = Cast<UClass>(LoadStruct);

		int32 StartingImportIndex = 0;
		// this function (for this linker) could be reentrant (see where we 
		// recursively call ResolveDeferredDependencies() for super-classes below);  
		// if that's the case, then we want to finish resolving the pending class 
		// before we continue on
		if (ResolvingPlaceholderStack.Num() > 0)
		{
			// Since this method is recursive, we don't need to needlessly loop over all the imports we've already 
			// resolved. However, we can only guarantee that the oldest entry in the 'resolving' stack is from a loop below.
			// Now that other places call ResolveDependencyPlaceholder(), the ResolvingPlaceholderStack may jump around and
			// skip some entries. The only certainty is that this function is the initial entry point for ResolveDependencyPlaceholder().
			const FPackageIndex& FirstResolvingIndex = ResolvingPlaceholderStack[0]->PackageIndex;
			if (FirstResolvingIndex.IsNull())
			{
				// if the placeholder's package index is null, that means we've already looped over the entire 
				// ImportMap, and moved on to the loop below it (where we resolve placeholders from ImportText() 
				// and such - they don't have entries in the ImportMap), so skip the ImportMap loop
				StartingImportIndex = ImportMap.Num();
			}
			else
			{
				DEFERRED_DEPENDENCY_CHECK(FirstResolvingIndex.IsImport());

				// Since the ImportMap loop below resolves ULinkerPlaceholderFunction's owner first (out of order), we cannot 
				// even guarantee that we've resolved everything prior to FirstResolvingIndex, so don't set StartingImportIndex in this case
// 				if (FirstResolvingIndex.IsImport())
// 				{
// 					StartingImportIndex = FirstResolvingIndex.ToImport()+1;
// 				}
			}

			while (ResolvingPlaceholderStack.Num() > 0)
			{
				FLinkerPlaceholderBase* ResolvingPlaceholder = ResolvingPlaceholderStack.Pop();
				// If this is a placeholder outside the ImportMap (from ImportText(), etc.), then it needs a PackagePath to
				// resolve. Don't worry that one isn't passed in as a param here, ResolveDependencyPlaceholder() will 
				// look it up itself in ImportPlaceholders (the param is just an optimization)
				ResolveDependencyPlaceholder(ResolvingPlaceholder, LoadClass);
			}

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			for (int32 ImportIndex = 0; ImportIndex < StartingImportIndex; ++ImportIndex)
			{
				if (UObject* ImportObj = ImportMap[ImportIndex].XObject)
				{
					DEFERRED_DEPENDENCY_CHECK(Cast<ULinkerPlaceholderClass>(ImportObj) == nullptr);
					DEFERRED_DEPENDENCY_CHECK(Cast<ULinkerPlaceholderFunction>(ImportObj) == nullptr);
				}
			}
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		}
		
		// because this loop could recurse (and end up finishing all of this for
		// us), we check HasUnresolvedDependencies() so we can early out  
		// from this loop in that situation (the loop has been finished elsewhere)
		for (int32 ImportIndex = StartingImportIndex; ImportIndex < ImportMap.Num() && HasUnresolvedDependencies(); ++ImportIndex)
		{
			FObjectImport& Import = ImportMap[ImportIndex];

			FLinkerLoad* SourceLinker = Import.SourceLinker;
			// we cannot rely on Import.SourceLinker being set, if you look 
			// at FLinkerLoad::CreateImport(), you'll see in game builds 
			// that we try to circumvent the normal Import loading with a
			// FindImportFast() call... if this is successful (the import 
			// has already been somewhat loaded), then we don't fill out the 
			// SourceLinker field
			if (SourceLinker == nullptr)
			{
				if (Import.XObject != nullptr)
				{
					SourceLinker = Import.XObject->GetLinker();
					//if (SourceLinker == nullptr)
					//{
					//	UPackage* ImportPkg = Import.XObject->GetOutermost();
					//	// we use this package as placeholder for our 
					//	// placeholders, so make sure to skip those (all other
					//	// imports should belong to another package)
					//	if (ImportPkg && ImportPkg != LinkerRoot)
					//	{
					//		SourceLinker = FindExistingLinkerForPackage(ImportPkg);
					//	}
					//}
				}					
			}

			const UPackage* SourcePackage = (SourceLinker != nullptr) ? SourceLinker->LinkerRoot : nullptr;
			// this package may not have introduced any (possible) cyclic 
			// dependencies, but it still could have been deferred (kept from
			// fully loading... we need to make sure metadata gets loaded, etc.)
			if ((SourcePackage != nullptr) && !SourcePackage->HasAnyFlags(RF_WasLoaded | RF_WillBeLoaded))
			{
				uint32 InternalLoadFlags = LoadFlags & (LOAD_NoVerify | LOAD_NoWarn | LOAD_Quiet);
				// make sure LoadAllObjects() is called for this package
				LoadPackageInternal(/*Outer =*/nullptr, SourceLinker->GetPackagePath(), InternalLoadFlags, this, nullptr/*InReaderOverride*/, nullptr/*InstancingContext*/, nullptr /* DiffPackagePath */); //-V595
			}

			DEFERRED_DEPENDENCY_CHECK(ResolvingPlaceholderStack.Num() == 0);
			if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(Import.XObject))
			{
				DEFERRED_DEPENDENCY_CHECK(PlaceholderClass->PackageIndex.ToImport() == ImportIndex);

				// NOTE: we don't check that this resolve successfully replaced any
				//       references (by the return value), because this resolve 
				//       could have been re-entered and completed by a nested call
				//       to the same function (for the same placeholder)
				ResolveDependencyPlaceholder(PlaceholderClass, LoadClass);
			}
			else if (ULinkerPlaceholderFunction* PlaceholderFunction = Cast<ULinkerPlaceholderFunction>(Import.XObject))
			{
				if (ULinkerPlaceholderClass* PlaceholderOwner = Cast<ULinkerPlaceholderClass>(PlaceholderFunction->GetOwnerClass()))
				{
					ResolveDependencyPlaceholder(PlaceholderOwner, LoadClass);
				}

				DEFERRED_DEPENDENCY_CHECK(PlaceholderFunction->PackageIndex.ToImport() == ImportIndex);
				ResolveDependencyPlaceholder(PlaceholderFunction, LoadClass);
			}
			else if (UScriptStruct* StructObj = Cast<UScriptStruct>(Import.XObject))
			{
				// in case this is a user defined struct, we have to resolve any 
				// deferred dependencies in the struct 
				if (SourceLinker != nullptr && !ShouldSkipResolveDeferredDependencies(StructObj))
				{
					SourceLinker->ResolveDeferredDependencies(StructObj);
				}
			}
			DEFERRED_DEPENDENCY_CHECK(ResolvingPlaceholderStack.Num() == 0);
		}

		// resolve any placeholders that were imported through methods like 
		// ImportText() (meaning the ImportMap wouldn't reference them)
		while (ImportPlaceholders.Num() > 0)
		{
			auto PlaceholderIt = ImportPlaceholders.CreateIterator();

			// store off the key before we resolve, in case this has been recursively removed
			const FName PlaceholderKey = PlaceholderIt.Key();
			ResolveDependencyPlaceholder(PlaceholderIt.Value(), LoadClass, PlaceholderKey);

			ImportPlaceholders.Remove(PlaceholderKey);
		}

		if (UStruct* SuperStruct = LoadStruct->GetSuperStruct())
		{
			if (!ShouldSkipResolveDeferredDependencies(SuperStruct))
			{
				FLinkerLoad* SuperLinker = SuperStruct->GetLinker();
				// NOTE: there is no harm in calling this when the super is not 
				//       "actively resolving deferred dependencies"... this condition  
				//       just saves on wasted ops, looping over the super's ImportMap
				if ((SuperLinker != nullptr) && SuperLinker->HasUnresolvedDependencies())
				{
					// a resolve could have already been started up the stack, and in turn  
					// started loading a different package that resulted in another (this) 
					// resolve beginning... in that scenario, the original resolve could be 
					// for this class's super and we want to make sure that finishes before
					// we regenerate this class (else the generated script code could end up 
					// with unwanted placeholder references; ones that would have been
					// resolved by the super's linker)
					SuperLinker->ResolveDeferredDependencies(SuperStruct);
				}
			}
		}

	// close the scope on ScopedResolveTracker (so LoadClass doesn't appear to 
	// be resolving through the rest of this function)
	}

	// @TODO: don't know if we need this, but could be good to have (as class 
	//        regeneration is about to force load a lot of this), BUT! this 
	//        doesn't work for map packages (because this would load the level's
	//        ALevelScriptActor instance BEFORE the class has been regenerated)
	//LoadAllObjects();

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	auto CheckPlaceholderReferences = [this](FLinkerPlaceholderBase* Placeholder)
	{
		UObject* PlaceholderObj = Placeholder->GetPlaceholderAsUObject();
		if (PlaceholderObj->GetOuter() == LinkerRoot)
		{
			// there shouldn't be any deferred dependencies (belonging to this 
			// linker) that need to be resolved by this point
			DEFERRED_DEPENDENCY_CHECK(!Placeholder->HasKnownReferences());

			if (!Placeholder->PackageIndex.IsNull() && ensure(Placeholder->PackageIndex.IsImport()))
			{
				const UObject* ImportObj = ImportMap[Placeholder->PackageIndex.ToImport()].XObject;
				DEFERRED_DEPENDENCY_CHECK(ImportObj != PlaceholderObj);
				DEFERRED_DEPENDENCY_CHECK(Cast<ULinkerPlaceholderClass>(ImportObj) == nullptr);
				DEFERRED_DEPENDENCY_CHECK(Cast<ULinkerPlaceholderFunction>(ImportObj) == nullptr);
			}
		}
	};

	for (TObjectIterator<ULinkerPlaceholderClass> PlaceholderIt; PlaceholderIt; ++PlaceholderIt)
	{
		CheckPlaceholderReferences(*PlaceholderIt);
	}
	for (TObjectIterator<ULinkerPlaceholderFunction> PlaceholderIt; PlaceholderIt; ++PlaceholderIt)
	{
		CheckPlaceholderReferences(*PlaceholderIt);
	}

	DEFERRED_DEPENDENCY_CHECK(ImportPlaceholders.Num() == 0);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

bool FLinkerLoad::HasUnresolvedDependencies() const
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	// checking (ResolvingPlaceholderStack.Num() <= 0) is not sufficient, 
	// because the linker could be in the midst of a nested resolve (for a 
	// struct, or super... see FLinkerLoad::ResolveDeferredDependencies)
	bool bIsClassExportUnresolved = FUnresolvedStructTracker::IsAssociatedStructUnresolved(this);

	// (ResolvingPlaceholderStack.Num() <= 0) should imply 
	// bIsClassExportUnresolved is true (but not the other way around)
	DEFERRED_DEPENDENCY_CHECK((ResolvingPlaceholderStack.Num() <= 0) || bIsClassExportUnresolved);
	
	return bIsClassExportUnresolved;

#else  // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return false;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

int32 FLinkerLoad::ResolveDependencyPlaceholder(FLinkerPlaceholderBase* PlaceholderIn, UClass* ReferencingClass, const FName ObjectPathIn)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	TGuardValue<uint32>  LoadFlagsGuard(LoadFlags, (LoadFlags & ~LOAD_DeferDependencyLoads));
	ResolvingPlaceholderStack.Push(PlaceholderIn);

	UObject* PlaceholderObj = PlaceholderIn->GetPlaceholderAsUObject();
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	DEFERRED_DEPENDENCY_CHECK(PlaceholderObj != nullptr);
	DEFERRED_DEPENDENCY_CHECK(PlaceholderObj->GetOutermost() == LinkerRoot);
	const int32 ResolvingStackDepth = ResolvingPlaceholderStack.Num();
#endif
	
	UObject* RealImportObj = nullptr;

	FName ObjectPathName = NAME_None;
	if (PlaceholderIn->PackageIndex.IsNull())
	{
		ObjectPathName = ObjectPathIn;
		if (!ObjectPathIn.IsValid() || ObjectPathIn.IsNone())
		{
			const FName* ObjectPathPtr = ImportPlaceholders.FindKey(PlaceholderIn);
			DEFERRED_DEPENDENCY_CHECK(ObjectPathPtr != nullptr);
			if (ObjectPathPtr)
			{
				ObjectPathName = *ObjectPathPtr;
			}
		}
		DEFERRED_DEPENDENCY_CHECK(ObjectPathName.IsValid() && !ObjectPathName.IsNone());

		// emulating the StaticLoadObject() call in FObjectPropertyBase::FindImportedObject(),
		// since this was most likely a placeholder 
		RealImportObj = StaticLoadObject(UObject::StaticClass(), /*Outer =*/nullptr, *ObjectPathName.ToString(), /*Filename =*/nullptr, (LOAD_NoWarn | LOAD_FindIfFail));
	}
	else
	{
		DEFERRED_DEPENDENCY_CHECK(PlaceholderIn->PackageIndex.IsImport());
		int32 const ImportIndex = PlaceholderIn->PackageIndex.ToImport();
		FObjectImport& Import = ImportMap[ImportIndex];

		if ((Import.XObject != nullptr) && (Import.XObject != PlaceholderObj))
		{
			DEFERRED_DEPENDENCY_CHECK(ResolvingPlaceholderStack.Num() > 0 && ResolvingPlaceholderStack.Top() == PlaceholderIn);
			DEFERRED_DEPENDENCY_CHECK(ResolvingPlaceholderStack.Num() == ResolvingStackDepth);

			RealImportObj = Import.XObject;
		}
		else
		{
			// clear the placeholder from the import, so that a call to CreateImport()
			// properly fills it in
			Import.XObject = nullptr;
			// NOTE: this is a possible point of recursion... CreateImport() could 
			//       continue to load a package already started up the stack and you 
			//       could end up in another ResolveDependencyPlaceholder() for some  
			//       other placeholder before this one has completely finished resolving
			RealImportObj = CreateImport(ImportIndex);
		}
	}

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	UFunction* AsFunction = Cast<UFunction>(RealImportObj);
	UClass* FunctionOwner = (AsFunction != nullptr) ? AsFunction->GetOwnerClass() : nullptr;
	// it's ok if super functions come in not fully loaded (missing 
	// RF_LoadCompleted... meaning it's in the middle of serializing in somewhere 
	// up the stack); the function will be forcefully ran through Preload(), 
	// when we regenerate the super class (see FRegenerationHelper::ForcedLoadMembers)
	bool const bIsSuperFunction   = (AsFunction != nullptr) && (ReferencingClass != nullptr) && ReferencingClass->IsChildOf(FunctionOwner);
	// it's also possible that the loaded version of this function has been 
	// thrown out and replaced with a regenerated version (presumably from a
	// blueprint compiling on load)... if that's the case, then this function 
	// will not have a corresponding linker assigned to it
	bool const bIsRegeneratedFunc = (AsFunction != nullptr) && (AsFunction->GetLinker() == nullptr);

	bool const bExpectsLoadCompleteFlag = (RealImportObj != nullptr) && !bIsSuperFunction && !bIsRegeneratedFunc;
	// if we can't rely on the Import object's RF_LoadCompleted flag, then its
	// owner class should at least have it
	DEFERRED_DEPENDENCY_CHECK( (RealImportObj == nullptr) || bExpectsLoadCompleteFlag ||
		(FunctionOwner && FunctionOwner->HasAnyFlags(RF_LoadCompleted)) );

	DEFERRED_DEPENDENCY_CHECK(RealImportObj != PlaceholderObj);
	DEFERRED_DEPENDENCY_CHECK(!bExpectsLoadCompleteFlag || RealImportObj->HasAnyFlags(RF_LoadCompleted));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	int32 ReplacementCount = 0;
	if (ReferencingClass != nullptr)
	{
		// @TODO: roll this into ULinkerPlaceholderClass's ResolveAllPlaceholderReferences()
		for (FImplementedInterface& Interface : ReferencingClass->Interfaces)
		{
			if (Interface.Class == PlaceholderObj)
			{
				++ReplacementCount;
				Interface.Class = CastChecked<UClass>(RealImportObj, ECastCheckedType::NullAllowed);
			}
		}
	}

	// make sure that we know what utilized this placeholder class... right now
	// we only expect UObjectProperties/UClassProperties/UInterfaceProperties/
	// FImplementedInterfaces to, but something else could have requested the 
	// class without logging itself with the placeholder... if the placeholder
	// doesn't have any known references (and it hasn't already been resolved in
	// some recursive call), then there is something out there still using this
	// placeholder class
	DEFERRED_DEPENDENCY_CHECK( (ReplacementCount > 0) || PlaceholderIn->HasKnownReferences() || PlaceholderIn->HasBeenFullyResolved() );

	ReplacementCount += PlaceholderIn->ResolveAllPlaceholderReferences(RealImportObj);

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	// @TODO: not an actual method, but would be nice to circumvent the need for bIsAsyncLoadRef below
	//FAsyncObjectsReferencer::Get().RemoveObject(PlaceholderObj);

	// there should not be any references left to this placeholder class 
	// (if there is, then we didn't log that referencer with the placeholder)
	FReferencerInformationList UnresolvedReferences;
	bool const bIsReferenced = false;// IsReferenced(PlaceholderObj, RF_NoFlags, /*bCheckSubObjects =*/false, &UnresolvedReferences);

	// when we're running with async loading there may be an acceptable 
	// reference left in FAsyncObjectsReferencer (which reports its refs  
	// through FGCObject::GGCObjectReferencer)... since garbage collection can  
	// be ran during async loading, FAsyncObjectsReferencer is in charge of  
	// holding onto objects that are spawned during the process (to ensure 
	// they're not thrown away prematurely)
	bool const bIsAsyncLoadRef = (UnresolvedReferences.ExternalReferences.Num() == 1) &&
		PlaceholderObj->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading) && (UnresolvedReferences.ExternalReferences[0].Referencer == FGCObject::GGCObjectReferencer);

	DEFERRED_DEPENDENCY_CHECK(!bIsReferenced || bIsAsyncLoadRef);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	// this could recurse back into ResolveDeferredDependencies(), which resolves all placeholders from this list,
	// so by the time we're returned here, the list may be empty
	if (ResolvingPlaceholderStack.Num() > 0)
	{
		DEFERRED_DEPENDENCY_CHECK(ResolvingPlaceholderStack.Top() == PlaceholderIn);
		DEFERRED_DEPENDENCY_CHECK(ResolvingPlaceholderStack.Num() == ResolvingStackDepth);

		ResolvingPlaceholderStack.Pop();
	}
	ImportPlaceholders.Remove(ObjectPathName);

	return ReplacementCount;
#else  // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return 0;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

void FLinkerLoad::PRIVATE_ForceLoadAllDependencies(UPackage* Package)
{
	if (FLinkerLoad* PkgLinker = FindExistingLinkerForPackage(Package))
	{
		PkgLinker->ResolveAllImports();
	}
}

void FLinkerLoad::ResolveAllImports()
{
	UE_TRACK_REFERENCING_PACKAGE_SCOPED(LinkerRoot->GetFName(), PackageAccessTrackingOps::NAME_Load);
	for (int32 ImportIndex = 0; ImportIndex < ImportMap.Num() && IsBlueprintFinalizationPending(); ++ImportIndex)
	{
		// first, make sure every import object is available... just because 
		// it isn't present in the map already, doesn't mean it isn't in the 
		// middle of a resolve (the CreateImport() brings in an export 
		// object from another package, which could be resolving itself)... 
		// 
		// don't fret, all these imports were bound to get created sooner or 
		// later (like when the blueprint was regenerated)
		//
		// NOTE: this is a possible root point for recursion... accessing a 
		//       separate package could continue its loading process which
		//       in turn, could end us back in this function before we ever  
		//       returned from this
		FObjectImport& Import = ImportMap[ImportIndex];

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		if (FLinkerLoad::IsImportLazyLoadEnabled())
		{
			using namespace UE::LinkerLoad;
			if (GetPropertyImportLoadBehavior(Import, *this) != EImportBehavior::Eager)
			{
				continue;
			}
		}
#endif

		UObject* ImportObject = CreateImport(ImportIndex);

		// see if this import is currently being resolved (presumably somewhere 
		// up the callstack)... if it is, we need to ensure that this dependency 
		// is fully resolved before we get to regenerating the blueprint (else,
		// we could end up with placeholder classes in our script-code)
		if (FUnresolvedStructTracker::IsImportStructUnresolved(ImportObject))
		{
			// because it is tracked by FUnresolvedStructTracker, it must be a struct
			DEFERRED_DEPENDENCY_CHECK(Cast<UStruct>(ImportObject) != nullptr);
			FLinkerLoad* SourceLinker = FindExistingLinkerForImport(ImportIndex);
			if (SourceLinker)
			{
				SourceLinker->ResolveDeferredDependencies((UStruct*)ImportObject);
			}
		}
	}
}

void FLinkerLoad::FinalizeBlueprint(UClass* LoadClass)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (!FBlueprintSupport::UseDeferredDependencyLoading())
	{
		return;
	}
	DEFERRED_DEPENDENCY_CHECK(LoadClass->HasAnyFlags(RF_LoadCompleted));

	//--------------------------------------
	// Phase 3: Finalize (serialize CDO & regenerate class)
	//--------------------------------------
	TGuardValue<uint32> LoadFlagsGuard(LoadFlags, LoadFlags & ~LOAD_DeferDependencyLoads);

	// we can get in a state where a sub-class is getting finalized here, before
	// its super-class has been "finalized" (like when the super's 
	// ResolveDeferredDependencies() ends up Preloading a sub-class), so we
	// want to make sure that its properly finalized before this sub-class is
	// (so we can have a properly formed sub-class)
	if (UClass* SuperClass = LoadClass->GetSuperClass())
	{
		FLinkerLoad* SuperLinker = SuperClass->GetLinker();
		if ((SuperLinker != nullptr) && SuperLinker->IsBlueprintFinalizationPending())
		{
			DEFERRED_DEPENDENCY_CHECK(SuperLinker->DeferredCDOIndex != INDEX_NONE || SuperLinker->bForceBlueprintFinalization);
			UObject* SuperCDO = SuperLinker->DeferredCDOIndex != INDEX_NONE ? ToRawPtr(SuperLinker->ExportMap[SuperLinker->DeferredCDOIndex].Object) : SuperClass->GetDefaultObject(false);
			// we MUST have the super fully serialized before we can finalize  
			// this (class and CDO); if the SuperCDO is already in the midst of 
			// serializing somewhere up the stack (and a cyclic dependency has  
			// landed us here, finalizing one of it's children), then it is 
			// paramount that we force it through serialization (so we reset the 
			// RF_NeedLoad guard, and leave it to ResolveDeferredExports, for it
			// to re-run the serialization)
			if ((SuperCDO != nullptr) && !SuperCDO->HasAnyFlags(RF_NeedLoad|RF_LoadCompleted) && !FBlueprintSupport::IsDeferredDependencyPlaceholder(SuperCDO))
			{
				SuperCDO->SetFlags(RF_NeedLoad);
			}
			SuperLinker->FinalizeBlueprint(SuperClass);
		}
	}

	// at this point, we're sure that LoadClass doesn't contain any class 
	// placeholders (because ResolveDeferredDependencies() was ran on it); 
	// however, once we get to regenerating/compiling the blueprint, the graph 
	// (nodes, pins, etc.) could bring in new dependencies that weren't part of 
	// the class... this would normally be all fine and well, but in complicated 
	// dependency chains those graph-dependencies could already be in the middle 
	// of resolving themselves somewhere up the stack... if we just continue  
	// along and let the blueprint compile, then it could end up with  
	// placeholder refs in its script code (which it bad); we need to make sure
	// that all dependencies don't have any placeholder classes left in them
	// 
	// we don't want this to be part of ResolveDeferredDependencies() 
	// because we don't want this to count as a linker's "class resolution 
	// phase"; at this point, it is ok if other blueprints compile with refs to  
	// this LoadClass since it doesn't have any placeholders left in it (we also 
	// don't want this linker externally claiming that it has resolving left to 
	// do, otherwise other linkers could want to finish this off when they don't
	// have to)... we do however need it here in FinalizeBlueprint(), because
	// we need it ran for any super-classes before we regen

	ResolveAllImports();

	// Now that imports have been resolved we optionally flush the compilation
	// queue. This is only done for level blueprints, which will have instances
	// of actors in them that cannot reliably be reinstanced on load (see usage
	// of Scene pointers in things like UActorComponent::ExecuteRegisterEvents)
	// - on load the Scene may not yet be created, meaning this code cannot 
	// correctly be run. We could address that, but avoiding reinstancings is
	// also a performance win:
#if WITH_EDITOR
	LoadClass->FlushCompilationQueueForLevel();
#endif

	// interfaces can invalidate classes which implement them (when the  
	// interface is regenerated), they essentially define the makeup of the  
	// implementing class; so here, like we do with the parent class above, we  
	// ensure that all implemented interfaces are finalized first - this helps  
	// avoid cyclic issues where an interface ends up invalidating a dependent  
	// class by being regenerated after the class (see UE-28846)
	for (const FImplementedInterface& InterfaceDesc : LoadClass->Interfaces)
	{
		FLinkerLoad* InterfaceLinker = (InterfaceDesc.Class) ? InterfaceDesc.Class->GetLinker() : nullptr;
		if ((InterfaceLinker != nullptr) && InterfaceLinker->IsBlueprintFinalizationPending())
		{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			// the interface import should have been properly resolved above, in 
			// ResolveAllImports()
			if (!ensure(!InterfaceLinker->HasUnresolvedDependencies()))
#else 
			if (InterfaceLinker->HasUnresolvedDependencies())
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			{
				InterfaceLinker->ResolveDeferredDependencies(InterfaceDesc.Class);
			}
			InterfaceLinker->FinalizeBlueprint(InterfaceDesc.Class);
		}
	}

	// replace any export placeholders that were created, and serialize in the 
	// class's CDO
	ResolveDeferredExports(LoadClass);

	// the above calls (ResolveAllImports(), ResolveDeferredExports(), etc.) 
	// could have caused some recursion... if it ended up finalizing a sub-class 
	// (of LoadClass), then that would have finalized this as well; so, before 
	// we continue, make sure that this didn't already get fully executed in 
	// some nested call
	if (IsBlueprintFinalizationPending())
	{
		int32 DeferredCDOIndexCopy = DeferredCDOIndex;
		UObject* CDO = DeferredCDOIndex != INDEX_NONE ? ToRawPtr(ExportMap[DeferredCDOIndexCopy].Object) : LoadClass->GetDefaultObject(false);
		// clear this so IsBlueprintFinalizationPending() doesn't report true:
		FLinkerLoad::bForceBlueprintFinalization = false;
		// clear this because we're processing this CDO now:
		DeferredCDOIndex = INDEX_NONE;

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		// at this point there should not be any instances of the Blueprint 
		// (else, we'd have to re-instance and that is too expensive an 
		// operation to have at load time)
		TArray<UObject*> ClassInstances;
		GetObjectsOfClass(LoadClass, ClassInstances, /*bIncludeDerivedClasses =*/true);

		// Filter out instances that are part of this package, they were handled in ResolveDeferredExports:
		ClassInstances.RemoveAllSwap([LoadClass](UObject* Obj){return Obj->GetOutermost() == LoadClass->GetOutermost();});

		for (UObject* ClassInst : ClassInstances)
		{
			// in the case that we do end up with instances, use this to find 
			// where they are referenced (to help sleuth out when/where they 
			// were created)
			FReferencerInformationList InstanceReferences;
			bool const bIsReferenced = false;// IsReferenced(ClassInst, RF_NoFlags, /*bCheckSubObjects =*/false, &InstanceReferences);
			DEFERRED_DEPENDENCY_CHECK(!bIsReferenced);
		}
		DEFERRED_DEPENDENCY_CHECK(ClassInstances.Num() == 0);

		UClass* BlueprintClass = DeferredCDOIndexCopy != INDEX_NONE ? Cast<UClass>(IndexToObject(ExportMap[DeferredCDOIndexCopy].ClassIndex)) : LoadClass;
		DEFERRED_DEPENDENCY_CHECK(BlueprintClass == LoadClass);
		DEFERRED_DEPENDENCY_CHECK(BlueprintClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

		// for cooked builds (we skip script serialization for editor builds), 
		// certain scripts can contain references to external dependencies; and 
		// since the script is serialized in with the class (functions) we want 
		// those dependencies deferred until now (we expect this to be the right 
		// spot, because in editor builds, with RegenerateBlueprintClass(), this 
		// is where script code is regenerated)
		FStructScriptLoader::ResolveDeferredScriptLoads(this);

		DEFERRED_DEPENDENCY_CHECK(ImportPlaceholders.Num() == 0);
		DEFERRED_DEPENDENCY_CHECK(LoadClass->GetOutermost() != GetTransientPackage());
		// if we enable deferred dependency loading for cooked assets, and if we're also
		// not in the editor context... we want to keep from regenerating in that scenario
#if !WITH_EDITOR
		if (!LoadClass->bCooked)
#endif
		{
			UObject* OldCDO = LoadClass->GetDefaultObject(false);
			if (RegenerateBlueprintClass(LoadClass, CDO))
			{
				// emulate class CDO serialization (RegenerateBlueprintClass() could 
				// have a side-effect where it overwrites the class's CDO; so we 
				// want to make sure that we don't overwrite that new CDO with a 
				// stale one)
				if (OldCDO == LoadClass->GetDefaultObject(false))
				{
					LoadClass->SetDefaultObject(CDO);
				}
			}
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

void FLinkerLoad::ResolveDeferredExports(UClass* LoadClass)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (!IsBlueprintFinalizationPending())
	{
		return;
	}

	DEFERRED_DEPENDENCY_CHECK(DeferredCDOIndex != INDEX_NONE || bForceBlueprintFinalization);

	// Handle deferred construction of the CDO and patch it into the export table. Any deferred ctor
	// initializer dependencies (e.g. subobject class overrides) should now be resolved at this point.
	if (DeferredCDOIndex != INDEX_NONE)
	{
		if (ULinkerPlaceholderExportObject* PlaceholderExport = Cast<ULinkerPlaceholderExportObject>(ExportMap[DeferredCDOIndex].Object))
		{
			LoadClass->SetDefaultObject(nullptr);
			PlaceholderExport->SetLinker(nullptr, INDEX_NONE);
			ExportMap[DeferredCDOIndex].ResetObject();
			UObject* ExportObj = CreateExport(DeferredCDOIndex);

			PlaceholderExport->ResolveAllPlaceholderReferences(ExportObj);
			ResolvedDeferredSubobjects(PlaceholderExport);
			PlaceholderExport->MarkAsGarbage();

			DEFERRED_DEPENDENCY_CHECK(LoadClass->GetDefaultObject(false) == ExportObj);
		}
	}

	UObject* BlueprintCDO = DeferredCDOIndex != INDEX_NONE ? ToRawPtr(ExportMap[DeferredCDOIndex].Object) : LoadClass->GetDefaultObject(false);
	DEFERRED_DEPENDENCY_CHECK(BlueprintCDO != nullptr);
	
	TArray<int32> DeferredTemplateObjects;

	if (!FBlueprintSupport::IsDeferredExportCreationDisabled())
	{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		auto IsPlaceholderReferenced = [](ULinkerPlaceholderExportObject* ExportPlaceholder)->bool
		{
			UObject* PlaceholderObj = ExportPlaceholder;

			FReferencerInformationList UnresolvedReferences;
			bool bIsReferenced = IsReferenced(PlaceholderObj, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, /*bCheckSubObjects =*/false, &UnresolvedReferences);

			if (bIsReferenced && IsAsyncLoading())
			{
				// if we're async loading, then we assume a single external 
				// reference belongs to FAsyncObjectsReferencer, which is allowable 
				bIsReferenced = (UnresolvedReferences.ExternalReferences.Num() != 1) || (UnresolvedReferences.InternalReferences.Num() > 0);
			}
			return bIsReferenced;
		};
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

		// we may have circumvented an export creation or two to avoid instantiating
		// an BP object before its class has been finalized (to avoid costly re-
		// instancing operations when the class ultimately finalizes)... so here, we
		// find those skipped exports and properly create them (before we finalize 
		// our own class)

		// Mark this linker as ResolvingDeferredExports so that we don't continue deferring exports
		// we clear this flag after the loop. We have no TGuardValue for flags and so I'm setting
		// and clearing the bit manually:
		LoadFlags |= LOAD_ResolvingDeferredExports;

		for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num() && IsBlueprintFinalizationPending(); ++ExportIndex)
		{
			FObjectExport& Export = ExportMap[ExportIndex];
			if (ULinkerPlaceholderExportObject* PlaceholderExport = Cast<ULinkerPlaceholderExportObject>(Export.Object))
			{
				if(Export.ClassIndex.IsExport())
				{
					DeferredTemplateObjects.Push(ExportIndex);
					continue;
				}

				if(PlaceholderExport->IsDeferredSubobject())
				{
					continue;
				}

				UClass* ExportClass = GetExportLoadClass(ExportIndex);
				// export class could be null... we create these placeholder 
				// exports for objects that are instances of an external 
				// (Blueprint) type, not knowing if that type (class) will 
				// successfully load... it may resolve to null in scenarios 
				// where its super class has been deleted, or its super belonged 
				// to a plugin that has been removed/disabled 
				if (ExportClass != nullptr)
				{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
					DEFERRED_DEPENDENCY_CHECK(!ExportClass->HasAnyClassFlags(CLASS_Intrinsic) && ExportClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint));
					FLinkerLoad* ClassLinker = ExportClass->GetLinker();
					DEFERRED_DEPENDENCY_CHECK((ClassLinker != nullptr) && (ClassLinker != this));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

					FScopedResolvingExportTracker ForceRegenGuard(this, ExportIndex);
					// make sure this export's class is fully regenerated before  
					// we instantiate it (so we don't have to re-inst on load)
					ForceRegenerateClass(ExportClass);

					if (PlaceholderExport != Export.Object)
					{
						DEFERRED_DEPENDENCY_CHECK( !IsPlaceholderReferenced(PlaceholderExport) );
						continue;
					}
				}

				// replace the placeholder with the proper object instance
				PlaceholderExport->SetLinker(nullptr, INDEX_NONE);
				Export.ResetObject();
				UObject* ExportObj = CreateExport(ExportIndex);

				// NOTE: we don't count how many references were resolved (and 
				//       assert on it), because this could have only been created as 
				//       part of the LoadAllObjects() pass (not for any specific 
				//       container object).
				PlaceholderExport->ResolveAllPlaceholderReferences(ExportObj);

				ResolvedDeferredSubobjects(PlaceholderExport);

				PlaceholderExport->MarkAsGarbage();

				// if we hadn't used a ULinkerPlaceholderExportObject in place of 
				// the expected export, then someone may have wanted it preloaded
				if (ExportObj != nullptr)
				{
					Preload(ExportObj);
				}
				DEFERRED_DEPENDENCY_CHECK( !IsPlaceholderReferenced(PlaceholderExport) );
			}
		}

		LoadFlags &= ~LOAD_ResolvingDeferredExports;
	}

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	// this helps catch any placeholder export objects that may be created 
	// between now and when DeferredCDOIndex is cleared (they won't be resolved,
	// so that is a problem!)
	FResolvingExportTracker::Get().FlagFullExportResolvePassComplete(this);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	// the ExportMap loop above could have recursed back into "finalization" for  
	// this asset, subsequently resolving all exports before this function could 
	// finish... that means there's no work left for this to do (and trying to
	// redo the work would cause a crash), so we guard here against that
	if (IsBlueprintFinalizationPending())
	{
		// have to prematurely set the CDO's linker so we can force a Preload()/
		// Serialization of the CDO before we regenerate the Blueprint class
		{
			if (DeferredCDOIndex != INDEX_NONE)
			{
				const EObjectFlags OldFlags = BlueprintCDO->GetFlags();
				BlueprintCDO->ClearFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects);
				BlueprintCDO->SetLinker(this, DeferredCDOIndex, /*bShouldDetatchExisting =*/false);
				BlueprintCDO->SetFlags(OldFlags);
			}
		}
		DEFERRED_DEPENDENCY_CHECK(BlueprintCDO->GetClass() == LoadClass);

		// should load the CDO (ensuring that it has been serialized in by the 
		// time we get to class regeneration)
		//
		// NOTE: this is point where circular dependencies could reveal 
		//       themselves, as the CDO could depend on a class not listed in 
		//       the package's imports
		//
		// NOTE: how we don't guard against re-entrant behavior... if the CDO 
		//       has already been "finalized", then its RF_NeedLoad flag would 
		//       be cleared (and this will do nothing the 2nd time around)
		Preload(BlueprintCDO);

		// Ensure that all subobject exports belonging to the CDO have been created. This is often handled by 
		// PreloadSubobjects in CreateExport, but they can get skipped. Subobjects need to be created here so
		// they can be correctly inherited by any child classes and they are correctly registered for any later
		// deferred fixups related to native class changes.
		for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
		{
			FObjectExport& Export = ExportMap[ExportIndex];
			FPackageIndex CheckOuterIndex = Export.OuterIndex;
			bool bInsideCDO = false;
			while (CheckOuterIndex.IsExport())
			{
				int32 OuterExportIndex = CheckOuterIndex.ToExport();
				if (OuterExportIndex == DeferredCDOIndex)
				{
					bInsideCDO = true;
					break;
				}
				else
				{
					// Handle nested subobjects
					CheckOuterIndex = ExportMap[OuterExportIndex].OuterIndex;
				}
			}

			if (bInsideCDO)
			{
				if (Export.Object == nullptr)
				{
					CreateExport(ExportIndex);
				}

				// In order to complete loading of the CDO we need to also preload its subobjects. Other CDOs 
				// will use these subobjects as archetypes for their own subobjects when they run InitSubobjectProperties
				if(Export.Object)
				{
					Preload(Export.Object);
				}
			}
		}

		{
			// Create any objects that (non CDO) objects that were deferred in this package:
			TGuardValue<int32> ClearDeferredCDOToPreventDeferExportCreation(DeferredCDOIndex, INDEX_NONE);
			for(int32 ExportIndex : DeferredTemplateObjects)
			{
				FObjectExport& Export = ExportMap[ExportIndex];
				ULinkerPlaceholderExportObject* PlaceholderExport = Cast<ULinkerPlaceholderExportObject>(Export.Object);
				if (ensure(PlaceholderExport) && !PlaceholderExport->IsMarkedResolved())
				{
					// replace the placeholder with the proper object instance
					PlaceholderExport->SetLinker(nullptr, INDEX_NONE);
					Export.ResetObject();
					UObject* ExportObj = CreateExport(ExportIndex);

					PlaceholderExport->ResolveAllPlaceholderReferences(ExportObj);
					ResolvedDeferredSubobjects(PlaceholderExport);

					PlaceholderExport->MarkAsGarbage();
					if (ExportObj != nullptr)
					{
						Preload(ExportObj);
					}
				}
			}
		}

		// sub-classes of this Blueprint could have had their CDO's 
		// initialization deferred (this occurs when the sub-class CDO is 
		// created before this super CDO has been fully serialized; we do this
		// because the sub-class's CDO would not have been initialized with 
		// accurate values)
		//
		// in that case, the sub-class CDOs are waiting around until their 
		// super CDO is fully loaded (which is now)... we want to do this here, 
		// before this (super) Blueprint gets regenerated, because after it's
		// regenerated the class layout (and property offsets) may no longer 
		// match the layout that sub-class CDOs were constructed with (making 
		// property copying dangerous)
		FDeferredObjInitializationHelper::ResolveDeferredInitsFromArchetype(BlueprintCDO);

		DEFERRED_DEPENDENCY_CHECK(BlueprintCDO->HasAnyFlags(RF_LoadCompleted));
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

void FLinkerLoad::ResolvePlaceholder(ULinkerPlaceholderExportObject* Placeholder)
{
	int32 ExportIndex = Placeholder->PackageIndex.ToExport();

	Placeholder->SetLinker(nullptr, INDEX_NONE);

	FObjectExport& Export = ExportMap[ExportIndex];
	Export.Object = nullptr;

	UObject* ReplacementObject = CreateExport(ExportIndex);
	Placeholder->ResolveAllPlaceholderReferences(ReplacementObject);
	Placeholder->MarkAsGarbage();
	
	// recurse:
	ResolvedDeferredSubobjects(Placeholder);

	// attempt to preload, we don't really care if this doesn't complete but we don't want to fail
	// to serialize an object:
	if (ReplacementObject != nullptr)
	{
		Preload(ReplacementObject);
	}
}

void FLinkerLoad::ResolvedDeferredSubobjects(ULinkerPlaceholderExportObject* OwningPlaceholder)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	ensure(OwningPlaceholder->IsMarkedResolved());
	for(ULinkerPlaceholderExportObject* PlaceholderSubobject : OwningPlaceholder->GetSubobjectPlaceholders() )
	{
		int32 ExportIndex = PlaceholderSubobject->PackageIndex.ToExport();
	
		PlaceholderSubobject->SetLinker(nullptr, INDEX_NONE);

		FObjectExport& Export = ExportMap[ExportIndex];

		Export.ResetObject();

		UObject* ReplacementObject = CreateExport(ExportIndex);
		PlaceholderSubobject->ResolveAllPlaceholderReferences(ReplacementObject);
		PlaceholderSubobject->MarkAsGarbage();

		// recurse:
		ResolvedDeferredSubobjects(PlaceholderSubobject);

		// serialize:
		if (ReplacementObject != nullptr)
		{
			Preload(ReplacementObject);
		}
	}
#endif//USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

void FLinkerLoad::ForceBlueprintFinalization()
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	check(!bForceBlueprintFinalization);
	bForceBlueprintFinalization = true;
#endif
}

bool FLinkerLoad::IsBlueprintFinalizationPending() const
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return (DeferredCDOIndex != INDEX_NONE) || bForceBlueprintFinalization;
#else  // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return false;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

bool FLinkerLoad::ForceRegenerateClass(UClass* ImportClass)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (FLinkerLoad* ClassLinker = ImportClass->GetLinker())
	{
		//
		// BE VERY CAREFUL with this! if these following statements are called 
		// in the wrong place, we could end up infinitely recursing

		Preload(ImportClass);
		DEFERRED_DEPENDENCY_CHECK(ImportClass->HasAnyFlags(RF_LoadCompleted));

		if (ClassLinker->HasUnresolvedDependencies())
		{
			ClassLinker->ResolveDeferredDependencies(ImportClass);
		}
		if (ClassLinker->IsBlueprintFinalizationPending())
		{
			ClassLinker->FinalizeBlueprint(ImportClass);
		}
		return true;
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return false;
}

bool FLinkerLoad::IsExportBeingResolved(int32 ExportIndex)
{
	FObjectExport& Export = ExportMap[ExportIndex];
	bool bIsExportClassBeingForceRegened = FResolvingExportTracker::Get().IsLinkerExportBeingResolved(this, ExportIndex);

	FPackageIndex OuterIndex = Export.OuterIndex;
	// since child exports require their outers be set upon creation, then those 
	// too count as being "resolved"... so here we check this export's outers too
	while (!bIsExportClassBeingForceRegened && OuterIndex.IsExport())
	{
		int32 OuterExportIndex = OuterIndex.ToExport();
		if (OuterExportIndex != INDEX_NONE)
		{
			FObjectExport& OuterExport = ExportMap[OuterExportIndex];
			bIsExportClassBeingForceRegened |= FResolvingExportTracker::Get().IsLinkerExportBeingResolved(this, OuterExportIndex);

			OuterIndex = OuterExport.OuterIndex;
		}
		else
		{
			break;
		}
	}
	return bIsExportClassBeingForceRegened;
}

void FLinkerLoad::ResetDeferredLoadingState()
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	DeferredCDOIndex = INDEX_NONE;
	bForceBlueprintFinalization = false;
	ResolvingPlaceholderStack.Empty();
	ImportPlaceholders.Empty();
	LoadFlags &= ~(LOAD_DeferDependencyLoads);

	FResolvingExportTracker::Get().Reset(this);
	FUnresolvedStructTracker::Reset(this);
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

bool FLinkerLoad::HasPerformedFullExportResolvePass()
{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	return FResolvingExportTracker::Get().HasPerformedFullExportResolvePass(this);
#else 
	return false;
#endif
	
}

UObject* FLinkerLoad::RequestPlaceholderValue(const FProperty* Property, const UClass* ObjectType, const TCHAR* ObjectPath)
{
#if !USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return nullptr;
#else
	FLinkerPlaceholderBase* Placeholder = nullptr;

	if (FBlueprintSupport::UseDeferredDependencyLoading() && (LoadFlags & LOAD_DeferDependencyLoads))
	{
		const FName ObjId(ObjectPath);
		if (FLinkerPlaceholderBase** PlaceholderPtr = ImportPlaceholders.Find(ObjId))
		{
			Placeholder = *PlaceholderPtr;
		}
		// right now we only support external parties requesting CLASS placeholders;
		// if there is a scenario where they're, through a different ObjectType, 
		// loading another Blueprint package when they shouldn't, then we need to 
		// handle that here as well
		else if (ObjectType->IsChildOf<UClass>())
		{
			// Class property values will typically always request the base UClass type via FObjectPropertyBase, so
			// we try and redirect to the actual class object value type to determine if a placeholder can be created.
			// Generally, we shouldn't be in here unless we're serializing an object property's value from exported T3D.
			if (const FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
			{
				if (ClassProperty->MetaClass)
				{
					// Note: We are interested in the value's underlying class type, not the value itself (which will be an instance of the class type).
					const UClass* ValueType = ClassProperty->MetaClass->GetClass();
					if (const ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(ClassProperty->MetaClass))
					{
						// If the type was deferred on import of the property value, redirect to the type of value whose load has been deferred (this should be a native UClass derivative).
						ValueType = PlaceholderClass->DeferredObjectType.Get();
					}

					if (ValueType)
					{
						checkf(ValueType->IsChildOf(ObjectType),
							TEXT("Requesting an import placeholder object for a class value type (%s) that is not a derivative of the required object type (%s)."),
							*ValueType->GetName(),
							*ObjectType->GetName());

						ObjectType = ValueType;
					}
				}
			}

			const FStringView ObjectPathStr(ObjectPath);
			// we don't need placeholders for native object references and for non-BP class objects (the 
			// calling code should properly handle null return values)
			if (!FPackageName::IsScriptPackage(ObjectPathStr) && ObjectType->HasAnyClassFlags(CLASS_NeedsDeferredDependencyLoading))
			{
				FPackageIndex ImportIndex;

				if (FindImport(ObjectPathStr, ImportIndex))
				{
					const FStringView ObjectName = FPackageName::ObjectPathToObjectName(ObjectPathStr);

					Placeholder = MakeImportPlaceholder<ULinkerPlaceholderClass>(LinkerRoot, ObjectType, ObjectName.GetData(), ImportIndex.ToImport());
					ImportPlaceholders.Add(ObjId, Placeholder);
				}
				else
				{
					UE_LOGF(LogBlueprintSupport, Error, "'%ls' was not found in the import table for package '%ls'.", ObjectPathStr.GetData(), *LinkerRoot->GetFullName());
				}
			}
		}
	}

	return Placeholder ? Placeholder->GetPlaceholderAsUObject() : nullptr;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

UObject* FLinkerLoad::FindImport(UClass* ImportClass, UObject* ImportOuter, const TCHAR* Name)
{	
	UObject* Result = StaticFindObject(ImportClass, ImportOuter, Name);
	return Result;
}

UObject* FLinkerLoad::FindImportFast(UClass* ImportClass, UObject* ImportOuter, FName Name, bool bFindObjectbyName)
{
	UObject* Result = nullptr;
	if (!bFindObjectbyName)
	{
		Result = StaticFindObjectFast(ImportClass, ImportOuter, Name, EFindObjectFlags::None);
	}
	else
	{
		Result = StaticFindFirstObject(ImportClass, *Name.ToString(), EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous, ELogVerbosity::Warning, TEXT("FindImportFast"));
	}

	return Result;
}

/*******************************************************************************
 * UObject
 ******************************************************************************/

bool UObject::IsInBlueprint() const
{
	// Exclude blueprint classes as they may be regenerated at any time
	// Need to exclude classes, CDOs, and their subobjects
	const UObject* TestObject = this;
 	while (TestObject)
 	{
 		const UClass *ClassObject = dynamic_cast<const UClass*>(TestObject);
		if (ClassObject && ClassObject->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
 		{
 			return true;
 		}
		else if (TestObject->HasAnyFlags(RF_ClassDefaultObject) 
			&& TestObject->GetClass() 
			&& TestObject->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
 		{
 			return true;
 		}
 		TestObject = TestObject->GetOuter();
 	}

	return false;
}

void UObject::DestroyNonNativeProperties()
{
	if (HasAnyInternalFlags(EInternalObjectFlags::AutoRTFMConstructionAborted))
	{
		// Object was constructed in a transaction that was aborted.
		// Return as the object is unlikely to be in a valid state.
		return;
	}

	// Destroy properties that won't be destroyed by the native destructor
	GetClass()->DestroyPersistentUberGraphFrame(this);

	for (FProperty* P = GetClass()->DestructorLink; P; P = P->DestructorLinkNext)
	{
		if (!P->GetOwnerClass()->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
		{
			// Non-native value, destroy.
			P->DestroyValue_InContainer(this);
		}
		else
		{
			// Native value, call to handle finish destroy.
			// Native properties appear here if they report true on ContainsFinishDestroy() during UStruct::Link().
			P->FinishDestroy_InContainer(this);
		}
	}
}

/*******************************************************************************
 * FObjectInitializer
 ******************************************************************************/

bool FObjectInitializer::InitNonNativeProperty(FProperty* Property, UObject* Data)
{
	if (!Property->GetOwnerClass()->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic)) // if this property belongs to a native class, it was already initialized by the class constructor
	{
		if (!Property->HasAnyPropertyFlags(CPF_ZeroConstructor)) // this stuff is already zero
		{
			Property->InitializeValue_InContainer(Data);
		}
		return true;
	}
	else
	{
		// we have reached a native base class, none of the rest of the properties will need initialization
		return false;
	}
}

/*******************************************************************************
 * FDeferredInitializationTrackerBase
 ******************************************************************************/

FObjectInitializer* FDeferredInitializationTrackerBase::Add(const UObject* InitDependency, const FObjectInitializer& DeferringInitializer)
{
	FObjectInitializer* DeferredInitializerCopy = nullptr;

	DEFERRED_DEPENDENCY_CHECK(InitDependency);
	if (InitDependency)
	{
		UObject* InstanceObj = DeferringInitializer.GetObj();
		ArchetypeInstanceMap.AddUnique(InitDependency, InstanceObj);

		DEFERRED_DEPENDENCY_CHECK(DeferredInitializers.Find(InstanceObj) == nullptr); // did we try to init the object twice?

		// NOTE: we copy the FObjectInitializer, because it is most likely in the process of being destroyed
		DeferredInitializerCopy = &DeferredInitializers.Add(InstanceObj, DeferringInitializer);
	}
	return DeferredInitializerCopy;
}

void FDeferredInitializationTrackerBase::ResolveArchetypeInstances(UObject* InitDependency)
{
	TArray<UObject*> ArchetypeInstances;
	ArchetypeInstanceMap.MultiFind(InitDependency, ArchetypeInstances);

	for (UObject* Instance : ArchetypeInstances)
	{
		DEFERRED_DEPENDENCY_CHECK(ResolvingObjects.Contains(Instance) == false);
		ResolvingObjects.Push(Instance);

		if (ResolveDeferredInitialization(InitDependency, Instance))
		{
			// For sub-objects, this has to come after ResolveDeferredInitialization(), since InitSubObjectProperties() is 
			// invoked there (which is where we fill this sub-object with values from the super)
			PreloadDeferredDependents(Instance);
		}
		
		DEFERRED_DEPENDENCY_CHECK(ResolvingObjects.Top() == Instance);
		ResolvingObjects.Pop();
	}

	ArchetypeInstanceMap.Remove(InitDependency);
}

bool FDeferredInitializationTrackerBase::IsInitializationDeferred(const UObject* Object) const
{
	return DeferredInitializers.Contains(Object);
}

bool FDeferredInitializationTrackerBase::DeferPreload(UObject* Object)
{
	const bool bDeferPreload = IsInitializationDeferred(Object);
	if (bDeferPreload && !IsResolving(Object))
	{
		DeferredPreloads.AddUnique(Object, Object);
	}
	return bDeferPreload;
}

bool FDeferredInitializationTrackerBase::IsResolving(UObject* ArchetypeInstance) const
{
	return ResolvingObjects.Contains(ArchetypeInstance);
}

bool FDeferredInitializationTrackerBase::ResolveDeferredInitialization(UObject* /*ResolvingObject*/, UObject* ArchetypeInstance)
{
	if (FObjectInitializer* DeferredInitializer = DeferredInitializers.Find(ArchetypeInstance))
	{
		// initializes and instances CDO properties (copies inherited values 
		// from the super's CDO)
		FScriptIntegrationObjectHelper::PostConstructInitObject(*DeferredInitializer);

		DeferredInitializers.Remove(ArchetypeInstance);
	}

	return true;
}

void FDeferredInitializationTrackerBase::PreloadDeferredDependents(UObject* ArchetypeInstance)
{
	TArray<UObject*> ObjsToPreload;
	DeferredPreloads.MultiFind(ArchetypeInstance, ObjsToPreload);

	for (UObject* Object : ObjsToPreload)
	{
		FLinkerLoad* Linker = Object->GetLinker();
		DEFERRED_DEPENDENCY_CHECK(Linker != nullptr);
		if (Linker)
		{
			Linker->Preload(Object);
		}
	}

	DeferredPreloads.Remove(ArchetypeInstance);
}

/*******************************************************************************
 * FDeferredCdoInitializationTracker
 ******************************************************************************/

bool FDeferredCdoInitializationTracker::DeferPreload(UObject* Object)
{
	bool bDeferPostload = false;

	if (Object->HasAnyFlags(RF_ClassDefaultObject))
	{
		// When the initialization has been deferred we have to make sure to
		// defer serialization as well - don't worry, for CDOs, Preload() will be invoked 
		// again from FinalizeBlueprint()->ResolveDeferredExports()
		bDeferPostload = !IsResolving(Object) && IsInitializationDeferred(Object);
	}
	else
	{
		auto ShouldDeferSubObjectPreload = [this, Object](UObject* OwnerObject)->bool
		{
			if (IsInitializationDeferred(OwnerObject))
			{
				const bool bDeferSubObjPostload = !IsResolving(OwnerObject);
				if (bDeferSubObjPostload)
				{
					DeferredPreloads.AddUnique(OwnerObject, Object);
				}
				return bDeferSubObjPostload;
			}
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			else if (OwnerObject)
			{
				UObject* OwnerClass = OwnerObject->GetClass();
				for (auto& DeferredCdo : DeferredInitializers)
				{
					// we used to index these by class, so to ensure the same behavior validate
					// our assumption that we can use the CDO object itself as the key (and that 
					// using the class wouldn't find a match instead)
					DEFERRED_DEPENDENCY_CHECK(DeferredCdo.Key->GetClass() != OwnerClass);
				}
			}
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			return false;
		};

		if (Object->HasAnyFlags(RF_DefaultSubObject))
		{
			UObject* SubObjOuter = Object->GetOuter();
			// NOTE: The outer of a DSO may not be a CDO like we want. It could 
			//       be something like a component template. Right now we ignore 
			//       those cases (IsDeferred() will reject this - only CDOs are 
			//       deferred in this struct), but if this case proves to be a problem, 
			//       then we may need to look up the outer chain, or see if the outer 
			//       sub-obj is deferred itself.
			bDeferPostload = ShouldDeferSubObjectPreload(SubObjOuter);
		}
		else if (Object->HasAnyFlags(RF_InheritableComponentTemplate))
		{
			UClass* OwningClass = Cast<UClass>(Object->GetOuter());

			DEFERRED_DEPENDENCY_CHECK(OwningClass && OwningClass->GetDefaultObject(false));
			if (OwningClass)
			{
				bDeferPostload = ShouldDeferSubObjectPreload(OwningClass->GetDefaultObject(false));
			}
		}
	}
	return bDeferPostload;
}

/*******************************************************************************
 * FDeferredSubObjInitializationTracker
 ******************************************************************************/

bool FDeferredSubObjInitializationTracker::ResolveDeferredInitialization(UObject* ResolvingObject, UObject* ArchetypeInstance)
{
	bool bInitializerRan = false;

	// If we deferred the sub-object because the super CDO wasn't ready, we still
	// need to check that its archetype is in a ready state (ready to be copied from)
	if (ResolvingObject->HasAnyFlags(RF_ClassDefaultObject))
	{
		if (FObjectInitializer* DeferredInitializer = DeferredInitializers.Find(ArchetypeInstance))
		{
			UObject* Archetype = DeferredInitializer->GetArchetype();
			// When this sub-object was created it's archetype object (the 
			// super's sub-obj) may not have been created yet. In that scenario, the 
			// component class's CDO would have been used in its place; now that 
			// the super is good, we should update the archetype
			if (ArchetypeInstance->HasAnyFlags(RF_ClassDefaultObject))
			{
				Archetype = UObject::GetArchetypeFromRequiredInfo(ArchetypeInstance->GetClass(), ArchetypeInstance->GetOuter(), ArchetypeInstance->GetFName(), ArchetypeInstance->GetFlags());
			}

			const bool bArchetypeLoadPending = Archetype &&
				(Archetype->HasAnyFlags(RF_NeedLoad) || (Archetype->HasAnyFlags(RF_WasLoaded) && !Archetype->HasAnyFlags(RF_LoadCompleted)));

			if (bArchetypeLoadPending)
			{
				// Archetype isn't ready, move the deferred initializer to wait for its archetype 
				ArchetypeInstanceMap.Add(Archetype, ArchetypeInstance);
				// don't need to add this to DeferredInitializers, as it is already there
			}
			else
			{
				bInitializerRan = FDeferredInitializationTrackerBase::ResolveDeferredInitialization(ResolvingObject, ArchetypeInstance);
			}
		}
	}
	else
	{
		bInitializerRan = FDeferredInitializationTrackerBase::ResolveDeferredInitialization(ResolvingObject, ArchetypeInstance);
	}

	return bInitializerRan;
}

/*******************************************************************************
 * FDeferredObjInitializationHelper
 ******************************************************************************/

FObjectInitializer* FDeferredObjInitializationHelper::DeferObjectInitializerIfNeeded(const FObjectInitializer& DeferringInitializer)
{
	FObjectInitializer* DeferredInitializerCopy = nullptr;

	UObject* TargetObj = DeferringInitializer.GetObj();
	if (TargetObj)
	{
		auto IsSuperCdoReadyToBeCopied = [](FDeferredCdoInitializationTracker& InCdoInitDeferalSys, const UClass* LoadClass, const UObject* SuperCDO)->bool
		{
			// RF_WasLoaded indicates that this Super was loaded from disk (and hasn't been regenerated on load)
			// regenerated CDOs will not have the RF_LoadCompleted
			const bool bSuperCdoLoadPending = InCdoInitDeferalSys.IsInitializationDeferred(SuperCDO) ||
				SuperCDO->HasAnyFlags(RF_NeedLoad) || (SuperCDO->HasAnyFlags(RF_WasLoaded) && !SuperCDO->HasAnyFlags(RF_LoadCompleted));

			if (bSuperCdoLoadPending)
			{
				const FLinkerLoad* ObjLinker = LoadClass->GetLinker();
				const bool bIsBpClassSerializing = ObjLinker && (ObjLinker->LoadFlags & LOAD_DeferDependencyLoads);
				const bool bIsResolvingDeferredObjs = LoadClass->HasAnyFlags(RF_LoadCompleted) &&
					ObjLinker && ObjLinker->IsBlueprintFinalizationPending();

				DEFERRED_DEPENDENCY_CHECK(bIsBpClassSerializing || bIsResolvingDeferredObjs);
				return !bIsBpClassSerializing && !bIsResolvingDeferredObjs;
			}
			return true;
		};

		auto IsObjectLoadPending = [](const UObject* InObject)
		{
			return InObject &&
				(InObject->HasAnyFlags(RF_NeedLoad) || (InObject->HasAnyFlags(RF_WasLoaded) && !InObject->HasAnyFlags(RF_LoadCompleted)));
		};

		const bool bIsCDO = TargetObj->HasAnyFlags(RF_ClassDefaultObject);
		if (bIsCDO)
		{
			const UClass* CdoClass = DeferringInitializer.GetClass();
			UClass* SuperClass = CdoClass->GetSuperClass();

			if (!CdoClass->IsNative() && !SuperClass->IsNative())
			{
				DEFERRED_DEPENDENCY_CHECK(CdoClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint));
				DEFERRED_DEPENDENCY_CHECK(SuperClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint));

				const UObject* SuperCDO = DeferringInitializer.GetArchetype();
				DEFERRED_DEPENDENCY_CHECK(SuperCDO && SuperCDO->HasAnyFlags(RF_ClassDefaultObject));
				// use the ObjectArchetype for the super CDO because the SuperClass may have a REINST CDO cached currently
				SuperClass = SuperCDO->GetClass();
				
				FDeferredCdoInitializationTracker& CdoInitDeferalSys = FDeferredCdoInitializationTracker::Get();
				if (!IsSuperCdoReadyToBeCopied(CdoInitDeferalSys, CdoClass, SuperCDO))
				{
					DeferredInitializerCopy = CdoInitDeferalSys.Add(SuperCDO, DeferringInitializer);
				}
			}
		}
		// since "InheritableComponentTemplate"s are not default sub-objects, 
		// they won't be fixed up by the owner's FObjectInitializer (CDO 
		// FObjectInitializers will init default sub-object properties, copying  
		// from the super's DSOs) - this means that we need to separately defer 
		// init'ing these sub-objects when their archetype hasn't been loaded yet 
		else if (TargetObj->HasAnyFlags(RF_InheritableComponentTemplate))
		{
			const UClass* OwnerClass = Cast<UClass>(TargetObj->GetOuter());
			DEFERRED_DEPENDENCY_CHECK(OwnerClass && OwnerClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint));
			const UClass* SuperClass = OwnerClass->GetSuperClass();

			if (SuperClass && !SuperClass->IsNative())
			{
				// It is possible that the archetype isn't even correct, if the 
				// super's sub-object hasn't even been created yet (in this case the 
				// component's CDO is used, which is probably wrong)
				// 
				// So if the super CDO isn't ready, we need to defer this sub-object
				const UObject* SuperCDO = SuperClass->GetDefaultObject(false);
				FDeferredCdoInitializationTracker& CdoInitDeferalSys = FDeferredCdoInitializationTracker::Get();
				if (!IsSuperCdoReadyToBeCopied(CdoInitDeferalSys, OwnerClass, SuperCDO))
				{
					FDeferredSubObjInitializationTracker& SubObjInitDeferalSys = FDeferredSubObjInitializationTracker::Get();
					DeferredInitializerCopy = SubObjInitDeferalSys.Add(SuperCDO, DeferringInitializer);
				}
			}

			// if it passed the super CDO check above, assume the archetype is kosher
			if (!DeferredInitializerCopy)
			{
				UObject* Archetype = DeferringInitializer.GetArchetype();
				if (IsObjectLoadPending(Archetype))
				{
					FDeferredSubObjInitializationTracker& SubObjInitDeferalSys = FDeferredSubObjInitializationTracker::Get();
					DeferredInitializerCopy = SubObjInitDeferalSys.Add(Archetype, DeferringInitializer);
				}
			}
		}
		else if (TargetObj->HasAnyFlags(RF_DefaultSubObject))
		{
			// Since users can override default subobject types with non-native subtypes from the editor side, we need to
			// ensure its non-native CDO has been fully serialized before we can allow those subobjects to be initialized.
			// Deferral can occur e.g. when the non-native subtype contains a strong reference to a non-native owner type,
			// resulting in a circular load dependency that can manifest if the non-native subobject type is loaded first.
			// In that case, we'll then defer that subobject's initialization until after we've serialized its type's CDO.
			const UClass* SubobjectClass = TargetObj->GetClass();
			if (ensure(SubobjectClass) && !SubobjectClass->IsNative())
			{
				DEFERRED_DEPENDENCY_CHECK(SubobjectClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint));

				// Grab the subobject type's CDO and verify that it's what we expect. At this point, any placeholder export
				// should at least have been created/resolved to an actual object, but may not be fully serialized just yet.
				UObject* SubobjectCDO = SubobjectClass->GetDefaultObject(false);
				DEFERRED_DEPENDENCY_CHECK(SubobjectCDO && SubobjectCDO->HasAnyFlags(RF_ClassDefaultObject));

				if (IsObjectLoadPending(SubobjectCDO))
				{
					FDeferredSubObjInitializationTracker& SubObjInitDeferalSys = FDeferredSubObjInitializationTracker::Get();
					DeferredInitializerCopy = SubObjInitDeferalSys.Add(SubobjectCDO, DeferringInitializer);
				}
			}
		}
	}

	return DeferredInitializerCopy;
}

bool FDeferredObjInitializationHelper::DeferObjectPreload(UObject* Object)
{
	return FDeferredCdoInitializationTracker::Get().DeferPreload(Object) || FDeferredSubObjInitializationTracker::Get().DeferPreload(Object);
}

void FDeferredObjInitializationHelper::ResolveDeferredInitsFromArchetype(UObject* Archetype)
{
	FDeferredCdoInitializationTracker& DeferredCdoTracker = FDeferredCdoInitializationTracker::Get();
	FDeferredSubObjInitializationTracker& DeferredSubObjTracker = FDeferredSubObjInitializationTracker::Get();

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	if (Archetype->HasAnyFlags(RF_ClassDefaultObject))
	{
		// we used to index the deferred initialization by class, so to ensure the same behavior validate
		// our assumption that we can use the CDO object itself as the key (and that using the class wouldn't find a match instead)
		auto IsDeferredByClass = [Archetype](const TMultiMap<const UObject*, UObject*>& ArchetypeMap)->bool
		{
			for (auto& DeferredObj : ArchetypeMap)
			{
				if (DeferredObj.Key->GetClass() == Archetype->GetClass())
				{
					return true;
				}
			}
			return false;
		};

		if (!DeferredCdoTracker.ArchetypeInstanceMap.Contains(Archetype))
		{
			DEFERRED_DEPENDENCY_CHECK(IsDeferredByClass(DeferredCdoTracker.ArchetypeInstanceMap) == false);
		}
		if (!DeferredSubObjTracker.ArchetypeInstanceMap.Contains(Archetype))
		{
			DEFERRED_DEPENDENCY_CHECK(IsDeferredByClass(DeferredSubObjTracker.ArchetypeInstanceMap) == false);
		}
	}
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	DeferredCdoTracker.ResolveArchetypeInstances(Archetype);
	DeferredSubObjTracker.ResolveArchetypeInstances(Archetype);
}

// don't want other files ending up with this internal define
#undef DEFERRED_DEPENDENCY_CHECK
