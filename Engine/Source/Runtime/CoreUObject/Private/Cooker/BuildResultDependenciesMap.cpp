// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/BuildResultDependenciesMap.h"

#if WITH_EDITOR
#include "Cooker/CookDependency.h"
#endif

#if WITH_EDITOR

namespace UE::Cook
{

void FBuildResultDependenciesMap::Add(FName Name, UE::Cook::FCookDependency CookDependency)
{
	FindOrAdd(Name).Add(MoveTemp(CookDependency));
}

void FBuildResultDependenciesMap::Append(FName Name, TArray<UE::Cook::FCookDependency> CookDependencies)
{
	FindOrAdd(Name).Append(MoveTemp(CookDependencies));
}

void FBuildResultDependenciesMap::Append(const FBuildResultDependenciesMap& Other)
{
	for (const TPair<FName, TArray<FCookDependency>>& Pair: Other)
	{
		FindOrAdd(Pair.Key).Append(Pair.Value);
	}
}

void FBuildResultDependenciesMap::Append(FBuildResultDependenciesMap&& Other)
{
	for (TPair<FName, TArray<FCookDependency>>& Pair : Other)
	{
		FindOrAdd(Pair.Key).Append(MoveTemp(Pair.Value));
	}
}

} // namespace UE::Cook

#endif // WITH_EDITOR