// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "ConsoleVariableData.h"

#include "HAL/IConsoleManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Misc/CoreMisc.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"

namespace UE::Cook
{
	FConsoleVariableData::FConsoleVariableData()
		: VariableName()
		, TargetPlatform(nullptr)
		, bFallbackToNonPlatformValue(false)
	{ }

	FConsoleVariableData::FConsoleVariableData(const FStringView& InVariableName, const ITargetPlatform* InTargetPlatform, bool bInFallbackToNonPlatformValue)
		: VariableName(InVariableName)
		, TargetPlatform(InTargetPlatform)
		, bFallbackToNonPlatformValue(bInFallbackToNonPlatformValue)
	{ }

	void FConsoleVariableData::Save(FCbWriter& Writer) const
	{
		Writer.BeginArray();
		Writer << VariableName;

		bool bHasTargetPlatform = TargetPlatform != nullptr;
		Writer << bHasTargetPlatform;

		if (bHasTargetPlatform)
		{
			Writer << TargetPlatform->PlatformName();
		}

		Writer << bFallbackToNonPlatformValue;

		Writer.EndArray();
	}

	bool FConsoleVariableData::TryLoad(FCbFieldView Value)
	{
		if (!Value.IsArray())
		{
			return false;
		}

		FCbFieldViewIterator ArrayIterator = Value.CreateViewIterator();

		if (!LoadFromCompactBinary(ArrayIterator++, VariableName))
		{
			return false;
		}

		bool bHasTargetPlatform = false;
		if (!LoadFromCompactBinary(ArrayIterator++, bHasTargetPlatform))
		{
			return false;
		}

		if (bHasTargetPlatform)
		{
			FString TargetPlatformName;
			if (!LoadFromCompactBinary(ArrayIterator++, TargetPlatformName))
			{
				return false;
			}

			ITargetPlatformManagerModule* TargetPlatformManager = GetTargetPlatformManager();
			TargetPlatform = TargetPlatformManager->FindTargetPlatform(TargetPlatformName);
			if (!TargetPlatform)
			{
				return false;
			}
		}

		if (!LoadFromCompactBinary(ArrayIterator++, bFallbackToNonPlatformValue))
		{
			return false;
		}

		return true;
	}

	FStringView FConsoleVariableData::GetConsoleVariableName() const
	{
		return VariableName;
	}

	bool FConsoleVariableData::TryResolveValue(FString& Value) const
	{
		IConsoleVariable* VariableInstance = IConsoleManager::Get().FindConsoleVariable(*VariableName, false);
		if (VariableInstance == nullptr)
		{
			return false;
		}

		if (TargetPlatform)
		{
			FName IniPlatformName = *TargetPlatform->IniPlatformName();

			//don't call directly GetPlatformValueVariable because it will create the cvar
			bool bHasPlaftormValue = VariableInstance->HasPlatformValueVariable(IniPlatformName);
			if (bHasPlaftormValue)
			{
				if (const IConsoleVariable* PlatformCVar = VariableInstance->GetPlatformValueVariable(IniPlatformName).Get())
				{
					Value = PlatformCVar->GetString();
				}
				else
				{
					return false;
				}
			}
			else if (bFallbackToNonPlatformValue)
			{
				Value = VariableInstance->GetString();
			}
			else
			{
				return false;
			}
		}
		else
		{
			Value = VariableInstance->GetString();
		}

		return true;
	}

	bool FConsoleVariableData::operator<(const FConsoleVariableData& Other) const
	{
		int32 CompareResult = VariableName.Compare(Other.VariableName);
		if (CompareResult != 0)
		{
			return CompareResult < 0;
		}

		if (TargetPlatform != Other.TargetPlatform)
		{
			if (!TargetPlatform)
			{
				return true;
			}
			else if (!Other.TargetPlatform)
			{
				return false;
			}
			else
			{
				CompareResult = TargetPlatform->PlatformName().Compare(Other.TargetPlatform->PlatformName());
				return CompareResult < 0;
			}
		}

		if (bFallbackToNonPlatformValue != Other.bFallbackToNonPlatformValue)
		{
			return bFallbackToNonPlatformValue;
		}

		return false;
	}

	bool FConsoleVariableData::operator==(const FConsoleVariableData& Other) const
	{
		return VariableName == Other.VariableName && TargetPlatform == Other.TargetPlatform && bFallbackToNonPlatformValue == Other.bFallbackToNonPlatformValue;
	}
}

#endif // #if WITH_EDITOR
