// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/UnrealString.h"
#include "Serialization/CompactBinary.h"

class FCbWriter;
class ITargetPlatform;

namespace UE::Cook
{

/* Describe how a console variable is loaded for FCookDependency. */
class FConsoleVariableData
{
public:
	FConsoleVariableData();
	FConsoleVariableData(const FStringView& InVariableName, const ITargetPlatform* InTargetPlatform, bool bInFallbackToNonPlatformValue);

	void Save(FCbWriter& Writer) const;
	bool TryLoad(FCbFieldView Value);

	FStringView GetConsoleVariableName() const;

	bool TryResolveValue(FString& Value) const;

	bool operator<(const FConsoleVariableData& Other) const;
	bool operator==(const FConsoleVariableData& Other) const;

private:
	FString VariableName;
	const ITargetPlatform* TargetPlatform;		// If non-null, query the platform specific value
	bool bFallbackToNonPlatformValue;			// If querying the platform specific value but it's not found then fallback to the non-platform value.

	friend FCbWriter& operator<<(FCbWriter& Writer, const UE::Cook::FConsoleVariableData& Data)
	{
		Data.Save(Writer);
		return Writer;
	}

	friend bool LoadFromCompactBinary(FCbFieldView Value, UE::Cook::FConsoleVariableData& Data)
	{
		return Data.TryLoad(Value);
	}
};

}

#endif // #if WITH_EDITOR
