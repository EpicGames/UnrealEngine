// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookArtifact.h"
#include "Logging/LogMacros.h"

#if WITH_EDITOR

DEFINE_LOG_CATEGORY_STATIC(LogCookArtifact, Log, All);

namespace UE::Cook::Artifact
{
// ---------------------------------------------------------------------------
// FAppendPackageMetaDataContext
// ---------------------------------------------------------------------------
void FAppendPackageMetaDataContext::SetAttachment(FName Key, FCbObject Value, EFieldStorage InFieldStorage/*= EFieldStorage::Attachment*/)
{
	UE_CLOG(Entries.ContainsByPredicate([&](const FEntry& E) { return E.Key == Key; }), LogCookArtifact, Error, TEXT("Key '%s' already exists in FAppendPackageMetaDataContext"), *Key.ToString());
	FEntry& NewEntry = Entries.AddDefaulted_GetRef();
	NewEntry.Key = Key;
	NewEntry.Value = Value;
	NewEntry.FieldStorage = InFieldStorage;
}

// ---------------------------------------------------------------------------
// FStoreDataInOplogContext
// ---------------------------------------------------------------------------

void FStoreDataInOplogContext::AppendOp(FName OpName, FCbObject Object)
{
	check(!Entries.ContainsByPredicate([&](const FEntry& E) { return E.Key == OpName; }));
	FEntry& NewEntry = Entries.AddDefaulted_GetRef();
	NewEntry.Key = OpName;
	NewEntry.Value = Object;
}

} // namespace UE::Cook::Artifact

#endif // WITH_EDITOR
