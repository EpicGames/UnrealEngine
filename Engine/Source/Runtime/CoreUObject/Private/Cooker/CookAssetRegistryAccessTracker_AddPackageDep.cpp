// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookAssetRegistryAccessTracker_AddPackageDep.h"

namespace UE::CookAssetRegistryAccessTracker
{
	static thread_local bool bAddPackageDependenciesEnabled = false;

	FAddPackageDependenciesScope::FAddPackageDependenciesScope()
		: bPreviousValue(bAddPackageDependenciesEnabled)
	{
		bAddPackageDependenciesEnabled = true;
	}

	FAddPackageDependenciesScope::~FAddPackageDependenciesScope()
	{
		bAddPackageDependenciesEnabled = bPreviousValue;
	}

	bool FAddPackageDependenciesScope::GetAddPackageDependenciesEnabled()
	{
		return bAddPackageDependenciesEnabled;
	}
}
