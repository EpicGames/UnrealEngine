// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookAssetRegistryAccessTracker_IgnoreScope.h"

namespace UE::CookAssetRegistryAccessTracker
{
	static thread_local bool bIgnoreAccessTracker = false;

	FIgnoreScope::FIgnoreScope()
		: bPreviousValue(bIgnoreAccessTracker)
	{
		bIgnoreAccessTracker = true;
	}

	FIgnoreScope::~FIgnoreScope()
	{
		bIgnoreAccessTracker = bPreviousValue;
	}

	bool FIgnoreScope::ShouldIgnoreAccessTracker()
	{
		return bIgnoreAccessTracker;
	}
}
