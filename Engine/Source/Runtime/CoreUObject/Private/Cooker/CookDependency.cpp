// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookDependency.h"

#if WITH_EDITOR
#include "Algo/Unique.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Cooker/ConsoleVariableData.h"
#include "Cooker/CookDependencyContext.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMath.h"
#include "Hash/Blake3.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/ConfigAccessData.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/Casts.h"
#include "Templates/UniquePtr.h"
#include "UObject/Class.h"
#endif

#if WITH_EDITOR

namespace UE::Cook::Dependency::Private
{

/**
 * Return the map from Name to function created by UE_COOK_DEPENDENCY_FUNCTION macros.
 * Initialized when first called and afterwards immutable
 */
const TMap<FName, FCookDependencyFunction>& GetCookDependencyFunctions();

} // namespace UE::Cook::Dependency::Private

namespace UE::Cook
{

const TCHAR* LexToString(ECookDependency CookDependency)
{
	switch (CookDependency)
	{
	case ECookDependency::None: return TEXT("None");
	case ECookDependency::File: return TEXT("File");
	case ECookDependency::ConsoleVariable: return TEXT("ConsoleVariable");
	case ECookDependency::NativeClass: return TEXT("NativeClass");
	case ECookDependency::Function: return TEXT("Function");
	case ECookDependency::TransitiveBuild: return TEXT("TransitiveBuild");
	case ECookDependency::Package: return TEXT("Package");
	case ECookDependency::Config: return TEXT("Config");
	case ECookDependency::SettingsObject: return TEXT("SettingsObject");
	case ECookDependency::AssetRegistryQuery: return TEXT("AssetRegistryQuery");
	case ECookDependency::RedirectionTarget: return TEXT("RedirectionTarget");
	default: return TEXT("Invalid");
	}
}

namespace ResultProjection
{

FName All("UE_Cook_ResultProjection_All");
FName PackageAndClass("UE_Cook_ResultProjection_PackageAndClass");
FName None("UE_Cook_ResultProjection_None");

}

namespace BuildResult
{

FName NAME_Save("Save");
FName NAME_Load("Load");

}

FCookDependency FCookDependency::File(FStringView InFileName)
{
	FCookDependency Result(ECookDependency::File);
	Result.StringData = InFileName;
	return Result;
}

FCookDependency FCookDependency::Function(FName InFunctionName, FCbFieldIterator&& InArgs)
{
	FCookDependency Result(ECookDependency::Function);
	Result.FunctionData.Name = InFunctionName;
	Result.FunctionData.Args = MoveTemp(InArgs);
	Result.FunctionData.Args.MakeRangeOwned();
	return Result;
}

FCookDependency FCookDependency::TransitiveBuild(FName PackageName)
{
	FCookDependency Result(ECookDependency::TransitiveBuild);
	Result.TransitiveBuildData.PackageName = PackageName;
	return Result;
}

FCookDependency FCookDependency::TransitiveBuildAndRuntime(FName PackageName)
{
	FCookDependency Result(ECookDependency::TransitiveBuild);
	Result.TransitiveBuildData.PackageName = PackageName;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Result.TransitiveBuildData.bAlsoAddRuntimeDependency = true;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	return Result;
}

FCookDependency FCookDependency::Package(FName PackageName)
{
	FCookDependency Result(ECookDependency::Package);
	Result.NameData = PackageName;
	return Result;
}

FCookDependency FCookDependency::ConsoleVariable(const FStringView& VariableName)
{
	return FCookDependency::ConsoleVariable(VariableName, nullptr, false);
}

FCookDependency FCookDependency::ConsoleVariable(const FStringView& VariableName, const ITargetPlatform* TargetPlatform, bool bFallbackToNonPlatform)
{
	FCookDependency Result(ECookDependency::ConsoleVariable);
	Result.ConsoleVariableData = MakeUnique<FConsoleVariableData>(VariableName, TargetPlatform, bFallbackToNonPlatform);
	return Result;
}

FCookDependency FCookDependency::Config(UE::ConfigAccessTracking::FConfigAccessData AccessData)
{
	FCookDependency Result(ECookDependency::Config);
	Result.ConfigAccessData.Reset(new UE::ConfigAccessTracking::FConfigAccessData(MoveTemp(AccessData)));
	return Result;
}

FCookDependency FCookDependency::Config(UE::ConfigAccessTracking::ELoadType LoadType, FName Platform,
	FName FileName, FName SectionName, FName ValueName)
{
	FCookDependency Result(ECookDependency::Config);
	Result.ConfigAccessData.Reset(new UE::ConfigAccessTracking::FConfigAccessData(
		LoadType, Platform, FileName, SectionName, ValueName, nullptr /* RequestPlatform */));
	return Result;
}

FCookDependency FCookDependency::Config(FName FileName, FName SectionName, FName ValueName)
{
	return Config(UE::ConfigAccessTracking::ELoadType::ConfigSystem, NAME_None,
		FileName, SectionName, ValueName);
}

FCookDependency FCookDependency::SettingsObject(const UObject* InObject)
{
	FCookDependency Result(ECookDependency::SettingsObject);
	if (InObject)
	{
		const UClass* Class = Cast<const UClass>(InObject);
		if (Class)
		{
			InObject = Class->GetDefaultObject();
		}
		else
		{
			Class = InObject->GetClass();
		}

		if (!InObject->IsRooted())
		{
			UE_LOGF(LogCore, Error, "Invalid FCookDependency::SettingsObject(%ls). The object is not in the root set and may be garbage collected. " "FCookDependency keeps a raw pointer to SettingsObjects and does not support pointers to objects that are not in the root set. " "The dependency will be ignored.",
				*InObject->GetPathName());
			InObject = nullptr;
		}
		else
		{
			if (!Class->HasAnyClassFlags(CLASS_Config | CLASS_PerObjectConfig))
			{
				UE_LOGF(LogCore, Error, "Invalid FCookDependency::SettingsObject(%ls). The object's class %ls is not a config class. CookDependency::SettingsObject only supports config classes. " "The dependency will be ignored.",
					*InObject->GetPathName(), *Class->GetPathName());
				InObject = nullptr;
			}
			else if (!Class->HasAnyClassFlags(CLASS_PerObjectConfig) && InObject != Class->GetDefaultObject())
			{
				UE_LOGF(LogCore, Error, "Invalid FCookDependency::SettingsObject(%ls). The object is not the ClassDefaultObject and its class %ls is not a per-object-config class. " "CookDependency::SettingsObject only supports the CDO or per-object-config objects. " "The dependency will be ignored.",
					*InObject->GetPathName(), *Class->GetPathName());
				InObject = nullptr;
			}
		}
	}
	Result.ObjectPtr = InObject;
	return Result;
}

FCookDependency FCookDependency::NativeClass(const UClass* InClass)
{
	if (InClass)
	{
		while (!InClass->IsNative())
		{
			InClass = InClass->GetSuperClass();
			check(InClass); // Every class other than UObject has a super class, and UObject is native.
		}
		return NativeClass(InClass->GetPathName());
	}
	else
	{
		return NativeClass(FStringView());
	}
}

FCookDependency FCookDependency::NativeClass(FStringView ClassPath)
{
	FCookDependency Result(ECookDependency::NativeClass);
	Result.StringData = ClassPath;
	return Result;
}

FCookDependency FCookDependency::AssetRegistryQuery(FARFilter Filter)
{
	FCookDependency Result(ECookDependency::AssetRegistryQuery);
	Filter.SortForSaving();
	if (!Filter.bIncludeOnlyOnDiskAssets)
	{
		UE_LOGF(LogCore, Warning,
			"An AssetRegistryQuery dependency has an ARFilter with bIncludeOnlyOnDiskAssets=false. " "This will create cookindeterminism since the packages in can vary between cooks. Caller should set bIncludeOnlyOnDiskAssets=true. " "For the recorded dependency we are forcing bIncludeOnlyOnDiskAssets=true. Callstack of the AssetRegistryQuery construction: "
		);
		FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
		Filter.bIncludeOnlyOnDiskAssets = true;
	}

	Result.ARFilter = MakeUnique<FARFilter>(MoveTemp(Filter));
	return Result;
}

FCookDependency FCookDependency::RedirectionTarget(FName PackageName)
{
	FCookDependency Result(ECookDependency::RedirectionTarget);
	Result.NameData = PackageName;
	return Result;
}

FCookDependency::FCookDependency()
	: Type(ECookDependency::None)
	, HashedValue()
{}

FCookDependency::FCookDependency(ECookDependency InType)
	: Type(InType)
	, HashedValue()
{
	Construct();
}

FCookDependency::~FCookDependency()
{
	Destruct();
}

FCookDependency::FCookDependency(const FCookDependency& Other)
	: Type(ECookDependency::None)
{
	*this = Other;
}

FCookDependency::FCookDependency(FCookDependency&& Other)
	: Type(ECookDependency::None)
{
	*this = MoveTemp(Other);
}

FCookDependency& FCookDependency::operator=(const FCookDependency& Other)
{
	Destruct();
	Type = Other.Type;
	HashedValue = Other.HashedValue;
	Construct();
	switch (Type)
	{
	case ECookDependency::None:
		break;
	case ECookDependency::File:
	case ECookDependency::NativeClass:
		StringData = Other.StringData;
		break;
	case ECookDependency::ConsoleVariable:
		ConsoleVariableData.Reset(new FConsoleVariableData(*Other.ConsoleVariableData));
		break;
	case ECookDependency::Function:
		FunctionData = Other.FunctionData;
		break;
	case ECookDependency::TransitiveBuild:
		TransitiveBuildData = Other.TransitiveBuildData;
		break;
	case ECookDependency::Package:
		NameData = Other.NameData;
		break;
	case ECookDependency::Config:
		ConfigAccessData.Reset(Other.ConfigAccessData.IsValid() ?
			new UE::ConfigAccessTracking::FConfigAccessData(*Other.ConfigAccessData) :
			nullptr);
		break;
	case ECookDependency::SettingsObject:
		ObjectPtr = Other.ObjectPtr;
		break;
	case ECookDependency::AssetRegistryQuery:
		ARFilter.Reset(Other.ARFilter.IsValid() ? new FARFilter(*Other.ARFilter) : nullptr);
		break;
	case ECookDependency::RedirectionTarget:
		NameData = Other.NameData;
		break;
	default:
		checkNoEntry();
		break;
	}
	return *this;
}

FCookDependency& FCookDependency::operator=(FCookDependency&& Other)
{
	Destruct();
	Type = Other.Type;
	HashedValue = Other.HashedValue;
	Construct();
	switch (Type)
	{
	case ECookDependency::None:
		break;
	case ECookDependency::File:
	case ECookDependency::NativeClass:
		StringData = MoveTemp(Other.StringData);
		break;
	case ECookDependency::ConsoleVariable:
		ConsoleVariableData = MoveTemp(Other.ConsoleVariableData);
		break;
	case ECookDependency::Function:
		FunctionData = MoveTemp(Other.FunctionData);
		break;
	case ECookDependency::TransitiveBuild:
		TransitiveBuildData = MoveTemp(Other.TransitiveBuildData);
		break;
	case ECookDependency::Package:
		NameData = MoveTemp(Other.NameData);
		break;
	case ECookDependency::Config:
		ConfigAccessData = MoveTemp(Other.ConfigAccessData);
		break;
	case ECookDependency::SettingsObject:
		ObjectPtr = Other.ObjectPtr;
		break;
	case ECookDependency::AssetRegistryQuery:
		ARFilter = MoveTemp(Other.ARFilter);
		break;
	case ECookDependency::RedirectionTarget:
		NameData = MoveTemp(Other.NameData);
		break;
	default:
		checkNoEntry();
		break;
	}
	return *this;
}

FString FCookDependency::GetDebugIdentifier() const
{
	switch (Type)
	{
	case ECookDependency::None:
		return FString();
	case ECookDependency::File:
	case ECookDependency::NativeClass:
		return StringData;
	case ECookDependency::ConsoleVariable:
		return ConsoleVariableData.IsValid() ? FString(ConsoleVariableData->GetConsoleVariableName()) : FString();
	case ECookDependency::Function:
		return FunctionData.Name.ToString();
	case ECookDependency::TransitiveBuild:
		return TransitiveBuildData.PackageName.ToString();
	case ECookDependency::Package:
		return NameData.ToString();
	case ECookDependency::Config:
		return ConfigAccessData.IsValid() ? ConfigAccessData->FullPathToString() : FString();
	case ECookDependency::SettingsObject:
		return ObjectPtr ? ObjectPtr->GetPathName() : FString();
	case ECookDependency::AssetRegistryQuery:
		return FString();
	case ECookDependency::RedirectionTarget:
		return NameData.ToString();
	default:
		checkNoEntry();
		return FString();
	}
}

UE::ConfigAccessTracking::FConfigAccessData FCookDependency::GetConfigAccessData() const
{
	return Type == ECookDependency::Config && ConfigAccessData.IsValid() ?
		*ConfigAccessData : UE::ConfigAccessTracking::FConfigAccessData();
}

FString FCookDependency::GetConfigPath() const
{
	return Type == ECookDependency::Config && ConfigAccessData.IsValid() ?
		ConfigAccessData->FullPathToString() : FString();
}

void FCookDependency::UpdateHash(FCookDependencyContext& Context, FBlake3Hash* OutCurrentValue) const
{
	switch (GetType())
	{
	case ECookDependency::None:
		// Nothing to add, Nones are never invalidated
		if (OutCurrentValue)
		{
			*OutCurrentValue = FBlake3Hash();
		}
		return;
	case ECookDependency::File:
	{
		const TCHAR* LocalFilename = GetFileName().GetData(); // contract for GetFileName is a null-terminated TCHAR*.
		LocalFilename = LocalFilename ? LocalFilename : TEXT("");
		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(LocalFilename, FILEREAD_Silent));
		if (!Ar)
		{
			Context.LogError(FString::Printf(
				TEXT("FCookDependency::File('%s') failed to UpdateHash: could not read file."),
				LocalFilename));
			if (OutCurrentValue)
			{
				*OutCurrentValue = FBlake3Hash();
			}
			return;
		}
		TArray64<uint8> Buffer;
		Buffer.SetNumUninitialized(64 * 1024);
		const int64 Size = Ar->TotalSize();
		int64 Position = 0;
		// Read in BufferSize chunks

		FBlake3 Hasher;
		while (Position < Size)
		{
			const int64 ReadNum = FMath::Min(Size - Position, (int64)Buffer.Num());
			Ar->Serialize(Buffer.GetData(), static_cast<uint64>(ReadNum));
			Hasher.Update(Buffer.GetData(), ReadNum);
			Position += ReadNum;
		}

		FBlake3Hash Hash = Hasher.Finalize();
		if (OutCurrentValue)
		{
			*OutCurrentValue = Hash;
		}
		Context.Update(&Hash.GetBytes(), sizeof(Hash.GetBytes()));

		break;
	}
	case ECookDependency::Function:
	{
		const TMap<FName, FCookDependencyFunction>& CookDependencyFunctions =
			UE::Cook::Dependency::Private::GetCookDependencyFunctions();
		const FCookDependencyFunction* Function = CookDependencyFunctions.Find(GetFunctionName());
		if (!Function)
		{
			Context.LogError(FString::Printf(
				TEXT("FCookDependency::Function('%s') failed to UpdateHash: Function not found."),
				*GetFunctionName().ToString()));
			if (OutCurrentValue)
			{
				*OutCurrentValue = FBlake3Hash();
			}
			return;
		}

		FCookDependencyContext::FErrorHandlerScope Scope = Context.ErrorHandlerScope([this](FString&& Message)
			{
				return FString::Printf(TEXT("FCookDependency::Function('%s') failed to UpdateHash: %s"),
				*GetFunctionName().ToString(), *Message);
			});

		FBlake3 NewHasher;
		void* OldHasher = Context.SetHasher(&NewHasher);
		(*Function)(GetFunctionArgs(), Context);
		
		Context.SetHasher(OldHasher);

		FBlake3Hash Hash = NewHasher.Finalize();
		if (OutCurrentValue)
		{
			*OutCurrentValue = Hash;
		}
		Context.Update(&Hash.GetBytes(), sizeof(Hash.GetBytes()));

		return;
	}
	case ECookDependency::TransitiveBuild:
		// Build dependencies do not impact the hash; they instead operate by marking the package
		// as invalidated based on the invalidation of other packages, in a separate pass after its hash is compared
		if (OutCurrentValue)
		{
			*OutCurrentValue = FBlake3Hash();
		}
		return;
	case ECookDependency::Package:
		Context.LogError(FString::Printf(
			TEXT("FCookDependency::Package('%s') failed to UpdateHash: Package dependencies do not implement UpdateHash and it should not be called on them."),
			*NameData.ToString()));
		if (OutCurrentValue)
		{
			*OutCurrentValue = FBlake3Hash();
		}
		return;
	case ECookDependency::ConsoleVariable:
	{
		if (!ConsoleVariableData.IsValid())
		{
			return;
		}

		FString Value;
		bool bVariableFound = ConsoleVariableData->TryResolveValue(Value);

		if (!bVariableFound)
		{
			FStringView ConsoleVariableName = ConsoleVariableData->GetConsoleVariableName();
			Context.LogError(FString::Printf(
				TEXT("FCookDependency::ConsoleVariable('%.*s') failed to UpdateHash: could not find console variable."),
				ConsoleVariableName.Len(), ConsoleVariableName.GetData()));

			if (OutCurrentValue)
			{
				*OutCurrentValue = FBlake3Hash();
			}
			return;
		}

		FBlake3Hash Hash = ConvertToHash(FUtf8String(Value));
		if (OutCurrentValue)
		{
			*OutCurrentValue = Hash;
		}
		Context.Update(&Hash.GetBytes(), sizeof(Hash.GetBytes()));

		return;
	}
	case ECookDependency::Config:
		Context.LogError(FString::Printf(
			TEXT("FCookDependency::Config('%s') failed to UpdateHash: Config dependencies do not implement UpdateHash and it should not be called on them."),
			*GetConfigPath()));
		if (OutCurrentValue)
		{
			*OutCurrentValue = FBlake3Hash();
		}
		return;
	case ECookDependency::SettingsObject:
		Context.LogError(FString::Printf(
			TEXT("FCookDependency::SettingsObject('%s') failed to UpdateHash: SettingsObject dependencies do not implement UpdateHash and it should not be called on them."),
			ObjectPtr ? *ObjectPtr->GetPathName() : TEXT("<null>")));
		if (OutCurrentValue)
		{
			*OutCurrentValue = FBlake3Hash();
		}
		return;
	case ECookDependency::NativeClass:
		Context.LogError(FString::Printf(
			TEXT("FCookDependency::NativeClass('%s') failed to UpdateHash: NativeClass dependencies do not implement UpdateHash and it should not be called on them."),
			*StringData));
		if (OutCurrentValue)
		{
			*OutCurrentValue = FBlake3Hash();
		}
		return;
	case ECookDependency::AssetRegistryQuery:
		if (ARFilter.IsValid())
		{
			IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr();
			if (AssetRegistry)
			{
				TArray<FName> PackageNames;
				if (!ARFilter->bIncludeOnlyOnDiskAssets)
				{
					UE_LOGF(LogCore, Error,
						"An AssetRegistryQuery dependency has an ARFilter with bIncludeOnlyOnDiskAssets=false. " "This is supposed to be prevented on construction of FCookDependency::AssetRegistryQuery.");
					ARFilter->bIncludeOnlyOnDiskAssets = true;
				}
				AssetRegistry->EnumerateAssets(*ARFilter, [&Context, &PackageNames](const FAssetData& AssetData)
					{
						PackageNames.Add(AssetData.PackageName);
						return true;
					}, UE::AssetRegistry::EEnumerateAssetsFlags::None);
				PackageNames.Sort(FNameLexicalLess());
				PackageNames.SetNum(Algo::Unique(PackageNames));
				TStringBuilder<256> PackageNameStr;

				FBlake3 Hasher;
				for (FName PackageName : PackageNames)
				{
					PackageNameStr.Reset();
					PackageNameStr << PackageName;
					Hasher.Update(*PackageNameStr, sizeof(**PackageNameStr) * PackageNameStr.Len());
				}

				FBlake3Hash Hash = Hasher.Finalize();
				if (OutCurrentValue)
				{
					*OutCurrentValue = Hash;
				}
				Context.Update(&Hash.GetBytes(), sizeof(Hash.GetBytes()));
			}
		}
		return;
	case ECookDependency::RedirectionTarget:
		Context.LogError(FString::Printf(
			TEXT("FCookDependency::RedirectionTarget('%s') failed to UpdateHash: RedirectionTarget dependencies do not implement UpdateHash and it should not be called on them."),
			*NameData.ToString()));
		if (OutCurrentValue)
		{
			*OutCurrentValue = FBlake3Hash();
		}
		return;
	default:
		checkNoEntry();
		return;
	}
}

void FCookDependency::SetValue(const FBlake3Hash& Hash)
{
	static_assert(sizeof(Hash) == ValueSizeInBytes);
	HashedValue = Hash;
}

FBlake3Hash FCookDependency::ConvertToHash(const FIoHash& Hash)
{
	FBlake3Hash OutHash;
	constexpr size_t OutSize = sizeof(OutHash.GetBytes()); // -V568
	static_assert(sizeof(Hash) <= OutSize);
	FMemory::Memcpy(&OutHash.GetBytes(), &Hash, sizeof(Hash));

	if constexpr (sizeof(Hash) < OutSize)
	{
		FMemory::Memzero(((uint8*)&OutHash.GetBytes()) + sizeof(Hash), OutSize - sizeof(Hash));
	}
	return OutHash;
}

FBlake3Hash FCookDependency::ConvertToHash(const FUtf8String& String)
{
	FBlake3Hash OutHash;
	constexpr size_t OutSize = sizeof(OutHash.GetBytes()); // -V568

	SIZE_T StringNumBytes = String.NumBytesWithoutNull();
	if (StringNumBytes > OutSize)
	{
		FUtf8StringView View(String);

		FBlake3 Hasher;
		OutHash = Hasher.HashBuffer(View.GetData(), View.NumBytes());
	}
	else
	{
		FMemory::Memcpy(&OutHash.GetBytes(), String.GetCharArray().GetData(), StringNumBytes);

		if (StringNumBytes < OutSize)
		{
			FMemory::Memzero(((uint8*) &OutHash.GetBytes()) + StringNumBytes, OutSize - StringNumBytes);
		}
	}
	return OutHash;
}

void FCookDependency::Construct()
{
	switch (Type)
	{
	case ECookDependency::None:
		break;
	case ECookDependency::File:
	case ECookDependency::NativeClass:
		new(&StringData) FString();
		break;
	case ECookDependency::ConsoleVariable:
		// Set the union's bytes equal to a TUniquePtr with a null internal pointer
		new(&ConsoleVariableData) TUniquePtr<FConsoleVariableData>(nullptr);
		break;
	case ECookDependency::Function:
		new(&FunctionData) FFunctionData();
		break;
	case ECookDependency::TransitiveBuild:
		new(&TransitiveBuildData) FTransitiveBuildData();
		break;
	case ECookDependency::Package:
		new(&NameData) FName();
		break;
	case ECookDependency::Config:
		// Set the union's bytes equal to a TUniquePtr with a null internal pointer
		new(&ConfigAccessData) TUniquePtr<UE::ConfigAccessTracking::FConfigAccessData>(nullptr);
		break;
	case ECookDependency::SettingsObject:
		ObjectPtr = nullptr;
		break;
	case ECookDependency::AssetRegistryQuery:
		// Set the union's bytes equal to a TUniquePtr with a null internal pointer
		new(&ARFilter) TUniquePtr<FARFilter>(nullptr);
		break;
	case ECookDependency::RedirectionTarget:
		new(&NameData) FName();
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FCookDependency::Destruct()
{
	switch (Type)
	{
	case ECookDependency::None:
		break;
	case ECookDependency::File:
	case ECookDependency::NativeClass:
		StringData.~FString();
		break;
	case ECookDependency::ConsoleVariable:
		ConsoleVariableData.~TUniquePtr<FConsoleVariableData>();
		break;
	case ECookDependency::Function:
		FunctionData.~FFunctionData();
		break;
	case ECookDependency::TransitiveBuild:
		TransitiveBuildData.~FTransitiveBuildData();
		break;
	case ECookDependency::Package:
		NameData.~FName();
		break;
	case ECookDependency::Config:
		// TUniquePtr's destructor will also destruct its internal pointer
		ConfigAccessData.~TUniquePtr<UE::ConfigAccessTracking::FConfigAccessData>();
		break;
	case ECookDependency::SettingsObject:
		ObjectPtr = nullptr;
		break;
	case ECookDependency::AssetRegistryQuery:
		// TUniquePtr's destructor will also destruct its internal pointer
		ARFilter.~TUniquePtr<FARFilter>();
		break;
	case ECookDependency::RedirectionTarget:
		NameData.~FName();
		break;
	default:
		checkNoEntry();
		break;
	}
}

void FCookDependency::Save(FCbWriter& Writer) const
{
	Writer.BeginArray();
	Writer << static_cast<uint8>(Type);
	switch (Type)
	{
	case ECookDependency::None:
		break;
	case ECookDependency::File:
	case ECookDependency::NativeClass:
		Writer << StringData;
		break;
	case ECookDependency::ConsoleVariable:
	{
		bool bIsValid = ConsoleVariableData.IsValid();
		Writer << bIsValid;

		if (bIsValid)
		{
			Writer << *ConsoleVariableData;
		}
		break;
	}
	case ECookDependency::Function:
		Writer << FunctionData.Name;
		Writer.BeginArray();
		for (FCbFieldViewIterator Iter(FunctionData.Args); Iter; ++Iter)
		{
			Writer << Iter;
		}
		Writer.EndArray();
		break;
	case ECookDependency::TransitiveBuild:
		Writer << TransitiveBuildData.PackageName;
		break;
	case ECookDependency::Package:
		Writer << NameData;
		break;
	case ECookDependency::Config:
		Writer << GetConfigPath();
		break;
	case ECookDependency::SettingsObject:
		// Settings objects are not persistable; save out an empty SettingsObject dependency
		break;
	case ECookDependency::AssetRegistryQuery:
	{
		bool bValid = ARFilter.IsValid();
		Writer << bValid;
		if (bValid)
		{
			Writer << *ARFilter;
		}
		break;
	}
	case ECookDependency::RedirectionTarget:
		Writer << NameData;
		break;
	default:
		checkNoEntry();
		break;
	}

	Writer.AddBinary(FMemoryView(RawValue, ValueSizeInBytes));
	
	Writer.EndArray();
}

bool FCookDependency::Load(FCbFieldView Value)
{
	*this = FCookDependency(ECookDependency::None);
	FCbArrayView ArrayView = Value.AsArrayView();
	if (ArrayView.Num() < 1)
	{
		return false;
	}
	FCbFieldViewIterator Field(Value.CreateViewIterator());
	int32 LocalTypeAsInt = Field.AsUInt8();
	if ((Field++).HasError() || LocalTypeAsInt >= static_cast<uint8>(ECookDependency::Count))
	{
		return false;
	}
	ECookDependency LocalType = static_cast<ECookDependency>(LocalTypeAsInt);

	switch (LocalType)
	{
	case ECookDependency::None:
		break;

	case ECookDependency::File:
	{
		FUtf8StringView LocalFilename = Field.AsString();
		if ((Field++).HasError())
		{
			return false;
		}
		*this = FCookDependency::File(WriteToString<256>(LocalFilename));
	}
	break;

	case ECookDependency::Function:
	{
		FName LocalFuncName;
		if (!LoadFromCompactBinary(Field++, LocalFuncName))
		{
			return false;
		}

		if (Field.IsArray())
		{
			FCbFieldViewIterator FunctionArgumentsIterator = Field.CreateViewIterator();
			*this = FCookDependency::Function(LocalFuncName, FCbFieldIterator::CloneRange(FunctionArgumentsIterator));
			FunctionData.Args.MakeRangeOwned();
			++Field;
		}
		else // If the arguments are not in an array, then this is the old format and all the fields until the end are function parameters
		{
			*this = FCookDependency::Function(LocalFuncName, FCbFieldIterator::CloneRange(Field));
			FunctionData.Args.MakeRangeOwned();
			Field.Reset(); // In the old format, all the fields are arguments of the function so there is nothing to read after.
		}
	}
	break;

	case ECookDependency::TransitiveBuild:
	{
		FName LocalPackageName;
		if (!LoadFromCompactBinary(Field++, LocalPackageName))
		{
			return false;
		}
		*this = FCookDependency::TransitiveBuild(LocalPackageName);
	}
	break;

	case ECookDependency::Package:
	{
		FName LocalPackageName;
		if (!LoadFromCompactBinary(Field++, LocalPackageName))
		{
			return false;
		}
		*this = FCookDependency::Package(LocalPackageName);
	}
	break;

	case ECookDependency::ConsoleVariable:
	{
		bool bIsValid = false;
		if (!LoadFromCompactBinary(Field++, bIsValid))
		{
			return false;
		}

		if (!bIsValid)
		{
			return false;
		}

		TUniquePtr<FConsoleVariableData> LocalConsoleVariableData = MakeUnique<FConsoleVariableData>();
		if (!LoadFromCompactBinary(Field++, *LocalConsoleVariableData))
		{
			return false;
		}

		*this = FCookDependency(ECookDependency::ConsoleVariable);
		ConsoleVariableData = MoveTemp(LocalConsoleVariableData);
	}
	break;

	case ECookDependency::Config:
	{
		FString LocalConfigPath;
		if (!LoadFromCompactBinary(Field++, LocalConfigPath))
		{
			return false;
		}
		if (LocalConfigPath.IsEmpty())
		{
			*this = FCookDependency(ECookDependency::Config);
		}
		else
		{
			*this = FCookDependency::Config(UE::ConfigAccessTracking::FConfigAccessData::Parse(LocalConfigPath));
		}
	}
	break;

	case ECookDependency::SettingsObject:
	{
		// Settings objects are not persistable; construct an empty SettingsObject dependency
		*this = FCookDependency::SettingsObject(nullptr);
	}
	break;

	case ECookDependency::NativeClass:
	{
		FString LocalClassPath;
		if (!LoadFromCompactBinary(Field++, LocalClassPath))
		{
			return false;
		}
		*this = FCookDependency::NativeClass(LocalClassPath);
	}
	break;

	case ECookDependency::AssetRegistryQuery:
	{
		bool bValid;
		if (!LoadFromCompactBinary(Field++, bValid))
		{
			return false;
		}
		if (!bValid)
		{
			*this = FCookDependency(ECookDependency::AssetRegistryQuery);
		}
		else
		{
			FARFilter LocalARFilter;
			if (!LoadFromCompactBinary(Field++, LocalARFilter))
			{
				return false;
			}
			*this = FCookDependency::AssetRegistryQuery(MoveTemp(LocalARFilter));
		}
	}
	break;

	case ECookDependency::RedirectionTarget:
	{
		FName LocalPackageName;
		if (!LoadFromCompactBinary(Field++, LocalPackageName))
		{
			return false;
		}
		*this = FCookDependency::RedirectionTarget(LocalPackageName);
	}
	break;

	default:
		break;
	}

	FMemoryView View = Field.AsBinaryView();
	if (View.GetSize() != ValueSizeInBytes)
	{
		return false;
	}

	FMemory::Memcpy(RawValue, View.GetData(), ValueSizeInBytes);
	
	return true;
}

bool FCookDependency::ConfigAccessDataLessThan(const UE::ConfigAccessTracking::FConfigAccessData& A,
	const UE::ConfigAccessTracking::FConfigAccessData& B)
{
	return A < B;
}

bool FCookDependency::ConfigAccessDataEqual(const UE::ConfigAccessTracking::FConfigAccessData& A,
	const UE::ConfigAccessTracking::FConfigAccessData& B)
{
	return A == B;
}

bool FCookDependency::ARFilterLessThan(const FARFilter& A, const FARFilter& B)
{
	return A < B;
}

bool FCookDependency::ARFilterEqual(const FARFilter& A, const FARFilter& B)
{
	return A == B;
}

bool FCookDependency::ConsoleVariableDataLessThan(const FConsoleVariableData& A, const FConsoleVariableData& B)
{
	return A < B;
}

bool FCookDependency::ConsoleVariableDataEqual(const FConsoleVariableData& A, const FConsoleVariableData& B)
{
	return A == B;
}

} // namespace UE::Cook

namespace UE::Cook::Dependency::Private
{

FCookDependencyFunctionRegistration* List = nullptr;
bool bGCookDependencyFunctionsInitialized = false;
TMap<FName, FCookDependencyFunction> GCookDependencyFunctions;

const TMap<FName, FCookDependencyFunction>& GetCookDependencyFunctions()
{
	if (!bGCookDependencyFunctionsInitialized)
	{
		GCookDependencyFunctions.Empty();
		FCookDependencyFunctionRegistration* Reg = UE::Cook::Dependency::Private::List;
		while (Reg)
		{
			FCookDependencyFunction& ExistingFunction = GCookDependencyFunctions.FindOrAdd(Reg->GetFName(), nullptr);
			checkf(ExistingFunction == nullptr || ExistingFunction == Reg->Function,
				TEXT("UE_COOK_DEPENDENCY_FUNCTION name '%s' is duplicated. UE_COOK_DEPENDENCY_FUNCTION names must be unique."),
				*Reg->GetFName().ToString());
			ExistingFunction = Reg->Function;
			Reg = Reg->Next;
		}
		bGCookDependencyFunctionsInitialized = true;
	}
	return GCookDependencyFunctions;
}

void FCookDependencyFunctionRegistration::Construct()
{
	using namespace UE::Cook::Dependency::Private;

	Next = UE::Cook::Dependency::Private::List;
	UE::Cook::Dependency::Private::List = this;
	bGCookDependencyFunctionsInitialized = false;
}

FCookDependencyFunctionRegistration::~FCookDependencyFunctionRegistration()
{
	using namespace UE::Cook::Dependency::Private;

	// Remove this from the list when destructed, but for better shutdown performance skip this cost during engine
	// exit, and just leave the list with dangling pointers, since the list can only be read by this destructor or by
	// GetCookDependencyFunctions, which is not called during shutdown.
	if (!IsEngineExitRequested())
	{
		if (UE::Cook::Dependency::Private::List == this)
		{
			UE::Cook::Dependency::Private::List = this->Next;
		}
		else
		{
			FCookDependencyFunctionRegistration* Current = UE::Cook::Dependency::Private::List;
			while (Current)
			{
				if (Current->Next == this)
				{
					Current->Next = this->Next;
					break;
				}
				Current = Current->Next;
			}
		}
		bGCookDependencyFunctionsInitialized = false;
	}
}

} // namespace UE::Cook::Dependency::Private

#endif // WITH_EDITOR