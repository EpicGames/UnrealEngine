// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookDependencyContext.h"

#include "Hash/Blake3.h"

#if WITH_EDITOR

namespace UE::Cook
{
FCookDependencyContext::FCookDependencyContext(void* InHasher,
	TUniqueFunction<void(ELogVerbosity::Type, FString&&)>&& InOnLog,
	TUniqueFunction<void(ELogVerbosity::Type)>&& InOnInvalidated,
	FName InPackageName)
	: PackageName(InPackageName)
	, Hasher(InHasher)
{
	OnLog = ([ParamOnLog = MoveTemp(InOnLog), ParamOnInvalidated = MoveTemp(InOnInvalidated)](ELogVerbosity::Type Type, FString&& Message, bool bInvalidated)
		{ 
			ParamOnLog(Type, MoveTemp(Message));
			if (bInvalidated)
			{
				ParamOnInvalidated(Type);
			}
		});
}

FCookDependencyContext::FCookDependencyContext(void* InHasher,
	TUniqueFunction<void(ELogVerbosity::Type, FString&&, bool)>&& InOnLog,
	FName InPackageName)
	: OnLog(MoveTemp(InOnLog))
	, PackageName(InPackageName)
	, Hasher(InHasher)
{ }

void FCookDependencyContext::LogError(FString Message)
{
	Log(ELogVerbosity::Error, MoveTemp(Message), /*bInvalidated*/ true);
}

void FCookDependencyContext::LogInvalidated(FString Message)
{
	Log(ELogVerbosity::Display, MoveTemp(Message), /*bInvalidated*/ true);
}

void FCookDependencyContext::Update(const void* Data, uint64 Size)
{
	FBlake3& Blake3Hasher = *(reinterpret_cast<FBlake3*>(Hasher));
	Blake3Hasher.Update(Data, Size);
}

void FCookDependencyContext::ReportInvalidated()
{
	Log(ELogVerbosity::Display, "", /*bInvalidated*/ true);
}

void FCookDependencyContext::ReportError()
{
	Log(ELogVerbosity::Error, "", /*bInvalidated*/ true);
}

void FCookDependencyContext::Log(ELogVerbosity::Type Verbosity, FString Message, bool bInvalidated)
{
	for (TUniqueFunction<FString(FString&&)>& Handler : ErrorHandlers)
	{
		Message = Handler(MoveTemp(Message));
	}
	OnLog(Verbosity, MoveTemp(Message), bInvalidated);
}

void* FCookDependencyContext::SetHasher(void* NewHasher)
{
	void* Temp = Hasher;
	Hasher = NewHasher;
	return Temp;
}

FCookDependencyContext::FErrorHandlerScope::FErrorHandlerScope(FCookDependencyContext& InContext)
	: Context(InContext)
{ }

FCookDependencyContext::FErrorHandlerScope::~FErrorHandlerScope()
{
	check(!Context.ErrorHandlers.IsEmpty());
	Context.ErrorHandlers.Pop(EAllowShrinking::No);
}

FCookDependencyContext::FErrorHandlerScope FCookDependencyContext::ErrorHandlerScope(
	TUniqueFunction<FString(FString&&)>&& ErrorHandler)
{
	ErrorHandlers.Add(MoveTemp(ErrorHandler));
	// We rely on Copy elision to not move-construct this scope, so that it only calls the destructor once.
	return FErrorHandlerScope(*this);
}
}

#endif //#if WITH_EDITOR
