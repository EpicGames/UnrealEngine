// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/CookEvents.h"

#if WITH_EDITOR
#include "Cooker/CookDependency.h"
#include "UObject/ICookInfo.h"
#include "UObject/SavePackage/SavePackageUtilities.h"
#include "UObject/SoftObjectPath.h"
#endif

#if WITH_EDITOR

namespace UE::Cook
{

void FCookEventContext::AddLoadBuildDependency(FCookDependency CookDependency)
{
	Data.BuildResultDependencies.Add(BuildResult::NAME_Load, MoveTemp(CookDependency));
}

void FCookEventContext::AddSaveBuildDependency(FCookDependency CookDependency)
{
	Data.BuildResultDependencies.Add(BuildResult::NAME_Save, MoveTemp(CookDependency));
}

void FCookEventContext::AddRuntimeDependency(FSoftObjectPath ObjectName)
{
	Data.CookRuntimeDependencies.Add(MoveTemp(ObjectName));
}

void FCookEventContext::HarvestCookRuntimeDependencies(UObject* HarvestReferencesFrom)
{
	UE::SavePackageUtilities::HarvestCookRuntimeDependencies(Data, HarvestReferencesFrom);
}

} // namespace UE::Cook

#endif // WITH_EDITOR
