// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Cooker/CookRand.h"

#include "Containers/UnrealString.h"
#include "Hash/Blake3.h"
#include "Misc/Char.h"
#include "Misc/Guid.h"
#include "Misc/StringBuilder.h"

namespace UE::Cook
{

thread_local TRefCountPtr<FCookRand> GCookRand;

FCookRand::FCookRand(int32 InSeed)
	: RandomStreamVar(InSeed)
{
}

FGuid FCookRand::RandomGuid() const
{
	FBlake3 Builder;

	// Use a number of random bits equal in size to FGuid, so that we do not unnecessarily restrict our random
	// output to some subset of possible guids.
	for (int32 WordIndex = 0; WordIndex < sizeof(FGuid) / sizeof(int32); ++WordIndex)
	{
		uint32 RandWord = RandomStreamVar.GetUnsignedInt();
		Builder.Update(&RandWord, sizeof(RandWord));
	}
	return FGuid::NewGuidFromHash(Builder.Finalize());
}

TRefCountPtr<FCookRand> FCookRand::GetThreadScope()
{
	return GCookRand;
}

int32 FCookRand::ConstructSeed(FStringView String)
{
	// Lower-case the string so that filepath capitalization differences do not alter the hash.
	// Convert the string to utf8 so that whether TCHAR is UTF8 or UTF16 does not alter the hash.
	TUtf8StringBuilder<1024> Normalized;
	for (TCHAR C : String)
	{
		C = FChar::ToLower(C);
		// Conversion operator<<(TCHAR) is not supported because it is unclear what it would do. So instead cast
		// our TCHAR to an FStringView and append that.
		Normalized << FStringView(&C, 1);
	}

	// Don't use GetTypeHash since GetTypeHash documentation explicitly warns that the values are not
	// guaranteed persistent between processes.
	FBlake3 Builder;
	Builder.Update(*Normalized, Normalized.Len() * sizeof(**Normalized));
	FBlake3Hash FullHash = Builder.Finalize();

	// Arbitrary shorter-length prefixes of Blake3 hashes are documented as the equivalent of running
	// the Blake3 hashing algorithm for the shorter length. So just take a 4-byte prefix of the Blake3Hash
	// to get our int32.
	int32 Result = 0;
	static_assert(sizeof(FBlake3Hash::ByteArray) >= sizeof(Result));
	// Use memcpy instead of reinterpret_cast to avoid technically undefined behavior due to misalignment.
	FMemory::Memcpy(&Result, FullHash.GetBytes(), sizeof(Result));
	return Result;
}

FCookRandScope::FCookRandScope(int32 Seed, ECookRandScope RandScope)
{
	if (ShouldAddNewInCurrentScope(RandScope))
	{
		bRestorePrevious = true;
		Previous = MoveTemp(GCookRand);
		GCookRand = MakeRefCount<FCookRand>(Seed);
	}
}

FCookRandScope::FCookRandScope(TRefCountPtr<FCookRand> CookRand, ECookRandScope RandScope)
{
	if (ShouldAddNewInCurrentScope(RandScope))
	{
		bRestorePrevious = true;
		Previous = MoveTemp(GCookRand);
		GCookRand = MoveTemp(CookRand);
	}
}

FCookRandScope::~FCookRandScope()
{
	if (bRestorePrevious)
	{
		GCookRand = MoveTemp(Previous);
	}
}

bool FCookRandScope::ShouldAddNewInCurrentScope(ECookRandScope RandScope)
{
	switch (RandScope)
	{
	case ECookRandScope::FindOrAdd:
		if (GCookRand)
		{
			return false;
		}
		break;
	case ECookRandScope::Overwrite:
		break;
	default:
		checkNoEntry();
		break;
	}
	return true;
}

}; // namespace UE::Cook

#endif // WITH_EDITOR
