// Copyright Epic Games, Inc. All Rights Reserved.

#include "Cooker/MPCollector.h"

#if WITH_EDITOR

namespace UE::Cook
{

static uint8 PlatformToIntImpl(const ITargetPlatform* Platform, TConstArrayView<const ITargetPlatform*> Platforms)
{
	if (Platform)
	{
		int32 Result = Platforms.IndexOfByKey(Platform);
		check(Result != INDEX_NONE && Result < static_cast<int32>(MAX_uint8));
		return static_cast<uint8>(Result);
	}
	else
	{
		return MAX_uint8;
	}
}

static const ITargetPlatform* IntToPlatformImpl(uint8 PlatformAsInt, TConstArrayView<const ITargetPlatform*> Platforms)
{
	int32 Index = static_cast<int32>(PlatformAsInt);
	if (Index < Platforms.Num())
	{
		return Platforms[Index];
	}
	return nullptr;
}

void FMPCollectorClientTickContext::AddMessage(FCbObject Object)
{
	Messages.Add(Object);
}

uint8 FMPCollectorClientTickContext::PlatformToInt(const ITargetPlatform* Platform) const
{
	return PlatformToIntImpl(Platform, Platforms);
}

const ITargetPlatform* FMPCollectorClientTickContext::IntToPlatform(uint8 PlatformAsInt) const
{
	return IntToPlatformImpl(PlatformAsInt, Platforms);
}

void FMPCollectorServerTickContext::AddMessage(FCbObject Object)
{
	Messages.Add(Object);
}

uint8 FMPCollectorServerTickContext::PlatformToInt(const ITargetPlatform* Platform) const
{
	return PlatformToIntImpl(Platform, Platforms);
}

const ITargetPlatform* FMPCollectorServerTickContext::IntToPlatform(uint8 PlatformAsInt) const
{
	return IntToPlatformImpl(PlatformAsInt, Platforms);
}

void FMPCollectorClientTickPackageContext::AddMessage(FCbObject Object)
{
	Messages.Emplace(nullptr, MoveTemp(Object));
}

void FMPCollectorClientTickPackageContext::AddAsyncMessage(TFuture<FCbObject>&& ObjectFuture)
{
	AsyncMessages.Emplace(nullptr, MoveTemp(ObjectFuture));
}

void FMPCollectorClientTickPackageContext::AddPlatformMessage(const ITargetPlatform* TargetPlatform, FCbObject Object)
{
	check(TargetPlatform && Platforms.Contains(TargetPlatform));
	Messages.Emplace(TargetPlatform, MoveTemp(Object));
}

void FMPCollectorClientTickPackageContext::AddAsyncPlatformMessage(const ITargetPlatform* TargetPlatform, TFuture<FCbObject>&& ObjectFuture)
{
	check(TargetPlatform && Platforms.Contains(TargetPlatform));
	AsyncMessages.Emplace(TargetPlatform, MoveTemp(ObjectFuture));
}

uint8 FMPCollectorClientTickPackageContext::PlatformToInt(const ITargetPlatform* Platform) const
{
	return PlatformToIntImpl(Platform, Platforms);
}

const ITargetPlatform* FMPCollectorClientTickPackageContext::IntToPlatform(uint8 PlatformAsInt) const
{
	return IntToPlatformImpl(PlatformAsInt, Platforms);
}

void FMPCollectorServerTickPackageContext::AddMessage(FCbObject Object)
{
	Messages.Emplace(MoveTemp(Object));
}

uint8 FMPCollectorServerTickPackageContext::PlatformToInt(const ITargetPlatform* Platform) const
{
	return PlatformToIntImpl(Platform, Platforms);
}

const ITargetPlatform* FMPCollectorServerTickPackageContext::IntToPlatform(uint8 PlatformAsInt) const
{
	return IntToPlatformImpl(PlatformAsInt, Platforms);
}

uint8 FMPCollectorClientMessageContext::PlatformToInt(const ITargetPlatform* Platform) const
{
	return PlatformToIntImpl(Platform, Platforms);
}

const ITargetPlatform* FMPCollectorClientMessageContext::IntToPlatform(uint8 PlatformAsInt) const
{
	return IntToPlatformImpl(PlatformAsInt, Platforms);
}

uint8 FMPCollectorServerMessageContext::PlatformToInt(const ITargetPlatform* Platform) const
{
	return PlatformToIntImpl(Platform, Platforms);
}

const ITargetPlatform* FMPCollectorServerMessageContext::IntToPlatform(uint8 PlatformAsInt) const
{
	return IntToPlatformImpl(PlatformAsInt, Platforms);
}

} // namespace UE::Cook

#endif // WITH_EDITOR