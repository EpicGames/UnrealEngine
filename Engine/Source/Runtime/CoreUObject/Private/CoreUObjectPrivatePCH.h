// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreUObjectSharedPCH.h"

// From Core:
#include "HAL/ThreadSafeBool.h"
#include "Internationalization/GatherableTextData.h"
#include "Internationalization/InternationalizationMetadata.h"
#include "Math/RandomStream.h"
#include "Misc/EngineVersion.h"
#include "Misc/NetworkGuid.h"
#include "Misc/ObjectThumbnail.h"
#include "Misc/OutputDeviceError.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"
#include "Serialization/CustomVersion.h"
#include "UObject/PropertyPortFlags.h"

// From CoreUObject:
#include "Blueprint/BlueprintSupport.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Misc/PackageName.h"
#include "Misc/WorldCompositionUtility.h"
#include "Serialization/SerializedPropertyScope.h"
#include "Templates/Casts.h"
#include "UObject/CoreNet.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerManager.h"
#include "UObject/LinkerPlaceholderBase.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/LinkerPlaceholderFunction.h"
#include "UObject/MetaData.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ObjectResource.h"
#include "UObject/Package.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/PropertyHelper.h"
#include "UObject/PropertyTag.h"
#include "UObject/ScriptInterface.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectThreadContext.h"
