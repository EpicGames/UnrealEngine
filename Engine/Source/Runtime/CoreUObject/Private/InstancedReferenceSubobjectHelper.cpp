// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedReferenceSubobjectHelper.h"
#include "UObject/PropertyOptional.h"
#include "UObject/Package.h"
#include "UObject/Package.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	
UObject* FInstancedPropertyPath::Resolve(const UObject* Container) const
{
	UStruct* CurrentContainerType = Container->GetClass();

	const TArray<FPropertyLink>& PropChainRef = PropertyChain;
	auto GetProperty = [&CurrentContainerType, &PropChainRef](int32 ChainIndex)->FProperty*
	{
		const FProperty* SrcProperty = PropChainRef[ChainIndex].PropertyPtr;
		return FindFProperty<FProperty>(CurrentContainerType, SrcProperty->GetFName());
	};

	auto GetArrayIndex = [&PropChainRef](int32 ChainIndex)->int32
	{
		return PropChainRef[ChainIndex].ArrayIndex == INDEX_NONE ? 0 : PropChainRef[ChainIndex].ArrayIndex;
	};

	const FProperty* CurrentProp = GetProperty(0);
	const uint8* ValuePtr = (CurrentProp) ? CurrentProp->ContainerPtrToValuePtr<uint8>(Container, GetArrayIndex(0)) : nullptr;

	for (int32 ChainIndex = 1; CurrentProp && ChainIndex < PropertyChain.Num(); ++ChainIndex)
	{
		const FPropertyLink& PropertyLink = PropertyChain[ChainIndex];

		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentProp))
		{
			if (!PropertyLink.PropertyPtr->SameType(ArrayProperty->Inner))
			{
				CurrentProp = nullptr;
				break;
			}

			const int32 TargetIndex = PropertyLink.ArrayIndex;
			check(TargetIndex != INDEX_NONE);

			FScriptArrayHelper ArrayHelper(ArrayProperty, ValuePtr);
			if (TargetIndex >= ArrayHelper.Num())
			{
				CurrentProp = nullptr;
				break;
			}

			CurrentProp = ArrayProperty->Inner;
			ValuePtr    = ArrayHelper.GetRawPtr(TargetIndex);
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(CurrentProp))
		{
			if (!PropertyLink.PropertyPtr->SameType(SetProperty->ElementProp))
			{
				CurrentProp = nullptr;
				break;
			}

			FScriptSetHelper SetHelper(SetProperty, ValuePtr);

			// Convert the logical index (recorded in the path) to the actual index used internally by the set.
			const int32 TargetIndex = SetHelper.FindInternalIndex(PropertyLink.ArrayIndex);
			if (SetHelper.IsValidIndex(TargetIndex))
			{
				CurrentProp = SetProperty->ElementProp;
				ValuePtr = SetHelper.GetElementPtr(TargetIndex);
			}
			else
			{
				CurrentProp = nullptr;
			}
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(CurrentProp))
		{
			FScriptMapHelper MapHelper(MapProperty, ValuePtr);

			// Convert the logical index (recorded in the path) to the actual index used internally by the map.
			const int32 TargetIndex = MapHelper.FindInternalIndex(PropertyLink.ArrayIndex);
			if (!MapHelper.IsValidIndex(TargetIndex))
			{
				CurrentProp = nullptr;
			}
			else if (!PropertyLink.bIsMapValue && PropertyLink.PropertyPtr->SameType(MapProperty->KeyProp))
			{
				ValuePtr = MapHelper.GetKeyPtr(TargetIndex);
				CurrentProp = MapProperty->KeyProp;
			}
			else if (PropertyLink.bIsMapValue && PropertyLink.PropertyPtr->SameType(MapProperty->ValueProp))
			{
				ValuePtr = MapHelper.GetValuePtr(TargetIndex);
				CurrentProp = MapProperty->ValueProp;
			}
			else
			{
				CurrentProp = nullptr;
			}
		}
		else if(const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(CurrentProp))
		{
			if(const uint8* InnerValuePtr = static_cast<const uint8*>(OptionalProperty->GetValuePointerForReadIfSet(ValuePtr)))
			{
				ValuePtr = InnerValuePtr;
				CurrentProp = OptionalProperty->GetValueProperty();
			}
			else
			{
				CurrentProp = nullptr;
			}
		}
		else
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(CurrentProp))
			{
				CurrentContainerType = StructProperty->Struct;
			}

			CurrentProp = GetProperty(ChainIndex);
			ValuePtr = (CurrentProp) ? CurrentProp->ContainerPtrToValuePtr<uint8>(ValuePtr, GetArrayIndex(ChainIndex)) : nullptr;
		}
	}

	const FObjectProperty* TargetProperty = CastField<FObjectProperty>(CurrentProp);
	if (TargetProperty && TargetProperty->HasAnyPropertyFlags(CPF_InstancedReference))
	{
		return TargetProperty->GetObjectPropertyValue(ValuePtr);
	}
	return nullptr;
}

template<typename T>
void FFindInstancedReferenceSubobjectHelper::ForEachInstancedSubObject(FInstancedPropertyPath& PropertyPath, T ContainerAddress, TFunctionRef<void(const FInstancedSubObjRef&, T)> ObjRefFunc)
{
	check(ContainerAddress);
	const FProperty* TargetProp = PropertyPath.Head();

	if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(TargetProp))
	{
		// Exit now if the array doesn't contain any instanced references.
		if (!ArrayProperty->HasAnyPropertyFlags(CPF_ContainsInstancedReference))
		{
			return;
		}

		FScriptArrayHelper ArrayHelper(ArrayProperty, ContainerAddress);
		for (int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ++ElementIndex)
		{
			T ValueAddress = ArrayHelper.GetRawPtr(ElementIndex);

			PropertyPath.Push(ArrayProperty->Inner, ElementIndex);
			ForEachInstancedSubObject(PropertyPath, ValueAddress, ObjRefFunc);
			PropertyPath.Pop();
		}
	}
	else if (const FMapProperty* MapProperty = CastField<const FMapProperty>(TargetProp))
	{
		// Exit now if the map doesn't contain any instanced references.
		if (!MapProperty->HasAnyPropertyFlags(CPF_ContainsInstancedReference))
		{
			return;
		}

		FScriptMapHelper MapHelper(MapProperty, ContainerAddress);
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			T KeyAddress = MapHelper.GetKeyPtr(It);
			T ValueAddress = MapHelper.GetValuePtr(It);

			// Note: Keep these as the logical (Nth) index in case the map changes internally after we construct the path or in case we resolve using a different object.
			PropertyPath.Push(MapProperty->KeyProp, It.GetLogicalIndex());
			ForEachInstancedSubObject(PropertyPath, KeyAddress, ObjRefFunc);
			PropertyPath.Pop();

			PropertyPath.Push(MapProperty->ValueProp, It.GetLogicalIndex(), true);
			ForEachInstancedSubObject(PropertyPath, ValueAddress, ObjRefFunc);
			PropertyPath.Pop();
		}
	}
	else if (const FSetProperty* SetProperty = CastField<const FSetProperty>(TargetProp))
	{
		// Exit now if the set doesn't contain any instanced references.
		if (!SetProperty->HasAnyPropertyFlags(CPF_ContainsInstancedReference))
		{
			return;
		}

		FScriptSetHelper SetHelper(SetProperty, ContainerAddress);
		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			T ValueAddress = SetHelper.GetElementPtr(It);

			// Note: Keep this as the logical (Nth) index in case the set changes internally after we construct the path or in case we resolve using a different object.
			PropertyPath.Push(SetProperty->ElementProp, It.GetLogicalIndex());
			ForEachInstancedSubObject(PropertyPath, ValueAddress, ObjRefFunc);
			PropertyPath.Pop();
		}
	}
	else if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(TargetProp))
	{
		// Exit early if the optional does not contain any instanced references.
		if (!OptionalProperty->HasAnyPropertyFlags(CPF_ContainsInstancedReference))
		{
			return;
		}
		
		if (T ValueAddress = static_cast<T>(OptionalProperty->GetValuePointerForReadOrReplaceIfSet(ContainerAddress)))
		{
			PropertyPath.Push(OptionalProperty->GetValueProperty());
			ForEachInstancedSubObject(PropertyPath, ValueAddress, ObjRefFunc);
			PropertyPath.Pop();
		}
	}
	else if (const FStructProperty* StructProperty = CastField<const FStructProperty>(TargetProp))
	{
		// Exit early if the struct does not contain any instanced references or if the struct is invalid.
		if (!StructProperty->HasAnyPropertyFlags(CPF_ContainsInstancedReference) || !StructProperty->Struct)
		{
			return;
		}

		for (FProperty* StructProp = StructProperty->Struct->RefLink; StructProp; StructProp = StructProp->NextRef)
		{
			for (int32 ArrayIdx = 0; ArrayIdx < StructProp->ArrayDim; ++ArrayIdx)
			{
				T ValueAddress = StructProp->ContainerPtrToValuePtr<uint8>(ContainerAddress, ArrayIdx);

				PropertyPath.Push(StructProp, ArrayIdx);
				ForEachInstancedSubObject(PropertyPath, ValueAddress, ObjRefFunc);
				PropertyPath.Pop();
			}
		}
	}
	else if (TargetProp->HasAllPropertyFlags(CPF_PersistentInstance))
	{
		ensure(TargetProp->HasAllPropertyFlags(CPF_InstancedReference));
		if (const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(TargetProp))
		{
			if (UObject* ObjectValue = ObjectProperty->GetObjectPropertyValue(ContainerAddress))
			{
				// don't need to push to PropertyPath, since this property is already at its head
				ObjRefFunc(FInstancedSubObjRef(ObjectValue, PropertyPath), ContainerAddress);
			}
		}
		else
		{
			// @todo figure out how to support verse property types
			//UE_LOGF(LogBlueprint, Error, "%ls: Skipping unknow property type(%ls)", ANSI_TO_TCHAR(__FUNCTION__), *TargetProp->GetName());
		}
	}
}

template COREUOBJECT_API void FFindInstancedReferenceSubobjectHelper::ForEachInstancedSubObject<void*>(FInstancedPropertyPath& PropertyPath, void* ContainerAddress, TFunctionRef<void(const FInstancedSubObjRef&, void*)> ObjRefFunc);
template COREUOBJECT_API void FFindInstancedReferenceSubobjectHelper::ForEachInstancedSubObject<const void*>(FInstancedPropertyPath& PropertyPath, const void* ContainerAddress, TFunctionRef<void(const FInstancedSubObjRef&, const void*)> ObjRefFunc);

void FFindInstancedReferenceSubobjectHelper::Duplicate(
	UObject* OldObject, 
	UObject* NewObject, 
	TMap<UObject*, UObject*>& ReferenceReplacementMap, 
	TArray<UObject*>& DuplicatedObjects, 
	TMap<UObject*, UObject*>* OptionalMappings)
{
	if (OldObject->GetClass()->HasAnyClassFlags(CLASS_HasInstancedReference) &&
		NewObject->GetClass()->HasAnyClassFlags(CLASS_HasInstancedReference))
	{
		TArray<FInstancedSubObjRef> OldInstancedSubObjects;
		GetInstancedSubObjects(OldObject, OldInstancedSubObjects);
		if (OldInstancedSubObjects.Num() > 0)
		{
			TArray<FInstancedSubObjRef> NewInstancedSubObjects;
			GetInstancedSubObjects(NewObject, NewInstancedSubObjects);
			for (const FInstancedSubObjRef& Obj : NewInstancedSubObjects)
			{
				const bool bNewObjectHasOldOuter = (Obj->GetOuter() == OldObject);
				if (bNewObjectHasOldOuter)
				{
					const bool bKeptByOld = OldInstancedSubObjects.Contains(Obj);
					const bool bNotHandledYet = !ReferenceReplacementMap.Contains(Obj) &&
						(!OptionalMappings || !OptionalMappings->Contains(Obj));
					if (bKeptByOld)
					{
						if (bNotHandledYet)
						{
							// This name may have been taken by an instanced subobject, since we're copying from
							// an object we want to recreate it here with a duplicate of the old object. I'm 
							// using a rename call here to reserve the name for ourself. These objects come from
							// FObjectInitializer::InstanceSubobjects()
							UObject* ExistingObject = StaticFindObjectFast(UObject::StaticClass(), NewObject, Obj->GetFName());
							if (ExistingObject)
							{
								ExistingObject->Rename(
									nullptr,
									GetTransientPackage(),
									REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
							}

							UObject* NewEditInlineSubobject = StaticDuplicateObject(Obj, NewObject, Obj->GetFName());
							ReferenceReplacementMap.Add(Obj, NewEditInlineSubobject);
							if(OptionalMappings)
							{
								OptionalMappings->Add(Obj, NewEditInlineSubobject);
							}

							// NOTE: we cannot patch OldObject's linker table here, since we don't 
							//       know the relation between the two objects (one could be of a 
							//       super class, and the other a child)

							// We also need to make sure to fixup any properties here
							DuplicatedObjects.Add(NewEditInlineSubobject);
						}
						else
						{
							// make sure the object is outered correctly:
							check(bNewObjectHasOldOuter);
							UObject* RealNewSubobject = nullptr;
							if (OptionalMappings)
							{
								UObject* const* Entry = OptionalMappings->Find(Obj);
								if (Entry)
								{
									RealNewSubobject = *Entry;
								}
							}

							if (!RealNewSubobject)
							{
								RealNewSubobject = ReferenceReplacementMap.FindChecked(Obj);
							}
							if (RealNewSubobject->GetOuter() != NewObject)
							{
								UObject* ExistingObject = StaticFindObjectFast(UObject::StaticClass(), NewObject, RealNewSubobject->GetFName());
								if (ExistingObject)
								{
									ExistingObject->Rename(
										nullptr,
										GetTransientPackage(),
										REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
									ReferenceReplacementMap.Add(ExistingObject, RealNewSubobject);
								}
								RealNewSubobject->Rename(
									*(RealNewSubobject->GetName()),
									NewObject,
									REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
							}
						}
					}
				}
			}
		}
	}
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS