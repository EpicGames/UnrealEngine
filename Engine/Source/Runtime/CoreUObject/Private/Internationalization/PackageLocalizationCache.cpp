// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/PackageLocalizationCache.h"
#include "HAL/PlatformTime.h"
#include "Misc/ScopeLock.h"
#include "Internationalization/Culture.h"
#include "Misc/PackageName.h"
#include "Misc/ConfigCacheIni.h"

DEFINE_LOG_CATEGORY(LogPackageLocalizationCache);

FPackageLocalizationCultureCache::FPackageLocalizationCultureCache(FPackageLocalizationCache* InOwnerCache, const FString& InCultureName)
	: OwnerCache(InOwnerCache)
{
	PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(InCultureName);

	// Query both UE style (eg, "en-US") and Verse style (eg, "en_US") localized assets
	for (const FString& PrioritizedCultureName : PrioritizedCultureNames)
	{
		FString VerseIdentifier = FCulture::CultureNameToVerseIdentifier(PrioritizedCultureName);
		if (PrioritizedCultureName != VerseIdentifier)
		{
			PrioritizedCultureNamesAndVerseIdentifiers.Add(MoveTemp(VerseIdentifier));
		}
		PrioritizedCultureNamesAndVerseIdentifiers.Add(PrioritizedCultureName);
	}
}

void FPackageLocalizationCultureCache::ConditionalUpdateCache()
{
	FScopeLock Lock(&LocalizedPackagesCS);
	ConditionalUpdateCache_NoLock();
}

void FPackageLocalizationCultureCache::ConditionalUpdateCache_NoLock()
{
	if (PendingSourceRootPathsToSearch.Num() == 0)
	{
		return;
	}

	SCOPED_BOOT_TIMING("FPackageLocalizationCultureCache::ConditionalUpdateCache_NoLock");
	const double CacheStartTime = FPlatformTime::Seconds();

	TMap<FString, TArray<FString>> NewSourceToLocalizedPaths;
	for (const FString& SourceRootPath : PendingSourceRootPathsToSearch)
	{
		TArray<FString>& LocalizedRootPaths = SourcePathsToLocalizedPaths.FindOrAdd(SourceRootPath);
		for (const FString& PrioritizedCultureName : PrioritizedCultureNamesAndVerseIdentifiers)
		{
			const FString LocalizedRootPath = SourceRootPath / TEXT("L10N") / PrioritizedCultureName;
			if (!LocalizedRootPaths.Contains(LocalizedRootPath))
			{
				LocalizedRootPaths.Add(LocalizedRootPath);
				NewSourceToLocalizedPaths.FindOrAdd(SourceRootPath).Add(LocalizedRootPath);
			}
			UE_LOGF(LogPackageLocalizationCache, Verbose, "Processing localized package path '%ls'...", *LocalizedRootPath);
		}
	}
	OwnerCache->FindLocalizedPackages(NewSourceToLocalizedPaths, SourcePackagesToLocalizedPackages);

	UE_LOGF(LogPackageLocalizationCache, Log, "Processed %d localized package path(s) for %d prioritized culture(s) in %0.6f seconds", PendingSourceRootPathsToSearch.Num(), PrioritizedCultureNames.Num(), FPlatformTime::Seconds() - CacheStartTime);

	PendingSourceRootPathsToSearch.Empty();
}

void FPackageLocalizationCultureCache::AddRootSourcePath(const FString& InRootPath)
{
	FScopeLock Lock(&LocalizedPackagesCS);

	// Add this to the list of paths to process - it will get picked up the next time the cache is updated
	PendingSourceRootPathsToSearch.AddUnique(InRootPath);
}

void FPackageLocalizationCultureCache::RemoveRootSourcePath(const FString& InRootPath)
{
	FScopeLock Lock(&LocalizedPackagesCS);

	// Remove it from the pending list
	PendingSourceRootPathsToSearch.Remove(InRootPath);

	// Make sure the StartsWith ends with a slash to avoid false positives (eg, "/Foo" also removing "/FooBar")
	const FString RootPathWithSlash = InRootPath / FString();

	// Remove all paths under this root
	for (auto It = SourcePathsToLocalizedPaths.CreateIterator(); It; ++It)
	{
		if (It->Key == InRootPath || It->Key.StartsWith(RootPathWithSlash))
		{
			It.RemoveCurrent();
			continue;
		}
	}

	// Remove all packages under this root
	FNameBuilder SourcePackageName;
	for (auto It = SourcePackagesToLocalizedPackages.CreateIterator(); It; ++It)
	{
		SourcePackageName.Reset();
		It->Key.AppendString(SourcePackageName);

		if (SourcePackageName.ToView().StartsWith(RootPathWithSlash))
		{
			It.RemoveCurrent();
			continue;
		}
	}
}

bool FPackageLocalizationCultureCache::AddPackage(const FString& InPackageName)
{
	if (!FPackageName::IsLocalizedPackage(InPackageName))
	{
		return false;
	}

	FScopeLock Lock(&LocalizedPackagesCS);

	// Is this package for a localized path that we care about
	for (const auto& SourcePathToLocalizedPathsPair : SourcePathsToLocalizedPaths)
	{
		for (const FString& LocalizedRootPath : SourcePathToLocalizedPathsPair.Value)
		{
			if (InPackageName.StartsWith(LocalizedRootPath, ESearchCase::IgnoreCase))
			{
				const FName SourcePackageName = *(SourcePathToLocalizedPathsPair.Key / InPackageName.Mid(LocalizedRootPath.Len() + 1)); // +1 for the trailing slash that isn't part of the string

				TArray<FName>& PrioritizedLocalizedPackageNames = SourcePackagesToLocalizedPackages.FindOrAdd(SourcePackageName);
				PrioritizedLocalizedPackageNames.AddUnique(*InPackageName);
				return true;
			}
		}
	}

	return false;
}

bool FPackageLocalizationCultureCache::RemovePackage(const FString& InPackageName)
{
	FScopeLock Lock(&LocalizedPackagesCS);

	if (FPackageName::IsLocalizedPackage(InPackageName))
	{
		const FName LocalizedPackageName = *InPackageName;

		// Try and find the corresponding localized package to remove
		// If the package was the last localized package for a source package, then we remove the whole mapping entry
		for (auto& SourcePackageToLocalizedPackagesPair : SourcePackagesToLocalizedPackages)
		{
			TArray<FName>& PrioritizedLocalizedPackageNames = SourcePackageToLocalizedPackagesPair.Value;
			if (PrioritizedLocalizedPackageNames.Remove(LocalizedPackageName) > 0)
			{
				if (PrioritizedLocalizedPackageNames.Num() == 0)
				{
					SourcePackagesToLocalizedPackages.Remove(SourcePackageToLocalizedPackagesPair.Key);
				}
				return true;
			}
		}
	}
	else
	{
		return SourcePackagesToLocalizedPackages.Remove(*InPackageName) > 0;
	}

	return false;
}

void FPackageLocalizationCultureCache::Empty()
{
	FScopeLock Lock(&LocalizedPackagesCS);

	PendingSourceRootPathsToSearch.Empty();
	SourcePathsToLocalizedPaths.Empty();
	SourcePackagesToLocalizedPackages.Empty();
}

FName FPackageLocalizationCultureCache::FindLocalizedPackageName(const FName InSourcePackageName)
{
	FScopeLock Lock(&LocalizedPackagesCS);

	ConditionalUpdateCache_NoLock();

	const TArray<FName>* const FoundPrioritizedLocalizedPackageNames = SourcePackagesToLocalizedPackages.Find(InSourcePackageName);
	const FName LocalizedPackageName = (FoundPrioritizedLocalizedPackageNames) ? (*FoundPrioritizedLocalizedPackageNames)[0] : NAME_None;

	UE_CLOGF(!LocalizedPackageName.IsNone(), LogPackageLocalizationCache, Verbose, "Found localized package '%ls' for source package '%ls'", *LocalizedPackageName.ToString(), *InSourcePackageName.ToString());

	return LocalizedPackageName;
}

FPackageLocalizationCache::FPackageLocalizationCache()
{
	// Read the asset group class information so we know which culture to use for packages based on the class of their primary asset
	{
		auto ReadAssetGroupClassSettings = [this](const TCHAR* InConfigLogName, const FString& InConfigFilename)
		{
			// The config is Group=Class, but we want Class=Group
			if (const FConfigSection* AssetGroupClassesSection = GConfig->GetSection(TEXT("Internationalization.AssetGroupClasses"), false, InConfigFilename))
			{
				for (const auto& SectionEntryPair : *AssetGroupClassesSection)
				{
					const FName GroupName = SectionEntryPair.Key;
					const FTopLevelAssetPath ClassName = UClass::TryConvertShortTypeNameToPathName<UStruct>(SectionEntryPair.Value.GetValue(), ELogVerbosity::Fatal, TEXT("ReadAssetGroupClassSettings"));
					
					const auto* AssetClassGroupPair = AssetClassesToAssetGroups.FindByPredicate([&](const TTuple<FTopLevelAssetPath, FName>& InAssetClassToAssetGroup)
					{
						return InAssetClassToAssetGroup.Key == ClassName;
					});

					if (AssetClassGroupPair)
					{
						UE_CLOGF(AssetClassGroupPair->Value != ClassName.GetAssetName(), LogPackageLocalizationCache, Warning, "Class '%ls' was already assigned to asset group '%ls', ignoring request to assign it to '%ls' from the %ls configuration.", *ClassName.ToString(), *AssetClassGroupPair->Value.ToString(), *GroupName.ToString(), InConfigLogName);
					}
					else
					{
						AssetClassesToAssetGroups.Add(MakeTuple(ClassName, GroupName));
						UE_LOGF(LogPackageLocalizationCache, Log, "Assigning class '%ls' to asset group '%ls' from the %ls configuration.", *ClassName.ToString(), *GroupName.ToString(), InConfigLogName);
					}
				}
			}
		};

		ReadAssetGroupClassSettings(TEXT("game"), GGameIni);
		ReadAssetGroupClassSettings(TEXT("engine"), GEngineIni);

		bPackageNameToAssetGroupDirty = true;
	}

	const FString CurrentCultureName = FInternationalization::Get().GetCurrentLanguage()->GetName();
	CurrentCultureCache = FindOrAddCacheForCulture_NoLock(CurrentCultureName);

	FInternationalization::Get().OnCultureChanged().AddRaw(this, &FPackageLocalizationCache::HandleCultureChanged);

	FPackageName::OnContentPathMounted().AddRaw(this, &FPackageLocalizationCache::HandleContentPathMounted);
	FPackageName::OnContentPathDismounted().AddRaw(this, &FPackageLocalizationCache::HandleContentPathDismounted);
}

FPackageLocalizationCache::~FPackageLocalizationCache()
{
	if (FInternationalization::IsAvailable())
	{
		FInternationalization::Get().OnCultureChanged().RemoveAll(this);
	}

	FPackageName::OnContentPathMounted().RemoveAll(this);
	FPackageName::OnContentPathDismounted().RemoveAll(this);
}

void FPackageLocalizationCache::InvalidateRootSourcePath(const FString& InRootPath)
{
	FScopeLock Lock(&LocalizedCachesCS);

	for (auto& CultureCachePair : AllCultureCaches)
	{
		CultureCachePair.Value->RemoveRootSourcePath(InRootPath);
		CultureCachePair.Value->AddRootSourcePath(InRootPath);
	}

	bPackageNameToAssetGroupDirty = true;
}

void FPackageLocalizationCache::ConditionalUpdateCache()
{
	FScopeLock Lock(&LocalizedCachesCS);

	for (auto& CultureCachePair : AllCultureCaches)
	{
		CultureCachePair.Value->ConditionalUpdateCache();
	}

	ConditionalUpdatePackageNameToAssetGroupCache_NoLock();
}

FName FPackageLocalizationCache::FindLocalizedPackageName(const FName InSourcePackageName)
{
	FScopeLock Lock(&LocalizedCachesCS);

	ConditionalUpdatePackageNameToAssetGroupCache_NoLock();

	if (PackageNameToAssetGroup.Num() > 0)
	{
		const FName AssetGroupName = PackageNameToAssetGroup.FindRef(InSourcePackageName);
		if (!AssetGroupName.IsNone())
		{
			const FCultureRef PrimaryAssetCulture = FInternationalization::Get().GetCurrentAssetGroupCulture(AssetGroupName);

			TSharedPtr<FPackageLocalizationCultureCache> CultureCache = FindOrAddCacheForCulture_NoLock(PrimaryAssetCulture->GetName());
			return (CultureCache.IsValid()) ? CultureCache->FindLocalizedPackageName(InSourcePackageName) : NAME_None;
		}
	}

	return (CurrentCultureCache.IsValid()) ? CurrentCultureCache->FindLocalizedPackageName(InSourcePackageName) : NAME_None;
}

FName FPackageLocalizationCache::FindLocalizedPackageNameForCulture(const FName InSourcePackageName, const FString& InCultureName)
{
	FScopeLock Lock(&LocalizedCachesCS);

	TSharedPtr<FPackageLocalizationCultureCache> CultureCache = FindOrAddCacheForCulture_NoLock(InCultureName);
	return (CultureCache.IsValid()) ? CultureCache->FindLocalizedPackageName(InSourcePackageName) : NAME_None;
}

TSharedPtr<FPackageLocalizationCultureCache> FPackageLocalizationCache::FindOrAddCacheForCulture_NoLock(const FString& InCultureName)
{
	if (InCultureName.IsEmpty())
	{
		return nullptr;
	}

	{
		auto* ExistingCache = AllCultureCaches.FindByPredicate([&](const TTuple<FString, TSharedPtr<FPackageLocalizationCultureCache>>& InCultureCachePair)
		{
			return InCultureCachePair.Key == InCultureName;
		});

		if (ExistingCache)
		{
			return ExistingCache->Value;
		}
	}

	TSharedPtr<FPackageLocalizationCultureCache> CultureCache = MakeShared<FPackageLocalizationCultureCache>(this, InCultureName);

	// Add the current set of root paths
	TArray<FString> RootPaths;
	FPackageName::QueryRootContentPaths(RootPaths);
	for (const FString& RootPath : RootPaths)
	{
		CultureCache->AddRootSourcePath(RootPath);
	}

	AllCultureCaches.Add(MakeTuple(InCultureName, CultureCache));
	return CultureCache;
}

void FPackageLocalizationCache::ConditionalUpdatePackageNameToAssetGroupCache_NoLock()
{
	if (!bPackageNameToAssetGroupDirty)
	{
		return;
	}

	PackageNameToAssetGroup.Reset();
	for (const auto& AssetClassGroupPair : AssetClassesToAssetGroups)
	{
		FindAssetGroupPackages(AssetClassGroupPair.Value, AssetClassGroupPair.Key);
	}

	bPackageNameToAssetGroupDirty = false;
}

void FPackageLocalizationCache::HandleContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath)
{
	FScopeLock Lock(&LocalizedCachesCS);

	for (auto& CultureCachePair : AllCultureCaches)
	{
		CultureCachePair.Value->AddRootSourcePath(InAssetPath);
	}

	bPackageNameToAssetGroupDirty = true;
}

void FPackageLocalizationCache::HandleContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath)
{
	FScopeLock Lock(&LocalizedCachesCS);

	for (auto& CultureCachePair : AllCultureCaches)
	{
		CultureCachePair.Value->RemoveRootSourcePath(InAssetPath);
	}

	bPackageNameToAssetGroupDirty = true;
}

void FPackageLocalizationCache::HandleCultureChanged()
{
	FScopeLock Lock(&LocalizedCachesCS);

	// Clear out all current caches and re-scan for the current culture
	CurrentCultureCache.Reset();
	AllCultureCaches.Empty();

	const FString CurrentCultureName = FInternationalization::Get().GetCurrentLanguage()->GetName();
	CurrentCultureCache = FindOrAddCacheForCulture_NoLock(CurrentCultureName);

	if (!GIsEditor)
	{
		// Preemptively update the cache outside of the editor
		const TArray<FCultureRef> CurrentCultures = FInternationalization::Get().GetCurrentCultures(/*bIncludeLanguage*/true, /*bIncludeLocale*/false, /*bIncludeAssetGroups*/true);
		for (const FCultureRef& CurrentCulture : CurrentCultures)
		{
			if (TSharedPtr<FPackageLocalizationCultureCache> CultureCache = FindOrAddCacheForCulture_NoLock(CurrentCulture->GetName()))
			{
				CultureCache->ConditionalUpdateCache();
			}
		}

		ConditionalUpdatePackageNameToAssetGroupCache_NoLock();
	}
}
