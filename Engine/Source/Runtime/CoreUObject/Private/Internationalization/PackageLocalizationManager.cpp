// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/PackageLocalizationManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Culture.h"
#include "Internationalization/IPackageLocalizationCache.h"
#include "Internationalization/PackageLocalizationCache.h"
#include "Async/UniqueLock.h"

DEFINE_LOG_CATEGORY_STATIC(LogPackageLocalizationManager, Log, All);

namespace PackageLocalizationInternal
{
	static bool GEnableLocalizationPackageRemapping = true;
	static FAutoConsoleVariableRef CVar_EnableLocalizationPackageRemapping(
		TEXT("localization.EnablePackageRemapping"),
		GEnableLocalizationPackageRemapping,
		TEXT("Disables identification of localization packages in order to improve startup time. Always false for the editor, optional (default true) for everything else."),
		ECVF_Default);

	bool ShouldSkipPackageRemapping()
	{
		return !GEnableLocalizationPackageRemapping;
	}
}

class FDefaultPackageLocalizationCache : public FPackageLocalizationCache
{
public:
	virtual ~FDefaultPackageLocalizationCache() {}

protected:
	//~ FPackageLocalizationCache interface
	virtual void FindLocalizedPackages(const TMap<FString, TArray<FString>>& NewSourceToLocalizedPaths, TMap<FName, TArray<FName>>& InOutSourcePackagesToLocalizedPackages) override;
	virtual void FindAssetGroupPackages(const FName InAssetGroupName, const FTopLevelAssetPath& InAssetClassName) override;
};

void FDefaultPackageLocalizationCache::FindLocalizedPackages(const TMap<FString, TArray<FString>>& NewSourceToLocalizedPaths, TMap<FName, TArray<FName>>& InOutSourcePackagesToLocalizedPackages)
{
	// Convert the package path to a filename with no extension (directory)
	for (const TPair<FString, TArray<FString>>& Pair : NewSourceToLocalizedPaths)
	{
		const FString& SourceRoot = Pair.Key;
		for (const FString& LocalizedRoot : Pair.Value)
		{
			FString LocalizedPackageFilePath;
			if (!FPackageName::TryConvertLongPackageNameToFilename(LocalizedRoot / TEXT(""), LocalizedPackageFilePath))
			{
				continue;
			}

			FPackageName::IteratePackagesInDirectory(LocalizedPackageFilePath, FPackageName::FPackageNameVisitor([&](const TCHAR* InPackageFileName) -> bool
			{
				const FString PackageSubPath = FPaths::ChangeExtension(InPackageFileName + LocalizedPackageFilePath.Len(), FString());
				const FName SourcePackageName = *(SourceRoot / PackageSubPath);
				const FName LocalizedPackageName = *(LocalizedRoot / PackageSubPath);

				TArray<FName>& PrioritizedLocalizedPackageNames = InOutSourcePackagesToLocalizedPackages.FindOrAdd(SourcePackageName);
				PrioritizedLocalizedPackageNames.AddUnique(LocalizedPackageName);

				return true;
			}));
		}
	}
}

void FDefaultPackageLocalizationCache::FindAssetGroupPackages(const FName InAssetGroupName, const FTopLevelAssetPath& InAssetClassName)
{
	// Not supported without the asset registry
}

void FPackageLocalizationManager::PerformLazyInitialization()
{
	if (PackageLocalizationInternal::ShouldSkipPackageRemapping())
	{
		UE_LOGF(LogPackageLocalizationManager, Display, "Localization packages disabled.");
		return;
	}

	if (!bIsInitialized.load(std::memory_order_acquire) && LazyInitFunc)
	{
		UE::TUniqueLock ScopeLock(InitializationMutex);
		if (ActiveCache.IsValid())
		{
			return;
		}

		LazyInitFunc(*this);

		if (!ActiveCache.IsValid())
		{
			UE_LOGF(LogPackageLocalizationManager, Warning, "InitializeFromLazyCallback was bound to a callback that didn't initialize the active cache.");
		}
	}
}

void FPackageLocalizationManager::InitializeFromLazyCallback(FLazyInitFunc InLazyInitFunc)
{
	LazyInitFunc = MoveTemp(InLazyInitFunc);
	ActiveCache.Reset();
	bIsInitialized = false;
}

void FPackageLocalizationManager::InitializeFromCache(const TSharedRef<IPackageLocalizationCache>& InCache)
{
	ActiveCache = InCache;

	// Only preemptively attempt to conditionally update the cache outside of the editor where such things
	// will happen almost immediately in a localized game, where as in the editor it's a bunch of work that
	// likely won't be used until using some localization menus in the editor.
	if (!GIsEditor && !PackageLocalizationInternal::ShouldSkipPackageRemapping())
	{
		ActiveCache->ConditionalUpdateCache();
	}

	bIsInitialized.store(true, std::memory_order_release);
}

void FPackageLocalizationManager::InitializeFromDefaultCache()
{
	InitializeFromCache(MakeShared<FDefaultPackageLocalizationCache>());
}

FName FPackageLocalizationManager::FindLocalizedPackageName(const FName InSourcePackageName)
{
	if (PackageLocalizationInternal::ShouldSkipPackageRemapping())
	{
		return InSourcePackageName;
	}

	PerformLazyInitialization();

	FName LocalizedPackageName;

	if (ActiveCache.IsValid())
	{
		LocalizedPackageName = ActiveCache->FindLocalizedPackageName(InSourcePackageName);
	}
	else
	{
		UE_LOGF(LogPackageLocalizationManager, Warning, "Localized package requested for '%ls' before the package localization manager cache was ready. Falling back to a non-cached look-up...", *InSourcePackageName.ToString());

		const FString CurrentCultureName = FInternationalization::Get().GetCurrentLanguage()->GetName();
		LocalizedPackageName = FindLocalizedPackageNameNoCache(InSourcePackageName, CurrentCultureName);
	}

	UE_CLOGF(!LocalizedPackageName.IsNone(), LogPackageLocalizationManager, Verbose, "Resolved localized package '%ls' for source package '%ls'", *LocalizedPackageName.ToString(), *InSourcePackageName.ToString());

	return LocalizedPackageName;
}

FName FPackageLocalizationManager::FindLocalizedPackageNameForCulture(const FName InSourcePackageName, const FString& InCultureName)
{
	if (PackageLocalizationInternal::ShouldSkipPackageRemapping())
	{
		return InSourcePackageName;
	}

	PerformLazyInitialization();

	FName LocalizedPackageName;

	if (ActiveCache.IsValid())
	{
		LocalizedPackageName = ActiveCache->FindLocalizedPackageNameForCulture(InSourcePackageName, InCultureName);
	}
	else
	{
		UE_LOGF(LogPackageLocalizationManager, Warning, "Localized package requested for '%ls' before the package localization manager cache was ready. Falling back to a non-cached look-up...", *InSourcePackageName.ToString());

		LocalizedPackageName = FindLocalizedPackageNameNoCache(InSourcePackageName, InCultureName);
	}

	UE_CLOGF(!LocalizedPackageName.IsNone(), LogPackageLocalizationManager, Verbose, "Resolved localized package '%ls' for source package '%ls'", *LocalizedPackageName.ToString(), *InSourcePackageName.ToString());

	return LocalizedPackageName;
}

FName FPackageLocalizationManager::FindLocalizedPackageNameNoCache(const FName InSourcePackageName, const FString& InCultureName) const
{
	if (PackageLocalizationInternal::ShouldSkipPackageRemapping())
	{
		return InSourcePackageName;
	}

	// Split the package name into its root and sub-path so that we can convert it into its localized variants for testing
	FString PackageNameRoot;
	FString PackageNameSubPath;
	{
		const FString SourcePackageNameStr = InSourcePackageName.ToString();

		TArray<FString> RootPaths;
		FPackageName::QueryRootContentPaths(RootPaths);

		for (const FString& RootPath : RootPaths)
		{
			if (SourcePackageNameStr.StartsWith(RootPath, ESearchCase::IgnoreCase))
			{
				PackageNameRoot = RootPath;
				PackageNameSubPath = SourcePackageNameStr.Mid(RootPath.Len());
				break;
			}
		}
	}

	if (PackageNameRoot.IsEmpty() || PackageNameSubPath.IsEmpty())
	{
		return NAME_None;
	}

	const TArray<FString> PrioritizedCultureNames = FInternationalization::Get().GetPrioritizedCultureNames(InCultureName);
	for (const FString& PrioritizedCultureName : PrioritizedCultureNames)
	{
		// Query both UE style (eg, "en-US") and Verse style (eg, "en_US") localized assets
		const FString VerseIdentifier = FCulture::CultureNameToVerseIdentifier(PrioritizedCultureName);
		if (PrioritizedCultureName != VerseIdentifier)
		{
			const FString LocalizedPackageNameForVerseIdentifier = PackageNameRoot / TEXT("L10N") / VerseIdentifier / PackageNameSubPath;
			if (FPackageName::DoesPackageExist(LocalizedPackageNameForVerseIdentifier))
			{
				return *LocalizedPackageNameForVerseIdentifier;
			}
		}

		const FString LocalizedPackageNameForPrioritizedCulture = PackageNameRoot / TEXT("L10N") / PrioritizedCultureName / PackageNameSubPath;
		if (FPackageName::DoesPackageExist(LocalizedPackageNameForPrioritizedCulture))
		{
			return *LocalizedPackageNameForPrioritizedCulture;
		}
	}

	return NAME_None;
}

void FPackageLocalizationManager::InvalidateRootSourcePath(const FString& InRootPath)
{
	if (ActiveCache.IsValid())
	{
		ActiveCache->InvalidateRootSourcePath(InRootPath);
	}
}

void FPackageLocalizationManager::ConditionalUpdateCache()
{
	if (!PackageLocalizationInternal::ShouldSkipPackageRemapping() && ActiveCache.IsValid())
	{
		ActiveCache->ConditionalUpdateCache();
	}
}

FPackageLocalizationManager& FPackageLocalizationManager::Get()
{
	static FPackageLocalizationManager PackageLocalizationManager;
	return PackageLocalizationManager;
}
