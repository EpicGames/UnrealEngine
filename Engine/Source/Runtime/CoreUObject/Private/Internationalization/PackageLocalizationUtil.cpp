// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/PackageLocalizationUtil.h"
#include "AssetRegistry/AssetData.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"

namespace FPackageLocalizationUtilInternal
{
	class L10NCulturesOnDiskVisitor : public IPlatformFile::FDirectoryVisitor
	{
	public:
		L10NCulturesOnDiskVisitor(const FStringView& InPackageName)
		{
			FString PackageName(InPackageName);
			FString MountPoint = FPackageName::GetPackageMountPoint(PackageName).ToString();
			FString MountPointRelativePath;
			FPackageName::TryConvertLongPackageNameToFilename(TEXT("/") + MountPoint, MountPointRelativePath);
			MountPointAbsolutePath = FPaths::ConvertRelativePathToFull(MountPointRelativePath);
		}

		const TArray<FString>& GetAllL10NCulturesIDOnDisk()
		{
			LazyVisit();
			return CulturesIDVisited;
		}

	private:
		void LazyVisit()
		{
			if (!bVisitedAlready)
			{
				bVisitedAlready = true;
				FString L10NAbsolutePath = MountPointAbsolutePath + TEXT("/L10N/");
				IFileManager::Get().IterateDirectory(*L10NAbsolutePath, *this);
			}
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			FString CultureAbsoluteDirectory(FilenameOrDirectory);
			int CultureSlashIndex = -1;
			CultureAbsoluteDirectory.FindLastChar('/', CultureSlashIndex);
			CulturesIDVisited.Add(CultureAbsoluteDirectory.RightChop(CultureSlashIndex + 1));
			return true;
		}

	private:
		FString MountPointAbsolutePath;

		bool bVisitedAlready = false;
		TArray<FString> CulturesIDVisited;
	};
}

bool FPackageLocalizationUtil::ConvertLocalizedToSource(const FStringView& InLocalized, FString& OutSource)
{
	int32 L10NStart = InLocalized.Find(TEXT("/L10N/"), ESearchCase::IgnoreCase);
	if (L10NStart != INDEX_NONE)
	{
		// .../L10N/fr/...
		//    ^ We matched here, so we need to walk over the L10N folder, and then walk over the culture code to find the range of characters to remove
		++L10NStart;
		int32 CultureStart = L10NStart + 5;
		int32 CultureEnd = CultureStart;
		for (; CultureEnd < InLocalized.Len(); ++CultureEnd)
		{
			if (InLocalized[CultureEnd] == TEXT('/'))
			{
				break;
			}
		}

		if (CultureEnd == InLocalized.Len())
		{
			return false;
		}

		OutSource = InLocalized;
		OutSource.RemoveAt(L10NStart, CultureEnd - L10NStart + 1, EAllowShrinking::No);
		return true;
	}

	return false;
}

void FPackageLocalizationUtil::ConvertToSource(const FStringView& InPath, FString& OutSource)
{
	if (!ConvertLocalizedToSource(InPath, OutSource))
	{
		OutSource = InPath;
	}
}

bool FPackageLocalizationUtil::ConvertSourceToLocalized(const FStringView& InSource, const FStringView& InCulture, FString& OutLocalized)
{
	if (!FPackageName::IsLocalizedPackage(InSource))
	{
		if (InSource.Len() > 0 && InSource[0] == TEXT('/'))
		{
			const int32 RootPathEnd = InSource.Find(TEXT("/"), 1, ESearchCase::IgnoreCase);
			if (RootPathEnd != INDEX_NONE)
			{
				FString Culture(InCulture);
				OutLocalized = InSource;
				OutLocalized.InsertAt(RootPathEnd, TEXT("/L10N") / Culture);
				return true;
			}
		}
	}

	return false;
}

bool FPackageLocalizationUtil::ConvertSourceToRegexLocalized(const FStringView& InSource, FString& OutLocalized)
{
	return ConvertSourceToLocalized(InSource, TEXT("*"), OutLocalized);
}

bool FPackageLocalizationUtil::GetLocalizedRoot(const FString& InPath, const FString& InCulture, FString& OutLocalized)
{
	if (InPath.Len() > 0 && InPath[0] == TEXT('/'))
	{
		const int32 RootPathEnd = InPath.Find(TEXT("/"), ESearchCase::IgnoreCase, ESearchDir::FromStart, 1);
		if (RootPathEnd != INDEX_NONE)
		{
			OutLocalized = InPath.Left(RootPathEnd);
			OutLocalized /= TEXT("L10N");
			if (InCulture.Len() > 0)
			{
				OutLocalized /= InCulture;
			}
			return true;
		}
	}

	return false;
}

bool FPackageLocalizationUtil::ExtractCultureFromLocalized(const FStringView& InLocalized, FString& OutCulture)
{
	const int32 L10NStart = InLocalized.Find(TEXT("/L10N/"), ESearchCase::IgnoreCase);
	if (L10NStart != INDEX_NONE)
	{
		// .../L10N/fr/...
		//    ^ We matched here, so we need to walk over the L10N folder, and then walk over the culture code
		const int32 CultureStart = L10NStart + 6;
		int32 CultureEnd = CultureStart;
		for (; CultureEnd < InLocalized.Len(); ++CultureEnd)
		{
			if (InLocalized[CultureEnd] == TEXT('/'))
			{
				break;
			}
		}

		if (CultureEnd == InLocalized.Len())
		{
			return false;
		}

		OutCulture = InLocalized.Mid(CultureStart, CultureEnd - CultureStart);
		return true;
	}

	return false;
}

void FPackageLocalizationUtil::GetLocalizedVariantsAbsolutePaths(const FStringView& InSource, TArray<FString>& OutLocalizedAbsolutePaths)
{
	FPackageLocalizationUtilInternal::L10NCulturesOnDiskVisitor L10NCulturesOnDisk(InSource);
	const TArray<FString>& CulturesID = L10NCulturesOnDisk.GetAllL10NCulturesIDOnDisk();

	for (const FString& CultureID : CulturesID)
	{
		FString LocalizedVariantPackage;
		ConvertSourceToLocalized(InSource, CultureID, LocalizedVariantPackage);
		FString OutFilename;
		if (FPackageName::DoesPackageExist(LocalizedVariantPackage, &OutFilename))
		{
			OutLocalizedAbsolutePaths.Add(FPaths::ConvertRelativePathToFull(OutFilename));
		}
	}
}
