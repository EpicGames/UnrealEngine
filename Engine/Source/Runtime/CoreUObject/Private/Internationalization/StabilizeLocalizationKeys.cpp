// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/StabilizeLocalizationKeys.h"
#include "UObject/PropertyOptional.h"
#include "UObject/TextProperty.h"

#if WITH_EDITOR

void StabilizeLocalizationKeys::StabilizeLocalizationKeysForProperty(FProperty* InProp, void* InPropData, const FString& InNamespace, const FString& InKeyRoot, const bool bAppendPropertyNameToKey)
{
	auto ShouldStabilizeInnerProperty = [](FProperty* InnerProperty) -> bool
	{
		return InnerProperty->IsA<FTextProperty>() || InnerProperty->IsA<FStructProperty>();
	};

	auto GetPropertyName = [InProp]() -> FString
	{
		// UserDefinedStructs should use the display name rather than the raw name, since the raw name doesn't match what the user generally sees or entered as the property name
		for (UStruct* PropOwnerStruct : InProp->GetOwnerStruct()->GetSuperStructIterator())
		{
			static const FName UserDefinedStructName = "UserDefinedStruct";
			if (PropOwnerStruct->GetClass()->GetFName() == UserDefinedStructName)
			{
				static const FName DisplayNameKey = "DisplayName";
				return InProp->HasMetaData(DisplayNameKey) ? InProp->GetMetaData(DisplayNameKey) : InProp->GetName();
			}
		}
		return InProp->GetName();
	};

	const FString PropKeyRoot = bAppendPropertyNameToKey ? FString::Printf(TEXT("%s_%s"), *InKeyRoot, *GetPropertyName()) : InKeyRoot;

	if (FTextProperty* TextProp = CastField<FTextProperty>(InProp))
	{
		for (int32 ArrIndex = 0; ArrIndex < TextProp->ArrayDim; ++ArrIndex)
		{
			void* PropValueData = ((uint8*)InPropData) + (TextProp->GetElementSize() * ArrIndex);

			FText* TextValuePtr = TextProp->GetPropertyValuePtr(PropValueData);
			check(TextValuePtr);

			if (TextValuePtr->IsInitializedFromString())
			{
				if (TextProp->ArrayDim > 1)
				{
					*TextValuePtr = FText::ChangeKey(InNamespace, FString::Printf(TEXT("%s_Index%d"), *PropKeyRoot, ArrIndex), *TextValuePtr);
				}
				else
				{
					*TextValuePtr = FText::ChangeKey(InNamespace, PropKeyRoot, *TextValuePtr);
				}
			}
		}
		return;
	}

	if (FStructProperty* StructProp = CastField<FStructProperty>(InProp))
	{
		for (int32 ArrIndex = 0; ArrIndex < StructProp->ArrayDim; ++ArrIndex)
		{
			void* PropValueData = ((uint8*)InPropData) + (StructProp->GetElementSize() * ArrIndex);

			if (StructProp->ArrayDim > 1)
			{
				StabilizeLocalizationKeysForStruct(
					StructProp->Struct,
					PropValueData,
					InNamespace,
					FString::Printf(TEXT("%s_Index%d"), *PropKeyRoot, ArrIndex)
					);
			}
			else
			{
				StabilizeLocalizationKeysForStruct(
					StructProp->Struct,
					PropValueData,
					InNamespace,
					PropKeyRoot
					);
			}
		}
		return;
	}

	if (FArrayProperty* ArrayProp = CastField<FArrayProperty>(InProp))
	{
		if (ShouldStabilizeInnerProperty(ArrayProp->Inner))
		{
			FScriptArrayHelper ScriptArrayHelper(ArrayProp, InPropData);

			const int32 ElementCount = ScriptArrayHelper.Num();
			for (int32 ArrIndex = 0; ArrIndex < ElementCount; ++ArrIndex)
			{
				StabilizeLocalizationKeysForProperty(
					ArrayProp->Inner,
					ScriptArrayHelper.GetRawPtr(ArrIndex),
					InNamespace,
					FString::Printf(TEXT("%s_Index%d"), *PropKeyRoot, ArrIndex),
					/*bAppendPropertyNameToKey*/false
					);
			}
		}
		return;
	}

	if (FSetProperty* SetProp = CastField<FSetProperty>(InProp))
	{
		if (ShouldStabilizeInnerProperty(SetProp->ElementProp))
		{
			FScriptSetHelper ScriptSetHelper(SetProp, InPropData);

			for (FScriptSetHelper::FIterator It(ScriptSetHelper); It; ++It)
			{
				StabilizeLocalizationKeysForProperty(
					SetProp->ElementProp,
					ScriptSetHelper.GetElementPtr(It),
					InNamespace,
					FString::Printf(TEXT("%s_Index%d"), *PropKeyRoot, It.GetLogicalIndex()),
					/*bAppendPropertyNameToKey*/false
					);
			}

			ScriptSetHelper.Rehash();
		}
		return;
	}

	if (FMapProperty* MapProp = CastField<FMapProperty>(InProp))
	{
		if (ShouldStabilizeInnerProperty(MapProp->KeyProp) || ShouldStabilizeInnerProperty(MapProp->ValueProp))
		{
			FScriptMapHelper ScriptMapHelper(MapProp, InPropData);

			for (FScriptMapHelper::FIterator It(ScriptMapHelper); It; ++It)
			{
				if (ShouldStabilizeInnerProperty(MapProp->KeyProp))
				{
					StabilizeLocalizationKeysForProperty(
						MapProp->KeyProp,
						ScriptMapHelper.GetKeyPtr(It),
						InNamespace,
						FString::Printf(TEXT("%s_KeyIndex%d"), *PropKeyRoot, It.GetLogicalIndex()),
						/*bAppendPropertyNameToKey*/false
						);
				}

				if (ShouldStabilizeInnerProperty(MapProp->ValueProp))
				{
					StabilizeLocalizationKeysForProperty(
						MapProp->ValueProp,
						ScriptMapHelper.GetValuePtr(It),
						InNamespace,
						FString::Printf(TEXT("%s_ValueIndex%d"), *PropKeyRoot, It.GetLogicalIndex()),
						/*bAppendPropertyNameToKey*/false
						);
				}
			}

			if (ShouldStabilizeInnerProperty(MapProp->KeyProp))
			{
				ScriptMapHelper.Rehash();
			}
		}
		return;
	}

	if (FOptionalProperty* OptionalProp = CastField<FOptionalProperty>(InProp))
	{
		FProperty* OptionalValueProp = OptionalProp->GetValueProperty();
		if (ShouldStabilizeInnerProperty(OptionalValueProp))
		{
			if (void* ValueData = OptionalProp->GetValuePointerForReplaceIfSet(InPropData))
			{
				StabilizeLocalizationKeysForProperty(
					OptionalValueProp,
					ValueData,
					InNamespace,
					PropKeyRoot,
					/*bAppendPropertyNameToKey*/false
					);
			}
		}
	}
}

void StabilizeLocalizationKeys::StabilizeLocalizationKeysForStruct(UStruct* InStruct, void* InStructData, const FString& InNamespace, const FString& InKeyRoot)
{
	for (TFieldIterator<FProperty> It(InStruct); It; ++It)
	{
		StabilizeLocalizationKeysForProperty(*It, It->ContainerPtrToValuePtr<void>(InStructData), InNamespace, InKeyRoot);
	}
}

#endif	// WITH_EDITOR
