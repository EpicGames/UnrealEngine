// Copyright Epic Games, Inc. All Rights Reserved.

#include "Internationalization/TextPackageNamespaceUtil.h"

#include "Hash/Blake3.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "Serialization/TextReferenceCollector.h"
#include "UObject/Class.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/TextProperty.h"

#if USE_STABLE_LOCALIZATION_KEYS

namespace TextNamespaceUtilImpl
{

const FName PackageLocalizationNamespaceKey = "PackageLocalizationNamespace";

FString FindOrAddPackageNamespace(UPackage* InPackage, const bool bCanAdd)
{
	checkf(!bCanAdd || GIsEditor, TEXT("An attempt was made to add a localization namespace while running as a non-editor. Please wrap the call to TextNamespaceUtil::EnsurePackageNamespace with a test for GIsEditor, or use TextNamespaceUtil::GetPackageNamespace instead."));

	if (InPackage)
	{
		const bool bIsTransientPackage = InPackage->HasAnyFlags(RF_Transient) || InPackage == GetTransientPackage();
		const FString PackageName = InPackage->GetName();

		if (FPackageName::IsScriptPackage(PackageName))
		{
			if (!bIsTransientPackage)
			{
				// Script packages use their name as the package namespace
				return PackageName;
			}
		}
		else
		{
			// Other packages store their package namespace as meta-data
			FMetaData& PackageMetaData = InPackage->GetMetaData();

			if (bCanAdd && !bIsTransientPackage)
			{
				FString& PackageLocalizationNamespaceValue = PackageMetaData.RootMetaDataMap.FindOrAdd(PackageLocalizationNamespaceKey);
				if (PackageLocalizationNamespaceValue.IsEmpty())
				{
					PackageLocalizationNamespaceValue = TextNamespaceUtil::GenerateDeterministicPackageNamespace(InPackage);
				}
				return PackageLocalizationNamespaceValue;
			}
			else
			{
				const FString* PackageLocalizationNamespaceValue = PackageMetaData.RootMetaDataMap.Find(PackageLocalizationNamespaceKey);
				if (PackageLocalizationNamespaceValue)
				{
					return *PackageLocalizationNamespaceValue;
				}
			}
		}
	}

	return FString();
}

void ClearPackageNamespace(UPackage* InPackage)
{
	if (InPackage)
	{
		const FString PackageName = InPackage->GetName();

		if (!FPackageName::IsScriptPackage(PackageName))
		{
			FMetaData& PackageMetaData = InPackage->GetMetaData();
			PackageMetaData.RootMetaDataMap.Remove(PackageLocalizationNamespaceKey);
		}
	}
}

void ForcePackageNamespace(UPackage* InPackage, const FString& InNamespace)
{
	if (InPackage)
	{
		const FString PackageName = InPackage->GetName();

		if (!FPackageName::IsScriptPackage(PackageName))
		{
			FMetaData& PackageMetaData = InPackage->GetMetaData();
			FString& PackageLocalizationNamespaceValue = PackageMetaData.RootMetaDataMap.FindOrAdd(PackageLocalizationNamespaceKey);
			PackageLocalizationNamespaceValue = InNamespace;
		}
	}
}

FString GenerateDeterministicPackageNamespace(const FString& InPackagePath, const FGuid& InNonUniqueGuid)
{
	// Make a deterministic new guid that will vary based on the package
	FBlake3 Builder;
	Builder.Update(&InNonUniqueGuid, sizeof(FGuid));
	Builder.Update(*InPackagePath, InPackagePath.Len() * sizeof(**InPackagePath));
	FBlake3Hash Hash = Builder.Finalize();
	// We use the first 16 bytes of the FIoHash to create the guid, there is
	// no specific reason why these were chosen, we could take any pattern or combination
	// of bytes.
	uint32* HashBytes = (uint32*)Hash.GetBytes();
	FGuid NewGuid(HashBytes[0], HashBytes[1], HashBytes[2], HashBytes[3]);

	return NewGuid.ToString();
}

bool HasDeterministicPackageNamespace(const FString& InPackagePath, const FString& InCurrentPackageNamespace, const FGuid& InPersistentGuid)
{
	if (!InCurrentPackageNamespace.IsEmpty())
	{
		if (FPackageName::IsScriptPackage(InPackagePath))
		{
			return true;
		}

		const FString& DeterministicPackageNamespace = GenerateDeterministicPackageNamespace(InPackagePath, InPersistentGuid);
		if (InCurrentPackageNamespace.Equals(DeterministicPackageNamespace))
		{
			return true;
		}
	}

	return false;
}

} // TextNamespaceUtilImpl

namespace TextNamespaceUtil
{
	class FUpdateTextsPackageNamespace::FImpl : public FArchiveUObject
	{
	public:
		FImpl(UPackage* InPackage)
			: Package(InPackage)
			, ActualPackageNamespace(TextNamespaceUtil::EnsurePackageNamespace(InPackage))
		{
		}

		void UpdateTextsPackageNamespace(TextNamespaceUtil::FUpdateTextsPackageNamespace* InOriginalUpdateTextsPackageNamespace)
		{
			this->SetIsSaving(true);

			TArray<UObject*> AllObjectsInPackage;
			GetObjectsWithOuter(Package, AllObjectsInPackage, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);

			for (UObject* Obj : AllObjectsInPackage)
			{
				UClass* CurrentClass = nullptr;
				if (InOriginalUpdateTextsPackageNamespace)
				{
					CurrentClass = Obj->GetClass();
				}

				while (CurrentClass)
				{
					const TextNamespaceUtil::FUpdateTextsPackageNamespace::FProcessObjectCallback* CustomCallback =
						TextNamespaceUtil::FUpdateTextsPackageNamespace::GetTypeSpecificProcessObjectCallbacks().Find(CurrentClass);
					if (CustomCallback)
					{
						(*CustomCallback)(Obj, *InOriginalUpdateTextsPackageNamespace);
						break;
					}
					else
					{
						CurrentClass = CurrentClass->GetSuperClass();
					}

				}

				if (CurrentClass == nullptr) // No callback was used for that class (and all super classes)
				{
					Obj->Serialize(*this);
				}
			}
		}

		bool UpdateSingleText(FText& InOutText)
		{
			if (!InOutText.ShouldGatherForLocalization())
			{
				return false;
			}

			const FTextId TextId = FTextInspector::GetTextId(InOutText);
			if (TextId.IsEmpty())
			{
				return false;
			}

			const FString CurrentNamespace = TextId.GetNamespace().ToString();
			const FString ExpectedNamespace = TextNamespaceUtil::BuildFullNamespace(CurrentNamespace, ActualPackageNamespace);
			if (CurrentNamespace.Equals(ExpectedNamespace, ESearchCase::CaseSensitive))
			{
				return false;
			}

			FTextKey NewNamespace;
			FTextKey NewKey;
			if (const FTextId* ExistingMapping = PackageTextIdMap.Find(TextId))
			{
				NewNamespace = ExistingMapping->GetNamespace();
				NewKey = ExistingMapping->GetKey();
			}
			else
			{
				NewNamespace = ExpectedNamespace;
				NewKey = FGuid::NewGuid().ToString();

				PackageTextIdMap.Add(TextId, FTextId(NewNamespace, NewKey));
			}

			InOutText = FText::ChangeKey(NewNamespace, NewKey, InOutText);
			return true;
		}

		void CallBackSerialize(UObject* InObject)
		{
			InObject->Serialize(*this);
		}

		using FArchiveUObject::operator<<; // For visibility of the overloads we don't override

		virtual FArchive& operator<<(FText& Value) override
		{
			UpdateSingleText(Value);
			return *this;
		}

	private:
		UPackage* Package;
		FString ActualPackageNamespace;
		TMap<FTextId, FTextId> PackageTextIdMap;
	};
} // TextNamespaceUtil

void TextNamespaceUtil::FUpdateTextsPackageNamespace::UpdatePackage(UPackage* InPackage)
{
	FUpdateTextsPackageNamespace UpdateTextsPackageNamespace(InPackage);
}

TextNamespaceUtil::FUpdateTextsPackageNamespace::FUpdateTextsPackageNamespace(UPackage* InPackage)
{
	PImpl = MakePimpl<FImpl>(InPackage);
	PImpl->UpdateTextsPackageNamespace(this);
}

bool TextNamespaceUtil::FUpdateTextsPackageNamespace::UpdateSingleText(FText& InOutText)
{
	return PImpl->UpdateSingleText(InOutText);
}

void TextNamespaceUtil::FUpdateTextsPackageNamespace::Serialize(UObject* InObject)
{
	PImpl->CallBackSerialize(InObject);
}

TextNamespaceUtil::FUpdateTextsPackageNamespace::FProcessObjectCallbackMap& TextNamespaceUtil::FUpdateTextsPackageNamespace::GetTypeSpecificProcessObjectCallbacks()
{
	static TextNamespaceUtil::FUpdateTextsPackageNamespace::FProcessObjectCallbackMap CallbackMap;
	return CallbackMap;
}

FString TextNamespaceUtil::GenerateDeterministicPackageNamespace(const UPackage* InPackage)
{
	FString PackagePath = InPackage->GetName();
	FGuid NonUniqueGuid = InPackage->GetPersistentGuid(); // Can be the same for duplicated packages
	return TextNamespaceUtilImpl::GenerateDeterministicPackageNamespace(PackagePath, NonUniqueGuid);
}

bool TextNamespaceUtil::HasDeterministicPackageNamespace(const UPackage* InPackage)
{
	if (InPackage)
	{
		FString PackagePath = InPackage->GetName();
		FString CurrentPackageNamespace = TextNamespaceUtil::GetPackageNamespace(InPackage);
		FGuid PersistentGuid = InPackage->GetPersistentGuid(); // Can be the same for duplicated packages
		return TextNamespaceUtilImpl::HasDeterministicPackageNamespace(PackagePath, CurrentPackageNamespace, PersistentGuid);
	}
	else
	{
		return false;
	}
}

bool TextNamespaceUtil::HasDeterministicPackageNamespace(UObject* InObject)
{
	UPackage* Package = InObject ? InObject->GetOutermost() : nullptr;
	return HasDeterministicPackageNamespace(Package);
}

bool TextNamespaceUtil::HasDeterministicPackageNamespace(const FString& InPackagePath, const FPackageFileSummary& InPackageFileSummary)
{
	const FString& CurrentPackageNamespace = InPackageFileSummary.LocalizationId;
	const FGuid& PersistentGuid = InPackageFileSummary.PersistentGuid;
	return TextNamespaceUtilImpl::HasDeterministicPackageNamespace(InPackagePath, CurrentPackageNamespace, PersistentGuid);
}

bool TextNamespaceUtil::ResetPackageNamespace(UPackage* InPackage)
{
	if (InPackage && !HasDeterministicPackageNamespace(InPackage))
	{
		TextNamespaceUtil::ClearPackageNamespace(InPackage);
		TextNamespaceUtil::EnsurePackageNamespace(InPackage);

		// We also need to update package namespace on all texts within the package. Otherwise, the cache could become invalid on save.
		TextNamespaceUtil::FUpdateTextsPackageNamespace::UpdatePackage(InPackage);

		return true;
	}

	return false;
}

bool TextNamespaceUtil::ResetPackageNamespace(UObject* InObject)
{
	UPackage* Package = InObject ? InObject->GetOutermost() : nullptr;
	return ResetPackageNamespace(Package);
}

FString TextNamespaceUtil::GetPackageNamespace(const UPackage* InPackage)
{
	return TextNamespaceUtilImpl::FindOrAddPackageNamespace(const_cast<UPackage*>(InPackage), /*bCanAdd*/false);
}

FString TextNamespaceUtil::GetPackageNamespace(const UObject* InObject)
{
	const UPackage* Package = InObject ? InObject->GetOutermost() : nullptr;
	return GetPackageNamespace(Package);
}

FString TextNamespaceUtil::EnsurePackageNamespace(UPackage* InPackage)
{
	return TextNamespaceUtilImpl::FindOrAddPackageNamespace(InPackage, /*bCanAdd*/true);
}

FString TextNamespaceUtil::EnsurePackageNamespace(UObject* InObject)
{
	UPackage* Package = InObject ? InObject->GetOutermost() : nullptr;
	return EnsurePackageNamespace(Package);
}

void TextNamespaceUtil::ClearPackageNamespace(UPackage* InPackage)
{
	TextNamespaceUtilImpl::ClearPackageNamespace(InPackage);
}

void TextNamespaceUtil::ClearPackageNamespace(UObject* InObject)
{
	UPackage* Package = InObject ? InObject->GetOutermost() : nullptr;
	ClearPackageNamespace(Package);
}

void TextNamespaceUtil::ForcePackageNamespace(UPackage* InPackage, const FString& InNamespace)
{
	TextNamespaceUtilImpl::ForcePackageNamespace(InPackage, InNamespace);
}

void TextNamespaceUtil::ForcePackageNamespace(UObject* InObject, const FString& InNamespace)
{
	UPackage* Package = InObject ? InObject->GetOutermost() : nullptr;
	ForcePackageNamespace(Package, InNamespace);
}

#endif // USE_STABLE_LOCALIZATION_KEYS

FText TextNamespaceUtil::CopyTextToPackage(const FText& InText, UPackage* InPackage, const ETextCopyMethod InCopyMethod, const bool bAlwaysApplyPackageNamespace)
{
#if USE_STABLE_LOCALIZATION_KEYS
	return CopyTextToPackage(InText, EnsurePackageNamespace(InPackage), InCopyMethod, bAlwaysApplyPackageNamespace);
#else	// USE_STABLE_LOCALIZATION_KEYS
	return CopyTextToPackage(InText, FString(), InCopyMethod, bAlwaysApplyPackageNamespace);
#endif	// USE_STABLE_LOCALIZATION_KEYS
}

FText TextNamespaceUtil::CopyTextToPackage(const FText& InText, UObject* InObject, const ETextCopyMethod InCopyMethod, const bool bAlwaysApplyPackageNamespace)
{
#if USE_STABLE_LOCALIZATION_KEYS
	return CopyTextToPackage(InText, EnsurePackageNamespace(InObject), InCopyMethod, bAlwaysApplyPackageNamespace);
#else	// USE_STABLE_LOCALIZATION_KEYS
	return CopyTextToPackage(InText, FString(), InCopyMethod, bAlwaysApplyPackageNamespace);
#endif	// USE_STABLE_LOCALIZATION_KEYS
}

FString TextNamespaceUtil::GenerateRandomTextKey()
{
	return FGuid::NewGuid().ToString();
}

FString TextNamespaceUtil::GenerateDeterministicTextKey(UObject* InTextOwner, const FTextProperty* InTextProperty, const bool bApplyPackageNamespace)
{
	return GenerateDeterministicTextKey(InTextOwner, InTextProperty->GetFName(), bApplyPackageNamespace);
}

FString TextNamespaceUtil::GenerateDeterministicTextKey(UObject* InTextOwner, const FName InTextPropertyName, const bool bApplyPackageNamespace)
{
	auto GetNameKeyHash = [](const FName Name)
	{
		FNameBuilder Builder(Name);
		return TextKeyUtil::HashString(Builder.ToView());
	};

	auto GetObjectKeyHash = [](const UObject* Obj)
	{
		FNameBuilder Builder;
		if (Obj)
		{
			Obj->GetPathName(nullptr, Builder);
		}
		return TextKeyUtil::HashString(Builder.ToView());
	};

	// Build a (hopefully) unique and deterministic key from a combination of the text owner and text property info
	FGuid KeyGuid(GetObjectKeyHash(InTextOwner->GetOuter()), GetNameKeyHash(InTextOwner->GetFName()), GetNameKeyHash(InTextOwner->GetClass()->GetFName()), GetNameKeyHash(InTextPropertyName));
#if USE_STABLE_LOCALIZATION_KEYS
	if (const FString PackageNamespace = bApplyPackageNamespace ? EnsurePackageNamespace(InTextOwner) : FString();
		!PackageNamespace.IsEmpty())
	{
		// Mix the package namespace hash into the outer hash
		KeyGuid.A = TextKeyUtil::HashString(PackageNamespace, KeyGuid.A);
	}
#endif // USE_STABLE_LOCALIZATION_KEYS
	return KeyGuid.ToString();
}

void TextNamespaceUtil::GetTextIdForEdit(UPackage* InPackage, const ETextEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey, TFunctionRef<FString()> InTextKeyGenerator, const bool bApplyPackageNamespace)
{
#if USE_STABLE_LOCALIZATION_KEYS
	bool bPersistKey = false;

	if (const FString PackageNamespace = bApplyPackageNamespace ? EnsurePackageNamespace(InPackage) : FString();
		!PackageNamespace.IsEmpty())
	{
		// Make sure the proposed namespace is using the correct namespace for this package
		OutStableNamespace = BuildFullNamespace(InProposedNamespace, PackageNamespace, /*bAlwaysApplyPackageNamespace*/true);

		if (InProposedNamespace.Equals(OutStableNamespace, ESearchCase::CaseSensitive) || InEditAction == ETextEditAction::Namespace)
		{
			// If the proposal was already using the correct namespace (or we just set the namespace), attempt to persist the proposed key too
			if (!InProposedKey.IsEmpty())
			{
				// If we changed the source text, then we can persist the key if this text is the *only* reference using that ID
				// If we changed the identifier, then we can persist the key only if doing so won't cause an identify conflict
				const FTextReferenceCollector::EComparisonMode ReferenceComparisonMode = InEditAction == ETextEditAction::SourceString ? FTextReferenceCollector::EComparisonMode::MatchId : FTextReferenceCollector::EComparisonMode::MismatchSource;
				const int32 RequiredReferenceCount = InEditAction == ETextEditAction::SourceString ? 1 : 0;

				int32 ReferenceCount = 0;
				FTextReferenceCollector Collector(InPackage, ReferenceComparisonMode, OutStableNamespace, InProposedKey, InTextSource, ReferenceCount);

				if (ReferenceCount == RequiredReferenceCount)
				{
					bPersistKey = true;
					OutStableKey = InProposedKey;
				}
			}
		}
		else if (InEditAction != ETextEditAction::Namespace)
		{
			// If our proposed namespace wasn't correct for our package, and we didn't just set it (which doesn't include the package namespace)
			// then we should clear out any user specified part of it
			OutStableNamespace = BuildFullNamespace(FString(), PackageNamespace, /*bAlwaysApplyPackageNamespace*/true);
		}
	}

	if (!bPersistKey)
#endif // USE_STABLE_LOCALIZATION_KEYS
	{
		OutStableKey = InTextKeyGenerator();

#if USE_STABLE_LOCALIZATION_KEYS
		// Make sure the new key didn't produce a conflict (as may sometimes happen when using deterministic text key generators).
		// If it has, generate a new key using the random text key generator instead.
		{
			int32 ReferenceCount = 0;
			FTextReferenceCollector Collector(InPackage, FTextReferenceCollector::EComparisonMode::MismatchSource, OutStableNamespace, OutStableKey, InTextSource, ReferenceCount);

			if (ReferenceCount != 0)
			{
				OutStableKey = GenerateRandomTextKey();
			}
		}
#endif // USE_STABLE_LOCALIZATION_KEYS
	}
}

bool TextNamespaceUtil::EditTextProperty(UObject* InTextOwner, const FTextProperty* InTextProperty, const ETextEditAction InEditAction, const FString& InEditValue, TFunctionRef<FString()> InTextKeyGenerator, const bool bApplyPackageNamespace)
{
	if (!InTextOwner->GetClass()->HasProperty(InTextProperty))
	{
		return false;
	}

	return EditTextProperty_Direct(InTextOwner->GetPackage(), InTextProperty->GetPropertyValuePtr_InContainer(InTextOwner), InTextProperty, InEditAction, InEditValue, InTextKeyGenerator, bApplyPackageNamespace);
}

bool TextNamespaceUtil::EditTextProperty_Direct(UPackage* InPackage, void* InTextValue, const FTextProperty* InTextProperty, const ETextEditAction InEditAction, const FString& InEditValue, TFunctionRef<FString()> InTextKeyGenerator, const bool bApplyPackageNamespace)
{
	if (InEditAction == ETextEditAction::SourceString && InEditValue.IsEmpty())
	{
		// Empty source strings always produce an empty text
		InTextProperty->SetPropertyValue(InTextValue, FText());
		return true;
	}

	const FText CurrentTextValue = InTextProperty->GetPropertyValue(InTextValue);
	const FTextId CurrentTextId = FTextInspector::GetTextId(CurrentTextValue);
	const FString* CurrentSourceString = FTextInspector::GetSourceString(CurrentTextValue);
	const bool bIsCurrentTextLocalized = !CurrentTextValue.IsCultureInvariant() && !CurrentTextValue.IsFromStringTable();

	// Verify the edited attribute has actually changed
	if (bIsCurrentTextLocalized)
	{
		switch (InEditAction)
		{
		case ETextEditAction::Namespace:
			if (const FString CurrentTextNamespace = StripPackageNamespace(CurrentTextId.GetNamespace().ToString());
				CurrentTextNamespace.Equals(InEditValue, ESearchCase::CaseSensitive))
			{
				return true;
			}
			break;

		case ETextEditAction::Key:
			if (CurrentTextId.GetKey().ToString().Equals(InEditValue, ESearchCase::CaseSensitive))
			{
				return true;
			}
			break;

		case ETextEditAction::SourceString:
			if (CurrentSourceString && CurrentSourceString->Equals(InEditValue, ESearchCase::CaseSensitive))
			{
				return true;
			}
			break;

		default:
			checkf(false, TEXT("Unknown ETextEditAction!"));
			break;
		}
	}

	const FString ProposedNamespace = (InEditAction == ETextEditAction::Namespace ? InEditValue : CurrentTextId.GetNamespace().ToString());
	const FString ProposedKey = (InEditAction == ETextEditAction::Key ? InEditValue : CurrentTextId.GetKey().ToString());
	FString SourceString = (InEditAction == ETextEditAction::SourceString ? InEditValue : CurrentSourceString ? *CurrentSourceString : FString());

	FString StableNamespace;
	FString StableKey;
	GetTextIdForEdit(InPackage, InEditAction, SourceString, ProposedNamespace, ProposedKey, StableNamespace, StableKey, InTextKeyGenerator, bApplyPackageNamespace);

	InTextProperty->SetPropertyValue(InTextValue, FText::AsLocalizable_Advanced(StableNamespace, StableKey, MoveTemp(SourceString)));
	return true;
}
