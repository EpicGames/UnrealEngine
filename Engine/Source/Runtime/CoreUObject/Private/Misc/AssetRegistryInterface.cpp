// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AssetRegistryInterface.h"

#include "Containers/Set.h"
#include "CoreGlobals.h"
#include "Misc/CommandLine.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

IAssetRegistryInterface* IAssetRegistryInterface::Default = nullptr;
IAssetRegistryInterface* IAssetRegistryInterface::GetPtr()
{
	return Default;
}

namespace UE::AssetRegistry
{
namespace Private
{
	IAssetRegistry* IAssetRegistrySingleton::Singleton = nullptr;
}

#if WITH_ENGINE && WITH_EDITOR
	FRWLock                  SkipClassesLock;
	TSet<FTopLevelAssetPath> SkipUncookedClasses;
	TSet<FTopLevelAssetPath> SkipCookedClasses;
	bool bInitializedSkipClasses = false;

	void FFiltering::SetSkipClasses(const TSet<FTopLevelAssetPath>& InSkipUncookedClasses, const TSet<FTopLevelAssetPath>& InSkipCookedClasses)
	{
		bInitializedSkipClasses = true;
		UE::TWriteScopeLock WriteLock(SkipClassesLock);
		SkipUncookedClasses = InSkipUncookedClasses;
		SkipCookedClasses = InSkipCookedClasses;
	}

	void FFiltering::InitializeShouldSkipAsset()
	{
		if (!bInitializedSkipClasses)
		{
			// Since we only collect these the first on-demand time, it is possible we will miss subclasses
			// from plugins that load later. This flaw is a rare edge case, though, and this solution will
			// be replaced eventually, so leaving it for now.
			{
				UE::TWriteScopeLock WriteLock(SkipClassesLock);
				Utils::PopulateSkipClasses(SkipUncookedClasses, SkipCookedClasses);
			}

			bInitializedSkipClasses = true;
		}
	}
#endif

#if WITH_ENGINE && WITH_EDITOR
namespace Utils
{

	bool ShouldSkipAsset(const FTopLevelAssetPath& AssetClass, uint32 PackageFlags, const TSet<FTopLevelAssetPath>& InSkipUncookedClasses, const TSet<FTopLevelAssetPath>& InSkipCookedClasses)
	{
		if (PackageFlags & PKG_ContainsNoAsset)
		{
			return true;
		}

		const bool bIsCooked = (PackageFlags & PKG_FilterEditorOnly);
		if ((bIsCooked && InSkipCookedClasses.Contains(AssetClass)) ||
			(!bIsCooked && InSkipUncookedClasses.Contains(AssetClass)))
		{
			return true;
		}
		return false;
	}

	bool ShouldSkipAsset(const UObject* InAsset, const TSet<FTopLevelAssetPath>& InSkipUncookedClasses, const TSet<FTopLevelAssetPath>& InSkipCookedClasses)
	{
		if (!InAsset)
		{
			return false;
		}
		UPackage* Package = InAsset->GetPackage();
		if (!Package)
		{
			return false;
		}
		return ShouldSkipAsset(InAsset->GetClass()->GetClassPathName(), Package->GetPackageFlags(), InSkipUncookedClasses, InSkipCookedClasses);
	}

	void PopulateSkipClasses(TSet<FTopLevelAssetPath>& OutSkipUncookedClasses, TSet<FTopLevelAssetPath>& OutSkipCookedClasses)
	{
		const FName NAME_EnginePackage(GetScriptPackageNameEngine());
		UPackage* EnginePackage = Cast<UPackage>(StaticFindObjectFast(UPackage::StaticClass(), nullptr, NAME_EnginePackage));
		{
			OutSkipUncookedClasses.Reset();

			const FName NAME_BlueprintGeneratedClass(GetClassNameBlueprintGeneratedClass());
			UClass* BlueprintGeneratedClass = nullptr;
			if (EnginePackage)
			{
				BlueprintGeneratedClass = Cast<UClass>(StaticFindObjectFast(UClass::StaticClass(), EnginePackage, NAME_BlueprintGeneratedClass));
			}
			if (!BlueprintGeneratedClass)
			{
				UE_LOGF(LogCore, Warning, "Could not find BlueprintGeneratedClass; will not be able to filter uncooked BPGC");
			}
			else
			{
				OutSkipUncookedClasses.Add(BlueprintGeneratedClass->GetClassPathName());
				for (TObjectIterator<UClass> It; It; ++It)
				{
					if (It->IsChildOf(BlueprintGeneratedClass) && !It->HasAnyClassFlags(CLASS_Abstract))
					{
						OutSkipUncookedClasses.Add(It->GetClassPathName());
					}
				}
			}
		}
		{
			OutSkipCookedClasses.Reset();

			const FName NAME_Blueprint(GetClassNameBlueprint());
			UClass* BlueprintClass = nullptr;
			if (EnginePackage)
			{
				BlueprintClass = Cast<UClass>(StaticFindObjectFast(UClass::StaticClass(), EnginePackage, NAME_Blueprint));
			}
			if (!BlueprintClass)
			{
				UE_LOGF(LogCore, Warning, "Could not find BlueprintClass; will not be able to filter cooked BP");
			}
			else
			{
				OutSkipCookedClasses.Add(BlueprintClass->GetClassPathName());
				for (TObjectIterator<UClass> It; It; ++It)
				{
					if (It->IsChildOf(BlueprintClass) && !It->HasAnyClassFlags(CLASS_Abstract))
					{
						OutSkipCookedClasses.Add(It->GetClassPathName());
					}
				}
			}
		}
	}

}
#endif

	bool FFiltering::ShouldSkipAsset(const FTopLevelAssetPath& AssetClass, uint32 PackageFlags)
	{
#if WITH_ENGINE && WITH_EDITOR
		// We do not yet support having UBlueprintGeneratedClasses be assets when the UBlueprint is also
		// an asset; the content browser does not handle the multiple assets correctly and displays this
		// class asset as if it is in a separate package. Revisit when we have removed the UBlueprint as an asset
		// or when we support multiple assets.
		InitializeShouldSkipAsset();

		UE::TReadScopeLock ReadLock(SkipClassesLock);
		return Utils::ShouldSkipAsset(AssetClass, PackageFlags, SkipUncookedClasses, SkipCookedClasses);
#else
		return false;
#endif //if WITH_ENGINE && WITH_EDITOR
	}

	bool FFiltering::ShouldSkipAsset(const UObject* InAsset)
	{
		if (!InAsset)
		{
			return false;
		}
		UPackage* Package = InAsset->GetPackage();
		if (!Package)
		{
			return false;
		}
		return ShouldSkipAsset(InAsset->GetClass()->GetClassPathName(), Package->GetPackageFlags());
	}

	void FFiltering::MarkDirty()
	{
#if WITH_ENGINE && WITH_EDITOR
		bInitializedSkipClasses = false;
#endif
	}

	static FName ScriptPackageNameCoreUObject("/Script/CoreUObject");
	static FName ScriptPackageNameEngine("/Script/Engine");
	static FName ScriptPackageNameBlueprintGraph("/Script/BlueprintGraph");
	static FName ScriptPackageNameUnrealEd("/Script/UnrealEd");
	static FName ClassNameObject("Object");
	static FName ClassNameObjectRedirector("ObjectRedirector");
	static FName ClassNameBlueprintCore("BlueprintCore");
	static FName ClassNameBlueprint("Blueprint");
	static FName ClassNameBlueprintGeneratedClass("BlueprintGeneratedClass");

	FName GetScriptPackageNameCoreUObject()
	{
		return ScriptPackageNameCoreUObject;
	}
	FName GetScriptPackageNameEngine()
	{
		return ScriptPackageNameEngine;
	}
	FName GetScriptPackageNameBlueprintGraph()
	{
		return ScriptPackageNameBlueprintGraph;
	}
	FName GetScriptPackageNameUnrealEd()
	{
		return ScriptPackageNameUnrealEd;
	}
	FName GetClassNameObject()
	{
		return ClassNameObject;
	}
	FName GetClassNameObjectRedirector()
	{
		return ClassNameObjectRedirector;
	}
	FName GetClassNameBlueprintCore()
	{
		return ClassNameBlueprintCore;
	}
	FName GetClassNameBlueprint()
	{
		return ClassNameBlueprint;
	}
	FName GetClassNameBlueprintGeneratedClass()
	{
		return ClassNameBlueprintGeneratedClass;
	}
	FTopLevelAssetPath GetClassPathObject()
	{
		return FTopLevelAssetPath(GetScriptPackageNameCoreUObject(), GetClassNameObject());
	}
	FTopLevelAssetPath GetClassPathObjectRedirector()
	{
		return FTopLevelAssetPath(GetScriptPackageNameCoreUObject(), GetClassNameObjectRedirector());
	}
	FTopLevelAssetPath GetClassPathBlueprintCore()
	{
		return FTopLevelAssetPath(GetScriptPackageNameEngine(), GetClassNameBlueprintCore());
	}
	FTopLevelAssetPath GetClassPathBlueprint()
	{
		return FTopLevelAssetPath(GetScriptPackageNameEngine(), GetClassNameBlueprint());
	}
	FTopLevelAssetPath GetClassPathBlueprintGeneratedClass()
	{
		return FTopLevelAssetPath(GetScriptPackageNameEngine(), GetClassNameBlueprintGeneratedClass());
	}

} // namespace UE::AssetRegistry
