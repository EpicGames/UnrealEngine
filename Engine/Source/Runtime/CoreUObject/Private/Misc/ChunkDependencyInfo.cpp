// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ChunkDependencyInfo.h"
#include "Misc/ConfigCacheIni.h"
#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Containers/Queue.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChunkDependencyInfo)

DEFINE_LOG_CATEGORY_STATIC(LogChunkDependencyInfo, Log, All);

namespace ChunkDependencyInfo
{
	// breadth first traversal
	static TArray<int32> BuildTopologicallySortedArray(const FChunkDependencyTreeNode* RootNode)
	{
		TArray<int32> OutArray;
		if (!RootNode)
		{
			return OutArray;
		}
		TQueue<const FChunkDependencyTreeNode*> Nodes;
		Nodes.Enqueue(RootNode);

		TSet<const FChunkDependencyTreeNode*> ProcessedNodes;
		const FChunkDependencyTreeNode* CurrentNode = nullptr;
		while (!Nodes.IsEmpty())
		{
			Nodes.Dequeue(CurrentNode);
			if (ProcessedNodes.Contains(CurrentNode))
			{
				continue;
			}
			ProcessedNodes.Add(CurrentNode);
			OutArray.Add(CurrentNode->ChunkID);
			for (const FChunkDependencyTreeNode& ChildNode : CurrentNode->ChildNodes)
			{
				Nodes.Enqueue(&ChildNode);
			}
		}

		return OutArray;
	}
}

UChunkDependencyInfo::UChunkDependencyInfo(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	CachedHighestChunk = -1;
}

const FChunkDependencyTreeNode* UChunkDependencyInfo::GetOrBuildChunkDependencyGraph(int32 HighestChunk, bool bForceRebuild)
{
	if (HighestChunk > CachedHighestChunk)
	{
		return BuildChunkDependencyGraph(HighestChunk);
	}
	else if (bForceRebuild)
	{
		return BuildChunkDependencyGraph(CachedHighestChunk);
	}
	return &RootTreeNode;
}

const FChunkDependencyTreeNode* UChunkDependencyInfo::BuildChunkDependencyGraph(int32 HighestChunk)
{
	// Reset any current tree
	RootTreeNode.ChunkID = 0;
	RootTreeNode.ChildNodes.Reset(0);

	ChildToParentMap.Reset();
	ChildToParentMap.Add(0);
	CachedHighestChunk = HighestChunk;

	// Ensure the DependencyArray is OK to work with.
	for (int32 DepIndex = DependencyArray.Num() - 1; DepIndex >= 0; DepIndex --)
	{
		const FChunkDependency& Dep = DependencyArray[DepIndex];
		if (Dep.ChunkID > HighestChunk)
		{
			HighestChunk = Dep.ChunkID;
		}
		if (Dep.ParentChunkID > HighestChunk)
		{
			HighestChunk = Dep.ParentChunkID;
		}
		if (Dep.ChunkID == Dep.ParentChunkID ||
			!ensureMsgf(Dep.ChunkID != 0, TEXT("Primary chunk 0 must not have a parent chunk, but ParentChunkID=%d was specified. This creates a cyclic dependency."), Dep.ParentChunkID))
		{
			// Remove cycles
			DependencyArray.RemoveAtSwap(DepIndex);
		}
	}
	// Add missing links (assumes they parent to chunk zero)
	for (int32 i = 1; i <= HighestChunk; ++i)
	{
		if (!DependencyArray.FindByPredicate([=](const FChunkDependency& RHS){ return i == RHS.ChunkID; }))
		{
			FChunkDependency Dep;
			Dep.ChunkID = i;
			Dep.ParentChunkID = 0;
			DependencyArray.Add(Dep);
		}
	}
	// Remove duplicates
	DependencyArray.StableSort([](const FChunkDependency& LHS, const FChunkDependency& RHS) { return LHS.ChunkID < RHS.ChunkID; });
	for (int32 i = 0; i < DependencyArray.Num() - 1;)
	{
		if (DependencyArray[i] == DependencyArray[i + 1])
		{
			DependencyArray.RemoveAt(i + 1);
		}
		else
		{
			++i;
		}
	}

	AddChildrenRecursive(RootTreeNode, DependencyArray, TSet<int32>());
	TopologicallySortedChunks = ChunkDependencyInfo::BuildTopologicallySortedArray(&RootTreeNode);
	return &RootTreeNode;
}

void UChunkDependencyInfo::PostInitProperties()
{
	Super::PostInitProperties();

	// Handle deprecated INI section since this class has moved from UnrealEd into CoreUObject module
	{
		TArray<FString> DeprecatedDependencyStrings;
		if (GConfig->GetArray(TEXT("/Script/UnrealEd.ChunkDependencyInfo"), TEXT("DependencyArray"), DeprecatedDependencyStrings, GEngineIni))
		{
			UE_LOGF(LogChunkDependencyInfo, Warning, "Configuration [/Script/UnrealEd.ChunkDependencyInfo]:DependencyArray is deprecated since UE 5.8; Use [/Script/CoreUObject.ChunkDependencyInfo]:DependencyArray instead!");
			if (DependencyArray.IsEmpty())
			{
				UScriptStruct* StructType = FChunkDependency::StaticStruct();
				for (const FString& Entry : DeprecatedDependencyStrings)
				{
					StructType->ImportText(*Entry, &DependencyArray.AddDefaulted_GetRef(), nullptr, PPF_None, GLog, StructType->GetName());
				}
			}
			else
			{
				UE_LOGF(LogChunkDependencyInfo, Error,
					"Conflict between deprecated config [/Script/UnrealEd.ChunkDependencyInfo]:DependencyArray (Size=%d) and new config [/Script/CoreUObject.ChunkDependencyInfo]:DependencyArray (Size=%d)",
					DeprecatedDependencyStrings.Num(), DependencyArray.Num());
			}
		}
	}
}

void UChunkDependencyInfo::AddChildrenRecursive(FChunkDependencyTreeNode& Node, TArray<FChunkDependency>& DepInfo, TSet<int32> Parents)
{
	if (Parents.Num() > 0)
	{
		ChildToParentMap.FindOrAdd(Node.ChunkID).Append(Parents);
	}

	Parents.Add(Node.ChunkID);
	auto ChildNodeIndices = DepInfo.FilterByPredicate(
		[&](const FChunkDependency& RHS) 
		{
			return Node.ChunkID == RHS.ParentChunkID;
		});
	for (const auto& ChildIndex : ChildNodeIndices)
	{
		Node.ChildNodes.Add(FChunkDependencyTreeNode(ChildIndex.ChunkID));
	}
	for (auto& Child : Node.ChildNodes) 
	{
		AddChildrenRecursive(Child, DepInfo, Parents);
	}
}

void UChunkDependencyInfo::RemoveRedundantChunks(TArray<int32>& ChunkIDs) const
{
	for (int32 ChunkIndex = ChunkIDs.Num() - 1; ChunkIndex >= 0; ChunkIndex--)
	{
		const TSet<int32>* FoundParents = ChildToParentMap.Find(ChunkIDs[ChunkIndex]);

		if (FoundParents)
		{
			for (int32 ParentChunk : *FoundParents)
			{
				if (ChunkIDs.Contains(ParentChunk))
				{
					ChunkIDs.RemoveAt(ChunkIndex);
					break;
				}
			}
		}
	}
}

int32 UChunkDependencyInfo::FindHighestSharedChunk(const TArray<int32>& ChunkIDs) const
{
	TArray<int32> TestChunkIds;
	TestChunkIds.Append(ChunkIDs);
	Algo::Sort(TestChunkIds);
	TestChunkIds.SetNum(Algo::Unique(TestChunkIds));

	for (int32 ChunkId : TestChunkIds)
	{
		if (!ChildToParentMap.Contains(ChunkId))
		{
			return INDEX_NONE;
		}
	}

	if (TestChunkIds.Num() == 0)
	{
		return INDEX_NONE;
	}
	if (TestChunkIds.Num() == 1)
	{
		return TestChunkIds[0];
	}

	TSet<int32> CommonParentSet;
	CommonParentSet.Append(ChildToParentMap[TestChunkIds[0]]);
	CommonParentSet.Add(TestChunkIds[0]);

	for (int32 Index = 1; Index < TestChunkIds.Num(); Index++)
	{
		TSet<int32> OtherSet;
		OtherSet.Append(ChildToParentMap[TestChunkIds[Index]]);
		OtherSet.Add(TestChunkIds[Index]);
		CommonParentSet = CommonParentSet.Intersect(OtherSet);
	}

	int32 HighestIndex = INDEX_NONE;
	for (int32 ChunkId : CommonParentSet)
	{
		HighestIndex = FMath::Max(HighestIndex, TopologicallySortedChunks.IndexOfByKey(ChunkId));
	}
	if (HighestIndex == INDEX_NONE)
	{
		UE_LOGF(LogChunkDependencyInfo, Error, "Unable to find parent.");
		return 0;
	}
	return TopologicallySortedChunks[HighestIndex];
}

void UChunkDependencyInfo::GetChunkDependencies(const int32 InChunk, TSet<int32>& OutChunkDependencies) const
{
	if (const TSet<int32>* Parents = ChildToParentMap.Find(InChunk))
	{
		OutChunkDependencies.Append(*Parents);
	}
}
