// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreUObjectPluginManager.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/PathViews.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/Package.h"
#include "Templates/Casts.h"
#include "UObject/UObjectHash.h"
#include "Logging/LogMacros.h"
#include "UObject/ObjectRename.h"
#include "HAL/IConsoleManager.h"
#include "UObject/ReferenceChainSearch.h"

namespace UE::CoreUObject::Private
{
	
	static bool GVerifyUnload = true;
	static FAutoConsoleVariableRef CVarVerifyPluginUnload(TEXT("PluginManager.VerifyUnload"),
		GVerifyUnload,
		TEXT("Verify plugin assets are no longer in memory when unloading."),
		ECVF_Default);

	static FAutoConsoleVariableRef CVarVerifyPluginUnloadOldName(TEXT("GameFeaturePlugin.VerifyUnload"),
		GVerifyUnload,
		TEXT("Verify plugin assets are no longer in memory when unloading. Deprecated use PluginManager.VerifyUnload instead"),
		ECVF_Default);

#if WITH_LOW_LEVEL_TESTS
	bool bEnsureOnLeakedPackages = false;
#endif

	static int32 GLeakedAssetTrace_Severity = 2;
	static int32 GLeakedAssetTrace_EditorSeverity = -1;
	static FAutoConsoleVariableRef CVarLeakedAssetTrace_Severity(
#if UE_BUILD_SHIPPING
		TEXT("PluginManager.LeakedAssetTrace.Severity.Shipping"),
#else
		TEXT("PluginManager.LeakedAssetTrace.Severity"),
#endif
		GLeakedAssetTrace_Severity,
		TEXT("Controls severity of logging when the engine detects that assets from an Game Feature Plugin were leaked during unloading or unmounting.\n")
		TEXT("0 - all reference tracing and logging is disabled\n")
		TEXT("1 - logs an error\n")
		TEXT("2 - ensure\n")
		TEXT("3 - fatal error\n"),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarLeakedAssetTrace_EditorSeverity(
		TEXT("PluginManager.LeakedAssetTrace.Severity.Editor"),
		GLeakedAssetTrace_EditorSeverity,
		TEXT("Controls severity of logging in Editor when the engine detects that assets from an Game Feature Plugin were leaked during unloading or unmounting. Overrides PluginManager.LeakedAssetTrace.Severity if set.\n")
		TEXT("0 - all reference tracing and logging is disabled\n")
		TEXT("1 - logs an error\n")
		TEXT("2 - ensure\n")
		TEXT("3 - fatal error\n"),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarLeakedAssetTrace_SeverityOldName(
#if UE_BUILD_SHIPPING
		TEXT("GameFeaturePlugin.LeakedAssetTrace.Severity.Shipping"),
#else
		TEXT("GameFeaturePlugin.LeakedAssetTrace.Severity"),
#endif
		GLeakedAssetTrace_Severity,
		TEXT("Controls severity of logging when the engine detects that assets from an Game Feature Plugin were leaked during unloading or unmounting. . Deprecated use GameFeaturePlugin.LeakedAssetTrace instead\n")
		TEXT("0 - all reference tracing and logging is disabled\n")
		TEXT("1 - logs an error\n")
		TEXT("2 - ensure\n")
		TEXT("3 - fatal error\n"),
		ECVF_Default
	);

	static bool GRenameLeakedPackages = true;
	static FAutoConsoleVariableRef CVarRenameLeakedPackages(
		TEXT("PluginManager.LeakedAssetTrace.RenameLeakedPackages"),
		GRenameLeakedPackages,
		TEXT("Should packages which are leaked after the Game Feature Plugin is unloaded or unmounted."),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarRenameLeakedPackagesOldName(
		TEXT("GameFeaturePlugin.LeakedAssetTrace.RenameLeakedPackages. Deprecated used PluginManager.LeakedAssetTrace.RenameLeakedPackages instead"),
		GRenameLeakedPackages,
		TEXT("Should packages which are leaked after the Game Feature Plugin is unloaded or unmounted."),
		ECVF_Default
	);

	static int32 GLeakedAssetTrace_TraceMode = (UE_BUILD_SHIPPING ? 0 : 1);
	static FAutoConsoleVariableRef CVarLeakedAssetTrace_TraceMode(
#if UE_BUILD_SHIPPING
		TEXT("PluginManager.LeakedAssetTrace.TraceMode.Shipping"),
#else
		TEXT("PluginManager.LeakedAssetTrace.TraceMode"),
#endif
		GLeakedAssetTrace_TraceMode,
		TEXT("Controls detail level of reference tracing when the engine detects that assets from a Game Feature Plugin were leaked during unloading or unmounting.\n")
		TEXT("0 - direct references only\n")
		TEXT("1 - full reference trace"),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarLeakedAssetTrace_TraceModeOldName(
#if UE_BUILD_SHIPPING
		TEXT("GameFeaturePlugin.LeakedAssetTrace.TraceMode.Shipping"),
#else
		TEXT("GameFeaturePlugin.LeakedAssetTrace.TraceMode"),
#endif
		GLeakedAssetTrace_TraceMode,
		TEXT("Controls detail level of reference tracing when the engine detects that assets from a Game Feature Plugin were leaked during unloading or unmounting. Deprecated used PluginManager.LeakedAssetTrace.TraceMode instead\n")
		TEXT("0 - direct references only\n")
		TEXT("1 - full reference trace"),
		ECVF_Default
	);

	static int32 GLeakedAssetTrace_MaxReportCount = 10;
	static FAutoConsoleVariableRef CVarLeakedAssetTrace_MaxReportCount(
		TEXT("PluginManager.LeakedAssetTrace.MaxReportCount"),
		GLeakedAssetTrace_MaxReportCount,
		TEXT("Max number of assets to report when we find leaked assets.\n"),
		ECVF_Default
	);

	static FAutoConsoleVariableRef CVarLeakedAssetTrace_MaxReportCountOldName(
		TEXT("GameFeaturePlugin.LeakedAssetTrace.MaxReportCount. Deprecated use PluginManager.LeakedAssetTrace.MaxReportCount instead"),
		GLeakedAssetTrace_MaxReportCount,
		TEXT("Max number of assets to report when we find leaked assets.\n"),
		ECVF_Default
	);

	static bool GLeakDetectorRunGCTwice = true;
	static FAutoConsoleVariableRef CVarLeakDetectorRunGCTwice(TEXT("PluginManager.RunGarbageCollectTwice"),
		GLeakDetectorRunGCTwice,
		TEXT("Runs GC twice so the first one clears objects from unmounted plugins ")
		TEXT("and second one clears any residue from running clear inside BeginDestroy."),
		ECVF_Default);

	DEFINE_LOG_CATEGORY_STATIC(PluginHandlerLog, Log, All);

	PluginHandler GPluginHandler;

	void PluginHandler::Install()
	{
		UE::PluginManager::Private::SetCoreUObjectPluginManager(GPluginHandler);
	}

	// Check if any assets from the plugin mount point have leaked, and if so trace them.
	// Then rename the assets and mark them as garbage to allow new copies of them to be loaded. 
	void HandlePossibleAssetLeaks(const TSet<FString>& PluginNames)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(HandlePossibleAssetLeaks);

		// Allow the editor/commandlet to customize its own severity during development.
#if !UE_BUILD_SHIPPING
#if WITH_EDITOR
		const int32 LeakedAssetSeverity = GLeakedAssetTrace_EditorSeverity >= 0 ? GLeakedAssetTrace_EditorSeverity : GLeakedAssetTrace_Severity;
#else
		const int32 LeakedAssetSeverity = GLeakedAssetTrace_EditorSeverity >= 0 && IsRunningCommandlet() ? GLeakedAssetTrace_EditorSeverity : GLeakedAssetTrace_Severity;
#endif
#else
		const int32 LeakedAssetSeverity = GLeakedAssetTrace_Severity;
#endif
		const bool bFindLeakedPackages = GVerifyUnload && (LeakedAssetSeverity != 0 || GRenameLeakedPackages);
		if (!bFindLeakedPackages)
		{
			return;
		}

		TArray<UPackage*> LeakedPackages;
		{
			// If the UObject hash knew about package mount roots, we could avoid this loop
			TRACE_CPUPROFILER_EVENT_SCOPE(PackageLoop);
			FPermanentObjectPoolExtents PermanentPool;
			ForEachObjectOfClass(UPackage::StaticClass(), [&](UObject* Obj)
				{
					if (UPackage* Package = CastChecked<UPackage>(Obj, ECastCheckedType::NullAllowed))
					{
						if (PermanentPool.Contains(Package))
						{
							return;
						}
						FNameBuilder NameBuffer(Package->GetFName());
						FStringView PluginNameOfPackage = FPathViews::GetMountPointNameFromPath(NameBuffer.ToView());
						if (PluginNames.ContainsByHash(GetTypeHash(PluginNameOfPackage), PluginNameOfPackage))
						{
							LeakedPackages.Add(Package);
						}
					}
				});
		}

		if (LeakedPackages.Num() == 0)
		{
			return;
		}

#if WITH_LOW_LEVEL_TESTS
		if (bEnsureOnLeakedPackages)
		{
			check(LeakedPackages.Num() == 0);
		}
#endif

		if (LeakedAssetSeverity != 0)
		{
			EPrintStaleReferencesOptions Options = EPrintStaleReferencesOptions::None;
			switch (LeakedAssetSeverity)
			{
			case 3:
				Options = EPrintStaleReferencesOptions::Fatal;
				break;
			case 2:
				Options = EPrintStaleReferencesOptions::Ensure | EPrintStaleReferencesOptions::Error;
				break;
			case 1:
			default:
				Options = EPrintStaleReferencesOptions::Error;
				break;
			}

			if (GLeakedAssetTrace_TraceMode == 0)
			{
				Options |= EPrintStaleReferencesOptions::Minimal;
			}

			// Make sure we are running on the game thread because we are going to be modifying
			// object flags to improve the stale reference reporting later on.
			check(IsInGameThread());

			TSet<UObject*> ObjectsThatHadStandaloneCleared;
			TArray<UObject*> ObjectsInPackage;

			TArray<UPackage*> PackagesToSearchFor;
			TMap<FString, int32> PackageCountPerMountPoint;
			PackageCountPerMountPoint.Reserve(PluginNames.Num());
			for (UPackage* Package : LeakedPackages)
			{
				ObjectsInPackage.Reset();

				// To improve the reporting for stale references we clear out the RF_Standalone flag on
				// every object in the packages we will check. The flags will be restored after the Find.
				GetObjectsWithPackage(Package, ObjectsInPackage, EGetObjectsFlags::None);

				for (UObject* Object : ObjectsInPackage)
				{
					if (Object->HasAnyFlags(RF_Standalone))
					{
						Object->ClearFlags(RF_Standalone);
						ObjectsThatHadStandaloneCleared.Add(Object);
					}
				}

				const FNameBuilder PackageName(Package->GetFName());
				const FString MountPointName(FPathViews::GetMountPointNameFromPath(PackageName.ToView()));

				if (PackageCountPerMountPoint.FindOrAdd(MountPointName)++ < GLeakedAssetTrace_MaxReportCount)
				{
					PackagesToSearchFor.Add(Package);
				}
			}

			const int32 OmittedCount = FMath::Max(0, LeakedPackages.Num() - PackagesToSearchFor.Num());
			UE_LOGF(PluginHandlerLog, Display, "Searching for references to %d leaked packages (%d omitted for speed)", LeakedPackages.Num(), OmittedCount);
			TArray<FString> ReferenceChains = FReferenceChainSearch::FindAndPrintStaleReferencesToObjects(MakeArrayView((UObject**)PackagesToSearchFor.GetData(), PackagesToSearchFor.Num()), Options);

			IPluginManager& Manager = IPluginManager::Get();
			if (Manager.OnLeakedPackagesFound().IsBound())
			{
				Manager.OnLeakedPackagesFound().Broadcast(PackageCountPerMountPoint);
			}

			for (UObject* Object : ObjectsThatHadStandaloneCleared)
			{
				Object->SetFlags(RF_Standalone);
			}
		}

		// Rename the packages that we are streaming out so that we can possibly reload another copy of them
		for (UPackage* Package : LeakedPackages)
		{
			UE_LOGF(PluginHandlerLog, Warning, "Marking leaking package %ls as Garbage", *Package->GetName());
			ForEachObjectWithPackage(Package, [](UObject* Object)
				{
					Object->MarkAsGarbage();
					return true;
				}, EGetObjectsFlags::None);

			Package->MarkAsGarbage();

			if (!UObject::IsGarbageEliminationEnabled()
#if WITH_LOW_LEVEL_TESTS
				|| GRenameLeakedPackages
#endif
				)
			{
				UE::Object::RenameLeakedPackage(Package);
			}
		}
	}

	void PluginHandler::OnPluginLoad(IPlugin& Plugin)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("CoreUObjectPluginManager OnPluginLoad");
		check(IsInGameThread());

		DeferredPluginsToGC.Remove(Plugin.GetName());
	}

	void PluginHandler::OnPluginUnload(IPlugin& Plugin)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("CoreUObjectPluginManager OnPluginUnload");
		check(IsInGameThread());

		DeferredPluginsToGC.Add(Plugin.GetName());

		if (SuppressGCRefCount == 0)
		{
			DeferGCUntilSafe();
		}
	}

	void PluginHandler::SuppressPluginUnloadGC()
	{
		check(IsInGameThread());

		++SuppressGCRefCount;
	}

	void PluginHandler::ResumePluginUnloadGC()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("CoreUObjectPluginManager ResumePluginUnloadGC");
		check(IsInGameThread());

		--SuppressGCRefCount;

		ensure(SuppressGCRefCount >= 0);

		if (SuppressGCRefCount == 0)
		{
			if (!DeferredPluginsToGC.IsEmpty())
			{
				DeferGCUntilSafe();
			}
		}
	}

	void PluginHandler::DeferGCUntilSafe()
	{
		if (!DeferredGCDelegate.IsValid())
		{
			DeferredGCDelegate = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float){
				const bool bPerformFullPurge = true;
				if (GLeakDetectorRunGCTwice)
				{
					// Run GC twice, first time to destroy objects holding StreambleHandles 
					// and second time to clear stale objects inside StreamableManager
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
				}
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, bPerformFullPurge);
				HandlePossibleAssetLeaks(DeferredPluginsToGC);
				DeferredPluginsToGC.Empty();
				DeferredGCDelegate.Reset();
				return false;
			}));
		}
	}
}