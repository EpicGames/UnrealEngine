// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ICoreUObjectPluginManager.h"
#include "Containers/Set.h"
#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"

namespace UE::CoreUObject::Private
{
	class PluginHandler : public UE::PluginManager::Private::ICoreUObjectPluginManager
	{
	public:
		static void Install();

		virtual void OnPluginLoad(IPlugin& Plugin) override;
		virtual void OnPluginUnload(IPlugin& Plugin) override;

		virtual void SuppressPluginUnloadGC() override;
		virtual void ResumePluginUnloadGC() override;

		virtual ~PluginHandler() = default;

	private:

		void DeferGCUntilSafe();

		TSet<FString> DeferredPluginsToGC;
		FTSTicker::FDelegateHandle DeferredGCDelegate;

		/** Ref count for deferring calls to OnPluginUnload. When the ref count reaches 0 we GC and leak test all deferred plugins */
		int32 SuppressGCRefCount = 0;
	};
}
