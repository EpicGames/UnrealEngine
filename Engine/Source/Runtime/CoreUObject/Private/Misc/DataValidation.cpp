// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DataValidation.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataToken.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataValidation)

namespace UE::DataValidation
{
	const FName MessageLogName("AssetCheck");
}

EDataValidationResult CombineDataValidationResults(EDataValidationResult Result1, EDataValidationResult Result2)
{
	/**
	 * Anything combined with an Invalid result is Invalid. Any result combined with a NotValidated result is the same result
	 *
	 * The combined results should match the following matrix
	 *
	 *				|	NotValidated	|	Valid	|	Invalid
	 * -------------+-------------------+-----------+----------
	 * NotValidated	|	NotValidated	|	Valid	|	Invalid
	 * Valid		|	Valid			|	Valid	|	Invalid
	 * Invalid		|	Invalid			|	Invalid	|	Invalid
	 *
	 */

	if (Result1 == EDataValidationResult::Invalid || Result2 == EDataValidationResult::Invalid)
	{
		return EDataValidationResult::Invalid;
	}

	if (Result1 == EDataValidationResult::Valid || Result2 == EDataValidationResult::Valid)
	{
		return EDataValidationResult::Valid;
	}

	return EDataValidationResult::NotValidated;
}
	
void FDataValidationContext::AddMessage(TSharedRef<FTokenizedMessage> InMessage)
{
	switch(InMessage->GetSeverity())
	{
	case EMessageSeverity::Error:
		++NumErrors;
		break;
	case EMessageSeverity::Warning:
	case EMessageSeverity::PerformanceWarning:
		++NumWarnings;
		break;
	case EMessageSeverity::Info:
	default:
		break;
	}	
	Issues.Emplace(InMessage);
}

TSharedRef<FTokenizedMessage> FDataValidationContext::AddMessage(const FAssetData& ForAsset, EMessageSeverity::Type InSeverity, FText InText)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(InSeverity);
	if (ForAsset.IsValid())
	{
		Message->AddToken(FAssetDataToken::Create(ForAsset));
	}
	if (!InText.IsEmpty())
	{
		Message->AddText(InText);
	}
	AddMessage(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FDataValidationContext::AddMessage(EMessageSeverity::Type InSeverity, FText InText)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(InSeverity, InText);
	AddMessage(Message);
	return Message;
}

void FDataValidationContext::SplitIssues(TArray<FText>& Warnings, TArray<FText>& Errors, TArray<TSharedRef<FTokenizedMessage>>* Messages) const
{
	for (const FIssue& Issue : GetIssues())
	{
		if (Issue.Severity == EMessageSeverity::Error || Issue.Severity == EMessageSeverity::Warning)
		{
			FText Message = Issue.Message;
			if (Issue.TokenizedMessage)
			{
				Message = Issue.TokenizedMessage->ToText();
			}

			if (Issue.Severity == EMessageSeverity::Error)
			{
				Errors.Add(MoveTemp(Message));
			}
			else if (Issue.Severity == EMessageSeverity::Warning)
			{
				Warnings.Add(MoveTemp(Message));
			}
		}

		if (Messages)
		{
			if (Issue.TokenizedMessage)
			{
				Messages->Add(Issue.TokenizedMessage.ToSharedRef());
			}
			else
			{
				Messages->Add(FTokenizedMessage::Create(Issue.Severity, Issue.Message));
			}
		}
	}
}
