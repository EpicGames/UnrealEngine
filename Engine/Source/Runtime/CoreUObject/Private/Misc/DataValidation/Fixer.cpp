// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/DataValidation/Fixer.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "DataValidationFixers"

namespace UE::DataValidation
{

TSharedRef<FFixToken> IFixer::CreateToken(const FText& Label)
{
	return FFixToken::Create(FText::Format(LOCTEXT("SingleFix", "Fix: {0}"), Label), AsShared(), 0);
}

}

#undef LOCTEXT_NAMESPACE
