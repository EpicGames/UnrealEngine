// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/EditorPathHelper.h"

#if WITH_EDITOR

#include "Misc/EditorPathObjectInterface.h"
#include "UObject/TopLevelAssetPath.h"
#include "HAL/IConsoleManager.h"

COREUOBJECT_API bool GIsEditorPathFeatureEnabled = false;
static FAutoConsoleVariableRef CVarIsEditorPathFeatureEnabled(
	TEXT("Editorpaths.Enabled"),
	GIsEditorPathFeatureEnabled,
	TEXT("Enable experimental Editor Path support."));

bool FEditorPathHelper::IsEnabled()
{
	return GIsEditorPathFeatureEnabled;
}

UObject* FEditorPathHelper::GetEditorPathOwner(const UObject* InObject)
{
	if (IsEnabled() && InObject)
	{
		if (IEditorPathObjectInterface* EditorPathObject = InObject->GetImplementingOuter<IEditorPathObjectInterface>())
		{
			return EditorPathObject->GetEditorPathOwner();
		}
	}

	return nullptr;
}

bool FEditorPathHelper::IsInEditorPath(const UObject* InEditorPathOwner, const UObject* InObject)
{
	if (InEditorPathOwner)
	{
		while (UObject* CurrentEditorPathOwner = FEditorPathHelper::GetEditorPathOwner(InObject))
		{
			if (CurrentEditorPathOwner == InEditorPathOwner)
			{
				return true;
			}

			InObject = CurrentEditorPathOwner;
		}
	}

	return false;
}

FSoftObjectPath FEditorPathHelper::GetEditorPath(const UObject* InObject)
{
	return GetEditorPathFromEditorPathOwner(InObject, nullptr);
}

FSoftObjectPath FEditorPathHelper::GetEditorPathFromReferencer(const UObject* InObject, const UObject* InReferencer)
{
	return GetEditorPathFromEditorPathOwner(InObject, GetEditorPathOwner(InReferencer));
}

FSoftObjectPath FEditorPathHelper::GetEditorPathFromEditorPathOwner(const UObject* InObject, const UObject* InEditorPathOwner)
{
	check(InObject);
	FSoftObjectPath SoftObjectPath(InObject);
	if (!IsEnabled())
	{
		return SoftObjectPath;
	}
	
	UObject* EditorPathOwner = GetEditorPathOwner(InObject);
	
	if (EditorPathOwner && EditorPathOwner != InEditorPathOwner)
	{
		TArray<FString> EditorPathOwnerNames;
		FString SubObjectString;
		FString SubObjectContext;
		check(SoftObjectPath.GetSubPathString().Split(TEXT("."), &SubObjectContext, &SubObjectString));
		check(SubObjectContext == TEXT("PersistentLevel"));

		EditorPathOwnerNames.Add(EditorPathOwner->GetName());

		while (UObject* ParentEditorPathOwner = GetEditorPathOwner(EditorPathOwner))
		{
			// EditorPath needs to be relative to InEditorPathOwner in case we have a valid InEditorPathOwner
			if (ParentEditorPathOwner == InEditorPathOwner)
			{
				break;
			}

			EditorPathOwner = ParentEditorPathOwner;
			EditorPathOwnerNames.Add(ParentEditorPathOwner->GetName());
		}

		TStringBuilder<512> SubPathStringBuilder;
		SubPathStringBuilder.Append(SubObjectContext);
		for (int i = EditorPathOwnerNames.Num() - 1; i >= 0; --i)
		{
			SubPathStringBuilder.Append(TEXT("."));
			SubPathStringBuilder.Append(EditorPathOwnerNames[i]);
		}

		if (!SubObjectString.IsEmpty())
		{
			SubPathStringBuilder.Append(TEXT("."));
			SubPathStringBuilder.Append(SubObjectString);
		}

		return FSoftObjectPath(FTopLevelAssetPath(EditorPathOwner->GetOutermostObject()), SubPathStringBuilder.ToString());
	}

	return SoftObjectPath;
}

#endif