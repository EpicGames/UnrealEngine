// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ExclusiveLoadPackageTimeTracker.h"
#include "Misc/OutputDeviceArchiveWrapper.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Misc/ScopeLock.h"
#include "UObject/UObjectHash.h"
#include "UObject/Class.h"

#if WITH_LOADPACKAGE_TIME_TRACKER

#define LOCTEXT_NAMESPACE "ExclusiveLoadPackageTimeTracker"

FExclusiveLoadPackageTimeTracker* FExclusiveLoadPackageTimeTracker::TrackerInstance;

FExclusiveLoadPackageTimeTracker::FExclusiveLoadPackageTimeTracker()
	: TrackerOverhead(0.f)
	, EndLoadName(FName("EndLoad"))
	, UnknownAssetName(FName("Unknown"))
	, DumpReportCommand(
		TEXT("LoadTimes.DumpReport"),
		*LOCTEXT("CommandText_DumpReport", "Dumps a report about the amount of time spent loading assets").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FExclusiveLoadPackageTimeTracker::DumpReportCommandHandler))
	, ResetReportCommand(
		TEXT("LoadTimes.Reset"),
		*LOCTEXT("CommandText_ResetReport", "Resets accumulated report data").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FExclusiveLoadPackageTimeTracker::ResetReportCommandHandler))
{
}

FExclusiveLoadPackageTimeTracker& FExclusiveLoadPackageTimeTracker::Construct()
{
	static FExclusiveLoadPackageTimeTracker Tracker;
	TrackerInstance = &Tracker;
	return Tracker;
}

void FExclusiveLoadPackageTimeTracker::InternalPushLoadPackage(FName PackageName)
{
	FScopeLock Lock(&TimesCritical);
	const double CurrentTime = FPlatformTime::Seconds();

	if (TimeStack.Num() > 0)
	{
		FLoadTime& Time = TimeStack.Last();
		Time.ExclusiveTime += (CurrentTime - Time.LastStartTime);
	}

	TimeStack.Push(FLoadTime(PackageName, CurrentTime));

	TrackerOverhead += (FPlatformTime::Seconds() - CurrentTime);
}

void FExclusiveLoadPackageTimeTracker::InternalPopLoadPackage(UPackage* LoadedPackage, UObject* LoadedAsset)
{
	// Resolve the classname first to avoid having to take a lock on the object hash table inside another lock
	// which can cause deadlock.
	FName ClassName = UnknownAssetName;
	if (LoadedAsset)
	{
		ClassName = LoadedAsset->GetClass()->GetFName();
	}
	else if (LoadedPackage)
	{
		ForEachObjectWithPackage(LoadedPackage, [&ClassName](UObject* Object)
		{
			if (Object && Object->IsAsset())
			{
				ClassName = Object->GetClass()->GetFName();
				return false;
			}
			return true;
		}, EGetObjectsFlags::None);
	}

	FScopeLock Lock(&TimesCritical);
	if (ensure(TimeStack.Num() > 0))
	{
		const double CurrentTime = FPlatformTime::Seconds();
		FLoadTime& Time = TimeStack.Last();
		Time.ExclusiveTime += (CurrentTime - Time.LastStartTime);

		double InclusiveTime = CurrentTime - Time.OriginalStartTime;
		FLoadTime* LoadTimePtr = LoadTimes.Find(Time.TimeName);
		if ( LoadTimePtr )
		{
			// Use the most recently discovered classname. This may be "Unknown" during async loading but future pops will correct it.
			LoadTimePtr->AssetClass = ClassName;
			LoadTimePtr->ExclusiveTime += Time.ExclusiveTime;
			LoadTimePtr->InclusiveTime += InclusiveTime;
		}
		else
		{
			LoadTimes.Add(Time.TimeName, FLoadTime(Time.TimeName, ClassName, Time.ExclusiveTime, InclusiveTime));
		}

		TimeStack.RemoveAt(TimeStack.Num() - 1, EAllowShrinking::No);

		if (TimeStack.Num() > 0)
		{
			const double NewCurrentTime = FPlatformTime::Seconds();
			FLoadTime& InnerTime = TimeStack.Last();
			InnerTime.LastStartTime = NewCurrentTime;
		}

		TrackerOverhead += (FPlatformTime::Seconds() - CurrentTime);
	}
}

void FExclusiveLoadPackageTimeTracker::InternalDumpReport(const TArray<FString>& Args) const
{
	FScopeLock Lock(&TimesCritical);

	const bool bLogOutputToFile = Args.ContainsByPredicate([](const FString& Arg) { return Arg.Compare(TEXT("FILE"), ESearchCase::IgnoreCase) == 0; });
	const bool bAlphaSort       = Args.ContainsByPredicate([](const FString& Arg) { return Arg.Compare(TEXT("-ALPHASORT"), ESearchCase::IgnoreCase) == 0; });

	FOutputDevice* ReportAr = GLog;
	FArchive* FileAr = nullptr;
	FOutputDeviceArchiveWrapper* FileArWrapper = nullptr;
	FString FilenameFull;

	if (bLogOutputToFile)
	{
		const FString PathName = *(FPaths::ProfilingDir() + TEXT("LoadReports/"));
		IFileManager::Get().MakeDirectory(*PathName);

		const FString Filename = CreateProfileFilename(TEXT(""), TEXT(".loadreport"), true);
		FilenameFull = PathName + Filename;

#if ALLOW_DEBUG_FILES
		FileAr = IFileManager::Get().CreateDebugFileWriter(*FilenameFull);
#else
		FileAr = IFileManager::Get().CreateFileWriter(*FilenameFull);
#endif
		FileArWrapper = new FOutputDeviceArchiveWrapper(FileAr);
		ReportAr = FileArWrapper;

		UE_LOGF(LogLoad, Log, "LoadTimes.DumpReport: saving to %ls", *FilenameFull);
	}

	/** Struct to assemble times for assets loaded by class */
	struct FTimeCount
	{
		FName AssetClass;
		double ExclusiveTime;
		int32 Count;

		FTimeCount() : ExclusiveTime(0), Count(0) {}
	};

	double LongestLoadTime = 0;
	double SlowAssetTime = 0;
	FName LongestLoadName;
	double TotalLoadTime = 0;
	const double SlowAssetThreshold = 0.10f;
	TMap<FName, FTimeCount> AssetTypeLoadTimes;
	for (const auto& LoadTimeIt : LoadTimes)
	{
		const FLoadTime& Time = LoadTimeIt.Value;
		if (IsPackageLoadTime(Time))
		{
			if (Time.ExclusiveTime > LongestLoadTime)
			{
				LongestLoadName = LoadTimeIt.Key;
				LongestLoadTime = Time.ExclusiveTime;
			}

			if (Time.ExclusiveTime > SlowAssetThreshold)
			{
				SlowAssetTime += Time.ExclusiveTime;
			}

			FTimeCount& TypeTimeCount = AssetTypeLoadTimes.FindOrAdd(Time.AssetClass);
			TypeTimeCount.AssetClass = Time.AssetClass;
			TypeTimeCount.ExclusiveTime += Time.ExclusiveTime;
			TypeTimeCount.Count++;
		}

		TotalLoadTime += Time.ExclusiveTime;
	}

	const FLoadTime* EndLoadTime = LoadTimes.Find(EndLoadName);
	const int32 NumNonAssetTimes = EndLoadTime ? 1 : 0;
	ReportAr->Logf(TEXT("Loaded: %d packages"), LoadTimes.Num() - NumNonAssetTimes);
	ReportAr->Logf(TEXT("Total time loading packages: %.3f seconds"), TotalLoadTime);
	ReportAr->Logf(TEXT("Time spent loading assets slower than %.1fms: %.3f seconds"), SlowAssetThreshold * 1000, SlowAssetTime);
	ReportAr->Logf(TEXT("Slowest asset: %s (%.1fms)"), *LongestLoadName.ToString(), LongestLoadTime * 1000);
	ReportAr->Logf(TEXT("Time spent in EndLoad: %.3f seconds"), EndLoadTime ? EndLoadTime->ExclusiveTime : 0.f);
	ReportAr->Logf(TEXT("Time spent in overhead tracking asset load times: %.6f seconds"), TrackerOverhead);

	ReportAr->Logf(TEXT("Dumping asset type load times sorted by exclusive time:"));
	TArray<FTimeCount> SortedAssetTypeLoadTimes;
	AssetTypeLoadTimes.GenerateValueArray(SortedAssetTypeLoadTimes);
	SortedAssetTypeLoadTimes.Sort([](const FTimeCount& A, const FTimeCount& B){ return A.ExclusiveTime >= B.ExclusiveTime; });
	for (const FTimeCount& TimeCount : SortedAssetTypeLoadTimes)
	{
		ReportAr->Logf(TEXT("    %.3f: %s (%d packages, %.1fms per package)"), TimeCount.ExclusiveTime, *TimeCount.AssetClass.ToString(), TimeCount.Count, TimeCount.ExclusiveTime / TimeCount.Count * 1000);
	}
	
	// Assets faster than this will be excluded
	double LowTimeThreshold = 0.05;

	const FString* LowTimeArg = Args.FindByPredicate([](const FString& Arg) { return Arg.StartsWith(TEXT("LOWTIME="), ESearchCase::IgnoreCase); });
	if (LowTimeArg != nullptr)
	{
		float LowTimeThresholdParam = 0.0F;
		if (FParse::Value(**LowTimeArg, TEXT("LOWTIME="), LowTimeThresholdParam))
		{
			LowTimeThreshold = (double)LowTimeThresholdParam;
		}
		
	}

	TArray<FLoadTime> SortedLoadTimes;
	LoadTimes.GenerateValueArray(SortedLoadTimes);
	
	{
		ReportAr->Logf(TEXT("Dumping all loaded assets by exclusive load time:"));
		if (bAlphaSort)
		{
			SortedLoadTimes.Sort([](const FLoadTime& A, const FLoadTime& B) { return A.TimeName.LexicalLess(B.TimeName); });
		}
		else
		{
			SortedLoadTimes.Sort([](const FLoadTime& A, const FLoadTime& B) { return A.ExclusiveTime >= B.ExclusiveTime; });
		}

		int32 LowThresholdCount = 0;
		double TotalLowTime = 0;
		for (int32 LoadTimeIdx = 0; LoadTimeIdx < SortedLoadTimes.Num(); ++LoadTimeIdx)
		{
			const FLoadTime& LoadTime = SortedLoadTimes[LoadTimeIdx];
			if (IsPackageLoadTime(LoadTime))
			{
				if (LoadTime.ExclusiveTime > LowTimeThreshold)
				{
					ReportAr->Logf(TEXT("    %.1fms: %s"), LoadTime.ExclusiveTime * 1000, *LoadTime.TimeName.ToString());
				}
				else
				{
					LowThresholdCount++;
					TotalLowTime += LoadTime.ExclusiveTime;
				}
			}
		}

		if (LowThresholdCount > 0)
		{
			ReportAr->Logf(TEXT("    ... skipped %d assets that loaded in less than %.1fms totaling %.1fms"), LowThresholdCount, LowTimeThreshold * 1000, TotalLowTime * 1000);
		}
	}

	{
		ReportAr->Logf(TEXT("Dumping all loaded assets by inclusive load time:"));
		if (bAlphaSort)
		{
			SortedLoadTimes.Sort([](const FLoadTime& A, const FLoadTime& B) { return A.TimeName.LexicalLess(B.TimeName); });
		}
		else
		{
			SortedLoadTimes.Sort([](const FLoadTime& A, const FLoadTime& B) { return A.InclusiveTime >= B.InclusiveTime; });
		}
		
		int32 LowThresholdCount = 0;
		double TotalLowTime = 0;
		for (int32 LoadTimeIdx = 0; LoadTimeIdx < SortedLoadTimes.Num(); ++LoadTimeIdx)
		{
			const FLoadTime& LoadTime = SortedLoadTimes[LoadTimeIdx];
			if (IsPackageLoadTime(LoadTime))
			{
				if (LoadTime.InclusiveTime > LowTimeThreshold)
				{
					ReportAr->Logf(TEXT("    %.1fms: %s"), LoadTime.InclusiveTime * 1000, *LoadTime.TimeName.ToString());
				}
				else
				{
					LowThresholdCount++;
					TotalLowTime += LoadTime.InclusiveTime;
				}
			}
		}

		if (LowThresholdCount > 0)
		{
			ReportAr->Logf(TEXT("    ... skipped %d assets slower than %.1fms totaling %.1fms"), LowThresholdCount, LowTimeThreshold * 1000, TotalLowTime * 1000);
		}
	}

	if (FileArWrapper != nullptr)
	{
		FileArWrapper->TearDown();
		delete FileArWrapper;
		delete FileAr;
	}
}

void FExclusiveLoadPackageTimeTracker::InternalResetReport()
{
	FScopeLock Lock(&TimesCritical);
	LoadTimes.Reset();
	TimeStack.Reset();
}

double FExclusiveLoadPackageTimeTracker::InternalGetExclusiveLoadTime(FName PackageName) const
{
	FScopeLock Lock(&TimesCritical);
	const FLoadTime* LoadTime = LoadTimes.Find(PackageName);
	if (LoadTime)
	{
		return LoadTime->ExclusiveTime;
	}
	
	return -1;
}

double FExclusiveLoadPackageTimeTracker::InternalGetInclusiveLoadTime(FName PackageName) const
{
	FScopeLock Lock(&TimesCritical);
	const FLoadTime* LoadTime = LoadTimes.Find(PackageName);
	if (LoadTime)
	{
		return LoadTime->InclusiveTime;
	}

	return -1;
}

bool FExclusiveLoadPackageTimeTracker::IsPackageLoadTime(const FLoadTime& Time) const
{
	return Time.TimeName != EndLoadName;
}

void FExclusiveLoadPackageTimeTracker::DumpReportCommandHandler(const TArray<FString>& Args)
{
	DumpReport(Args);
}

void FExclusiveLoadPackageTimeTracker::ResetReportCommandHandler(const TArray<FString>& Args)
{
	ResetReport();
}


#undef LOCTEXT_NAMESPACE

#endif //WITH_LOADPACKAGE_TIME_TRACKER
