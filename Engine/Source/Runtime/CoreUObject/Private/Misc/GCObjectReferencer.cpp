// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GCObjectReferencer.cpp: Implementation of UGCObjectReferencer
=============================================================================*/

#include "AutoRTFM.h"
#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Templates/Casts.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GarbageCollectionVerification.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "VerseVM/VVMHeap.h"

#include <atomic>

// Global GC state flags
extern std::atomic<bool> GObjIncrementalPurgeIsInProgress;
extern std::atomic<bool> GObjUnhashUnreachableIsInProgress;

namespace UE::GC
{
	inline constexpr FStringView UnknownGCObjectName = TEXTVIEW("Unknown FGCObject");
}

struct UGCObjectReferencer::FImpl
{
	/** Critical section used when adding and removing objects */
	FTransactionallySafeCriticalSection ReferencedObjectsCritical;
	/** Objects without EFlags::AddStableNativeReferencesOnly */
	TArray<FGCObject*> RemainingReferencedObjects;
	/** Objects with EFlags::AddStableNativeReferencesOnly */
	TArray<FGCObject*> InitialReferencedObjects;
};


UGCObjectReferencer::UGCObjectReferencer(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
, Impl(new FImpl)
{}

UGCObjectReferencer::UGCObjectReferencer(FVTableHelper& Helper)
: Super(Helper)
, Impl(new FImpl)
{}

UGCObjectReferencer::~UGCObjectReferencer() = default;

void UGCObjectReferencer::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UGCObjectReferencer* This = CastChecked<UGCObjectReferencer>(InThis);

	check(!This->CurrentlySerializingObject);

	// Only GC protects us against adding new UObjects, but this is called through other means outside of GC that needs to be protected.
	UE::TConditionalScopeLock ReferencedObjectsLock(This->Impl->ReferencedObjectsCritical, !IsGarbageCollecting());
	// Let each registered object handle its AddReferencedObjects call
	
	if (Collector.NeedsInitialReferences())
	{
		for (FGCObject* Object : This->Impl->InitialReferencedObjects)
		{
			This->CurrentlySerializingObject = Object;
			Object->AddReferencedObjects(Collector);
		}
	}

	for (FGCObject* Object : This->Impl->RemainingReferencedObjects)
	{
		This->CurrentlySerializingObject = Object;
		Object->AddReferencedObjects(Collector);
	}

	This->CurrentlySerializingObject = nullptr;
	Super::AddReferencedObjects(This, Collector);
}

static bool IsUnrealGCMarkingPassInProgress()
{
	return !GObjUnhashUnreachableIsInProgress && !GObjIncrementalPurgeIsInProgress && IsGarbageCollectingAndLockingUObjectHashTables();
}

static bool IsFrankenGCDestroyingPassInProgress()
{
#if WITH_VERSE_VM
	// The FrankenGC object-destruction pass occurs immediately after the marking 
	// pass of both Verse and Unreal has finished and reached a steady-state. 
	// Unfortunately `IsUnrealGCMarkingPassInProgress` still returns true at that 
	// point, even though all marking has actually concluded. 
	// We detect this case here.
	return UE::GC::GIsFrankenGCCollecting && Verse::FHeap::IsDestroying();
#else
	return false;
#endif
}

void UGCObjectReferencer::AddObject(FGCObject* Object)
{
	check(Object);
	check(!IsUnrealGCMarkingPassInProgress() || IsFrankenGCDestroyingPassInProgress());

	UE::TScopeLock ReferencedObjectsLock(Impl->ReferencedObjectsCritical);

	TArray<FGCObject*>* ReferencedObjects = Object->bCanMakeInitialReferences ? &Impl->InitialReferencedObjects : &Impl->RemainingReferencedObjects;

	// Make sure there are no duplicates. Should be impossible...
	checkSlow(!ReferencedObjects->Contains(Object));
	ReferencedObjects->Add(Object);

#if WITH_EDITORONLY_DATA
	OnGCObjectAdded.Broadcast(Object);
#endif // WITH_EDITORONLY_DATA
}

void UGCObjectReferencer::RemoveObject(FGCObject* Object)
{
	check(Object);
	check(!IsUnrealGCMarkingPassInProgress() || IsFrankenGCDestroyingPassInProgress());
	
	UE::TScopeLock ReferencedObjectsLock(Impl->ReferencedObjectsCritical);

	TArray<FGCObject*>* ReferencedObjects = Object->bCanMakeInitialReferences ? &Impl->InitialReferencedObjects : &Impl->RemainingReferencedObjects;
	int32 NumRemoved = ReferencedObjects->RemoveSingleSwap(Object);
	check(NumRemoved == 1);
}

bool UGCObjectReferencer::GetReferencerName(UObject* Object, FString& OutName, bool bOnlyIfAddingReferenced) const
{
	if (bOnlyIfAddingReferenced)
	{
		if (!CurrentlySerializingObject)
		{
			return false;
		}
		OutName = CurrentlySerializingObject->GetReferencerName();
		FString ReferencerProperty;
		if (CurrentlySerializingObject->GetReferencerPropertyName(Object, ReferencerProperty))
		{
			OutName += TEXT(":") + ReferencerProperty;
		}
		return true;
	}

	// Let each registered object handle its AddReferencedObjects call
	for (TArrayView<FGCObject*> ReferencedObjects : { MakeArrayView(Impl->RemainingReferencedObjects),
													  MakeArrayView(Impl->InitialReferencedObjects) })
	{
		for (FGCObject* GCReporter : ReferencedObjects)
		{
			TArray<UObject*> ObjectArray;
			FReferenceFinder Collector(ObjectArray);

			check(GCReporter);
			GCReporter->AddReferencedObjects(Collector);

			if (ObjectArray.Contains(Object))
			{
				OutName = GCReporter->GetReferencerName();
				FString ReferencerProperty;
				if (GCReporter->GetReferencerPropertyName(Object, ReferencerProperty))
				{
					OutName += TEXT(":") + ReferencerProperty;
				}
				return true;
			}
		}
	}

	return false;
}

void UGCObjectReferencer::FinishDestroy()
{
	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		// Make sure FGCObjects that are around after exit purge don't
		// reference this object.
		check( FGCObject::GGCObjectReferencer == this );
		FGCObject::GGCObjectReferencer = nullptr;
		Impl.Reset();
	}

	Super::FinishDestroy();
}


class FInitialReferenceCollector final : public FReferenceCollector
{
	TArray<UObject**>& Result;

	virtual void AddStableReference(UObject** Object) override
	{
		Result.Add(Object);
	}

	virtual void AddStableReferenceArray(TArray<UObject*>* Objects) override
	{
		for (UObject*& Object : *Objects)
		{
			Result.Add(&Object);
		}
	}

	virtual void AddStableReferenceSet(TSet<UObject*>* Objects) override
	{
		for (UObject*& Object : *Objects)
		{
			Result.Add(&Object);
		}
	}
	
	virtual void AddStableReference(TObjectPtr<UObject>* Object) override
	{
		Result.Add(&UE::Core::Private::Unsafe::Decay(*Object));
	}

	virtual void AddStableReferenceArray(TArray<TObjectPtr<UObject>>* Objects) override
	{
		for (TObjectPtr<UObject>& Object : *Objects)
		{
			Result.Add(&UE::Core::Private::Unsafe::Decay(Object));
		}
	}

	virtual void AddStableReferenceSet(TSet<TObjectPtr<UObject>>* Objects) override
	{
		for (TObjectPtr<UObject>& Object : *Objects)
		{
			Result.Add(&UE::Core::Private::Unsafe::Decay(Object));
		}
	}
  
	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		checkf(false, TEXT("FGCObject constructed with AddStableNativeReferencesOnly should only call AddStableReference, not HandleObjectReference"));
	}
  
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual void HandleVCellReference(Verse::VCell* InCell, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		checkf(false, TEXT("FGCObject constructed with AddStableNativeReferencesOnly should only call AddStableReference, not HandleVCellReference"));
	}
#endif

	virtual void SetIsProcessingNativeReferences(bool) override
	{
		checkf(false, TEXT("FGCObject constructed with AddStableNativeReferencesOnly should never flip to non-native references"));
	}

	virtual bool IsIgnoringArchetypeRef() const override { return false;}
	virtual bool IsIgnoringTransient() const override {	return false; }

public:
	FInitialReferenceCollector(TArray<UObject**>& Out) : Result(Out) {}
};


void UGCObjectReferencer::AddInitialReferences(TArray<UObject**>& Out)
{
	FInitialReferenceCollector Collector(Out);
	for (FGCObject* Object : Impl->InitialReferencedObjects)
	{
		Object->AddReferencedObjects(Collector);
	}
}

UGCObjectReferencer* FGCObject::GGCObjectReferencer = nullptr;

// This variable allows inline declaration of GetReferencerName(), which
// works around a Linux/.so undefined typeinfo link error for plugins using -frtti
const TCHAR* FGCObject::UnknownGCObjectName = UE::GC::UnknownGCObjectName.GetData();

UE_AUTORTFM_ALWAYS_OPEN
void FGCObject::StaticInit()
{
	if (GGCObjectReferencer == nullptr)
	{
		GGCObjectReferencer = NewObject<UGCObjectReferencer>();
		GGCObjectReferencer->AddToRoot();
	}
}

void FGCObject::RegisterGCObject()
{
	// Some objects can get created after the engine started shutting down (lazy init of singletons etc).
	if (!IsEngineExitRequested() && !bReferenceAdded)
	{
		StaticInit();

		GGCObjectReferencer->AddObject(this);

		bReferenceAdded = true;
	}
}

void FGCObject::UnregisterGCObject()
{
	// GObjectSerializer will be NULL if this object gets destroyed after the exit purge.
	// We want to make sure we remove any objects that were added to the GGCObjectReferencer during Init when exiting
	if (GGCObjectReferencer && bReferenceAdded)
	{
		GGCObjectReferencer->RemoveObject(this);

		bReferenceAdded = false;
	}
}
