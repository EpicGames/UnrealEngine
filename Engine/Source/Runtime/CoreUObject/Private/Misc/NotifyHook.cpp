// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnMisc.cpp: Various core platform-independent functions.
=============================================================================*/

// Core includes.
#include "Misc/NotifyHook.h"
#include "UObject/UnrealType.h"

void FNotifyHook::NotifyPreChange( FEditPropertyChain* PropertyAboutToChange )
{
	NotifyPreChange( PropertyAboutToChange != NULL && PropertyAboutToChange->Num() > 0 ? PropertyAboutToChange->GetActiveNode()->GetValue() : NULL );
}

void FNotifyHook::NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged )
{
	NotifyPostChange( PropertyChangedEvent, PropertyThatChanged != NULL && PropertyThatChanged->Num() > 0 ? PropertyThatChanged->GetActiveNode()->GetValue() : NULL );
}


