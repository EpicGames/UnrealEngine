// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/PackageAccessTracking.h"

#include "Async/InheritedContext.h"
#include "Cooker/CookDependency.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageName.h"
#include "UObject/Object.h"
#include "UObject/Package.h"

#if UE_WITH_PACKAGE_ACCESS_TRACKING
thread_local PackageAccessTracking_Private::FPackageAccessRefScope* PackageAccessTracking_Private::FPackageAccessRefScope::CurrentThreadScope = nullptr;

namespace PackageAccessTracking_Private
{

bool IsBuildOpName(FName OpName)
{
	//If this was a Package Access done in the context of a save or load, we record it as a build dependency.
	//If we want to capture script package dependencies, we may also need to record accesses under the "PackageAccessTrackingOps::NAME_CreateDefaultObject" operation
	//which occurs from "UObjectLoadAllCompiledInDefaultProperties" and possibly elsewhere.
	return (OpName == PackageAccessTrackingOps::NAME_Load)
		|| (OpName == PackageAccessTrackingOps::NAME_PreLoad)
		|| (OpName == PackageAccessTrackingOps::NAME_PostLoad)
		|| (OpName == PackageAccessTrackingOps::NAME_Save)
		|| (OpName == PackageAccessTrackingOps::NAME_CookerBuildObject)
		// || (OpName == PackageAccessTrackingOps::NAME_CreateDefaultObject)
		;
}

namespace
{
	struct FCapturedPackageTrackingData
	{
		FName PackageName;
		FName OpName;
		FName BuildOpName;
		FName CookResultProjection;
		const UObject* Object;          // TODO: may need FWeakObjectPtr if GC becomes problematic
		const ITargetPlatform* TargetPlatform;
		bool bHasData;
	};
}

FTrackedData::FTrackedData(FName InPackageName, FName InOpName, FName InCookResultProjection,
	const ITargetPlatform* InTargetPlatform, const UObject* InObject)
	: PackageName(InPackageName)
	, OpName(InOpName)
	, BuildOpName(IsBuildOpName(InOpName) ? InOpName : NAME_None)
	, CookResultProjection(InCookResultProjection)
	, Object(InObject)
	, TargetPlatform(InTargetPlatform)
{
}

FTrackedData::FTrackedData(FName InPackageName, FName InOpName, FName InBuildOpName,
	FName InCookResultProjection, const ITargetPlatform* InTargetPlatform, const UObject* InObject)
	: PackageName(InPackageName)
	, OpName(InOpName)
	, BuildOpName(InBuildOpName)
	, CookResultProjection(InCookResultProjection)
	, Object(InObject)
	, TargetPlatform(InTargetPlatform)
{
}

FTrackedData::FTrackedData(FTrackedData& DirectData, FTrackedData* OuterAccumulatedData)
{
	if (OuterAccumulatedData)
	{
		// NOTE: The accumulated PackageName is used as the referencer
		// The referencer is always the PackageName from the latest scope with a PackageName, NOT the PackageName from the latest scope with a BuildOpName.
		// Consider a situation where in the process of loading package A,
		// we call PostLoad on an object on package B which references an object in package C.  The SearchThreadScope with OpName == Load is for package A,
		// but we want to record that B depends on C, not that A depends on C.  The InnermostThreadScope will have a package name of B and an OpName of PostLoad
		// and so we want the package name from the InnermostThreadScope while having searched upwards for a scope with an OpName == Load.

		// The accumulated PackageName is the most recent non-empty PackageName
		PackageName = !DirectData.PackageName.IsNone() ? DirectData.PackageName : OuterAccumulatedData->PackageName;

		// The accumulated OpName is the most recent non-empty OpName
		OpName = !DirectData.OpName.IsNone() ? DirectData.OpName : OuterAccumulatedData->OpName;

		CookResultProjection = !DirectData.CookResultProjection.IsNone() ?
			DirectData.CookResultProjection : OuterAccumulatedData->CookResultProjection;

		// The accumulated BuildOpName is by default the most recent non-empty BuildOpName
		// ResetContext ops set it back to none
		if (OpName == PackageAccessTrackingOps::NAME_ResetContext)
		{
			BuildOpName = NAME_None;
		}
		else
		{
			BuildOpName = !DirectData.BuildOpName.IsNone() ? DirectData.BuildOpName : OuterAccumulatedData->BuildOpName;
		}

		// The accumulated object is the most recent non-null Object
		Object = DirectData.Object ? DirectData.Object : OuterAccumulatedData->Object;

		// The accumulated TargetPlatform is the most recent non-empty TargetPlatform
		TargetPlatform = DirectData.TargetPlatform ? DirectData.TargetPlatform : OuterAccumulatedData->TargetPlatform;
	}
	else
	{
		PackageName = DirectData.PackageName;
		OpName = DirectData.OpName;
		BuildOpName = DirectData.BuildOpName;
		CookResultProjection = DirectData.CookResultProjection;
		Object = DirectData.Object;
		TargetPlatform = DirectData.TargetPlatform;
	}
}

UE::FInheritedContextExtension& FPackageAccessRefScope::GetInheritedContextExtension()
{
	static UE::FInheritedContextExtension Extension = []()
	{
		UE::FInheritedContextExtension Ext;
		Ext.Impl = MakeShared<UE::FInheritedContextExtension::FInterface>();
		Ext.Impl->Capture = [](void* DataDst, void* /*UserData*/)
		{
			auto* Dst = static_cast<FCapturedPackageTrackingData*>(DataDst);
			FTrackedData* Accumulated = GetCurrentThreadAccumulatedData();
			if (Accumulated)
			{
				Dst->PackageName = Accumulated->PackageName;
				Dst->OpName = Accumulated->OpName;
				Dst->BuildOpName = Accumulated->BuildOpName;
				Dst->CookResultProjection = Accumulated->CookResultProjection;
				Dst->Object = Accumulated->Object;
				Dst->TargetPlatform = Accumulated->TargetPlatform;
				Dst->bHasData = true;
			}
			else
			{
				Dst->bHasData = false;
			}
		};
		Ext.Impl->Apply = [](const void* DataSrc, void* SaveDst, void* /*UserData*/)
		{
			auto* Src = static_cast<const FCapturedPackageTrackingData*>(DataSrc);
			FPackageAccessRefScope** SaveSlot = static_cast<FPackageAccessRefScope**>(SaveDst);
			if (Src->bHasData)
			{
				*SaveSlot = new FPackageAccessRefScope(FInheritedContextTag{},
					Src->PackageName, Src->OpName, Src->BuildOpName,
					Src->CookResultProjection, Src->Object, Src->TargetPlatform);
			}
			else
			{
				*SaveSlot = nullptr;
			}
		};
		Ext.Impl->Restore = [](const void* SaveSrc, void* /*UserData*/)
		{
			FPackageAccessRefScope* const* Slot = static_cast<FPackageAccessRefScope* const*>(SaveSrc);
			delete *Slot; // this may delete nullptr, which is fine
		};
		static_assert(std::is_trivially_destructible_v<FCapturedPackageTrackingData>);
		Ext.Impl->Destroy = nullptr;
		Ext.Impl->DataSize = static_cast<uint32>(FMath::Max(sizeof(FCapturedPackageTrackingData), sizeof(FPackageAccessRefScope*)));
		Ext.Impl->DataAlign = static_cast<uint32>(FMath::Max(alignof(FCapturedPackageTrackingData), alignof(FPackageAccessRefScope*)));
		Ext.Impl->UserData = nullptr;
		return Ext;
	}();
	return Extension;
}

FPackageAccessRefScope::FPackageAccessRefScope(FInheritedContextTag,
	FName InPackageName, FName InOpName, FName InBuildOpName,
	FName InCookResultProjection, const UObject* InObject,
	const ITargetPlatform* InTargetPlatform)
	: DirectData(InPackageName, InOpName, InBuildOpName, InCookResultProjection, InTargetPlatform, InObject)
	, AccumulatedData(DirectData, GetCurrentThreadAccumulatedData())
	, Outer(CurrentThreadScope)
{
	CurrentThreadScope = this;
}

FPackageAccessRefScope::FPackageAccessRefScope(FName InPackageName, FName InOpName,
	FName InCookResultProjection, const ITargetPlatform* InTargetPlatform, const UObject* InObject)
	: DirectData(InPackageName, InOpName, InCookResultProjection, InTargetPlatform, InObject)
	, AccumulatedData(DirectData, GetCurrentThreadAccumulatedData())
	, Outer(CurrentThreadScope)
{
	CurrentThreadScope = this;

	if (!Outer)
	{
		InheritedExtensionScope = MakeUnique<UE::FInheritedContextExtensionScope>(GetInheritedContextExtension());
	}
}

FPackageAccessRefScope::FPackageAccessRefScope(FName InPackageName, FName InOpName)
	: FPackageAccessRefScope(InPackageName, InOpName
#if WITH_EDITOR
	, UE::Cook::ResultProjection::All
#else
	, NAME_None
#endif
	, nullptr, nullptr)
{
}

FPackageAccessRefScope::FPackageAccessRefScope(const UObject* InObject, FName InOpName)
	: FPackageAccessRefScope(InObject->GetPackage()->GetFName(), InOpName
#if WITH_EDITOR
	, UE::Cook::ResultProjection::All
#else
	, NAME_None
#endif
	, nullptr, InObject)
{
}

FPackageAccessRefScope::FPackageAccessRefScope(FName InOpName)
	: FPackageAccessRefScope(NAME_None, InOpName
#if WITH_EDITOR
	, UE::Cook::ResultProjection::All
#else
	, NAME_None
#endif
	, nullptr, nullptr)
{
}

FPackageAccessRefScope::FPackageAccessRefScope(const ITargetPlatform* InTargetPlatform)
	: FPackageAccessRefScope(NAME_None, NAME_None, NAME_None, InTargetPlatform, nullptr)
{
}

FPackageAccessRefScope::FPackageAccessRefScope(FPackageAccessRefScope::ECookResultProjectionType,
	FName InCookResultProjection)
	: FPackageAccessRefScope(NAME_None, NAME_None, InCookResultProjection, nullptr, nullptr)
{
}

FPackageAccessRefScope::~FPackageAccessRefScope()
{
	check(CurrentThreadScope == this);
	CurrentThreadScope = Outer;
}

void FPackageAccessRefScope::SetPackageName(FName InPackageName)
{
	checkfSlow(FPackageName::IsValidLongPackageName(InPackageName.ToString(), true), TEXT("Invalid package name: %s"), *InPackageName.ToString());
	checkf(CurrentThreadScope == this, TEXT("Invalid SetPackageName while a child scope is already on the stack. We have not yet implemented propagating a change in PackageName to child scopes' AccumulatedData."));

	DirectData.PackageName = InPackageName;
	if (Outer)
	{
		// Make sure this line matches the PackageName line in the FTrackedData accumulating constructor
		AccumulatedData.PackageName = !DirectData.PackageName.IsNone() ? DirectData.PackageName : Outer->AccumulatedData.PackageName;
	}
	else
	{
		AccumulatedData.PackageName = DirectData.PackageName;
	}
}

FPackageAccessRefScope* FPackageAccessRefScope::GetCurrentThreadScope()
{
	return CurrentThreadScope;
}

FTrackedData* FPackageAccessRefScope::GetCurrentThreadAccumulatedData()
{
	return CurrentThreadScope ? &CurrentThreadScope->AccumulatedData : nullptr;
}


} // PackageAccessTracking_Private
#endif // UE_WITH_OBJECT_HANDLE_TRACKING
