// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/PackageAccessTrackingOps.h"

#if UE_WITH_PACKAGE_ACCESS_TRACKING
FName PackageAccessTrackingOps::NAME_Load(TEXT("Load"));
FName PackageAccessTrackingOps::NAME_PreLoad(TEXT("PreLoad"));
FName PackageAccessTrackingOps::NAME_PostLoad(TEXT("PostLoad"));
FName PackageAccessTrackingOps::NAME_Save(TEXT("Save"));
FName PackageAccessTrackingOps::NAME_CreateDefaultObject(TEXT("CreateDefaultObject"));
FName PackageAccessTrackingOps::NAME_CookerBuildObject(TEXT("CookerBuildObject"));
FName PackageAccessTrackingOps::NAME_ResetContext(TEXT("ResetContext"));
FName PackageAccessTrackingOps::NAME_NoAccessExpected(TEXT("NoAccessExpected"));

#endif // UE_WITH_OBJECT_HANDLE_TRACKING
