// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectHash.cpp: Unreal object name hashes
=============================================================================*/

#include "Misc/PackageName.h"

#include "Algo/Find.h"
#include "Algo/FindLast.h"
#include "Algo/Sort.h"
#include "AssetRegistry/AssetData.h"
#include "Containers/StringView.h"
#include "Containers/VersePath.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/CriticalSection.h"
#include "HAL/FileManager.h"
#include "HAL/ThreadHeartBeat.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "IO/IoDispatcher.h"
#include "IO/IoDispatcherInternal.h"
#include "Misc/App.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/AutomationTest.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CoreUObjectPluginManager.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PackageName/MountPointPrivate.h"
#include "Misc/PackageName/PackageNamePrivate.h"
#include "Misc/PackagePath.h"
#include "Misc/PackageSegment.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Stats/Stats.h"
#include "String/Find.h"
#include "String/ParseTokens.h"
#include "Templates/UniquePtr.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/SoftObjectPath.h"

DEFINE_LOG_CATEGORY(LogPackageName);

#define LOCTEXT_NAMESPACE "PackageNames"

FPackageName::FMountPointEvent FPackageName::OnMountPointMountedEvent;
FPackageName::FOnContentPathMountedEvent FPackageName::OnContentPathMountedEvent;

FPackageName::FMountPointEvent FPackageName::OnMountPointDismountedEvent;
FPackageName::FOnContentPathDismountedEvent FPackageName::OnContentPathDismountedEvent;

/** Delegate used to check whether a package exist without using the filesystem. */
FPackageName::FDoesPackageExistOverride FPackageName::DoesPackageExistOverrideDelegate;

namespace PackageNameConstants
{
	// Minimum theoretical package name length ("/A/B") is 4
	const int32 MinPackageNameLength = 4;
}

bool FPackageName::IsShortPackageName(FStringView PossiblyLongName)
{
	// Long names usually have / as first character so check from the front
	for (TCHAR Char : PossiblyLongName)
	{
		if (Char == '/')
		{
			return false;
		}
	}

	return true;
}

bool FPackageName::IsShortPackageName(const FString& PossiblyLongName)
{
	return IsShortPackageName(FStringView(PossiblyLongName));
}

bool FPackageName::IsShortPackageName(FName PossiblyLongName)
{
	// Only get "plain" part of the name. The number suffix, e.g. "_123", can't contain slashes.
	TCHAR Buffer[NAME_SIZE];
	uint32 Len = PossiblyLongName.GetPlainNameString(Buffer);
	return IsShortPackageName(FStringView(Buffer, static_cast<int32>(Len)));
}

FString FPackageName::GetShortName(FStringView LongName)
{
	// Get everything after the last slash
	int32 IndexOfLastSlash = INDEX_NONE;
	LongName.FindLastChar(TEXT('/'), IndexOfLastSlash);
	return FString(LongName.Mid(IndexOfLastSlash + 1));
}

FString FPackageName::GetShortName(const UPackage* Package)
{
	check(Package != NULL);
	return GetShortName(Package->GetName());
}

FString FPackageName::GetShortName(const FName& LongName)
{
	return GetShortName(LongName.ToString());
}

FString FPackageName::GetShortName(const TCHAR* LongName)
{
	return GetShortName(FString(LongName));
}

FName FPackageName::GetShortFName(FStringView LongName)
{
	// Get everything after the last slash
	int32 IndexOfLastSlash = INDEX_NONE;
	LongName.FindLastChar(TEXT('/'), IndexOfLastSlash);
	return FName(LongName.Mid(IndexOfLastSlash + 1));
}

FName FPackageName::GetShortFName(const FName& LongName)
{
	TCHAR LongNameStr[FName::StringBufferSize];
	LongName.ToString(LongNameStr);

	if (const TCHAR* Slash = FCString::Strrchr(LongNameStr, '/'))
	{
		return FName(Slash + 1);
	}

	return LongName;
}

FName FPackageName::GetShortFName(const TCHAR* LongName)
{
	if (LongName == nullptr)
	{
		return FName();
	}

	if (const TCHAR* Slash = FCString::Strrchr(LongName, '/'))
	{
		return FName(Slash + 1);
	}

	return FName(LongName);
}

bool FPackageName::TryConvertGameRelativePackagePathToLocalPath(FStringView RelativePackagePath, FString& OutLocalPath)
{
	if (RelativePackagePath.StartsWith(TEXT("/"), ESearchCase::CaseSensitive))
	{
		// If this starts with /, this includes a root like /engine
		return FPackageName::TryConvertLongPackageNameToFilename(FString(RelativePackagePath), OutLocalPath);
	}
	else
	{
		// This is relative to /game
		const FString AbsoluteGameContentDir = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
		OutLocalPath = AbsoluteGameContentDir / FString(RelativePackagePath);
		return true;
	}
}

FLongPackagePathsSingleton& FLongPackagePathsSingleton::Get()
{
	static FLongPackagePathsSingleton Singleton;
	return Singleton;
}

/**
 * Function to bind to IPluginManager::FRegisterMountPointDelegate, which expects a function with the same
 * arguments as RegisterMountPoint but with a void return value.
 */
static void RegisterMountPointForPluginManager(const FString& RootPath, const FString& ContentPath)
{
	(void)FPackageName::RegisterMountPoint(RootPath, ContentPath);
}

void FLongPackagePathsSingleton::OnCoreUObjectInitialized()
{
	// Allow the plugin manager to mount new content paths by exposing access through a delegate.  PluginManager is 
	// a Core class, but content path functionality is added at the CoreUObject level.
	IPluginManager& PluginManager = IPluginManager::Get();
	PluginManager.SetRegisterMountPointDelegate(IPluginManager::FRegisterMountPointDelegate::CreateStatic(&RegisterMountPointForPluginManager));
	PluginManager.SetUnRegisterMountPointDelegate(IPluginManager::FRegisterMountPointDelegate::CreateStatic(&FPackageName::UnRegisterMountPoint));
	UE::CoreUObject::Private::PluginHandler::Install();
}

bool FLongPackagePathsSingleton::TryNormalizePaths(FStringView InLongPackageName, FStringView InLocalPath,
	FString& OutLongPackageName, FString& OutLocalPathStandard, FString& OutLocalPathAbsolute, bool bSilenceLogs)
{
	if (!TryNormalizeLongPackageNamePath(InLongPackageName, OutLongPackageName, bSilenceLogs))
	{
		return false;
	}
	if (!TryNormalizeLocalPath(InLocalPath, OutLocalPathStandard, OutLocalPathAbsolute, bSilenceLogs))
	{
		return false;
	}
	return true;
}

bool FLongPackagePathsSingleton::TryNormalizeLongPackageNamePath(FStringView InLongPackageName,
	FString& OutLongPackageName, bool bSilenceLogs)
{
	int32 NumBackSlashes = 0;
	int32 LenLongPackageName = InLongPackageName.Len();
	for (int32 N = 0; N < LenLongPackageName; ++N)
	{
		TCHAR C = InLongPackageName.GetData()[N];
		if (C == '\\')
		{
			++NumBackSlashes;
		}
	}
	if (LenLongPackageName == 0 || InLongPackageName[0] != '/' || NumBackSlashes > 0)
	{
		if (!bSilenceLogs)
		{
			UE_LOGF(LogPackageName, Error,
				"Invalid LongPackageName for mount point: \"%.*ls\". Callstack:",
				InLongPackageName.Len(), InLongPackageName.GetData());
			FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
		}
		return false;
	}

	OutLongPackageName = InLongPackageName;
	if (!OutLongPackageName.EndsWith("/"))
	{
		OutLongPackageName += TEXT("/");
	}
	return true;
}

bool FLongPackagePathsSingleton::TryNormalizeLocalPath(FStringView InLocalPath,
	FString& OutLocalPathStandard, FString& OutLocalPathAbsolute, bool bSilenceLogs)
{
	// If a relative path is passed, convert to an absolute path 
	OutLocalPathAbsolute = InLocalPath;
	if (FPaths::IsRelative(OutLocalPathAbsolute))
	{
		OutLocalPathAbsolute = FPaths::ConvertRelativePathToFull(OutLocalPathAbsolute);

		// Revert to original path if unable to convert to full path
		if (OutLocalPathAbsolute.Len() <= 1)
		{
			OutLocalPathAbsolute = InLocalPath;
			if (!bSilenceLogs)
			{
				UE_LOGF(LogPackageName, Warning, "Unable to convert mount point relative path: %.*ls",
					InLocalPath.Len(), InLocalPath.GetData());
			}
		}
	}

	// Convert to a relative path using the FileManager
	OutLocalPathStandard = IFileManager::Get().ConvertToRelativePath(*OutLocalPathAbsolute);

	// Make sure the path ends in a trailing path separator for consistency.
	if (!OutLocalPathStandard.EndsWith(TEXT("/"), ESearchCase::CaseSensitive))
	{
		OutLocalPathStandard += TEXT("/");
	}

	// Recompute the absolute path from the standard path, to prevent the possibility of there being any text
	// difference between a path passed in as an absolute path and a path passed in as a standard path.
	TStringBuilder<256> AbsolutePathBuilder;
	FPathViews::ToAbsolutePath(OutLocalPathStandard, AbsolutePathBuilder);
	OutLocalPathAbsolute = AbsolutePathBuilder;

	return true;
}

TRefCountPtr<UE::PackageName::IMountPoint> FLongPackagePathsSingleton::InsertMountPoint(const FString& RootPath, const FString& ContentPath)
{
	using namespace UE::PackageName;

	FMountPointKeyByStrings SearchKey;
	FString LocalPathAbsolute;
	if (!TryNormalizePaths(RootPath, ContentPath, SearchKey.LongPackageName,
		SearchKey.LocalPathStandard, LocalPathAbsolute, false /* bSilenceLogs */))
	{
		return TRefCountPtr<UE::PackageName::IMountPoint>();
	}
	uint32 SearchHash = GetTypeHash(SearchKey);

	TRefCountPtr<IMountPoint> PublicMountPoint;
	FMountPoint* MountPoint = nullptr;
	{
		UE::TWriteScopeLock ScopeLock(MountLock);

		// Look in the existing MountPoints to see if this one already exists, so we can update the existing pointer.
		FMountPointKeyByPtr* Existing = MountPoints.FindByHash(SearchHash, SearchKey);
		if (Existing)
		{
			MountPoint = Existing->GetMountPoint();
		}
		else
		{
			MountPoint = new FMountPoint(FString(SearchKey.LongPackageName), FString(SearchKey.LocalPathStandard),
				FString(LocalPathAbsolute), EMountFlags::Removable);
			MountPoints.AddByHash(SearchHash, FMountPointKeyByPtr(*MountPoint));
		}
		PublicMountPoint = MountPoint;
		RootPathTree.FindOrAdd(MountPoint->GetRootPath()) = MountPoint;
		ContentPathTree.FindOrAdd(MountPoint->GetContentPathRelative()) = MountPoint;
		ContentPathTree.FindOrAdd(MountPoint->GetContentPathAbsolute()) = MountPoint;

		bool bRemoveFromContainer;
		MountPoint->SetMounted(true, ScopeLock, PublicMountPoint, bRemoveFromContainer);
		// Set mount order to highest priority, replacing earlier mount order if it already existed.
		MountPoint->SetMountOrder(NextMountOrder++);
		if (NextMountOrder == 0)
		{
			// Overflow, due to e.g. removing and adding a mountpoint too many times.
			ReassignMountOrders();
		}
		check(!bRemoveFromContainer); // Should only be settable to true when calling SetMounted(false).
	}

	BroadcastMountPointMounted(PublicMountPoint, MountPoint);
	return PublicMountPoint;
}

void FLongPackagePathsSingleton::BroadcastMountPointMounted(
	const TRefCountPtr<UE::PackageName::IMountPoint>& PublicMountPoint, UE::PackageName::FMountPoint* MountPoint)
{
	using namespace UE::PackageName;

	check(PublicMountPoint && PublicMountPoint.GetReference() == MountPoint);

	// Downgrade the log messages on startup to Verbose to reduce startup spam
#if !NO_LOGGING
	FString Message = FString::Printf(TEXT("FPackageName: Mount point added: '%s' mounted to '%s'"),
		*MountPoint->GetContentPathRelative(), *MountPoint->GetRootPath());
	if (IsEngineStartupModuleLoadingComplete())
	{
		UE_LOGF(LogPackageName, Log, "%ls", *Message);
	}
	else
	{
		UE_LOGF(LogPackageName, Verbose, "%ls", *Message);
	}
#endif

	// Let subscribers know that a new content path was mounted
	FPackageName::OnMountPointMounted().Broadcast(PublicMountPoint);
	FPackageName::OnContentPathMounted().Broadcast(MountPoint->GetRootPath(), MountPoint->GetContentPathRelative());
}

void FLongPackagePathsSingleton::InsertMountPoint(const TRefCountPtr<UE::PackageName::IMountPoint>& PublicMountPoint)
{
	using namespace UE::PackageName;

	if (!PublicMountPoint)
	{
		return;
	}
	FMountPoint* MountPoint = PublicMountPoint->AsFMountPoint();
	if (!MountPoint)
	{
		UE_LOGF(LogPackageName, Error,
			"IMountPoint passed to FPackageName that is not an FMountPoint subclass. This is unexpected, all IMountPoints should be FMountPoint. " "LongPackageName: %.*ls, LocalPathStandard: %.*ls",
			PublicMountPoint->GetLongPackageName().Len(), PublicMountPoint->GetLongPackageName().GetData(),
			PublicMountPoint->GetLocalPathStandard().Len(), PublicMountPoint->GetLocalPathStandard().GetData());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
		return;
	}

	{
		UE::TWriteScopeLock ScopeLock(MountLock);

		RootPathTree.FindOrAdd(MountPoint->GetRootPath()) = MountPoint;
		ContentPathTree.FindOrAdd(MountPoint->GetContentPathRelative()) = MountPoint;
		ContentPathTree.FindOrAdd(MountPoint->GetContentPathAbsolute()) = MountPoint;

		bool bRemoveFromContainer;
		MountPoint->SetMounted(true, ScopeLock, PublicMountPoint, bRemoveFromContainer);
		// Set mount order to highest priority, replacing earlier mount order if it already existed.
		MountPoint->SetMountOrder(NextMountOrder++);
		if (NextMountOrder == 0)
		{
			// Overflow, due to e.g. removing and adding a mountpoint too many times.
			ReassignMountOrders();
		}
		check(!bRemoveFromContainer); // Should only be settable to true when calling SetMounted(false).
	}

	BroadcastMountPointMounted(PublicMountPoint, MountPoint);
}

void FLongPackagePathsSingleton::ReassignMountOrders()
{
	using namespace UE::PackageName;

	// Called within writelock on MountLock
	TArray<FMountPoint*> SortedMountPoints;
	SortedMountPoints.Reserve(MountPoints.Num());
	for (const FMountPointKeyByPtr& Ptr : MountPoints)
	{
		SortedMountPoints.Add(Ptr.GetMountPoint());
	}

	// Sort the list from lowest in priority to highest in priority, so we can traverse front to
	// back assigning increasing-in-value and therefore increasing-in-priority NextMountOrder.
	Algo::Sort(SortedMountPoints, [](const FMountPoint* A, const FMountPoint* B)
		{
			return A->GetMountOrder() < B->GetMountOrder();
		});
	NextMountOrder = 0;
	for (FMountPoint* MountPoint : SortedMountPoints)
	{
		MountPoint->SetMountOrder(NextMountOrder++);
	}
}

void FLongPackagePathsSingleton::RemoveMountPoint(const FString& RootPath, const FString& ContentPath)
{
	using namespace UE::PackageName;

	FMountPointKeyByStrings SearchKey;
	FString LocalPathAbsolute;
	if (!TryNormalizePaths(RootPath, ContentPath, SearchKey.LongPackageName,
		SearchKey.LocalPathStandard, LocalPathAbsolute, true /* bSilenceLogs */))
	{
		UE_LOGF(LogPackageName, Display,
			"FPackageName: Mount point remove failed: invalid input LongPackageName ('%ls') or LocalPath ('%ls').",
			*RootPath, *ContentPath);
		return;
	}
	uint32 SearchHash = GetTypeHash(SearchKey);

	TRefCountPtr<IMountPoint> PublicMountPoint;
	FMountPoint* MountPoint = nullptr;
	{
		UE::TWriteScopeLock ScopeLock(MountLock);
		FMountPointKeyByPtr* ExistingPtr = MountPoints.FindByHash(SearchHash, SearchKey);
		if (!ExistingPtr)
		{
			UE_LOGF(LogPackageName, Display,
				"FPackageName: Mount point remove failed: no MountPoint found mapping '%ls' to '%ls'",
				*SearchKey.LocalPathStandard, *SearchKey.LongPackageName);
			return;
		}
		MountPoint = ExistingPtr->GetMountPoint();
		PublicMountPoint = MountPoint;
		if (!TryRemoveMountPointInternal(PublicMountPoint, ScopeLock, MountPoint))
		{
			return;
		}
	}
	BroadcastMountPointDismounted(PublicMountPoint, MountPoint);
}

bool FLongPackagePathsSingleton::TryRemoveMountPointInternal(
	const TRefCountPtr<UE::PackageName::IMountPoint>& InternallyHeldMountPoint,
	UE::TWriteScopeLock<FTransactionallySafeRWLock>& ProofOfWriteLock, UE::PackageName::FMountPoint* MountPoint)
{
	using namespace UE::PackageName;

	// Called within the WriteLock
	// Caller must hold a single internal refcount in addition to any externally held refcount; that single internal
	// refcount is required by the SetMounted call within this function to decide whether the MountPoint is externally referenced
	if (!MountPoint->IsRemovable())
	{
		UE_LOGF(LogPackageName, Display, "FPackageName: Mount point remove failed: MountPoint mapping '%ls' to '%ls' is a default mapping and is not removable.",
			*MountPoint->GetContentPathRelative(), *MountPoint->GetRootPath());
		return false;
	}

	bool bRootPathExisted;
	bool bContentPathRelativeExisted;
	bool bContentPathAbsoluteExisted;
	RootPathTree.Remove(MountPoint->GetRootPath(), &bRootPathExisted);
	ContentPathTree.Remove(MountPoint->GetContentPathRelative(), &bContentPathRelativeExisted);
	ContentPathTree.Remove(MountPoint->GetContentPathAbsolute(), &bContentPathAbsoluteExisted);
	if (bRootPathExisted || bContentPathRelativeExisted || bContentPathAbsoluteExisted)
	{
		// See if there is an existing mounted MountPoint that was overridden by the removed mount point and now
		// needs to be published as the MountPoint for one of its paths.
		FMountPoint* NewRootPath = nullptr;
		FMountPoint* NewContentPathRelative = nullptr;
		FMountPoint* NewContentPathAbsolute = nullptr;
		for (const FMountPointKeyByPtr& OtherMountPtr : MountPoints)
		{
			FMountPoint* OtherMount = OtherMountPtr.GetMountPoint();
			CA_ASSUME(OtherMount != nullptr);
			if (OtherMount == MountPoint || !OtherMount->IsMounted())
			{
				continue;
			}
			if (bRootPathExisted && OtherMount->GetRootPath().Equals(
				MountPoint->GetRootPath(), ESearchCase::IgnoreCase))
			{
				if (NewRootPath == nullptr
					|| OtherMount->GetMountOrder() > NewRootPath->GetMountOrder())
				{
					NewRootPath = OtherMount;
				}
			}
			if (bContentPathRelativeExisted && OtherMount->GetContentPathRelative().Equals(
				MountPoint->GetContentPathRelative(), ESearchCase::IgnoreCase))
			{
				if (NewContentPathRelative == nullptr
					|| OtherMount->GetMountOrder() > NewContentPathRelative->GetMountOrder())
				{
					NewContentPathRelative = OtherMount;
				}
			}
			if (bContentPathAbsoluteExisted && OtherMount->GetContentPathAbsolute().Equals(
				MountPoint->GetContentPathAbsolute(), ESearchCase::IgnoreCase))
			{
				if (NewContentPathAbsolute == nullptr
					|| OtherMount->GetMountOrder() > NewContentPathAbsolute->GetMountOrder())
				{
					NewContentPathAbsolute = OtherMount;
				}
			}
		}
		if (NewRootPath)
		{
			RootPathTree.FindOrAdd(NewRootPath->GetRootPath()) = NewRootPath;
		}
		if (NewContentPathRelative)
		{
			ContentPathTree.FindOrAdd(NewContentPathRelative->GetContentPathRelative()) = NewContentPathRelative;
		}
		if (NewContentPathAbsolute)
		{
			ContentPathTree.FindOrAdd(NewContentPathAbsolute->GetContentPathAbsolute()) = NewContentPathAbsolute;
		}
	}

	bool bRemoveFromContainer;
	MountPoint->SetMounted(false, ProofOfWriteLock, InternallyHeldMountPoint, bRemoveFromContainer);
	if (bRemoveFromContainer)
	{
		MountPoints.Remove(FMountPointKeyByPtr(*MountPoint));
	}

	return true;
}

void FLongPackagePathsSingleton::BroadcastMountPointDismounted(
	const TRefCountPtr<UE::PackageName::IMountPoint>& PublicMountPoint, UE::PackageName::FMountPoint* MountPoint)
{
	using namespace UE::PackageName;

	check(PublicMountPoint && PublicMountPoint.GetReference() == MountPoint);
	UE_LOGF(LogPackageName, Display, "FPackageName: Mount point removed: '%ls' unmounted from '%ls'",
		*MountPoint->GetContentPathRelative(), *MountPoint->GetRootPath());
	// Let subscribers know that a content path was unmounted
	FPackageName::OnMountPointDismounted().Broadcast(PublicMountPoint);
	FPackageName::OnContentPathDismounted().Broadcast(MountPoint->GetRootPath(), MountPoint->GetContentPathRelative());
}

void FLongPackagePathsSingleton::RemoveMountPoint(const TRefCountPtr<UE::PackageName::IMountPoint>& PublicMountPoint)
{
	using namespace UE::PackageName;

	if (!PublicMountPoint)
	{
		return;
	}
	FMountPoint* MountPoint = PublicMountPoint->AsFMountPoint();
	if (!MountPoint)
	{
		UE_LOGF(LogPackageName, Error,
			"IMountPoint passed to FPackageName that is not an FMountPoint subclass. This is unexpected, all IMountPoints should be FMountPoint. " "LongPackageName: %.*ls, LocalPathStandard: %.*ls",
			PublicMountPoint->GetLongPackageName().Len(), PublicMountPoint->GetLongPackageName().GetData(),
			PublicMountPoint->GetLocalPathStandard().Len(), PublicMountPoint->GetLocalPathStandard().GetData());
		FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
		return;
	}

	// The SetMounted function assumes we take an additional refcount in this function, and uses that to decide
	// that the MountPoint is no longer externally referenced. It is externally referenced (our caller passed in
	// a refcountptr to it that they still have), so we need to increment refcount to let SetMounted know that
	// it is externally referenced.
	TRefCountPtr<IMountPoint> InternallyHeldMountPoint = PublicMountPoint;
	{
		UE::TWriteScopeLock ScopeLock(MountLock);
		if (!TryRemoveMountPointInternal(InternallyHeldMountPoint, ScopeLock, MountPoint))
		{
			return;
		}
	}
	BroadcastMountPointDismounted(InternallyHeldMountPoint, MountPoint);
}

TRefCountPtr<UE::PackageName::IMountPoint>
FLongPackagePathsSingleton::FindMountPointByRootPackageName(FStringView RootLongPackageName)
{
	using namespace UE::PackageName;

	FString LongPackageName;
	if (!TryNormalizeLongPackageNamePath(RootLongPackageName, LongPackageName, true /* bSilenceLogs */))
	{
		return TRefCountPtr<UE::PackageName::IMountPoint>();
	}

	{
		UE::TReadScopeLock ScopeLock(MountLock);

		// First look for a Mounted MountPoint because its a common case that we can do fast
		FMountPoint** ExistingPtr = RootPathTree.Find(LongPackageName);
		if (ExistingPtr)
		{
			return TRefCountPtr<IMountPoint>(*ExistingPtr);
		}

		// We don't have a lookup map by RootName for overridden or non-mounted MountPoints, so search our entire list.
		FMountPoint* HighestPriorityExisting = nullptr;
		for (const FMountPointKeyByPtr& ExistingKey : MountPoints)
		{
			FMountPoint* Existing = ExistingKey.GetMountPoint();
			if (Existing->GetLongPackageName().Equals(LongPackageName, ESearchCase::IgnoreCase))
			{
				if (FMountPoint::IsHigherOverridePriority(Existing, HighestPriorityExisting))
				{
					HighestPriorityExisting = Existing;
				}
			}
		}
		if (HighestPriorityExisting)
		{
			return TRefCountPtr<IMountPoint>(HighestPriorityExisting);
		}
	}
	return TRefCountPtr<UE::PackageName::IMountPoint>();
}

TRefCountPtr<UE::PackageName::IMountPoint>
FLongPackagePathsSingleton::FindMountPointByRootLocalPath(FStringView RootLocalPath)
{
	using namespace UE::PackageName;

	FString LocalPathStandard;
	FString LocalPathAbsolute;
	if (!TryNormalizeLocalPath(RootLocalPath, LocalPathStandard, LocalPathAbsolute, true /* bSilenceLogs */))
	{
		return TRefCountPtr<UE::PackageName::IMountPoint>();
	}

	{
		UE::TReadScopeLock ScopeLock(MountLock);

		// First look for a Mounted MountPoint because its a common case that we can do fast
		FMountPoint** ExistingPtr = ContentPathTree.Find(LocalPathStandard);
		if (ExistingPtr)
		{
			return TRefCountPtr<IMountPoint>(*ExistingPtr);
		}

		// We don't have a lookup map by LocalPath for overridden or non-mounted MountPoints, so search our entire list.
		FMountPoint* HighestPriorityExisting = nullptr;
		for (const FMountPointKeyByPtr& ExistingKey : MountPoints)
		{
			FMountPoint* Existing = ExistingKey.GetMountPoint();
			if (Existing->GetLocalPathStandard().Equals(LocalPathStandard, ESearchCase::IgnoreCase))
			{
				if (FMountPoint::IsHigherOverridePriority(Existing, HighestPriorityExisting))
				{
					HighestPriorityExisting = Existing;
				}
			}
		}
		if (HighestPriorityExisting)
		{
			return TRefCountPtr<IMountPoint>(HighestPriorityExisting);
		}
	}
	return TRefCountPtr<UE::PackageName::IMountPoint>();
}

TRefCountPtr<UE::PackageName::IMountPoint>
FLongPackagePathsSingleton::FindMountPoint(FStringView RootLongPackageName, FStringView RootLocalPath)
{
	using namespace UE::PackageName;

	FMountPointKeyByStrings SearchKey;
	FString LocalPathAbsolute;
	if (!TryNormalizePaths(RootLongPackageName, RootLocalPath, SearchKey.LongPackageName,
		SearchKey.LocalPathStandard, LocalPathAbsolute, true /* bSilenceLogs */))
	{
		return TRefCountPtr<UE::PackageName::IMountPoint>();
	}
	uint32 SearchHash = GetTypeHash(SearchKey);

	{
		UE::TReadScopeLock ScopeLock(MountLock);
		FMountPoint* MountPoint = nullptr;

		FMountPointKeyByPtr* Existing = MountPoints.FindByHash(SearchHash, SearchKey);
		if (Existing)
		{
			return TRefCountPtr<IMountPoint>(Existing->GetMountPoint());
		}
		return TRefCountPtr<UE::PackageName::IMountPoint>();
	}
}

TRefCountPtr<UE::PackageName::IMountPoint>
FLongPackagePathsSingleton::FindOrAddMountPoint(FStringView RootLongPackageName, FStringView RootLocalPath)
{
	using namespace UE::PackageName;

	FMountPointKeyByStrings SearchKey;
	FString LocalPathAbsolute;
	if (!TryNormalizePaths(RootLongPackageName, RootLocalPath, SearchKey.LongPackageName,
		SearchKey.LocalPathStandard, LocalPathAbsolute, false /* bSilenceLogs */))
	{
		return TRefCountPtr<UE::PackageName::IMountPoint>();
	}
	uint32 SearchHash = GetTypeHash(SearchKey);

	// Look in the existing MountPoints to see if this one already exists, so we can return the existing pointer.
	{
		UE::TReadScopeLock ReadScopeLock(MountLock);
		FMountPointKeyByPtr* Existing = MountPoints.FindByHash(SearchHash, SearchKey);
		if (Existing)
		{
			return TRefCountPtr<IMountPoint>(Existing->GetMountPoint());
		}
	}

	{
		UE::TWriteScopeLock WriteScopeLock(MountLock);

		// Look again in MountPoints in case it was added when we dropped the readlock.
		FMountPointKeyByPtr* Existing = MountPoints.FindByHash(SearchHash, SearchKey);
		if (Existing)
		{
			return TRefCountPtr<IMountPoint>(Existing->GetMountPoint());
		}

		FMountPoint* MountPoint = new FMountPoint(MoveTemp(SearchKey.LongPackageName),
			MoveTemp(SearchKey.LocalPathStandard), MoveTemp(LocalPathAbsolute), EMountFlags::Removable);
		MountPoints.AddByHash(SearchHash, FMountPointKeyByPtr(*MountPoint));
		return TRefCountPtr<UE::PackageName::IMountPoint>(MountPoint);
	}
}

TRefCountPtr<UE::PackageName::IMountPoint>
FLongPackagePathsSingleton::FindMountPointByChildLongPackageName(FStringView ChildLongPackageName)
{
	using namespace UE::PackageName;

	FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();

	UE::TReadScopeLock ScopeLock(Paths.MountLock);
	FMountPoint* const* MountPointPtr = Paths.RootPathTree.FindClosestValue(ChildLongPackageName);
	if (!MountPointPtr)
	{
		return TRefCountPtr<IMountPoint>();
	}
	FMountPoint* MountPoint = *MountPointPtr;
	// DirectoryTree is less conservative than IsParentPathOf; e.g. d:/Dir//Root will match
	// d:/Dir/Root/Child in DirectoryTree but not in IsParentPathOf. Treat this as not found,
	// since no mountdir will match it according to IsParentPathOf.
	if (!FPathViews::IsParentPathOf(MountPoint->GetRootPath(), ChildLongPackageName))
	{
		return TRefCountPtr<IMountPoint>();
	}
	return TRefCountPtr<IMountPoint>(MountPoint);
}

bool FLongPackagePathsSingleton::MountPointExists(FStringView RootPath)
{
	UE::TReadScopeLock ScopeLock(MountLock);
	return RootPathTree.Contains(RootPath) || RootPath == MemoryRootPath;
}

FLongPackagePathsSingleton::FLongPackagePathsSingleton()
#if !UE_BUILD_SHIPPING
	: DumpMountPointsCommand(
		TEXT("PackageName.DumpMountPoints"),
		*LOCTEXT("CommandText_DumpMountPoints", "Print registered LongPackagePath mount points").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FLongPackagePathsSingleton::ExecDumpMountPoints))
	, RegisterMountPointCommand(
		TEXT("PackageName.RegisterMountPoint"),
		*LOCTEXT("CommandText_RegisterMountPoint", "<RootPath> <ContentPath> // Register a LongPackagePath mount point").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FLongPackagePathsSingleton::ExecInsertMountPoint))
	, UnregisterMountPointCommand(
		TEXT("PackageName.UnregisterMountPoint"),
		*LOCTEXT("CommandText_UnregisterMountPoint", "<RootPath> <ContentPath> // Remove a LongPackagePath mount point").ToString(),
		FConsoleCommandWithArgsDelegate::CreateRaw(this, &FLongPackagePathsSingleton::ExecRemoveMountPoint))
#endif
{
	using namespace UE::PackageName;

	SCOPED_BOOT_TIMING("FPackageName::FLongPackagePathsSingleton");

	ConfigRootPath = TEXT("/Config/");
	EngineRootPath = TEXT("/Engine/");
	GameRootPath   = TEXT("/Game/");
	ScriptRootPath = TEXT("/Script/");
	MemoryRootPath = TEXT("/Memory/");
	TempRootPath   = TEXT("/Temp/");

	VerseSubPath = TEXT("/_Verse");

	EngineContentPath      = FPaths::EngineContentDir();
	ContentPathShort       = TEXT("../../Content/");
	EngineShadersPath      = FPaths::EngineDir() / TEXT("Shaders/");
	EngineShadersPathShort = TEXT("../../Shaders/");
	GameContentPath        = FPaths::ProjectContentDir();
	GameConfigPath         = FPaths::ProjectConfigDir();
	GameScriptPath         = FPaths::ProjectDir() / TEXT("Script/");
	GameSavedPath          = FPaths::ProjectSavedDir();

	FString RebasedGameDir = FString::Printf(TEXT("../../../%s/"), FApp::GetProjectName());

	GameContentPathRebased = RebasedGameDir / TEXT("Content/");
	GameConfigPathRebased  = RebasedGameDir / TEXT("Config/");
	GameScriptPathRebased  = RebasedGameDir / TEXT("Script/");
	GameSavedPathRebased   = RebasedGameDir / TEXT("Saved/");
	UE::TWriteScopeLock ScopeLock(MountLock);

	auto LocalInsertMountPoint = [this, &ScopeLock](const FString& RootPath, const FString& ContentPathRelative,
		EMountFlags InFlags)
	{
		// Don't call TryNormalizePaths, because that calls FPaths::ConvertRelativePathToFull followed by
		// IFileManager::Get().ConvertToRelativePath, and that combination will convert ../../../Engine/Content
		// into ../../Content, which we don't want here; we want to enter in the full ../../.. path for some entries.
		TStringBuilder<256> AbsolutePathBuilder;
		FPathViews::ToAbsolutePath(ContentPathRelative, AbsolutePathBuilder);
		FMountPointKeyByStrings SearchKey(RootPath, ContentPathRelative);
		FMountPointKeyByPtr* ExistingFullKey = MountPoints.FindByHash(GetTypeHash(SearchKey), SearchKey);
		if (ExistingFullKey)
		{
			// Duplicates can occur in our manual list if e.g. GameContentPathRebased == GameContentPath.
			// If we call LocalInsertMountPoint with the same pair of LongPackageName, LocalPath as one we've done
			// earlier, just skip it.
			return;
		}

		FMountPoint* MountPoint = new FMountPoint(FString(RootPath), FString(ContentPathRelative),
			FString(AbsolutePathBuilder), InFlags);
		TRefCountPtr<IMountPoint> MountPointRefCount = MountPoint;
		MountPoint->SetMountOrder(NextMountOrder++);
		bool bAlreadyExisted;
		MountPoints.Add(FMountPointKeyByPtr(*MountPoint), &bAlreadyExisted);
		check(!bAlreadyExisted); // Impossible for it to already exist, because we called Find above.

		bool bRemoveFromContainer;
		MountPoint->SetMounted(true, ScopeLock, MountPointRefCount, bRemoveFromContainer);
		check(!bRemoveFromContainer); // Should only be settable to true when calling SetMounted(false).

		// We are appending to the MountPoint list, and will have only a few duplicates. If any duplicates do occur,
		// keep the MountPoint that was appended earlier as the registered mountpoint, and mark it as higher priority
		// than the current one.
		FMountPoint*& ExistingRootPath = RootPathTree.FindOrAdd(MountPoint->GetRootPath());
		if (!ExistingRootPath)
		{
			ExistingRootPath = MountPoint;
		}
		else
		{
			if (ExistingRootPath->GetMountOrder() < MountPoint->GetMountOrder())
			{
				ExistingRootPath->SetMountOrder(NextMountOrder++);
			}
		}
		FMountPoint*& ExistingContentPathRelative = ContentPathTree.FindOrAdd(MountPoint->GetContentPathRelative());
		if (!ExistingContentPathRelative)
		{
			ExistingContentPathRelative = MountPoint;
		}
		else
		{
			if (ExistingContentPathRelative->GetMountOrder() < MountPoint->GetMountOrder())
			{
				ExistingContentPathRelative->SetMountOrder(NextMountOrder++);
			}
		}
		FMountPoint*& ExistingContentPathAbsolute = ContentPathTree.FindOrAdd(MountPoint->GetContentPathAbsolute());
		if (!ExistingContentPathAbsolute)
		{
			ExistingContentPathAbsolute = MountPoint;
		}
		else
		{
			if (ExistingContentPathAbsolute->GetMountOrder() < MountPoint->GetMountOrder())
			{
				ExistingContentPathAbsolute->SetMountOrder(NextMountOrder++);
			}
		}
	};
		
	LocalInsertMountPoint(EngineRootPath, EngineContentPath, EMountFlags::NotReadOnly);
	LocalInsertMountPoint(EngineRootPath, EngineShadersPath, EMountFlags::NotReadOnly | EMountFlags::Alias);
	LocalInsertMountPoint(GameRootPath,   GameContentPath, EMountFlags::NotReadOnly);
	LocalInsertMountPoint(ScriptRootPath, GameScriptPath, EMountFlags::ReadOnly);
	LocalInsertMountPoint(TempRootPath,   GameSavedPath, EMountFlags::ReadOnly);
	LocalInsertMountPoint(ConfigRootPath, GameConfigPath, EMountFlags::ReadOnly);

	// Add other LocalPaths that have different sets of .. but are the same location on disk
	LocalInsertMountPoint(EngineRootPath, EngineShadersPathShort, EMountFlags::NotReadOnly | EMountFlags::Alias);
	if (FPaths::IsSamePath(GameContentPath, ContentPathShort))
	{
		// ../../Content points to the the Engine directory, the same as ../../../Engine/Content
		// But if the /Game is pointing to ../../../Engine/Content as well, then map ../../Content to
		// /Game instead of /Engine
		LocalInsertMountPoint(GameRootPath, ContentPathShort, EMountFlags::NotReadOnly | EMountFlags::Alias);
	}
	else
	{
		LocalInsertMountPoint(EngineRootPath, ContentPathShort, EMountFlags::NotReadOnly | EMountFlags::Alias);
	}
	LocalInsertMountPoint(GameRootPath,   GameContentPathRebased, EMountFlags::NotReadOnly | EMountFlags::Alias);
	LocalInsertMountPoint(ScriptRootPath, GameScriptPathRebased, EMountFlags::ReadOnly | EMountFlags::Alias);
	LocalInsertMountPoint(TempRootPath,   GameSavedPathRebased, EMountFlags::ReadOnly | EMountFlags::Alias);
	LocalInsertMountPoint(ConfigRootPath, GameConfigPathRebased, EMountFlags::ReadOnly | EMountFlags::Alias);
}

FLongPackagePathsSingleton::~FLongPackagePathsSingleton()
{
	using namespace UE::PackageName;

	// We require that our external callers drop their IMountPoint refcounts, but we do not require that they unmount
	// all of the MountPoints they registered with RegisterMountPoint. So we may have MountPoints remaining that
	// are still mounted and therefore we still hold a refcount. Unmount them all now to drop that final refcount and
	// free the memory. This happens at end of process, so this is important mainly for leak detection.
	TArray<TRefCountPtr<IMountPoint>> RemainingMountPoints;
	{
		UE::TWriteScopeLock ScopeLock(MountLock);
		RemainingMountPoints.Reserve(MountPoints.Num());
		for (FMountPointKeyByPtr& Key : MountPoints)
		{
			FMountPoint* MountPoint = Key.GetMountPoint();
			TRefCountPtr<IMountPoint>& MountPointRefCount = RemainingMountPoints.Emplace_GetRef(MountPoint);
			bool bRemoveFromContainer;
			MountPoint->SetMounted(false, ScopeLock, MountPointRefCount, bRemoveFromContainer);
			if (!bRemoveFromContainer)
			{
				// UE_LOG is likely no longer available; we will have to rely on leak detector reporting this instead.
				constexpr bool bAllowedToCallLog = false;
				if (bAllowedToCallLog)
				{
					UE_LOGF(LogPackageName, Error,
						"MountPoint is still referenced during FLongPackagePathsSingleton destruction; this will leak memory or will cause a crash if the MountPoint is released later. MountPoint: %ls, %ls",
						*MountPoint->GetRootPath(), *MountPoint->GetContentPathRelative());
				}
			}
		}
		RootPathTree.Empty();
		ContentPathTree.Empty();
		MountPoints.Empty();
	}

	// Emptying the list will drop the final refcount for all of the remaining MountPoints and will cause their destruction.
	RemainingMountPoints.Empty();
}

#if !UE_BUILD_SHIPPING
void FLongPackagePathsSingleton::ExecDumpMountPoints(const TArray<FString>& Args)
{
	using namespace UE::PackageName;

	UE_LOGF(LogPackageName, Log, "Valid mount points:");

	TArray<TRefCountPtr<FMountPoint>> ReportMountPoints;
	{
		UE::TReadScopeLock ScopeLock(MountLock);
		ReportMountPoints.Reserve(MountPoints.Num());
		for (const FMountPointKeyByPtr& MountPoint : MountPoints)
		{
			if (MountPoint.GetMountPoint()->IsMounted())
			{
				ReportMountPoints.Emplace(MountPoint.GetMountPoint());
			}
		}
	}

	// Sort the mount points from highest priority to lowest priority.
	Algo::Sort(ReportMountPoints, [](const TRefCountPtr<FMountPoint>& A, const TRefCountPtr<FMountPoint>& B)
		{
			return A->GetMountOrder() > B->GetMountOrder();
		});
	for (TRefCountPtr<FMountPoint>& MountPoint : ReportMountPoints)
	{
		UE_LOGF(LogPackageName, Log, "	'%ls' -> '%ls'", *MountPoint->GetRootPath(), *MountPoint->GetContentPathRelative());
	}

	UE_LOGF(LogPackageName, Log, "Removable mount points:");
	for (TRefCountPtr<FMountPoint>& MountPoint : ReportMountPoints)
	{
		if (MountPoint->IsRemovable())
		{
			UE_LOGF(LogPackageName, Log, "	'%ls'", *MountPoint->GetRootPath());
		}
	}
}

void FLongPackagePathsSingleton::ExecInsertMountPoint(const TArray<FString>& Args)
{
	if ( Args.Num() < 2 )
	{
		UE_LOGF(LogPackageName, Log, "Usage: PackageName.RegisterMountPoint <RootPath> <ContentPath>");
		UE_LOGF(LogPackageName, Log, "Example ContentPath: '../../../ProjectName/Content/'");
		UE_LOGF(LogPackageName, Log, "Example RootPath: '/Game/'");
		return;
	}

	const FString& RootPath = Args[0];
	const FString& ContentPath = Args[1];

	if (RootPath.IsEmpty() || ContentPath.IsEmpty())
	{
		UE_LOGF(LogPackageName, Error, "PackageName.RegisterMountPoint: Invalid empty path, expected non-empty RootPath and ContentPath.");
		return;
	}

	if (ContentPath[0] == TEXT('/'))
	{
		UE_LOGF(LogPackageName, Error, "PackageName.RegisterMountPoint: Invalid ContentPath, should not start with '/'! Example: '../../../ProjectName/Content/'");
		return;
	}

	if (RootPath[0] != TEXT('/'))
	{
		UE_LOGF(LogPackageName, Error, "PackageName.RegisterMountPoint: Invalid RootPath, should start with a '/'! Example: '/Game/'");
		return;
	}
		
	Get().InsertMountPoint(RootPath, ContentPath);
}

void FLongPackagePathsSingleton::ExecRemoveMountPoint(const TArray<FString>& Args)
{
	using namespace UE::PackageName;

	if ( Args.Num() < 1 || Args.Num() > 2 )
	{
		UE_LOGF(LogPackageName, Log, "Usage: PackageName.UnregisterMountPoint <Path>");
		UE_LOGF(LogPackageName, Log, "Removes either a Root or Content path if the path is unambiguous");
		UE_LOGF(LogPackageName, Log, "Usage: PackageName.UnregisterMountPoint <RootPath> <ContentPath>");
		UE_LOGF(LogPackageName, Log, "Removes a specific Root path to Content path mapping");
		UE_LOGF(LogPackageName, Log, "Example ContentPath: '../../../ProjectName/Content/'");
		UE_LOGF(LogPackageName, Log, "Example RootPath: '/Game/'");
		return;
	}

	for (const FString& Arg : Args)
	{
		if (Arg.IsEmpty())
		{
			UE_LOGF(LogPackageName, Error, "PackageName.UnregisterMountPoint: Invalid empty path, expected non-empty root or content paths.");
			return;
		}
	}

	if (Args.Num() == 1)
	{
		const FString& Path = Args[0];
		const bool IsRootPath = Path[0] == TEXT('/');

		FString LongPackageName;
		FString LocalPathStandard;

		if (IsRootPath)
		{
			if (!TryNormalizeLongPackageNamePath(Path, LongPackageName, false /* bSilenceLogs */))
			{
				return;
			}
			UE::TReadScopeLock ScopeLock(MountLock);

			TArray<FString> ContentPathRelatives;
			for (const FMountPointKeyByPtr& ExistingKey : MountPoints)
			{
				FMountPoint* ExistingMount = ExistingKey.GetMountPoint();
				if (ExistingMount->IsMounted() && ExistingMount->GetRootPath() == LongPackageName)
				{
					ContentPathRelatives.Add(ExistingMount->GetContentPathRelative());
				}
			}
			if (ContentPathRelatives.Num() == 0)
			{
				UE_LOGF(LogPackageName, Error, "PackageName.UnregisterMountPoint: Root path '%ls' is not mounted!",
					*LongPackageName);
				return;
			}

			if (ContentPathRelatives.Num() > 1)
			{
				UE_LOGF(LogPackageName, Error,
					"PackageName.UnregisterMountPoint: Root path '%ls' is mounted to multiple content paths, specify content path to unmount explicitly!",
					*LongPackageName);
				for (const FString& ContentPathRelative : ContentPathRelatives)
				{
					UE_LOGF(LogPackageName, Error, "- %ls", *ContentPathRelative);
				}
				return;
			}

			LocalPathStandard = ContentPathRelatives[0];
		}
		else
		{
			FString LocalPathAbsolute;
			if (!TryNormalizeLocalPath(Path, LocalPathStandard, LocalPathAbsolute, false /* bSilenceLogs */))
			{
				return;
			}

			UE::TReadScopeLock ScopeLock(MountLock);

			TArray<FString> RootPaths;
			for (const FMountPointKeyByPtr& ExistingKey : MountPoints)
			{
				FMountPoint* ExistingMount = ExistingKey.GetMountPoint();
				if (ExistingMount->IsMounted() && ExistingMount->GetContentPathRelative() == LocalPathStandard)
				{
					RootPaths.Add(ExistingMount->GetRootPath());
				}
			}

			if (RootPaths.Num() == 0)
			{
				UE_LOGF(LogPackageName, Error, "PackageName.UnregisterMountPoint: Content path '%ls' is not mounted!", *LocalPathStandard);
				return;
			}

			if (RootPaths.Num() > 1)
			{
				UE_LOGF(LogPackageName, Error, "PackageName.UnregisterMountPoint: Content path '%ls' is mounted to multiple root paths, specify root path to unmount explicitly!", *LocalPathStandard);
				for (const FString& MountRootPath: RootPaths)
				{
					UE_LOGF(LogPackageName, Error, "- %ls", *MountRootPath);
				}
				return;
			}

			LongPackageName = RootPaths[0];
		}

		Get().RemoveMountPoint(LongPackageName, LocalPathStandard);
	}
	else
	{
		Get().RemoveMountPoint(Args[0], Args[1]);
	}
}
#endif

void FPackageName::InternalFilenameToLongPackageName(FStringView InFilename, FStringBuilderBase& OutPackageName)
{
	using namespace UE::PackageName;

	const FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();
	FString Filename(InFilename);
	FPaths::NormalizeFilename(Filename);

	// Convert to relative path if it's not already a long package name
	bool bIsValidLongPackageName = false;
	{
		UE::TReadScopeLock ScopeLock(Paths.MountLock);
		const FMountPoint* const* MountPointPtr = Paths.RootPathTree.FindClosestValue(Filename);
		if (MountPointPtr)
		{
			bIsValidLongPackageName = true;
		}
	}

	if (!bIsValidLongPackageName)
	{
		Filename = IFileManager::Get().ConvertToRelativePath(*Filename);
		if (InFilename.Len() > 0 && InFilename[InFilename.Len() - 1] == '/')
		{
			// If InFilename ends in / but converted doesn't, add the / back
			bool bEndsInSlash = Filename.Len() > 0 && Filename[Filename.Len() - 1] == '/';

			if (!bEndsInSlash)
			{
				Filename += TEXT("/");
			}
		}
	}
	
	FStringView SplitOutPath;
	FStringView SplitOutName;
	FStringView SplitOutExtension;
	FPathViews::Split(Filename, SplitOutPath, SplitOutName, SplitOutExtension);

	// Do not strip the extension from paths with an empty pathname; if we do the caller can't tell the difference
	// between /Path/.ext and /Path/
	FStringView Result;
	if (!SplitOutName.IsEmpty() || SplitOutExtension.IsEmpty())
	{
		Result = FStringView(SplitOutPath.GetData(), UE_PTRDIFF_TO_INT32(SplitOutName.GetData() + SplitOutName.Len() - SplitOutPath.GetData()));
	}

	if (bIsValidLongPackageName && Result.Len() != Filename.Len())
	{
		UE_LOGF(LogPackageName, Warning, "TryConvertFilenameToLongPackageName was passed an ObjectPath (%.*ls) rather than a PackageName or FilePath; it will be converted to the PackageName. "
			"Accepting ObjectPaths is deprecated behavior and will be removed in a future release; TryConvertFilenameToLongPackageName will fail on ObjectPaths.", InFilename.Len(), InFilename.GetData());
	}

	auto TryGetLocalPathResult = [&Paths, &OutPackageName](FStringView SearchName)
	{
		UE::TReadScopeLock ScopeLock(Paths.MountLock);
		const FMountPoint* const* MountPointPtr = Paths.ContentPathTree.FindClosestValue(SearchName);
		if (!MountPointPtr)
		{
			return false;
		}
		const FMountPoint* MountPoint = *MountPointPtr;
		FStringView RelPath;
		if (!FPathViews::TryMakeChildPathRelativeTo(SearchName, MountPoint->GetContentPathRelative(), RelPath))
		{
			if (!FPathViews::TryMakeChildPathRelativeTo(SearchName, MountPoint->GetContentPathAbsolute(), RelPath))
			{
				// DirectoryTree is less conservative than TryMakeChildPathRelativeTo; e.g. d:/Dir//Root will match
				// d:/Dir/Root/Child in DirectoryTree but not in TryMakeChildPathRelativeTo. Treat this as PathNotMounted,
				// since no mountdir will match it according to TryMakeChildPathRelativeTo.
				return false;
			}
		}
		OutPackageName << MountPoint->GetRootPath();
		FPathViews::AppendPath(OutPackageName, RelPath);
		return true;
	};

	if (TryGetLocalPathResult(Result))
	{
		return;
	}

	// if we get here, we haven't converted to a package name, and it may be because the path was an unnormalized absolute ContentPath.
	// In that case, normalize it and check the ContentPathTree again
	if (!bIsValidLongPackageName)
	{
		// reset to the incoming string
		Filename = InFilename;
		if (!FPaths::IsRelative(Filename))
		{
			FPaths::NormalizeFilename(Filename);
			Result = FPathViews::GetBaseFilenameWithPath(Filename);
			if (TryGetLocalPathResult(Result))
			{
				return;
			}
		}
	}

	// Either the input string was already a LongPackageName, or we did not find a mapping for it. Return the last attempted Result
	OutPackageName << Result;
}

bool FPackageName::TryConvertFilenameToLongPackageName(const FString& InFilename, FString& OutPackageName, FString* OutFailureReason, const EConvertFlags Flags)
{
	TStringBuilder<256> FailureReasonBuilder;
	FStringBuilderBase* FailureReasonBuilderPtr = nullptr;
	if (OutFailureReason != nullptr)
	{
		FailureReasonBuilderPtr = &FailureReasonBuilder;
	}

	TStringBuilder<256> PackageNameBuilder;
	const bool bResult = TryConvertFilenameToLongPackageName(MakeStringView(InFilename), PackageNameBuilder, FailureReasonBuilderPtr, Flags);
	if (bResult)
	{
		OutPackageName = PackageNameBuilder.ToView();
	}
	else if (OutFailureReason != nullptr)
	{
		*OutFailureReason = FailureReasonBuilder.ToView();
	}
	return bResult;
}

bool FPackageName::TryConvertFilenameToLongPackageName(FStringView InFilename, FStringBuilderBase& OutPackageName, FStringBuilderBase* OutFailureReason /*= nullptr*/, const EConvertFlags Flags)
{
	TStringBuilder<256> LongPackageNameBuilder;
	InternalFilenameToLongPackageName(InFilename, LongPackageNameBuilder);
	const FStringView LongPackageName = LongPackageNameBuilder.ToView();

	if (LongPackageName.IsEmpty())
	{
		if (OutFailureReason != nullptr)
		{
			FStringView FilenameWithoutExtension = FPathViews::GetBaseFilenameWithPath(InFilename);
			OutFailureReason->Reset();
			*OutFailureReason << TEXTVIEW("FilenameToLongPackageName failed to convert '") << InFilename << TEXTVIEW("'. ");
			*OutFailureReason << TEXTVIEW("The Result would be indistinguishable from using '") << FilenameWithoutExtension << TEXTVIEW("' as the InFilename.");
		}
		return false;
	}

	// we don't support loading packages from outside of well defined places
	int32 CharacterIndex;
	const bool bContainsDot = LongPackageName.FindChar(TEXT('.'), CharacterIndex);
	const bool bContainsBackslash = LongPackageName.FindChar(TEXT('\\'), CharacterIndex);
	const bool bContainsColon = LongPackageName.FindChar(TEXT(':'), CharacterIndex);

	// For Verse files, the filenames can have dots in them to allow for `*.*.verse` to be considered as valid Verse snippets.
	if (!((!EnumHasAnyFlags(Flags, EConvertFlags::AllowDots) && bContainsDot) || bContainsBackslash || bContainsColon))
	{
		OutPackageName = LongPackageName;
		return true;
	}

	// if the package name resolution failed and a relative path was provided, convert to an absolute path
	// as content may be mounted in a different relative path to the one given
	if (FPathViews::IsRelativePath(InFilename))
	{
		TStringBuilder<256> AbsPath;
		FPathViews::ToAbsolutePath(InFilename, AbsPath);
		if (!FPathViews::IsRelativePath(AbsPath) && AbsPath.Len() > 1)
		{
			if (TryConvertFilenameToLongPackageName(AbsPath, OutPackageName, nullptr, Flags))
			{
				return true;
			}
		}
	}

	if (OutFailureReason != nullptr)
	{
		TStringBuilder<16> InvalidChars;
		if (bContainsDot)
		{
			InvalidChars << TEXT('.');
		}
		if (bContainsBackslash)
		{
			InvalidChars << TEXT('\\');
		}
		if (bContainsColon)
		{
			InvalidChars << TEXT(':');
		}
		OutFailureReason->Reset();
		*OutFailureReason << TEXTVIEW("FilenameToLongPackageName failed to convert '") << InFilename << TEXTVIEW("'. ");
		*OutFailureReason << TEXTVIEW("Attempt result was '") << LongPackageName << TEXTVIEW("', but the path contains illegal characters '") << InvalidChars << TEXTVIEW("'.");
	}

	return false;
}

FString FPackageName::FilenameToLongPackageName(const FString& InFilename)
{
	using namespace UE::PackageName;

	FString FailureReason;
	FString Result;
	if (!TryConvertFilenameToLongPackageName(InFilename, Result, &FailureReason))
	{
		TArray<FString> ContentRootsArrayRelative;
		TArray<FString> ContentRootsArrayAbsolute;
		{
			const auto& Paths = FLongPackagePathsSingleton::Get();
			UE::TReadScopeLock ScopeLock(Paths.MountLock);
			for (const FMountPointKeyByPtr& MountPointKey : Paths.MountPoints)
			{
				FMountPoint* MountPoint = MountPointKey.GetMountPoint();
				if (MountPoint->IsMounted())
				{
					ContentRootsArrayRelative.Add(MountPoint->GetContentPathRelative());
					ContentRootsArrayAbsolute.Add(MountPoint->GetContentPathAbsolute());
				}
			}
		}

		UE_LOGF(LogPackageName, Display, "FilenameToLongPackageName failed, we will issue a fatal log. Diagnostics:"
			"\n\tInFilename=%ls" "\n\tConvertToRelativePath=%ls"
			"\n\tConvertRelativePathToFull=%ls"
			"\n\tRootDir=%ls"
			"\n\tBaseDir=%ls"
			"\n\tContentRoots listed below...",
			*InFilename, *IFileManager::Get().ConvertToRelativePath(*InFilename),
			*FPaths::ConvertRelativePathToFull(InFilename), FPlatformMisc::RootDir(), FPlatformProcess::BaseDir()
			);
		
		if (ensure(ContentRootsArrayRelative.Num() == ContentRootsArrayAbsolute.Num()))
		{
			for (int32 RootIdx = 0; RootIdx < ContentRootsArrayRelative.Num(); ++RootIdx)
			{
				const FString& RelativeRoot = ContentRootsArrayRelative[RootIdx];
				const FString& AbsoluteRoot = ContentRootsArrayAbsolute[RootIdx];
				UE_LOGF(LogPackageName, Display, "\t\t%ls", *RelativeRoot);
				UE_LOGF(LogPackageName, Display, "\t\t\t%ls", *AbsoluteRoot)
			}
		}
		
		UE_LOGF(LogPackageName, Fatal, "%ls", *FailureReason);
	}
	return Result;
}

bool FPackageName::TryConvertLongPackageNameToFilename(const FString& InLongPackageName, FString& OutFilename, const FString& InExtension)
{
	return TryConvertLongPackageNameToFilename(FStringView(InLongPackageName), OutFilename, InExtension);
}

bool FPackageName::TryConvertLongPackageNameToFilename(FStringView InLongPackageName, FString& OutFilename, FStringView InExtension)
{
	using namespace UE::PackageName;

	const auto& Paths = FLongPackagePathsSingleton::Get();
	UE::TReadScopeLock ScopeLock(Paths.MountLock);
	const FMountPoint* const* MountPointPtr = Paths.RootPathTree.FindClosestValue(InLongPackageName);
	if (!MountPointPtr)
	{
		// Note that we return false for the root folder "/".
		return false;
	}

	const FMountPoint* MountPoint = *MountPointPtr;
	FStringView RelPath;
	if (!FPathViews::TryMakeChildPathRelativeTo(InLongPackageName, MountPoint->GetRootPath(), RelPath))
	{
		// DirectoryTree is less conservative than TryMakeChildPathRelativeTo; e.g. d:/Dir//Root will match
		// d:/Dir/Root/Child in DirectoryTree but not in TryMakeChildPathRelativeTo. Treat this as PathNotMounted,
		// since no mountdir will match it according to TryMakeChildPathRelativeTo.
		return false;
	}
	TStringBuilder<256> Builder;
	Builder << MountPoint->GetContentPathRelative();
	FPathViews::AppendPath(Builder, RelPath);
	Builder << InExtension;
	OutFilename = Builder.ToView();
	return true;
}

FString FPackageName::LongPackageNameToFilename(const FString& InLongPackageName, const FString& InExtension)
{
	FString Result;
	if (!TryConvertLongPackageNameToFilename(InLongPackageName, Result, InExtension))
	{
		UE_LOGF(LogPackageName, Fatal,"LongPackageNameToFilename failed to convert '%ls'. Path does not map to any roots.", *InLongPackageName);
	}
	return Result;
}

bool FPackageName::TryConvertToMountedPath(FStringView InFilePathOrPackageName, FString* OutLocalPathNoExtension,
	FString* OutPackageName, FString* OutObjectName, FString* OutSubObjectName, FString* OutExtension,
	EFlexNameType* OutFlexNameType, EErrorCode* OutFailureReason)
{
	TStringBuilder<256> MountPointPackageName;
	TStringBuilder<256> MountPointFilePath;
	TStringBuilder<256> PackageNameRelPath;
	TStringBuilder<128> ObjectPath;
	TStringBuilder<16> CustomExtension;
	EPackageExtension ExtensionType;

	bool bResult = TryConvertToMountedPathComponents(InFilePathOrPackageName, MountPointPackageName,
		MountPointFilePath, PackageNameRelPath, ObjectPath, ExtensionType, CustomExtension, OutFlexNameType,
		OutFailureReason);
	if (!bResult)
	{
		if (OutLocalPathNoExtension) OutLocalPathNoExtension->Reset();
		if (OutPackageName) OutPackageName->Reset();
		if (OutObjectName) OutObjectName->Reset();
		if (OutSubObjectName) OutSubObjectName->Reset();
		if (OutExtension) OutExtension->Reset();
		if (OutFlexNameType) *OutFlexNameType = EFlexNameType::Invalid;
		return false;
	}

	if (OutLocalPathNoExtension) *OutLocalPathNoExtension = FString(MountPointFilePath) + PackageNameRelPath;
	if (OutPackageName) *OutPackageName = FString(MountPointPackageName) + PackageNameRelPath;
	if (OutObjectName || OutSubObjectName)
	{
		FStringView ObjectPathView(ObjectPath);
		int32 SubObjectStart;
		if (ObjectPathView.FindChar(SUBOBJECT_DELIMITER_CHAR, SubObjectStart))
		{
			if (OutObjectName) *OutObjectName = ObjectPathView.Left(SubObjectStart);
			if (OutSubObjectName) *OutSubObjectName = ObjectPathView.RightChop(SubObjectStart + 1);
		}
		else
		{
			if (OutObjectName) *OutObjectName = ObjectPathView;
			if (OutSubObjectName) OutSubObjectName->Reset();
		}
	}
	if (OutExtension)
	{
		*OutExtension = ExtensionType == EPackageExtension::Custom ? *CustomExtension : LexToString(ExtensionType);
	}
	return true;
}
FString FPackageName::GetLongPackagePath(const FString& InLongPackageName)
{
	int32 IndexOfLastSlash = INDEX_NONE;
	if (InLongPackageName.FindLastChar(TEXT('/'), IndexOfLastSlash))
	{
		return InLongPackageName.Left(IndexOfLastSlash);
	}
	else
	{
		return InLongPackageName;
	}
}

void FPackageName::SplitFullObjectPath(const FString& InFullObjectPath, FString& OutClassName,
	FString& OutPackageName, FString& OutObjectName, FString& OutSubObjectName, bool bDetectClassName)
{
	FStringView ClassName;
	FStringView PackageName;
	FStringView ObjectName;
	FStringView SubObjectName;
	SplitFullObjectPath(InFullObjectPath, ClassName, PackageName, ObjectName, SubObjectName, bDetectClassName);
	OutClassName = ClassName;
	OutPackageName = PackageName;
	OutObjectName = ObjectName;
	OutSubObjectName = SubObjectName;
}

void FPackageName::SplitFullObjectPath(FStringView InFullObjectPath, FStringView& OutClassName,
	FStringView& OutPackageName, FStringView& OutObjectName, FStringView& OutSubObjectName, bool bDetectClassName)
{
	FStringView Remaining = InFullObjectPath.TrimStartAndEnd();

	auto ExtractBeforeDelim = [&Remaining](TCHAR Delim, FStringView& OutStringView)
	{
		int32 DelimIndex;
		if (Remaining.FindChar(Delim, DelimIndex))
		{
			OutStringView = Remaining.Left(DelimIndex);
			Remaining.RightChopInline(DelimIndex + 1);
			return true;
		}
		else
		{
			OutStringView.Reset();
			return false;
		}
	};

	// If we are handling class names, split on the first space. If we are not, or there is no space,
	// then ClassName is empty and the remaining string is PackageName.ObjectName:SubObjectName
	if (bDetectClassName)
	{
		ExtractBeforeDelim(TEXT(' '), OutClassName);
	}
	else
	{
		OutClassName.Reset();
	}
	if (ExtractBeforeDelim(TEXT('.'), OutPackageName))
	{
		if (ExtractBeforeDelim(TEXT(':'), OutObjectName))
		{
			OutSubObjectName = Remaining;
		}
		else
		{
			// If no :, then the remaining string is ObjectName
			OutObjectName = Remaining;
			OutSubObjectName = FStringView();
		}
	}
	else 
	{
		// If no '.', then the remaining string is PackageName
		OutPackageName = Remaining;
		OutObjectName = FStringView();
		OutSubObjectName = FStringView();
	}
}

void FPackageName::SplitFullObjectPath(const FString& InFullObjectPath, FString& OutClassName,
	FString& OutPackageName, FString& OutObjectName, TArray<FString>& OutSubobjectNames, bool bDetectClassName)
{
	FStringView ClassName;
	FStringView PackageName;
	FStringView ObjectName;
	TArray<FStringView> SubObjectNames;
	SplitFullObjectPath(InFullObjectPath, ClassName, PackageName, ObjectName, SubObjectNames, bDetectClassName);
	OutClassName = ClassName;
	OutPackageName = PackageName;
	OutObjectName = ObjectName;
	OutSubobjectNames.Empty();
	OutSubobjectNames.Append(SubObjectNames);
}

void FPackageName::SplitFullObjectPath(FStringView InFullObjectPath, FStringView& OutClassName,
	FStringView& OutPackageName, FStringView& OutObjectName, TArray<FStringView>& OutSubobjectNames, bool bDetectClassName)
{
	FStringView FullSubobjectString;
	SplitFullObjectPath(InFullObjectPath, OutClassName, OutPackageName, OutObjectName, FullSubobjectString, bDetectClassName);

	if (FullSubobjectString.Len() > 0)
	{
		OutSubobjectNames.Empty();

		auto ExtractBeforeDotDelim = [&FullSubobjectString](FStringView& OutStringView)
		{
			int32 DelimIndex;
			if (FullSubobjectString.FindChar(TCHAR('.'), DelimIndex))
			{
				OutStringView = FullSubobjectString.Left(DelimIndex);
				FullSubobjectString.RightChopInline(DelimIndex + 1);
				return true;
			}
			else
			{
				OutStringView.Reset();
				return false;
			}
		};

		bool bFoundDelim = false;
		do
		{
			FStringView SubobjectName;
			bFoundDelim = ExtractBeforeDotDelim(SubobjectName);
			const bool bSubobjectNameIsNotEmpty = (SubobjectName.Len() > 0);
			if (bFoundDelim && bSubobjectNameIsNotEmpty)
			{
				OutSubobjectNames.Add(SubobjectName);
			}
			else if (FullSubobjectString.Len() > 0)
			{
				OutSubobjectNames.Add(FullSubobjectString);
			}
		} while (bFoundDelim);
	}
}

FString FPackageName::GetLongPackageAssetName(const FString& InLongPackageName)
{
	return GetShortName(InLongPackageName);
}

bool FPackageName::DoesPackageNameContainInvalidCharacters(FStringView InLongPackageName, FText* OutReason)
{
	EErrorCode Reason;
	if (DoesPackageNameContainInvalidCharacters(InLongPackageName, &Reason))
	{
		if (OutReason) *OutReason = FormatErrorAsText(InLongPackageName, Reason);
		return true;
	}
	return false;
}

bool FPackageName::DoesPackageNameContainInvalidCharacters(FStringView InLongPackageName, EErrorCode* OutReason /*= nullptr */)
{
	// See if the name contains invalid characters.
	TStringBuilder<32> MatchedInvalidChars;
	for (const TCHAR* InvalidCharacters = INVALID_LONGPACKAGE_CHARACTERS; *InvalidCharacters; ++InvalidCharacters)
	{
		int32 OutIndex;
		if (InLongPackageName.FindChar(*InvalidCharacters, OutIndex))
		{
			MatchedInvalidChars += *InvalidCharacters;
		}
	}
	if (MatchedInvalidChars.Len())
	{
		if (OutReason) *OutReason = EErrorCode::PackageNameContainsInvalidCharacters;
		return true;
	}
	if (OutReason) *OutReason = EErrorCode::PackageNameUnknown;
	return false;
}

bool FPackageName::IsValidTextForLongPackageName(FStringView InLongPackageName, FText* OutReason)
{
	EErrorCode Reason;
	if (!IsValidTextForLongPackageName(InLongPackageName, &Reason))
	{
		if (OutReason) *OutReason = FormatErrorAsText(InLongPackageName, Reason);
		return false;
	}
	return true;
}

bool FPackageName::IsValidTextForLongPackageName(FStringView InLongPackageName, EErrorCode* OutReason /*= nullptr */)
{
	// All package names must contain a leading slash, root, slash and name, at minimum theoretical length ("/A/B") is 4
	if (InLongPackageName.Len() < PackageNameConstants::MinPackageNameLength)
	{
		if (OutReason) *OutReason = EErrorCode::LongPackageNames_PathTooShort;
		return false;
	}
	// Package names start with a leading slash.
	if (InLongPackageName[0] != '/')
	{
		if (OutReason) *OutReason = EErrorCode::LongPackageNames_PathWithNoStartingSlash;
		return false;
	}
	// Package names do not end with a trailing slash.
	if (InLongPackageName[InLongPackageName.Len() - 1] == '/')
	{
		if (OutReason) *OutReason = EErrorCode::LongPackageNames_PathWithTrailingSlash;
		return false;
	}
	if (InLongPackageName.Contains(TEXT("//")))
	{
		if (OutReason) *OutReason = EErrorCode::LongPackageNames_PathWithDoubleSlash;
		return false;
	}
	// Check for invalid characters
	if (DoesPackageNameContainInvalidCharacters(InLongPackageName, OutReason))
	{
		return false;
	}
	if (OutReason) *OutReason = EErrorCode::PackageNameUnknown;
	return true;
}

bool FPackageName::IsValidLongPackageName(FStringView InLongPackageName, bool bIncludeReadOnlyRoots, FText* OutReason)
{
	using namespace UE::PackageName;

	EErrorCode Reason;
	if (!IsValidLongPackageName(InLongPackageName, bIncludeReadOnlyRoots, &Reason))
	{
		if (OutReason)
		{
			if (Reason == EErrorCode::PackageNamePathNotMounted)
			{
				const FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();

				UE::TReadScopeLock ScopeLock(Paths.MountLock);
				TStringBuilder<2048> ValidRootsString;
				for (const FMountPointKeyByPtr& MountPointKey : Paths.MountPoints)
				{
					FMountPoint* MountPoint = MountPointKey.GetMountPoint();
					if (MountPoint->IsMounted() && (bIncludeReadOnlyRoots || !MountPoint->IsReadOnly()))
					{
						ValidRootsString << MountPoint->GetRootPath() << TEXT(", ");
					}
				}
				if (bIncludeReadOnlyRoots)
				{
					const FString& ExtraMountRootPath = Paths.MemoryRootPath;
					ValidRootsString << ExtraMountRootPath << TEXT(", ");
				}
				ValidRootsString.RemoveSuffix(2); // Remove the trailing ", "
				*OutReason = FText::Format( NSLOCTEXT("Core", "LongPackageNames_InvalidRoot", "Path does not start with a valid root. Path must begin with one of: {0}"),
					FText::FromString( FString(ValidRootsString) ) );
			}
			else
			{
				*OutReason = FormatErrorAsText(InLongPackageName, Reason);
			}
		}
		return false;
	}
	return true;
}

bool FPackageName::IsValidLongPackageName(FStringView InLongPackageName, bool bIncludeReadOnlyRoots /*= false*/, EErrorCode* OutReason /*= nullptr */)
{
	using namespace UE::PackageName;

	if (!IsValidTextForLongPackageName(InLongPackageName, OutReason))
	{
		return false;
	}

	// Check valid roots
	const FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();
	UE::TReadScopeLock ScopeLock(Paths.MountLock);

	const FMountPoint* const* MountPointPtr = Paths.RootPathTree.FindClosestValue(InLongPackageName);
	if (MountPointPtr)
	{
		if (bIncludeReadOnlyRoots || !(*MountPointPtr)->IsReadOnly())
		{
			// DirectoryTree is less conservative than IsParentPathOf; e.g. d:/Dir//Root will match
			// d:/Dir/Root/Child in DirectoryTree but not in IsParentPathOf. Treat this as PathNotMounted,
			// since no mountdir will match it according to IsParentPathOf.
			if (FPathViews::IsParentPathOf((*MountPointPtr)->GetRootPath(), InLongPackageName))
			{
				if (OutReason) *OutReason = EErrorCode::PackageNameUnknown;
				return true;
			}
		}
	}
	else
	{
		if (bIncludeReadOnlyRoots)
		{
			const FString& ExtraPackageRoot = Paths.MemoryRootPath;
			if (FPathViews::IsParentPathOf(ExtraPackageRoot, InLongPackageName))
			{
				if (OutReason) *OutReason = EErrorCode::PackageNameUnknown;
				return true;
			}
		}
	}

	if (OutReason) *OutReason = EErrorCode::PackageNamePathNotMounted;
	return false;
}

bool FPackageName::IsValidObjectPath(FStringView InObjectPath, FText* OutReason)
{
	FStringView PackageName;
	FStringView RemainingObjectPath;

	// Check for package delimiter
	int32 ObjectDelimiterIdx;
	if (InObjectPath.FindChar(TEXT('.'), ObjectDelimiterIdx))
	{
		if (ObjectDelimiterIdx == InObjectPath.Len() - 1)
		{
			if (OutReason)
			{
				*OutReason = NSLOCTEXT("Core", "ObjectPath_EndWithPeriod", "Object Path may not end with .");
			}
			return false;
		}

		PackageName = InObjectPath.Mid(0, ObjectDelimiterIdx);
		RemainingObjectPath = InObjectPath.Mid(ObjectDelimiterIdx + 1);
	}
	else
	{
		PackageName = InObjectPath;
	}

	if (!IsValidLongPackageName(PackageName, true, OutReason))
	{
		return false;
	}

	if (RemainingObjectPath.Len() > 0)
	{
		FText PathContext = NSLOCTEXT("Core", "ObjectPathContext", "Object Path");
		if (!FName::IsValidXName(RemainingObjectPath, INVALID_OBJECTPATH_CHARACTERS, OutReason, &PathContext))
		{
			return false;
		}

		TCHAR LastChar = RemainingObjectPath[RemainingObjectPath.Len() - 1];
		if (LastChar == '.' || LastChar == ':')
		{
			if (OutReason)
			{
				*OutReason = NSLOCTEXT("Core", "ObjectPath_PathWithTrailingSeperator", "Object Path may not end with : or .");
			}
			return false;
		}

		int32 SlashIndex;
		if (RemainingObjectPath.FindChar(TEXT('/'), SlashIndex))
		{
			if (OutReason)
			{
				*OutReason = NSLOCTEXT("Core", "ObjectPath_SlashAfterPeriod", "Object Path may not have / after first .");
			}

			return false;
		}
	}

	return true;
}

bool FPackageName::IsValidPath(FStringView InPath)
{
	const FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();
	UE::TReadScopeLock ScopeLock(Paths.MountLock);
	// Note that we return false for the root folder "/".
	return Paths.RootPathTree.FindClosestValue(InPath) != nullptr ||
		IsMemoryPackage(InPath);
}

TRefCountPtr<UE::PackageName::IMountPoint> FPackageName::RegisterMountPoint(const FString& RootPath, const FString& ContentPath)
{
	return FLongPackagePathsSingleton::Get().InsertMountPoint(RootPath, ContentPath);
}

void FPackageName::RegisterMountPoint(const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint)
{
	FLongPackagePathsSingleton::Get().InsertMountPoint(MountPoint);
}

void FPackageName::UnRegisterMountPoint(const FString& RootPath, const FString& ContentPath)
{
	FLongPackagePathsSingleton::Get().RemoveMountPoint(RootPath, ContentPath);
}

void FPackageName::UnRegisterMountPoint(const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint)
{
	FLongPackagePathsSingleton::Get().RemoveMountPoint(MountPoint);
}

bool FPackageName::MountPointExists(FStringView RootPath)
{
	return FLongPackagePathsSingleton::Get().MountPointExists(RootPath);
}

bool FPackageName::MountPointExists(const FString& RootPath)
{
	return FLongPackagePathsSingleton::Get().MountPointExists(RootPath);
}

bool FPackageName::MountPointExists(const TCHAR* RootPath)
{
	return FLongPackagePathsSingleton::Get().MountPointExists(FStringView(RootPath));
}

TRefCountPtr<UE::PackageName::IMountPoint>
FPackageName::FindMountPointByRootPackageName(FStringView RootLongPackageName)
{
	return FLongPackagePathsSingleton::Get().FindMountPointByRootPackageName(RootLongPackageName);
}

TRefCountPtr<UE::PackageName::IMountPoint>
FPackageName::FindMountPointByRootLocalPath(FStringView RootLocalPath)
{
	return FLongPackagePathsSingleton::Get().FindMountPointByRootLocalPath(RootLocalPath);
}

TRefCountPtr<UE::PackageName::IMountPoint>
FPackageName::FindMountPoint(FStringView RootLongPackageName, FStringView RootLocalPath)
{
	return FLongPackagePathsSingleton::Get().FindMountPoint(RootLongPackageName, RootLocalPath);
}

TRefCountPtr<UE::PackageName::IMountPoint>
FPackageName::FindOrAddMountPoint(FStringView RootLongPackageName, FStringView RootLocalPath)
{
	return FLongPackagePathsSingleton::Get().FindOrAddMountPoint(RootLongPackageName, RootLocalPath);
}

TRefCountPtr<UE::PackageName::IMountPoint>
FPackageName::FindMountPointByChildLongPackageName(FStringView ChildLongPackageName)
{
	return FLongPackagePathsSingleton::Get().FindMountPointByChildLongPackageName(ChildLongPackageName);
}

FName FPackageName::GetPackageMountPoint(FStringView InPackagePath, bool InWithoutSlashes)
{
	using namespace UE::PackageName;

	FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();
	
	UE::TReadScopeLock ScopeLock(Paths.MountLock);
	int32 WithoutSlashes = InWithoutSlashes ? 1 : 0;
	auto TryRootPath = [InPackagePath, WithoutSlashes](FStringView RootPath)
	{
		if (FPathViews::IsParentPathOf(RootPath, InPackagePath))
		{
			return FName(RootPath.Mid(WithoutSlashes, RootPath.Len() - (2 * WithoutSlashes)));
		}
		return FName();
	};
	const FMountPoint* const* MountPointPtr = Paths.RootPathTree.FindClosestValue(InPackagePath);
	if (MountPointPtr)
	{
		// DirectoryTree is less conservative than IsParentPathOf; e.g. d:/Dir//Root will match
		// d:/Dir/Root/Child in DirectoryTree but not in IsParentPathOf. Treat this as PathNotMounted,
		// since no mountdir will match it according to IsParentPathOf.
		return TryRootPath((*MountPointPtr)->GetRootPath());
	}
	else
	{
		const FString& ExtraPackageRoot = Paths.MemoryRootPath;
		FName Result = TryRootPath(ExtraPackageRoot);
		if (Result.IsValid())
		{
			return Result;
		}
	}

	return FName();
}

FName FPackageName::GetPackageMountPoint(const FString& InPackagePath, bool InWithoutSlashes)
{
	return GetPackageMountPoint(FStringView(InPackagePath), InWithoutSlashes);
}

FName FPackageName::GetPackageMountPoint(const TCHAR* InPackagePath, bool InWithoutSlashes)
{
	return GetPackageMountPoint(FStringView(InPackagePath), InWithoutSlashes);
}

FString FPackageName::GetContentPathForPackageRoot(FStringView InMountPoint)
{
	using namespace UE::PackageName;

	FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();

	UE::TReadScopeLock ScopeLock(Paths.MountLock);
	const FMountPoint* const* MountPointPtr = Paths.RootPathTree.FindClosestValue(InMountPoint);
	if (!MountPointPtr)
	{
		return FString();
	}
	const FMountPoint* MountPoint = *MountPointPtr;
	return MountPoint->GetContentPathAbsolute();
}

bool FPackageName::TryConvertToMountedPathComponents(FStringView InFilePathOrPackageName,
	FStringBuilderBase& OutMountPointPackageName, FStringBuilderBase& OutMountPointFilePath,
	FStringBuilderBase& OutRelPath, FStringBuilderBase& OutObjectName, EPackageExtension& OutExtension,
	FStringBuilderBase& OutCustomExtension, EFlexNameType* OutFlexNameType, EErrorCode* OutFailureReason)
{
	auto ClearSuccessOutputs = [&OutMountPointPackageName, &OutMountPointFilePath, &OutRelPath,
		&OutObjectName, &OutExtension, &OutCustomExtension, OutFlexNameType]()
	{
		OutMountPointPackageName.Reset();
		OutMountPointFilePath.Reset();
		OutRelPath.Reset();
		OutObjectName.Reset();
		OutExtension = EPackageExtension::Unspecified;
		OutCustomExtension.Reset();
		if (OutFlexNameType) *OutFlexNameType = EFlexNameType::Invalid;
	};

	EFlexNameType PathFlexNameType;
	bool bResult = TryGetMountPointForPath(InFilePathOrPackageName, OutMountPointPackageName,
		OutMountPointFilePath, OutRelPath, &PathFlexNameType, OutFailureReason);
	if (!bResult)
	{
		// Complain about spaces in the name before pathnotmounted.
		// "Texture2D /Engine/Foo" should be a spaces error, not a PathNotMounted error.
		int32 UnusedIndex;
		if (OutFailureReason && *OutFailureReason == EErrorCode::PackageNamePathNotMounted &&
			InFilePathOrPackageName.FindChar(TEXT(' '), UnusedIndex))
		{
			*OutFailureReason = EErrorCode::PackageNameSpacesNotAllowed;
		}
		ClearSuccessOutputs();
		return false;
	}

	OutObjectName.Reset();
	OutExtension = EPackageExtension::Unspecified;
	OutCustomExtension.Reset();

	if (PathFlexNameType == EFlexNameType::LocalPath)
	{
		// Remove Extension from OutRelPath and put it into OutExtension
		int32 ExtensionStart;
		OutExtension = FPackagePath::ParseExtension(OutRelPath, &ExtensionStart);
		if (OutExtension == EPackageExtension::Custom)
		{
			OutCustomExtension << FStringView(OutRelPath).RightChop(ExtensionStart);
		}
		OutRelPath.RemoveSuffix(OutRelPath.Len() - ExtensionStart);

	}
	else if (PathFlexNameType == EFlexNameType::ObjectPath)
	{
		// Legacy behavior; convert ObjectPaths to packageName
		TStringBuilder<256> ObjectPath;
		ObjectPath << OutMountPointPackageName << OutRelPath;
		FStringView ClassName;
		FStringView PackageName;
		FStringView ObjectName;
		FStringView SubObjectName;
		SplitFullObjectPath(ObjectPath, ClassName, PackageName, ObjectName, SubObjectName);
		if (ClassName.Len() > 0)
		{
			// If there is no classname, but the packagename or objectname has spaces in the name (which is invalid),
			// it will reach this location as well. This function does not need to care about spaces in the objectname;
			// silently let that invalidity pass.
			// Test whether we are in the case of no class, but spaces in the objectname, by checking whether the PackageName
			// with class detection is invalid, but the packagename without class detection is valid.
			bool bObjectNameErrorCase = false;
			if (!IsValidTextForLongPackageName(PackageName))
			{
				SplitFullObjectPath(ObjectPath, ClassName, PackageName, ObjectName, SubObjectName, false /* bDetectClassName */);
				if (IsValidTextForLongPackageName(PackageName))
				{
					bObjectNameErrorCase = true;
				}
			}
			if (!bObjectNameErrorCase)
			{
				// It's not the object error case, so it's either a fullobjectpath with class or it is a packagename with spaces
				// Both of those are unrecoverable errors for this function, and we can't easily distinguish them. Report
				// the error as spaces are invalid.
				ClearSuccessOutputs();
				if (OutFailureReason) *OutFailureReason = EErrorCode::PackageNameSpacesNotAllowed;
				return false;
			}
		}

		if (!IsValidTextForLongPackageName(PackageName, OutFailureReason))
		{
			ClearSuccessOutputs();
			return false;
		}
		OutObjectName << ObjectName;
		if (SubObjectName.Len())
		{
			OutObjectName << SUBOBJECT_DELIMITER << SubObjectName;
		}
		check(FStringView(PackageName).StartsWith(OutMountPointPackageName));
		int32 RelPathPackageNameLen = PackageName.Len() - OutMountPointPackageName.Len();
		OutRelPath.RemoveSuffix(OutRelPath.Len() - RelPathPackageNameLen);
	}
	else
	{
		check(PathFlexNameType == EFlexNameType::PackageName);
		TStringBuilder<256> PackageNameBuffer;
		PackageNameBuffer << OutMountPointPackageName << OutRelPath;
		FStringView PackageName(PackageNameBuffer);
		FStringView PackageNameNoTrailingSlash = PackageName;
		if (PackageName.Len() > 0 && FPathViews::IsSeparator(PackageName[PackageName.Len() - 1]))
		{
			// IsValidTextForLongPackageName rejects packagenames with a trailing slash, but we want to allow that
			// because this function allows both files and directories
			PackageNameNoTrailingSlash.LeftChopInline(1);
		}
		int32 UnusedIndex;
		if (PackageNameNoTrailingSlash.FindChar(TEXT(' '), UnusedIndex))
		{
			ClearSuccessOutputs();
			if (OutFailureReason) *OutFailureReason = EErrorCode::PackageNameSpacesNotAllowed;
			return false;
		}
		else if (!IsValidTextForLongPackageName(PackageNameNoTrailingSlash, OutFailureReason))
		{
			ClearSuccessOutputs();
			return false;
		}
	}

	if (OutFlexNameType) *OutFlexNameType = PathFlexNameType;
	if (OutFailureReason) *OutFailureReason = EErrorCode::PackageNameUnknown;
	return true;
}

bool FPackageName::TryGetMountPointForPath(FStringView InFilePathOrPackageName, FStringBuilderBase& OutMountPointPackageName, FStringBuilderBase& OutMountPointFilePath, FStringBuilderBase& OutRelPath, EFlexNameType* OutFlexNameType, EErrorCode* OutFailureReason)
{
	using namespace UE::PackageName;

	OutMountPointPackageName.Reset();
	OutMountPointFilePath.Reset();
	OutRelPath.Reset();

	if (InFilePathOrPackageName.IsEmpty())
	{
		if (OutFlexNameType)
		{
			*OutFlexNameType = EFlexNameType::Invalid;
		}
		if (OutFailureReason)
		{
			*OutFailureReason = EErrorCode::PackageNameEmptyPath;
		}
		return false;
	}

	TStringBuilder<256> PossibleAbsFilePath;
	FPathViews::ToAbsolutePath(InFilePathOrPackageName, PossibleAbsFilePath);
	FStringView RelPath;
	const FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();
	UE::TReadScopeLock ScopeLock(Paths.MountLock);
	const FMountPoint* const* MountPointPtr = Paths.RootPathTree.FindClosestValue(InFilePathOrPackageName);
	if (MountPointPtr)
	{
		const FMountPoint* MountPoint = *MountPointPtr;
		if (!FPathViews::TryMakeChildPathRelativeTo(InFilePathOrPackageName, MountPoint->GetRootPath(), RelPath))
		{
			// DirectoryTree is less conservative than TryMakeChildPathRelativeTo; e.g. d:/Dir//Root will match
			// d:/Dir/Root/Child in DirectoryTree but not in TryMakeChildPathRelativeTo. Treat this as PathNotMounted,
			// since no mountdir will match it according to TryMakeChildPathRelativeTo.
			if (OutFlexNameType)
			{
				*OutFlexNameType = EFlexNameType::Invalid;
			}
			if (OutFailureReason)
			{
				*OutFailureReason = EErrorCode::PackageNamePathNotMounted;
			}
			return false;
		}
		OutMountPointPackageName << MountPoint->GetRootPath();
		OutMountPointFilePath << MountPoint->GetContentPathRelative();
		OutRelPath << RelPath;
		if (OutFlexNameType)
		{
			if (Algo::Find(RelPath, TEXT('.')))
			{
				*OutFlexNameType = EFlexNameType::ObjectPath;
			}
			else
			{
				*OutFlexNameType = EFlexNameType::PackageName;
			}
		}
		if (OutFailureReason)
		{
			*OutFailureReason = EErrorCode::PackageNameUnknown;
		}
		return true;
	}

	MountPointPtr = Paths.ContentPathTree.FindClosestValue(PossibleAbsFilePath);
	if (MountPointPtr)
	{
		const FMountPoint* MountPoint = *MountPointPtr;
		if (!FPathViews::TryMakeChildPathRelativeTo(PossibleAbsFilePath, MountPoint->GetContentPathAbsolute(), RelPath))
		{
			if (!FPathViews::TryMakeChildPathRelativeTo(PossibleAbsFilePath, MountPoint->GetContentPathRelative(), RelPath))
			{
				// DirectoryTree is less conservative than TryMakeChildPathRelativeTo; e.g. d:/Dir//Root will match
				// d:/Dir/Root/Child in DirectoryTree but not in TryMakeChildPathRelativeTo. Treat this as PathNotMounted,
				// since no mountdir will match it according to TryMakeChildPathRelativeTo.
				if (OutFlexNameType)
				{
					*OutFlexNameType = EFlexNameType::Invalid;
				}
				if (OutFailureReason)
				{
					*OutFailureReason = EErrorCode::PackageNamePathNotMounted;
				}
				return false;
			}
		}
		OutMountPointPackageName << MountPoint->GetRootPath();
		OutMountPointFilePath << MountPoint->GetContentPathRelative();
		OutRelPath << RelPath;
		if (OutFlexNameType)
		{
			*OutFlexNameType = EFlexNameType::LocalPath;
		}
		if (OutFailureReason)
		{
			*OutFailureReason = EErrorCode::PackageNameUnknown;
		}
		return true;
	}
	if (OutFlexNameType)
	{
		*OutFlexNameType = EFlexNameType::Invalid;
	}
	if (OutFailureReason)
	{
		if (FPathViews::TryMakeChildPathRelativeTo(InFilePathOrPackageName, Paths.MemoryRootPath, RelPath))
		{
			*OutFailureReason = EErrorCode::PackageNamePathIsMemoryOnly;
		}
		else
		{
			*OutFailureReason = EErrorCode::PackageNamePathNotMounted;
		}
	}
	return false;
}

FPackageName::FGetExplanationForUnavailablePackageDelegate& FPackageName::GetExplanationForUnavailablePackageDelegate()
{
	static FPackageName::FGetExplanationForUnavailablePackageDelegate Delegate;
	return Delegate;
}

void FPackageName::GetExplanationForUnavailablePackage(const FName& UnavailablePackageName, FStringBuilderBase& InOutExplanation)
{
	FString PackageRoot = FString::Printf(TEXT("/%s"), *FPackageName::SplitPackageNameRoot(UnavailablePackageName, nullptr));
	if (FPackageName::MountPointExists(PackageRoot))
	{
		// The mount point exists but we failed to load the asset. Maybe the package itself didn't exist?
		const EPackageLocationFilter BothLocationsMask = (EPackageLocationFilter)((uint8)EPackageLocationFilter::FileSystem | (uint8)EPackageLocationFilter::IoDispatcher);
		FPackagePath SkippedPackagePath;
		const FString& UnavailablePackageNameString = UnavailablePackageName.ToString();
		bool ConvertedToPath = FPackagePath::TryFromMountedName(UnavailablePackageNameString, SkippedPackagePath);
		if (ensure(ConvertedToPath))
		{
			EPackageLocationFilter PackageLocations = DoesPackageExistEx(SkippedPackagePath, BothLocationsMask);

			if (PackageLocations == EPackageLocationFilter::None)
			{
				InOutExplanation.Appendf(TEXT("FPackageName: Skipped package %s has a valid, mounted, mount point but does not exist either on disk or in iostore."),
					*UnavailablePackageNameString);
				SkippedPackagePath.SetHeaderExtension(EPackageExtension::EmptyString); // Suppress the warning from FPackagePath about unknown asset
				const FString& LocalFullPath = SkippedPackagePath.GetLocalFullPath();
				if (LocalFullPath.Len())
				{
					FString PathToPrint = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*LocalFullPath);
					if (PathToPrint.Len() == 0)
					{
						PathToPrint = LocalFullPath;
					}
					InOutExplanation.Appendf(TEXT(" The uncooked file would be expected on disk at '%s'. Perhaps it has been deleted or was not synced?\n"), *PathToPrint);
				}
				else
				{
					InOutExplanation.Append(TEXT("\n"));
				}
			}
			else
			{
				InOutExplanation.Appendf(TEXT("FPackageName: Skipped package %s was found %s but for some reason could not be found or used by the loader. This could occur if the loader is only considering cooked packages but the package exists only on disk.\n"),
					*UnavailablePackageNameString,
					(PackageLocations == BothLocationsMask) ? TEXT("on disk and in iodispatcher") :
					((PackageLocations == EPackageLocationFilter::FileSystem) ? TEXT("on disk") : TEXT("in iodispatcher")));
			}
		}
	}
	else
	{
		InOutExplanation.Appendf(TEXT("FPackageName: Unable to identify a valid mount point associated with skipped package %s. The package root is unknown.\n"), *UnavailablePackageName.ToString());
	}

	// The plugin manager needs to be called directly rather than via delegate due to dependency issue (CoreUObject depends on Projects and not vice versa)
	IPluginManager::Get().GetExplanationForUnavailablePackage(UnavailablePackageName, InOutExplanation);

	// Give other systems an opportunity to add to this explanation
	GetExplanationForUnavailablePackageDelegate().Broadcast(UnavailablePackageName, InOutExplanation);
}

FString FPackageName::GetModuleScriptPackageName(FStringView InModuleName)
{
	return FString::Printf(TEXT("/Script/%.*s"), InModuleName.Len(), InModuleName.GetData());
}

FName FPackageName::GetModuleScriptPackageName(FName InModuleName)
{
	return FName(WriteToString<128>(TEXT("/Script/"), InModuleName));
}

bool FPackageName::TryConvertScriptPackageNameToModuleName(FStringView PackageName, FStringView& OutModuleName)
{
	constexpr FStringView ScriptPrefix(TEXTVIEW("/Script/"));
	if (!PackageName.StartsWith(ScriptPrefix))
	{
		OutModuleName.Reset();
		return false;
	}
	OutModuleName = PackageName.RightChop(ScriptPrefix.Len());
	return true;
};


FString FPackageName::ConvertToLongScriptPackageName(const TCHAR* InShortName)
{
	if (IsShortPackageName(FString(InShortName)))
	{
		return GetModuleScriptPackageName(FStringView(InShortName));
	}
	else
	{
		return InShortName;
	}
}

// Short to long script package name map.
static TMap<FName, FName> ScriptPackageNames;


// @todo: This stuff needs to be eliminated as soon as we can make sure that no legacy short package names
//        are in use when referencing class names in UObject module "class packages"
void FPackageName::RegisterShortPackageNamesForUObjectModules()
{
	// @todo: Ideally we'd only be processing UObject modules, not every module, but we have
	//        no way of knowing which modules may contain UObjects (without say, having UBT save a manifest.)
	// @todo: This stuff is a bomb waiting to explode.  Because short package names can
	//        take precedent over other object names, modules can reserve names for other types!
	TArray<FName> AllModuleNames;
	FModuleManager::Get().FindModules( TEXT( "*" ), AllModuleNames );
	for( TArray<FName>::TConstIterator ModuleNameIt( AllModuleNames ); ModuleNameIt; ++ModuleNameIt )
	{
		ScriptPackageNames.Add(*ModuleNameIt, GetModuleScriptPackageName(*ModuleNameIt));
	}
}

FName* FPackageName::FindScriptPackageName(FName InShortName)
{
	return ScriptPackageNames.Find(InShortName);
}

bool FPackageName::FindPackageFileWithoutExtension(const FString& InPackageFilename, FString& OutFilename)
{
	FPackagePath PackagePath = FPackagePath::FromLocalPath(InPackageFilename);
	if (IPackageResourceManager::Get().DoesPackageExist(PackagePath, &PackagePath))
	{
		OutFilename = PackagePath.GetLocalFullPath();
		return true;
	}
	else
	{
		return false;
	}
}

bool FPackageName::FixPackageNameCase(FString& LongPackageName, FStringView Extension)
{
	FPackagePath PackagePath;
	if (!FPackagePath::TryFromPackageName(LongPackageName, PackagePath))
	{
		return false;
	}
	if (!IPackageResourceManager::Get().TryMatchCaseOnDisk(PackagePath, &PackagePath))
	{
		return false;
	}
	TStringBuilder<256> DiskPackageName;
	PackagePath.AppendPackageName(DiskPackageName);
	check(FStringView(LongPackageName).Equals(DiskPackageName, ESearchCase::IgnoreCase));
	LongPackageName = DiskPackageName;
	return true;
}

bool FPackageName::DoesPackageExist(const FString& LongPackageName, FString* OutFilename, bool InAllowTextFormats)
{
	FPackagePath PackagePath;
	EErrorCode FailureReason;
	if (!TryConvertToMountedPackagePath(LongPackageName, PackagePath, FailureReason))
	{
		FString Message = FString::Printf(TEXT("DoesPackageExist called on PackageName that will always return false. Reason: %s"), *FormatErrorAsString(LongPackageName, FailureReason));
		UE_LOGF(LogPackageName, Warning, "%ls", *Message);
		return false;
	}
	if (!DoesPackageExist(PackagePath, false /* bMatchCaseOnDisk */, &PackagePath))
	{
		return false;
	}
	if (!InAllowTextFormats && IsTextPackageExtension(PackagePath.GetHeaderExtension()))
	{
		return false;
	}
	if (OutFilename)
	{
		*OutFilename = PackagePath.GetLocalFullPath();
	}
	return true;
}

bool FPackageName::DoesPackageExist(const FPackagePath& PackagePath, FPackagePath* OutPackagePath)
{
	return DoesPackageExist(PackagePath, false, OutPackagePath);
}

bool FPackageName::DoesPackageExist(const FPackagePath& PackagePath, bool bMatchCaseOnDisk, FPackagePath* OutPackagePath)
{
	return DoesPackageExistEx(PackagePath, EPackageLocationFilter::Any, bMatchCaseOnDisk, OutPackagePath) != EPackageLocationFilter::None;
}

FPackageName::EPackageLocationFilter FPackageName::DoesPackageExistEx(const FPackagePath& PackagePath, EPackageLocationFilter Filter, bool bMatchCaseOnDisk, FPackagePath* OutPackagePath)
{
	// DoesPackageExist returns false for local filenames that are in unmounted directories, even if those files exist on the local disk
	if (!PackagePath.IsMountedPath())
	{
		return EPackageLocationFilter::None;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FPackageName::DoesPackageExistEx);

#if WITH_EDITOR
	IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr();
	if (AssetRegistry)
	{
		FName CorrectCasePackageName;
		FAssetPackageData AssetPackageData;
		FName PackageName = PackagePath.GetPackageFName();
		UE::AssetRegistry::EExists ExistsStatus = AssetRegistry->TryGetAssetPackageData(PackageName, AssetPackageData, CorrectCasePackageName, true /*bFailIfLockHeld*/);
		if (ExistsStatus == UE::AssetRegistry::EExists::Exists)
		{
			EPackageLocationFilter PackageLocation = AssetPackageData.GetPackageLocation();

			// If the registry doesn't have this information at all, we can't trust it (e.g. reading older cache version)
			if (PackageLocation != EPackageLocationFilter::None)
			{
				PackageLocation = EPackageLocationFilter((uint8)Filter & (uint8)PackageLocation);
				if (!OutPackagePath)
				{
					return PackageLocation;
				}

				*OutPackagePath = FPackagePath::FromPackageNameUnchecked(MoveTemp(CorrectCasePackageName));
				if ((uint8)PackageLocation & (uint8)EPackageLocationFilter::FileSystem)
				{
					// If an OutPackagePath was provided but a custom or unspecified extension was used we must use the disk
					// scanning code futher down to allow the PackageResourceManager to handle the unspecified and custom extension cases
					if (AssetPackageData.Extension != EPackageExtension::Unspecified && AssetPackageData.Extension != EPackageExtension::Custom)
					{
						OutPackagePath->SetHeaderExtension(AssetPackageData.Extension);
						return PackageLocation;
					}
					// Intentionally do not return a EPackageLocationFilter if an OutPackagePath was provided but a custom or 
					// unspecified extension was used. In such cases we must use the disk scanning code futher down to allow the 
					// PackageResourceManager to handle the unspecified and custom extension cases
				}
				else
				{
					// IODispatcher packagepaths use the EmptyString HeaderExtension to indicate they are an IODispatcher path.
					// We do need to provide the EmptyString extension, rather than leaving it unspecified, to suppress wasted effort and
					// warnings from FPackagePath about missing extensions.
					// For IODispatcher paths we respect an existing header extension and don't overwrite it. Unknown whether that's
					// important; it has been that way for a while so some systems may be relying on it.
					if (OutPackagePath->GetHeaderExtension() == EPackageExtension::Unspecified)
					{
						OutPackagePath->SetHeaderExtension(EPackageExtension::EmptyString);
					}
					return PackageLocation;
				}
			}
		}
	}
#endif

	return InternalDoesPackageExistEx(PackagePath, Filter, bMatchCaseOnDisk, OutPackagePath);
}

bool FPackageName::TryConvertToMountedPackagePath(const FString& InPath, FPackagePath& OutPackagePath, EErrorCode& OutFailureReason)
{
	SCOPED_LOADTIMER(TryConvertToPackagePath);
	TStringBuilder<64> PackageNameRoot;
	TStringBuilder<64> FilePathRoot;
	TStringBuilder<256> RelPath;
	TStringBuilder<64> UnusedObjectName; // DoesPackageExist accepts ObjectPaths and ignores the ObjectName portion and uses only the PackageName
	TStringBuilder<16> CustomExtension;
	EPackageExtension Extension;
	if (!FPackageName::TryConvertToMountedPathComponents(InPath, PackageNameRoot, FilePathRoot, RelPath, UnusedObjectName, Extension, CustomExtension, nullptr /* OutFlexNameType */, &OutFailureReason))
	{
		return false;
	}
	OutPackagePath = FPackagePath::FromMountedComponents(PackageNameRoot, FilePathRoot, RelPath, Extension, CustomExtension);
	return true;
}

FPackageName::EPackageLocationFilter FPackageName::InternalDoesPackageExistEx(const FString& LongPackageName, EPackageLocationFilter Filter, bool bMatchCaseOnDisk, FPackagePath* OutPackagePath)
{
	FPackagePath PackagePath;
	EErrorCode FailureReason;
	if (!TryConvertToMountedPackagePath(LongPackageName, PackagePath, FailureReason))
	{
		FString Message = FString::Printf(TEXT("DoesPackageExist called on PackageName that will always return false. Reason: %s"), *FormatErrorAsString(LongPackageName, FailureReason));
		UE_LOGF(LogPackageName, Warning, "%ls", *Message);
		return EPackageLocationFilter::None;
	}

	return InternalDoesPackageExistEx(PackagePath, Filter, bMatchCaseOnDisk, OutPackagePath);
}

FPackageName::EPackageLocationFilter FPackageName::InternalDoesPackageExistEx(const FPackagePath& PackagePath, EPackageLocationFilter Filter, bool bMatchCaseOnDisk, FPackagePath* OutPackagePath)
{
	TStringBuilder<256> PackageName;
	PackagePath.AppendPackageName(PackageName);

	// Once we have the real Package Name, we can exit early if it's a script package - they exist only in memory.
	if (IsScriptPackage(PackageName))
	{
		return EPackageLocationFilter::None;
	}

	if (IsMemoryPackage(PackageName))
	{
		return EPackageLocationFilter::None;
	}

	FText Reason;
	if (!FPackageName::IsValidTextForLongPackageName(PackageName, &Reason))
	{
		UE_LOGF(LogPackageName, Error, "DoesPackageExist: DoesPackageExist FAILED: '%ls' is not a long packagename name. Reason: %ls", *PackageName, *Reason.ToString());
		return EPackageLocationFilter::None;
	}

	EPackageLocationFilter Result = EPackageLocationFilter::None;

	if ((uint8)Filter & (uint8)EPackageLocationFilter::FileSystem)
	{
		bool bFoundInFileSystem = false;
		if (bMatchCaseOnDisk)
		{
			bFoundInFileSystem = IPackageResourceManager::Get().TryMatchCaseOnDisk(PackagePath, OutPackagePath);
		}
		else
		{
			bFoundInFileSystem = IPackageResourceManager::Get().DoesPackageExist(PackagePath, OutPackagePath);
		}

		if (bFoundInFileSystem)
		{
			Result = EPackageLocationFilter((uint8)Result | (uint8)EPackageLocationFilter::FileSystem);

			// if we just want to find any existence, then we are done
			if (Filter == EPackageLocationFilter::Any)
			{
				return Result;
			}
		}
	}

	if (((uint8)Filter & (uint8)EPackageLocationFilter::IoDispatcher))
	{
		bool bFoundInIoDispatcher = false;
		if (DoesPackageExistOverrideDelegate.IsBound())
		{
			if (DoesPackageExistOverrideDelegate.Execute(PackagePath.GetPackageFName()))
			{
				bFoundInIoDispatcher = true;
			}
		}
		else if (FIoDispatcher::IsInitialized())
		{
			bFoundInIoDispatcher = FIoDispatcher::Get().DoesChunkExist(CreatePackageDataChunkId(FPackageId::FromName(PackagePath.GetPackageFName())));
		}

		if (bFoundInIoDispatcher)
		{
			if (OutPackagePath)
			{
				*OutPackagePath = PackagePath;
				// IODispatcher packagepaths use the EmptyString HeaderExtension to indicate they are an IODispatcher path.
				// We do need to provide the EmptyString extension, rather than leaving it unspecified, to suppress wasted effort and
				// warnings from FPackagePath about missing extensions.
				// For IODispatcher paths we respect an existing header extension and don't overwrite it. Unknown whether that's
				// important; it has been that way for a while so some systems may be relying on it.
				if (OutPackagePath->GetHeaderExtension() == EPackageExtension::Unspecified)
				{
					OutPackagePath->SetHeaderExtension(EPackageExtension::EmptyString);
				}
			}

			Result = EPackageLocationFilter((uint8)Result | (uint8)EPackageLocationFilter::IoDispatcher);

			// if we just want to find any existence, then we are done
			if (Filter == EPackageLocationFilter::Any)
			{
				return Result;
			}
		}
	}

	return Result;
}

bool FPackageName::SearchForPackageOnDisk(const FString& PackageName, FString* OutLongPackageName, FString* OutFilename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FPackageName::SearchForPackageOnDisk);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FPackageName::SearchForPackageOnDisk"), STAT_PackageName_SearchForPackageOnDisk, STATGROUP_LoadTime);

	// This function may take a long time to complete, so suspend heartbeat measure while we're here
	FSlowHeartBeatScope SlowHeartBeatScope;

	bool bResult = false;
	double StartTime = FPlatformTime::Seconds();
	if (FPackageName::IsShortPackageName(PackageName) == false)
	{
		// If this is long package name, revert to using DoesPackageExist because it's a lot faster.
		FString Filename;
		if (DoesPackageExist(PackageName, &Filename))
		{
			if (OutLongPackageName)
			{
				*OutLongPackageName = PackageName;
			}
			if (OutFilename)
			{
				*OutFilename = Filename;
			}
			bResult = true;
		}
	}
	else
	{
		// Attempt to find package by its short name by searching in the known content paths.
		TArray<TPair<FString,FString>> RootsNameAndFile;
		{
			TArray<FString> RootContentPaths;
			FPackageName::QueryRootContentPaths( RootContentPaths );
			for( TArray<FString>::TConstIterator RootPathIt( RootContentPaths ); RootPathIt; ++RootPathIt )
			{
				const FString& RootPackageName = *RootPathIt;
				const FString& RootFilePath = FPackageName::LongPackageNameToFilename(RootPackageName, TEXT(""));
				RootsNameAndFile.Add(TPair<FString,FString>(RootPackageName, RootFilePath));
			}
		}

		int32 ExtensionStart;
		EPackageExtension RequiredExtension = FPackagePath::ParseExtension(PackageName, &ExtensionStart);
		if (RequiredExtension == EPackageExtension::Custom || ExtensionToSegment(RequiredExtension) != EPackageSegment::Header)
		{
			UE_LOGF(LogPackageName, Warning, "SearchForPackageOnDisk: Invalid extension in packagename %ls. Searching for any header extension instead.", *PackageName);
			RequiredExtension = EPackageExtension::Unspecified;
		}
		TStringBuilder<128> PackageWildCard;
		PackageWildCard << FStringView(PackageName).Left(ExtensionStart) << TEXT(".*");

		FPackagePath FirstResult;
		TArray<FPackagePath> FoundResults;
		IPackageResourceManager& PackageResourceManager = IPackageResourceManager::Get();
		for (const TPair<FString, FString>& RootNameAndFile : RootsNameAndFile)
		{
			const FString& RootPackageName = RootNameAndFile.Get<0>();
			const FString& RootFilePath = RootNameAndFile.Get<1>();
			check(RootPackageName.EndsWith(TEXT("/")));
			check(RootFilePath.EndsWith(TEXT("/")));
			// Search directly on disk. Very slow!
			FoundResults.Reset();
			PackageResourceManager.FindPackagesRecursive(FoundResults, RootPackageName, RootFilePath, FStringView(), PackageWildCard);

			for (const FPackagePath& FoundPackagePath: FoundResults)
			{			
				FStringView UnusedCustomExtension;
				if (RequiredExtension != EPackageExtension::Unspecified && FoundPackagePath.GetHeaderExtension() != RequiredExtension)
				{
					continue;
				}

				bResult = true;
				if (OutLongPackageName || OutFilename)
				{
					if (!FirstResult.IsEmpty())
					{
						UE_LOGF(LogPackageName, Warning, "SearchForPackageOnDisk: Found ambiguous long package name for '%ls'. Returning '%ls', but could also be '%ls'.", *PackageName,
							*FirstResult.GetDebugNameWithExtension(), *FoundPackagePath.GetDebugNameWithExtension());
					}
					else
					{
						if (OutLongPackageName)
						{
							*OutLongPackageName = FoundPackagePath.GetPackageName();
						}
						if (OutFilename)
						{
							*OutFilename = FoundPackagePath.GetLocalFullPath();
						}
						FirstResult = FoundPackagePath;
					}
				}
			}
			if (bResult)
			{
				break;
			}
		}
	}
	const double ThisTime = FPlatformTime::Seconds() - StartTime;

	if ( bResult )
	{
		UE_LOGF(LogPackageName, Log, "SearchForPackageOnDisk took %7.3fs to resolve %ls.", ThisTime, *PackageName);
	}
	else
	{
		UE_LOGF(LogPackageName, Log, "SearchForPackageOnDisk took %7.3fs, but failed to resolve %ls.", ThisTime, *PackageName);
	}

	return bResult;
}

bool FPackageName::TryConvertShortPackagePathToLongInObjectPath(const FString& ObjectPath, FString& ConvertedObjectPath)
{
	FString PackagePath;
	FString ObjectName;

	int32 DotPosition = ObjectPath.Find(TEXT("."), ESearchCase::CaseSensitive);
	if (DotPosition != INDEX_NONE)
	{
		PackagePath = ObjectPath.Mid(0, DotPosition);
		ObjectName = ObjectPath.Mid(DotPosition + 1);
	}
	else
	{
		PackagePath = ObjectPath;
	}

	FString LongPackagePath;
	if (!SearchForPackageOnDisk(PackagePath, &LongPackagePath))
	{
		return false;
	}

	ConvertedObjectPath = FString::Printf(TEXT("%s.%s"), *LongPackagePath, *ObjectName);
	return true;
}

FString FPackageName::GetNormalizedObjectPath(const FString& ObjectPath)
{
	if (!ObjectPath.IsEmpty() && FPackageName::IsShortPackageName(ObjectPath))
	{
		FString LongPath;

		UE_LOGF(LogPackageName, Warning, "Asset path \"%ls\" is in short form, which is unsupported and -- even if valid -- resolving it will be really slow.", *ObjectPath);
		UE_LOGF(LogPackageName, Warning, "Please consider resaving package in order to speed-up loading.");
		
		if (!FPackageName::TryConvertShortPackagePathToLongInObjectPath(ObjectPath, LongPath))
		{
			UE_LOGF(LogPackageName, Warning, "Asset path \"%ls\" could not be resolved.", *ObjectPath);
		}

		return LongPath;
	}
	else
	{
		return ObjectPath;
	}
}

FString FPackageName::GetDelegateResolvedPackagePath(const FString& InSourcePackagePath)
{
	if (FCoreDelegates::PackageNameResolvers.Num() > 0)
	{
		bool WasResolved = false;

		// If the path is /Game/Path/Foo.Foo only worry about resolving the /Game/Path/Foo
		FString PathName = InSourcePackagePath;
		FString ObjectName;
		int32 DotIndex = INDEX_NONE;

		if (PathName.FindChar(TEXT('.'), DotIndex))
		{
			ObjectName = PathName.Mid(DotIndex + 1);
			PathName.LeftInline(DotIndex, EAllowShrinking::No);
		}

		for (auto Delegate : FCoreDelegates::PackageNameResolvers)
		{
			FString ResolvedPath;
			if (Delegate.Execute(PathName, ResolvedPath))
			{
				UE_LOGF(LogPackageName, Display, "Package '%ls' was resolved to '%ls'", *PathName, *ResolvedPath);
				PathName = ResolvedPath;
				WasResolved = true;
			}
		}

		if (WasResolved)
		{
			// If package was passed in with an object, add that back on by deriving it from the package name
			if (ObjectName.Len())
			{
				int32 LastSlashIndex = INDEX_NONE;
				if (PathName.FindLastChar(TEXT('/'), LastSlashIndex))
				{
					ObjectName = PathName.Mid(LastSlashIndex + 1);
				}

				PathName += TEXT(".");
				PathName += ObjectName;
			}

			return PathName;
		}
	}

	return InSourcePackagePath;
}

FString FPackageName::GetSourcePackagePath(const FString& InLocalizedPackagePath)
{
	// This function finds the start and end point of the "/L10N/<culture>" part of the path so that it can be removed
	auto GetL10NTrimRange = [](const FString& InPath, int32& OutL10NStart, int32& OutL10NLength)
	{
		const TCHAR* CurChar = *InPath;

		// Must start with a slash
		if (*CurChar++ != TEXT('/'))
		{
			return false;
		}

		// Find the end of the first part of the path, eg /Game/
		while (*CurChar && *CurChar++ != TEXT('/')) {}
		if (!*CurChar)
		{
			// Found end-of-string
			return false;
		}

		if (FCString::Strnicmp(CurChar, TEXT("L10N/"), 5) == 0) // StartsWith "L10N/"
		{
			CurChar -= 1; // -1 because we need to eat the slash before L10N
			OutL10NStart = UE_PTRDIFF_TO_INT32(CurChar - *InPath);
			OutL10NLength = 6; // "/L10N/"

			// Walk to the next slash as that will be the end of the culture code
			CurChar += OutL10NLength;
			while (*CurChar && *CurChar++ != TEXT('/')) { ++OutL10NLength; }

			return true;
		}
		else if (FCString::Stricmp(CurChar, TEXT("L10N")) == 0) // Is "L10N"
		{
			CurChar -= 1; // -1 because we need to eat the slash before L10N
			OutL10NStart = UE_PTRDIFF_TO_INT32(CurChar - *InPath);
			OutL10NLength = 5; // "/L10N"

			return true;
		}

		return false;
	};

	FString SourcePackagePath = InLocalizedPackagePath;

	int32 L10NStart = INDEX_NONE;
	int32 L10NLength = 0;
	if (GetL10NTrimRange(SourcePackagePath, L10NStart, L10NLength))
	{
		SourcePackagePath.RemoveAt(L10NStart, L10NLength);
	}

	return SourcePackagePath;
}

FString FPackageName::GetLocalizedPackagePath(const FString& InSourcePackagePath)
{
	const FName LocalizedPackageName = FPackageLocalizationManager::Get().FindLocalizedPackageName(*InSourcePackagePath);
	return (LocalizedPackageName.IsNone()) ? InSourcePackagePath : LocalizedPackageName.ToString();
}

FString FPackageName::GetLocalizedPackagePath(const FString& InSourcePackagePath, const FString& InCultureName)
{
	const FName LocalizedPackageName = FPackageLocalizationManager::Get().FindLocalizedPackageNameForCulture(*InSourcePackagePath, InCultureName);
	return (LocalizedPackageName.IsNone()) ? InSourcePackagePath : LocalizedPackageName.ToString();
}

UE::Core::FVersePath FPackageName::GetVersePath(const FTopLevelAssetPath& AssetPath)
{
	// Ignore empty paths, or paths that specify a package rather than an object within the package
	if (!AssetPath.IsValid() || AssetPath.GetAssetName().IsNone())
	{
		return {};
	}

	TStringBuilder<128> PackageNameBuilder;
	AssetPath.GetPackageName().ToString(PackageNameBuilder);
	const FStringView PackageName = PackageNameBuilder.ToView();
	
	// If the mount point is invalid, we can't create a vpath from it
	bool bHadClassesPrefix = false;
	const FStringView MountPointName = FPathViews::GetMountPointNameFromPath(PackageName, &bHadClassesPrefix);
	if (MountPointName.IsEmpty() || bHadClassesPrefix)
	{
		return {};
	}

	// If the object isn't mounted under a plugin, the object doesn't have a vpath
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(MountPointName);
	if (!Plugin)
	{
		return {};
	}

	// If the plugin doesn't have a root vpath, the object doesn't have a vpath
	const FString& PluginVersePath = Plugin->GetVersePath();
	if (PluginVersePath.IsEmpty())
	{
		return {};
	}

	FStringView RelativePath = PackageName.RightChop(MountPointName.Len() + 1);

	// Remove the leaf name
	int32 IndexOfLastSlash = INDEX_NONE;
	if (RelativePath.FindLastChar(TEXT('/'), IndexOfLastSlash))
	{
		if (!ensureMsgf(IndexOfLastSlash != RelativePath.Len() - 1, TEXT("Top level asset path '%s' missing leaf name."), *PackageNameBuilder))
		{
			return {};
		}

		RelativePath.LeftInline(IndexOfLastSlash);
	}
	if (!ensureMsgf(IndexOfLastSlash != INDEX_NONE, TEXT("Top level asset path '%s' should have at least two forward slashes."), *PackageNameBuilder))
	{
		return {};
	}

	TStringBuilder<128> AssetNameBuilder;
	AssetPath.GetAssetName().ToString(AssetNameBuilder);

	FString VerseModule = FPaths::Combine(PluginVersePath, RelativePath, AssetNameBuilder);

	UE::Core::FVersePath Result;
#if NO_LOGGING
	UE::Core::FVersePath::TryMake(Result, MoveTemp(VerseModule));
#else
	if (!UE::Core::FVersePath::TryMake(Result, VerseModule))
	{
		static thread_local TSet<FString> AlreadyLogged;

		bool bAlreadyInSet = false;
		AlreadyLogged.Add(VerseModule, &bAlreadyInSet);
		if (!bAlreadyInSet)
		{
			UE_LOGF(LogCore, Display, "Unable to make a VersePath for object '%ls' with path '%ls'", *AssetPath.ToString(), *VerseModule);
		}
	}
#endif

	return Result;
}

UE::Core::FVersePath FPackageName::LongPackagePathToVersePath(FStringView LongPackagePath)
{
	// If the mount point is invalid, we can't create a Verse path from it.
	bool bHadClassesPrefix = false;
	const FStringView MountPointName = FPathViews::GetMountPointNameFromPath(LongPackagePath, &bHadClassesPrefix);
	if (MountPointName.IsEmpty() || bHadClassesPrefix)
	{
		return {};
	}

	// If the package path isn't mounted under a plugin, the package path doesn't have a Verse path.
	TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(MountPointName);
	if (!Plugin)
	{
		return {};
	}

	// If the plugin doesn't have a root vpath, the object doesn't have a Verse path.
	const FString& PluginVersePath = Plugin->GetVersePath();
	if (PluginVersePath.IsEmpty())
	{
		return {};
	}

	// Don't FPaths::Combine if getting the Verse path for the root plugin path, FPaths::Combine will add a trailing '/' making the Verse path invalid.
	const int32 CharCount = MountPointName.Len() + 1;
	UE::Core::FVersePath Result;
	UE::Core::FVersePath::TryMake(Result, LongPackagePath.Len() == CharCount ? PluginVersePath : FPaths::Combine(PluginVersePath, LongPackagePath.RightChop(CharCount)));
	return Result;
}

const FString& FPackageName::GetAssetPackageExtension()
{
	static FString AssetPackageExtension(LexToString(EPackageExtension::Asset));
	return AssetPackageExtension;
}

const FString& FPackageName::GetMapPackageExtension()
{
	static FString MapPackageExtension(LexToString(EPackageExtension::Map));
	return MapPackageExtension;
}

const FString& FPackageName::GetTextAssetPackageExtension()
{
	static FString TextAssetPackageExtension(LexToString(EPackageExtension::TextAsset));
	return TextAssetPackageExtension;
}

const FString& FPackageName::GetTextMapPackageExtension()
{
	static FString TextMapPackageExtension(LexToString(EPackageExtension::TextMap));
	return TextMapPackageExtension;
}

bool FPackageName::IsTextPackageExtension(const TCHAR* Ext)
{
	return IsTextAssetPackageExtension(Ext) || IsTextMapPackageExtension(Ext);
}

bool FPackageName::IsTextPackageExtension(EPackageExtension Extension)
{
	return Extension == EPackageExtension::TextAsset || Extension == EPackageExtension::TextMap;
}

bool FPackageName::IsTextAssetPackageExtension(const TCHAR* Ext)
{
	FStringView TextAssetPackageExtension(LexToString(EPackageExtension::TextAsset));
	if (*Ext != TEXT('.') && *Ext != TEXT('\0'))
	{
		return (TextAssetPackageExtension.RightChop(1) == Ext);
	}
	else
	{
		return (TextAssetPackageExtension == Ext);
	}
}

bool FPackageName::IsTextMapPackageExtension(const TCHAR* Ext)
{
	FStringView TextMapPackageExtension(LexToString(EPackageExtension::TextMap));
	if (*Ext != TEXT('.') && *Ext != TEXT('\0'))
	{
		return (TextMapPackageExtension.RightChop(1) == Ext);
	}
	else
	{
		return (TextMapPackageExtension == Ext);
	}
}

bool FPackageName::IsPackageExtension( const TCHAR* Ext )
{
	return IsAssetPackageExtension(Ext) || IsMapPackageExtension(Ext);
}

bool FPackageName::IsPackageExtension(EPackageExtension Extension)
{
	return Extension == EPackageExtension::Asset || Extension == EPackageExtension::Map;
}

bool FPackageName::IsAssetPackageExtension(const TCHAR* Ext)
{
	FStringView AssetPackageExtension(LexToString(EPackageExtension::Asset));
	if (*Ext != TEXT('.'))
	{
		return (AssetPackageExtension.RightChop(1) == Ext);
	}
	else
	{
		return (AssetPackageExtension == Ext);
	}
}

bool FPackageName::IsMapPackageExtension(const TCHAR* Ext)
{
	FStringView MapPackageExtension(LexToString(EPackageExtension::Map));
	if (*Ext != TEXT('.'))
	{
		return (MapPackageExtension.RightChop(1) == Ext);
	}
	else
	{
		return (MapPackageExtension == Ext);
	}
}

bool FPackageName::IsVerseExtension(const TCHAR* Ext)
{
	FStringView VerseExtension = TEXTVIEW(".verse");
	FStringView VModuleExtension = TEXTVIEW(".vmodule");
	if (*Ext != TEXT('.'))
	{
		return (VerseExtension.RightChop(1) == Ext) || (VModuleExtension.RightChop(1) == Ext);
	}
	else
	{
		return (VerseExtension == Ext) || (VModuleExtension == Ext);
	}
}

const TCHAR* FPackageName::GetGeneratedPackageSubPath()
{
	return TEXT("_Generated_");
}

bool FPackageName::IsUnderGeneratedPackageSubPath(FStringView FileOrLongPackagePath)
{
	FStringView GeneratedSubDir(GetGeneratedPackageSubPath());
	int32 Index = FileOrLongPackagePath.Find(GeneratedSubDir, ESearchCase::IgnoreCase);
	if (Index < 0)
	{
		return false;
	}
	if (Index == 0 || !FPathViews::IsSeparator(FileOrLongPackagePath[Index - 1]))
	{
		// "_Generated_/..." or ".../Prefix_Generated_/...", not the /_Generated_/ subfolder we're looking for.
		return false;
	}
	if (Index + GeneratedSubDir.Len() < FileOrLongPackagePath.Len()
		&& !FPathViews::IsSeparator(FileOrLongPackagePath[Index + GeneratedSubDir.Len()]))
	{
		// ".../_Generated_Suffix/...", not the /_Generated_/ subfolder we're looking for.
		return false;
	}
	return true;
}


bool FPackageName::FindPackagesInDirectory( TArray<FString>& OutPackages, const FString& RootDir )
{
	// Keep track if any package has been found. Can't rely only on OutPackages.Num() > 0 as it may not be empty.
	const int32 PreviousPackagesCount = OutPackages.Num();
	IteratePackagesInDirectory(RootDir, [&OutPackages](const TCHAR* PackageFilename) -> bool
	{
		OutPackages.Add(PackageFilename);
		return true;
	});
	return OutPackages.Num() > PreviousPackagesCount;
}

bool FPackageName::FindPackagesInDirectories(TArray<FString>& OutPackages, const TArrayView<const FString>& RootDirs)
{
	TSet<FString> Packages;
	TArray<FString> DirPackages;
	for (const FString& RootDir : RootDirs)
	{
		DirPackages.Reset();
		FindPackagesInDirectory(DirPackages, RootDir);
		for (FString& DirPackage : DirPackages)
		{
			Packages.Add(MoveTemp(DirPackage));
		}
	}
	OutPackages.Reserve(Packages.Num() + OutPackages.Num());
	for (FString& Package : Packages)
	{
		OutPackages.Add(MoveTemp(Package));
	}
	return Packages.Num() > 0;
}

void FPackageName::IteratePackagesInDirectory(const FString& RootDir, const FPackageNameVisitor& Callback)
{
	auto LocalCallback = [&Callback](const FPackagePath& PackagePath) -> bool
	{
		return Callback(*PackagePath.GetLocalFullPath());
	};

	TStringBuilder<256> PackageNameRoot;
	TStringBuilder<256> FilePathRoot;
	TStringBuilder<256> RelRootDir;
	if (TryGetMountPointForPath(RootDir, PackageNameRoot, FilePathRoot, RelRootDir))
	{
		IPackageResourceManager::Get().IteratePackagesInPath(PackageNameRoot, FilePathRoot, RelRootDir, LocalCallback);
	}
	else
	{
		// Searching a localonly path
		IPackageResourceManager::Get().IteratePackagesInLocalOnlyDirectory(RootDir, LocalCallback);
	}
}

void FPackageName::IteratePackagesInDirectory(const FString& RootDir, const FPackageNameStatVisitor& Callback)
{
	auto LocalCallback = [&Callback](const FPackagePath& PackagePath, const FFileStatData& StatData) -> bool
	{
		return Callback(*PackagePath.GetLocalFullPath(), StatData);
	};

	TStringBuilder<256> PackageNameRoot;
	TStringBuilder<256> FilePathRoot;
	TStringBuilder<256> RelRootDir;
	if (TryGetMountPointForPath(RootDir, PackageNameRoot, FilePathRoot, RelRootDir))
	{
		IPackageResourceManager::Get().IteratePackagesStatInPath(PackageNameRoot, FilePathRoot, RelRootDir, LocalCallback);
	}
	else
	{
		// Searching a localonly path
		IPackageResourceManager::Get().IteratePackagesStatInLocalOnlyDirectory(RootDir, LocalCallback);
	}
}

void FPackageName::QueryRootContentPaths(TArray<FString>& OutRootContentPaths, bool bIncludeReadOnlyRoots, bool bWithoutLeadingSlashes, bool bWithoutTrailingSlashes)
{
	using namespace UE::PackageName;

	const FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();

	{
		TArray<TRefCountPtr<FMountPoint>> ReportMountPoints;
		{
			UE::TReadScopeLock ScopeLock(Paths.MountLock);
			ReportMountPoints.Reserve(Paths.MountPoints.Num());
			for (const FMountPointKeyByPtr& MountPointKey : Paths.MountPoints)
			{
				FMountPoint* MountPoint = MountPointKey.GetMountPoint();
				if (MountPoint->IsMounted())
				{
					ReportMountPoints.Emplace(MountPoint);
				}
			}
		}
		// Sort ReportMountPoints from highest priority to lowest priority. 
		Algo::Sort(ReportMountPoints, [](const TRefCountPtr<FMountPoint>& A, const TRefCountPtr<FMountPoint>& B)
			{
				return A->GetMountOrder() > B->GetMountOrder();
			});

		OutRootContentPaths.Reserve(ReportMountPoints.Num() + (bIncludeReadOnlyRoots ? 1 : 0));
		// Maintain the legacy order: non-removable non-readonly, then removable, then non-removable readonly
		for (TRefCountPtr<FMountPoint>& MountPoint : ReportMountPoints)
		{
			if (!MountPoint->IsRemovable() && !MountPoint->IsReadOnly() && !MountPoint->IsAlias())
			{
				OutRootContentPaths.Add(MountPoint->GetRootPath());
			}
		}
		for (TRefCountPtr<FMountPoint>& MountPoint : ReportMountPoints)
		{
			if (MountPoint->IsRemovable())
			{
				OutRootContentPaths.Add(MountPoint->GetRootPath());
			}
		}
		if (bIncludeReadOnlyRoots)
		{
			for (TRefCountPtr<FMountPoint>& MountPoint : ReportMountPoints)
			{
				if (!MountPoint->IsRemovable() && MountPoint->IsReadOnly() && !MountPoint->IsAlias())
				{
					OutRootContentPaths.Add(MountPoint->GetRootPath());
				}
			}
			const FString& ExtraRoot = Paths.MemoryRootPath;
			OutRootContentPaths.Add(ExtraRoot);
		}
	}

	if (bWithoutTrailingSlashes || bWithoutLeadingSlashes)
	{
		for (FString& It : OutRootContentPaths)
		{
			if (bWithoutTrailingSlashes && It.Len() > 1 && It[It.Len() - 1] == TEXT('/'))
			{
				It.RemoveAt(It.Len() - 1, /*Count*/ 1, EAllowShrinking::No);
			}

			if (bWithoutLeadingSlashes && It.Len() > 1 && It[0] == TEXT('/'))
			{
				It.RemoveAt(0, /*Count*/ 1, EAllowShrinking::No);
			}
		}
	}
}

TArray<FString> FPackageName::QueryMountPointLocalAbsPaths()
{
	using namespace UE::PackageName;

	const FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();
	TArray<FString> OutAbsPaths;

	TArray<TRefCountPtr<FMountPoint>> ReportMountPoints;
	{
		UE::TReadScopeLock ScopeLock(Paths.MountLock);
		ReportMountPoints.Reserve(Paths.MountPoints.Num());
		for (const FMountPointKeyByPtr& MountPointKey : Paths.MountPoints)
		{
			FMountPoint* MountPoint = MountPointKey.GetMountPoint();
			if (MountPoint->IsMounted())
			{
				ReportMountPoints.Emplace(MountPoint);
			}
		}
	}
	// Sort ReportMountPoints from highest priority to lowest priority.
	Algo::Sort(ReportMountPoints, [](const TRefCountPtr<FMountPoint>& A, const TRefCountPtr<FMountPoint>& B)
		{
			return A->GetMountOrder() > B->GetMountOrder();
		});

	OutAbsPaths.Reserve(ReportMountPoints.Num());
	for (TRefCountPtr<FMountPoint>& MountPoint : ReportMountPoints)
	{
		if (!MountPoint->IsAlias() && !MountPoint->GetContentPathAbsolute().IsEmpty())
		{
			OutAbsPaths.Add(MountPoint->GetContentPathAbsolute());
		}
	}
	return OutAbsPaths;
}

void FPackageName::OnCoreUObjectInitialized()
{
	FLongPackagePathsSingleton::Get().OnCoreUObjectInitialized();
}

template<class T>
bool ParseExportTextPathImpl(const T& InExportTextPath, T* OutClassName, T* OutObjectPath)
{
	int32 Index;
	if (InExportTextPath.FindChar('\'', Index) && InExportTextPath.IsValidIndex(Index + 1) && InExportTextPath[InExportTextPath.Len() - 1] == '\'') // IsValidIndex checks that the found ' isn't the same one that's at the end of the string
	{
		if (OutClassName)
		{
			*OutClassName = InExportTextPath.Left(Index);
		}

		if (OutObjectPath)
		{
			*OutObjectPath = InExportTextPath.Mid(Index + 1, InExportTextPath.Len() - Index - 2); // -2 because we're stripping the first and last '
		}

		return true;
	}
	
	return false;
}

bool FPackageName::ParseExportTextPath(const FString& InExportTextPath, FString* OutClassName, FString* OutObjectPath)
{
	return ParseExportTextPathImpl(InExportTextPath, OutClassName, OutObjectPath);
}

bool FPackageName::ParseExportTextPath(FWideStringView InExportTextPath, FWideStringView* OutClassName, FWideStringView* OutObjectPath)
{
	return ParseExportTextPathImpl(InExportTextPath, OutClassName, OutObjectPath);
}

bool FPackageName::ParseExportTextPath(FAnsiStringView InExportTextPath, FAnsiStringView* OutClassName, FAnsiStringView* OutObjectPath)
{
	return ParseExportTextPathImpl(InExportTextPath, OutClassName, OutObjectPath);
}

bool FPackageName::ParseExportTextPath(const TCHAR* InExportTextPath, FStringView* OutClassName, FStringView* OutObjectPath)
{
	return ParseExportTextPath(FStringView(InExportTextPath), OutClassName, OutObjectPath);
}

template <class T>
T ExportTextPathToObjectPathImpl(const T& InExportTextPath)
{
	T ObjectPath;
	if (FPackageName::ParseExportTextPath(InExportTextPath, nullptr, &ObjectPath))
	{
		return ObjectPath;
	}
	
	// Could not parse the export text path. Could already be an object path, just return it back.
	return InExportTextPath;
}

FWideStringView FPackageName::ExportTextPathToObjectPath(FWideStringView InExportTextPath)
{
	return ExportTextPathToObjectPathImpl(InExportTextPath);
}

FAnsiStringView FPackageName::ExportTextPathToObjectPath(FAnsiStringView InExportTextPath)
{
	return ExportTextPathToObjectPathImpl(InExportTextPath);
}

FString FPackageName::ExportTextPathToObjectPath(const FString& InExportTextPath)
{
	return ExportTextPathToObjectPathImpl(InExportTextPath);
}

FString FPackageName::ExportTextPathToObjectPath(const TCHAR* InExportTextPath)
{
	return ExportTextPathToObjectPath(FString(InExportTextPath));
}

void FPackageName::SplitPackageName(FStringView InPackageName, FStringView* OutPackageRoot,
	FStringView* OutPackagePath, FStringView* OutPackageLeafName, EPathFormatFlags Flags)
{
	auto SetOutput = [OutPackageRoot, OutPackagePath, OutPackageLeafName]
	(FStringView InPackageRoot, FStringView InPackagePath, FStringView InPackageLeafName)
		{
			if (OutPackageRoot)
			{
				*OutPackageRoot = InPackageRoot;
			}
			if (OutPackagePath)
			{
				*OutPackagePath = InPackagePath;
			}
			if (OutPackageLeafName)
			{
				*OutPackageLeafName = InPackageLeafName;
			}
		};

	if (!InPackageName.StartsWith(TEXT("/")))
	{
		SetOutput(FStringView(), FStringView(), FStringView());
		return;
	}

	int32 SecondSlashIndex;
	if (!InPackageName.RightChop(1).FindChar('/', SecondSlashIndex))
	{
		SetOutput(FStringView(), FStringView(), FStringView());
		return;
	}
	check(SecondSlashIndex != INDEX_NONE);
	// Change back to an index relative to the beginnning of InPackageName rather than relative to InPackageName[1].
	SecondSlashIndex += 1; 

	FStringView PackageRoot(InPackageName.Left(SecondSlashIndex + 1));
	FStringView RemainingPath = InPackageName.RightChop(SecondSlashIndex + 1);
	FStringView PackagePath;
	FStringView PackageNameLeaf;

	// Don't use FPathViews, because it has different rules for edge cases than we do, and adds some complexity
	// around Extension to convert between its rules and our rules.

	int32 LastSlashIndex;
	if (!RemainingPath.FindLastChar('/', LastSlashIndex))
	{
		// "/root/leaf"
		PackageNameLeaf = RemainingPath;
	}
	else
	{
		if (LastSlashIndex == RemainingPath.Len() - 1)
		{
			// "/root/path/"
			PackagePath = RemainingPath;
		}
		else
		{
			// "/root/path/leaf"
			PackagePath = RemainingPath.Left(LastSlashIndex + 1);
			PackageNameLeaf = RemainingPath.RightChop(LastSlashIndex + 1);
		}
	}

	// Convert to the requested output format
	if (!EnumHasAnyFlags(Flags, EPathFormatFlags::MountPointLeadingSlash))
	{
		PackageRoot.RightChopInline(1);
	}
	if (!EnumHasAnyFlags(Flags, EPathFormatFlags::MountPointTrailingSlash))
	{
		PackageRoot.LeftChopInline(1);
	}
	SetOutput(PackageRoot, PackagePath, PackageNameLeaf);
}

void FPackageName::SplitPackageName(const FString& InLongPackageName, FString* OutPackageRoot,
	FString* OutPackagePath, FString* OutPackageLeafName, EPathFormatFlags Flags)
{
	FStringView PackageRoot;
	FStringView PackagePath;
	FStringView PackageLeafName;
	SplitPackageName(InLongPackageName, &PackageRoot, &PackagePath, &PackageLeafName, Flags);
	if (OutPackageRoot)
	{
		*OutPackageRoot = PackageRoot;
	}
	if (OutPackagePath)
	{
		*OutPackagePath = PackagePath;
	}
	if (OutPackageLeafName)
	{
		*OutPackageLeafName = PackageLeafName;
	}
}

bool FPackageName::SplitLongPackageName(const FString& InLongPackageName, FString& OutPackageRoot,
	FString& OutPackagePath, FString& OutPackageName, const bool bStripRootLeadingSlash)
{
	if (!IsValidPath(InLongPackageName))
	{
		OutPackageRoot.Reset();
		OutPackagePath.Reset();
		OutPackageName.Reset();
		return false;
	}

	SplitPackageName(InLongPackageName, &OutPackageRoot, &OutPackagePath, &OutPackageName,
		bStripRootLeadingSlash ? EPathFormatFlags::MountPointTrailingSlash : EPathFormatFlags::MountPointSlashes);
	return true;
}

FString FPackageName::SplitPackageNameRoot(FName InPackageName, FString* OutRelativePath, EPathFormatFlags Flags)
{
	TStringBuilder<FName::StringBufferSize> PackageNameStr(InPlace, InPackageName);
	FStringView PackageName(PackageNameStr);

	FStringView RelativePath;
	FStringView Root = SplitPackageNameRoot(PackageName, &RelativePath, Flags);

	if (OutRelativePath)
	{
		*OutRelativePath = FString(RelativePath);
	}
	return FString(Root);
}

FStringView FPackageName::SplitPackageNameRoot(FStringView InPackageName, FStringView* OutRelativePath,
	EPathFormatFlags Flags)
{
	if (!InPackageName.StartsWith(TEXT("/")))
	{
		if (OutRelativePath)
		{
			*OutRelativePath = InPackageName;
		}
		return FStringView();
	}

	int32 SecondSlashIndex;
	if (!InPackageName.RightChop(1).FindChar('/', SecondSlashIndex))
	{
		if (OutRelativePath)
		{
			*OutRelativePath = FStringView();
		}
		FStringView PackageRoot = InPackageName;
		// Convert to the requested output format
		if (!EnumHasAnyFlags(Flags, EPathFormatFlags::MountPointLeadingSlash))
		{
			PackageRoot.RightChopInline(1);
		}

		return PackageRoot;
	}
	check(SecondSlashIndex != INDEX_NONE);
	// Change back to an index relative to the beginnning of InPackageName rather than relative to InPackageName[1].
	SecondSlashIndex += 1;

	if (OutRelativePath)
	{
		*OutRelativePath = InPackageName.RightChop(SecondSlashIndex + 1);
	}

	FStringView PackageRoot = InPackageName.Left(SecondSlashIndex + 1);
	// Convert to the requested output format
	if (!EnumHasAnyFlags(Flags, EPathFormatFlags::MountPointLeadingSlash))
	{
		PackageRoot.RightChopInline(1);
	}
	if (!EnumHasAnyFlags(Flags, EPathFormatFlags::MountPointTrailingSlash))
	{
		PackageRoot.LeftChopInline(1);
	}
	return PackageRoot;
};


template<class T>
T ObjectPathToPackageNameImpl(const T& InObjectPath)
{
	// Check for package delimiter
	int32 ObjectDelimiterIdx;
	if (InObjectPath.FindChar('.', ObjectDelimiterIdx))
	{
		return InObjectPath.Mid(0, ObjectDelimiterIdx);
	}

	// No object delimiter. The path must refer to the package name directly.
	return InObjectPath;
}

FWideStringView FPackageName::ObjectPathToPackageName(FWideStringView InObjectPath)
{
	return ObjectPathToPackageNameImpl(InObjectPath);
}

FAnsiStringView FPackageName::ObjectPathToPackageName(FAnsiStringView InObjectPath)
{
	return ObjectPathToPackageNameImpl(InObjectPath);
}

FString FPackageName::ObjectPathToPackageName(const FString& InObjectPath)
{
	return ObjectPathToPackageNameImpl(InObjectPath);
}

template<class T>
T ObjectPathToPathWithinPackageImpl(const T& InObjectPath)
{
	// Check for package delimiter
	int32 ObjectDelimiterIdx;
	if (InObjectPath.FindChar('.', ObjectDelimiterIdx))
	{
		return InObjectPath.Mid(ObjectDelimiterIdx + 1);
	}

	// No object delimiter. The path must refer to the package name directly.
	return T();
}

FWideStringView FPackageName::ObjectPathToPathWithinPackage(FWideStringView InObjectPath)
{
	return ObjectPathToPathWithinPackageImpl(InObjectPath);
}

FAnsiStringView FPackageName::ObjectPathToPathWithinPackage(FAnsiStringView InObjectPath)
{
	return ObjectPathToPathWithinPackageImpl(InObjectPath);
}

FString FPackageName::ObjectPathToPathWithinPackage(const FString& InObjectPath)
{
	return ObjectPathToPathWithinPackageImpl(InObjectPath);
}

template<class T>
T ObjectPathToOuterPathImpl(const T& InObjectPath)
{
	auto* LeafObjectDelimeterPtr = Algo::FindLastByPredicate(InObjectPath, [](auto Ch)
	{
		return Ch == ':' || Ch == '.';
	});

	if (LeafObjectDelimeterPtr)
	{
		int32 LeafObjectDelimeterIdx = UE_PTRDIFF_TO_INT32(LeafObjectDelimeterPtr - GetData(InObjectPath));
		return InObjectPath.Left(LeafObjectDelimeterIdx);
	}

	// No object or subobject delimiters. The path must refer to the object name directly (i.e. a package).
	return T();
}

FString FPackageName::ObjectPathToOuterPath(const FString& InObjectPath)
{
	return ObjectPathToOuterPathImpl(InObjectPath);
}

FAnsiStringView FPackageName::ObjectPathToOuterPath(FAnsiStringView InObjectPath)
{
	return ObjectPathToOuterPathImpl(InObjectPath);
}

FWideStringView FPackageName::ObjectPathToOuterPath(FWideStringView InObjectPath)
{
	return ObjectPathToOuterPathImpl(InObjectPath);
}

template<class T>
T ObjectPathToSubObjectPathImpl(const T& InObjectPath)
{
	// Check for a subobject
	int32 SubObjectDelimiterIdx;
	if (InObjectPath.FindChar(':', SubObjectDelimiterIdx))
	{
		return InObjectPath.Mid(SubObjectDelimiterIdx + 1);
	}

	// Check for a top level object
	int32 ObjectDelimiterIdx;
	if (InObjectPath.FindChar('.', ObjectDelimiterIdx))
	{
		return InObjectPath.Mid(ObjectDelimiterIdx + 1);
	}

	// No object or subobject delimiters. The path must refer to the object name directly (i.e. a package).
	return InObjectPath;
}

FWideStringView FPackageName::ObjectPathToSubObjectPath(FWideStringView InObjectPath)
{
	return ObjectPathToSubObjectPathImpl(InObjectPath);
}

FAnsiStringView FPackageName::ObjectPathToSubObjectPath(FAnsiStringView InObjectPath)
{
	return ObjectPathToSubObjectPathImpl(InObjectPath);
}

FString FPackageName::ObjectPathToSubObjectPath(const FString& InObjectPath)
{
	return ObjectPathToSubObjectPathImpl(InObjectPath);
}

template<class T>
T ObjectPathToObjectNameImpl(const T& InObjectPath)
{
	auto* LeafObjectDelimeterPtr = Algo::FindLastByPredicate(InObjectPath, [](auto Ch)
	{
		return Ch == ':' || Ch == '.';
	});

	if (LeafObjectDelimeterPtr)
	{
		int32 LeafObjectDelimeterIdx = UE_PTRDIFF_TO_INT32(LeafObjectDelimeterPtr - GetData(InObjectPath));
		return InObjectPath.Mid(LeafObjectDelimeterIdx + 1);
	}

	// No object or subobject delimiters. The path must refer to the object name directly (i.e. a package).
	return InObjectPath;
}

FString FPackageName::ObjectPathToObjectName(const FString& InObjectPath)
{
	return ObjectPathToObjectNameImpl(InObjectPath);
}

FAnsiStringView FPackageName::ObjectPathToObjectName(FAnsiStringView InObjectPath)
{
	return ObjectPathToObjectNameImpl(InObjectPath);
}

FWideStringView FPackageName::ObjectPathToObjectName(FWideStringView InObjectPath)
{
	return ObjectPathToObjectNameImpl(InObjectPath);
}

template<class CharType>
static void ObjectPathSplitFirstNameImpl(TStringView<CharType> Text, TStringView<CharType>& OutFirst,
	TStringView<CharType>& OutRemainder)
{
	int32 DelimiterIndex = UE::String::FindFirstOfAnyChar(Text, { CharType(':'), CharType('.') });
	if (DelimiterIndex < 0)
	{
		OutFirst = Text;
		OutRemainder.Reset();
		return;
	}
	OutFirst = Text.Left(DelimiterIndex);
	OutRemainder = Text.RightChop(DelimiterIndex + 1);
}

void FPackageName::ObjectPathSplitFirstName(FWideStringView Text, FWideStringView& OutFirst,
	FWideStringView& OutRemainder)
{
	ObjectPathSplitFirstNameImpl(Text, OutFirst, OutRemainder);
}

void FPackageName::ObjectPathSplitFirstName(FAnsiStringView Text, FAnsiStringView& OutFirst, FAnsiStringView& OutRemainder)
{
	ObjectPathSplitFirstNameImpl(Text, OutFirst, OutRemainder);
}

void FPackageName::ObjectPathAppend(FStringBuilderBase& ObjectPath, FStringView NextName)
{
	if (ObjectPath.Len() == 0)
	{
		ObjectPath << NextName;
		return;
	}
	if (NextName.IsEmpty())
	{
		return;
	}
	if (NextName[0] == '/')
	{
		ObjectPath.Reset();
		ObjectPath << NextName;
		return;
	}

	int32 NumObjectDelimiters = 0;
	{
		int32 LastSlash;
		FStringView ObjectPathView(ObjectPath);
		ObjectPathView.FindLastChar('/', LastSlash);
		if (LastSlash == INDEX_NONE)
		{
			// Not a full object path. Always append with '.'
			NumObjectDelimiters = 2;
		}
		else
		{
			ObjectPathView.RightChopInline(LastSlash + 1);
			for (NumObjectDelimiters = 0;
				NumObjectDelimiters < 2; // Stop counting after the second delimiter since behavior no longer changes
				++NumObjectDelimiters)
			{
				int32 NextDelimiter = UE::String::FindFirstOfAnyChar(ObjectPathView, { TCHAR('.'), TCHAR(':') });
				if (NextDelimiter == INDEX_NONE)
				{
					break;
				}
				ObjectPathView.RightChopInline(NextDelimiter + 1);
			}
		}
	}

	UE::String::ParseTokensMultiple(NextName, { TCHAR(':'), TCHAR('.') },
	[&ObjectPath, &NumObjectDelimiters](FStringView NextSingleName)
	{
		TCHAR Delimiter = NumObjectDelimiters++ == 1 ? ':' : '.';
		ObjectPath << Delimiter << NextSingleName;
	},
	UE::String::EParseTokensOptions::SkipEmpty);
}

FString FPackageName::ObjectPathCombine(FStringView ObjectPath, FStringView NextName)
{
	TStringBuilder<256> Base(InPlace, ObjectPath);
	ObjectPathAppend(Base, NextName);
	return FString(Base);
}

bool FPackageName::IsVersePackage(FStringView InPackageName)
{
	return InPackageName.Contains(FLongPackagePathsSingleton::Get().VerseSubPath);
}

bool FPackageName::IsScriptPackage(FStringView InPackageName)
{
	return FPathViews::IsParentPathOf(FLongPackagePathsSingleton::Get().ScriptRootPath, InPackageName);
}

bool FPackageName::IsMemoryPackage(FStringView InPackageName)
{
	return FPathViews::IsParentPathOf(FLongPackagePathsSingleton::Get().MemoryRootPath, InPackageName);
}

bool FPackageName::IsTempPackage(FStringView InPackageName)
{
	return FPathViews::IsParentPathOf(FLongPackagePathsSingleton::Get().TempRootPath, InPackageName);
}

bool FPackageName::IsInEngineTransientPackages(FStringView InPackageName)
{
	constexpr FStringView EngineTransientPrefix(TEXTVIEW("/Engine/Transient"));
	return InPackageName.StartsWith(EngineTransientPrefix, ESearchCase::IgnoreCase) &&
		(InPackageName.Len() == EngineTransientPrefix.Len() || InPackageName[EngineTransientPrefix.Len()] == '/');
}

bool FPackageName::IsLocalizedPackage(FStringView InPackageName)
{
	// Minimum valid package name length is "/A/L10N"
	if (InPackageName.Len() < 7)
	{
		return false;
	}

	const TCHAR* CurChar = InPackageName.GetData();
	const TCHAR* EndChar = InPackageName.GetData() + InPackageName.Len();

	// Must start with a slash
	if (CurChar == EndChar || *CurChar++ != TEXT('/'))
	{
		return false;
	}

	// Find the end of the first part of the path, eg /Game/
	while (CurChar != EndChar && *CurChar++ != TEXT('/')) {}
	if (CurChar == EndChar)
	{
		// Found end-of-string
		return false;
	}

	// Are we part of the L10N folder?
	FStringView Remaining(CurChar, UE_PTRDIFF_TO_INT32(EndChar - CurChar));
	// Is "L10N" or StartsWith "L10N/" 
	return Remaining.StartsWith(TEXTVIEW("L10N"), ESearchCase::IgnoreCase) && (Remaining.Len() == 4 || Remaining[4] == '/');
}

FString FPackageName::FormatErrorAsString(FStringView InPath, EErrorCode ErrorCode)
{
	FText ErrorText = FormatErrorAsText(InPath, ErrorCode);
	return ErrorText.ToString();
}

FText FPackageName::FormatErrorAsText(FStringView InPath, EErrorCode ErrorCode)
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("InPath"), FText::FromString(FString(InPath)));
	switch (ErrorCode)
	{
	case EErrorCode::PackageNameUnknown:
		return FText::Format(NSLOCTEXT("Core", "PackageNameUnknownError", "Input '{InPath}' caused undocumented internal error."), Args);
	case EErrorCode::PackageNameEmptyPath:
		return FText::Format(NSLOCTEXT("Core", "PackageNameEmptyPath", "Input '{InPath}' was empty."), Args);
	case EErrorCode::PackageNamePathNotMounted:
		return FText::Format(NSLOCTEXT("Core", "PackageNamePathNotMounted", "Input '{InPath}' is not a child of an existing mount point."), Args);
	case EErrorCode::PackageNamePathIsMemoryOnly:
		return FText::Format(NSLOCTEXT("Core", "PackageNamePathIsMemoryOnly", "Input '{InPath}' is in a memory-only mount point."), Args);
	case EErrorCode::PackageNameSpacesNotAllowed:
		return FText::Format(NSLOCTEXT("Core", "PackageNameSpacesNotAllowed", "Input '{InPath}' includes a space. A space is not valid in ObjectPaths unless it is delimiting a ClassName in a full ObjectPath, and full ObjectPaths are not supported in this function."), Args);
	case EErrorCode::PackageNameContainsInvalidCharacters:
		Args.Add(TEXT("IllegalNameCharacters"), FText::FromString(FString(INVALID_LONGPACKAGE_CHARACTERS)));
		return FText::Format(NSLOCTEXT("Core", "PackageNameContainsInvalidCharacters", "Input '{InPath}' contains one of the invalid characters for LongPackageNames: '{IllegalNameCharacters}'."), Args);
	case EErrorCode::LongPackageNames_PathTooShort:
	{
		// This has to be an FFormatOrderedArguments until we change the localized text string for it.
		FFormatOrderedArguments OrderedArgs;
		OrderedArgs.Add(FText::AsNumber(PackageNameConstants::MinPackageNameLength));
		OrderedArgs.Add(FText::FromString(FString(InPath)));
		return FText::Format(NSLOCTEXT("Core", "LongPackageNames_PathTooShort", "Input '{1}' contains fewer than the minimum number of characters {0} for LongPackageNames."), OrderedArgs);
	}
	case EErrorCode::LongPackageNames_PathWithNoStartingSlash:
		return FText::Format(NSLOCTEXT("Core", "LongPackageNames_PathWithNoStartingSlash", "Input '{InPath}' does not start with a '/', which is required for LongPackageNames."), Args);
	case EErrorCode::LongPackageNames_PathWithTrailingSlash:
		return FText::Format(NSLOCTEXT("Core", "LongPackageNames_PathWithTrailingSlash", "Input '{InPath}' ends with a '/', which is invalid for LongPackageNames."), Args);
	case EErrorCode::LongPackageNames_PathWithDoubleSlash:
		return FText::Format(NSLOCTEXT("Core", "LongPackageNames_PathWithDoubleSlash", "Input '{InPath}' contains '//', which is invalid for LongPackageNames."), Args);
	default:
		UE_LOGF(LogPackageName, Warning, "FPackageName::FormatErrorAsText: Invalid ErrorCode %d", static_cast<int32>(ErrorCode));
		return FText::Format(NSLOCTEXT("Core", "PackageNameUndocumentedError", "Input '{InPath} caused undocumented internal error."), Args);
	}
}

#if !UE_BUILD_SHIPPING
void ConsoleCommandImplConvertFilenameToLongPackageName(const TArray<FString>& Args)
{
	if (Args.IsEmpty())
	{
		UE_LOGF(LogPackageName, Display, "Invalid arguments. Usage: PackageName.ConvertFilenameToLongPackageName <Filename>.");
		return;
	}

	FString LongPackageName;
	if (!FPackageName::TryConvertFilenameToLongPackageName(Args[0], LongPackageName))
	{
		UE_LOGF(LogPackageName, Display, "");
		return;
	}

	UE_LOGF(LogPackageName, Display, "%ls", *LongPackageName);
}

static FAutoConsoleCommand ConsoleCommandConvertFilenameToLongPackageName(
	TEXT("PackageName.ConvertFilenameToLongPackageName"),
	TEXT("Prints the corresponding packagename for a filename at a given localpath, according to the current registered mount points. Prints empty string if not mounted."),
	FConsoleCommandWithArgsDelegate::CreateStatic(ConsoleCommandImplConvertFilenameToLongPackageName)
);

void ConsoleCommandImplConvertLongPackageNameToFilename(const TArray<FString>& Args)
{
	if (Args.IsEmpty())
	{
		UE_LOGF(LogPackageName, Display, "Invalid arguments. Usage: PackageName.ConvertFilenameToLongPackageName <Filename>.");
		return;
	}

	FString Filename;
	if (!FPackageName::TryConvertLongPackageNameToFilename(Args[0], Filename))
	{
		UE_LOGF(LogPackageName, Display, "");
		return;
	}

	UE_LOGF(LogPackageName, Display, "%ls", *Filename);
}

static FAutoConsoleCommand ConsoleCommandConvertLongPackageNameToFilename(
	TEXT("PackageName.ConvertLongPackageNameToFilename"),
	TEXT("Prints the corresponding local filename for a given packagename, according to the current registered mount points. Prints empty string if not mounted."),
	FConsoleCommandWithArgsDelegate::CreateStatic(ConsoleCommandImplConvertLongPackageNameToFilename)
);
#endif

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPackageNameTests, "System.Core.Misc.PackageNames", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)

bool FPackageNameTests::RunTest(const FString& Parameters)
{
	auto AggressiveNormalize = [](FStringView Path)
	{
		FString Norm(Path);
		Norm.ReplaceCharInline('\\', '/');
		// Strip trailing /
		if (Norm.Len() >= 2)
		{
			TCHAR Last = Norm[Norm.Len() - 1];
			TCHAR Next = Norm[Norm.Len() - 2];
			if (Last == '/' && (Next != '/' && Next != ':'))
			{
				Norm.LeftChopInline(1);
			}
		}
		// Replace // with / except at beginning
		if (Norm.StartsWith(TEXT("//")))
		{
			FString Suffix = Norm.RightChop(2);
			Suffix.ReplaceInline(TEXT("//"), TEXT("/"));
			Norm = TEXT("//") + Suffix;
		}
		else
		{
			Norm.ReplaceInline(TEXT("//"), TEXT("/"));
		}
		return Norm;
	};
	auto IsSamePath = [&AggressiveNormalize](FStringView A, FStringView B)
	{
		FString NormA(AggressiveNormalize(A));
		FString NormB(AggressiveNormalize(B));
		return FPaths::IsSamePath(NormA, NormB);
	};

	// Localized paths tests
	{
		auto TestIsLocalizedPackage = [&](const FString& InPath, const bool InExpected)
		{
			const bool bResult = FPackageName::IsLocalizedPackage(InPath);
			if (bResult != InExpected)
			{
				AddError(FString::Printf(TEXT("Path '%s' failed FPackageName::IsLocalizedPackage (got '%d', expected '%d')."), *InPath, bResult, InExpected));
			}
		};
		
		TestIsLocalizedPackage(TEXT("/Game"), false);
		TestIsLocalizedPackage(TEXT("/Game/MyAsset"), false);
		TestIsLocalizedPackage(TEXT("/Game/L10N"), true);
		TestIsLocalizedPackage(TEXT("/Game/L10N/en"), true);
		TestIsLocalizedPackage(TEXT("/Game/L10N/en/MyAsset"), true);
	}

	// Source path tests
	{
		auto TestGetSourcePackagePath = [this](const FString& InPath, const FString& InExpected)
		{
			const FString Result = FPackageName::GetSourcePackagePath(InPath);
			if (Result != InExpected)
			{
				AddError(FString::Printf(TEXT("Path '%s' failed FPackageName::GetSourcePackagePath (got '%s', expected '%s')."), *InPath, *Result, *InExpected));
			}
		};

		TestGetSourcePackagePath(TEXT("/Game"), TEXT("/Game"));
		TestGetSourcePackagePath(TEXT("/Game/MyAsset"), TEXT("/Game/MyAsset"));
		TestGetSourcePackagePath(TEXT("/Game/L10N"), TEXT("/Game"));
		TestGetSourcePackagePath(TEXT("/Game/L10N/en"), TEXT("/Game"));
		TestGetSourcePackagePath(TEXT("/Game/L10N/en/MyAsset"), TEXT("/Game/MyAsset"));
	}

	// ObjectPath conversions
	{
		FStringView SourceObjectPaths[] = {
			TEXT("/Game/MyAsset.MyAsset:SubObject.AnotherObject"),
			TEXT("/Game/MyAsset.MyAsset:SubObject"),
			TEXT("/Game/MyAsset.MyAsset"),
			TEXT("/Game/MyAsset"),
		};

		auto RunObjectPathTests = [this, &SourceObjectPaths](FStringView SubTestName, TArrayView<const FStringView> ExpectedOutputPaths, TFunctionRef<FStringView(FStringView)> ObjectPathFunc)
		{
			check(UE_ARRAY_COUNT(SourceObjectPaths) == ExpectedOutputPaths.Num());

			for (int32 Index = 0; Index < UE_ARRAY_COUNT(SourceObjectPaths); ++Index)
			{
				FStringView ActualOutputPath = ObjectPathFunc(SourceObjectPaths[Index]);
				if (ActualOutputPath != ExpectedOutputPaths[Index])
				{
					AddError(*WriteToString<256>(SubTestName, TEXT(": Expected '"), ExpectedOutputPaths[Index], TEXT("' but got '"), ActualOutputPath, TEXT("' for input '"), SourceObjectPaths[Index], TEXT("'")));
				}
			}
		};

		// ObjectPathToPackageName
		{
			FStringView ExpectedOutputPaths[] = {
				TEXT("/Game/MyAsset"),
				TEXT("/Game/MyAsset"),
				TEXT("/Game/MyAsset"),
				TEXT("/Game/MyAsset"),
			};
			RunObjectPathTests(TEXT("ObjectPathToPackageName"), ExpectedOutputPaths, [](FStringView ObjectPath) { return FPackageName::ObjectPathToPackageName(ObjectPath); });
		}

		// ObjectPathToPathWithinPackage
		{
			FStringView ExpectedOutputPaths[] = {
				TEXT("MyAsset:SubObject.AnotherObject"),
				TEXT("MyAsset:SubObject"),
				TEXT("MyAsset"),
				TEXT(""),
			};
			RunObjectPathTests(TEXT("ObjectPathToPathWithinPackage"), ExpectedOutputPaths, [](FStringView ObjectPath) { return FPackageName::ObjectPathToPathWithinPackage(ObjectPath); });
		}

		// ObjectPathToOuterPath
		{
			FStringView ExpectedOutputPaths[] = {
				TEXT("/Game/MyAsset.MyAsset:SubObject"),
				TEXT("/Game/MyAsset.MyAsset"),
				TEXT("/Game/MyAsset"),
				TEXT(""),
			};
			RunObjectPathTests(TEXT("ObjectPathToOuterPath"), ExpectedOutputPaths, [](FStringView ObjectPath) { return FPackageName::ObjectPathToOuterPath(ObjectPath); });
		}

		// ObjectPathToSubObjectPath
		{
			FStringView ExpectedOutputPaths[] = {
				TEXT("SubObject.AnotherObject"),
				TEXT("SubObject"),
				TEXT("MyAsset"),
				TEXT("/Game/MyAsset"),
			};
			RunObjectPathTests(TEXT("ObjectPathToSubObjectPath"), ExpectedOutputPaths, [](FStringView ObjectPath) { return FPackageName::ObjectPathToSubObjectPath(ObjectPath); });
		}

		// ObjectPathToObjectName
		{
			FStringView ExpectedOutputPaths[] = {
				TEXT("AnotherObject"),
				TEXT("SubObject"),
				TEXT("MyAsset"),
				TEXT("/Game/MyAsset"),
			};
			RunObjectPathTests(TEXT("ObjectPathToObjectName"), ExpectedOutputPaths, [](FStringView ObjectPath) { return FPackageName::ObjectPathToObjectName(ObjectPath); });
		}

		// ObjectPathSplitFirstName
		{
			auto ObjectPathSplitFirstNameTest = [this](FStringView Text, FStringView ExpectedFirst, FStringView ExpectedRemainder)
				{
					FStringView ActualFirst;
					FStringView ActualRemainder;
					FPackageName::ObjectPathSplitFirstName(Text, ActualFirst, ActualRemainder);
					if (!ActualFirst.Equals(ExpectedFirst, ESearchCase::CaseSensitive) ||
						!ActualRemainder.Equals(ExpectedRemainder, ESearchCase::CaseSensitive))
					{
						AddError(*WriteToString<256>(TEXT("ObjectPathSplitFirstName"), TEXT(": Expected {'"),
							ExpectedFirst, TEXT("', '"), ExpectedRemainder, TEXT("'} but got {'"),
							ActualFirst, TEXT("', '"), ActualRemainder, TEXT("'} for input '"), Text, TEXT("'")));
					}
				};
			ObjectPathSplitFirstNameTest(TEXT("/Game/MyAsset.MyAsset:SubObject.AnotherObject"), TEXT("/Game/MyAsset"), TEXT("MyAsset:SubObject.AnotherObject"));
			ObjectPathSplitFirstNameTest(TEXT("/Game/MyAsset.MyAsset:SubObject"), TEXT("/Game/MyAsset"), TEXT("MyAsset:SubObject"));
			ObjectPathSplitFirstNameTest(TEXT("/Game/MyAsset.MyAsset"), TEXT("/Game/MyAsset"), TEXT("MyAsset"));
			ObjectPathSplitFirstNameTest(TEXT("/Game/MyAsset"), TEXT("/Game/MyAsset"), TEXT(""));
			ObjectPathSplitFirstNameTest(TEXT("MyAsset:SubObject"), TEXT("MyAsset"), TEXT("SubObject"));
			ObjectPathSplitFirstNameTest(TEXT("MyAsset.SubObject"), TEXT("MyAsset"), TEXT("SubObject"));
			ObjectPathSplitFirstNameTest(TEXT("MyAsset"), TEXT("MyAsset"), TEXT(""));
			ObjectPathSplitFirstNameTest(TEXT(""), TEXT(""), TEXT(""));
		}

		// ObjectPathAppend
		{
			auto ObjectPathAppendTest = [this](FStringView A, FStringView B, FStringView Expected)
				{
					TStringBuilder<256> Base(InPlace, A);
					FPackageName::ObjectPathAppend(Base, B);
					if (!Base.ToView().Equals(Expected, ESearchCase::CaseSensitive))
					{
						AddError(*WriteToString<256>(TEXT("ObjectPathAppend"), TEXT(": Expected '"),
							Expected, TEXT("' but got '"), Base, TEXT("' for input {'"),
							A, TEXT("', '"), B, TEXT("'}")));
					}
				};
			ObjectPathAppendTest(TEXT("/Package"), TEXT("Object"), TEXT("/Package.Object"));
			ObjectPathAppendTest(TEXT("/Package.Object"), TEXT("SubObject"), TEXT("/Package.Object:SubObject"));
			ObjectPathAppendTest(TEXT("/Package.Object:SubObject"), TEXT("NextSubObject"), TEXT("/Package.Object:SubObject.NextSubObject"));
			ObjectPathAppendTest(TEXT("/Package"), TEXT("Object.SubObject"), TEXT("/Package.Object:SubObject"));
			ObjectPathAppendTest(TEXT("/Package"), TEXT("/OtherPackage.Object:SubObject"), TEXT("/OtherPackage.Object:SubObject"));
			ObjectPathAppendTest(TEXT("/Package"), TEXT(""), TEXT("/Package"));
			ObjectPathAppendTest(TEXT(""), TEXT("/Package.Object:SubObject"), TEXT("/Package.Object:SubObject"));
			ObjectPathAppendTest(TEXT(""), TEXT("Object"), TEXT("Object"));
			ObjectPathAppendTest(TEXT(""), TEXT(""), TEXT(""));
		}
	}

	// TryConvertToMountedPath
	{
		auto TestConvert = [this](FStringView InPath, bool bExpectedResult, FStringView ExpectedLocalPath,
			FStringView ExpectedPackageName, FStringView ExpectedObjectName, FStringView ExpectedSubObjectName,
			FStringView ExpectedExtension, FPackageName::EFlexNameType ExpectedFlexNameType,
			FPackageName::EErrorCode ExpectedFailureReason)
		{
			bool bActualResult;
			FString ActualLocalPath;
			FString ActualPackageName;
			FString ActualObjectName;
			FString ActualSubObjectName;
			FString ActualExtension;
			FPackageName::EFlexNameType ActualFlexNameType;
			FPackageName::EErrorCode ActualFailureReason;

			bActualResult = FPackageName::TryConvertToMountedPath(InPath, &ActualLocalPath, &ActualPackageName,
				&ActualObjectName, &ActualSubObjectName, &ActualExtension, &ActualFlexNameType, &ActualFailureReason);
			if (bActualResult != bExpectedResult || ActualLocalPath != ExpectedLocalPath ||
				ActualPackageName != ExpectedPackageName || ActualObjectName != ExpectedObjectName ||
				ActualSubObjectName != ExpectedSubObjectName || ActualExtension != ExpectedExtension ||
				ActualFlexNameType != ExpectedFlexNameType || ActualFailureReason != ExpectedFailureReason)
			{
				AddError(FString::Printf(TEXT("Path '%.*s' failed FPackageName::TryConvertToMountedPath\n"
					"got      %s,'%s','%s','%s','%s','%s',%d,%d,\nexpected %s,'%.*s','%.*s','%.*s','%.*s','%.*s',%d,%d."),
					InPath.Len(), InPath.GetData(),
					bActualResult ? TEXT("true") : TEXT("false"),
					*ActualLocalPath, *ActualPackageName, *ActualObjectName, *ActualSubObjectName, *ActualExtension,
					(int32)ActualFlexNameType, (int32)ActualFailureReason,
					bExpectedResult ? TEXT("true") : TEXT("false"),
					ExpectedLocalPath.Len(), ExpectedLocalPath.GetData(),
					ExpectedPackageName.Len(), ExpectedPackageName.GetData(),
					ExpectedObjectName.Len(), ExpectedObjectName.GetData(),
					ExpectedSubObjectName.Len(), ExpectedSubObjectName.GetData(),
					ExpectedExtension.Len(), ExpectedExtension.GetData(),
					(int32)ExpectedFlexNameType, (int32)ExpectedFailureReason));
			}
		};
		FString EngineFileRoot = FPaths::EngineContentDir();
		auto EngineLocalPath = [&EngineFileRoot](FStringView RelPath)
		{
			return FPaths::Combine(EngineFileRoot, FString(RelPath));
		};
		TestConvert(TEXT("/Engine"), true, EngineFileRoot, TEXT("/Engine/"),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::PackageName, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(TEXT("/Engine/"), true, EngineFileRoot, TEXT("/Engine/"),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::PackageName, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(EngineFileRoot, true, EngineFileRoot, TEXT("/Engine/"),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::LocalPath, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(TEXT("/Engine/Foo"), true, EngineLocalPath(TEXT("Foo")), TEXT("/Engine/Foo"),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::PackageName, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(EngineLocalPath(TEXT("Foo")), true, EngineLocalPath(TEXT("Foo")), TEXT("/Engine/Foo"),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::LocalPath, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(EngineLocalPath(TEXT("Foo.uasset")), true, EngineLocalPath(TEXT("Foo")), TEXT("/Engine/Foo"),
			FStringView(), FStringView(), TEXT(".uasset"),
			FPackageName::EFlexNameType::LocalPath, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(EngineLocalPath(TEXT("Foo.Bar")), true, EngineLocalPath(TEXT("Foo")), TEXT("/Engine/Foo"),
			FStringView(), FStringView(), TEXT(".Bar"),
			FPackageName::EFlexNameType::LocalPath, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(TEXT("/Engine/Foo.Bar"), true, EngineLocalPath(TEXT("Foo")), TEXT("/Engine/Foo"),
			TEXT("Bar"), FStringView(), FStringView(),
			FPackageName::EFlexNameType::ObjectPath, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(TEXT("/Engine/Foo.Bar:Baz"), true, EngineLocalPath(TEXT("Foo")), TEXT("/Engine/Foo"),
			TEXT("Bar"), TEXT("Baz"), FStringView(),
			FPackageName::EFlexNameType::ObjectPath, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(TEXT("/Engine/Foo.Spaces Ignored"), true, EngineLocalPath(TEXT("Foo")), TEXT("/Engine/Foo"),
			TEXT("Spaces Ignored"), FStringView(), FStringView(),
			FPackageName::EFlexNameType::ObjectPath, FPackageName::EErrorCode::PackageNameUnknown);
		TestConvert(TEXT("/Engine/Foo.Spaces Ignored:For Spaces"), true, EngineLocalPath(TEXT("Foo")), TEXT("/Engine/Foo"),
			TEXT("Spaces Ignored"), TEXT("For Spaces"), FStringView(),
			FPackageName::EFlexNameType::ObjectPath, FPackageName::EErrorCode::PackageNameUnknown);

		TestConvert(TEXT("/Engine/Package Spaces"), false, FStringView(), FStringView(),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::Invalid, FPackageName::EErrorCode::PackageNameSpacesNotAllowed);
		TestConvert(TEXT("/Engine/PackageNameWithInvalidChar?"), false, FStringView(), FStringView(),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::Invalid, FPackageName::EErrorCode::PackageNameContainsInvalidCharacters);
		TestConvert(TEXT("Texture2D /Engine/Foo"), false, FStringView(), FStringView(),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::Invalid, FPackageName::EErrorCode::PackageNameSpacesNotAllowed);
		TestConvert(TEXT("/FPackageNameTests_DNE/Foo"), false, FStringView(), FStringView(),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::Invalid, FPackageName::EErrorCode::PackageNamePathNotMounted);
		TestConvert(TEXT("../../../FPackageNameTests_DNE/Foo"), false, FStringView(), FStringView(),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::Invalid, FPackageName::EErrorCode::PackageNamePathNotMounted);
		TestConvert(TEXT(""), false, FStringView(), FStringView(),
			FStringView(), FStringView(), FStringView(),
			FPackageName::EFlexNameType::Invalid, FPackageName::EErrorCode::PackageNameEmptyPath);
	}

	// TryGetMountPointForPath
	{
		auto TestTryGetMountPointForPath = [this, &IsSamePath](FStringView InPath, bool bExpectedResult, FStringView ExpectedPackageName,
			FStringView ExpectedFilePath, FStringView ExpectedRelPath, FPackageName::EFlexNameType ExpectedFlexNameType,
			FPackageName::EErrorCode ExpectedFailureReason)
		{
			bool bActualResult;
			TStringBuilder<256> ActualMountPointPackageName;
			TStringBuilder<256> ActualMountPointFilePath;
			TStringBuilder<256> ActualRelPath;
			FPackageName::EFlexNameType ActualFlexNameType;
			FPackageName::EErrorCode ActualFailureReason;

			bActualResult = FPackageName::TryGetMountPointForPath(InPath, ActualMountPointPackageName,
				ActualMountPointFilePath, ActualRelPath, &ActualFlexNameType, &ActualFailureReason);
			bool bActualMatchesExpected = bActualResult == bExpectedResult &&
				ActualFlexNameType == ExpectedFlexNameType && ActualFailureReason == ExpectedFailureReason;
			if (bActualMatchesExpected && bExpectedResult)
			{
				bActualMatchesExpected =
					IsSamePath(FString(ExpectedPackageName), FString(ActualMountPointPackageName)) &&
					IsSamePath(FString(ExpectedFilePath), FString(ActualMountPointFilePath)) &&
					IsSamePath(FString(ExpectedRelPath), FString(ActualRelPath));
			}
			if (!bActualMatchesExpected)
			{
				if (!bActualResult)
				{
					ActualMountPointPackageName.Reset();
					ActualMountPointFilePath.Reset();
					ActualRelPath.Reset();
				}
				AddError(FString::Printf(TEXT("Path '%.*s' failed FPackageName::TestTryGetMountPointForPath\n")
					TEXT("got      %s,'%s','%s','%s',%d,%d,\nexpected %s,'%s','%s','%s',%d,%d."),
					InPath.Len(), InPath.GetData(),
					(bActualResult ? TEXT("true") : TEXT("false")),
					*FString(ActualMountPointPackageName), *FString(ActualMountPointFilePath),
					*FString(ActualRelPath), (int32)ActualFlexNameType, (int32)ActualFailureReason,
					(bExpectedResult ? TEXT("true") : TEXT("false")),
					*FString(ExpectedPackageName), *FString(ExpectedFilePath),
					*FString(ExpectedRelPath), (int32)ExpectedFlexNameType, (int32)ExpectedFailureReason));
			}
		};

		TestTryGetMountPointForPath(FPaths::ProjectContentDir(), true, TEXT("/Game"), FPaths::ProjectContentDir(),
			TEXT(""), FPackageName::EFlexNameType::LocalPath, FPackageName::EErrorCode::PackageNameUnknown);
		TestTryGetMountPointForPath(TEXT("/Game"), true, TEXT("/Game"), FPaths::ProjectContentDir(),
			TEXT(""), FPackageName::EFlexNameType::PackageName, FPackageName::EErrorCode::PackageNameUnknown);

		// For a malformed path of this type:
		// d:/root/QAGame/Content -> d:/root/QAGame//Content
		// TestTryGetMountPointForPath should fail
		FString MalformedContentDir = FPaths::ProjectContentDir();
		MalformedContentDir.ReplaceCharInline('\\', '/');
		int32 LastSlashIndex;
		if (FStringView(MalformedContentDir).LeftChop(1).FindLastChar('/', LastSlashIndex))
		{
			MalformedContentDir = MalformedContentDir.Left(LastSlashIndex) + TEXT("/") + MalformedContentDir.RightChop(LastSlashIndex);
			TestTryGetMountPointForPath(MalformedContentDir, false, TEXT(""), TEXT(""),
				TEXT(""), FPackageName::EFlexNameType::Invalid, FPackageName::EErrorCode::PackageNamePathNotMounted);
		}
	}

	return true;
}

// Tests that are too expensive to run as a SmokeFilter
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPackageNameTestsExtended, "System.Core.Misc.PackageNamesExtended", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FPackageNameTestsExtended::RunTest(const FString& Parameters)
{
	// Mounting and Unmounting Paths
	{
		FString TestMountLongPackageName("/PackageNameTestDNE");
		FString TestMountLocalPathRoot("../../../Engine/Intermediate/PackageNameTestDNE");
		FString TestMountLocalPath("../../../Engine/Intermediate/PackageNameTestDNE/Content");
		FString TestLongPackageName("/PackageNameTestDNE/Package1");
		FString TestLocalPath("../../../Engine/Intermediate/PackageNameTestDNE/Content/Package1.uasset");
		FString TestExtension(".uasset");
		FString ActualLocalPath;
		ON_SCOPE_EXIT
		{
			IFileManager::Get().DeleteDirectory(*TestMountLocalPathRoot, false /* bRequireExists */, true /* Delete Tree */);
		};

		if (FPackageName::TryConvertLongPackageNameToFilename(TestLongPackageName, ActualLocalPath, TestExtension))
		{
			AddError(FString("Mounting test: TestPath is unexpectedly before Register."));
		}

		// Register,UnRegister,Register,Unregister works
		// Register
		FPackageName::RegisterMountPoint(TestMountLongPackageName, TestMountLocalPath);
		if (!FPackageName::TryConvertLongPackageNameToFilename(TestLongPackageName, ActualLocalPath, TestExtension))
		{
			AddError(FString("Mounting test: TestPath not mounted after Register."));
		}
		else if (!FPaths::IsSamePath(TestLocalPath, ActualLocalPath))
		{
			AddError(FString("Mounting test: TestPath mounted to wrong location after Register."));
		}

		// Unregister
		FPackageName::UnRegisterMountPoint(TestMountLongPackageName, TestMountLocalPath);
		if (FPackageName::TryConvertLongPackageNameToFilename(TestLongPackageName, ActualLocalPath, TestExtension))
		{
			AddError(FString("Mounting test: TestPath is unexpectedly mounted after Register, Unregister."));
		}

		// Register
		FPackageName::RegisterMountPoint(TestMountLongPackageName, TestMountLocalPath);
		if (!FPackageName::TryConvertLongPackageNameToFilename(TestLongPackageName, ActualLocalPath, TestExtension))
		{
			AddError(FString("Mounting test: TestPath not mounted after Register, Unregister, Register."));
		}
		else if (!FPaths::IsSamePath(TestLocalPath, ActualLocalPath))
		{
			AddError(FString("Mounting test: TestPath mounted to wrong location after Register, Unregister, Register."));
		}

		// Unregister
		FPackageName::UnRegisterMountPoint(TestMountLongPackageName, TestMountLocalPath);
		if (FPackageName::TryConvertLongPackageNameToFilename(TestLongPackageName, ActualLocalPath, TestExtension))
		{
			AddError(FString("Mounting test: TestPath is unexpectedly mounted after Register, Unregister, Register, Unregister."));
		}

		// Register,Register a duplicate, Unregister should remove all duplicates

		// Register
		FPackageName::RegisterMountPoint(TestMountLongPackageName, TestMountLocalPath);
		if (!FPackageName::TryConvertLongPackageNameToFilename(TestLongPackageName, ActualLocalPath, TestExtension))
		{
			AddError(FString("Mounting test: TestPath not mounted after Register."));
		}
		else if (!FPaths::IsSamePath(TestLocalPath, ActualLocalPath))
		{
			AddError(FString("Mounting test: TestPath mounted to wrong location after Register."));
		}
		// Register a duplicate
		FPackageName::RegisterMountPoint(TestMountLongPackageName, TestMountLocalPath);
		if (!FPackageName::TryConvertLongPackageNameToFilename(TestLongPackageName, ActualLocalPath, TestExtension))
		{
			AddError(FString("Mounting test: TestPath not mounted after Register, RegisterDuplicate."));
		}
		else if (!FPaths::IsSamePath(TestLocalPath, ActualLocalPath))
		{
			AddError(FString("Mounting test: TestPath mounted to wrong location after Register, RegisterDuplicate."));
		}
		// Unregister
		FPackageName::UnRegisterMountPoint(TestMountLongPackageName, TestMountLocalPath);
		if (FPackageName::TryConvertLongPackageNameToFilename(TestLongPackageName, ActualLocalPath, TestExtension))
		{
			AddError(FString("Mounting test: TestPath is unexpectedly mounted after Register, RegisterDuplicate, Unregister."));
		}
	}
	return true;
}

#endif //WITH_DEV_AUTOMATION_TESTS
#undef LOCTEXT_NAMESPACE // "PackageNames"
