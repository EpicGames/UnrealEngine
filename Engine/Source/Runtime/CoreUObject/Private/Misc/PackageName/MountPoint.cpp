// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/MountPoint.h"

#include "MountPointPrivate.h"
#include "PackageNamePrivate.h"

namespace UE::PackageName
{

FMountPoint::FMountPoint(FString&& InLongPackageName, FString&& InLocalPathStandard,
	FString&& InLocalPathAbsolute, EMountFlags Flags)
	: LongPackageName(MoveTemp(InLongPackageName))
	, LocalPathStandard(MoveTemp(InLocalPathStandard))
	, LocalPathAbsolute(MoveTemp(InLocalPathAbsolute))
	, bRemovable(!!(Flags & EMountFlags::Removable))
	, bReadOnly(!!(Flags & EMountFlags::ReadOnly))
	, bAlias(!!(Flags & EMountFlags::Alias))
{
}

bool FMountPoint::IsMounted() const
{
	return bMounted.load(std::memory_order_relaxed);
}

void FMountPoint::SetMounted(bool bInMounted, UE::TWriteScopeLock<FTransactionallySafeRWLock>& /* ProofOfWriteLock */,
	const TRefCountPtr<IMountPoint>& /* ProofOfCallerRefCount */, bool& bOutRemoveFromContainer)
{
	bOutRemoveFromContainer = false;

	// bMounted is only modified within a write lock, so we do not need compare_exchange
	bool bWasMounted = bMounted.load(std::memory_order_relaxed);

	// Our caller holds a refcount, to prevent the possiblility of refcount going to zero when we update mounted
	// status; refcount should only go to zero once we leave the WriteLock. If we are already mounted, we should
	// additionally hold a refcount for being mounted.
	check(RefCount.load(std::memory_order_relaxed) >= (bWasMounted ? 2U : 1U));
	if (bWasMounted == bInMounted)
	{
		return;
	}

	bMounted.store(bInMounted, std::memory_order_relaxed);

	// While mounted, we hold an internal refcount on the MountPoint.
	if (bInMounted)
	{
		AddRef();
	}
	else
	{
		Release();

		// If there is no further external refcount, as an optimization, allow our caller to remove
		// us from the FLongPackagePathsSingleton.MountPoints container, possibly more efficiently
		// than we will be able to do it in the upcoming TryUnregisterAndDelete.
		bOutRemoveFromContainer = RefCount.load(std::memory_order_relaxed) == 1;
		bAlreadyRemovedFromContainer = bOutRemoveFromContainer;
	}
}

void FMountPoint::AddRef() const
{
	RefCount.fetch_add(1, std::memory_order_relaxed);
}

FReturnedRefCountValue FMountPoint::Release() const
{
	uint32 OldValue = RefCount.fetch_add(-1, std::memory_order_relaxed);
	check(OldValue > 0);
	if (OldValue == 1)
	{
		const_cast<FMountPoint*>(this)->TryUnregisterAndDelete();
	}
	return FReturnedRefCountValue{ OldValue - 1 };
}

void FMountPoint::TryUnregisterAndDelete()
{
	// Another thread may be in the process of finding the pointer to this from
	// FLongPackagePathsSingleton.RootPathTree, and it might AddRef after the decrement in our caller function.
	// It does that Find and AddRef inside a WriteLock on FLongPackagePathsSingleton.MountLock.
	// Enter the WriteLock ourselves and check that the RefCount is still 0 within the writelock
	// before unregister and delete.
	FLongPackagePathsSingleton& Paths = FLongPackagePathsSingleton::Get();
	UE::TWriteScopeLock<FTransactionallySafeRWLock> ScopeLock(Paths.MountLock);
	if (RefCount.load(std::memory_order_relaxed) != 0)
	{
		return;
	}

	// We may have set bAlreadyRemovedFromContainer when unmounting, in which case we can skip the effort of removing
	// from MountPoints
	if (!bAlreadyRemovedFromContainer)
	{
		// It should be impossible for an FMountPoint to be in one of the mounted containers when refcount is 0, because
		// we AddRef it when mounted and do not Release until unmounted. Assert that it is not in any of the moounted containers.
#if DO_CHECK
		FMountPoint** Existing = Paths.RootPathTree.Find(LongPackageName);
		check(!Existing || *Existing != this);
		Existing = Paths.ContentPathTree.Find(LocalPathStandard);
		check(!Existing || *Existing != this);
		Existing = Paths.ContentPathTree.Find(LocalPathAbsolute);
		check(!Existing || *Existing != this);
#endif

		Paths.MountPoints.Remove(FMountPointKeyByPtr(*this));
	}

	delete this;
}

FReturnedRefCountValue FMountPoint::GetRefCount() const
{
	return FReturnedRefCountValue{ RefCount.load(std::memory_order_relaxed) };
}

} // namespace UE::PackageName