// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/MountPoint.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"

#include <atomic>

namespace UE::PackageName
{

enum class EMountFlags
{
	Removable = 0x1,
	NotRemovable = 0x0,
	ReadOnly = 0x2,
	NotReadOnly = 0x0,
	Alias = 0x4,
	NotAlias = 0x0,
};
ENUM_CLASS_FLAGS(EMountFlags);

// Keytypes for use in TSets that look up mountpoints by their paths. We support both a KeyType struct with a pointer,
// to save memory, and a KeyType struct with strings, for when we have the paths but not an FMountPoint.
struct FMountPointKeyByPtr;
struct FMountPointKeyByStrings;
extern uint32 GetTypeHash(const FMountPointKeyByPtr& Key);
extern uint32 GetTypeHash(const FMountPointKeyByStrings& Key);


/**
 * The internal implementation of IMountPoint. FMountPoints are present in FLongPackagePathsSingleton.MountPoints
 * while allocated; they are removed from that set, inside a critical section, when their refcount goes to zero.
 */
class FMountPoint final : public IMountPoint
{
public:
	FMountPoint(FString&& InLongPackageName, FString&& InLocalPathStandard,
		FString&& InLocalPathAbsolute, EMountFlags Flags);

	// IMountPoint API
	virtual FStringView GetLongPackageName() const override;
	virtual FStringView GetLocalPathStandard() const override;
	virtual FStringView GetLocalPathAbsolute() const override;

	virtual bool IsRemovable() const override;
	virtual bool IsReadOnly() const override;
	virtual bool IsMounted() const override;

	virtual void AddRef() const override;
	virtual FReturnedRefCountValue Release() const override;
	virtual FReturnedRefCountValue GetRefCount() const override;
protected:
	virtual FMountPoint* AsFMountPoint() override;
public:

	// FMountPoint additional API

	/** Same value as GetLongPackageName */
	const FString& GetRootPath() const;
	/** Same value as GetLocalPathStandard */
	const FString& GetContentPathRelative() const;
	/** Same value as GetLocalPathAbsolute */
	const FString& GetContentPathAbsolute() const;

	/** Whether this mountpoint is the same RootPath as another mountpoint but bound to a different ContentPath. */
	bool IsAlias() const;

	/**
	 * Mark mounted or unmounted. Must only be called within Paths.MountLock, so that the
	 * change in value is atomic with external queries about whether a path is mounted.
	 * When unmounting, caller must hold a refcount to prevent the possibility of deletion.
	 *
	 * Caller should hold exactly 1 refcount when unmounting. In that case bOutRemoveFromContainer will
	 * be set to true and we will skip the effort in TryUnregisterAndDelete of removing it from the container.
	 */
	void SetMounted(bool bInMounted, UE::TWriteScopeLock<FTransactionallySafeRWLock>& ProofOfWriteLock,
		const TRefCountPtr<IMountPoint>& ProofOfCallerRefCount, bool& bOutRemoveFromContainer);

	/**
	 * Get/Set the Ephemeral ordering value. MountPoints that have the same RootPath or ContentPath as other
	 * mountpoints are selectively returned from queries where they overlap based on higher priority == higher value.
	 * Setting higher values higher in priority allows us to easily implement our contract that MountPoints added
	 * later are higher in priority than MountPoints added earlier.
	 * The exact value of this field can change arbitrarily in MountPoints, when e.g. we reorder the list. Only
	 * the relative ordering is guaranteed unchanged for an existing MountPoint.
	 */
	uint32 GetMountOrder() const;
	void SetMountOrder(uint32 MountOrder);

	bool Equals(const FMountPointKeyByStrings& Key) const;
	static uint32 GetTypeHashKey(const FString& LongPackageNameOfKey, const FString& LocalPathStandardOfKey);
	static bool KeysAreEqual(const FString& LongPackageNameA, const FString& LocalPathStandardA,
		const FString& LongPackageNameB, const FString& LocalPathStandardB);
	static bool IsHigherOverridePriority(FMountPoint* A, FMountPoint* B);

private:
	/**
	 * Called when refcount goes to zero. Enters the critical section and if still zero in critical section,
	 * removes this from FMountPoint's registration maps and deletes this.
	 */
	void TryUnregisterAndDelete();

	/** Const field set in Constructor */
	const FString LongPackageName;
	/** Const field set in Constructor */
	const FString LocalPathStandard;
	/** Const field set in Constructor */
	const FString LocalPathAbsolute;
	/** Const field set in Constructor */
	const bool bRemovable : 1;
	/** Const field set in Constructor */
	const bool bReadOnly : 1;
	/** Const field set in Constructor */
	const bool bAlias : 1;
	/** Read/Write only under FLongPackagePathsSingleton.MountLock */
	bool bAlreadyRemovedFromContainer : 1 = false;
	/** Read/Write only under FLongPackagePathsSingleton.MountLock */
	uint32 MountOrder = 0;
	/** Atomic field read/write from all threads */
	mutable std::atomic<uint32> RefCount{ 0 };
	/** Atomic field readable from all threads, writable only under FLongPackagePathsSingleton.MountLock */
	std::atomic<bool> bMounted{ false };
};

/**
 * A wrapper around FMountPoint* that provides the KeyFuncs for use in a TSet.
 * This type must only be used in the reference maps for FMountPoints, and removed before FMountPoint is destructed,
 * so that we can keep a raw pointer and not have to handle the complexity of accounting for this structure's
 * TRefCountPtr when deciding whether an FMountPoint is no longer referenced.
 */
struct FMountPointKeyByPtr
{
public:
	explicit FMountPointKeyByPtr(FMountPoint& InMountPoint);
	bool operator==(const FMountPointKeyByPtr& Other) const;
	bool operator==(const FMountPointKeyByStrings& Other) const;
	FMountPoint* GetMountPoint() const;

private:
	FMountPoint* MountPoint = nullptr;
	friend uint32 UE::PackageName::GetTypeHash(const FMountPointKeyByPtr& Key);
	friend UE::PackageName::FMountPointKeyByStrings;
};

/**
 * A structure that holds the same strings held by an FMountPoint that is comparable with FMountPointKeyByPtr,
 * to support queries against a TSet<FMountPointKeyByPtr>.
 */
struct FMountPointKeyByStrings
{
	FMountPointKeyByStrings() = default;
	FMountPointKeyByStrings(const FMountPoint& InMountPoint);
	FMountPointKeyByStrings(FString InLongPackageName, FString InLocalPathStandard);

	bool operator==(const FMountPointKeyByPtr& Other) const;
	bool operator==(const FMountPointKeyByStrings& Other) const;

	FString LongPackageName;
	FString LocalPathStandard;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


inline FStringView FMountPoint::GetLongPackageName() const
{
	return LongPackageName;
}

inline FStringView FMountPoint::GetLocalPathStandard() const
{
	return LocalPathStandard;
}

inline FStringView FMountPoint::GetLocalPathAbsolute() const
{
	return LocalPathAbsolute;
}

inline FMountPoint* FMountPoint::AsFMountPoint()
{
	return this;
}

inline bool FMountPoint::IsRemovable() const
{
	return bRemovable;
}

inline bool FMountPoint::IsReadOnly() const
{
	return bReadOnly;
}

inline const FString& FMountPoint::GetRootPath() const
{
	return LongPackageName;
}

inline const FString& FMountPoint::GetContentPathRelative() const
{
	return LocalPathStandard;
}

inline const FString& FMountPoint::GetContentPathAbsolute() const
{
	return LocalPathAbsolute;
}

inline bool FMountPoint::IsAlias() const
{
	return bAlias;
}

inline uint32 FMountPoint::GetMountOrder() const
{
	return MountOrder;
}

inline void FMountPoint::SetMountOrder(uint32 InMountOrder)
{
	MountOrder = InMountOrder;
}

inline FMountPointKeyByPtr::FMountPointKeyByPtr(FMountPoint& InMountPoint)
	: MountPoint(&InMountPoint)
{
}

inline bool FMountPointKeyByPtr::operator==(const FMountPointKeyByPtr& Other) const
{
	return FMountPoint::KeysAreEqual(MountPoint->GetRootPath(), MountPoint->GetContentPathRelative(),
		Other.MountPoint->GetRootPath(), Other.MountPoint->GetContentPathRelative());
}

inline bool FMountPointKeyByPtr::operator==(const FMountPointKeyByStrings& Other) const
{
	return FMountPoint::KeysAreEqual(MountPoint->GetRootPath(), MountPoint->GetContentPathRelative(),
		Other.LongPackageName, Other.LocalPathStandard);
}

inline FMountPoint* FMountPointKeyByPtr::GetMountPoint() const
{
	return MountPoint;
}

inline uint32 GetTypeHash(const FMountPointKeyByPtr& Key)
{
	return FMountPoint::GetTypeHashKey(Key.MountPoint->GetRootPath(), Key.MountPoint->GetContentPathRelative());
}

inline FMountPointKeyByStrings::FMountPointKeyByStrings(const FMountPoint& InMountPoint)
	: LongPackageName(InMountPoint.GetLongPackageName())
	, LocalPathStandard(InMountPoint.GetLocalPathStandard())
{
}

inline FMountPointKeyByStrings::FMountPointKeyByStrings(FString InLongPackageName, FString InLocalPathStandard)
	: LongPackageName(MoveTemp(InLongPackageName))
	, LocalPathStandard(MoveTemp(InLocalPathStandard))
{
}

inline bool FMountPointKeyByStrings::operator==(const FMountPointKeyByPtr& Other) const
{
	return FMountPoint::KeysAreEqual(LongPackageName, LocalPathStandard,
		Other.MountPoint->GetRootPath(), Other.MountPoint->GetContentPathRelative());
}

inline bool FMountPointKeyByStrings::operator==(const FMountPointKeyByStrings& Other) const
{
	return FMountPoint::KeysAreEqual(LongPackageName, LocalPathStandard,
		Other.LongPackageName, Other.LocalPathStandard);
}

inline uint32 GetTypeHash(const FMountPointKeyByStrings& Key)
{
	return FMountPoint::GetTypeHashKey(Key.LongPackageName, Key.LocalPathStandard);
}

inline bool FMountPoint::Equals(const FMountPointKeyByStrings& Key) const
{
	return KeysAreEqual(LongPackageName, LocalPathStandard, Key.LongPackageName, Key.LocalPathStandard);
}

inline uint32 FMountPoint::GetTypeHashKey(const FString& LongPackageNameOfKey, const FString& LocalPathStandardOfKey)
{
	return 101 * GetTypeHash(LongPackageNameOfKey) + GetTypeHash(LocalPathStandardOfKey);
}

inline bool FMountPoint::KeysAreEqual(const FString& LongPackageNameA, const FString& LocalPathStandardA,
	const FString& LongPackageNameB, const FString& LocalPathStandardB)
{
	return LongPackageNameA.Equals(LongPackageNameB, ESearchCase::IgnoreCase)
		&& LocalPathStandardA.Equals(LocalPathStandardB, ESearchCase::IgnoreCase);
}

inline bool FMountPoint::IsHigherOverridePriority(FMountPoint* A, FMountPoint* B)
{
	// Must be called within FLongPackagePathsSingleton->MountLock so that GetMountOrder is readable.
	if ((A != nullptr) != (B != nullptr))
	{
		return A != nullptr;
	}
	if (A == nullptr)
	{
		return false; // equal
	}
	CA_ASSUME(B != nullptr);

	if (A->IsMounted() != B->IsMounted())
	{
		return A->IsMounted();
	}
	return A->GetMountOrder() > B->GetMountOrder();
}

} // namespace UE::PackageName