// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/DirectoryTree.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/IConsoleManager.h"
#include "Misc/PackageName.h"
#include "Misc/PackageName/MountPointPrivate.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/UniquePtr.h"

struct FLongPackagePathsSingleton
{
	mutable FTransactionallySafeRWLock MountLock;

	FString ConfigRootPath;
	FString EngineRootPath;
	FString GameRootPath;
	FString ScriptRootPath;
	FString MemoryRootPath;
	FString TempRootPath;

	FString VerseSubPath;

	FString EngineContentPath;
	FString ContentPathShort;
	FString EngineShadersPath;
	FString EngineShadersPathShort;
	FString GameContentPath;
	FString GameConfigPath;
	FString GameScriptPath;
	FString GameSavedPath;
	FString GameContentPathRebased;
	FString GameConfigPathRebased;
	FString GameScriptPathRebased;
	FString GameSavedPathRebased;

	TDirectoryTree<UE::PackageName::FMountPoint*> RootPathTree;
	TDirectoryTree<UE::PackageName::FMountPoint*> ContentPathTree;
	/**
	 * Set of all MountPoints, possibly unmounted, looked up by SearchKey={LongPackageName,LocalPath}. 
	 * Elements added later are assigned a higher priority and will be the ones findable in RootPathTree
	 * and ContentPathTree if they have the same LongPackageName or LocalPath as another MountPoint.
	 * This override system allows aliasing both of two LocalPaths that go to the same LongPackageName and two
	 * LongPackageNames that go to the same LocalPath.
	 */
	TSet<UE::PackageName::FMountPointKeyByPtr> MountPoints;

	/** Assigned and incremented to each new mountpoint, to establish override priority. */
	uint32 NextMountOrder = 0;

	// singleton
	static FLongPackagePathsSingleton& Get();

	/** Initialization function to setup content paths that cannot be run until CoreUObject/PluginManager have been initialized. */
	void OnCoreUObjectInitialized();

	/**
	 * Given LongPackageName and LocalPath, return the consistently-formatted LongPackageName, LocalPathStandard, and
	 * LocalPathAbsolute, suitable for lookup in our maps of string to MountPoint.
	 * LocalPathStandard is a relative path and is consistent with FileManager relative paths.
	 * @return false if the input strings are not recognizable as mountpoint strings, else true.
	 */
	bool TryNormalizePaths(FStringView InLongPackageName, FStringView InLocalPath,
		FString& OutLongPackageName, FString& OutLocalPathStandard, FString& OutLocalPathAbsolute, bool bSilenceLogs);
	bool TryNormalizeLongPackageNamePath(FStringView InLongPackageName, FString& OutLongPackageName,
		bool bSilenceLogs);
	bool TryNormalizeLocalPath(FStringView InLocalPath, FString& OutLocalPathStandard,
		FString& OutLocalPathAbsolute, bool bSilenceLogs);

	TRefCountPtr<UE::PackageName::IMountPoint> InsertMountPoint(const FString& RootPath, const FString& ContentPath);
	void InsertMountPoint(const TRefCountPtr<UE::PackageName::IMountPoint>& InMountPoint);

	/** This will remove a previously inserted mount point. */
	void RemoveMountPoint(const FString& RootPath, const FString& ContentPath);
	void RemoveMountPoint(const TRefCountPtr<UE::PackageName::IMountPoint>& InMountPoint);

	TRefCountPtr<UE::PackageName::IMountPoint> FindMountPointByRootPackageName(FStringView RootLongPackageName);
	TRefCountPtr<UE::PackageName::IMountPoint> FindMountPointByRootLocalPath(FStringView RootLocalPath);
	TRefCountPtr<UE::PackageName::IMountPoint> FindMountPoint(FStringView RootLongPackageName,
		FStringView RootLocalPath);
	TRefCountPtr<UE::PackageName::IMountPoint> FindOrAddMountPoint(FStringView RootLongPackageName,
		FStringView RootLocalPath);
	TRefCountPtr<UE::PackageName::IMountPoint> FindMountPointByChildLongPackageName(FStringView ChildLongPackageName);

	/** Checks whether the specific root path is a valid mount point. */
	bool MountPointExists(FStringView RootPath);

private:
#if !UE_BUILD_SHIPPING
	const FAutoConsoleCommand DumpMountPointsCommand;
	const FAutoConsoleCommand RegisterMountPointCommand;
	const FAutoConsoleCommand UnregisterMountPointCommand;
#endif
	
	FLongPackagePathsSingleton();
	~FLongPackagePathsSingleton();
	void ReassignMountOrders();
	void BroadcastMountPointMounted(const TRefCountPtr<UE::PackageName::IMountPoint>& PublicMountPoint,
		UE::PackageName::FMountPoint* MountPoint);
	bool TryRemoveMountPointInternal(const TRefCountPtr<UE::PackageName::IMountPoint>& InternallyHeldMountPoint,
		UE::TWriteScopeLock<FTransactionallySafeRWLock>& ProofOfWriteLock, UE::PackageName::FMountPoint* MountPoint);
	void BroadcastMountPointDismounted(const TRefCountPtr<UE::PackageName::IMountPoint>& PublicMountPoint,
		UE::PackageName::FMountPoint* MountPoint);

#if !UE_BUILD_SHIPPING
	void ExecDumpMountPoints(const TArray<FString>& Args);
	void ExecInsertMountPoint(const TArray<FString>& Args);
	void ExecRemoveMountPoint(const TArray<FString>& Args);
#endif
};