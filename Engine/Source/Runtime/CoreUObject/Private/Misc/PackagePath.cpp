// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/PackagePath.h"

#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/PackageName.h"
#include "Misc/PackageSegment.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/PackageResourceManager.h"

const TCHAR* LexToString(EPackageSegment PackageSegment)
{
	switch (PackageSegment)
	{
	case EPackageSegment::Header:
		return TEXT("Header");
	case EPackageSegment::Exports:
		return TEXT("Exports");
	case EPackageSegment::BulkDataDefault:
		return TEXT("BulkDataDefault");
	case EPackageSegment::BulkDataOptional:
		return TEXT("BulkDataOptional");
	case EPackageSegment::BulkDataMemoryMapped:
		return TEXT("BulkDataMemoryMapped");
	case EPackageSegment::PayloadSidecar:
		return TEXT("PayloadSidecar");
	default:
		checkNoEntry();
		return TEXT("");
	}
}

EPackageSegment ExtensionToSegment(EPackageExtension PackageExtension)
{
	switch (PackageExtension)
	{
	case EPackageExtension::Unspecified:
		return EPackageSegment::Header;
	case EPackageExtension::Asset:
		return EPackageSegment::Header;
	case EPackageExtension::Map:
		return EPackageSegment::Header;
	case EPackageExtension::TextAsset:
		return EPackageSegment::Header;
	case EPackageExtension::TextMap:
		return EPackageSegment::Header;
	case EPackageExtension::Custom:
		return EPackageSegment::Header;
	case EPackageExtension::EmptyString:
		return EPackageSegment::Header;
	case EPackageExtension::Exports:
		return EPackageSegment::Exports;
	case EPackageExtension::BulkDataDefault:
		return EPackageSegment::BulkDataDefault;
	case EPackageExtension::BulkDataOptional:
		return EPackageSegment::BulkDataDefault;
	case EPackageExtension::BulkDataMemoryMapped:
		return EPackageSegment::BulkDataDefault;
	case EPackageExtension::PayloadSidecar:
		return EPackageSegment::PayloadSidecar;
	default:
		checkNoEntry();
		return EPackageSegment::Header;
	}
}

EPackageExtension SegmentToExtension(EPackageSegment PackageSegment)
{
	switch (PackageSegment)
	{
	case EPackageSegment::Header:
		return EPackageExtension::Unspecified;
	case EPackageSegment::Exports:
		return EPackageExtension::Exports;
	case EPackageSegment::BulkDataDefault:
		return EPackageExtension::BulkDataDefault;
	case EPackageSegment::BulkDataOptional:
		return EPackageExtension::BulkDataOptional;
	case EPackageSegment::BulkDataMemoryMapped:
		return EPackageExtension::BulkDataMemoryMapped;
	case EPackageSegment::PayloadSidecar:
		return EPackageExtension::PayloadSidecar;
	default:
		checkNoEntry();
		return EPackageExtension::Unspecified;
	}
}

const TCHAR* LexToString(EPackageExtension PackageExtension)
{
	switch (PackageExtension)
	{
	case EPackageExtension::Unspecified:
		return TEXT("");
	case EPackageExtension::Asset:
		return TEXT(".uasset");
	case EPackageExtension::Map:
		return TEXT(".umap");
	case EPackageExtension::TextAsset:
		return TEXT(".utxt");
	case EPackageExtension::TextMap:
		return TEXT(".utxtmap");
	case EPackageExtension::Custom:
		return TEXT(".CustomExtension");
	case EPackageExtension::EmptyString:
		return TEXT("");
	case EPackageExtension::Exports:
		return TEXT(".uexp");
	case EPackageExtension::BulkDataDefault:
		return TEXT(".ubulk");
	case EPackageExtension::BulkDataOptional:
		return TEXT(".uptnl");
	case EPackageExtension::BulkDataMemoryMapped:
		return TEXT(".m.ubulk");
	case EPackageExtension::PayloadSidecar:
		return TEXT(".upayload");
	default:
		checkNoEntry();
		return TEXT("");
	}
}

EPackageExtension FPackagePath::ParseExtension(FStringView Filename, int32* OutExtensionStart)
{
	FStringView Extension = FPathViews::GetExtension(Filename, true /* bIncludeDot */);
	if (OutExtensionStart)
	{
		*OutExtensionStart = Filename.Len() - Extension.Len();
	}
	
	if (Extension.IsEmpty())
	{
		return EPackageExtension::Unspecified;
	}
	if (Extension.Equals(TEXTVIEW(".uasset"), ESearchCase::IgnoreCase))
	{
		return EPackageExtension::Asset;
	}
	if (Extension.Equals(TEXTVIEW(".umap"), ESearchCase::IgnoreCase))
	{
		return EPackageExtension::Map;
	}
	if (Extension.Equals(TEXTVIEW(".utxt"), ESearchCase::IgnoreCase))
	{
		return EPackageExtension::TextAsset;
	}
	if (Extension.Equals(TEXTVIEW(".utxtmap"), ESearchCase::IgnoreCase))
	{
		return EPackageExtension::TextMap;
	}
	if (Extension.Equals(TEXTVIEW(".uexp"), ESearchCase::IgnoreCase))
	{
		return EPackageExtension::Exports;
	}
	if (Extension.Equals(TEXTVIEW(".ubulk"), ESearchCase::IgnoreCase))
	{
		// .m.ubulk
		// .ubulk
		FStringView MemoryMappedExtension(TEXTVIEW(".m"));
		if (FPathViews::GetExtension(Filename.LeftChop(Extension.Len()), true /* bIncludeDot */).Equals(MemoryMappedExtension, ESearchCase::IgnoreCase))
		{
			if (OutExtensionStart)
			{
				*OutExtensionStart -= MemoryMappedExtension.Len();
			}
			return EPackageExtension::BulkDataMemoryMapped;
		}
		else
		{
			return EPackageExtension::BulkDataDefault;
		}
	}
	if (Extension.Equals(TEXTVIEW(".uptnl"), ESearchCase::IgnoreCase))
	{
		return EPackageExtension::BulkDataOptional;
	}
	if (Extension.Equals(TEXTVIEW(".upayload"), ESearchCase::IgnoreCase))
	{
		return EPackageExtension::PayloadSidecar;
	}
	return EPackageExtension::Custom;
}

const TCHAR* FPackagePath::GetOptionalSegmentExtensionModifier()
{
	return TEXT(".o");
}

const TCHAR* FPackagePath::GetExternalActorsFolderName()
{
	return TEXT("__ExternalActors__");
}

const TCHAR* FPackagePath::GetExternalObjectsFolderName()
{
	return TEXT("__ExternalObjects__");
}

FPackagePath::FPackagePath(const FPackagePath& Other)
{
	*this = Other;
}

void FPackagePath::Empty()
{
	*this = FPackagePath();
}

FPackagePath FPackagePath::FromPackageNameChecked(FStringView InPackageName)
{
	FPackagePath PackagePath;
	if (!TryFromPackageName(InPackageName, PackagePath))
	{
		UE_LOGF(LogPackageName, Error, "FromPackageNameChecked: Invalid LongPackageName \"%.*ls\"",
			InPackageName.Len(), InPackageName.GetData())
	}
	return PackagePath;
}

FPackagePath FPackagePath::FromPackageNameChecked(FName InPackageName)
{
	TStringBuilder<256> PackageNameString;
	InPackageName.AppendString(PackageNameString);
	return FromPackageNameChecked(PackageNameString);
}

FPackagePath FPackagePath::FromPackageNameChecked(const TCHAR* InPackageName)
{
	return FromPackageNameChecked(FStringView(InPackageName));
}

bool FPackagePath::TryFromPackageName(const TCHAR* InPackageName, FPackagePath& OutPackagePath)
{
	return TryFromPackageName(FStringView(InPackageName), OutPackagePath);
}

FPackagePath FPackagePath::FromLocalPath(FStringView InFilename)
{
	EPackageSegment PackageSegment;
	return FromLocalPath(InFilename, PackageSegment);
}

bool FPackagePath::operator!=(const FPackagePath& Other) const
{
	return !(*this == Other);
}

FString FPackagePath::GetPackageName() const
{
	TStringBuilder<256> Result;
	AppendPackageName(Result);
	return FString(Result);
}

FString FPackagePath::GetLocalFullPath() const
{
	return GetLocalFullPath(EPackageSegment::Header);
}

EPackageExtension FPackagePath::GetHeaderExtension() const
{
	return HeaderExtension;
}

EPackageExtension FPackagePath::GetExtension(EPackageSegment PackageSegment, FStringView& OutCustomExtension) const
{
	switch (PackageSegment)
	{
	case EPackageSegment::Header:
		OutCustomExtension = GetCustomExtension();
		return HeaderExtension;
	default:
		OutCustomExtension = FStringView();
		return SegmentToExtension(PackageSegment);
	}
}

FStringView FPackagePath::GetExtensionString(EPackageSegment PackageSegment) const
{
	switch (PackageSegment)
	{
	case EPackageSegment::Header:
		if (HeaderExtension == EPackageExtension::Custom)
		{
			return GetCustomExtension();
		}
		else
		{
			return LexToString(HeaderExtension);
		}
	default:
		return LexToString(SegmentToExtension(PackageSegment));
	}
}

namespace UE
{
namespace PackagePathPrivate
{
	EPackageExtension AllExtensions[] =
	{
		EPackageExtension::Unspecified,
		EPackageExtension::Asset,
		EPackageExtension::Map,
		EPackageExtension::TextAsset,
		EPackageExtension::TextMap,
		EPackageExtension::Custom,
		EPackageExtension::EmptyString,
		EPackageExtension::Exports,
		EPackageExtension::BulkDataDefault,
		EPackageExtension::BulkDataOptional,
		EPackageExtension::BulkDataMemoryMapped,
		EPackageExtension::PayloadSidecar,
	};
	static_assert(static_cast<int>(EPackageExtension::Unspecified) == 0 && EPackageExtensionCount == UE_ARRAY_COUNT(AllExtensions), "Need to add new extensions to AllExtensions array");
}
}

TConstArrayView<EPackageExtension> FPackagePath::GetPossibleExtensions(EPackageSegment PackageSegment) const
{
	FStringView OutCustomExtension;
	EPackageExtension Extension = GetExtension(PackageSegment, OutCustomExtension);
	if (Extension != EPackageExtension::Unspecified)
	{
		return TConstArrayView<EPackageExtension>(&UE::PackagePathPrivate::AllExtensions[static_cast<int>(Extension)], 1);
	}
	else
	{
#if WITH_TEXT_ARCHIVE_SUPPORT
		constexpr int NumHeaderSearchExtensions = 4;
#else
		constexpr int NumHeaderSearchExtensions = 2;
#endif
		static_assert(static_cast<int>(EPackageExtension::TextMap) == static_cast<int>(EPackageExtension::Asset) + 3, "Need to update the list of header extensions");
		return TConstArrayView<EPackageExtension>(&UE::PackagePathPrivate::AllExtensions[static_cast<int>(EPackageExtension::Asset)], NumHeaderSearchExtensions);
	}
}

FString FPackagePath::GetDebugName() const
{
	return GetDebugName(EPackageSegment::Header);
}

FText FPackagePath::GetDebugNameText() const
{
	return GetDebugNameText(EPackageSegment::Header);
}

FText FPackagePath::GetDebugNameText(EPackageSegment PackageSegment) const
{
	return FText::FromString(GetDebugName(PackageSegment));
}

FString FPackagePath::GetDebugNameWithExtension() const
{
	return GetDebugNameWithExtension(EPackageSegment::Header);
}

FString FPackagePath::GetDebugNameWithExtension(EPackageSegment PackageSegment) const
{
	FString Result = GetDebugName(EPackageSegment::Header);
	Result += GetExtensionString(PackageSegment);
	return Result;
}

#if UE_SUPPORT_FULL_PACKAGEPATH

FPackagePath& FPackagePath::operator=(const FPackagePath& Other)
{
	if (this == &Other)
	{
		return *this;
	}
	PathDataLen = Other.PathDataLen;
	PackageNameRootLen = Other.PackageNameRootLen;
	FilePathRootLen = Other.FilePathRootLen;
	ExtensionLen = Other.ExtensionLen;
	IdType = Other.IdType;
	HeaderExtension = Other.HeaderExtension;

	int32 StringDataLen = Other.GetStringDataLen();
	if (StringDataLen > 0)
	{
		StringData = MakeUnique<TCHAR[]>(StringDataLen);
		FMemory::Memcpy(StringData.Get(), Other.StringData.Get(), StringDataLen * sizeof(StringData.Get()[0]));
	}
	return *this;
}

bool FPackagePath::TryFromMountedName(FStringView InPackageNameOrFilePath, FPackagePath& OutPackagePath)
{
	TStringBuilder<64> PackageNameRoot;
	TStringBuilder<64> FilePathRoot;
	TStringBuilder<256> RelPath;
	TStringBuilder<64> ObjectName;
	TStringBuilder<16> CustomExtension;
	EPackageExtension Extension;
	if (FPackageName::TryConvertToMountedPathComponents(InPackageNameOrFilePath, PackageNameRoot, FilePathRoot,
		RelPath, ObjectName, Extension, CustomExtension))
	{
		if (ObjectName.Len() > 0)
		{
			UE_LOGF(LogPackageName, Warning,
				"FPackagePath::TryFromMountedName was passed an ObjectPath (%.*ls) rather than a PackageName or FilePath;" " it will be converted to the PackageName." " Accepting ObjectPaths is deprecated behavior and will be removed in a future release;" " TryFromMountedName will fail on ObjectPaths.",
					InPackageNameOrFilePath.Len(), InPackageNameOrFilePath.GetData());
		}
		OutPackagePath = FromMountedComponents(PackageNameRoot, FilePathRoot, RelPath, Extension, CustomExtension);
		return true;
	}

	return false;
}

bool FPackagePath::TryFromPackageName(FStringView InPackageName, FPackagePath& OutPackagePath)
{
	if (!FPackageName::IsValidTextForLongPackageName(InPackageName))
	{
		return false;
	}
	OutPackagePath.IdType = EPackageIdType::PackageOnlyPath;
	OutPackagePath.SetStringData(InPackageName, FStringView(), FStringView(), FStringView());
	return true;
}

bool FPackagePath::TryFromPackageName(FName InPackageName, FPackagePath& OutPackagePath)
{
	TCHAR Buffer[FName::StringBufferSize];
	FStringView PackageName(Buffer, InPackageName.ToString(Buffer));
	return TryFromPackageName(PackageName, OutPackagePath);
}

FPackagePath FPackagePath::FromPackageNameUnchecked(FName InPackageName)
{
	FPackagePath PackagePath;
	TStringBuilder<256> PackageNameString;
	InPackageName.AppendString(PackageNameString);
	PackagePath.IdType = EPackageIdType::PackageOnlyPath;
	PackagePath.SetStringData(PackageNameString, FStringView(), FStringView(), FStringView());
	return PackagePath;
}

FPackagePath FPackagePath::FromLocalPath(FStringView InFilename, EPackageSegment& PackageSegment)
{
	FPackagePath PackagePath;
	PackagePath.IdType = EPackageIdType::LocalOnlyPath;
	int32 ExtensionStart;
	EPackageExtension Extension = ParseExtension(InFilename, &ExtensionStart);
	FStringView CustomExtension;
	if (Extension == EPackageExtension::Custom)
	{
		CustomExtension = InFilename.RightChop(ExtensionStart);
	}
	PackageSegment = ExtensionToSegment(Extension);
	if (PackageSegment != EPackageSegment::Header)
	{
		Extension = EPackageExtension::Unspecified;
		CustomExtension = FStringView();
	}

	FString Filename(InFilename.Left(ExtensionStart));
	FPaths::NormalizeFilename(Filename);

	PackagePath.SetStringData(Filename, FStringView(), FStringView(), CustomExtension);
	PackagePath.HeaderExtension = Extension;
	return PackagePath;
}

FPackagePath FPackagePath::FromMountedComponents(FStringView PackageNameRoot, FStringView FilePathRoot,
	FStringView RelPath, EPackageExtension InExtension, FStringView InCustomExtension)
{
	if (ExtensionToSegment(InExtension) != EPackageSegment::Header)
	{
		InExtension = EPackageExtension::Unspecified;
		InCustomExtension = FStringView();
	}

	FPackagePath PackagePath;
	PackagePath.IdType = EPackageIdType::MountedPath;
	PackagePath.SetStringData(RelPath, PackageNameRoot, FilePathRoot, InCustomExtension);
	PackagePath.HeaderExtension = InExtension;
	return PackagePath;
}

bool FPackagePath::TryMatchCase(const FPackagePath& SourcePackagePath, FStringView FilePathToMatch,
	FPackagePath& OutPackagePath)
{
	switch (SourcePackagePath.IdType)
	{
	case EPackageIdType::Empty:
		return false;
	case EPackageIdType::MountedPath:
	{
		FString RootPath = FPaths::ConvertRelativePathToFull(FString(SourcePackagePath.GetFilePathRoot()));
		FString RelativePathOnDisk = FPaths::ConvertRelativePathToFull(
			FString(FPathViews::GetBaseFilenameWithPath(FilePathToMatch)));

		if (!FPaths::MakePathRelativeTo(RelativePathOnDisk, *RootPath) ||
			!FStringView(RelativePathOnDisk).Equals(SourcePackagePath.GetPathData(), ESearchCase::IgnoreCase))
		{
			return false;
		}
		// Note that OutPackagePath might be the same as SourcePackagePath, so create a temporary instead of
		// assigning it directly to ensure that the fields we are reading remain available until assignment is done
		FPackagePath Result = FromMountedComponents(SourcePackagePath.GetPackageNameRoot(),
			SourcePackagePath.GetFilePathRoot(), RelativePathOnDisk,
			SourcePackagePath.GetHeaderExtension(), SourcePackagePath.GetCustomExtension());
		OutPackagePath = MoveTemp(Result);
		return true;
	}
	case EPackageIdType::PackageOnlyPath:
	{
		if (!SourcePackagePath.TryConvertToMounted())
		{
			return false;
		}
		return TryMatchCase(SourcePackagePath, FilePathToMatch, OutPackagePath);
	}
	case EPackageIdType::LocalOnlyPath:
	{
		FString SourceBasePath = FPaths::ConvertRelativePathToFull(SourcePackagePath.GetLocalBaseFilenameWithPath());
		FString TargetBasePath = FPaths::ConvertRelativePathToFull(
			FString(FPathViews::GetBaseFilenameWithPath(FilePathToMatch)));
		if (!SourceBasePath.Equals(TargetBasePath, ESearchCase::IgnoreCase))
		{
			return false;
		}
		// Note that OutPackagePath might be the same as SourcePackagePath, so create a temporary instead of
		// assigning it directly to ensure that the fields we are reading remain available until assignment is done
		FPackagePath Result = FromLocalPath(TargetBasePath);
		Result.SetHeaderExtension(SourcePackagePath.HeaderExtension, SourcePackagePath.GetCustomExtension());
		OutPackagePath = MoveTemp(Result);
		return true;
	}
	default:
		checkNoEntry();
		return false;
	}
}

bool FPackagePath::operator==(const FPackagePath& Other) const
{
	// Note that HeaderExtension is not required for equality
	auto CompareByLocalPath = [](const FPackagePath& A, const FPackagePath& B)
	{
		FString ABaseFilenameWithPath = FPaths::ConvertRelativePathToFull(A.GetLocalBaseFilenameWithPath());
		FString BBaseFilenameWithPath = FPaths::ConvertRelativePathToFull(B.GetLocalBaseFilenameWithPath());
		return ABaseFilenameWithPath.Equals(BBaseFilenameWithPath, ESearchCase::IgnoreCase);
	};
	auto CompareByPackageName = [](const FPackagePath& A, const FPackagePath& B)
	{
		TStringBuilder<256> AName;
		TStringBuilder<256> BName;
		A.AppendPackageName(AName);
		B.AppendPackageName(BName);
		return FStringView(AName).Equals(FStringView(BName), ESearchCase::IgnoreCase);
	};

	switch (IdType)
	{
	case EPackageIdType::Empty:
		return Other.IsEmpty();
	case EPackageIdType::MountedPath:
		switch (Other.IdType)
		{
		case EPackageIdType::Empty:
			return false;
		case EPackageIdType::MountedPath:
			return CompareByPackageName(*this, Other);
		case EPackageIdType::PackageOnlyPath:
			return CompareByPackageName(*this, Other);
		case EPackageIdType::LocalOnlyPath:
			return CompareByLocalPath(*this, Other);
		default:
			checkNoEntry();
			return false;
		}
	case EPackageIdType::PackageOnlyPath:
		switch (Other.IdType)
		{
		case EPackageIdType::Empty:
			return false;
		case EPackageIdType::MountedPath:
			return CompareByPackageName(*this, Other);
		case EPackageIdType::PackageOnlyPath:
			return CompareByPackageName(*this, Other);
		case EPackageIdType::LocalOnlyPath:
			if (!TryConvertToMounted() || !Other.TryConvertToMounted())
			{
				return false;
			}
			return CompareByPackageName(*this, Other);
		default:
			checkNoEntry();
			return false;
		}
	case EPackageIdType::LocalOnlyPath:
		switch (Other.IdType)
		{
		case EPackageIdType::Empty:
			return false;
		case EPackageIdType::MountedPath:
			return CompareByLocalPath(*this, Other);
		case EPackageIdType::PackageOnlyPath:
			if (!TryConvertToMounted() || !Other.TryConvertToMounted())
			{
				return false;
			}
			return CompareByPackageName(*this, Other);
		case EPackageIdType::LocalOnlyPath:
			return CompareByLocalPath(*this, Other);
		default:
			checkNoEntry();
			return false;
		}
	default:
		checkNoEntry();
		return false;
	}
}

FArchive& operator<<(FArchive& Ar, FPackagePath& PackagePath)
{
	checkf(!Ar.IsPersistent(), TEXT("PackagePath is transient and does not support serialization to Persistent storage."));
	uint8 IntIdType = static_cast<uint8>(PackagePath.IdType);
	uint8 IntHeaderExtension = static_cast<uint8>(PackagePath.HeaderExtension);
	TUniquePtr<TCHAR[]> StringData = nullptr;

	Ar << IntIdType;
	Ar << IntHeaderExtension;
	Ar << PackagePath.PathDataLen;
	Ar << PackagePath.PackageNameRootLen;
	Ar << PackagePath.FilePathRootLen;
	Ar << PackagePath.ExtensionLen;

	const int32 StringDataLen = PackagePath.GetStringDataLen();
	if (Ar.IsLoading())
	{
		PackagePath.IdType = static_cast<FPackagePath::EPackageIdType>(IntIdType);
		PackagePath.HeaderExtension = static_cast<EPackageExtension>(IntHeaderExtension);
		if (StringDataLen > 0)
		{
			FString SerializerString;
			Ar << SerializerString;
			PackagePath.StringData.Reset(new TCHAR[StringDataLen]);
			FMemory::Memcpy(PackagePath.StringData.Get(), *SerializerString, StringDataLen * sizeof(SerializerString[0]));
		}
		else
		{
			PackagePath.StringData.Reset();
		}
	}
	else
	{
		if (StringDataLen > 0)
		{
			FString SerializerString = FString::ConstructFromPtrSize(PackagePath.StringData.Get(), StringDataLen);
			Ar << SerializerString;
		}
	}
	return Ar;
}

bool FPackagePath::IsEmpty() const
{
	return IdType == EPackageIdType::Empty;
}

bool FPackagePath::IsMountedPath() const
{
	switch (IdType)
	{
	case EPackageIdType::Empty:
		return false;
	case EPackageIdType::MountedPath:
		return true;
	case EPackageIdType::PackageOnlyPath:
		return TryConvertToMounted();
	case EPackageIdType::LocalOnlyPath:
		return TryConvertToMounted();
	default:
		checkNoEntry();
		return false;
	}
}

bool FPackagePath::HasPackageName() const
{
	switch (IdType)
	{
	case EPackageIdType::Empty:
		return false;
	case EPackageIdType::MountedPath:
		return true;
	case EPackageIdType::PackageOnlyPath:
		return true;
	case EPackageIdType::LocalOnlyPath:
		return TryConvertToMounted();
	default:
		checkNoEntry();
		return false;
	}
}

bool FPackagePath::HasLocalPath() const
{
	switch (IdType)
	{
	case EPackageIdType::Empty:
		return false;
	case EPackageIdType::MountedPath:
		return true;
	case EPackageIdType::PackageOnlyPath:
		return TryConvertToMounted();
	case EPackageIdType::LocalOnlyPath:
		return true;
	default:
		checkNoEntry();
		return false;
	}
}

void FPackagePath::AppendPackageName(FStringBuilderBase& Builder) const
{
	switch (IdType)
	{
	case EPackageIdType::Empty:
		return;
	case EPackageIdType::MountedPath:
		Builder << GetPackageNameRoot() << GetPathData();
		return;
	case EPackageIdType::PackageOnlyPath:
		Builder << GetPathData();
		return;
	case EPackageIdType::LocalOnlyPath:
		if (!TryConvertToMounted())
		{
			return;
		}
		AppendPackageName(Builder);
		return;
	default:
		checkNoEntry();
		return;
	}
}

FName FPackagePath::GetPackageFName() const
{
	TStringBuilder<256> Builder;
	AppendPackageName(Builder);
	if (Builder.Len() == 0)
	{
		return NAME_None;
	}
	return FName(Builder);
}

FString FPackagePath::GetLocalFullPath(EPackageSegment PackageSegment) const
{
	TStringBuilder<256> Result;
	AppendLocalFullPath(Result, PackageSegment);
	return FString(Result);
}

void FPackagePath::AppendLocalFullPath(FStringBuilderBase& Builder) const
{
	AppendLocalFullPath(Builder, EPackageSegment::Header);
}

void FPackagePath::AppendLocalFullPath(FStringBuilderBase& Builder, EPackageSegment PackageSegment) const
{
	const int32 BeforeLen = Builder.Len();
	AppendLocalBaseFilenameWithPath(Builder);
	if (Builder.Len() == BeforeLen)
	{
		return;
	}

	FStringView UnusedCustomExtension;
	EPackageExtension Extension = GetExtension(PackageSegment, UnusedCustomExtension);
	if (Extension == EPackageExtension::Custom)
	{
		Builder << GetCustomExtension();
	}
	else
	{
		if (Extension == EPackageExtension::Unspecified)
		{
			check(PackageSegment == EPackageSegment::Header);
			FPackagePath FoundPackage;
			if (FPackageName::DoesPackageExist(*this, &FoundPackage))
			{
				Extension = FoundPackage.GetHeaderExtension();
				// DoesPackageExist should not search for files with non-standard extensions, so we don't handle CustomExtensions here
				check(Extension != EPackageExtension::Custom);

				// Specify this path's Extension now that we know it
				SetHeaderExtension(Extension);
			}
			else
			{
				UE_LOGF(LogPackageName, Warning,
					"GetLocalFullPath called on FPackagePath %ls which has an unspecified header extension," " and the path does not exist on disk. Assuming EPackageExtension::Asset.",
					Builder.ToString() + BeforeLen); // Can't use GetDebugName since that can call this function recursively
				// This warning is a bug in the calling code so we need a callstack to diagnose. But only write it on
				// the first occurrence to prevent spam.
				UE_CALL_ONCE([]()
					{
						FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
					});
				Extension = EPackageExtension::Asset;
			}
		}
		Builder << LexToString(Extension);
	}
}

FString FPackagePath::GetLocalBaseFilenameWithPath() const
{
	TStringBuilder<256> Result;
	AppendLocalBaseFilenameWithPath(Result);
	return FString(Result);
}

void FPackagePath::AppendLocalBaseFilenameWithPath(FStringBuilderBase& Builder) const
{
	switch (IdType)
	{
	case EPackageIdType::Empty:
		return;
	case EPackageIdType::MountedPath:
		Builder << GetFilePathRoot() << GetPathData();
		return;
	case EPackageIdType::PackageOnlyPath:
		if (!TryConvertToMounted())
		{
			return;
		}
		AppendLocalBaseFilenameWithPath(Builder);
		return;
	case EPackageIdType::LocalOnlyPath:
		Builder << GetPathData();
		return;
	default:
		checkNoEntry();
		return;
	}
}

FString FPackagePath::GetDebugName(EPackageSegment PackageSegment) const
{
	FString Result;
	switch (IdType)
	{
	case EPackageIdType::MountedPath:
		Result = GetPackageName();
		break;
	case EPackageIdType::PackageOnlyPath:
		Result = GetPackageName();
		break;
	case EPackageIdType::LocalOnlyPath:
		Result = GetLocalFullPath();
		break;
	case EPackageIdType::Empty:
		Result = FString();
		break;
	default:
		// Don't assert here, since this function can be called in the watch window on possibly invalid data
		Result = TEXT("InvalidPackagePathIdType");
		break;
	}
	if (PackageSegment != EPackageSegment::Header)
	{
		Result += FString::Printf(TEXT("(%s)"), LexToString(PackageSegment));
	}
	return Result;
}

FString FPackagePath::GetPackageNameOrFallback() const
{
	switch (IdType)
	{
	case EPackageIdType::Empty:
		return TEXT("");
	case EPackageIdType::MountedPath:
		return GetPackageName();
	case EPackageIdType::PackageOnlyPath:
		return GetPackageName();
	case EPackageIdType::LocalOnlyPath:
		return GetLocalFullPath();
	default:
		checkNoEntry();
		return TEXT("");
	}
}

void FPackagePath::SetHeaderExtension(EPackageExtension Extension, FStringView CustomExtension) const
{
	if (IsEmpty())
	{
		return;
	}
	check(ExtensionToSegment(Extension) == EPackageSegment::Header);
	if (Extension == EPackageExtension::Custom)
	{
		SetStringData(GetPathData(), GetPackageNameRoot(), GetFilePathRoot(), CustomExtension);
	}
	HeaderExtension = Extension;
}

FStringView FPackagePath::GetPathData() const
{
	return FStringView(StringData.Get(), PathDataLen);
}

FStringView FPackagePath::GetPackageNameRoot() const
{
	return FStringView(StringData.Get() + PathDataLen, PackageNameRootLen);
}

FStringView FPackagePath::GetFilePathRoot() const
{
	return FStringView(StringData.Get() + PathDataLen + PackageNameRootLen, FilePathRootLen);
}

FStringView FPackagePath::GetCustomExtension() const
{
	return FStringView(StringData.Get() + PathDataLen + PackageNameRootLen + FilePathRootLen, ExtensionLen);
}

void FPackagePath::SetStringData(FStringView PathData, FStringView PackageNameRoot, FStringView FilePathRoot, FStringView CustomExtension) const
{
	checkf(PathData.Len() <= TNumericLimits<decltype(PathDataLen)>::Max(),
		TEXT("Maximum length for FPackagePath::PathData is %d"),
		TNumericLimits<decltype(PathDataLen)>::Max());
	checkf(PackageNameRoot.Len() <= TNumericLimits<decltype(PackageNameRootLen)>::Max(),
		TEXT("Maximum length for FPackagePath::PackageNameRoot is %d"),
		TNumericLimits<decltype(PackageNameRootLen)>::Max());
	checkf(FilePathRoot.Len() <= TNumericLimits<decltype(FilePathRootLen)>::Max(),
		TEXT("Maximum length for FPackagePath::FilePathRoot is %d"), TNumericLimits<decltype(FilePathRootLen)>::Max());
	checkf(CustomExtension.Len() <= TNumericLimits<decltype(ExtensionLen)>::Max(),
		TEXT("Maximum length for FPackagePath::CustomExtension is %d"), TNumericLimits<decltype(ExtensionLen)>::Max());
	PathDataLen = static_cast<uint16>(PathData.Len());
	PackageNameRootLen = static_cast<uint16>(PackageNameRoot.Len());
	FilePathRootLen = static_cast<uint16>(FilePathRoot.Len());
	ExtensionLen = static_cast<uint16>(CustomExtension.Len());

	int32 StringDataLen = GetStringDataLen();
	if (StringDataLen != 0)
	{
		// Make sure we handle the input views being views into the current StringData;
		// copy them to the new StringData before deleting the old
		TUniquePtr<TCHAR[]> NewStringData(new TCHAR[StringDataLen]);
		TCHAR* StringPtr = NewStringData.Get();
		int32 Offset = 0;
		FMemory::Memcpy(StringPtr + Offset, PathData.GetData(), PathDataLen * sizeof(PathData[0]));
		Offset += PathDataLen;
		FMemory::Memcpy(StringPtr + Offset, PackageNameRoot.GetData(), PackageNameRootLen * sizeof(PackageNameRoot[0]));
		Offset += PackageNameRootLen;
		FMemory::Memcpy(StringPtr + Offset, FilePathRoot.GetData(), FilePathRootLen * sizeof(FilePathRoot[0]));
		Offset += FilePathRootLen;
		FMemory::Memcpy(StringPtr + Offset, CustomExtension.GetData(), ExtensionLen * sizeof(CustomExtension[0]));
		StringData = MoveTemp(NewStringData);		
	}
	else
	{
		StringData.Reset();
	}
}

int32 FPackagePath::GetStringDataLen() const
{
	return static_cast<int32>(PathDataLen + PackageNameRootLen + FilePathRootLen + ExtensionLen);
}

bool FPackagePath::TryConvertToMounted() const
{
	FPackagePath NewPath;
	switch (IdType)
	{
	case EPackageIdType::Empty:
		return false;
	case EPackageIdType::MountedPath:
		return true;
	case EPackageIdType::PackageOnlyPath:
	{
		if (!TryFromMountedName(GetPathData(), NewPath))
		{
			return false;
		}
		break;
	}
	case EPackageIdType::LocalOnlyPath:
	{
		if (!TryFromMountedName(GetPathData(), NewPath))
		{
			return false;
		}
		break;
	}
	default:
		checkNoEntry();
		return false;
	}

	IdType = EPackageIdType::MountedPath;
	SetStringData(NewPath.GetPathData(), NewPath.GetPackageNameRoot(), NewPath.GetFilePathRoot(), GetCustomExtension());
	return true;
}

#else

FPackagePath& FPackagePath::operator=(const FPackagePath& Other)
{
	PackageName = Other.PackageName;
	HeaderExtension = Other.HeaderExtension;
	return *this;
}

bool FPackagePath::TryFromMountedName(FStringView InPackageNameOrHeaderFilePath, FPackagePath& OutPackagePath)
{
	if (FPackageName::IsValidLongPackageName(InPackageNameOrHeaderFilePath))
	{
		OutPackagePath.PackageName = FName(InPackageNameOrHeaderFilePath);
	}
	else
	{
		OutPackagePath.PackageName = FName(FPackageName::FilenameToLongPackageName(FString(InPackageNameOrHeaderFilePath)));
	}
	return true;
}

bool FPackagePath::TryFromPackageName(FStringView InPackageName, FPackagePath& OutPackagePath)
{
	if (!FPackageName::IsValidTextForLongPackageName(InPackageName))
	{
		return false;
	}
	OutPackagePath.PackageName = FName(InPackageName);
	return true;
}

bool FPackagePath::TryFromPackageName(FName InPackageName, FPackagePath& OutPackagePath)
{
	TStringBuilder<256> PackageNameString;
	InPackageName.AppendString(PackageNameString);
	if (!FPackageName::IsValidTextForLongPackageName(PackageNameString))
	{
		return false;
	}
	OutPackagePath.PackageName = InPackageName;
	return true;
}

FPackagePath FPackagePath::FromPackageNameUnchecked(FName InPackageName)
{
	FPackagePath PackagePath;
	PackagePath.PackageName = InPackageName;
	return PackagePath;
}

FPackagePath FPackagePath::FromLocalPath(FStringView InFilename, EPackageSegment& OutPackageSegment)
{
	int32 ExtensionStart;
	EPackageExtension Extension = ParseExtension(InFilename, &ExtensionStart);
	OutPackageSegment = ExtensionToSegment(Extension);
	if (OutPackageSegment != EPackageSegment::Header)
	{
		Extension = EPackageExtension::Unspecified;
	}
	FString FileNameString(InFilename.Left(ExtensionStart));
	FString PackageNameString;
	if (!FPackageName::TryConvertFilenameToLongPackageName(FileNameString, PackageNameString))
	{
		UE_LOGF(LogPackageName, Error, "FromLocalPath: Failed converting filename \"%ls\" to package name", *FileNameString);
	}
	FPackagePath PackagePath;
	PackagePath.PackageName = FName(PackageNameString);
	PackagePath.HeaderExtension = Extension;
	return PackagePath;
}

FPackagePath FPackagePath::FromMountedComponents(FStringView PackageNameRoot, FStringView FilePathRoot, FStringView RelPath,
	EPackageExtension InExtension, FStringView InCustomExtension)
{
	if (ExtensionToSegment(InExtension) != EPackageSegment::Header)
	{
		InExtension = EPackageExtension::Unspecified;
		InCustomExtension = FStringView();
	}
	if (!InCustomExtension.IsEmpty())
	{
		checkNoEntry();
	}
	TStringBuilder<256> FileNameString;
	FileNameString.Append(FilePathRoot);
	FileNameString.Append(RelPath);

	FString PackageNameString;
	if (!FPackageName::TryConvertFilenameToLongPackageName(FString(FileNameString), PackageNameString))
	{
		UE_LOGF(LogPackageName, Error, "FromMountedComponents: Invalid FileName \"%ls\"", *FileNameString);
	}
	FPackagePath PackagePath;
	PackagePath.PackageName = FName(PackageNameString);
	PackagePath.HeaderExtension = InExtension;
	return PackagePath;
}

bool FPackagePath::TryMatchCase(const FPackagePath& SourcePackagePath, FStringView FilePathToMatch, FPackagePath& OutPackagePath)
{
	return true;
}

bool FPackagePath::operator==(const FPackagePath& Other) const
{
	return Other.PackageName == PackageName;
}

FArchive& operator<<(FArchive& Ar, FPackagePath& PackagePath)
{
	unimplemented();
	return Ar;
}

bool FPackagePath::IsEmpty() const
{
	return PackageName.IsNone();
}

bool FPackagePath::IsMountedPath() const
{
	return true;
}

void FPackagePath::AppendPackageName(FStringBuilderBase& Builder) const
{
	PackageName.AppendString(Builder);
}

FName FPackagePath::GetPackageFName() const
{
	return PackageName;
}

FString FPackagePath::GetLocalFullPath(EPackageSegment PackageSegment) const
{
	if (!PackageName.IsNone())
	{
		FString PackageNameString = PackageName.ToString();
		EPackageExtension Extension = PackageSegment == EPackageSegment::Header ? HeaderExtension : SegmentToExtension(PackageSegment);
		FString Result;
		if (FPackageName::TryConvertLongPackageNameToFilename(PackageNameString, Result, LexToString(Extension)))
		{
			return Result;
		}
		else
		{
			UE_LOGF(LogPackageName, Error, "GetLocalFullPath: Failed converting package name \"%ls\" to file name", *PackageNameString);
		}
	}
	return TEXT("");
}

FString FPackagePath::GetLocalBaseFilenameWithPath() const
{
	if (!PackageName.IsNone())
	{
		FString PackageNameString = PackageName.ToString();
		FString Result;
		if (FPackageName::TryConvertLongPackageNameToFilename(PackageNameString, Result))
		{
			return Result;
		}
		else
		{
			UE_LOGF(LogPackageName, Error, "GetLocalBaseFilenameWithPath: Failed converting package name \"%ls\" to file name", *PackageNameString);
		}
	}
	return TEXT("");
}

FString FPackagePath::GetDebugName(EPackageSegment PackageSegment) const
{
	return PackageName.ToString();
}

FString FPackagePath::GetPackageNameOrFallback() const
{
	return GetPackageName();
}

void FPackagePath::SetHeaderExtension(EPackageExtension Extension, FStringView CustomExtension) const
{
	if (!CustomExtension.IsEmpty())
	{
		checkNoEntry();
	}
	HeaderExtension = Extension;
}

FStringView FPackagePath::GetCustomExtension() const
{
	return FStringView();
}

#endif //UE_SUPPORT_FULL_PACKAGEPATH
