// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS

#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"

#include "Containers/StringView.h"
#include "Misc/StringBuilder.h"

class FPackagePathTest : FAutomationTestBase
{
private:
	TStringBuilder<256> ErrorText;

public:
	FPackagePathTest(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

	template <typename FmtType, typename... Types>
	const TCHAR* Errorf(const FmtType&  Format, Types... Args)
	{
		ErrorText.Reset();
		ErrorText.Appendf(Format, Forward<Types>(Args)...);
		return ErrorText.ToString();
	}

	bool RunTest(const FString& Parameters)
	{
		RunTestParseExtension();
		RunTestIsPackageExtension();
		return !HasAnyErrors();
	}

	void RunTestParseExtension()
	{
		auto TestExtension = [this](EPackageExtension Extension)
		{
			TStringBuilder<64> Path;
			const FStringView PathWithoutExtension = TEXTVIEW("d:\\Root\\Dir\\Filename");
			Path << PathWithoutExtension << LexToString(Extension);
			int32 ExtensionStart;
			EPackageExtension ParsedExtension = FPackagePath::ParseExtension(Path, &ExtensionStart);
			TestEqual(Errorf(TEXT("ParsedExtension returns correct extension for %s"), LexToString(Extension)), ParsedExtension, Extension);
			TestEqual(Errorf(TEXT("ParsedExtension sets correct ExtensionStart for %s"), LexToString(Extension)), ExtensionStart, PathWithoutExtension.Len());
		};
		static_assert(EPackageExtensionCount == 12, "Need to add extensions to the test");
		TestExtension(EPackageExtension::Unspecified);
		TestExtension(EPackageExtension::Asset);
		TestExtension(EPackageExtension::Map);
		TestExtension(EPackageExtension::TextAsset);
		TestExtension(EPackageExtension::TextMap);
		//TestExtension(EPackageExtension::Custom); // Custom requires a separate test
		//TestExtension(EPackageExtension::EmptyString); // Parse never returns EmptyString; it returns Unspecified instead
		TestExtension(EPackageExtension::Exports);
		TestExtension(EPackageExtension::BulkDataDefault);
		TestExtension(EPackageExtension::BulkDataOptional);
		TestExtension(EPackageExtension::BulkDataMemoryMapped);
		TestExtension(EPackageExtension::PayloadSidecar);

		// Test Custom
		{
			TStringBuilder<64> Path;
			const FStringView PathWithoutExtension = TEXTVIEW("d:\\Root\\Dir\\Filename");
			Path << PathWithoutExtension << TEXT(".SomeOtherExtension");
			int32 ExtensionStart;
			EPackageExtension ParsedExtension = FPackagePath::ParseExtension(Path, &ExtensionStart);
			TestEqual(Errorf(TEXT("ParsedExtension returns correct extension for %s"), LexToString(EPackageExtension::Custom)), ParsedExtension, EPackageExtension::Custom);
			TestEqual(Errorf(TEXT("ParsedExtension sets correct ExtensionStart for %s"), LexToString(EPackageExtension::Custom)), ExtensionStart, PathWithoutExtension.Len());
		}
	}

	void RunTestIsPackageExtension()
	{
		TestTrue(TEXT("IsPackageExtension returns true for .uasset"), FPackageName::IsPackageExtension(TEXT(".uasset")));
		TestTrue(TEXT("IsAssetPackageExtension returns true for .uasset"), FPackageName::IsAssetPackageExtension(TEXT(".uasset")));
		TestFalse(TEXT("IsMapPackageExtension returns false for .uasset"), FPackageName::IsMapPackageExtension(TEXT(".uasset")));
		TestTrue(TEXT("IsPackageExtension returns true for uasset"), FPackageName::IsPackageExtension(TEXT("uasset")));
		TestTrue(TEXT("IsPackageExtension returns true for .umap"), FPackageName::IsPackageExtension(TEXT(".umap")));
		TestFalse(TEXT("IsAssetPackageExtension returns false for .umap"), FPackageName::IsAssetPackageExtension(TEXT(".umap")));
		TestTrue(TEXT("IsMapPackageExtension returns true for .umap"), FPackageName::IsMapPackageExtension(TEXT(".umap")));
		TestTrue(TEXT("IsPackageExtension returns true for umap"), FPackageName::IsPackageExtension(TEXT("umap")));
		TestFalse(TEXT("IsPackageExtension returns false for .set"), FPackageName::IsPackageExtension(TEXT(".set")));
		TestFalse(TEXT("IsPackageExtension returns false for set"), FPackageName::IsPackageExtension(TEXT("set")));
		TestFalse(TEXT("IsPackageExtension returns false for .ap"), FPackageName::IsPackageExtension(TEXT(".set")));
		TestFalse(TEXT("IsPackageExtension returns false for ap"), FPackageName::IsPackageExtension(TEXT("ap")));
		TestFalse(TEXT("IsPackageExtension returns false for .uassetWarglBargl"), FPackageName::IsPackageExtension(TEXT(".uassetWarglBargl")));
		TestFalse(TEXT("IsPackageExtension returns false for uassetWarglBargl"), FPackageName::IsPackageExtension(TEXT("uassetWarglBargl")));
		TestFalse(TEXT("IsPackageExtension returns false for .umapBarglWargl"), FPackageName::IsPackageExtension(TEXT(".umapBarglWargl")));
		TestFalse(TEXT("IsPackageExtension returns false for umapBarglWargl"), FPackageName::IsPackageExtension(TEXT("umapBarglWargl")));
		TestFalse(TEXT("IsPackageExtension returns false for .utxt"), FPackageName::IsPackageExtension(TEXT(".utxt")));
		TestFalse(TEXT("IsPackageExtension returns false for utxt"), FPackageName::IsPackageExtension(TEXT("utxt")));
		TestFalse(TEXT("IsPackageExtension returns false for .utxtmap"), FPackageName::IsPackageExtension(TEXT(".utxtmap")));
		TestFalse(TEXT("IsPackageExtension returns false for utxtmap"), FPackageName::IsPackageExtension(TEXT("utxtmap")));
		TestFalse(TEXT("IsPackageExtension returns false for .ubulk"), FPackageName::IsPackageExtension(TEXT(".ubulk")));
		TestFalse(TEXT("IsPackageExtension returns false for ubulk"), FPackageName::IsPackageExtension(TEXT("ubulk")));
		TestFalse(TEXT("IsPackageExtension returns false for empty string"), FPackageName::IsPackageExtension(TEXT("")));
		TestFalse(TEXT("IsPackageExtension returns false for \".\""), FPackageName::IsPackageExtension(TEXT(".")));

		TestTrue(TEXT("IsTextPackageExtension returns true for .utxt"), FPackageName::IsTextPackageExtension(TEXT(".utxt")));
		TestTrue(TEXT("IsTextAssetPackageExtension returns true for .utxt"), FPackageName::IsTextAssetPackageExtension(TEXT(".utxt")));
		TestFalse(TEXT("IsTextMapPackageExtension returns false for .utxt"), FPackageName::IsTextMapPackageExtension(TEXT(".utxt")));
		TestTrue(TEXT("IsTextPackageExtension returns true for utxt"), FPackageName::IsTextPackageExtension(TEXT("utxt")));
		TestTrue(TEXT("IsTextPackageExtension returns true for .utxtmap"), FPackageName::IsTextPackageExtension(TEXT(".utxtmap")));
		TestFalse(TEXT("IsTextAssetPackageExtension returns false for .utxtmap"), FPackageName::IsTextAssetPackageExtension(TEXT(".utxtmap")));
		TestTrue(TEXT("IsTextMapPackageExtension returns true for .utxtmap"), FPackageName::IsTextMapPackageExtension(TEXT(".utxtmap")));
		TestTrue(TEXT("IsTextPackageExtension returns true for utxtmap"), FPackageName::IsTextPackageExtension(TEXT("utxtmap")));
		TestFalse(TEXT("IsTextPackageExtension returns false for .xt"), FPackageName::IsPackageExtension(TEXT(".xt")));
		TestFalse(TEXT("IsTextPackageExtension returns false for xt"), FPackageName::IsPackageExtension(TEXT("xt")));
		TestFalse(TEXT("IsTextPackageExtension returns false for .map"), FPackageName::IsPackageExtension(TEXT(".map")));
		TestFalse(TEXT("IsTextPackageExtension returns false for map"), FPackageName::IsPackageExtension(TEXT("map")));
		TestFalse(TEXT("IsTextPackageExtension returns false for .utxtWarglBargl"), FPackageName::IsPackageExtension(TEXT(".utxtWarglBargl")));
		TestFalse(TEXT("IsTextPackageExtension returns false for utxtWarglBargl"), FPackageName::IsPackageExtension(TEXT("utxtWarglBargl")));
		TestFalse(TEXT("IsTextPackageExtension returns false for .utxtmapBarglWargl"), FPackageName::IsPackageExtension(TEXT(".utxtmapBarglWargl")));
		TestFalse(TEXT("IsTextPackageExtension returns false for utxtmapBarglWargl"), FPackageName::IsPackageExtension(TEXT("utxtmapBarglWargl")));
		TestFalse(TEXT("IsTextPackageExtension returns false for .uasset"), FPackageName::IsTextPackageExtension(TEXT(".uasset")));
		TestFalse(TEXT("IsTextPackageExtension returns false for uasset"), FPackageName::IsTextPackageExtension(TEXT("uasset")));
		TestFalse(TEXT("IsTextPackageExtension returns false for .umap"), FPackageName::IsTextPackageExtension(TEXT(".umap")));
		TestFalse(TEXT("IsTextPackageExtension returns false for umap"), FPackageName::IsTextPackageExtension(TEXT("umap")));
		TestFalse(TEXT("IsTextPackageExtension returns false for .ubulk"), FPackageName::IsTextPackageExtension(TEXT(".ubulk")));
		TestFalse(TEXT("IsTextPackageExtension returns false for ubulk"), FPackageName::IsTextPackageExtension(TEXT("ubulk")));
		TestFalse(TEXT("IsTextPackageExtension returns false for empty string"), FPackageName::IsTextPackageExtension(TEXT("")));
		TestFalse(TEXT("IsTextPackageExtension returns false for \".\""), FPackageName::IsTextPackageExtension(TEXT(".")));
	}
};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FPackagePathTestSubClass, FPackagePathTest, "System.Core.Misc.PackagePath", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FPackagePathTestSubClass::RunTest(const FString& Parameters)
{
	return FPackagePathTest::RunTest(Parameters);
}


#endif // #if WITH_DEV_AUTOMATION_TESTS