// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/RedirectCollector.h"
#include "Algo/Transform.h"
#include "Misc/CoreDelegates.h"
#include "UObject/CoreRedirects.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "UObject/ObjectRedirector.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/SoftObjectPath.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"

#if WITH_EDITOR

DEFINE_LOG_CATEGORY_STATIC(LogRedirectors, Log, All);

FAutoConsoleCommand CVarResolveAllSoftObjects(
	TEXT("RedirectCollector.ResolveAllSoftObjectPaths"),
	TEXT("Attempts to load / resolve all currently referenced Soft Object Paths"),
	FConsoleCommandDelegate::CreateLambda([]
	{
		GRedirectCollector.ResolveAllSoftObjectPaths();
	})
);

void RedirectCollectorDumpAllAssetRedirects();

FAutoConsoleCommand CVarRedirectCollectorDumpAllAssetRedirects(
	TEXT("redirectcollector.DumpAllAssetRedirects"),
	TEXT("Prints all tracked redirectors to the log."),
	FConsoleCommandDelegate::CreateStatic(&RedirectCollectorDumpAllAssetRedirects)
);

void RedirectCollectorDumpAllAssetRedirects()
{
	FString FullyQualifiedFileName = FPaths::ProfilingDir() + FString::Printf(TEXT("AllRedirects (%s).csv"), *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));
	TUniquePtr<FArchive> OutputFile(IFileManager::Get().CreateFileWriter(*FullyQualifiedFileName));
	if (OutputFile.IsValid())
	{
		TAnsiStringBuilder<4096> StringBuilder;

		GRedirectCollector.EnumerateRedirectsUnderLock([&StringBuilder](FRedirectCollector::FRedirectionData& Data)
			{
				StringBuilder << WriteToAnsiString<256>(Data.GetSource().ToString()).ToString()
					<< TEXT(",")
					<< WriteToAnsiString<256>(Data.GetFirstTarget().ToString()).ToString()
					<< LINE_TERMINATOR;
			});
		OutputFile->Serialize(StringBuilder.GetData(), StringBuilder.Len());
	}
}

void FRedirectCollector::OnSoftObjectPathLoaded(const FSoftObjectPath& ObjectPath, FArchive* InArchive)
{
	if (ObjectPath.IsNull() || !GIsEditor)
	{
		// No need to track empty strings, or in standalone builds
		return;
	}

	FSoftObjectPathThreadContext& ThreadContext = FSoftObjectPathThreadContext::Get();

	FName PackageName, PropertyName;
	ESoftObjectPathCollectType CollectType = ESoftObjectPathCollectType::AlwaysCollect;
	ESoftObjectPathSerializeType SerializeType = ESoftObjectPathSerializeType::AlwaysSerialize;

	ThreadContext.GetSerializationOptions(PackageName, PropertyName, CollectType, SerializeType, InArchive);

	// We don't need to keep records for NonPackage Or PackageHeader CollectTypes, but we do need to keep records
	// for all SoftObjectPaths in UObjects, even for the ones marked NeverCollect, so early exit for !IsObjectType
	// CollectTypes but not necessarily for !IsCollectable CollectTypes.
	if (!UE::SoftObjectPath::IsObjectType(CollectType))
	{
		return;
	}

	const bool bReferencedByEditorOnlyProperty = (CollectType == ESoftObjectPathCollectType::EditorOnlyCollect);
	FTopLevelAssetPath AssetPath = ObjectPath.GetAssetPath();

	FScopeLock ScopeLock(&CriticalSection);
	if (UE::SoftObjectPath::IsCollectable(CollectType))
	{
		// Add this reference to the soft object inclusion list for the cook's iterative traversal of the soft dependency graph
		FSoftObjectPathProperty SoftObjectPathProperty(FSoftObjectPath(AssetPath, {}), PropertyName, bReferencedByEditorOnlyProperty);
		SoftObjectPathMap.FindOrAdd(PackageName).Add(SoftObjectPathProperty);
	}

	if (ShouldTrackPackageReferenceTypes())
	{
		// Add the referenced package to the potential-exclusion list for the cook's up-front traversal of the soft dependency graph
		TStringBuilder<FName::StringBufferSize> ObjectPathString;
		ObjectPath.AppendString(ObjectPathString);
		FName ReferencedPackageName = FName(FPackageName::ObjectPathToPackageName(ObjectPathString));
		if (PackageName != ReferencedPackageName)
		{
			TMap<FName, ESoftObjectPathCollectType>& PackageReferences = PackageReferenceTypes.FindOrAdd(PackageName);
			ESoftObjectPathCollectType& ExistingCollectType = PackageReferences.FindOrAdd(ReferencedPackageName, ESoftObjectPathCollectType::NeverCollect);
			ExistingCollectType = FMath::Max(ExistingCollectType, CollectType);
		}
	}
}

void FRedirectCollector::CollectSavedSoftPackageReferences(FName ReferencingPackage, const TSet<FName>& PackageNames, bool bEditorOnlyReferences)
{
	TArray<FSoftObjectPathProperty, TInlineAllocator<4>> SoftObjectPathArray;
	Algo::Transform(PackageNames, SoftObjectPathArray, [ReferencingPackage, bEditorOnlyReferences](const FName& PackageName)
		{
			return FSoftObjectPathProperty(FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath(PackageName, NAME_None)), NAME_None, bEditorOnlyReferences);
		});

	FScopeLock ScopeLock(&CriticalSection);
	SoftObjectPathMap.FindOrAdd(ReferencingPackage).Append(SoftObjectPathArray);
}

void FRedirectCollector::ResolveAllSoftObjectPaths(FName FilterPackage)
{	
	auto LoadSoftObjectPathLambda = [this](const FSoftObjectPathProperty& SoftObjectPathProperty, const FName ReferencerPackageName)
	{
		FSoftObjectPath ToLoadPath = SoftObjectPathProperty.GetObjectPath();
		const FString ToLoad = ToLoadPath.ToString();

		if (ToLoad.Len() > 0 )
		{
			UE_LOGF(LogRedirectors, Verbose, "Resolving Soft Object Path '%ls'", *ToLoad);
			UE_CLOGF(SoftObjectPathProperty.GetPropertyName().ToString().Len(), LogRedirectors, Verbose, "    Referenced by '%ls'", *SoftObjectPathProperty.GetPropertyName().ToString());

			int32 DotIndex = ToLoad.Find(TEXT("."));
			FString PackageName = DotIndex != INDEX_NONE ? ToLoad.Left(DotIndex) : ToLoad;

			// If is known missing don't try
			if (FLinkerLoad::IsKnownMissingPackage(FName(*PackageName)))
			{
				return;
			}

			UObject *Loaded = LoadObject<UObject>(NULL, *ToLoad, NULL, SoftObjectPathProperty.GetReferencedByEditorOnlyProperty() ? LOAD_EditorOnly | LOAD_NoWarn : LOAD_NoWarn, NULL);

			if (Loaded)
			{
				FSoftObjectPath Dest(Loaded);
				UE_LOGF(LogRedirectors, Verbose, "    Resolved to '%ls'", *Dest.ToString());
				if (Dest.ToString() != ToLoad)
				{
					AddObjectPathRedirectionInternal(ToLoadPath, Dest);
					FCoreRedirects::RecordAddedObjectRedirector(ToLoadPath, Dest);
				}
			}
			else
			{
				const FString Referencer = SoftObjectPathProperty.GetPropertyName().ToString().Len() ? SoftObjectPathProperty.GetPropertyName().ToString() : TEXT("Unknown");
				UE_LOGF(LogRedirectors, Display, "Soft Object Path '%ls' was not found when resolving paths! (Referencer '%ls:%ls')", *ToLoad, *ReferencerPackageName.ToString(), *Referencer);
			}
		}
	};

	FScopeLock ScopeLock(&CriticalSection);

	FSoftObjectPathMap KeepSoftObjectPathMap;
	KeepSoftObjectPathMap.Reserve(SoftObjectPathMap.Num());
	while (SoftObjectPathMap.Num())
	{
		FSoftObjectPathMap LocalSoftObjectPathMap;
		Swap(SoftObjectPathMap, LocalSoftObjectPathMap);

		for (TPair<FName, FSoftObjectPathPropertySet>& CurrentPackage : LocalSoftObjectPathMap)
		{
			const FName& CurrentPackageName = CurrentPackage.Key;
			FSoftObjectPathPropertySet& SoftObjectPathProperties = CurrentPackage.Value;

			if ((FilterPackage != NAME_None) && // not using a filter
				(FilterPackage != CurrentPackageName) && // this is the package we are looking for
				(CurrentPackageName != NAME_None) // if we have an empty package name then process it straight away
				)
			{
				// If we have a valid filter and it doesn't match, skip processing of this package and keep it
				KeepSoftObjectPathMap.FindOrAdd(CurrentPackageName).Append(MoveTemp(SoftObjectPathProperties));
				continue;
			}

			// This will call LoadObject which may trigger OnSoftObjectPathLoaded and add new soft object paths to the SoftObjectPathMap
			for (const FSoftObjectPathProperty& SoftObjecPathProperty : SoftObjectPathProperties)
			{
				LoadSoftObjectPathLambda(SoftObjecPathProperty, CurrentPackageName);
			}
		}
	}
	PackageReferenceTypes.Empty();

	check(SoftObjectPathMap.Num() == 0);
	// Add any non processed packages back into the global map for the next time this is called
	Swap(SoftObjectPathMap, KeepSoftObjectPathMap);
	// we shouldn't have any references left if we decided to resolve them all
	check((SoftObjectPathMap.Num() == 0) || (FilterPackage != NAME_None));
}

void FRedirectCollector::ProcessSoftObjectPathPackageList(FName FilterPackage, bool bGetEditorOnly, TSet<FName>& OutReferencedPackages)
{
	TSet<FSoftObjectPathProperty> SoftObjectPathProperties;
	{
		FScopeLock ScopeLock(&CriticalSection);
		// always remove all data for the processed FilterPackage, in addition to processing it to populate OutReferencedPackages
		if (!SoftObjectPathMap.RemoveAndCopyValue(FilterPackage, SoftObjectPathProperties))
		{
			return;
		}
	}

	// potentially add soft object path package names to OutReferencedPackages
	OutReferencedPackages.Reserve(SoftObjectPathProperties.Num());
	for (const FSoftObjectPathProperty& SoftObjectPathProperty : SoftObjectPathProperties)
	{
		if (!SoftObjectPathProperty.GetReferencedByEditorOnlyProperty() || bGetEditorOnly)
		{
			FSoftObjectPath ToLoadPath = SoftObjectPathProperty.GetObjectPath();
			FString PackageNameString = FPackageName::ObjectPathToPackageName(ToLoadPath.ToString());
			OutReferencedPackages.Add(FName(*PackageNameString));
		}
	}
}

bool FRedirectCollector::RemoveAndCopySoftObjectPathExclusions(FName PackageName, TSet<FName>& OutExcludedReferences)
{
	OutExcludedReferences.Reset();
	FScopeLock ScopeLock(&CriticalSection);
	TMap<FName, ESoftObjectPathCollectType> PackageTypes;
	if (!PackageReferenceTypes.RemoveAndCopyValue(PackageName, PackageTypes))
	{
		return false;
	}

	for (TPair<FName, ESoftObjectPathCollectType>& Pair : PackageTypes)
	{
		if (Pair.Value < ESoftObjectPathCollectType::AlwaysCollect)
		{
			OutExcludedReferences.Add(Pair.Key);
		}
	}
	return OutExcludedReferences.Num() != 0;
}

void FRedirectCollector::OnStartupPackageLoadComplete()
{
	// When startup packages are done loading, we never track any more regardless whether we were before
	FScopeLock ScopeLock(&CriticalSection);
	TrackingReferenceTypesState = ETrackingReferenceTypesState::Disabled;
}

void FRedirectCollector::GetAllSourcePathsForTargetPath(const FSoftObjectPath& TargetPath, TArray<FSoftObjectPath>& OutSourcePaths) const
{
	FScopeLock ScopeLock(&CriticalSection);

	OutSourcePaths.Reset();
	if (const ObjectPathSourcesArray* Sources = ObjectPathRedirectionReverseMap.Find(TargetPath))
	{
		OutSourcePaths = *Sources;
	}
}

void FRedirectCollector::AddObjectPathRedirectionInternal(const FSoftObjectPath& Source, const FSoftObjectPath& Destination)
{
	FSimpleOrChainedRedirect& ExistingRedirect = ObjectPathRedirectionMap.FindOrAdd(Source);
	if (!ExistingRedirect.GetFirstTarget().IsNull())
	{
		if (ExistingRedirect.GetFirstTarget() == Destination)
		{
			return;
		}

		// We are replacing a redirect not adding one. 
		// That means we need to remove all old sources that had us in their chain to their final destination and
		// then add them to their new final destination.

		// Bootstrap the replacement destination in as a SimpleRedirect with First == Final. We will replace it if
		// necessary with a ChainedRedirect in the loop below, because it will be one of the
		// SourcesThatWentToOldDestination.
		// We need its GetFirstTarget == Destination present in the ObjectPathRedirectionMap for the calls to
		// TraverseToFinalTarget to work for all of the redirectors that chain into it.
		FSoftObjectPath OldFinalTarget = ExistingRedirect.GetFinalTarget();
		ExistingRedirect = FSimpleOrChainedRedirect(Destination);

		// Get all redirects that that had OldFinalTarget as their FinalTarget, and clear OldFinalTarget from
		// the reverse map; we will reconstruct it if necessary.
		ObjectPathSourcesArray SourcesThatWentToOldTarget;
		ObjectPathRedirectionReverseMap.RemoveAndCopyValue(OldFinalTarget, SourcesThatWentToOldTarget);

		// For all redirects (including the one from our Source argument) that had OldFinalTarget as their
		// FinalTarget, calculate their new FinalTarget using the graph of FirstTargets, and set the data
		// for their FinalTarget and for their entry in the reverse map.
		for (const FSoftObjectPath& SourceThatWentToOldTarget : SourcesThatWentToOldTarget)
		{
			if (FSimpleOrChainedRedirect* OldRedirectionTarget
				= ObjectPathRedirectionMap.Find(SourceThatWentToOldTarget))
			{
				FSoftObjectPath FirstTarget = OldRedirectionTarget->GetFirstTarget();
				const FSoftObjectPath& FinalTarget = TraverseToFinalTarget(&FirstTarget);

				ObjectPathRedirectionReverseMap.FindOrAdd(FinalTarget).AddUnique(SourceThatWentToOldTarget);

				*OldRedirectionTarget = FSimpleOrChainedRedirect::ConstructSimpleOrChained(
					FirstTarget, FinalTarget);
			}
		}
	}
	else
	{
		// Add FirstTarget data for the new redirect before calling TraverseToFinalTarget. An empty FirstTarget value
		// is invalid, and we might encounter it during TraverseToFinalTarget if there is a cycle.
		ExistingRedirect = FSimpleOrChainedRedirect(Destination);
		const FSoftObjectPath& FinalTarget = TraverseToFinalTarget(&Destination);
		if (FinalTarget != Destination)
		{
			ExistingRedirect = FSimpleOrChainedRedirect(Destination, FinalTarget);
		}

		// Add the redirect's source to the reverse map for its FinalTarget, and if it was chained into by any
		// existing redirects, remove them from the reverse lookup for Source, change their FinalTarget to FinalTarget,
		// and add them to the reverse lookup for FinalTarget.
		ObjectPathSourcesArray OldReverseLookupArray;
		ObjectPathRedirectionReverseMap.RemoveAndCopyValue(Source, OldReverseLookupArray);
		for (const FSoftObjectPath& ChainedPath : OldReverseLookupArray)
		{
			FSimpleOrChainedRedirect* ChainedRedirect = ObjectPathRedirectionMap.Find(ChainedPath);
			check(ChainedRedirect); // It should be in ObjectPathRedirectionMap because it was in the reverse map
			*ChainedRedirect = FSimpleOrChainedRedirect(ChainedRedirect->GetFirstTarget(), FinalTarget);
		}

		ObjectPathSourcesArray& FinalTargetArray = ObjectPathRedirectionReverseMap.FindOrAdd(FinalTarget);
		FinalTargetArray.Add(Source);
		FinalTargetArray.Append(MoveTemp(OldReverseLookupArray));
	}
}

const FSoftObjectPath& FRedirectCollector::TraverseToFinalTarget(const FSoftObjectPath* FirstTarget) const
{
	// Called within the CriticalSection

	// Our input argument FirstTarget needs to be a pointer so that we are guaranteed it points to an LValue
	// and we can return a reference to it. We need to support being called with a const FSoftObjectPath, but
	// if we accepted const FSoftObjectPath& then we might get passed an X value and would not be allowed to
	// return a pointer to it.
	const FSimpleOrChainedRedirect* NextRedirect = ObjectPathRedirectionMap.Find(*FirstTarget);
	if (!NextRedirect)
	{
		// This is the most common case; handle it as cheaply as possible
		return *FirstTarget;
	}

	// Handle cycles in the graph of redirections
	TSet<FSoftObjectPath> SeenPaths;
	SeenPaths.Add(*FirstTarget);

	const FSimpleOrChainedRedirect* CurrentRedirect;
	do
	{
		CurrentRedirect = NextRedirect;
		const FSoftObjectPath& CurrentTarget = CurrentRedirect->GetFirstTarget();
		bool bAlreadySeen = false;
		SeenPaths.Add(CurrentTarget, &bAlreadySeen);
		if (bAlreadySeen)
		{
			// A cycle; return the first path we encountered in the cycle, which is CurrentTarget

			ensureMsgf(!SeenPaths.Contains(CurrentTarget),
				TEXT("Found circular redirect from %s to itself! Setting FinalDestination of %s to %s."),
				*CurrentTarget.ToString(), *FirstTarget->ToString(), *CurrentTarget.ToString());
			UE_LOGF(LogRedirectors, Error, "Logging redirection chain: ");
			for (const FSoftObjectPath& Entry : SeenPaths)
			{
				UE_LOGF(LogRedirectors, Error, " %ls", *Entry.ToString());
			}
			UE_LOGF(LogRedirectors, Error, " %ls", *CurrentTarget.ToString());

			return CurrentTarget;
		}

		NextRedirect = ObjectPathRedirectionMap.Find(CurrentTarget);
	} while (NextRedirect);

	return CurrentRedirect->GetFirstTarget();
}

bool FRedirectCollector::ShouldTrackPackageReferenceTypes()
{
	// Called from within CriticalSection
	if (TrackingReferenceTypesState == ETrackingReferenceTypesState::Uninitialized)
	{
		// OnStartupPackageLoadComplete has not been called yet. Turn tracking on/off depending on whether the
		// run mode needs it.
		TrackingReferenceTypesState = IsRunningCookCommandlet() ? ETrackingReferenceTypesState::Enabled : ETrackingReferenceTypesState::Disabled;
	}
	return TrackingReferenceTypesState == ETrackingReferenceTypesState::Enabled;
}

void FRedirectCollector::AddAssetPathRedirection(const FSoftObjectPath& OriginalPath, const FSoftObjectPath& RedirectedPath)
{
	FScopeLock ScopeLock(&CriticalSection);

	if (!ensureMsgf(!OriginalPath.IsNull(), TEXT("Cannot add redirect from Name_None!")))
	{
		return;
	}

	FSoftObjectPath FinalRedirection = GetAssetPathRedirection(RedirectedPath);
	if (FinalRedirection == OriginalPath)
	{
		// If RedirectedPath points back to OriginalPath, remove that to avoid a circular reference
		// This can happen when renaming assets in the editor but not actually dropping redirectors because it was new
		TryRemoveObjectPathRedirectionInternal(RedirectedPath);
	}

	// This replaces an existing mapping, can happen in the editor if things are renamed twice
	AddObjectPathRedirectionInternal(OriginalPath, RedirectedPath);
	FCoreRedirects::RecordAddedObjectRedirector(OriginalPath, RedirectedPath);
}

bool FRedirectCollector::TryRemoveObjectPathRedirectionInternal(const FSoftObjectPath& Source)
{
	FSimpleOrChainedRedirect OldRedirect;
	if (!ObjectPathRedirectionMap.RemoveAndCopyValue(Source, OldRedirect))
	{
		return false;
	}

	FCoreRedirects::RecordRemovedObjectRedirector(Source, OldRedirect.GetFirstTarget());

	// Get all redirects that that had OldFinalTarget as their FinalTarget, and clear OldFinalTarget from
	// the reverse map; we will reconstruct it if necessary.
	ObjectPathSourcesArray SourcesThatWentToOldTarget;
	ObjectPathRedirectionReverseMap.RemoveAndCopyValue(OldRedirect.GetFinalTarget(), SourcesThatWentToOldTarget);

	// For all redirects (except the one from our Source argument) that had OldFinalTarget as their
	// FinalTarget, calculate their new FinalTarget using the graph of FirstTargets, and set the data
	// for their FinalTarget and for their entry in the reverse map.
	for (const FSoftObjectPath& SourceThatWentToOldTarget : SourcesThatWentToOldTarget)
	{
		if (SourceThatWentToOldTarget == Source)
		{
			continue;
		}

		if (FSimpleOrChainedRedirect* OldRedirectionTarget
			= ObjectPathRedirectionMap.Find(SourceThatWentToOldTarget))
		{
			FSoftObjectPath FirstTarget = OldRedirectionTarget->GetFirstTarget();
			const FSoftObjectPath& FinalTarget = TraverseToFinalTarget(&FirstTarget);

			ObjectPathRedirectionReverseMap.FindOrAdd(FinalTarget).AddUnique(SourceThatWentToOldTarget);

			*OldRedirectionTarget = FSimpleOrChainedRedirect::ConstructSimpleOrChained(FirstTarget, FinalTarget);
		}
	}
	return true;
}

void FRedirectCollector::RemoveAssetPathRedirection(const FSoftObjectPath& OriginalPath)
{
	FScopeLock ScopeLock(&CriticalSection);

	TryRemoveObjectPathRedirectionInternal(OriginalPath);
}

FSoftObjectPath FRedirectCollector::GetAssetPathRedirection(const FSoftObjectPath& OriginalPath) const
{
	FScopeLock ScopeLock(&CriticalSection);

	const FSimpleOrChainedRedirect* Redirection;
	if (Redirection = ObjectPathRedirectionMap.Find(OriginalPath); Redirection)
	{
		return Redirection->GetFinalTarget();
	}
	if (Redirection = ObjectPathRedirectionMap.Find(OriginalPath.GetWithoutSubPath()); Redirection)
	{
		return FSoftObjectPath(Redirection->GetFinalTarget().GetAssetPath(), OriginalPath.GetSubPathString());
	}
	return FSoftObjectPath();
}

void FRedirectCollector::EnumerateRedirectsUnderLock(TFunctionRef<void(FRedirectionData&)> Callback) const
{
	FScopeLock ScopeLock(&CriticalSection);
	
	for (const TPair<FSoftObjectPath, FSimpleOrChainedRedirect>& Pair : ObjectPathRedirectionMap)
	{
		FRedirectionData RedirectionData{ Pair.Key, Pair.Value.GetFirstTarget(), Pair.Value.GetFinalTarget() };
		Callback(RedirectionData);
	}
}

FRedirectCollector GRedirectCollector;
#if WITH_AUTOMATION_WORKER

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRedirectCollectorReverseLookupTest,
	"System.Core.Misc.RedirectCollector.ReverseLookup",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FRedirectCollectorReverseLookupTest::RunTest(const FString& Parameters)
{
	FScopeLock ScopeLock(&GRedirectCollector.CriticalSection);
	bool bSuccess = true;

	// Validate that every forward redirect has a corresponding reverse redirect
	for (const TPair<FSoftObjectPath, FRedirectCollector::FSimpleOrChainedRedirect>& ForwardRedirect
		: GRedirectCollector.ObjectPathRedirectionMap)
	{
		bool bFoundReverseEntry = false;
		if (const FRedirectCollector::ObjectPathSourcesArray* Sources =
			GRedirectCollector.ObjectPathRedirectionReverseMap.Find(ForwardRedirect.Value.GetFinalTarget()))
		{
			if (Sources->Contains(ForwardRedirect.Key))
			{
				bFoundReverseEntry = true;
			}
		}

		if (!bFoundReverseEntry)
		{
			bSuccess = false;
			AddError(FString::Printf(TEXT("Failed to find matching reverse lookup for redirect %s --> (%s, %s)"),
				*ForwardRedirect.Key.ToString(), *ForwardRedirect.Value.GetFirstTarget().ToString(),
				*ForwardRedirect.Value.GetFinalTarget().ToString()));
		}
	}

	// Validate that every reverse redirect has a corresponding forward redirect
	for (const TPair<FSoftObjectPath, FRedirectCollector::ObjectPathSourcesArray>& ReverseRedirectList
		: GRedirectCollector.ObjectPathRedirectionReverseMap)
	{
		for (const FSoftObjectPath& Source : ReverseRedirectList.Value)
		{
			bool bFoundForwardEntry = false;
			if (const FRedirectCollector::FSimpleOrChainedRedirect* Target
				= GRedirectCollector.ObjectPathRedirectionMap.Find(Source))
			{
				if (Target->GetFinalTarget() == ReverseRedirectList.Key)
				{
					bFoundForwardEntry = true;
				}
			}

			if (!bFoundForwardEntry)
			{
				bSuccess = false;
				AddError(FString::Printf(TEXT("Failed to find matching forward lookup for reverse redirect %s <-- %s"),
					*ReverseRedirectList.Key.ToString(), *Source.ToString()));
			}
		}
	}
	return bSuccess;
}
#endif // WITH_AUTOMATION_WORKER
#endif // WITH_EDITOR
