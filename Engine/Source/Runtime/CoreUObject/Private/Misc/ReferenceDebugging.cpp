// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/ReferenceDebugging.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

void FindPackageReferencesInObject(const UObject* RootObject, const TArray<FName>& PackagesReferenced, TFunctionRef<void(FName /*PackagePath*/, const FString& /*PropertyPath*/)> Callback)
{
	TArray<FString> PathStack;
	TFunction<void(const UObject*)> InnerLoop = [&InnerLoop, &PackagesReferenced, &PathStack, Callback](const UObject* RootObject)
	{
		for (TPropertyValueIterator<FObjectProperty> It(RootObject->GetClass(), RootObject, EPropertyValueIteratorFlags::FullRecursion, EFieldIteratorFlags::ExcludeDeprecated); It; ++It)
		{
			const FObjectProperty* Property = It.Key();
			if (const UObject* Object = Property->GetObjectPtrPropertyValueRef(It.Value()))
			{
				if (Object->HasAnyFlags(RF_Public) && Object->IsAsset())
				{
					const FName ObjectPackageName = Object->GetOutermost()->GetFName();
					if (PackagesReferenced.Contains(ObjectPackageName))
					{
						TStringBuilder<FName::StringBufferSize> FullPropertyPath;
						for (const FString& Path : PathStack)
						{
							FullPropertyPath.Append(Path);
							FullPropertyPath.Append(TEXT("→"));
						}

						FullPropertyPath.Append(It.GetPropertyPathDebugString());
						
						Callback(ObjectPackageName, FullPropertyPath.ToString());
					}
				}
				else if (Property->HasAllPropertyFlags(CPF_ExportObject) && Object->GetClass()->HasAllClassFlags(CLASS_EditInlineNew))
				{
					TStringBuilder<FName::StringBufferSize> PathFragment;
					PathFragment.Append(It.GetPropertyPathDebugString());

					// When we encounter an embedded UObject, to make the path easy to understand we should include the
					// class name in the expression, the idea being that the user will see something like,
					// 
					PathFragment.Append(TEXT("("));
					PathFragment.Append(GetNameSafe(Object->GetClass()));
					PathFragment.Append(TEXT(")"));

					{
						PathStack.Add(PathFragment.ToString());

						// Add the inner properties of this instanced object.
						InnerLoop(Object);
					
						PathStack.Pop();
					}
				}
			}
		}
	};

	InnerLoop(RootObject);
}
