// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	TextBuffer.cpp: UObject class for storing text
=============================================================================*/

#include "Misc/TextBuffer.h"
#include "UObject/Class.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextBuffer)


/* UTextBuffer constructors
 *****************************************************************************/

UTextBuffer::UTextBuffer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UTextBuffer::UTextBuffer(const TCHAR* InText)
	: UObject(FObjectInitializer::Get())
	, Text(InText)
{}

UTextBuffer::UTextBuffer (const FObjectInitializer& ObjectInitializer, const TCHAR* InText)
	: UObject(ObjectInitializer)
	, Text(InText)
{}


/* FOutputDevice interface
 *****************************************************************************/

void UTextBuffer::Serialize (const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category)
{
	Text += (TCHAR*)Data;
}

void UTextBuffer::Serialize( const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time )
{
	Text += (TCHAR*)Data;
}

/* UObject interface
 *****************************************************************************/

void UTextBuffer::Serialize (FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << Pos << Top << Text;
}

IMPLEMENT_FSTRUCTUREDARCHIVE_SERIALIZER(UTextBuffer);
