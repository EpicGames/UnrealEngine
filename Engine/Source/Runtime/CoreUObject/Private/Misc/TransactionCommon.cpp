// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TransactionCommon.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/UnrealType.h"

namespace UE::Transaction
{

FPersistentObjectRef::FPersistentObjectRef(UObject* InObject)
{
	// Resolve the leaf-most object that is UseOuter() == false, without any UseOuter() == true objects above it
	// This is needed as there may be a break in the stable reference chain, eg, Actor -> Component -> InstancedObject, 
	// may have a UseOuter() == true Component despite the InstancedObject itself being UseOuter() == false, in which 
	// case Actor needs to be the root object rather than InstancedObject
	RootObject = nullptr;
	{
		auto UseOuter = [](const UObject* Obj)
		{
			check(Obj);

			const bool bIsClassCDO = Obj->HasAllFlags(RF_ClassDefaultObject) && Obj->GetClass()->GetDefaultObject(false) == Obj; // Guard against a trashed CDO
			const bool bIsDSO = Obj->HasAnyFlags(RF_DefaultSubObject);
			if (bIsClassCDO || bIsDSO)
			{
				return true;
			}

			for (OnUseOuterDelegate Delegate : GetOnUseOuterDelegates())
			{
				if (Delegate(Obj))
				{
					return true;
				}
			}

			return false;
		};

		// First pass to find the correct root object
		for (UObject* CurObject = InObject; CurObject; CurObject = CurObject->GetOuter())
		{
			if (CurObject->IsCapturingAsRootObjectForTransaction())
			{
				// Explicitly a root object; stop walking
				if (!RootObject)
				{
					RootObject = CurObject;
				}
				break;
			}
			if (UseOuter(CurObject))
			{
				// This reference is unstable, so reset any current root object and keep going
				RootObject = nullptr;
			}
			else if (!RootObject)
			{
				RootObject = CurObject;
			}
		}

		// Second pass to fill SubObjectHierarchyIDs with the objects under the root object
		for (UObject* CurObject = InObject; CurObject && CurObject != RootObject; CurObject = CurObject->GetOuter())
		{
			SubObjectHierarchyIDs.Add(CurObject->GetFName());
		}
	}
	check(RootObject);

	if (SubObjectHierarchyIDs.Num() > 0)
	{
		ReferenceType = EReferenceType::SubObject;
		Algo::Reverse(SubObjectHierarchyIDs);
	}
	else
	{
		ReferenceType = EReferenceType::RootObject;
	}

	// Make sure that when we look up the object we find the same thing:
	checkSlow(Get() == InObject);
}

UObject* FPersistentObjectRef::Get() const
{
	if (ReferenceType == EReferenceType::SubObject)
	{
		check(SubObjectHierarchyIDs.Num() > 0);

		UObject* CurrentObject = nullptr;

		// Do we have a valid cached pointer?
		if (!CachedRootObject.IsExplicitlyNull() && SubObjectHierarchyIDs.Num() == CachedSubObjectHierarchy.Num())
		{
			// Root object is a pointer test
			{
				CurrentObject = CachedRootObject.GetEvenIfUnreachable();
				if (CurrentObject != RootObject)
				{
					CurrentObject = nullptr;
				}
			}

			// All other sub-objects are a name test
			for (int32 SubObjectIndex = 0; CurrentObject && SubObjectIndex < SubObjectHierarchyIDs.Num(); ++SubObjectIndex)
			{
				CurrentObject = CachedSubObjectHierarchy[SubObjectIndex].GetEvenIfUnreachable();
				if (CurrentObject && CurrentObject->GetFName() != SubObjectHierarchyIDs[SubObjectIndex])
				{
					CurrentObject = nullptr;
				}
			}
		}
		if (CurrentObject)
		{
			return CurrentObject;
		}

		// Cached pointer is invalid
		CachedRootObject.Reset();
		CachedSubObjectHierarchy.Reset();

		// Try to find and cache the subobject
		CachedRootObject = RootObject;
		CurrentObject = RootObject;
		for (int32 SubObjectIndex = 0; CurrentObject && SubObjectIndex < SubObjectHierarchyIDs.Num(); ++SubObjectIndex)
		{
			CurrentObject = StaticFindObjectFast(UObject::StaticClass(), CurrentObject, SubObjectHierarchyIDs[SubObjectIndex]);
			CachedSubObjectHierarchy.Add(CurrentObject);
		}
		if (CurrentObject)
		{
			check(!CachedRootObject.IsExplicitlyNull() && SubObjectHierarchyIDs.Num() == CachedSubObjectHierarchy.Num());
			return CurrentObject;
		}

		// Cached pointer is invalid
		CachedRootObject.Reset();
		CachedSubObjectHierarchy.Reset();

		return nullptr;
	}

	return RootObject;
}

void FPersistentObjectRef::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(RootObject);

	if (ReferenceType == EReferenceType::SubObject)
	{
		// We can't refresh the resolved pointers during ARO, as it's not safe to call FindObject to update the cache if stale
		// Instead we'll just ARO whatever we may have cached, as this may result in the resolved pointers being updated anyway
		// Note: This is needed as sub-objects may be subject to GC while inside the transaction buffer, as the references from their root 
		// object may have been removed (eg, a component on an actor will no longer be referenced by the actor after a delete operation)
		for (TWeakObjectPtr<UObject>& CachedSubObject : CachedSubObjectHierarchy)
		{
			Collector.AddReferencedObject(CachedSubObject);
		}
	}
}

TArray<FPersistentObjectRef::OnUseOuterDelegate>& FPersistentObjectRef::GetOnUseOuterDelegates()
{
	static TArray<FPersistentObjectRef::OnUseOuterDelegate> Delegates;
	return Delegates;
}

FSerializedTaggedData FSerializedTaggedData::FromOffsetAndSize(const int64 InOffset, const int64 InSize)
{
	return FSerializedTaggedData{ InOffset, InSize };
}

FSerializedTaggedData FSerializedTaggedData::FromStartAndEnd(const int64 InStart, const int64 InEnd)
{
	return FSerializedTaggedData{ InStart, InEnd - InStart };
}

void FSerializedTaggedData::AppendSerializedData(const int64 InOffset, const int64 InSize)
{
	if (DataOffset == INDEX_NONE)
	{
		DataOffset = InOffset;
		DataSize = InSize;
	}
	else
	{
		DataOffset = FMath::Min(DataOffset, InOffset);
		DataSize = FMath::Max(InOffset + InSize - DataOffset, DataSize);
	}
}

void FSerializedTaggedData::AppendSerializedData(const FSerializedTaggedData& InData)
{
	if (InData.HasSerializedData())
	{
		AppendSerializedData(InData.DataOffset, InData.DataSize);
	}
}

bool FSerializedTaggedData::HasSerializedData() const
{
	return DataOffset != INDEX_NONE && DataSize != 0;
}


void FSerializedObjectData::Read(void* Dest, int64 Offset, int64 Num) const
{
	checkSlow(Offset + Num <= Data.Num());
	FMemory::Memcpy(Dest, &Data[Offset], Num);
}

void FSerializedObjectData::Write(const void* Src, int64 Offset, int64 Num)
{
	if (Offset == Data.Num())
	{
		Data.AddUninitialized(Num);
	}
	FMemory::Memcpy(&Data[Offset], Src, Num);
}


const FName TaggedDataKey_UnknownData = ".UnknownData";
const FName TaggedDataKey_ScriptData = ".ScriptData";

FSerializedObjectDataReader::FSerializedObjectDataReader(const FSerializedObject& InSerializedObject)
	: SerializedObject(InSerializedObject)
{
	SetIsLoading(true);
}

void FSerializedObjectDataReader::Serialize(void* SerData, int64 Num)
{
	if (Num)
	{
		SerializedObject.SerializedData.Read(SerData, Offset, Num);
		Offset += Num;
	}
}

FArchive& FSerializedObjectDataReader::operator<<(FName& N)
{
	int32 NameIndex = 0;
	*this << NameIndex;
	N = SerializedObject.ReferencedNames[NameIndex];
	return *this;
}

FArchive& FSerializedObjectDataReader::operator<<(UObject*& Res)
{
	int32 ObjectIndex = 0;
	*this << ObjectIndex;
	if (ObjectIndex != INDEX_NONE)
	{
		Res = SerializedObject.ReferencedObjects[ObjectIndex].Get();
	}
	else
	{
		Res = nullptr;
	}
	return *this;
}


namespace Internal
{

FSerializedObjectDataWriterCommon::FSerializedObjectDataWriterCommon(FSerializedObjectData& InSerializedData)
	: SerializedData(InSerializedData)
{
	SetIsSaving(true);
}

void FSerializedObjectDataWriterCommon::Serialize(void* SerData, int64 Num)
{
	if (Num)
	{
		int64 DataIndex = Offset;
		SerializedData.Write(SerData, Offset, Num);
		Offset += Num;

		OnDataSerialized(DataIndex, Num);
	}
}

} // namespace Internal


FSerializedObjectDataWriter::FSerializedObjectDataWriter(FSerializedObject& InSerializedObject)
	: Internal::FSerializedObjectDataWriterCommon(InSerializedObject.SerializedData)
	, SerializedObject(InSerializedObject)
{
	for (int32 ObjIndex = 0; ObjIndex < SerializedObject.ReferencedObjects.Num(); ++ObjIndex)
	{
		ObjectMap.Add(SerializedObject.ReferencedObjects[ObjIndex].Get(), ObjIndex);
	}

	for (int32 NameIndex = 0; NameIndex < SerializedObject.ReferencedNames.Num(); ++NameIndex)
	{
		NameMap.Add(SerializedObject.ReferencedNames[NameIndex], NameIndex);
	}
}

FArchive& FSerializedObjectDataWriter::operator<<(FName& N)
{
	int32 NameIndex = INDEX_NONE;
	const int32* NameIndexPtr = NameMap.Find(N);
	if (NameIndexPtr)
	{
		NameIndex = *NameIndexPtr;
	}
	else
	{
		NameIndex = SerializedObject.ReferencedNames.Add(N);
		NameMap.Add(N, NameIndex);
	}

	return *this << NameIndex;
}

FArchive& FSerializedObjectDataWriter::operator<<(UObject*& Res)
{
	int32 ObjectIndex = INDEX_NONE;
	const int32* ObjIndexPtr = ObjectMap.Find(Res);
	if (ObjIndexPtr)
	{
		ObjectIndex = *ObjIndexPtr;
	}
	else if (Res)
	{
		ObjectIndex = SerializedObject.ReferencedObjects.Add(FPersistentObjectRef(Res));
		ObjectMap.Add(Res, ObjectIndex);
	}

	return *this << ObjectIndex;
}


FDiffableObjectDataWriter::FDiffableObjectDataWriter(FDiffableObject& InDiffableObject, TArrayView<const FProperty*> InPropertiesToSerialize)
	: Internal::FSerializedObjectDataWriterCommon(InDiffableObject.SerializedData)
	, DiffableObject(InDiffableObject)
	, PropertiesToSerialize(InPropertiesToSerialize)
{
	SetWantBinaryPropertySerialization(true);
}

FName FDiffableObjectDataWriter::GetTaggedDataKey() const
{
	FName TaggedDataKey;

	// Is this known property data?
	if (TaggedDataKey.IsNone())
	{
		const FArchiveSerializedPropertyChain* PropertyChain = GetSerializedPropertyChain();
		if (PropertyChain && PropertyChain->GetNumProperties() > 0)
		{
			TaggedDataKey = CachedSerializedTaggedPropertyKey.SyncCache(*PropertyChain);
		}
	}

	// Is this script data?
	if (TaggedDataKey.IsNone() && bIsPerformingScriptSerialization)
	{
		bWasUsingTaggedDataKey_ScriptData = true;
		TaggedDataKey = FName(TaggedDataKey_ScriptData, TaggedDataKeyIndex_ScriptData);
	}
	else if (bWasUsingTaggedDataKey_ScriptData)
	{
		++TaggedDataKeyIndex_ScriptData;
		bWasUsingTaggedDataKey_ScriptData = false;
	}

	// Is this unknown data?
	if (TaggedDataKey.IsNone())
	{
		bWasUsingTaggedDataKey_UnknownData = true;
		TaggedDataKey = FName(TaggedDataKey_UnknownData, TaggedDataKeyIndex_UnknownData);
	}
	else if (bWasUsingTaggedDataKey_UnknownData)
	{
		++TaggedDataKeyIndex_UnknownData;
		bWasUsingTaggedDataKey_UnknownData = false;
	}

	return TaggedDataKey;
}

bool FDiffableObjectDataWriter::DoesObjectMatchDiffableObject(const UObject* Obj) const
{
	FNameBuilder ObjPathName;
	Obj->GetPathName(nullptr, ObjPathName);
	return FName(ObjPathName) == DiffableObject.ObjectInfo.ObjectPathName;
}

bool FDiffableObjectDataWriter::ShouldSkipProperty(const FProperty* InProperty) const
{
	return (PropertiesToSerialize.Num() > 0 && !PropertiesToSerialize.Contains(InProperty))
		|| InProperty->HasAnyPropertyFlags(CPF_NonTransactional | CPF_Deprecated)
		|| Internal::FSerializedObjectDataWriterCommon::ShouldSkipProperty(InProperty);
}

void FDiffableObjectDataWriter::MarkScriptSerializationStart(const UObject* Obj)
{
	FArchiveUObject::MarkScriptSerializationStart(Obj);

	if (DoesObjectMatchDiffableObject(Obj))
	{
		bIsPerformingScriptSerialization = true;
	}
}

void FDiffableObjectDataWriter::MarkScriptSerializationEnd(const UObject* Obj)
{
	FArchiveUObject::MarkScriptSerializationEnd(Obj);

	if (DoesObjectMatchDiffableObject(Obj))
	{
		bIsPerformingScriptSerialization = false;
	}
}

void FDiffableObjectDataWriter::OnDataSerialized(int64 InOffset, int64 InNum)
{
	// Track this offset index in the serialized data
	const FName SerializedTaggedDataKey = GetTaggedDataKey();
	if (!SerializedTaggedDataKey.IsNone())
	{
		FSerializedTaggedData& SerializedTaggedData = CachedSerializedTaggedDataEntry.SyncCache(DiffableObject, SerializedTaggedDataKey);
		SerializedTaggedData.AppendSerializedData(InOffset, InNum);
	}
}

FArchive& FDiffableObjectDataWriter::operator<<(FName& N)
{
	uint32 NameDisplayIndex = N.GetDisplayIndex().ToUnstableInt();
	int32 NameNumericSuffix = N.GetNumber();
	return *this << NameDisplayIndex << NameNumericSuffix;
}

FArchive& FDiffableObjectDataWriter::operator<<(UObject*& Res)
{
	FName ResPathName;
	if (Res)
	{
		FNameBuilder ResPathNameBuilder;
		Res->GetPathName(nullptr, ResPathNameBuilder);
		ResPathName = FName(ResPathNameBuilder);
	}
	return *this << ResPathName;
}

FName FDiffableObjectDataWriter::FCachedPropertyKey::SyncCache(const FArchiveSerializedPropertyChain& InPropertyChain)
{
	const uint32 CurrentUpdateCount = InPropertyChain.GetUpdateCount();
	if (CurrentUpdateCount != LastUpdateCount)
	{
		CachedKey = InPropertyChain.GetNumProperties() > 0 ? InPropertyChain.GetPropertyFromRoot(0)->GetFName() : FName();
		LastUpdateCount = CurrentUpdateCount;
	}

	return CachedKey;
}

FSerializedTaggedData& FDiffableObjectDataWriter::FCachedTaggedDataEntry::SyncCache(FDiffableObject& InDiffableObject, const FName InSerializedTaggedDataKey)
{
	if (CachedKey != InSerializedTaggedDataKey)
	{
		CachedKey = InSerializedTaggedDataKey;
		CachedEntryPtr = &InDiffableObject.SerializedTaggedData.FindOrAdd(InSerializedTaggedDataKey);
	}

	return *CachedEntryPtr;
}


namespace DiffUtil
{
static FArchive* GCurrentDiffableObjectDataWriter = nullptr; // Set before GetDiffableObject calls its serialize impl
	
bool IsGeneratingDiffableObject(const FArchive& Ar)
{
	return GCurrentDiffableObjectDataWriter == &Ar;
}

FDiffableObject GetDiffableObject(const UObject* Object, const FGetDiffableObjectOptions& Options)
{
	FDiffableObject DiffableObject;
	DiffableObject.SetObject(Object);

	if (Options.bSerializeEvenIfPendingKill || !DiffableObject.ObjectInfo.bIsPendingKill)
	{
		FDiffableObjectDataWriter DiffWriter(DiffableObject, Options.PropertiesToSerialize);
		TGuardValue<FArchive*> GuardGCurrentDiffableObjectDataWriter(GCurrentDiffableObjectDataWriter, &DiffWriter);
		
		switch (Options.ObjectSerializationMode)
		{
		case EGetDiffableObjectMode::SerializeObject:
			const_cast<UObject*>(Object)->Serialize(DiffWriter);
			break;
		case EGetDiffableObjectMode::SerializeProperties:
			const_cast<UObject*>(Object)->SerializeScriptProperties(DiffWriter);
			break;
		case EGetDiffableObjectMode::Custom:
			check(Options.CustomSerializer);
			Options.CustomSerializer(DiffWriter);
			break;
		default:
			checkf(false, TEXT("Unknown EGetDiffableObjectMode!"));
			break;
		}
	}

	return DiffableObject;
}

const FDiffableObject& FDiffableObjectArchetypeCache::GetArchetypeDiffableObject(const UObject* Archetype, const FGetDiffableObjectOptions& Options)
{
	if (Archetype)
	{
		if (const FDiffableObject* ExistingDiffableObject = ArchetypeDiffableObjects.Find(Archetype))
		{
			return *ExistingDiffableObject;
		}
		else
		{
			return ArchetypeDiffableObjects.Add(Archetype, GetDiffableObject(Archetype, Options));
		}
	}

	static const FDiffableObject EmptyDiffableObject;
	return EmptyDiffableObject;
}

void GenerateObjectDataDiff_Impl(const FDiffableObject& OldDiffableObject, const FDiffableObject& NewDiffableObject, FTransactionObjectDeltaChange& OutDeltaChange, const FGenerateObjectDiffOptions& DiffOptions)
{
	auto IsTaggedDataBlockIdentical = [&OldDiffableObject, &NewDiffableObject, &OutDeltaChange](const FName TaggedDataKey, const FSerializedTaggedData& OldSerializedTaggedData, const FSerializedTaggedData& NewSerializedTaggedData) -> bool
	{
		// Binary compare the serialized data to see if something has changed for this property
		bool bIsTaggedDataIdentical = OldSerializedTaggedData.DataSize == NewSerializedTaggedData.DataSize;
		if (bIsTaggedDataIdentical && OldSerializedTaggedData.HasSerializedData())
		{
			bIsTaggedDataIdentical = FMemory::Memcmp(OldDiffableObject.SerializedData.GetPtr(OldSerializedTaggedData.DataOffset), NewDiffableObject.SerializedData.GetPtr(NewSerializedTaggedData.DataOffset), NewSerializedTaggedData.DataSize) == 0;
		}
		return bIsTaggedDataIdentical;
	};

	auto ShouldCompareTaggedData = [&DiffOptions](const FName TaggedDataKey) -> bool
	{
		if (TaggedDataKey.GetComparisonIndex() == TaggedDataKey_ScriptData.GetComparisonIndex())
		{
			// Never compare script data, as it's assumed to be overhead around the tagged property serialization
			return false;
		}

		if (DiffOptions.ShouldSkipProperty && DiffOptions.ShouldSkipProperty(TaggedDataKey))
		{
			return false;
		}

		return true;
	};

	auto ShouldCompareAsNonPropertyData = [](const FName TaggedDataKey)
	{
		return TaggedDataKey.GetComparisonIndex() == TaggedDataKey_UnknownData.GetComparisonIndex();
	};

	for (const TPair<FName, FSerializedTaggedData>& NewNamePropertyPair : NewDiffableObject.SerializedTaggedData)
	{
		if (!ShouldCompareTaggedData(NewNamePropertyPair.Key))
		{
			continue;
		}

		const FSerializedTaggedData* OldSerializedTaggedData = OldDiffableObject.SerializedTaggedData.Find(NewNamePropertyPair.Key);
		if (ShouldCompareAsNonPropertyData(NewNamePropertyPair.Key))
		{
			if (DiffOptions.bFullDiff && !OutDeltaChange.bHasNonPropertyChanges && (!OldSerializedTaggedData || !IsTaggedDataBlockIdentical(NewNamePropertyPair.Key, *OldSerializedTaggedData, NewNamePropertyPair.Value)))
			{
				OutDeltaChange.bHasNonPropertyChanges = true;
			}
		}
		else if (OldSerializedTaggedData)
		{
			if (!IsTaggedDataBlockIdentical(NewNamePropertyPair.Key, *OldSerializedTaggedData, NewNamePropertyPair.Value))
			{
				OutDeltaChange.ChangedProperties.AddUnique(NewNamePropertyPair.Key);
			}
		}
		else if (DiffOptions.bFullDiff)
		{
			// Missing property, assume that the property changed
			OutDeltaChange.ChangedProperties.AddUnique(NewNamePropertyPair.Key);
		}
	}

	if (DiffOptions.bFullDiff)
	{
		for (const TPair<FName, FSerializedTaggedData>& OldNamePropertyPair : OldDiffableObject.SerializedTaggedData)
		{
			if (!ShouldCompareTaggedData(OldNamePropertyPair.Key) || NewDiffableObject.SerializedTaggedData.Contains(OldNamePropertyPair.Key))
			{
				continue;
			}

			if (ShouldCompareAsNonPropertyData(OldNamePropertyPair.Key))
			{
				OutDeltaChange.bHasNonPropertyChanges = true;
			}
			else
			{
				// Missing property, assume that the property changed
				OutDeltaChange.ChangedProperties.AddUnique(OldNamePropertyPair.Key);
			}
		}
	}
}

FTransactionObjectDeltaChange GenerateObjectDiff(const FDiffableObject& OldDiffableObject, const FDiffableObject& NewDiffableObject, const FGenerateObjectDiffOptions& DiffOptions, FDiffableObjectArchetypeCache* ArchetypeCache)
{
	FTransactionObjectDeltaChange DeltaChange;
	GenerateObjectDiff(OldDiffableObject, NewDiffableObject, DeltaChange, DiffOptions, ArchetypeCache);
	return DeltaChange;
}

void GenerateObjectDiff(const FDiffableObject& OldDiffableObject, const FDiffableObject& NewDiffableObject, FTransactionObjectDeltaChange& OutDeltaChange, const FGenerateObjectDiffOptions& DiffOptions, FDiffableObjectArchetypeCache* ArchetypeCache)
{
	if (DiffOptions.bFullDiff)
	{
		OutDeltaChange.bHasNameChange |= OldDiffableObject.ObjectInfo.ObjectName != NewDiffableObject.ObjectInfo.ObjectName;
		OutDeltaChange.bHasOuterChange |= OldDiffableObject.ObjectInfo.ObjectOuterPathName != NewDiffableObject.ObjectInfo.ObjectOuterPathName;
		OutDeltaChange.bHasExternalPackageChange |= OldDiffableObject.ObjectInfo.ObjectExternalPackageName != NewDiffableObject.ObjectInfo.ObjectExternalPackageName;
		OutDeltaChange.bHasPendingKillChange |= OldDiffableObject.ObjectInfo.bIsPendingKill != NewDiffableObject.ObjectInfo.bIsPendingKill;
	}

	if (DiffOptions.bDiffDataEvenIfPendingKill || OldDiffableObject.ObjectInfo.bIsPendingKill == NewDiffableObject.ObjectInfo.bIsPendingKill)
	{
		GenerateObjectDataDiff_Impl(OldDiffableObject, NewDiffableObject, OutDeltaChange, DiffOptions);
	}
	else
	{
		if (!NewDiffableObject.ObjectInfo.bIsPendingKill)
		{
			check(OldDiffableObject.ObjectInfo.bIsPendingKill);

			FDiffableObjectArchetypeCache LocalArchetypeCache;
			FDiffableObjectArchetypeCache& ArchetypeCacheRef = ArchetypeCache ? *ArchetypeCache : LocalArchetypeCache;

			// The object has been restored to life, so use the archetype state when generating the data diff (as there likely isn't any data cached for the old object; see GetDiffableObject)
			const FDiffableObject& DiffableArchetypeObject = ArchetypeCacheRef.GetArchetypeDiffableObject(NewDiffableObject.ObjectArchetype.Get(), DiffOptions.ArchetypeOptions);
			GenerateObjectDataDiff_Impl(DiffableArchetypeObject, NewDiffableObject, OutDeltaChange, DiffOptions);
		}
		// else, the object has become pending kill, so there's nothing more to diff after calculating the object info change above (as there likely isn't any cached data for the new object; see GetDiffableObject)
	}
}

} // namespace DiffUtil

} // namespace UE::Transaction
