// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/TransactionObjectEvent.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/Class.h"
#include "Misc/EditorPathHelper.h"

void FTransactionObjectId::SetObject(const UObject* Object)
{
	auto GetPathNameHelper = [](const UObject* ObjectToGet) -> FName
	{
		FNameBuilder ObjectPathNameBuilder;
		ObjectToGet->GetPathName(nullptr, ObjectPathNameBuilder);
		return FName(ObjectPathNameBuilder);
	};

	auto ObjectPathHelper = [GetPathNameHelper](const UObject* ObjectToGet) -> FName
	{
#if WITH_EDITOR
		FSoftObjectPath Path = FEditorPathHelper::GetEditorPath(ObjectToGet);
		return FName(Path.ToString());
#else
		return GetPathNameHelper(ObjectToGet);
#endif
	};
#if WITH_EDITOR
	const bool bEditorPathHelper = FEditorPathHelper::IsEnabled();
#else
	const bool bEditorPathHelper = false;
#endif
	const UPackage* ObjectPackage = Object->GetPackage();
	ObjectPackageName = bEditorPathHelper ? ObjectPathHelper(ObjectPackage) : ObjectPackage->GetFName();
	ObjectName = Object->GetFName();
	ObjectPathName = ObjectPathHelper(Object);
	ObjectOuterPathName = Object->GetOuter() ? ObjectPathHelper(Object->GetOuter()) : FName();
	ObjectExternalPackageName = Object->GetExternalPackage() ? Object->GetExternalPackage()->GetFName() : FName();
	ObjectClassPathName = GetPathNameHelper(Object->GetClass());
}
