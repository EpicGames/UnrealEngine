// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/UObjectToken.h"
#include "UObject/Object.h"

FOnMessageTokenActivated FUObjectToken::DefaultMessageTokenActivated;

FUObjectToken::FOnGetDisplayName FUObjectToken::DefaultGetObjectDisplayName;

FUObjectToken::FUObjectToken( const UObject* InObject,  const FText& InLabelOverride )
	: ObjectBeingReferenced(InObject)
{
	if (InObject)
	{
		OriginalObjectPathName = InObject->GetPathName();
	}
	
	if ( !InLabelOverride.IsEmpty() )
	{
		CachedText = InLabelOverride;
	}
	else
	{
		if ( DefaultGetObjectDisplayName.IsBound() )
		{
			CachedText = DefaultGetObjectDisplayName.Execute(InObject, false);
		}
		else if ( InObject )
		{
			CachedText = FText::FromString( InObject->GetPathName() );
		}
		else
		{
			CachedText = NSLOCTEXT("MessageLog", "NoneObjectToken", "<None>");
		}
	}
}

TSharedRef<FUObjectToken> FUObjectToken::Create(const UObject* InObject, const FText& InLabelOverride)
{
	return MakeShareable(new FUObjectToken(InObject, InLabelOverride));
}

const FOnMessageTokenActivated& FUObjectToken::GetOnMessageTokenActivated() const
{
	if(MessageTokenActivated.IsBound())
	{
		return MessageTokenActivated;
	}
	else
	{
		return DefaultMessageTokenActivated;
	}
}
