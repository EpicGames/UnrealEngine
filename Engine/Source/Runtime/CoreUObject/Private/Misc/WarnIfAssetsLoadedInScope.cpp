// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/WarnIfAssetsLoadedInScope.h"

#include "Algo/AnyOf.h"
#include "AssetRegistry/AssetData.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "Logging/MessageLog.h"
#include "HAL/PlatformStackWalk.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/CommandLine.h"

#define LOCTEXT_NAMESPACE "FWarnIfAssetsLoadedInScope"

#if WITH_EDITOR

FWarnIfAssetsLoadedInScope::FWarnIfAssetsLoadedInScope()
{
    AssetsLoadedHandle = FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &FWarnIfAssetsLoadedInScope::OnAssetLoaded);
}

FWarnIfAssetsLoadedInScope::FWarnIfAssetsLoadedInScope(TConstArrayView<FAssetData> InSpecificAssets)
    : SpecificAssets(InSpecificAssets)
{
    AssetsLoadedHandle = FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &FWarnIfAssetsLoadedInScope::OnAssetLoaded);
}

FWarnIfAssetsLoadedInScope::~FWarnIfAssetsLoadedInScope()
{
    FCoreUObjectDelegates::OnAssetLoaded.Remove(AssetsLoadedHandle);

    FMessageLog AssetLoadsLog("EditorErrors");

    for (const FLoadEvent& LoadEvent : ObjectsLoaded)
    {
        TSharedRef<FTokenizedMessage> ErrorMessage = AssetLoadsLog.Error();
        ErrorMessage->AddToken(FAssetNameToken::Create(LoadEvent.ObjectPath.ToString(),FText::FromString(LoadEvent.ObjectPath.ToString())));
        ErrorMessage->AddToken(FTextToken::Create(LOCTEXT("DontLoadAssetsNow", "was loaded")));

        for (const FProgramCounterSymbolInfo& CallInfo : LoadEvent.Callstack)
        {
        	ANSICHAR HumanReadableString[1024] = { '\0' };
        	if (FPlatformStackWalk::SymbolInfoToHumanReadableString(CallInfo, HumanReadableString, sizeof(HumanReadableString)))
        	{
        		ErrorMessage->AddToken(FTextToken::Create(FText::FromString(FString(HumanReadableString) + TEXT("\n"))));	
        	}
        }
    }

	if (ObjectsLoaded.Num() > 0)
	{
        AssetLoadsLog.Notify(LOCTEXT("AssetsLoadedUnexpectedly", "Asset Unexpectedly Loaded!"));
    }
}

void FWarnIfAssetsLoadedInScope::OnAssetLoaded(UObject* InObject)
{
	// Don't warn people if the switch isn't enabled.
	if (!IsEnabled())
	{
		return;
	}

	// If the handle isn't valid we can avoid all this, as there's no way anything was captured.
	if (!AssetsLoadedHandle.IsValid())
	{
		return;
	}
	
	if (SpecificAssets.Num() == 0 || Algo::AnyOf(SpecificAssets, [InObject](const FAssetData& Asset){ return Asset.IsUAsset(InObject); }))
	{
		FLoadEvent Event;
		Event.ObjectPath = FSoftObjectPath(InObject);
		Event.Callstack = FPlatformStackWalk::GetStack(4, 15);
		ObjectsLoaded.Add(MoveTemp(Event));
	}
}

bool FWarnIfAssetsLoadedInScope::IsEnabled() const
{
	auto WarnIfAssetsLoadedEnabled = []()
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("WarnIfAssetsLoaded")))
		{
			return true;
		}
		return false;
	};
	static const bool bWarnIfAssetsLoadedEnabled = WarnIfAssetsLoadedEnabled();
	
	return bWarnIfAssetsLoadedEnabled;
}

#endif

#undef LOCTEXT_NAMESPACE