// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WorldCompositionUtility.cpp : Support structures for world composition
=============================================================================*/
#include "Misc/WorldCompositionUtility.h"
#include "HAL/FileManager.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/Linker.h"
#include "Templates/UniquePtr.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

FArchive& operator<<( FArchive& Ar, FWorldTileLayer& D )
{
	// Serialized with FPackageFileSummary
	Ar << D.Name << D.Reserved0 << D.Reserved1;
		
	if (Ar.UEVer() >= VER_UE4_WORLD_LEVEL_INFO_UPDATED)
	{
		Ar << D.StreamingDistance;
	}

	if (Ar.UEVer() >= VER_UE4_WORLD_LAYER_ENABLE_DISTANCE_STREAMING)
	{
		Ar << D.DistanceStreamingEnabled;
	}
	
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FWorldTileLayer& D)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	const FPackageFileVersion Version = Slot.GetUnderlyingArchive().UEVer();

	// Serialized with FPackageFileSummary
	Record << SA_VALUE(TEXT("Name"), D.Name) << SA_VALUE(TEXT("Reserved0"), D.Reserved0) << SA_VALUE(TEXT("Reserved1"), D.Reserved1);

	if (Version >= VER_UE4_WORLD_LEVEL_INFO_UPDATED)
	{
		Record << SA_VALUE(TEXT("StreamingDistance"), D.StreamingDistance);
	}

	if (Version >= VER_UE4_WORLD_LAYER_ENABLE_DISTANCE_STREAMING)
	{
		Record << SA_VALUE(TEXT("DistanceStreamingEnabled"), D.DistanceStreamingEnabled);
	}
}

FArchive& operator<<( FArchive& Ar, FWorldTileLODInfo& D )
{
	// Serialized with FPackageFileSummary
	Ar << D.RelativeStreamingDistance 
		<< D.Reserved0
		<< D.Reserved1
		<< D.Reserved2
		<< D.Reserved3;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FWorldTileLODInfo& D)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	// Serialized with FPackageFileSummary
	Record << SA_VALUE(TEXT("RelativeStreamingDistance"), D.RelativeStreamingDistance)
		<< SA_VALUE(TEXT("Reserved0"), D.Reserved0)
		<< SA_VALUE(TEXT("Reserved1"), D.Reserved1)
		<< SA_VALUE(TEXT("Reserved2"), D.Reserved2)
		<< SA_VALUE(TEXT("Reserved3"), D.Reserved3);
}

void FWorldTileInfo::StreamArchive(FArchive& Ar)
{
	FWorldTileInfo& D = *this;
	// Serialized with FPackageFileSummary
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::WorldCompositionTile3DOffset)
	{
		FIntPoint Position2D;
		Ar << Position2D;
		D.Position = FIntVector(Position2D.X, Position2D.Y, 0);
	}
	else
	{
		Ar << D.Position;
	}

	Ar << D.Bounds;
	Ar << D.Layer;
	
	if (Ar.UEVer() >= VER_UE4_WORLD_LEVEL_INFO_UPDATED)
	{
		Ar << D.bHideInTileView << D.ParentTilePackageName;
	}
	
	if (Ar.UEVer() >= VER_UE4_WORLD_LEVEL_INFO_LOD_LIST)
	{
		Ar << D.LODList;
	}
	
	if (Ar.UEVer() >= VER_UE4_WORLD_LEVEL_INFO_ZORDER)
	{
		Ar << D.ZOrder;
	}
		
	if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
	{
		Ar << D.AbsolutePosition;
	}
}

void FWorldTileInfo::StreamStructuredArchive(FStructuredArchive::FSlot Slot)
{
	FWorldTileInfo& D = *this;
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	const FPackageFileVersion ArchiveVersion = Slot.GetUnderlyingArchive().UEVer();

	// Serialized with FPackageFileSummary
	Record << SA_VALUE(TEXT("Position"), D.Position) << SA_VALUE(TEXT("Bounds"), D.Bounds) << SA_VALUE(TEXT("Layer"), D.Layer);

	if (ArchiveVersion >= VER_UE4_WORLD_LEVEL_INFO_UPDATED)
	{
		Record << SA_VALUE(TEXT("HideInTileView"), D.bHideInTileView) << SA_VALUE(TEXT("ParentTilePackageName"), D.ParentTilePackageName);
	}

	if (ArchiveVersion >= VER_UE4_WORLD_LEVEL_INFO_LOD_LIST)
	{
		Record << SA_VALUE(TEXT("LODList"), D.LODList);
	}

	if (ArchiveVersion >= VER_UE4_WORLD_LEVEL_INFO_ZORDER)
	{
		Record << SA_VALUE(TEXT("ZOrder"), D.ZOrder);
	}

	if (Slot.GetUnderlyingArchive().GetPortFlags() & PPF_DuplicateForPIE)
	{
		Record << SA_VALUE(TEXT("AbsolutePosition"), D.AbsolutePosition);
	}
}

bool FWorldTileInfo::Read(const FString& InPackageFileName, FWorldTileInfo& OutInfo)
{
	// Fill with default information
	OutInfo = FWorldTileInfo();

	// Create a file reader to load the file
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*InPackageFileName));
	if (FileReader == NULL)
	{
		// Couldn't open the file
		return false;
	}

	// Read package file summary from the file
	FPackageFileSummary FileSummary;
	(*FileReader) << FileSummary;

	// Make sure this is indeed a package
	if (FileSummary.Tag != PACKAGE_FILE_TAG)
	{
		// Unrecognized or malformed package file
		return false;
	}

	// Does the package contains a level info?
	if (FileSummary.WorldTileInfoDataOffset != 0)
	{
		// Seek the the part of the file where the structure lives
		FileReader->Seek(FileSummary.WorldTileInfoDataOffset);

		//make sure the filereader gets the correct version number (it defaults to latest version)
		FileReader->SetUEVer(FileSummary.GetFileVersionUE());
		FileReader->SetEngineVer(FileSummary.SavedByEngineVersion);
		FileReader->SetLicenseeUEVer(FileSummary.GetFileVersionLicenseeUE());
		FileReader->SetCustomVersions(FileSummary.GetCustomVersionContainer());
		
		// Load the structure
		*FileReader << OutInfo;
	}
	
	return true;
}