// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveCrc32.h"
#include "Serialization/ArchiveObjectCrc32.h"

#include "UObject/Object.h"


FArchiveCrc32::FArchiveCrc32(uint32 InCRC, UObject* InRootObject)
	: CRC(InCRC)
	, InitialCRC(InCRC)
	, RootObject(InRootObject)
{
}

void FArchiveCrc32::Serialize(void* Data, int64 Num)
{
	uint8* BytePointer = static_cast<uint8*>(Data);
	while (Num > 0)
	{
		const int32 BytesToHash = static_cast<int32>(FMath::Min(Num, int64(MAX_int32)));

		CRC = FCrc::MemCrc32(BytePointer, BytesToHash, CRC);

		Num -= BytesToHash;
		BytePointer += BytesToHash;
	}
}

FArchive& FArchiveCrc32::operator<<(class FName& Name)
{
	CRC = FCrc::StrCrc32(*Name.ToString(), CRC);
	return *this;
}

FArchive& FArchiveCrc32::operator<<(class UObject*& Object)
{
	if (!RootObject || !Object || !Object->IsIn(RootObject))
	{
		CRC = FCrc::StrCrc32(*GetPathNameSafe(Object), CRC);
	}
	else
	{
		CRC = FArchiveObjectCrc32().Crc32(Object, RootObject, CRC);
	}

	return *this;
}

void FArchiveCrc32::Reset()
{
	FArchiveUObject::Reset();
	CRC = InitialCRC;
}
