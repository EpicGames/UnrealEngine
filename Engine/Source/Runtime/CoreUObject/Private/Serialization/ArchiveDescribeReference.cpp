// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveDescribeReference.h"
#include "UObject/UnrealType.h"


FArchiveDescribeReference::FArchiveDescribeReference( UObject* Src, UObject* InTarget, FOutputDevice& InOutput ) : Source(Src)
	, Target(InTarget)
	, Output(InOutput)
{
	// use the optimized RefLink to skip over properties which don't contain object references
	ArIsObjectReferenceCollector = true;

	ArIgnoreArchetypeRef				= false;
	ArIgnoreOuterRef					= true;
	ArIgnoreClassRef					= false;

	Source->Serialize( *this );
}

FArchive& FArchiveDescribeReference::operator<<( class UObject*& Obj )
{
	if (Obj == Target)
	{
		if (GetSerializedProperty())
		{
			Output.Logf(TEXT("        [%s]"), *GetSerializedProperty()->GetFullName());
		}
		else
		{
			Output.Logf(TEXT("        [native]"));
		}
		PTRINT BigOffset = ((uint8*)&Obj) - (uint8*)Source;
		if (BigOffset > 0 && BigOffset < Source->GetClass()->GetPropertiesSize())
		{
			int32 Offset = int32(BigOffset);
			UClass* UseClass = Source->GetClass();
			UClass* SuperClass = UseClass->GetSuperClass();
			while (1)
			{
				if (!SuperClass || Offset >= SuperClass->GetPropertiesSize())
				{
					break;
				}
				UseClass = SuperClass;
				SuperClass = UseClass->GetSuperClass();
			}
			Output.Logf(TEXT("            class %s offset %d, offset from UObject %d "), *UseClass->GetName(), SuperClass ? Offset - SuperClass->GetPropertiesSize() : Offset, Offset);
		}
	}
	return *this;
}
