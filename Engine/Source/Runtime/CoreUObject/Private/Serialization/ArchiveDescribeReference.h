// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"

/** Helper archive class to describe a reference **/
class FArchiveDescribeReference : public FArchiveUObject
{
public:
	/**
	 * Constructor
	 *
	 * @param	Src		the object to serialize which may contain a references
	 */
	FArchiveDescribeReference(UObject* Src, UObject* InTarget, FOutputDevice& InOutput);
	virtual FString GetArchiveName() const { return TEXT("FArchiveFindAllRefs"); }

	UObject*		Source;
	UObject*		Target;
	FOutputDevice&	Output;
private:
	/** Serialize a reference **/
	FArchive& operator<<( class UObject*& Obj );
};
