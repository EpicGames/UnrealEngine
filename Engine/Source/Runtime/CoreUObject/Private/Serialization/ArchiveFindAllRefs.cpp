// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveFindAllRefs.h"
#include "UObject/Object.h"


FArchiveFindAllRefs::FArchiveFindAllRefs( UObject* Src )
{
	// use the optimized RefLink to skip over properties which don't contain object references
	ArIsObjectReferenceCollector = true;

	ArIgnoreArchetypeRef				= false;
	ArIgnoreOuterRef					= true;
	ArIgnoreClassRef					= false;

	Src->Serialize( *this );
}

FArchive& FArchiveFindAllRefs::operator<<( class UObject*& Obj )
{
	if (Obj)
	{
		References.AddUnique(Obj);
	}
	return *this;
}
