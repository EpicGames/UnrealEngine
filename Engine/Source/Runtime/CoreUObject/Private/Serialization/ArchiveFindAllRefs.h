// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"

/** Helper archive class to find all references, used by the cycle finder **/
class FArchiveFindAllRefs : public FArchiveUObject
{
public:
	/**
	 * Constructor
	 *
	 * @param	Src		the object to serialize which may contain a references
	 */
	FArchiveFindAllRefs(UObject* Src);
	virtual FString GetArchiveName() const { return TEXT("FArchiveFindAllRefs"); }

	/** List of all direct references from the Object passed to the constructor **/
	TArray<UObject*>	References;

private:
	/** Serialize a reference **/
	FArchive& operator<<( class UObject*& Obj );
};
