// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveFindCulprit.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

/*----------------------------------------------------------------------------
	FArchiveFindCulprit.
----------------------------------------------------------------------------*/
/**
 * Constructor
 *
 * @param	InFind				the object that we'll be searching for references to
 * @param	Src					the object to serialize which may contain a reference to InFind
 * @param	InPretendSaving		if true, marks the archive as saving and persistent, so that a different serialization codepath is followed
 */
FArchiveFindCulprit::FArchiveFindCulprit( UObject* InFind, UObject* Src, bool InPretendSaving )
: Find(InFind), Count(0), PretendSaving(InPretendSaving)
{
	// use the optimized RefLink to skip over properties which don't contain object references
	ArIsObjectReferenceCollector = true;

	// Outers are relevant as subobjects end up tagged incorrectly
	ArIgnoreOuterRef = false;

	if( PretendSaving )
	{
		this->SetIsSaving(true);
		this->SetIsPersistent(true);
	}

	Src->Serialize( *this );
}

FArchive& FArchiveFindCulprit::operator<<( UObject*& Obj )
{
	if (Obj == Find)
	{
		if (GetSerializedProperty() != nullptr)
		{
			Referencers.AddUnique(GetSerializedProperty());
		}
		Count++;
	}

	if (PretendSaving && IsValid(Obj))
	{
		if ((!Obj->HasAnyFlags(RF_Transient) || Obj->HasAnyFlags(RF_Public)) && !Obj->HasAnyMarks(OBJECTMARK_TagExp))
		{
			if (Obj->HasAnyFlags(RF_Standalone) || Obj->HasAnyInternalFlags(EInternalObjectFlags::Native | EInternalObjectFlags::RootSet))
			{
				// serialize the object's Outer if this object could potentially be rooting the object we're attempting to find references to
				// otherwise, it's just spam
				UObject* Outer = Obj->GetOuter();
				*this << Outer;
			}

			// serialize the object's ObjectArchetype
			UObject* ObjectArchetype = Obj->GetArchetype();
			*this << ObjectArchetype;
		}
	}
	return *this;
}
