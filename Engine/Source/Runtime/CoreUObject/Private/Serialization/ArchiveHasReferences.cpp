// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveHasReferences.h"

#include "UObject/UObjectIterator.h"

FArchiveHasReferences::FArchiveHasReferences(UObject* InTarget, const TSet<UObject*>& InPotentiallyReferencedObjects)
	: Target(InTarget)
	, PotentiallyReferencedObjects(InPotentiallyReferencedObjects)
	, Result(false)
{
	check(InTarget);

	ArIsObjectReferenceCollector = true;
	InTarget->Serialize(*this);

	class FArchiveProxyCollector : public FReferenceCollector
	{
		/** Archive we are a proxy for */
		FArchive& Archive;
	public:
		FArchiveProxyCollector(FArchive& InArchive)
			: Archive(InArchive)
		{
		}
		virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
		{
			Archive << Object;
		}
		virtual void HandleObjectReferences(UObject** InObjects, const int32 ObjectNum, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
		{
			for (int32 ObjectIndex = 0; ObjectIndex < ObjectNum; ++ObjectIndex)
			{
				UObject*& Object = InObjects[ObjectIndex];
				Archive << Object;
			}
		}
		virtual bool IsIgnoringArchetypeRef() const override
		{
			return false;
		}
		virtual bool IsIgnoringTransient() const override
		{
			return false;
		}
	} ArchiveProxyCollector(*this);

	if(!Result)
	{
		InTarget->GetClass()->CallAddReferencedObjects(InTarget, ArchiveProxyCollector);
	}
}

FArchive& FArchiveHasReferences::operator<<( UObject*& Obj )
{
	if ( Obj != nullptr && Obj != Target )
	{
		if(PotentiallyReferencedObjects.Contains(Obj))
		{
			Result = true;
		}
	}
	return *this;
}

TArray<UObject*> FArchiveHasReferences::GetAllReferencers(const TArray<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore )
{
	return GetAllReferencers(TSet<UObject*>(Referencees), ObjectsToIgnore);
}

TArray<UObject*> FArchiveHasReferences::GetAllReferencers(const TSet<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore )
{
	TArray<UObject*> Ret;
	if(Referencees.Num() > 0)
	{
		for (FThreadSafeObjectIterator It; It; ++It)
		{
			UObject* PotentialReferencer = *It;
			if (ObjectsToIgnore && ObjectsToIgnore->Contains(PotentialReferencer))
			{
				continue;
			}

			if ( !Referencees.Contains(PotentialReferencer) )
			{
				// serialize the object, looking for any to SourceObjectsSet:
				FArchiveHasReferences HasReferences(PotentialReferencer, Referencees);
				if(HasReferences.HasReferences())
				{
					Ret.Add(PotentialReferencer);
				}
			}
		}
	}
	return Ret;
}

