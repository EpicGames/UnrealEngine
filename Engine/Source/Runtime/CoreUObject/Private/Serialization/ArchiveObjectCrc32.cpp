// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveObjectCrc32.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

DEFINE_LOG_CATEGORY_STATIC(LogArchiveObjectCrc32, Log, All);

//#define DEBUG_ARCHIVE_OBJECT_CRC32

/*----------------------------------------------------------------------------
	FArchiveObjectCrc32
----------------------------------------------------------------------------*/

FArchiveObjectCrc32::FArchiveObjectCrc32()
	: MemoryWriter(SerializedObjectData)
	, ObjectBeingSerialized(NULL)
	, RootObject(NULL)
{
	ArIgnoreOuterRef = true;

	// Set FArchiveObjectCrc32 to be a saving archive instead of a reference collector.
	// Reference collection causes FSoftObjectPtrs to be serialized by their weak pointer,
	// which doesn't give a stable CRC.  Serializing these to a saving archive will
	// use a string reference instead, which is a more meaningful CRC'able state.
	SetIsSaving(true);
}

FArchiveObjectCrc32::~FArchiveObjectCrc32() = default;

void FArchiveObjectCrc32::Serialize(void* Data, int64 Length)
{
	MemoryWriter.Serialize(Data, Length);
}

FArchive& FArchiveObjectCrc32::operator<<(class FName& Name)
{
	checkSlow(ObjectBeingSerialized);

	// Don't include the name of the object being serialized, since that isn't technically part of the object's state
	if(Name != ObjectBeingSerialized->GetFName())
	{
		MemoryWriter << Name;
	}

	return *this;
}

FArchive& FArchiveObjectCrc32::operator<<(class UObject*& Object)
{
	FObjectPtr ObjectPtr(Object);
	return *this << ObjectPtr;
}

FArchive& FArchiveObjectCrc32::operator<<(FObjectPtr& ObjectPtr)
{
	if (!ObjectPtr)
	{
		FString UniqueName("None");
		*this << UniqueName;
	}
	else if (!RootObject || !ObjectPtr.IsIn(FObjectPtr(const_cast<UObject*>(RootObject))))
	{
		FString UniqueName = ObjectPtr.GetPathName();
		*this << UniqueName;
	}
	else
	{
		ObjectsToSerialize.Enqueue(ObjectPtr.Get());
	}

	return *this;
}

uint32 FArchiveObjectCrc32::Crc32(UObject* Object, UObject* Root, uint32 CRC)
{
#ifdef DEBUG_ARCHIVE_OBJECT_CRC32
	const double StartTime = FPlatformTime::Seconds();
	UE_LOGF(LogArchiveObjectCrc32, Log, "### Calculating CRC for object: %ls with outer: %ls", *Object->GetName(), Object->GetOuter() ? *Object->GetOuter()->GetName() : TEXT("NULL"));
#endif
	RootObject = Root;
	if (Object)
	{
		TSet<UObject*> SerializedObjects;

		// Start with the given object
		ObjectsToSerialize.Enqueue(Object);

		// Continue until we no longer have any objects to serialized
		while (ObjectsToSerialize.Dequeue(Object))
		{
			bool bAlreadyProcessed = false;
			SerializedObjects.Add(Object, &bAlreadyProcessed);
			// If we haven't already serialized this object
			if (!bAlreadyProcessed)
			{
#ifdef DEBUG_ARCHIVE_OBJECT_CRC32
				UE_LOGF(LogArchiveObjectCrc32, Log, "- Serializing object: %ls with outer: %ls", *Object->GetName(), Object->GetOuter() ? *Object->GetOuter()->GetName() : TEXT("NULL"));
#endif
				// Serialize it
				ObjectBeingSerialized = Object;
				if (!CustomSerialize(Object))
				{
					Object->Serialize(*this);
				}
				ObjectBeingSerialized = NULL;

				// Calculate the CRC, compounding it with the checksum calculated from the previous object
				CRC = FCrc::MemCrc32(SerializedObjectData.GetData(), SerializedObjectData.Num(), CRC);
#ifdef DEBUG_ARCHIVE_OBJECT_CRC32
				UE_LOGF(LogArchiveObjectCrc32, Log, "=> object: '%ls', total size: %d bytes, checksum: 0x%08x", *GetPathNameSafe(Object), SerializedObjectData.Num(), CRC);
#endif
				// Cleanup
				MemoryWriter.Seek(0L);
				SerializedObjectData.Empty();
			}
		}

		// Cleanup
		SerializedObjects.Empty();
		RootObject = NULL;
	}

#ifdef DEBUG_ARCHIVE_OBJECT_CRC32
	UE_LOGF(LogArchiveObjectCrc32, Log, "### Finished (%.02f ms), final checksum: 0x%08x", (FPlatformTime::Seconds() - StartTime) * 1000.0f, CRC);
#endif
	return CRC;
}

uint32 FArchiveObjectCrc32::Crc32(UObject* Object, uint32 CRC)
{
	return Crc32(Object, Object, CRC);
}
