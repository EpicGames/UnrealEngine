// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveShowReferences.h"
#include "Templates/Casts.h"

/*----------------------------------------------------------------------------
	FArchiveShowReferences.
----------------------------------------------------------------------------*/
/**
 * Constructor
 *
 * @param	inOutputAr		archive to use for logging results
 * @param	LimitOuter		only consider objects that have this object as its Outer
 * @param	inTarget		object to show referencers to
 * @param	inExclude		list of objects that should be ignored if encountered while serializing Target
 */
FArchiveShowReferences::FArchiveShowReferences( FOutputDevice& inOutputAr, UObject* inOuter, UObject* inSource, TArray<UObject*>& inExclude )
: SourceObject(inSource)
, SourceOuter(inOuter)
, OutputAr(inOutputAr)
, Exclude(inExclude)
, DidRef(false)
{
	ArIsObjectReferenceCollector = true;

	// there are several types of objects we don't want listed, for different reasons.
	// Prevent them from being logged by adding them to our Found list before we start
	// serialization, so that they won't be listed

	// quick sanity check
	check(SourceObject);
	check(SourceObject->IsValidLowLevel());

	// every object we serialize obviously references our package
	Found.AddUnique(SourceOuter);

	// every object we serialize obviously references its class and its class's parent classes
	for ( UClass* ObjectClass = SourceObject->GetClass(); ObjectClass; ObjectClass = ObjectClass->GetSuperClass() )
	{
		Found.AddUnique( ObjectClass );
	}

	// similarly, if the object is a class, they all obviously reference their parent classes
	if ( UClass* SourceObjectClass = dynamic_cast<UClass*>(SourceObject) )
	{
		for ( UClass* ParentClass = SourceObjectClass->GetSuperClass(); ParentClass; ParentClass = ParentClass->GetSuperClass() )
		{
			Found.AddUnique( ParentClass );
		}
	}

	// OK, now we're all set to go - let's see what Target is referencing.
	SourceObject->Serialize( *this );
}

FArchive& FArchiveShowReferences::operator<<( UObject*& Obj )
{
	if( Obj && Obj->GetOuter() != SourceOuter )
	{
		int32 i;
		for( i=0; i<Exclude.Num(); i++ )
		{
			if( Exclude[i] == Obj->GetOuter() )
			{
				break;
			}
		}

		if( i==Exclude.Num() )
		{
			if( !DidRef )
			{
				OutputAr.Logf( TEXT("   %s references:"), *Obj->GetFullName() );
			}

			OutputAr.Logf( TEXT("      %s"), *Obj->GetFullName() );

			DidRef=1;
		}
	}
	return *this;
}
