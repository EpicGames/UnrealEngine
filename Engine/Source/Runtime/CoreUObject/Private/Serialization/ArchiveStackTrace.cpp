// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveStackTrace.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrProperty.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/UnrealType.h"
#include "HAL/PlatformStackWalk.h"
#include "Serialization/AsyncLoading.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceHelper.h"
#include "Serialization/StaticMemoryReader.h"
#include "UObject/PropertyTempVal.h"
#include "UObject/LinkerLoad.h"
#include "Serialization/LargeMemoryReader.h"
#include "UObject/LinkerManager.h"
#include "Misc/PackageName.h"
#include "Templates/UniquePtr.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/ScopeExit.h"
#include "Compression/CompressionUtil.h"
#include "Serialization/ZenPackageHeader.h"
#include "UObject/PropertyOptional.h"

DEFINE_LOG_CATEGORY_STATIC(LogArchiveDiff, Log, All);

#if !NO_LOGGING
/** Helper class that holds runtime generated constants for log output formatting */
struct FDiffFormatHelper
{
public:
	const TCHAR* const Indent;
	const TCHAR* const LineTerminator;

	FDiffFormatHelper()
		: Indent(FCString::Spc(FOutputDeviceHelper::FormatLogLine(ELogVerbosity::Warning, LogArchiveDiff.GetCategoryName(), TEXT(""), GPrintLogTimes).Len()))
		, LineTerminator(TEXT("\n")) // because LINE_TERMINATOR doesn't work well wit EC
	{}
	static FDiffFormatHelper& Get()
	{
		static FDiffFormatHelper Instance;
		return Instance;
	}
};
#endif // !NO_LOGGING

class FIgnoreDiffManager
{
	int32 IgnoreCount = 0;
	int32 DisabledCount = 0;

public:
	FIgnoreDiffManager() = default;
	void Push()
	{
		IgnoreCount++;
	}
	void Pop()
	{
		IgnoreCount--;
		check(IgnoreCount >= 0);
	}
	bool ShouldIgnoreDiff() const
	{
		return !!IgnoreCount;
	}
	void PushDisabled()
	{
		DisabledCount++;
	}
	void PopDisabled()
	{
		DisabledCount--;
		check(DisabledCount >= 0);
	}
	bool ShouldBypassDiff() const
	{
		return !!DisabledCount;
	}
};

thread_local FIgnoreDiffManager GIgnoreDiffManager;

static const ANSICHAR* DebugDataStackMarker = "\r\nDebugDataStack:\r\n";

FArchiveStackTraceIgnoreScope::FArchiveStackTraceIgnoreScope(bool bInIgnore /* = true */)
	: bIgnore(bInIgnore)
{
	if (bIgnore)
	{
		GIgnoreDiffManager.Push();
	}
}
FArchiveStackTraceIgnoreScope::~FArchiveStackTraceIgnoreScope()
{
	if (bIgnore)
	{
		GIgnoreDiffManager.Pop();
	}
}

FArchiveStackTraceDisabledScope::FArchiveStackTraceDisabledScope()
{
	GIgnoreDiffManager.PushDisabled();
}

FArchiveStackTraceDisabledScope::~FArchiveStackTraceDisabledScope()
{
	GIgnoreDiffManager.PopDisabled();
}

namespace UE::ArchiveStackTrace
{

bool LoadPackageIntoMemory(const TCHAR* InFilename, UE::ArchiveStackTrace::FPackageData& OutPackageData,
	TUniquePtr<uint8, FDeleteByFree>& OutLoadedBytes)
{
	TUniquePtr<FArchive> UAssetFileArchive(IFileManager::Get().CreateFileReader(InFilename));
	if (!UAssetFileArchive || UAssetFileArchive->TotalSize() == 0)
	{
		// The package doesn't exist on disk
		OutLoadedBytes.Reset();
		OutPackageData.Data = nullptr;
		OutPackageData.Size = 0;
		OutPackageData.HeaderSize = 0;
		OutPackageData.StartOffset = 0;
		return false;
	}
	else
	{
		// Handle EDL packages (uexp files)
		TUniquePtr<FArchive> ExpFileArchive = nullptr;
		OutPackageData.Size = UAssetFileArchive->TotalSize();
		{
			FString UExpFilename = FPaths::ChangeExtension(InFilename, TEXT("uexp"));
			ExpFileArchive.Reset(IFileManager::Get().CreateFileReader(*UExpFilename));
			if (ExpFileArchive)
			{
				// The header size is the current package size
				OutPackageData.HeaderSize = OutPackageData.Size;
				// Grow the buffer size to append the uexp file contents
				OutPackageData.Size += ExpFileArchive->TotalSize();
			}
		}
		OutLoadedBytes.Reset(reinterpret_cast<uint8*>(FMemory::Malloc(OutPackageData.Size)));
		OutPackageData.Data = OutLoadedBytes.Get();
		UAssetFileArchive->Serialize(OutPackageData.Data, UAssetFileArchive->TotalSize());

		if (ExpFileArchive)
		{
			// If uexp file is present, append its contents at the end of the buffer
			ExpFileArchive->Serialize(OutPackageData.Data + OutPackageData.HeaderSize, ExpFileArchive->TotalSize());
		}
	}

	return true;
}

void ForceKillPackageAndLinker(FLinkerLoad* Linker)
{
	UPackage* Package = Linker->LinkerRoot;
	Linker->Detach();
	FLinkerManager::Get().RemoveLinker(Linker);
	if (Package)
	{
		Package->ClearPackageFlags(PKG_ContainsMapData | PKG_ContainsMap);
		Package->MarkAsGarbage();
	}
}

bool ShouldIgnoreDiff()
{
	return GIgnoreDiffManager.ShouldIgnoreDiff();
}
bool ShouldBypassDiff()
{
	return GIgnoreDiffManager.ShouldBypassDiff();
}

} // namespace UE::ArchiveStackTrace

FArchiveStackTraceReader::FSerializeData::FSerializeData(int64 InOffset, int64 InSize, UObject* InObject, FProperty* InProperty)
: Offset(InOffset)
, Size(InSize)
, Count(1)
, Object(InObject)
, PropertyName(InProperty->GetFName())
, FullPropertyName(GetFullNameSafe(InProperty))
{}

FArchiveStackTraceReader::FArchiveStackTraceReader(const TCHAR* InFilename, const uint8* InData, const int64 Num)
	: FLargeMemoryReader(InData, Num, ELargeMemoryReaderFlags::TakeOwnership, InFilename)
	, ThreadContext(FUObjectThreadContext::Get())
{

}

void FArchiveStackTraceReader::Serialize(void* OutData, int64 Num)
{
	bool bAddData = true;
	FSerializeData NewData(Tell(), Num, ThreadContext.GetSerializeContext()->SerializedObject, GetSerializedProperty());
	if (SerializeTrace.Num())
	{
		FSerializeData& Last = SerializeTrace.Last();
		if (NewData.IsContiguousSerialization(Last))
		{
			SerializeTrace.Add(NewData);
		}
		else
		{
			Last.Size += Num;
			Last.Count++;
		}
	}
	else
	{		
		SerializeTrace.Add(NewData);
	} 
	FLargeMemoryReader::Serialize(OutData, Num);
}

FArchiveStackTraceReader* FArchiveStackTraceReader::CreateFromFile(const TCHAR* InFilename)
{
	FArchiveStackTraceReader* Reader = nullptr;
	TUniquePtr<uint8, UE::ArchiveStackTrace::FDeleteByFree> PackageBytes;
	UE::ArchiveStackTrace::FPackageData PackageData;
	if (UE::ArchiveStackTrace::LoadPackageIntoMemory(InFilename, PackageData, PackageBytes))
	{
		Reader = new FArchiveStackTraceReader(InFilename, PackageBytes.Release(), PackageData.Size);
	}
	return Reader;
}
