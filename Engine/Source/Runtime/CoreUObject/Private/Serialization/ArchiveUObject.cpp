// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveUObject.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "Serialization/SerializedPropertyScope.h"
#include "UObject/CoreRedirects.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/UnrealType.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Serialization/ArchiveReplaceObjectRef.h"
#include "Misc/EditorPathHelper.h"
#include "Misc/PackageName.h"

DEFINE_LOG_CATEGORY_STATIC(LogArchiveUObject, Log, All);

/*----------------------------------------------------------------------------
	FArchiveUObject.
----------------------------------------------------------------------------*/
/**
 * Lazy object pointer serialization.  Lazy object pointers only have weak references to objects and
 * won't serialize the object when gathering references for garbage collection.  So in many cases, you
 * don't need to bother serializing lazy object pointers.  However, serialization is required if you
 * want to load and save your object.
 */
FArchive& FArchiveUObject::SerializeLazyObjectPtr(FArchive& Ar, FLazyObjectPtr& Value)
{
	// We never serialize our reference while the garbage collector is harvesting references
	// to objects, because we don't want weak object pointers to keep objects from being garbage
	// collected.  That would defeat the whole purpose of a weak object pointer!
	// However, when modifying both kinds of references we want to serialize and writeback the updated value.
	// We only want to write the modified value during reference fixup if the data is loaded
	if (!Ar.IsObjectReferenceCollector() || Ar.IsModifyingWeakAndStrongReferences())
	{
#if WITH_EDITORONLY_DATA
		// When transacting, just serialize as a guid since the object may
		// not be in memory and you don't want to save a nullptr in this case.
		if (Ar.IsTransacting())
		{
			if (Ar.IsLoading())
			{
				// Reset before serializing to clear the internal weak pointer. 
				Value.Reset();
			}
			Ar << Value.GetUniqueID();
		}
		else
#endif
		{
			UObject* Object = Value.Get();

			Ar << Object;

			if (Ar.IsLoading() || (Object && Ar.IsModifyingWeakAndStrongReferences()))
			{
				Value = Object;
			}
		}
	}

	return Ar;
}

FArchive& FArchiveUObject::SerializeObjectPtr(FArchive& Ar, FObjectPtr& Value)
{
	if (Ar.IsCountingMemory() && !(Ar.IsLoading() || Ar.IsSaving()) && !Value.IsResolved())
	{
		return Ar;
	}

	// Default behavior is to fully resolve the reference (if we're not loading) and send it through
	// the raw UObject* serialization codepath and ensure that the result is saved back into
	// the FObjectPtr afterwards.  There will be many use cases where this is
	// not what we want to do, but this should be a reasonable default that
	// allows FObjectPtrs to be treated like raw UObject*'s by default.

	// This dummy value is used when we're not intending for the incoming value to be meaningful (it may be uninitialized memory)
	// in those cases, we don't attempt to resolve the object reference and instead feed this dummy value in with the expectation
	// that the UObject* serialization codepath is going to overwrite it.  If for any reason it is not overwritten, the FObjectPtr
	// will remain its initial value.  Note that the dummy value is chosen to represent an unaligned value that can't be a valid
	// address for an object.
	UObject* const DummyValue = (UObject* const)0xFFFF'FEFB'F123'4567;
	UObject* Object = DummyValue;
	if (!Ar.IsLoading())
	{
		Object = Value.Get();
	}
	Ar << Object;
	if ((Ar.IsLoading() || Ar.IsModifyingWeakAndStrongReferences()) && (Object != DummyValue))
	{
		Value = Object;
	}
	return Ar;
}

FArchive& FArchiveUObject::SerializeSoftObjectPtr(FArchive& Ar, FSoftObjectPtr& Value)
{
	if (Ar.IsSaving() || Ar.IsLoading())
	{
		if (Ar.IsLoading())
		{
			// Reset before serializing to clear the internal weak pointer. 
			Value.ResetWeakPtr();
		}
		Ar << Value.GetUniqueID();
	}
	else if (!Ar.IsObjectReferenceCollector() || Ar.IsModifyingWeakAndStrongReferences())
	{
		// Treat this like a weak pointer object, as we are doing something like replacing references in memory
		UObject* Object = Value.Get();

		Ar << Object;

		if (Ar.IsLoading() || (Object && Ar.IsModifyingWeakAndStrongReferences()))
		{
#if WITH_EDITOR
			Value = FEditorPathHelper::GetEditorPath(Object);
#else
			Value = Object;
#endif
		}
	}

	return Ar;
}

FArchive& FArchiveUObject::SerializeSoftObjectPath(FArchive& Ar, FSoftObjectPath& Value)
{
	Value.SerializePath(Ar);
	return Ar;
}

FArchive& FArchiveUObject::SerializeWeakObjectPtr(FArchive& Ar, FWeakObjectPtr& Value)
{
	// NOTE: When changing this function, make sure to update the SavePackage.cpp version in the import and export tagger.
	
	// We never serialize our reference while the garbage collector is harvesting references
	// to objects, because we don't want weak object pointers to keep objects from being garbage
	// collected.  That would defeat the whole purpose of a weak object pointer!
	// However, when modifying both kinds of references we want to serialize and writeback the updated value.
	if (!Ar.IsObjectReferenceCollector() || Ar.IsModifyingWeakAndStrongReferences())
	{
		UObject* Object = Value.Get(true);
	
		Ar << Object;
	
		// Only commit the value using the same checks that we have in map/set to align with rehashing, otherwise those containers can become inoperable and lead to crashes
		if (Ar.IsLoading() || (Ar.IsModifyingWeakAndStrongReferences() && !Ar.IsSaving()))
		{
			Value = Object;
		}
	}

	return Ar;
}

/*----------------------------------------------------------------------------
	FObjectAndNameAsStringProxyArchive.
----------------------------------------------------------------------------*/

FObjectAndNameAsStringProxyArchive::FObjectAndNameAsStringProxyArchive(FArchive& InInnerArchive, bool bInLoadIfFindFails)
	: FNameAsStringProxyArchive(InInnerArchive)
	, bLoadIfFindFails(bInLoadIfFindFails)
{
}

FObjectAndNameAsStringProxyArchive::~FObjectAndNameAsStringProxyArchive() = default;

/**
 * Serialize the given UObject* as an FString
 */
FArchive& FObjectAndNameAsStringProxyArchive::operator<<(UObject*& Obj)
{
	if (IsLoading())
	{
		// load the path name to the object
		FString LoadedString;
		InnerArchive << LoadedString;

		// if it's empty, let's exit early
		if (LoadedString.IsEmpty())
		{
			Obj = nullptr;
			return *this;
		}

		// look up the object by fully qualified pathname
		Obj = FindObject<UObject>(nullptr, *LoadedString, EFindObjectFlags::None);
		// If we couldn't find it, and we want to load it, do that
		if(!Obj && bLoadIfFindFails)
		{
			Obj = LoadObject<UObject>(nullptr, *LoadedString);
		}

		// Check for core redirections
		if (!Obj && bResolveCoreRedirects)
		{
			const FCoreRedirectObjectName OldRedirectObjectName(LoadedString);

			// We're matching the logic in FSoftObjectPath::FixupCoreRedirects
			const bool bIsNative = FPackageName::IsScriptPackage(LoadedString);
			const FCoreRedirectObjectName NewRedirectObjectName = FCoreRedirects::GetRedirectedName(
				bIsNative ? ECoreRedirectFlags::Type_AllMask : ECoreRedirectFlags::Type_Object,
				OldRedirectObjectName
			);

			if (NewRedirectObjectName != OldRedirectObjectName)
			{
				// look up the redirected object by fully qualified pathname
				LoadedString = NewRedirectObjectName.ToString();
				Obj = FindObject<UObject>(nullptr, *LoadedString, EFindObjectFlags::None);

				// If we couldn't find it, and we want to load it, do that
				if (!Obj && bLoadIfFindFails)
				{
					Obj = LoadObject<UObject>(nullptr, *LoadedString);
				}
			}
		}

		if (bResolveRedirectors && Obj && Obj->IsA(UObjectRedirector::StaticClass()))
		{
			int32 Count = 0;
			constexpr int32 MaxCountBeforeUseSet = 5;
			while (Obj && Obj->IsA(UObjectRedirector::StaticClass()) && ++Count <= MaxCountBeforeUseSet)
			{
				Obj = static_cast<UObjectRedirector*>(Obj)->DestinationObject;
			}
			if (Obj && Obj->IsA(UObjectRedirector::StaticClass()))
			{
				TSet<UObject*> Seen;
				Seen.Add(Obj);
				while (Obj && Obj->IsA(UObjectRedirector::StaticClass()))
				{
					Obj = static_cast<UObjectRedirector*>(Obj)->DestinationObject;
					bool bExists;
					Seen.Add(Obj, &bExists);
					if (bExists)
					{
						// Cycle, return null
						Obj = nullptr;
					}
				}
			}
		}
	}
	else if (Obj)
	{
		// save out the fully qualified object name
		FString SavedString(Obj->GetPathName());
		InnerArchive << SavedString;
	}
	else
	{
		// for null pointer, output empty string
		FString EmptyString;
		InnerArchive << EmptyString;
	}
	return *this;
}

FArchive& FObjectAndNameAsStringProxyArchive::operator<<(FWeakObjectPtr& Obj)
{
	return FArchiveUObject::SerializeWeakObjectPtr(*this, Obj);
}

FArchive& FObjectAndNameAsStringProxyArchive::operator<<(FSoftObjectPtr& Value)
{
	if (IsLoading())
	{
		// Reset before serializing to clear the internal weak pointer. 
		Value.ResetWeakPtr();
	}
	*this << Value.GetUniqueID();
	return *this;
}

FArchive& FObjectAndNameAsStringProxyArchive::operator<<(FSoftObjectPath& Value)
{
	Value.SerializePath(*this);
	return *this;
}

FArchive& FObjectAndNameAsStringProxyArchive::operator<<(FObjectPtr& Obj)
{
	return FArchiveUObject::SerializeObjectPtr(*this, Obj);
}

void FSerializedPropertyScope::PushProperty()
{
	if (Property)
	{
		Ar.PushSerializedProperty(Property, Property->IsEditorOnlyProperty());
	}
}

void FSerializedPropertyScope::PopProperty()
{
	if (Property)
	{
		Ar.PopSerializedProperty(Property, Property->IsEditorOnlyProperty());
	}
}

const TMap<UObject*, TArray<FProperty*>>& FArchiveReplaceObjectRefBase::GetReplacedReferences() const
{ 
	ensure(bTrackReplacedReferences);
	return ReplacedReferences; 
}

void FArchiveReplaceObjectRefBase::SerializeObject(UObject* ObjectToSerialize)
{
	// Simple FReferenceCollector proxy for FArchiveReplaceObjectRefBase
	class FReplaceObjectRefCollector : public FReferenceCollector
	{
		FArchive& Ar;
		bool bAllowReferenceElimination;
	public:
		FReplaceObjectRefCollector(FArchive& InAr)
			: Ar(InAr)
			, bAllowReferenceElimination(true)
		{
		}
		virtual bool IsIgnoringArchetypeRef() const override
		{
			return Ar.IsIgnoringArchetypeRef();
		}
		virtual bool IsIgnoringTransient() const override
		{
			return false;
		}
		virtual void AllowEliminatingReferences(bool bAllow) override
		{
			bAllowReferenceElimination = bAllow;
		}
		virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
		{
			if (bAllowReferenceElimination)
			{
				FProperty* NewSerializedProperty = const_cast<FProperty*>(InReferencingProperty);
				FSerializedPropertyScope SerializedPropertyScope(Ar, NewSerializedProperty ? NewSerializedProperty : Ar.GetSerializedProperty());
				Ar << InObject;
			}
		}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		virtual void HandleVCellReference(Verse::VCell* InCell, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
		{
			if (bAllowReferenceElimination)
			{
				FProperty* NewSerializedProperty = const_cast<FProperty*>(InReferencingProperty);
				FSerializedPropertyScope SerializedPropertyScope(Ar, NewSerializedProperty ? NewSerializedProperty : Ar.GetSerializedProperty());
				Ar << InCell;
			}
		}
#endif
	} ReplaceRefCollector(*this);

	// serialization for class default objects must be deterministic (since class 
	// default objects may be serialized during script compilation while the script
	// and C++ versions of a class are not in sync), so use SerializeTaggedProperties()
	// rather than the native Serialize() function
	UClass* ObjectClass = ObjectToSerialize->GetClass();
	if (ObjectToSerialize->HasAnyFlags(RF_ClassDefaultObject))
	{		
		StartSerializingDefaults();
		if (!WantBinaryPropertySerialization() && (IsLoading() || IsSaving()))
		{
			ObjectClass->SerializeTaggedProperties(*this, (uint8*)ObjectToSerialize, ObjectClass, nullptr);
		}
		else
		{
			ObjectClass->SerializeBin(*this, ObjectToSerialize);
		}
		StopSerializingDefaults();
	}
	else
	{
		ObjectToSerialize->Serialize(*this);
	}
	ObjectClass->CallAddReferencedObjects(ObjectToSerialize, ReplaceRefCollector);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FRunningContext Context = Verse::FRunningContextPromise{};
	for (int32 CellIter = 0; CellIter < PendingSerializationCells.Num(); CellIter++)
	{
		AutoRTFM::UnreachableIfClosed("VerseVM does not support serialization in the closed");
		PendingSerializationCells[CellIter]->Serialize(Context, *this);
	}
	PendingSerializationCells.Reset();
#endif
}
