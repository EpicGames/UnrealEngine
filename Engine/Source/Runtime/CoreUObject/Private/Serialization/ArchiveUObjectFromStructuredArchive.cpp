// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "UObject/LinkerSave.h"
#include "Interfaces/ITargetPlatform.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

FArchiveUObjectFromStructuredArchiveImpl::FArchiveUObjectFromStructuredArchiveImpl(FStructuredArchive::FSlot Slot)
	: Super(Slot)
{
}

FArchive& FArchiveUObjectFromStructuredArchiveImpl::operator<<(FLazyObjectPtr& Value)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		if (IsLoading())
		{
			int32 ObjectIdx = 0;
			Serialize(&ObjectIdx, sizeof(ObjectIdx));
			Value = LazyObjectPtrs[ObjectIdx];
		}
		else
		{
			int32* IdxPtr = LazyObjectPtrToIndex.Find(Value);
			if (IdxPtr == nullptr)
			{
				IdxPtr = &(LazyObjectPtrToIndex.Add(Value));
				*IdxPtr = LazyObjectPtrs.Add(Value);
			}
			Serialize(IdxPtr, sizeof(*IdxPtr));
		}
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}

FArchive& FArchiveUObjectFromStructuredArchiveImpl::operator<<(FSoftObjectPtr& Value)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		if (IsLoading())
		{
			int32 ObjectIdx = 0;
			Serialize(&ObjectIdx, sizeof(ObjectIdx));
			Value = SoftObjectPtrs[ObjectIdx];
		}
		else
		{
			int32* IdxPtr = SoftObjectPtrToIndex.Find(Value);
			if (IdxPtr == nullptr)
			{
				IdxPtr = &(SoftObjectPtrToIndex.Add(Value));
				*IdxPtr = SoftObjectPtrs.Add(Value);
			}
			Serialize(IdxPtr, sizeof(*IdxPtr));
		}
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}

FArchive& FArchiveUObjectFromStructuredArchiveImpl::operator<<(FSoftObjectPath& Value)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		if (IsLoading())
		{
			int32 ObjectIdx = 0;
			Serialize(&ObjectIdx, sizeof(ObjectIdx));
			Value = SoftObjectPaths[ObjectIdx];
		}
		else
		{
			int32* IdxPtr = SoftObjectPathToIndex.Find(Value);
			if (IdxPtr == nullptr)
			{
				IdxPtr = &(SoftObjectPathToIndex.Add(Value));
				*IdxPtr = SoftObjectPaths.Add(Value);
			}

#if 1
			// Temp workaround for emulating the behaviour of soft asset path serialization. Use these thread specific overrides to determine if we should actually save
			// a reference to the path. If not, we still record the path in our list so that we get the correct behaviour later on when we pass these references down to the underlying
			// archive
			FName PackageName, PropertyName;
			ESoftObjectPathCollectType CollectType = ESoftObjectPathCollectType::AlwaysCollect;
			ESoftObjectPathSerializeType SerializeType = ESoftObjectPathSerializeType::AlwaysSerialize;
			FSoftObjectPathThreadContext& ThreadContext = FSoftObjectPathThreadContext::Get();
			ThreadContext.GetSerializationOptions(PackageName, PropertyName, CollectType, SerializeType, this);

			if (SerializeType == ESoftObjectPathSerializeType::AlwaysSerialize)
			{
				Serialize(IdxPtr, sizeof(*IdxPtr));
			}
#else
			Serialize(IdxPtr, sizeof(*IdxPtr));
#endif
		}
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}

FArchive& FArchiveUObjectFromStructuredArchiveImpl::operator<<(FWeakObjectPtr& Value)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		if (IsLoading())
		{
			int32 ObjectIdx = 0;
			Serialize(&ObjectIdx, sizeof(ObjectIdx));
			Value = WeakObjectPtrs[ObjectIdx];
		}
		else
		{
			int32* IdxPtr = WeakObjectPtrToIndex.Find(Value);
			if (IdxPtr == nullptr)
			{
				IdxPtr = &(WeakObjectPtrToIndex.Add(Value));
				*IdxPtr = WeakObjectPtrs.Add(Value);
			}
			Serialize(IdxPtr, sizeof(*IdxPtr));
		}
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}

FArchive& FArchiveUObjectFromStructuredArchiveImpl::operator<<(FObjectPtr& Value)
{
	OpenArchive();

	if (InnerArchive.IsTextFormat())
	{
		UObject* Obj = Value.Get();
		*this << Obj;
		Value = Obj;
	}
	else
	{
		InnerArchive << Value;
	}
	return *this;
}

void FArchiveUObjectFromStructuredArchiveImpl::PushFileRegionType(EFileRegionType Type)
{
	check(CurrentFileRegionType == EFileRegionType::None);
	check(Type != EFileRegionType::None);

	CurrentFileRegionType = Type;
	FileRegionStart = Tell();
}

void FArchiveUObjectFromStructuredArchiveImpl::PopFileRegionType()
{
	check(CurrentFileRegionType != EFileRegionType::None);

	if (IsCooking() && CookingTarget()->SupportsFeature(ETargetPlatformFeatures::CookFileRegionMetadata))
	{
		FLinkerSave* LinkerSave = Cast<FLinkerSave>(GetLinker());
		check(LinkerSave);

		int64 Length = Tell() - FileRegionStart;
		if (Length > 0)
		{
			LinkerSave->FileRegions.Add(FFileRegion(FileRegionStart, Length, CurrentFileRegionType));
		}
	}

	CurrentFileRegionType = EFileRegionType::None;
}

bool FArchiveUObjectFromStructuredArchiveImpl::Finalize(FStructuredArchive::FRecord Record)
{
	check(CurrentFileRegionType == EFileRegionType::None);

	bool bShouldSerialize = Super::Finalize(Record);
	if (bShouldSerialize)
	{
		TOptional<FStructuredArchive::FSlot> LazyObjectPtrsSlot = Record.TryEnterField(TEXT("LazyObjectPtrs"), LazyObjectPtrs.Num() > 0);
		if (LazyObjectPtrsSlot.IsSet())
		{
			LazyObjectPtrsSlot.GetValue() << LazyObjectPtrs;
		}

		TOptional<FStructuredArchive::FSlot> SoftObjectPtrsSlot = Record.TryEnterField(TEXT("SoftObjectPtrs"), SoftObjectPtrs.Num() > 0);
		if (SoftObjectPtrsSlot.IsSet())
		{
			SoftObjectPtrsSlot.GetValue() << SoftObjectPtrs;
		}

		TOptional<FStructuredArchive::FSlot> SoftObjectPathsSlot = Record.TryEnterField(TEXT("SoftObjectPaths"), SoftObjectPaths.Num() > 0);
		if (SoftObjectPathsSlot.IsSet())
		{
			SoftObjectPathsSlot.GetValue() << SoftObjectPaths;
		}

		TOptional<FStructuredArchive::FSlot> WeakObjectPtrsSlot = Record.TryEnterField(TEXT("WeakObjectPtrs"), WeakObjectPtrs.Num() > 0);
		if (WeakObjectPtrsSlot.IsSet())
		{
			WeakObjectPtrsSlot.GetValue() << WeakObjectPtrs;
		}
	}
	return bShouldSerialize;
}

#endif