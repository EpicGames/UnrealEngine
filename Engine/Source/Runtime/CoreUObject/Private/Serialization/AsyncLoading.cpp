// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnAsyncLoading.cpp: Unreal async loading code.
=============================================================================*/

#include "Serialization/AsyncLoading.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "HAL/Event.h"
#include "HAL/RunnableThread.h"
#include "Misc/PackageSegment.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Stats/StatsMisc.h"
#include "Misc/CoreStats.h"
#include "HAL/IConsoleManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageName.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/TrackedActivity.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/Linker.h"
#include "UObject/CoreRedirects.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerLoadImportBehavior.h"
#include "UObject/PackageResourceManager.h"
#include "Serialization/DeferredMessageLog.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/ObjectSerializeAccessScope.h"
#include "UObject/LinkerManager.h"
#include "Misc/Paths.h"
#include "Misc/PlayInEditorLoadingScope.h"
#include "Serialization/AsyncLoadingThread.h"
#include "Misc/ExclusiveLoadPackageTimeTracker.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/ExceptionHandling.h"
#include "Templates/GuardValueAccessors.h"
#include "Serialization/AsyncLoadingPrivate.h"
#include "UObject/UObjectHash.h"
#include "Templates/UniquePtr.h"
#include "Serialization/BufferReader.h"
#include "Async/TaskGraphInterfaces.h"
#include "Blueprint/BlueprintSupport.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/IPlatformFileOpenLogWrapper.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "UObject/GarbageCollectionInternal.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Serialization/LoadTimeTracePrivate.h"
#include "Misc/PathViews.h"

#define FIND_MEMORY_STOMPS (1 && (PLATFORM_WINDOWS || PLATFORM_UNIX) && !WITH_EDITORONLY_DATA)

DEFINE_LOG_CATEGORY(LogLoadingDev);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, FileIO);

//#pragma clang optimize off

/*-----------------------------------------------------------------------------
	Async loading stats.
-----------------------------------------------------------------------------*/

DECLARE_MEMORY_STAT(TEXT("Streaming Memory Used"),STAT_StreamingAllocSize,STATGROUP_Memory);

DECLARE_CYCLE_STAT(TEXT("Tick AsyncPackage"),STAT_FAsyncPackage_Tick,STATGROUP_AsyncLoad);

DECLARE_CYCLE_STAT(TEXT("CreateLinker AsyncPackage"),STAT_FAsyncPackage_CreateLinker,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("FinishLinker AsyncPackage"),STAT_FAsyncPackage_FinishLinker,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("LoadImports AsyncPackage"),STAT_FAsyncPackage_LoadImports,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("CreateImports AsyncPackage"),STAT_FAsyncPackage_CreateImports,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("CreateMetaData AsyncPackage"),STAT_FAsyncPackage_CreateMetaData,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("CreateExports AsyncPackage"),STAT_FAsyncPackage_CreateExports,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("FreeReferencedImports AsyncPackage"), STAT_FAsyncPackage_FreeReferencedImports, STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("Precache AsyncArchive"), STAT_FAsyncArchive_Precache, STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("PreLoadObjects AsyncPackage"),STAT_FAsyncPackage_PreLoadObjects,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("ExternalReadDependencies AsyncPackage"),STAT_FAsyncPackage_ExternalReadDependencies,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("PostLoadObjects AsyncPackage"),STAT_FAsyncPackage_PostLoadObjects,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("FinishObjects AsyncPackage"),STAT_FAsyncPackage_FinishObjects,STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("CreateAsyncPackagesFromQueue"), STAT_FAsyncPackage_CreateAsyncPackagesFromQueue, STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("ProcessAsyncLoading AsyncLoadingThread"), STAT_FAsyncLoadingThread_ProcessAsyncLoading, STATGROUP_AsyncLoad);
DECLARE_CYCLE_STAT(TEXT("Async Loading Time Detailed"), STAT_AsyncLoadingTimeDetailed, STATGROUP_AsyncLoad);

DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Total PostLoadObjects time GT"), STAT_FAsyncPackage_TotalPostLoadGameThread, STATGROUP_AsyncLoadGameThread);

DECLARE_FLOAT_ACCUMULATOR_STAT( TEXT( "Async loading block time" ), STAT_AsyncIO_AsyncLoadingBlockingTime, STATGROUP_AsyncIO );
DECLARE_FLOAT_ACCUMULATOR_STAT( TEXT( "Async package precache wait time" ), STAT_AsyncIO_AsyncPackagePrecacheWaitTime, STATGROUP_AsyncIO );

LLM_DEFINE_TAG(UObject_FAsyncPackage);


/** Helper function for profiling load times */
static FName StaticGetNativeClassName(UClass* InClass)
{
	while(InClass && !InClass->HasAnyClassFlags(CLASS_Native))
	{
		InClass = InClass->GetSuperClass();
	}

	return InClass ? InClass->GetFName() : NAME_None;
}

/** Returns true if we're inside a FGCScopeGuard */
bool IsGarbageCollectionLocked();

static int32 GAsyncLoadingThreadEnabled = 0;
static FAutoConsoleVariableRef CVarAsyncLoadingThreadEnabled(
	TEXT("s.AsyncLoadingThreadEnabled"),
	GAsyncLoadingThreadEnabled,
	TEXT("Placeholder console variable, currently not used in runtime."),
	ECVF_Default
	);

static int32 GAsyncLoadingAlwaysProcessPackages = 0;
static FAutoConsoleVariableRef CVarAsyncLoadingAlwaysProcessPackages(
	TEXT("s.AsyncLoadingAlwaysProcessPackages"),
	GAsyncLoadingAlwaysProcessPackages,
	TEXT("When flushing, will process all packages instead of only what's needed. (Used to avoid a hard to repro potential deadlock)"),
	ECVF_Default
	);

static int32 GFlushStreamingOnExit = 1;
static FAutoConsoleVariableRef CFlushStreamingOnExit(
	TEXT("s.FlushStreamingOnExit"),
	GFlushStreamingOnExit,
	TEXT("Placeholder console variable, currently not used in runtime."),
	ECVF_Default
);

static int32 GEventDrivenLoaderEnabledInCookedBuilds = 0;
static FAutoConsoleVariableRef CVarEventDrivenLoaderEnabled(
	TEXT("s.EventDrivenLoaderEnabled"),
	GEventDrivenLoaderEnabledInCookedBuilds,
	TEXT("Placeholder console variable, currently not used in runtime."),
	ECVF_Default
);

int32 GMaxReadyRequestsToStallMB = 30;
static FAutoConsoleVariableRef CVar_MaxReadyRequestsToStallMB(
	TEXT("s.MaxReadyRequestsToStallMB"),
	GMaxReadyRequestsToStallMB,
	TEXT("Controls the maximum amount memory for unhandled IO requests before we stall the pak precacher to let the CPU catch up (in megabytes).")
);

int32 GMaxPrecacheRequestsInFlight = 2;
static FAutoConsoleVariableRef CVar_MaxPrecacheRequestsInFlight(
	TEXT("s.MaxPrecacheRequestsInFlight"),
	GMaxPrecacheRequestsInFlight,
	TEXT("Controls the maximum amount of precache requests to have in flight.")
);

int32 GMaxIncomingRequestsToStall = 100;
static FAutoConsoleVariableRef CVar_MaxIncomingRequestsToStall(
	TEXT("s.MaxIncomingRequestsToStall"),
	GMaxIncomingRequestsToStall,
	TEXT("Controls the maximum number of unhandled IO requests before we stall the pak precacher to let the CPU catch up.")
);

int32 GProcessPrestreamingRequests = 0;
static FAutoConsoleVariableRef CVar_ProcessPrestreamingRequests(
	TEXT("s.ProcessPrestreamingRequests"),
	GProcessPrestreamingRequests,
	TEXT("If non-zero, then we process prestreaming requests in cooked builds.")
	);

int32 GEditorLoadPrecacheSizeKB = 0;
static FAutoConsoleVariableRef CVar_EditorLoadPrecacheSizeKB(
	TEXT("s.EditorLoadPrecacheSizeKB"),
	GEditorLoadPrecacheSizeKB,
	TEXT("Size, in KB, to precache when loading packages in the editor.")
);

int32 GAsyncLoadingPrecachePriority = (int32)AIOP_MIN;
static FAutoConsoleVariableRef CVarAsyncLoadingPrecachePriority(
	TEXT("s.AsyncLoadingPrecachePriority"),
	GAsyncLoadingPrecachePriority,
	TEXT("Priority of asyncloading precache requests"),
	ECVF_Default
);

EAsyncIOPriorityAndFlags GetAsyncIOPriority()
{
	check(GAsyncLoadingPrecachePriority >= AIOP_MIN && GAsyncLoadingPrecachePriority <= AIOP_MAX);
	EAsyncIOPriorityAndFlags Priority = (EAsyncIOPriorityAndFlags)FMath::Clamp(GAsyncLoadingPrecachePriority, (int32)AIOP_MIN, (int32)AIOP_MAX);
	return Priority;
}

EAsyncIOPriorityAndFlags GetAsyncIOPrecachePriorityAndFlags()
{
	return GetAsyncIOPriority() | AIOP_FLAG_PRECACHE;
}

DEFINE_LOG_CATEGORY_STATIC(LogAsyncArchive, Display, All);
DECLARE_MEMORY_STAT(TEXT("FAsyncArchive Buffers"), STAT_FAsyncArchiveMem, STATGROUP_Memory);

//#define ASYNC_WATCH_FILE "SM_Boots_Wings.uasset"
#define TRACK_SERIALIZE (0)
#define MIN_REMAIN_TIME (0.00101f)   // wait(0) is very different than wait(tiny) so we cut things off well before roundoff error could cause us to block when we didn't intend to. Also the granularity of the event API is 1ms.




FORCEINLINE void FAsyncArchive::LogItem(const TCHAR* Item, int64 Offset, int64 Size, double StartTime)
{
	if (UE_LOG_ACTIVE(LogAsyncArchive, Verbose)
#if defined(ASYNC_WATCH_FILE)
		|| PackagePath.GetLocalFullPath().Contains(TEXT(ASYNC_WATCH_FILE))
#endif
		)
	{
		FString FileName = PackagePath.GetLocalFullPath();

		static double GlobalStartTime(FPlatformTime::Seconds());
		double Now(FPlatformTime::Seconds());

		float ThisTime = (StartTime != 0.0) ? float(1000.0 * (Now - StartTime)) : 0.0f;

		if (!UE_LOG_ACTIVE(LogAsyncArchive, VeryVerbose) && ThisTime < 1.0f
#if defined(ASYNC_WATCH_FILE)
			&& !FileName.Contains(TEXT(ASYNC_WATCH_FILE))
#endif
			)
		{
			return;
		}

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("%32s%3s    %12lld %12lld    %6.2fms    (+%9.2fms)      %s\r\n"),
			Item,
			(ThisTime > 1.0f) ? TEXT("***") : TEXT(""),
			Offset,
			((Size == MAX_int64) ? TotalSize() : Offset + Size),
			ThisTime,
			float(1000.0 * (Now - GlobalStartTime)),
			*FileName);
	}
}

int32 FMaxPackageSummarySize::Value = 8192;

void FMaxPackageSummarySize::Init()
{
	// this is used for the initial precache and should be large enough to find the actual Sum.TotalHeaderSize
	// the editor packages may not have the AdditionalPackagesToCook array stripped so we need to allocate more memory
#if WITH_EDITORONLY_DATA
	const int32 MinimumPackageSummarySize = 1024;
	check(GConfig || IsEngineExitRequested());
	Value = 16384;
	if (GConfig)
	{
		GConfig->GetInt(TEXT("/Script/Engine.StreamingSettings"), TEXT("s.MaxPackageSummarySize"), Value, GEngineIni);
		if (Value <= MinimumPackageSummarySize)
		{
			UE_LOGF(LogStreaming, Warning, "Invalid minimum package file summary size (s.MaxPackageSummarySize=%d), %d is min.", Value, MinimumPackageSummarySize);
			Value = MinimumPackageSummarySize;
		}
	}
#endif
}

#define USE_DETAILED_FASYNCARCHIVE_MEMORY_TRACKING 0

#if USE_DETAILED_FASYNCARCHIVE_MEMORY_TRACKING
class FAsyncArchiveMemTracker
{
	TMap<FString, int64> AllocatedMem;
	FCriticalSection AllocatedMemCritical;

public:

	void Allocate(const FString& Filename, int64 Mem)
	{
		FScopeLock AllocatedMemLock(&AllocatedMemCritical);
		int64& AllocatedMemAmount = AllocatedMem.FindOrAdd(Filename);
		AllocatedMemAmount += Mem;
	}

	void Deallocate(const FString& Filename, int64 Mem)
	{
		FScopeLock AllocatedMemLock(&AllocatedMemCritical);
		int64& AllocatedMemAmount = AllocatedMem.FindOrAdd(Filename);
		AllocatedMemAmount -= Mem;
		check(AllocatedMemAmount >= 0);
		if (AllocatedMemAmount == 0)
		{
			AllocatedMem.Remove(Filename);
		}
	}

	void Dump()
	{
		FScopeLock AllocatedMemLock(&AllocatedMemCritical);

		UE_LOGF(LogStreaming, Display, "Dumping FAsyncArchie allocated memory (%d)", AllocatedMem.Num());
		for (TPair<FString, int64>& ArchiveMem : AllocatedMem)
		{
			UE_LOGF(LogStreaming, Display, "  %ls %lldb", *ArchiveMem.Key, ArchiveMem.Value);
		}
	}
} GAsyncArchiveMemTracker;

void DumpAsyncArchiveMem(const TArray<FString>& Args)
{
	GAsyncArchiveMemTracker.Dump();
}

static FAutoConsoleCommand GDumpSerializeCmd(
	TEXT("DumpAsyncArchiveMem"),
	TEXT("Debug command to dump the memory allocated by existing FAsyncArchive."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&DumpAsyncArchiveMem)
);
#endif // USE_DETAILED_FASYNCARCHIVE_MEMORY_TRACKING

static FCriticalSection SummaryRacePreventer;

FAsyncArchive::FAsyncArchive(const FPackagePath& InPackagePath, FLinkerLoad* InOwner, TFunction<void()>&& InSummaryReadyCallback)
	: Handle(nullptr)
	, SizeRequestPtr(nullptr)
	, EditorPrecacheRequestPtr(nullptr)
	, SummaryRequestPtr(nullptr)
	, SummaryPrecacheRequestPtr(nullptr)
	, ReadRequestPtr(nullptr)
	, CanceledReadRequestPtr(nullptr)
	, PrecacheBuffer(nullptr)
	, FileSize(-1)
	, CurrentPos(0)
	, PrecacheStartPos(0)
	, PrecacheEndPos(0)
	, ReadRequestOffset(0)
	, ReadRequestSize(0)
	, HeaderSize(0)
	, HeaderSizeWhenReadingExportsFromSplitFile(0)
	, LoadPhase(ELoadPhase::WaitingForSize)
	, LoadError(ELoadError::Unknown)
	, bCookedForEDLInEditor(false)
	, bNeedsEngineVersionChecks(true)
	, PackagePath(InPackagePath)
	, OpenTime(FPlatformTime::Seconds())
	, SummaryReadTime(0.0)
	, ExportReadTime(0.0)
	, SummaryReadyCallback(Forward<TFunction<void()>>(InSummaryReadyCallback))
	, OwnerLinker(InOwner)
{
	// Set the Archive flags for code that uses this as an FArchive
	SetIsLoading(true);

	LogItem(TEXT("Open"));
	FOpenAsyncPackageResult OpenResult = IPackageResourceManager::Get().OpenAsyncReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header);
	Handle = OpenResult.Handle.Release();
	check(Handle); // OpenAsyncReadPackage guarantees a non-null return value; the handle will fail to read later if the path does not exist
	if (OpenResult.Format != EPackageFormat::Binary)
	{
		SetError();
		LoadError = ELoadError::UnsupportedFormat;
	}
	bNeedsEngineVersionChecks = OpenResult.bNeedsEngineVersionChecks;

	ReadCallbackFunction = [this](bool bWasCancelled, IAsyncReadRequest* Request)
	{
		ReadCallback(bWasCancelled, Request);
	};

	SizeRequestPtr = Handle->SizeRequest(&ReadCallbackFunction);

}

FAsyncArchive::~FAsyncArchive()
{
	UE_CLOGF(OwnerLinker && !(OwnerLinker->GetLoader_Unsafe() == this && OwnerLinker->IsDestroyingLoader()), LogStreaming, Fatal,
		"Destroying FAsyncArchive %ls that belongs to linker %ls outside of the linker's DestroyLoader code!", *GetArchiveName(), *OwnerLinker->GetArchiveName());

	// Invalidate any precached data and free memory.
	FlushCache();
	if (Handle)
	{
		delete Handle;
		Handle = nullptr;
	}
	LogItem(TEXT("~FAsyncArchive"), 0, 0);
}

void FAsyncArchive::ReadCallback(bool bWasCancelled, IAsyncReadRequest* Request)
{
	if (bWasCancelled || IsError())
	{
		if (!IsError())
		{
			LoadError = ELoadError::Cancelled;
		}
		SetError();
		return; // we don't do much with this, the code on the other thread knows how to deal with my request
	}
	if (LoadPhase == ELoadPhase::WaitingForSize)
	{
		LoadPhase = ELoadPhase::WaitingForSummary;
		FileSize = Request->GetSizeResults();
		if (FileSize < 32)
		{
			SetError();
			LoadError = FileSize > 0 ? ELoadError::CorruptData : ELoadError::FileDoesNotExist;
		}
		else
		{
			int64 Size = FMath::Min<int64>(FMaxPackageSummarySize::Value, FileSize);
			LogItem(TEXT("Starting Summary"), 0, Size);
			SummaryRequestPtr = Handle->ReadRequest(0, Size, GetAsyncIOPriority(), &ReadCallbackFunction);
			// I need a precache request here to keep the memory alive until I submit the header request
			SummaryPrecacheRequestPtr = Handle->ReadRequest(0, Size, GetAsyncIOPrecachePriorityAndFlags() );
#if WITH_EDITOR
			if (FileSize > Size && GEditorLoadPrecacheSizeKB > 0)
			{
				const int64 MaxEditorPrecacheSize = int64(GEditorLoadPrecacheSizeKB) * 1024;
				EditorPrecacheRequestPtr = Handle->ReadRequest(Size, FMath::Min<int64>(FileSize - Size, MaxEditorPrecacheSize), GetAsyncIOPrecachePriorityAndFlags());
			}
#endif
		}
	}
	else if (LoadPhase == ELoadPhase::WaitingForSummary)
	{
		uint8* Mem = Request->GetReadResults();
		if (!Mem)
		{
			SetError();
			LoadError = ELoadError::CorruptData;
			FPlatformMisc::MemoryBarrier();
			LoadPhase = ELoadPhase::WaitingForHeader;
		}
		else
		{
			FBufferReader Ar(Mem, FMath::Min<int64>(FMaxPackageSummarySize::Value, FileSize),/*bInFreeOnClose=*/ false, /*bIsPersistent=*/ true);
			FPackageFileSummary Sum;
			Ar << Sum;
			if (Ar.IsError() || Sum.TotalHeaderSize > FileSize
				|| Sum.IsFileVersionTooOld() || (IsEnforcePackageCompatibleVersionCheck() && Sum.IsFileVersionTooNew()))
			{
				SetError();
				LoadError = ELoadError::CorruptData;
			}
			else
			{
				FScopeLock Lock(&SummaryRacePreventer);
				//@todoio change header format to put the TotalHeaderSize at the start of the file
				// we need to be sure that we can at least get the size from the initial request. This is an early warning that custom versions are starting to get too big, relocate the total size to be at offset 4!
				checkf(Ar.Tell() < FMaxPackageSummarySize::Value / 2,
					TEXT("The initial read request was too small (%d) compared to package %s header size (%lld). Try increasing s.MaxPackageSummarySize value in DefaultEngine.ini."),
					FMaxPackageSummarySize::Value, *PackagePath.GetDebugName(), Ar.Tell());
				
				// Support for cooked EDL packages in the editor
				bCookedForEDLInEditor = !FPlatformProperties::RequiresCookedData() && (Sum.GetPackageFlags() & PKG_FilterEditorOnly) && Sum.PreloadDependencyCount > 0 && Sum.PreloadDependencyOffset > 0;

				HeaderSize = Sum.TotalHeaderSize;
				LogItem(TEXT("Starting Header"), 0, HeaderSize);
				PrecacheInternal(0, HeaderSize);
				FPlatformMisc::MemoryBarrier();
				LoadPhase = ELoadPhase::WaitingForHeader;
			}
			FMemory::Free(Mem);
		}
	}
	else
	{
		check(0); // we don't use callbacks for other phases
	}
}

void FAsyncArchive::FlushPrecacheBlock()
{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	DiscardInlineBufferAndUpdateCurrentPos();
#endif
	if (PrecacheBuffer)
	{
		DEC_MEMORY_STAT_BY(STAT_FAsyncArchiveMem, PrecacheEndPos - PrecacheStartPos);
		FMemory::Free(PrecacheBuffer);
#if USE_DETAILED_FASYNCARCHIVE_MEMORY_TRACKING
		GAsyncArchieMemTracker.Deallocate(FileName, PrecacheEndPos - PrecacheStartPos);
#endif
	}
	PrecacheBuffer = nullptr;
	PrecacheStartPos = 0;
	PrecacheEndPos = 0;
}

void FAsyncArchive::FlushCache()
{
	bool nNonRedundantFlush = PrecacheEndPos || PrecacheBuffer || ReadRequestPtr;
	LogItem(TEXT("Flush"));
	WaitForIntialPhases();
	WaitRead(); // this deals with the read request
	CompleteCancel(); // this deals with the cancel request, important this is last because completing other things leaves cancels to process
	FlushPrecacheBlock();

	if (EditorPrecacheRequestPtr)
	{
		EditorPrecacheRequestPtr->WaitCompletion();
		delete EditorPrecacheRequestPtr;
		EditorPrecacheRequestPtr = nullptr;
	}

	if (Handle)
	{
		Handle->ShrinkHandleBuffers();
	}

	if ((UE_LOG_ACTIVE(LogAsyncArchive, Verbose) 
#if defined(ASYNC_WATCH_FILE)
		|| FileName.Contains(TEXT(ASYNC_WATCH_FILE))
#endif
		) && nNonRedundantFlush)
	{
		double Now(FPlatformTime::Seconds());
		float TotalLifetime = float(1000.0 * (Now - OpenTime));

		if (!UE_LOG_ACTIVE(LogAsyncArchive, VeryVerbose) && TotalLifetime < 100.0f
#if defined(ASYNC_WATCH_FILE)
			&& !FileName.Contains(TEXT(ASYNC_WATCH_FILE))
#endif
			)
		{
			return;
		}

		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Flush     Lifeitme %6.2fms   Open->Summary %6.2fms    Summary->Export1 %6.2fms    Export1->Now %6.2fms       %s\r\n"),
			TotalLifetime,
			float(1000.0 * (SummaryReadTime - OpenTime)),
			float(1000.0 * (ExportReadTime - SummaryReadTime)),
			float(1000.0 * (Now - ExportReadTime)),
			*PackagePath.GetDebugName());
#if defined(ASYNC_WATCH_FILE)
		if (FileName.Contains(TEXT(ASYNC_WATCH_FILE)))
		{
			UE_LOGF(LogAsyncArchive, Warning, "Handy Breakpoint after flush.");
		}
#endif
	}

}

bool FAsyncArchive::Close()
{
	// Invalidate any precached data and free memory.
	FlushCache();
	// Return true if there were NO errors, false otherwise.
	return !IsError();
}

bool FAsyncArchive::SetCompressionMap(TArray<FCompressedChunk>* InCompressedChunks, ECompressionFlags InCompressionFlags)
{
	check(0); // no support for compression
	return false;
}

int64 FAsyncArchive::TotalSize()
{
	if (SizeRequestPtr)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FArchiveAsync2_TotalSize);
		SizeRequestPtr->WaitCompletion();
		if (HeaderSizeWhenReadingExportsFromSplitFile)
		{
			FileSize = SizeRequestPtr->GetSizeResults();
		}
		delete SizeRequestPtr;
		SizeRequestPtr = nullptr;
	}
	return FileSize + HeaderSizeWhenReadingExportsFromSplitFile;
}

#if DEVIRTUALIZE_FLinkerLoad_Serialize
FORCEINLINE void FAsyncArchive::SetPosAndUpdatePrecacheBuffer(int64 Pos)
{
	check(Pos >= 0 && Pos <= TotalSizeOrMaxInt64IfNotReady());
	if (Pos < PrecacheStartPos || Pos >= PrecacheEndPos)
	{
		ActiveFPLB->Reset();
		CurrentPos = Pos;
	}
	else
	{
		check(PrecacheBuffer);
		ActiveFPLB->OriginalFastPathLoadBuffer = PrecacheBuffer;
		ActiveFPLB->StartFastPathLoadBuffer = PrecacheBuffer + (Pos - PrecacheStartPos);
		ActiveFPLB->EndFastPathLoadBuffer = PrecacheBuffer + (PrecacheEndPos - PrecacheStartPos);
		CurrentPos = PrecacheStartPos;
	}
	check(Tell() == Pos);
}
#endif

void FAsyncArchive::Seek(int64 InPos)
{
	if (LoadPhase < ELoadPhase::ProcessingExports)
	{
		// Auto-detect when exports are in a separate file so we only activate that mode when needed.
		if (!HeaderSizeWhenReadingExportsFromSplitFile && HeaderSize && TotalSize() == HeaderSize)
		{
			if (InPos >= HeaderSize)
			{
				FirstExportStarting();
			}
		}
	}
	checkf(InPos >= 0 && InPos <= TotalSizeOrMaxInt64IfNotReady(), TEXT("Bad position in FAsyncArchive::Seek. Filename:%s InPos:%llu, Size:%llu"),
		*PackagePath.GetDebugName(), InPos, TotalSizeOrMaxInt64IfNotReady());
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	SetPosAndUpdatePrecacheBuffer(InPos);
#else
	CurrentPos = InPos;
#endif
}

bool FAsyncArchive::WaitRead(double TimeLimit)
{
	if (ReadRequestPtr)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FArchiveAsync2_WaitRead);
		const int64 Offset = ReadRequestOffset;
		const int64 Size = ReadRequestSize;
		check(Size > 0);
		const double StartTime = FPlatformTime::Seconds();
		const bool bResult = ReadRequestPtr->WaitCompletion((float)TimeLimit);
		LogItem(TEXT("Wait Read"), Offset, Size, StartTime);
		if (!bResult)
		{
			return false;
		}
		CompleteRead();
	}
	return true;
}

void FAsyncArchive::CompleteRead()
{
	double StartTime = FPlatformTime::Seconds();
	check(LoadPhase != ELoadPhase::WaitingForSize && LoadPhase != ELoadPhase::WaitingForSummary);
	check(ReadRequestPtr && ReadRequestPtr->PollCompletion());
	if (PrecacheBuffer)
	{
		FlushPrecacheBlock();
	}
	if (!IsError())
	{
		uint8* Mem = ReadRequestPtr->GetReadResults();
		if (!Mem)
		{
			SetError();
			LoadError = ELoadError::CorruptData;
		}
		else
		{
			PrecacheBuffer = Mem;
			PrecacheStartPos = ReadRequestOffset;
			PrecacheEndPos = ReadRequestOffset + ReadRequestSize;
			check(ReadRequestSize > 0 && PrecacheStartPos >= 0);
			INC_MEMORY_STAT_BY(STAT_FAsyncArchiveMem, PrecacheEndPos - PrecacheStartPos);
#if USE_DETAILED_FASYNCARCHIVE_MEMORY_TRACKING
			GAsyncArchiveMemTracker.Allocate(FileName, PrecacheEndPos - PrecacheStartPos);
#endif
			// keeps the last cache block of the header around until we process the first export
			if (LoadPhase != ELoadPhase::ProcessingExports && Handle->UsesCache())
			{
				CompleteCancel();
				CanceledReadRequestPtr = Handle->ReadRequest(PrecacheEndPos - HeaderSizeWhenReadingExportsFromSplitFile - 1, 1, GetAsyncIOPrecachePriorityAndFlags());
			}
		}
	}

	delete ReadRequestPtr;
	ReadRequestPtr = nullptr;
	LogItem(TEXT("CompleteRead"), ReadRequestOffset, ReadRequestSize);
	ReadRequestOffset = 0;
	ReadRequestSize = 0;
}

void FAsyncArchive::CompleteCancel()
{
	if (CanceledReadRequestPtr)
	{
		double StartTime = FPlatformTime::Seconds();
		CanceledReadRequestPtr->WaitCompletion();
		//check(!CanceledReadRequestPtr->GetReadResults()); // this should have been canceled
		delete CanceledReadRequestPtr;
		CanceledReadRequestPtr = nullptr;
		LogItem(TEXT("Complete Cancel"), 0, 0, StartTime);
	}
}


void FAsyncArchive::CancelRead()
{
	if (ReadRequestPtr)
	{
		ReadRequestPtr->Cancel();
		CompleteCancel();
		CanceledReadRequestPtr = ReadRequestPtr;
		ReadRequestPtr = nullptr;
	}
	ReadRequestOffset = 0;
	ReadRequestSize = 0;
}

bool FAsyncArchive::WaitForIntialPhases(double InTimeLimit)
{
	if (SizeRequestPtr || SummaryRequestPtr || SummaryPrecacheRequestPtr)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FArchiveAsync2_WaitForIntialPhases);
		double StartTime = FPlatformTime::Seconds();
		if (SizeRequestPtr)
		{
			if (SizeRequestPtr->WaitCompletion((float)InTimeLimit))
			{
				delete SizeRequestPtr;
				SizeRequestPtr = nullptr;
			}
			else
			{
				check(InTimeLimit > 0.0);
				return false;
			}
		}
		
		{
			if (SummaryRequestPtr)
			{
				double TimeLimit = 0.0;
				if (InTimeLimit > 0.0)
				{
					TimeLimit = InTimeLimit - (FPlatformTime::Seconds() - StartTime);
					if (TimeLimit < MIN_REMAIN_TIME)
					{
						return false;
					}
				}
				if (SummaryRequestPtr->WaitCompletion((float)TimeLimit))
				{
					delete SummaryRequestPtr;
					SummaryRequestPtr = nullptr;
				}
				else
				{
					check(InTimeLimit > 0.0);
					return false;
				}
			}
			if (SummaryPrecacheRequestPtr)
			{
				double TimeLimit = 0.0;
				if (InTimeLimit > 0.0)
				{
					TimeLimit = InTimeLimit - (FPlatformTime::Seconds() - StartTime);
					if (TimeLimit < MIN_REMAIN_TIME)
					{
						return false;
					}
				}
				if (SummaryPrecacheRequestPtr->WaitCompletion((float)TimeLimit))
				{
					delete SummaryPrecacheRequestPtr;
					SummaryPrecacheRequestPtr = nullptr;
				}
				else
				{
					check(InTimeLimit > 0.0);
					return false;
				}
			}
		}
		LogItem(TEXT("Wait Summary"), 0, HeaderSize, StartTime);
	}
	return true;
}

bool FAsyncArchive::PrecacheInternal(int64 RequestOffset, int64 RequestSize, bool bApplyMinReadSize, IAsyncReadRequest* Read)
{
	// CAUTION! This is possibly called the first time from a random IO thread.

	bool bIsWaitingForSummary =( LoadPhase == ELoadPhase::WaitingForSummary);

	bool bReadIsActualRequest = !Handle->UsesCache();

	if (!bIsWaitingForSummary)
	{
		if (RequestSize == 0 || (RequestOffset >= PrecacheStartPos && RequestOffset + RequestSize <= PrecacheEndPos))
		{
			// ready
			delete Read;
			return true;
		}
		if (ReadRequestPtr && RequestOffset >= ReadRequestOffset && RequestOffset + RequestSize <= ReadRequestOffset + ReadRequestSize)
		{
			// current request contains request
			bool bResult = false;
			if (ReadRequestPtr->PollCompletion())
			{
				CompleteRead();
				if (GetLoadError() == ELoadError::Unknown)
				{
					check(RequestOffset >= PrecacheStartPos && RequestOffset + RequestSize <= PrecacheEndPos);
					bResult = true;
				}
			}
			delete Read;
			return bResult;
		}
		if (ReadRequestPtr)
		{
			// this one does not have what we need
			UE_LOGF(LogStreaming, Warning, "FAsyncArchive::PrecacheInternal Canceled read for %ls  Offset = %lld   Size = %lld", *PackagePath.GetDebugName(), RequestOffset, ReadRequestSize);
			CancelRead();
		}
	}
	check(!ReadRequestPtr);
	ReadRequestOffset = RequestOffset;
	ReadRequestSize = RequestSize;


	if (bApplyMinReadSize && !bIsWaitingForSummary && !bReadIsActualRequest)
	{
#if WITH_EDITOR
		static int64 MinimumReadSize = 1024 * 1024;
#else
		static int64 MinimumReadSize = 65536;
#endif
		checkSlow(MinimumReadSize >= 2048 && MinimumReadSize <= 1024 * 1024); // not a hard limit, but we should be loading at least a reasonable amount of data
		if (ReadRequestSize < MinimumReadSize)
		{ 
			ReadRequestSize = MinimumReadSize;
			int64 LocalFileSize = TotalSize();
			ReadRequestSize = FMath::Min(ReadRequestOffset + ReadRequestSize, LocalFileSize) - ReadRequestOffset;
		}
	}
	if (ReadRequestSize <= 0)
	{
		SetError();
		LoadError = ELoadError::CorruptData;
		return true;
	}
	double StartTime = FPlatformTime::Seconds();
#if defined(ASYNC_WATCH_FILE)
	if (FileName.Contains(TEXT(ASYNC_WATCH_FILE)) && ReadRequestOffset == 80203)
	{
		UE_LOGF(LogAsyncArchive, Warning, "Handy Breakpoint Read");
	}
#endif
	check(ReadRequestOffset - HeaderSizeWhenReadingExportsFromSplitFile >= 0 && ReadRequestSize > 0);

	if (Read && bReadIsActualRequest)
	{
		ReadRequestPtr = Read;
		Read = nullptr;
	}
	else
	{
	// caution, this callback can fire before this even returns....and so bIsWaitingForSummary must be a local variable or we could get all confused by concurrency!
		ReadRequestPtr = Handle->ReadRequest(ReadRequestOffset - HeaderSizeWhenReadingExportsFromSplitFile, ReadRequestSize, GetAsyncIOPriority(), nullptr);
	}
	delete Read;
	if (!bIsWaitingForSummary && ReadRequestPtr->PollCompletion())
	{
		LogItem(TEXT("Read Start Hot"), ReadRequestOffset - HeaderSizeWhenReadingExportsFromSplitFile, ReadRequestSize, StartTime);
		CompleteRead();
		check(RequestOffset >= PrecacheStartPos && RequestOffset + RequestSize <= PrecacheEndPos);
		return true;
	}
	else if (bIsWaitingForSummary)
	{
		LogItem(TEXT("Read Start Summary"), ReadRequestOffset - HeaderSizeWhenReadingExportsFromSplitFile, ReadRequestSize, StartTime);
	}
	else 
	{
		LogItem(TEXT("Read Start Cold"), ReadRequestOffset - HeaderSizeWhenReadingExportsFromSplitFile, ReadRequestSize, StartTime);
	}
	return false;
}

void FAsyncArchive::FirstExportStarting()
{
	ExportReadTime = FPlatformTime::Seconds();
	LogItem(TEXT("Exports"));
	LoadPhase = ELoadPhase::ProcessingExports;

	{
		// Detect when exports are in a separate file before trying to activate that mode.
		if (!HeaderSizeWhenReadingExportsFromSplitFile && HeaderSize && TotalSize() == HeaderSize)
		{
			FlushCache();
			if (Handle)
			{
				delete Handle;
				Handle = nullptr;
			}

			HeaderSizeWhenReadingExportsFromSplitFile = HeaderSize;

			FOpenAsyncPackageResult OpenResult = IPackageResourceManager::Get().OpenAsyncReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Exports);
			Handle = OpenResult.Handle.Release();
			check(Handle); // OpenAsyncReadPackage guarantees a non-null return value; the handle will fail to read later if the path does not exist

			check(!SizeRequestPtr);
			SizeRequestPtr = Handle->SizeRequest();
			if (SizeRequestPtr->PollCompletion())
			{
				TotalSize(); // complete the request
			}
		}
	}
}

IAsyncReadRequest* FAsyncArchive::MakeEventDrivenPrecacheRequest(int64 Offset, int64 BytesToRead, FAsyncFileCallBack* CompleteCallback)
{
	check(false);
	if (LoadPhase == ELoadPhase::WaitingForFirstExport)
	{
		// we need to avoid tearing down the old file and requests until we have the one in flight
		HeaderSizeWhenReadingExportsFromSplitFile = HeaderSize;
		IAsyncReadFileHandle* NewHandle;
		{
			double StartTime = FPlatformTime::Seconds();
			FOpenAsyncPackageResult OpenResult = IPackageResourceManager::Get().OpenAsyncReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Exports);
			NewHandle = OpenResult.Handle.Release();
			check(NewHandle); // OpenAsyncReadPackage guarantees a non-null return value; the handle will fail to read later if the path does not exist
			LogItem(TEXT("Open UExp"), Offset - HeaderSizeWhenReadingExportsFromSplitFile, BytesToRead, StartTime);
		}
		{
			double StartTime = FPlatformTime::Seconds();

			check(Offset - HeaderSizeWhenReadingExportsFromSplitFile >= 0);

			EAsyncIOPriorityAndFlags Prio = NewHandle->UsesCache() ? GetAsyncIOPrecachePriorityAndFlags() : GetAsyncIOPriority();

			IAsyncReadRequest* Precache = NewHandle->ReadRequest(Offset - HeaderSizeWhenReadingExportsFromSplitFile, BytesToRead, Prio, CompleteCallback);
			FlushCache();
			if (Handle)
			{
				delete Handle;
				Handle = nullptr;
			}
			Handle = NewHandle;

			FirstExportStarting();

			check(!SizeRequestPtr);
			SizeRequestPtr = Handle->SizeRequest();
			if (SizeRequestPtr->PollCompletion())
			{
				TotalSize(); // complete the request
			}
			LogItem(TEXT("First Precache"), Offset - HeaderSizeWhenReadingExportsFromSplitFile, BytesToRead, StartTime);
			return Precache;
		}
	}
	double StartTime = FPlatformTime::Seconds();
	check(Offset - HeaderSizeWhenReadingExportsFromSplitFile >= 0);
	check(Offset + BytesToRead <= TotalSizeOrMaxInt64IfNotReady());
	EAsyncIOPriorityAndFlags Prio = Handle->UsesCache() ? GetAsyncIOPrecachePriorityAndFlags() : GetAsyncIOPriority();
	IAsyncReadRequest* Precache = Handle->ReadRequest(Offset - HeaderSizeWhenReadingExportsFromSplitFile, BytesToRead, Prio, CompleteCallback);
	LogItem(TEXT("Event Precache"), Offset - HeaderSizeWhenReadingExportsFromSplitFile, BytesToRead, StartTime);
	return Precache;
}


bool FAsyncArchive::PrecacheWithTimeLimit(int64 RequestOffset, int64 RequestSize, bool bUseTimeLimit, bool bUseFullTimeLimit, double TickStartTime, double TimeLimit)
{
#if defined(ASYNC_WATCH_FILE)
	if (FileName.Contains(TEXT(ASYNC_WATCH_FILE)) && RequestOffset == 878129)
	{
		UE_LOGF(LogAsyncArchive, Warning, "Handy Breakpoint Raw Precache %lld", RequestOffset);
	}
#endif
	if (LoadPhase == ELoadPhase::WaitingForSize || LoadPhase == ELoadPhase::WaitingForSummary || LoadPhase == ELoadPhase::WaitingForHeader)
	{
		check(0); // this is a precache for an export, why is the summary not read yet?
		return false;
	}
	if (LoadPhase == ELoadPhase::WaitingForFirstExport)
	{
		FirstExportStarting();
	}
	if (!bUseTimeLimit)
	{
		return true; // we will stream and do the blocking on the serialize calls
	}
	bool bResult = PrecacheInternal(RequestOffset, RequestSize);
	if (!bResult && bUseFullTimeLimit)
	{
		double RemainingTime = TimeLimit - (FPlatformTime::Seconds() - TickStartTime);
		if (RemainingTime > MIN_REMAIN_TIME && WaitRead(RemainingTime))
		{
			bResult = true;
		}
	}
	return bResult;
}

bool FAsyncArchive::Precache(int64 RequestOffset, int64 RequestSize)
{
	if (LoadPhase == ELoadPhase::WaitingForSize || LoadPhase == ELoadPhase::WaitingForSummary)
	{
		return false;
	}
	if (LoadPhase == ELoadPhase::WaitingForHeader)
	{
		//@todoio, it would be nice to check that when we read the header, we don't read any more than we really need...i.e. no "minimum read size"
		check(RequestOffset == 0 && RequestOffset + RequestSize <= HeaderSize);
	}
	return PrecacheInternal(RequestOffset, RequestSize);
}

bool FAsyncArchive::PrecacheForEvent(IAsyncReadRequest* Read, int64 RequestOffset, int64 RequestSize)
{
	check(int32(LoadPhase) > int32(ELoadPhase::WaitingForHeader));
	return PrecacheInternal(RequestOffset, RequestSize, false, Read);
}


void FAsyncArchive::StartReadingHeader()
{
	//LogItem(TEXT("Start Header"));
	WaitForIntialPhases();
	if (!IsError())
	{
		if (int32(LoadPhase) < int32(ELoadPhase::WaitingForHeader))
		{
			FScopeLock Lock(&SummaryRacePreventer);
		}
		check(LoadPhase == ELoadPhase::WaitingForHeader && ReadRequestPtr);
		WaitRead();
	}
}

void FAsyncArchive::EndReadingHeader()
{
	LogItem(TEXT("End Header"));
	
	if (!IsError())
	{
		check(LoadPhase == ELoadPhase::WaitingForHeader);
		LoadPhase = ELoadPhase::WaitingForFirstExport;
		FlushPrecacheBlock();
	}
}

bool FAsyncArchive::ReadyToStartReadingHeader(bool bUseTimeLimit, bool bUseFullTimeLimit, double TickStartTime, double TimeLimit)
{
	if (SummaryReadTime == 0.0)
	{
		SummaryReadTime = FPlatformTime::Seconds();
	}
	if (!bUseTimeLimit)
	{
		return true; // we will stream and do the blocking on the serialize calls
	}
	if (LoadPhase == ELoadPhase::WaitingForSize || LoadPhase == ELoadPhase::WaitingForSummary)
	{
		if (bUseFullTimeLimit)
		{
			double RemainingTime = TimeLimit - (FPlatformTime::Seconds() - TickStartTime);
			if (RemainingTime < MIN_REMAIN_TIME || !WaitForIntialPhases(RemainingTime))
			{
				return false; // not ready
			}
		}
		else
		{
			return false; // not ready, not going to wait
		}
	}
	check(LoadPhase == ELoadPhase::WaitingForHeader);
	LogItem(TEXT("Ready For Header"));
	return true;
}

#if TRACK_SERIALIZE
void CallSerializeHook();
#endif

void FAsyncArchive::Serialize(void* Data, int64 Count)
{
	if (!Count || IsError())
	{
		return;
	}
	check(Count > 0);
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	if (ActiveFPLB->StartFastPathLoadBuffer + Count <= ActiveFPLB->EndFastPathLoadBuffer)
	{
		// this wasn't one of the cases we devirtualized; we can short circut here to avoid resettting the buffer when we don't need to
		FMemory::ParallelMemcpy(Data, ActiveFPLB->StartFastPathLoadBuffer, Count);
		ActiveFPLB->StartFastPathLoadBuffer += Count;
		return;
	}

	DiscardInlineBufferAndUpdateCurrentPos();
#endif

#if TRACK_SERIALIZE
	CallSerializeHook();
#endif

#if PLATFORM_DESKTOP
	// Show a message box indicating, possible, corrupt data (desktop platforms only)
	if (CurrentPos + Count > TotalSize())
	{
		FText ErrorMessage, ErrorCaption;
		GConfig->GetText(TEXT("/Script/Engine.Engine"),
			TEXT("SerializationOutOfBoundsErrorMessage"),
			ErrorMessage,
			GEngineIni);
		GConfig->GetText(TEXT("/Script/Engine.Engine"),
			TEXT("SerializationOutOfBoundsErrorMessageCaption"),
			ErrorCaption,
			GEngineIni);

		FMessageDialog::Open(EAppMsgType::Ok, ErrorMessage, ErrorCaption);
	}
#endif
	// Ensure we aren't reading beyond the end of the file
	checkf(CurrentPos + Count <= TotalSizeOrMaxInt64IfNotReady(), TEXT("Seeked past end of file %s (%lld / %lld)"), *PackagePath.GetDebugName(), CurrentPos + Count, TotalSize());

	int64 BeforeBlockOffset = 0;
	int64 BeforeBlockSize = 0;
	int64 AfterBlockOffset = 0;
	int64 AfterBlockSize = 0;

	if (CurrentPos + Count <= PrecacheStartPos || CurrentPos >= PrecacheEndPos)
	{
		// no overlap with current buffer
		AfterBlockOffset = CurrentPos;
		AfterBlockSize = Count;
	}
	else
	{
		if (CurrentPos >= PrecacheStartPos)
		{
			// no before block and head of desired block is in the cache
			int64 CopyLen = FMath::Min(PrecacheEndPos - CurrentPos, Count);
			check(CopyLen > 0);
			check(PrecacheBuffer);
			FMemory::Memcpy(Data, PrecacheBuffer + CurrentPos - PrecacheStartPos, CopyLen);
			AfterBlockSize = Count - CopyLen;
			check(AfterBlockSize >= 0);
			AfterBlockOffset = PrecacheEndPos;
		}
		else
		{
			// first part of the block is not in the cache
			BeforeBlockSize = PrecacheStartPos - CurrentPos;
			check(BeforeBlockSize > 0);
			BeforeBlockOffset = CurrentPos;
			if (CurrentPos + Count > PrecacheStartPos)
			{
				// tail of desired block is in the cache
				int64 CopyLen = FMath::Min(PrecacheEndPos - CurrentPos - BeforeBlockSize, Count - BeforeBlockSize);
				check(CopyLen > 0);
				check(PrecacheBuffer);
				FMemory::Memcpy(((uint8*)Data) + BeforeBlockSize, PrecacheBuffer, CopyLen);
				AfterBlockSize = Count - CopyLen - BeforeBlockSize;
				check(AfterBlockSize >= 0);
				AfterBlockOffset = PrecacheEndPos;
			}
		}
	}
	if (BeforeBlockSize)
	{
		LogItem(TEXT("Sync Before Block"), BeforeBlockOffset, BeforeBlockSize);
		if (!PrecacheInternal(BeforeBlockOffset, BeforeBlockSize))
		{
		WaitRead();
		}
		if (IsError())
		{
			return;
		}
		check(BeforeBlockOffset >= PrecacheStartPos && BeforeBlockOffset + BeforeBlockSize <= PrecacheEndPos);
		check(PrecacheBuffer);
		FMemory::Memcpy(Data, PrecacheBuffer + BeforeBlockOffset - PrecacheStartPos, BeforeBlockSize);
	}
	if (AfterBlockSize)
	{
#if defined(ASYNC_WATCH_FILE)
		if (FileName.Contains(TEXT(ASYNC_WATCH_FILE)))
		{
			UE_LOGF(LogAsyncArchive, Warning, "Handy Breakpoint AfterBlockSize");
		}
#endif
		LogItem(TEXT("Sync After Block"), AfterBlockOffset, AfterBlockSize);
		check(int32(LoadPhase) > int32(ELoadPhase::WaitingForSummary));

		int64_t OldPrecacheStartPos = PrecacheStartPos;
		int64_t OldPrecacheEndPos = PrecacheEndPos;
		void* OldRead = ReadRequestPtr;
		int64_t OldReadRequestOffset = ReadRequestOffset;
		int64_t OldReadRequestSize = ReadRequestSize;

		int64_t OldFileSize = FileSize;
		int64_t OldHeaderSizeWhenReadingExportsFromSplitFile = HeaderSizeWhenReadingExportsFromSplitFile;



		if (!PrecacheInternal(AfterBlockOffset, AfterBlockSize))
		{
			verify(WaitRead());
			void* OldRead2 = ReadRequestPtr;
			if (!IsError())
			{
				checkf(AfterBlockOffset >= PrecacheStartPos && AfterBlockOffset + AfterBlockSize <= PrecacheEndPos, 
					TEXT("Sync After Block Wait ????  %lld %lld     %lld %lld <-  %lld %lld     %lld %lld <-  %lld %lld    %p <- %p <- %p    %lld %lld <-  %lld %lld"), 
					AfterBlockOffset, AfterBlockSize, 
					PrecacheStartPos, PrecacheEndPos, OldPrecacheStartPos, OldPrecacheEndPos,
					ReadRequestOffset, ReadRequestSize, OldReadRequestOffset, OldReadRequestSize,
					ReadRequestPtr, OldRead2, OldRead,
					HeaderSizeWhenReadingExportsFromSplitFile, FileSize, OldHeaderSizeWhenReadingExportsFromSplitFile, OldFileSize
				);
			}
		}
		if (IsError())
		{
			return;
		}
		checkf(AfterBlockOffset >= PrecacheStartPos && AfterBlockOffset + AfterBlockSize <= PrecacheEndPos, TEXT("Sync After Block ????   %lld %lld %lld %lld"), AfterBlockOffset, AfterBlockSize, PrecacheStartPos, PrecacheEndPos);
		check(PrecacheBuffer);
		FMemory::Memcpy(((uint8*)Data) + Count - AfterBlockSize, PrecacheBuffer + AfterBlockOffset - PrecacheStartPos, AfterBlockSize);
	}
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	SetPosAndUpdatePrecacheBuffer(CurrentPos + Count);
#else
	CurrentPos += Count;
#endif
}

#if DEVIRTUALIZE_FLinkerLoad_Serialize
void FAsyncArchive::DiscardInlineBufferAndUpdateCurrentPos()
{
	CurrentPos += (ActiveFPLB->StartFastPathLoadBuffer - ActiveFPLB->OriginalFastPathLoadBuffer);
	ActiveFPLB->Reset();
}
#endif

bool IsEventDrivenLoaderEnabled()
{
	return FPlatformProperties::RequiresCookedData();
}