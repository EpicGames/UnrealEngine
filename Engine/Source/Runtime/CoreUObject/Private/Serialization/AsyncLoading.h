// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AsyncLoading.h: Unreal async loading definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "UObject/ObjectResource.h"
#include "UObject/GCObject.h"
#include "Serialization/AsyncPackage.h"
#include "UObject/Package.h"
#include "Templates/Casts.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/LinkerInstancingContext.h"
#include "Templates/RefCounting.h"
#include "Serialization/AsyncPackageLoader.h"

#include "Async/AsyncFileHandle.h"

