// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnAsyncLoading.cpp: Unreal async loading code.
=============================================================================*/

#include "Serialization/AsyncLoading2.h"
#include "Algo/AnyOf.h"
#include "Algo/Partition.h"
#include "AutoRTFM.h"
#include "IO/IoDispatcher.h"
#include "Serialization/AsyncPackageLoader.h"
#include "Serialization/PackageStore.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "HAL/Event.h"
#include "HAL/ThreadManager.h"
#include "HAL/RunnableThread.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/ScopeLock.h"
#include "Stats/StatsMisc.h"
#include "Containers/StripedMap.h"
#include "Misc/CoreStats.h"
#include "HAL/IConsoleManager.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Blueprint/BlueprintSupport.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/AssetReadLoggerConfig.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "Misc/ReverseIterate.h"
#include "Misc/StringBuilder.h"
#include "UObject/ObjectResource.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/NameBatchSerialization.h"
#include "UObject/UObjectGlobalsInternal.h"
#include "Serialization/DeferredMessageLog.h"
#include "UObject/UObjectThreadContext.h"
#include "Misc/Paths.h"
#include "Misc/PlayInEditorLoadingScope.h"
#include "Misc/ExclusiveLoadPackageTimeTracker.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/ExceptionHandling.h"
#include "UObject/UObjectHash.h"
#include "Templates/Casts.h"
#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"
#include "Serialization/BufferReader.h"
#include "Async/TaskGraphInterfaces.h"
#include "Async/UniqueLock.h"
#include "Blueprint/BlueprintSupport.h"
#include "HAL/LowLevelMemTracker.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "UObject/UObjectArchetypeHelper.h"
#include "Misc/MTAccessDetector.h"
#include "UObject/GarbageCollectionInternal.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "Serialization/LoadTimeTracePrivate.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/AsyncPackage.h"
#include "Serialization/UnversionedPropertySerialization.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "Serialization/Zenaphore.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectRedirector.h"
#include "Serialization/BulkData.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/MemoryReader.h"
#include "UObject/UObjectClusters.h"
#include "UObject/CookedMetaData.h"
#include "UObject/LinkerInstancingContext.h"
#include "UObject/LinkerLoadImportBehavior.h"
#include "UObject/ObjectHandlePrivate.h"
#include "UObject/ObjectSerializeAccessScope.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Async/Async.h"
#include "Sanitizer/RaceDetector.h"
#include "Templates/GuardValueAccessors.h"
#include "Async/ParallelFor.h"
#include "Async/ManualResetEvent.h"
#include "Async/RecursiveMutex.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/IPlatformFileOpenLogWrapper.h"
#include "Modules/ModuleManager.h"
#include "Containers/MpscQueue.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/CoreDelegates.h"
#include "Misc/PathViews.h"
#include "UObject/LinkerLoad.h"
#include "IO/IoPriorityQueue.h"
#include "UObject/CoreRedirects.h"
#include "Serialization/ZenPackageHeader.h"
#include "Trace/Trace.h"
#include "Containers/Ticker.h"
#include "Logging/MessageLog.h"
#include "VerseVM/VVMVerse.h"
#include "CoreGlobalsInternal.h"
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/VVMContext.h"
#include "VerseVM/VVMCppClassInfo.h"
#include "VerseVM/VVMPackage.h"
#include "VerseVM/VVMStructuredArchiveVisitor.h"
#endif

#include <atomic>

TAutoConsoleVariable<bool> CVar_CheckBlueprintCDOWillBePostLoadedAfterPreload(
	TEXT("s.CheckBlueprintCDOWillBePostLoadedAfterPreload"),
	true,
	TEXT("Determines whether or not to check that Class Default Object (CDO)s that are compiled from Blueprints (including old VerseVM classes) are"
		"set to be post-loaded and are in the process of being loaded."
		"When disabled, we will still `ensure` regardless, but will not treat these as fatal errors.")
);


#if CSV_PROFILER
TAutoConsoleVariable<bool> CVarAsyncLoadingCsvRegions(
	TEXT("s.AsyncLoadingCsvRegions"),
	false,
	TEXT("If true, emit CSV regions for async loading. These are quite spammy so we want them off by default")
);
#endif

template<typename PayloadType> using TAsyncAtomic = std::atomic<PayloadType>;

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, FileIO);
CSV_DEFINE_STAT(FileIO, FrameCompletedExportBundleLoadsKB);

LLM_DEFINE_TAG(UObject_FAsyncPackage2);
LLM_DEFINE_TAG(AsyncLoading_GlobalImportStore);

// Scope helper class for region instrumentation
struct FAsyncLoadingCsvRegionScope
{
#if CSV_PROFILER
	FAsyncLoadingCsvRegionScope()
	{
		bEnable = CVarAsyncLoadingCsvRegions.GetValueOnAnyThread();
		if (bEnable)
		{
			CSV_REGION_BEGIN(FileIO, TEXT("AsyncLoading"));
		}
	}
	~FAsyncLoadingCsvRegionScope()
	{
		if (bEnable)
		{
			CSV_REGION_END(FileIO, TEXT("AsyncLoading"));
		}
	}
	bool bEnable = false;
#else
	FAsyncLoadingCsvRegionScope() {}
	~FAsyncLoadingCsvRegionScope() {}
#endif
};


FArchive& operator<<(FArchive& Ar, FZenPackageVersioningInfo& VersioningInfo)
{
	Ar << VersioningInfo.ZenVersion;
	Ar << VersioningInfo.PackageVersion;
	Ar << VersioningInfo.LicenseeVersion;
	VersioningInfo.CustomVersions.Serialize(Ar);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FZenPackageImportedPackageNamesContainer& Container)
{
	TArray<FDisplayNameEntryId> NameEntries;
	if (Ar.IsSaving())
	{
#if ALLOW_NAME_BATCH_SAVING
		NameEntries.Reserve(Container.Names.Num());
		for (FName ImportedPackageName : Container.Names)
		{
			NameEntries.Emplace(ImportedPackageName);
		}
		SaveNameBatch(NameEntries, Ar);
		for (FName ImportedPackageName : Container.Names)
		{
			int32 Number = ImportedPackageName.GetNumber();
			Ar << Number;
		}
#else
		check(false);
#endif
	}
	else
	{
		NameEntries = LoadNameBatch(Ar);
		Container.Names.SetNum(NameEntries.Num());
		for (int32 Index = 0; Index < NameEntries.Num(); ++Index)
		{
			int32 Number;
			Ar << Number;
			Container.Names[Index] = NameEntries[Index].ToName(Number);
		}
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FExportBundleEntry& ExportBundleEntry)
{
	Ar << ExportBundleEntry.LocalExportIndex;
	Ar << ExportBundleEntry.CommandType;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FDependencyBundleEntry& DependencyBundleEntry)
{
	Ar << DependencyBundleEntry.LocalImportOrExportIndex;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FDependencyBundleHeader& DependencyBundleHeader)
{
	Ar << DependencyBundleHeader.FirstEntryIndex;
	for (int32 I = 0; I < FExportBundleEntry::ExportCommandType_Count; ++I)
	{
		for (int32 J = 0; J < FExportBundleEntry::ExportCommandType_Count; ++J)
		{
			Ar << DependencyBundleHeader.EntryCount[I][J];
		}
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FScriptObjectEntry& ScriptObjectEntry)
{
	Ar << ScriptObjectEntry.Mapped;
	Ar << ScriptObjectEntry.GlobalIndex;
	Ar << ScriptObjectEntry.OuterIndex;
	Ar << ScriptObjectEntry.CDOClassIndex;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FExportMapEntry& ExportMapEntry)
{
	Ar << ExportMapEntry.CookedSerialOffset;
	Ar << ExportMapEntry.CookedSerialSize;
	Ar << ExportMapEntry.ObjectName;
	Ar << ExportMapEntry.OuterIndex;
	Ar << ExportMapEntry.ClassIndex;
	Ar << ExportMapEntry.SuperIndex;
	Ar << ExportMapEntry.TemplateIndex;
	Ar << ExportMapEntry.PublicExportHash;

	uint32 ObjectFlags = uint32(ExportMapEntry.ObjectFlags);
	Ar << ObjectFlags;

	if (Ar.IsLoading())
	{
		ExportMapEntry.ObjectFlags = EObjectFlags(ObjectFlags);
	}

	uint8 FilterFlags = uint8(ExportMapEntry.FilterFlags);
	Ar << FilterFlags;

	if (Ar.IsLoading())
	{
		ExportMapEntry.FilterFlags = EExportFilterFlags(FilterFlags);
	}

	Ar.Serialize(&ExportMapEntry.Pad, sizeof(ExportMapEntry.Pad));

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FCellExportMapEntry& CellExportMapEntry)
{
	Ar << CellExportMapEntry.CookedSerialOffset;
	Ar << CellExportMapEntry.CookedSerialLayoutSize;
	Ar << CellExportMapEntry.CookedSerialSize;
	Ar << CellExportMapEntry.CppClassInfo;
	Ar << CellExportMapEntry.PublicExportHash;

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FBulkDataMapEntry& BulkDataEntry)
{
	Ar << BulkDataEntry.SerialOffset;
	Ar << BulkDataEntry.DuplicateSerialOffset;
	Ar << BulkDataEntry.SerialSize;
	Ar << BulkDataEntry.Flags;
	Ar << BulkDataEntry.CookedIndex;
	Ar.Serialize(&BulkDataEntry.Pad, 3);

	return Ar;
}

uint64 FPackageObjectIndex::GenerateImportHashFromObjectPath(const FStringView& ObjectPath)
{
	TArray<TCHAR, TInlineAllocator<FName::StringBufferSize>> FullImportPath;
	const int32 Len = ObjectPath.Len();
	FullImportPath.AddUninitialized(Len);
	for (int32 I = 0; I < Len; ++I)
	{
		if (ObjectPath[I] == TEXT('.') || ObjectPath[I] == TEXT(':'))
		{
			FullImportPath[I] = TEXT('/');
		}
		else
		{
			FullImportPath[I] = TChar<TCHAR>::ToLower(ObjectPath[I]);
		}
	}
	uint64 Hash = CityHash64(reinterpret_cast<const char*>(FullImportPath.GetData()), Len * sizeof(TCHAR));
	Hash &= ~(3ull << 62ull);
	return Hash;
}

uint64 FPackageObjectIndex::GenerateImportHashFromVersePath(FUtf8StringView VersePath)
{
	uint64 Hash = CityHash64(reinterpret_cast<const char*>(VersePath.GetData()), VersePath.Len());
	Hash &= ~(3ull << 62ull);
	return Hash;
}

void FindAllRuntimeScriptPackages(FRuntimeScriptPackages& OutPackages)
{
	OutPackages.Script.Empty(256);
	OutPackages.VerseVNI.Empty(256);
	ForEachObjectOfClass(UPackage::StaticClass(), [&OutPackages](UObject* InPackageObj)
	{
		UPackage* Package = CastChecked<UPackage>(InPackageObj);
		if (Package->HasAnyPackageFlags(PKG_CompiledIn))
		{
			TCHAR Buffer[FName::StringBufferSize];
			FStringView NameView(Buffer, Package->GetFName().ToString(Buffer));
			if (NameView.StartsWith(TEXT("/Script/"), ESearchCase::CaseSensitive))
			{
				OutPackages.Script.Add(Package);
			}
			else if (NameView.Contains(TEXT("/_Verse/VNI/"), ESearchCase::CaseSensitive))
			{
				OutPackages.VerseVNI.Add(Package);
			}
		}
	}, /*bIncludeDerivedClasses*/false);
}

#ifndef ALT2_VERIFY_LINKERLOAD_MATCHES_IMPORTSTORE
#define ALT2_VERIFY_LINKERLOAD_MATCHES_IMPORTSTORE 0
#endif

#ifndef ALT2_ENABLE_NEW_ARCHIVE_FOR_LINKERLOAD
#define ALT2_ENABLE_NEW_ARCHIVE_FOR_LINKERLOAD 0
#endif

#ifndef ALT2_LOG_VERBOSE
#define ALT2_LOG_VERBOSE DO_CHECK
#endif

#ifndef ALT2_DUMP_STATE_ON_HANG
#define ALT2_DUMP_STATE_ON_HANG !UE_BUILD_SHIPPING
#endif

#ifndef ALT2_ENABLE_PACKAGE_DEPENDENCY_DEBUGGING
#define ALT2_ENABLE_PACKAGE_DEPENDENCY_DEBUGGING 0
#endif

#ifndef ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING
#define ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING !UE_BUILD_SHIPPING && !IS_PROGRAM
#endif

#if ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING
static bool GEnableScriptImportsDebugging = true;
static FAutoConsoleVariableRef CEnableScriptImportsDebugging(
	TEXT("s.EnableScriptImportsDebugging"),
	GEnableScriptImportsDebugging,
	TEXT("Loads all native script objects (classes, enums, structs, CDOs) that were available in the editor during cook.")
	TEXT("Provides more informative log messages at the cost of up to ~1.5 MiB memory and ~1 MiB FName memory depending on project size."),
	ECVF_Default
);
#endif

#define checkObject(Object, expr) checkf(expr, TEXT("Object='%s' (%p), Flags=%s, InternalFlags=0x%08X") \
									, (Object ? *Object->GetFullName() : TEXT("null")) \
									, Object \
									, (Object ? *LexToString(Object->GetFlags()) : TEXT("None")) \
									, uint32{EnumToUnderlyingType(Object ? Object->GetInternalFlags() : EInternalObjectFlags::None)})

#define ensureObject(Object, expr) ensureMsgf(expr, TEXT("Object='%s' (%p), Flags=%s, InternalFlags=0x%08X") \
									, (Object ? *Object->GetFullName() : TEXT("null")) \
									, Object \
									, (Object ? *LexToString(Object->GetFlags()) : TEXT("None")) \
									, uint32{EnumToUnderlyingType(Object ? Object->GetInternalFlags() : EInternalObjectFlags::None)})

static TSet<FPackageId> GAsyncLoading2_DebugPackageIds;
static FString GAsyncLoading2_DebugPackageNamesString;
static TSet<FPackageId> GAsyncLoading2_VerbosePackageIds;
static FString GAsyncLoading2_VerbosePackageNamesString;
static int32 GAsyncLoading2_VerboseLogFilter = 2; //None=0,Filter=1,All=2
#if !UE_BUILD_SHIPPING
static void ParsePackageNames(const FString& PackageNamesString, TSet<FPackageId>& PackageIds)
{
	TArray<FString> Args;
	const TCHAR* Delimiters[] = { TEXT(","), TEXT(" ") };
	PackageNamesString.ParseIntoArray(Args, Delimiters, UE_ARRAY_COUNT(Delimiters), true);
	PackageIds.Reserve(PackageIds.Num() + Args.Num());
	for (const FString& PackageName : Args)
	{
		if (PackageName.Len() > 0 && FChar::IsDigit(PackageName[0]))
		{
			uint64 Value;
			LexFromString(Value, *PackageName);
			PackageIds.Add(*(FPackageId*)(&Value));
		}
		else
		{
			PackageIds.Add(FPackageId::FromName(FName(*PackageName)));
		}
	}
}
static FAutoConsoleVariableRef CVar_DebugPackageNames(
	TEXT("s.DebugPackageNames"),
	GAsyncLoading2_DebugPackageNamesString,
	TEXT("Add debug breaks for all listed package names, also automatically added to s.VerbosePackageNames."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
		{
			GAsyncLoading2_DebugPackageIds.Reset();
			ParsePackageNames(Variable->GetString(), GAsyncLoading2_DebugPackageIds);
			ParsePackageNames(Variable->GetString(), GAsyncLoading2_VerbosePackageIds);
			GAsyncLoading2_VerboseLogFilter = GAsyncLoading2_VerbosePackageIds.Num() > 0 ? 1 : 2;
		}),
	ECVF_Default);
static FAutoConsoleVariableRef CVar_VerbosePackageNames(
	TEXT("s.VerbosePackageNames"),
	GAsyncLoading2_VerbosePackageNamesString,
	TEXT("Restrict verbose logging to listed package names."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
		{
			GAsyncLoading2_VerbosePackageIds.Reset();
			ParsePackageNames(Variable->GetString(), GAsyncLoading2_VerbosePackageIds);
			GAsyncLoading2_VerboseLogFilter = GAsyncLoading2_VerbosePackageIds.Num() > 0 ? 1 : 2;
		}),
	ECVF_Default);
#endif

#define UE_ASYNC_PACKAGE_DEBUG(PackageDesc) \
if (GAsyncLoading2_DebugPackageIds.Contains((PackageDesc).UPackageId)) \
{ \
	UE_DEBUG_BREAK(); \
}

#define UE_ASYNC_UPACKAGE_DEBUG(UPackage) \
if (GAsyncLoading2_DebugPackageIds.Contains((UPackage)->GetPackageId())) \
{ \
	UE_DEBUG_BREAK(); \
}

#define UE_ASYNC_PACKAGEID_DEBUG(PackageId) \
if (GAsyncLoading2_DebugPackageIds.Contains(PackageId)) \
{ \
	UE_DEBUG_BREAK(); \
}

// The ELogVerbosity::VerbosityMask is used to silence PVS,
// using constexpr gave the same warning, and the disable comment can can't be used in a macro: //-V501 
// warning V501: There are identical sub-expressions 'ELogVerbosity::Verbose' to the left and to the right of the '<' operator.
#define UE_ASYNC_PACKAGE_LOG(Verbosity, PackageDesc, LogDesc, Format, ...) \
if ((ELogVerbosity::Type(ELogVerbosity::Verbosity & ELogVerbosity::VerbosityMask) < ELogVerbosity::Verbose) || \
	(GAsyncLoading2_VerboseLogFilter == 2) || \
	(GAsyncLoading2_VerboseLogFilter == 1 && GAsyncLoading2_VerbosePackageIds.Contains((PackageDesc).UPackageId))) \
{ \
	UE_LOG(LogStreaming, Verbosity, LogDesc TEXT(": %s (0x%s) %s (0x%s) - ") Format, \
		*(PackageDesc).UPackageName.ToString(), \
		*LexToString((PackageDesc).UPackageId), \
		*(PackageDesc).PackagePathToLoad.GetPackageFName().ToString(), \
		*LexToString((PackageDesc).PackageIdToLoad), \
		##__VA_ARGS__); \
}

#define UE_ASYNC_PACKAGE_CLOG(Condition, Verbosity, PackageDesc, LogDesc, Format, ...) \
if ((Condition)) \
{ \
	UE_ASYNC_PACKAGE_LOG(Verbosity, PackageDesc, LogDesc, Format, ##__VA_ARGS__); \
}

#if ALT2_LOG_VERBOSE
#define UE_ASYNC_PACKAGE_LOG_VERBOSE(Verbosity, PackageDesc, LogDesc, Format, ...) \
	UE_ASYNC_PACKAGE_LOG(Verbosity, PackageDesc, LogDesc, Format, ##__VA_ARGS__)
#define UE_ASYNC_PACKAGE_CLOG_VERBOSE(Condition, Verbosity, PackageDesc, LogDesc, Format, ...) \
	UE_ASYNC_PACKAGE_CLOG(Condition, Verbosity, PackageDesc, LogDesc, Format, ##__VA_ARGS__)
#else
#define UE_ASYNC_PACKAGE_LOG_VERBOSE(Verbosity, PackageDesc, LogDesc, Format, ...)
#define UE_ASYNC_PACKAGE_CLOG_VERBOSE(Condition, Verbosity, PackageDesc, LogDesc, Format, ...)
#endif

static bool GRemoveUnreachableObjectsOnGT = false;
static FAutoConsoleVariableRef CVarGRemoveUnreachableObjectsOnGT(
	TEXT("s.RemoveUnreachableObjectsOnGT"),
	GRemoveUnreachableObjectsOnGT,
	TEXT("Remove unreachable objects from GlobalImportStore on the GT from the GC callback NotifyUnreachableObjects (slow)."),
	ECVF_Default
);

static bool GShrinkGlobalImportStoreOnGT = false;
static FAutoConsoleVariableRef CVarGShrinkGlobalImportStoreOnGT(
	TEXT("s.ShrinkGlobalImportStoreOnGT"),
	GShrinkGlobalImportStoreOnGT,
	TEXT("Shrink the GlobalImportStore's working set of memory on the GT from the GC callback NotifyUnreachableObjects."),
	ECVF_Default
);

#if UE_BUILD_DEBUG || (UE_BUILD_DEVELOPMENT && !WITH_EDITOR)
static bool GVerifyUnreachableObjects = true;
static bool GVerifyObjectLoadFlags = true;
#else
static bool GVerifyUnreachableObjects = false;
static bool GVerifyObjectLoadFlags = false;
#endif

static FAutoConsoleVariableRef CVarGVerifyUnreachableObjects(
	TEXT("s.VerifyUnreachableObjects"),
	GVerifyUnreachableObjects,
	TEXT("Run GlobalImportStore verifications for unreachable objects from the GC callback NotifyUnreachableObjects (slow)."),
	ECVF_Default
);

static FAutoConsoleVariableRef CVarGVerifyObjectLoadFlags(
	TEXT("s.VerifyObjectLoadFlags"),
	GVerifyObjectLoadFlags,
	TEXT("Run AsyncFlags verifications for all objects when finished loading from the GC callback NotifyUnreachableObjects (slow)."),
	ECVF_Default
);

static bool GAllowMultithreadedLoading = false;
static FAutoConsoleVariableRef CVarGAllowMultithreadedLoading(
	TEXT("s.AllowMultithreadedLoading"),
	GAllowMultithreadedLoading,
	TEXT("Allow some loading steps to be fully multithreaded."),
	ECVF_Default
);

#if WITH_EDITOR || ENABLE_PGO_PROFILE
static float GAsyncCreatePackagesFromQueueMaxTimeLimitMs = 0.0f;
#else
static float GAsyncCreatePackagesFromQueueMaxTimeLimitMs = 2.5f;
#endif
static FAutoConsoleVariableRef CVarAsyncCreatePackagesFromQueueMaxTimeLimitMs(
	TEXT("s.AsyncCreatePackagesFromQueueMaxTimeLimitMs"),
	GAsyncCreatePackagesFromQueueMaxTimeLimitMs,
	TEXT("Limit the time spent in CreateAsyncPackagesFromQueue when running on the ALT.\n")
	TEXT("This can improve throughput when s.AllowMultithreadedLoading is enabled by allowing MT work to kick off earlier."),
	ECVF_Default
);


static bool GOnlyProcessRequiredPackagesWhenSyncLoading = true;
static FAutoConsoleVariableRef CVarGOnlyProcessRequiredPackagesWhenSyncLoading(
	TEXT("s.OnlyProcessRequiredPackagesWhenSyncLoading"),
	GOnlyProcessRequiredPackagesWhenSyncLoading,
	TEXT("When sync loading a package process only that package and its imports"),
	ECVF_Default
);

static bool GFailLoadOnNotInstalledImport = true;
static FAutoConsoleVariableRef CVarGFailLoadOnNotInstalledImport(
	TEXT("s.FailLoadOnNotInstalledImport"),
	GFailLoadOnNotInstalledImport,
	TEXT("Fail package load if an imported package is not installed"),
	ECVF_Default
);

static bool GFailLoadOnMissingImport = false;
static FAutoConsoleVariableRef CVarGFailLoadOnMissingImport(
	TEXT("s.FailLoadOnMissingImport"),
	GFailLoadOnMissingImport,
	TEXT("Fail package load if an imported package is missing (cannot be found when it should be, i.e. this does not apply to packages that are NotInstalled)"),
	ECVF_Default
);

static bool GAllowPackageRedirectorSupport = WITH_EDITOR;
static FAutoConsoleVariableRef CVarGAllowPackageRedirectorSupport(
	TEXT("s.AllowPackageRedirectorSupport"),
	GAllowPackageRedirectorSupport,
	TEXT("Whether to use package redirector support."),
	ECVF_Default
);

#if WITH_EDITOR
// This is important for the editor because the linker can end up recreating missing exports from its CreateImport function.
// If we don't put the package in the zenloader loading queue, we can be left with objects in a RF_NeedLoad state that will never be actually loaded.
// For the runtime, this should not happen since LinkerLoad is not involved, and we don't want to pay the memory and performance taxes involved in recreating missing exports.
static bool GReloadPackagesWithGCedExports = true;
static FAutoConsoleVariableRef CVarGReloadPackagesWithGCedExports(
	TEXT("s.ReloadPackagesWithGCedExports"),
	GReloadPackagesWithGCedExports,
	TEXT("When active, packages with exports that have been garbage collected will go throught loading again even if they are currently in memory"),
	ECVF_Default
);
#endif

#if USING_INSTRUMENTATION
static float GStallDetectorTimeout = 1200.0f; // 10x factor when instrumentation is enabled
#else
static float GStallDetectorTimeout = 120.0f;
#endif

static FAutoConsoleVariableRef CVarGStallDetectorTimeout(
	TEXT("s.StallDetectorTimeout"),
	GStallDetectorTimeout,
	TEXT("Time in seconds after which we consider the loader stalled if no progress is being made"),
	ECVF_Default
);

static int32 GStallDetectorIdleLoops = 50;
static FAutoConsoleVariableRef CVarGStallDetectorIdleLoops(
	TEXT("s.StallDetectorIdleLoops"),
	GStallDetectorIdleLoops,
	TEXT("The minimum amount of idle loops before considering the loader stalled if no progress is being made"),
	ECVF_Default
);

static bool GAsyncLoading2_AllowPreemptingPackagesDuringGC = true;
static FAutoConsoleVariableRef CVar_AllowPreemptingPackagesDuringGC(
	TEXT("s.AllowPreemptingPackagesDuringGC"),
	GAsyncLoading2_AllowPreemptingPackagesDuringGC,
	TEXT("Allow the async loading thread to get pre-empted by garbage collection while it's creating packages."),
	ECVF_Default);

#if USING_INSTRUMENTATION
static bool GDetectRaceDuringLoading = false;
static FAutoConsoleVariableRef CVarDetectRaceDuringLoading(
	TEXT("s.DetectRaceDuringLoading"),
	GDetectRaceDuringLoading,
	TEXT("Activate the race detector during loading periods"),
	ECVF_Default
);

static float GDetectRaceGracePeriod = 1.0f;
static FAutoConsoleVariableRef CVarDetectRaceGracePeriod(
	TEXT("s.DetectRaceGracePeriod"),
	GDetectRaceGracePeriod,
	TEXT("Amount of seconds to stay on after all loading is done. Can improve performance by reducing the rate of toggling."),
	ECVF_Default
);
#endif // USING_INSTRUMENTATION

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, Basic);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(CORE_API, FileIO);

TRACE_DECLARE_ATOMIC_INT_COUNTER(AsyncLoadingQueuedPackages, TEXT("AsyncLoading/PackagesQueued"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(AsyncLoadingLoadingPackages, TEXT("AsyncLoading/PackagesLoading"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(AsyncLoadingPackagesWithRemainingWork, TEXT("AsyncLoading/PackagesWithRemainingWork"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(AsyncLoadingPendingIoRequests, TEXT("AsyncLoading/PendingIoRequests"));
TRACE_DECLARE_ATOMIC_MEMORY_COUNTER(AsyncLoadingTotalLoaded, TEXT("AsyncLoading/TotalLoaded"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(AsyncLoadingFileSystemLoads, TEXT("AsyncLoadingThread/FileSystemLoads"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(AsyncLoadingIoStoreLoads, TEXT("AsyncLoadingThread/IoStoreLoads"));

FString FormatPackageId(FPackageId PackageId)
{
#if WITH_PACKAGEID_NAME_MAP
	return FString::Printf(TEXT("0x%s (%s)"), *LexToString(PackageId), *PackageId.GetName().ToString());
#else
	return FString::Printf(TEXT("0x%s"), *LexToString(PackageId));
#endif
}

// Check if an export should be loaded or not on the current platform/build
// Returns true if the export should be skipped, false if it should be kept
static bool AsyncLoading2_ShouldSkipLoadingExport(const EExportFilterFlags FilterFlags)
{
#if WITH_EDITOR
	return false;
#elif UE_SERVER
	return !!(static_cast<uint32>(FilterFlags) & static_cast<uint32>(EExportFilterFlags::NotForServer));
#elif !WITH_SERVER_CODE
	return !!(static_cast<uint32>(FilterFlags) & static_cast<uint32>(EExportFilterFlags::NotForClient));
#else
	static const bool bIsDedicatedServer = !GIsClient && GIsServer;
	static const bool bIsClientOnly = GIsClient && !GIsServer;

	if (bIsDedicatedServer && static_cast<uint32>(FilterFlags) & static_cast<uint32>(EExportFilterFlags::NotForServer))
	{
		return true;
	}

	if (bIsClientOnly && static_cast<uint32>(FilterFlags) & static_cast<uint32>(EExportFilterFlags::NotForClient))
	{
		return true;
	}

	return false;
#endif
}

/** Returns the CoreRedirected package name. Otherwise returns the name unchanged. */
FName ApplyPackageNameRedirections(FName PackageName)
{
	FName RedirectedName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(NAME_None, NAME_None, PackageName)).PackageName;
	if (RedirectedName != PackageName)
	{
		UE_LOGF(LogStreaming, Verbose, "Applying core redirection for package %ls to package %ls", *PackageName.ToString(), *RedirectedName.ToString());
	}

	return RedirectedName;
}

/** Returns localized package name to load when running as -game or using loose cooked files in a runtime process. Otherwise returns the name unchanged. */
FName ApplyLooseFileLocalizationPackageNameRedirects(FName PackageName)
{
	if (!GIsEditor && FAsyncLoadingThreadSettings::Get().bLooseFileLoadingEnabled)
	{
		UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(INDEX_NONE);
		FName RedirectedName = FPackageLocalizationManager::Get().FindLocalizedPackageName(PackageName);
		if (!RedirectedName.IsNone())
		{
			UE_LOGF(LogStreaming, Verbose, "Applying localization redirection for package %ls to package %ls", *PackageName.ToString(), *RedirectedName.ToString());
			PackageName = RedirectedName;
		}
	}
	return PackageName;
}

struct FAsyncPackage2;
class FAsyncLoadingThread2;

struct FExportObject
{
	UObject* Object = nullptr;
	UObject* TemplateObject = nullptr;
	UObject* SuperObject = nullptr;
	bool bFiltered = false;
	bool bExportLoadFailed = false;
	bool bWasFoundInMemory = false;
};
struct FExportCell
{
	Verse::VCell* Cell = nullptr;
	bool bSerialized = false;
};

struct FPackageReferencer
{
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	FName ReferencerPackageName;
	FName ReferencerPackageOp;
#endif

#if WITH_EDITOR
	ECookLoadType CookLoadType = ECookLoadType::Unspecified;
#endif

	static FPackageReferencer FromImport(FName ReferencerName)
	{
		FPackageReferencer Result;
#if UE_WITH_PACKAGE_ACCESS_TRACKING
		Result.ReferencerPackageName = ReferencerName;
		Result.ReferencerPackageOp = PackageAccessTrackingOps::NAME_Load;
#endif
#if WITH_EDITOR
		Result.CookLoadType = ECookLoadType::Unspecified;
#endif
		return Result;
	}
};

struct FLoadedObjectRenamed
{
	int32 ObjectIndex;
	UObject* Object;
};

struct FPackageRequest
{
	int32 RequestId = -1;
	int32 Priority = -1;
	EPackageFlags PackageFlags = PKG_None;
#if WITH_EDITOR
	uint32 LoadFlags = LOAD_None;
	int32 PIEInstanceID = INDEX_NONE;
#endif
	FLinkerInstancingContext InstancingContext;
	FName CustomName;
	FPackagePath PackagePath;
	TUniquePtr<FLoadPackageAsyncDelegate> PackageLoadedDelegate;
	TSharedPtr<FLoadPackageAsyncProgressDelegate> PackageProgressDelegate;
	FPackageRequest* Next = nullptr;
	FPackageReferencer PackageReferencer;

	FLinkerInstancingContext* GetInstancingContext()
	{
		return &InstancingContext;
	}

	static FPackageRequest Create(int32 RequestId, EPackageFlags PackageFlags, uint32 LoadFlags, int32 PIEInstanceID, int32 Priority, const FLinkerInstancingContext* InstancingContext, const FPackagePath& PackagePath, FName CustomName, TUniquePtr<FLoadPackageAsyncDelegate> PackageLoadedDelegate, TSharedPtr<FLoadPackageAsyncProgressDelegate> PackageProgressDelegate, FPackageReferencer PackageReferencer)
	{
		return FPackageRequest
		{
			RequestId,
			Priority,
			PackageFlags,
#if WITH_EDITOR
			LoadFlags,
			PIEInstanceID,
#endif
			InstancingContext ? FLinkerInstancingContext(*InstancingContext, FLinkerInstancingContext::EContextSharingMode::Link) : FLinkerInstancingContext(),
			CustomName,
			PackagePath,
			MoveTemp(PackageLoadedDelegate),
			PackageProgressDelegate,
			nullptr,
			PackageReferencer
		};
	}
};

const TCHAR* LexToString(EPackageLoader PackageLoader)
{
	switch (PackageLoader)
	{
	case EPackageLoader::LinkerLoad: return TEXT("LinkerLoad");
	case EPackageLoader::Zen: return TEXT("Zen");
	case EPackageLoader::Unknown: return TEXT("Unknown");
	default:
	checkNoEntry();
	return TEXT("Not Implemented");
	}
};

struct FAsyncPackageDesc2
{
	// A unique request id for each external call to LoadPackage
	int32 RequestID;
	// Package priority
	TSAN_ATOMIC(int32) Priority;
	/** The flags that should be applied to the package */
	EPackageFlags PackageFlags;
#if WITH_EDITOR
	uint32 LoadFlags;
	/** PIE instance ID this package belongs to, INDEX_NONE otherwise */
	int32 PIEInstanceID;
#endif
	/** Instancing context, maps original package to their instanced counterpart, used to remap imports. */
	FLinkerInstancingContext InstancingContext;
	// The package id of the UPackage being loaded
	// It will be used as key when tracking active async packages
	FPackageId UPackageId;
	// The id of the package being loaded from disk
	FPackageId PackageIdToLoad;
	// The name of the UPackage being loaded
	// Set to none for imported packages up until the package summary has been serialized
	FName UPackageName;
	// The package path of the package being loaded from disk
	// Set to none for imported packages up until the package summary has been serialized
	FPackagePath PackagePathToLoad;
	// Package referencer
	FPackageReferencer PackageReferencer;
	// If the package can be imported
	bool bCanBeImported;
	// Indicates which loader to use when loading the package
	EPackageLoader Loader;

	static FAsyncPackageDesc2 FromPackageRequest(
		FPackageRequest& Request,
		FName UPackageName,
		FPackageId PackageIdToLoad,
		EPackageLoader Loader)
	{
		// Note, we intentionally create FAsyncPackageDesc2 for missing packages for deferred 
		// processing so it's possible for the Loader to be EPackageLoader::Unknown
		return FAsyncPackageDesc2
		{
			Request.RequestId,
			Request.Priority,
			Request.PackageFlags,
#if WITH_EDITOR
			Request.LoadFlags,
			Request.PIEInstanceID,
#endif
			MoveTemp(Request.InstancingContext),
			FPackageId::FromName(UPackageName),
			PackageIdToLoad,
			UPackageName,
			MoveTemp(Request.PackagePath),
			Request.PackageReferencer,
			true,
			Loader
		};
	}

	static FAsyncPackageDesc2 FromPackageImport(
		const FAsyncPackageDesc2& ImportingPackageDesc,
		FName UPackageName,
		FPackageId ImportedPackageId,
		FPackageId PackageIdToLoad,
		FPackagePath&& PackagePathToLoad,
		EPackageLoader Loader)
	{
		// Note, we intentionally create FAsyncPackageDesc2 for missing packages for deferred 
		// processing so it's possible for the Loader to be EPackageLoader::Unknown
		return FAsyncPackageDesc2
		{
			INDEX_NONE,
			ImportingPackageDesc.Priority,
			PKG_None,
#if WITH_EDITOR
			LOAD_None,
			INDEX_NONE,
#endif
			FLinkerInstancingContext(),
			ImportedPackageId,
			PackageIdToLoad,
			UPackageName,
			MoveTemp(PackagePathToLoad),
			FPackageReferencer::FromImport(ImportingPackageDesc.UPackageName),
			true,
			Loader
		};
	}
};

struct FUnreachableObject
{
	FPackageId PackageId;
	int32 ObjectIndex = -1;
	FName ObjectName;
};

using FUnreachableObjects = TArray<FUnreachableObject>;

// Provides us with a way to profile contention.
class FAsyncLoadingMutex
{
	UE::FSharedMutex Mutex;
public:
	void Lock()
	{
		if (!Mutex.TryLock())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncLoadingMutexWriteLockContention);
			Mutex.Lock();
		}
	}
	
	bool IsLocked() const
	{
		return Mutex.IsLocked();
	}
	
	void Unlock()
	{
		Mutex.Unlock();
	}

	void LockShared()
	{
		if (!Mutex.TryLockShared())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncLoadingMutexReadLockContention);
			Mutex.LockShared();
		}
	}

	bool IsLockShared() const
	{
		return Mutex.IsLockShared();
	}

	void UnlockShared()
	{
		Mutex.UnlockShared();
	}
};

// This needs to stay below 48 bytes as much as possible
// in the runtime version, the next bin size is currently 64 bytes.
class FLoadedPackageRef
{
private:
	friend class FGlobalImportStore;
	friend class FPublicExportMapBasicTest;
	friend class FPublicExportMapIterationTest;
	friend class FPublicExportMapPreInsertPublicExportsTest;
	template <int32 Type> friend class FLockedPackageRef;

	class FPublicExportMap
	{
	public:
		static const int32 InvalidValue = -1;

		FPublicExportMap() = default;

		FPublicExportMap(const FPublicExportMap&) = delete;

		FPublicExportMap(FPublicExportMap&& Other)
		{
			Allocation = Other.Allocation;
			Capacity = Other.Capacity;
			Count = Other.Count;
			SingleItemValue = Other.SingleItemValue;
			Other.Allocation = nullptr;
			Other.Capacity = 0;
			Other.Count = 0;
		};

		FPublicExportMap& operator=(const FPublicExportMap&) = delete;

		FPublicExportMap& operator=(FPublicExportMap&& Other)
		{
			if (Capacity > 1)
			{
				FMemory::Free(Allocation);
			}
			Allocation = Other.Allocation;
			Capacity = Other.Capacity;
			Count = Other.Count;
			SingleItemValue = Other.SingleItemValue;
			Other.Allocation = nullptr;
			Other.Capacity = 0;
			Other.Count = 0;
			return *this;
		}

		~FPublicExportMap()
		{
			if (Capacity > 1)
			{
				FMemory::Free(Allocation);
			}
		}
		void Reserve(int32 NewCapacity)
		{
			if (NewCapacity <= Capacity)
			{
				return;
			}

			if (NewCapacity > 1)
			{
				TArrayView<uint64> OldKeys = GetKeys();
				TArrayView<int32> OldValues = GetValues();
				const uint64 OldKeysSize = Capacity * sizeof(uint64);
				const uint64 NewKeysSize = NewCapacity * sizeof(uint64);
				const uint64 OldValuesSize = Capacity * sizeof(int32);
				const uint64 NewValuesSize = NewCapacity * sizeof(int32);
				const uint64 KeysToAddSize = NewKeysSize - OldKeysSize;
				const uint64 ValuesToAddSize = NewValuesSize - OldValuesSize;

				uint8* NewAllocation = reinterpret_cast<uint8*>(FMemory::Malloc(NewKeysSize + NewValuesSize));
				FMemory::Memzero(NewAllocation, KeysToAddSize); // Insert new keys initialized to zero
				FMemory::Memcpy(NewAllocation + KeysToAddSize, OldKeys.GetData(), OldKeysSize); // Copy old keys
				FMemory::Memset(NewAllocation + NewKeysSize, 0xFF, ValuesToAddSize); // Insert new values initialized to InvalidValue
				FMemory::Memcpy(NewAllocation + NewKeysSize + ValuesToAddSize, OldValues.GetData(), OldValuesSize); // Copy old values

				if (Capacity > 1)
				{
					FMemory::Free(Allocation);
				}
				Allocation = NewAllocation;
			}
			Capacity = NewCapacity;
		}

		void PreInsertPublicExports(TConstArrayView<TConstArrayView<FExportMapEntry>> ExportLists)
		{
			if (Capacity <= 1)
			{
				int32 PublicExportCount = 0;
				uint64 LastNewKeySeen = 0;
				for (TConstArrayView<FExportMapEntry> Exports : ExportLists)
				{
					for (const FExportMapEntry& Entry : Exports)
					{
						if (Entry.PublicExportHash && !AsyncLoading2_ShouldSkipLoadingExport(Entry.FilterFlags))
						{
							PublicExportCount++;
							LastNewKeySeen = Entry.PublicExportHash;
						}
					}
				}

				if (PublicExportCount == 0)
				{
					return;
				}

				// Either: 
				// 	We're empty so we can just insert all the hashes with an associated value of -1 and sort.
				//  We have a single pair in the SingleItemKey/SingleItemValue slots that we want to keep the mapping for.
				const uint64 OldKey = Capacity == 1 ? SingleItemKey : 0;
				const int32 OldValue = Capacity == 1 ? SingleItemValue : InvalidValue;

				// We will be clearing all values during preinsert, thus Count is reset. We will restore
				// Count when we Store() any valid OldValue below
				Count = 0;

				if (PublicExportCount == 1)
				{
					SingleItemKey = LastNewKeySeen;
					SingleItemValue = InvalidValue;
					Capacity = 1;
				}
				else
				{
					const int32 NewCapacity = PublicExportCount;
					const uint64 NewKeysSize = NewCapacity * sizeof(uint64);
					const uint64 NewValuesSize = NewCapacity * sizeof(int32);
					uint8* NewAllocation = reinterpret_cast<uint8*>(FMemory::Malloc(NewKeysSize + NewValuesSize));

					TArrayView<uint64> NewKeys = MakeArrayView(reinterpret_cast<uint64*>(NewAllocation), NewCapacity);
					TArrayView<int32> NewValues = MakeArrayView(reinterpret_cast<int32*>(NewAllocation + sizeof(uint64) * NewCapacity), NewCapacity);
					int32 Index = 0;

					for (TConstArrayView<FExportMapEntry> Exports : ExportLists)
					{
						for (const FExportMapEntry& Entry : Exports)
						{
							if (Entry.PublicExportHash && !AsyncLoading2_ShouldSkipLoadingExport(Entry.FilterFlags))
							{
								NewKeys[Index++] = Entry.PublicExportHash;
							}
						}
					}

					Algo::Sort(NewKeys);

					// Set all values to -1
					FMemory::Memset(NewValues.GetData(), 0xFF, NewValues.NumBytes());
					Allocation = NewAllocation;
					Capacity = NewCapacity;
				}

				// If we had a single object previously, re-store its old value.
				// If the old key was present in Exports, this will overwrite the -1 we just stored.
				// If it wasn't in Exports, this will grow the storage, but we consider that unlikely at present.
				if (OldValue != InvalidValue)
				{
					Store(OldKey, OldValue);
				}
			}
			else
			{
				check(Allocation);
				const int32 OldCapacity = Capacity;
				TArrayView<uint64> OldKeys = GetKeys();
				TArrayView<int32> OldValues = GetValues();
				TArray<uint64, TInlineAllocator<256>> KeysToAdd;
				int32 MaxKeysToAdd = 0;
				for (TConstArrayView<FExportMapEntry> Exports : ExportLists)
				{
					MaxKeysToAdd += Exports.Num();
				}
				KeysToAdd.Reserve(MaxKeysToAdd);

				for (TConstArrayView<FExportMapEntry> Exports : ExportLists)
				{
					for (const FExportMapEntry& Entry : Exports)
					{
						if (Entry.PublicExportHash && !AsyncLoading2_ShouldSkipLoadingExport(Entry.FilterFlags))
						{
							int32 Index = Algo::LowerBound(OldKeys, Entry.PublicExportHash);
							if (Index >= Capacity || OldKeys[Index] != Entry.PublicExportHash)
							{
								KeysToAdd.Add(Entry.PublicExportHash);
							}
						}
					}
				}

				// We expect the most common case here to be KeysToAdd.Num() == 0 as we're likely to be reloading the same package data.
				// If we aren't, it implies some UGC/delivered content scenario where the old content was not fully GC'd before attemping 
				// to reload the newly downloaded version.
				if (KeysToAdd.Num() > 0)
				{
					// Sort keys to add so that we can interleave old and new keys to maintain sorting
					Algo::Sort(KeysToAdd);
					const int32 KeysToAddCount = KeysToAdd.Num();

					// At this point, Capacity > 1 and KeysToAdd.Num() >= 1, so we don't need to consider the single object case and will always allocate.
					const int32 NewCapacity = OldCapacity + KeysToAddCount;
					const uint64 NewKeysSize = NewCapacity * sizeof(uint64);
					const uint64 NewValuesSize = NewCapacity * sizeof(int32);
					uint8* NewAllocation = reinterpret_cast<uint8*>(FMemory::Malloc(NewKeysSize + NewValuesSize));

					TArrayView<uint64> NewKeys = MakeArrayView(reinterpret_cast<uint64*>(NewAllocation), NewCapacity);
					TArrayView<int32> NewValues = MakeArrayView(reinterpret_cast<int32*>(NewAllocation + sizeof(uint64) * NewCapacity), NewCapacity);
					for (int32 InsertIndex = 0, OldIndex = 0, ToAddIndex = 0; InsertIndex < NewCapacity; ++InsertIndex)
					{
						if (OldKeys.IsValidIndex(OldIndex)
						&& (!KeysToAdd.IsValidIndex(ToAddIndex) || OldKeys[OldIndex] < KeysToAdd[ToAddIndex]))
						{
							NewKeys[InsertIndex] = OldKeys[OldIndex];
							NewValues[InsertIndex] = OldValues[OldIndex];
							++OldIndex;
						}
						else
						{
							NewKeys[InsertIndex] = KeysToAdd[ToAddIndex];
							NewValues[InsertIndex] = InvalidValue;
							++ToAddIndex;
						}
					}
					FMemory::Free(Allocation);
					Allocation = NewAllocation;
					Capacity = NewCapacity;
				}
			}
		}

		void Store(uint64 ExportHash, int32 ObjectIndex, int32 Index = INDEX_NONE)
		{
			checkf(ExportHash, TEXT("Invalid to store a hash of 0 in this map"));
			checkf(ObjectIndex != InvalidValue, TEXT("Use Remove() instead"));

			TArrayView<uint64> Keys = GetKeys();
			TArrayView<int32> Values = GetValues();
			if (Index == INDEX_NONE)
			{
				Index = Algo::LowerBound(Keys, ExportHash);
			}
			if (Index < Capacity && Keys[Index] == ExportHash)
			{
				// Slot already exists so reuse it
				int32& ExistingValue = Values[Index];

				// If we are storing a valid value on top of an invalid value 
				// we have increased the number of valid indices
				Count += ExistingValue == InvalidValue;
				ExistingValue = ObjectIndex;
				return;
			}

			if (Capacity == 0 || Keys[0] != 0)
			{
				// No free slots so we need to add one (will be inserted at the beginning of the array)
				Reserve(Capacity + 1);
				Keys = GetKeys();
				Values = GetValues();
			}
			else
			{
				--Index; // Update insertion index to one before the lower bound item
			}

			if (Index > 0)
			{
				// Move items down
				FMemory::Memmove(Keys.GetData(), Keys.GetData() + 1, Index * sizeof(uint64));
				FMemory::Memmove(Values.GetData(), Values.GetData() + 1, Index * sizeof(int32));
			}
			Keys[Index] = ExportHash;
			Values[Index] = ObjectIndex;
			Count++;
		}

		bool Remove(uint64 ExportHash)
		{
			TArrayView<uint64> Keys = GetKeys();
			int32 Index = Algo::LowerBound(Keys, ExportHash);
			if (Index < Capacity && Keys[Index] == ExportHash)
			{
				TArrayView<int32> Values = GetValues();
				int32& ExistingValue = Values[Index];

				const bool bRemovedValue = ExistingValue != InvalidValue;
				Count -= bRemovedValue;
				ExistingValue = InvalidValue;

				return bRemovedValue;
			}

			return false;
		}

		// Returns the value stored if any, returning InvalidValue for not present or removed values
		int32 Find(uint64 ExportHash, int32* OutHashIndex = nullptr)
		{
			TArrayView<uint64> Keys = GetKeys();
			int32 Index = Algo::LowerBound(Keys, ExportHash);
			if (OutHashIndex)
			{
				*OutHashIndex = Index;
			}
			if (Index < Capacity && Keys[Index] == ExportHash)
			{
				TArrayView<int32> Values = GetValues();
				return Values[Index];
			}
			return InvalidValue;
		}

		[[nodiscard]] bool PinForGC(TArray<int32>& OutUnreachableObjectIndices)
		{
			OutUnreachableObjectIndices.Reset();
			int ValidExportCount = 0;
			for (int32& ObjectIndex : GetValues())
			{
				if (ObjectIndex >= 0)
				{
					FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(ObjectIndex);
					if (!ObjectItem->IsUnreachable())
					{
						UObject* Object = static_cast<UObject*>(ObjectItem->GetObject());
						checkObject(Object, !ObjectItem->HasAnyFlags(EInternalObjectFlags::LoaderImport));
						ObjectItem->SetFlags(EInternalObjectFlags::LoaderImport);
						ValidExportCount++;
					}
					else
					{
						OutUnreachableObjectIndices.Reserve(Capacity);
						OutUnreachableObjectIndices.Add(ObjectIndex);
						ObjectIndex = InvalidValue;
						Count--;
					}
				}
			}
			check(ValidExportCount == Count);
			return OutUnreachableObjectIndices.Num() == 0;
		}

		void UnpinForGC()
		{
			for (int32 ObjectIndex : GetValues())
			{
				if (ObjectIndex >= 0)
				{
					UObject* Object = static_cast<UObject*>(GUObjectArray.IndexToObject(ObjectIndex)->GetObject());
					checkObject(Object, Object->HasAnyInternalFlags(EInternalObjectFlags::LoaderImport));
					Object->AtomicallyClearInternalFlags(EInternalObjectFlags::LoaderImport);
				}
			}
		}

		TArrayView<uint64> GetKeys()
		{
			if (Capacity == 1)
			{
				return MakeArrayView(&SingleItemKey, 1);
			}
			else
			{
				return MakeArrayView(reinterpret_cast<uint64*>(Allocation), Capacity);
			}
		}

		TArrayView<int32> GetValues()
		{
			if (Capacity == 1)
			{
				return MakeArrayView(&SingleItemValue, 1);
			}
			else
			{
				return MakeArrayView(reinterpret_cast<int32*>(Allocation + Capacity * sizeof(uint64)), Capacity);
			}
		}

		class FValueIterator
		{
			TArrayView<int32> Array;
			int32 Pos;
		public:
			FValueIterator(const TArrayView<int32>& Data) : Array(Data), Pos(0)
			{
				const int32 ArraySize = Array.Num();

				// Move to the first valid element
				while (Pos < ArraySize && Array[Pos] == FPublicExportMap::InvalidValue)
				{
					++Pos;
				}
			}

			FORCEINLINE bool operator== (const FValueIterator& Other) const
			{
				return Pos == Other.Pos && Array.GetData() == Other.Array.GetData();
			}

			FORCEINLINE bool operator!= (const FValueIterator& Other) const
			{
				return !(Pos == Other.Pos && Array.GetData() == Other.Array.GetData());
			}

			FORCEINLINE explicit operator bool() const
			{
				return Pos < Array.Num();
			}

			FORCEINLINE FValueIterator& operator++()
			{
				const int32 ArraySize = Array.Num();
				do
				{
					++Pos;
				} while (Pos < ArraySize && Array[Pos] == FPublicExportMap::InvalidValue);
				return *this;
			}

			FORCEINLINE int32 operator*()
			{
				return Array[Pos];
			}

			FORCEINLINE void RemoveCurrent()
			{
				Array[Pos] = FPublicExportMap::InvalidValue;
			}
		};

		FValueIterator CreateValueIterator()
		{
			if (Capacity == 1)
			{
				return FValueIterator{ MakeArrayView(&SingleItemValue, 1) };
			}
			else
			{
				return FValueIterator{ MakeArrayView(reinterpret_cast<int32*>(Allocation + Capacity * sizeof(uint64)), Capacity) };
			}
		}

		inline int32 Num() const
		{
			return Count;
		}

	private:
		union
		{
			uint8* Allocation = nullptr;
			uint64 SingleItemKey;
		};

		// Capacity is the total number of keys stored, not necessarily number of keys that map to valid values
		// Capacity == 1 means we use SingleItemKey and SingleItemValue 
		// Capacity > 1 means we use Allocation 
		// Count represents the total valid values (non-invalid values, the objects may still be unreachable)
		int32 Count = 0;
		int32 Capacity = 0;
		int32 SingleItemValue = -1;
	};

	FPublicExportMap PublicExportMap;
	FName OriginalPackageName;
	int32 PackageObjectIndex = -1;
	uint32 PackageRefCount = 0;         // Package references during loading activity. We've seen 16-bit not being enough, 24-bit should probably be enough if we need to compact it in the future.
#if WITH_EDITOR
	int32 ExportCount = -1;
#endif
	std::atomic<uint16> RefCount = 1;   // Only taken by thread activity, the max count should be proportional to the number of worker threads.

	// This mutex protects all the fields of this class via a FLockedPackageRef at the exception of RefCount, 
	// which is a typical reference counting using an atomic for synchronization.
	FAsyncLoadingMutex Mutex;
	EPackageExtension PackageHeaderExtension = EPackageExtension::Unspecified;
	EPackageLoader PackageLoader = EPackageLoader::Unknown;
	uint8 bWasFullyLoaded : 1 = false;				// Was the package loading ever completed
	uint8 bAreAllPublicExportsLoaded : 1 = false;	// Are all loaded objects still valid
	uint8 bIsNotInstalled : 1 = false;
	uint8 bIsMissing : 1 = false;
	uint8 bHasFailed : 1 = false;
	uint8 bHasBeenLoadedDebug : 1 = false;
	
	void Reset()
	{
		check(Mutex.IsLocked());
		PublicExportMap = FPublicExportMap();
		OriginalPackageName = FName();
		PackageObjectIndex = -1;
		PackageRefCount = 0;
#if WITH_EDITOR
		ExportCount = -1;
#endif
		PackageHeaderExtension = EPackageExtension::Unspecified;
		PackageLoader = EPackageLoader::Unknown;
		bWasFullyLoaded = false;
		bAreAllPublicExportsLoaded = false;
		bIsNotInstalled = false;
		bIsMissing = false;
		bHasFailed = false;
		bHasBeenLoadedDebug = false;
	}
public:
	void AddRef()
	{
		check(GetRefCount() < TNumericLimits<decltype(RefCount.load())>::Max());
		RefCount.fetch_add(1, std::memory_order_relaxed);
	}

	void Release()
	{
		if (RefCount.fetch_sub(1, std::memory_order_relaxed) == 1)
		{
			delete this;
		}
	}
	
	int32 GetRefCount() const
	{
		return RefCount.load(std::memory_order_relaxed);
	}

	FLoadedPackageRef() = default;

	FLoadedPackageRef(const FLoadedPackageRef& Other) = delete;
	FLoadedPackageRef(FLoadedPackageRef&& Other) = delete;
	FLoadedPackageRef& operator=(const FLoadedPackageRef& Other) = delete;
	FLoadedPackageRef& operator=(FLoadedPackageRef&& Other) = delete;

	void AddPackageRef()
	{
		check(Mutex.IsLocked());
		check(PackageRefCount < TNumericLimits<decltype(PackageRefCount)>::Max());
		++PackageRefCount;
	}

	void ReleasePackageRef()
	{
		check(Mutex.IsLocked());
		check(PackageRefCount > 0);
		--PackageRefCount;
	}

	inline int32 GetPackageRefCount() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return PackageRefCount;
	}

	inline FName GetOriginalPackageName() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return OriginalPackageName;
	}

	inline bool HasPackage() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return PackageObjectIndex >= 0;
	}

	inline bool HasErrors() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return bHasFailed || bIsMissing || bIsNotInstalled;
	}

	inline bool IsFailed() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return bHasFailed;
	}

	inline bool IsMissing() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return bIsMissing;
	}

	inline bool IsNotInstalled() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return bIsNotInstalled;
	}

	inline EPackageLoader GetPackageLoader() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return PackageLoader;
	}

	inline EPackageExtension GetPackageHeaderExtension() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return PackageHeaderExtension;
	}

	inline UPackage* GetPackage() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		if (HasPackage())
		{
			return static_cast<UPackage*>(GUObjectArray.IndexToObject(PackageObjectIndex)->GetObject());
		}
		return nullptr;
	}

	inline void SetPackage(UPackage* InPackage)
	{
		check(Mutex.IsLocked());
		check(!bWasFullyLoaded);
		check(!bAreAllPublicExportsLoaded);
		check(!bIsMissing);
		check(!bIsNotInstalled);
		check(!bHasFailed);
		check(!HasPackage());
		if (InPackage)
		{
			PackageObjectIndex = GUObjectArray.ObjectToIndex(InPackage);
			OriginalPackageName = InPackage->GetFName();
		}
		else
		{
			PackageObjectIndex = -1;
			OriginalPackageName = FName();
		}
	}

	void RemoveUnreferencedObsoletePackage()
	{
		check(PackageRefCount == 0);
		Reset();
	}

	void ReplaceReferencedRenamedPackage(UPackage* NewPackage)
	{
		check(Mutex.IsLocked());
		// keep RefCount and PublicExportMap while resetting all other state,
		// the public exports will be replaced one by one from StoreGlobalObject
		bWasFullyLoaded = false;
		bAreAllPublicExportsLoaded = false;
		bIsMissing = false;
		bHasFailed = false;
		bHasBeenLoadedDebug = false;
		PackageObjectIndex = GUObjectArray.ObjectToIndex(NewPackage);
		OriginalPackageName = NewPackage->GetFName();
	}

	inline bool AreAllPublicExportsLoaded() const
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return bAreAllPublicExportsLoaded && OriginalPackageName == GetPackage()->GetFName();
	}

	inline void SetAllPublicExportsLoaded(bool bSnapshotExportCount)
	{
		check(Mutex.IsLocked());
		check(!bIsMissing);
		check(!bIsNotInstalled);
		check(!bHasFailed);
		check(HasPackage());
		bIsMissing = false;
		bWasFullyLoaded = true;
		bAreAllPublicExportsLoaded = true;
		bHasBeenLoadedDebug = true;
#if WITH_EDITOR
		if (bSnapshotExportCount)
		{
			ExportCount = PublicExportMap.Num();
		}
#endif
	}

	inline void SetIsMissingPackage()
	{
		check(Mutex.IsLocked());
		check(!bWasFullyLoaded);
		check(!bAreAllPublicExportsLoaded);
		check(!HasPackage());
		bIsMissing = true;
		bWasFullyLoaded = false;
		bAreAllPublicExportsLoaded = false;
	}

	inline void SetIsNotInstalledPackage()
	{
		check(Mutex.IsLocked());
		check(!bWasFullyLoaded);
		check(!bAreAllPublicExportsLoaded);
		check(!HasPackage());
		bIsNotInstalled = true;
		bWasFullyLoaded = false;
		bAreAllPublicExportsLoaded = false;
	}

	inline void ClearErrorFlags()
	{
		check(Mutex.IsLocked());
		bIsNotInstalled = false;
		bIsMissing = false;
		bHasFailed = false;
	}

	inline void SetHasFailed()
	{
		check(Mutex.IsLocked());
		bHasFailed = true;
	}

	FPublicExportMap::FValueIterator GetPublicExportObjectIndices()
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		return PublicExportMap.CreateValueIterator();
	}

	/** Insert all the given exports into the map by their hash, storing a non-present/null value for any objects that are not already in the map */
	void PreInsertPublicExports(TConstArrayView<TConstArrayView<FExportMapEntry>> Exports)
	{
		check(Mutex.IsLocked());
		PublicExportMap.PreInsertPublicExports(Exports);
	}

	void StorePublicExport(uint64 ExportHash, int32 HashIndex, UObject* Object)
	{
		check(Mutex.IsLocked());
		PublicExportMap.Store(ExportHash, GUObjectArray.ObjectToIndex(Object), HashIndex);
	}

	void RemovePublicExport(uint64 ExportHash, FName ObjectName = NAME_None)
	{
		check(Mutex.IsLocked());
		check(!bIsMissing);
		check(!bIsNotInstalled);
		check(HasPackage());
		if (PublicExportMap.Remove(ExportHash))
		{
			UE_LOG(LogStreaming, VeryVerbose, TEXT("Package %s got its export %" UINT64_X_FMT " removed %s"), *GetPackage()->GetPathName(), ExportHash, *ObjectName.ToString());
			bAreAllPublicExportsLoaded = false;
		}
	}

	UObject* GetPublicExport(uint64 ExportHash, int32* OutHashIndex = nullptr)
	{
		check(Mutex.IsLocked() || Mutex.IsLockShared());
		int32 ObjectIndex = PublicExportMap.Find(ExportHash, OutHashIndex);
		if (ObjectIndex >= 0)
		{
			return static_cast<UObject*>(GUObjectArray.IndexToObject(ObjectIndex)->GetObject());
		}
		return nullptr;
	}

	void PinPublicExportsForGC(TArray<int32>& OutUnreachableObjectIndices)
	{
		check(Mutex.IsLocked());
		UPackage* Package = GetPackage();
		UE_ASYNC_UPACKAGE_DEBUG(Package);

		if (GUObjectArray.IsDisregardForGC(Package))
		{
			return;
		}
		// If we're missing export table objects, reset the export loaded flags
		// but never overwrite it to true because if the UPackage was created outside
		// of the loader it wont have any exports in its map yet and we don't want
		// to make the loader think that the package was already loaded.
		if (!PublicExportMap.PinForGC(OutUnreachableObjectIndices))
		{
			bAreAllPublicExportsLoaded = false;
		}
#if WITH_EDITOR
		// Only reload if we ever got snapshotted after a proper load and the export count has shrinked since the package was last loaded
		if (GReloadPackagesWithGCedExports && bAreAllPublicExportsLoaded && ExportCount > PublicExportMap.Num())
		{
			UE_LOGF(LogStreaming, Log, "Reloading %ls because %d on %d exports were GCed since it was loaded", *Package->GetPathName(), ExportCount - PublicExportMap.Num(), ExportCount);
			bAreAllPublicExportsLoaded = false;
		}
#endif
		checkObject(Package, !Package->HasAnyInternalFlags(EInternalObjectFlags::LoaderImport));
		Package->SetInternalFlags(EInternalObjectFlags::LoaderImport);
	}

	void UnpinPublicExportsForGC()
	{
		check(Mutex.IsLocked());
		UPackage* Package = GetPackage();
		UE_ASYNC_UPACKAGE_DEBUG(Package);

		if (GUObjectArray.IsDisregardForGC(Package))
		{
			return;
		}
		PublicExportMap.UnpinForGC();
		checkObject(Package, Package->HasAnyInternalFlags(EInternalObjectFlags::LoaderImport));
		Package->AtomicallyClearInternalFlags(EInternalObjectFlags::LoaderImport);
	}
};

class FLoadedPackageCellsRef
{
	friend class FAsyncLoadingVerseRoot;

private:
	mutable UE::FMutex Mutex;
	TAsyncAtomic<bool> bPinned = true;
	TMap<uint64, Verse::VCell*> PublicExportMap;

public:
	FLoadedPackageCellsRef(EInPlace) {}

	void RemoveUnreferencedObsoletePackage()
	{
		UE::TUniqueLock Lock(Mutex);
		PublicExportMap.Empty();
	}

	void StorePublicCellExport(uint64 ExportHash, Verse::VCell* Cell)
	{
		check(bPinned);
		UE::TUniqueLock Lock(Mutex);
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		Verse::FRunningContext Context = Verse::FRunningContextPromise{};
		Context.RunWriteBarrier(Cell);
#endif
		PublicExportMap.Add(ExportHash, Cell);
	}

	Verse::VCell* GetPublicCellExport(uint64 ExportHash) const
	{
		check(bPinned);
		UE::TUniqueLock Lock(Mutex);
		return PublicExportMap.FindRef(ExportHash);
	}

	void PinPublicCellExportsForGC()
	{
		UE::TUniqueLock Lock(Mutex);
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		Verse::FRunningContext Context = Verse::FRunningContextPromise{};
		for (auto It = PublicExportMap.CreateIterator(); It; ++It)
		{
			if (Verse::VCell* Cell = Context.RunWeakReadBarrier(It->Value))
			{
				Context.RunWriteBarrier(Cell);
			}
			else
			{
				It.RemoveCurrent();
			}
		}
#endif
		bPinned.store(true, std::memory_order_release);
	}

	void UnpinPublicCellExportsForGC()
	{
		bPinned.store(false, std::memory_order_release);
	}
};

struct FAsyncLoadingSharedMutexStripedMapLockingPolicy
{
	typedef FAsyncLoadingMutex                  MutexType;
	typedef UE::TUniqueLock<MutexType>          ExclusiveLockType;
	typedef UE::TSharedLock<MutexType>          SharedLockType;
};

template <
	int32 BucketCount,
	typename KeyType,
	typename ValueType,
	typename SetAllocator = FDefaultSetAllocator,
	typename KeyFuncs = TDefaultMapHashableKeyFuncs<KeyType, ValueType, false>,
	typename LockingPolicy = FAsyncLoadingSharedMutexStripedMapLockingPolicy
>
using TAsyncLoadingStripedMap = TStripedMap<BucketCount, KeyType, ValueType, SetAllocator, KeyFuncs, LockingPolicy>;

#if ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING
class FCookedScriptObjectsDebug
{
private:
	TMap<FPackageObjectIndex, TPair<FMinimalName, FPackageObjectIndex>> Objects;

	void LoadDebugData();
	void GetPathNameInternal(const TPair<FMinimalName, FPackageObjectIndex>& Entry, FStringBuilderBase& PathName, int32& NumParts) const;
public:
	FCookedScriptObjectsDebug();
	~FCookedScriptObjectsDebug();
	bool IsEnabled() const { return Objects.Num() > 0; }
	void TrimDebugData(const TAsyncLoadingStripedMap<4, FPackageObjectIndex, UObject*>& RuntimeScriptObjects);
	void OnCompiledInUObjectsRegistered(FName, ECompiledInUObjectsRegisteredStatus Status);
	FString GetPathName(FPackageObjectIndex Index) const;
};
#endif

template <int32 Type>
class FLockedPackageRef
{
private:
	TRefCountPtr<FLoadedPackageRef> PackageRef;
public:
	FLockedPackageRef(const FLockedPackageRef&) = delete;
	FLockedPackageRef& operator=(const FLockedPackageRef&) = delete;

	FLockedPackageRef(FLockedPackageRef&& Other)
	{
		PackageRef  = Other.PackageRef;
		Other.PackageRef = nullptr;
	}

	FLockedPackageRef& operator=(FLockedPackageRef&& Other)
	{
		if (this != &Other)
		{
			Reset();
			PackageRef = Other.PackageRef;
			Other.PackageRef = nullptr;
		}

		return *this;
	}

	FLoadedPackageRef* Get() const { return PackageRef.GetReference(); }

	FLockedPackageRef() = default;

	FLoadedPackageRef& operator*() { return *PackageRef; }
	FLoadedPackageRef* operator ->()
	{
		return PackageRef.GetReference();
	}

	operator bool() const
	{
		return PackageRef != nullptr;
	}

	void Reset()
	{
		if (PackageRef)
		{
			static_assert(Type >= 0 && Type <= 1);
			if constexpr (Type == 0)
			{
				PackageRef->Mutex.UnlockShared();
			}
			else
			{
				PackageRef->Mutex.Unlock();
			}

			PackageRef = nullptr;
		}
	}

	FLockedPackageRef(TRefCountPtr<FLoadedPackageRef> InPackageRef)
		: PackageRef(InPackageRef)
	{
		if (PackageRef)
		{
			static_assert(Type >= 0 && Type <= 1);
			if constexpr (Type == 0)
			{
				PackageRef->Mutex.LockShared();
			}
			else
			{
				PackageRef->Mutex.Lock();
			}
		}
	}

	~FLockedPackageRef()
	{
		Reset();
	}
};

using FReadLockedPackageRef = FLockedPackageRef<0>;
using FWriteLockedPackageRef = FLockedPackageRef<1>;

class FGlobalImportStore
{
	friend class FAsyncLoadingVerseRoot;

private:
	// Reference back to our parent
	FAsyncLoadingThread2& AsyncLoadingThread;
	// Packages in active loading or completely loaded packages, with Desc.UPackageId as key.
	// Does not track temp packages with custom UPackage names, since they are never imorted by other packages.
	TAsyncLoadingStripedMap<4, FPackageId, TRefCountPtr<FLoadedPackageRef>> Packages;
 	// Contention is currently negligible so only use TStripedMap as a thread-safe TMap with just 1 stripe.
	// Packages in active loading that export VCells. Stored separately from FLoadedPackageRefs to save space.
	TAsyncLoadingStripedMap<4, FPackageId, FLoadedPackageCellsRef> PackageCells;
	// All native script objects (structs, enums, classes, CDOs and their subobjects) from the initial load phase
	TAsyncLoadingStripedMap<4, FPackageObjectIndex, UObject*> ScriptObjects;
	// Built-in VCells always defined by the Verse VM.
	TAsyncLoadingStripedMap<4, FPackageObjectIndex, Verse::VCell*> ScriptCells;
	// All currently loaded public export objects from any loaded package
	TAsyncLoadingStripedMap<4, int32, FPublicExportKey> ObjectIndexToPublicExport;
#if ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING
public:
	FCookedScriptObjectsDebug CookedScriptObjectsDebug;
private:
#endif

	// Process all the deferred cleanups for packages that are finished loading
	void FlushDeferredCleanupPackagesQueue();
public:
	FGlobalImportStore(FAsyncLoadingThread2& InAsyncLoadingThread)
		: AsyncLoadingThread(InAsyncLoadingThread)
	{
		LLM_SCOPE_BYTAG(AsyncLoading_GlobalImportStore);
		Packages.Reserve(32768);
		ScriptObjects.Reserve(32768);
		ObjectIndexToPublicExport.Reserve(32768);
	}

	int32 GetStoredPackagesCount() const
	{
		return Packages.Num();
	}

	int32 GetStoredScriptObjectsCount() const
	{
		return ScriptObjects.Num();
	}

	SIZE_T GetStoredScriptObjectsAllocatedSize() const
	{
		return ScriptObjects.GetAllocatedSize();
	}

	int32 GetStoredPublicExportsCount() const
	{
		return ObjectIndexToPublicExport.Num();
	}

	inline FReadLockedPackageRef FindPackageRef(FPackageId PackageId) const
	{
		FReadLockedPackageRef PackageRef;
		Packages.FindAndApply(PackageId, 
			[&PackageRef](const TRefCountPtr<FLoadedPackageRef>& Value)
			{
				// Take the package lock while inside the stripe lock to avoid races with package removal.
				PackageRef = Value;
			}
		);
		return PackageRef;
	}

	inline FWriteLockedPackageRef FindPackageRefForWrite(FPackageId PackageId)
	{
		FWriteLockedPackageRef PackageRef;
		Packages.FindAndApply(PackageId,
			[&PackageRef](const TRefCountPtr<FLoadedPackageRef>& Value)
			{
				// Take the package lock while inside the stripe lock to avoid races with package removal.
				PackageRef = Value;
			}
		);
		return PackageRef;
	}

	inline FReadLockedPackageRef FindPackageRefChecked(FPackageId PackageId, FName Name = FName()) const
	{
		FReadLockedPackageRef PackageRef = FindPackageRef(PackageId);
		if (!PackageRef)
		{
			UE_LOGF(LogStreaming, Fatal, "FindPackageRefChecked: Package %ls (0x%ls) has been deleted", *Name.ToString(), *LexToString(PackageId));
		}
		return PackageRef;
	}

	inline FWriteLockedPackageRef FindPackageRefCheckedForWrite(FPackageId PackageId, FName Name = FName())
	{
		FWriteLockedPackageRef PackageRef = FindPackageRefForWrite(PackageId);
		if (!PackageRef)
		{
			UE_LOGF(LogStreaming, Fatal, "FindPackageRefChecked: Package %ls (0x%ls) has been deleted", *Name.ToString(), *LexToString(PackageId));
		}
		return PackageRef;
	}

	FWriteLockedPackageRef FindOrAddPackageRef(FPackageId PackageId)
	{
		FWriteLockedPackageRef PackageRef;
		Packages.FindOrProduceAndApply(PackageId,
			[]() { return TRefCountPtr<FLoadedPackageRef>(new FLoadedPackageRef(), false); },
			[&PackageRef](const TRefCountPtr<FLoadedPackageRef>& Value)
			{
				// Take the package lock while inside the stripe lock to avoid races with package removal.
				PackageRef = Value;
			}
		);
		return PackageRef;
	}

	FWriteLockedPackageRef TryRemovePackage(FPackageId PackageId)
	{
		FWriteLockedPackageRef RemovedPackageRef;
		if (Packages.RemoveIf(PackageId,
			[&RemovedPackageRef](TRefCountPtr<FLoadedPackageRef>& Value)
			{
				// Take the lock on the package while we're in the stripe lock to make sure no other thread is using that package during its removal.
				RemovedPackageRef = Value;
				return true;
			}))
		{
			check(RemovedPackageRef->GetPackageRefCount() == 0);

			// We should be the last remaining refcount
			check(RemovedPackageRef->GetRefCount() == 1);
		}

		return RemovedPackageRef;
	}

	inline FWriteLockedPackageRef AddPackageRef(FPackageId PackageId, FName PackageNameIfKnown, EPackageLoader PackageLoaderIfKnown, EPackageExtension PackageHeaderExtensionIfKnown)
	{
		LLM_SCOPE_BYTAG(AsyncLoading_GlobalImportStore);

		// This is currently required by at least 2 different code path marked as such below.
		// To only flush on demand for these 2 code paths, we'd need to refactor this function so that it's possible to retry a second time in case of failure.
		// If we do that refactor, we need to take very special care that we don't leave a new Package uninitialized for other threads to see during our retry.
		// Since that's a non trivial refactor, the flush has been put back here for now to fix asserts caused when packages weren't cleaned fast enough.
		FlushDeferredCleanupPackagesQueue();

		FWriteLockedPackageRef PackageRef = FindOrAddPackageRef(PackageId);

		// Detect package that have been renamed but are still referenced and attempt to get rid of the references
		// by flushing the deferred delete packages queue so that we can load the proper package instead.
		if (PackageRef->GetPackageRefCount() > 0)
		{
			if (UPackage* Package = PackageRef->GetPackage())
			{
				if (PackageRef->GetOriginalPackageName() != Package->GetFName())
				{
					// This path requires that FlushDeferredCleanupPackagesQueue has been run to get rid of all the pending references to packages
					// Flush failed to clear the remaining references, warn since we want to know about this.
					if (PackageRef->GetPackageRefCount() > 0)
					{
						UE_LOGF(LogStreaming, Warning, "Package %ls was renamed to %ls but is unexpectedly still being referenced by other packages being loaded", *PackageRef->GetOriginalPackageName().ToString(), *Package->GetFName().ToString());
					}
				}
			}
		}

		// is this the first reference to a package that already exists?
		if (PackageRef->GetPackageRefCount() == 0)
		{
			// Remove stale package before searching below as its possible a UEDPIE package got trashed and replaced by a new one
			// and its important that we find the one that replaced it so we don't try to load it if its a PKG_InMemoryOnly package.
			if (UPackage* Package = PackageRef->GetPackage())
			{
				if (Package->IsUnreachable() || PackageRef->GetOriginalPackageName() != Package->GetFName())
				{
					RemoveUnreferencedObsoletePackage(PackageRef);
				}
			}
#if WITH_EDITOR
			if (!PackageRef->HasPackage() && !PackageNameIfKnown.IsNone())
			{
				UPackage* FoundPackage = FindObjectFast<UPackage>(nullptr, PackageNameIfKnown);
				if (FoundPackage)
				{
					// We need to maintain a 1:1 relationship between PackageId and Package object
					// to avoid having a costly 1:N lookup during GC. If we find an existing package
					// that already has a valid packageid in our table, it means it was renamed
					// after it finished loading. Since our mapping is now "stale" we remove
					// it before replacing the package ID in the package object, otherwise
					// the mapping would leak during GC and we could end up crashing when
					// trying to access the ref after the GC has run.
					FPackageId OldPackageId = FoundPackage->GetPackageId();
					if (OldPackageId.IsValid() && OldPackageId != PackageId)
					{
						// This path requires that FlushDeferredCleanupPackagesQueue has been run to get rid of all the pending references to packages
						if (FWriteLockedPackageRef RemovedPackageRef = TryRemovePackage(OldPackageId))
						{
							UE_LOGF(LogStreaming, Log,
								"FGlobalImportStore:AddPackageRef: Dropping stale reference to package %ls (0x%ls) that has been renamed to %ls (0x%ls)",
								*RemovedPackageRef->GetOriginalPackageName().ToString(),
								*LexToString(OldPackageId),
								*FoundPackage->GetName(),
								*LexToString(PackageId)
							);

							PackageCells.Remove(OldPackageId);
							RemoveUnreferencedObsoletePackage(RemovedPackageRef);
						}
					}

					if (PackageRef->bIsMissing)
					{
						check(!PackageRef->bIsNotInstalled);
						PackageRef->bIsMissing = false;
						UE_LOGF(LogStreaming, Warning,
							"FGlobalImportStore:AddPackageRef: Found reference to previously missing package %ls (0x%ls)",
							*FoundPackage->GetName(),
							*LexToString(PackageId)
						);
					}
					else if (PackageRef->bIsNotInstalled)
					{
						check(!PackageRef->bIsMissing);
						PackageRef->bIsNotInstalled = false;
						// Note, unlike missing packages, it is expected that we may later find a reference to a 
						// previously not installed package so we do not emit a warning
						UE_LOGF(LogStreaming, Log,
							"FGlobalImportStore:AddPackageRef: Found reference to previously not installed package %ls (0x%ls)",
							*FoundPackage->GetName(),
							*LexToString(PackageId)
						);
					}

					PackageRef->SetPackage(FoundPackage);
					FoundPackage->SetCanBeImportedFlag(true);
					FoundPackage->SetPackageId(PackageId);
				}
			}
			if (PackageRef->HasPackage())
			{
				if (PackageRef->GetPackage()->bHasBeenFullyLoaded && !PackageRef->IsFailed())
				{
					const bool bSnapshotExportCount = false;
					PackageRef->SetAllPublicExportsLoaded(bSnapshotExportCount);
				}
			}
#endif
			if (UPackage* Package = PackageRef->GetPackage())
			{
				if (Package->IsUnreachable() || PackageRef->GetOriginalPackageName() != Package->GetFName())
				{
					UE_CLOGF(!Package->IsUnreachable(), LogStreaming, Log,
						"FGlobalImportStore:AddPackageRef: Dropping renamed package %ls before reloading %ls (0x%ls)",
						*Package->GetName(),
						*PackageRef->GetOriginalPackageName().ToString(),
						*LexToString(Package->GetPackageId()));

					RemoveUnreferencedObsoletePackage(PackageRef);
				}
				else
				{
					TArray<int32> UnreachableObjectIndices;
					PackageRef->PinPublicExportsForGC(UnreachableObjectIndices);

					for (int32 ObjectIndex : UnreachableObjectIndices)
					{
						ObjectIndexToPublicExport.Remove(ObjectIndex);
					}

					PackageCells.FindAndApply(PackageId,
						[](FLoadedPackageCellsRef& PackageCellsRef)
						{
							PackageCellsRef.PinPublicCellExportsForGC();
						}
					);
				}
			}

			// Must be done last since we may reset PackageRef during RemoveUnreferencedObsoletePackage above
			PackageRef->PackageHeaderExtension = PackageHeaderExtensionIfKnown;
			PackageRef->PackageLoader = PackageLoaderIfKnown;
		}
		else
		{
			// Packages that exist only in memory might later be represented on disk, so if that changes
			// allow specifying their extensions and loader if they are specified to keep our state in sync.
			// Since our package has a RefCount > 0, we should only further specify/promote the extension and load
			// location, and not allow lowering the extension or location to an unknown value (since it was
			// either unknown upon creation or it was something of actual meaning that needs to remain the same)
			if (PackageRef->PackageHeaderExtension == EPackageExtension::Unspecified)
			{
				PackageRef->PackageHeaderExtension = PackageHeaderExtensionIfKnown;
			}
			if (PackageRef->PackageLoader == EPackageLoader::Unknown)
			{
				PackageRef->PackageLoader = PackageLoaderIfKnown;
			}
		}
		PackageRef->AddPackageRef();
		return PackageRef;
	}

	inline void ReleasePackageRef(FPackageId PackageId, FPackageId FromPackageId = FPackageId())
	{
		FWriteLockedPackageRef PackageRef = FindPackageRefCheckedForWrite(PackageId);

		PackageRef->ReleasePackageRef();

#if DO_CHECK
		ensureMsgf(!PackageRef->bHasBeenLoadedDebug || PackageRef->bWasFullyLoaded || PackageRef->bIsMissing || PackageRef->bIsNotInstalled || PackageRef->bHasFailed,
			TEXT("LoadedPackageRef from None (0x%s) to %s (0x%s) should not have been released when the package is not complete.")
			TEXT("RefCount=%d, AreAllExportsLoaded=%d, IsMissing=%d, IsNotInstalled=%d, HasFailed=%d, HasBeenLoaded=%d"),
			*LexToString(FromPackageId),
			*PackageRef->GetOriginalPackageName().ToString(),
			*LexToString(PackageId),
			PackageRef->GetPackageRefCount(),
			PackageRef->bAreAllPublicExportsLoaded,
			PackageRef->bIsMissing,
			PackageRef->bIsNotInstalled,
			PackageRef->bHasFailed,
			PackageRef->bHasBeenLoadedDebug);

		if (PackageRef->bWasFullyLoaded)
		{
			check(!PackageRef->bIsMissing && !PackageRef->bIsNotInstalled);
		}
		if (PackageRef->bIsMissing || PackageRef->bIsNotInstalled)
		{
			check(!PackageRef->bWasFullyLoaded);
		}
#endif
		// is this the last reference to a loaded package?
		if (PackageRef->GetPackageRefCount() == 0 && PackageRef->HasPackage())
		{
			PackageRef->UnpinPublicExportsForGC();

			PackageCells.FindAndApply(PackageId,
				[](FLoadedPackageCellsRef& PackageCellsRef)
				{
					PackageCellsRef.UnpinPublicCellExportsForGC();
				}
			);
		}
	}

	void VerifyLoadedPackages() const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VerifyLoadedPackages);
		Packages.ParallelForEach(
			[](const TPair<FPackageId, TRefCountPtr<FLoadedPackageRef>>& Pair)
			{
				const FPackageId& PackageId = Pair.Key;
				FReadLockedPackageRef PackageRef = Pair.Value;
				ensureMsgf(PackageRef->GetPackageRefCount() == 0,
					TEXT("PackageId '%s' with ref count %d should not have a ref count now")
					TEXT(", or this check is incorrectly reached during active loading."),
					*FormatPackageId(PackageId),
					PackageRef->GetPackageRefCount());
			}
		);
	}

	bool RemoveDeferredPublicExport(int32 ObjectIndex, UObject* Object)
	{
		check(ObjectIndex >= 0);

		FPublicExportKey PublicExportKey;
		if (ObjectIndexToPublicExport.FindAndApply(ObjectIndex,
			[&PublicExportKey](const FPublicExportKey& InPublicExportKey)
			{
				PublicExportKey = InPublicExportKey;
			}
		))
		{
			// We are dealing with deferred object removal/renames. In which case we can't be sure the passed in Object
			// is still valid, so lookup the object via index and only remove if we have a match. If we have a mismatch
			// the object was already collected and would have been removed during collection.
			FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(ObjectIndex);
			if (!ObjectItem->IsUnreachable())
			{
				UObject* FoundObject = static_cast<UObject*>(ObjectItem->GetObject());
				if (FoundObject == Object)
				{
					FPackageId PackageId = PublicExportKey.GetPackageId();
					UE_ASYNC_PACKAGEID_DEBUG(PackageId);
					FWriteLockedPackageRef PackageRef = FindPackageRefForWrite(PackageId);
					PackageRef->RemovePublicExport(PublicExportKey.GetExportHash(), Object->GetFName());

					// Allow this object to now be collectable
					ObjectIndexToPublicExport.Remove(ObjectIndex);
					checkf(Object, TEXT("We expect renamed loaded objects to remain alive until we remove the LoaderImport flag."));
					Object->AtomicallyClearInternalFlags(EInternalObjectFlags::LoaderImport);
				}

				return true;
			}
		}
		return false;
	}

	void CleanupPublicExportMap(FWriteLockedPackageRef& PackageRef)
	{
		for (auto It = PackageRef->PublicExportMap.CreateValueIterator(); It; ++It)
		{
			const int32 ObjectIndex = *It;
			ObjectIndexToPublicExport.Remove(ObjectIndex);
		}
	}

	void RemoveUnreferencedObsoletePackage(FWriteLockedPackageRef& PackageRef)
	{
		UPackage* OldPackage = PackageRef->GetPackage();
		UE_ASYNC_UPACKAGE_DEBUG(OldPackage);

		if (GVerifyUnreachableObjects)
		{
			VerifyPackageForRemoval(*PackageRef);
		}

		CleanupPublicExportMap(PackageRef);

		PackageCells.FindAndApply(OldPackage->GetPackageId(),
			[](FLoadedPackageCellsRef& PackageCellsRef)
			{
				PackageCellsRef.RemoveUnreferencedObsoletePackage();
			}
		);

		PackageRef->RemoveUnreferencedObsoletePackage();
		// Reset PackageId to prevent a double remove from GC NotifyUnreachableObjects
		OldPackage->SetPackageId(FPackageId());
	}

	void ReplaceReferencedRenamedPackageNoLock(FLoadedPackageRef& PackageRef, UPackage* NewPackage)
	{
		UPackage* OldPackage = PackageRef.GetPackage();
		UE_ASYNC_UPACKAGE_DEBUG(OldPackage);

		PackageRef.ReplaceReferencedRenamedPackage(NewPackage);
		// Clear LoaderImport so GC may destroy this package
		OldPackage->AtomicallyClearInternalFlags(EInternalObjectFlags::LoaderImport);
		// Update PackageId to prevent us from removing our updated package ref from GC
		OldPackage->SetPackageId(FPackageId::FromName(OldPackage->GetFName()));
	}

	void RemovePackages(const FUnreachableObjects& ObjectsToRemove)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemovePackages);
		for (const FUnreachableObject& Item : ObjectsToRemove)
		{
			const FPackageId& PackageId = Item.PackageId;
			if (PackageId.IsValid())
			{
				RemovePackage(PackageId);
			}
		}
	}

	void RemovePackage(FPackageId PackageId)
	{
		UE_ASYNC_PACKAGEID_DEBUG(PackageId);

		bool bRemoved = false;
		if (FWriteLockedPackageRef RemovedPackageRef = TryRemovePackage(PackageId))
		{
			bRemoved = true;
			CleanupPublicExportMap(RemovedPackageRef);
		}
		
		{
			bool bRemovedCells = PackageCells.Remove(PackageId) > 0;
#if DO_CHECK
			ensureMsgf(!bRemovedCells || bRemoved, TEXT("Removed %s from cell package map when there should have been nothing to remove"), *LexToString(PackageId));
#endif
		}
	}

	void RemovePublicExports(const FUnreachableObjects& ObjectsToRemove)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemovePublicExports);

		FPackageId LastPackageId;
		FWriteLockedPackageRef PackageRef;
		for (const FUnreachableObject& Item : ObjectsToRemove)
		{
			int32 ObjectIndex = Item.ObjectIndex;
			check(ObjectIndex >= 0);

			FPublicExportKey PublicExportKey;
			if (ObjectIndexToPublicExport.RemoveAndCopyValue(ObjectIndex, PublicExportKey))
			{
				FPackageId PackageId = PublicExportKey.GetPackageId();
				if (PackageId != LastPackageId)
				{
					// Reset the current PackageRef since our intention is to get another one
					// and we need to release the sharedlock otherwise we could deadlock if
					// a writelock inserts itself on the mutex, we won't be able to acquire
					// the new readlock until the writelock has been satisfied, but if we don't release
					// our sharedlock, it would prevent the writelock from making progress
					// so we would effectively deadlock.
					PackageRef.Reset();
					UE_ASYNC_PACKAGEID_DEBUG(PackageId);
					LastPackageId = PackageId;
					PackageRef = FindPackageRefForWrite(PackageId);
				}
				if (PackageRef)
				{
					PackageRef->RemovePublicExport(PublicExportKey.GetExportHash(), Item.ObjectName);
				}
			}
		}
	}

	void VerifyObjectForRemoval(UObject* GCObject, FPackageId* InPackageId = nullptr, FLoadedPackageRef* InPackageRef = nullptr) const
	{
		int32 ObjectIndex = GUObjectArray.ObjectToIndex(GCObject);

		FPublicExportKey PublicExportKey;
		if (ObjectIndexToPublicExport.FindAndApply(ObjectIndex,
			[&PublicExportKey](const FPublicExportKey& InPublicExportKey)
			{
				PublicExportKey = InPublicExportKey;
			}
		))
		{
			// This is a bit messy but exist to avoid locking the same package twice
			// in the same callstack since the locks aren't recursive and making them
			// recursive just for this use case would impact both memory and performance and is not worth it.
			FPackageId PackageId = PublicExportKey.GetPackageId();
			FReadLockedPackageRef ReadLockedPackageRef;
			FLoadedPackageRef* PackageRef = nullptr;
			if (InPackageId && InPackageRef && PackageId == *InPackageId)
			{
				PackageRef = InPackageRef;
			}
			else
			{
				ReadLockedPackageRef = FindPackageRef(PackageId);
				PackageRef = ReadLockedPackageRef.Get();
			}
			
			if (PackageRef)
			{
				UObject* ExistingObject = PackageRef->GetPublicExport(PublicExportKey.GetExportHash());
				UE_CLOGF(!ExistingObject, LogStreaming, Fatal,
					"FGlobalImportStore::VerifyObjectForRemoval: The loaded public export object '%ls' with flags (ObjectFlags=%x, InternalObjectFlags=%x) and id %ls:0x%llX is missing in GlobalImportStore. " "Reason unknown. Double delete? Bug or hash collision?",
					*GCObject->GetFullName(),
					GCObject->GetFlags(),
					int(GCObject->GetInternalFlags()),
					*FormatPackageId(PackageId),
					PublicExportKey.GetExportHash());

				UE_CLOGF(ExistingObject != GCObject, LogStreaming, Fatal,
					"FGlobalImportStore::VerifyObjectForRemoval: The loaded public export object '%ls' with flags (ObjectFlags=%x, InternalObjectFlags=%x) and id %ls:0x%llX is not matching the object '%ls' in GlobalImportStore. " "Reason unknown. Overwritten after it was added? Bug or hash collision?",
					*GCObject->GetFullName(),
					GCObject->GetFlags(),
					int(GCObject->GetInternalFlags()),
					*FormatPackageId(PackageId),
					PublicExportKey.GetExportHash(),
					*ExistingObject->GetFullName());
			}
			else
			{
				UE_LOGF(LogStreaming, Warning,
					"FGlobalImportStore::VerifyObjectForRemoval: The package for the serialized GC object '%ls' with flags (ObjectFlags=%x, InternalObjectFlags=%x) and id %ls:0x%llX is missing in GlobalImportStore. " "Most likely this object has been moved into this package after it was loaded, while the original package is still around.",
					*GCObject->GetFullName(),
					GCObject->GetFlags(),
					int(GCObject->GetInternalFlags()),
					*FormatPackageId(PackageId),
					PublicExportKey.GetExportHash());
			}
		}
	}
	
	void VerifyPackageForRemoval(FLoadedPackageRef& PackageRef) const
	{
		UPackage* Package = PackageRef.GetPackage();
		FPackageId PackageId = Package->GetPackageId();

		UE_CLOGF(PackageRef.GetPackageRefCount() > 0, LogStreaming, Fatal,
			"FGlobalImportStore::VerifyPackageForRemoval: %ls (0x%ls) - " "Package removed while still being referenced, RefCount %d > 0.",
			*Package->GetName(),
			*LexToString(PackageId),
			PackageRef.GetPackageRefCount());

		for (auto It = PackageRef.GetPublicExportObjectIndices(); It; ++It)
		{
			const int32 ObjectIndex = *It;
			UObject* Object = static_cast<UObject*>(GUObjectArray.IndexToObject(ObjectIndex)->GetObject());
			ensureMsgf(!Object->HasAnyInternalFlags(EInternalObjectFlags::LoaderImport) || GUObjectArray.IsDisregardForGC(Object),
					TEXT("FGlobalImportStore::VerifyPackageForRemoval: The loaded public export object '%s' with flags (ObjectFlags=%x, InternalObjectFlags=%x) and id %s is probably still referenced by the loader."),
					*Object->GetFullName(),
					uint32{EnumToUnderlyingType(Object->GetFlags())},
					uint32{EnumToUnderlyingType(Object->GetInternalFlags())},
					*FormatPackageId(PackageId));

			bool bExists = 
				ObjectIndexToPublicExport.FindAndApply(ObjectIndex,
					[Object, Package, PackageId](const FPublicExportKey& PublicExportKey)
					{
						FPackageId ObjectPackageId = PublicExportKey.GetPackageId();
						UE_CLOGF(ObjectPackageId != PackageId, LogStreaming, Fatal,
							"FGlobalImportStore::VerifyPackageForRemoval: %ls (%ls) - " "The loaded public export object '%ls' has a mismatching package id %ls in GlobalImportStore.",
							*Package->GetName(),
							*FormatPackageId(PackageId),
							*Object->GetFullName(),
							*FormatPackageId(ObjectPackageId));
					}
				);
			
			UE_CLOGF(!bExists, LogStreaming, Fatal,
						"FGlobalImportStore::VerifyPackageForRemoval: %ls (%ls) - " "The loaded public export object '%ls' is missing in GlobalImportStore.",
						*Package->GetName(),
						*FormatPackageId(PackageId),
						*Object->GetFullName());

			VerifyObjectForRemoval(Object, &PackageId, &PackageRef);
		}
	}

	inline UObject* FindPublicExportObjectUnchecked(const FPublicExportKey& Key) const
	{
		FReadLockedPackageRef PackageRef = FindPackageRef(Key.GetPackageId());
		if (!PackageRef)
		{
			return nullptr;
		}
		return PackageRef->GetPublicExport(Key.GetExportHash());
	}

	inline UObject* FindPublicExportObject(const FPublicExportKey& Key) const
	{
		UObject* Object = FindPublicExportObjectUnchecked(Key);
		checkf(!Object || !Object->IsUnreachable(), TEXT("%s"), Object ? *Object->GetFullName() : TEXT("null"));
		return Object;
	}

	inline UObject* FindScriptImportObject(FPackageObjectIndex GlobalIndex) const
	{
		check(GlobalIndex.IsScriptImport());
		UObject* Object = nullptr;
		Object = ScriptObjects.FindRef(GlobalIndex);
		return Object;
	}

	Verse::VCell* FindPublicExportCell(const FPublicExportKey& Key) const
	{
		Verse::VCell* Cell = nullptr;
		PackageCells.FindAndApply(Key.GetPackageId(),
			[&Key, &Cell](const FLoadedPackageCellsRef& PackageCellsRef)
			{
				Cell = PackageCellsRef.GetPublicCellExport(Key.GetExportHash());
			}
		);
		return Cell;
	}

	Verse::VCell* FindScriptImportCell(FPackageObjectIndex GlobalIndex) const
	{
		check(GlobalIndex.IsScriptImport());
		return ScriptCells.FindRef(GlobalIndex);
	}

	void StoreGlobalObject(FPackageId PackageId, uint64 ExportHash, UObject* Object)
	{
		LLM_SCOPE_BYTAG(AsyncLoading_GlobalImportStore);

		check(PackageId.IsValid());
		check(ExportHash != 0);
		int32 ObjectIndex = GUObjectArray.ObjectToIndex(Object);
		FPublicExportKey Key = FPublicExportKey::MakeKey(PackageId, ExportHash);
		FPublicExportKey ExistingKey;

 		bool bPreviousExportRemoved = false;
		{
			FWriteLockedPackageRef PackageRef = FindPackageRefCheckedForWrite(Key.GetPackageId());

			int32 HashIndex = INDEX_NONE;
			UObject* ExistingObject = PackageRef->GetPublicExport(Key.GetExportHash(), &HashIndex);
			if (ExistingObject && ExistingObject != Object)
			{
				int32 ExistingObjectIndex = GUObjectArray.ObjectToIndex(ExistingObject);

				UE_LOGF(LogStreaming, Verbose,
					"FGlobalImportStore::StoreGlobalObject: The constructed public export object '%ls' with index %d and id %ls:0x%llX collides with object '%ls' (ObjectFlags=%X, InternalObjectFlags=%x) with index %d in GlobalImportStore. " "The existing object will be replaced since it or its package was most likely renamed after it was loaded the first time.",
					Object ? *Object->GetFullName() : TEXT("null"),
					ObjectIndex,
					*FormatPackageId(Key.GetPackageId()),
					Key.GetExportHash(),
					*ExistingObject->GetFullName(),
					ExistingObject->GetFlags(),
					int(ExistingObject->GetInternalFlags()),
					ExistingObjectIndex);

				ExistingObject->AtomicallyClearInternalFlags(EInternalObjectFlags::LoaderImport);
				ObjectIndexToPublicExport.Remove(ExistingObjectIndex);
			}
		
			bPreviousExportRemoved = ObjectIndexToPublicExport.RemoveIf(ObjectIndex,
				[this, ObjectIndex, Object, &Key, &ExistingKey](const FPublicExportKey& InExistingKey)
				{
					ExistingKey = InExistingKey;
					return InExistingKey != Key;
				}
			) > 0;

			PackageRef->StorePublicExport(ExportHash, HashIndex, Object);
			ObjectIndexToPublicExport.Add(ObjectIndex, Key);
		}

		if (bPreviousExportRemoved)
		{
			UE_LOGF(LogStreaming, Verbose,
						"FGlobalImportStore::StoreGlobalObject: The constructed public export object '%ls' with index %d and id %ls:0x%llX already exists in GlobalImportStore but with a different key %ls:0x%llX." "The existing object will be replaced since it or its package was most likely renamed after it was loaded the first time.",
						Object ? *Object->GetFullName() : TEXT("null"),
						ObjectIndex,
						*FormatPackageId(Key.GetPackageId()), Key.GetExportHash(),
						*FormatPackageId(ExistingKey.GetPackageId()), ExistingKey.GetExportHash());

			// Break the link with the old package now because otherwise, we wouldn't be able to remove the export
			// during GC since the ObjectIndex can only be linked to a single package ref.
			if (FWriteLockedPackageRef ExistingPackageRef = FindPackageRefForWrite(ExistingKey.GetPackageId()))
			{
				ExistingPackageRef->RemovePublicExport(ExistingKey.GetExportHash());
			}
		}
	}

	void StoreGlobalCell(FPackageId PackageId, uint64 ExportHash, Verse::VCell* Cell)
	{
		LLM_SCOPE_BYTAG(AsyncLoading_GlobalImportStore);
		PackageCells.FindOrProduceAndApplyForWrite(PackageId,
			[]()
			{
				return InPlace;
			},
			[ExportHash, Cell](FLoadedPackageCellsRef& PackageCellsRef)
			{
				PackageCellsRef.StorePublicCellExport(ExportHash, Cell);
			}
		);
	}

	void FindAllScriptObjects(bool bVerifyOnly);
	void RegistrationComplete();

	void AddScriptObject(FStringView PackageName, FStringView Name, UObject* Object)
	{
		LLM_SCOPE_BYTAG(AsyncLoading_GlobalImportStore);
		TRACE_CPUPROFILER_EVENT_SCOPE(FGlobalImportStore::AddScriptObject);

		TStringBuilder<FName::StringBufferSize> FullName;
		FPathViews::Append(FullName, PackageName);
		FPathViews::Append(FullName, Name);
		FPackageObjectIndex GlobalImportIndex = FPackageObjectIndex::FromScriptPath(FullName);

#if WITH_EDITOR
		FPackageObjectIndex PackageGlobalImportIndex = FPackageObjectIndex::FromScriptPath(PackageName);
		ScriptObjects.Add(PackageGlobalImportIndex, Object->GetOutermost());
#endif
		ScriptObjects.Add(GlobalImportIndex, Object);

		TStringBuilder<FName::StringBufferSize> SubObjectName;
		ForEachObjectWithOuter(Object, [this, &SubObjectName](UObject* SubObject)
			{
				if (SubObject->HasAnyFlags(RF_Public))
				{
					SubObjectName.Reset();
					SubObject->GetPathName(nullptr, SubObjectName);
					FPackageObjectIndex SubObjectGlobalImportIndex = FPackageObjectIndex::FromScriptPath(SubObjectName);
					ScriptObjects.Add(SubObjectGlobalImportIndex, SubObject);
				}
			}, EGetObjectsFlags::IncludeNestedObjects);
	}

	void RemoveUnreachableScriptObjects()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemoveUnreachableScriptObjects);

		const int32 RemovedCount = ScriptObjects.ParallelRemoveIf([](const TPair<FPackageObjectIndex, UObject*>& Entry)
		{
			const UObject* ScriptObject = Entry.Value;
			ensure(ScriptObject->IsValidLowLevel());

			if (ScriptObject->IsUnreachable())
			{
				UE_LOGF(LogStreaming, Verbose, "RemoveUnreachableScriptObjects: %ls", *ScriptObject->GetName());
				return true;
			}
			else
			{
				return false;
			}
		});

		UE_CLOGF(RemovedCount, LogStreaming, Display, "RemoveUnreachableScriptObjects: removed %d entries", RemovedCount);
	}

	void Shrink()
	{
		LLM_SCOPE_BYTAG(AsyncLoading_GlobalImportStore);
		TRACE_CPUPROFILER_EVENT_SCOPE(FGlobalImportStore::Shrink);
		Packages.Shrink();
		PackageCells.Shrink();
		ScriptObjects.Shrink();
		ScriptCells.Shrink();
		ObjectIndexToPublicExport.Shrink();
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	void AddScriptCellPackage(Verse::VPackage* Package)
	{
		LLM_SCOPE_BYTAG(AsyncLoading_GlobalImportStore);
		for (int32 I = 0; I < Package->NumDefinitions(); ++I)
		{
			FUtf8StringView VersePath = Package->GetDefinitionName(I).AsStringView();
			Verse::VCell* Cell = Package->GetDefinition(I).ExtractCell();
			FPackageObjectIndex GlobalImportIndex = FPackageObjectIndex::FromVersePath(VersePath);
			ScriptCells.Add(GlobalImportIndex, Cell);
		}
	}
#endif
};

struct FAsyncPackageHeaderData : public FZenPackageHeader
{
	// Backed by allocation in FAsyncPackageData
	TArrayView<FPackageId> ImportedPackageIds;
	TArrayView<FAsyncPackage2*> ImportedAsyncPackagesView;
	TArrayView<FExportObject> ExportsView;
	TArrayView<FExportCell> CellExportsView;
	TArrayView<FExportBundleEntry> ExportBundleEntriesCopyForPostLoad; // TODO: Can we use ConstructedObjects or Exports instead for posloading?
};

struct FAsyncPackageLinkerLoadHeaderData
{
	TArray<uint64> ImportedPublicExportHashes;
	TArray<FPackageObjectIndex> ImportMap;
	TArray<FExportMapEntry> ExportMap; // Note: Currently only using the public export hash field
};

struct FPackageImportStore
{
	FGlobalImportStore& GlobalImportStore;

	FPackageImportStore(FGlobalImportStore& InGlobalImportStore)
		: GlobalImportStore(InGlobalImportStore)
	{
	}

	inline FPackageObjectIndex GetGlobalImportIndex(const FAsyncPackageHeaderData& Header, FPackageIndex LocalIndex)
	{
		check(LocalIndex.IsImport());
		check(Header.ImportMap.Num() > 0);
		const int32 LocalImportIndex = LocalIndex.ToImport();
		return LocalImportIndex < Header.ImportMap.Num() ? Header.ImportMap[LocalImportIndex] : FPackageObjectIndex();
	}

	inline UObject* GetImportObject(const FAsyncPackageHeaderData& Header, FPackageObjectIndex GlobalIndex)
	{
		check(GlobalIndex.IsImport());
		UObject* Object = nullptr;
		if (GlobalIndex.IsScriptImport())
		{
			Object = GlobalImportStore.FindScriptImportObject(GlobalIndex);
		}
		else if (GlobalIndex.IsPackageImport())
		{
			Object = GlobalImportStore.FindPublicExportObject(FPublicExportKey::FromPackageImport(GlobalIndex, Header.ImportedPackageIds, Header.ImportedPublicExportHashes));
#if WITH_EDITOR
			if (UObjectRedirector* Redirector = dynamic_cast<UObjectRedirector*>(Object))
			{
				Redirector->ConditionalPreload();
				// Make sure we're not trying to access a redirector that has yet to be loaded
				Object = Redirector->DestinationObject;
			}
#endif
		}
		else
		{
			check(GlobalIndex.IsNull());
		}
		return Object;
	}

	FString GetImportName(const FAsyncPackageHeaderData& Header, FPackageObjectIndex GlobalIndex)
	{
		bool bIsInvalidPackageImport = GlobalIndex.IsPackageImport() && !Header.ImportedPackageIds.IsValidIndex(GlobalIndex.ToPackageImportRef().GetImportedPackageIndex());
		const UObject* Import = bIsInvalidPackageImport ? nullptr : GetImportObject(Header, GlobalIndex);
		if (Import)
		{
			return Import->GetFName().ToString();
		}
#if ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING
		else if (GlobalIndex.IsScriptImport())
		{
			return GlobalImportStore.CookedScriptObjectsDebug.GetPathName(GlobalIndex);
		}
#endif
		return FString();
	}

	void GetUnresolvedCDOs(const FAsyncPackageHeaderData& Header, TArray<UClass*, TInlineAllocator<8>>& Classes)
	{
		for (const FPackageObjectIndex& Index : Header.ImportMap)
		{
			if (!Index.IsScriptImport())
			{
				continue;
			}

			UObject* Object = GlobalImportStore.FindScriptImportObject(Index);
			if (!Object)
			{
				continue;
			}

			UClass* Class = Cast<UClass>(Object);
			if (!Class)
			{
				continue;
			}

			// Filter out CDOs that are themselves classes,
			// like Default__BlueprintGeneratedClass of type UBlueprintGeneratedClass
			if (Class->HasAnyFlags(RF_ClassDefaultObject))
			{
				continue;
			}

			// Add dependency on any script CDO that has not been created and initialized yet
			UObject* CDO = Class->GetDefaultObject(/*bCreateIfNeeded*/ false);
			if (!CDO || CDO->HasAnyFlags(RF_NeedInitialization))
			{
				UE_LOGF(LogStreaming, Log, "Package %ls has a dependency on pending script CDO for '%ls' (0x%llX)",
					*Header.PackageName.ToString(), *Class->GetFullName(), Index.Value());
				Classes.AddUnique(Class);
			}
		}
	}

	inline void StoreGlobalObject(FPackageId PackageId, uint64 ExportHash, UObject* Object)
	{
		GlobalImportStore.StoreGlobalObject(PackageId, ExportHash, Object);
	}

	inline Verse::VCell* FindOrGetImportCell(const FAsyncPackageHeaderData& Header, FPackageObjectIndex GlobalIndex)
	{
		check(GlobalIndex.IsImport());
		Verse::VCell* Cell = nullptr;
		if (GlobalIndex.IsScriptImport())
		{
			Cell = GlobalImportStore.FindScriptImportCell(GlobalIndex);
		}
		else if (GlobalIndex.IsPackageImport())
		{
			Cell = GlobalImportStore.FindPublicExportCell(FPublicExportKey::FromPackageImport(GlobalIndex, Header.ImportedPackageIds, Header.ImportedPublicExportHashes));
		}
		else
		{
			check(GlobalIndex.IsNull());
		}
		return Cell;
	}

	void StoreGlobalCell(FPackageId PackageId, uint64 ExportHash, Verse::VCell* Cell)
	{
		GlobalImportStore.StoreGlobalCell(PackageId, ExportHash, Cell);
	}

public:
	bool ContainsImportedPackageReference(FPackageId ImportedPackageId)
	{
		return !!GlobalImportStore.FindPackageRef(ImportedPackageId);
	}

	FWriteLockedPackageRef AddImportedPackageReference(FPackageId ImportedPackageId, FName PackageNameIfKnown, EPackageLoader PackageLoaderIfKnown, EPackageExtension PackageHeaderExtensionIfKnown)
	{
		return GlobalImportStore.AddPackageRef(ImportedPackageId, PackageNameIfKnown, PackageLoaderIfKnown, PackageHeaderExtensionIfKnown);
	}

	void AddPackageReference(const FAsyncPackageDesc2& Desc)
	{
		if (Desc.bCanBeImported)
		{
			FWriteLockedPackageRef PackageRef = GlobalImportStore.AddPackageRef(Desc.UPackageId, Desc.UPackageName, Desc.Loader, Desc.PackagePathToLoad.GetHeaderExtension());
			PackageRef->ClearErrorFlags();
		}
	}

	void ReleaseImportedPackageReferences(const FAsyncPackageDesc2& Desc, const TArrayView<const FPackageId>& ImportedPackageIds)
	{
		for (const FPackageId& ImportedPackageId : ImportedPackageIds)
		{
			GlobalImportStore.ReleasePackageRef(ImportedPackageId, Desc.UPackageId);
		}
	}

	void ReleasePackageReference(const FAsyncPackageDesc2& Desc)
	{
		if (Desc.bCanBeImported)
		{
			GlobalImportStore.ReleasePackageRef(Desc.UPackageId);
		}
	}
};

class FExportArchive final : public FArchive
{
public:
	FExportArchive(const FIoBuffer& IoBuffer)
		: IoDispatcher(FIoDispatcher::Get())
	{
#if (!DEVIRTUALIZE_FLinkerLoad_Serialize)
		ActiveFPLB = &InlineFPLB;
#endif
		ActiveFPLB->OriginalFastPathLoadBuffer = IoBuffer.Data();
		ActiveFPLB->StartFastPathLoadBuffer = ActiveFPLB->OriginalFastPathLoadBuffer;
		ActiveFPLB->EndFastPathLoadBuffer = IoBuffer.Data() + IoBuffer.DataSize();
	}

	~FExportArchive()
	{
	}
	void ExportBufferBegin(UObject* Object, uint64 InExportSerialOffset, uint64 InExportSerialSize)
	{
		CurrentExport = Object;
		ExportSerialOffset = HeaderData->PackageSummary->HeaderSize + InExportSerialOffset;
		ExportSerialSize = InExportSerialSize;
		ActiveFPLB->StartFastPathLoadBuffer = ActiveFPLB->OriginalFastPathLoadBuffer + ExportSerialOffset;
	}

	void ExportBufferEnd()
	{
		CurrentExport = nullptr;
		ExportSerialOffset = 0;
		ExportSerialSize = 0;
		ActiveFPLB->StartFastPathLoadBuffer = ActiveFPLB->OriginalFastPathLoadBuffer;
	}

	void CheckBufferPosition(const TCHAR* Text, uint64 Offset = 0)
	{
#if DO_CHECK
		const uint64 BufferPosition = (ActiveFPLB->StartFastPathLoadBuffer - ActiveFPLB->OriginalFastPathLoadBuffer) + Offset;
		const bool bIsInsideExportBuffer =
			(ExportSerialOffset <= BufferPosition) && (BufferPosition <= ExportSerialOffset + ExportSerialSize);

		UE_ASYNC_PACKAGE_CLOG(
			!bIsInsideExportBuffer,
			Error, *PackageDesc, TEXT("FExportArchive::InvalidPosition"),
			TEXT("%s: Position %llu is outside of the current export buffer (%lld,%lld)."),
			Text,
			BufferPosition,
			ExportSerialOffset, ExportSerialOffset + ExportSerialSize);
#endif
	}

	void Skip(int64 InBytes)
	{
		CheckBufferPosition(TEXT("InvalidSkip"), InBytes);
		ActiveFPLB->StartFastPathLoadBuffer += InBytes;
	}

	virtual int64 TotalSize() override
	{
		return ExportSerialSize;
	}

	virtual int64 Tell() override
	{
		return (ActiveFPLB->StartFastPathLoadBuffer - ActiveFPLB->OriginalFastPathLoadBuffer) - ExportSerialOffset;
	}

	virtual void Seek(int64 Position) override
	{
		ActiveFPLB->StartFastPathLoadBuffer = ActiveFPLB->OriginalFastPathLoadBuffer + ExportSerialOffset + Position;
		CheckBufferPosition(TEXT("InvalidSeek"));
	}

	virtual void Serialize(void* Data, int64 Length) override
	{
		if (!Length || ArIsError)
		{
			return;
		}
		CheckBufferPosition(TEXT("InvalidSerialize"), (uint64)Length);
		FMemory::Memcpy(Data, ActiveFPLB->StartFastPathLoadBuffer, Length);
		ActiveFPLB->StartFastPathLoadBuffer += Length;
	}

	virtual FString GetArchiveName() const override
	{
		return PackageDesc ? PackageDesc->UPackageName.ToString() : TEXT("FExportArchive");
	}

	void UsingCustomVersion(const FGuid& Key) override {};
	using FArchive::operator<<; // For visibility of the overloads we don't override

	/** FExportArchive will be created on the stack so we do not want BulkData objects caching references to it. */
	virtual FArchive* GetCacheableArchive()
	{
		return nullptr;
	}

	//~ Begin FArchive::FArchiveUObject Interface
	virtual FArchive& operator<<(FObjectPtr& Value) override { return FArchiveUObject::SerializeObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override { return FArchiveUObject::SerializeWeakObjectPtr(*this, Value); }
	//~ End FArchive::FArchiveUObject Interface

	//~ Begin FArchive::FLinkerLoad Interface
	UObject* GetArchetypeFromLoader(const UObject* Obj) { return TemplateForGetArchetypeFromLoader; }

	virtual bool AttachExternalReadDependency(FExternalReadCallback& ReadCallback) override
	{
		ExternalReadDependencies->Add(ReadCallback);
		return true;
	}

	FORCENOINLINE void HandleBadExportIndex(int32 ExportIndex, UObject*& Object)
	{
		UE_ASYNC_PACKAGE_LOG(Fatal, *PackageDesc, TEXT("ObjectSerializationError"),
			TEXT("%s: Bad export index %d/%d."),
			CurrentExport ? *CurrentExport->GetFullName() : TEXT("null"), ExportIndex, HeaderData->ExportsView.Num());

		Object = nullptr;
	}

	FORCENOINLINE void HandleBadImportIndex(int32 ImportIndex, UObject*& Object)
	{
		UE_ASYNC_PACKAGE_LOG(Fatal, *PackageDesc, TEXT("ObjectSerializationError"),
			TEXT("%s: Bad import index %d/%d."),
			CurrentExport ? *CurrentExport->GetFullName() : TEXT("null"), ImportIndex, HeaderData->ImportMap.Num());
		Object = nullptr;
	}

	FORCENOINLINE void HandleBadExportIndex(int32 CellExportIndex, Verse::VCell*& Cell)
	{
		UE_ASYNC_PACKAGE_LOG(Fatal, *PackageDesc, TEXT("ObjectSerializationError"),
			TEXT("%s: Bad cell export index %d/%d."),
			CurrentExport ? *CurrentExport->GetFullName() : TEXT("null"), CellExportIndex, HeaderData->CellExportsView.Num());
		Cell = nullptr;
	}

	FORCENOINLINE void HandleBadImportIndex(int32 ImportIndex, Verse::VCell*& Cell)
	{
		UE_ASYNC_PACKAGE_LOG(Fatal, *PackageDesc, TEXT("ObjectSerializationError"),
			TEXT("%s: Bad import index %d/%d."),
			CurrentExport ? *CurrentExport->GetFullName() : TEXT("null"), ImportIndex, HeaderData->CellImportMap.Num());
		Cell = nullptr;
	}

	virtual FArchive& operator<<(UObject*& Object) override
	{
		FPackageIndex Index;
		FArchive& Ar = *this;
		Ar << Index;

		if (Index.IsNull())
		{
			Object = nullptr;
		}
		else if (Index.IsExport())
		{
			const int32 ExportIndex = Index.ToExport();
			if (ExportIndex < HeaderData->ExportsView.Num())
			{
				Object = HeaderData->ExportsView[ExportIndex].Object;

#if ALT2_LOG_VERBOSE
				const FExportMapEntry& Export = HeaderData->ExportMap[ExportIndex];
				FName ObjectName = HeaderData->NameMap.GetName(Export.ObjectName);
				UE_ASYNC_PACKAGE_CLOG_VERBOSE(!Object, VeryVerbose, *PackageDesc,
					TEXT("FExportArchive: Object"), TEXT("Export %s at index %d is null."),
					*ObjectName.ToString(),
					ExportIndex);
#endif
			}
			else
			{
				HandleBadExportIndex(ExportIndex, Object);
			}
		}
		else
		{
			FPackageObjectIndex GlobalIndex = ImportStore->GetGlobalImportIndex(*HeaderData, Index);
			if (!GlobalIndex.IsNull())
			{
				Object = ImportStore->GetImportObject(*HeaderData, GlobalIndex);

				UE_ASYNC_PACKAGE_CLOG(!Object, Log, *PackageDesc,
					TEXT("FExportArchive: Object"), TEXT("Import index %d is null (0x%llX - '%s')"),
					Index.ToImport(), GlobalIndex.Value(), *ImportStore->GetImportName(*HeaderData, GlobalIndex));
			}
			else
			{
				HandleBadImportIndex(Index.ToImport(), Object);
			}
		}
		return *this;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Cell) override
	{
		FPackageIndex Index;
		FArchive& Ar = *this;
		Ar << Index;

		if (Index.IsNull())
		{
			Cell = nullptr;
		}
		else if (Index.IsExport())
		{
			int32 CellExportIndex = Index.ToExport() - HeaderData->ExportsView.Num();
			if (HeaderData->CellExportsView.IsValidIndex(CellExportIndex))
			{
				Cell = HeaderData->CellExportsView[CellExportIndex].Cell;
			}
			else
			{
				HandleBadExportIndex(CellExportIndex, Cell);
			}
		}
		else
		{
			int32 CellImportIndex = Index.ToImport() - HeaderData->ImportMap.Num();
			if (HeaderData->CellImportMap.IsValidIndex(CellImportIndex))
			{
				FPackageObjectIndex GlobalIndex = HeaderData->CellImportMap[CellImportIndex];
				Cell = ImportStore->FindOrGetImportCell(*HeaderData, GlobalIndex);
				check(Cell);
			}
			else
			{
				HandleBadImportIndex(Index.ToImport(), Cell);
			}
		}
		return *this;
	}
#endif

	inline virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override
	{
		FArchive& Ar = *this;
		FUniqueObjectGuid ID;
		Ar << ID;
		LazyObjectPtr = ID;
		return Ar;
	}

	inline virtual FArchive& operator<<(FSoftObjectPtr& Value) override
	{
		FArchive& Ar = *this;
		FSoftObjectPath ID;
		ID.Serialize(Ar);
		Value = ID;
		return Ar;
	}

	inline virtual FArchive& operator<<(FSoftObjectPath& Value) override
	{
		FArchive& Ar = FArchiveUObject::SerializeSoftObjectPath(*this, Value);
		FixupSoftObjectPathForInstancedPackage(Value);
		return Ar;
	}

	FORCENOINLINE void HandleBadNameIndex(int32 NameIndex, FName& Name)
	{
		UE_ASYNC_PACKAGE_LOG(Fatal, *PackageDesc, TEXT("ObjectSerializationError"),
			TEXT("%s: Bad name index %d/%d."),
			CurrentExport ? *CurrentExport->GetFullName() : TEXT("null"), NameIndex, HeaderData->NameMap.Num());
		Name = FName();
		SetCriticalError();
	}

	inline virtual FArchive& operator<<(FName& Name) override
	{
		FArchive& Ar = *this;
		uint32 NameIndex;
		Ar << NameIndex;
		uint32 Number = 0;
		Ar << Number;

		FMappedName MappedName = FMappedName::Create(NameIndex, Number, FMappedName::EType::Package);
		if (!HeaderData->NameMap.TryGetName(MappedName, Name))
		{
			HandleBadNameIndex(NameIndex, Name);
		}
		return *this;
	}

	inline virtual bool SerializeBulkData(FBulkData& BulkData, const FBulkDataSerializationParams& Params) override
	{
		const FPackageId& PackageId = PackageDesc->PackageIdToLoad;
		const uint16 ChunkIndex = bIsOptionalSegment ? 1 : 0;

		UE::BulkData::Private::FBulkMetaData& Meta = BulkData.BulkMeta;
		FBulkDataCookedIndex CookedIndex;
		int64 DuplicateSerialOffset = -1;
		SerializeBulkMeta(Meta, CookedIndex, DuplicateSerialOffset, Params.ElementSize);

		const bool bIsInline = Meta.HasAnyFlags(BULKDATA_PayloadAtEndOfFile) == false;
		if (bIsInline)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (int64 PayloadSize = Meta.GetSize(); PayloadSize > 0 && Meta.HasAnyFlags(BULKDATA_Unused) == false)
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					// Set the offset from the beginning of the I/O chunk in order to make inline bulk data reloadable
					const int64 ExportBundleChunkOffset = (ActiveFPLB->StartFastPathLoadBuffer - ActiveFPLB->OriginalFastPathLoadBuffer);
					Meta.SetOffset(ExportBundleChunkOffset);
					BulkData.BulkChunkId = CreateIoChunkId(PackageId.Value(), ChunkIndex, EIoChunkType::ExportBundleData);
					Serialize(BulkData.ReallocateData(PayloadSize), PayloadSize);
				}
		}
		else if (Meta.HasAnyFlags(BULKDATA_MemoryMappedPayload))
		{
#if UE_DISABLE_COOKEDINDEX_FOR_MEMORYMAPPED
			CookedIndex = FBulkDataCookedIndex::Default;
#endif //UE_DISABLE_COOKEDINDEX_FOR_MEMORYMAPPED

			BulkData.BulkChunkId = CreateBulkDataIoChunkId(PackageId.Value(), ChunkIndex, CookedIndex.GetValue(), EIoChunkType::MemoryMappedBulkData);

			if (Params.bAttemptMemoryMapping)
			{
				TIoStatusOr<FIoMappedRegion> Status = IoDispatcher.OpenMapped(BulkData.BulkChunkId, FIoReadOptions(Meta.GetOffset(), Meta.GetSize()));

				if (Status.IsOk())
				{
					FIoMappedRegion Mapping = Status.ConsumeValueOrDie();
					BulkData.DataAllocation.SetMemoryMappedData(&BulkData, Mapping.MappedFileHandle, Mapping.MappedFileRegion);
				}
				else
				{
					UE_LOGF(LogSerialization, Warning, "Memory map bulk data from chunk '%ls', offset '%lld', size '%lld' FAILED: %ls",
						*LexToString(BulkData.BulkChunkId), Meta.GetOffset(), Meta.GetSize(), *Status.Status().ToString());

					BulkData.ForceBulkDataResident();
				}
			}
		}
		else
		{
			const EIoChunkType ChunkType = Meta.HasAnyFlags(BULKDATA_OptionalPayload) ? EIoChunkType::OptionalBulkData : EIoChunkType::BulkData;
			BulkData.BulkChunkId = CreateBulkDataIoChunkId(PackageId.Value(), ChunkIndex, CookedIndex.GetValue(), ChunkType);

			if (Meta.HasAnyFlags(BULKDATA_DuplicateNonOptionalPayload))
			{
#if UE_DISABLE_COOKEDINDEX_FOR_NONDUPLICATE
				CookedIndex = FBulkDataCookedIndex::Default;
#endif //UE_DISABLE_COOKEDINDEX_FOR_NONDUPLICATE

				const FIoChunkId OptionalChunkId = CreateBulkDataIoChunkId(PackageId.Value(), ChunkIndex, CookedIndex.GetValue(), EIoChunkType::OptionalBulkData);

				if (IoDispatcher.DoesChunkExist(OptionalChunkId))
				{
					BulkData.BulkChunkId = OptionalChunkId;
					Meta.ClearFlags(BULKDATA_DuplicateNonOptionalPayload);
					Meta.AddFlags(BULKDATA_OptionalPayload);
					Meta.SetOffset(DuplicateSerialOffset);
				}
			}
		}

		return true;
	}
	//~ End FArchive::FLinkerLoad Interface

private:
	inline void SerializeBulkMeta(UE::BulkData::Private::FBulkMetaData& Meta, FBulkDataCookedIndex& CookedIndex, int64& DuplicateSerialOffset, int32 ElementSize)
	{
		using namespace UE::BulkData::Private;
		FArchive& Ar = *this;

		if (UNLIKELY(HeaderData->BulkDataMap.IsEmpty()))
		{
			FBulkMetaData::FromSerialized(Ar, ElementSize, Meta, DuplicateSerialOffset);
		}
		else
		{
			const int32 Size = sizeof(FBulkDataMapEntry);
			int32 EntryIndex = INDEX_NONE;
			Ar << EntryIndex;
			const FBulkDataMapEntry& Entry = HeaderData->BulkDataMap[EntryIndex];
			Meta.SetFlags(static_cast<EBulkDataFlags>(Entry.Flags));
			Meta.SetOffset(Entry.SerialOffset);
			Meta.SetSize(Entry.SerialSize);

#if !USE_RUNTIME_BULKDATA
			// If the payload was compressed at package level then we will not be able to decompress it properly as that requires
			// us to know the compressed size (SizeOnDisk) which we do not keep track of when the package is stored by the IoDispatcher.
			// The BULKDATA_SerializeCompressed flag is removed during cooking/staging so the flag should never be set at this point,
			// the assert is just a paranoid safety check.
			checkf(Meta.HasAnyFlags(BULKDATA_SerializeCompressed) == false, TEXT("Package level compression is not supported by the IoDispatcher: '%s'"), *PackageDesc->UPackageName.ToString());

			// Since we know that the payload is not compressed there is no difference between the in memory size and the size of disk
			Meta.SetSizeOnDisk(Entry.SerialSize);
#endif //!USE_RUNTIME_BULKDATA

			DuplicateSerialOffset = Entry.DuplicateSerialOffset;
			CookedIndex = Entry.CookedIndex;
		}

		Meta.AddFlags(static_cast<EBulkDataFlags>(BULKDATA_UsesIoDispatcher | BULKDATA_LazyLoadable));
#if WITH_EDITOR
		if (GIsEditor)
		{
			Meta.ClearFlags(BULKDATA_SingleUse);
		}
#endif
	}

	friend FAsyncPackage2;
	FIoDispatcher& IoDispatcher;
#if (!DEVIRTUALIZE_FLinkerLoad_Serialize)
	FArchive::FFastPathLoadBuffer InlineFPLB;
	FArchive::FFastPathLoadBuffer* ActiveFPLB;
#endif

	UObject* TemplateForGetArchetypeFromLoader = nullptr;

	FAsyncPackageDesc2* PackageDesc = nullptr;
	FPackageImportStore* ImportStore = nullptr;
	TArray<FExternalReadCallback>* ExternalReadDependencies;
	const FAsyncPackageHeaderData* HeaderData = nullptr;
	const FLinkerInstancingContext* InstanceContext = nullptr;
	UObject* CurrentExport = nullptr;
	uint64 ExportSerialOffset = 0;
	uint64 ExportSerialSize = 0;
	bool bIsOptionalSegment = false;

	void FixupSoftObjectPathForInstancedPackage(FSoftObjectPath& InOutSoftObjectPath)
	{
		if (InstanceContext)
		{
			InstanceContext->FixupSoftObjectPath(InOutSoftObjectPath);
		}
	}
};

enum class EAsyncPackageLoadingState2 : uint8
{
	NewPackage,
	WaitingForIo,
	ProcessPackageSummary,
	ProcessDependencies,
	WaitingForDependencies,
	DependenciesReady,
	CreateExports,
	WaitingForImportDependencies,
	ResolveImports,
	PreloadExports,
	WaitingForExternalReads,
	ExportsDone,
	PostLoad,
	DeferredPostLoad,
	DeferredPostLoadDone,
	Finalize,
	PostLoadInstances,
	CreateClusters,
	Complete,
	DeferredClearImportedPackages,
	DeferredDelete,
};

const TCHAR* LexToString(EAsyncPackageLoadingState2 AsyncPackageLoadingState)
{
	switch (AsyncPackageLoadingState)
	{
	case EAsyncPackageLoadingState2::NewPackage: return TEXT("NewPackage");
	case EAsyncPackageLoadingState2::WaitingForIo: return TEXT("WaitingForIo");
	case EAsyncPackageLoadingState2::ProcessPackageSummary: return TEXT("ProcessPackageSummary");
	case EAsyncPackageLoadingState2::ProcessDependencies: return TEXT("ProcessDependencies");
	case EAsyncPackageLoadingState2::WaitingForDependencies: return TEXT("WaitingForDependencies");
	case EAsyncPackageLoadingState2::DependenciesReady: return TEXT("DependenciesReady");
	case EAsyncPackageLoadingState2::CreateExports: return TEXT("CreateExports");
	case EAsyncPackageLoadingState2::WaitingForImportDependencies: return TEXT("WaitingForImportDependencies");
	case EAsyncPackageLoadingState2::ResolveImports: return TEXT("ResolveImports");
	case EAsyncPackageLoadingState2::PreloadExports: return TEXT("PreloadExports");
	case EAsyncPackageLoadingState2::WaitingForExternalReads: return TEXT("WaitingForExternalReads");
	case EAsyncPackageLoadingState2::ExportsDone: return TEXT("ExportsDone");
	case EAsyncPackageLoadingState2::PostLoad: return TEXT("PostLoad");
	case EAsyncPackageLoadingState2::DeferredPostLoad: return TEXT("DeferredPostLoad");
	case EAsyncPackageLoadingState2::DeferredPostLoadDone: return TEXT("DeferredPostLoadDone");
	case EAsyncPackageLoadingState2::Finalize: return TEXT("Finalize");
	case EAsyncPackageLoadingState2::PostLoadInstances: return TEXT("PostLoadInstances");
	case EAsyncPackageLoadingState2::CreateClusters: return TEXT("CreateClusters");
	case EAsyncPackageLoadingState2::Complete: return TEXT("Complete");
	case EAsyncPackageLoadingState2::DeferredClearImportedPackages: return TEXT("DeferredClearImportedPackages");
	case EAsyncPackageLoadingState2::DeferredDelete: return TEXT("DeferredDelete");
	default:
	checkNoEntry();
	return TEXT("Unknown");
	}
};

class FEventLoadGraphAllocator;
struct FAsyncLoadEventSpec;
struct FAsyncLoadingThreadState2;
class FAsyncLoadEventQueue2;
class FAsyncLoadingSyncLoadContext;

enum class EEventLoadNodeExecutionResult : uint8
{
	Timeout,
	Complete,
};

/** [EDL] Event Load Node */
class FEventLoadNode2
{
public:
	FEventLoadNode2(const FAsyncLoadEventSpec* InSpec, FAsyncPackage2* InPackage, int32 InBarrierCount);
	void DependsOn(FEventLoadNode2* Other);
	void AddBarrier();
	void AddBarrier(int32 Count);
	void ReleaseBarrier(FAsyncLoadingThreadState2* ThreadState);
	EEventLoadNodeExecutionResult Execute(FAsyncLoadingThreadState2& ThreadState);

	FAsyncPackage2* GetPackage() const
	{
		return Package;
	}

	uint64 GetSyncLoadContextId() const;

	const FAsyncLoadEventSpec* GetSpec() const
	{
		return Spec;
	}
private:
	friend class FAsyncLoadEventQueue2;
	friend struct FAsyncPackage2;
	friend TIoPriorityQueue<FEventLoadNode2>;

	FEventLoadNode2() = default;

	void ProcessDependencies(FAsyncLoadingThreadState2& ThreadState);
	void Fire(FAsyncLoadingThreadState2* ThreadState);
	void ParallelLoadingWork(FAsyncLoadingThreadState2* InParentState);

	const FAsyncLoadEventSpec* Spec = nullptr;
	FAsyncPackage2* Package = nullptr;
	union
	{
		FEventLoadNode2* SingleDependent;
		FEventLoadNode2** MultipleDependents;
	};
	FEventLoadNode2* Prev = nullptr;
	FEventLoadNode2* Next = nullptr;
	uint32 DependenciesCount = 0;
	uint32 DependenciesCapacity = 0;
	int32 Priority = 0;
	TAsyncAtomic<int32> BarrierCount{ 0 };
	enum EQueueStatus
	{
		QueueStatus_None = 0,
		QueueStatus_Local = 1,
		QueueStatus_External = 2
	};
	uint8 QueueStatus = QueueStatus_None;
	TAsyncAtomic<bool> bIsUpdatingDependencies{ false };
	TAsyncAtomic<bool> bIsDone{ false };
};

class FAsyncLoadEventGraphAllocator
{
public:
	FEventLoadNode2** AllocArcs(uint32 Count)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(AllocArcs);
		SIZE_T Size = Count * sizeof(FEventLoadNode2*);
		TotalArcCount += Count;
		TotalAllocated += Size;
		return reinterpret_cast<FEventLoadNode2**>(FMemory::Malloc(Size));
	}

	void FreeArcs(FEventLoadNode2** Arcs, uint32 Count)
	{
		//TRACE_CPUPROFILER_EVENT_SCOPE(FreeArcs);
		FMemory::Free(Arcs);
		SIZE_T Size = Count * sizeof(FEventLoadNode2*);
		TotalAllocated -= Size;
		TotalArcCount -= Count;
	}

	TAsyncAtomic<int64> TotalArcCount{ 0 };
	TAsyncAtomic<int64> TotalAllocated{ 0 };
};

/** Queue based on DepletableMpscQueue.h
	Producers will add nodes to the linked list defined by Sentinel->Tail
	Nodes will be executed from the linked list defined by LocalHead->LocalTail
	All the nodes from the producer queue will be moved to the local queue when needed */
class FAsyncLoadEventQueue2
{
public:
	FAsyncLoadEventQueue2();
	~FAsyncLoadEventQueue2();

	void SetZenaphore(FZenaphore* InZenaphore)
	{
		Zenaphore = InZenaphore;
	}

	void SetWakeEvent(UE::FManualResetEvent* InEvent)
	{
		WakeEvent = InEvent;
	}

	void SetOwnerThread(const FAsyncLoadingThreadState2* InOwnerThread)
	{
		OwnerThread = InOwnerThread;
	}

	bool PopAndExecute(FAsyncLoadingThreadState2& ThreadState);
	void Push(FAsyncLoadingThreadState2* ThreadState, FEventLoadNode2* Node);
	bool ExecuteSyncLoadEvents(FAsyncLoadingThreadState2& ThreadState);
	void UpdatePackagePriority(FAsyncPackage2* Package);
	bool IsEmptyForDebug()
	{
		FScopeLock Lock(&ExternalCritical);
		return LocalQueue.IsEmpty() && ExternalQueue.IsEmpty();
	}
private:
	void PushLocal(FEventLoadNode2* Node);
	void PushExternal(FEventLoadNode2* Node);
	bool GetMaxPriorityInExternalQueue(int32& OutMaxPriority)
	{
		int64 StateValue = ExternalQueueState.load();
		if (StateValue == MIN_int64)
		{
			return false;
		}
		else
		{
			OutMaxPriority = static_cast<int32>(StateValue);
			return true;
		}
	}
	void UpdateExternalQueueState()
	{
		if (ExternalQueue.IsEmpty())
		{
			ExternalQueueState.store(MIN_int64);
		}
		else
		{
			ExternalQueueState.store(ExternalQueue.GetMaxPriority());
		}
	}

	const FAsyncLoadingThreadState2* OwnerThread = nullptr;
	FZenaphore* Zenaphore = nullptr;
	UE::FManualResetEvent* WakeEvent = nullptr;
	TIoPriorityQueue<FEventLoadNode2> LocalQueue;
	FCriticalSection ExternalCritical;
	TIoPriorityQueue<FEventLoadNode2> ExternalQueue;
	TAsyncAtomic<int64> ExternalQueueState = MIN_int64;
	FEventLoadNode2* TimedOutEventNode = nullptr;
	int32 ExecuteSyncLoadEventsCallCounter = 0;
};

enum class EExecutionType
{
	Normal,
	Immediate,
	Parallel,
};

struct FAsyncLoadEventSpec
{
	typedef EEventLoadNodeExecutionResult(*FAsyncLoadEventFunc)(FAsyncLoadingThreadState2&, FAsyncPackage2*);
	FAsyncLoadEventFunc Func = nullptr;
	FAsyncLoadEventQueue2* EventQueue = nullptr;
	EExecutionType ExecutionType = EExecutionType::Normal;
	const TCHAR* Name = nullptr;
};

struct FAsyncLoadingThreadState2
{
	struct FTimeLimitScope
	{
		bool bUseTimeLimit = false;
		double TimeLimit = 0.0;
		double StartTime = 0.0;
		FAsyncLoadingThreadState2& ThreadState;

		FTimeLimitScope(FAsyncLoadingThreadState2& InThreadState, bool bInUseTimeLimit, double InTimeLimit, bool bCondition=true)
			: ThreadState(InThreadState)
		{
			bUseTimeLimit = ThreadState.bUseTimeLimit;
			TimeLimit = ThreadState.TimeLimit;
			StartTime = ThreadState.StartTime;

			if (bCondition)
			{
				ThreadState.bUseTimeLimit = bInUseTimeLimit;
				ThreadState.TimeLimit = InTimeLimit;
				if (bInUseTimeLimit)
				{
					ThreadState.StartTime = FPlatformTime::Seconds();
				}
				else
				{
					ThreadState.StartTime = 0.0;
				}
			}
		}

		~FTimeLimitScope()
		{
			ThreadState.bUseTimeLimit = bUseTimeLimit;
			ThreadState.TimeLimit = TimeLimit;
			ThreadState.StartTime = StartTime;
		}
	};

	static void Set(FAsyncLoadingThreadState2* State)
	{
		check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
		check(!FPlatformTLS::GetTlsValue(TlsSlot));
		FPlatformTLS::SetTlsValue(TlsSlot, State);
	}

	static void Reset()
	{
		check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
		FPlatformTLS::SetTlsValue(TlsSlot, nullptr);
	}

	static FAsyncLoadingThreadState2* Get()
	{
		check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
		return static_cast<FAsyncLoadingThreadState2*>(FPlatformTLS::GetTlsValue(TlsSlot));
	}

	bool IsTimeLimitExceeded(const TCHAR* InLastTypeOfWorkPerformed = nullptr, UObject* InLastObjectWorkWasPerformedOn = nullptr)
	{
		bool bTimeLimitExceeded = false;

		if (bUseTimeLimit)
		{
			double CurrentTime = FPlatformTime::Seconds();
			bTimeLimitExceeded = CurrentTime - StartTime > TimeLimit;

			if (bTimeLimitExceeded && GWarnIfTimeLimitExceeded)
			{
				IsTimeLimitExceededPrint(StartTime, CurrentTime, LastTestTime, TimeLimit, InLastTypeOfWorkPerformed, InLastObjectWorkWasPerformedOn);
			}

			LastTestTime = CurrentTime;
		}

		if (!bTimeLimitExceeded)
		{
			const bool bGCIsWaiting = IsGarbageCollectionWaiting();

			// Do not time out sync loads due to GC 
			if (bGCIsWaiting && SyncLoadContextStack.IsEmpty() && !IsInParallelLoadingThread())
			{
				bTimeLimitExceeded = true;
				UE_LOGF(LogStreaming, Verbose, "Timing out async loading (last work %ls) due to Garbage Collection request", InLastTypeOfWorkPerformed);
			}
		}

		return bTimeLimitExceeded;
	}

	bool UseTimeLimit() const
	{
		return bUseTimeLimit;
	}

	void MarkAsActive()
	{
		if (ParentState)
		{
			ParentState->MarkAsActive();
		}
		else
		{
			bIsActive.store(true, std::memory_order_relaxed);
		}
	}

	void ResetActivity()
	{
		bIsActive.store(false, std::memory_order_relaxed);
	}

	bool IsActive() const
	{
		return bIsActive.load(std::memory_order_relaxed);
	}

	TArray<FEventLoadNode2*> NodesToFire;
	TArray<FEventLoadNode2*> CurrentlyExecutingEventNodeStack;
	TArray<FAsyncLoadingSyncLoadContext*> SyncLoadContextStack;
	TArray<FAsyncPackage2*> PackagesOnStack;
	TMpscQueue<FAsyncLoadingSyncLoadContext*> SyncLoadContextsCreatedOnOtherThreads;
	TMpscQueue<FAsyncPackage2*> PackagesToReprioritize;
	FAsyncLoadingThreadState2* ParentState = nullptr;
	bool bIsAsyncLoadingThread = false;
	bool bCanAccessAsyncLoadingThreadData = false;
	// used to probe activity for stall detection by the game thread
	// we use relaxed memory ordering with a simple store to avoid any form of costly interlocked operations.
	TAsyncAtomic<bool> bIsActive = false;
	bool bShouldFireNodes = true;
	bool bUseTimeLimit = false;

	double TimeLimit = 0.0;
	double StartTime = 0.0;
	double LastTestTime = -1.0;
	static uint32 TlsSlot;
	const bool bIsWorkerThread = false; // can only be set at construction time
};

uint32 FAsyncLoadingThreadState2::TlsSlot;

/**
 * Event node.
 */
enum EEventLoadNode2 : uint8
{
	Package_ProcessSummary,
	Package_ProcessDependencies,
	Package_DependenciesReady,
	Package_CreateExports,
	Package_ResolveImports,
	Package_PreloadExports,
	Package_ExportsSerialized,
	Package_PostLoad,
	Package_DeferredPostLoad,
	Package_NumPhases,
};

struct FAsyncPackageData
{
	uint8* MemoryBuffer0 = nullptr;
	uint8* MemoryBuffer1 = nullptr;
	TArrayView<FExportObject> Exports;
	TArrayView<FExportCell> CellExports;
	TArrayView<FAsyncPackage2*> ImportedAsyncPackages;
	TArrayView<const FShaderHash> ShaderMapHashes;
	int32 TotalExportBundleCount = 0;
	TAsyncAtomic<bool> bCellExportsInitialized = false;
};

struct FAsyncPackageSerializationState
{
	FIoRequest IoRequest;

	void ReleaseIoRequest()
	{
		IoRequest.Release();
	}
};

#if ALT2_ENABLE_NEW_ARCHIVE_FOR_LINKERLOAD
class FLinkerLoadArchive2 final
	: public FArchive
{
public:
	FLinkerLoadArchive2(const FPackagePath& InPackagePath)
	{
		FOpenAsyncPackageResult UassetOpenResult = IPackageResourceManager::Get().OpenAsyncReadPackage(InPackagePath, EPackageSegment::Header);
		UassetFileHandle = UassetOpenResult.Handle.Release();
		check(UassetFileHandle); // OpenAsyncReadPackage guarantees a non-null return value; the handle will fail to read later if the path does not exist
		if (UassetOpenResult.Format != EPackageFormat::Binary)
		{
			UE_LOGF(LogStreaming, Fatal, "Only binary assets are supported"); // TODO
			SetError();
			return;
		}
		bNeedsEngineVersionChecks = UassetOpenResult.bNeedsEngineVersionChecks;

		if (FPlatformProperties::RequiresCookedData())
		{
			FOpenAsyncPackageResult UexpOpenResult = IPackageResourceManager::Get().OpenAsyncReadPackage(InPackagePath, EPackageSegment::Exports);
			UexpFileHandle = UexpOpenResult.Handle.Release();
			check(UexpFileHandle); // OpenAsyncReadPackage guarantees a non-null return value; the handle will fail to read later if the path does not exist
			if (UexpOpenResult.Format != EPackageFormat::Binary)
			{
				UE_LOGF(LogStreaming, Fatal, "Only binary assets are supported"); // TODO
				SetError();
				return;
			}
		}
		else
		{
			UexpSize = 0;
		}
	}

	virtual ~FLinkerLoadArchive2()
	{
		WaitForRequests();
		delete UassetFileHandle;
		delete UexpFileHandle;
	}

	bool NeedsEngineVersionChecks() const
	{
		return bNeedsEngineVersionChecks;
	}

	void BeginRead(FEventLoadNode2* InDependentNode)
	{
		check(PendingSizeRequests == 0);
		check(PendingReadRequests == 0);
		check(!DependentNode);
		if (UexpFileHandle)
		{
			PendingSizeRequests = 2;
			PendingReadRequests = 2;
		}
		else
		{
			PendingSizeRequests = 1;
			PendingReadRequests = 1;
		}
		DependentNode = InDependentNode;
		StartSizeRequests();
	}

	virtual int64 TotalSize() override
	{
		check(bDone);
		return ActiveFPLB->EndFastPathLoadBuffer - ActiveFPLB->OriginalFastPathLoadBuffer;
	}

	virtual int64 Tell() override
	{
		check(bDone);
		return ActiveFPLB->StartFastPathLoadBuffer - ActiveFPLB->OriginalFastPathLoadBuffer;
	}

	virtual void Seek(int64 InPos) override
	{
		check(bDone);
		ActiveFPLB->StartFastPathLoadBuffer = ActiveFPLB->OriginalFastPathLoadBuffer + InPos;
		check(ActiveFPLB->StartFastPathLoadBuffer <= ActiveFPLB->EndFastPathLoadBuffer);
	}

	virtual void Serialize(void* Data, int64 Length) override
	{
		check(bDone);
		if (!Length || IsError())
		{
			return;
		}
		check(ActiveFPLB->StartFastPathLoadBuffer + Length <= ActiveFPLB->EndFastPathLoadBuffer);
		FMemory::Memcpy(Data, ActiveFPLB->StartFastPathLoadBuffer, Length);
		ActiveFPLB->StartFastPathLoadBuffer += Length;
	}

	/*virtual bool Close() override;
	virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize) override;
	virtual void FlushCache() override;*/

private:
	void StartSizeRequests()
	{
		FAsyncFileCallBack UassetSizeRequestCallbackFunction = [this](bool bWasCancelled, IAsyncReadRequest* Request)
			{
				if (!bWasCancelled)
				{
					UassetSize = Request->GetSizeResults();
				}
				if (--PendingSizeRequests == 0)
				{
					StartReadRequests();
				}
			};
		UassetSizeRequest = UassetFileHandle->SizeRequest(&UassetSizeRequestCallbackFunction);
		if (UexpFileHandle)
		{
			FAsyncFileCallBack UexpSizeRequestCallbackFunction = [this](bool bWasCancelled, IAsyncReadRequest* Request)
				{
					if (!bWasCancelled)
					{
						UexpSize = Request->GetSizeResults();
					}
					if (--PendingSizeRequests == 0)
					{
						StartReadRequests();
					}
				};
			UexpSizeRequest = UexpFileHandle->SizeRequest(&UexpSizeRequestCallbackFunction);
		}
	}

	void StartReadRequests()
	{
		FAsyncFileCallBack ReadRequestCallbackFunction = [this](bool bWasCancelled, IAsyncReadRequest* Request)
			{
				if (bWasCancelled)
				{
					bFailed = true;
				}
				if (--PendingReadRequests == 0)
				{
					FinishedReading();
				}
			};
		if (UassetSize <= 0 || (UexpFileHandle && UexpSize <= 0))
		{
			SetError();
			FinishedReading();
			return;
		}
		IoBuffer = FIoBuffer(UassetSize + UexpSize);
		UassetReadRequest = UassetFileHandle->ReadRequest(0, UassetSize, AIOP_Normal, &ReadRequestCallbackFunction, IoBuffer.Data());
		if (UexpFileHandle)
		{
			UexpReadRequest = UexpFileHandle->ReadRequest(0, UexpSize, AIOP_Normal, &ReadRequestCallbackFunction, IoBuffer.Data() + UassetSize);
		}
	}

	void FinishedReading()
	{
		ActiveFPLB->OriginalFastPathLoadBuffer = IoBuffer.Data();
		ActiveFPLB->StartFastPathLoadBuffer = IoBuffer.Data();
		ActiveFPLB->EndFastPathLoadBuffer = IoBuffer.Data() + IoBuffer.DataSize();
		bDone = true;
		DependentNode->ReleaseBarrier();
		DependentNode = nullptr;
	}

	void WaitForRequests()
	{
		if (UassetSizeRequest)
		{
			UassetSizeRequest->WaitCompletion();
			delete UassetSizeRequest;
			UassetSizeRequest = nullptr;
		}
		if (UexpSizeRequest)
		{
			UexpSizeRequest->WaitCompletion();
			delete UexpSizeRequest;
			UexpSizeRequest = nullptr;
		}
		if (UassetReadRequest)
		{
			UassetReadRequest->WaitCompletion();
			delete UassetReadRequest;
			UassetReadRequest = nullptr;
		}
		if (UexpReadRequest)
		{
			UexpReadRequest->WaitCompletion();
			delete UexpReadRequest;
			UexpReadRequest = nullptr;
		}
	}

#if (!DEVIRTUALIZE_FLinkerLoad_Serialize)
	FArchive::FFastPathLoadBuffer InlineFPLB;
	FArchive::FFastPathLoadBuffer* ActiveFPLB = &InlineFPLB;
#endif
	FEventLoadNode2* DependentNode = nullptr;
	FIoBuffer IoBuffer;
	int64 Offset = 0;
	IAsyncReadFileHandle* UassetFileHandle = nullptr;
	IAsyncReadFileHandle* UexpFileHandle = nullptr;
	IAsyncReadRequest* UassetSizeRequest = nullptr;
	IAsyncReadRequest* UexpSizeRequest = nullptr;
	IAsyncReadRequest* UassetReadRequest = nullptr;
	IAsyncReadRequest* UexpReadRequest = nullptr;
	int64 UassetSize = -1;
	int64 UexpSize = -1;
	TAsyncAtomic<int8> PendingSizeRequests = 0;
	TAsyncAtomic<int8> PendingReadRequests = 0;
	TAsyncAtomic<bool> bDone = false;
	TAsyncAtomic<bool> bFailed = false;
	bool bNeedsEngineVersionChecks = false;
};
#endif // ALT2_ENABLE_NEW_ARCHIVE_FOR_LINKERLOAD

/**
* Node in a lock-free intrusive linked list of progress callbacks.
* Each entry has its own InvocationLock to serialize callback invocations,
* ensuring in-order delivery even when multiple threads call CallProgressCallbacks concurrently.
* The Next pointer is immutable after the node is pushed onto the list.
*/
struct FProgressCallbackEntry
{
	TSharedPtr<FLoadPackageAsyncProgressDelegate> Delegate;
	UE::FMutex InvocationLock;
	EAsyncLoadingProgress LastNotifiedProgress = EAsyncLoadingProgress::Failed; // Failed = sentinel for "nothing notified yet"
	std::atomic<FProgressCallbackEntry*> Next = nullptr;
};
	
// Fire events up to the current progress cooperatively between threads in case some were missed (i.e. late load request registration).
void CallProgressCallback(FProgressCallbackEntry& Entry, FName PackageName, UPackage* LoadedPackage, EAsyncLoadingProgress CurrentProgress)
{
	// Progress states in dispatch order
	static constexpr EAsyncLoadingProgress ProgressOrder[] = {
		EAsyncLoadingProgress::Started,
		EAsyncLoadingProgress::Read,
		EAsyncLoadingProgress::Serialized,
		EAsyncLoadingProgress::FullyLoaded
	};

	// Per-entry lock serializes invocations for this callback, ensuring
	// in-order delivery even when multiple threads call concurrently.
	UE::TUniqueLock Lock(Entry.InvocationLock);
	if (CurrentProgress > Entry.LastNotifiedProgress)
	{
		for (EAsyncLoadingProgress Progress : ProgressOrder)
		{
			if (Progress > Entry.LastNotifiedProgress && Progress <= CurrentProgress)
			{
				FLoadPackageAsyncProgressParams Params
				{
					.PackageName = PackageName,
					.LoadedPackage = LoadedPackage,
					.ProgressType = Progress
				};
				Entry.Delegate->Invoke(Params);
			}
		}
		Entry.LastNotifiedProgress = CurrentProgress;
	}
}

EAsyncLoadingProgress GetLoadingProgressFromState(EAsyncPackageLoadingState2 AsyncPackageLoadingState)
{
	if (AsyncPackageLoadingState >= EAsyncPackageLoadingState2::ExportsDone)
	{
		return EAsyncLoadingProgress::Serialized;
	}
	if (AsyncPackageLoadingState >= EAsyncPackageLoadingState2::ProcessPackageSummary)
	{
		return EAsyncLoadingProgress::Read;
	}
	if (AsyncPackageLoadingState >= EAsyncPackageLoadingState2::WaitingForIo)
	{
		return EAsyncLoadingProgress::Started;
	}

	return EAsyncLoadingProgress::Failed;
}

/**
* Structure containing intermediate data required for async loading of all exports of a package.
*/

struct FAsyncPackage2
{
	friend struct FAsyncPackageScope2;
	friend class FAsyncLoadingThread2;
	friend class FAsyncLoadEventQueue2;
	friend class FAsyncLoadingVerseRoot;

	FAsyncPackage2(
		FAsyncLoadingThreadState2& ThreadState,
		const FAsyncPackageDesc2& InDesc,
		FAsyncLoadingThread2& InAsyncLoadingThread,
		FAsyncLoadEventGraphAllocator& InGraphAllocator,
		const FAsyncLoadEventSpec* EventSpecs);
	virtual ~FAsyncPackage2();


	void AddRef()
	{
		RefCount.fetch_add(1);
	}

	bool TryAddRef()
	{
		for (;;)
		{
			int32 CurrentRefCount = RefCount.load();
			if (CurrentRefCount == 0)
			{
				return false;
			}
			if (RefCount.compare_exchange_strong(CurrentRefCount, CurrentRefCount + 1))
			{
				return true;
			}
		}
	}

	void Release();

	bool TryAddHeaderRef()
	{
		for (;;)
		{
			int32 CurrentRefCount = HeaderRefCount.load(std::memory_order_relaxed);
			if (CurrentRefCount == 0)
			{
				return false;
			}
			
			if (HeaderRefCount.compare_exchange_strong(CurrentRefCount, CurrentRefCount + 1))
			{
				return true;
			}
		}
	}

	void ReleaseHeaderRef()
	{
		int32 OldRefCount = HeaderRefCount.fetch_sub(1);
		check(OldRefCount > 0);
		if (OldRefCount == 1)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncPackage2::ReleaseHeaderRef);
			HeaderData.Reset();
			SerializationState.ReleaseIoRequest();
#if WITH_EDITOR
			if (OptionalSegmentHeaderData.IsSet())
			{
				OptionalSegmentHeaderData->Reset();
				OptionalSegmentSerializationState->ReleaseIoRequest();
			}
#endif
		}
	}

	void ClearImportedPackages();

	/**
	 * @return Time load begun. This is NOT the time the load was requested in the case of other pending requests.
	 */
	double GetLoadStartTime() const;

	void AddCompletionCallback(TUniquePtr<FLoadPackageAsyncDelegate>&& Callback);
	void AddProgressCallback(TSharedPtr<FLoadPackageAsyncProgressDelegate> Callback);

	FORCEINLINE UPackage* GetLinkerRoot() const
	{
		return LinkerRoot;
	}

	/** Returns true if loading has failed */
	FORCEINLINE bool HasLoadFailed() const
	{
		return LoadStatus != EAsyncLoadingResult::Succeeded;
	}

	FORCEINLINE EAsyncLoadingResult::Type GetLoadStatus() const
	{
		return LoadStatus;
	}

	/** Adds new request ID to the existing package */
	void AddRequestID(FAsyncLoadingThreadState2& ThreadState, int32 Id);

	uint64 GetSyncLoadContextId() const
	{
		return SyncLoadContextId;
	}

	void AddConstructedObject(UObject* Object, bool bSubObjectThatAlreadyExists)
	{
		UE::TUniqueLock ScopeLock(ConstructedObjectsMutex);

		// this is required even with the lock above since other
		// parts are using ConstructedObjects without any locking
		// but we are expecting that only this part runs concurrently.

		UE_MT_SCOPED_WRITE_ACCESS(ConstructedObjectsAccessDetector);

		if (bSubObjectThatAlreadyExists)
		{
			ConstructedObjects.AddUnique(Object);
		}
		else
		{
			// Mark objects created during async loading process (e.g. from within PostLoad or CreateExport) as async loaded so they 
			// cannot be found. This requires also keeping track of them so we can remove the async loading flag later one when we 
			// finished routing PostLoad to all objects.
			// 
			// Once we pass the publish gate, we mark new objects in phase 2, they are generally preloaded as soon
			// as they are created. However, it leaves a small window where GT could find a RF_NeedLoad object from phase 2...
			if (ObjectsNowInPhase2)
			{
				Object->SetInternalFlags(EInternalObjectFlags::AsyncLoadingPhase2 | EInternalObjectFlags::Async);

				// This must be cleared after Phase2 is set to avoid any window where the object can be erroneously found.
				Object->ClearInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1);
			}
			else
			{
				Object->SetInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1 | EInternalObjectFlags::Async);
			}

			ConstructedObjects.Add(Object);
		}
	}

	void MoveConstructedObjectsToPhase2();
	void ClearConstructedObjects();

	/** Class specific callback for initializing non-native objects */
	EAsyncPackageState::Type PostLoadInstances(FAsyncLoadingThreadState2& ThreadState);

	/** Creates GC clusters from loaded objects */
	EAsyncPackageState::Type CreateClusters(FAsyncLoadingThreadState2& ThreadState);

	void ImportPackagesRecursive(FAsyncLoadingThreadState2& ThreadState, FIoBatch& IoBatch, FPackageStore& PackageStore);
	void StartLoading(FAsyncLoadingThreadState2& ThreadState, FIoBatch& IoBatch);

	void InitializeLinkerLoadState(const FLinkerInstancingContext* InstancingContext);
	void CreateLinker(const FLinkerInstancingContext* InstancingContext);
	void DetachLinker();

private:
	void ImportPackagesRecursiveInner(FAsyncLoadingThreadState2& ThreadState, FIoBatch& IoBatch, FPackageStore& PackageStore, FAsyncPackageHeaderData& Header);

	uint8 PackageNodesMemory[EEventLoadNode2::Package_NumPhases * sizeof(FEventLoadNode2)];
	TArrayView<FEventLoadNode2> PackageNodes;
	/** Basic information associated with this package */
	FAsyncPackageDesc2 Desc;
	FAsyncPackageData Data;
	FAsyncPackageHeaderData HeaderData;
	FAsyncPackageSerializationState SerializationState;
	TSet<FAsyncPackage2*> AdditionalImportedAsyncPackages;

#if WITH_EDITOR
	TOptional<FAsyncPackageHeaderData> OptionalSegmentHeaderData;
	TOptional<FAsyncPackageSerializationState> OptionalSegmentSerializationState;
	bool bRequestOptionalChunk = false;
#endif

	struct FLinkerLoadState
	{
		FLinkerLoad* Linker = nullptr;
		int32 ProcessingImportedPackageIndex = 0;
		int32 CreateImportIndex = 0;
		int32 CreateExportIndex = 0;
		int32 SerializeExportIndex = 0;
		int32 PostLoadExportIndex = 0;
#if WITH_METADATA
		int32 MetaDataIndex = -1;
#endif // WITH_METADATA
		bool bIsCurrentlyResolvingImports = false;
		bool bIsCurrentlyCreatingExports = false;
		bool bContainsClasses = false;

		FAsyncPackageLinkerLoadHeaderData LinkerLoadHeaderData;
	};
	TOptional<FLinkerLoadState> LinkerLoadState;

	/** Cached async loading thread object this package was created by */
	FAsyncLoadingThread2& AsyncLoadingThread;
	FAsyncLoadEventGraphAllocator& GraphAllocator;
	FPackageImportStore ImportStore;
	/** Package which is going to have its exports and imports loaded */
	UPackage* LinkerRoot = nullptr;
	// The sync load context associated with this package
	TAsyncAtomic<uint64>			SyncLoadContextId = 0;
	/** Time load begun. This is NOT the time the load was requested in the case of pending requests.	*/
	double						LoadStartTime = 0.0;
	/** RefCount on the package itself */
	TAsyncAtomic<int32>			RefCount = 1;
	/** RefCount on the HeaderData information to avoid use-after-free from other threads */
	TAsyncAtomic<int32>			HeaderRefCount = 1;
	bool						bHasStartedImportingPackages = false;
	int32						ProcessedExportBundlesCount = 0;
	/** Current bundle entry index in the current export bundle */
	int32						ExportBundleEntryIndex = 0;
	/** Current index into ExternalReadDependencies array used to spread wating for external reads over several frames			*/
	int32						ExternalReadIndex = 0;
	/** Current index into DeferredClusterObjects array used to spread routing CreateClusters over several frames			*/
	int32						DeferredClusterIndex = 0;
	/** Current index into Export objects array used to spread routing PostLoadInstances over several frames			*/
	int32						PostLoadInstanceIndex = 0;
	/** Current loading state of a package. */
	TAsyncAtomic<EAsyncPackageLoadingState2> AsyncPackageLoadingState{ EAsyncPackageLoadingState2::NewPackage };
	/** Whether the constructed objects have been moved to the second loading phase. */
	TAsyncAtomic<bool> ObjectsNowInPhase2 = false;
	/** Protects against the same package being serialized from different threads. */
	UE::FMutex                  SerializationMutex;

	struct FAllDependenciesState
	{
		FAsyncPackage2* WaitingForPackage = nullptr;
		FAsyncPackage2* PackagesWaitingForThisHead = nullptr;
		FAsyncPackage2* PackagesWaitingForThisTail = nullptr;
		FAsyncPackage2* PrevLink = nullptr;
		FAsyncPackage2* NextLink = nullptr;
		uint32 LastTick = 0;
		int32 PreOrderNumber = -1;
		bool bAssignedToStronglyConnectedComponent = false;
		bool bAllDone = false;

		void UpdateTick(int32 CurrentTick)
		{
			if (LastTick != CurrentTick)
			{
				LastTick = CurrentTick;
				PreOrderNumber = -1;
				bAssignedToStronglyConnectedComponent = false;
			}
		}

		static void AddToWaitList(FAllDependenciesState FAsyncPackage2::* StateMemberPtr, FAsyncPackage2* WaitListPackage, FAsyncPackage2* PackageToAdd)
		{
			check(WaitListPackage);
			check(PackageToAdd);
			check(WaitListPackage != PackageToAdd);
			FAllDependenciesState& WaitListPackageState = WaitListPackage->*StateMemberPtr;
			FAllDependenciesState& PackageToAddState = PackageToAdd->*StateMemberPtr;

			if (PackageToAddState.WaitingForPackage == WaitListPackage)
			{
				return;
			}
			if (PackageToAddState.WaitingForPackage)
			{
				FAllDependenciesState::RemoveFromWaitList(StateMemberPtr, PackageToAddState.WaitingForPackage, PackageToAdd);
			}

			check(!PackageToAddState.PrevLink);
			check(!PackageToAddState.NextLink);
			if (WaitListPackageState.PackagesWaitingForThisTail)
			{
				FAllDependenciesState& WaitListTailState = WaitListPackageState.PackagesWaitingForThisTail->*StateMemberPtr;
				check(!WaitListTailState.NextLink);
				WaitListTailState.NextLink = PackageToAdd;
				PackageToAddState.PrevLink = WaitListPackageState.PackagesWaitingForThisTail;
			}
			else
			{
				check(!WaitListPackageState.PackagesWaitingForThisHead);
				WaitListPackageState.PackagesWaitingForThisHead = PackageToAdd;
			}

			WaitListPackageState.PackagesWaitingForThisTail = PackageToAdd;
			WaitListPackage->AddRef();
			PackageToAddState.WaitingForPackage = WaitListPackage;
		}

		static void RemoveFromWaitList(FAllDependenciesState FAsyncPackage2::* StateMemberPtr, FAsyncPackage2* WaitListPackage, FAsyncPackage2* PackageToRemove)
		{
			check(WaitListPackage);
			check(PackageToRemove);

			FAllDependenciesState& WaitListPackageState = WaitListPackage->*StateMemberPtr;
			FAllDependenciesState& PackageToRemoveState = PackageToRemove->*StateMemberPtr;

			check(PackageToRemoveState.WaitingForPackage == WaitListPackage);
			if (PackageToRemoveState.PrevLink)
			{
				FAllDependenciesState& PrevLinkState = PackageToRemoveState.PrevLink->*StateMemberPtr;
				PrevLinkState.NextLink = PackageToRemoveState.NextLink;
			}
			else
			{
				check(WaitListPackageState.PackagesWaitingForThisHead == PackageToRemove);
				WaitListPackageState.PackagesWaitingForThisHead = PackageToRemoveState.NextLink;
			}

			if (PackageToRemoveState.NextLink)
			{
				FAllDependenciesState& NextLinkState = PackageToRemoveState.NextLink->*StateMemberPtr;
				NextLinkState.PrevLink = PackageToRemoveState.PrevLink;
			}
			else
			{
				check(WaitListPackageState.PackagesWaitingForThisTail == PackageToRemove);
				WaitListPackageState.PackagesWaitingForThisTail = PackageToRemoveState.PrevLink;
			}

			PackageToRemoveState.PrevLink = nullptr;
			PackageToRemoveState.NextLink = nullptr;
			FAsyncPackage2* WaitingForPackage = PackageToRemoveState.WaitingForPackage;
			PackageToRemoveState.WaitingForPackage = nullptr;
			WaitingForPackage->Release();
		}
	};
	FAllDependenciesState		AllDependenciesSetupState;
	FAllDependenciesState		AllDependenciesImportState;
	FAllDependenciesState		AllDependenciesPostLoadReadyState;
	FAllDependenciesState		AllDependenciesFullyLoadedState;

	/** True if our load has failed */
	EAsyncLoadingResult::Type	LoadStatus = EAsyncLoadingResult::Succeeded;
	/** True if this package was created by this async package */
	bool						bCreatedLinkerRoot = false;

	UE::FMutex ConstructedObjectsMutex;
	/** List of ConstructedObjects = Exports + UPackage + ObjectsCreatedFromExports */
	TArray<UObject*> ConstructedObjects;

	typedef TArray<int32, TInlineAllocator<1>> FRequestIDs;
	/** List of all request handles. */
	FRequestIDs RequestIDs;

	/** Detects if the constructed objects are improperly accessed by different threads at the same time. */
	UE_MT_DECLARE_MRSW_RECURSIVE_ACCESS_DETECTOR(ConstructedObjectsAccessDetector);
	TArray<FExternalReadCallback> ExternalReadDependencies;

	// The fields below are only valid until the package is marked as completed

	/** List of all request ids which caused the import of this package. */
	FRequestIDs ImportRequestIDs;
	/** Callbacks called when we finished loading this package */
	TArray<TUniquePtr<FLoadPackageAsyncDelegate>, TInlineAllocator<2>> CompletionCallbacks;
	/** Lock-free intrusive linked list of progress callbacks. @see FProgressCallbackEntry */
	std::atomic<FProgressCallbackEntry*> ProgressCallbacksHead = nullptr;

public:

	FAsyncLoadingThread2& GetAsyncLoadingThread()
	{
		return AsyncLoadingThread;
	}

	FAsyncLoadEventGraphAllocator& GetGraphAllocator()
	{
		return GraphAllocator;
	}

	static FAsyncPackage2* GetCurrentlyExecutingPackage(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* PackageToFilter = nullptr);

	static EEventLoadNodeExecutionResult Event_CreateExports(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	static EEventLoadNodeExecutionResult Event_ResolveImports(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	static EEventLoadNodeExecutionResult Event_PreloadExports(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	static EEventLoadNodeExecutionResult Event_ProcessPackageSummary(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	static EEventLoadNodeExecutionResult Event_ProcessDependencies(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	static EEventLoadNodeExecutionResult Event_DependenciesReady(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	static EEventLoadNodeExecutionResult Event_ExportsDone(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	static EEventLoadNodeExecutionResult Event_PostLoad(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	static EEventLoadNodeExecutionResult Event_DeferredPostLoad(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);

	bool CanRunNodeAsync(const FAsyncLoadEventSpec* Spec) const;

	void EventDrivenCreateExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex);
	void EventDrivenSerializeExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex, FExportArchive* Ar);

	void EventDrivenCreateCellExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex, FExportArchive* Ar);
	void EventDrivenSerializeCellExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex, FExportArchive* Ar);

	UObject* EventDrivenIndexToObject(const FAsyncPackageHeaderData& Header, FPackageObjectIndex Index, bool bCheckSerialized);
	template<class T>
	T* CastEventDrivenIndexToObject(const FAsyncPackageHeaderData& Header, FPackageObjectIndex Index, bool bCheckSerialized)
	{
		UObject* Result = EventDrivenIndexToObject(Header, Index, bCheckSerialized);
		if (!Result)
		{
			return nullptr;
		}
		return CastChecked<T>(Result);
	}

	FEventLoadNode2& GetPackageNode(EEventLoadNode2 Phase);

	/** [EDL] End Event driven loader specific stuff */

	void CallProgressCallbacks();
private:
	void InitializeExportArchive(FExportArchive& Ar, bool bIsOptionalSegment);
	void CreatePackageNodes(const FAsyncLoadEventSpec* EventSpecs);
	void SetupScriptDependencies();
#if ALT2_ENABLE_PACKAGE_DEPENDENCY_DEBUGGING
	bool HasDependencyToPackageDebug(FAsyncPackage2* Package);
	void CheckThatAllDependenciesHaveReachedStateDebug(FAsyncLoadingThreadState2& ThreadState, EAsyncPackageLoadingState2 PackageState, EAsyncPackageLoadingState2 PackageStateForCircularDependencies);
#endif
	struct FUpdateDependenciesStateRecursiveContext
	{
		FUpdateDependenciesStateRecursiveContext(FAllDependenciesState FAsyncPackage2::* InStateMemberPtr, EAsyncPackageLoadingState2 InWaitForPackageState, uint32 InCurrentTick, TFunctionRef<void(FAsyncPackage2*)> InOnStateReached)
			: StateMemberPtr(InStateMemberPtr)
			, WaitForPackageState(InWaitForPackageState)
			, OnStateReached(InOnStateReached)
			, CurrentTick(InCurrentTick)
		{

		}

		FAllDependenciesState FAsyncPackage2::* StateMemberPtr;
		EAsyncPackageLoadingState2 WaitForPackageState;
		TFunctionRef<void(FAsyncPackage2*)> OnStateReached;
		TArray<FAsyncPackage2*, TInlineAllocator<512>> S;
		TArray<FAsyncPackage2*, TInlineAllocator<512>> P;
		uint32 CurrentTick;
		int32 C = 0;
	};
	FAsyncPackage2* UpdateDependenciesStateRecursive(FAsyncLoadingThreadState2& ThreadState, FUpdateDependenciesStateRecursiveContext& Context);
	void WaitForAllDependenciesToReachState(FAsyncLoadingThreadState2& ThreadState, FAllDependenciesState FAsyncPackage2::* StateMemberPtr, EAsyncPackageLoadingState2 WaitForPackageState, uint32& CurrentTickVariable, TFunctionRef<void(FAsyncPackage2*)> OnStateReached);
	void ConditionalBeginProcessPackageExports(FAsyncLoadingThreadState2& ThreadState);
	void ConditionalFinishLoading(FAsyncLoadingThreadState2& ThreadState);
	void ConditionalBeginPostLoadPhase(FAsyncLoadingThreadState2& ThreadState);
	void ConditionalReleasePartialRequests(FAsyncLoadingThreadState2& ThreadState);
	void ConditionalBeginPostLoad(FAsyncLoadingThreadState2& ThreadState);
	void ConditionalBeginDeferredPostLoad(FAsyncLoadingThreadState2& ThreadState);
	EEventLoadNodeExecutionResult ProcessLinkerLoadPackageSummary(FAsyncLoadingThreadState2& ThreadState);
	bool CreateLinkerLoadExports(FAsyncLoadingThreadState2& ThreadState);
	bool CreateRuntimeExports(FAsyncLoadingThreadState2& ThreadState);
	void ConditionalBeginResolveImports(FAsyncLoadingThreadState2& ThreadState);
	bool ResolveLinkerLoadImports(FAsyncLoadingThreadState2& ThreadState);
	bool ResolveRuntimeImports(FAsyncLoadingThreadState2& ThreadState);
	bool PreloadLinkerLoadExports(FAsyncLoadingThreadState2& ThreadState);
	bool PreloadRuntimeExports(FAsyncLoadingThreadState2& ThreadState);
	EEventLoadNodeExecutionResult ExecutePostLoadLinkerLoadPackageExports(FAsyncLoadingThreadState2& ThreadState);
	EEventLoadNodeExecutionResult ExecuteDeferredPostLoadLinkerLoadPackageExports(FAsyncLoadingThreadState2& ThreadState);

	void ProcessExportDependencies(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex, FExportBundleEntry::EExportCommandType CommandType);
	int32 GetPublicExportIndex(uint64 ExportHash, FAsyncPackageHeaderData*& OutHeader);
	FString GetNameFromPackageObjectIndex(const FAsyncPackageHeaderData& Header, FPackageObjectIndex Index);
	void ConditionalCreateExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex);
	void ConditionalSerializeExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex);
	void ConditionalCreateImport(const FAsyncPackageHeaderData& Header, int32 LocalImportIndex);
	void ConditionalSerializeImport(const FAsyncPackageHeaderData& Header, int32 LocalImportIndex);
	int32 GetPublicCellExportIndexAsLocalIndex(uint64 CellExportHash, FAsyncPackageHeaderData*& OutHeader);
	void ConditionalCreateCellExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex);
	void ConditionalSerializeCellExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex);
	void ConditionalCreateCellImport(const FAsyncPackageHeaderData& Header, int32 LocalImportIndex);
	void ConditionalSerializeCellImport(const FAsyncPackageHeaderData& Header, int32 LocalImportIndex);

	/**
	 * Begin async loading process. Simulates parts of BeginLoad.
	 *
	 * Objects created during BeginAsyncLoad and EndAsyncLoad will have EInternalObjectFlags::AsyncLoading set
	 */
	void BeginAsyncLoad();
	/**
	 * End async loading process. Simulates parts of EndLoad().
	 */
	void EndAsyncLoad();
	/**
	 * Create UPackage
	 *
	 * @return true
	 */
	void CreateUPackage();

	/**
	 * Finish up UPackage
	 *
	 * @return true
	 */
	void FinishUPackage();

	/**
	 * Finalizes external dependencies till time limit is exceeded
	 *
	 * @return Complete if all dependencies are finished, TimeOut otherwise
	 */
	enum EExternalReadAction { ExternalReadAction_Poll, ExternalReadAction_Wait };
	EAsyncPackageState::Type ProcessExternalReads(FAsyncLoadingThreadState2& ThreadState, EExternalReadAction Action);

#if ALT2_DUMP_STATE_ON_HANG
	void DumpStateImpl(TSet<FAsyncPackage2*>& Set, int32 Indent = 0, TMultiMap<FEventLoadNode2*, FEventLoadNode2*>* MappedNodes = nullptr);
#endif
public:
#if ALT2_DUMP_STATE_ON_HANG
	void DumpState();
#endif

	/** Serialization context for this package */
	FUObjectSerializeContext* GetSerializeContext();
};

class FAsyncLoadingSyncLoadContext
{
public:
	FAsyncLoadingSyncLoadContext(TConstArrayView<int32> InRequestIDs, FAsyncPackage2* CurrentlyExecutingPackage)
		: RequestIDs(InRequestIDs)
	{
		// We need to be careful that we aren't taking reference to something that has been marked for deferred delete already
		if (CurrentlyExecutingPackage && CurrentlyExecutingPackage->TryAddRef())
		{
			RequestingPackage = CurrentlyExecutingPackage;
		}

		RequestedPackages.AddZeroed(RequestIDs.Num());
		ContextId = NextContextId++;
		if (NextContextId == 0)
		{
			NextContextId = 1;
		}
	}

	~FAsyncLoadingSyncLoadContext()
	{
		if (RequestingPackage)
		{
			RequestingPackage->Release();
		}

		for (FAsyncPackage2* RequestedPackage : RequestedPackages)
		{
			if (RequestedPackage)
			{
				RequestedPackage->Release();
			}
		}
	}

	void AddRef()
	{
		++RefCount;
	}

	void ReleaseRef()
	{
		int32 NewRefCount = --RefCount;
		check(NewRefCount >= 0);
		if (NewRefCount == 0)
		{
			delete this;
		}
	}

	FAsyncPackage2* GetRequestingPackage()
	{
		return RequestingPackage;
	}

	TArray<FAsyncPackage2*, TInlineAllocator<4>>& GetRequestedPackages()
	{
		return RequestedPackages;
	}

	FAsyncPackage2* GetRequestedPackage(int32 Index)
	{
		return RequestedPackages[Index];
	}

	void SetRequestedPackage(int32 Index, FAsyncPackage2* RequestedPackage)
	{
		check(RequestedPackage);

		// We need to be careful that we aren't taking reference to something that has been marked for deferred delete already
		if (RequestedPackage->TryAddRef())
		{
			RequestedPackages[Index] = RequestedPackage;
		}
	}

	bool IsRequestedPackageValid(int32 Index) const
	{
		return RequestedPackages[Index] != nullptr;
	}

	int32 FindRequestedPackage(FAsyncPackage2* Package)
	{
		return RequestedPackages.Find(Package);
	}

	uint64 ContextId;
	TArray<int32, TInlineAllocator<4>> RequestIDs;
	TAsyncAtomic<bool> bHasFoundRequestedPackages{ false };
	UE::FManualResetEvent DoneEvent;
private:
	TArray<FAsyncPackage2*, TInlineAllocator<4>> RequestedPackages;
	FAsyncPackage2* RequestingPackage = nullptr;
	TAsyncAtomic<int32> RefCount = 1;

	static TAsyncAtomic<uint64> NextContextId;
};

TAsyncAtomic<uint64> FAsyncLoadingSyncLoadContext::NextContextId{ 1 };

class FAsyncLoadingThread2 final
	: public FRunnable
	, public IAsyncPackageLoader
{
	friend struct FAsyncPackage2;
	friend class FAsyncLoadingVerseRoot;
public:
	FAsyncLoadingThread2(FIoDispatcher& IoDispatcher);
	virtual ~FAsyncLoadingThread2();

	virtual ELoaderType GetLoaderType() const override
	{
		return ELoaderType::ZenLoader;
	}

private:
	/** Thread to run the worker FRunnable on */
	static constexpr int32 DefaultAsyncPackagesReserveCount = 512;
	FRunnableThread* Thread;
	TAsyncAtomic<bool> bStopRequested = false;
	TAsyncAtomic<int32> SuspendRequestedCount = 0;
	TAsyncAtomic<int32> FullFlushCount = 0;

	bool bHasRegisteredAllScriptObjects = false;
	/** [ASYNC/GAME THREAD] true if the async thread is actually started. We don't start it until after we boot because the boot process on the game thread can create objects that are also being created by the loader */
	TAsyncAtomic<bool> bThreadStarted = false;

#if !UE_BUILD_SHIPPING
	FPlatformFileOpenLog* FileOpenLogWrapper = nullptr;
#endif

	/** [ASYNC/GAME THREAD] Event used to signal loading should be cancelled */
	FEvent* CancelLoadingEvent;
	/** [ASYNC/GAME THREAD] Event used to signal that the async loading thread should be suspended */
	FEvent* ThreadSuspendedEvent;
	/** [ASYNC/GAME THREAD] Event used to signal that the async loading thread has resumed */
	FEvent* ThreadResumedEvent;
	TArray<FAsyncPackage2*> LoadedPackagesToProcess;
	/** [ASYNC/GAME THREAD] Event used to signal the main thread that new processing is needed. Only used when ALT is active. */
	UE::FManualResetEvent MainThreadWakeEvent;

#if WITH_EDITOR
	/** [GAME THREAD] */
	TArray<UObject*> EditorLoadedAssets;
#endif
#if WITH_EDITOR || UE_ENABLE_ASSET_READ_LOGGER
	/** [GAME THREAD] Packages that have completed loading, drained by ConditionalProcessCallbacks to broadcast OnEndLoadPackage. */
	TArray<UPackage*> CompletedUPackagesForBroadcast;
#endif
	TArray<UObject*> HotfixableLoadedAssets;
	/** [ASYNC/GAME THREAD] Packages to be cleaned up from async thread */
	TMpscQueue<FAsyncPackage2*> DeferredCleanupPackages;
	/** Used to protect DeferredCleanupPackages when dequeuing since its a single consumer queue. */
	UE::FMutex DeferredCleanupPackagesLock;

	struct FCompletedPackageRequest
	{
		FName PackageName;
		EAsyncLoadingResult::Type Result = EAsyncLoadingResult::Succeeded;
		UPackage* UPackage = nullptr;
		FAsyncPackage2* AsyncPackage = nullptr;
		FAsyncPackage2::FRequestIDs RequestIDs;
		TArray<TUniquePtr<FLoadPackageAsyncDelegate>, TInlineAllocator<2>> CompletionCallbacks;
		FProgressCallbackEntry* ProgressCallbacksHead = nullptr;
		FAsyncPackage2::FRequestIDs ImportRequestIDs;

		static FCompletedPackageRequest FromUnreslovedPackage(
			const FAsyncPackageDesc2& Desc,
			EAsyncLoadingResult::Type Result,
			TUniquePtr<FLoadPackageAsyncDelegate>&& CompletionCallback)
		{
			FCompletedPackageRequest Res =
			{
				.PackageName = Desc.UPackageName,
				.Result = Result
			};
			Res.CompletionCallbacks.Add(MoveTemp(CompletionCallback));
			Res.RequestIDs.Add(Desc.RequestID);
			return Res;
		}

		static FCompletedPackageRequest FromLoadedPackage(
			FAsyncPackage2* Package)
		{
			FCompletedPackageRequest Request
			{
				.PackageName = Package->Desc.UPackageName,
				.Result = Package->GetLoadStatus(),
				.UPackage = Package->LinkerRoot,
				.AsyncPackage = Package,
				.RequestIDs = Package->RequestIDs, // must be a copy as this array is accessed after the package marked as completed
				.CompletionCallbacks = MoveTemp(Package->CompletionCallbacks),
				.ProgressCallbacksHead = Package->ProgressCallbacksHead.exchange(nullptr, std::memory_order_acquire),
				.ImportRequestIDs = MoveTemp(Package->ImportRequestIDs)
			};

			return Request;
		}

		void CallCompletionCallbacks()
		{
			checkSlow(IsInGameThread());

#if WITH_EDITOR
			UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(AsyncPackage ? AsyncPackage->Desc.PIEInstanceID : INDEX_NONE);
#endif

			if (CompletionCallbacks.Num() != 0)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(PackageCompletionCallbacks);
				for (TUniquePtr<FLoadPackageAsyncDelegate>& CompletionCallback : CompletionCallbacks)
				{
					CompletionCallback->ExecuteIfBound(PackageName, UPackage, Result);
				}
				CompletionCallbacks.Empty();
			}

			if (ProgressCallbacksHead)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(PackageProgressCallbacks_Completion);

				FProgressCallbackEntry* Entry = ProgressCallbacksHead;
				while (Entry)
				{
					if (Result == EAsyncLoadingResult::Succeeded)
					{
						CallProgressCallback(*Entry, PackageName, UPackage, EAsyncLoadingProgress::FullyLoaded);
					}
					else
					{
						FLoadPackageAsyncProgressParams Params
						{
							.PackageName = PackageName,
							.LoadedPackage = UPackage,
							.ProgressType = EAsyncLoadingProgress::Failed
						};
						Entry->Delegate->Invoke(Params);
					}

					FProgressCallbackEntry* Next = Entry->Next;
					delete Entry;
					Entry = Next;
				}
			}
		}
	};
	TArray<FCompletedPackageRequest> CompletedPackageRequests;
	TArray<FCompletedPackageRequest> FailedPackageRequests;
	FCriticalSection FailedPackageRequestsCritical;

	/** 
	* Packages in active loading with GetAsyncPackageId() as key. 
	* Using a FSharedMutexStripedMapLockingPolicy as we don't need transaction support since the loader
	* always runs in the open, and the loader requires shared recursive locking when flushing.
	*/
	TStripedMap<32, FPackageId, FAsyncPackage2*, FDefaultSetAllocator, 
		TDefaultMapHashableKeyFuncs<FPackageId, FAsyncPackage2*, false>, FSharedMutexStripedMapLockingPolicy> AsyncPackageLookup;

	TMpscQueue<FAsyncPackage2*> ExternalReadQueue;
	TAsyncAtomic<int32> PendingIoRequestsCounter{ 0 };

	/** List of all pending package requests */
	TSet<int32> PendingRequests;
	/** Synchronization object for PendingRequests list */
	FCriticalSection PendingRequestsCritical;
	TMap<int32, FAsyncPackage2*> RequestIdToPackageMap; // Only accessed from the async loading thread

	/** [ASYNC/GAME THREAD] Number of package load requests in the async loading queue */
	TAsyncAtomic<int32> QueuedPackagesCounter{ 0 };
	/** [ASYNC/GAME THREAD] Number of packages being loaded on the async thread and post loaded on the game thread */
	TAsyncAtomic<int32> LoadingPackagesCounter{ 0 };
	/** Encapsulate our counter to make sure all decrements are monitored. */
	class FPackagesWithRemainingWorkCounter
	{
		UE::FManualResetEvent* WakeEvent = nullptr;
		TAsyncAtomic<int32> PackagesWithRemainingWorkCounter{ 0 };
	public:
		void SetWakeEvent(UE::FManualResetEvent* InWakeEvent) { WakeEvent = InWakeEvent; }
		int32 operator++() { return ++PackagesWithRemainingWorkCounter; }
		int32 operator++(int) { return PackagesWithRemainingWorkCounter++; }
		operator int32() const { return PackagesWithRemainingWorkCounter; }

		// Only implement prefix
		int32 operator--()
		{
			int32 newValue = --PackagesWithRemainingWorkCounter;
			if (newValue == 0)
			{
				if (WakeEvent)
				{
					WakeEvent->Notify();
				}
			}
			return newValue;
		}
	};
	/** [ASYNC/GAME THREAD] While this is non-zero there's work left to do */
	FPackagesWithRemainingWorkCounter PackagesWithRemainingWorkCounter;

	TAsyncAtomic<int32> AsyncThreadReady;

	/** When cancelling async loading: list of package requests to cancel */
	TArray<FAsyncPackageDesc2*> QueuedPackagesToCancel;
	/** When cancelling async loading: list of packages to cancel */
	TSet<FAsyncPackage2*> PackagesToCancel;

	/** Async loading thread ID */
	TAsyncAtomic<uint32> AsyncLoadingThreadID;

	/** I/O Dispatcher */
	FIoDispatcher& IoDispatcher;

	FPackageStore& PackageStore;
	FGlobalImportStore GlobalImportStore;
	TMpscQueue<FPackageRequest> PackageRequestQueue;
	TArray<FAsyncPackage2*> PendingPackages;

	/** [GAME THREAD] Initial load pending CDOs */
	TMap<UClass*, TArray<FEventLoadNode2*>> PendingCDOs;
	TArray<UClass*> PendingCDOsRecursiveStack;

	/** [ASYNC/GAME THREAD] Unreachable objects from last NotifyUnreachableObjects callback from GC. */
	FCriticalSection UnreachableObjectsCritical;
	FUnreachableObjects UnreachableObjects;

	/** [ASYNC/GAME THREAD] Object Renames can occur on any thread, will be flushed by the ALT exclusively. */
	TMpscQueue<FLoadedObjectRenamed> DeferredLoadedObjectRenameQueue;

	TUniquePtr<FAsyncLoadingThreadState2> GameThreadState;
	TUniquePtr<FAsyncLoadingThreadState2> AsyncLoadingThreadState;

	uint32 ConditionalBeginProcessExportsTick = 0;
	uint32 ConditionalBeginResolveImportsTick = 0;
	uint32 ConditionalBeginPostLoadTick = 0;
	uint32 ConditionalFinishLoadingTick = 0;

public:

	//~ Begin FRunnable Interface.
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	//~ End FRunnable Interface

	/** Start the async loading thread */
	virtual void StartThread() override;

	/** [EDL] Event queue */
	FZenaphore AltZenaphore;
	FAsyncLoadEventGraphAllocator GraphAllocator;
	FAsyncLoadEventQueue2 EventQueue;
	FAsyncLoadEventQueue2 MainThreadEventQueue;
	TArray<FAsyncLoadEventSpec> EventSpecs;

	/** True if multithreaded async loading is currently being used. */
	inline virtual bool IsMultithreaded() override
	{
		return bThreadStarted.load(std::memory_order_acquire);
	}

	/** Sets the current state of async loading */
	void EnterAsyncLoadingTick()
	{
		AsyncLoadingTickCounter++;
	}

	void LeaveAsyncLoadingTick()
	{
		AsyncLoadingTickCounter--;
		check(AsyncLoadingTickCounter >= 0);
	}

	/** Gets the current state of async loading */
	bool GetIsInAsyncLoadingTick() const
	{
		return !!AsyncLoadingTickCounter;
	}

	/** Returns true if packages are currently being loaded on the async thread */
	inline virtual bool IsAsyncLoadingPackages() override
	{
		return PackagesWithRemainingWorkCounter != 0;
	}

	/** Returns true this codes runs on the async loading thread */
	virtual bool IsInAsyncLoadThread() override
	{
		if (IsMultithreaded())
		{
			// We still need to report we're in async loading thread even if 
			// we're on game thread but inside of async loading code (PostLoad mostly)
			// to make it behave exactly like the non-threaded version
			uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
			if (CurrentThreadId == AsyncLoadingThreadID ||
				(IsInGameThread() && GetIsInAsyncLoadingTick()))
			{
				return true;
			}
			return false;
		}
		else
		{
			return IsInGameThread() && GetIsInAsyncLoadingTick();
		}
	}

	/** Returns true if async loading is suspended */
	inline virtual bool IsAsyncLoadingSuspended() override
	{
		return SuspendRequestedCount.load(std::memory_order_relaxed) > 0;
	}

	virtual void NotifyConstructedDuringAsyncLoading(UObject* Object, bool bSubObjectThatAlreadyExists) override;

	virtual void NotifyUnreachableObjects(const TArrayView<FUObjectItem*>& UnreachableObjects) override;

	virtual void NotifyRegistrationEvent(FName PackageName, FName Name, ENotifyRegistrationType NotifyRegistrationType, ENotifyRegistrationPhase NotifyRegistrationPhase, FTypeConstructFunc* InRegister, bool InbDynamic, UObject* FinishedObject) override;

	virtual void NotifyScriptVersePackage(Verse::VPackage* Package) override;

	void NotifyCompiledVersePackage(Verse::VPackage* VersePackage, UPackage* Package, TArrayView<UObject*> Exports);

	virtual void NotifyRegistrationComplete() override;

	// This is only safe to call from ALT since we're not adding any refs to the package
	// but that's OK on ALT since we destroy the packages from ALT.
	FORCEINLINE FAsyncPackage2* GetAsyncPackage(const FPackageId& PackageId)
	{
		return AsyncPackageLookup.FindRef(PackageId);
	}

	void UpdatePackagePriority(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package);
	void UpdatePackagePriorityRecursive(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package, int32 NewPriority);

	FAsyncPackage2* FindOrInsertPackage(FAsyncLoadingThreadState2& ThreadState, FAsyncPackageDesc2& InDesc, bool& bInserted, FAsyncPackage2* ImportedByPackage, TUniquePtr<FLoadPackageAsyncDelegate>&& PackageLoadedDelegate = TUniquePtr<FLoadPackageAsyncDelegate>(), TSharedPtr<FLoadPackageAsyncProgressDelegate> PackageProgressDelegate = nullptr);
	void QueueUnresolvedPackage(FAsyncLoadingThreadState2& ThreadState, EPackageStoreEntryStatus PackageStatus, FAsyncPackageDesc2& PackageDesc, TUniquePtr<FLoadPackageAsyncDelegate>&& LoadPackageAsyncDelegate, TSharedPtr<FLoadPackageAsyncProgressDelegate> LoadPackageAsyncProgressDelegate);

	/**
	* [ASYNC* THREAD] Loads all packages
	*
	* @param OutPackagesProcessed Number of packages processed in this call.
	* @return The current state of async loading
	*/
	EAsyncPackageState::Type ProcessAsyncLoadingFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool& bDidSomething);

	/**
	* [GAME THREAD] Ticks game thread side of async loading.
	*
	* @param bUseTimeLimit True if time limit should be used [time-slicing].
	* @param bUseFullTimeLimit True if full time limit should be used [time-slicing].
	* @param TimeLimit Maximum amount of time that can be spent in this call [time-slicing].
	* @param FlushTree Package dependency tree to be flushed
	* @return The current state of async loading
	*/
	EAsyncPackageState::Type TickAsyncLoadingFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit, TConstArrayView<int32> FlushRequestIDs, bool& bDidSomething);

	/**
	* [ASYNC THREAD] Main thread loop
	*
	* @param bUseTimeLimit True if time limit should be used [time-slicing].
	* @param bUseFullTimeLimit True if full time limit should be used [time-slicing].
	* @param TimeLimit Maximum amount of time that can be spent in this call [time-slicing].
	* @param FlushTree Package dependency tree to be flushed
	*/
	EAsyncPackageState::Type TickAsyncThreadFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool& bDidSomething);

	/** Initializes async loading thread */
	virtual void InitializeLoading() override;

	virtual void ShutdownLoading() override;

	virtual bool ShouldAlwaysLoadPackageAsync(const FPackagePath& PackagePath) override
	{
		return true;
	}

	int32 LoadPackageInternal(const FPackagePath& InPackagePath, FName InCustomName, TUniquePtr<FLoadPackageAsyncDelegate>&& InCompletionDelegate, TSharedPtr<FLoadPackageAsyncProgressDelegate> InProgressDelegate, EPackageFlags InPackageFlags, int32 InPIEInstanceID, int32 InPackagePriority, const FLinkerInstancingContext* InInstancingContext, uint32 InLoadFlags);

	virtual int32 LoadPackage(
		const FPackagePath& InPackagePath,
		FName InCustomName,
		FLoadPackageAsyncDelegate InCompletionDelegate,
		EPackageFlags InPackageFlags,
		int32 InPIEInstanceID,
		int32 InPackagePriority,
		const FLinkerInstancingContext* InstancingContext = nullptr,
		uint32 InLoadFlags = LOAD_None) override;

	virtual int32 LoadPackage(
		const FPackagePath& InPackagePath,
		FLoadPackageAsyncOptionalParams Params) override;

	EAsyncPackageState::Type ProcessLoadingFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit);

	inline virtual EAsyncPackageState::Type ProcessLoading(bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit) override
	{
		FAsyncLoadingThreadState2& ThreadState = *FAsyncLoadingThreadState2::Get();
		return ProcessLoadingFromGameThread(ThreadState, bUseTimeLimit, bUseFullTimeLimit, TimeLimit);
	}

	EAsyncPackageState::Type ProcessLoadingUntilCompleteFromGameThread(FAsyncLoadingThreadState2& ThreadState, TFunctionRef<bool()> CompletionPredicate, double TimeLimit);

	inline virtual EAsyncPackageState::Type ProcessLoadingUntilComplete(TFunctionRef<bool()> CompletionPredicate, double TimeLimit) override
	{
		FAsyncLoadingThreadState2& ThreadState = *FAsyncLoadingThreadState2::Get();
		return ProcessLoadingUntilCompleteFromGameThread(ThreadState, CompletionPredicate, TimeLimit);
	}

	virtual void CancelLoading() override;

	virtual void SuspendLoading() override;

	virtual void ResumeLoading() override;

	virtual void FlushLoading(TConstArrayView<int32> RequestIds) override;

	void FlushLoadingFromLoadingThread(FAsyncLoadingThreadState2& ThreadState, TConstArrayView<int32> RequestIds);
	void FlushLoadingFromParallelLoadingThread(FAsyncLoadingThreadState2& ThreadState, TConstArrayView<int32> RequestIds);
	void WarnAboutPotentialSyncLoadStall(FAsyncLoadingSyncLoadContext* SyncLoadContext);

	virtual int32 GetNumQueuedPackages() override
	{
		return QueuedPackagesCounter;
	}

	virtual int32 GetNumAsyncPackages() override
	{
		return LoadingPackagesCounter;
	}

	/**
	 * [GAME THREAD] Gets the load percentage of the specified package
	 * @param PackageName Name of the package to return async load percentage for
	 * @return Percentage (0-100) of the async package load or -1 of package has not been found
	 */
	virtual float GetAsyncLoadPercentage(const FName& PackageName) override;

	/**
	 * [ASYNC/GAME THREAD] Checks if a request ID already is added to the loading queue
	 */
	bool ContainsRequestID(int32 RequestID)
	{
		FScopeLock Lock(&PendingRequestsCritical);
		return PendingRequests.Contains(RequestID);
	}

	/**
	 * [ASYNC/GAME THREAD] Checks if there are any pending requests
	 */
	bool ContainsAnyPendingRequests()
	{
		FScopeLock Lock(&PendingRequestsCritical);
		return !PendingRequests.IsEmpty();
	}

	/**
	 * [ASYNC/GAME THREAD] Checks if a request ID already is added to the loading queue
	 */
	bool ContainsAnyRequestID(TConstArrayView<int32> RequestIDs)
	{
		FScopeLock Lock(&PendingRequestsCritical);
		return Algo::AnyOf(RequestIDs, [this](int32 RequestID) { return PendingRequests.Contains(RequestID); });
	}

	/**
	 * [ASYNC/GAME THREAD] Adds a request ID to the list of pending requests
	 */
	void AddPendingRequest(int32 RequestID)
	{
		FScopeLock Lock(&PendingRequestsCritical);
		PendingRequests.Add(RequestID);
	}

	/**
	 * [ASYNC/GAME THREAD] Removes a request ID from the list of pending requests
	 */
	void RemovePendingRequests(FAsyncLoadingThreadState2& ThreadState, const FAsyncPackage2::FRequestIDs& RequestIDs)
	{
		int32 RemovedCount = 0;
		{
			FScopeLock Lock(&PendingRequestsCritical);
			for (int32 ID : RequestIDs)
			{
				RemovedCount += PendingRequests.Remove(ID);
				TRACE_LOADTIME_END_REQUEST(ID);
			}
			if (PendingRequests.IsEmpty())
			{
				PendingRequests.Empty(DefaultAsyncPackagesReserveCount);
			}
		}

		if (RemovedCount > 0)
		{
			// Any removed pending request is of interest to main thread as it might unblock a flush.
			if (ThreadState.bIsAsyncLoadingThread)
			{
				MainThreadWakeEvent.Notify();
			}
			// Removal from the GT might unblock the ALT for flushes coming from other threads.
			else if (IsMultithreaded())
			{
				AltZenaphore.NotifyOne();
			}
		}
	}

	void AddPendingCDOs(FAsyncPackage2* Package, TArray<UClass*, TInlineAllocator<8>>& Classes)
	{
		check(IsInGameThread()); // this is accessing PendingCDOsRecursiveStack which is GT only
		for (UClass* Class : Classes)
		{
			// Don't add a dependency on a CDO that is currently being created on the stack as this would cause a deadlock.
			// StaticFind will be able to find the CDO and a second call to CreateDefaultObject would return the pointer
			// so the only risk left is if the package referencing the CDO needs it to be fully constructed, which we 
			// can't possibly satisfy because of the circular dependency.
			if (!PendingCDOsRecursiveStack.Contains(Class))
			{
				TArray<FEventLoadNode2*>& Nodes = PendingCDOs.FindOrAdd(Class);
				FEventLoadNode2& Node = Package->GetPackageNode(Package_DependenciesReady);
				Node.AddBarrier();
				Nodes.Add(&Node);
			}
		}
	}

private:
	bool TryGetExistingLoadedPackagePath(FPackageId InPackageIdToLoad, FName InPackageNameToLoad, EPackageExtension& OutPackageExtension, EPackageLoader& OutPackageLoader, FPackageStoreEntry& OutZenStoreEntry)
	{
		// Lookup if we have the package in the importstore already to avoid expensive calls in TryGetPackagePathFromFileSystem
		if (FReadLockedPackageRef PackageRef = GlobalImportStore.FindPackageRef(InPackageIdToLoad))
		{
			// If a package has failed or wasn't loaded from the loader, fail the lookup: we cannot trust these references
			// and it's possible their actual state is now different than what was cached in the GlobalImportStore.
			// Also, if a newer version of the package exists, we don't want to look up in this cache, as we may be going
			// between loaders (cooked -> uncooked)
			if (!PackageRef->HasErrors() && PackageRef->GetPackageLoader() != EPackageLoader::Unknown &&
				(PackageRef->GetPackage() == nullptr || !PackageRef->GetPackage()->HasAnyFlags(RF_NewerVersionExists)))
			{
				OutPackageLoader = PackageRef->GetPackageLoader();
				OutPackageExtension = PackageRef->GetPackageHeaderExtension();

				if (OutPackageLoader == EPackageLoader::Zen)
				{
					// @todo: get the name? it's not used really 
					PackageStore.GetPackageStoreEntry(InPackageIdToLoad, InPackageNameToLoad, OutZenStoreEntry);
				}
				return true;
			}
		}
		return false;
	}

	EPackageStoreEntryStatus TryGetPackageInfoFromPackageStore(FPackageId& InOutPackageIdToLoad, FName& InOutPackageNameToLoad, FName& InOutUPackageName, FPackageStoreEntry& OutPackageStoreEntry)
	{
		EPackageStoreEntryStatus PackageStatus = PackageStore.GetPackageStoreEntry(InOutPackageIdToLoad, InOutPackageNameToLoad, OutPackageStoreEntry);

		if (PackageStatus != EPackageStoreEntryStatus::Missing)
		{
			if (PackageStatus == EPackageStoreEntryStatus::Ok && (GFailLoadOnMissingImport || GFailLoadOnNotInstalledImport))
			{
				// We need to confirm that this package can be fully imported so walk its transitive dependencies
				// If we have a loaded package already that isn't failed we don't need to check it's dependencies
				bool bFoundMissing = false;
				bool bFoundNotInstalled = false;
				const bool bLocalFailLoadOnMissingImport = GFailLoadOnMissingImport;
				const bool bLocalFailLoadOnNotInstalledImport = GFailLoadOnNotInstalledImport;
				FPackageId UnloadableImportPackageId;
				TArray<FPackageId> ImportsToCheck(OutPackageStoreEntry.ImportedPackageIds);
				TSet<FPackageId> ImportsSeen(OutPackageStoreEntry.ImportedPackageIds); // Out of paranoia, don't loop forever if there is somehow a circular import chain
				while (!ImportsToCheck.IsEmpty() && !UnloadableImportPackageId.IsValid())
				{
					FPackageId ImportId = ImportsToCheck.Pop();
					if (FReadLockedPackageRef PackageRef = GlobalImportStore.FindPackageRef(ImportId))
					{
						if (!PackageRef->HasErrors())
						{
							continue;
						}
						else if (bLocalFailLoadOnMissingImport && PackageRef->IsMissing())
						{
							bFoundMissing |= PackageRef->IsMissing();
							UnloadableImportPackageId = ImportId;
							break;
						}
						else if (bLocalFailLoadOnNotInstalledImport && PackageRef->IsNotInstalled())
						{
							bFoundNotInstalled |= PackageRef->IsNotInstalled();
							UnloadableImportPackageId = ImportId;
							break;
						}
					}

					FPackageStoreEntry ImportEntry;
					FName ImportName;
					EPackageStoreEntryStatus ImportStatus = PackageStore.GetPackageStoreEntry(ImportId, ImportName, ImportEntry);
					bFoundMissing |= bLocalFailLoadOnMissingImport && (ImportStatus == EPackageStoreEntryStatus::Missing);
					bFoundNotInstalled |= bLocalFailLoadOnNotInstalledImport && (ImportStatus == EPackageStoreEntryStatus::NotInstalled);
					if (bFoundMissing || bFoundNotInstalled)
					{
						UnloadableImportPackageId = ImportId;
						break;
					}

					// Append all the imports we haven't checked yet
					for (FPackageId ImportToAdd : ImportEntry.ImportedPackageIds)
					{
						bool bIsAlreadyInSet = false;
						ImportsSeen.Add(ImportToAdd, &bIsAlreadyInSet);
						if (!bIsAlreadyInSet)
						{
							ImportsToCheck.Add(ImportToAdd);
						}
					}
				}

				if (UnloadableImportPackageId.IsValid())
				{
					check(bFoundMissing || bFoundNotInstalled);
					//
					// Note: Update the FAsyncLoadingTestFailedImportPropagation test if you edit these messages
					//
					if (bFoundMissing)
					{
						UE_LOGF(LogStreaming, Display, "Package %ls (0x%ls) will not be loaded due to Missing imported package (0x%ls)",
							*InOutPackageNameToLoad.ToString(), *LexToString(InOutPackageIdToLoad), *LexToString(UnloadableImportPackageId));
						return EPackageStoreEntryStatus::Missing;
					}
					else
					{
						UE_LOGF(LogStreaming, Display, "Package %ls (0x%ls) will not be loaded due to the NotInstalled imported package (0x%ls), however it may become available later.",
							*InOutPackageNameToLoad.ToString(), *LexToString(InOutPackageIdToLoad), *LexToString(UnloadableImportPackageId));
						return EPackageStoreEntryStatus::NotInstalled;
					}
				}
			}

#if WITH_EDITORONLY_DATA
			if (OutPackageStoreEntry.LoaderType == EPackageLoader::LinkerLoad)
			{
				// in case we are loading loose, update the casing to match the name on disk
				if (InOutUPackageName == InOutPackageNameToLoad)
				{
					InOutUPackageName = OutPackageStoreEntry.LinkerLoadCaseCorrectedPackageName;
				}
				InOutPackageNameToLoad = OutPackageStoreEntry.LinkerLoadCaseCorrectedPackageName;
			}
#endif
			if (OutPackageStoreEntry.LoaderType == EPackageLoader::LinkerLoad)
			{
				TRACE_COUNTER_INCREMENT(AsyncLoadingFileSystemLoads);
			}
			else
			{
				TRACE_COUNTER_INCREMENT(AsyncLoadingIoStoreLoads);
			}
		}

		return PackageStatus;

	};

	void ConditionalProcessCallbacks();

public:
	bool ProcessDeferredCleanupPackagesQueue(int32 MaxCount = MAX_int32)
	{
		bool bDidSomething = false;
		if (!DeferredCleanupPackages.IsEmpty())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ProcessDeferredCleanupPackagesQueue);
			UE::TUniqueLock ScopeLock(DeferredCleanupPackagesLock);

			FAsyncPackage2* Package = nullptr;
			int32 Count = 0;
			while (++Count <= MaxCount && DeferredCleanupPackages.Dequeue(Package))
			{
				if (Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::DeferredClearImportedPackages)
				{
					Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::DeferredDelete;
					Package->ClearImportedPackages();
				}
				else
				{
					check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::DeferredDelete);
					DeleteAsyncPackage(Package);
				}
				bDidSomething = true;
			}
		}
		return bDidSomething;
	}

private:
	void OnGarbageCollectStarted();

#if WITH_EDITOR
	void OnLoadedObjectRenamed(UObject* Obj, UObject* OldOuter, FName OldName);
#endif
	void FlushDeferredLoadedObjectRenameQueue();

	void CollectUnreachableObjects(TArrayView<FUObjectItem*> UnreachableObjectItems, FUnreachableObjects& OutUnreachableObjects);

	void RemoveUnreachableObjects(FUnreachableObjects& ObjectsToRemove);

	bool ProcessPendingCDOs(FAsyncLoadingThreadState2& ThreadState)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ProcessPendingCDOs);

		bool bDidSomething = false;
		UClass* Class = nullptr;
		const uint64 SyncLoadContextId = ThreadState.SyncLoadContextStack.Num() > 0 ? ThreadState.SyncLoadContextStack.Top()->ContextId : 0;
		for (TMap<UClass*, TArray<FEventLoadNode2*>>::TIterator It = PendingCDOs.CreateIterator(); It; ++It)
		{
			UClass* CurrentClass = It.Key();
			if (PendingCDOsRecursiveStack.Num() > 0)
			{
				bool bAnyParentOnStack = false;
				UClass* Super = CurrentClass;
				while (Super)
				{
					if (PendingCDOsRecursiveStack.Contains(Super))
					{
						bAnyParentOnStack = true;
						break;
					}
					Super = Super->GetSuperClass();
				}
				if (bAnyParentOnStack)
				{
					continue;
				}
			}

			const TArray<FEventLoadNode2*>& Nodes = It.Value();
			for (const FEventLoadNode2* Node : Nodes)
			{
				const uint64 NodeContextId = Node->GetSyncLoadContextId();
				if (NodeContextId >= SyncLoadContextId)
				{
					Class = CurrentClass;
					break;
				}
			}

			if (Class != nullptr)
			{
				break;
			}
		}

		if (Class)
		{
			TArray<FEventLoadNode2*> Nodes;
			PendingCDOs.RemoveAndCopyValue(Class, Nodes);

			UE_LOGF(LogStreaming, Log,
				"ProcessPendingCDOs: Creating CDO for '%ls' for SyncLoadContextId %llu, releasing %d nodes. %d CDOs remaining.",
				*Class->GetFullName(), SyncLoadContextId, Nodes.Num(), PendingCDOs.Num());
			PendingCDOsRecursiveStack.Push(Class);
			UObject* CDO = Class->GetDefaultObject(/*bCreateIfNeeded*/ true);
			verify(PendingCDOsRecursiveStack.Pop() == Class);

			ensureAlwaysMsgf(CDO, TEXT("Failed to create CDO for %s"), *Class->GetFullName());
			UE_LOGF(LogStreaming, Verbose, "ProcessPendingCDOs: Created CDO for '%ls'.", *Class->GetFullName());

			for (FEventLoadNode2* Node : Nodes)
			{
				Node->ReleaseBarrier(&ThreadState);
			}

			bDidSomething = true;
		}
		return bDidSomething;
	}

	/**
	* [GAME THREAD] Performs game-thread specific operations on loaded packages (not-thread-safe PostLoad, callbacks)
	*
	* @param bUseTimeLimit True if time limit should be used [time-slicing].
	* @param bUseFullTimeLimit True if full time limit should be used [time-slicing].
	* @param TimeLimit Maximum amount of time that can be spent in this call [time-slicing].
	* @param FlushTree Package dependency tree to be flushed
	* @return The current state of async loading
	*/
	EAsyncPackageState::Type ProcessLoadedPackagesFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool& bDidSomething, TConstArrayView<int32> FlushRequestIDs = {});

	void GatherCompletedPackagesToFlush(FAsyncLoadingThreadState2& ThreadState, TConstArrayView<int32> FlushRequestIDs, TArray<FCompletedPackageRequest>& OutRequestsToProcess);
	void IncludePackageInSyncLoadContextRecursive(FAsyncLoadingThreadState2& ThreadState, uint64 ContextId, FAsyncPackage2* Package);
	void UpdateSyncLoadContext(FAsyncLoadingThreadState2& ThreadState, bool bAutoHandleSyncLoadContext = true);
	void CleanupSyncLoadContext(FAsyncLoadingThreadState2& ThreadState);
	bool CreateAsyncPackagesFromQueue(FAsyncLoadingThreadState2& ThreadState, bool bForceExitForGarbageCollect = false);

	FAsyncPackage2* CreateAsyncPackage(FAsyncLoadingThreadState2& ThreadState, const FAsyncPackageDesc2& Desc)
	{
		UE_ASYNC_PACKAGE_DEBUG(Desc);

		return new FAsyncPackage2(ThreadState, Desc, *this, GraphAllocator, EventSpecs.GetData());
	}

	void InitializeAsyncPackageFromPackageStore(FAsyncLoadingThreadState2& ThreadState, FIoBatch* IoBatch, FAsyncPackage2* AsyncPackage, const FPackageStoreEntry& PackageStoreEntry)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(InitializeAsyncPackageFromPackageStore);
		UE_ASYNC_PACKAGE_DEBUG(AsyncPackage->Desc);
		UE_TRACE_METADATA_SCOPE_PACKAGE_ID(AsyncPackage->Desc.UPackageId);
		UE_TRACE_PACKAGE_NAME(AsyncPackage->Desc.UPackageId, AsyncPackage->Desc.UPackageName);

		FAsyncPackageData& Data = AsyncPackage->Data;

		const int32 ImportedPackagesCount = PackageStoreEntry.ImportedPackageIds.Num();
		const uint64 ImportedPackageIdsMemSize = Align(sizeof(FPackageId) * ImportedPackagesCount, 8);
#if WITH_EDITOR
		const bool bRequestOptionalChunk = PackageStoreEntry.bReplaceChunkWithOptional;
		const bool bHasOptionalSegment = PackageStoreEntry.bHasOptionalSegment;
		const int32 OptionalSegmentImportedPackagesCount = PackageStoreEntry.OptionalSegmentImportedPackageIds.Num();
		const uint64 OptionalSegmentImportedPackageIdsMemSize = Align(sizeof(FPackageId) * OptionalSegmentImportedPackagesCount, 8);

		const int32 TotalImportedPackagesCount = ImportedPackagesCount + OptionalSegmentImportedPackagesCount;
		const int32 TotalExportBundleCount = bHasOptionalSegment ? 2 : 1;
#else
		const int32 TotalImportedPackagesCount = ImportedPackagesCount;
		const int32 TotalExportBundleCount = 1;
#endif
		const int32 ShaderMapHashesCount = PackageStoreEntry.ShaderMapHashes.Num();

		const uint64 ImportedPackagesMemSize = Align(sizeof(FAsyncPackage2*) * TotalImportedPackagesCount, 8);
		const uint64 ShaderMapHashesMemSize = Align(sizeof(FShaderHash) * ShaderMapHashesCount, 8);
		const uint64 MemoryBufferSize =
#if WITH_EDITOR
			OptionalSegmentImportedPackageIdsMemSize +
#endif
			ImportedPackageIdsMemSize +
			ImportedPackagesMemSize +
			ShaderMapHashesMemSize;

		Data.MemoryBuffer0 = reinterpret_cast<uint8*>(FMemory::Malloc(MemoryBufferSize));

		uint8* DataPtr = Data.MemoryBuffer0;

		Data.TotalExportBundleCount = TotalExportBundleCount;

		Data.ShaderMapHashes = MakeArrayView(reinterpret_cast<const FShaderHash*>(DataPtr), ShaderMapHashesCount);
		FMemory::Memcpy((void*)Data.ShaderMapHashes.GetData(), PackageStoreEntry.ShaderMapHashes.GetData(), sizeof(FShaderHash) * ShaderMapHashesCount);
		DataPtr += ShaderMapHashesMemSize;
		Data.ImportedAsyncPackages = MakeArrayView(reinterpret_cast<FAsyncPackage2**>(DataPtr), TotalImportedPackagesCount);
		FMemory::Memzero(DataPtr, ImportedPackagesMemSize);
		DataPtr += ImportedPackagesMemSize;

		FAsyncPackageHeaderData& HeaderData = AsyncPackage->HeaderData;
		HeaderData.ImportedPackageIds = MakeArrayView(reinterpret_cast<FPackageId*>(DataPtr), ImportedPackagesCount);
		FMemory::Memcpy((void*)HeaderData.ImportedPackageIds.GetData(), PackageStoreEntry.ImportedPackageIds.GetData(), sizeof(FPackageId) * ImportedPackagesCount);
		DataPtr += ImportedPackageIdsMemSize;

		HeaderData.ImportedAsyncPackagesView = Data.ImportedAsyncPackages;
#if WITH_EDITOR
		if (bHasOptionalSegment)
		{
			AsyncPackage->OptionalSegmentSerializationState.Emplace();
			FAsyncPackageHeaderData& OptionalSegmentHeaderData = AsyncPackage->OptionalSegmentHeaderData.Emplace();
			OptionalSegmentHeaderData.ImportedPackageIds = MakeArrayView(reinterpret_cast<FPackageId*>(DataPtr), OptionalSegmentImportedPackagesCount);
			FMemory::Memcpy((void*)OptionalSegmentHeaderData.ImportedPackageIds.GetData(), PackageStoreEntry.OptionalSegmentImportedPackageIds.GetData(), sizeof(FPackageId) * OptionalSegmentImportedPackagesCount);
			DataPtr += OptionalSegmentImportedPackageIdsMemSize;

			HeaderData.ImportedAsyncPackagesView = Data.ImportedAsyncPackages.Left(ImportedPackagesCount);
			OptionalSegmentHeaderData.ImportedAsyncPackagesView = Data.ImportedAsyncPackages.Right(OptionalSegmentImportedPackagesCount);
		}
		// track if later we need to request the optional chunk instead of regular
		AsyncPackage->bRequestOptionalChunk = bRequestOptionalChunk;
#endif
		check(DataPtr - Data.MemoryBuffer0 == MemoryBufferSize);

#if WITH_EDITOR
		const bool bCanImportPackagesWithIdsOnly = false;
#else          
		const bool bIsZenPackage = !AsyncPackage->LinkerLoadState.IsSet();
		bool bCanImportPackagesWithIdsOnly = bIsZenPackage;
#endif
		if (bCanImportPackagesWithIdsOnly)
		{
			check(IoBatch);
			AsyncPackage->ImportPackagesRecursive(ThreadState, *IoBatch, PackageStore);
		}
	}

	void FinishInitializeAsyncPackage(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* AsyncPackage)
	{
		FAsyncPackageData& Data = AsyncPackage->Data;
		FAsyncPackageHeaderData& HeaderData = AsyncPackage->HeaderData;
		int32 TotalExportCount = HeaderData.ExportMap.Num();
		const uint64 ExportBundleEntriesCopyMemSize = Align(HeaderData.ExportBundleEntries.Num() * sizeof(FExportBundleEntry), 8);
#if WITH_EDITOR
		FAsyncPackageHeaderData* OptionalSegmentHeaderData = AsyncPackage->OptionalSegmentHeaderData.GetPtrOrNull();
		uint64 OptionalSegmentExportBundleEntriesCopyMemSize = 0;
		if (OptionalSegmentHeaderData)
		{
			TotalExportCount += OptionalSegmentHeaderData->ExportMap.Num();
			OptionalSegmentExportBundleEntriesCopyMemSize = Align(OptionalSegmentHeaderData->ExportBundleEntries.Num() * sizeof(FExportBundleEntry), 8);
		}
#endif
		const uint64 ExportsMemSize = Align(sizeof(FExportObject) * TotalExportCount, 8);
		const uint64 CellExportsMemSize = Align(sizeof(FExportCell) * HeaderData.CellExportMap.Num(), 8);

		const uint64 MemoryBufferSize =
			ExportsMemSize +
			CellExportsMemSize +
#if WITH_EDITOR
			OptionalSegmentExportBundleEntriesCopyMemSize +
#endif
			ExportBundleEntriesCopyMemSize;

		Data.MemoryBuffer1 = reinterpret_cast<uint8*>(FMemory::Malloc(MemoryBufferSize));

		uint8* DataPtr = Data.MemoryBuffer1;

		Data.Exports = MakeArrayView(reinterpret_cast<FExportObject*>(DataPtr), TotalExportCount);
		DataPtr += ExportsMemSize;
		Data.CellExports = MakeArrayView(reinterpret_cast<FExportCell*>(DataPtr), HeaderData.CellExportMap.Num());
		DataPtr += CellExportsMemSize;
		HeaderData.ExportBundleEntriesCopyForPostLoad = MakeArrayView(reinterpret_cast<FExportBundleEntry*>(DataPtr), HeaderData.ExportBundleEntries.Num());
		FMemory::Memcpy(DataPtr, HeaderData.ExportBundleEntries.GetData(), HeaderData.ExportBundleEntries.Num() * sizeof(FExportBundleEntry));
		DataPtr += ExportBundleEntriesCopyMemSize;

		HeaderData.ExportsView = Data.Exports;

#if WITH_EDITOR
		if (OptionalSegmentHeaderData)
		{
			OptionalSegmentHeaderData->ExportBundleEntriesCopyForPostLoad = MakeArrayView(reinterpret_cast<FExportBundleEntry*>(DataPtr), OptionalSegmentHeaderData->ExportBundleEntries.Num());
			FMemory::Memcpy(DataPtr, OptionalSegmentHeaderData->ExportBundleEntries.GetData(), OptionalSegmentHeaderData->ExportBundleEntries.Num() * sizeof(FExportBundleEntry));
			DataPtr += OptionalSegmentExportBundleEntriesCopyMemSize;

			HeaderData.ExportsView = Data.Exports.Left(HeaderData.ExportCount);
			OptionalSegmentHeaderData->ExportsView = Data.Exports.Right(OptionalSegmentHeaderData->ExportCount);
		}
#endif

		HeaderData.CellExportsView = Data.CellExports;

		check(DataPtr - Data.MemoryBuffer1 == MemoryBufferSize);

		AsyncPackage->ConstructedObjects.Reserve(Data.Exports.Num() + 1); // +1 for UPackage, may grow dynamically beyond that
		for (FExportObject& Export : Data.Exports)
		{
			Export = FExportObject();
		}
		for (FExportCell& CellExport : Data.CellExports)
		{
			CellExport = FExportCell();
		}
		Data.bCellExportsInitialized.store(true, std::memory_order_release);
	}

	void FinishImportingPackages(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* AsyncPackage)
	{
		if (!AsyncPackage->bHasStartedImportingPackages)
		{
#if WITH_EDITOR
			UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(AsyncPackage ? AsyncPackage->Desc.PIEInstanceID : INDEX_NONE);
#endif

			FIoBatch IoBatch = IoDispatcher.NewBatch();
			{
				FPackageStoreReadScope _(PackageStore);
				AsyncPackage->ImportPackagesRecursive(ThreadState, IoBatch, PackageStore);
			}
			IoBatch.Issue();
		}
	}

	void DeleteAsyncPackage(FAsyncPackage2* Package)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DeleteAsyncPackage);
		UE_ASYNC_PACKAGE_DEBUG(Package->Desc);
		for (int32 RequestId : Package->RequestIDs)
		{
			RequestIdToPackageMap.Remove(RequestId);
		}
		if (RequestIdToPackageMap.IsEmpty())
		{
			RequestIdToPackageMap.Empty(DefaultAsyncPackagesReserveCount);
		}
		delete Package;
		--PackagesWithRemainingWorkCounter;
		TRACE_COUNTER_SET(AsyncLoadingPackagesWithRemainingWork, PackagesWithRemainingWorkCounter);
	}

	/** Number of times we re-entered the async loading tick, mostly used by singlethreaded ticking. Debug purposes only. */
	int32 AsyncLoadingTickCounter;

	class FLoaderStats
	{
		struct FPartialFlushInfo
		{
			static constexpr int32 MaxCallstackDepth = 32;
			int32 UniqueCallstackIndex = -1;
			FName RequestedPackage;
			FName RequestingPackage;
			EAsyncPackageLoadingState2 RequestedPackageState;
			EAsyncPackageLoadingState2 RequestingPackageState;

			inline bool operator==(const FPartialFlushInfo& Other) const
			{
				return RequestedPackage == Other.RequestedPackage
					&& RequestingPackage == Other.RequestingPackage;
			}

			inline friend uint32 GetTypeHash(const FPartialFlushInfo& Info)
			{
				return HashCombine(GetTypeHash(Info.RequestedPackageState), GetTypeHash(Info.RequestingPackageState));
			}
		};

	public:
		enum ELogMode : uint8
		{
			Inline,				// Stats are recorded and logged immediately
			Dump,				// Stats are recorded and logged during ShutdownLoading() and DumpState()
			DumpWithCallstack	// Same as Dump but callstacks are also recorded
		};

		void Clear();
		void DumpToLog();
		void RecordPartialFlush(const FAsyncPackage2* RequestedPackage, const FAsyncPackage2* RequestingPackage);
		void Log(FStringBuilderBase& Builder, const FPartialFlushInfo& PartialFlush);

		inline void SetLogMode(ELogMode Mode)
		{
#if !UE_BUILD_SHIPPING
			LogMode = Mode;
#endif
		}
		struct FUniqueCallstack
		{
			TArray<uint64> Callstack;
			bool bHasLogged = false;
		};
		TArray<FUniqueCallstack> UniqueCallstacks;
		TMap<TArray<uint64>, int32> UniqueCallstackMap;

#if !UE_BUILD_SHIPPING
	private:
		TSet<FPartialFlushInfo> PartialFlushes;
		ELogMode LogMode;
#endif
	} LoaderStats;
};

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
class FAsyncLoadingVerseRoot : Verse::FGlobalHeapRoot, Verse::FGlobalHeapCensusRoot
{
	UE::FMutex Mutex;
	FAsyncLoadingThread2* AsyncLoadingThread = nullptr;

public:
	void SetAsyncLoadingThread(FAsyncLoadingThread2* InAsyncLoadingThread)
	{
		UE::TUniqueLock Lock(Mutex);
		AsyncLoadingThread = InAsyncLoadingThread;
	}

	void Visit(Verse::FMarkStackVisitor& Visitor) override { VisitImpl(Visitor); }
	void Visit(Verse::FAbstractVisitor& Visitor) override { VisitImpl(Visitor); }

	// Called from the Verse GC thread or game thread when marking.
	template <class VisitorType>
	void VisitImpl(VisitorType& Visitor)
	{
		UE::TUniqueLock Lock(Mutex);
		if (AsyncLoadingThread == nullptr)
		{
			return;
		}

		// Packages keep all their cell exports alive until they are done loading.
		// This plays the role of ConstructedObjects and the KeepFlags aspect of EInternalObjectFlags::Async.
		{
			AsyncLoadingThread->AsyncPackageLookup.ForEach(
				[&Visitor](TPair<FPackageId, FAsyncPackage2*>& Pair)
				{
					if (Pair.Value->Data.bCellExportsInitialized.load(std::memory_order_acquire))
					{
						for (FExportCell& CellExport : Pair.Value->Data.CellExports)
						{
							Visitor.Visit(CellExport.Cell, TEXT(""));
						}
					}
				}
			);
		}

		// Packages keep all their public cell exports alive while they are referenced by a loading package.
		// This plays the role of PinPublicExportsForGC/UnpinPublicExportsForGC and EInternalObjectFlags::LoaderImport
		{
			FGlobalImportStore& GlobalImportStore = AsyncLoadingThread->GlobalImportStore;
			GlobalImportStore.PackageCells.ForEach(
				[&Visitor](const TPair<FPackageId, FLoadedPackageCellsRef>& PackageCellsRef)
				{
					if (PackageCellsRef.Value.bPinned.load(std::memory_order_acquire))
					{
						UE::TUniqueLock PackageCellsRefLock(PackageCellsRef.Value.Mutex);
						for (const TPair<uint64, Verse::VCell*>& CellExport : PackageCellsRef.Value.PublicExportMap)
						{
							Visitor.Visit(CellExport.Value, TEXT(""));
						}
					}
				}
			);
		}
	}

	virtual void ConductCensus() override
	{
		UE::TUniqueLock Lock(Mutex);
		if (AsyncLoadingThread == nullptr)
		{
			return;
		}

		// Packages only hold their public cell exports weakly when they are not in use.
		// This plays the role of NotifyUnreachableObjects and RemovePublicExports.
		// Note: bAreAllPublicExportsLoaded does not account for VCells, and we do not currently
		//   support reloading on top of existing VCells either.
		{
			FGlobalImportStore& GlobalImportStore = AsyncLoadingThread->GlobalImportStore;
			GlobalImportStore.PackageCells.ForEach(
				[](TPair<FPackageId, FLoadedPackageCellsRef>& PackageCellsRef)
				{
					if (!PackageCellsRef.Value.bPinned.load(std::memory_order_acquire))
					{
						UE::TUniqueLock PackageCellsRefLock(PackageCellsRef.Value.Mutex);
						for (auto It = PackageCellsRef.Value.PublicExportMap.CreateIterator(); It; ++It)
						{
							if (!Verse::FHeap::IsMarked(It->Value))
							{
								It.RemoveCurrent();
							}
						}
					}
				}
			);
		}
	}
};

static Verse::TLazyInitialized<FAsyncLoadingVerseRoot> AsyncLoadingVerseRoot;
#endif

COREUOBJECT_API TDelegate<void(Verse::VPackage*, UPackage*, TArrayView<UObject*>)> NotifyCompiledVersePackageDelegate;

/**
 * Updates FUObjectThreadContext with the current package when processing it.
 * FUObjectThreadContext::AsyncPackage is used by NotifyConstructedDuringAsyncLoading.
 */
struct FAsyncPackageScope2
{
	/** Outer scope package */
	void* PreviousPackage;
	IAsyncPackageLoader* PreviousAsyncPackageLoader;

	/** Cached ThreadContext so we don't have to access it again */
	FUObjectThreadContext& ThreadContext;

	FAsyncPackageScope2(FAsyncPackage2* InPackage)
		: ThreadContext(FUObjectThreadContext::Get())
	{
		PreviousPackage = ThreadContext.AsyncPackage;
		ThreadContext.AsyncPackage = InPackage;
		PreviousAsyncPackageLoader = ThreadContext.AsyncPackageLoader;
		ThreadContext.AsyncPackageLoader = &InPackage->AsyncLoadingThread;
	}
	~FAsyncPackageScope2()
	{
		ThreadContext.AsyncPackage = PreviousPackage;
		ThreadContext.AsyncPackageLoader = PreviousAsyncPackageLoader;
	}
};

/** Just like TGuardValue for FAsyncLoadingThread::AsyncLoadingTickCounter but only works for the game thread */
struct FAsyncLoadingTickScope2
{
	FAsyncLoadingThread2& AsyncLoadingThread;
	bool bNeedsToLeaveAsyncTick;

	FAsyncLoadingTickScope2(FAsyncLoadingThread2& InAsyncLoadingThread)
		: AsyncLoadingThread(InAsyncLoadingThread)
		, bNeedsToLeaveAsyncTick(false)
	{
		if (IsInGameThread())
		{
			AsyncLoadingThread.EnterAsyncLoadingTick();
			bNeedsToLeaveAsyncTick = true;
		}
	}
	~FAsyncLoadingTickScope2()
	{
		if (bNeedsToLeaveAsyncTick)
		{
			AsyncLoadingThread.LeaveAsyncLoadingTick();
		}
	}
};

void FAsyncLoadingThread2::InitializeLoading()
{
#if !UE_BUILD_SHIPPING
	{
		FString DebugPackageNamesString;
		FParse::Value(FCommandLine::Get(), TEXT("-s.DebugPackageNames="), DebugPackageNamesString);
		ParsePackageNames(DebugPackageNamesString, GAsyncLoading2_DebugPackageIds);
		FString VerbosePackageNamesString;
		FParse::Value(FCommandLine::Get(), TEXT("-s.VerbosePackageNames="), VerbosePackageNamesString);
		ParsePackageNames(VerbosePackageNamesString, GAsyncLoading2_VerbosePackageIds);
		ParsePackageNames(DebugPackageNamesString, GAsyncLoading2_VerbosePackageIds);
		GAsyncLoading2_VerboseLogFilter = GAsyncLoading2_VerbosePackageIds.Num() > 0 ? 1 : 2;
	}

	FileOpenLogWrapper = (FPlatformFileOpenLog*)(FPlatformFileManager::Get().FindPlatformFile(FPlatformFileOpenLog::GetTypeName()));

	if (IsRunningCookCommandlet())
	{
		// Don't overwhelm cook logs with stat logging during cooking and instead 
		// accumulate records to flush when we shutdown the loader
		LoaderStats.SetLogMode(FLoaderStats::ELogMode::DumpWithCallstack);
	}
	else
	{
		// When not cooking, log immediately so we don't bear any memory overhead from recording stats
		// and can ensure we have logged records should the application crash before we can call DumpToLog
		LoaderStats.SetLogMode(FLoaderStats::ELogMode::Inline);
	}
#endif

	PackageStore.OnPendingEntriesAdded().AddLambda([this]()
	{
		AltZenaphore.NotifyOne();
	});

	AsyncThreadReady.fetch_add(1);

	UE_LOGF(LogStreaming, Log, "AsyncLoading2 - Initialized");
}

void FAsyncLoadingThread2::UpdatePackagePriority(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UpdatePackagePriority);
	EAsyncPackageLoadingState2 LoadingState = Package->AsyncPackageLoadingState;

	// TODO: Shader requests are not tracked so they won't be reprioritized correctly here
	if (LoadingState <= EAsyncPackageLoadingState2::WaitingForIo)
	{
		Package->SerializationState.IoRequest.UpdatePriority(Package->Desc.Priority);
#if WITH_EDITOR
		if (Package->OptionalSegmentSerializationState.IsSet())
		{
			Package->OptionalSegmentSerializationState->IoRequest.UpdatePriority(Package->Desc.Priority);
		}
#endif
	}
	if (LoadingState <= EAsyncPackageLoadingState2::PostLoad)
	{
		EventQueue.UpdatePackagePriority(Package);
	}
	if (LoadingState == EAsyncPackageLoadingState2::DeferredPostLoad)
	{
		if (ThreadState.bIsAsyncLoadingThread || IsInParallelLoadingThread())
		{
			if (Package->TryAddRef())
			{
				GameThreadState->PackagesToReprioritize.Enqueue(Package);

				// Repriorization of packages is of interest to main thread as it could unblock a flush.
				MainThreadWakeEvent.Notify();
			}
		}
		else
		{
			MainThreadEventQueue.UpdatePackagePriority(Package);
		}
	}
}

void FAsyncLoadingThread2::UpdatePackagePriorityRecursive(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package, int32 NewPriority)
{
	if (Package->Desc.Priority >= NewPriority)
	{
		return;
	}

	Package->Desc.Priority = NewPriority;
	UpdatePackagePriority(ThreadState, Package);

	// For LinkerLoad, dependencies are loaded in ProcessPackageSummary so if this is called before,
	// early out to avoid races and we'll reprocess them in the ProcessDependencies step
	if (Package->LinkerLoadState.IsSet() &&
		Package->AsyncPackageLoadingState <= EAsyncPackageLoadingState2::ProcessPackageSummary)
	{
		return;
	}

	for (FAsyncPackage2* ImportedPackage : Package->Data.ImportedAsyncPackages)
	{
		if (ImportedPackage)
		{
			UpdatePackagePriorityRecursive(ThreadState, ImportedPackage, NewPriority);
		}
	}

	for (FAsyncPackage2* ImportedPackage : Package->AdditionalImportedAsyncPackages)
	{
		UpdatePackagePriorityRecursive(ThreadState, ImportedPackage, NewPriority);
	}
}

void FAsyncPackage2::ConditionalBeginPostLoad(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ConditionalBeginPostLoad);

	check(AsyncPackageLoadingState == EAsyncPackageLoadingState2::ExportsDone);
	AsyncPackageLoadingState = EAsyncPackageLoadingState2::PostLoad;
	ConditionalReleasePartialRequests(ThreadState);
	ConditionalBeginPostLoadPhase(ThreadState);
}

void FAsyncPackage2::ConditionalBeginDeferredPostLoad(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ConditionalBeginDeferredPostLoad);

	// Move everything to the second phase so that StaticFind from inside postload can
	// see objects ready for postload.
	MoveConstructedObjectsToPhase2();

	check(AsyncPackageLoadingState == EAsyncPackageLoadingState2::PostLoad);
	AsyncPackageLoadingState = EAsyncPackageLoadingState2::DeferredPostLoad;
	ConditionalReleasePartialRequests(ThreadState);

	GetPackageNode(Package_DeferredPostLoad).ReleaseBarrier(&ThreadState);
}

FAsyncPackage2* FAsyncLoadingThread2::FindOrInsertPackage(FAsyncLoadingThreadState2& ThreadState, FAsyncPackageDesc2& Desc, bool& bInserted, FAsyncPackage2* ImportedByPackage, TUniquePtr<FLoadPackageAsyncDelegate>&& PackageLoadedDelegate, TSharedPtr<FLoadPackageAsyncProgressDelegate> PackageProgressDelegate)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FindOrInsertPackage);
	FAsyncPackage2* Package = nullptr;
	bInserted = false;
	{
		AsyncPackageLookup.FindOrProduceAndApplyForWrite(Desc.UPackageId,
			[&]() -> FAsyncPackage2*
			{
				Package = CreateAsyncPackage(ThreadState, Desc);
				checkf(Package, TEXT("Failed to create async package %s"), *Desc.UPackageName.ToString());
				++LoadingPackagesCounter;
				TRACE_COUNTER_SET(AsyncLoadingLoadingPackages, LoadingPackagesCounter);
				bInserted = true;
				return Package;
			},
			[&](FAsyncPackage2* InPackage)
			{
				Package = InPackage;

				if (Desc.RequestID > 0)
				{
					Package->AddRequestID(ThreadState, Desc.RequestID);
				}
				if (ImportedByPackage)
				{
					Package->ImportRequestIDs.Append(ImportedByPackage->RequestIDs);
				}
				if (Desc.Priority > Package->Desc.Priority)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(UpdatePackagePriority);
					UpdatePackagePriorityRecursive(ThreadState, Package, Desc.Priority);
				}

				if (PackageLoadedDelegate.IsValid())
				{
					Package->AddCompletionCallback(MoveTemp(PackageLoadedDelegate));
				}

				if (PackageProgressDelegate.IsValid())
				{
					Package->AddProgressCallback(PackageProgressDelegate);
				}
			}
		);
	}

	return Package;
}

void FAsyncLoadingThread2::IncludePackageInSyncLoadContextRecursive(FAsyncLoadingThreadState2& ThreadState, uint64 ContextId, FAsyncPackage2* Package)
{
	if (Package->SyncLoadContextId >= ContextId)
	{
		return;
	}

	if (Package->AsyncPackageLoadingState >= EAsyncPackageLoadingState2::Complete)
	{
		return;
	}

	UE_ASYNC_PACKAGE_LOG(VeryVerbose, Package->Desc, TEXT("IncludePackageInSyncLoadContextRecursive"), TEXT("Setting SyncLoadContextId to %" UINT64_FMT), ContextId);

	Package->SyncLoadContextId = ContextId;

	if (Package->Desc.Priority < MAX_int32)
	{
		Package->Desc.Priority = MAX_int32;
		UpdatePackagePriority(ThreadState, Package);
	}

	// For LinkerLoad, dependencies are loaded in ProcessPackageSummary so if this is called before,
	// early out to avoid races and we'll reprocess them in the ProcessDependencies step
	if (Package->LinkerLoadState.IsSet() &&
		Package->AsyncPackageLoadingState <= EAsyncPackageLoadingState2::ProcessPackageSummary)
	{
		return;
	}

	for (FAsyncPackage2* ImportedPackage : Package->Data.ImportedAsyncPackages)
	{
		if (ImportedPackage && ImportedPackage->SyncLoadContextId < ContextId)
		{
			IncludePackageInSyncLoadContextRecursive(ThreadState, ContextId, ImportedPackage);
		}
	}

	for (FAsyncPackage2* ImportedPackage : Package->AdditionalImportedAsyncPackages)
	{
		if (ImportedPackage->SyncLoadContextId < ContextId)
		{
			IncludePackageInSyncLoadContextRecursive(ThreadState, ContextId, ImportedPackage);
		}
	}
}

UE_TRACE_EVENT_BEGIN(CUSTOM_LOADTIMER_LOG, CreateAsyncPackage, NoSync)
UE_TRACE_EVENT_FIELD(uint64, PackageId)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PackageName)
UE_TRACE_EVENT_END()

bool FAsyncLoadingThread2::CreateAsyncPackagesFromQueue(FAsyncLoadingThreadState2& ThreadState, bool bForceExitForGarbageCollect)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateAsyncPackagesFromQueue);

	// Package creation needs access to all objects
	TGuardValue GuardVisibilityFilter(FUObjectThreadContext::Get().AsyncVisibilityFilter, EInternalObjectFlags::None);

	bool bPackagesCreated = false;
	const int32 TimeSliceGranularity = (ThreadState.UseTimeLimit() || bForceExitForGarbageCollect) ? 4 : MAX_int32;

	FIoBatch IoBatch = IoDispatcher.NewBatch();

	FPackageStoreReadScope _(PackageStore);

	for (auto It = PendingPackages.CreateIterator(); It; ++It)
	{
		ThreadState.MarkAsActive();

		FAsyncPackage2* PendingPackage = *It;
		FPackageStoreEntry PackageEntry;
		EPackageStoreEntryStatus PendingPackageStatus = PackageStore.GetPackageStoreEntry(PendingPackage->Desc.PackageIdToLoad,
			PendingPackage->Desc.UPackageName, PackageEntry);
		if (PendingPackageStatus == EPackageStoreEntryStatus::Ok)
		{
			SCOPED_CUSTOM_LOADTIMER(CreateAsyncPackage)
				ADD_CUSTOM_LOADTIMER_META(CreateAsyncPackage, PackageId, PendingPackage->Desc.UPackageId.Value())
				ADD_CUSTOM_LOADTIMER_META(CreateAsyncPackage, PackageName, *WriteToString<256>(PendingPackage->Desc.UPackageName));
			InitializeAsyncPackageFromPackageStore(ThreadState, &IoBatch, PendingPackage, PackageEntry);
			PendingPackage->StartLoading(ThreadState, IoBatch);
			It.RemoveCurrent();
		}
		else if (PendingPackageStatus == EPackageStoreEntryStatus::Missing)
		{
			// Initialize package with a fake package store entry
			FPackageStoreEntry FakePackageEntry;
			InitializeAsyncPackageFromPackageStore(ThreadState, &IoBatch, PendingPackage, FakePackageEntry);
			// Simulate StartLoading() getting back a failed IoRequest and let it go through all package states
			PendingPackage->AsyncPackageLoadingState = EAsyncPackageLoadingState2::WaitingForIo;
			PendingPackage->LoadStatus = EAsyncLoadingResult::FailedMissing;
			PendingPackage->GetPackageNode(EEventLoadNode2::Package_ProcessSummary).ReleaseBarrier(&ThreadState);
			// Remove from PendingPackages
			It.RemoveCurrent();
		}
		else if (PendingPackageStatus == EPackageStoreEntryStatus::NotInstalled)
		{
			// Initialize package with a fake package store entry
			FPackageStoreEntry FakePackageEntry;
			InitializeAsyncPackageFromPackageStore(ThreadState, &IoBatch, PendingPackage, FakePackageEntry);
			// Simulate StartLoading() getting back a failed IoRequest and let it go through all package states
			PendingPackage->AsyncPackageLoadingState = EAsyncPackageLoadingState2::WaitingForIo;
			PendingPackage->LoadStatus = EAsyncLoadingResult::FailedNotInstalled;
			PendingPackage->GetPackageNode(EEventLoadNode2::Package_ProcessSummary).ReleaseBarrier(&ThreadState);
			// Remove from PendingPackages
			It.RemoveCurrent();
		}
	}
	for (;;)
	{
		int32 NumDequeued = 0;
		while (NumDequeued < TimeSliceGranularity)
		{
			ThreadState.MarkAsActive();

			TOptional<FPackageRequest> OptionalRequest = PackageRequestQueue.Dequeue();
			if (!OptionalRequest.IsSet())
			{
				break;
			}

			--QueuedPackagesCounter;
			++NumDequeued;
			TRACE_COUNTER_SET(AsyncLoadingQueuedPackages, QueuedPackagesCounter);

			FPackageRequest& Request = OptionalRequest.GetValue();
			EPackageStoreEntryStatus PackageStatus = EPackageStoreEntryStatus::Missing;
			EPackageLoader PackageLoader = EPackageLoader::Zen;
			FPackageStoreEntry PackageEntry;
			FName PackageNameToLoad = Request.PackagePath.GetPackageFName();
			TCHAR NameBuffer[FName::StringBufferSize];
			uint32 NameLen = PackageNameToLoad.ToString(NameBuffer);
			FStringView PackageNameStr = FStringView(NameBuffer, NameLen);
			if (!FPackageName::IsValidLongPackageName(PackageNameStr))
			{
				FString NewPackageNameStr;
				if (FPackageName::TryConvertFilenameToLongPackageName(FString(PackageNameStr), NewPackageNameStr))
				{
					PackageNameToLoad = *NewPackageNameStr;
				}
			}

			FName UPackageName = PackageNameToLoad;

			if (GAllowPackageRedirectorSupport)
			{
				// Redirects should redirect both the UPackage name as well as the package name to load
				PackageNameToLoad = ApplyPackageNameRedirections(PackageNameToLoad);
				UPackageName = PackageNameToLoad;
			}

			// Localization should only update the package name to load
			PackageNameToLoad = ApplyLooseFileLocalizationPackageNameRedirects(PackageNameToLoad);

			FPackageId PackageIdToLoad = FPackageId::FromName(PackageNameToLoad);
			{
				FName SourcePackageName;
				FPackageId RedirectedToPackageId;
				if (PackageStore.GetPackageRedirectInfo(PackageIdToLoad, SourcePackageName, RedirectedToPackageId))
				{
					PackageIdToLoad = RedirectedToPackageId;
					Request.PackagePath.Empty(); // We no longer know the path but it will be set again when serializing the package summary
					PackageNameToLoad = NAME_None;
					UPackageName = SourcePackageName;
				}
			}

			FPackagePath::TryFromPackageName(PackageNameToLoad, Request.PackagePath);
			EPackageExtension Extension;
			if (!PackageNameToLoad.IsNone() && TryGetExistingLoadedPackagePath(PackageIdToLoad, PackageNameToLoad, Extension, PackageLoader, PackageEntry))
			{
				Request.PackagePath.SetHeaderExtension(Extension);
				PackageStatus = EPackageStoreEntryStatus::Ok;
			}
			else
			{
				PackageStatus = TryGetPackageInfoFromPackageStore(PackageIdToLoad, PackageNameToLoad, UPackageName, PackageEntry);

				// try again after looking for an matching active load request
				if (PackageStatus == EPackageStoreEntryStatus::Missing)
				{
					// While there is an active load request for (InName=/Temp/PackageABC_abc, InPackageToLoadFrom=/Game/PackageABC), then allow these requests too:
					// (InName=/Temp/PackageA_abc, InPackageToLoadFrom=/Temp/PackageABC_abc) and (InName=/Temp/PackageABC_xyz, InPackageToLoadFrom=/Temp/PackageABC_abc)
					FAsyncPackage2* Package = GetAsyncPackage(PackageIdToLoad);
					if (Package)
					{
						PackageIdToLoad = Package->Desc.PackageIdToLoad;
						PackageNameToLoad = Request.PackagePath.GetPackageFName();
						PackageStatus = TryGetPackageInfoFromPackageStore(PackageIdToLoad, PackageNameToLoad, UPackageName, PackageEntry);
					}
				}

				if (PackageStatus != EPackageStoreEntryStatus::Missing)
				{
					Request.PackagePath.SetHeaderExtension(PackageEntry.PackageExtension);
					PackageLoader = PackageEntry.LoaderType;
				}
			}
			// Fixup CustomName to handle any input string that can be converted to a long package name.
			if (!Request.CustomName.IsNone())
			{
				NameLen = Request.CustomName.ToString(NameBuffer);
				PackageNameStr = FStringView(NameBuffer, NameLen);
				if (!FPackageName::IsValidLongPackageName(PackageNameStr))
				{
					FString NewPackageNameStr;
					if (FPackageName::TryConvertFilenameToLongPackageName(FString(PackageNameStr), NewPackageNameStr))
					{
						Request.CustomName = *NewPackageNameStr;
					}
				}
				UPackageName = Request.CustomName;
			}

			FAsyncPackageDesc2 PackageDesc = FAsyncPackageDesc2::FromPackageRequest(Request, UPackageName, PackageIdToLoad, PackageLoader);
			if (PackageStatus == EPackageStoreEntryStatus::Missing || PackageStatus == EPackageStoreEntryStatus::NotInstalled)
			{
				QueueUnresolvedPackage(ThreadState, PackageStatus, PackageDesc, MoveTemp(Request.PackageLoadedDelegate), MoveTemp(Request.PackageProgressDelegate));
			}
			else
			{
				bool bInserted;
				FAsyncPackage2* Package = FindOrInsertPackage(ThreadState, PackageDesc, bInserted, nullptr, MoveTemp(Request.PackageLoadedDelegate), MoveTemp(Request.PackageProgressDelegate));
				checkf(Package, TEXT("Failed to find or insert package %s"), *PackageDesc.UPackageName.ToString());

				if (bInserted)
				{
					UE_ASYNC_PACKAGE_LOG(Verbose, PackageDesc, TEXT("CreateAsyncPackages: AddPackage"),
						TEXT("Start loading package."));
#if !UE_BUILD_SHIPPING
					if (FileOpenLogWrapper)
					{
						FileOpenLogWrapper->AddPackageToOpenLog(*PackageDesc.UPackageName.ToString());
					}
#endif
					if (PackageStatus == EPackageStoreEntryStatus::Pending)
					{
						PendingPackages.Add(Package);
					}
					else
					{
						SCOPED_CUSTOM_LOADTIMER(CreateAsyncPackage)
							ADD_CUSTOM_LOADTIMER_META(CreateAsyncPackage, PackageId, PackageDesc.UPackageId.Value())
							ADD_CUSTOM_LOADTIMER_META(CreateAsyncPackage, PackageName, NameBuffer);

						check(PackageStatus == EPackageStoreEntryStatus::Ok);
						check(PackageLoader != EPackageLoader::Unknown);

						if (PackageLoader == EPackageLoader::LinkerLoad)
						{
							Package->InitializeLinkerLoadState(&PackageDesc.InstancingContext);
						}
						else
						{
							check(PackageLoader == EPackageLoader::Zen);
							Package->Desc.InstancingContext = PackageDesc.InstancingContext;
							InitializeAsyncPackageFromPackageStore(ThreadState, &IoBatch, Package, PackageEntry);
						}
						Package->StartLoading(ThreadState, IoBatch);
					}
				}
				else
				{
					UE_ASYNC_PACKAGE_LOG_VERBOSE(Verbose, PackageDesc, TEXT("CreateAsyncPackages: UpdatePackage"),
						TEXT("Package is already being loaded."));
					--PackagesWithRemainingWorkCounter;
					TRACE_COUNTER_SET(AsyncLoadingPackagesWithRemainingWork, PackagesWithRemainingWorkCounter);
				}

				RequestIdToPackageMap.Add(PackageDesc.RequestID, Package);
			}
		}

		bPackagesCreated |= NumDequeued > 0;

		if (!NumDequeued || ThreadState.IsTimeLimitExceeded(TEXT("CreateAsyncPackagesFromQueue")))
		{
			break;
		}
	}

	IoBatch.Issue();

	return bPackagesCreated;
}

FEventLoadNode2::FEventLoadNode2(const FAsyncLoadEventSpec* InSpec, FAsyncPackage2* InPackage, int32 InBarrierCount)
	: Spec(InSpec)
	, Package(InPackage)
	, BarrierCount(InBarrierCount)
{
	check(Spec);
	check(Package);
}

void FEventLoadNode2::DependsOn(FEventLoadNode2* Other)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DependsOn);
	check(!bIsDone.load(std::memory_order_relaxed));
	bool bExpected = false;
	// Set modification flag before checking the done flag
	// If we're currently in ProcessDependencies the done flag will have been set and we won't do anything
	// If ProcessDependencies is called during this call it will wait for the modification flag to be cleared
	while (!Other->bIsUpdatingDependencies.compare_exchange_strong(bExpected, true))
	{
		// Note: Currently only the async loading thread is calling DependsOn so this will never be contested
		bExpected = false;
	}
	if (!Other->bIsDone.load())
	{
		++BarrierCount;
		if (Other->DependenciesCount == 0)
		{
			Other->SingleDependent = this;
			Other->DependenciesCount = 1;
		}
		else
		{
			if (Other->DependenciesCount == 1)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(DependsOnAlloc);
				FEventLoadNode2* FirstDependency = Other->SingleDependent;
				uint32 NewDependenciesCapacity = 4;
				Other->DependenciesCapacity = NewDependenciesCapacity;
				Other->MultipleDependents = Package->GetGraphAllocator().AllocArcs(NewDependenciesCapacity);
				Other->MultipleDependents[0] = FirstDependency;
			}
			else if (Other->DependenciesCount == Other->DependenciesCapacity)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(DependsOnRealloc);
				FEventLoadNode2** OriginalDependents = Other->MultipleDependents;
				uint32 OldDependenciesCapcity = Other->DependenciesCapacity;
				SIZE_T OldDependenciesSize = OldDependenciesCapcity * sizeof(FEventLoadNode2*);
				uint32 NewDependenciesCapacity = OldDependenciesCapcity * 2;
				Other->DependenciesCapacity = NewDependenciesCapacity;
				Other->MultipleDependents = Package->GetGraphAllocator().AllocArcs(NewDependenciesCapacity);
				FMemory::Memcpy(Other->MultipleDependents, OriginalDependents, OldDependenciesSize);
				Package->GetGraphAllocator().FreeArcs(OriginalDependents, OldDependenciesCapcity);
			}
			Other->MultipleDependents[Other->DependenciesCount++] = this;
		}
	}
	Other->bIsUpdatingDependencies.store(false);
}

void FEventLoadNode2::AddBarrier()
{
	check(!bIsDone.load(std::memory_order_relaxed));
	++BarrierCount;
}

void FEventLoadNode2::AddBarrier(int32 Count)
{
	check(!bIsDone.load(std::memory_order_relaxed));
	BarrierCount += Count;
}

void FEventLoadNode2::ReleaseBarrier(FAsyncLoadingThreadState2* ThreadState)
{
	check(BarrierCount > 0);
	if (--BarrierCount == 0)
	{
		Fire(ThreadState);
	}
}

class FAsyncLoadingThreadState2Scope
{
public:
	FAsyncLoadingThreadState2Scope(FAsyncLoadingThreadState2& ThreadState)
	{
		check(FAsyncLoadingThreadState2::Get() == nullptr);
		FAsyncLoadingThreadState2::Set(&ThreadState);
	}

	~FAsyncLoadingThreadState2Scope()
	{
		check(FAsyncLoadingThreadState2::Get()->CurrentlyExecutingEventNodeStack.Num() == 0);
		FAsyncLoadingThreadState2::Reset();
	}

	FAsyncLoadingThreadState2& Get()
	{
		return *FAsyncLoadingThreadState2::Get();
	}
};

void FEventLoadNode2::ParallelLoadingWork(FAsyncLoadingThreadState2* InParentState)
{
#if USING_INSTRUMENTATION
	if (GDetectRaceDuringLoading)
	{
		UE::Sanitizer::RaceDetector::ToggleFilterOtherThreads(true);
	}
	UE::Sanitizer::RaceDetector::FRaceDetectorScope RaceDetectorScope(GDetectRaceDuringLoading, GDetectRaceGracePeriod);
#endif

	LLM_SCOPE(ELLMTag::AsyncLoading);
	FTaskTagScope Scope(ETaskTag::EParallelLoadingThread);
	FAsyncLoadingThreadState2 LocalThreadState{ .ParentState = InParentState, .bIsWorkerThread = true };

	FAsyncLoadingThreadState2Scope WorkerStateScope(LocalThreadState);

	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	// Firing a node can come from anywhere, needs to remove any current visibility filter
	// A proper visibility filter is expected to be setup properly in each different nodes
	TGuardValue GuardVisibilityFilter(ThreadContext.AsyncVisibilityFilter, EInternalObjectFlags::None);

	// Used in NotifyConstructedDuringAsyncLoading to dispatch any created objects
	// to this package if they don't belong to any other package.
	ThreadContext.AsyncPackage = Package;

	EEventLoadNodeExecutionResult Result = Execute(LocalThreadState);
	check(Result == EEventLoadNodeExecutionResult::Complete);

	ThreadContext.AsyncPackage = nullptr;

	// We should never destroy a thread state that still has nodes to fire
	check(LocalThreadState.NodesToFire.Num() == 0);

	// This is never used for zen loader, everything is handled in NotifyConstructedDuringAsyncLoading
	// Reset to avoid accumulation and shutdown error complaining about objects being present in the context.
	ThreadContext.GetSerializeContext()->PRIVATE_GetObjectsLoadedInternalUseOnly().Reset();
}

void FEventLoadNode2::Fire(FAsyncLoadingThreadState2* ThreadState)
{
	if (Spec->ExecutionType == EExecutionType::Immediate && ThreadState)
	{
		// Firing a node can come from anywhere, needs to remove any current visibility filter
		TGuardValue GuardVisibilityFilter(FUObjectThreadContext::Get().AsyncVisibilityFilter, EInternalObjectFlags::None);

		EEventLoadNodeExecutionResult Result = Execute(*ThreadState);
		check(Result == EEventLoadNodeExecutionResult::Complete);

		return;
	}
	else if (GAllowMultithreadedLoading && Spec->ExecutionType == EExecutionType::Parallel && Package->CanRunNodeAsync(Spec))
	{
		// We need the GC lock to go wide, otherwise we just queue the task normally since a GC is imminent

		// Always take the async lock to avoid imbalance since we're releasing in a different thread
		constexpr bool bShouldLock = true;

		if (FGCCSyncObject::Get().TryLockAsync(bShouldLock))
		{
			UE::Tasks::Launch(
				TEXT("ParallelLoadingWork"),
				[this, ParentState = ThreadState]()
				{
#if WITH_VERSE_VM
					Verse::FRunningContext::Create([this, ParentState](Verse::FRunningContext Context) { ParallelLoadingWork(ParentState); });
#else
					ParallelLoadingWork(ParentState);
#endif

					FGCCSyncObject::Get().UnlockAsync(bShouldLock);
				}
			);

			return;
		}

		// fallthrough to push the task to the queue
	}

	Spec->EventQueue->Push(ThreadState, this);
}

EEventLoadNodeExecutionResult FEventLoadNode2::Execute(FAsyncLoadingThreadState2& ThreadState)
{
	ThreadState.MarkAsActive();

	// This will avoid a race where the function being called releases a barrier
	// that allows the package to move forward on other threads and get deleted
	// before we have the time to process dependencies. This is only required
	// when outside ALT since ALT is responsible for deleting packages so it
	// cannot race with itself.
	if (ThreadState.bIsWorkerThread)
	{
		Package->AddRef();
	}

	check(BarrierCount.load(std::memory_order_relaxed) == 0);
	EEventLoadNodeExecutionResult Result;
	{
		ThreadState.CurrentlyExecutingEventNodeStack.Push(this);
		Result = Spec->Func(ThreadState, Package);
		ThreadState.CurrentlyExecutingEventNodeStack.Pop();
	}
	if (Result == EEventLoadNodeExecutionResult::Complete)
	{
		ProcessDependencies(ThreadState);
	}

	if (ThreadState.bIsWorkerThread)
	{
		Package->Release();
	}
	return Result;
}

void FEventLoadNode2::ProcessDependencies(FAsyncLoadingThreadState2& ThreadState)
{
	//TRACE_CPUPROFILER_EVENT_SCOPE(ProcessDependencies);
	// Set the done flag before checking the modification flag
	bIsDone.store(true);
	while (bIsUpdatingDependencies.load())
	{
		FPlatformProcess::Sleep(0);
	}

	if (DependenciesCount == 1)
	{
		check(SingleDependent->BarrierCount > 0);
		if (--SingleDependent->BarrierCount == 0)
		{
			ThreadState.NodesToFire.Push(SingleDependent);
		}
	}
	else if (DependenciesCount != 0)
	{
		FEventLoadNode2** Current = MultipleDependents;
		FEventLoadNode2** End = MultipleDependents + DependenciesCount;
		for (; Current < End; ++Current)
		{
			FEventLoadNode2* Dependent = *Current;
			check(Dependent->BarrierCount > 0);
			if (--Dependent->BarrierCount == 0)
			{
				ThreadState.NodesToFire.Push(Dependent);
			}
		}
		Package->GetGraphAllocator().FreeArcs(MultipleDependents, DependenciesCapacity);
	}
	if (ThreadState.bShouldFireNodes)
	{
		ThreadState.bShouldFireNodes = false;
		while (ThreadState.NodesToFire.Num())
		{
			ThreadState.NodesToFire.Pop(EAllowShrinking::No)->Fire(&ThreadState);
		}
		ThreadState.bShouldFireNodes = true;
	}
}

uint64 FEventLoadNode2::GetSyncLoadContextId() const
{
	return Package->GetSyncLoadContextId();
}

FAsyncLoadEventQueue2::FAsyncLoadEventQueue2()
{
}

FAsyncLoadEventQueue2::~FAsyncLoadEventQueue2()
{
}

void FAsyncLoadEventQueue2::Push(FAsyncLoadingThreadState2* ThreadState, FEventLoadNode2* Node)
{
	if (OwnerThread == ThreadState)
	{
		PushLocal(Node);
	}
	else
	{
		PushExternal(Node);
	}
}

void FAsyncLoadEventQueue2::PushLocal(FEventLoadNode2* Node)
{
	check(!Node->QueueStatus);
	int32 Priority = Node->Package->Desc.Priority;
	Node->QueueStatus = FEventLoadNode2::QueueStatus_Local;
	LocalQueue.Push(Node, Priority);
}

void FAsyncLoadEventQueue2::PushExternal(FEventLoadNode2* Node)
{
	{
		int32 Priority = Node->Package->Desc.Priority;
		FScopeLock Lock(&ExternalCritical);
		check(!Node->QueueStatus);
		Node->QueueStatus = FEventLoadNode2::QueueStatus_External;
		ExternalQueue.Push(Node, Priority);
		UpdateExternalQueueState();
	}
	if (Zenaphore)
	{
		Zenaphore->NotifyOne();
	}
	if (WakeEvent)
	{
		WakeEvent->Notify();
	}
}

bool FAsyncLoadEventQueue2::PopAndExecute(FAsyncLoadingThreadState2& ThreadState)
{
	// By default, nodes are all executed without visibility filter
	TGuardValue GuardVisibilityFilter(FUObjectThreadContext::Get().AsyncVisibilityFilter, EInternalObjectFlags::None);

	if (TimedOutEventNode)
	{
		// Backup and reset the node before executing it in case we end up with a recursive flush call, we don't want the same node run multiple time.
		FEventLoadNode2* LocalTimedOutEventNode = TimedOutEventNode;
		TimedOutEventNode = nullptr;

		EEventLoadNodeExecutionResult Result = LocalTimedOutEventNode->Execute(ThreadState);
		if (Result == EEventLoadNodeExecutionResult::Timeout)
		{
			TimedOutEventNode = LocalTimedOutEventNode;
		}
		return true;
	}

	bool bPopFromExternalQueue = false;
	int32 MaxPriorityInExternalQueue;
	if (GetMaxPriorityInExternalQueue(MaxPriorityInExternalQueue))
	{
		if (LocalQueue.IsEmpty() || MaxPriorityInExternalQueue > LocalQueue.GetMaxPriority())
		{
			bPopFromExternalQueue = true;
		}
	}
	FEventLoadNode2* Node;
	if (bPopFromExternalQueue)
	{
		FScopeLock Lock(&ExternalCritical);
		Node = ExternalQueue.Pop();
		check(Node);
		UpdateExternalQueueState();
	}
	else
	{
		Node = LocalQueue.Pop();
	}
	if (!Node)
	{
		return false;
	}
	Node->QueueStatus = FEventLoadNode2::QueueStatus_None;

	EEventLoadNodeExecutionResult Result = Node->Execute(ThreadState);
	if (Result == EEventLoadNodeExecutionResult::Timeout)
	{
		TimedOutEventNode = Node;
	}
	return true;
}

bool FAsyncLoadEventQueue2::ExecuteSyncLoadEvents(FAsyncLoadingThreadState2& ThreadState)
{
	// By default, nodes are all executed without visibility filter
	TGuardValue GuardVisibilityFilter(FUObjectThreadContext::Get().AsyncVisibilityFilter, EInternalObjectFlags::None);

	check(!ThreadState.SyncLoadContextStack.IsEmpty());
	FAsyncLoadingSyncLoadContext& SyncLoadContext = *ThreadState.SyncLoadContextStack.Top();

	int32 ThisCallCounter = ++ExecuteSyncLoadEventsCallCounter;

	auto ShouldExecuteNode = [&SyncLoadContext](FEventLoadNode2& Node) -> bool
		{
			return Node.Package->SyncLoadContextId >= SyncLoadContext.ContextId;
		};

	bool bDidSomething = false;
	if (TimedOutEventNode && ShouldExecuteNode(*TimedOutEventNode))
	{
		// Backup and reset the node before executing it in case we end up with a recursive flush call, we don't want the same node run multiple time.
		FEventLoadNode2* LocalTimedOutEventNode = TimedOutEventNode;
		TimedOutEventNode = nullptr;

		EEventLoadNodeExecutionResult Result = LocalTimedOutEventNode->Execute(ThreadState);
		check(Result == EEventLoadNodeExecutionResult::Complete); // we can't timeout during a sync load operation
		bDidSomething = true;
	}

	int32 MaxPriorityInExternalQueue;
	bool bTakeFromExternalQueue = GetMaxPriorityInExternalQueue(MaxPriorityInExternalQueue) && MaxPriorityInExternalQueue == MAX_int32;
	if (bTakeFromExternalQueue)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MergeIntoLocalQueue);
		// Take all the max prio items from the external queue and put in the local queue. This breaks the queue status value
		// of the items in the external queue but we know that we'll never reprioritize them again so it doesn't matter
		FScopeLock Lock(&ExternalCritical);
		ExternalQueue.MergeInto(LocalQueue, MAX_int32);
		UpdateExternalQueueState();
	}
	for (auto It = LocalQueue.CreateIterator(MAX_int32); It; ++It)
	{
		FEventLoadNode2& Node = *It;
		if (ShouldExecuteNode(Node))
		{
			It.RemoveCurrent();
			Node.QueueStatus = FEventLoadNode2::QueueStatus_None;
			EEventLoadNodeExecutionResult Result = Node.Execute(ThreadState);
			check(Result == EEventLoadNodeExecutionResult::Complete); // we can't timeout during a sync load operation
			if (ExecuteSyncLoadEventsCallCounter != ThisCallCounter)
			{
				// ExecuteSyncLoadEvents was called recursively and our view of the list might have been compromised, start over
				return true;
			}
			bDidSomething = true;
		}
	}
	if (!bDidSomething && ThreadState.bIsAsyncLoadingThread)
	{
		return PopAndExecute(ThreadState);
	}
	return bDidSomething;
}

void FAsyncLoadEventQueue2::UpdatePackagePriority(FAsyncPackage2* Package)
{
	FScopeLock Lock(&ExternalCritical);
	auto ReprioritizeNode = [this](FEventLoadNode2& Node)
		{
			if (Node.Spec->EventQueue == this && Node.Priority < Node.Package->Desc.Priority)
			{
				if (Node.QueueStatus == FEventLoadNode2::QueueStatus_Local)
				{
					LocalQueue.Reprioritize(&Node, Node.Package->Desc.Priority);
				}
				else if (Node.QueueStatus == FEventLoadNode2::QueueStatus_External)
				{
					ExternalQueue.Reprioritize(&Node, Node.Package->Desc.Priority);
				}
			}
		};

	for (FEventLoadNode2& Node : Package->PackageNodes)
	{
		ReprioritizeNode(Node);
	}
	UpdateExternalQueueState();
}

FUObjectSerializeContext* FAsyncPackage2::GetSerializeContext()
{
	return FUObjectThreadContext::Get().GetSerializeContext();
}

void FAsyncPackage2::SetupScriptDependencies()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SetupScriptDependencies);

	// UObjectLoadAllCompiledInDefaultProperties is creating CDOs from a flat list.
	// During initial laod, if a CDO called LoadObject for this package it may depend on other CDOs later in the list.
	// Then collect them here, and wait for them to be created before allowing this package to proceed.
	TArray<UClass*, TInlineAllocator<8>> UnresolvedCDOs;
	ImportStore.GetUnresolvedCDOs(HeaderData, UnresolvedCDOs);
#if WITH_EDITOR
	if (OptionalSegmentHeaderData.IsSet())
	{
		ImportStore.GetUnresolvedCDOs(*OptionalSegmentHeaderData, UnresolvedCDOs);
	}
#endif
	if (!UnresolvedCDOs.IsEmpty())
	{
		AsyncLoadingThread.AddPendingCDOs(this, UnresolvedCDOs);
	}
}

void FGlobalImportStore::FindAllScriptObjects(bool bVerifyOnly)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGlobalImportStore::FindAllScriptObjects);

	TStringBuilder<FName::StringBufferSize> Name;
	TArray<UObject*> Objects;
	FRuntimeScriptPackages ScriptPackages;
	FindAllRuntimeScriptPackages(ScriptPackages);

	auto FindAllScriptObjectsInPackages = [this, bVerifyOnly, &Name, &Objects](const TArray<UPackage*>& InPackages, bool bIsVerseVNIPackage)
		{
			for (UPackage* Package : InPackages)
			{
#if WITH_EDITOR
				Name.Reset();
				Package->GetPathName(nullptr, Name);
				FPackageObjectIndex PackageGlobalImportIndex = FPackageObjectIndex::FromScriptPath(Name);

				if (bVerifyOnly)
				{
					if (!ScriptObjects.Contains(PackageGlobalImportIndex))
					{
						UE_LOGF(LogStreaming, Display, "Script package %ls (0x%016llX) is missing a NotifyRegistrationEvent from the initial load phase.",
							*Package->GetFullName(),
							PackageGlobalImportIndex.Value());
					}
				}
				else
				{
					ScriptObjects.Add(PackageGlobalImportIndex, Package);
				}
#endif
				Objects.Reset();
				GetObjectsWithOuter(Package, Objects, EGetObjectsFlags::IncludeNestedObjects);
				for (UObject* Object : Objects)
				{
					if (Object->HasAnyFlags(RF_Public))
					{
						// Unlike things in /Scripts/, with Verse VNI objects, there is a mix of UHT generated types, which will always be 
						// available, and Verse compiler generated types which need to be cooked and packaged.  We don't want the compiler
						// generated types to be included in this collection.
						if (bIsVerseVNIPackage && !Verse::VerseVM::IsUHTGeneratedVerseVNIObject(Object))
						{
							continue;
						}
						Name.Reset();
						Object->GetPathName(nullptr, Name);
						FPackageObjectIndex GlobalImportIndex = FPackageObjectIndex::FromScriptPath(Name);
						if (bVerifyOnly)
						{
							if (!ScriptObjects.Contains(GlobalImportIndex))
							{
								UE_LOGF(LogStreaming, Warning, "Script object %ls (0x%016llX) is missing a NotifyRegistrationEvent from the initial load phase.",
									*Object->GetFullName(),
									GlobalImportIndex.Value());
							}
						}
						else
						{
							ScriptObjects.Add(GlobalImportIndex, Object);
						}
					}
				}
			}
		};

	FindAllScriptObjectsInPackages(ScriptPackages.Script, false);
	FindAllScriptObjectsInPackages(ScriptPackages.VerseVNI, true);
}

#if ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING

FCookedScriptObjectsDebug::FCookedScriptObjectsDebug()
{
	if (FPlatformProperties::RequiresCookedData())
	{
		// register to sync load all the debug data just before we start loading the CDOs for the first module
		// this is too early to check for GEnableScriptImportsDebugging if it has been set from the command line
		FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.AddRaw(this, &FCookedScriptObjectsDebug::OnCompiledInUObjectsRegistered);
	}
}

FCookedScriptObjectsDebug::~FCookedScriptObjectsDebug()
{
	FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.RemoveAll(this);
}

void FCookedScriptObjectsDebug::OnCompiledInUObjectsRegistered(FName, ECompiledInUObjectsRegisteredStatus Status)
{
	if (Status == ECompiledInUObjectsRegisteredStatus::PreCDO)
	{
		if (GEnableScriptImportsDebugging)
		{
			LoadDebugData();
		}
		FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.RemoveAll(this);
	}
}

void FCookedScriptObjectsDebug::LoadDebugData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FCookedScriptObjectsDebug::LoadDebugData);

	FIoDispatcher& IoDispatcher = FIoDispatcher::Get();
	FIoChunkId ChunkId = CreateIoChunkId(0, 0, EIoChunkType::ScriptObjects);
	// Do a quick sync check if we are running in a valid scenario (cooked client or server) where we actually have the script objects chunk.
	// If not, early out since the iodispatcher may not even be fully initialized yet and the read may never complete.
	if (!IoDispatcher.DoesChunkExist(ChunkId))
	{
		return;
	}

	FIoBatch IoBatch = IoDispatcher.NewBatch();
	FIoRequest Request = IoBatch.Read(ChunkId, FIoReadOptions(), IoDispatcherPriority_Max);
	FEventRef Event;
	IoBatch.IssueAndTriggerEvent(Event.Get());
	Event->Wait();

	if (const FIoBuffer* Buffer = Request.GetResult())
	{
		FLargeMemoryReader Ar(Buffer->Data(), Buffer->DataSize());
		TArray<FDisplayNameEntryId> NameMap = LoadNameBatch(Ar);
		int32 NumObjects = 0;
		Ar << NumObjects;
		Objects.Reserve(NumObjects);
		TConstArrayView<FScriptObjectEntry> Entries =
			MakeConstArrayView(reinterpret_cast<const FScriptObjectEntry*>(Buffer->Data() + Ar.Tell()), NumObjects);
		for (const FScriptObjectEntry& Entry : Entries)
		{
			FName Name = NameMap[Entry.Mapped.GetIndex()].ToName(Entry.Mapped.GetNumber());
			Objects.Emplace(Entry.GlobalIndex, { FMinimalName(Name), Entry.OuterIndex });
		}
		UE_LOGF(LogStreaming, Log, "AsyncLoading2 - LoadDebugData: Loaded %d cooked script object entries (%.2f KB)",
			Objects.Num(), (float)Objects.GetAllocatedSize() / 1024.f);
	}
}

void FCookedScriptObjectsDebug::TrimDebugData(const TAsyncLoadingStripedMap<4, FPackageObjectIndex, UObject*>& RuntimeScriptObjects)
{
	if (!IsEnabled())
	{
		return;
	}
	TRACE_CPUPROFILER_EVENT_SCOPE(FCookedScriptObjectsDebug::TrimDebugData);
	// Remove debug data for all cooked script objects that are available in the runtime,
	// just keep the missing ones and their outers.
	const int32 Num = Objects.Num();
	TBitArray<> Remove(false, Num);
	TBitArray<> Visited(false, Num);

	int32 NumRemaining = Num;
	TFunction<void(int32)> KeepOuters = [this, &KeepOuters, &Remove, &Visited, &NumRemaining](int32 I)
		{
			if (!Visited[I])
			{
				Visited[I] = true;
				if (Remove[I])
				{
					++NumRemaining;
					Remove[I] = false;
				}
				const TPair<FMinimalName, FPackageObjectIndex>& Entry = Objects.Get(FSetElementId::FromInteger(I)).Value;
				if (!Entry.Value.IsNull())
				{
					FSetElementId OuterId = Objects.FindId(Entry.Value);
					KeepOuters(OuterId.AsInteger());
				}
			}
		};
	// 1. Mark all runtime objects to be removed
	RuntimeScriptObjects.ForEach(
		[this, &Remove, &NumRemaining](const TPair<FPackageObjectIndex, UObject*>& Runtime)
		{
			FSetElementId Id = Objects.FindId(Runtime.Key);
			if (Id.IsValidId())
			{
				Remove[Id.AsInteger()] = true;
				--NumRemaining;
			}
		}
	);
	
	// 2. Keep the outers for all objects that are kept
	for (int I = 0; I < Num; ++I)
	{
		if (!Remove[I])
		{
			KeepOuters(I);
		}
	}

	// 3. Remove the remaining runtime objects
	if (NumRemaining != Num)
	{
		TMap<FPackageObjectIndex, TPair<FMinimalName, FPackageObjectIndex>> RemainingObjects;
		RemainingObjects.Reserve(NumRemaining);

		for (int I = 0; I < Num; ++I)
		{
			if (!Remove[I])
			{
				RemainingObjects.Add(Objects.Get(FSetElementId::FromInteger(I)));
			}
		}

		check(RemainingObjects.Num() == NumRemaining);

		Swap(RemainingObjects, Objects);
	}

	UE_LOGF(LogStreaming, Log, "AsyncLoading2 - TrimDebugData: Kept %d/%d cooked script object entries (%.2f KB)",
		NumRemaining, Num, (float)Objects.GetAllocatedSize() / 1024.f);
}

void FCookedScriptObjectsDebug::GetPathNameInternal(const TPair<FMinimalName, FPackageObjectIndex>& Entry, FStringBuilderBase& PathName, int32& NumParts) const
{
	const TPair<FMinimalName, FPackageObjectIndex>* Outer = Objects.Find(Entry.Value);
	if (Outer)
	{
		GetPathNameInternal(*Outer, PathName, NumParts);
		// Use ? as divider for longer paths where we don't know if ':' or '.' should be used.
		if (NumParts == 1)
		{
			PathName.AppendChar('.');
		}
		else if (NumParts == 2)
		{
			PathName.AppendChar(SUBOBJECT_DELIMITER_CHAR);
		}
		else if (NumParts > 2)
		{
			// we don't know if this should be a dot or a colon separator
			PathName.AppendChar('?');
		}
	}
	FName(Entry.Key).AppendString(PathName);
	++NumParts;
}

FString FCookedScriptObjectsDebug::GetPathName(FPackageObjectIndex Index) const
{
	const TPair<FMinimalName, FPackageObjectIndex>* Entry = Objects.Find(Index);
	if (Entry)
	{
		TStringBuilder<256> PathName;
		int32 NumParts = 0;
		GetPathNameInternal(*Entry, PathName, NumParts);
		return PathName.ToString();
	}
	return FString();
}

#endif // ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING 

void FGlobalImportStore::RegistrationComplete()
{
#if WITH_EDITOR
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindAllScriptObjects);
		FindAllScriptObjects(/*bVerifyOnly*/false);
	}
#elif DO_CHECK
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindAllScriptObjectsVerify);
		FindAllScriptObjects(/*bVerifyOnly*/true);
	}
#endif

	ScriptObjects.Shrink();
#if ALT2_ENABLE_SCRIPT_IMPORTS_DEBUGGING
	CookedScriptObjectsDebug.TrimDebugData(ScriptObjects);
#endif
}

void FAsyncPackage2::ImportPackagesRecursiveInner(FAsyncLoadingThreadState2& ThreadState, FIoBatch& IoBatch, FPackageStore& PackageStore, FAsyncPackageHeaderData& Header)
{
	const TArrayView<FPackageId>& ImportedPackageIds = Header.ImportedPackageIds;
	const int32 ImportedPackageCount = ImportedPackageIds.Num();
	if (!ImportedPackageCount)
	{
		return;
	}
	TArray<FName>& ImportedPackageNames = Header.ImportedPackageNames;
	bool bHasImportedPackageNames = !ImportedPackageNames.IsEmpty();
	check(ImportedPackageNames.Num() == 0 || ImportedPackageNames.Num() == ImportedPackageCount);
	for (int32 LocalImportedPackageIndex = 0; LocalImportedPackageIndex < ImportedPackageCount; ++LocalImportedPackageIndex)
	{
		FPackageId ImportedPackageId = ImportedPackageIds[LocalImportedPackageIndex];
		EPackageStoreEntryStatus ImportedPackageStatus = EPackageStoreEntryStatus::Missing;
		EPackageLoader ImportedPackageLoader = EPackageLoader::Unknown;
		FPackageStoreEntry ImportedPackageEntry;
		FName ImportedPackageUPackageName = bHasImportedPackageNames ? ImportedPackageNames[LocalImportedPackageIndex] : NAME_None;
		FName ImportedPackageNameToLoad = ImportedPackageUPackageName;
		FPackageId ImportedPackageIdToLoad = ImportedPackageId;

		if (GAllowPackageRedirectorSupport)
		{
			if (!ImportedPackageNameToLoad.IsNone())
			{
				// Redirects should redirect both the UPackage name as well as the package name to load
				const FName NewPackageNameToLoad = ApplyPackageNameRedirections(ImportedPackageNameToLoad);
				if (ImportedPackageNameToLoad != NewPackageNameToLoad)
				{
					ImportedPackageNameToLoad = NewPackageNameToLoad;
					ImportedPackageIdToLoad = FPackageId::FromName(ImportedPackageNameToLoad);
					ImportedPackageUPackageName = ImportedPackageNameToLoad;
					ImportedPackageId = ImportedPackageIdToLoad;
					// Rewrite the import table
					ImportedPackageIds[LocalImportedPackageIndex] = ImportedPackageId;
					ImportedPackageNames[LocalImportedPackageIndex] = ImportedPackageUPackageName;
				}
			}
		}

		bool bIsInstanced = false;
		if (bHasImportedPackageNames && LinkerLoadState.IsSet())
		{
			// Use the instancing context remap from disk package name to upackage name
			const FLinkerInstancingContext& InstancingContext = LinkerLoadState->Linker->GetInstancingContext();
			ImportedPackageUPackageName = InstancingContext.RemapPackage(ImportedPackageNameToLoad);
			if (ImportedPackageUPackageName != ImportedPackageNameToLoad)
			{
				bIsInstanced = true;
				if (ImportedPackageUPackageName.IsNone())
				{
					ImportedPackageIdToLoad = FPackageId::FromName(NAME_None);
					ImportedPackageNameToLoad = NAME_None;
				}
				ImportedPackageId = FPackageId::FromName(ImportedPackageUPackageName);
				// Rewrite the import table
				ImportedPackageIds[LocalImportedPackageIndex] = ImportedPackageId;
				ImportedPackageNames[LocalImportedPackageIndex] = ImportedPackageUPackageName;
			}
		}
		else
		{
			// Use the instancing context remap from disk package name to upackage name
			const FLinkerInstancingContext& InstancingContext = Desc.InstancingContext;

			FPackageId RemappedImportedPackageId = InstancingContext.RemapPackageId(ImportedPackageId);

			if (RemappedImportedPackageId != ImportedPackageId)
			{
				bIsInstanced = true;
				ImportedPackageId = RemappedImportedPackageId;
				// Rewrite the import table
				ImportedPackageIds[LocalImportedPackageIndex] = ImportedPackageId;
			}
		}

		// Note, localization should apply after remapping from instancing has occurred
		{
			FName LocalizedPackageNameToLoad = ApplyLooseFileLocalizationPackageNameRedirects(ImportedPackageNameToLoad);
			if (LocalizedPackageNameToLoad != ImportedPackageNameToLoad)
			{
				// Localization should only update the package name to load
				ImportedPackageNameToLoad = LocalizedPackageNameToLoad;
				ImportedPackageIdToLoad = FPackageId::FromName(ImportedPackageNameToLoad);
			}
		}

		{
			FName SourcePackageName;
			FPackageId RedirectedToPackageId;
			if (PackageStore.GetPackageRedirectInfo(ImportedPackageIdToLoad, SourcePackageName, RedirectedToPackageId))
			{
				if (ImportedPackageUPackageName.IsNone())
				{
					ImportedPackageUPackageName = SourcePackageName;
				}
				ImportedPackageIdToLoad = RedirectedToPackageId;
				ImportedPackageNameToLoad = NAME_None;
			}
		}

		FPackagePath ImportedPackagePath;
		EPackageLoader PackageLoader = EPackageLoader::Zen;
		EPackageExtension ImportedPackageExtension = EPackageExtension::Unspecified;

		FPackagePath::TryFromPackageName(ImportedPackageNameToLoad, ImportedPackagePath);
		PackageLoader = LinkerLoadState.IsSet() ? EPackageLoader::LinkerLoad : PackageLoader;

		if (!ImportedPackageNameToLoad.IsNone() && AsyncLoadingThread.TryGetExistingLoadedPackagePath(ImportedPackageId, ImportedPackageNameToLoad, ImportedPackageExtension, ImportedPackageLoader, ImportedPackageEntry))
		{
			ImportedPackageStatus = EPackageStoreEntryStatus::Ok;
		}
		else
		{
			ImportedPackageStatus = AsyncLoadingThread.TryGetPackageInfoFromPackageStore(ImportedPackageIdToLoad, ImportedPackageNameToLoad, ImportedPackageUPackageName, ImportedPackageEntry);
			if (ImportedPackageStatus != EPackageStoreEntryStatus::Missing)
			{
				ImportedPackageExtension = ImportedPackageEntry.PackageExtension;
				ImportedPackageLoader = ImportedPackageEntry.LoaderType;
			}
			else
			{
				ImportedPackageLoader = EPackageLoader::Unknown;
			}
		}
		ImportedPackagePath.SetHeaderExtension(ImportedPackageExtension);

		FAsyncPackage2* ImportedPackage = nullptr;
		bool bInserted = false;
		bool bIsFullyLoaded = false;
		{
			// Add the imported package reference and confirm that its loader and extension match what was found from the package store or import store directly, 
			// and if not, ensure the package was marked as missing or not installed
			FWriteLockedPackageRef ImportedPackageRef = ImportStore.AddImportedPackageReference(ImportedPackageId, ImportedPackageUPackageName, ImportedPackageLoader, ImportedPackagePath.GetHeaderExtension());
			checkf((ImportedPackageStatus == EPackageStoreEntryStatus::Missing || ImportedPackageStatus == EPackageStoreEntryStatus::NotInstalled)
				|| ImportedPackageRef->GetPackageLoader() == ImportedPackageLoader,
				TEXT("ImportPackageStatus=%s, ImportedPackageRef.GetPackageLoader()=%s, ImportedPackageLoader=%s"),
				LexToString(ImportedPackageStatus), LexToString(ImportedPackageRef->GetPackageLoader()), LexToString(ImportedPackageLoader));
			checkf((ImportedPackageStatus == EPackageStoreEntryStatus::Missing || ImportedPackageStatus == EPackageStoreEntryStatus::NotInstalled)
				|| ImportedPackageRef->GetPackageHeaderExtension() == ImportedPackagePath.GetHeaderExtension(),
				TEXT("ImportPackageStatus=%s, ImportedPackageRef.GetPackageHeaderExtension()=%s, ImportedPackagePath.GetHeaderExtension()=%s"),
				LexToString(ImportedPackageStatus), LexToString(ImportedPackageRef->GetPackageHeaderExtension()), LexToString(ImportedPackagePath.GetHeaderExtension()));
			
			bIsFullyLoaded = ImportedPackageRef->AreAllPublicExportsLoaded();

			if (ImportedPackageLoader == EPackageLoader::LinkerLoad && (!ImportedPackageRef->HasPackage() || (!ImportedPackageRef->GetPackage()->GetLinker() && !ImportedPackageRef->GetPackage()->HasAnyPackageFlags(PKG_InMemoryOnly))))
			{
				// If we're importing a linker load package and it doesn't have its linker we need to reload it, otherwise we can't reliably link to its imports
				// Note: Legacy loader path appears to do this only for uncooked packages in the editor?
				bIsFullyLoaded = false;
			}

			const bool bIsNotInstalled = ImportedPackageStatus == EPackageStoreEntryStatus::NotInstalled;
			const bool bIsMissing = ImportedPackageStatus == EPackageStoreEntryStatus::Missing;
			if (bIsFullyLoaded)
			{
				ImportedPackage = AsyncLoadingThread.GetAsyncPackage(ImportedPackageId);
				if (!ImportedPackage)
				{
					continue;
				}
				bInserted = false;
			}
			else if (bIsNotInstalled)
			{
				UE_ASYNC_PACKAGE_LOG(Log, Desc, TEXT("ImportPackages: SkipPackage"),
					TEXT("Skipping imported package %s (0x%s) because it or one of its dependencies is not installed"), *ImportedPackageNameToLoad.ToString(), *LexToString(ImportedPackageId));
				ImportedPackageRef->SetIsNotInstalledPackage();
				continue;
			}
			else if (bIsMissing)
			{
				if (!ImportedPackageRef->HasPackage()) // If we found a package it's not actually missing but we can't load it anyway
				{
					UE_ASYNC_PACKAGE_LOG(Log, Desc, TEXT("ImportPackages: SkipPackage"),
						TEXT("Skipping non mounted imported package %s (0x%s)"), *ImportedPackageNameToLoad.ToString(), *LexToString(ImportedPackageId));
					if (!FLinkerLoad::IsKnownMissingPackage(ImportedPackageNameToLoad))
					{
						FName PackagePathToLoadName = Desc.PackagePathToLoad.GetPackageFName();
						ExecuteOnGameThread(TEXT("GetExplanationForUnavailablePackage"), [ImportedPackageNameToLoad, ImportedPackageId, PackagePathToLoadName]()
						{
							FString ImportedPackageNameToLoadString = ImportedPackageNameToLoad.ToString();
							FString PackagePathToLoadString = PackagePathToLoadName.ToString();
							TStringBuilder<2048> Explanation;
							FPackageName::GetExplanationForUnavailablePackage(ImportedPackageNameToLoad, Explanation);
							if (Explanation.Len())
							{
								FMessageLog("LoadErrors").Info(FText::FormatNamed(NSLOCTEXT("Core", "AsyncLoading_SkippedPackage_Explanation", "While trying to load package {MissingPackage}, a dependent package {DependentPackage} ({DependentPackageId}) was not available. Additional explanatory information follows:\n{Explanation}"),
									TEXT("MissingPackage"), FText::FromString(PackagePathToLoadString),
									TEXT("DependentPackage"), FText::FromString(ImportedPackageNameToLoadString),
									TEXT("DependentPackageId"), FText::FromString(LexToString(ImportedPackageId)),
									TEXT("Explanation"), FText::FromString(Explanation.ToString())));
							}
							else
							{
								FMessageLog("LoadErrors").Info(FText::FormatNamed(NSLOCTEXT("Core", "AsyncLoading_SkippedPackage_NoExplanation", "While trying to load package {MissingPackage}, a dependent package {DependentPackage} ({DependentPackageId}) was not available. No additional explanation was available."),
									TEXT("MissingPackage"), FText::FromString(PackagePathToLoadString),
									TEXT("DependentPackage"), FText::FromString(ImportedPackageNameToLoadString),
									TEXT("DependentPackageId"), FText::FromString(LexToString(ImportedPackageId))));
							}

							return false;
						});
					}
					ImportedPackageRef->SetIsMissingPackage();
				}
				continue;
			}
			else
			{
				// Release this to avoid deadlocks
				ImportedPackageRef.Reset();
				FAsyncPackageDesc2 PackageDesc = FAsyncPackageDesc2::FromPackageImport(Desc, ImportedPackageUPackageName, ImportedPackageId, ImportedPackageIdToLoad, MoveTemp(ImportedPackagePath), ImportedPackageLoader);
				ImportedPackage = AsyncLoadingThread.FindOrInsertPackage(ThreadState, PackageDesc, bInserted, this);
			}
		}

		checkf(ImportedPackage, TEXT("Failed to find or insert imported package with id '%s'"), *FormatPackageId(ImportedPackageId));
		TRACE_LOADTIME_ASYNC_PACKAGE_IMPORT_DEPENDENCY(this, ImportedPackage);

		if (bInserted)
		{
			UE_ASYNC_PACKAGE_LOG(Verbose, Desc, TEXT("ImportPackages: AddPackage"),
			TEXT("Start loading imported package with id '%s'"), *FormatPackageId(ImportedPackageId));
			++AsyncLoadingThread.PackagesWithRemainingWorkCounter;
			TRACE_COUNTER_SET(AsyncLoadingPackagesWithRemainingWork, AsyncLoadingThread.PackagesWithRemainingWorkCounter);

			// This should never fail since we just did the insert.
			ImportedPackage->AddRef();
		}
		else
		{
			UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("ImportPackages: UpdatePackage"),
				TEXT("Imported package with id '%s' is already being loaded."), *FormatPackageId(ImportedPackageId));

			// When using ALT, we could end up with a package that is finishing its loading on the GT and the last ref released.
			// In such a case, we need to avoid taking a ref otherwise we would end up with a use-after-free.
			if (!ImportedPackage->TryAddRef())
			{
				ensureMsgf(bIsFullyLoaded, TEXT("Found a package being destructed that is not marked as fully loaded"));
				continue;
			}
		}

		Header.ImportedAsyncPackagesView[LocalImportedPackageIndex] = ImportedPackage;

		if (PackageLoader != ImportedPackageLoader)
		{
			UE_ASYNC_PACKAGE_LOG(VeryVerbose, Desc, TEXT("ImportPackages: AddDependency"),
				TEXT("Adding package dependency to %s import '%s'."), LexToString(ImportedPackageLoader), *ImportedPackage->Desc.UPackageName.ToString());

			// When importing a linker load package from a zen package or vice versa we need to wait for all the exports in the imported package to be created
			// and serialized before we can start processing our own exports
			GetPackageNode(Package_DependenciesReady).DependsOn(&ImportedPackage->GetPackageNode(Package_ExportsSerialized));
		}

		if (bInserted)
		{
			if (ImportedPackageStatus == EPackageStoreEntryStatus::Pending)
			{
				AsyncLoadingThread.PendingPackages.Add(ImportedPackage);
			}
			else
			{
				check(ImportedPackageStatus == EPackageStoreEntryStatus::Ok);

				if (ImportedPackageLoader == EPackageLoader::LinkerLoad)
				{
					// Only propagate the instancing context if the imported package is also instanced
					ImportedPackage->InitializeLinkerLoadState(bIsInstanced ? &LinkerLoadState->Linker->GetInstancingContext() : nullptr);
				}
				else
				{
					if (bIsInstanced)
					{
						ImportedPackage->Desc.InstancingContext = Desc.InstancingContext;
					}
					AsyncLoadingThread.InitializeAsyncPackageFromPackageStore(ThreadState, &IoBatch, ImportedPackage, ImportedPackageEntry);
				}
				ImportedPackage->StartLoading(ThreadState, IoBatch);
			}
		}
	}
}

void FAsyncPackage2::ImportPackagesRecursive(FAsyncLoadingThreadState2& ThreadState, FIoBatch& IoBatch, FPackageStore& PackageStore)
{
	if (bHasStartedImportingPackages)
	{
		return;
	}
	bHasStartedImportingPackages = true;

	// No additional imports should be present when ImportPackagesRecursive is called since this should only be called during
	// the early phases of loading.
	check(AdditionalImportedAsyncPackages.IsEmpty());

	if (Data.ImportedAsyncPackages.IsEmpty())
	{
		return;
	}

	ImportPackagesRecursiveInner(ThreadState, IoBatch, PackageStore, HeaderData);
#if WITH_EDITOR
	if (OptionalSegmentHeaderData.IsSet())
	{
		ImportPackagesRecursiveInner(ThreadState, IoBatch, PackageStore, OptionalSegmentHeaderData.GetValue());
	}
#endif

	if (SyncLoadContextId)
	{
		for (FAsyncPackage2* ImportedPackage : Data.ImportedAsyncPackages)
		{
			if (ImportedPackage)
			{
				AsyncLoadingThread.IncludePackageInSyncLoadContextRecursive(ThreadState, SyncLoadContextId, ImportedPackage);
			}
		}

		for (FAsyncPackage2* ImportedPackage : AdditionalImportedAsyncPackages)
		{
			AsyncLoadingThread.IncludePackageInSyncLoadContextRecursive(ThreadState, SyncLoadContextId, ImportedPackage);
		}
	}

	UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("ImportPackages: ImportsDone"),
		TEXT("All imported packages are now being loaded."));
}

void FAsyncPackage2::InitializeLinkerLoadState(const FLinkerInstancingContext* InstancingContext)
{
	LinkerLoadState.Emplace();
	CreateUPackage();
	CreateLinker(InstancingContext);
}

void FAsyncPackage2::CreateLinker(const FLinkerInstancingContext* InstancingContext)
{
	uint32 LinkerFlags = (LOAD_Async | LOAD_NoVerify | LOAD_SkipLoadImportedPackages);
#if WITH_EDITOR
	LinkerFlags |= Desc.LoadFlags;
	if ((Desc.PackageFlags & PKG_PlayInEditor) != 0 && (GIsEditor || !FApp::IsGame()))
	{
		LinkerFlags |= LOAD_PackageForPIE;
	}
#endif
	FLinkerLoad* Linker = FLinkerLoad::FindExistingLinkerForPackage(LinkerRoot);
	if (!Linker)
	{
		FUObjectSerializeContext* LoadContext = GetSerializeContext();
#if ALT2_ENABLE_NEW_ARCHIVE_FOR_LINKERLOAD
		Linker = new FLinkerLoad(LinkerRoot, Desc.PackagePathToLoad, LinkerFlags, InstancingContext ? *InstancingContext : FLinkerInstancingContext());
		LinkerRoot->SetLinker(Linker);
		FLinkerLoadArchive2* Loader = new FLinkerLoadArchive2(Desc.PackagePathToLoad);
		Linker->SetLoader(Loader, Loader->NeedsEngineVersionChecks());
#else
		Linker = FLinkerLoad::CreateLinkerAsync(LoadContext, LinkerRoot, Desc.PackagePathToLoad, LinkerFlags, InstancingContext, TFunction<void()>([]() {}));
#endif
	}
	else
	{
		Linker->LoadFlags |= LinkerFlags;
	}
	check(Linker);
	check(Linker->LinkerRoot == LinkerRoot);
	check(!Linker->AsyncRoot);
	TRACE_LOADTIME_ASYNC_PACKAGE_LINKER_ASSOCIATION(this, Linker);
	Linker->AsyncRoot = this;
	LinkerLoadState->Linker = Linker;
#if ALT2_ENABLE_NEW_ARCHIVE_FOR_LINKERLOAD
	Linker->ResetStatusInfo();
#endif
}

void FAsyncPackage2::DetachLinker()
{
	if (LinkerLoadState.IsSet() && LinkerLoadState->Linker)
	{
		// We're no longer keeping the imports alive so clear them from the linker
		for (FObjectImport& ObjectImport : LinkerLoadState->Linker->ImportMap)
		{
			ObjectImport.XObject = nullptr;
			ObjectImport.SourceLinker = nullptr;
			ObjectImport.SourceIndex = INDEX_NONE;
		}
		check(LinkerLoadState->Linker->AsyncRoot == this);
		LinkerLoadState->Linker->AsyncRoot = nullptr;
		LinkerLoadState->Linker = nullptr;
	}
}

namespace UE::AsyncPackage2
{
EAsyncLoadingResult::Type AsyncLoadingResultFromIoError(EIoErrorCode Error)
{
	switch (Error)
	{
	case EIoErrorCode::Ok:
	return EAsyncLoadingResult::Succeeded;
	case EIoErrorCode::NotInstalled:
	return EAsyncLoadingResult::FailedNotInstalled;
	}

	return EAsyncLoadingResult::Failed;
}
}

void FAsyncPackage2::StartLoading(FAsyncLoadingThreadState2& ThreadState, FIoBatch& IoBatch)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(StartLoading);
	UE_TRACE_METADATA_SCOPE_PACKAGE_ID(Desc.UPackageId);
	UE_TRACE_PACKAGE_NAME(Desc.UPackageId, Desc.UPackageName);

	LoadStartTime = FPlatformTime::Seconds();

	AsyncPackageLoadingState = EAsyncPackageLoadingState2::WaitingForIo;

	// CallProgressCallbacks will read AsyncPackageLoadingState and dispatch progress so it needs
	// to be called after we set the state to WaitingForIo to have an effect.
	CallProgressCallbacks();

	FIoReadOptions ReadOptions;
	if (LinkerLoadState.IsSet())
	{
#if ALT2_ENABLE_NEW_ARCHIVE_FOR_LINKERLOAD
		static_cast<FLinkerLoadArchive2*>(LinkerLoadState->Linker->GetLoader())->BeginRead(&GetPackageNode(EEventLoadNode2::Package_ProcessSummary));
#else
		GetPackageNode(EEventLoadNode2::Package_ProcessSummary).ReleaseBarrier(&ThreadState);
#endif
		return;
	}

#if WITH_EDITOR
	if (OptionalSegmentHeaderData.IsSet())
	{
		int32 LocalPendingIoRequestsCounter = AsyncLoadingThread.PendingIoRequestsCounter.fetch_add(1) + 1;
		TRACE_COUNTER_SET(AsyncLoadingPendingIoRequests, LocalPendingIoRequestsCounter);

		GetPackageNode(EEventLoadNode2::Package_ProcessSummary).AddBarrier();
		OptionalSegmentSerializationState->IoRequest = IoBatch.ReadWithCallback(CreateIoChunkId(Desc.PackageIdToLoad.Value(), 1, EIoChunkType::ExportBundleData),
			ReadOptions,
			Desc.Priority,
			[this](TIoStatusOr<FIoBuffer> Result)
			{
				if (!Result.IsOk())
				{
					UE_ASYNC_PACKAGE_LOG(Warning, Desc, TEXT("StartBundleIoRequests: FailedRead"),
						TEXT("Failed reading optional chunk for package: %s"), *Result.Status().ToString());
					LoadStatus = UE::AsyncPackage2::AsyncLoadingResultFromIoError(Result.Status().GetErrorCode());
				}
				int32 LocalPendingIoRequestsCounter = AsyncLoadingThread.PendingIoRequestsCounter.fetch_sub(1) - 1;
				TRACE_COUNTER_SET(AsyncLoadingPendingIoRequests, LocalPendingIoRequestsCounter);
				FAsyncLoadingThread2& LocalAsyncLoadingThread = AsyncLoadingThread;
				GetPackageNode(EEventLoadNode2::Package_ProcessSummary).ReleaseBarrier(nullptr);
				if (LocalPendingIoRequestsCounter == 0)
				{
					LocalAsyncLoadingThread.AltZenaphore.NotifyOne();
				}
			});
	}
#endif

	int32 LocalPendingIoRequestsCounter = AsyncLoadingThread.PendingIoRequestsCounter.fetch_add(1) + 1;
	TRACE_COUNTER_SET(AsyncLoadingPendingIoRequests, LocalPendingIoRequestsCounter);


#if WITH_EDITOR
	const uint16 ChunkIndex = bRequestOptionalChunk ? 1 : 0;
#else
	const uint16 ChunkIndex = 0;
#endif
	FIoChunkId ChunkId = CreateIoChunkId(Desc.PackageIdToLoad.Value(), ChunkIndex, EIoChunkType::ExportBundleData);
	SerializationState.IoRequest = IoBatch.ReadWithCallback(ChunkId,
		ReadOptions,
		Desc.Priority,
		[this](TIoStatusOr<FIoBuffer> Result)
		{
			if (Result.IsOk())
			{
				TRACE_COUNTER_ADD(AsyncLoadingTotalLoaded, Result.ValueOrDie().DataSize());
				CSV_CUSTOM_STAT_DEFINED(FrameCompletedExportBundleLoadsKB, float((double)Result.ValueOrDie().DataSize() / 1024.0), ECsvCustomStatOp::Accumulate);
			}
			else
			{
				UE_ASYNC_PACKAGE_LOG(Warning, Desc, TEXT("StartBundleIoRequests: FailedRead"),
					TEXT("Failed reading chunk for package: %s"), *Result.Status().ToString());
				LoadStatus = UE::AsyncPackage2::AsyncLoadingResultFromIoError(Result.Status().GetErrorCode());
			}
			int32 LocalPendingIoRequestsCounter = AsyncLoadingThread.PendingIoRequestsCounter.fetch_sub(1) - 1;
			TRACE_COUNTER_SET(AsyncLoadingPendingIoRequests, LocalPendingIoRequestsCounter);
			FAsyncLoadingThread2& LocalAsyncLoadingThread = AsyncLoadingThread;
			GetPackageNode(EEventLoadNode2::Package_ProcessSummary).ReleaseBarrier(nullptr);
			if (LocalPendingIoRequestsCounter == 0)
			{
				LocalAsyncLoadingThread.AltZenaphore.NotifyOne();
			}
		});

	if (!Data.ShaderMapHashes.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(StartShaderMapRequests);
		auto ReadShaderMapFunc = [this, &IoBatch](const FIoChunkId& ChunkId, FGraphEventRef GraphEvent)
			{
#if USE_MMAPPED_SHADERARCHIVE
				FIoDispatcher& IoDispatcher = FIoDispatcher::Get();
				TIoStatusOr<FIoMappedRegion> MappedRegionStatus = IoDispatcher.OpenMapped(ChunkId, FIoReadOptions());
				if (MappedRegionStatus.IsOk())
				{
					GraphEvent->DispatchSubsequents();
					GraphEvent.SafeRelease();
					return FCoreDelegates::FShaderReadRequestResult(FIoRequest(), MoveTemp(MappedRegionStatus));
				}
#endif
				GetPackageNode(Package_ExportsSerialized).AddBarrier();
				int32 LocalPendingIoRequestsCounter = AsyncLoadingThread.PendingIoRequestsCounter.fetch_add(1) + 1;
				TRACE_COUNTER_SET(AsyncLoadingPendingIoRequests, LocalPendingIoRequestsCounter);
#if USE_MMAPPED_SHADERARCHIVE
				return FCoreDelegates::FShaderReadRequestResult(
#else
				return (// Avoid calling FIoRequest(FIoRequest...
#endif
				IoBatch.ReadWithCallback(ChunkId, FIoReadOptions(), Desc.Priority,
					[this, GraphEvent](TIoStatusOr<FIoBuffer> Result)
					{
						GraphEvent->DispatchSubsequents();
						int32 LocalPendingIoRequestsCounter = AsyncLoadingThread.PendingIoRequestsCounter.fetch_sub(1) - 1;
						TRACE_COUNTER_SET(AsyncLoadingPendingIoRequests, LocalPendingIoRequestsCounter);
						FAsyncLoadingThread2& LocalAsyncLoadingThread = AsyncLoadingThread;
						GetPackageNode(Package_ExportsSerialized).ReleaseBarrier(nullptr);
						if (LocalPendingIoRequestsCounter == 0)
						{
							LocalAsyncLoadingThread.AltZenaphore.NotifyOne();
						}
					})
#if USE_MMAPPED_SHADERARCHIVE
					, MoveTemp(MappedRegionStatus)
#endif
				);
			};
		FCoreDelegates::PreloadPackageShaderMaps.ExecuteIfBound(Data.ShaderMapHashes, ReadShaderMapFunc);
	}
}

EEventLoadNodeExecutionResult FAsyncPackage2::ProcessLinkerLoadPackageSummary(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessLinkerLoadPackageSummary);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(LinkerLoadState->Linker->GetPackagePath().GetPackageFName(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(UPackage::StaticClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, UPackage::StaticClass()->GetFName(), LinkerLoadState->Linker->GetPackagePath().GetPackageFName());
	SCOPED_LOADTIMER_ASSET_TEXT(*LinkerLoadState->Linker->GetDebugName());

#if ALT2_ENABLE_NEW_ARCHIVE_FOR_LINKERLOAD
	FLinkerLoad::ELinkerStatus LinkerResult = FLinkerLoad::LINKER_Failed;
	if (!LinkerLoadState->Linker->GetLoader()->IsError())
	{
		LinkerLoadState->Linker->bUseTimeLimit = false;
		TRACE_LOADTIME_PROCESS_SUMMARY_SCOPE(LinkerLoadState->Linker);
		LinkerResult = LinkerLoadState->Linker->ProcessPackageSummary(nullptr);
	}
#else
	TRACE_LOADTIME_PROCESS_SUMMARY_SCOPE(this);
	FLinkerLoad::ELinkerStatus LinkerResult = LinkerLoadState->Linker->Tick(/* RemainingTimeLimit */ 0.0, /* bUseTimeLimit */ false, /* bUseFullTimeLimit */ false, nullptr);
#endif
	check(LinkerResult != FLinkerLoad::LINKER_TimedOut); // TODO: Add support for timeouts here
	if (LinkerResult == FLinkerLoad::LINKER_Failed)
	{
		LoadStatus = EAsyncLoadingResult::FailedLinker;
	}
	check(LinkerLoadState->Linker->HasFinishedInitialization() || LinkerResult == FLinkerLoad::LINKER_Failed);

	LinkerLoadState->LinkerLoadHeaderData.ImportMap.SetNum(LinkerLoadState->Linker->ImportMap.Num());
	TArray<FName, TInlineAllocator<128>> ImportedPackageNames;
	TArray<FPackageId, TInlineAllocator<128>> ImportedPackageIds;
	for (int32 ImportIndex = 0; ImportIndex < LinkerLoadState->Linker->ImportMap.Num(); ++ImportIndex)
	{
		const FObjectImport& LinkerImport = LinkerLoadState->Linker->ImportMap[ImportIndex];
		TArray<int32, TInlineAllocator<128>> PathComponents;
		int32 PathIndex = ImportIndex;
		PathComponents.Push(PathIndex);
		while (LinkerLoadState->Linker->ImportMap[PathIndex].OuterIndex.IsImport() && !LinkerLoadState->Linker->ImportMap[PathIndex].HasPackageName())
		{
			PathIndex = LinkerLoadState->Linker->ImportMap[PathIndex].OuterIndex.ToImport();
			PathComponents.Push(PathIndex);
		}
		int32 PackageImportIndex = PathComponents.Top();
		FObjectImport& PackageImport = LinkerLoadState->Linker->ImportMap[PackageImportIndex];
		FName ImportPackageName;
		const bool bImportHasPackageName = PackageImport.HasPackageName();
		if (bImportHasPackageName)
		{
			ImportPackageName = PackageImport.GetPackageName();
		}
		else
		{
			ImportPackageName = PackageImport.ObjectName;
		}

		// if the ImportPackageName is unset (NAME_None) then we skip processing it, it's likely an orphaned entry left over from a package redirector
		if (ImportPackageName.IsNone())
		{
			continue;
		}

		TCHAR NameStr[FName::StringBufferSize];
		uint32 NameLen = ImportPackageName.ToString(NameStr);
		bool bIsScriptImport = FPackageName::IsScriptPackage(FStringView(NameStr, NameLen));
		if (bIsScriptImport)
		{
			check(!bImportHasPackageName);
			TStringBuilder<256> FullPath;
			while (!PathComponents.IsEmpty())
			{
				NameLen = LinkerLoadState->Linker->ImportMap[PathComponents.Pop(EAllowShrinking::No)].ObjectName.ToString(NameStr);
				FPathViews::Append(FullPath, FStringView(NameStr, NameLen));
				LinkerLoadState->LinkerLoadHeaderData.ImportMap[ImportIndex] = FPackageObjectIndex::FromScriptPath(FullPath);
			}
		}
		else
		{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
			auto AssetRegistry = IAssetRegistryInterface::GetPtr();
			if (UE::LinkerLoad::CanLazyImport(*AssetRegistry, LinkerImport, *LinkerLoadState->Linker))
			{
				continue;
			}
#endif
			FPackageId ImportedPackageId = FPackageId::FromName(ImportPackageName);
			int32 ImportedPackageIndex = ImportedPackageIds.AddUnique(ImportedPackageId);
			if (ImportedPackageIndex == ImportedPackageNames.Num())
			{
				ImportedPackageNames.AddDefaulted();
			}
			ImportedPackageNames[ImportedPackageIndex] = ImportPackageName;
			bool bIsPackageImport = ImportIndex == PackageImportIndex;
			if (!bIsPackageImport || bImportHasPackageName)
			{
				if (!bImportHasPackageName)
				{
					PathComponents.Pop(EAllowShrinking::No);
				}
				TStringBuilder<256> PackageRelativeExportPath;
				while (!PathComponents.IsEmpty())
				{
					NameLen = LinkerLoadState->Linker->ImportMap[PathComponents.Pop(EAllowShrinking::No)].ObjectName.ToString(NameStr);
					for (uint32 I = 0; I < NameLen; ++I)
					{
						NameStr[I] = TChar<TCHAR>::ToLower(NameStr[I]);
					}
					PackageRelativeExportPath.AppendChar('/');
					PackageRelativeExportPath.Append(FStringView(NameStr, NameLen));
				}
				FPackageImportReference PackageImportRef(ImportedPackageIndex, LinkerLoadState->LinkerLoadHeaderData.ImportedPublicExportHashes.Num());
				LinkerLoadState->LinkerLoadHeaderData.ImportMap[ImportIndex] = FPackageObjectIndex::FromPackageImportRef(PackageImportRef);
				LinkerLoadState->LinkerLoadHeaderData.ImportedPublicExportHashes.Add(CityHash64(reinterpret_cast<const char*>(PackageRelativeExportPath.GetData() + 1), (PackageRelativeExportPath.Len() - 1) * sizeof(TCHAR)));
			}
		}
	}

	LinkerLoadState->LinkerLoadHeaderData.ExportMap.SetNum(LinkerLoadState->Linker->ExportMap.Num());
	for (int32 ExportIndex = 0; ExportIndex < LinkerLoadState->Linker->ExportMap.Num(); ++ExportIndex)
	{
		FObjectExport& ObjectExport = LinkerLoadState->Linker->ExportMap[ExportIndex];
		if ((ObjectExport.ObjectFlags & RF_ClassDefaultObject) != 0)
		{
			LinkerLoadState->bContainsClasses |= true;
		}

		if ((ObjectExport.ObjectFlags & (RF_Public | RF_GeneratePublicHash)) > 0 || ObjectExport.bGeneratePublicHash || ObjectExport.OuterIndex.IsImport())
		{
			TArray<int32, TInlineAllocator<128>> FullPath;
			int32 PathIndex = ExportIndex;
			FullPath.Push(PathIndex);
			while (LinkerLoadState->Linker->ExportMap[PathIndex].OuterIndex.IsExport())
			{
				PathIndex = LinkerLoadState->Linker->ExportMap[PathIndex].OuterIndex.ToExport();
				FullPath.Push(PathIndex);
			}
			TStringBuilder<256> PackageRelativeExportPath;
			while (!FullPath.IsEmpty())
			{
				TCHAR NameStr[FName::StringBufferSize];
				uint32 NameLen = LinkerLoadState->Linker->ExportMap[FullPath.Pop(EAllowShrinking::No)].ObjectName.ToString(NameStr);
				for (uint32 I = 0; I < NameLen; ++I)
				{
					NameStr[I] = TChar<TCHAR>::ToLower(NameStr[I]);
				}
				PackageRelativeExportPath.AppendChar('/');
				PackageRelativeExportPath.Append(FStringView(NameStr, NameLen));
			}
			uint64 PublicExportHash = CityHash64(reinterpret_cast<const char*>(PackageRelativeExportPath.GetData() + 1), (PackageRelativeExportPath.Len() - 1) * sizeof(TCHAR));
			LinkerLoadState->LinkerLoadHeaderData.ExportMap[ExportIndex].PublicExportHash = PublicExportHash;
		}
	}

	FPackageStoreEntry PackageStoreEntry;
	PackageStoreEntry.ImportedPackageIds = ImportedPackageIds;
	AsyncLoadingThread.InitializeAsyncPackageFromPackageStore(ThreadState, nullptr, this, PackageStoreEntry);

	HeaderData.ImportedPackageNames = ImportedPackageNames;
	HeaderData.ImportedPublicExportHashes = LinkerLoadState->LinkerLoadHeaderData.ImportedPublicExportHashes;
	HeaderData.ImportMap = LinkerLoadState->LinkerLoadHeaderData.ImportMap;
	HeaderData.ExportMap = LinkerLoadState->LinkerLoadHeaderData.ExportMap;

	AsyncLoadingThread.FinishInitializeAsyncPackage(ThreadState, this);

	// Package that can't be imported are never registered in the global import store, don't try to search for them
	if (Desc.bCanBeImported)
	{
		FWriteLockedPackageRef PackageRef = AsyncLoadingThread.GlobalImportStore.FindPackageRefCheckedForWrite(Desc.UPackageId, Desc.UPackageName);
		if (!HasLoadFailed())
		{
			PackageRef->PreInsertPublicExports({ LinkerLoadState->LinkerLoadHeaderData.ExportMap });

#if WITH_METADATA
			UE::TUniqueLock LinkerLock(LinkerLoadState->Linker->Mutex);
			// Create metadata object, this needs to happen before any other package wants to use our exports
			LinkerLoadState->MetaDataIndex = LinkerLoadState->Linker->LoadMetaDataFromExportMap(false);
			if (LinkerLoadState->MetaDataIndex >= 0)
			{
				FObjectExport& LinkerExport = LinkerLoadState->Linker->ExportMap[LinkerLoadState->MetaDataIndex];
				FExportObject& ExportObject = Data.Exports[LinkerLoadState->MetaDataIndex];
				ExportObject.Object = LinkerExport.Object;

				ExportObject.bWasFoundInMemory = !!LinkerExport.Object; // Make sure that the async flags are cleared in ClearConstructedObjects
				ExportObject.bExportLoadFailed = LinkerExport.bExportLoadFailed;
				ExportObject.bFiltered = LinkerExport.bWasFiltered;
			}
#endif // WITH_METADATA
		}
		else
		{
			PackageRef->SetHasFailed();
		}
	}

	AsyncPackageLoadingState = EAsyncPackageLoadingState2::ProcessDependencies;
	GetPackageNode(Package_ProcessDependencies).ReleaseBarrier(&ThreadState);

	return EEventLoadNodeExecutionResult::Complete;
}

bool FAsyncPackage2::PreloadLinkerLoadExports(FAsyncLoadingThreadState2& ThreadState)
{
	// Serialize exports
	UE::TUniqueLock LinkerLock(LinkerLoadState->Linker->Mutex);

	const int32 ExportCount = LinkerLoadState->Linker->ExportMap.Num();
	check(LinkerLoadState->Linker->ExportMap.Num() == Data.Exports.Num());
	while (LinkerLoadState->SerializeExportIndex < ExportCount)
	{
		ThreadState.MarkAsActive();

		const int32 ExportIndex = LinkerLoadState->SerializeExportIndex++;
		FExportObject& ExportObject = Data.Exports[ExportIndex];
		FObjectExport& LinkerExport = LinkerLoadState->Linker->ExportMap[ExportIndex];

		if (UObject* Object = LinkerExport.Object)
		{
			if (Object->HasAnyFlags(RF_NeedLoad))
			{
				UE_ASYNC_PACKAGE_LOG(VeryVerbose, Desc, TEXT("PreloadLinkerLoadExports"), TEXT("Preloading export %d: %s"), ExportIndex, *Object->GetPathName());
				UE_TRACK_REFERENCING_PACKAGE_SCOPED(Object, PackageAccessTrackingOps::NAME_PreLoad);
				LinkerLoadState->Linker->Preload(Object);
			}
		}

		// The linker export table can be patched during reinstantiation/serialization. We need to adjust our own export table if needed.
		if (ExportObject.Object != LinkerExport.Object)
		{
			UE_ASYNC_PACKAGE_LOG(Verbose, Desc, TEXT("PreloadLinkerLoadExports"), TEXT("Patching export %d: %s -> %s"), ExportIndex, *GetPathNameSafe(ExportObject.Object), *GetPathNameSafe(LinkerExport.Object));
			ExportObject.Object = LinkerExport.Object;
			ExportObject.bWasFoundInMemory = !!ExportObject.Object;
			ExportObject.bExportLoadFailed = LinkerExport.bExportLoadFailed;
		}

		if (ThreadState.IsTimeLimitExceeded(TEXT("SerializeLinkerLoadExports")))
		{
			return false;
		}
	}

	return true;
}

bool FAsyncPackage2::ResolveLinkerLoadImports(FAsyncLoadingThreadState2& ThreadState)
{
	check(AsyncPackageLoadingState >= EAsyncPackageLoadingState2::WaitingForImportDependencies);

	check(!LinkerLoadState->bIsCurrentlyResolvingImports);
	TGuardValue<bool> GuardIsCurrentlyResolvingImports(LinkerLoadState->bIsCurrentlyResolvingImports, true);

	// Validate that all imports are in the appropriate state and had their exports created
	const int32 ImportedPackagesCount = Data.ImportedAsyncPackages.Num();
	for (int32 ImportIndex = 0; ImportIndex < ImportedPackagesCount; ++ImportIndex)
	{
		FAsyncPackage2* ImportedPackage = Data.ImportedAsyncPackages[ImportIndex];
		if (ImportedPackage)
		{
			if (ImportedPackage->LinkerLoadState.IsSet())
			{
				if (ImportedPackage->AsyncPackageLoadingState < EAsyncPackageLoadingState2::WaitingForImportDependencies)
				{
					check(ThreadState.PackagesOnStack.Contains(ImportedPackage));
					UE_LOGF(LogStreaming, Warning, "Package %ls might be missing an import from package %ls because of a circular dependency between them.",
						*Desc.UPackageName.ToString(),
						*ImportedPackage->Desc.UPackageName.ToString());
				}
			}
			else
			{
				// A dependency is added for zen imports in ImportPackagesRecursiveInner that should prevent
				// us from getting a zen package in a state before its exports are ready. 
				// Just verify that it's working as intended.
				if (ImportedPackage->AsyncPackageLoadingState < EAsyncPackageLoadingState2::ExportsDone)
				{
					check(ThreadState.PackagesOnStack.Contains(ImportedPackage));
					UE_LOGF(LogStreaming, Warning, "Package %ls might be missing an import from cooked package %ls because it's exports are not yet ready.",
						*Desc.UPackageName.ToString(),
						*ImportedPackage->Desc.UPackageName.ToString());
				}
			}
		}
	}

	const int32 ImportCount = HeaderData.ImportMap.Num();
	while (LinkerLoadState->CreateImportIndex < ImportCount)
	{
		const int32 ImportIndex = LinkerLoadState->CreateImportIndex++;
		const FPackageObjectIndex& GlobalImportIndex = HeaderData.ImportMap[ImportIndex];
		if (!GlobalImportIndex.IsNull())
		{
			UObject* FromImportStore = ImportStore.GetImportObject(HeaderData, GlobalImportIndex);
#if ALT2_VERIFY_LINKERLOAD_MATCHES_IMPORTSTORE
			/*if (Desc.UPackageId.ValueForDebugging() == 0xF37353A71BF5C938 && ImportIndex == 0x0000005d)
			{
				UE_DEBUG_BREAK();
			}*/
			UObject* FromLinker = LinkerLoadState->Linker->CreateImport(ImportIndex);
			if (FromImportStore != FromLinker)
			{
				bool bIsAcceptableDeviation = false;
				FObjectImport& LinkerImport = LinkerLoadState->Linker->ImportMap[ImportIndex];
				if (FromLinker)
				{
					check(LinkerImport.SourceLinker);
					check(LinkerImport.SourceIndex >= 0);
					FObjectExport& SourceExport = LinkerImport.SourceLinker->ExportMap[LinkerImport.SourceIndex];
					if (!FromImportStore && SourceExport.bExportLoadFailed)
					{
						bIsAcceptableDeviation = true; // Exports can be marked as failed after they have already been returned to CreateImport. Is this a bug or a feature?
					}
					else if (FromImportStore && FromImportStore->GetName() == FromLinker->GetName() && FromLinker->GetOutermost() == GetTransientPackage())
					{
						bIsAcceptableDeviation = true; // Linker are sometimes stuck with objects that have been moved to the transient package. Is this a bug or a feature?
					}
				}
				check(bIsAcceptableDeviation);
			}
#endif
			FObjectImport& LinkerImport = LinkerLoadState->Linker->ImportMap[ImportIndex];
			if (!LinkerImport.XObject && FromImportStore)
			{
				LinkerImport.XObject = FromImportStore;
				LinkerImport.SourceIndex = FromImportStore->GetLinkerIndex();
				LinkerImport.SourceLinker = FromImportStore->GetLinker();

				UE_ASYNC_PACKAGE_LOG(VeryVerbose, Desc, TEXT("ResolveLinkerLoadImports"), TEXT("Resolved import %d: %s"), ImportIndex, *FromImportStore->GetPathName());
			}

			if (FromImportStore == nullptr)
			{
				UE_ASYNC_PACKAGE_LOG(Verbose, Desc, TEXT("ResolveLinkerLoadImports"), TEXT("Could not resolve import %d"), ImportIndex);
			}
		}
	}

	return true;
}

bool FAsyncPackage2::CreateLinkerLoadExports(FAsyncLoadingThreadState2& ThreadState)
{
	check(AsyncPackageLoadingState >= EAsyncPackageLoadingState2::DependenciesReady);

	check(!LinkerLoadState->bIsCurrentlyCreatingExports);
	TGuardValue<bool> GuardIsCurrentlyCreatingExports(LinkerLoadState->bIsCurrentlyCreatingExports, true);

	UE::TUniqueLock LinkerLock(LinkerLoadState->Linker->Mutex);

	// Create exports
	const int32 ExportCount = LinkerLoadState->Linker->ExportMap.Num();
	while (LinkerLoadState->CreateExportIndex < ExportCount)
	{
		const int32 ExportIndex = LinkerLoadState->CreateExportIndex++;
#if WITH_METADATA
		if (ExportIndex == LinkerLoadState->MetaDataIndex)
		{
			continue;
		}
#endif // WITH_METADATA

		ThreadState.MarkAsActive();

		FObjectExport& LinkerExport = LinkerLoadState->Linker->ExportMap[ExportIndex];
		FExportObject& ExportObject = Data.Exports[ExportIndex];

		if (LinkerExport.ClassIndex.IsNull())
		{
			// During FixupExportMap, LinkerLoad may null class indices for exports whose class is FCoreRedirects::IsKnownMissing. We cannot load these types, and since 
			// it would have been an error to serialize these types in the first place, we can assume if the class is null, it is a missing type we can ignore.

			// Fixup will also null the OuterIndex and ObjectName. Assert these are as expected, and that we aren't just hitting some corrupt state.
			checkf(LinkerExport.OuterIndex.IsNull() && LinkerExport.ObjectName == NAME_None, TEXT("Trying to create a linker load export that has a null class but non null outer or name"));

			ExportObject.bWasFoundInMemory = false;
			ExportObject.Object = nullptr;
			continue;
		}

		if (UObject* Object = LinkerLoadState->Linker->CreateExport(ExportIndex))
		{
			checkf(!Object->IsUnreachable(), TEXT("Trying to store an unreachable object '%s' in the import store"), *Object->GetFullName());
			ExportObject.Object = Object;
			ExportObject.bWasFoundInMemory = true; // Make sure that the async flags are cleared in ClearConstructedObjects
			EInternalObjectFlags FlagsToSet = EInternalObjectFlags::Async;
			uint64 PublicExportHash = LinkerLoadState->LinkerLoadHeaderData.ExportMap[ExportIndex].PublicExportHash;
			if (Desc.bCanBeImported && PublicExportHash)
			{
				FlagsToSet |= EInternalObjectFlags::LoaderImport;
				ImportStore.StoreGlobalObject(Desc.UPackageId, PublicExportHash, Object);
			}
			Object->SetInternalFlags(FlagsToSet);
		}
		else
		{
			ExportObject.bExportLoadFailed = LinkerExport.bExportLoadFailed;
			if (!ExportObject.bExportLoadFailed)
			{
				ExportObject.bFiltered = true;
			}
		}

		if (ThreadState.IsTimeLimitExceeded(TEXT("CreateLinkerLoadExports")))
		{
			return false;
		}
	}

	return true;
}

EEventLoadNodeExecutionResult FAsyncPackage2::ExecutePostLoadLinkerLoadPackageExports(FAsyncLoadingThreadState2& ThreadState)
{
#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Desc.PIEInstanceID);
#endif

	if (!HasLoadFailed())
	{
		SCOPED_LOADTIMER(PostLoadObjectsTime);
		TRACE_LOADTIME_POSTLOAD_SCOPE;

		FAsyncLoadingTickScope2 InAsyncLoadingTick(AsyncLoadingThread);

		// Begin async loading, simulates BeginLoad
		BeginAsyncLoad();

		FUObjectSerializeContext* LoadContext = GetSerializeContext();
		TArray<UObject*>& ThreadObjLoaded = LoadContext->PRIVATE_GetObjectsLoadedInternalUseOnly();

		// End async loading, simulates EndLoad
		ON_SCOPE_EXIT{ ThreadObjLoaded.Reset(); EndAsyncLoad(); };

		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		TGuardValue<bool> GuardIsRoutingPostLoad(ThreadContext.IsRoutingPostLoad, true);

		// Only postload exports instead of constructed objects to avoid cross-package interference.
		// Also this step is only about running the thread-safe postload that are ready to run, the rest will be deferred.
		// This code applies to both non threaded loader and threaded loader to exercise the same code path for both
		// Any non thread-safe or not ready postload is automatically being moved to the deferred phase.
		while (LinkerLoadState->PostLoadExportIndex < Data.Exports.Num())
		{
			const int32 ExportIndex = LinkerLoadState->PostLoadExportIndex++;
			const FExportObject& Export = Data.Exports[ExportIndex];

			if (UObject* Object = Export.Object)
			{
				if (Object->HasAnyFlags(RF_NeedPostLoad) && CanPostLoadOnAsyncLoadingThread(Object) && Object->IsReadyForAsyncPostLoad())
				{
#if WITH_EDITOR
					SCOPED_LOADTIMER_ASSET_TEXT(*Object->GetPathName());
#endif
					ThreadContext.CurrentlyPostLoadedObjectByALT = Object;
					Object->ConditionalPostLoad();
					ThreadContext.CurrentlyPostLoadedObjectByALT = nullptr;
				}

				if (ThreadState.IsTimeLimitExceeded(TEXT("ExecutePostLoadLinkerLoadPackageExports")))
				{
					return EEventLoadNodeExecutionResult::Timeout;
				}
			}
		}
	}

	// Reset this to be reused for the deferred postload phase
	LinkerLoadState->PostLoadExportIndex = 0;

	ConditionalBeginDeferredPostLoad(ThreadState);
	return EEventLoadNodeExecutionResult::Complete;
}

EEventLoadNodeExecutionResult FAsyncPackage2::ExecuteDeferredPostLoadLinkerLoadPackageExports(FAsyncLoadingThreadState2& ThreadState)
{
	SCOPED_LOADTIMER(PostLoadDeferredObjectsTime);
	TRACE_LOADTIME_POSTLOAD_SCOPE;

	FAsyncLoadingTickScope2 InAsyncLoadingTick(AsyncLoadingThread);

#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Desc.PIEInstanceID);
#endif

	FUObjectSerializeContext* LoadContext = GetSerializeContext();
	TArray<UObject*>& ThreadObjLoaded = LoadContext->PRIVATE_GetObjectsLoadedInternalUseOnly();

	ON_SCOPE_EXIT{ ThreadObjLoaded.Reset(); };

	// We can't return timeout during a flush as we're expected to be able to finish
	const bool bIsReadyForAsyncPostLoadAllowed = ThreadState.SyncLoadContextStack.IsEmpty() && AsyncLoadingThread.FullFlushCount.load(std::memory_order_relaxed) == 0;

	UE_MT_SCOPED_READ_ACCESS(ConstructedObjectsAccessDetector);

	// Go through both ConstructedObjects and export table as its possible to reload objects in the export table
	// without them being constructed and that would lead to missing postloads.
	// ConstructedObjects can be appended to during conditional postloads, so make sure to always take the latest value.
	const int32 ExportsCount = Data.Exports.Num();
	while (LinkerLoadState->PostLoadExportIndex < ExportsCount + ConstructedObjects.Num())
	{
		const int32 ObjectIndex = LinkerLoadState->PostLoadExportIndex++;

		if (ObjectIndex < ExportsCount)
		{
			FExportObject& ExportObject = Data.Exports[ObjectIndex];
			FObjectExport& LinkerExport = LinkerLoadState->Linker->ExportMap[ObjectIndex];
			// The linker export table can be patched during reinstantiation. We need to adjust our own export table if needed.
			if (!LinkerExport.bExportLoadFailed)
			{
				if (ExportObject.Object != LinkerExport.Object)
				{
					UE_ASYNC_PACKAGE_LOG(Verbose, Desc, TEXT("ExecuteDeferredPostLoadLinkerLoadPackageExports"), TEXT("Patching export %d: %s -> %s"), ObjectIndex, *GetPathNameSafe(ExportObject.Object), *GetPathNameSafe(LinkerExport.Object));
					ExportObject.Object = LinkerExport.Object;
					ExportObject.bWasFoundInMemory = !!LinkerExport.Object;
				}
			}
		}

		UObject* Object = ObjectIndex < ExportsCount ? Data.Exports[ObjectIndex].Object : ConstructedObjects[ObjectIndex - ExportsCount];
		if (Object && Object->HasAnyFlags(RF_NeedPostLoad))
		{
			// Only allow to wait when there is no flush waiting on us
			if (bIsReadyForAsyncPostLoadAllowed && !Object->IsReadyForAsyncPostLoad())
			{
				--LinkerLoadState->PostLoadExportIndex;
				return EEventLoadNodeExecutionResult::Timeout;
			}
#if WITH_EDITOR
			SCOPED_LOADTIMER_ASSET_TEXT(*Object->GetPathName());
#endif
			Object->ConditionalPostLoad();
		}

		if (ThreadState.IsTimeLimitExceeded(TEXT("ExecuteDeferredPostLoadLinkerLoadPackageExports")))
		{
			return EEventLoadNodeExecutionResult::Timeout;
		}
	}

	AsyncPackageLoadingState = EAsyncPackageLoadingState2::DeferredPostLoadDone;
	ConditionalFinishLoading(ThreadState);

	return EEventLoadNodeExecutionResult::Complete;
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_CreateExports(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Event_CreateExports);
	UE_ASYNC_PACKAGE_DEBUG(Package->Desc);

	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::CreateExports);

	FAsyncPackageScope2 Scope(Package);
#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Package->Desc.PIEInstanceID);
#endif

	if (Package->LinkerLoadState.IsSet())
	{
		if (!Package->CreateLinkerLoadExports(ThreadState))
		{
			return EEventLoadNodeExecutionResult::Timeout;
		}
	}
	else
	{
		if (!Package->CreateRuntimeExports(ThreadState))
		{
			return EEventLoadNodeExecutionResult::Timeout;
		}
	}

	Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::WaitingForImportDependencies;
	Package->ConditionalBeginResolveImports(ThreadState);
	return EEventLoadNodeExecutionResult::Complete;
}

void FAsyncPackage2::ConditionalBeginResolveImports(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ConditionalBeginResolveImports);

	WaitForAllDependenciesToReachState(ThreadState, &FAsyncPackage2::AllDependenciesImportState, EAsyncPackageLoadingState2::WaitingForImportDependencies, AsyncLoadingThread.ConditionalBeginResolveImportsTick,
		[&ThreadState](FAsyncPackage2* Package)
		{
			check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::WaitingForImportDependencies);
			Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::ResolveImports;
			Package->GetPackageNode(Package_ResolveImports).ReleaseBarrier(&ThreadState);
		}
	);
}

void FAsyncPackage2::ConditionalBeginPostLoadPhase(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ConditionalBeginPostLoadPhase);

	// This phase is important so that all dependencies have finished serialization before we start postloading. Otherwise we could end up with RF_NeedLoad
	// objects or worse, race conditions between postload trying to access properties, and serialize writing to those properties.
	WaitForAllDependenciesToReachState(ThreadState, &FAsyncPackage2::AllDependenciesPostLoadReadyState, EAsyncPackageLoadingState2::PostLoad, AsyncLoadingThread.ConditionalBeginPostLoadTick,
		[&ThreadState](FAsyncPackage2* Package)
		{
			check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::PostLoad);

			Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::PostLoad;
			Package->GetPackageNode(Package_PostLoad).ReleaseBarrier(&ThreadState);
		}
	);
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_ResolveImports(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Event_ResolveImports);
	UE_ASYNC_PACKAGE_DEBUG(Package->Desc);

	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::ResolveImports);

	FAsyncPackageScope2 Scope(Package);
#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Package->Desc.PIEInstanceID);
#endif

	if (Package->LinkerLoadState.IsSet())
	{
		if (!Package->ResolveLinkerLoadImports(ThreadState))
		{
			return EEventLoadNodeExecutionResult::Timeout;
		}
	}

	Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::PreloadExports;
	Package->GetPackageNode(Package_PreloadExports).ReleaseBarrier(&ThreadState);
	return EEventLoadNodeExecutionResult::Complete;
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_PreloadExports(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Event_PreloadExports);
	UE_ASYNC_PACKAGE_DEBUG(Package->Desc);

	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::PreloadExports);

	FAsyncPackageScope2 Scope(Package);
#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Package->Desc.PIEInstanceID);
#endif

	if (Package->LinkerLoadState.IsSet())
	{
		if (!Package->PreloadLinkerLoadExports(ThreadState))
		{
			return EEventLoadNodeExecutionResult::Timeout;
		}

		if (Package->LinkerLoadState->Linker->ExternalReadDependencies.Num())
		{
			Package->ExternalReadDependencies.Append(MoveTemp(Package->LinkerLoadState->Linker->ExternalReadDependencies));
		}
	}
	else
	{
		if (!Package->PreloadRuntimeExports(ThreadState))
		{
			return EEventLoadNodeExecutionResult::Timeout;
		}
	}

	if (Package->ExternalReadDependencies.Num() == 0)
	{
		Package->GetPackageNode(Package_ExportsSerialized).ReleaseBarrier(&ThreadState);
	}
	else
	{
		Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::WaitingForExternalReads;
		Package->AsyncLoadingThread.ExternalReadQueue.Enqueue(Package);
		if (IsInParallelLoadingThread())
		{
			Package->AsyncLoadingThread.AltZenaphore.NotifyOne();
		}
	}
	return EEventLoadNodeExecutionResult::Complete;
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_ProcessPackageSummary(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Event_ProcessPackageSummary);
	UE_ASYNC_PACKAGE_DEBUG(Package->Desc);

	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::WaitingForIo);
	Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::ProcessPackageSummary;

	FAsyncPackageScope2 Scope(Package);

#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Package->Desc.PIEInstanceID);
#endif

	Package->CallProgressCallbacks();

	if (Package->LinkerLoadState.IsSet())
	{
		return Package->ProcessLinkerLoadPackageSummary(ThreadState);
	}
	if (Package->HasLoadFailed())
	{
		if (Package->Desc.bCanBeImported)
		{
			FWriteLockedPackageRef PackageRef = Package->AsyncLoadingThread.GlobalImportStore.FindPackageRefCheckedForWrite(Package->Desc.UPackageId, Package->Desc.UPackageName);
			PackageRef->SetHasFailed();
		}
	}
	else
	{
		TRACE_LOADTIME_PROCESS_SUMMARY_SCOPE(Package);
		check(Package->ExportBundleEntryIndex == 0);

		static_cast<FZenPackageHeader&>(Package->HeaderData) = FZenPackageHeader::MakeView(Package->SerializationState.IoRequest.GetResultOrDie().GetView());
#if WITH_EDITOR
		FAsyncPackageHeaderData* OptionalSegmentHeaderData = Package->OptionalSegmentHeaderData.GetPtrOrNull();
		if (OptionalSegmentHeaderData)
		{
			static_cast<FZenPackageHeader&>(*OptionalSegmentHeaderData) = FZenPackageHeader::MakeView(Package->OptionalSegmentSerializationState->IoRequest.GetResultOrDie().GetView());
		}
#endif
		if (Package->Desc.bCanBeImported)
		{
			FWriteLockedPackageRef PackageRef = Package->AsyncLoadingThread.GlobalImportStore.FindPackageRefCheckedForWrite(Package->Desc.UPackageId, Package->Desc.UPackageName);
#if WITH_EDITOR
			if (Package->OptionalSegmentHeaderData.IsSet())
			{
				PackageRef->PreInsertPublicExports({ Package->HeaderData.ExportMap, Package->OptionalSegmentHeaderData->ExportMap });
			}
			else
#endif
			{
				PackageRef->PreInsertPublicExports({ Package->HeaderData.ExportMap });
			}
		}

		check(Package->Desc.PackageIdToLoad == FPackageId::FromName(Package->HeaderData.PackageName));
		if (Package->Desc.PackagePathToLoad.IsEmpty())
		{
			Package->Desc.PackagePathToLoad = FPackagePath::FromPackageNameUnchecked(Package->HeaderData.PackageName);
		}
		// Imported packages won't have a UPackage name set unless they were redirected, in which case they will have the source package name
		if (Package->Desc.UPackageName.IsNone())
		{
			Package->Desc.UPackageName = Package->Desc.InstancingContext.RemapPackage(Package->HeaderData.PackageName);
		}
		check(Package->Desc.UPackageId == FPackageId::FromName(Package->Desc.UPackageName));
#if WITH_EDITORONLY_DATA
		// Determinism in capitalization when loading cooked packages (EPackageLoader::Zen) from uncooked:
		// If the loaded/imported package name has been provided and is case-insensitive equal to the one in the
		// package summary, then use the deterministic capitalization from the package summary.
		if (Package->Desc.Loader != EPackageLoader::LinkerLoad // LinkerLoad handles capitalization in TryGetPackageInfoFromPackageStore
			&& Package->Desc.UPackageName == Package->HeaderData.PackageName) // case-insensitive compare
		{
			Package->Desc.UPackageName = Package->HeaderData.PackageName;
		}
#endif
		Package->CreateUPackage();
		Package->LinkerRoot->SetPackageFlags(Package->HeaderData.PackageSummary->PackageFlags);
#if WITH_EDITOR
		Package->LinkerRoot->bIsCookedForEditor = !!(Package->HeaderData.PackageSummary->PackageFlags & PKG_FilterEditorOnly);
#endif
		if (const FZenPackageVersioningInfo* VersioningInfo = Package->HeaderData.VersioningInfo.GetPtrOrNull())
		{
			Package->LinkerRoot->SetLinkerPackageVersion(VersioningInfo->PackageVersion);
			Package->LinkerRoot->SetLinkerLicenseeVersion(VersioningInfo->LicenseeVersion);
			Package->LinkerRoot->SetLinkerCustomVersions(VersioningInfo->CustomVersions);
		}
		else
		{
			Package->LinkerRoot->SetLinkerPackageVersion(GPackageFileUEVersion);
			Package->LinkerRoot->SetLinkerLicenseeVersion(GPackageFileLicenseeUEVersion);
		}

		// Check if the package is instanced
		FName PackageNameToLoad = Package->Desc.PackagePathToLoad.GetPackageFName();
		if (Package->Desc.UPackageName != PackageNameToLoad)
		{
			Package->Desc.InstancingContext.BuildPackageMapping(PackageNameToLoad, Package->Desc.UPackageName);
		}

		TRACE_LOADTIME_PACKAGE_SUMMARY(Package, Package->HeaderData.PackageName, Package->HeaderData.PackageSummary->HeaderSize, Package->HeaderData.ImportMap.Num(), Package->HeaderData.ExportMap.Num(), Package->Desc.Priority);
	}

	Package->AsyncLoadingThread.FinishInitializeAsyncPackage(ThreadState, Package);

	Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::ProcessDependencies;
	Package->GetPackageNode(Package_ProcessDependencies).ReleaseBarrier(&ThreadState);

	return EEventLoadNodeExecutionResult::Complete;
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_ProcessDependencies(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::WaitingForDependencies;

	Package->AsyncLoadingThread.FinishImportingPackages(ThreadState, Package);

	if (!Package->AsyncLoadingThread.bHasRegisteredAllScriptObjects)
	{
		Package->SetupScriptDependencies();
	}
	Package->GetPackageNode(Package_DependenciesReady).ReleaseBarrier(&ThreadState);
	return EEventLoadNodeExecutionResult::Complete;
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_DependenciesReady(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Event_DependenciesReady);
	UE_ASYNC_PACKAGE_DEBUG(Package->Desc);

	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::WaitingForDependencies);

	Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::DependenciesReady;
	Package->ConditionalBeginProcessPackageExports(ThreadState);
	return EEventLoadNodeExecutionResult::Complete;
}

void FAsyncPackage2::CallProgressCallbacks()
{
	FProgressCallbackEntry* Entry = ProgressCallbacksHead.load(std::memory_order_acquire);
	if (!Entry)
	{
		return;
	}

	EAsyncLoadingProgress CurrentProgress = GetLoadingProgressFromState(AsyncPackageLoadingState);
	if (CurrentProgress <= EAsyncLoadingProgress::Failed)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncPackage2::CallProgressCallbacks);

	while (Entry)
	{
		CallProgressCallback(*Entry, Desc.UPackageName, GetLinkerRoot(), CurrentProgress);
		Entry = Entry->Next.load(std::memory_order_relaxed);
	}
}

void FAsyncPackage2::InitializeExportArchive(FExportArchive& Ar, bool bIsOptionalSegment)
{
	Ar.SetUEVer(LinkerRoot->GetLinkerPackageVersion());
	Ar.SetLicenseeUEVer(LinkerRoot->GetLinkerLicenseeVersion());
	// Ar.SetEngineVer(Summary.SavedByEngineVersion); // very old versioning scheme
	if (!LinkerRoot->GetLinkerCustomVersions().GetAllVersions().IsEmpty())
	{
		Ar.SetCustomVersions(LinkerRoot->GetLinkerCustomVersions());
	}
	Ar.SetUseUnversionedPropertySerialization((LinkerRoot->GetPackageFlags() & PKG_UnversionedProperties) != 0);
	Ar.SetIsLoadingFromCookedPackage((LinkerRoot->GetPackageFlags() & PKG_Cooked) != 0);
	Ar.SetIsLoading(true);
	Ar.SetIsPersistent(true);
	if (LinkerRoot->GetPackageFlags() & PKG_FilterEditorOnly)
	{
		Ar.SetFilterEditorOnly(true);
	}
	Ar.ArAllowLazyLoading = true;

	// FExportArchive special fields
	Ar.PackageDesc = &Desc;
	Ar.HeaderData = &HeaderData;
#if WITH_EDITOR
	if (bIsOptionalSegment)
	{
		Ar.HeaderData = OptionalSegmentHeaderData.GetPtrOrNull();
		check(Ar.HeaderData);
	}
#endif
	Ar.ImportStore = &ImportStore;
	Ar.ExternalReadDependencies = &ExternalReadDependencies;
	Ar.InstanceContext = &Desc.InstancingContext;
	Ar.bIsOptionalSegment = bIsOptionalSegment;
}

// When possible, deferring serialization will enable some nodes to run in parallel
// At some point, we will also need to defer them if we want to better mix cooked and uncooked loading.
inline bool CanDeferSerialization(FExportObject& Export)
{
	return GAllowMultithreadedLoading &&
		// Classes, CDOs and Archetype are serialized during the create export phase since they are required to be loaded by dependencies
		!Export.Object->GetClass()->IsChildOf(UField::StaticClass()) &&
		!Export.Object->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) &&
		!Export.Object->GetClass()->IsChildOf(UCookedMetaDataBase::StaticClass());
}

bool FAsyncPackage2::CreateRuntimeExports(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateRuntimeExports);
	UE_ASYNC_PACKAGE_DEBUG(Desc);

	check(!LinkerLoadState.IsSet());
	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	check(AsyncPackageLoadingState >= EAsyncPackageLoadingState2::DependenciesReady);

	if (HasLoadFailed())
	{
		return true;
	}

	FAsyncPackageScope2 Scope(this);
#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Desc.PIEInstanceID);
#endif

	while (ProcessedExportBundlesCount < Data.TotalExportBundleCount)
	{
		UE_ASYNC_PACKAGE_LOG(VeryVerbose, Desc, TEXT("CreateRuntimeExports"), TEXT("Beginning Creating Exports for runtime bundle %d"), ProcessedExportBundlesCount);

		bool bIsOptionalSegment = false;
#if WITH_EDITOR
		const FAsyncPackageHeaderData* LocalHeaderData;
		FAsyncPackageSerializationState* LocalSerializationState;
		bIsOptionalSegment = ProcessedExportBundlesCount == 1;

		if (bIsOptionalSegment)
		{
			LocalHeaderData = OptionalSegmentHeaderData.GetPtrOrNull();
			check(LocalHeaderData);
			LocalSerializationState = OptionalSegmentSerializationState.GetPtrOrNull();
			check(LocalSerializationState);
		}
		else
		{
			check(ProcessedExportBundlesCount == 0);
			LocalHeaderData = &HeaderData;
			LocalSerializationState = &SerializationState;
		}
#else
		const FAsyncPackageHeaderData* LocalHeaderData = &HeaderData;
		FAsyncPackageSerializationState* LocalSerializationState = &SerializationState;
#endif
		const FIoBuffer& IoBuffer = LocalSerializationState->IoRequest.GetResultOrDie();
		FExportArchive Ar(IoBuffer);
		InitializeExportArchive(Ar, bIsOptionalSegment);

		while (ExportBundleEntryIndex < LocalHeaderData->ExportBundleEntries.Num())
		{
			const FExportBundleEntry& BundleEntry = LocalHeaderData->ExportBundleEntries[ExportBundleEntryIndex];
			if (BundleEntry.LocalExportIndex < uint32(LocalHeaderData->ExportMap.Num()))
			{
				const FExportMapEntry& ExportMapEntry = LocalHeaderData->ExportMap[BundleEntry.LocalExportIndex];
				FExportObject& Export = LocalHeaderData->ExportsView[BundleEntry.LocalExportIndex];

				if (BundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Create)
				{
					if (!Export.Object)
					{
						EventDrivenCreateExport(*LocalHeaderData, BundleEntry.LocalExportIndex);
					}
				}
				// We cannot test RF_NeedLoad since it is removed before we start serializing.
				// Since under multithreaded mode, we might race, we always enter the serialization
				// race unless we are positive the serialization is completed.
				else if (Export.Object && !Export.Object->HasAnyFlags(RF_LoadCompleted, std::memory_order_acquire) && !CanDeferSerialization(Export))
				{
					check(BundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Serialize);

					EventDrivenSerializeExport(*LocalHeaderData, BundleEntry.LocalExportIndex, &Ar);
				}
			}
			else
			{
				uint32 CellExportIndex = BundleEntry.LocalExportIndex - LocalHeaderData->ExportMap.Num();
				const FCellExportMapEntry& CellExportMapEntry = LocalHeaderData->CellExportMap[CellExportIndex];
				FExportCell& CellExport = LocalHeaderData->CellExportsView[CellExportIndex];

				if (BundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Create)
				{
					if (!CellExport.Cell)
					{
						EventDrivenCreateCellExport(*LocalHeaderData, BundleEntry.LocalExportIndex, &Ar);
					}
				}
				else
				{
					check(BundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Serialize);
					check(CellExport.Cell);
					EventDrivenSerializeCellExport(*LocalHeaderData, BundleEntry.LocalExportIndex, &Ar);
				}
			}
			++ExportBundleEntryIndex;
		}

		ExportBundleEntryIndex = 0;

		UE_ASYNC_PACKAGE_LOG(VeryVerbose, Desc, TEXT("CreateRuntimeExports"), TEXT("Finished Creating Exports for runtime bundle %d"), ProcessedExportBundlesCount);

		if (++ProcessedExportBundlesCount == Data.TotalExportBundleCount)
		{
			ProcessedExportBundlesCount = 0;
			return true;
		}
	}

	return false;
}

bool FAsyncPackage2::PreloadRuntimeExports(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(PreloadRuntimeExports);
	UE_ASYNC_PACKAGE_DEBUG(Desc);

	check(!LinkerLoadState.IsSet());
	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	check(AsyncPackageLoadingState >= EAsyncPackageLoadingState2::ResolveImports);

	if (HasLoadFailed())
	{
		return true;
	}

	FAsyncPackageScope2 Scope(this);
#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Desc.PIEInstanceID);
#endif

	while (ProcessedExportBundlesCount < Data.TotalExportBundleCount)
	{
		UE_ASYNC_PACKAGE_LOG(VeryVerbose, Desc, TEXT("PreloadRuntimeExports"), TEXT("Beginning Preloads of runtime export bundle %d"), ProcessedExportBundlesCount);

		bool bIsOptionalSegment = false;
#if WITH_EDITOR
		const FAsyncPackageHeaderData* LocalHeaderData;
		FAsyncPackageSerializationState* LocalSerializationState;
		bIsOptionalSegment = ProcessedExportBundlesCount == 1;

		if (bIsOptionalSegment)
		{
			LocalHeaderData = OptionalSegmentHeaderData.GetPtrOrNull();
			check(LocalHeaderData);
			LocalSerializationState = OptionalSegmentSerializationState.GetPtrOrNull();
			check(LocalSerializationState);
		}
		else
		{
			check(ProcessedExportBundlesCount == 0);
			LocalHeaderData = &HeaderData;
			LocalSerializationState = &SerializationState;
		}
#else
		const FAsyncPackageHeaderData* LocalHeaderData = &HeaderData;
		FAsyncPackageSerializationState* LocalSerializationState = &SerializationState;
#endif
		const FIoBuffer& IoBuffer = LocalSerializationState->IoRequest.GetResultOrDie();
		FExportArchive Ar(IoBuffer);
		InitializeExportArchive(Ar, bIsOptionalSegment);

		while (ExportBundleEntryIndex < LocalHeaderData->ExportBundleEntries.Num())
		{
			const FExportBundleEntry& BundleEntry = LocalHeaderData->ExportBundleEntries[ExportBundleEntryIndex];
			if (BundleEntry.LocalExportIndex < uint32(LocalHeaderData->ExportMap.Num()))
			{
				const FExportMapEntry& ExportMapEntry = LocalHeaderData->ExportMap[BundleEntry.LocalExportIndex];
				FExportObject& Export = LocalHeaderData->ExportsView[BundleEntry.LocalExportIndex];

				if (BundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Serialize)
				{
					// We cannot test RF_NeedLoad since it is removed before we start serializing.
					// Since under multithreaded mode, we might race, we always enter the serialization
					// race unless we are positive the serialization is completed.
					if (Export.Object && !Export.Object->HasAnyFlags(RF_LoadCompleted, std::memory_order_acquire))
					{
						EventDrivenSerializeExport(*LocalHeaderData, BundleEntry.LocalExportIndex, &Ar);
					}
				}
			}
			else
			{
				uint32 CellExportIndex = BundleEntry.LocalExportIndex - LocalHeaderData->ExportMap.Num();
				const FCellExportMapEntry& CellExportMapEntry = LocalHeaderData->CellExportMap[CellExportIndex];
				FExportCell& CellExport = LocalHeaderData->CellExportsView[CellExportIndex];

				if (BundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Serialize)
				{
					if (!CellExport.bSerialized)
					{
						EventDrivenSerializeCellExport(*LocalHeaderData, BundleEntry.LocalExportIndex, &Ar);
					}
				}
			}
			++ExportBundleEntryIndex;
		}

		ExportBundleEntryIndex = 0;

		UE_ASYNC_PACKAGE_LOG(VeryVerbose, Desc, TEXT("PreloadRuntimeExports"), TEXT("Finished Preloads of runtime export bundle %d"), ProcessedExportBundlesCount);

		if (++ProcessedExportBundlesCount == Data.TotalExportBundleCount)
		{
			ProcessedExportBundlesCount = 0;
			// We own one ref by default since the ref is initialized to 1.
			ReleaseHeaderRef();
			return true;
		}
	}

	return false;
}

UObject* FAsyncPackage2::EventDrivenIndexToObject(const FAsyncPackageHeaderData& Header, FPackageObjectIndex Index, bool bCheckSerialized)
{
	UObject* Result = nullptr;
	if (Index.IsNull())
	{
		return Result;
	}
	if (Index.IsExport())
	{
		const FExportObject& Export = Header.ExportsView[Index.ToExport()];
		Result = Export.Object;
		UE_CLOGF(!Result, LogStreaming, Warning, "Missing Dependency, missing export (%ls) in package %ls. \n\tExport Details - Index: 0x%llX, Super: '%ls', Template: '%ls', LoadFailed: %ls, Filtered: %ls (%ls), FoundInMemory: %ls",
			*Header.NameMap.GetName(Header.ExportMap[Index.ToExport()].ObjectName).ToString(),
			*Desc.PackagePathToLoad.GetPackageFName().ToString(),
			Index.Value(),
			(Export.SuperObject ? *Export.SuperObject->GetPathName() : TEXT("null")),
			(Export.TemplateObject ? *Export.TemplateObject->GetPathName() : TEXT("null")),
			(Export.bExportLoadFailed ? TEXT("true") : TEXT("false")),
			(Export.bFiltered ? TEXT("true") : TEXT("false")),
			LexToString(Header.ExportMap[Index.ToExport()].FilterFlags),
			(Export.bWasFoundInMemory ? TEXT("true") : TEXT("false")));
	}
	else if (Index.IsImport())
	{
		Result = ImportStore.GetImportObject(Header, Index);
		UE_CLOGF(!Result, LogStreaming, Warning, "Missing Dependency, missing %ls import %ls(0x%llX) for package %ls",
			Index.IsScriptImport() ? TEXT("script") : TEXT("package"),
			*ImportStore.GetImportName(Header, Index),
			Index.Value(),
			*Desc.PackagePathToLoad.GetPackageFName().ToString());
	}
#if DO_CHECK
	if (Result && bCheckSerialized)
	{
		bool bIsSerialized = Index.IsScriptImport() || Result->IsA(UPackage::StaticClass()) || Result->HasAllFlags(RF_WasLoaded | RF_LoadCompleted);
		if (!bIsSerialized)
		{
			UE_LOGF(LogStreaming, Warning, "Missing Dependency, '%ls' (0x%llX) for package %ls has not been serialized yet.",
				*Result->GetFullName(),
				Index.Value(),
				*Desc.PackagePathToLoad.GetPackageFName().ToString());
		}
	}
	if (Result)
	{
		UE_CLOGF(Result->IsUnreachable(), LogStreaming, Fatal, "Returning an object  (%ls) from EventDrivenIndexToObject that is unreachable.", *Result->GetFullName());
	}
#endif
	return Result;
}

void FAsyncPackage2::ProcessExportDependencies(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex, FExportBundleEntry::EExportCommandType CommandType)
{
	static_assert(FExportBundleEntry::ExportCommandType_Count == 2, "Expected the only export command types fo be Create and Serialize");

	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessExportDependencies);
	const FDependencyBundleHeader& DependencyBundle = Header.DependencyBundleHeaders[LocalExportIndex];
	if (DependencyBundle.FirstEntryIndex < 0)
	{
		return;
	}
	int32 RunningIndex = DependencyBundle.FirstEntryIndex;
	if (CommandType == FExportBundleEntry::ExportCommandType_Serialize)
	{
		// Skip over the dependency entries for Create
		for (int32 Index = 0; Index < FExportBundleEntry::ExportCommandType_Count; ++Index)
		{
			RunningIndex += DependencyBundle.EntryCount[FExportBundleEntry::ExportCommandType_Create][Index];
		}
	}

	for (int32 Index = DependencyBundle.EntryCount[CommandType][FExportBundleEntry::ExportCommandType_Create]; Index > 0; --Index)
	{
		const FDependencyBundleEntry& Dep = Header.DependencyBundleEntries[RunningIndex++];
		if (Dep.LocalImportOrExportIndex.IsExport())
		{
			if (Dep.LocalImportOrExportIndex.ToExport() < Header.ExportsView.Num())
			{
				ConditionalCreateExport(Header, Dep.LocalImportOrExportIndex.ToExport());
			}
			else
			{
				ConditionalCreateCellExport(Header, Dep.LocalImportOrExportIndex.ToExport());
			}
		}
		else
		{
			if (Dep.LocalImportOrExportIndex.ToImport() < Header.ImportMap.Num())
			{
				ConditionalCreateImport(Header, Dep.LocalImportOrExportIndex.ToImport());
			}
			else
			{
				ConditionalCreateCellImport(Header, Dep.LocalImportOrExportIndex.ToImport());
			}
		}
	}

	for (int32 Index = DependencyBundle.EntryCount[CommandType][FExportBundleEntry::ExportCommandType_Serialize]; Index > 0; Index--)
	{
		const FDependencyBundleEntry& Dep = Header.DependencyBundleEntries[RunningIndex++];
		if (Dep.LocalImportOrExportIndex.IsExport())
		{
			if (Dep.LocalImportOrExportIndex.ToExport() < Header.ExportsView.Num())
			{
				ConditionalSerializeExport(Header, Dep.LocalImportOrExportIndex.ToExport());
			}
			else
			{
				ConditionalSerializeCellExport(Header, Dep.LocalImportOrExportIndex.ToExport());
			}
		}
		else
		{
			if (Dep.LocalImportOrExportIndex.ToImport() < Header.ImportMap.Num())
			{
				ConditionalSerializeImport(Header, Dep.LocalImportOrExportIndex.ToImport());
			}
			else
			{
				ConditionalSerializeCellImport(Header, Dep.LocalImportOrExportIndex.ToImport());
			}
		}
	}
}

int32 FAsyncPackage2::GetPublicExportIndex(uint64 ExportHash, FAsyncPackageHeaderData*& OutHeader)
{
	for (int32 ExportIndex = 0; ExportIndex < HeaderData.ExportMap.Num(); ++ExportIndex)
	{
		if (HeaderData.ExportMap[ExportIndex].PublicExportHash == ExportHash)
		{
			OutHeader = &HeaderData;
			return ExportIndex;
		}
	}
#if WITH_EDITOR
	if (FAsyncPackageHeaderData* OptionalSegmentHeaderDataPtr = OptionalSegmentHeaderData.GetPtrOrNull())
	{
		for (int32 ExportIndex = 0; ExportIndex < OptionalSegmentHeaderDataPtr->ExportMap.Num(); ++ExportIndex)
		{
			if (OptionalSegmentHeaderDataPtr->ExportMap[ExportIndex].PublicExportHash == ExportHash)
			{
				OutHeader = OptionalSegmentHeaderDataPtr;
				return HeaderData.ExportMap.Num() + ExportIndex;
			}
		}
	}
#endif
	return -1;
}

void FAsyncPackage2::ConditionalCreateExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex)
{
	if (!Header.ExportsView[LocalExportIndex].Object)
	{
		FAsyncPackageScope2 Scope(this);
		EventDrivenCreateExport(Header, LocalExportIndex);
	}
}

void FAsyncPackage2::ConditionalSerializeExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex)
{
	FExportObject& Export = Header.ExportsView[LocalExportIndex];

	if (!Export.Object && !(Export.bFiltered || Export.bExportLoadFailed))
	{
		ConditionalCreateExport(Header, LocalExportIndex);
	}

	if (!Export.Object || (Export.bFiltered || Export.bExportLoadFailed))
	{
		return;
	}

	// We cannot test RF_NeedLoad since it is removed before we start serializing.
	// Since under multithreaded mode, we might race, we always enter the serialization
	// race unless we are positive the serialization is completed.
	if (!Export.Object->HasAllFlags(RF_LoadCompleted, std::memory_order_acquire))
	{
		FAsyncPackageScope2 Scope(this);
		EventDrivenSerializeExport(Header, LocalExportIndex, nullptr);
	}
}

void FAsyncPackage2::ConditionalCreateImport(const FAsyncPackageHeaderData& Header, int32 LocalImportIndex)
{
	const FPackageObjectIndex& ObjectIndex = Header.ImportMap[LocalImportIndex];
	check(ObjectIndex.IsPackageImport());
	if (UObject* FromImportStore = ImportStore.GetImportObject(Header, ObjectIndex))
	{
		return;
	}

	FPackageImportReference PackageImportRef = ObjectIndex.ToPackageImportRef();
	FAsyncPackage2* SourcePackage = Header.ImportedAsyncPackagesView[PackageImportRef.GetImportedPackageIndex()];
	if (!SourcePackage)
	{
		return;
	}
	uint64 ExportHash = Header.ImportedPublicExportHashes[PackageImportRef.GetImportedPublicExportHashIndex()];
	FAsyncPackageHeaderData* SourcePackageHeader = nullptr;

	// If the header ref is at 0, it means all the serialization is done for that package, we just raced with it and missed the cue.
	if (SourcePackage->TryAddHeaderRef())
	{
		int32 ExportIndex = SourcePackage->GetPublicExportIndex(ExportHash, SourcePackageHeader);
		if (ExportIndex >= 0)
		{
			SourcePackage->ConditionalCreateExport(*SourcePackageHeader, ExportIndex);
		}
		SourcePackage->ReleaseHeaderRef();
	}
}

void FAsyncPackage2::ConditionalSerializeImport(const FAsyncPackageHeaderData& Header, int32 LocalImportIndex)
{
	const FPackageObjectIndex& ObjectIndex = Header.ImportMap[LocalImportIndex];
	check(ObjectIndex.IsPackageImport());

	if (UObject* FromImportStore = ImportStore.GetImportObject(Header, ObjectIndex))
	{
		// We cannot test RF_NeedLoad since it is removed before we start serializing.
		// Since under multithreaded mode, we might race, we always enter the serialization
		// race unless we are positive the serialization is completed.
		if (FromImportStore->HasAllFlags(RF_LoadCompleted, std::memory_order_acquire))
		{
			return;
		}
	}

	FPackageImportReference PackageImportRef = ObjectIndex.ToPackageImportRef();
	FAsyncPackage2* SourcePackage = Header.ImportedAsyncPackagesView[PackageImportRef.GetImportedPackageIndex()];
	if (!SourcePackage)
	{
		return;
	}
	uint64 ExportHash = Header.ImportedPublicExportHashes[PackageImportRef.GetImportedPublicExportHashIndex()];
	FAsyncPackageHeaderData* SourcePackageHeader = nullptr;

	// If the header ref is at 0, it means all the serialization is done for that package, we just raced with it and missed the cue.
	if (SourcePackage->TryAddHeaderRef())
	{
		int32 ExportIndex = SourcePackage->GetPublicExportIndex(ExportHash, SourcePackageHeader);
		if (ExportIndex >= 0)
		{
			SourcePackage->ConditionalSerializeExport(*SourcePackageHeader, ExportIndex);
		}
		SourcePackage->ReleaseHeaderRef();
	}
}

int32 FAsyncPackage2::GetPublicCellExportIndexAsLocalIndex(uint64 CellExportHash, FAsyncPackageHeaderData*& OutHeader)
{
	for (int32 CellExportIndex = 0; CellExportIndex < HeaderData.CellExportMap.Num(); ++CellExportIndex)
	{
		if (HeaderData.CellExportMap[CellExportIndex].PublicExportHash == CellExportHash)
		{
			OutHeader = &HeaderData;
			return CellExportIndex + HeaderData.ExportMap.Num();
		}
	}
	return -1;
}

void FAsyncPackage2::ConditionalCreateCellExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex)
{
	const int32 CellExportIndex = LocalExportIndex - Header.ExportMap.Num();
	if (!Header.CellExportsView[CellExportIndex].Cell)
	{
		FAsyncPackageScope2 Scope(this);
		EventDrivenCreateCellExport(Header, LocalExportIndex, nullptr);
	}
}

void FAsyncPackage2::ConditionalSerializeCellExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex)
{
	ConditionalCreateCellExport(Header, LocalExportIndex);

	const int32 CellExportIndex = LocalExportIndex - Header.ExportMap.Num();
	if (!Header.CellExportsView[CellExportIndex].bSerialized)
	{
		FAsyncPackageScope2 Scope(this);
		EventDrivenSerializeCellExport(Header, LocalExportIndex, nullptr);
	}
}

void FAsyncPackage2::ConditionalCreateCellImport(const FAsyncPackageHeaderData& Header, int32 LocalImportIndex)
{
	const int32 CellImportIndex = LocalImportIndex - Header.ImportMap.Num();
	const FPackageObjectIndex& CellIndex = Header.CellImportMap[CellImportIndex];
	check(CellIndex.IsPackageImport());
	if (Verse::VCell* FromImportStore = ImportStore.FindOrGetImportCell(Header, CellIndex))
	{
		return;
	}

	FPackageImportReference PackageImportRef = CellIndex.ToPackageImportRef();
	FAsyncPackage2* SourcePackage = Header.ImportedAsyncPackagesView[PackageImportRef.GetImportedPackageIndex()];
	if (!SourcePackage)
	{
		return;
	}
	uint64 CellExportHash = Header.ImportedPublicExportHashes[PackageImportRef.GetImportedPublicExportHashIndex()];
	FAsyncPackageHeaderData* SourcePackageHeader = nullptr;

	// If the header ref is at 0, it means all the serialization is done for that package, we just raced with it and missed the cue.
	if (SourcePackage->TryAddHeaderRef())
	{
		const int32 LocalExportIndex = SourcePackage->GetPublicCellExportIndexAsLocalIndex(CellExportHash, SourcePackageHeader);
		if (LocalExportIndex >= 0)
		{
			SourcePackage->ConditionalCreateCellExport(*SourcePackageHeader, LocalExportIndex);
		}
		SourcePackage->ReleaseHeaderRef();
	}
}

void FAsyncPackage2::ConditionalSerializeCellImport(const FAsyncPackageHeaderData& Header, int32 LocalImportIndex)
{
	const int32 CellImportIndex = LocalImportIndex - Header.ImportMap.Num();
	const FPackageObjectIndex& CellIndex = Header.CellImportMap[CellImportIndex];
	check(CellIndex.IsPackageImport());

	FPackageImportReference PackageImportRef = CellIndex.ToPackageImportRef();
	FAsyncPackage2* SourcePackage = Header.ImportedAsyncPackagesView[PackageImportRef.GetImportedPackageIndex()];
	if (!SourcePackage)
	{
		return;
	}
	uint64 CellExportHash = Header.ImportedPublicExportHashes[PackageImportRef.GetImportedPublicExportHashIndex()];
	FAsyncPackageHeaderData* SourcePackageHeader = nullptr;

	// If the header ref is at 0, it means all the serialization is done for that package, we just raced with it and missed the cue.
	if (SourcePackage->TryAddHeaderRef())
	{
		const int32 LocalExportIndex = SourcePackage->GetPublicCellExportIndexAsLocalIndex(CellExportHash, SourcePackageHeader);
		if (LocalExportIndex >= 0)
		{
			SourcePackage->ConditionalSerializeCellExport(*SourcePackageHeader, LocalExportIndex);
		}
		SourcePackage->ReleaseHeaderRef();
	}
}

FString FAsyncPackage2::GetNameFromPackageObjectIndex(const FAsyncPackageHeaderData& Header, FPackageObjectIndex Index)
{
	FString Out;
	if (Index.IsExport())
	{
		if (LinkerLoadState.IsSet())
		{
			Out = LinkerLoadState->Linker->ExportMap[Index.ToExport()].ObjectName.ToString();
		}
		else
		{
			FName Name(NAME_None);
			Header.NameMap.TryGetName(Header.ExportMap[Index.ToExport()].ObjectName, Name);
			Out = Name.ToString();
		}
	}
	else if (Index.IsImport())
	{
		Out = ImportStore.GetImportName(Header, Index);
	}

	return Out;
}

namespace AsyncLoading2::Private
{

class FExcludedClassSet
{
	FName           Name;
	TSet<FString>   ExcludedClassNames;
	TSet<UClass*>   ExcludedClasses;
	TSet<UClass*>   ExcludedNativeClasses;
	FRWLock         Lock;
	uint64          ClassesVersionNumber = 0;
	FDelegateHandle ConfigDelegate;

	void UpdateClasses()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FExcludedClassSet::UpdateClasses);

		FWriteScopeLock ScopeLock(Lock);

		uint64 CurrentVersion = GetRegisteredClassesVersionNumber();

		if (ExcludedClassNames.Num())
		{
			ExcludedClasses.Reset();

			for (auto It = ExcludedClassNames.CreateIterator(); It; ++It)
			{
				if (UClass* Class = FindObject<UClass>(nullptr, *It))
				{
					if (Class->IsNative())
					{
						ExcludedNativeClasses.Add(Class);

						// We don't need to look at native classes ever again
						It.RemoveCurrent();
					}
					else
					{
						ExcludedClasses.Add(Class);
					}
				}
			}
		}

		ClassesVersionNumber = CurrentVersion;
	}

	void ResetNoLock()
	{
		ExcludedClassNames.Reset();
		ExcludedClasses.Reset();
		ExcludedNativeClasses.Reset();
		ClassesVersionNumber = 0;
	}

	void OnConfigSectionChanged()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FExcludedClassSet::OnConfigSectionChanged);

		FWriteScopeLock ScopeLock(Lock);

		ResetNoLock();

		if (GConfig && GConfig->IsReadyForUse())
		{
			const TCHAR* SectionName = FPlatformProperties::RequiresCookedData() ? TEXT("/Script/Engine.StreamingSettings") : TEXT("/Script/Engine.EditorStreamingSettings");

			if (const FConfigSection* ConfigSection = GConfig->GetSection(SectionName, false, GEngineIni))
			{
				for (const auto& It : *ConfigSection)
				{
					if (It.Key == Name)
					{
						ExcludedClassNames.Add(It.Value.GetValue());
					}
				}
			}
		}
	}
public:
	FExcludedClassSet(const TCHAR* InName)
		: Name(InName)
	{
		ConfigDelegate = FCoreDelegates::TSOnConfigSectionsChanged().AddLambda(
			[this](const FString&, const TSet<FString>&)
			{
				OnConfigSectionChanged();
			}
		);

		OnConfigSectionChanged();
	}

	~FExcludedClassSet()
	{
		FCoreDelegates::TSOnConfigSectionsChanged().Remove(ConfigDelegate);
	}

	bool IsExcluded(UClass* Class)
	{
		while (true)
		{
			UE::TReadScopeLock ScopeLock(Lock);

			uint64 CurrentVersion = GetRegisteredClassesVersionNumber();
			if (ClassesVersionNumber != CurrentVersion)
			{
				// In case classes need to be updated, we have to release the read lock and proceed
				ScopeLock.ReadUnlock();

				UpdateClasses();

				continue;
			}

			for (UClass* ExcludedClass : ExcludedClasses)
			{
				if (Class->IsChildOf(ExcludedClass))
				{
					return true;
				}
			}

			for (UClass* ExcludedClass : ExcludedNativeClasses)
			{
				if (Class->IsChildOf(ExcludedClass))
				{
					return true;
				}
			}

			return false;
		}
	}
};

} // namespace AsyncLoading2::Private

bool FAsyncPackage2::CanRunNodeAsync(const FAsyncLoadEventSpec* Spec) const
{
	if (Spec->Func == &FAsyncPackage2::Event_PreloadExports)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncPackage2::CanRunNodeAsync);

		static AsyncLoading2::Private::FExcludedClassSet ParallelPreloadExclusions(TEXT("ParallelPreloadExclusions"));

		const int32 ExportCount = Data.Exports.Num();
		for (int32 ExportIndex = 0; ExportIndex < ExportCount; ++ExportIndex)
		{
			FExportObject& Export = Data.Exports[ExportIndex];

			if (Export.Object)
			{
				if (ParallelPreloadExclusions.IsExcluded(Export.Object->GetClass()))
				{
					return false;
				}
			}
		}
	}

	return true;
}

class FSerializationMutex
{
private:
	UE::FMutex& Mutex;
public:
	FSerializationMutex(UE::FMutex& InMutex)
		: Mutex(InMutex)
	{
		if (!Mutex.TryLock())
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SerializationMutexContention);
			Mutex.Lock();
		}
	}

	~FSerializationMutex()
	{
		Mutex.Unlock();
	}
};

void FAsyncPackage2::EventDrivenCreateExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(CreateExport);

	const FExportMapEntry& Export = Header.ExportMap[LocalExportIndex];
	FExportObject& ExportObject = Header.ExportsView[LocalExportIndex];
	UObject*& Object = ExportObject.Object;

	TRACE_LOADTIME_CREATE_EXPORT_SCOPE(this, &Object);

	FName ObjectName;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ObjectNameFixup);
		ObjectName = Header.NameMap.GetName(Export.ObjectName);
	}

	ExportObject.bFiltered = AsyncLoading2_ShouldSkipLoadingExport(Export.FilterFlags);
	if (ExportObject.bFiltered || ExportObject.bExportLoadFailed)
	{
		if (ExportObject.bExportLoadFailed)
		{
			UE_ASYNC_PACKAGE_LOG(Warning, Desc, TEXT("CreateExport"), TEXT("Skipped failed export %s"), *ObjectName.ToString());
		}
		else
		{
			UE_ASYNC_PACKAGE_LOG_VERBOSE(Verbose, Desc, TEXT("CreateExport"), TEXT("Skipped filtered export %s"), *ObjectName.ToString());
		}
		return;
	}

	ProcessExportDependencies(Header, LocalExportIndex, FExportBundleEntry::ExportCommandType_Create);

	FSerializationMutex ScopeLock(SerializationMutex);

	// Another thread might have done it for us while we were waiting...
	if (Object)
	{
		return;
	}

	bool bIsCompleteyLoaded = false;
	UClass* LoadClass = Export.ClassIndex.IsNull() ? UClass::StaticClass() : CastEventDrivenIndexToObject<UClass>(Header, Export.ClassIndex, true);
	UObject* ThisParent = Export.OuterIndex.IsNull() ? LinkerRoot : EventDrivenIndexToObject(Header, Export.OuterIndex, false);

	if (!LoadClass)
	{
		UE_ASYNC_PACKAGE_LOG(Error, Desc, TEXT("CreateExport"), TEXT("Could not find class object (0x%llX - '%s') for %s"), Export.ClassIndex.Value(), *GetNameFromPackageObjectIndex(Header, Export.ClassIndex), *ObjectName.ToString());
		ExportObject.bExportLoadFailed = true;
		return;
	}
	if (!ThisParent)
	{
		UE_ASYNC_PACKAGE_LOG(Error, Desc, TEXT("CreateExport"), TEXT("Could not find outer (0x%llX - '%s') object for %s"), Export.OuterIndex.Value(), *GetNameFromPackageObjectIndex(Header, Export.OuterIndex), *ObjectName.ToString());
		ExportObject.bExportLoadFailed = true;
		return;
	}
	check(!dynamic_cast<UObjectRedirector*>(ThisParent));
	if (!Export.SuperIndex.IsNull())
	{
		ExportObject.SuperObject = EventDrivenIndexToObject(Header, Export.SuperIndex, false);
		if (!ExportObject.SuperObject)
		{
			UE_ASYNC_PACKAGE_LOG(Error, Desc, TEXT("CreateExport"), TEXT("Could not find SuperStruct object (0x%llX - '%s') for %s"), Export.SuperIndex.Value(), *GetNameFromPackageObjectIndex(Header, Export.SuperIndex), *ObjectName.ToString());
			ExportObject.bExportLoadFailed = true;
			return;
		}
	}
	// Find the Archetype object for the one we are loading.
	check(!Export.TemplateIndex.IsNull());
	ExportObject.TemplateObject = EventDrivenIndexToObject(Header, Export.TemplateIndex, true);
	if (!ExportObject.TemplateObject)
	{
		UE_ASYNC_PACKAGE_LOG(Error, Desc, TEXT("CreateExport"), TEXT("Could not find template object (0x%llX - '%s') for %s"), Export.TemplateIndex.Value(), *GetNameFromPackageObjectIndex(Header, Export.TemplateIndex), *ObjectName.ToString());
		ExportObject.bExportLoadFailed = true;
		return;
	}

	if ((Export.ObjectFlags & RF_ClassDefaultObject) == 0
		&& !ExportObject.TemplateObject->IsA(LoadClass))
	{
		UE_ASYNC_PACKAGE_LOG(Error, Desc, TEXT("CreateExport"), TEXT("Export class type (%s) differs from the template object type (%s)"),
			*(LoadClass->GetFullName()),
			*(ExportObject.TemplateObject->GetClass()->GetFullName()));
		ExportObject.bExportLoadFailed = true;
		return;
	}

	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetLinkerRoot(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(LoadClass, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(ObjectName, LoadClass->GetFName(), GetLinkerRoot()->GetFName());

	// Try to find existing object first as we cannot in-place replace objects, could have been created by other export in this package
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FindExport);
		Object = StaticFindObjectFastInternal(nullptr, ThisParent, ObjectName, EFindObjectFlags::ExactClass);
	}

	const bool bIsNewObject = !Object;

	// Object is found in memory.
	if (Object)
	{
		// If it has the AsyncLoading flag set it was created during the current load of this package (likely as a subobject)
		if (!Object->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
		{
			ExportObject.bWasFoundInMemory = true;
		}
		// If this object was allocated but never loaded (components created by a constructor, CDOs, etc) make sure it gets loaded
		// Do this for all subobjects created in the native constructor.
		const EObjectFlags ObjectFlags = Object->GetFlags();
		bIsCompleteyLoaded = !!(ObjectFlags & RF_LoadCompleted);
		if (!bIsCompleteyLoaded)
		{
			check(!(ObjectFlags & (RF_NeedLoad | RF_WasLoaded))); // If export exist but is not completed, it is expected to have been created from a native constructor and not from EventDrivenCreateExport, but who knows...?
			if (ObjectFlags & RF_ClassDefaultObject)
			{
				// never call PostLoadSubobjects on class default objects, this matches the behavior of the old linker where
				// StaticAllocateObject prevents setting of RF_NeedPostLoad and RF_NeedPostLoadSubobjects, but FLinkerLoad::Preload
				// assigns RF_NeedPostLoad for blueprint CDOs:
				Object->SetFlags(RF_NeedLoad | RF_NeedPostLoad | RF_WasLoaded);
			}
			else
			{
				Object->SetFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_WasLoaded);
			}
		}
	}
	else
	{
		// we also need to ensure that the template has set up any instances
		ExportObject.TemplateObject->ConditionalPostLoadSubobjects();

		check(!GVerifyObjectReferencesOnly); // not supported with the event driven loader
		// Create the export object, marking it with the appropriate flags to
		// indicate that the object's data still needs to be loaded.
		EObjectFlags ObjectLoadFlags = Export.ObjectFlags;
		ObjectLoadFlags = EObjectFlags(ObjectLoadFlags | RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_WasLoaded);

		// If we are about to create a CDO, we need to ensure that all parent sub-objects are loaded
		// to get default value initialization to work.
#if DO_CHECK
		if ((ObjectLoadFlags & RF_ClassDefaultObject) != 0)
		{
			UClass* SuperClass = LoadClass->GetSuperClass();
			UObject* SuperCDO = SuperClass ? SuperClass->GetDefaultObject() : nullptr;
			check(!SuperCDO || ExportObject.TemplateObject == SuperCDO); // the template for a CDO is the CDO of the super
			if (SuperClass && !SuperClass->IsNative())
			{
				check(SuperCDO);
				if (SuperClass->HasAnyFlags(RF_NeedLoad))
				{
					UE_LOGF(LogStreaming, Fatal, "Super %ls had RF_NeedLoad while creating %ls", *SuperClass->GetFullName(), *ObjectName.ToString());
					return;
				}
				if (SuperCDO->HasAnyFlags(RF_NeedLoad))
				{
					UE_LOGF(LogStreaming, Fatal, "Super CDO %ls had RF_NeedLoad while creating %ls", *SuperCDO->GetFullName(), *ObjectName.ToString());
					return;
				}
				TArray<UObject*> SuperSubObjects;
				GetObjectsWithOuter(SuperCDO, SuperSubObjects, EGetObjectsFlags::None, /*ExclusionFlags=*/ RF_NoFlags, /*InternalExclusionFlags=*/ EInternalObjectFlags::Native);

				for (UObject* SubObject : SuperSubObjects)
				{
					if (SubObject->HasAnyFlags(RF_NeedLoad))
					{
						UE_LOGF(LogStreaming, Fatal, "Super CDO subobject %ls had RF_NeedLoad while creating %ls", *SubObject->GetFullName(), *ObjectName.ToString());
						return;
					}
				}
			}
			else
			{
				checkf(ExportObject.TemplateObject->IsA(LoadClass),
					TEXT("ExportObject.TemplateObject: '%s' is not a child of class: '%s'"),
					*GetNameSafe(ExportObject.TemplateObject),
					*GetNameSafe(LoadClass));
			}
		}
#endif
		checkf(!LoadClass->HasAnyFlags(RF_NeedLoad),
			TEXT("LoadClass %s had RF_NeedLoad while creating %s"), *LoadClass->GetFullName(), *ObjectName.ToString());
		checkf(!(LoadClass->GetDefaultObject() && LoadClass->GetDefaultObject()->HasAnyFlags(RF_NeedLoad)),
			TEXT("Class CDO %s had RF_NeedLoad while creating %s"), *LoadClass->GetDefaultObject()->GetFullName(), *ObjectName.ToString());
		checkf(!ExportObject.TemplateObject->HasAnyFlags(RF_NeedLoad),
			TEXT("Template %s had RF_NeedLoad while creating %s"), *ExportObject.TemplateObject->GetFullName(), *ObjectName.ToString());

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConstructObject);
			FStaticConstructObjectParameters Params(LoadClass);
			Params.Outer = ThisParent;
			Params.Name = ObjectName;
			Params.SetFlags = ObjectLoadFlags;
			Params.Template = ExportObject.TemplateObject;
			Params.bAssumeTemplateIsArchetype = true;
			Object = StaticConstructObject_Internal(Params);
		}

		if (GIsInitialLoad || GUObjectArray.IsOpenForDisregardForGC())
		{
			Object->AddToRoot();
		}

		check(Object->GetClass() == LoadClass);
		check(Object->GetFName() == ObjectName);
	}

	check(Object);
	EInternalObjectFlags FlagsToSet = EInternalObjectFlags::Async;

	if (Export.OuterIndex.IsImport())
	{
		Object->SetExternalPackage(LinkerRoot);
	}

	if (Desc.bCanBeImported && Export.PublicExportHash)
	{
		FlagsToSet |= EInternalObjectFlags::LoaderImport;
		ImportStore.StoreGlobalObject(Desc.UPackageId, Export.PublicExportHash, Object);

		UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("CreateExport"), TEXT("Created %s export %s. Tracked as %s:0x%llX"),
			Object->HasAnyFlags(RF_Public) ? TEXT("public") : TEXT("private"), *Object->GetPathName(), *FormatPackageId(Desc.UPackageId), Export.PublicExportHash);
	}
	else
	{
		UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("CreateExport"), TEXT("Created %s export %s. Not tracked."),
			Object->HasAnyFlags(RF_Public) ? TEXT("public") : TEXT("private"), *Object->GetPathName());
	}
	Object->SetInternalFlags(FlagsToSet);
}

void FAsyncPackage2::EventDrivenSerializeExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex, FExportArchive* Ar)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_FAsyncPackage2);
	TRACE_CPUPROFILER_EVENT_SCOPE(SerializeExport);

	const FExportMapEntry& Export = Header.ExportMap[LocalExportIndex];
	FExportObject& ExportObject = Header.ExportsView[LocalExportIndex];
	UObject* Object = ExportObject.Object;
	check(Object || (ExportObject.bFiltered || ExportObject.bExportLoadFailed));

	TRACE_LOADTIME_SERIALIZE_EXPORT_SCOPE(Object, Export.CookedSerialSize);

	if (ExportObject.bFiltered || ExportObject.bExportLoadFailed || Object == nullptr)
	{
		if (ExportObject.bExportLoadFailed)
		{
			UE_ASYNC_PACKAGE_LOG(Warning, Desc, TEXT("SerializeExport"),
				TEXT("Skipped failed export %s"), *Header.NameMap.GetName(Export.ObjectName).ToString());
		}
		else if (ExportObject.bFiltered)
		{
			UE_ASYNC_PACKAGE_LOG_VERBOSE(Verbose, Desc, TEXT("SerializeExport"),
				TEXT("Skipped filtered export %s"), *Header.NameMap.GetName(Export.ObjectName).ToString());
		}
		else
		{
			UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("SerializeExport"),
				TEXT("Skipped nullptr object for export %s"), *Header.NameMap.GetName(Export.ObjectName).ToString());
		}
		return;
	}

	// We cannot test RF_NeedLoad since it is removed before we start serializing.
	// Since under multithreaded mode, we might race, we always enter the serialization
	// race unless we are positive the serialization is completed.
	if (Object->HasAnyFlags(RF_LoadCompleted, std::memory_order_acquire))
	{
		UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("SerializeExport"),
			TEXT("Skipped already serialized export %s"), *Header.NameMap.GetName(Export.ObjectName).ToString());
		return;
	}

	ProcessExportDependencies(Header, LocalExportIndex, FExportBundleEntry::ExportCommandType_Serialize);

	FSerializationMutex ScopeLock(SerializationMutex);

	// Another thread might have done it for us while we were waiting...
	if (ExportObject.Object && !ExportObject.Object->HasAnyFlags(RF_NeedLoad))
	{
		return;
	}

	TOptional<FExportArchive> LocalAr;
	if (!Ar)
	{
#if WITH_EDITOR
		if (&Header == OptionalSegmentHeaderData.GetPtrOrNull())
		{
			Ar = &LocalAr.Emplace(OptionalSegmentSerializationState->IoRequest.GetResultOrDie());
			InitializeExportArchive(*Ar, true);
		}
		else
#endif
		{
			Ar = &LocalAr.Emplace(SerializationState.IoRequest.GetResultOrDie());
			InitializeExportArchive(*Ar, false);
		}
	}

	// If this is a struct, make sure that its parent struct is completely loaded
	if (UStruct* Struct = dynamic_cast<UStruct*>(Object))
	{
		if (UStruct* SuperStruct = dynamic_cast<UStruct*>(ExportObject.SuperObject))
		{
			Struct->SetSuperStruct(SuperStruct);
			if (UClass* ClassObject = dynamic_cast<UClass*>(Object))
			{
				ClassObject->Bind();
			}
		}
	}

	const UClass* LoadClass = Export.ClassIndex.IsNull() ? UClass::StaticClass() : CastEventDrivenIndexToObject<UClass>(Header, Export.ClassIndex, true);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object->GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(LoadClass, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET(Object, LoadClass);

	// cache archetype
	// prevents GetArchetype from hitting the expensive GetArchetypeFromRequiredInfoImpl
	check(ExportObject.TemplateObject);
	FObjectArchetypeHelper::CacheArchetypeForObject(Object, ExportObject.TemplateObject);

	Object->ClearFlags(RF_NeedLoad);

	FUObjectSerializeContext* LoadContext = GetSerializeContext();
	UObject* PrevSerializedObject = LoadContext->SerializedObject;
	LoadContext->SerializedObject = Object;

	Ar->ExportBufferBegin(Object, Export.CookedSerialOffset, Export.CookedSerialSize);

	const int64 Pos = Ar->Tell();

	check(!Ar->TemplateForGetArchetypeFromLoader);
	Ar->TemplateForGetArchetypeFromLoader = ExportObject.TemplateObject;

	if (Object->HasAnyFlags(RF_ClassDefaultObject))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SerializeDefaultObject);
		Object->GetClass()->SerializeDefaultObject(Object, *Ar);
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SerializeObject);
		UE_SERIALIZE_ACCCESS_SCOPE(Object);
		Object->Serialize(*Ar);
	}
	Ar->TemplateForGetArchetypeFromLoader = nullptr;

	const uint64 ReadBytes = uint64(Ar->Tell() - Pos);
	if (Ar->IsError())
	{
		const uint64 SeekBytes = Export.CookedSerialSize - ReadBytes;
		Ar->Seek(SeekBytes);
		Ar->ClearError();
		UE_ASYNC_PACKAGE_LOG(Error, Desc, TEXT("ObjectSerializationError"),
			TEXT("%s: Serialization error handled, %lld of %lld bytes was read, %lld bytes was seeked. The serialized object may be in a bad state."),
			Object ? *Object->GetFullName() : TEXT("null"), ReadBytes, Export.CookedSerialSize, SeekBytes);
	}
	else
	{
		UE_ASYNC_PACKAGE_CLOG(
			Export.CookedSerialSize != ReadBytes, Fatal, Desc, TEXT("ObjectSerializationError"),
			TEXT("%s: Serial size mismatch: Expected read size %lld, Actual read size %lld"),
			Object ? *Object->GetFullName() : TEXT("null"), Export.CookedSerialSize, ReadBytes);
	}

	Ar->ExportBufferEnd();

	Object->SetFlags(RF_LoadCompleted);
	LoadContext->SerializedObject = PrevSerializedObject;

	if (Object->HasAnyFlags(RF_ClassDefaultObject) && Object->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		if (CVar_CheckBlueprintCDOWillBePostLoadedAfterPreload.GetValueOnAnyThread())
		{
			checkObject(Object, Object->HasAllFlags(RF_NeedPostLoad | RF_WasLoaded));
		}
		else
		{
			ensureObject(Object, Object->HasAllFlags(RF_NeedPostLoad | RF_WasLoaded));
		}
	}

	UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("SerializeExport"), TEXT("Serialized export %s"), *Object->GetPathName());

	// push stats so that we don't overflow number of tags per thread during blocking loading
	LLM_PUSH_STATS_FOR_ASSET_TAGS();
}

void FAsyncPackage2::EventDrivenCreateCellExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex, FExportArchive* Ar)
{
	const int32 CellExportIndex = LocalExportIndex - Header.ExportMap.Num();
	const FCellExportMapEntry& Export = Header.CellExportMap[CellExportIndex];
	FExportCell& ExportCell = Header.CellExportsView[CellExportIndex];

	ProcessExportDependencies(Header, LocalExportIndex, FExportBundleEntry::ExportCommandType_Create);

	FSerializationMutex ScopeLock(SerializationMutex);

	// Another thread might have done it for us while we were waiting...
	if (ExportCell.Cell)
	{
		return;
	}

	Verse::VCell* Cell = ExportCell.Cell;
	TOptional<FExportArchive> LocalAr;
	if (!Ar)
	{
		Ar = &LocalAr.Emplace(SerializationState.IoRequest.GetResultOrDie());
		InitializeExportArchive(*Ar, false);
	}

	Ar->ExportBufferBegin(nullptr, Export.CookedSerialOffset, Export.CookedSerialLayoutSize);

	FName CppClassInfoName = Header.NameMap.GetName(Export.CppClassInfo);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FRunningContext Context = Verse::FRunningContextPromise{};
	Verse::VCppClassInfo* CppClassInfo = Verse::VCppClassInfoRegistry::GetCppClassInfo(*CppClassInfoName.ToString());
	FStructuredArchiveFromArchive StructuredArchive(*Ar);
	Verse::FStructuredArchiveVisitor Visitor(Context, StructuredArchive.GetSlot().EnterRecord());
	CppClassInfo->SerializeLayout(Context, Cell, Visitor);

	Context.RunWriteBarrier(Cell);
	ExportCell.Cell = Cell;
#endif

	Ar->ExportBufferEnd();

	if (Desc.bCanBeImported && Export.PublicExportHash)
	{
		ImportStore.StoreGlobalCell(Desc.UPackageId, Export.PublicExportHash, Cell);
	}
}

void FAsyncPackage2::EventDrivenSerializeCellExport(const FAsyncPackageHeaderData& Header, int32 LocalExportIndex, FExportArchive* Ar)
{
	AutoRTFM::UnreachableIfClosed();

	const int32 CellExportIndex = LocalExportIndex - Header.ExportMap.Num();
	const FCellExportMapEntry& Export = Header.CellExportMap[CellExportIndex];
	FExportCell& ExportCell = Header.CellExportsView[CellExportIndex];
	Verse::VCell* Cell = ExportCell.Cell;
	check(Cell);

	ProcessExportDependencies(Header, LocalExportIndex, FExportBundleEntry::ExportCommandType_Serialize);

	FSerializationMutex ScopeLock(SerializationMutex);

	// Another thread might have done it for us while we were waiting...
	if (ExportCell.bSerialized)
	{
		return;
	}

	TOptional<FExportArchive> LocalAr;
	if (!Ar)
	{
		Ar = &LocalAr.Emplace(SerializationState.IoRequest.GetResultOrDie());
		InitializeExportArchive(*Ar, false);
	}

	ExportCell.bSerialized = true;

	Ar->ExportBufferBegin(nullptr, Export.CookedSerialOffset, Export.CookedSerialSize);
	Ar->Skip(Export.CookedSerialLayoutSize);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
	FStructuredArchiveFromArchive StructuredArchive(*Ar);
	Verse::FStructuredArchiveVisitor Visitor(Context, StructuredArchive.GetSlot().EnterRecord());
	Cell->Serialize(Context, Visitor);
#endif

	Ar->ExportBufferEnd();
}

FAsyncPackage2* FAsyncPackage2::GetCurrentlyExecutingPackage(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* PackageToFilter)
{
	FAsyncPackage2* CurrentlyExecutingPackage = nullptr;
	for (int32 Index = ThreadState.CurrentlyExecutingEventNodeStack.Num() - 1; Index >= 0; --Index)
	{
		FAsyncPackage2* Package = ThreadState.CurrentlyExecutingEventNodeStack[Index]->GetPackage();
		if (Package != nullptr && Package != PackageToFilter)
		{
			CurrentlyExecutingPackage = Package;
			break;
		}
	}
	return CurrentlyExecutingPackage;
}

void FAsyncPackage2::ConditionalReleasePartialRequests(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ConditionalReleasePartialRequests);

	// We need to look at all syncload context now that its possible for any threads to push a context as they are not strictly recursive anymore.
	// This means the request we're trying to satisfy is not necessarily on top of the sync context stack anymore.
	for (FAsyncLoadingSyncLoadContext* SyncLoadContext : ThreadState.SyncLoadContextStack)
	{
		if (FAsyncPackage2* RequestingPackage = SyncLoadContext->GetRequestingPackage())
		{
			EAsyncPackageLoadingState2 RequesterState = RequestingPackage->AsyncPackageLoadingState;

			if (AsyncPackageLoadingState > RequesterState)
			{
				int32 Index = SyncLoadContext->FindRequestedPackage(this);
				if (Index != INDEX_NONE)
				{
					int32 RequestId = SyncLoadContext->RequestIDs[Index];
					// Release the sync context request tied to our package, allowing the current flush to exit.
					UE_LOGF(LogStreaming, Log, "Package %ls has reached state %ls > %ls, releasing request %d to allow recursive sync load to finish",
						*Desc.UPackageName.ToString(),
						LexToString(AsyncPackageLoadingState),
						LexToString(RequesterState),
						RequestId
					);
					AsyncLoadingThread.RemovePendingRequests(ThreadState, { RequestId });

#if WITH_EDITOR
					// If the package we release hasn't been postloaded yet, we'll prevent it from being postloaded until
					// its blueprint classes have been compiled if we find any.
					FScopedClassDependencyGather* ClassDependencyGather = FScopedClassDependencyGather::GetAuthoritativeScope();
					if (ClassDependencyGather != nullptr && AsyncPackageLoadingState < EAsyncPackageLoadingState2::DeferredPostLoad)
					{
						UE_LOGF(LogStreaming, Display, "Package %ls postload will not proceed until its blueprint has been compiled",
							*Desc.UPackageName.ToString()
						);

						GetPackageNode(EEventLoadNode2::Package_DeferredPostLoad).AddBarrier();

						ClassDependencyGather->OnScopeExitDelegate.AddLambda(
							[this, &ThreadState]()
							{
								GetPackageNode(EEventLoadNode2::Package_DeferredPostLoad).ReleaseBarrier(&ThreadState);
							}
						);
					}
#endif

					switch (AsyncPackageLoadingState)
					{
					case EAsyncPackageLoadingState2::ExportsDone:
					{
						// As a general rule, we want any synchronous load during deserialization to act like an import in order
						// to make sure the requested package is part of the loading of the requesting package even if the
						// sync load was released to avoid a deadlock. This will make sure that by the time the requesting package
						// finishes loading, the requested package will be done too.
						bool bIsAlreadyInSet = false;
						RequestingPackage->AdditionalImportedAsyncPackages.FindOrAdd(this, &bIsAlreadyInSet);
						if (!bIsAlreadyInSet)
						{
							this->AddRef();

							UE_LOGF(LogStreaming, Display, "Package %ls is adding a dynamic import to package %ls because of a recursive sync load",
								*RequestingPackage->Desc.UPackageName.ToString(),
								*Desc.UPackageName.ToString()
							);
						}
					}
					break;
					case EAsyncPackageLoadingState2::PostLoad:
					[[fallthrough]];
					case EAsyncPackageLoadingState2::DeferredPostLoad:
					// We don't need to add a dynamic import if the package is already in postload since it wouldn't provide any benefits...
					break;
					default:
					// If anything else begins to call ConditionalReleasePartialRequests, we'll need to consider what needs to be done here
					// checkNoEntry();
					break;
					}
				}
			}
		}
	}
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_ExportsDone(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Event_ExportsDone);
	UE_ASYNC_PACKAGE_DEBUG(Package->Desc);

	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);

	check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::WaitingForExternalReads ||
		  Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::PreloadExports);

	Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::ExportsDone;

	if (!Package->HasLoadFailed() && Package->Desc.bCanBeImported)
	{
		FWriteLockedPackageRef PackageRef = Package->AsyncLoadingThread.GlobalImportStore.FindPackageRefCheckedForWrite(Package->Desc.UPackageId, Package->Desc.UPackageName);
		const bool bSnapshotExportCount = true;
		PackageRef->SetAllPublicExportsLoaded(bSnapshotExportCount);
	}

	if (!Package->Data.ShaderMapHashes.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ReleasePreloadedShaderMaps);
		FCoreDelegates::ReleasePreloadedPackageShaderMaps.ExecuteIfBound(Package->Data.ShaderMapHashes);
	}

	Package->CallProgressCallbacks();

	Package->ConditionalReleasePartialRequests(ThreadState);
	Package->ConditionalBeginPostLoad(ThreadState);
	return EEventLoadNodeExecutionResult::Complete;
}

#if ALT2_ENABLE_PACKAGE_DEPENDENCY_DEBUGGING
bool FAsyncPackage2::HasDependencyToPackageDebug(FAsyncPackage2* Package)
{
	TSet<FAsyncPackage2*> Visited;
	TArray<FAsyncPackage2*> Stack;
	for (FAsyncPackage2* ImportedPackage : Data.ImportedAsyncPackages)
	{
		if (ImportedPackage)
		{
			Stack.Push(ImportedPackage);
		}
	}
	while (!Stack.IsEmpty())
	{
		FAsyncPackage2* InnerPackage = Stack.Top();
		Stack.Pop();
		Visited.Add(InnerPackage);
		if (InnerPackage == Package)
		{
			return true;
		}
		for (FAsyncPackage2* ImportedPackage : InnerPackage->Data.ImportedAsyncPackages)
		{
			if (ImportedPackage && !Visited.Contains(ImportedPackage))
			{
				Stack.Push(ImportedPackage);
			}
		}
	}
	return false;
}

void FAsyncPackage2::CheckThatAllDependenciesHaveReachedStateDebug(FAsyncLoadingThreadState2& ThreadState, EAsyncPackageLoadingState2 PackageState, EAsyncPackageLoadingState2 PackageStateForCircularDependencies)
{
	TSet<FAsyncPackage2*> Visited;
	TArray<TTuple<FAsyncPackage2*, TArray<FAsyncPackage2*>>> Stack;

	TArray<FAsyncPackage2*> DependencyChain;
	DependencyChain.Add(this);
	Stack.Push(MakeTuple(this, DependencyChain));
	while (!Stack.IsEmpty())
	{
		TTuple<FAsyncPackage2*, TArray<FAsyncPackage2*>> PackageAndDependencyChain = Stack.Top();
		Stack.Pop();
		FAsyncPackage2* Package = PackageAndDependencyChain.Get<0>();
		DependencyChain = PackageAndDependencyChain.Get<1>();

		for (FAsyncPackage2* ImportedPackage : Package->Data.ImportedAsyncPackages)
		{
			if (ImportedPackage && !Visited.Contains(ImportedPackage) && !ThreadState.PackagesOnStack.Contains(Package))
			{
				TArray<FAsyncPackage2*> NextDependencyChain = DependencyChain;
				NextDependencyChain.Add(ImportedPackage);

				check(ImportedPackage->AsyncPackageLoadingState >= PackageStateForCircularDependencies);
				if (ImportedPackage->AsyncPackageLoadingState < PackageState)
				{
					bool bHasCircularDependencyToPackage = ImportedPackage->HasDependencyToPackageDebug(this);
					check(bHasCircularDependencyToPackage);
				}

				Visited.Add(ImportedPackage);
				Stack.Push(MakeTuple(ImportedPackage, NextDependencyChain));
			}
		}
	}
}
#endif

FAsyncPackage2* FAsyncPackage2::UpdateDependenciesStateRecursive(FAsyncLoadingThreadState2& ThreadState, FUpdateDependenciesStateRecursiveContext& Context)
{
	FAllDependenciesState& ThisState = this->*Context.StateMemberPtr;

	check(ThisState.PreOrderNumber < 0);

	if (ThisState.bAllDone)
	{
		return nullptr;
	}

	FAsyncPackage2* WaitingForPackage = ThisState.WaitingForPackage;
	if (WaitingForPackage)
	{
		if (WaitingForPackage->AsyncPackageLoadingState >= Context.WaitForPackageState)
		{
			FAllDependenciesState::RemoveFromWaitList(Context.StateMemberPtr, WaitingForPackage, this);
			WaitingForPackage = nullptr;
		}
		else if (ThreadState.PackagesOnStack.Contains(WaitingForPackage))
		{
			FAllDependenciesState::RemoveFromWaitList(Context.StateMemberPtr, WaitingForPackage, this);
			WaitingForPackage = nullptr;
		}
		else
		{
			return WaitingForPackage;
		}
	}

	ThisState.PreOrderNumber = Context.C;
	++Context.C;
	Context.S.Push(this);
	Context.P.Push(this);

	auto ProcessImportedPackage =
		[&ThreadState, &Context, &WaitingForPackage](FAsyncPackage2* ImportedPackage) -> bool
		{
			if (!ImportedPackage)
			{
				return true;
			}

			if (ThreadState.PackagesOnStack.Contains(ImportedPackage))
			{
				return true;
			}

			FAllDependenciesState& ImportedPackageState = ImportedPackage->*Context.StateMemberPtr;
			if (ImportedPackageState.bAllDone)
			{
				return true;
			}

			if (ImportedPackage->AsyncPackageLoadingState < Context.WaitForPackageState)
			{
				WaitingForPackage = ImportedPackage;
				return false;
			}

			ImportedPackageState.UpdateTick(Context.CurrentTick);
			if (ImportedPackageState.PreOrderNumber < 0)
			{
				WaitingForPackage = ImportedPackage->UpdateDependenciesStateRecursive(ThreadState, Context);
				if (WaitingForPackage)
				{
					return false;
				}
			}
			else if (!ImportedPackageState.bAssignedToStronglyConnectedComponent)
			{
				while ((Context.P.Top()->*Context.StateMemberPtr).PreOrderNumber > ImportedPackageState.PreOrderNumber)
				{
					Context.P.Pop();
				}
			}
			if (ImportedPackageState.WaitingForPackage)
			{
				WaitingForPackage = ImportedPackageState.WaitingForPackage;
				return false;
			}

			return true;
		};

	bool bContinueToProcessPackage = true;
	for (FAsyncPackage2* ImportedPackage : Data.ImportedAsyncPackages)
	{
		if (!ProcessImportedPackage(ImportedPackage))
		{
			bContinueToProcessPackage = false;
			break;
		}
	}

	if (bContinueToProcessPackage)
	{
		for (FAsyncPackage2* ImportedPackage : AdditionalImportedAsyncPackages)
		{
			if (!ProcessImportedPackage(ImportedPackage))
			{
				break;
			}
		}
	}

	if (Context.P.Top() == this)
	{
		FAsyncPackage2* InStronglyConnectedComponent;
		do
		{
			InStronglyConnectedComponent = Context.S.Pop();
			FAllDependenciesState& InStronglyConnectedComponentState = InStronglyConnectedComponent->*Context.StateMemberPtr;
			InStronglyConnectedComponentState.bAssignedToStronglyConnectedComponent = true;
			check(InStronglyConnectedComponent->AsyncPackageLoadingState >= Context.WaitForPackageState);
			if (WaitingForPackage)
			{
#if ALT2_ENABLE_PACKAGE_DEPENDENCY_DEBUGGING
				check(HasDependencyToPackageDebug(WaitingForPackage));
#endif
				FAllDependenciesState::AddToWaitList(Context.StateMemberPtr, WaitingForPackage, InStronglyConnectedComponent);
			}
			else
			{
				InStronglyConnectedComponentState.bAllDone = true;
#if ALT2_ENABLE_PACKAGE_DEPENDENCY_DEBUGGING
				InStronglyConnectedComponent->CheckThatAllDependenciesHaveReachedStateDebug(ThreadState, InStronglyConnectedComponent->AsyncPackageLoadingState, Context.WaitForPackageState);
#endif
				Context.OnStateReached(InStronglyConnectedComponent);
			}
		} while (InStronglyConnectedComponent != this);
		Context.P.Pop();
	}

	return WaitingForPackage;
}

void FAsyncPackage2::WaitForAllDependenciesToReachState(FAsyncLoadingThreadState2& ThreadState, FAllDependenciesState FAsyncPackage2::* StateMemberPtr, EAsyncPackageLoadingState2 WaitForPackageState, uint32& CurrentTickVariable, TFunctionRef<void(FAsyncPackage2*)> OnStateReached)
{
	check(AsyncPackageLoadingState == WaitForPackageState);
	++CurrentTickVariable;

	FUpdateDependenciesStateRecursiveContext Context(StateMemberPtr, WaitForPackageState, CurrentTickVariable, OnStateReached);

	FAllDependenciesState& ThisState = this->*StateMemberPtr;
	check(!ThisState.bAllDone);
	ThisState.UpdateTick(CurrentTickVariable);
	UpdateDependenciesStateRecursive(ThreadState, Context);
	check(ThisState.bAllDone || (ThisState.WaitingForPackage && ThisState.WaitingForPackage->AsyncPackageLoadingState < WaitForPackageState));

	while (FAsyncPackage2* WaitingPackage = ThisState.PackagesWaitingForThisHead)
	{
		FAllDependenciesState& WaitingPackageState = WaitingPackage->*StateMemberPtr;
		WaitingPackageState.UpdateTick(CurrentTickVariable);
		if (WaitingPackageState.PreOrderNumber < 0)
		{
			WaitingPackage->UpdateDependenciesStateRecursive(ThreadState, Context);
		}
		check(WaitingPackageState.bAllDone || (WaitingPackageState.WaitingForPackage && WaitingPackageState.WaitingForPackage->AsyncPackageLoadingState < WaitForPackageState));
	}
}

void FAsyncPackage2::ConditionalBeginProcessPackageExports(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ConditionalBeginProcessPackageExports);

	WaitForAllDependenciesToReachState(ThreadState, &FAsyncPackage2::AllDependenciesSetupState, EAsyncPackageLoadingState2::DependenciesReady, AsyncLoadingThread.ConditionalBeginProcessExportsTick,
		[&ThreadState](FAsyncPackage2* Package)
		{
			check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::DependenciesReady);

			Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::CreateExports;
			Package->GetPackageNode(EEventLoadNode2::Package_CreateExports).ReleaseBarrier(&ThreadState);
		});
}

void FAsyncPackage2::ConditionalFinishLoading(FAsyncLoadingThreadState2& ThreadState)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ConditionalFinishLoading);
	WaitForAllDependenciesToReachState(ThreadState, &FAsyncPackage2::AllDependenciesFullyLoadedState, EAsyncPackageLoadingState2::DeferredPostLoadDone, AsyncLoadingThread.ConditionalFinishLoadingTick,
		[&ThreadState](FAsyncPackage2* Package)
		{
			check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::DeferredPostLoadDone);
			Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::Finalize;
			Package->AsyncLoadingThread.LoadedPackagesToProcess.Add(Package);

			// Any update to LoadedPackagesToProcess is of interest to the main thread if we are on ALT.
			if (ThreadState.bIsAsyncLoadingThread)
			{
				Package->AsyncLoadingThread.MainThreadWakeEvent.Notify();
			}
		});
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_PostLoad(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(Event_PostLoad);
	UE_ASYNC_PACKAGE_DEBUG(Package->Desc);

	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::PostLoad);
	check(Package->ExternalReadDependencies.Num() == 0);

	FAsyncPackageScope2 PackageScope(Package);

	if (Package->LinkerLoadState.IsSet())
	{
		return Package->ExecutePostLoadLinkerLoadPackageExports(ThreadState);
	}

#if WITH_EDITOR
	UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Package->Desc.PIEInstanceID);
#endif

	EEventLoadNodeExecutionResult LoadingState = EEventLoadNodeExecutionResult::Complete;

	if (!Package->HasLoadFailed())
	{
		// Begin async loading, simulates BeginLoad
		Package->BeginAsyncLoad();

		SCOPED_LOADTIMER(PostLoadObjectsTime);
		TRACE_LOADTIME_POSTLOAD_SCOPE;

		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		TGuardValue<bool> GuardIsRoutingPostLoad(ThreadContext.IsRoutingPostLoad, true);

		const bool bAsyncPostLoadEnabled = FAsyncLoadingThreadSettings::Get().bAsyncPostLoadEnabled;
		const bool bIsMultithreaded = Package->AsyncLoadingThread.IsMultithreaded();

		while (Package->ProcessedExportBundlesCount < Package->Data.TotalExportBundleCount)
		{
#if WITH_EDITOR
			const FAsyncPackageHeaderData* HeaderData;
			if (Package->ProcessedExportBundlesCount == 1)
			{
				HeaderData = Package->OptionalSegmentHeaderData.GetPtrOrNull();
				check(HeaderData);
			}
			else
			{
				check(Package->ProcessedExportBundlesCount == 0);
				HeaderData = &Package->HeaderData;
			}
#else
			const FAsyncPackageHeaderData* HeaderData = &Package->HeaderData;
#endif

			while (Package->ExportBundleEntryIndex < HeaderData->ExportBundleEntriesCopyForPostLoad.Num())
			{
				const FExportBundleEntry& BundleEntry = HeaderData->ExportBundleEntriesCopyForPostLoad[Package->ExportBundleEntryIndex];
				if (ThreadState.IsTimeLimitExceeded(TEXT("Event_PostLoadExportBundle")))
				{
					LoadingState = EEventLoadNodeExecutionResult::Timeout;
					break;
				}

				if (BundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Serialize)
				{
					do
					{
						if (uint32(HeaderData->ExportsView.Num()) <= BundleEntry.LocalExportIndex)
						{
							break;
						}

						FExportObject& Export = HeaderData->ExportsView[BundleEntry.LocalExportIndex];
						if (Export.bFiltered || Export.bExportLoadFailed)
						{
							break;
						}

						UObject* Object = Export.Object;
						check(Object);
						checkObject(Object, !Object->HasAnyFlags(RF_NeedLoad));
						if (!Object->HasAnyFlags(RF_NeedPostLoad))
						{
							break;
						}

						check(Object->IsReadyForAsyncPostLoad());
						if (!bIsMultithreaded || (bAsyncPostLoadEnabled && CanPostLoadOnAsyncLoadingThread(Object)))
						{
#if WITH_EDITOR
							SCOPED_LOADTIMER_ASSET_TEXT(*Object->GetPathName());
#endif
							ThreadContext.CurrentlyPostLoadedObjectByALT = Object;
							Object->ConditionalPostLoad();
							ThreadContext.CurrentlyPostLoadedObjectByALT = nullptr;
						}
					} while (false);
				}
				++Package->ExportBundleEntryIndex;
			}

			if (LoadingState == EEventLoadNodeExecutionResult::Timeout)
			{
				break;
			}

			Package->ExportBundleEntryIndex = 0;
			Package->ProcessedExportBundlesCount++;
		}

		// End async loading, simulates EndLoad
		Package->EndAsyncLoad();
	}
	else
	{
		// On failure, just move to the next step
		Package->ProcessedExportBundlesCount = Package->Data.TotalExportBundleCount;
	}

	if (LoadingState == EEventLoadNodeExecutionResult::Timeout)
	{
		return LoadingState;
	}

	Package->ExportBundleEntryIndex = 0;

	if (Package->ProcessedExportBundlesCount == Package->Data.TotalExportBundleCount)
	{
		Package->ProcessedExportBundlesCount = 0;

		Package->ConditionalBeginDeferredPostLoad(ThreadState);
	}

	return EEventLoadNodeExecutionResult::Complete;
}

EEventLoadNodeExecutionResult FAsyncPackage2::Event_DeferredPostLoad(FAsyncLoadingThreadState2& ThreadState, FAsyncPackage2* Package)
{
	SCOPE_CYCLE_COUNTER(STAT_FAsyncPackage_PostLoadObjectsGameThread);
	TRACE_CPUPROFILER_EVENT_SCOPE(Event_DeferredPostLoad);
	UE_ASYNC_PACKAGE_DEBUG(Package->Desc);
	check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::DeferredPostLoad);

	check(FUObjectThreadContext::Get().AsyncVisibilityFilter == EInternalObjectFlags::None);
	// Prevent objects still being loaded from being visible from the game thread during postload.
	// This avoids StaticFind, ObjectIterators or SoftObjectPtr from being able to resolve RF_NeedLoad objects along with other potential
	// race condition during creation and serialization.
	TGuardValue GuardVisibilityFilter(FUObjectThreadContext::Get().AsyncVisibilityFilter, EInternalObjectFlags::AsyncLoadingPhase1);

	FAsyncPackageScope2 PackageScope(Package);
	TGuardValue<bool> GuardIsRoutingPostLoad(PackageScope.ThreadContext.IsRoutingPostLoad, true);

	if (Package->LinkerLoadState.IsSet())
	{
		return Package->ExecuteDeferredPostLoadLinkerLoadPackageExports(ThreadState);
	}

	EEventLoadNodeExecutionResult LoadingState = EEventLoadNodeExecutionResult::Complete;

	if (!Package->HasLoadFailed())
	{
		SCOPED_LOADTIMER(PostLoadDeferredObjectsTime);
		TRACE_LOADTIME_POSTLOAD_SCOPE;

		FAsyncLoadingTickScope2 InAsyncLoadingTick(Package->AsyncLoadingThread);

		while (Package->ProcessedExportBundlesCount < Package->Data.TotalExportBundleCount)
		{
#if WITH_EDITOR
			UE::Core::Private::FPlayInEditorLoadingScope PlayInEditorIDScope(Package->Desc.PIEInstanceID);

			const FAsyncPackageHeaderData* HeaderData;
			if (Package->ProcessedExportBundlesCount == 1)
			{
				HeaderData = Package->OptionalSegmentHeaderData.GetPtrOrNull();
				check(HeaderData);
			}
			else
			{
				check(Package->ProcessedExportBundlesCount == 0);
				HeaderData = &Package->HeaderData;
			}
#else
			const FAsyncPackageHeaderData* HeaderData = &Package->HeaderData;
#endif

			while (Package->ExportBundleEntryIndex < HeaderData->ExportBundleEntriesCopyForPostLoad.Num())
			{
				const FExportBundleEntry& BundleEntry = HeaderData->ExportBundleEntriesCopyForPostLoad[Package->ExportBundleEntryIndex];
				if (ThreadState.IsTimeLimitExceeded(TEXT("Event_DeferredPostLoadExportBundle")))
				{
					LoadingState = EEventLoadNodeExecutionResult::Timeout;
					break;
				}

				if (BundleEntry.CommandType == FExportBundleEntry::ExportCommandType_Serialize)
				{
					do
					{
						if (uint32(HeaderData->ExportsView.Num()) <= BundleEntry.LocalExportIndex)
						{
							break;
						}

						FExportObject& Export = HeaderData->ExportsView[BundleEntry.LocalExportIndex];
						if (Export.bFiltered || Export.bExportLoadFailed)
						{
							break;
						}

						UObject* Object = Export.Object;
						check(Object);
						checkObject(Object, !Object->HasAnyFlags(RF_NeedLoad));
						if (Object->HasAnyFlags(RF_NeedPostLoad))
						{
#if WITH_EDITOR
							SCOPED_LOADTIMER_ASSET_TEXT(*Object->GetPathName());
#endif
							PackageScope.ThreadContext.CurrentlyPostLoadedObjectByALT = Object;
							{
								FScopeCycleCounterUObject ConstructorScope(Object, GET_STATID(STAT_FAsyncPackage_PostLoadObjectsGameThread));
								LLM_SCOPE(ELLMTag::UObject);
								LLM_SCOPE_BYTAG(UObject_FAsyncPackage2);
								LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object->GetPackage(), ELLMTagSet::Assets);
								LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object->GetClass(), ELLMTagSet::AssetClasses);
								UE_TRACE_METADATA_SCOPE_ASSET(Object, Object->GetClass());
								Object->ConditionalPostLoad();
							}
							PackageScope.ThreadContext.CurrentlyPostLoadedObjectByALT = nullptr;
						}
					} while (false);
				}
				++Package->ExportBundleEntryIndex;
			}

			if (LoadingState == EEventLoadNodeExecutionResult::Timeout)
			{
				break;
			}

			Package->ExportBundleEntryIndex = 0;
			Package->ProcessedExportBundlesCount++;
		}
	}
	else
	{
		// On failure, just move to the next step
		Package->ProcessedExportBundlesCount = Package->Data.TotalExportBundleCount;
	}

	if (LoadingState == EEventLoadNodeExecutionResult::Timeout)
	{
		return LoadingState;
	}

	Package->ExportBundleEntryIndex = 0;

	if (Package->ProcessedExportBundlesCount == Package->Data.TotalExportBundleCount)
	{
		Package->ProcessedExportBundlesCount = 0;
		check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::DeferredPostLoad);
		Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::DeferredPostLoadDone;
		Package->ConditionalFinishLoading(ThreadState);
	}

	return EEventLoadNodeExecutionResult::Complete;
}

FEventLoadNode2& FAsyncPackage2::GetPackageNode(EEventLoadNode2 Phase)
{
	check(Phase < EEventLoadNode2::Package_NumPhases);
	return PackageNodes[Phase];
}

void FAsyncLoadingThread2::CleanupSyncLoadContext(FAsyncLoadingThreadState2& ThreadState)
{
	if (ThreadState.bIsAsyncLoadingThread)
	{
		while (!ThreadState.SyncLoadContextStack.IsEmpty())
		{
			FAsyncLoadingSyncLoadContext* SyncLoadContext = ThreadState.SyncLoadContextStack.Top();

			if (ContainsAnyRequestID(SyncLoadContext->RequestIDs))
			{
				// We still have some processing to do for this request, abort cleanup
				break;
			}

			// Request is fulfilled, release it
			UE_LOG(LogStreaming, VeryVerbose, TEXT("Popping ALT SyncLoadContext %" UINT64_FMT), SyncLoadContext->ContextId);
			SyncLoadContext->DoneEvent.Notify();
			SyncLoadContext->ReleaseRef();
			ThreadState.SyncLoadContextStack.Pop();
		}
	}
}

void FAsyncLoadingThread2::UpdateSyncLoadContext(FAsyncLoadingThreadState2& ThreadState, bool bAutoHandleSyncLoadContext)
{
	if (ThreadState.bIsAsyncLoadingThread && bAutoHandleSyncLoadContext)
	{
		// Retire complete/invalid contexts for which we aren't loading any requests
		// before pushing any more contexts on top
		CleanupSyncLoadContext(ThreadState);

		bool bRequiresSecondCleanup = false;
		FAsyncLoadingSyncLoadContext* CreatedOnOtherThread;
		while (ThreadState.SyncLoadContextsCreatedOnOtherThreads.Dequeue(CreatedOnOtherThread))
		{
			ThreadState.SyncLoadContextStack.Push(CreatedOnOtherThread);

			bRequiresSecondCleanup = true;
			UE_LOG(LogStreaming, VeryVerbose, TEXT("Pushing ALT SyncLoadContext %" UINT64_FMT), CreatedOnOtherThread->ContextId);
		}

		// Retire newly pushed complete/invalid contexts for which we aren't loading any requests
		if (bRequiresSecondCleanup)
		{
			CleanupSyncLoadContext(ThreadState);
		}
	}
	else
	{
		// Auto handling is only for async loading thread, all other threads manage
		// their sync context on their own.
		bAutoHandleSyncLoadContext = false;
	}

	if (ThreadState.SyncLoadContextStack.IsEmpty())
	{
		return;
	}

	FAsyncLoadingSyncLoadContext* SyncLoadContext = ThreadState.SyncLoadContextStack.Top();

	// When manually managed, we need to verify if the request on the stack is already fulfilled
	if (!bAutoHandleSyncLoadContext && !ContainsAnyRequestID(SyncLoadContext->RequestIDs))
	{
		return;
	}

	if (ThreadState.bCanAccessAsyncLoadingThreadData && !SyncLoadContext->bHasFoundRequestedPackages.load(std::memory_order_relaxed))
	{
		// Ensure that we've created the package we're waiting for
		CreateAsyncPackagesFromQueue(ThreadState);
		int32 FoundPackages = 0;
		for (int32 i = 0; i < SyncLoadContext->RequestIDs.Num(); ++i)
		{
			int32 RequestID = SyncLoadContext->RequestIDs[i];
			if (SyncLoadContext->IsRequestedPackageValid(i))
			{
				++FoundPackages;
			}
			else if (FAsyncPackage2* RequestedPackage = RequestIdToPackageMap.FindRef(RequestID))
			{
				// Set RequestedPackage before setting bHasFoundRequestedPackage so that another thread looking at RequestedPackage
				// after validating that bHasFoundRequestedPackage is true would see the proper value.
				SyncLoadContext->SetRequestedPackage(i, RequestedPackage);

				if (FAsyncPackage2* RequestingPackage = SyncLoadContext->GetRequestingPackage())
				{
					// If the flush is coming from a step before the requesting package is back on GT, there is no way to fully flush
					// the requested package unless its already done. We have no choice but to trigger partial loading in that case.
					if (RequestingPackage->AsyncPackageLoadingState < EAsyncPackageLoadingState2::DeferredPostLoad &&
						RequestedPackage->AsyncPackageLoadingState < EAsyncPackageLoadingState2::Complete)
					{
						LoaderStats.RecordPartialFlush(RequestedPackage, RequestingPackage);

						// Check if partial loading rules allow to release the package right now.
						RequestedPackage->ConditionalReleasePartialRequests(ThreadState);
					}
				}

				IncludePackageInSyncLoadContextRecursive(ThreadState, SyncLoadContext->ContextId, RequestedPackage);
				++FoundPackages;
			}
		}

		// Only set when full list is available 
		if (FoundPackages == SyncLoadContext->RequestIDs.Num())
		{
			SyncLoadContext->bHasFoundRequestedPackages.store(true, std::memory_order_release);
		}
	}
	if (SyncLoadContext->bHasFoundRequestedPackages.load(std::memory_order_acquire))
	{
		bool bMayHaveFulfilledRequest = false;
		for (int32 i = 0; i < SyncLoadContext->RequestIDs.Num(); ++i)
		{
			int32 RequestID = SyncLoadContext->RequestIDs[i];
			FAsyncPackage2* RequestedPackage = SyncLoadContext->GetRequestedPackage(i);
			if (RequestedPackage && ThreadState.PackagesOnStack.Contains(RequestedPackage))
			{
				// Flushing a package while it's already being processed on the stack, if we're done preloading we let it pass and remove the request id
				bool bPreloadIsDone = RequestedPackage->AsyncPackageLoadingState >= EAsyncPackageLoadingState2::DeferredPostLoad;
				UE_CLOGF(!bPreloadIsDone, LogStreaming, Warning, "Flushing package %ls while it's being preloaded in the same callstack is not possible. Releasing request %d to unblock.", *RequestedPackage->Desc.UPackageName.ToString(), RequestID);
				RemovePendingRequests(ThreadState, { RequestID });
				bMayHaveFulfilledRequest = true;
			}
		}

		if (bAutoHandleSyncLoadContext && bMayHaveFulfilledRequest)
		{
			CleanupSyncLoadContext(ThreadState);
		}
	}
}

EAsyncPackageState::Type FAsyncLoadingThread2::ProcessAsyncLoadingFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool& bDidSomething)
{
	SCOPED_LOADTIMER(AsyncLoadingTime);
	CSV_SCOPED_TIMING_STAT(FileIO, AsyncLoadingTime);

	check(IsInGameThread());

	// By default, game thread can only access objects in Phase 2. Further scoping is needed to access everything.
	TGuardValue GuardVisibilityFilter(FUObjectThreadContext::Get().AsyncVisibilityFilter, EInternalObjectFlags::AsyncLoadingPhase1);

	FAsyncLoadingTickScope2 InAsyncLoadingTick(*this);
	uint32 LoopIterations = 0;

	while (true)
	{
		do
		{
			if ((++LoopIterations) % 32 == 31)
			{
				// We're not multithreaded and flushing async loading
				// Update heartbeat after 32 events
				FThreadHeartBeat::Get().HeartBeat();
				FCoreDelegates::OnAsyncLoadingFlushUpdate.Broadcast();
			}

			if (ThreadState.IsTimeLimitExceeded(TEXT("ProcessAsyncLoadingFromGameThread")))
			{
				return EAsyncPackageState::TimeOut;
			}

			if (IsAsyncLoadingSuspended())
			{
				return EAsyncPackageState::TimeOut;
			}

			if (QueuedPackagesCounter || !PendingPackages.IsEmpty())
			{
				if (CreateAsyncPackagesFromQueue(ThreadState))
				{
					bDidSomething = true;
					break;
				}
				else
				{
					return EAsyncPackageState::TimeOut;
				}
			}

			if (!ThreadState.SyncLoadContextStack.IsEmpty() && ThreadState.SyncLoadContextStack.Top()->ContextId)
			{
				if (EventQueue.ExecuteSyncLoadEvents(ThreadState))
				{
					bDidSomething = true;
					break;
				}
			}
			else if (EventQueue.PopAndExecute(ThreadState))
			{
				bDidSomething = true;
				break;
			}

			if (!ExternalReadQueue.IsEmpty())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(WaitingForExternalReads);

				FAsyncPackage2* Package = nullptr;
				ExternalReadQueue.Dequeue(Package);

				EAsyncPackageState::Type Result = Package->ProcessExternalReads(ThreadState, FAsyncPackage2::ExternalReadAction_Wait);
				check(Result == EAsyncPackageState::Complete);

				bDidSomething = true;
				break;
			}

			if (ProcessDeferredCleanupPackagesQueue(1))
			{
				bDidSomething = true;
				break;
			}

			return EAsyncPackageState::Complete;
		} while (false);
	}
	check(false);
	return EAsyncPackageState::Complete;
}

EAsyncPackageState::Type FAsyncLoadingThread2::ProcessLoadedPackagesFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool& bDidSomething, TConstArrayView<int32> FlushRequestIDs)
{
	EAsyncPackageState::Type Result = EAsyncPackageState::Complete;

	if (IsMultithreaded() &&
		ENamedThreads::GetRenderThread() == ENamedThreads::GameThread &&
		!FTaskGraphInterface::Get().IsThreadProcessingTasks(ENamedThreads::GameThread)) // render thread tasks are actually being sent to the game thread.
	{
		// The async loading thread might have queued some render thread tasks (we don't have a render thread yet, so these are actually sent to the game thread)
		// We need to process them now before we do any postloads.
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
		if (ThreadState.IsTimeLimitExceeded(TEXT("ProcessLoadedPackagesFromGameThread")))
		{
			return EAsyncPackageState::TimeOut;
		}
	}

	TArray<FAsyncPackage2*, TInlineAllocator<4>> LocalCompletedAsyncPackages;
	for (;;)
	{
		FPlatformMisc::PumpEssentialAppMessages();

		if (ThreadState.IsTimeLimitExceeded(TEXT("ProcessAsyncLoadingFromGameThread")))
		{
			Result = EAsyncPackageState::TimeOut;
			break;
		}

		bool bLocalDidSomething = false;
		FAsyncPackage2* PackageToRepriortize;
		while (ThreadState.PackagesToReprioritize.Dequeue(PackageToRepriortize))
		{
			MainThreadEventQueue.UpdatePackagePriority(PackageToRepriortize);
			PackageToRepriortize->Release();
		}
		uint64 SyncLoadContextId = !ThreadState.SyncLoadContextStack.IsEmpty() ? ThreadState.SyncLoadContextStack.Top()->ContextId : 0;
		if (SyncLoadContextId)
		{
			bLocalDidSomething |= MainThreadEventQueue.ExecuteSyncLoadEvents(ThreadState);
		}
		else
		{
			bLocalDidSomething |= MainThreadEventQueue.PopAndExecute(ThreadState);
		}

		for (int32 PackageIndex = 0; PackageIndex < LoadedPackagesToProcess.Num(); ++PackageIndex)
		{
			SCOPED_LOADTIMER(ProcessLoadedPackagesTime);
			FAsyncPackage2* Package = LoadedPackagesToProcess[PackageIndex];
			if (Package->SyncLoadContextId < SyncLoadContextId)
			{
				continue;
			}
			bLocalDidSomething = true;
			UE_ASYNC_PACKAGE_DEBUG(Package->Desc);
			check(Package->AsyncPackageLoadingState >= EAsyncPackageLoadingState2::Finalize &&
				  Package->AsyncPackageLoadingState <= EAsyncPackageLoadingState2::CreateClusters);

			if (Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::Finalize)
			{
				TArray<UObject*> CDODefaultSubobjects;
				// Clear async loading flags (we still want EInternalObjectFlags::Async, but EInternalObjectFlags::AsyncLoading can be cleared)
				for (const FExportObject& Export : Package->Data.Exports)
				{
					if (Export.bFiltered || Export.bExportLoadFailed)
					{
						continue;
					}

					UObject* Object = Export.Object;

					// CDO need special handling, no matter if it's listed in DeferredFinalizeObjects
					UObject* CDOToHandle = ((Object != nullptr) && Object->HasAnyFlags(RF_ClassDefaultObject)) ? Object : nullptr;

					// Clear AsyncLoading in CDO's subobjects.
					if (CDOToHandle != nullptr)
					{
						CDOToHandle->GetDefaultSubobjects(CDODefaultSubobjects);
						for (UObject* SubObject : CDODefaultSubobjects)
						{
							if (SubObject && SubObject->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
							{
								SubObject->AtomicallyClearInternalFlags(EInternalObjectFlags_AsyncLoading);
							}
						}
						CDODefaultSubobjects.Reset();
					}
				}

				Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::PostLoadInstances;
			}

			if (Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::PostLoadInstances)
			{
				SCOPE_CYCLE_COUNTER(STAT_FAsyncPackage_PostLoadInstancesGameThread);
				if (Package->PostLoadInstances(ThreadState) == EAsyncPackageState::Complete)
				{
					Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::CreateClusters;
				}
				else
				{
					// PostLoadInstances timed out
					Result = EAsyncPackageState::TimeOut;
				}
			}


			if (Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::CreateClusters)
			{
				SCOPE_CYCLE_COUNTER(STAT_FAsyncPackage_CreateClustersGameThread);
				if (Package->HasLoadFailed() || !CanCreateObjectClusters())
				{
					Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::Complete;
				}
				else if (Package->CreateClusters(ThreadState) == EAsyncPackageState::Complete)
				{
					// All clusters created, it's safe to delete the package
					Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::Complete;
				}
				else
				{
					// Cluster creation timed out
					Result = EAsyncPackageState::TimeOut;
				}
			}

			// push stats so that we don't overflow number of tags per thread during blocking loading
			LLM_PUSH_STATS_FOR_ASSET_TAGS();

			if (Result == EAsyncPackageState::TimeOut)
			{
				break;
			}

			check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::Complete);

			Package->FinishUPackage();

			{
				if (!Package->HasLoadFailed())
				{
#if WITH_EDITOR || UE_ENABLE_ASSET_READ_LOGGER
					// Track completed packages for later OnEndLoadPackage broadcast (editor or logger-enabled builds)
					CompletedUPackagesForBroadcast.Add(Package->LinkerRoot);
#endif

					UE_MT_SCOPED_READ_ACCESS(Package->ConstructedObjectsAccessDetector);
					for (UObject* Object : Package->ConstructedObjects)
					{
#if WITH_EDITOR
						if (Object->IsAsset())
						{
							EditorLoadedAssets.Add(Object);
						}
#endif
						if (FCoreUObjectDelegates::IsHotfixableAssetClass(Object->GetClass()))
						{
							HotfixableLoadedAssets.Add(Object);
						}
					}
				}

				Package->ClearConstructedObjects();

				Package->DetachLinker();

				// If we were to perform the removal before DetachLinker, it would be possible
				// for a new async package to be created, and we could race trying to attach
				// the linker to the new package before its detached by this one.
				AsyncPackageLookup.Remove(Package->Desc.UPackageId);
			}

			// Remove the package from the list before we trigger the callbacks, 
			// this is to ensure we can re-enter FlushAsyncLoading from any of the callbacks
			LoadedPackagesToProcess.RemoveAt(PackageIndex--);

			// Incremented on the Async Thread, now decrement as we're done with this package
			--LoadingPackagesCounter;

			TRACE_COUNTER_SET(AsyncLoadingLoadingPackages, LoadingPackagesCounter);

			LocalCompletedAsyncPackages.Add(Package);
		}

		{
			FScopeLock _(&FailedPackageRequestsCritical);
			CompletedPackageRequests.Append(MoveTemp(FailedPackageRequests));
			FailedPackageRequests.Reset();
		}
		for (FAsyncPackage2* Package : LocalCompletedAsyncPackages)
		{
			UE_ASYNC_PACKAGE_DEBUG(Package->Desc);
			UE_ASYNC_PACKAGE_LOG(Verbose, Package->Desc, TEXT("GameThread: LoadCompleted"),
				TEXT("All loading of package is done, and the async package and load request will be deleted."));

			check(Package->AsyncPackageLoadingState == EAsyncPackageLoadingState2::Complete);
			Package->AsyncPackageLoadingState = EAsyncPackageLoadingState2::DeferredClearImportedPackages;
			DeferredCleanupPackages.Enqueue(Package);

			if (Package->CompletionCallbacks.IsEmpty() && Package->ProgressCallbacksHead.load(std::memory_order_relaxed) == nullptr)
			{
				RemovePendingRequests(ThreadState, Package->RequestIDs);
				Package->Release();
			}
			else
			{
				// Note: We need to keep the package alive until the callback has been executed
				CompletedPackageRequests.Add(FCompletedPackageRequest::FromLoadedPackage(Package));
			}
		}

		if (LocalCompletedAsyncPackages.Num())
		{
			// Notify ALT for processing DeferredCleanupPackages
			AltZenaphore.NotifyOne();

			LocalCompletedAsyncPackages.Reset();
		}

		// Move CompletedPackageRequest out of the global collection to prevent it from changing from within the callbacks.
		// If we're flushing a specific request only call callbacks for that request
		TArray<FCompletedPackageRequest> RequestsToProcess;
		GatherCompletedPackagesToFlush(ThreadState, FlushRequestIDs, RequestsToProcess);
		bLocalDidSomething |= !!RequestsToProcess.Num();

		// Call callbacks in a batch in a stack-local array after all other work has been done to handle
		// callbacks that may call FlushAsyncLoading
		for (FCompletedPackageRequest& CompletedPackageRequest : RequestsToProcess)
		{
			TRACE_COUNTER_SET(AsyncLoadingPackagesWithRemainingWork, PackagesWithRemainingWorkCounter);
			CompletedPackageRequest.CallCompletionCallbacks();
			if (CompletedPackageRequest.AsyncPackage)
			{
				CompletedPackageRequest.AsyncPackage->Release();
			}
			else
			{
				// Requests for missing packages have no AsyncPackage but they count as packages with remaining work
				--PackagesWithRemainingWorkCounter;
			}
		}

		if (!bLocalDidSomething)
		{
			break;
		}

		bDidSomething = true;

		if (FlushRequestIDs.Num() != 0 && !ContainsAnyRequestID(FlushRequestIDs))
		{
			// The only packages we care about have finished loading, so we're good to exit
			break;
		}
	}

	return Result;
}

void FAsyncLoadingThread2::GatherCompletedPackagesToFlush(FAsyncLoadingThreadState2& ThreadState, TConstArrayView<int32> FlushRequestIds, TArray<FCompletedPackageRequest>& OutRequestsToProcess)
{
	if (CompletedPackageRequests.IsEmpty())
	{
		return;
	}

	if (FlushRequestIds.Num())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GatherCompletedPackagesToFlush);

		// Gather all completed packages affected by the flush request (i.e. the package being flushed
		// and any completed imports loaded as a result). To do so, we build an adjacency list from the 
		// CompletedRequests, and perform a depth-first sort for each package being flushed. We will append 
		// the DFS results from each flush while maintaining a 'visited' set. This set will prevent loopiung on cycles
		// but also allow us to prevent duplication when we first flush a package that is a sub-tree of some other package
		// we are also going to flush later.
		TMap<int32, int32, TInlineSetAllocator<64>>  RequestIdToCompletePackageIndex;
		for (int32 CompletedPackageRequestIndex = 0; CompletedPackageRequestIndex < CompletedPackageRequests.Num(); ++CompletedPackageRequestIndex)
		{
			FCompletedPackageRequest& CompletedPackageRequest = CompletedPackageRequests[CompletedPackageRequestIndex];
			for (int32 RequestId : CompletedPackageRequest.RequestIDs)
			{
				RequestIdToCompletePackageIndex.Add(RequestId, CompletedPackageRequestIndex);
			}
		}

		// Adjacency list of completed packages
		typedef TSet<int32, DefaultKeyFuncs<int32>, TInlineSetAllocator<64>> FInlineIndexSet;
		TMap<int32, FInlineIndexSet, TInlineSetAllocator<64>> IndexToImportIndices;
		for (int32 CompletedPackageRequestIndex = 0; CompletedPackageRequestIndex < CompletedPackageRequests.Num(); ++CompletedPackageRequestIndex)
		{
			FCompletedPackageRequest& CompletedPackageRequest = CompletedPackageRequests[CompletedPackageRequestIndex];
			for (int32 ImportRequestId : CompletedPackageRequest.ImportRequestIDs)
			{
				int32* ImportCompletedPackageRequestIndex = RequestIdToCompletePackageIndex.Find(ImportRequestId);
				if (ImportCompletedPackageRequestIndex && *ImportCompletedPackageRequestIndex != CompletedPackageRequestIndex)
				{
					FInlineIndexSet& Imports = IndexToImportIndices.FindOrAdd(*ImportCompletedPackageRequestIndex);
					Imports.Add(CompletedPackageRequestIndex);
				}
			}
		}

		FInlineIndexSet Visited;
		TArray<int32, TInlineAllocator<64>> Stack;
		TArray<int32, TInlineAllocator<64>> DFS;
		TArray<int32, TInlineAllocator<64>> CompletedPackageIndicesToProcess;
		// For each root (flushed) package, depth-first sort
		for (int32 FlushRequestId : FlushRequestIds)
		{
			int32* CompletedPackageIndexToFlush = RequestIdToCompletePackageIndex.Find(FlushRequestId);
			if (!CompletedPackageIndexToFlush || Visited.Contains(*CompletedPackageIndexToFlush))
			{
				continue;
			}
			Stack.Push(*CompletedPackageIndexToFlush);
			DFS.Reset();
			do
			{
				int32 CompletedPackageIndex = Stack.Pop();
				bool bAlreadyInserted = false;
				Visited.Add(CompletedPackageIndex, &bAlreadyInserted);

				// Cycles are common in loading graphs so we must defend against them
				if (!bAlreadyInserted)
				{
					DFS.Add(CompletedPackageIndex);
				}

				FInlineIndexSet* ImportIndices = IndexToImportIndices.Find(CompletedPackageIndex);
				if (ImportIndices)
				{
					for (int32 ImportIndex : *ImportIndices)
					{
						if (!Visited.Contains(ImportIndex))
						{
							Stack.Push(ImportIndex);
						}
					}
				}
			} while (!Stack.IsEmpty());

			for (int32 Index : ReverseIterate(DFS))
			{
				CompletedPackageIndicesToProcess.Add(Index);
			}
		}

		for (int32 ProcessIndex : CompletedPackageIndicesToProcess)
		{
			FCompletedPackageRequest& CompletedPackageRequest = CompletedPackageRequests[ProcessIndex];
			RemovePendingRequests(ThreadState, CompletedPackageRequest.RequestIDs);
			OutRequestsToProcess.Push(MoveTemp(CompletedPackageRequest));
		}

		// Sort the indices to remove elements from the end of the array first 
		// so our indices are not invalidated as we perform removes and reduce the memmoving involved
		CompletedPackageIndicesToProcess.Sort([](int32 A, int32 B) { return A > B; });
		for (int32 ProcessIndex : CompletedPackageIndicesToProcess)
		{
			CompletedPackageRequests.RemoveAt(ProcessIndex);
		}
	}
	else
	{
		OutRequestsToProcess.Reserve(CompletedPackageRequests.Num());
		for (FCompletedPackageRequest& CompletedPackageRequest : CompletedPackageRequests)
		{
			RemovePendingRequests(ThreadState, CompletedPackageRequest.RequestIDs);
			OutRequestsToProcess.Emplace(MoveTemp(CompletedPackageRequest));
		}
		CompletedPackageRequests.Empty();
	}
}

EAsyncPackageState::Type FAsyncLoadingThread2::TickAsyncLoadingFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit, TConstArrayView<int32> FlushRequestIDs, bool& bDidSomething)
{
	SCOPE_CYCLE_COUNTER(STAT_FAsyncPackage_TickAsyncLoadingGameThread);
	//TRACE_INT_VALUE(QueuedPackagesCounter, QueuedPackagesCounter);
	//TRACE_INT_VALUE(GraphNodeCount, GraphAllocator.TotalNodeCount);
	//TRACE_INT_VALUE(GraphArcCount, GraphAllocator.TotalArcCount);
	//TRACE_MEMORY_VALUE(GraphMemory, GraphAllocator.TotalAllocated);

	check(IsInGameThread());
	check(!IsGarbageCollecting());

	// By default, game thread can only access objects in Phase 2. Further scoping is needed to access everything.
	TGuardValue GuardVisibilityFilter(FUObjectThreadContext::Get().AsyncVisibilityFilter, EInternalObjectFlags::AsyncLoadingPhase1);

#if WITH_EDITOR
	// In the editor loading cannot be part of a transaction as it cannot be undone, and may result in recording half-loaded objects. So we suppress any active transaction while in this stack, and set the editor loading flag
	TGuardValue<ITransaction*> SuppressTransaction(GUndo, nullptr);
	TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, GIsEditor || UE::GetIsEditorLoadingPackage());
#endif

	const bool bLoadingSuspended = IsAsyncLoadingSuspended();
	EAsyncPackageState::Type Result = bLoadingSuspended ? EAsyncPackageState::PendingImports : EAsyncPackageState::Complete;

	if (!bLoadingSuspended)
	{
		// Use a time limit scope to restore the time limit to the old values when exiting the scope
		// This is required to ensure a reentrant call here doesn't overwrite time limits permanently.
		FAsyncLoadingThreadState2::FTimeLimitScope TimeLimitScope(ThreadState, bUseTimeLimit, TimeLimit);

		const bool bIsMultithreaded = FAsyncLoadingThread2::IsMultithreaded();

		if (!bIsMultithreaded)
		{
			RemoveUnreachableObjects(UnreachableObjects);
			FlushDeferredLoadedObjectRenameQueue();
		}
		UpdateSyncLoadContext(ThreadState);

		Result = ProcessLoadedPackagesFromGameThread(ThreadState, bDidSomething, FlushRequestIDs);
		if (bUseTimeLimit && !GIsEditor)
		{
			double TimeLimitUsedForProcessLoaded = FPlatformTime::Seconds() - ThreadState.StartTime;
			UE_CLOGF(TimeLimitUsedForProcessLoaded > 0.1, LogStreaming, Warning, "Took %6.2fms to ProcessLoadedPackages", float(TimeLimitUsedForProcessLoaded) * 1000.0f);
		}

		if (!bIsMultithreaded && Result != EAsyncPackageState::TimeOut)
		{
			Result = TickAsyncThreadFromGameThread(ThreadState, bDidSomething);
		}

		if (Result != EAsyncPackageState::TimeOut)
		{
			if (!bDidSomething && PendingCDOs.Num() > 0)
			{
				bDidSomething = ProcessPendingCDOs(ThreadState);
			}

			// Flush deferred messages
			if (!IsAsyncLoadingPackages())
			{
				FDeferredMessageLog::Flush();
			}
		}

		// Call update callback once per tick on the game thread
		FCoreDelegates::OnAsyncLoadingFlushUpdate.Broadcast();
	}

	ConditionalProcessCallbacks();

	return Result;
}

FAsyncLoadingThread2::FAsyncLoadingThread2(FIoDispatcher& InIoDispatcher)
	: Thread(nullptr)
	, IoDispatcher(InIoDispatcher)
	, PackageStore(FPackageStore::Get())
	, GlobalImportStore(*this)
{
	AutoRTFM::UnreachableIfClosed();

	EventQueue.SetZenaphore(&AltZenaphore);

	PendingPackages.Reserve(DefaultAsyncPackagesReserveCount);
	RequestIdToPackageMap.Reserve(DefaultAsyncPackagesReserveCount);

	EventSpecs.AddDefaulted(EEventLoadNode2::Package_NumPhases);
	EventSpecs[EEventLoadNode2::Package_ProcessSummary] = { &FAsyncPackage2::Event_ProcessPackageSummary, &EventQueue, EExecutionType::Parallel, TEXT("ProcessSummary") };
	EventSpecs[EEventLoadNode2::Package_ProcessDependencies] = { &FAsyncPackage2::Event_ProcessDependencies, &EventQueue, EExecutionType::Normal, TEXT("ProcessDependencies") };
	EventSpecs[EEventLoadNode2::Package_DependenciesReady] = { &FAsyncPackage2::Event_DependenciesReady, &EventQueue, EExecutionType::Normal, TEXT("DependenciesReady") };
	EventSpecs[EEventLoadNode2::Package_CreateExports] = { &FAsyncPackage2::Event_CreateExports, &EventQueue, EExecutionType::Normal, TEXT("CreateExports") };
	EventSpecs[EEventLoadNode2::Package_ResolveImports] = { &FAsyncPackage2::Event_ResolveImports, &EventQueue, EExecutionType::Normal, TEXT("ResolveImports") };
	EventSpecs[EEventLoadNode2::Package_PreloadExports] = { &FAsyncPackage2::Event_PreloadExports, &EventQueue, EExecutionType::Parallel, TEXT("PreloadExports") };
	EventSpecs[EEventLoadNode2::Package_ExportsSerialized] = { &FAsyncPackage2::Event_ExportsDone, &EventQueue, EExecutionType::Normal, TEXT("ExportsSerialized") };
	EventSpecs[EEventLoadNode2::Package_PostLoad] = { &FAsyncPackage2::Event_PostLoad, &EventQueue, EExecutionType::Normal, TEXT("PostLoad") };
	EventSpecs[EEventLoadNode2::Package_DeferredPostLoad] = { &FAsyncPackage2::Event_DeferredPostLoad, &MainThreadEventQueue, EExecutionType::Normal, TEXT("DeferredPostLoad") };

	CancelLoadingEvent = FPlatformProcess::GetSynchEventFromPool();
	ThreadSuspendedEvent = FPlatformProcess::GetSynchEventFromPool();
	ThreadResumedEvent = FPlatformProcess::GetSynchEventFromPool();
	AsyncLoadingTickCounter = 0;

	FCoreUObjectDelegates::GetGarbageCollectStartedDelegate().AddRaw(this, &FAsyncLoadingThread2::OnGarbageCollectStarted);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnLoadedObjectRenamed.AddRaw(this, &FAsyncLoadingThread2::OnLoadedObjectRenamed);
#endif

	FAsyncLoadingThreadState2::TlsSlot = FPlatformTLS::AllocTlsSlot();
	GameThreadState = MakeUnique<FAsyncLoadingThreadState2>();
	GameThreadState->bCanAccessAsyncLoadingThreadData = true;
	EventQueue.SetOwnerThread(GameThreadState.Get());
	MainThreadEventQueue.SetOwnerThread(GameThreadState.Get());
	FAsyncLoadingThreadState2::Set(GameThreadState.Get());

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FHeap::Initialize();
	AsyncLoadingVerseRoot->SetAsyncLoadingThread(this);
#endif

	NotifyCompiledVersePackageDelegate.BindRaw(this, &FAsyncLoadingThread2::NotifyCompiledVersePackage);

	// if we are on an uncooked platform query the editor streaming settings
	const TCHAR* SectionName = FPlatformProperties::RequiresCookedData() ? TEXT("/Script/Engine.StreamingSettings") : TEXT("/Script/Engine.EditorStreamingSettings");
	
	bool bConfigValue = false;

	// This is a CVar but for consistency, we allow setting it with the other streaming settings
	if (GConfig->GetBool(SectionName, TEXT("s.AllowMultithreadedLoading"), bConfigValue, GEngineIni))
	{
		GAllowMultithreadedLoading = bConfigValue;
	}

	// Support blanket removal of async loading and multithreaded loading for commandlet if desired.
	if (IsRunningCommandlet())
	{
		if (GConfig->GetBool(SectionName, TEXT("s.NoCommandletAsyncLoading"), bConfigValue, GEngineIni) && bConfigValue)
		{
			FAsyncLoadingThreadSettings::Get().bAsyncLoadingThreadEnabled = false;
		}

		if (GConfig->GetBool(SectionName, TEXT("s.NoCommandletMultithreadedLoading"), bConfigValue, GEngineIni) && bConfigValue)
		{
			GAllowMultithreadedLoading = false;
		}
	}

	// A bit more constrained on the cook only case if we want async loading in some commandlet that aren't cook related
	if (IsRunningCookCommandlet())
	{
		if (GConfig->GetBool(SectionName, TEXT("s.NoCookCommandletAsyncLoading"), bConfigValue, GEngineIni) && bConfigValue)
		{
			FAsyncLoadingThreadSettings::Get().bAsyncLoadingThreadEnabled = false;
		}

		if (GConfig->GetBool(SectionName, TEXT("s.NoCookCommandletMultithreadedLoading"), bConfigValue, GEngineIni) && bConfigValue)
		{
			GAllowMultithreadedLoading = false;
		}
	}

#if WITH_EDITOR
	if (FAsyncLoadingThreadSettings::Get().bAsyncLoadingThreadEnabled)
	{
		const TCHAR* FlushStreamingOnGCStr = TEXT("gc.FlushStreamingOnGC");
		IConsoleVariable* FlushStreamingOnGCVar = IConsoleManager::Get().FindConsoleVariable(FlushStreamingOnGCStr);
		check(FlushStreamingOnGCVar);
		const int CurrentFlushStreamingOnGCValue = FlushStreamingOnGCVar->GetInt();

		UE_CLOGF(CurrentFlushStreamingOnGCValue == 0, LogStreaming, Display, "Using AsyncLoadingThread in Editor; forcing %ls=1", FlushStreamingOnGCStr);
		FlushStreamingOnGCVar->Set(1);
	}
#endif

	UE_LOGF(LogStreaming, Display, "AsyncLoading2 - Created: Async Loading Thread: %ls, Async Post Load: %ls, Multithreaded: %ls",
		FAsyncLoadingThreadSettings::Get().bAsyncLoadingThreadEnabled ? TEXT("true") : TEXT("false"),
		FAsyncLoadingThreadSettings::Get().bAsyncPostLoadEnabled ? TEXT("true") : TEXT("false"),
		GAllowMultithreadedLoading ? TEXT("true") : TEXT("false"));
}

FAsyncLoadingThread2::~FAsyncLoadingThread2()
{
	NotifyCompiledVersePackageDelegate.Unbind();

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	AsyncLoadingVerseRoot->SetAsyncLoadingThread(nullptr);
#endif

	if (Thread)
	{
		ShutdownLoading();
	}

	LoaderStats.DumpToLog();
}

void FAsyncLoadingThread2::ShutdownLoading()
{
	FCoreUObjectDelegates::GetPreGarbageCollectDelegate().RemoveAll(this);

	delete Thread;
	Thread = nullptr;
	FPlatformProcess::ReturnSynchEventToPool(CancelLoadingEvent);
	CancelLoadingEvent = nullptr;
	FPlatformProcess::ReturnSynchEventToPool(ThreadSuspendedEvent);
	ThreadSuspendedEvent = nullptr;
	FPlatformProcess::ReturnSynchEventToPool(ThreadResumedEvent);
	ThreadResumedEvent = nullptr;
}

void FAsyncLoadingThread2::StartThread()
{
	// Make sure the GC sync object is created before we start the thread (apparently this can happen before we call InitUObject())
	FGCCSyncObject::Create();

	// Clear game thread initial load arrays
	check(PendingCDOs.Num() == 0);
	PendingCDOs.Empty();
	check(PendingCDOsRecursiveStack.Num() == 0);
	PendingCDOsRecursiveStack.Empty();

	if (FAsyncLoadingThreadSettings::Get().bAsyncLoadingThreadEnabled && !Thread)
	{
		AsyncLoadingThreadState = MakeUnique<FAsyncLoadingThreadState2>();
		EventQueue.SetOwnerThread(AsyncLoadingThreadState.Get());

		// When using ALT, we want to wake the main thread ASAP in case it's sleeping for lack of something to do during flush.
		MainThreadEventQueue.SetWakeEvent(&MainThreadWakeEvent);
		PackagesWithRemainingWorkCounter.SetWakeEvent(&MainThreadWakeEvent);

		AsyncLoadingThreadState->bIsAsyncLoadingThread = true;
		AsyncLoadingThreadState->bCanAccessAsyncLoadingThreadData = true;
		GameThreadState->bCanAccessAsyncLoadingThreadData = false;
		UE_LOGF(LogStreaming, Log, "Starting Async Loading Thread.");
		bThreadStarted.store(true, std::memory_order_release);
		FPlatformMisc::MemoryBarrier();
		UE::Trace::ThreadGroupBegin(TEXT("AsyncLoading"));
		Thread = FRunnableThread::Create(this, TEXT("FAsyncLoadingThread"), 0, TPri_Normal);
		UE::Trace::ThreadGroupEnd();
	}

	UE_LOGF(LogStreaming, Log, "AsyncLoading2 - Thread Started: %ls, IsInitialLoad: %ls",
		FAsyncLoadingThreadSettings::Get().bAsyncLoadingThreadEnabled ? TEXT("true") : TEXT("false"),
		GIsInitialLoad ? TEXT("true") : TEXT("false"));
}

bool FAsyncLoadingThread2::Init()
{
	return true;
}

uint32 FAsyncLoadingThread2::Run()
{
	const FTaskTagScope AsyncLoadingScope(ETaskTag::EAsyncLoadingThread);

	LLM_SCOPE(ELLMTag::AsyncLoading);

	AsyncLoadingThreadID = FPlatformTLS::GetCurrentThreadId();

	FAsyncLoadingThreadState2::Set(AsyncLoadingThreadState.Get());

	TRACE_LOADTIME_START_ASYNC_LOADING();

	FPlatformProcess::SetThreadAffinityMask(FPlatformAffinity::GetAsyncLoadingThreadMask());
	FMemory::SetupTLSCachesOnCurrentThread();

	FAsyncLoadingThreadState2& ThreadState = *FAsyncLoadingThreadState2::Get();

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FIOContext Context = Verse::FIOContext::CreateForManualStackScanning();
#endif

	FZenaphoreWaiter Waiter(AltZenaphore, TEXT("WaitForEvents"));
	enum class EMainState : uint8
	{
		Suspended,
		Loading,
		Waiting,
	};
	EMainState PreviousState = EMainState::Loading;
	EMainState CurrentState = EMainState::Loading;
	while (!bStopRequested.load(std::memory_order_relaxed))
	{
		if (CurrentState == EMainState::Suspended)
		{
			// suspended, sleep until loading can be resumed
			while (!bStopRequested.load(std::memory_order_relaxed))
			{
				if (SuspendRequestedCount.load(std::memory_order_relaxed) == 0 && !IsGarbageCollectionWaiting())
				{
					ThreadResumedEvent->Trigger();
					CurrentState = EMainState::Loading;
					break;
				}

				FPlatformProcess::Sleep(0.001f);
			}
		}
		else if (CurrentState == EMainState::Waiting)
		{
			// no packages in flight and waiting for new load package requests,
			// or done serializing and waiting for deferred deletes of packages being postloaded
			Waiter.Wait();
			CurrentState = EMainState::Loading;
		}
		else if (CurrentState == EMainState::Loading)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AsyncLoadingTime);
			CSV_SCOPED_TIMING_STAT(FileIO, AsyncLoadingTime);

#if USING_INSTRUMENTATION
			if (GDetectRaceDuringLoading)
			{
				UE::Sanitizer::RaceDetector::ToggleFilterOtherThreads(true);
			}
			UE::Sanitizer::RaceDetector::FRaceDetectorScope RaceDetectorScope(GDetectRaceDuringLoading, GDetectRaceGracePeriod);
#endif
			FAsyncLoadingCsvRegionScope CsvRegionScope;
			bool bShouldSuspend = false;
			bool bShouldWaitForExternalReads = false;
			while (!bStopRequested.load(std::memory_order_relaxed))
			{
				ThreadState.MarkAsActive();

				if (bShouldSuspend || SuspendRequestedCount.load(std::memory_order_relaxed) > 0 || IsGarbageCollectionWaiting())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(SuspendAsyncLoading);
					ThreadSuspendedEvent->Trigger();
					CurrentState = EMainState::Suspended;
					break;
				}

				{
					FGCScopeGuard GCGuard;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
					Context.SetIsInManuallyEmptyStack(false);
					Verse::FRunningContext RunningContext = Context.AcquireAccessForManualStackScanning();
					ON_SCOPE_EXIT
					{
						RunningContext.RelinquishAccessForManualStackScanning();
						Context.SetIsInManuallyEmptyStack(true);
					};
#endif

					// The first thing we need to do is cleanup the global import table from unreachable objects
					// to avoid racing with the IncrementalPurgeGarbage that happens outside of the GC guard.
					{
						FScopeLock UnreachableObjectsLock(&UnreachableObjectsCritical);
						RemoveUnreachableObjects(UnreachableObjects);
					}

					FlushDeferredLoadedObjectRenameQueue();

					if (bShouldWaitForExternalReads)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(WaitingForExternalReads);
						FAsyncPackage2* Package = nullptr;
						ExternalReadQueue.Dequeue(Package);
						check(Package);
						EAsyncPackageState::Type Result = Package->ProcessExternalReads(ThreadState, FAsyncPackage2::ExternalReadAction_Wait);
						check(Result == EAsyncPackageState::Complete);
						bShouldWaitForExternalReads = false;
						continue;
					}


					if (QueuedPackagesCounter || !PendingPackages.IsEmpty())
					{
						TRACE_CPUPROFILER_EVENT_SCOPE_CONDITIONAL(TimeLimitedCreateAsyncPackages, GAsyncCreatePackagesFromQueueMaxTimeLimitMs > .0f);
						// Use a time limit scope to restore the time limit to the old values when exiting the scope
						// This is required to ensure a reentrant call here doesn't overwrite time limits permanently.
						// Note: this is only active when GAsyncCreatePackagesFromQueueMaxTimeLimitMs > 0
						FAsyncLoadingThreadState2::FTimeLimitScope TimeLimitScope( ThreadState, true, double(GAsyncCreatePackagesFromQueueMaxTimeLimitMs/1000.0f), GAsyncCreatePackagesFromQueueMaxTimeLimitMs > .0f);

						if (CreateAsyncPackagesFromQueue(ThreadState, GAsyncLoading2_AllowPreemptingPackagesDuringGC))
						{
							// Fall through to FAsyncLoadEventQueue2 processing unless we need to suspend
							if (SuspendRequestedCount.load(std::memory_order_relaxed) > 0 || IsGarbageCollectionWaiting())
							{
								bShouldSuspend = true;
								continue;
							}
						}
					}

					// do as much event queue processing as we possibly can
					{
						bool bDidSomething = false;
						bool bPopped = false;
						do
						{
							bPopped = false;
							UpdateSyncLoadContext(ThreadState);
							if (!ThreadState.SyncLoadContextStack.IsEmpty() && ThreadState.SyncLoadContextStack.Top()->ContextId)
							{
								if (EventQueue.ExecuteSyncLoadEvents(ThreadState))
								{
									// The first thing we should do after execution is making sure we release
									// any sync load context that have been fulfilled
									CleanupSyncLoadContext(ThreadState);
									bPopped = true;
									bDidSomething = true;
								}
							}
							else if (EventQueue.PopAndExecute(ThreadState))
							{
								bPopped = true;
								bDidSomething = true;
							}
							if (SuspendRequestedCount.load(std::memory_order_relaxed) > 0 || IsGarbageCollectionWaiting())
							{
								bShouldSuspend = true;
								bDidSomething = true;
								bPopped = false;
								break;
							}
						} while (bPopped);

						if (bDidSomething)
						{
							continue;
						}
					}

					{
						FAsyncPackage2** Package = ExternalReadQueue.Peek();
						if (Package != nullptr)
						{
							TRACE_CPUPROFILER_EVENT_SCOPE(PollExternalReads);
							check(*Package);
							EAsyncPackageState::Type Result = (*Package)->ProcessExternalReads(ThreadState, FAsyncPackage2::ExternalReadAction_Poll);
							if (Result == EAsyncPackageState::Complete)
							{
								ExternalReadQueue.Dequeue();
								continue;
							}
						}
					}

					if (ProcessDeferredCleanupPackagesQueue(100))
					{
						continue;
					}
				} // release FGCScopeGuard

				if (PendingIoRequestsCounter.load() > 0)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(WaitingForIo);
					Waiter.Wait();
					continue;
				}

				if (!ExternalReadQueue.IsEmpty())
				{
					bShouldWaitForExternalReads = true;
					continue;
				}

				// no async loading work left to do for now
				CurrentState = EMainState::Waiting;
				break;
			}
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Context.ReleaseForManualStackScanning();
#endif

	return 0;
}

EAsyncPackageState::Type FAsyncLoadingThread2::TickAsyncThreadFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool& bDidSomething)
{
	check(IsInGameThread());
	EAsyncPackageState::Type Result = EAsyncPackageState::Complete;

	if (AsyncThreadReady)
	{
		if (ThreadState.IsTimeLimitExceeded(TEXT("TickAsyncThreadFromGameThread")))
		{
			Result = EAsyncPackageState::TimeOut;
		}
		else
		{
			Result = ProcessAsyncLoadingFromGameThread(ThreadState, bDidSomething);
		}
	}

	return Result;
}

void FAsyncLoadingThread2::Stop()
{
	SuspendRequestedCount.fetch_add(1);
	bStopRequested.store(true);
	AltZenaphore.NotifyAll();
}

void FAsyncLoadingThread2::CancelLoading()
{
	check(false);
	// TODO
}

void FAsyncLoadingThread2::SuspendLoading()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SuspendLoading);
	UE_CLOGF(!IsInGameThread() || IsInSlateThread(), LogStreaming, Fatal, "Async loading can only be suspended from the main thread");
	int32 OldCount = SuspendRequestedCount.fetch_add(1);
	if (OldCount == 0)
	{
		UE_LOGF(LogStreaming, Log, "Suspending async loading");
		if (IsMultithreaded())
		{
			TRACE_LOADTIME_SUSPEND_ASYNC_LOADING();
			AltZenaphore.NotifyAll();
			ThreadSuspendedEvent->Wait();
		}
	}
	else
	{
		UE_LOGF(LogStreaming, Verbose, "Async loading is already suspended (count: %d)", OldCount + 1);
	}
}

void FAsyncLoadingThread2::ResumeLoading()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ResumeLoading);
	UE_CLOGF(!IsInGameThread() || IsInSlateThread(), LogStreaming, Fatal, "Async loading can only be resumed from the main thread");
	int32 OldCount = SuspendRequestedCount.fetch_sub(1);
	UE_CLOG(OldCount < 1, LogStreaming, Fatal, TEXT("Trying to resume async loading when it's not suspended"));
	if (OldCount == 1)
	{
		UE_LOGF(LogStreaming, Log, "Resuming async loading");
		if (IsMultithreaded())
		{
			ThreadResumedEvent->Wait();
			TRACE_LOADTIME_RESUME_ASYNC_LOADING();
		}
	}
	else
	{
		UE_LOGF(LogStreaming, Verbose, "Async loading is still suspended (count: %d)", OldCount - 1);
	}
}

float FAsyncLoadingThread2::GetAsyncLoadPercentage(const FName& PackageName)
{
	return -1.0f;
}

static void VerifyObjectLoadFlagsWhenFinishedLoading()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(VerifyObjectLoadFlagsWhenFinishedLoading);

	const EInternalObjectFlags AsyncFlags =
		EInternalObjectFlags_AsyncLoading;

	const EObjectFlags LoadIntermediateFlags =
		EObjectFlags::RF_NeedLoad | EObjectFlags::RF_WillBeLoaded |
		EObjectFlags::RF_NeedPostLoad | RF_NeedPostLoadSubobjects;

	ParallelFor(TEXT("VerifyObjectLoadFlagsDebugTask"), GUObjectArray.GetObjectArrayNum(), 512,
		[AsyncFlags, LoadIntermediateFlags](int32 ObjectIndex)
	{
		FUObjectItem* ObjectItem = &GUObjectArray.GetObjectItemArrayUnsafe()[ObjectIndex];
		if (UObject* Obj = static_cast<UObject*>(ObjectItem->GetObject()))
		{
			const EInternalObjectFlags InternalFlags = Obj->GetInternalFlags();
			const EObjectFlags Flags = Obj->GetFlags();
			const bool bHasAnyAsyncFlags = !!(InternalFlags & AsyncFlags);
			const bool bHasAnyLoadIntermediateFlags = !!(Flags & LoadIntermediateFlags);
			const bool bHasLoaderImportFlag = !!(InternalFlags & EInternalObjectFlags::LoaderImport);
			const bool bWasLoaded = !!(Flags & RF_WasLoaded);
			const bool bLoadCompleted = !!(Flags & RF_LoadCompleted);

			ensureMsgf(!bHasAnyLoadIntermediateFlags,
				TEXT("Object '%s' (ObjectFlags=%X, InternalObjectFlags=%x) should not have any load flags now")
				TEXT(", or this check is incorrectly reached during active loading."),
				*Obj->GetFullName(),
				uint32{EnumToUnderlyingType(Flags)},
				uint32{EnumToUnderlyingType(InternalFlags)});

			ensureMsgf(!bHasLoaderImportFlag || GUObjectArray.IsDisregardForGC(Obj),
				TEXT("Object '%s' (ObjectFlags=%X, InternalObjectFlags=%x) should not have the LoaderImport flag now")
				TEXT(", or this check is incorrectly reached during active loading."),
				*Obj->GetFullName(),
				uint32{EnumToUnderlyingType(Flags)},
				uint32{EnumToUnderlyingType(InternalFlags)});

			if (bWasLoaded)
			{
				const bool bIsPackage = Obj->IsA(UPackage::StaticClass());

				ensureMsgf(bIsPackage || bLoadCompleted,
					TEXT("Object '%s' (ObjectFlags=%x, InternalObjectFlags=%x) is a serialized object and should be completely loaded now")
					TEXT(", or this check is incorrectly reached during active loading."),
					*Obj->GetFullName(),
					uint32{EnumToUnderlyingType(Flags)},
					uint32{EnumToUnderlyingType(InternalFlags)});

				ensureMsgf(!bHasAnyAsyncFlags,
					TEXT("Object '%s' (ObjectFlags=%x, InternalObjectFlags=%x) is a serialized object and should not have any async flags now")
					TEXT(", or this check is incorrectly reached during active loading."),
					*Obj->GetFullName(),
					uint32{EnumToUnderlyingType(Flags)},
					uint32{EnumToUnderlyingType(InternalFlags)});
			}
		}
	});
	UE_LOGF(LogStreaming, Log, "Verified load flags when finished active loading.");
}

void FAsyncLoadingThread2::CollectUnreachableObjects(
	TArrayView<FUObjectItem*> UnreachableObjectItems,
	FUnreachableObjects& OutUnreachableObjects)
{
	check(IsInGameThread());

	OutUnreachableObjects.SetNum(UnreachableObjectItems.Num());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CollectUnreachableObjects);
		ParallelFor(TEXT("CollectUnreachableObjectsTask"), UnreachableObjectItems.Num(), 2048,
			[&UnreachableObjectItems, &OutUnreachableObjects, bUsingLinkerLoad = FAsyncLoadingThreadSettings::Get().bLooseFileLoadingEnabled](int32 Index)
		{
			UObject* Object = static_cast<UObject*>(UnreachableObjectItems[Index]->GetObject());

			FUnreachableObject& Item = OutUnreachableObjects[Index];
			Item.ObjectIndex = GUObjectArray.ObjectToIndex(Object);
			Item.ObjectName = Object->GetFName();

			UObject* Outer = UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Object);
			if (!Outer)
			{
				UPackage* Package = static_cast<UPackage*>(Object);
				if (Package->bCanBeImported)
				{
					Item.PackageId = Package->GetPackageId();
				}
			}

			if (bUsingLinkerLoad)
			{
				// Clear garbage objects from linker export tables
				// Normally done from UObject::BeginDestroy but we need to do it already here
				if (FLinkerLoad* ObjectLinker = Object->GetLinker())
				{
					Object->SetLinker(nullptr, INDEX_NONE);
				}
			}
		});
	}

	if (GVerifyUnreachableObjects)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(VerifyUnreachableObjects);
		ParallelFor(TEXT("VerifyUnreachableObjectsDebugTask"), UnreachableObjectItems.Num(), 512,
			[this, &UnreachableObjectItems](int32 Index)
		{
			UObject* Object = static_cast<UObject*>(UnreachableObjectItems[Index]->GetObject());
			UObject* Outer = UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Object);
			if (!Outer)
			{
				UPackage* Package = static_cast<UPackage*>(Object);
				if (Package->bCanBeImported)
				{
					FPackageId PackageId = Package->GetPackageId();
					FReadLockedPackageRef PackageRef = GlobalImportStore.FindPackageRef(PackageId);
					if (PackageRef)
					{
						GlobalImportStore.VerifyPackageForRemoval(*PackageRef);
					}
				}
			}
			GlobalImportStore.VerifyObjectForRemoval(Object);
		});
	}
}

void FAsyncLoadingThread2::OnGarbageCollectStarted()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncLoadingThread2::OnGarbageCollectStarted);
	// Flush the cleanup queue so that we don't prevent packages from being garbage collected if we're done with them
	ProcessDeferredCleanupPackagesQueue();
}

void FAsyncLoadingThread2::RemoveUnreachableObjects(FUnreachableObjects& ObjectsToRemove)
{
	if (ObjectsToRemove.Num() == 0)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(RemoveUnreachableObjects);

	const int32 ObjectCount = ObjectsToRemove.Num();

	const int32 OldLoadedPackageCount = GlobalImportStore.GetStoredPackagesCount();
	const int32 OldPublicExportCount = GlobalImportStore.GetStoredPublicExportsCount();

	const double StartTime = FPlatformTime::Seconds();

	GlobalImportStore.RemovePackages(ObjectsToRemove);
	GlobalImportStore.RemovePublicExports(ObjectsToRemove);

	// No need to keep Array slack memory as it may not be entirely filled over the time
	ObjectsToRemove.Empty();

	const int32 NewLoadedPackageCount = GlobalImportStore.GetStoredPackagesCount();
	const int32 NewPublicExportCount = GlobalImportStore.GetStoredPublicExportsCount();
	const int32 RemovedLoadedPackageCount = OldLoadedPackageCount - NewLoadedPackageCount;
	const int32 RemovedPublicExportCount = OldPublicExportCount - NewPublicExportCount;

	const double StopTime = FPlatformTime::Seconds();
	UE_LOGF(LogStreaming, Log,
		"%.3f ms for processing %d objects in RemoveUnreachableObjects(Queued=%d, Async=%d). " "Removed %d (%d->%d) packages and %d (%d->%d) public exports.",
		(StopTime - StartTime) * 1000,
		ObjectCount,
		GetNumQueuedPackages(), GetNumAsyncPackages(),
		RemovedLoadedPackageCount, OldLoadedPackageCount, NewLoadedPackageCount,
		RemovedPublicExportCount, OldPublicExportCount, NewPublicExportCount);
}

void FAsyncLoadingThread2::NotifyUnreachableObjects(const TArrayView<FUObjectItem*>& UnreachableObjectItems)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NotifyUnreachableObjects);

	if (GExitPurge)
	{
		return;
	}

	FScopeLock UnreachableObjectsLock(&UnreachableObjectsCritical);

	// unreachable objects from last GC should typically have been processed already,
	// if not handle them here before adding new ones
	RemoveUnreachableObjects(UnreachableObjects);
	
	UE::Tasks::FTask RemoveUnreachableScriptObjectsTask = UE::Tasks::Launch(
		TEXT("RemoveUnreachableScriptObject"),
		[this]()
		{
			GlobalImportStore.RemoveUnreachableScriptObjects();
		});

	CollectUnreachableObjects(UnreachableObjectItems, UnreachableObjects);

	RemoveUnreachableScriptObjectsTask.Wait();

	if (GVerifyObjectLoadFlags && !IsAsyncLoading())
	{
		GlobalImportStore.VerifyLoadedPackages();
		VerifyObjectLoadFlagsWhenFinishedLoading();
	}

	if (GRemoveUnreachableObjectsOnGT)
	{
		RemoveUnreachableObjects(UnreachableObjects);
	}

	if (GShrinkGlobalImportStoreOnGT)
	{
		GlobalImportStore.Shrink();
	}

	// wake up ALT to remove unreachable objects
	AltZenaphore.NotifyAll();
}

/**
 * Call back into the async loading code to inform of the creation of a new object
 * @param Object		Object created
 * @param bSubObjectThatAlreadyExists	Object created as a sub-object of a loaded object
 */
void FAsyncLoadingThread2::NotifyConstructedDuringAsyncLoading(UObject* Object, bool bSubObjectThatAlreadyExists)
{
	check(Object);
	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	FAsyncPackage2* ContextAsyncPackage = (FAsyncPackage2*)ThreadContext.AsyncPackage;
	UPackage* ObjectPackage = Object->GetPackage();

	if (!ContextAsyncPackage || !ObjectPackage)
	{
		// Something is creating objects on the async loading thread outside of the actual async loading code
		// e.g. ShaderCodeLibrary::OnExternalReadCallback doing FTaskGraphInterface::Get().WaitUntilTaskCompletes(Event);
		// Or the object doesn't have a package, in which case we didn't load it from disk. Created objects not part of the package should be cleaned up by their creator
		Object->AtomicallyClearInternalFlags(EInternalObjectFlags_AsyncLoading);
		UE_LOGF(LogStreaming, VeryVerbose, "Constructed object '%ls' during loading will not be post-loaded due to its package '%ls' (null and transient package sub-objects are ignored).", *Object->GetFullName(), (ObjectPackage ? *ObjectPackage->GetFullName() : TEXT("null")));
		return;
	}

	auto IsTargetPackageSafeForAdd =
		[this](FAsyncPackage2* TargetPackage)
		{
			// if the loader isn't multithreaded, it's fine to add constructed packages as long as the package
			// is not done with its deferred postload phase.
			if (!IsMultithreaded())
			{
				return TargetPackage->AsyncPackageLoadingState <= EAsyncPackageLoadingState2::DeferredPostLoad;
			}

			// To avoid race conditions, ALT packages should only be added to by ALT threads. 
			if (TargetPackage->AsyncPackageLoadingState < EAsyncPackageLoadingState2::DeferredPostLoad)
			{
				return !IsInGameThread();
			}

			// To avoid race conditions, GT packages should only be added to by GT threads.
			if (TargetPackage->AsyncPackageLoadingState == EAsyncPackageLoadingState2::DeferredPostLoad)
			{
				return IsInGameThread();
			}

			return false;
		};

#if WITH_EDITOR
	// In editor, objects from other packages might be constructed with the wrong package currently on stack (i.e. reinstantiation).
	// It's a lot cleaner and easier to properly dispatch them to their respective package than adding scopes everywhere.
	if (FLinkerLoad* LinkerLoad = ObjectPackage->GetLinker())
	{
		if (FAsyncPackage2* AsyncPackage = (FAsyncPackage2*)LinkerLoad->AsyncRoot)
		{
			if (IsTargetPackageSafeForAdd(AsyncPackage))
			{
				check(AsyncPackage->Desc.UPackageId == ObjectPackage->GetPackageId());
				AsyncPackage->AddConstructedObject(Object, bSubObjectThatAlreadyExists);
				return;
			}
		}
	}

	const FPackageId ObjectPackageId = ObjectPackage->GetPackageId();
	if (ObjectPackageId.IsValid())
	{
		bool bReturn = false;
		AsyncPackageLookup.FindAndApply(ObjectPackageId,
			[&](FAsyncPackage2* AsyncPackage)
			{
				if (IsTargetPackageSafeForAdd(AsyncPackage))
				{
					AsyncPackage->AddConstructedObject(Object, bSubObjectThatAlreadyExists);
					bReturn = true;
				}
			}
		);

		if (bReturn)
		{
			return;
		}
	}
#endif

	if (IsTargetPackageSafeForAdd(ContextAsyncPackage))
	{
		UE_CLOGF(ObjectPackage->GetPackageId().IsValid() && ContextAsyncPackage->Desc.UPackageId != ObjectPackage->GetPackageId(), LogStreaming, VeryVerbose,
			"Constructed object '%ls' is part of package '%ls' which is no longer postloading objects. This object will be postloaded with the FUObjectThreadContext AsyncPackage '%ls'(%ls) instead.",
			*Object->GetFullName(),
			*ObjectPackage->GetFullName(),
			*ContextAsyncPackage->Desc.UPackageName.ToString(),
			LexToString(ContextAsyncPackage->AsyncPackageLoadingState));

		ContextAsyncPackage->AddConstructedObject(Object, bSubObjectThatAlreadyExists);
		return;
	}

	Object->AtomicallyClearInternalFlags(EInternalObjectFlags_AsyncLoading);
	UE_LOGF(LogStreaming, Error, "Object '%ls' was created while the FUObjectThreadContext's async package '%ls(0x%ls)' is done post-loading objects. This should not happening (perhaps a FAsyncPackageScope2 is missing?) This object may be left in a partially loaded state and may leak.",
		*Object->GetFullName(), *ContextAsyncPackage->Desc.UPackageName.ToString(), *LexToString(ContextAsyncPackage->Desc.UPackageId));
}

void FAsyncLoadingThread2::NotifyRegistrationEvent(
	FName PackageName,
	FName Name,
	ENotifyRegistrationType NotifyRegistrationType,
	ENotifyRegistrationPhase NotifyRegistrationPhase,
	FTypeConstructFunc* InRegister,
	bool InbDynamic,
	UObject* FinishedObject)
{
	if (NotifyRegistrationPhase == ENotifyRegistrationPhase::NRP_Finished)
	{
		ensureMsgf(FinishedObject, TEXT("FinishedObject was not provided by NotifyRegistrationEvent when called with ENotifyRegistrationPhase::NRP_Finished, see call stack for offending code."));
		FNameBuilder PackageNameBuilder(PackageName);
		FNameBuilder NameBuilder(Name);
		GlobalImportStore.AddScriptObject(PackageNameBuilder.ToView(), NameBuilder.ToView(), FinishedObject);
	}
}

void FAsyncLoadingThread2::NotifyScriptVersePackage(Verse::VPackage* Package)
{
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	GlobalImportStore.AddScriptCellPackage(Package);
#endif
}

void FAsyncLoadingThread2::NotifyCompiledVersePackage(Verse::VPackage* VersePackage, UPackage* Package, TArrayView<UObject*> Exports)
{
	FName PackageName = Package->GetFName();
	FPackageId PackageId = FPackageId::FromName(PackageName);

	// Construct an export map to reserve space in the package, rather than inserting objects directly.
	// StoreGlobalObject does not amortize export map reallocations, so this avoids a quadratic slowdown.
	TArray<FExportMapEntry> ExportMap;
	ExportMap.Reserve(Exports.Num());
	for (UObject* Object : Exports)
	{
		ensure(Object->HasAllFlags(RF_Public));
		Object->SetInternalFlags(EInternalObjectFlags::LoaderImport);
		Object->SetFlags(RF_WasLoaded | RF_LoadCompleted);

		TArray<FName, TInlineAllocator<64>> FullPath;
		FullPath.Add(Object->GetFName());
		UObject* Outer = Object->GetOuter();
		while (Outer)
		{
			FullPath.Add(Outer->GetFName());
			Outer = Outer->GetOuter();
		}
		TStringBuilder<256> PackageRelativeExportPath;
		for (int32 PathIndex = FullPath.Num() - 2; PathIndex >= 0; --PathIndex)
		{
			TCHAR NameStr[FName::StringBufferSize];
			uint32 NameLen = FullPath[PathIndex].ToString(NameStr);
			for (uint32 I = 0; I < NameLen; ++I)
			{
				NameStr[I] = TChar<TCHAR>::ToLower(NameStr[I]);
			}
			PackageRelativeExportPath.AppendChar('/');
			PackageRelativeExportPath.Append(FStringView(NameStr, NameLen));
		}
		uint64 ExportHash = CityHash64(reinterpret_cast<const char*>(PackageRelativeExportPath.GetData() + 1), (PackageRelativeExportPath.Len() - 1) * sizeof(TCHAR));

		ExportMap.Add({ .PublicExportHash = ExportHash });
	}

	{
		FWriteLockedPackageRef PackageRef = GlobalImportStore.AddPackageRef(PackageId, PackageName, EPackageLoader::Unknown, EPackageExtension::Unspecified);
		PackageRef->PreInsertPublicExports({ ExportMap });

#if !WITH_EDITOR
		ensureMsgf(!PackageRef->HasPackage(), TEXT("Compiled Verse package %s has already been added to the loader."), *PackageName.ToString());
		PackageRef->SetPackage(Package);
		Package->SetCanBeImportedFlag(true);
		Package->SetPackageId(PackageId);
#endif
		PackageRef->SetAllPublicExportsLoaded(true);
		Package->SetInternalFlags(EInternalObjectFlags::LoaderImport);
	}
	for (int32 Index = 0; Index < Exports.Num(); ++Index)
	{
		GlobalImportStore.StoreGlobalObject(PackageId, ExportMap[Index].PublicExportHash, Exports[Index]);
	}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	for (int32 Index = 0; Index < VersePackage->NumDefinitions(); Index++)
	{
		if (Verse::VCell* Cell = VersePackage->GetDefinition(Index).DynamicCast<Verse::VCell>())
		{
			FUtf8StringView VersePath = VersePackage->GetDefinitionName(Index).AsStringView();
			uint64 ExportHash = CityHash64(reinterpret_cast<const char*>(VersePath.GetData()), VersePath.Len());
			GlobalImportStore.StoreGlobalCell(PackageId, ExportHash, Cell);
		}
	}
#endif
	GlobalImportStore.ReleasePackageRef(PackageId);
}

void FAsyncLoadingThread2::NotifyRegistrationComplete()
{
	GlobalImportStore.RegistrationComplete();
	bHasRegisteredAllScriptObjects = true;

	UE_LOGF(LogStreaming, Log,
		"AsyncLoading2 - NotifyRegistrationComplete: Registered %d public script object entries (%.2f KB)",
		GlobalImportStore.GetStoredScriptObjectsCount(), (float)GlobalImportStore.GetStoredScriptObjectsAllocatedSize() / 1024.f);
}

#if WITH_EDITOR
void FAsyncLoadingThread2::OnLoadedObjectRenamed(UObject* Obj, UObject* OldOuter, FName OldName)
{
	const int32 Index = GUObjectArray.ObjectToIndex(Obj);
	DeferredLoadedObjectRenameQueue.Enqueue(Index, Obj);
}
#endif

void FAsyncLoadingThread2::FlushDeferredLoadedObjectRenameQueue()
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncLoadingThread2::FlushDeferredLoadedObjectRenameQueue);
	FLoadedObjectRenamed RenamedObject;
	while (DeferredLoadedObjectRenameQueue.Dequeue(RenamedObject))
	{
		// Attempt to remove this object as a public export (it might not be). 
		// 
		// If we just renamed a loaded object to no longer match the public export, we want the export removed.
		// 
		// If we just renamed a new object to have the same name as a previously removed public export, do nothing. Why?
		// We can't rename new object instances on top of public export names without having first renamed/removed them. So if we 
		// are in this call with an object that was renamed to be "back" in the export table we must have already marked the package 
		// as not fully loaded. Thus, any load of the package will now either reload the removed export from disk or find the newly renamed 
		// object instance in memory (if it is indeed an object being renamed back into the export table) and use that.
		GlobalImportStore.RemoveDeferredPublicExport(RenamedObject.ObjectIndex, RenamedObject.Object);
	}
#endif
}

/*-----------------------------------------------------------------------------
	FAsyncPackage implementation.
-----------------------------------------------------------------------------*/

/**
* Constructor
*/
FAsyncPackage2::FAsyncPackage2(
	FAsyncLoadingThreadState2& ThreadState,
	const FAsyncPackageDesc2& InDesc,
	FAsyncLoadingThread2& InAsyncLoadingThread,
	FAsyncLoadEventGraphAllocator& InGraphAllocator,
	const FAsyncLoadEventSpec* EventSpecs)
	: Desc(InDesc)
	, AsyncLoadingThread(InAsyncLoadingThread)
	, GraphAllocator(InGraphAllocator)
	, ImportStore(AsyncLoadingThread.GlobalImportStore)
{
	TRACE_LOADTIME_NEW_ASYNC_PACKAGE(this);
	AddRequestID(ThreadState, Desc.RequestID);

	CreatePackageNodes(EventSpecs);

	ImportStore.AddPackageReference(Desc);
}

void FAsyncPackage2::CreatePackageNodes(const FAsyncLoadEventSpec* EventSpecs)
{
	const int32 BarrierCount = 1;

	FEventLoadNode2* Node = reinterpret_cast<FEventLoadNode2*>(PackageNodesMemory);
	for (int32 Phase = 0; Phase < EEventLoadNode2::Package_NumPhases; ++Phase)
	{
		new (Node + Phase) FEventLoadNode2(EventSpecs + Phase, this, BarrierCount);
	}
	PackageNodes = MakeArrayView(Node, EEventLoadNode2::Package_NumPhases);
}

FAsyncPackage2::~FAsyncPackage2()
{
	TRACE_LOADTIME_DESTROY_ASYNC_PACKAGE(this);
	UE_ASYNC_PACKAGE_LOG(Verbose, Desc, TEXT("AsyncThread: Deleted"), TEXT("Package deleted."));

	FProgressCallbackEntry* Head = ProgressCallbacksHead.exchange(nullptr, std::memory_order_relaxed);
	while (Head)
	{
		FProgressCallbackEntry* Next = Head->Next.load(std::memory_order_relaxed);
		delete Head;
		Head = Next;
	}

	ImportStore.ReleaseImportedPackageReferences(Desc, HeaderData.ImportedPackageIds);
#if WITH_EDITOR
	if (OptionalSegmentHeaderData.IsSet())
	{
		ImportStore.ReleaseImportedPackageReferences(Desc, OptionalSegmentHeaderData->ImportedPackageIds);
	}
#endif
	ImportStore.ReleasePackageReference(Desc);

	checkf(RefCount.load(std::memory_order_relaxed) == 0, TEXT("RefCount is not 0 when deleting package %s"),
		*Desc.PackagePathToLoad.GetPackageFName().ToString());

	checkf(ConstructedObjects.Num() == 0, TEXT("ClearConstructedObjects() has not been called for package %s"),
		*Desc.PackagePathToLoad.GetPackageFName().ToString());

	FMemory::Free(Data.MemoryBuffer0);
	FMemory::Free(Data.MemoryBuffer1);
}

void FAsyncPackage2::Release()
{
	int32 OldRefCount = RefCount.fetch_sub(1);
	check(OldRefCount > 0);
	if (OldRefCount == 1)
	{
		FAsyncLoadingThread2& AsyncLoadingThreadLocal = AsyncLoadingThread;
		AsyncLoadingThreadLocal.DeferredCleanupPackages.Enqueue(this);
		AsyncLoadingThreadLocal.AltZenaphore.NotifyOne();
	}
}

void FAsyncPackage2::ClearImportedPackages()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ClearImportedPackages);
	// Reset the ImportedAsyncPackages member array before releasing the async package references.
	// Release may queue up the ImportedAsyncPackage for deletion on the ALT.
	// ImportedAsyncPackages of this package can still be accessed by both the GT and the ALT.
	TArrayView<FAsyncPackage2*> LocalImportedAsyncPackages = Data.ImportedAsyncPackages;
	Data.ImportedAsyncPackages = MakeArrayView(Data.ImportedAsyncPackages.GetData(), 0);
	for (FAsyncPackage2* ImportedAsyncPackage : LocalImportedAsyncPackages)
	{
		if (ImportedAsyncPackage)
		{
			ImportedAsyncPackage->Release();
		}
	}

	TSet<FAsyncPackage2*> LocalAdditionalImportedAsyncPackages = MoveTemp(AdditionalImportedAsyncPackages);
	for (FAsyncPackage2* ImportedAsyncPackage : LocalAdditionalImportedAsyncPackages)
	{
		ImportedAsyncPackage->Release();
	}
}

void FAsyncPackage2::MoveConstructedObjectsToPhase2()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MoveConstructedObjectsToPhase2);
	{
		UE_MT_SCOPED_READ_ACCESS(ConstructedObjectsAccessDetector);
		for (UObject* Object : ConstructedObjects)
		{
			checkObject(Object, !Object->HasAnyFlags(RF_NeedLoad | RF_NeedInitialization));
			Object->SetInternalFlags(EInternalObjectFlags::AsyncLoadingPhase2);
			Object->AtomicallyClearInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1);
		}
	}

	TArray<UObject*> CDODefaultSubobjects;
	for (FExportObject& Export : Data.Exports)
	{
		UObject* Object = Export.Object;

		// CDO need special handling, no matter if it's listed in DeferredFinalizeObjects
		UObject* CDOToHandle = ((Object != nullptr) && Object->HasAnyFlags(RF_ClassDefaultObject)) ? Object : nullptr;

		if (CDOToHandle != nullptr)
		{
			CDOToHandle->GetDefaultSubobjects(CDODefaultSubobjects);
			for (UObject* SubObject : CDODefaultSubobjects)
			{
				if (SubObject && SubObject->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1))
				{
					checkObject(SubObject, !SubObject->HasAnyFlags(RF_NeedLoad | RF_NeedInitialization));
					SubObject->SetInternalFlags(EInternalObjectFlags::AsyncLoadingPhase2);
					SubObject->AtomicallyClearInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1);
				}
			}
			CDODefaultSubobjects.Reset();
		}

		if (Object)
		{
			checkObject(Object, !Object->HasAnyFlags(RF_NeedLoad | RF_NeedInitialization));
			if (Object->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1))
			{
				Object->SetInternalFlags(EInternalObjectFlags::AsyncLoadingPhase2);
				Object->AtomicallyClearInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1);
			}
		}
	}

	if (LinkerRoot && LinkerRoot->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1))
	{
		checkObject(LinkerRoot, !LinkerRoot->HasAnyFlags(RF_NeedLoad | RF_NeedInitialization));
		LinkerRoot->SetInternalFlags(EInternalObjectFlags::AsyncLoadingPhase2);
		LinkerRoot->AtomicallyClearInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1);
	}

	ObjectsNowInPhase2 = true;
}

void FAsyncPackage2::ClearConstructedObjects()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ClearConstructedObjects);
	UE_MT_SCOPED_WRITE_ACCESS(ConstructedObjectsAccessDetector);

	for (UObject* Object : ConstructedObjects)
	{
		Object->AtomicallyClearInternalFlags(EInternalObjectFlags_AsyncLoading | EInternalObjectFlags::Async);
	}
	ConstructedObjects.Empty();

	const int32 ExportCount = Data.Exports.Num();
	for (int32 ExportIndex = 0; ExportIndex < ExportCount; ++ExportIndex)
	{
		FExportObject& Export = Data.Exports[ExportIndex];
		if (Export.bWasFoundInMemory)
		{
			checkf(Export.Object, TEXT("Export %d in package '%s' found in memory is missing:\n\t\tSuper: '%s', Template: '%s', LoadFailed: %s, Filtered: %s (%s), FoundInMemory: %s"),
				ExportIndex, // The header is cleared by this point so we only have the export index we can provide for backtracking purposes
				*Desc.PackagePathToLoad.GetPackageFName().ToString(),
				(Export.SuperObject ? *Export.SuperObject->GetPathName() : TEXT("null")),
				(Export.TemplateObject ? *Export.TemplateObject->GetPathName() : TEXT("null")),
				(Export.bExportLoadFailed ? TEXT("true") : TEXT("false")),
				(Export.bFiltered ? TEXT("true") : TEXT("false")),
				LexToString(HeaderData.ExportMap[ExportIndex].FilterFlags),
				(Export.bWasFoundInMemory ? TEXT("true") : TEXT("false")));

			Export.Object->AtomicallyClearInternalFlags(EInternalObjectFlags_AsyncLoading | EInternalObjectFlags::Async);
		}
		else
		{
			checkObject(Export.Object, !Export.Object || !Export.Object->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading | EInternalObjectFlags::Async));
		}
	}

	if (LinkerRoot)
	{
		LinkerRoot->AtomicallyClearInternalFlags(EInternalObjectFlags_AsyncLoading | EInternalObjectFlags::Async);
	}
}

void FAsyncPackage2::AddRequestID(FAsyncLoadingThreadState2& ThreadState, int32 Id)
{
	if (Id > 0)
	{
		if (Desc.RequestID == INDEX_NONE)
		{
			// For debug readability
			Desc.RequestID = Id;
		}

		// We expect the amount of aliasing to be very low so add unique to reduce memory
		RequestIDs.AddUnique(Id);

		// The Id is most likely already present because it's added as soon as the request is created.
		AsyncLoadingThread.AddPendingRequest(Id);

		TRACE_LOADTIME_ASYNC_PACKAGE_REQUEST_ASSOCIATION(this, Id);
	}
}

/**
 * @return Time load begun. This is NOT the time the load was requested in the case of other pending requests.
 */
double FAsyncPackage2::GetLoadStartTime() const
{
	return LoadStartTime;
}

/**
 * Begin async loading process. Simulates parts of BeginLoad.
 *
 * Objects created during BeginAsyncLoad and EndAsyncLoad will have EInternalObjectFlags::AsyncLoading set
 */
void FAsyncPackage2::BeginAsyncLoad()
{
	if (IsInGameThread())
	{
		AsyncLoadingThread.EnterAsyncLoadingTick();
	}

	// this won't do much during async loading except increase the load count which causes IsLoading to return true
	FUObjectSerializeContext* LoadContext = GetSerializeContext();
	BeginLoad(LoadContext);
}

/**
 * End async loading process. Simulates parts of EndLoad().
 */
void FAsyncPackage2::EndAsyncLoad()
{
	check(AsyncLoadingThread.IsAsyncLoadingPackages());

	// this won't do much during async loading except decrease the load count which causes IsLoading to return false
	FUObjectSerializeContext* LoadContext = GetSerializeContext();
	EndLoad(LoadContext);

	if (IsInGameThread())
	{
		AsyncLoadingThread.LeaveAsyncLoadingTick();
	}
}

void FAsyncPackage2::CreateUPackage()
{
	check(!LinkerRoot);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPackageFind);
		LinkerRoot = FindObjectFast<UPackage>(/*Outer*/nullptr, Desc.UPackageName);
	}
	if (!LinkerRoot)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UPackageCreate);
		UE_TRACK_REFERENCING_PACKAGE_SCOPED(Desc.PackageReferencer.ReferencerPackageName, Desc.PackageReferencer.ReferencerPackageOp);

#if WITH_EDITOR
		FCookLoadScope CookLoadScope(Desc.PackageReferencer.CookLoadType);
#endif
		// Add scope so that this constructed object is assigned to ourself and not another package up the stack in ImportPackagesRecursive.
		FAsyncPackageScope2 Scope(this);
		LinkerRoot = NewObject<UPackage>(/*Outer*/nullptr, Desc.UPackageName);
		bCreatedLinkerRoot = true;
	}

#if WITH_EDITOR
	// Do not overwrite PIEInstanceID for package that might already have one from a previous load
	// or an in-memory only package created specifically for PIE.
	if (!LinkerRoot->bHasBeenFullyLoaded && LinkerRoot->GetLoadedPath().IsEmpty())
	{
		LinkerRoot->SetPIEInstanceID(Desc.PIEInstanceID);
	}
#endif

	LinkerRoot->SetFlags(RF_Public | RF_WillBeLoaded);
	LinkerRoot->SetLoadedPath(Desc.PackagePathToLoad);
	LinkerRoot->SetCanBeImportedFlag(Desc.bCanBeImported);
	LinkerRoot->SetPackageId(Desc.UPackageId);
	LinkerRoot->SetPackageFlags(Desc.PackageFlags);

	EInternalObjectFlags FlagsToSet = EInternalObjectFlags::Async;
	if (Desc.bCanBeImported)
	{
		FlagsToSet |= EInternalObjectFlags::LoaderImport;
	}
	LinkerRoot->SetInternalFlags(FlagsToSet);

	// update global import store
	// temp packages can't be imported and are never stored or found in global import store
	if (Desc.bCanBeImported)
	{
		FWriteLockedPackageRef PackageRef = AsyncLoadingThread.GlobalImportStore.FindPackageRefCheckedForWrite(Desc.UPackageId, Desc.UPackageName);
		UPackage* ExistingPackage = PackageRef->GetPackage();
		if (!ExistingPackage)
		{
			PackageRef->SetPackage(LinkerRoot);
		}
		else if (ExistingPackage != LinkerRoot)
		{
			UE_ASYNC_PACKAGE_LOG(Warning, Desc, TEXT("CreateUPackage: ReplacePackage"),
				TEXT("Replacing renamed package %s (0x%s) while being referenced by the loader, RefCount=%d"),
				*ExistingPackage->GetName(), *LexToString(ExistingPackage->GetPackageId()),
				PackageRef->GetPackageRefCount());

			AsyncLoadingThread.GlobalImportStore.ReplaceReferencedRenamedPackageNoLock(*PackageRef, LinkerRoot);
		}
	}

	if (bCreatedLinkerRoot)
	{
		UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("CreateUPackage: AddPackage"),
			TEXT("New UPackage created."));
	}
	else
	{
		UE_ASYNC_PACKAGE_LOG_VERBOSE(VeryVerbose, Desc, TEXT("CreateUPackage: UpdatePackage"),
			TEXT("Existing UPackage updated."));
	}

	UE_TRACE_PACKAGE_NAME(Desc.UPackageId, Desc.UPackageName);
}

EAsyncPackageState::Type FAsyncPackage2::ProcessExternalReads(FAsyncLoadingThreadState2& ThreadState, EExternalReadAction Action)
{
	check(AsyncPackageLoadingState == EAsyncPackageLoadingState2::WaitingForExternalReads);
	double WaitTime;
	if (Action == ExternalReadAction_Poll)
	{
		WaitTime = -1.f;
	}
	else// if (Action == ExternalReadAction_Wait)
	{
		WaitTime = 0.f;
	}

	while (ExternalReadIndex < ExternalReadDependencies.Num())
	{
		FExternalReadCallback& ReadCallback = ExternalReadDependencies[ExternalReadIndex];
		if (!ReadCallback(WaitTime))
		{
			return EAsyncPackageState::TimeOut;
		}
		++ExternalReadIndex;
	}

	ExternalReadDependencies.Empty();
	GetPackageNode(Package_ExportsSerialized).ReleaseBarrier(&ThreadState);
	return EAsyncPackageState::Complete;
}

EAsyncPackageState::Type FAsyncPackage2::PostLoadInstances(FAsyncLoadingThreadState2& ThreadState)
{
	if (HasLoadFailed())
	{
		return EAsyncPackageState::Complete;
	}
	const int32 ExportCount = Data.Exports.Num();
	while (PostLoadInstanceIndex < ExportCount && !ThreadState.IsTimeLimitExceeded(TEXT("PostLoadInstances")))
	{
		const FExportObject& Export = Data.Exports[PostLoadInstanceIndex++];

		if (Export.Object && !(Export.bFiltered || Export.bExportLoadFailed))
		{
			UClass* ObjClass = Export.Object->GetClass();
			ObjClass->PostLoadInstance(Export.Object);
		}
	}
	return PostLoadInstanceIndex == ExportCount ? EAsyncPackageState::Complete : EAsyncPackageState::TimeOut;
}

EAsyncPackageState::Type FAsyncPackage2::CreateClusters(FAsyncLoadingThreadState2& ThreadState)
{
	const int32 ExportCount = Data.Exports.Num();
	while (DeferredClusterIndex < ExportCount)
	{
		const FExportObject& Export = Data.Exports[DeferredClusterIndex++];

		if (!(Export.bFiltered || Export.bExportLoadFailed) && Export.Object->CanBeClusterRoot())
		{
			Export.Object->CreateCluster();
			if (DeferredClusterIndex < ExportCount && ThreadState.IsTimeLimitExceeded(TEXT("CreateClusters")))
			{
				break;
			}
		}
	}

	return DeferredClusterIndex == ExportCount ? EAsyncPackageState::Complete : EAsyncPackageState::TimeOut;
}

void FAsyncPackage2::FinishUPackage()
{
	if (LinkerRoot && !HasLoadFailed())
	{
		// Mark package as having been fully loaded and update load time.
		LinkerRoot->MarkAsFullyLoaded();
		LinkerRoot->SetFlags(RF_WasLoaded);
		LinkerRoot->ClearFlags(RF_WillBeLoaded);
		LinkerRoot->SetLoadTime((float)(FPlatformTime::Seconds() - LoadStartTime));
	}
}

void FAsyncLoadingThread2::ConditionalProcessCallbacks()
{
	check(IsInGameThread());

	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	if (ThreadContext.IsRoutingPostLoad || !GameThreadState->SyncLoadContextStack.IsEmpty())
	{
		return;
	}

#if WITH_EDITOR
	// Prevent objects still being loaded from being accessible from the game thread.
	TGuardValue GuardVisibilityFilter(ThreadContext.AsyncVisibilityFilter, EInternalObjectFlags::AsyncLoadingPhase1);

	FBlueprintSupport::FlushReinstancingQueue();
#endif

	// Editor builds always broadcast OnEndLoadPackage; non-editor builds only when the asset read logger is compiled in.
#if WITH_EDITOR || UE_ENABLE_ASSET_READ_LOGGER
	while (!CompletedUPackagesForBroadcast.IsEmpty()
#if WITH_EDITOR
		|| !EditorLoadedAssets.IsEmpty()
#endif
		)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncLoadingThread2::ConditionalProcessEditorCallbacks);

		TArray<UPackage*> LocalCompletedUPackages;
		Swap(LocalCompletedUPackages, CompletedUPackagesForBroadcast);

		// Call the global delegate for package endloads and set the bHasBeenLoaded flag that is used to
		// check which packages have reached this state
		for (UPackage* CompletedUPackage : LocalCompletedUPackages)
		{
			CompletedUPackage->SetHasBeenEndLoaded(true);
		}

		FCoreUObjectDelegates::OnEndLoadPackage.Broadcast(
			FEndLoadPackageContext{ LocalCompletedUPackages, 0, false /* bSynchronous */ });

#if WITH_EDITOR
		// In editor builds, call the asset load callback. This happens in both editor and standalone to match EndLoad.
		TArray<UObject*> LocalEditorLoadedAssets;
		Swap(LocalEditorLoadedAssets, EditorLoadedAssets);
		for (UObject* LoadedObject : LocalEditorLoadedAssets)
		{
			if (LoadedObject)
			{
				UE_TRACK_REFERENCING_PACKAGE_SCOPED(LoadedObject->GetPackage()->GetFName(), PackageAccessTrackingOps::NAME_Load);
				FCoreUObjectDelegates::OnAssetLoaded.Broadcast(LoadedObject);
			}
		}
#endif
	}
#endif // UE_ENABLE_ASSET_READ_LOGGER

	while (!HotfixableLoadedAssets.IsEmpty())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncLoadingThread2::ConditionalProcessHotfixCallbacks);

		TArray<UObject*> LocalHotfixableLoadedAssets;
		Swap(LocalHotfixableLoadedAssets, HotfixableLoadedAssets);

		for (UObject* LoadedObject : LocalHotfixableLoadedAssets)
		{
			if (LoadedObject)
			{
				FCoreUObjectDelegates::OnHotfixableAssetLoaded.Broadcast(LoadedObject);
			}
		}
	}
}

void FAsyncPackage2::AddCompletionCallback(TUniquePtr<FLoadPackageAsyncDelegate>&& Callback)
{
	// This is to ensure that there is no one trying to subscribe to a already loaded package
	//check(!bLoadHasFinished && !HasLoadFailed());
	CompletionCallbacks.Emplace(MoveTemp(Callback));
}

void FAsyncPackage2::AddProgressCallback(TSharedPtr<FLoadPackageAsyncProgressDelegate> Callback)
{
	FProgressCallbackEntry* Entry = new FProgressCallbackEntry();
	Entry->Delegate = MoveTemp(Callback);

	// Lock-free push onto intrusive linked list
	FProgressCallbackEntry* OldHead = ProgressCallbacksHead.load(std::memory_order_relaxed);
	while (true)
	{
		Entry->Next.store(OldHead, std::memory_order_relaxed);
		if (ProgressCallbacksHead.compare_exchange_weak(OldHead, Entry, std::memory_order_release, std::memory_order_relaxed))
		{
			break;
		}
	}

	EAsyncLoadingProgress CurrentProgress = GetLoadingProgressFromState(AsyncPackageLoadingState);
	if (CurrentProgress > EAsyncLoadingProgress::Failed)
	{
		CallProgressCallback(*Entry, Desc.UPackageName, GetLinkerRoot(), CurrentProgress);
	}
}

UE_TRACE_EVENT_BEGIN(CUSTOM_LOADTIMER_LOG, LoadAsyncPackageInternal, NoSync)
UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PackageName)
UE_TRACE_EVENT_END()

int32 FAsyncLoadingThread2::LoadPackageInternal(const FPackagePath& InPackagePath, FName InCustomName, TUniquePtr<FLoadPackageAsyncDelegate>&& InCompletionDelegate, TSharedPtr<FLoadPackageAsyncProgressDelegate> InProgressDelegate, EPackageFlags InPackageFlags, int32 InPIEInstanceID, int32 InPackagePriority, const FLinkerInstancingContext* InInstancingContext, uint32 InLoadFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(LoadPackage);

	const FName PackageNameToLoad = InPackagePath.GetPackageFName();
	if (InCustomName == PackageNameToLoad)
	{
		InCustomName = NAME_None;
	}
	SCOPED_CUSTOM_LOADTIMER(LoadAsyncPackageInternal)
		ADD_CUSTOM_LOADTIMER_META(LoadAsyncPackageInternal, PackageName, *WriteToString<256>(PackageNameToLoad));

	if (FCoreDelegates::GetOnAsyncLoadPackage().IsBound())
	{
		const FName PackageName = InCustomName.IsNone() ? PackageNameToLoad : InCustomName;
		FCoreDelegates::GetOnAsyncLoadPackage().Broadcast(PackageName.ToString());
	}

	// Generate new request ID and add it immediately to the global request list (it needs to be there before we exit
	// this function, otherwise it would be added when the packages are being processed on the async thread).
	const int32 RequestId = IAsyncPackageLoader::GetNextRequestId();
	TRACE_LOADTIME_BEGIN_REQUEST(RequestId);
	AddPendingRequest(RequestId);

	FPackageReferencer PackageReferencer;
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	PackageAccessTracking_Private::FTrackedData* AccumulatedScopeData = PackageAccessTracking_Private::FPackageAccessRefScope::GetCurrentThreadAccumulatedData();
	if (AccumulatedScopeData)
	{
		PackageReferencer.ReferencerPackageName = AccumulatedScopeData->PackageName;
		PackageReferencer.ReferencerPackageOp = AccumulatedScopeData->OpName;
	}
#endif
#if WITH_EDITOR
	PackageReferencer.CookLoadType = FCookLoadScope::GetCurrentValue();
#endif
	PackageRequestQueue.Enqueue(FPackageRequest::Create(RequestId, InPackageFlags, InLoadFlags, InPIEInstanceID, InPackagePriority, InInstancingContext, InPackagePath, InCustomName, MoveTemp(InCompletionDelegate), MoveTemp(InProgressDelegate), PackageReferencer));
	++QueuedPackagesCounter;
	++PackagesWithRemainingWorkCounter;

	TRACE_COUNTER_SET(AsyncLoadingQueuedPackages, QueuedPackagesCounter);
	TRACE_COUNTER_SET(AsyncLoadingPackagesWithRemainingWork, PackagesWithRemainingWorkCounter);

	AltZenaphore.NotifyOne();

	return RequestId;
}

int32 FAsyncLoadingThread2::LoadPackage(const FPackagePath& InPackagePath, FLoadPackageAsyncOptionalParams InParams)
{
	return LoadPackageInternal(InPackagePath, InParams.CustomPackageName, MoveTemp(InParams.CompletionDelegate), InParams.ProgressDelegate, InParams.PackageFlags, InParams.PIEInstanceID, InParams.PackagePriority, InParams.InstancingContext, InParams.LoadFlags);
}

int32 FAsyncLoadingThread2::LoadPackage(const FPackagePath& InPackagePath, FName InCustomName, FLoadPackageAsyncDelegate InCompletionDelegate, EPackageFlags InPackageFlags, int32 InPIEInstanceID, int32 InPackagePriority, const FLinkerInstancingContext* InInstancingContext, uint32 InLoadFlags)
{
	// Allocate delegate before going async, it is not safe to copy delegates by value on other threads
	TUniquePtr<FLoadPackageAsyncDelegate> CompletionDelegate = InCompletionDelegate.IsBound()
		? MakeUnique<FLoadPackageAsyncDelegate>(MoveTemp(InCompletionDelegate))
		: TUniquePtr<FLoadPackageAsyncDelegate>();

	return LoadPackageInternal(InPackagePath, InCustomName, MoveTemp(CompletionDelegate), nullptr, InPackageFlags, InPIEInstanceID, InPackagePriority, InInstancingContext, InLoadFlags);
}

void FAsyncLoadingThread2::QueueUnresolvedPackage(FAsyncLoadingThreadState2& ThreadState, EPackageStoreEntryStatus PackageStatus, FAsyncPackageDesc2& PackageDesc, TUniquePtr<FLoadPackageAsyncDelegate>&& PackageLoadedDelegate, TSharedPtr<FLoadPackageAsyncProgressDelegate> PackageProgressDelegate)
{
	const FName FailedPackageName = PackageDesc.UPackageName;

	static TSet<FName> SkippedPackages;
	bool bIsAlreadySkipped = false;

	SkippedPackages.Add(FailedPackageName, &bIsAlreadySkipped);

	bool bIssueWarning = !bIsAlreadySkipped;
#if WITH_EDITOR
	bIssueWarning &= ((PackageDesc.LoadFlags & (LOAD_NoWarn | LOAD_Quiet)) == 0);
#endif
	if (bIssueWarning)
	{
		bool bIsScriptPackage = FPackageName::IsScriptPackage(WriteToString<FName::StringBufferSize>(FailedPackageName));
		bIssueWarning &= !bIsScriptPackage;
	}

	if (PackageStatus == EPackageStoreEntryStatus::NotInstalled)
	{
		UE_CLOGF(bIssueWarning, LogStreaming, Warning,
			"LoadPackage: SkipPackage: %ls (0x%ls) - The package to load, or one of its dependencies (see logs above), does not exist on disk or in the loader but may be installed on demand later.",
			*FailedPackageName.ToString(), *LexToString(PackageDesc.PackageIdToLoad));
	}
	else
	{
		UE_CLOGF(bIssueWarning, LogStreaming, Warning,
			"LoadPackage: SkipPackage: %ls (0x%ls) - The package to load does not exist on disk or in the loader",
			*FailedPackageName.ToString(), *LexToString(PackageDesc.PackageIdToLoad));
	}

	if (PackageProgressDelegate.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PackageProgressCallback_Failed);

		FLoadPackageAsyncProgressParams Params
		{
			.PackageName = FailedPackageName,
			.LoadedPackage = nullptr,
			.ProgressType = EAsyncLoadingProgress::Failed
		};

		PackageProgressDelegate->Invoke(Params);
	}

	if (PackageLoadedDelegate.IsValid())
	{
		EAsyncLoadingResult::Type Result = EAsyncLoadingResult::FailedMissing;
		if (PackageStatus == EPackageStoreEntryStatus::NotInstalled)
		{
			Result = EAsyncLoadingResult::FailedNotInstalled;
		}

		FScopeLock _(&FailedPackageRequestsCritical);
		FailedPackageRequests.Add(
			FCompletedPackageRequest::FromUnreslovedPackage(PackageDesc, Result, MoveTemp(PackageLoadedDelegate)));
	}
	else
	{
		RemovePendingRequests(ThreadState, { PackageDesc.RequestID });
		--PackagesWithRemainingWorkCounter;
		TRACE_COUNTER_SET(AsyncLoadingPackagesWithRemainingWork, PackagesWithRemainingWorkCounter);
	}
}

EAsyncPackageState::Type FAsyncLoadingThread2::ProcessLoadingFromGameThread(FAsyncLoadingThreadState2& ThreadState, bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit)
{
	SCOPE_CYCLE_COUNTER(STAT_AsyncLoadingTime);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(AsyncLoading);

	// CSV_CUSTOM_STAT(FileIO, EDLEventQueueDepth, (int32)GraphAllocator.TotalNodeCount, ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT(FileIO, QueuedPackagesQueueDepth, GetNumQueuedPackages(), ECsvCustomStatOp::Set);
	CSV_CUSTOM_STAT_MINIMAL(FileIO, ExistingQueuedPackagesQueueDepth, GetNumAsyncPackages(), ECsvCustomStatOp::Set);

	bool bDidSomething = false;
	TickAsyncLoadingFromGameThread(ThreadState, bUseTimeLimit, bUseFullTimeLimit, TimeLimit, {}, bDidSomething);
	return IsAsyncLoadingPackages() ? EAsyncPackageState::TimeOut : EAsyncPackageState::Complete;
}

void FAsyncLoadingThread2::WarnAboutPotentialSyncLoadStall(FAsyncLoadingSyncLoadContext* SyncLoadContext)
{
	for (int32 Index = 0; Index < SyncLoadContext->RequestIDs.Num(); ++Index)
	{
		int32 RequestId = SyncLoadContext->RequestIDs[Index];
		if (ContainsRequestID(RequestId))
		{
			FAsyncPackage2* Package = SyncLoadContext->GetRequestedPackage(Index);
			UE_LOGF(LogStreaming, Warning, "A flush request appear to be stuck waiting on package %ls at state %ls to reach state > %ls",
				*Package->Desc.UPackageName.ToString(),
				LexToString(Package->AsyncPackageLoadingState),
				LexToString(SyncLoadContext->GetRequestingPackage()->AsyncPackageLoadingState)
			);
		}
	}
}

void FAsyncLoadingThread2::FlushLoadingFromParallelLoadingThread(FAsyncLoadingThreadState2& ThreadState, TConstArrayView<int32> RequestIDs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FlushLoadingFromParallelLoadingThread);

	if (RequestIDs.IsEmpty())
	{
		return;
	}

	FAsyncLoadingSyncLoadContext* SyncLoadContext = new FAsyncLoadingSyncLoadContext(RequestIDs, FAsyncPackage2::GetCurrentlyExecutingPackage(ThreadState));

	UE_LOG(LogStreaming, VeryVerbose, TEXT("Pushing ALT Worker SyncLoadContext %" UINT64_FMT), SyncLoadContext->ContextId);
	ThreadState.SyncLoadContextStack.Push(SyncLoadContext);

	SyncLoadContext->AddRef();
	AsyncLoadingThreadState->SyncLoadContextsCreatedOnOtherThreads.Enqueue(SyncLoadContext);
	AltZenaphore.NotifyOne();

	TRACE_CPUPROFILER_EVENT_FLUSH();
	SyncLoadContext->DoneEvent.Wait();
	check(!ContainsAnyRequestID(SyncLoadContext->RequestIDs));

	check(ThreadState.SyncLoadContextStack.Top() == SyncLoadContext);
	UE_LOG(LogStreaming, VeryVerbose, TEXT("Popping ALT Worker SyncLoadContext %" UINT64_FMT), ThreadState.SyncLoadContextStack.Top()->ContextId);
	ThreadState.SyncLoadContextStack.Pop();
	SyncLoadContext->ReleaseRef();
	AltZenaphore.NotifyOne();
}

void FAsyncLoadingThread2::FlushLoadingFromLoadingThread(FAsyncLoadingThreadState2& ThreadState, TConstArrayView<int32> RequestIDs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FlushLoadingFromLoadingThread);

	if (RequestIDs.IsEmpty())
	{
		return;
	}

	FAsyncLoadingSyncLoadContext* SyncLoadContext = new FAsyncLoadingSyncLoadContext(RequestIDs, FAsyncPackage2::GetCurrentlyExecutingPackage(ThreadState));

	UE_LOG(LogStreaming, VeryVerbose, TEXT("Pushing ALT SyncLoadContext %" UINT64_FMT), SyncLoadContext->ContextId);
	AsyncLoadingThreadState->SyncLoadContextStack.Push(SyncLoadContext);

	// We handle the context push/pop manually since we don't interact with the main thread during this time.
	static constexpr bool bAutoHandleSyncLoadContext = false;
	UpdateSyncLoadContext(ThreadState, bAutoHandleSyncLoadContext);

	int64 DidNothingCount = 0;
	while (ContainsAnyRequestID(SyncLoadContext->RequestIDs))
	{
		const bool bDidSomething = EventQueue.ExecuteSyncLoadEvents(ThreadState);
		if (bDidSomething)
		{
			DidNothingCount = 0;
		}
		else if (DidNothingCount++ == 100)
		{
			WarnAboutPotentialSyncLoadStall(SyncLoadContext);
		}
	}

	check(AsyncLoadingThreadState->SyncLoadContextStack.Top() == SyncLoadContext);
	UE_LOG(LogStreaming, VeryVerbose, TEXT("Popping ALT SyncLoadContext %" UINT64_FMT), AsyncLoadingThreadState->SyncLoadContextStack.Top()->ContextId);
	AsyncLoadingThreadState->SyncLoadContextStack.Pop();
	delete SyncLoadContext;
}

void FAsyncLoadingThread2::FlushLoading(TConstArrayView<int32> RequestIDs)
{
	if (IsAsyncLoadingPackages())
	{
		LLM_SCOPE(ELLMTag::AsyncLoading);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		// Prevent objects still being loaded from being accessible from the game thread.
		TGuardValue GuardVisibilityFilter(FUObjectThreadContext::Get().AsyncVisibilityFilter, EInternalObjectFlags::AsyncLoadingPhase1);

		const bool bIsFlushSupportedOnCurrentThread = IsInGameThread() || IsInAsyncLoadingThread() || IsInParallelLoadingThread();

		ELoaderType LoaderType = GetLoaderType();

		if (!bIsFlushSupportedOnCurrentThread)
		{
			// As there is no ensure for test builds, use custom stacktrace logging so we can see these problems in playtest and fix them.
			// But just return in shipping build to avoid crashing as the side effect of missing flushes are not always fatal.
			// Don't forget to adjust unittest in FLoadingTests_InvalidFlush_FromWorker if changing this.
#if !UE_BUILD_SHIPPING
			const FString& ThreadName = FThreadManager::GetThreadName(FPlatformTLS::GetCurrentThreadId());
			const FString Heading =
				FString::Printf(
					TEXT("The current loader '%s' is unable to FlushAsyncLoading from the current thread '%s'. Flush will be ignored."),
					LexToString(LoaderType),
					*ThreadName
				);
			FDebug::DumpStackTraceToLog(*Heading, ELogVerbosity::Error);
#endif
			return;
		}

		// Flushing async loading while loading is suspend will result in infinite stall
		UE_CLOGF(SuspendRequestedCount.load(std::memory_order_relaxed) > 0, LogStreaming, Fatal, "Cannot Flush Async Loading while async loading is suspended");

		if (RequestIDs.Num() != 0 && !ContainsAnyRequestID(RequestIDs))
		{
			return;
		}

		FAsyncLoadingThreadState2* ThreadState = FAsyncLoadingThreadState2::Get();
		if (ThreadState && ThreadState->bIsAsyncLoadingThread)
		{
			FlushLoadingFromLoadingThread(*ThreadState, RequestIDs);
			return;
		}

		if (ThreadState && IsInParallelLoadingThread())
		{
			FlushLoadingFromParallelLoadingThread(*ThreadState, RequestIDs);
			return;
		}

#if WITH_EDITOR
		// In the editor loading cannot be part of a transaction as it cannot be undone, and may result in recording half-loaded objects. So we suppress any active transaction while in this stack, and set the editor loading flag
		TGuardValue<ITransaction*> SuppressTransaction(GUndo, nullptr);
		TGuardValueAccessors<bool> IsEditorLoadingPackageGuard(UE::GetIsEditorLoadingPackage, UE::SetIsEditorLoadingPackage, GIsEditor || UE::GetIsEditorLoadingPackage());
#endif

		const bool bIsFullFlush = RequestIDs.IsEmpty();
		if (bIsFullFlush)
		{
			FullFlushCount.fetch_add(1, std::memory_order_relaxed);
		}

		ON_SCOPE_EXIT
		{
			if (bIsFullFlush)
			{
				FullFlushCount.fetch_sub(1, std::memory_order_relaxed);
			}
		};

		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_FAsyncPackage_FlushAsyncLoadingGameThread, !bIsFullFlush);
		CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_FAsyncPackage_FlushAllAsyncLoadingGameThread, bIsFullFlush);

		// if the sync count is 0, then this flush is not triggered from a sync load, broadcast the delegate in that case
		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		if (ThreadContext.SyncLoadUsingAsyncLoaderCount == 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FCoreDelegates::OnAsyncLoadingFlush);
			FCoreDelegates::OnAsyncLoadingFlush.Broadcast();
		}

		double StartTime = FPlatformTime::Seconds();
		double LogFlushTime = StartTime;

		FAsyncPackage2* CurrentlyExecutingPackage = nullptr;
		if (GameThreadState->CurrentlyExecutingEventNodeStack.Num() > 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(HandleCurrentlyExecutingEventNode);

			UE_CLOGF(RequestIDs.Num() == 0, LogStreaming, Fatal, "Flushing async loading while creating, serializing or postloading an object is not permitted");
			CurrentlyExecutingPackage = GameThreadState->CurrentlyExecutingEventNodeStack.Top()->GetPackage();
			GameThreadState->PackagesOnStack.Push(CurrentlyExecutingPackage);
			// Update the state of any package that is waiting for the currently executing one
			while (FAsyncPackage2* WaitingPackage = CurrentlyExecutingPackage->AllDependenciesFullyLoadedState.PackagesWaitingForThisHead)
			{
				FAsyncPackage2::FAllDependenciesState::RemoveFromWaitList(&FAsyncPackage2::AllDependenciesFullyLoadedState, CurrentlyExecutingPackage, WaitingPackage);
				WaitingPackage->ConditionalFinishLoading(*GameThreadState);
			}
		}

		FAsyncLoadingSyncLoadContext* SyncLoadContext = nullptr;
		if (RequestIDs.Num() != 0 && GOnlyProcessRequiredPackagesWhenSyncLoading)
		{
			SyncLoadContext = new FAsyncLoadingSyncLoadContext(RequestIDs, CurrentlyExecutingPackage);

			UE_LOG(LogStreaming, VeryVerbose, TEXT("Pushing GT SyncLoadContext %" UINT64_FMT), SyncLoadContext->ContextId);

			GameThreadState->SyncLoadContextStack.Push(SyncLoadContext);
			if (AsyncLoadingThreadState)
			{
				SyncLoadContext->AddRef();
				AsyncLoadingThreadState->SyncLoadContextsCreatedOnOtherThreads.Enqueue(SyncLoadContext);
				AltZenaphore.NotifyOne();
			}
		}

		// Flush async loaders by not using a time limit. Needed for e.g. garbage collection.
		{
			double LastActivity = 0.0;
			int32  IdleLoopCount = 0;
			while (IsAsyncLoadingPackages())
			{
				bool bDidSomething = false;
				EAsyncPackageState::Type Result = TickAsyncLoadingFromGameThread(*GameThreadState, false, false, 0.0, RequestIDs, bDidSomething);
				if (RequestIDs.Num() != 0 && !ContainsAnyRequestID(RequestIDs))
				{
					break;
				}

				if (!bDidSomething)
				{
					// Early out in case we end up with a full flush with no more request to process
					// (i.e. Could happen if a full flush is called from inside the last package completion callback).
					if (RequestIDs.Num() == 0 && !ContainsAnyPendingRequests())
					{
						break;
					}

					if (LastActivity == 0.0)
					{
						LastActivity = FPlatformTime::Seconds();

						// The loop count offers additional protection against timeout that could arise during debugging.
						IdleLoopCount = 0;

						// Mark the ALT as inactive, we'll then monitor IsActive() to see if the thread has done anything since the reset.
						if (AsyncLoadingThreadState)
						{
							AsyncLoadingThreadState->ResetActivity();
						}
					}
					else if (GStallDetectorTimeout != 0.0f && PendingIoRequestsCounter.load() == 0 && FPlatformTime::Seconds() - LastActivity > GStallDetectorTimeout && ++IdleLoopCount > GStallDetectorIdleLoops)
					{
#if ALT2_DUMP_STATE_ON_HANG
						{
							if (GameThreadState->CurrentlyExecutingEventNodeStack.Num() > 0)
							{
								UE_LOGF(LogStreaming, Warning, "============ Currently executing nodes on stack =============");

								int32 Index = 0;
								for (FEventLoadNode2* Node : GameThreadState->CurrentlyExecutingEventNodeStack)
								{
									UE_LOGF(LogStreaming, Warning, "#%d: Package %ls executing node %ls", Index++, *Node->GetPackage()->Desc.UPackageName.ToString(), Node->GetSpec()->Name);
								}

								UE_LOGF(LogStreaming, Warning, "============");
							}

							if (SyncLoadContext)
							{
								for (FAsyncPackage2* Package : SyncLoadContext->GetRequestedPackages())
								{
									if (Package)
									{
										Package->DumpState();
									}
								}
							}
							else
							{
								// as_const to allow us to take the a shared lock when calling ForEach as DumpState will also take a shared lock
								std::as_const(AsyncPackageLookup).ForEach(
									[](auto& Pair)
									{
										if (FAsyncPackage2* Package = Pair.Value)
										{
											Package->DumpState();
										}
									}
								);
							}
						}
#endif // ALT2_DUMP_STATE_ON_HANG
						UE_LOGF(LogStreaming, Fatal, "Loading is stuck, flush will never finish");
					}
				}
				else
				{
					LastActivity = 0.0;
				}

				if (IsMultithreaded())
				{
					// Update the heartbeat and sleep a little if we had nothing to do unless ALT has made progress.
					// If we're not multithreading, the heartbeat is updated after each package has been processed.
					FThreadHeartBeat::Get().HeartBeat();

					// Only going idle if nothing has been done
					if (!bDidSomething)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(MainThreadWaitingOnAsyncLoadingThread);
						// Still let the main thread tick at 60fps for processing message loop/etc.
						MainThreadWakeEvent.WaitFor(UE::FMonotonicTimeSpan::FromMilliseconds(16));
						// Reset the manual event right after we wake up so we don't miss any trigger.
						// Worst case, we'll do an empty spin before going back to sleep.
						MainThreadWakeEvent.Reset();

						// Reset the stall detector if there was any activity on the ALT.
						if (AsyncLoadingThreadState && AsyncLoadingThreadState->IsActive())
						{
							LastActivity = 0.0;
						}
					}

					// Flush logging when running cook-on-the-fly and waiting for packages
					if (IsRunningCookOnTheFly() && FPlatformTime::Seconds() - LogFlushTime > 1.0)
					{
						GLog->FlushThreadedLogs(EOutputDeviceRedirectorFlushOptions::Async);
						LogFlushTime = FPlatformTime::Seconds();
					}
				}
				else if (!bDidSomething)
				{
					// When running on a single core (e.g., dedicated server), yield to give other
					// threads a chance to make progress on IO and serialization work.
					FPlatformProcess::YieldThread();
				}

				// push stats so that we don't overflow number of tags per thread during blocking loading
				LLM_PUSH_STATS_FOR_ASSET_TAGS();
			}
		}

		if (SyncLoadContext)
		{
			check(GameThreadState->SyncLoadContextStack.Top() == SyncLoadContext);

			UE_LOG(LogStreaming, VeryVerbose, TEXT("Popping GT SyncLoadContext %" UINT64_FMT), SyncLoadContext->ContextId);

			SyncLoadContext->ReleaseRef();
			GameThreadState->SyncLoadContextStack.Pop();
			AltZenaphore.NotifyOne();
		}

		if (CurrentlyExecutingPackage)
		{
			check(GameThreadState->PackagesOnStack.Top() == CurrentlyExecutingPackage);
			GameThreadState->PackagesOnStack.Pop();
		}

		ConditionalProcessCallbacks();

		// If we asked to flush everything, we should no longer have anything in the pipeline
		check(RequestIDs.Num() != 0 || !IsAsyncLoadingPackages() || !ContainsAnyPendingRequests());
	}
}

EAsyncPackageState::Type FAsyncLoadingThread2::ProcessLoadingUntilCompleteFromGameThread(FAsyncLoadingThreadState2& ThreadState, TFunctionRef<bool()> CompletionPredicate, double TimeLimit)
{
	if (!IsAsyncLoadingPackages())
	{
		return EAsyncPackageState::Complete;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessLoadingUntilComplete);
	SCOPE_CYCLE_COUNTER(STAT_FAsyncPackage_FlushAsyncLoadingGameThread);

	// Flushing async loading while loading is suspend will result in infinite stall
	UE_CLOGF(SuspendRequestedCount.load(std::memory_order_relaxed) > 0, LogStreaming, Fatal, "Cannot Flush Async Loading while async loading is suspended");

	bool bUseTimeLimit = TimeLimit > 0.0f;
	double TimeLoadingPackage = 0.0f;

	// If there's an active sync load context we need to suppress is for the duration of this call since we've no idea what the CompletionPredicate is waiting for
	uint64 NoSyncLoadContextId = 0;
	uint64& SyncLoadContextId = ThreadState.SyncLoadContextStack.IsEmpty() ? NoSyncLoadContextId : ThreadState.SyncLoadContextStack.Top()->ContextId;
	TGuardValue<uint64> GuardSyncLoadContextId(SyncLoadContextId, 0);

	bool bLoadingComplete = !IsAsyncLoadingPackages() || CompletionPredicate();
	while (!bLoadingComplete && (!bUseTimeLimit || TimeLimit > 0.0f))
	{
		double TickStartTime = FPlatformTime::Seconds();
		if (ProcessLoadingFromGameThread(ThreadState, bUseTimeLimit, bUseTimeLimit, TimeLimit) == EAsyncPackageState::Complete)
		{
			return EAsyncPackageState::Complete;
		}

		if (IsMultithreaded())
		{
			// Update the heartbeat and sleep. If we're not multithreading, the heartbeat is updated after each package has been processed
			// only update the heartbeat up to the limit of the hang detector to ensure if we get stuck in this loop that the hang detector gets a chance to trigger
			if (TimeLoadingPackage < FThreadHeartBeat::Get().GetHangDuration())
			{
				FThreadHeartBeat::Get().HeartBeat();
			}
			FPlatformProcess::SleepNoStats(0.0001f);
		}

		double TimeDelta = (FPlatformTime::Seconds() - TickStartTime);
		TimeLimit -= TimeDelta;
		TimeLoadingPackage += TimeDelta;

		bLoadingComplete = !IsAsyncLoadingPackages() || CompletionPredicate();
	}

	return bLoadingComplete ? EAsyncPackageState::Complete : EAsyncPackageState::TimeOut;
}

void FGlobalImportStore::FlushDeferredCleanupPackagesQueue()
{
	AsyncLoadingThread.ProcessDeferredCleanupPackagesQueue();
}

void FAsyncLoadingThread2::FLoaderStats::Clear()
{
#if !UE_BUILD_SHIPPING
	PartialFlushes.Empty();
#endif
}

void FAsyncLoadingThread2::FLoaderStats::DumpToLog()
{

#if !UE_BUILD_SHIPPING
	if ((LogMode != ELogMode::Dump) && (LogMode != ELogMode::DumpWithCallstack))
	{
		return;
	}

	UE_LOGF(LogStreaming, Display, "============ Loader Stats ============");

	UE_LOGF(LogStreaming, Display, "Unique Partial Flushes: %d", PartialFlushes.Num());
	TStringBuilder<1024> Builder;
	for (const FPartialFlushInfo& PartialFlush : PartialFlushes)
	{
		Log(Builder, PartialFlush);
	}

	UE_LOGF(LogStreaming, Display, "============ Loader Stats End ============");
#endif

}

void FAsyncLoadingThread2::FLoaderStats::RecordPartialFlush(const FAsyncPackage2* RequestedPackage, const FAsyncPackage2* RequestingPackage)
{
#if !UE_BUILD_SHIPPING
	FPartialFlushInfo PartialFlush
	{
		.RequestedPackage = RequestedPackage->Desc.UPackageName,
		.RequestingPackage = RequestingPackage->Desc.UPackageName,
		.RequestedPackageState = RequestedPackage->AsyncPackageLoadingState,
		.RequestingPackageState = RequestingPackage->AsyncPackageLoadingState,
	};

	if (LogMode == ELogMode::DumpWithCallstack)
	{
		TArray<uint64> Callstack;
		Callstack.SetNumUninitialized(FPartialFlushInfo::MaxCallstackDepth);

		const int32 NumStackFramesToIgnore = 3; // Ignore this callstack and 2 frames from CaptureStackBackTrace
		uint32 NumFrames = FPlatformStackWalk::CaptureStackBackTrace(Callstack.GetData(), Callstack.Num());
		Callstack.SetNum(NumFrames);

		int32& ExistingIndex = UniqueCallstackMap.FindOrAdd(Callstack, -1);
		if (ExistingIndex == -1)
		{
			ExistingIndex = UniqueCallstacks.Num();
			FUniqueCallstack& UniqueCallstack = UniqueCallstacks.Emplace_GetRef();
			UniqueCallstack.Callstack = MoveTemp(Callstack);
		}
		PartialFlush.UniqueCallstackIndex = ExistingIndex;
	}

	if (LogMode == ELogMode::Inline)
	{
		TStringBuilder<1024> Builder;
		Log(Builder, PartialFlush);
	}
	else
	{
		PartialFlushes.Add(MoveTemp(PartialFlush));
	}
#endif
}

void FAsyncLoadingThread2::FLoaderStats::Log(FStringBuilderBase& Builder, const FPartialFlushInfo& PartialFlush)
{
#if !UE_BUILD_SHIPPING
	FUniqueCallstack* Callstack = PartialFlush.UniqueCallstackIndex < 0
		? nullptr : &UniqueCallstacks[PartialFlush.UniqueCallstackIndex];

	//
	// Note: Update the FLoadingTests_RecursiveLoads_FullFlushFrom_Serialize test if you edit this error message
	//
	Builder.Reset();
	Builder.Appendf(
		TEXT("Partially loaded package %s (state: %s) to avoid deadlock when it was recursively flushed by package %s (state: %s). UniqueCallstack(%d)%s"),
		*PartialFlush.RequestedPackage.ToString(),
		LexToString(PartialFlush.RequestedPackageState),
		*PartialFlush.RequestingPackage.ToString(),
		LexToString(PartialFlush.RequestingPackageState),
		PartialFlush.UniqueCallstackIndex,
		(!Callstack || Callstack->bHasLogged) ? TEXT(".") : TEXT(" - first occurrence - Callstack:\n")
	);

	if (Callstack && !Callstack->bHasLogged)
	{
		Callstack->bHasLogged = true;
		FString HumanReadableString;
		for (const uint64 ProgramCounter : Callstack->Callstack)
		{
			FProgramCounterSymbolInfoEx SymbolInfo;
			FPlatformStackWalk::ProgramCounterToSymbolInfoEx(ProgramCounter, SymbolInfo);
			FPlatformStackWalk::SymbolInfoToHumanReadableStringEx(SymbolInfo, HumanReadableString);

			Builder.Append(TEXT("\t"));
			Builder.Append(HumanReadableString);
			Builder.Append(TEXT("\n"));

			HumanReadableString.Reset();
		}
	}

	UE_LOGF(LogStreaming, Display, "%ls", Builder.ToString());
#endif
}

#if ALT2_DUMP_STATE_ON_HANG

void FAsyncPackage2::DumpState()
{
	UE_LOGF(LogStreaming, Warning, "============ Dumping State of Package %ls ============", *Desc.UPackageName.ToString());

	TSet<FAsyncPackage2*> Set;
	DumpStateImpl(Set);

	UE_LOGF(LogStreaming, Warning, "============");

	AsyncLoadingThread.LoaderStats.DumpToLog();
}

void FAsyncPackage2::DumpStateImpl(TSet<FAsyncPackage2*>& Set, int32 Indent, TMultiMap<FEventLoadNode2*, FEventLoadNode2*>* MappedNodes)
{
	Set.Add(this);

	auto GetPackageType = [](FAsyncPackage2* Package)
		{
			return LexToString(Package->Desc.Loader);
		};

	auto FormatPackage = [&GetPackageType](FAsyncPackage2* Package)
		{
			return FString::Printf(TEXT("%s package %s state %s"), GetPackageType(Package), *Package->Desc.UPackageName.ToString(), LexToString(Package->AsyncPackageLoadingState));
		};

	UE_LOGF(LogStreaming, Warning, "%ls%ls:", FCString::Spc(Indent), *FormatPackage(this));

	auto VisitNodes = [](FEventLoadNode2& Node, TFunction<void(FEventLoadNode2*)> Visitor)
		{
			if (Node.DependenciesCount == 1)
			{
				Visitor(Node.SingleDependent);
			}
			else if (Node.DependenciesCount != 0)
			{
				FEventLoadNode2** Current = Node.MultipleDependents;
				FEventLoadNode2** End = Node.MultipleDependents + Node.DependenciesCount;
				for (; Current < End; ++Current)
				{
					Visitor(*Current);
				}
			}
		};

	auto DumpNode = [&FormatPackage, &VisitNodes](TMultiMap<FEventLoadNode2*, FEventLoadNode2*>& MappedNodes, const TCHAR* Header, FEventLoadNode2& Node, int32 Indent)
		{
			bool bIsEmpty = Node.Spec->EventQueue->IsEmptyForDebug();

			if (Node.DependenciesCount == 0 && Node.BarrierCount == 0 && bIsEmpty)
			{
				return false;
			}

			if (Header)
			{
				UE_LOGF(LogStreaming, Warning, "%ls %ls", FCString::Spc(Indent), Header);
			}
			Indent++;

			UE_LOGF(LogStreaming, Warning, "%ls Node %ls (BarrierCount %d, EventQueue: %ls)", FCString::Spc(Indent), Node.GetSpec()->Name, Node.BarrierCount.load(), bIsEmpty ? TEXT("Empty") : TEXT("NonEmpty"));

			TArray<FEventLoadNode2*> WaitingOn;
			MappedNodes.MultiFind(&Node, WaitingOn);

			for (FEventLoadNode2* WaitingNode : WaitingOn)
			{
				UE_LOGF(LogStreaming, Warning, "%ls Waiting on node %ls from %ls", FCString::Spc(Indent + 1), WaitingNode->GetSpec()->Name, *FormatPackage(WaitingNode->GetPackage()));
			}

			VisitNodes(Node,
				[Indent, &FormatPackage](FEventLoadNode2* Node)
				{
					UE_LOGF(LogStreaming, Warning, "%ls Will trigger node %ls for %ls", FCString::Spc(Indent + 1), Node->GetSpec()->Name, *FormatPackage(Node->GetPackage()));
				}
			);

			return true;
		};

	TMultiMap<FEventLoadNode2*, FEventLoadNode2*> LocalMappedNode;
	if (MappedNodes == nullptr)
	{
		MappedNodes = &LocalMappedNode;

		// as_const to allow us to take the a shared lock as we might be already in a shared lock
		std::as_const(AsyncLoadingThread).AsyncPackageLookup.ForEach(
			[&](auto& Pair)
			{
				if (FAsyncPackage2* Package = Pair.Value)
				{
					for (uint8 PhaseIndex = 0; PhaseIndex < Package_NumPhases; ++PhaseIndex)
					{
						FEventLoadNode2& Node = Package->GetPackageNode((EEventLoadNode2)PhaseIndex);
						VisitNodes(Node, [&](FEventLoadNode2* Dependent) { LocalMappedNode.AddUnique(Dependent, &Node); });
					}
				}
			}
		);
	}

	{
		// as_const to allow us to take the a shared lock as we might be already in a shared lock
		std::as_const(AsyncLoadingThread).AsyncPackageLookup.ForEach(
			[&](auto& Pair)
			{
				if (FAsyncPackage2* Package = Pair.Value)
				{
					if (Package->HeaderData.ImportedPackageIds.Contains(this->Desc.UPackageId))
					{
						UE_LOGF(LogStreaming, Warning, " %ls referenced by package %ls", FCString::Spc(Indent), *FormatPackage(Package));
					}
				}
			}
		);
	}

	for (uint8 PhaseIndex = 0; PhaseIndex < Package_NumPhases; ++PhaseIndex)
	{
		EEventLoadNode2 EventNode = (EEventLoadNode2)PhaseIndex;
		DumpNode(*MappedNodes, nullptr, GetPackageNode(EventNode), Indent + 1);
	}

	for (FAsyncPackage2* Import : Data.ImportedAsyncPackages)
	{
		if (Import && Import->AsyncPackageLoadingState < EAsyncPackageLoadingState2::DeferredDelete)
		{
			UE_LOGF(LogStreaming, Warning, " %ls imports %ls package %ls state %ls", FCString::Spc(Indent), GetPackageType(Import), *Import->Desc.UPackageName.ToString(), LexToString(Import->AsyncPackageLoadingState));
		}
	}

	for (FAsyncPackage2* Import : AdditionalImportedAsyncPackages)
	{
		if (Import->AsyncPackageLoadingState < EAsyncPackageLoadingState2::DeferredDelete)
		{
			UE_LOGF(LogStreaming, Warning, " %ls dynamically imports %ls package %ls state %ls", FCString::Spc(Indent), GetPackageType(Import), *Import->Desc.UPackageName.ToString(), LexToString(Import->AsyncPackageLoadingState));
		}
	}

	if (AllDependenciesSetupState.WaitingForPackage)
	{
		UE_LOGF(LogStreaming, Warning, " %ls AllDependenciesSetupState is waiting on %ls", FCString::Spc(Indent), *FormatPackage(AllDependenciesSetupState.WaitingForPackage));
	}
	if (AllDependenciesImportState.WaitingForPackage)
	{
		UE_LOGF(LogStreaming, Warning, " %ls AllDependenciesImportState is waiting on %ls", FCString::Spc(Indent), *FormatPackage(AllDependenciesImportState.WaitingForPackage));
	}
	if (AllDependenciesPostLoadReadyState.WaitingForPackage)
	{
		UE_LOGF(LogStreaming, Warning, " %ls AllDependenciesPostLoadReadyState is waiting on %ls", FCString::Spc(Indent), *FormatPackage(AllDependenciesPostLoadReadyState.WaitingForPackage));
	}

	if (AllDependenciesFullyLoadedState.WaitingForPackage)
	{
		UE_LOGF(LogStreaming, Warning, " %ls AllDependenciesFullyLoadedState is waiting on %ls", FCString::Spc(Indent), *FormatPackage(AllDependenciesFullyLoadedState.WaitingForPackage));
	}

	for (FAsyncPackage2* Import : Data.ImportedAsyncPackages)
	{
		if (Import && Import->AsyncPackageLoadingState < EAsyncPackageLoadingState2::DeferredDelete)
		{
			if (!Set.Contains(Import))
			{
				UE_LOGF(LogStreaming, Warning, "");
				Import->DumpStateImpl(Set, 0, MappedNodes);
			}
		}
	}

	for (FAsyncPackage2* Import : AdditionalImportedAsyncPackages)
	{
		if (Import && Import->AsyncPackageLoadingState < EAsyncPackageLoadingState2::DeferredDelete)
		{
			if (!Set.Contains(Import))
			{
				UE_LOGF(LogStreaming, Warning, "");
				Import->DumpStateImpl(Set, 0, MappedNodes);
			}
		}
	}
}

#endif // ALT2_DUMP_STATE_ON_HANG

IAsyncPackageLoader* MakeAsyncPackageLoader2(FIoDispatcher& InIoDispatcher)
{
	return new FAsyncLoadingThread2(InIoDispatcher);
}

#if WITH_DEV_AUTOMATION_TESTS 
#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPublicExportMapBasicTest, "System.Core.Loading.FPublicExportMap.Basic", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter);
bool FPublicExportMapBasicTest::RunTest(const FString& Parameters)
{
	typedef FLoadedPackageRef::FPublicExportMap FPublicExportMap;

	FPublicExportMap Map;
	int32 HashIndex = INDEX_NONE;
	TestEqual(TEXT("Empty map has no count"), Map.Num(), 0);
	TestEqual(TEXT("Empty map has no keys"), Map.GetKeys().Num(), 0);
	TestEqual(TEXT("Empty map has no values"), Map.GetValues().Num(), 0);

	Map.Store(1, 2);
	TestEqual(TEXT("Find stored key"), Map.Find(1, &HashIndex), 2);
	TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
	TestEqual(TEXT("Store increased size"), Map.Num(), 1);

	// May resize
	Map.Store(2, 3);
	HashIndex = INDEX_NONE;
	TestEqual(TEXT("Find stored key"), Map.Find(2, &HashIndex), 3);
	TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
	TestEqual(TEXT("Store increased size"), Map.Num(), 2);

	HashIndex = INDEX_NONE;
	TestEqual(TEXT("Find removed key fails"), Map.Find(3, &HashIndex), FPublicExportMap::InvalidValue);
	Map.Store(3,4,HashIndex);
	int32 NewHashIndex = INDEX_NONE;
	TestEqual(TEXT("Find stored key"), Map.Find(3, &NewHashIndex), 4);
	TestEqual(TEXT("Same hash index"), NewHashIndex, HashIndex);
	TestBetweenInclusive(TEXT("Find stored key hash index"), NewHashIndex, 0, Map.GetValues().Num());
	TestEqual(TEXT("Store increased size"), Map.Num(), 3);
	TestEqual(TEXT("Remove existing key"), Map.Remove(3), true);
	TestEqual(TEXT("Removed decreased count"), Map.Num(), 2);

	TestEqual(TEXT("Remove existing key"), Map.Remove(1), true);
	TestEqual(TEXT("Removed decreased count"), Map.Num(), 1);
	TestEqual(TEXT("Remove removed key fails"), Map.Remove(1), false);
	TestEqual(TEXT("Failed remove does not change count"), Map.Num(), 1);
	TestEqual(TEXT("Find removed key fails"), Map.Find(1), FPublicExportMap::InvalidValue);

	TestEqual(TEXT("Remove existing key"), Map.Remove(2), true);
	TestEqual(TEXT("Removed decreased count"), Map.Num(), 0);
	TestEqual(TEXT("Remove removed key fails"), Map.Remove(2), false);
	TestEqual(TEXT("Failed remove does not change count"), Map.Num(), 0);
	TestEqual(TEXT("Find removed key fails"), Map.Find(2), FPublicExportMap::InvalidValue);
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPublicExportMapIterationTest, "System.Core.Loading.FPublicExportMap.Iteration", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter);
bool FPublicExportMapIterationTest::RunTest(const FString& Parameters)
{
	typedef FLoadedPackageRef::FPublicExportMap FPublicExportMap;

	FPublicExportMap Map;
	int32 HashIndex = INDEX_NONE;
	TArray<uint64> Keys = { 1, 2, 3, 4, 5, 6, 7, 8 };
	TArray<int32> Values = { 1, 2, 3, 4, 5, 6, 7, 8 };
	check(Keys.Num() == Values.Num());

	TestEqual(TEXT("Empty map has no elements"), Map.Num(), 0);
	for (int32 i = 0; i < Keys.Num(); ++i)
	{
		Map.Store(Keys[i], Values[i]);
	}

	TSet<uint64> FoundKeys;
	for (uint64 Key : Map.GetKeys())
	{
		FoundKeys.Add(Key);
	}
	TSet<int32> FoundValues;
	for (int32 Value : Map.GetValues())
	{
		FoundValues.Add(Value);
	}

	TestEqual(TEXT("Stored keys matches expected count"), Map.GetKeys().Num(), FoundKeys.Num());
	for (uint64 Key : Keys)
	{
		TestTrue(TEXT("All stored keys can be found"), FoundKeys.Contains(Key));
	}

	TestEqual(TEXT("Stored values matches expected count"), Map.GetValues().Num(), FoundValues.Num());
	for (uint64 Key : Keys)
	{
		TestTrue(TEXT("All stored values can be found"), FoundKeys.Contains(Key));
	}

	for (uint64 Key : Keys)
	{
		TestEqual(TEXT("Lookup works"), Map.Find(Key, &HashIndex), Values[(int32)(Key - 1)]);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
	}
	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FPublicExportMapPreInsertPublicExportsTest, "System.Core.Loading.FPublicExportMap.PreInsertPublicExports", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter);
bool FPublicExportMapPreInsertPublicExportsTest::RunTest(const FString& Parameters)
{
	typedef FLoadedPackageRef::FPublicExportMap FPublicExportMap;

	// Empty map
	{
		FPublicExportMap Map;
		int32 HashIndex = INDEX_NONE;
		TArray<FExportMapEntry> Exports = { {.PublicExportHash = 1 }, {.PublicExportHash = 2 }, {.PublicExportHash = 3 }, {.PublicExportHash = 4 } };
		int32 OriginaMapCount = Map.Num();

		TestEqual(TEXT("Empty map has no count"), OriginaMapCount, 0);
		TestEqual(TEXT("Empty map has no keys"), Map.GetKeys().Num(), OriginaMapCount);
		TestEqual(TEXT("Empty map has no values"), Map.GetValues().Num(), OriginaMapCount);
		Map.PreInsertPublicExports({ Exports });
		TestEqual(TEXT("Empty map has no count"), Map.Num(), OriginaMapCount);
		TestEqual(TEXT("PreInsert has populated our keys"), Map.GetKeys().Num(), Exports.Num());
		TestEqual(TEXT("PreInsert has populated our values (should mostly be InvalidValue)"), Map.GetValues().Num(), Exports.Num());

		for (uint64 Key : Map.GetKeys())
		{
			TestEqual(TEXT("All preinserted keys are initialized to InvalidValue"), Map.Find(Key), FPublicExportMap::InvalidValue);
		}
		TestEqual(TEXT("PreInsertPublicExports does not store new values"), Map.Num(), 0);

		// Overwrite preinserted value
		TestEqual(TEXT("Find stored key with no value set"), Map.Find(Exports[0].PublicExportHash), FPublicExportMap::InvalidValue);
		Map.Store(Exports[0].PublicExportHash, (int32)Exports[0].PublicExportHash);
		TestEqual(TEXT("Find stored key"), Map.Find(Exports[0].PublicExportHash, &HashIndex), (int32)Exports[0].PublicExportHash);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 1);

		// Insert new value
		TestEqual(TEXT("Find missing key"), Map.Find(111), FPublicExportMap::InvalidValue);
		Map.Store(111, 111);
		TestEqual(TEXT("Find stored key"), Map.Find(111, &HashIndex), 111);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 2);

		TestEqual(TEXT("Remove existing key"), Map.Remove(111), true);
		TestEqual(TEXT("Remove decreased count"), Map.Num(), OriginaMapCount + 1);
		TestEqual(TEXT("Remove removed key fails"), Map.Remove(111), false);
		TestEqual(TEXT("Failed remove did not increase count"), Map.Num(), OriginaMapCount + 1);
		TestEqual(TEXT("Find removed key returns no value set"), Map.Find(111), FPublicExportMap::InvalidValue);
	}

	// Empty map, populate a single key
	{
		FPublicExportMap Map;
		int32 HashIndex = INDEX_NONE;
		TArray<FExportMapEntry> Exports = { {.PublicExportHash = 1 } };
		int32 OriginaMapCount = Map.Num();

		TestEqual(TEXT("Empty map has no count"), OriginaMapCount, 0);
		TestEqual(TEXT("Empty map has no keys"), Map.GetKeys().Num(), OriginaMapCount);
		TestEqual(TEXT("Empty map has no values"), Map.GetValues().Num(), OriginaMapCount);
		Map.PreInsertPublicExports({ Exports });
		TestEqual(TEXT("Empty map has no count"), Map.Num(), OriginaMapCount);
		TestEqual(TEXT("PreInsert has populated our keys"), Map.GetKeys().Num(), Exports.Num());
		TestEqual(TEXT("PreInsert has populated our values (should mostly be InvalidValue)"), Map.GetValues().Num(), Exports.Num());

		for (uint64 Key : Map.GetKeys())
		{
			TestEqual(TEXT("All preinserted keys are initialized to InvalidValue"), Map.Find(Key), FPublicExportMap::InvalidValue);
		}
		TestEqual(TEXT("PreInsertPublicExports does not store new values"), Map.Num(), 0);

		// Overwrite preinserted value
		TestEqual(TEXT("Find stored key with no value set"), Map.Find(Exports[0].PublicExportHash), FPublicExportMap::InvalidValue);
		Map.Store(Exports[0].PublicExportHash, (int32)Exports[0].PublicExportHash);
		TestEqual(TEXT("Find stored key"), Map.Find(Exports[0].PublicExportHash, &HashIndex), (int32)Exports[0].PublicExportHash);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 1);

		// Insert new value
		TestEqual(TEXT("Find missing key"), Map.Find(111), FPublicExportMap::InvalidValue);
		Map.Store(111, 111);
		TestEqual(TEXT("Find stored key"), Map.Find(111, &HashIndex), 111);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 2);

		TestEqual(TEXT("Remove existing key"), Map.Remove(111), true);
		TestEqual(TEXT("Remove decreased count"), Map.Num(), OriginaMapCount + 1);
		TestEqual(TEXT("Remove removed key fails"), Map.Remove(111), false);
		TestEqual(TEXT("Failed remove did not increase count"), Map.Num(), OriginaMapCount + 1);
		TestEqual(TEXT("Find removed key returns no value set"), Map.Find(111), FPublicExportMap::InvalidValue);
	}

	// One existing entry (preinsert multiple keys, some overlapping)
	{
		FPublicExportMap Map;
		int32 HashIndex = INDEX_NONE;
		TArray<FExportMapEntry> Exports = { {.PublicExportHash = 1 }, {.PublicExportHash = 2 }, {.PublicExportHash = 3 }, {.PublicExportHash = 4 } };

		uint64 PreExistingKey = 2;
		int32 PreExistingValue = 2;
		Map.Store(PreExistingKey, PreExistingValue);
		int32 OriginaMapCount = Map.Num();

		TestEqual(TEXT("Find stored key"), Map.Find(PreExistingKey, &HashIndex), PreExistingValue);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), OriginaMapCount, 1);
		TestEqual(TEXT("Store increased keys"), Map.GetKeys().Num(), OriginaMapCount);
		TestEqual(TEXT("Store increased values"), Map.GetValues().Num(), OriginaMapCount);

		Map.PreInsertPublicExports({ Exports });
		TestEqual(TEXT("PreInsert doesn't add new values"), Map.Num(), OriginaMapCount);
		TestEqual(TEXT("PreInsert has populated our keys"), Map.GetKeys().Num(), Exports.Num());
		TestEqual(TEXT("PreInsert has populated our values (should mostly be InvalidValue)"), Map.GetValues().Num(), Exports.Num());

		bool bFoundPreExistingKey = false;
		for (uint64 Key : Map.GetKeys())
		{
			if (Key == PreExistingKey)
			{
				bFoundPreExistingKey = true;
				TestEqual(TEXT("Pre-existing keys remain intact after PreInsertPublicExports"), Map.Find(Key, &HashIndex), PreExistingValue);
			}
			else
			{
				TestEqual(TEXT("All preinserted keys are initialized to InvalidValue"), Map.Find(Key), FPublicExportMap::InvalidValue);
			}
		}
		TestEqual(TEXT("GetKeys has preexisting key"), bFoundPreExistingKey, true);
		TestEqual(TEXT("PreInsertPublicExports does not add new values"), Map.Num(), OriginaMapCount);

		// Overwrite preinserted value
		TestEqual(TEXT("Find stored key with no value set"), Map.Find(Exports[0].PublicExportHash), FPublicExportMap::InvalidValue);
		Map.Store(Exports[0].PublicExportHash, (int32)Exports[0].PublicExportHash);
		TestEqual(TEXT("Find stored key"), Map.Find(Exports[0].PublicExportHash, &HashIndex), (int32)Exports[0].PublicExportHash);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 1);

		// Insert new value
		TestEqual(TEXT("Find missing key"), Map.Find(111), FPublicExportMap::InvalidValue);
		Map.Store(111, 111);
		TestEqual(TEXT("Find stored key"), Map.Find(111, &HashIndex), 111);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 2);

		TestEqual(TEXT("Remove existing key"), Map.Remove(111), true);
		TestEqual(TEXT("Remove decreased count"), Map.Num(), OriginaMapCount + 1);
		TestEqual(TEXT("Remove removed key fails"), Map.Remove(111), false);
		TestEqual(TEXT("Failed remove did not increase count"), Map.Num(), OriginaMapCount + 1);
		TestEqual(TEXT("Find removed key returns no value set"), Map.Find(111), FPublicExportMap::InvalidValue);
	}

	// One existing entry (preinsert a single entry overlapping)
	{
		FPublicExportMap Map;
		int32 HashIndex = INDEX_NONE;
		TArray<FExportMapEntry> Exports = { {.PublicExportHash = 1 } };

		uint64 PreExistingKey = 1;
		int32 PreExistingValue = 1;
		Map.Store(PreExistingKey, PreExistingValue);
		int32 OriginaMapCount = Map.Num();

		TestEqual(TEXT("Find stored key"), Map.Find(PreExistingKey, &HashIndex), PreExistingValue);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), OriginaMapCount, 1);
		TestEqual(TEXT("Store increased keys"), Map.GetKeys().Num(), OriginaMapCount);
		TestEqual(TEXT("Store increased values"), Map.GetValues().Num(), OriginaMapCount);

		Map.PreInsertPublicExports({ Exports });
		TestEqual(TEXT("PreInsert doesn't add new values"), Map.Num(), OriginaMapCount);
		TestEqual(TEXT("PreInsert has populated our keys"), Map.GetKeys().Num(), Exports.Num());
		TestEqual(TEXT("PreInsert has populated our values (should mostly be InvalidValue)"), Map.GetValues().Num(), Exports.Num());

		bool bFoundPreExistingKey = false;
		for (uint64 Key : Map.GetKeys())
		{
			if (Key == PreExistingKey)
			{
				bFoundPreExistingKey = true;
				TestEqual(TEXT("Pre-existing keys remain intact after PreInsertPublicExports"), Map.Find(Key, &HashIndex), PreExistingValue);
				TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
			}
			else
			{
				TestEqual(TEXT("All preinserted keys are initialized to InvalidValue"), Map.Find(Key), FPublicExportMap::InvalidValue);
			}
		}
		TestEqual(TEXT("GetKeys has preexisting key"), bFoundPreExistingKey, true);
		TestEqual(TEXT("PreInsertPublicExports does not add new values"), Map.Num(), OriginaMapCount);
	}

	// One existing entry (preinsert a single entry non-overlapping)
	{
		FPublicExportMap Map;
		int32 HashIndex = INDEX_NONE;
		TArray<FExportMapEntry> Exports = { {.PublicExportHash = 1 } };

		uint64 PreExistingKey = 2;
		int32 PreExistingValue = 2;
		Map.Store(PreExistingKey, PreExistingValue);
		int32 OriginaMapCount = Map.Num();

		TestEqual(TEXT("Find stored key"), Map.Find(PreExistingKey, &HashIndex), PreExistingValue);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), OriginaMapCount, 1);
		TestEqual(TEXT("Store increased keys"), Map.GetKeys().Num(), OriginaMapCount);
		TestEqual(TEXT("Store increased values"), Map.GetValues().Num(), OriginaMapCount);

		Map.PreInsertPublicExports({ Exports });
		TestEqual(TEXT("PreInsert doesn't add new values"), Map.Num(), OriginaMapCount);
		TestEqual(TEXT("PreInsert has populated our keys"), Map.GetKeys().Num(), Exports.Num() + OriginaMapCount);
		TestEqual(TEXT("PreInsert has populated our values (should mostly be InvalidValue)"), Map.GetValues().Num(), Exports.Num() + OriginaMapCount);

		bool bFoundPreExistingKey = false;
		for (uint64 Key : Map.GetKeys())
		{
			if (Key == PreExistingKey)
			{
				bFoundPreExistingKey = true;
				TestEqual(TEXT("Pre-existing keys remain intact after PreInsertPublicExports"), Map.Find(Key, &HashIndex), PreExistingValue);
				TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
			}
			else
			{
				TestEqual(TEXT("All preinserted keys are initialized to InvalidValue"), Map.Find(Key), FPublicExportMap::InvalidValue);
			}
		}
		TestEqual(TEXT("GetKeys has preexisting key"), bFoundPreExistingKey, true);
		TestEqual(TEXT("PreInsertPublicExports does not add new values"), Map.Num(), OriginaMapCount);

		// Overwrite preinserted value
		TestEqual(TEXT("Find stored key with no value set"), Map.Find(Exports[0].PublicExportHash), FPublicExportMap::InvalidValue);
		Map.Store(Exports[0].PublicExportHash, (int32)Exports[0].PublicExportHash);
		TestEqual(TEXT("Find stored key"), Map.Find(Exports[0].PublicExportHash, &HashIndex), (int32)Exports[0].PublicExportHash);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 1);
	}

	// One existing entry (preinsert multiple keys, non-overlapping)
	{
		FPublicExportMap Map;
		int32 HashIndex = INDEX_NONE;
		TArray<FExportMapEntry> Exports = { {.PublicExportHash = 1 }, {.PublicExportHash = 2 }, {.PublicExportHash = 3 }, {.PublicExportHash = 4 } };

		uint64 PreExistingKey = 7;
		int32 PreExistingValue = 8;
		Map.Store(PreExistingKey, PreExistingValue);
		int32 OriginaMapCount = Map.Num();

		TestEqual(TEXT("Find stored key"), Map.Find(PreExistingKey, &HashIndex), PreExistingValue);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), OriginaMapCount, 1);
		TestEqual(TEXT("Store increased keys"), Map.GetKeys().Num(), OriginaMapCount);
		TestEqual(TEXT("Store increased values"), Map.GetValues().Num(), OriginaMapCount);

		Map.PreInsertPublicExports({ Exports });
		TestEqual(TEXT("PreInsertPublicExports has populated our keys"), Map.GetKeys().Num(), Exports.Num() + OriginaMapCount);
		TestEqual(TEXT("PreInsert has populated our values (should mostly be InvalidValue)"), Map.GetValues().Num(), Exports.Num() + OriginaMapCount);

		bool bFoundPreExistingKey = false;
		for (uint64 Key : Map.GetKeys())
		{
			if (Key == PreExistingKey)
			{
				bFoundPreExistingKey = true;
				TestEqual(TEXT("Pre-existing keys remain intact after PreInsertPublicExports"), Map.Find(Key, &HashIndex), PreExistingValue);
				TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
			}
			else
			{
				TestEqual(TEXT("All preinserted keys are initialized to InvalidValue"), Map.Find(Key), FPublicExportMap::InvalidValue);
			}
		}
		TestEqual(TEXT("GetKeys has preexisting key"), bFoundPreExistingKey, true);
		TestEqual(TEXT("PreInsertPublicExports does not add new values"), Map.Num(), OriginaMapCount);

		// Overwrite preinserted value
		TestEqual(TEXT("Find stored key with no value set"), Map.Find(Exports[0].PublicExportHash), FPublicExportMap::InvalidValue);
		Map.Store(Exports[0].PublicExportHash, (int32)Exports[0].PublicExportHash);
		TestEqual(TEXT("Find stored key"), Map.Find(Exports[0].PublicExportHash, &HashIndex), (int32)Exports[0].PublicExportHash);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 1);

		// Insert new value
		TestEqual(TEXT("Find missing key"), Map.Find(111), FPublicExportMap::InvalidValue);
		Map.Store(111, 111);
		TestEqual(TEXT("Find stored key"), Map.Find(111, &HashIndex), 111);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), OriginaMapCount + 2);

		TestEqual(TEXT("Remove existing key"), Map.Remove(111), true);
		TestEqual(TEXT("Remove decreased count"), Map.Num(), OriginaMapCount + 1);
		TestEqual(TEXT("Remove removed key fails"), Map.Remove(111), false);
		TestEqual(TEXT("Failed remove did not increase count"), Map.Num(), OriginaMapCount + 1);
		TestEqual(TEXT("Find removed key returns no value set"), Map.Find(111), FPublicExportMap::InvalidValue);
	}

	// Many pre-existing keys
	{
		FPublicExportMap Map;
		int32 HashIndex = INDEX_NONE;
		// Note we are explicitly overlapping key 1 with the preinsert key 1
		// Note we are mixing ranges of preinserted and preexisting keys so we are forced to sort the keys together
		uint64 OverlappingKey = 1;
		TArray<uint64> PreExistingKeys = { 11, 2, 13, 4, 15, 6, 17, 8, OverlappingKey };
		TArray<int32> PreExistingValues = { 11, 2, 13, 4, 15, 6, 17, 8, (int32)OverlappingKey };
		check(PreExistingKeys.Num() == PreExistingValues.Num());

		TArray<FExportMapEntry> Exports = { {.PublicExportHash = 12}, {.PublicExportHash = OverlappingKey  }, {.PublicExportHash = 3 }, {.PublicExportHash = 14 } };
		for (int32 i = 0; i < PreExistingKeys.Num(); ++i)
		{
			Map.Store(PreExistingKeys[i], PreExistingValues[i]);
		}
		int32 OriginaMapCount = Map.Num();

		TSet<uint64> AllKeys;
		AllKeys.Append(PreExistingKeys);
		for (FExportMapEntry& Export : Exports)
		{
			AllKeys.Add(Export.PublicExportHash);
		}

		TestEqual(TEXT("Store increased count"), OriginaMapCount, PreExistingValues.Num());
		TestEqual(TEXT("Store increased keys"), Map.GetKeys().Num(), OriginaMapCount);
		TestEqual(TEXT("Store increased values"), Map.GetValues().Num(), OriginaMapCount);

		Map.PreInsertPublicExports({ Exports });
		TestEqual(TEXT("PreInsertPublicExports has populated our keys"), Map.GetKeys().Num(), AllKeys.Num());
		TestEqual(TEXT("PreInsert has populated our values (should mostly be InvalidValue)"), Map.GetValues().Num(), AllKeys.Num());

		TSet<uint64> FoundPreExistingKeys;
		FoundPreExistingKeys.Reserve(PreExistingKeys.Num());
		for (uint64 Key : Map.GetKeys())
		{
			if (PreExistingKeys.Contains(Key))
			{
				FoundPreExistingKeys.Add(Key);
				TestEqual(TEXT("Pre-existing keys remain intact after PreInsertPublicExports"), Map.Find(Key, &HashIndex), PreExistingValues[PreExistingKeys.IndexOfByKey(Key)]);
				TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
			}
			else if (Key == OverlappingKey)
			{
				TestEqual(TEXT("Ensure preinsert didn't overwrite the existing value"), Map.Find(Key, &HashIndex), (int32)OverlappingKey);
				TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
			}
			else
			{
				TestEqual(TEXT("All preinserted keys are initialized to InvalidValue"), Map.Find(Key), FPublicExportMap::InvalidValue);
			}
		}
		TestEqual(TEXT("All preexisting keys are still in the map"), FoundPreExistingKeys.Difference(TSet<uint64>(PreExistingKeys)).Num(), 0);
		TestEqual(TEXT("PreInsertPublicExports does not add new values"), Map.Num(), OriginaMapCount);

		// Overwrite preinserted value
		TestEqual(TEXT("Find stored key with no value set"), Map.Find(Exports[0].PublicExportHash), FPublicExportMap::InvalidValue);
		Map.Store(Exports[0].PublicExportHash, (int32)Exports[0].PublicExportHash);
		TestEqual(TEXT("Find stored key"), Map.Find(Exports[0].PublicExportHash, &HashIndex), (int32)Exports[0].PublicExportHash);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), PreExistingValues.Num() + 1);

		// Insert new value
		TestEqual(TEXT("Find missing key"), Map.Find(111), FPublicExportMap::InvalidValue);
		Map.Store(111, 111);
		TestEqual(TEXT("Find stored key"), Map.Find(111, &HashIndex), 111);
		TestBetweenInclusive(TEXT("Find stored key hash index"), HashIndex, 0, Map.GetValues().Num());
		TestEqual(TEXT("Store increased count"), Map.Num(), PreExistingValues.Num() + 2);

		TestEqual(TEXT("Remove existing key"), Map.Remove(111), true);
		TestEqual(TEXT("Remove decreased count"), Map.Num(), PreExistingValues.Num() + 1);
		TestEqual(TEXT("Remove removed key fails"), Map.Remove(111), false);
		TestEqual(TEXT("Failed remove did not increase count"), Map.Num(), PreExistingValues.Num() + 1);
		TestEqual(TEXT("Find removed key returns no value set"), Map.Find(111), FPublicExportMap::InvalidValue);
	}
	return true;
}

#endif // WITH_LOW_LEVEL_TESTS
