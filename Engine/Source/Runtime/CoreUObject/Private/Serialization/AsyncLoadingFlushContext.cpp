// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/AsyncLoadingFlushContext.h"

#include "Containers/Ticker.h"
#include "Serialization/AsyncPackageLoader.h"

DEFINE_LOG_CATEGORY(AsyncLoadingFlush);

bool IsAsyncLoadingCoreUObjectInternal();

int32 FAsyncLoadingFlushContext::NextId = 0;

FAsyncLoadingFlushContext::FAsyncLoadingFlushContext(const FString& Context)
	: Context(Context)
	, Id(NextId++)
{
}

FAsyncLoadingFlushContext::FAsyncLoadingFlushContext(FString&& Context)
	: Context(MoveTemp(Context))
	, Id(NextId++)
{
}

FAsyncLoadingFlushContext::~FAsyncLoadingFlushContext()
{
	CleanupTickers();
}

void FAsyncLoadingFlushContext::Flush(const FOnAsyncLoadingFlushComplete& OnFlushComplete)
{
	check(IsInGameThread() && !IsInSlateThread());
	
	// It is only valid to call flush once on a context.
	check(!OnFlushCompleteDelegate.IsBound());
	
	// A delegate is required.
	check(OnFlushComplete.IsBound());

	OnFlushCompleteDelegate = OnFlushComplete;

	// Check each frame whether async loading has completed.
	LoadingCompleteCheckDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FAsyncLoadingFlushContext::OnAsyncLoadingCheck));

	// Periodically warn when waiting for async loading to complete.
	WaitingWarnDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FAsyncLoadingFlushContext::OnAsyncLoadingWarn), LoadingFlushWarnIntervalSeconds);

	StartTime = FPlatformTime::Seconds();
	UE_LOGF(AsyncLoadingFlush, Verbose, "Context[%ls:%d] Flushing async loading.", *Context, Id);
}

void FAsyncLoadingFlushContext::CleanupTickers()
{
	if (LoadingCompleteCheckDelegateHandle.IsValid())
	{
		FTSTicker::RemoveTicker(LoadingCompleteCheckDelegateHandle);
		LoadingCompleteCheckDelegateHandle.Reset();
	}

	if (WaitingWarnDelegateHandle.IsValid())
	{
		FTSTicker::RemoveTicker(WaitingWarnDelegateHandle);
		WaitingWarnDelegateHandle.Reset();
	}
}

bool FAsyncLoadingFlushContext::OnAsyncLoadingCheck(float DeltaTime)
{
	if (IsAsyncLoadingCoreUObjectInternal())
	{
		// Continue rescheduling the check.
		return true;
	}

	// Async loading completed, remove warning logger.
	CleanupTickers();

	const double CurrentTimeSeconds = FPlatformTime::Seconds();
	UE_LOGF(AsyncLoadingFlush, Verbose, "Context[%ls:%d] Flushed in %f seconds.", *Context, Id, CurrentTimeSeconds - StartTime);

	// Signal to user that flush has completed.
	FOnAsyncLoadingFlushComplete OnFlushComplete = MoveTemp(OnFlushCompleteDelegate);
	OnFlushComplete.ExecuteIfBound();
	// do not access member variables below this point in case firing the delegate deleted this object.

	// Unregister check.
	return false;
}

bool FAsyncLoadingFlushContext::OnAsyncLoadingWarn(float DeltaTime)
{
	UE_LOGF(AsyncLoadingFlush, Warning, "Context[%ls:%d] Waiting on async loading to complete. Total wait time: %f seconds.", *Context, Id, FPlatformTime::Seconds() - StartTime);
	return true;
}
