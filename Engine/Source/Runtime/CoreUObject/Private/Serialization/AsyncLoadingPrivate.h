// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveProxy.h"
#include "UObject/Linker.h"
#include "Async/AsyncFileHandle.h"

DECLARE_LOG_CATEGORY_EXTERN(LogLoadingDev, Fatal, All);

class FAsyncArchive final: public FArchive
{
public:
	enum class ELoadPhase
	{
		WaitingForSize,
		WaitingForSummary,
		WaitingForHeader,
		WaitingForFirstExport,
		ProcessingExports,
	};
	enum class ELoadError : uint8
	{
		Unknown,
		UnsupportedFormat,
		FileDoesNotExist,
		CorruptData,
		Cancelled,
	};

	FAsyncArchive(const FPackagePath& InPackagePath, FLinkerLoad* InOwner, TFunction<void()>&& InSummaryReadyCallback);
	COREUOBJECT_API virtual ~FAsyncArchive ();

	/** Archive overrides */
	COREUOBJECT_API virtual bool Close() override;
	COREUOBJECT_API virtual bool SetCompressionMap(TArray<FCompressedChunk>* CompressedChunks, ECompressionFlags CompressionFlags) override;
	COREUOBJECT_API virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize) override;
	COREUOBJECT_API virtual void Serialize(void* Data, int64 Num) override;
	FORCEINLINE virtual int64 Tell() override
	{
#if DEVIRTUALIZE_FLinkerLoad_Serialize
		return CurrentPos + (ActiveFPLB->StartFastPathLoadBuffer - ActiveFPLB->OriginalFastPathLoadBuffer);
#else
		return CurrentPos;
#endif
	}
	COREUOBJECT_API virtual int64 TotalSize() override;
	COREUOBJECT_API virtual void Seek(int64 InPos) override;
	COREUOBJECT_API virtual void FlushCache() override;
	virtual FString GetArchiveName() const override 
	{
		return PackagePath.GetDebugName();
	}

	/** AsyncArchive interface */
	COREUOBJECT_API bool PrecacheWithTimeLimit(int64 PrecacheOffset, int64 PrecacheSize, bool bUseTimeLimit, bool bUseFullTimeLimit, double TickStartTime, double TimeLimit);
	COREUOBJECT_API bool PrecacheForEvent(IAsyncReadRequest* Read, int64 PrecacheOffset, int64 PrecacheSize);
	COREUOBJECT_API void FlushPrecacheBlock();
	COREUOBJECT_API bool ReadyToStartReadingHeader(bool bUseTimeLimit, bool bUseFullTimeLimit, double TickStartTime, double TimeLimit);
	COREUOBJECT_API void StartReadingHeader();
	COREUOBJECT_API void EndReadingHeader();
	COREUOBJECT_API IAsyncReadRequest* MakeEventDrivenPrecacheRequest(int64 Offset, int64 BytesToRead, FAsyncFileCallBack* CompleteCallback);
	COREUOBJECT_API void LogItem(const TCHAR* Item, int64 Offset = 0, int64 Size = 0, double StartTime = 0.0);

	bool IsCookedForEDLInEditor() const
	{
		return bCookedForEDLInEditor;
	}

	ELoadError GetLoadError() const
	{
		return LoadError;
	}
	bool NeedsEngineVersionChecks() const
	{
		return bNeedsEngineVersionChecks;
	}


private:
#if DEVIRTUALIZE_FLinkerLoad_Serialize
	/**
	* Updates CurrentPos based on StartFastPathLoadBuffer and sets both StartFastPathLoadBuffer and EndFastPathLoadBuffer to null
	*/
	COREUOBJECT_API void DiscardInlineBufferAndUpdateCurrentPos();

	COREUOBJECT_API void SetPosAndUpdatePrecacheBuffer(int64 Pos);

#endif
	COREUOBJECT_API void FirstExportStarting();
	COREUOBJECT_API bool WaitRead(double TimeLimit = 0.0);
	COREUOBJECT_API void CompleteRead();
	COREUOBJECT_API void CancelRead();
	COREUOBJECT_API void CompleteCancel();
	COREUOBJECT_API bool WaitForIntialPhases(double TimeLimit = 0.0);
	COREUOBJECT_API void ReadCallback(bool bWasCancelled, IAsyncReadRequest*);
	COREUOBJECT_API bool PrecacheInternal(int64 PrecacheOffset, int64 PrecacheSize, bool bApplyMinReadSize = true, IAsyncReadRequest* Read = nullptr);
	
	FORCEINLINE int64 TotalSizeOrMaxInt64IfNotReady()
	{

		return SizeRequestPtr ? MAX_int64 : (FileSize + HeaderSizeWhenReadingExportsFromSplitFile);
	}

	IAsyncReadFileHandle* Handle;
	IAsyncReadRequest* SizeRequestPtr;
	IAsyncReadRequest* EditorPrecacheRequestPtr;

	IAsyncReadRequest* SummaryRequestPtr;
	IAsyncReadRequest* SummaryPrecacheRequestPtr;

	IAsyncReadRequest* ReadRequestPtr;
	IAsyncReadRequest* CanceledReadRequestPtr;

	/** Buffer containing precached data.											*/
	uint8* PrecacheBuffer;
	/** Cached file size															*/
	int64 FileSize;
	/** Current position of archive.												*/
	int64 CurrentPos;
	/** Start position of current precache request.									*/
	int64 PrecacheStartPos;
	/** End position (exclusive) of current precache request.						*/
	int64 PrecacheEndPos;

	int64 ReadRequestOffset;
	int64 ReadRequestSize;

	int64 HeaderSize;
	int64 HeaderSizeWhenReadingExportsFromSplitFile;

	ELoadPhase LoadPhase;
	ELoadError LoadError;

	/** If true, this package is a cooked EDL package loaded in uncooked builds */
	bool bCookedForEDLInEditor;
	/** True if the linker should do version and corruption checks on bytes of this archive. */
	bool bNeedsEngineVersionChecks;

	FAsyncFileCallBack ReadCallbackFunction;
	/** Cached PackagePath for debugging.												*/
	FPackagePath PackagePath;
	double OpenTime;
	double SummaryReadTime;
	double ExportReadTime;

	TFunction<void()> SummaryReadyCallback;
	FAsyncFileCallBack ReadCallbackFunctionForLinkerLoad;

	FLinkerLoad* OwnerLinker;
};
