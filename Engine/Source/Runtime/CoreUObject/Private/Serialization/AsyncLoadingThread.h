// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AsyncLoadingThread.h: Unreal async loading code.
=============================================================================*/
#pragma once

#include "CoreMinimal.h"
#include "Algo/AnyOf.h"
#include "HAL/ThreadSafeCounter.h"
#include "UObject/ObjectMacros.h"
#include "Serialization/AsyncLoading.h"
#include "Serialization/LoadTimeTracePrivate.h"
#include "Misc/ScopeLock.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "HAL/Runnable.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/LowLevelMemTracker.h"
#include "Serialization/AsyncPackageLoader.h"
#include "AutoRTFM.h"

#include <atomic>

/** Holds the maximum package summary size that can be set via ini files
  * This is used for the initial precache and should be large enough to hold the actual Sum.TotalHeaderSize 
  */
struct FMaxPackageSummarySize
{
	static int32 Value;
	static void Init();
};
