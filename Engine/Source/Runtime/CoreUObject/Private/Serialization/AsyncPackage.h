// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AsyncPackage.h: Unreal async loading definitions.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/PackagePath.h"
#include "Misc/PackageAccessTracking.h"
#include "Templates/UniquePtr.h"
#include "UObject/ICookInfo.h"
#include "UObject/LinkerInstancingContext.h"

struct FAsyncPackageDesc
{
	/** Handle for the caller */
	int32 RequestID;
	/** Name of the UPackage to create. */
	FName Name;
	/** PackagePath of the package to load. */
	FPackagePath PackagePath;
	/** Delegate called on completion of loading. This delegate can only be created and consumed on the game thread */
	TUniquePtr<FLoadPackageAsyncDelegate> PackageLoadedDelegate;
	/** The flags that should be applied to the package */
	EPackageFlags PackageFlags;
	/** Package loading priority. Higher number is higher priority. */
	TAsyncLoadPriority Priority;
	/** PIE instance ID this package belongs to, INDEX_NONE otherwise */
	int32 PIEInstanceID;
	/** Instancing context, maps original package to their instanced counterpart, used to remap imports. */
	FLinkerInstancingContext InstancingContext;
#if UE_WITH_PACKAGE_ACCESS_TRACKING
	FName ReferencerPackageName;
	FName ReferencerPackageOp;
#endif
#if WITH_EDITOR
	ECookLoadType CookLoadType;
#endif

	const FLinkerInstancingContext* GetInstancingContext() const { return &InstancingContext; }
	void SetInstancingContext(FLinkerInstancingContext InInstancingContext) { InstancingContext = MoveTemp(InInstancingContext); }

	FAsyncPackageDesc(int32 InRequestID, const FName& InName, const FPackagePath& InPackagePath, TUniquePtr<FLoadPackageAsyncDelegate>&& InCompletionDelegate = TUniquePtr<FLoadPackageAsyncDelegate>(), EPackageFlags InPackageFlags = PKG_None, int32 InPIEInstanceID = INDEX_NONE, TAsyncLoadPriority InPriority = 0)
		: RequestID(InRequestID)
		, Name(InName)
		, PackagePath(InPackagePath)
		, PackageLoadedDelegate(MoveTemp(InCompletionDelegate))
		, PackageFlags(InPackageFlags)
		, Priority(InPriority)
		, PIEInstanceID(InPIEInstanceID)
#if WITH_EDITOR
		, CookLoadType(ECookLoadType::Unspecified)
#endif
	{
		check(!PackagePath.IsEmpty());
	}

	/** This constructor does not modify the package loaded delegate as this is not safe outside the game thread */
	FAsyncPackageDesc(const FAsyncPackageDesc& OldPackage)
		: RequestID(OldPackage.RequestID)
		, Name(OldPackage.Name)
		, PackagePath(OldPackage.PackagePath)
		, PackageFlags(OldPackage.PackageFlags)
		, Priority(OldPackage.Priority)
		, PIEInstanceID(OldPackage.PIEInstanceID)
		, InstancingContext(OldPackage.InstancingContext, FLinkerInstancingContext::EContextSharingMode::Link)
#if UE_WITH_PACKAGE_ACCESS_TRACKING
		, ReferencerPackageName(OldPackage.ReferencerPackageName)
		, ReferencerPackageOp(OldPackage.ReferencerPackageOp)
#endif
#if WITH_EDITOR
		, CookLoadType(OldPackage.CookLoadType)
#endif
	{
	}

	/** This constructor will explicitly copy the package loaded delegate and invalidate the old one */
	FAsyncPackageDesc(const FAsyncPackageDesc& OldPackage, TUniquePtr<FLoadPackageAsyncDelegate>&& InPackageLoadedDelegate)
		: FAsyncPackageDesc(OldPackage)
	{
		PackageLoadedDelegate = MoveTemp(InPackageLoadedDelegate);
	}

#if DO_GUARD_SLOW
	~FAsyncPackageDesc()
	{
		checkSlow(!PackageLoadedDelegate.IsValid() || IsInGameThread());
	}
#endif
};
