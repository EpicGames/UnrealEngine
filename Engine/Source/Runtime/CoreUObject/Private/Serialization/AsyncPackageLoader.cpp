// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/AsyncPackageLoader.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/AsyncLoadingThread.h"
#include "Serialization/AsyncLoading2.h"
#include "Serialization/TransactionallySafeAsyncLoading.h"
#include "UObject/GCObject.h"
#include "UObject/LinkerLoad.h"
#include "IO/PackageId.h"
#include "Misc/CoreDelegates.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "IO/IoDispatcher.h"
#include "IO/IoDispatcherInternal.h"
#include "HAL/IConsoleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "AutoRTFM.h"

static bool GUseOldLoaderAsFallback = false;
static FAutoConsoleVariableRef CVarGUseOldLoaderAsFallback(
	TEXT("s.UseOldLoaderAsFallback"),
	GUseOldLoaderAsFallback,
	TEXT("When active, the old loader will be used as the fallback instead of zenloader"),
	ECVF_Default
);

#define DO_TRACK_ASYNC_LOAD_REQUESTS (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)

#if DO_TRACK_ASYNC_LOAD_REQUESTS
#include "Containers/StackTracker.h"
#include "Misc/OutputDeviceFile.h"
#endif

volatile int32 GIsLoaderCreated;
TUniquePtr<IAsyncPackageLoader> GPackageLoader;
bool GAsyncLoadingAllowed = true;
bool GIoDispatcherInitializedByLoader = false;

FThreadSafeCounter IAsyncPackageLoader::NextPackageRequestId;

int32 IAsyncPackageLoader::GetNextRequestId()
{
	return NextPackageRequestId.Increment();
}

class FEarlyRegistrationEventsRecorder
{
public:
	FEarlyRegistrationEventsRecorder()
	{
		RecordedEvents.Reserve(16388);
	}

	void NotifyRegistrationEvent(FName PackageName, FName Name, ENotifyRegistrationType NotifyRegistrationType, ENotifyRegistrationPhase NotifyRegistrationPhase, FTypeConstructFunc* RegisterFunc, bool bDynamic, UObject* FinishedObject)
	{
		RecordedEvents.Add({ PackageName, Name, NotifyRegistrationType, NotifyRegistrationPhase, RegisterFunc, FinishedObject, bDynamic });
	}

	void Replay(IAsyncPackageLoader& PackageLoader)
	{
		UE_LOGF(LogStreaming, Log, "NotifyRegistrationEvent: Replay %d entries", RecordedEvents.Num());
		for (const FRecordedEvent& Event : RecordedEvents)
		{
			PackageLoader.NotifyRegistrationEvent(Event.PackageName, Event.Name, Event.NotifyRegistrationType, Event.NotifyRegistrationPhase, Event.RegisterFunc, Event.bDynamic, Event.FinishedObject);
		}
		RecordedEvents.Empty();
	}

private:
	struct FRecordedEvent
	{
		FName PackageName;
		FName Name;
		ENotifyRegistrationType NotifyRegistrationType;
		ENotifyRegistrationPhase NotifyRegistrationPhase;
		FTypeConstructFunc* RegisterFunc;
		UObject* FinishedObject;
		bool bDynamic;
	};

	TArray<FRecordedEvent> RecordedEvents;
};

static FEarlyRegistrationEventsRecorder& GetEarlyRegistrationEventsRecorder()
{
	static FEarlyRegistrationEventsRecorder Singleton;
	return Singleton;
}

#if !UE_BUILD_SHIPPING
static void LoadPackageCommand(const TArray<FString>& Args)
{
	for (const FString& PackageName : Args)
	{
		UE_LOGF(LogStreaming, Display, "LoadPackageCommand: %ls - Requested", *PackageName);
		UPackage* Package = LoadPackage(nullptr, *PackageName, LOAD_None);
		UE_LOGF(LogStreaming, Display, "LoadPackageCommand: %ls - %ls",
			*PackageName, (Package != nullptr) ? TEXT("Loaded") : TEXT("Failed"));
	}
}

static void LoadPackageAsyncCommand(const TArray<FString>& Args)
{
	for (const FString& PackageName : Args)
	{
		UE_LOGF(LogStreaming, Display, "LoadPackageAsyncCommand: %ls - Requested", *PackageName);
		LoadPackageAsync(PackageName, FLoadPackageAsyncDelegate::CreateLambda(
			[](const FName& PackageName, UPackage* Package, EAsyncLoadingResult::Type Result)
		{
			UE_LOGF(LogStreaming, Display, "LoadPackageAsyncCommand: %ls - %ls",
				*PackageName.ToString(), (Package != nullptr) ? TEXT("Loaded") : TEXT("Failed"));
		}
		));
	}
}

static FAutoConsoleCommand CVar_LoadPackageCommand(
	TEXT("LoadPackage"),
	TEXT("Loads packages by names. Usage: LoadPackage <package name> [<package name> ...]"),
	FConsoleCommandWithArgsDelegate::CreateStatic(LoadPackageCommand));

static FAutoConsoleCommand CVar_LoadPackageAsyncCommand(
	TEXT("LoadPackageAsync"),
	TEXT("Loads packages async by names. Usage: LoadPackageAsync <package name> [<package name> ...]"),
	FConsoleCommandWithArgsDelegate::CreateStatic(LoadPackageAsyncCommand));
#endif

const FName PrestreamPackageClassNameLoad = FName("PrestreamPackage");

FAsyncLoadingThreadSettings::FAsyncLoadingThreadSettings()
	: bAsyncLoadingThreadEnabled(false)
	, bAsyncPostLoadEnabled(false)
	, bLooseFileLoadingEnabled(!!WITH_EDITOR)
{
#if THREADSAFE_UOBJECTS
	check(GConfig);
	// if we are on an uncooked platform query the editor streaming settings
	const TCHAR* SectionName = FPlatformProperties::RequiresCookedData() ? TEXT("/Script/Engine.StreamingSettings") : TEXT("/Script/Engine.EditorStreamingSettings");

	// if we are on an uncooked platform the default value for async loading is disabled while it is enabled for cooked platform
	bool bConfigValue = FPlatformProperties::RequiresCookedData() ? true : false;

	GConfig->GetBool(SectionName, TEXT("s.AsyncLoadingThreadEnabled"), bConfigValue, GEngineIni);
	bool bCommandLineDisable = FParse::Param(FCommandLine::Get(), TEXT("NoAsyncLoadingThread"));
	bool bCommandLineEnable = FParse::Param(FCommandLine::Get(), TEXT("AsyncLoadingThread"));
	bAsyncLoadingThreadEnabled = bCommandLineEnable || (bConfigValue && FApp::ShouldUseThreadingForPerformance() && !bCommandLineDisable);

	// if we are on an uncooked platform the default value for async loading is disabled while it is enabled for cooked platform
	bConfigValue = FPlatformProperties::RequiresCookedData() ? true : false;

	GConfig->GetBool(SectionName, TEXT("s.AsyncPostLoadEnabled"), bConfigValue, GEngineIni);
	bCommandLineDisable = FParse::Param(FCommandLine::Get(), TEXT("NoAsyncPostLoad"));
	bCommandLineEnable = FParse::Param(FCommandLine::Get(), TEXT("AsyncPostLoad"));
	bAsyncPostLoadEnabled = bCommandLineEnable || (bConfigValue && FApp::ShouldUseThreadingForPerformance() && !bCommandLineDisable);

	// By default the editor will look for uncooked files on disk and attempt to load them first
	bConfigValue = !!WITH_EDITOR;
	GConfig->GetBool(SectionName, TEXT("s.LooseFileLoadingEnabled"), bConfigValue, GEngineIni);
	bCommandLineEnable = FParse::Param(FCommandLine::Get(), TEXT("loosefileloading"));
	bLooseFileLoadingEnabled = bConfigValue || bCommandLineEnable;
#endif
}

FAsyncLoadingThreadSettings& FAsyncLoadingThreadSettings::Get()
{
	static FAsyncLoadingThreadSettings Settings;
	return Settings;
}

bool IsNativeCodePackage(UPackage* Package)
{
	return (Package && Package->HasAnyPackageFlags(PKG_CompiledIn));
}

/** Checks if the object can have PostLoad called on the Async Loading Thread */
bool CanPostLoadOnAsyncLoadingThread(UObject* Object)
{
	if (Object->IsPostLoadThreadSafe())
	{
		bool bCanPostLoad = true;
		// All outers should also be safe to call PostLoad on ALT
		for (UObject* Outer = Object->GetOuter(); Outer && bCanPostLoad; Outer = Outer->GetOuter())
		{
			bCanPostLoad = !Outer->HasAnyFlags(RF_NeedPostLoad) || Outer->IsPostLoadThreadSafe();
		}
		return bCanPostLoad;
	}
	return false;
}

IAsyncPackageLoader& GetAsyncPackageLoader()
{
	check(GPackageLoader.Get());
	return *GPackageLoader;
}

void SetAsyncLoadingAllowed(bool bAllowAsyncLoading)
{
	GAsyncLoadingAllowed = bAllowAsyncLoading;
}

void InitAsyncThread()
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	
	if (!FIoDispatcher::IsInitialized())
	{
		FIoDispatcher::Initialize();
		GIoDispatcherInitializedByLoader = true;
	}

	// If there is no IODispatcher data, enable the loose file loading path for zenloader so 
	// the user can attempt to find content on disk, otherwise they will have no loading capabilities
	if (!FIoDispatcherInternal::HasPackageData())
	{
		FAsyncLoadingThreadSettings::Get().bLooseFileLoadingEnabled = true;
	}

	GPackageLoader.Reset(MakeAsyncPackageLoader2(FIoDispatcher::Get()));

#if UE_AUTORTFM_ENABLED
	// If we are using AutoRTFM we need to layer a transactionally safe package loader over
	// whatever loader was to be used.
	GPackageLoader.Reset(MakeTransactionallySafeAsyncPackageLoader(GPackageLoader.Release()));
#endif

	FPlatformAtomics::InterlockedIncrement(&GIsLoaderCreated);

	FCoreDelegates::OnSyncLoadPackage.AddStatic([](const FString&) { GSyncLoadCount++; });
 
	GetEarlyRegistrationEventsRecorder().Replay(*GPackageLoader);
	GPackageLoader->InitializeLoading();
}

void ShutdownAsyncThread()
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	if (GPackageLoader)
	{
		GPackageLoader->ShutdownLoading();
		GPackageLoader.Reset(nullptr);
	}

	if (GIoDispatcherInitializedByLoader)
	{
		FIoDispatcher::Shutdown();
	}
}

bool IsInAsyncLoadingThreadCoreUObjectInternal()
{
	if (GPackageLoader)
	{
		return GPackageLoader->IsInAsyncLoadThread();
	}
	else
	{
		return false;
	}
}

void FlushAsyncLoading(int32 RequestId /* = INDEX_NONE */)
{
	if (RequestId == INDEX_NONE)
	{
		FlushAsyncLoading(TConstArrayView<int32>());
	}
	else
	{
		FlushAsyncLoading(MakeArrayView({RequestId}));
	}
}

void FlushAsyncLoading(TConstArrayView<int32> RequestIds)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FlushAsyncLoading);
	CSV_SCOPED_TIMING_STAT_EXCLUSIVE(FlushAsyncLoading);
	CSV_EVENT_GLOBAL_NOLOG(TEXT("FlushAsyncLoading"));

#if defined(WITH_CODE_GUARD_HANDLER) && WITH_CODE_GUARD_HANDLER
	void CheckImageIntegrityAtRuntime();
	CheckImageIntegrityAtRuntime();
#endif
	LLM_SCOPE(ELLMTag::AsyncLoading);

	if (GPackageLoader)
	{
#if NO_LOGGING == 0
		if (IsAsyncLoading() && IsInGameThread())
		{
			// Log the flush, but only display once per frame to avoid log spam.
			static uint64 LastFrameNumber = -1;
			TStringBuilder<1024> Buffer;
			if (LastFrameNumber != GFrameNumber)
			{
				UE_LOGF(LogStreaming, Display, "FlushAsyncLoading(%ls): %d QueuedPackages, %d AsyncPackages", *Buffer.Join(RequestIds, TEXT(",")), GPackageLoader->GetNumQueuedPackages(), GPackageLoader->GetNumAsyncPackages());
			}
			else
			{
				UE_LOGF(LogStreaming, Log, "FlushAsyncLoading(%ls): %d QueuedPackages, %d AsyncPackages", *Buffer.Join(RequestIds, TEXT(",")), GPackageLoader->GetNumQueuedPackages(), GPackageLoader->GetNumAsyncPackages());
			}
			LastFrameNumber = GFrameNumber;
		}
#endif
		GPackageLoader->FlushLoading(RequestIds);
	}
}

EAsyncPackageState::Type ProcessAsyncLoadingUntilComplete(TFunctionRef<bool()> CompletionPredicate, double TimeLimit)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	return GetAsyncPackageLoader().ProcessLoadingUntilComplete(CompletionPredicate, TimeLimit);
}

int32 GetNumAsyncPackages()
{
	return GetAsyncPackageLoader().GetNumAsyncPackages();
}

EAsyncPackageState::Type ProcessAsyncLoading(bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	TRACE_CPUPROFILER_EVENT_SCOPE(ProcessAsyncLoading);
	return GetAsyncPackageLoader().ProcessLoading(bUseTimeLimit, bUseFullTimeLimit, TimeLimit);
}

bool IsAsyncLoadingCoreUObjectInternal()
{
	// GIsInitialLoad guards the async loading thread from being created too early
	return GetAsyncPackageLoader().IsAsyncLoadingPackages();
}

bool IsAsyncLoadingMultithreadedCoreUObjectInternal()
{
	// GIsInitialLoad guards the async loading thread from being created too early
	return GetAsyncPackageLoader().IsMultithreaded();
}

ELoaderType GetLoaderTypeInternal()
{
	return GetAsyncPackageLoader().GetLoaderType();
}

void SuspendAsyncLoadingInternal()
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	check(IsInGameThread() && !IsInSlateThread());
	GetAsyncPackageLoader().SuspendLoading();
}

void ResumeAsyncLoadingInternal()
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	check(IsInGameThread() && !IsInSlateThread());
	GetAsyncPackageLoader().ResumeLoading();
}

bool IsAsyncLoadingSuspendedInternal()
{
	return GetAsyncPackageLoader().IsAsyncLoadingSuspended();
}

#if DO_TRACK_ASYNC_LOAD_REQUESTS
static TAutoConsoleVariable<int32> CVarTrackAsyncLoadRequests_Enable(
	TEXT("TrackAsyncLoadRequests.Enable"),
	0,
	TEXT("If > 0 then remove aliases from the counting process. This essentialy merges addresses that have the same human readable string. It is slower.")
);

static TAutoConsoleVariable<int32> CVarTrackAsyncLoadRequests_Dedupe(
	TEXT("TrackAsyncLoadRequests.Dedupe"),
	0,
	TEXT("If > 0 then deduplicate requests to async load the same package in the report.")
);

static TAutoConsoleVariable<int32> CVarTrackAsyncLoadRequests_RemoveAliases(
	TEXT("TrackAsyncLoadRequests.RemoveAliases"),
	1,
	TEXT("If > 0 then remove aliases from the counting process. This essentialy merges addresses that have the same human readable string. It is slower.")
);

static TAutoConsoleVariable<int32> CVarTrackAsyncLoadRequests_StackIgnore(
	TEXT("TrackAsyncLoadRequests.StackIgnore"),
	5,
	TEXT("Number of items to discard from the top of a stack frame."));

static TAutoConsoleVariable<int32> CVarTrackAsyncLoadRequests_StackLen(
	TEXT("TrackAsyncLoadRequests.StackLen"),
	12,
	TEXT("Maximum number of stack frame items to keep. This improves aggregation because calls that originate from multiple places but end up in the same place will be accounted together."));

static TAutoConsoleVariable<int32> CVarTrackAsyncLoadRequests_Threshhold(
	TEXT("TrackAsyncLoadRequests.Threshhold"),
	0,
	TEXT("Minimum number of hits to include in the report."));

static TAutoConsoleVariable<int32> CVarTrackAsyncLoadRequests_DumpAfterCsvProfiling(
	TEXT("TrackAsyncLoadRequests.DumpAfterCsvProfiling"),
	1,
	TEXT("If > 0, dumps tracked async load requests to a file when csv profiling ends."));


struct FTrackAsyncLoadRequests
{
	FStackTracker StackTracker;
	FCriticalSection CritSec;

	struct FUserData
	{
		struct FLoadRequest
		{
			FString RequestName;
			int32 Priority;

			FLoadRequest(FString InName, int32 InPriority)
				: RequestName(MoveTemp(InName))
				, Priority(InPriority)
			{
			}
		};
		TArray<FLoadRequest> Requests;
	};

	static FTrackAsyncLoadRequests& Get()
	{
		static FTrackAsyncLoadRequests Instance;
		return Instance;
	}

	FTrackAsyncLoadRequests()
		: StackTracker(&UpdateStack, &ReportStack, &DeleteUserData, true)
	{
#if CSV_PROFILER
		FCsvProfiler::Get()->OnCSVProfileEnd().AddRaw(this, &FTrackAsyncLoadRequests::DumpRequestsAfterCsvProfiling);
#endif
	}

	static void UpdateStack(const FStackTracker::FCallStack& CallStack, void* InUserData)
	{
		FUserData* NewUserData = (FUserData*)InUserData;
		FUserData* OldUserData = (FUserData*)CallStack.UserData;

		OldUserData->Requests.Append(NewUserData->Requests);
	}

	static void ReportStack(const FStackTracker::FCallStack& CallStack, uint64 TotalStackCount, FOutputDevice& Ar)
	{
		FUserData* UserData = (FUserData*)CallStack.UserData;
		bool bOldSuppress = Ar.GetSuppressEventTag();
		Ar.SetSuppressEventTag(true);

		if (CVarTrackAsyncLoadRequests_Dedupe->GetInt() > 0)
		{
			Ar.Logf(TEXT("Requested package names (Deduped):"));
			Ar.Logf(TEXT("===================="));
			TSet<FString> Seen;
			for (auto& Request : UserData->Requests)
			{
				if (!Seen.Contains(Request.RequestName))
				{
					Ar.Logf(TEXT("%d %s"), Request.Priority, *Request.RequestName);
					Seen.Add(Request.RequestName);
				}
			}
		}
		else
		{
			Ar.Logf(TEXT("Requested package names:"));
			Ar.Logf(TEXT("===================="));
			for (const auto& Request : UserData->Requests)
			{
				Ar.Logf(TEXT("%d %s"), Request.Priority, *Request.RequestName);
			}
		}
		Ar.Logf(TEXT("===================="));
		Ar.SetSuppressEventTag(bOldSuppress);
	}
	
	static void DeleteUserData(void* InUserData)
	{
		FUserData* UserData = (FUserData*)InUserData;
		delete UserData;
	}

	void TrackRequest(const FString& InName, const TCHAR* InPackageToLoadFrom, int32 InPriority)
	{
		if (CVarTrackAsyncLoadRequests_Enable->GetInt() == 0)
		{
			return;
		}

		FUserData* UserData = new FUserData;
		UserData->Requests.Emplace(InPackageToLoadFrom ? FString(InPackageToLoadFrom) : InName, InPriority);

		FScopeLock Lock(&CritSec);
		StackTracker.CaptureStackTrace(CVarTrackAsyncLoadRequests_StackIgnore->GetInt(), (void*)UserData, CVarTrackAsyncLoadRequests_StackLen->GetInt(), CVarTrackAsyncLoadRequests_Dedupe->GetBool());
	}

	void Reset()
	{
		FScopeLock Lock(&CritSec);
		StackTracker.ResetTracking();
	}

	void DumpRequests(bool bReset)
	{
		FScopeLock Lock(&CritSec);
		StackTracker.DumpStackTraces(CVarTrackAsyncLoadRequests_Threshhold->GetInt(), *GLog);
		if (bReset)
		{
			StackTracker.ResetTracking();
		}
	}

	void DumpRequestsToFile(bool bReset)
	{
		FString Filename = FPaths::ProfilingDir() / FString::Printf(TEXT("AsyncLoadRequests_%s.log"), *FDateTime::Now().ToString());
		FOutputDeviceFile Out(*Filename, true);
		Out.SetSuppressEventTag(true);

		UE_LOGF(LogStreaming, Display, "Dumping async load requests & callstacks to %ls", *Filename);

		FScopeLock Lock(&CritSec);
		StackTracker.DumpStackTraces(CVarTrackAsyncLoadRequests_Threshhold->GetInt(), Out);
		if (bReset)
		{
			StackTracker.ResetTracking();
		}
	}

#if CSV_PROFILER
	void DumpRequestsAfterCsvProfiling()
	{
		if (CVarTrackAsyncLoadRequests_DumpAfterCsvProfiling->GetInt() > 0)
		{
			DumpRequestsToFile(false);
		}
	}
#endif
};

static FAutoConsoleCommand TrackAsyncLoadRequests_Reset
(
	TEXT("TrackAsyncLoadRequests.Reset"),
	TEXT("Reset tracked async load requests"),
	FConsoleCommandDelegate::CreateLambda([]() { FTrackAsyncLoadRequests::Get().Reset(); })
);

static FAutoConsoleCommand TrackAsyncLoadRequests_Dump
(
	TEXT("TrackAsyncLoadRequests.Dump"),
	TEXT("Dump tracked async load requests and reset tracking"),
	FConsoleCommandDelegate::CreateLambda([]() { FTrackAsyncLoadRequests::Get().DumpRequests(true); })
);

static FAutoConsoleCommand TrackAsyncLoadRequests_DumpToFile
(
	TEXT("TrackAsyncLoadRequests.DumpToFile"),
	TEXT("Dump tracked async load requests and reset tracking"),
	FConsoleCommandDelegate::CreateLambda([]() { FTrackAsyncLoadRequests::Get().DumpRequestsToFile(true); })
);
#endif

static FPackagePath GetLoadPackageAsyncPackagePath(FStringView InPackageNameOrFilePath)
{
	FPackagePath PackagePath;
	if (!FPackagePath::TryFromMountedName(InPackageNameOrFilePath, PackagePath))
	{
		// Legacy behavior from FAsyncLoadingThread::LoadPackage: handle asset strings with class references: ClassName'PackageName'. 
		FStringView ExportTextPackagePath;
		if (FPackageName::ParseExportTextPath(InPackageNameOrFilePath, nullptr /* OutClassName */, &ExportTextPackagePath))
		{
			if (FPackagePath::TryFromMountedName(ExportTextPackagePath, PackagePath))
			{
				UE_LOGF(LogStreaming, Warning, "Deprecation warning: calling LoadPackage with the export text format of a package name (ClassName'PackageName') is deprecated and will be removed in a future release.");
			}
		}
	}

	// If PackagePath is empty at this point, then we are going to fail because its not mounted. But pass the input name 
	// into PackagePath (preferably as a packagename) to provide to the caller's CompletionCallback
	if (PackagePath.IsEmpty())
	{
		if (!FPackagePath::TryFromPackageName(InPackageNameOrFilePath, PackagePath))
		{
			PackagePath = FPackagePath::FromLocalPath(InPackageNameOrFilePath);
		}
	}

	return PackagePath;
}

int32 LoadPackageAsync(const FPackagePath& InPackagePath, FLoadPackageAsyncOptionalParams InOptionalParams)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	UE_CLOGF(!GAsyncLoadingAllowed && !IsInAsyncLoadingThread(), LogStreaming, Fatal, "Requesting async load of \"%ls\" when async loading is not allowed (after shutdown). Please fix higher level code.", *InPackagePath.GetDebugName());
#if DO_TRACK_ASYNC_LOAD_REQUESTS
	FTrackAsyncLoadRequests::Get().TrackRequest(InPackagePath.GetDebugName(), nullptr, InOptionalParams.PackagePriority);
#endif
	return GetAsyncPackageLoader().LoadPackage(InPackagePath, MoveTemp(InOptionalParams));
}

int32 LoadPackageAsync(const FPackagePath& InPackagePath,
		FName InPackageNameToCreate /* = NAME_None*/,
		FLoadPackageAsyncDelegate InCompletionDelegate /*= FLoadPackageAsyncDelegate()*/,
		EPackageFlags InPackageFlags /*= PKG_None*/,
		int32 InPIEInstanceID /*= INDEX_NONE*/,
		int32 InPackagePriority /*= 0*/,
		const FLinkerInstancingContext* InstancingContext /*=nullptr*/,
		uint32 LoadFlags /*=LOAD_None*/)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	UE_CLOGF(!GAsyncLoadingAllowed && !IsInAsyncLoadingThread(), LogStreaming, Fatal, "Requesting async load of \"%ls\" when async loading is not allowed (after shutdown). Please fix higher level code.", *InPackagePath.GetDebugName());
#if DO_TRACK_ASYNC_LOAD_REQUESTS
	FTrackAsyncLoadRequests::Get().TrackRequest(InPackagePath.GetDebugName(), nullptr, InPackagePriority);
#endif
	return GetAsyncPackageLoader().LoadPackage(InPackagePath, InPackageNameToCreate, InCompletionDelegate, InPackageFlags, InPIEInstanceID, InPackagePriority, InstancingContext, LoadFlags);
}

int32 LoadPackageAsync(const FString& InName, const FGuid* InGuid /* nullptr*/)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	FPackagePath PackagePath = GetLoadPackageAsyncPackagePath(InName);
	return LoadPackageAsync(PackagePath, NAME_None /* InPackageNameToCreate */, FLoadPackageAsyncDelegate(), PKG_None, INDEX_NONE, 0, nullptr);
}

int32 LoadPackageAsync(const FString& InName, FLoadPackageAsyncDelegate CompletionDelegate, int32 InPackagePriority /*= 0*/, EPackageFlags InPackageFlags /*= PKG_None*/, int32 InPIEInstanceID /*= INDEX_NONE*/)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	FPackagePath PackagePath = GetLoadPackageAsyncPackagePath(InName);
	return LoadPackageAsync(PackagePath, NAME_None /* InPackageNameToCreate */, CompletionDelegate, InPackageFlags, InPIEInstanceID, InPackagePriority, nullptr);
}

int32 LoadPackageAsync(const FString& InName, FLoadPackageAsyncOptionalParams InOptionalParams)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	FPackagePath PackagePath = GetLoadPackageAsyncPackagePath(InName);
	return LoadPackageAsync(PackagePath, MoveTemp(InOptionalParams));
}

int32 LoadPackageAsync(const FString& InName, const FGuid* InGuid /*= nullptr*/, const TCHAR* InPackageToLoadFrom, FLoadPackageAsyncDelegate InCompletionDelegate /*= FLoadPackageAsyncDelegate()*/, EPackageFlags InPackageFlags /*= PKG_None*/, int32 InPIEInstanceID /*= INDEX_NONE*/, int32 InPackagePriority /*= 0*/, const FLinkerInstancingContext* InstancingContext /*=nullptr*/)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	FPackagePath PackagePath = GetLoadPackageAsyncPackagePath(InPackageToLoadFrom ? FStringView(InPackageToLoadFrom) : FStringView(InName));
	FName InPackageNameToCreate;
	if (InPackageToLoadFrom)
	{
		// We're loading from PackageToLoadFrom. If InName is different, it is the PackageNameToCreate. InName might be a filepath, so use FPackagePath to convert it to a PackagePath
		FPackagePath PackagePathToCreate;
		if (FPackagePath::TryFromMountedName(InName, PackagePathToCreate))
		{
			InPackageNameToCreate = PackagePathToCreate.GetPackageFName();
		}
	}
	return LoadPackageAsync(PackagePath, InPackageNameToCreate, MoveTemp(InCompletionDelegate), InPackageFlags, InPIEInstanceID, InPackagePriority, InstancingContext);
}

void CancelAsyncLoading()
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	// Cancelling async loading while loading is suspend will result in infinite stall
	UE_CLOGF(GetAsyncPackageLoader().IsAsyncLoadingSuspended(), LogStreaming, Fatal, "Cannot Cancel Async Loading while async loading is suspended.");
	GetAsyncPackageLoader().CancelLoading();

	if (!IsEngineExitRequested())
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
	}

	const EInternalObjectFlags AsyncFlags = EInternalObjectFlags::Async | EInternalObjectFlags_AsyncLoading;
	for (int32 ObjectIndex = 0; ObjectIndex < GUObjectArray.GetObjectArrayNum(); ++ObjectIndex)
	{
		FUObjectItem* ObjectItem = &GUObjectArray.GetObjectItemArrayUnsafe()[ObjectIndex];
		if (UObject* Obj = static_cast<UObject*>(ObjectItem->GetObject()))
		{
			check(!Obj->HasAnyInternalFlags(AsyncFlags));
		}
	}
}

float GetAsyncLoadPercentage(const FName& PackageName)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	return GetAsyncPackageLoader().GetAsyncLoadPercentage(PackageName);
}

void NotifyRegistrationEvent(FName PackageName, FName Name, ENotifyRegistrationType NotifyRegistrationType, ENotifyRegistrationPhase NotifyRegistrationPhase, FTypeConstructFunc* InRegister, bool InbDynamic, UObject* FinishedObject)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);

	if (GPackageLoader)
	{
		GPackageLoader->NotifyRegistrationEvent(PackageName, Name, NotifyRegistrationType, NotifyRegistrationPhase, InRegister, InbDynamic, FinishedObject);
	}
	else
	{
		GetEarlyRegistrationEventsRecorder().NotifyRegistrationEvent(PackageName, Name, NotifyRegistrationType, NotifyRegistrationPhase, InRegister, InbDynamic, FinishedObject);
	}
}

void NotifyScriptVersePackage(Verse::VPackage* Package)
{
	check(GPackageLoader);
	GPackageLoader->NotifyScriptVersePackage(Package);
}

void NotifyRegistrationComplete()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(NotifyRegistrationComplete);
	LLM_SCOPE(ELLMTag::AsyncLoading);
	GPackageLoader->NotifyRegistrationComplete();
	FlushAsyncLoading();
	GPackageLoader->StartThread();
}

void NotifyUnreachableObjects(const TArrayView<FUObjectItem*>& UnreachableObjects)
{
	LLM_SCOPE(ELLMTag::AsyncLoading);
	GetAsyncPackageLoader().NotifyUnreachableObjects(UnreachableObjects);
}

double GFlushAsyncLoadingTime = 0.0;
uint32 GFlushAsyncLoadingCount = 0;
uint32 GSyncLoadCount = 0;

void ResetAsyncLoadingStats()
{
	check(IsInGameThread());
	GFlushAsyncLoadingTime = 0.0;
	GFlushAsyncLoadingCount = 0;
	GSyncLoadCount = 0;
}

int32 GWarnIfTimeLimitExceeded = 0;
static FAutoConsoleVariableRef CVarWarnIfTimeLimitExceeded(
	TEXT("s.WarnIfTimeLimitExceeded"),
	GWarnIfTimeLimitExceeded,
	TEXT("Enables log warning if time limit for time-sliced package streaming has been exceeded."),
	ECVF_Default
);

float GTimeLimitExceededMultiplier = 1.5f;
static FAutoConsoleVariableRef CVarTimeLimitExceededMultiplier(
	TEXT("s.TimeLimitExceededMultiplier"),
	GTimeLimitExceededMultiplier,
	TEXT("Multiplier for time limit exceeded warning time threshold."),
	ECVF_Default
);

float GTimeLimitExceededMinTime = 0.005f;
static FAutoConsoleVariableRef CVarTimeLimitExceededMinTime(
	TEXT("s.TimeLimitExceededMinTime"),
	GTimeLimitExceededMinTime,
	TEXT("Minimum time the time limit exceeded warning will be triggered by."),
	ECVF_Default
);

void IsTimeLimitExceededPrint(
	double InTickStartTime,
	double CurrentTime,
	double LastTestTime,
	double InTimeLimit,
	const TCHAR* InLastTypeOfWorkPerformed,
	UObject* InLastObjectWorkWasPerformedOn)
{
	static double LastPrintStartTime = -1.0;
	// Log single operations that take longer than time limit (but only in cooked builds)
	if (LastPrintStartTime != InTickStartTime &&
		(CurrentTime - InTickStartTime) > GTimeLimitExceededMinTime &&
		(CurrentTime - InTickStartTime) > (GTimeLimitExceededMultiplier * InTimeLimit))
	{
		double EstimatedTimeForThisStep = (CurrentTime - InTickStartTime) * 1000.0;
		if (LastTestTime > InTickStartTime)
		{
			EstimatedTimeForThisStep = (CurrentTime - LastTestTime) * 1000.0;
		}
		LastPrintStartTime = InTickStartTime;
		UE_LOGF(LogStreaming, Warning, "IsTimeLimitExceeded: %ls %ls Load Time %5.2fms   Last Step Time %5.2fms",
			InLastTypeOfWorkPerformed ? InLastTypeOfWorkPerformed : TEXT("unknown"),
			InLastObjectWorkWasPerformedOn ? *InLastObjectWorkWasPerformedOn->GetFullName() : TEXT("nullptr"),
			(CurrentTime - InTickStartTime) * 1000,
			EstimatedTimeForThisStep);
	}
}
