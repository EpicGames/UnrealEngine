// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/BasePackageWriter.h"

#include "Serialization/LargeMemoryWriter.h"

#if WITH_EDITOR
#include "HAL/FileManager.h"
#include "HAL/PlatformMisc.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/SavePackage.h"
#endif

namespace UE::PackageWriter::Private
{

TUniquePtr<FLargeMemoryWriter> BaseCreateLinkerArchive(FName PackageName,
	UObject* Asset, uint16 /*MultiOutputIndex*/)
{
	// The LargeMemoryWriter does not need to be persistent; the LinkerSave wraps it and reports Persistent=true
	bool bIsPersistent = false;
	return TUniquePtr<FLargeMemoryWriter>(new FLargeMemoryWriter(0, bIsPersistent, *PackageName.ToString()));
}

TUniquePtr<FLargeMemoryWriter> BaseCreateLinkerExportsArchive(FName PackageName,
	UObject* Asset, uint16 /*MultiOutputIndex*/)
{
	const bool bPersistent = true;
	return MakeUnique<FLargeMemoryWriter>(0, bPersistent, *PackageName.ToString());
}

#if WITH_EDITOR

static void WriteToFile(const FString& Filename, const FCompositeBuffer& Buffer)
{
	IFileManager& FileManager = IFileManager::Get();

	struct FFailureReason
	{
		uint32 LastErrorCode = 0;
		bool bSizeMatchFailed = false;
		int64 ExpectedSize = 0;
		int64 ActualSize = 0;
		bool bArchiveError = false;
	};
	TOptional<FFailureReason> FailureReason;

	for (int32 Tries = 0; Tries < 3; ++Tries)
	{
		FArchive* Ar = FileManager.CreateFileWriter(*Filename);
		if (!Ar)
		{
			if (!FailureReason)
			{
				FailureReason = FFailureReason{ FPlatformMisc::GetLastError(), false };
			}
			continue;
		}

		int64 DataSize = 0;
		for (const FSharedBuffer& Segment : Buffer.GetSegments())
		{
			int64 SegmentSize = static_cast<int64>(Segment.GetSize());
			Ar->Serialize(const_cast<void*>(Segment.GetData()), SegmentSize);
			DataSize += SegmentSize;
		}
		bool bArchiveError = Ar->IsError();
		delete Ar;

		int64 ActualSize = FileManager.FileSize(*Filename);
		if (ActualSize != DataSize)
		{
			if (!FailureReason)
			{
				FailureReason = FFailureReason{ 0, true, DataSize, ActualSize, bArchiveError };
			}
			FileManager.Delete(*Filename);
			continue;
		}
		return;
	}

	FString ReasonText;
	if (FailureReason && FailureReason->bSizeMatchFailed)
	{
		ReasonText = FString::Printf(TEXT("Unexpected file size. Tried to write %" INT64_FMT " but resultant size was %" INT64_FMT ".%s")
			TEXT(" Another operation is modifying the file, or the write operation failed to write completely."),
			FailureReason->ExpectedSize, FailureReason->ActualSize, FailureReason->bArchiveError ? TEXT(" Ar->Serialize failed.") : TEXT(""));
	}
	else if (FailureReason && FailureReason->LastErrorCode != 0)
	{
		TCHAR LastErrorText[1024];
		FPlatformMisc::GetSystemErrorMessage(LastErrorText, UE_ARRAY_COUNT(LastErrorText), FailureReason->LastErrorCode);
		ReasonText = LastErrorText;
	}
	else
	{
		ReasonText = TEXT("Unknown failure reason.");
	}
	UE_LOGF(LogSavePackage, Fatal, "SavePackage Async write %ls failed: %ls", *Filename, *ReasonText);
}

void HashAndWrite(const UE::PackageWriter::Private::FWriteFileData& FileData, FMD5& AccumulatedHash,
	const TRefCountPtr<FPackageHashes>& PackageHashes, IPackageWriter::EWriteOptions WriteOptions)
{
	//@todo: FH: Should we calculate the hash of both output, currently only the main package output hash is calculated
	if (EnumHasAnyFlags(WriteOptions, IPackageWriter::EWriteOptions::ComputeHash) && FileData.bContributeToHash)
	{
		for (const FSharedBuffer& Segment : FileData.Buffer.GetSegments())
		{
			AccumulatedHash.Update(static_cast<const uint8*>(Segment.GetData()), Segment.GetSize());
		}

		if (FileData.ChunkId.IsValid())
		{
			FBlake3 ChunkHash;
			for (const FSharedBuffer& Segment : FileData.Buffer.GetSegments())
			{
				ChunkHash.Update(static_cast<const uint8*>(Segment.GetData()), Segment.GetSize());
			}
			FIoHash FinalHash(ChunkHash.Finalize());
			PackageHashes->ChunkHashes.Add(FileData.ChunkId, FinalHash);
		}
	}

	if ((FileData.bIsSidecar && EnumHasAnyFlags(WriteOptions, IPackageWriter::EWriteOptions::WriteSidecars)) ||
		(!FileData.bIsSidecar && EnumHasAnyFlags(WriteOptions, IPackageWriter::EWriteOptions::WritePackage)))
	{
		const FString* WriteFilename = &FileData.Filename;
		FString FilenameBuffer;
		if (EnumHasAnyFlags(WriteOptions, IPackageWriter::EWriteOptions::SaveForDiff))
		{
			FilenameBuffer = FPaths::Combine(FPaths::GetPath(FileData.Filename),
				FPaths::GetBaseFilename(FileData.Filename) + TEXT("_ForDiff")
				+ FPaths::GetExtension(FileData.Filename, true));
			WriteFilename = &FilenameBuffer;
		}
		WriteToFile(*WriteFilename, FileData.Buffer);

		if (FileData.Regions.Num() > 0)
		{
			TArray<uint8> Memory;
			FMemoryWriter Ar(Memory);
			FFileRegion::SerializeFileRegions(Ar, const_cast<TArray<FFileRegion>&>(FileData.Regions));

			WriteToFile(*WriteFilename + FFileRegion::RegionsFileExtension,
				FCompositeBuffer(FSharedBuffer::MakeView(Memory.GetData(), Memory.Num())));
		}
	}
}

#endif

} // UE::PackageWriter::Private

TUniquePtr<FLargeMemoryWriter> FBasePackageWriter::CreateLinkerArchive(FName PackageName,
	UObject* Asset, uint16 MultiOutputIndex)
{
	return UE::PackageWriter::Private::BaseCreateLinkerArchive(PackageName, Asset, MultiOutputIndex);
}

TUniquePtr<FLargeMemoryWriter> FBasePackageWriter::CreateLinkerExportsArchive(FName PackageName,
	UObject* Asset, uint16 MultiOutputIndex)
{
	return UE::PackageWriter::Private::BaseCreateLinkerExportsArchive(PackageName, Asset, MultiOutputIndex);
}

TUniquePtr<FLargeMemoryWriter> FBaseCookedPackageWriter::CreateLinkerArchive(FName PackageName,
	UObject* Asset, uint16 MultiOutputIndex)
{
	return UE::PackageWriter::Private::BaseCreateLinkerArchive(PackageName, Asset, MultiOutputIndex);
}

TUniquePtr<FLargeMemoryWriter> FBaseCookedPackageWriter::CreateLinkerExportsArchive(FName PackageName,
	UObject* Asset, uint16 MultiOutputIndex)
{
	return UE::PackageWriter::Private::BaseCreateLinkerExportsArchive(PackageName, Asset, MultiOutputIndex);
}
