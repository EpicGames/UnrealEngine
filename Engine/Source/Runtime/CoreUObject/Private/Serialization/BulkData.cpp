// Copyright Epic Games, Inc. All Rights Reserved.
#include "Serialization/BulkData.h"

#include "Async/MappedFileHandle.h"
#include "HAL/IConsoleManager.h"
#include "IO/IoDispatcher.h"
#include "IO/IoOffsetLength.h"
#include "Math/GuardedInt.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "Misc/SourceLocation.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Templates/Casts.h"
#include "UObject/Package.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerSave.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/PackageResourceIoDispatcherBackend.h"

/** Whether to track information of how bulk data is being used */
#define TRACK_BULKDATA_USE 0

#if TRACK_BULKDATA_USE
#include "Misc/ScopeLock.h"
#endif // TRACK_BULKDATA_USE

// If set to 0 then we will pretend that optional data does not exist, useful for testing.
#define ALLOW_OPTIONAL_DATA 1

//////////////////////////////////////////////////////////////////////////////

const FBulkDataCookedIndex FBulkDataCookedIndex::Default;

//////////////////////////////////////////////////////////////////////////////

FStringBuilderBase& LexToString(EBulkDataFlags Flags, FStringBuilderBase& Sb)
{
	#define TEST_AND_ADD_FLAG(Sb, Flags, Contains)\
	{\
		if ((uint32(Flags) & uint32(Contains)) == uint32(Contains))\
		{\
			if (Sb.Len())\
			{\
				Sb.Append(TEXT("|"));\
			}\
			Sb.Append(TEXT(#Contains));\
		}\
	}

	if (uint32(Flags) == BULKDATA_None)
	{
		Sb.Append("BULKDATA_None");
		return Sb;
	}

	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_PayloadAtEndOfFile);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_SerializeCompressedZLIB);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_ForceSingleElementSerialization);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_SingleUse);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_ForceInlinePayload);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_SerializeCompressed);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_PayloadInSeparateFile);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_Force_NOT_InlinePayload);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_OptionalPayload);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_MemoryMappedPayload);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_Size64Bit);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_DuplicateNonOptionalPayload);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_NoOffsetFixUp);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_WorkspaceDomainPayload);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_LazyLoadable);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_UsesIoDispatcher);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_DataIsMemoryMapped);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_HasAsyncReadPending);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_AlwaysAllowDiscard);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_BadDataVersion);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_ForceStreamPayload);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_SerializeCompressedBitWindow);
	TEST_AND_ADD_FLAG(Sb, Flags, BULKDATA_Unused);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return Sb;
}

FString LexToString(EBulkDataFlags Flags)
{
	TStringBuilder<256> Sb;
	return LexToString(Flags, Sb).ToString();
}

EIoChunkType GetIoChunkTypeFromFlags(EBulkDataFlags Flags)
{
	if (!FBulkData::HasFlags(Flags, BULKDATA_PayloadAtEndOfFile))
	{
		return EIoChunkType::ExportBundleData;
	}

	if (FBulkData::HasFlags(Flags, BULKDATA_OptionalPayload))
	{
		return EIoChunkType::OptionalBulkData;
	}

	if (FBulkData::HasFlags(Flags, BULKDATA_MemoryMappedPayload))
	{
		return EIoChunkType::MemoryMappedBulkData;
	}

	return EIoChunkType::BulkData;
}

const FIoFilenameHash FALLBACK_IO_FILENAME_HASH = INVALID_IO_FILENAME_HASH - 1;

FIoFilenameHash MakeIoFilenameHash(const FPackagePath& PackagePath)
{
	if (!PackagePath.IsEmpty())
	{
		FString BaseFileName = PackagePath.GetLocalBaseFilenameWithPath().ToLower();
		const FIoFilenameHash Hash = FCrc::StrCrc32<TCHAR>(*BaseFileName);
		return Hash != INVALID_IO_FILENAME_HASH ? Hash : FALLBACK_IO_FILENAME_HASH;
	}
	else
	{
		return INVALID_IO_FILENAME_HASH;
	}
}

FIoFilenameHash MakeIoFilenameHash(const FString& Filename)
{
	if (!Filename.IsEmpty())
	{
		FString BaseFileName = FPaths::GetBaseFilename(Filename).ToLower();
		const FIoFilenameHash Hash = FCrc::StrCrc32<TCHAR>(*BaseFileName);
		return Hash != INVALID_IO_FILENAME_HASH ? Hash : FALLBACK_IO_FILENAME_HASH;
	}
	else
	{
		return INVALID_IO_FILENAME_HASH;
	}
}

FIoFilenameHash MakeIoFilenameHash(const FIoChunkId& ChunkID)
{
	if (ChunkID.IsValid())
	{
		const FIoFilenameHash Hash = GetTypeHash(ChunkID);
		return Hash != INVALID_IO_FILENAME_HASH ? Hash : FALLBACK_IO_FILENAME_HASH;
	}
	else
	{
		return INVALID_IO_FILENAME_HASH;
	}
}

//////////////////////////////////////////////////////////////////////////////

namespace UE::BulkData::Private
{

/** Open the bulk data chunk fo reading. */
bool OpenReadBulkData(
	const FBulkMetaData& BulkMeta,
	const FIoChunkId& BulkChunkId,
	int64 Offset,
	int64 Size,
	EAsyncIOPriorityAndFlags Priority,
	TFunction<void(FArchive& Ar)>&& Read);

/** Open async read file handle for the specified bulk data chunk ID. */
TUniquePtr<IAsyncReadFileHandle> OpenAsyncReadBulkData(
	const FBulkMetaData& BulkMeta,
	const FIoChunkId& BulkChunkId,
	uint64 ChunkSize,
	uint64 AvailableChunkSize);

/** Create bulk data streaming request. */
TUniquePtr<IBulkDataIORequest> CreateStreamingRequest(
	const FBulkMetaData& BulkMeta,
	const FIoChunkId& BulkChunkId,
	int64 Offset,
	int64 Size,
	EAsyncIOPriorityAndFlags Priority,
	FBulkDataIORequestCallBack* CompleteCallback,
	uint8* UserSuppliedMemory);

/** Try memory map the chunk specified by the bulk data ID. */
bool TryMemoryMapBulkData(
	const FBulkMetaData& BulkMeta,
	const FIoChunkId& BulkChunkId,
	int64 Offset,
	int64 Size,
	FIoMappedRegion& OutRegion);

//////////////////////////////////////////////////////////////////////////////

static FArchive& SerializeAsInt32(FArchive& Ar, int64& Value)
{
	check(!Ar.IsSaving() || (MIN_int32 <= Value && Value <= MAX_int32));
	int32 ValueAsInt32 = static_cast<int32>(Value);
	Ar << ValueAsInt32;
	Value = ValueAsInt32;

	return Ar;
}

//////////////////////////////////////////////////////////////////////////////

FArchive& operator<<(FArchive& Ar, FBulkMetaResource& BulkMeta)
{
	if (Ar.IsSaving() && BulkMeta.SizeOnDisk >= (1LL << 31))
	{
		FBulkData::SetBulkDataFlagsOn(BulkMeta.Flags, BULKDATA_Size64Bit);
	}

	Ar << BulkMeta.Flags;

	if (UNLIKELY(BulkMeta.Flags & BULKDATA_Size64Bit))
	{
		Ar << BulkMeta.ElementCount;
		Ar << BulkMeta.SizeOnDisk;
		Ar << BulkMeta.Offset;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (UNLIKELY(BulkMeta.Flags & BULKDATA_BadDataVersion))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			if (Ar.IsLoading())
			{
				uint16 DummyValue;
				Ar << DummyValue;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				BulkMeta.Flags = static_cast<EBulkDataFlags>(BulkMeta.Flags & ~BULKDATA_BadDataVersion);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		if (BulkMeta.Flags & BULKDATA_DuplicateNonOptionalPayload)
		{
			Ar << BulkMeta.DuplicateFlags;
			Ar << BulkMeta.DuplicateSizeOnDisk;
			Ar << BulkMeta.DuplicateOffset;
		}
	}
	else
	{
		SerializeAsInt32(Ar, BulkMeta.ElementCount);
		SerializeAsInt32(Ar, BulkMeta.SizeOnDisk);
		Ar << BulkMeta.Offset;
		
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (UNLIKELY(BulkMeta.Flags & BULKDATA_BadDataVersion))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			if (Ar.IsLoading())
			{
				uint16 DummyValue;
				Ar << DummyValue;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
				BulkMeta.Flags = static_cast<EBulkDataFlags>(BulkMeta.Flags & ~BULKDATA_BadDataVersion);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
		
		if (BulkMeta.Flags & BULKDATA_DuplicateNonOptionalPayload)
		{
			Ar << BulkMeta.DuplicateFlags;
			SerializeAsInt32(Ar, BulkMeta.DuplicateSizeOnDisk);
			Ar << BulkMeta.DuplicateOffset;
		}
	}

	return Ar;
}

/** 
 * Attempt to return the best debug name we can from a given archive.
 * First we check to see the archive has a FLinkerLoad from which we
 * can find the name of the package being loaded from. If not we 
 * return an unknown string as the FArchive name is often not very 
 * useful for identification purposes.
 */
FString GetDebugNameFromArchive(FArchive& Ar)
{
	if (FLinkerLoad* LinkerLoad = Cast<FLinkerLoad>(Ar.GetLinker()))
	{
		return LinkerLoad->GetDebugName();
	}
	else
	{
		return TEXT("Unknown");
	}
}

bool FBulkMetaData::FromSerialized(FArchive& Ar, int64 ElementSize, FBulkMetaData& OutMetaData, int64& OutDuplicateOffset)
{
	check(Ar.IsLoading());

	// Ensure all fields are initialized.
	//
	// Note that this will reset the lock status as well, so callers may want to first check that 
	// the bulk data is unlocked before the lock state is lost.
	OutMetaData = FBulkMetaData();

	if (Ar.IsError())
	{
		return false;
	}

	FBulkMetaResource Resource;
	Ar << Resource;

	if (Ar.IsError())
	{
		// Note that setting the error flag on the archive is not enough to stop the package from being loaded so for now we 
		// need to fatal error to prevent the process from continuing to use the corrupted package.
		UE_LOGF(LogSerialization, Fatal, "Bulkdata error when serializing '%ls', could not serialize FBulkMetaResource correctly", *GetDebugNameFromArchive(Ar));
		return false;
	}

	if (Resource.ElementCount > 0)
	{
		const FGuardedInt64 MetadataSize = FGuardedInt64(Resource.ElementCount) * ElementSize;
		if (MetadataSize.IsValid())
		{
			OutMetaData.SetSize(MetadataSize.Get(0));
		}
		else
		{
			// We should only get here if the package is severely corrupted (which should get detected earlier in package loading)
			// Note that setting the error flag on the archive is not enough to stop the package from being loaded so for now we 
			// need to fatal error to prevent the process from continuing to use the corrupted package.
			Ar.SetError();

			UE_LOG(LogSerialization, Fatal,
				TEXT("Bulkdata error when serializing '%s', ElementCount (%" INT64_FMT ") and ElementSize (%" INT64_FMT ") would cause an int64 overflow"),
				*GetDebugNameFromArchive(Ar),
				Resource.ElementCount,
				ElementSize);
			
			return false;
		}
	}

	OutMetaData.SetSizeOnDisk(Resource.SizeOnDisk);
	OutMetaData.SetOffset(Resource.Offset);
	OutMetaData.SetFlags(Resource.Flags);

	check(Resource.ElementCount < 0 || OutMetaData.GetSize() == Resource.ElementCount * ElementSize);
	check(OutMetaData.GetOffset() == Resource.Offset);
	check(OutMetaData.GetFlags() == Resource.Flags);

#if !USE_RUNTIME_BULKDATA
	check(Resource.ElementCount < 0 || OutMetaData.GetSizeOnDisk() == Resource.SizeOnDisk);
#endif

	OutDuplicateOffset = Resource.DuplicateOffset;

	return true;
}

FIoOffsetAndLength FBulkMetaData::GetOffsetAndLength() const
{
	const uint64 Offset = GetOffset();
	const uint64 Size = GetSize();

	return Offset >= 0 && Size > 0
		? FIoOffsetAndLength(static_cast<int64>(Offset), static_cast<int64>(Size))
		: FIoOffsetAndLength();
}

} // namespace UE::BulkData::Private

/*-----------------------------------------------------------------------------
	Memory management
-----------------------------------------------------------------------------*/

void FBulkData::FAllocatedPtr::Free(FBulkData* Owner)
{
	if (!Owner->IsDataMemoryMapped())
	{
		FMemory::Free(Allocation.RawData);
		Allocation.RawData = nullptr;
	}
	else
	{
		delete Allocation.MemoryMappedData;
		Allocation.MemoryMappedData = nullptr;
	}
}

void* FBulkData::FAllocatedPtr::ReallocateData(FBulkData* Owner, int64 SizeInBytes)
{
	checkf(!Owner->IsDataMemoryMapped(),  TEXT("Trying to reallocate a memory mapped BulkData object without freeing it first!"));

	Allocation.RawData = FMemory::Realloc(Allocation.RawData, SizeInBytes, DEFAULT_ALIGNMENT);

	return Allocation.RawData;
}

void FBulkData::FAllocatedPtr::SetData(FBulkData* Owner, void* Buffer)
{
	checkf(Allocation.RawData == nullptr, TEXT("Trying to assign a BulkData object without freeing it first!"));

	Allocation.RawData = Buffer;
}

void FBulkData::FAllocatedPtr::SetMemoryMappedData(FBulkData* Owner, IMappedFileHandle* MappedHandle, IMappedFileRegion* MappedRegion)
{
	checkf(Allocation.MemoryMappedData == nullptr, TEXT("Trying to assign a BulkData object without freeing it first!"));
	Allocation.MemoryMappedData = new FOwnedBulkDataPtr(MappedHandle, MappedRegion);

	Owner->SetBulkDataFlags(BULKDATA_DataIsMemoryMapped);
}

void* FBulkData::FAllocatedPtr::GetAllocationForWrite(const FBulkData* Owner) const
{
	if (!Owner->IsDataMemoryMapped())
	{
		return Allocation.RawData;
	}
	else
	{
		return nullptr;
	}
}

const void* FBulkData::FAllocatedPtr::GetAllocationReadOnly(const FBulkData* Owner) const
{
	if (!Owner->IsDataMemoryMapped())
	{
		return Allocation.RawData;
	}
	else if (Allocation.MemoryMappedData != nullptr)
	{
		return Allocation.MemoryMappedData->GetPointer();
	}
	else
	{
		return nullptr;
	}
}

FOwnedBulkDataPtr* FBulkData::FAllocatedPtr::StealFileMapping(FBulkData* Owner)
{
	FOwnedBulkDataPtr* Ptr;
	if (!Owner->IsDataMemoryMapped())
	{
		Ptr = new FOwnedBulkDataPtr(Allocation.RawData);
	}
	else
	{
		Ptr = Allocation.MemoryMappedData;
		Owner->ClearBulkDataFlags(BULKDATA_DataIsMemoryMapped);
	}	

	Allocation.RawData = nullptr;

	return Ptr;
}

void FBulkData::FAllocatedPtr::Swap(FBulkData* Owner, void** DstBuffer)
{
	if (!Owner->IsDataMemoryMapped())
	{
		::Swap(*DstBuffer, Allocation.RawData);
	}
	else
	{
		const int64 DataSize = Owner->GetBulkDataSize();

		*DstBuffer = FMemory::Malloc(DataSize, DEFAULT_ALIGNMENT);
		FMemory::Memcpy(*DstBuffer, Allocation.MemoryMappedData->GetPointer(), DataSize);

		delete Allocation.MemoryMappedData;
		Allocation.MemoryMappedData = nullptr;

		Owner->ClearBulkDataFlags(BULKDATA_DataIsMemoryMapped);
	}	
}

/*-----------------------------------------------------------------------------
	Constructors and operators
-----------------------------------------------------------------------------*/

DECLARE_STATS_GROUP(TEXT("Bulk Data"), STATGROUP_BulkData, STATCAT_Advanced);

#if TRACK_BULKDATA_USE

/** Simple wrapper for tracking the bulk data usage in the thread-safe way. */
struct FThreadSafeBulkDataToObjectMap
{
	static FThreadSafeBulkDataToObjectMap& Get()
	{
		static FThreadSafeBulkDataToObjectMap Instance;
		return Instance;
	}

	void Add( FBulkData* Key, UObject* Value )
	{
		FScopeLock ScopeLock(&CriticalSection);
		BulkDataToObjectMap.Add(Key,Value);
	}

	void Remove( FBulkData* Key )
	{
		FScopeLock ScopeLock(&CriticalSection);
		BulkDataToObjectMap.Remove(Key);
	}

	FCriticalSection& GetLock() 
	{
		return CriticalSection;
	}

	const TMap<FBulkData*,UObject*>::TConstIterator GetIterator() const
	{
		return BulkDataToObjectMap.CreateConstIterator();
	}

protected:
	/** Map from bulk data pointer to object it is contained by */
	TMap<FBulkData*,UObject*> BulkDataToObjectMap;

	/** CriticalSection. */
	FCriticalSection CriticalSection;
};

/**
 * Helper structure associating an object and a size for sorting purposes.
 */
struct FObjectAndSize
{
	FObjectAndSize( const UObject* InObject, int64 InSize )
	:	Object( InObject )
	,	Size( InSize )
	{}

	/** Object associated with size. */
	const UObject*	Object;
	/** Size associated with object. */
	int64			Size;
};

/** Hash function required for TMap support */
uint32 GetTypeHash( const FBulkData* BulkData )
{
	return PointerHash(BulkData);
}

#endif

FOwnedBulkDataPtr::~FOwnedBulkDataPtr()
{
	if (AllocatedData)
	{
		FMemory::Free(AllocatedData);
	}
	else
	{
		if (MappedRegion || MappedHandle)
		{
			delete MappedRegion;
			delete MappedHandle;
		}
	}
}

const void* FOwnedBulkDataPtr::GetPointer()
{
	// return the pointer that the caller can use
	return AllocatedData ? AllocatedData : (MappedRegion ? MappedRegion->GetMappedPtr() : nullptr);
}

/**
 * Copy constructor. Use the common routine to perform the copy.
 *
 * @param Other the source array to copy
 */
FBulkData::FBulkData( const FBulkData& Other )
{
	*this = Other;

#if TRACK_BULKDATA_USE
	FThreadSafeBulkDataToObjectMap::Get().Add( this, NULL );
#endif
}

/**
 * Virtual destructor, free'ing allocated memory.
 */
FBulkData::~FBulkData()
{
	check(IsUnlocked());

	// Free memory.
	FreeData();

#if WITH_EDITOR
	// Detach from archive.
	if( AttachedAr )
	{
		AttachedAr->DetachBulkData( this, false );
		check( AttachedAr == nullptr );
	}
#endif // WITH_EDITOR

#if TRACK_BULKDATA_USE
	FThreadSafeBulkDataToObjectMap::Get().Remove( this );
#endif
}

/**
 * Copies the source array into this one after detaching from archive.
 *
 * @param Other the source array to copy
 */
FBulkData& FBulkData::operator=( const FBulkData& Other )
{
	checkf(IsUnlocked(), TEXT("Attempting to modify a BulkData object that is locked"));
	checkf(Other.BulkMeta.GetLockStatus() != LOCKSTATUS_ReadWriteLock, TEXT("Attempting to read from a BulkData object that is locked for write"));

	RemoveBulkData();
	
	BulkMeta = Other.BulkMeta;
	BulkChunkId = Other.BulkChunkId;

	if (!Other.IsDataMemoryMapped() || !Other.IsInSeparateFile())
	{
		if (const void* Src = Other.GetDataBufferReadOnly())
		{
			void* Dst = ReallocateData(GetBulkDataSize());
			FMemory::Memcpy(Dst, Src, GetBulkDataSize());
		}
	}
	else
	{
		FIoMappedRegion MappedRegion;
		if (UE::BulkData::Private::TryMemoryMapBulkData(BulkMeta, BulkChunkId, BulkMeta.GetOffset(), BulkMeta.GetSize(), MappedRegion))
		{
			DataAllocation.SetMemoryMappedData(this, MappedRegion.MappedFileHandle, MappedRegion.MappedFileRegion);
		}
	}

	return *this;
}

/*-----------------------------------------------------------------------------
	Static functions.
-----------------------------------------------------------------------------*/

/**
 * Dumps detailed information of bulk data usage.
 *
 * @param Log FOutputDevice to use for logging
 */
void FBulkData::DumpBulkDataUsage( FOutputDevice& Log )
{
#if TRACK_BULKDATA_USE
	// Arrays about to hold per object and per class size information.
	TArray<FObjectAndSize> PerObjectSizeArray;
	TArray<FObjectAndSize> PerClassSizeArray;

	{
		FScopeLock Lock(&FThreadSafeBulkDataToObjectMap::Get().GetLock());

		// Iterate over all "live" bulk data and add size to arrays if it is loaded.
		for( auto It(FThreadSafeBulkDataToObjectMap::Get().GetIterator()); It; ++It )
		{
			const FBulkData*		BulkData	= It.Key();
			const UObject*			Owner		= It.Value();
			// Only add bulk data that is consuming memory to array.
			if( Owner && BulkData->IsBulkDataLoaded() && BulkData->GetBulkDataSize() > 0 )
			{
				// Per object stats.
				PerObjectSizeArray.Add( FObjectAndSize( Owner, BulkData->GetBulkDataSize() ) );

				// Per class stats.
				bool bFoundExistingPerClassSize = false;
				// Iterate over array, trying to find existing entry.
				for( int32 PerClassIndex=0; PerClassIndex<PerClassSizeArray.Num(); PerClassIndex++ )
				{
					FObjectAndSize& PerClassSize = PerClassSizeArray[ PerClassIndex ];
					// Add to existing entry if found.
					if( PerClassSize.Object == Owner->GetClass() )
					{
						PerClassSize.Size += BulkData->GetBulkDataSize();
						bFoundExistingPerClassSize = true;
						break;
					}
				}
				// Add new entry if we didn't find an existing one.
				if( !bFoundExistingPerClassSize )
				{
					PerClassSizeArray.Add( FObjectAndSize( Owner->GetClass(), BulkData->GetBulkDataSize() ) );
				}
			}
		}
	}

	/** Compare operator, sorting by size in descending order */
	struct FCompareFObjectAndSize
	{
		FORCEINLINE bool operator()( const FObjectAndSize& A, const FObjectAndSize& B ) const
		{
			return B.Size < A.Size;
		}
	};

	// Sort by size.
	PerObjectSizeArray.Sort( FCompareFObjectAndSize() );
	PerClassSizeArray.Sort( FCompareFObjectAndSize() );

	// Log information.
	UE_LOGF(LogSerialization, Log, "");
	UE_LOGF(LogSerialization, Log, "Per class summary of bulk data use:");
	for( int32 PerClassIndex=0; PerClassIndex<PerClassSizeArray.Num(); PerClassIndex++ )
	{
		const FObjectAndSize& PerClassSize = PerClassSizeArray[ PerClassIndex ];
		Log.Logf( TEXT("  %5lld KByte of bulk data for Class %s"), PerClassSize.Size / 1024, *PerClassSize.Object->GetPathName() );
	}
	UE_LOGF(LogSerialization, Log, "");
	UE_LOGF(LogSerialization, Log, "Detailed per object stats of bulk data use:");
	for( int32 PerObjectIndex=0; PerObjectIndex<PerObjectSizeArray.Num(); PerObjectIndex++ )
	{
		const FObjectAndSize& PerObjectSize = PerObjectSizeArray[ PerObjectIndex ];
		Log.Logf( TEXT("  %5lld KByte of bulk data for %s"), PerObjectSize.Size / 1024, *PerObjectSize.Object->GetFullName() );
	}
	UE_LOGF(LogSerialization, Log, "");
#else
	UE_LOGF(LogSerialization, Log, "Please recompiled with TRACK_BULKDATA_USE set to 1 in UnBulkData.cpp.");
#endif
}


/*-----------------------------------------------------------------------------
	Accessors.
-----------------------------------------------------------------------------*/

/**
 * Returns the size of the bulk data in bytes.
 *
 * @return Size of the bulk data in bytes
 */
int64 FBulkData::GetBulkDataSize() const
{
	return BulkMeta.GetSize();
}

/**
 * Returns the size of the bulk data on disk. This can differ from GetBulkDataSize if
 * BULKDATA_SerializeCompressed is set.
 *
 * @return Size of the bulk data on disk or INDEX_NONE in case there's no association
 */
int64 FBulkData::GetBulkDataSizeOnDisk() const
{
	return BulkMeta.GetSizeOnDisk();
}
/**
 * Returns the offset into the file the bulk data is located at.
 *
 * @return Offset into the file or INDEX_NONE in case there is no association
 */
int64 FBulkData::GetBulkDataOffsetInFile() const
{
	return BulkMeta.GetOffset();
}
/**
 * Returns whether the bulk data is stored compressed on disk.
 *
 * @return true if data is compressed on disk, false otherwise
 */
bool FBulkData::IsStoredCompressedOnDisk() const
{
	return BulkMeta.HasAnyFlags(BULKDATA_SerializeCompressed);
}

bool FBulkData::CanLoadFromDisk() const
{
#if WITH_EDITOR
	return AttachedAr != nullptr || BulkChunkId.IsValid();
#else
	return BulkChunkId.IsValid();
#endif // WITH_EDITOR
}

bool FBulkData::DoesExist() const
{
#if !ALLOW_OPTIONAL_DATA
	if (IsOptional())
	{
		return false;
	}
#endif
	return FIoDispatcher::Get().DoesChunkExist(BulkChunkId, BulkMeta.GetOffsetAndLength());
}

/**
 * Returns flags usable to decompress the bulk data
 * 
 * @return COMPRESS_NONE if the data was not compressed on disk, or valid flags to pass to FCompression::UncompressMemory for this data
 */
FName FBulkData::GetDecompressionFormat() const
{
	return GetDecompressionFormat(BulkMeta.GetFlags());
}

FName FBulkData::GetDecompressionFormat(EBulkDataFlags InFlags)
{
	return (InFlags & BULKDATA_SerializeCompressedZLIB) ? NAME_Zlib : NAME_None;
}

/**
 * Returns whether the bulk data is currently loaded and resident in memory.
 *
 * @return true if bulk data is loaded, false otherwise
 */
bool FBulkData::IsBulkDataLoaded() const
{
	return DataAllocation.IsLoaded();
}

/*-----------------------------------------------------------------------------
	Data retrieval and manipulation.
-----------------------------------------------------------------------------*/

/**
 * Retrieves a copy of the bulk data.
 *
 * @param Dest [in/out] Pointer to pointer going to hold copy, can point to NULL pointer in which case memory is allocated
 * @param bDiscardInternalCopy Whether to discard/ free the potentially internally allocated copy of the data
 */
void FBulkData::GetCopy( void** Dest, bool bDiscardInternalCopy )
{
	check( IsUnlocked() );
	check( Dest );

	const int64 BulkDataSize = GetBulkDataSize();

	// Passed in memory is going to be used.
	if( *Dest )
	{
		// The data is already loaded so we can simply use a mempcy.
		if( IsBulkDataLoaded() )
		{
			// Copy data into destination memory.
			FMemory::Memcpy( *Dest, GetDataBufferReadOnly(), BulkDataSize );
			// Discard internal copy if wanted and we're still attached to an archive or if we're
			// single use bulk data.
			if( bDiscardInternalCopy && CanDiscardInternalData() )
			{
#if USE_RUNTIME_BULKDATA && !WITH_LOW_LEVEL_TESTS 
				UE_LOGF(LogSerialization, Warning, "FBulkData::GetCopy both copied and discarded it's data, passing in an empty pointer would avoid an extra allocate and memcpy!");
#endif
				FreeData();
			}
		}
		// Data isn't currently loaded so we need to load it from disk.
		else
		{
			FIoBuffer Buffer(FIoBuffer::Wrap, *Dest, BulkDataSize);
			TryLoadDataIntoMemory(Buffer);
		}
	}
	// Passed in memory is NULL so we need to allocate some.
	else
	{
		// The data is already loaded so we can simply use a mempcy.
		if (IsBulkDataLoaded())
		{
			// If the internal copy should be discarded and we are still attached to an archive we can
			// simply "return" the already existing copy and NULL out the internal reference. We can
			// also do this if the data is single use like e.g. when uploading texture data.
			if (bDiscardInternalCopy && CanDiscardInternalData())
			{
				DataAllocation.Swap(this, Dest);
			}
			// Can't/ Don't discard so we need to allocate and copy.
			else
			{
				if (BulkDataSize != 0)
				{
					// Allocate enough memory for data...
					*Dest = FMemory::Malloc(BulkDataSize, GetBulkDataAlignment());

					// ... and copy it into memory now pointed to by out parameter.
					FMemory::Memcpy( *Dest, GetDataBufferReadOnly(), BulkDataSize );
				}
				else
				{
					*Dest = nullptr;
				}
			}
		}
		// Data isn't currently loaded so we need to load it from disk.
		else
		{
			if (BulkDataSize != 0)
			{
				FIoBuffer Buffer(BulkDataSize);
				if (TryLoadDataIntoMemory(Buffer))
				{
					*Dest = Buffer.Release().ConsumeValueOrDie();
				}
			}
			else
			{
				*Dest = nullptr;
			}
		}
	}
}

/**
 * Locks the bulk data and returns a pointer to it.
 *
 * @param	LockFlags	Flags determining lock behavior
 */
void* FBulkData::Lock(uint32 LockFlags, UE::FSourceLocation Location)
{
	check(IsUnlocked());
	
	// Make sure bulk data is loaded.
	MakeSureBulkDataIsLoaded();

#if WITH_EDITOR && DO_CHECK
	LockSourceLocation.Emplace(Location);
#endif
		
	// Read-write operations are allowed on returned memory.
	if( LockFlags & LOCK_READ_WRITE )
	{
#if WITH_EDITOR
		// We need to detach from the archive to not be able to clobber changes by serializing
		// over them.
		if( AttachedAr )
		{
			// Detach bulk data. This will call DetachFromArchive which in turn will clear AttachedAr.
			AttachedAr->DetachBulkData( this, false );
			check( AttachedAr == nullptr );
		}
#endif // WITH_EDITOR
		// This has to be set after the DetachBulkData because we can't detach a locked bulkdata
		BulkMeta.SetLockStatus(LOCKSTATUS_ReadWriteLock);
		ClearBulkDataFlags(BULKDATA_LazyLoadable);
		return GetDataBufferForWrite();
	}
	else if( LockFlags & LOCK_READ_ONLY )
	{
		BulkMeta.SetLockStatus(LOCKSTATUS_ReadOnlyLock);
		return (void*)GetDataBufferReadOnly(); // Cast the const away, icky but our hands are tied by the original API at this time
	}
	else
	{
		UE_LOGF(LogSerialization, Fatal, "Unknown lock flag '%u'", LockFlags);
		return nullptr;
	}
}

const void* FBulkData::LockReadOnly(UE::FSourceLocation Location) const
{
	check(IsUnlocked());
	
	FBulkData* mutable_this = const_cast<FBulkData*>(this);

	// Make sure bulk data is loaded.
	mutable_this->MakeSureBulkDataIsLoaded();

	// Only read operations are allowed on returned memory.
	mutable_this->BulkMeta.SetLockStatus(LOCKSTATUS_ReadOnlyLock);
#if WITH_EDITOR && DO_CHECK
	mutable_this->LockSourceLocation.Emplace(Location);
#endif

	return GetDataBufferReadOnly();
}

void* FBulkData::Realloc(int64 ElementCount, int64 ElementSize)
{
	checkf(IsLocked(), TEXT("BulkData must be locked for 'write' before reallocating!"));
#if !WITH_EDITOR
	checkf(!CanLoadFromDisk(), TEXT("Cannot re-allocate a FBulkDataBase object that represents a file on disk!"));
#endif //!WITH_EDITOR

	// We might want to consider this a valid use case if anyone can come up with one?
	checkf(!IsUsingIODispatcher(), TEXT("Attempting to re-allocate data loaded from the IoDispatcher"));

	const int64 BulkDataSize = ElementCount * ElementSize;
	check(BulkDataSize <= MaxBulkDataSize);
	BulkMeta.SetSize(BulkDataSize);
	ReallocateData(BulkDataSize);

	return GetDataBufferForWrite();
}

/** 
 * Unlocks bulk data after which point the pointer returned by Lock no longer is valid.
 */
void FBulkData::Unlock() const
{
	check(IsUnlocked() == false);

	FBulkData* mutable_this = const_cast<FBulkData*>(this);

	mutable_this->BulkMeta.SetLockStatus(LOCKSTATUS_Unlocked);
#if WITH_EDITOR && DO_CHECK
	mutable_this->LockSourceLocation.Reset();
#endif

	// Free pointer if we're guaranteed to only to access the data once.
	if (BulkMeta.HasAnyFlags(BULKDATA_SingleUse))
	{
		mutable_this->FreeData();
	}
}

/**
 * Clears/ removes the bulk data and resets element count to 0.
 */
void FBulkData::RemoveBulkData()
{
	check(IsUnlocked());

#if WITH_EDITOR
	// Detach from archive without loading first.
	if( AttachedAr )
	{
		AttachedAr->DetachBulkData( this, false );
		check( AttachedAr == nullptr );
	}
#endif // WITH_EDITOR
	
	// Resize to 0 elements.
	BulkMeta.SetSize(0);
	FreeData();
	ClearBulkDataFlags(BULKDATA_LazyLoadable);
}

/**
 * Deallocates bulk data without detaching the archive.
 */
bool FBulkData::UnloadBulkData()
{
#if WITH_EDITOR
	if (IsUnlocked())
	{
		FreeData();
		return true;
	}
#endif
	return false;
}

#if !USE_RUNTIME_BULKDATA
/**
* Load the bulk data using a file reader. Works when no archive is attached to the bulk data.
* @return Whether the operation succeeded.
*/
bool FBulkData::LoadBulkDataWithFileReader()
{
#if WITH_EDITOR
	if (IsBulkDataLoaded() == false && CanLoadBulkDataWithFileReader())
	{
		const int64 BulkDataOffset = GetBulkDataOffsetInFile();
		const int64 BulkDataSizeOnDisk = GetBulkDataSizeOnDisk();

		const bool bOk = UE::BulkData::Private::OpenReadBulkData(
			BulkMeta,
			BulkChunkId,
			BulkDataOffset,
			BulkDataSizeOnDisk,
			AIOP_High,
			[this](FArchive& Ar)
			{
				const int64 BulkDataSize = GetBulkDataSize();
				void* DataBuffer = ReallocateData(BulkDataSize);
				SerializeBulkData(Ar, DataBuffer, BulkDataSize, BulkMeta.GetFlags());
			});

		return bOk;
	}
#endif
	return false;
}

bool FBulkData::CanLoadBulkDataWithFileReader() const
{
#if WITH_EDITOR
	return BulkChunkId.IsValid();
#else
	return false;
#endif
}
#endif // !USE_RUNTIME_BULKDATA

/**
 * Forces the bulk data to be resident in memory and detaches the archive.
 */
void FBulkData::ForceBulkDataResident()
{
	// Make sure bulk data is loaded.
	MakeSureBulkDataIsLoaded();

#if WITH_EDITOR
	// Detach from the archive 
	if( AttachedAr )
	{
		// Detach bulk data. This will call DetachFromArchive which in turn will clear AttachedAr.
		AttachedAr->DetachBulkData( this, false );
		check( AttachedAr == nullptr );
	}
#endif // WITH_EDITOR
}

/**
 * Sets the passed in bulk data flags.
 *
 * @param BulkDataFlagsToSet	Bulk data flags to set
 */
void FBulkData::SetBulkDataFlags( uint32 BulkDataFlagsToSet )
{
	BulkMeta.SetFlags(static_cast<EBulkDataFlags>(BulkMeta.GetFlags() | BulkDataFlagsToSet));
}

void FBulkData::ResetBulkDataFlags(uint32 BulkDataFlagsToSet)
{
	BulkMeta.SetFlags(static_cast<EBulkDataFlags>(BulkDataFlagsToSet));
}

/**
* Gets the current bulk data flags.
*
* @return Bulk data flags currently set
*/
uint32 FBulkData::GetBulkDataFlags() const
{
	return BulkMeta.GetFlags();
}

#if WITH_EDITOR

void FBulkData::SetCookedIndex(FBulkDataCookedIndex Index)
{
	CookedIndex = Index;
}

#endif //WITH_EDITOR

/**
* Gets the current bulk data alignment.
*
* @return Bulk data alignment currently set
*/
uint32 FBulkData::GetBulkDataAlignment() const
{
	return DEFAULT_ALIGNMENT;
}

/**
 * Clears the passed in bulk data flags.
 *
 * @param BulkDataFlagsToClear	Bulk data flags to clear
 */
void FBulkData::ClearBulkDataFlags( uint32 BulkDataFlagsToClear )
{
	BulkMeta.SetFlags(static_cast<EBulkDataFlags>(BulkMeta.GetFlags() & ~BulkDataFlagsToClear));
}

FIoChunkId FBulkData::CreateChunkId() const
{
	return BulkChunkId; 
}

FString FBulkData::GetDebugName() const
{
	return LexToString(BulkChunkId);
}

/*-----------------------------------------------------------------------------
	Serialization.
-----------------------------------------------------------------------------*/

int32 GMinimumBulkDataSizeForAsyncLoading = 131072;
static FAutoConsoleVariableRef CVarMinimumBulkDataSizeForAsyncLoading(
	TEXT("s.MinBulkDataSizeForAsyncLoading"),
	GMinimumBulkDataSizeForAsyncLoading,
	TEXT("Minimum time the time limit exceeded warning will be triggered by."),
	ECVF_Default
	);

bool FBulkData::NeedsOffsetFixup() const
{
	return BulkMeta.HasAnyFlags(BULKDATA_NoOffsetFixUp) == false;
}

void FBulkData::SetBulkDataFlagsOn(EBulkDataFlags& InOutAccumulator, EBulkDataFlags FlagsToSet)
{
	InOutAccumulator = static_cast<EBulkDataFlags>(InOutAccumulator | FlagsToSet);
}

void FBulkData::ClearBulkDataFlagsOn(EBulkDataFlags& InOutAccumulator, EBulkDataFlags FlagsToClear)
{
	InOutAccumulator = static_cast<EBulkDataFlags>(InOutAccumulator & ~FlagsToClear);
}

bool FBulkData::HasFlags(EBulkDataFlags Flags, EBulkDataFlags Contains)
{
	return (uint32(Flags) & uint32(Contains)) == Contains; 
}

void FBulkData::Serialize(FArchive& Ar, UObject* Owner, bool bAttemptFileMapping, int32 ElementSize, EFileRegionType FileRegionType)
{
	DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("FBulkData::Serialize"), STAT_UBD_Serialize, STATGROUP_Memory, EStatFlags::Verbose);
	SCOPE_CYCLE_COUNTER(STAT_UBD_Serialize);

	SCOPED_LOADTIMER(BulkData_Serialize);

	check(IsUnlocked());
	
	check(!bAttemptFileMapping || Ar.IsLoading()); // makes no sense to map unless we are loading

#if !WITH_EDITOR
	FBulkDataCookedIndex CookedIndex;	// Dummy value to pass into ::SerializeBulkData, as this is the editor we will only
										// ever be loading and so this value will not be used.
#endif //!WITH_EDITOR

	if (Ar.SerializeBulkData(*this, FBulkDataSerializationParams {Owner, ElementSize, FileRegionType, bAttemptFileMapping, CookedIndex }))
	{
		// Just early out when the archive overrides the serialization of bulk data
		return;
	}

#if !USE_RUNTIME_BULKDATA
	if (Ar.IsTransacting())
	{
		// Special case for transacting bulk data arrays.

		// Constructing the object during load will save it to the transaction buffer.
		// We need to cancel that save because trying to load the bulk data now would break.
		bool bActuallySave = Ar.IsSaving() && (!Owner || !Owner->HasAnyFlags(RF_NeedLoad));

		Ar << bActuallySave;

		if (bActuallySave)
		{
			if (Ar.IsLoading())
			{
				// Flags for bulk data.
				EBulkDataFlags BulkDataFlags;
				Ar << BulkDataFlags;
				BulkMeta.SetFlags(BulkDataFlags);
				
				// Number of elements in array.
				int64 ElementCount;
				Ar << ElementCount;

				BulkMeta.SetSize(ElementCount * ElementSize);

				// Allocate bulk data.
				void* DataBuffer = ReallocateData(GetBulkDataSize());

				// Deserialize bulk data.
				SerializeBulkData(Ar, DataBuffer, GetBulkDataSize(), BulkMeta.GetFlags());
			}
			else if (Ar.IsSaving())
			{
				// Size of bulk data
				const int64 BulkDataSize = BulkMeta.GetSize();
				// Flags for bulk data.
				EBulkDataFlags BulkDataFlags = BulkMeta.GetFlags();
				Ar << BulkDataFlags;
				// Number of elements in array.
				int64 ElementCount = BulkDataSize / ElementSize;
				Ar << ElementCount;

				// Don't attempt to load or serialize BulkData if the current size is 0.
				// This could be a newly constructed BulkData that has not yet been loaded, 
				// and allocating 0 bytes now will cause a crash when we load.
				if (BulkDataSize > 0)
				{
					// Make sure bulk data is loaded.
					MakeSureBulkDataIsLoaded();

					// Serialize bulk data.
					SerializeBulkData(Ar, GetDataBufferForWrite(), BulkDataSize, BulkDataFlags);
				}
			}
		}
	}
	else
#endif // !USE_RUNTIME_BULKDATA
	if (Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && !Ar.ShouldSkipBulkData())
	{
			using namespace UE::BulkData::Private;
#if TRACK_BULKDATA_USE
		FThreadSafeBulkDataToObjectMap::Get().Add( this, Owner );
#endif
		if (Ar.IsLoading())
		{
			checkf(IsUnlocked(), TEXT("Serialize bulk data FAILED, bulk data is locked"));

			FBulkMetaData::FromSerialized(Ar, ElementSize, BulkMeta);
			BulkChunkId = FIoChunkId::InvalidChunkId;

			const int64 BulkDataSize = GetBulkDataSize();
			SerializeBulkData(Ar, ReallocateData(BulkDataSize), BulkDataSize, BulkMeta.GetFlags());
#if WITH_EDITOR
			if (GIsEditor)
			{
				ClearBulkDataFlags(BULKDATA_SingleUse);
			}
#endif
		}
#if !USE_RUNTIME_BULKDATA
		else if (Ar.IsSaving())
		{
			MakeSureBulkDataIsLoaded();

			FBulkMetaResource SerializedMeta;
			SerializedMeta.Flags = BulkMeta.GetFlags();
			SerializedMeta.ElementCount = GetBulkDataSize() / ElementSize;
			SerializedMeta.SizeOnDisk = GetBulkDataSize();

			if (SerializeBulkDataElements != nullptr)
			{
				// Force 64 bit precision when using custom element serialization
				SetBulkDataFlagsOn(SerializedMeta.Flags, static_cast<EBulkDataFlags>(BULKDATA_Size64Bit));
			}

			const EBulkDataFlags FlagsToClear = static_cast<EBulkDataFlags>(BULKDATA_PayloadAtEndOfFile | BULKDATA_PayloadInSeparateFile | BULKDATA_WorkspaceDomainPayload | BULKDATA_ForceSingleElementSerialization);
			FBulkData::ClearBulkDataFlagsOn(SerializedMeta.Flags, FlagsToClear);

			const int64 MetaOffset = Ar.Tell();
			Ar << SerializedMeta;

			TOptional<EFileRegionType> FileRegionTypeOptional;
			if (FileRegionType != EFileRegionType::None)
			{
				FileRegionTypeOptional = FileRegionType;
			}
			SerializedMeta.Offset = Ar.Tell();
			SerializedMeta.SizeOnDisk = SerializePayload(Ar, SerializedMeta.Flags, FileRegionTypeOptional);

			if (SerializeBulkDataElements != nullptr)
			{
				SerializedMeta.ElementCount = SerializedMeta.SizeOnDisk / ElementSize;
			}

			{
				FArchive::FScopeSeekTo _(Ar, MetaOffset);
				Ar << SerializedMeta;
			}
		}
#endif // !USE_RUNTIME_BULKDATA
	}
}

#if WITH_EDITOR

void FBulkData::SetFlagsFromDiskWrittenValues(
	EBulkDataFlags InBulkDataFlags,
	int64 InBulkDataOffsetInFile,
	int64 InBulkDataSizeOnDisk,
	int64 LinkerSummaryBulkDataStartOffset)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(!(InBulkDataFlags & BULKDATA_BadDataVersion)); // This is a legacy flag that should no longer be set when saving
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (GIsEditor)
	{
		InBulkDataFlags = static_cast<EBulkDataFlags>(InBulkDataFlags & ~BULKDATA_SingleUse);
	}

	// We are no longer loading from iostore, even if we were before; our data is now stored in the loose file we saved
	InBulkDataFlags = static_cast<EBulkDataFlags>(InBulkDataFlags & ~BULKDATA_UsesIoDispatcher);

	const bool bPayloadInline = !(InBulkDataFlags & BULKDATA_PayloadAtEndOfFile);
	const bool bPayloadInSeparateFile = !bPayloadInline && (InBulkDataFlags & BULKDATA_PayloadInSeparateFile);
	// fix up the file offset if the offset is relative to the file's bulkdata section
	if (!bPayloadInline && NeedsOffsetFixup())
	{
		check(LinkerSummaryBulkDataStartOffset >= 0);
		InBulkDataOffsetInFile += LinkerSummaryBulkDataStartOffset;
	}

	BulkMeta.SetFlags(InBulkDataFlags);
	BulkMeta.SetOffset(InBulkDataOffsetInFile);
	BulkMeta.SetSizeOnDisk(InBulkDataSizeOnDisk);
}
#endif

void FBulkData::GetBulkDataVersions(FArchive& InlineArchive, FPackageFileVersion& OutUEVersion,
	int32& OutLicenseeUEVersion, FCustomVersionContainer& OutCustomVersions) const
{
	FName PackageName;
	EPackageSegment Segment;
	bool bExternal;

	if (UE::TryGetPackageNameFromChunkId(BulkChunkId, PackageName, Segment, bExternal))
	{
		IPackageResourceManager& ResourceMgr = IPackageResourceManager::Get();

		if (TUniquePtr<FArchive> Ar = ResourceMgr.OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, PackageName.ToString()))
		{
			FPackageFileSummary Summary;
			*Ar << Summary;

			if (Ar->IsError() == false && Summary.Tag == PACKAGE_FILE_TAG)
			{
				OutUEVersion = Summary.GetFileVersionUE();
				OutLicenseeUEVersion = Summary.GetFileVersionLicenseeUE();
				OutCustomVersions = Summary.GetCustomVersionContainer();
				return;
			}
		}
	}

	OutUEVersion = InlineArchive.UEVer();
	OutLicenseeUEVersion = InlineArchive.LicenseeUEVer();
	OutCustomVersions = InlineArchive.GetCustomVersions();
}

/*-----------------------------------------------------------------------------
	Accessors for friend classes FLinkerLoad and content cookers.
-----------------------------------------------------------------------------*/

#if WITH_EDITOR
/**
 * Detaches the bulk data from the passed in archive. Needs to match the archive we are currently
 * attached to.
 *
 * @param Ar						Archive to detach from
 * @param bEnsureBulkDataIsLoaded	whether to ensure that bulk data is load before detaching from archive
 */
void FBulkData::DetachFromArchive( FArchive* Ar, bool bEnsureBulkDataIsLoaded )
{
	check( Ar );
	check( Ar == AttachedAr || AttachedAr == nullptr || AttachedAr->IsProxyOf(Ar) );
#if DO_CHECK
	if (!IsUnlocked())
	{
		FLinker* FromLinker = Ar->GetLinker();
#if WITH_EDITOR
		UE::FSourceLocation* Location = LockSourceLocation.GetPtrOrNull();
#else
		UE::FSourceLocation* Location = nullptr;
#endif // WITH_EDITOR
		checkf(false, TEXT("Bulk data cannot be detached while still locked. BulkData: %s, Linker: %s, Archive: %s. LockLocation: %s %s(%d)"), 
			*GetDebugName(), FromLinker ? *FromLinker->GetDebugName() : TEXT("nullptr"), *Ar->GetArchiveName(),
			Location ? StringCast<TCHAR>(Location->GetFunctionName()).Get() : TEXT("null"),
			Location ? StringCast<TCHAR>(Location->GetFileName()).Get() : TEXT("null"),
			Location ? Location->GetLine() : 0
		);
	}
#endif // DO_CHECK

	// Make sure bulk data is loaded.
	if( bEnsureBulkDataIsLoaded )
	{
		MakeSureBulkDataIsLoaded();
	}

	// Detach from archive.
	AttachedAr = nullptr;
	Linker = nullptr;
	BulkChunkId = FIoChunkId::InvalidChunkId;
}
#endif // WITH_EDITOR

void FBulkData::StoreCompressedOnDisk( FName CompressionFormat )
{
	if( CompressionFormat != GetDecompressionFormat() )
	{
		//Need to force this to be resident so we don't try to load data as though it were compressed when it isn't.
		ForceBulkDataResident();

		if( CompressionFormat == NAME_None )
		{
			// clear all compression settings
			ClearBulkDataFlags(BULKDATA_SerializeCompressed);
		}
		else
		{
			// right BulkData only knows zlib
			check(CompressionFormat == NAME_Zlib);
			const uint32 FlagToSet = CompressionFormat == NAME_Zlib ? BULKDATA_SerializeCompressedZLIB : BULKDATA_None;
			SetBulkDataFlags(FlagToSet);

			// make sure we are not forcing the bulkdata to be stored inline if we use compression
			ClearBulkDataFlags(BULKDATA_ForceInlinePayload);
		}
	}
}

void FBulkData::SerializeBulkData(FArchive& Ar, void* Data, int64 DataSize, EBulkDataFlags InBulkDataFlags)
{
	SCOPED_LOADTIMER(BulkData_SerializeBulkData);

#if !USE_RUNTIME_BULKDATA
	if (SerializeBulkDataElements != nullptr)
	{
		(*SerializeBulkDataElements)(Ar, Data, DataSize, InBulkDataFlags);
		return;
	}
#endif

	// skip serializing of unused data
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (InBulkDataFlags & BULKDATA_Unused)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return;
	}

	if (DataSize == 0)
	{
		return;
	}

	if (InBulkDataFlags & BULKDATA_SerializeCompressed)
	{
		Ar.SerializeCompressed(Data, DataSize, GetDecompressionFormat(InBulkDataFlags), COMPRESS_NoFlags, false);
	}
	else
	{
		Ar.Serialize(Data, DataSize);
	}
}

int64 FBulkData::SerializePayload(FArchive& Ar, EBulkDataFlags SerializationFlags, const TOptional<EFileRegionType>& RegionType)
{
	check(Ar.IsSaving());

	MakeSureBulkDataIsLoaded();

	const int64 PayloadStart = Ar.Tell();

	if (int64 PayloadSize = GetBulkDataSize(); PayloadSize > 0
#if !USE_RUNTIME_BULKDATA
	|| (SerializeBulkDataElements != nullptr)
#endif
	)
	{
		if (RegionType)
		{
			Ar.PushFileRegionType(*RegionType);
		}

		SerializeBulkData(Ar, GetDataBufferForWrite(), PayloadSize, SerializationFlags);

		if (RegionType)
		{
			Ar.PopFileRegionType();
		}
	}

	return Ar.Tell() - PayloadStart;
}

IAsyncReadFileHandle* FBulkData::OpenAsyncReadHandle() const
{
	uint64 AvailableChunkSize = 0;
	TIoStatusOr<uint64> ChunkSize = FIoDispatcher::Get().GetSizeForChunk(BulkChunkId, BulkMeta.GetOffsetAndLength(), AvailableChunkSize);

	return UE::BulkData::Private::OpenAsyncReadBulkData(
		BulkMeta,
		BulkChunkId,
		ChunkSize.IsOk() ? ChunkSize.ValueOrDie() : 0,
		AvailableChunkSize).Release();
}

IBulkDataIORequest* FBulkData::CreateStreamingRequest(EAsyncIOPriorityAndFlags Priority, FBulkDataIORequestCallBack* CompleteCallback, uint8* UserSuppliedMemory) const
{
	const int64 DataSize = GetBulkDataSize();

	return CreateStreamingRequest(0, DataSize, Priority, CompleteCallback, UserSuppliedMemory);
}

IBulkDataIORequest* FBulkData::CreateStreamingRequest(int64 OffsetInBulkData, int64 BytesToRead, EAsyncIOPriorityAndFlags Priority, FBulkDataIORequestCallBack* CompleteCallback, uint8* UserSuppliedMemory) const
{
	return UE::BulkData::Private::CreateStreamingRequest(
		BulkMeta,
		BulkChunkId,
		BulkMeta.GetOffset() + OffsetInBulkData,
		BytesToRead,
		Priority,
		CompleteCallback,
		UserSuppliedMemory).Release();
}

/**
 * Loads the bulk data if it is not already loaded.
 */
void FBulkData::MakeSureBulkDataIsLoaded()
{
	if (IsBulkDataLoaded())
	{
		return;
	}

	const int64 BulkDataSize = GetBulkDataSize();

	if (BulkDataSize == 0)
	{
		return;
	}

	void* Dest = ReallocateData(BulkDataSize);

	if (TryLoadDataIntoMemory(FIoBuffer(FIoBuffer::Wrap, Dest, BulkDataSize)) == false)
	{
		DataAllocation.Free(this);
	}
}

/**
 * Loads the data from disk into the specified memory block. This requires us still being attached to an
 * archive we can use for serialization.
 *
 * @param Dest Memory to serialize data into
 */
bool FBulkData::TryLoadDataIntoMemory(FIoBuffer Dest)
{
	const int64 BulkDataOffset = GetBulkDataOffsetInFile();
	const int64 BulkDataSize = GetBulkDataSize();
	const int64 BulkDataSizeOnDisk = GetBulkDataSizeOnDisk();

	// Early out if we have no data to load
	if (BulkDataSize == 0)
	{
		return false;
	}
	
	check(Dest.GetSize() == BulkDataSize);

#if WITH_EDITOR
	// BulkDatas in the same package share AttachedAr that they get from the LinkerLoad of the package.
	// To make calls to those BulkDatas threadsafe, we need to not use AttachedAr when called multithreaded.
	// Also don't use the attached archive when loading from the editor domain and the bulk data is referenced
	// from the original .uasset file
	if (IsInGameThread() && !IsInSeparateFile() && !IsInExternalResource())
	{
		if (FArchive* Ar = AttachedAr)
		{
			{
				FArchive::FScopeSeekTo _(*Ar, BulkDataOffset);
				
#if UE_INTERNAL_LINKER_BOUNDS
				FArchiveBoundsScope Bounds(Ar->SetBounds(Dest.GetSize()));
#endif//UE_INTERNAL_LINKER_BOUNDS
				
				SerializeBulkData(*Ar, Dest.GetData(), Dest.GetSize(), BulkMeta.GetFlags());
			}
			Ar->FlushCache();
		
			return true;
		}
	}
#endif // WITH_EDITOR
	   
	if (CanLoadFromDisk() == false)
	{
		UE_LOGF(LogSerialization, Error, "Attempting to load a BulkData object that cannot be loaded from disk");
		return false;
	}

	const bool bOk = UE::BulkData::Private::OpenReadBulkData(
		BulkMeta,
		BulkChunkId,
		BulkDataOffset,
		BulkDataSizeOnDisk,
		AIOP_High,
		[this, BulkDataSize, &Dest](FArchive& Ar)
		{
			SerializeBulkData(Ar, Dest.GetData(), BulkDataSize, BulkMeta.GetFlags());
		});

	return bOk;
}

bool FBulkData::CanDiscardInternalData() const
{	
	return BulkMeta.HasAnyFlags(static_cast<EBulkDataFlags>(BULKDATA_AlwaysAllowDiscard | BULKDATA_SingleUse)) || CanLoadFromDisk();
}

void FFormatContainer::Serialize(FArchive& Ar, UObject* Owner, const TArray<FName>* FormatsToSave, bool bSingleUse, uint16 InAlignment, bool bInline, bool bMapped)
{
	if (Ar.IsLoading())
	{
		int32 NumFormats = 0;
		Ar << NumFormats;
		for (int32 Index = 0; Index < NumFormats; Index++)
		{
			FName Name;
			Ar << Name;
			FByteBulkData& Bulk = GetFormat(Name);
			Bulk.Serialize(Ar, Owner, INDEX_NONE, false);
		}
	}
	else
	{
		check(Ar.IsCooking() && FormatsToSave); // this thing is for cooking only, and you need to provide a list of formats

		int32 NumFormats = 0;
		for (const TPair<FName, FByteBulkData*>& Format : Formats)
		{
			const FName Name = Format.Key;
			FByteBulkData* Bulk = Format.Value;
			check(Bulk);
			if (FormatsToSave->Contains(Name) && Bulk->GetBulkDataSize() > 0)
			{
				NumFormats++;
			}
		}
		Ar << NumFormats;
		for (const TPair<FName, FByteBulkData*>& Format : Formats)
		{
			FName Name = Format.Key;
			FByteBulkData* Bulk = Format.Value;
			if (FormatsToSave->Contains(Name) && Bulk->GetBulkDataSize() > 0)
			{
				NumFormats--;
				Ar << Name;
				// Force this kind of bulk data (physics, etc) to be stored inline for streaming
				const uint32 OldBulkDataFlags = Bulk->GetBulkDataFlags();
				if (bInline)
				{
					Bulk->SetBulkDataFlags(BULKDATA_ForceInlinePayload);
					Bulk->ClearBulkDataFlags(BULKDATA_Force_NOT_InlinePayload | BULKDATA_MemoryMappedPayload);
				}
				else
				{
					Bulk->SetBulkDataFlags(BULKDATA_Force_NOT_InlinePayload);
					if (bMapped)
					{
						Bulk->SetBulkDataFlags(BULKDATA_MemoryMappedPayload);
					}
					Bulk->ClearBulkDataFlags(BULKDATA_ForceInlinePayload);

				}
				if (bSingleUse)
				{
					Bulk->SetBulkDataFlags(BULKDATA_SingleUse);
				}
				Bulk->Serialize(Ar, Owner, INDEX_NONE, false);
				Bulk->ClearBulkDataFlags(0xFFFFFFFF);
				Bulk->SetBulkDataFlags(OldBulkDataFlags);
			}
		}
		check(NumFormats == 0);
	}
}

void FFormatContainer::SerializeAttemptMappedLoad(FArchive& Ar, UObject* Owner)
{
	check(Ar.IsLoading());
	int32 NumFormats = 0;
	Ar << NumFormats;
	for (int32 Index = 0; Index < NumFormats; Index++)
	{
		FName Name;
		Ar << Name;
		FByteBulkData& Bulk = GetFormat(Name);
		Bulk.Serialize(Ar, Owner, -1, true);
	}
}
