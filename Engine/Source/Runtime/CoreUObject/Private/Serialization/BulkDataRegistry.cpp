// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/BulkDataRegistry.h"

#if WITH_EDITOR

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/EditorBulkData.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/UObjectGlobals.h"

DEFINE_LOG_CATEGORY(LogBulkDataRegistry);

const TCHAR* LexToString(UE::BulkDataRegistry::ERegisterResult Value)
{
	using namespace UE::BulkDataRegistry;
	switch (Value)
	{
	case ERegisterResult::Success:
		return TEXT("Success");
	case ERegisterResult::AlreadyExists:
		return TEXT("AlreadyExists");
	default:
		return TEXT("InvalidResultCode");
	}
}

namespace UE::BulkDataRegistry::Private
{

IBulkDataRegistry* GBulkDataRegistry = nullptr;
FSetBulkDataRegistry GSetBulkDataRegistry;

/** A stub class to provide a return value from IBulkDataRegistry::Get() when the registery is disabled. */
class FBulkDataRegistryNull : public IBulkDataRegistry
{
public:
	FBulkDataRegistryNull() = default;
	virtual ~FBulkDataRegistryNull() {}

	virtual UE::BulkDataRegistry::ERegisterResult
		TryRegister(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData) override
	{
		return UE::BulkDataRegistry::ERegisterResult::Success;
	}
	virtual void UpdateRegistrationData(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData) override {}
	virtual void Unregister(const UE::Serialization::FEditorBulkData& BulkData) override {}
	virtual void OnExitMemory(const UE::Serialization::FEditorBulkData& BulkData) override {}
	virtual void UpdatePlaceholderPayloadId(const UE::Serialization::FEditorBulkData& BulkData) override {};
	virtual TFuture<UE::BulkDataRegistry::FMetaData> GetMeta(const FGuid& BulkDataId) override
	{
		TPromise<UE::BulkDataRegistry::FMetaData> Promise;
		Promise.SetValue(UE::BulkDataRegistry::FMetaData{ FIoHash(), 0 });
		return Promise.GetFuture();
	}
	virtual TFuture<UE::BulkDataRegistry::FData> GetData(const FGuid& BulkDataId) override
	{
		TPromise<UE::BulkDataRegistry::FData> Promise;
		Promise.SetValue(UE::BulkDataRegistry::FData{ FCompressedBuffer() });
		return Promise.GetFuture();
	}
	virtual bool TryGetBulkData(const FGuid & BulkDataId, UE::Serialization::FEditorBulkData* OutBulk = nullptr,
		FName* OutOwner = nullptr) override
	{
		return false;
	}
	virtual uint64 GetBulkDataResaveSize(FName PackageName) override
	{
		return 0;
	}
};

class FBulkDataRegistryTrackBulkDataToResave : public FBulkDataRegistryNull
{
public:
	FBulkDataRegistryTrackBulkDataToResave() = default;
	virtual ~FBulkDataRegistryTrackBulkDataToResave() {}

	virtual UE::BulkDataRegistry::ERegisterResult
		TryRegister(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData) override
	{
		ResaveSizeTracker.Register(Owner, BulkData);
		return UE::BulkDataRegistry::ERegisterResult::Success;
	}
	virtual void UpdateRegistrationData(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData) override
	{
		ResaveSizeTracker.UpdateRegistrationData(Owner, BulkData);
	}
	virtual uint64 GetBulkDataResaveSize(FName PackageName) override
	{
		return ResaveSizeTracker.GetBulkDataResaveSize(PackageName);
	}

private:
	FResaveSizeTracker ResaveSizeTracker;
};

/** Returns if the current running commandlet requires the BulkDataRegistry or not */
bool IsAllowedForCurrentCommandlet()
{
	/** Cache the results so we don't have to redo this every time the function is called */
	struct FConfigEnabled
	{
		FConfigEnabled()
		{
			FStringView CommandletToken(TEXTVIEW("commandlet"));

			FString CommandletName = GetCommandletNameFromCmdline();
			if (CommandletName.EndsWith(CommandletToken))
			{
				CommandletName.LeftChopInline(CommandletToken.Len(), EAllowShrinking::No);
			}

			TArray<FString> CommandletsUsingRegistry;
			GConfig->GetArray(TEXT("BulkDataRegistry"), TEXT("CommandletsUsingBulkDataRegistry"), CommandletsUsingRegistry, GEditorIni);

			for (FString& CommandletUsingRegistry : CommandletsUsingRegistry)
			{
				if (CommandletUsingRegistry.EndsWith(CommandletToken))
				{
					CommandletUsingRegistry.LeftChopInline(CommandletToken.Len(), EAllowShrinking::No);
				}

				if (CommandletUsingRegistry == CommandletName)
				{
					bEnabled = true;
					break;
				}
			}
		}

		bool bEnabled = false;

	} static ConfigEnabled;

	return ConfigEnabled.bEnabled;
}

}

bool IBulkDataRegistry::IsEnabled()
{
	if (IsRunningCommandlet() && !IsRunningCookCommandlet() && !UE::BulkDataRegistry::Private::IsAllowedForCurrentCommandlet())
	{
		// Disabled in other commandlets because they do not tick the BulkDataRegistry.
		// When not ticked, the BulkDataRegistry will never free memory and will hold onto all memory
		// from loaded BulkDatas until process shutdown; GC will not cause it to be released.
		return false;
	}

	bool bEnabled = true;
	GConfig->GetBool(TEXT("CookSettings"), TEXT("BulkDataRegistryEnabled"), bEnabled, GEditorIni);
	return bEnabled;
}

IBulkDataRegistry& IBulkDataRegistry::Get()
{
	check(UE::BulkDataRegistry::Private::GBulkDataRegistry);
	return *UE::BulkDataRegistry::Private::GBulkDataRegistry;
}

void IBulkDataRegistry::Initialize()
{
	using namespace UE::BulkDataRegistry::Private;

	if (IsEnabled())
	{
		FSetBulkDataRegistry& SetRegistryDelegate = GetSetBulkDataRegistryDelegate();
		if (SetRegistryDelegate.IsBound())
		{
			// Allow the editor or licensee project to define the BulkDataRegistry
			GBulkDataRegistry = SetRegistryDelegate.Execute();
		}
	}
	else if (IsEditorDomainEnabled() >= EEditorDomainEnabled::PackageResourceManager)
	{
		GBulkDataRegistry = new FBulkDataRegistryTrackBulkDataToResave();
	}

	// Assign the null BulkDataRegistry if it was disabled or not set by a higher level source
	if (!GBulkDataRegistry)
	{
		GBulkDataRegistry = new FBulkDataRegistryNull();
	}
}

void IBulkDataRegistry::Shutdown()
{
	delete UE::BulkDataRegistry::Private::GBulkDataRegistry;
	UE::BulkDataRegistry::Private::GBulkDataRegistry = nullptr;
}

FSetBulkDataRegistry& IBulkDataRegistry::GetSetBulkDataRegistryDelegate()
{
	return UE::BulkDataRegistry::Private::GSetBulkDataRegistry;
}

namespace UE::GlobalBuildInputResolver::Private
{
UE::DerivedData::IBuildInputResolver* GGlobalBuildInputResolver = nullptr;
}

UE::DerivedData::IBuildInputResolver* GetGlobalBuildInputResolver()
{
	return UE::GlobalBuildInputResolver::Private::GGlobalBuildInputResolver;
}

void SetGlobalBuildInputResolver(UE::DerivedData::IBuildInputResolver* InResolver)
{
	UE::GlobalBuildInputResolver::Private::GGlobalBuildInputResolver = InResolver;
}

namespace UE::BulkDataRegistry::Private
{

FResaveSizeTracker::FResaveSizeTracker()
{
	FCoreUObjectDelegates::OnEndLoadPackage.AddRaw(this, &FResaveSizeTracker::OnEndLoadPackage);
	ELoadingPhase::Type CurrentPhase = IPluginManager::Get().GetLastCompletedLoadingPhase();
	if (CurrentPhase == ELoadingPhase::None || CurrentPhase < ELoadingPhase::PostEngineInit)
	{
		// Our contract says that we need to keep information up until OnEnginePostInit is called on all
		// subscribers to it, so we need to subscribe to the first event we can find that occurs after OnEnginePostInit
		// is done. OnAllModuleLoadingPhasesComplete seems to be it.
		FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddRaw(this, &FResaveSizeTracker::OnAllModuleLoadingPhasesComplete);
	}
	else
	{
		OnAllModuleLoadingPhasesComplete();
	}
	FCoreUObjectDelegates::OnEndLoadPackage.AddRaw(this, &FResaveSizeTracker::OnEndLoadPackage);
}

FResaveSizeTracker::~FResaveSizeTracker()
{
	FCoreUObjectDelegates::OnEndLoadPackage.RemoveAll(this);
	FCoreDelegates::OnAllModuleLoadingPhasesComplete.RemoveAll(this);
}

void FResaveSizeTracker::OnAllModuleLoadingPhasesComplete()
{
	bPostEngineInitComplete = true;

	FReadScopeLock ScopeLock(Lock);
	DeferredRemove.Reserve(PackageBulkResaveSize.Num());
	for (const TPair<FName, uint64>& Pair : PackageBulkResaveSize)
	{
		DeferredRemove.Add(Pair.Key);
	}
}

void FResaveSizeTracker::Register(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData)
{
	if (!BulkData.GetIdentifier().IsValid() || !BulkData.IsMemoryOnlyPayload())
	{
		return;
	}

	if (!Owner
		|| !Owner->GetFileSize() // We only track disk packages
		|| (bPostEngineInitComplete && Owner->GetHasBeenEndLoaded()) // We only record BulkDatas that are loaded before the package finishes loading
		)
	{
		return;
	}

	FWriteScopeLock ScopeLock(Lock);
	PackageBulkResaveSize.FindOrAdd(Owner->GetFName()) += BulkData.GetPayloadSize();
}

void FResaveSizeTracker::UpdateRegistrationData(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData)
{
	// Not yet implemented; we keep the values only from the first registration
	// Implementing this would require keeping more data; we will need to know the previous payload size to subtract it
}

uint64 FResaveSizeTracker::GetBulkDataResaveSize(FName PackageName)
{
	FReadScopeLock ScopeLock(Lock);
	return PackageBulkResaveSize.FindRef(PackageName);
}

void FResaveSizeTracker::OnEndLoadPackage(const FEndLoadPackageContext& Context)
{
	if (!bPostEngineInitComplete)
	{
		return;
	}

	TArray<FName> PackageNames;
	PackageNames.Reserve(Context.LoadedPackages.Num());
	for (UPackage* LoadedPackage : Context.LoadedPackages)
	{
		PackageNames.Add(LoadedPackage->GetFName());
	}

	FWriteScopeLock ScopeLock(Lock);
	// The contract for GetBulkDataResaveSize specifies that we must answer correctly until
	// OnEndLoadPackage is complete. This includes being called from other subscribers to OnEndLoadPackage
	// that might run after us. So we defer the removals from PackageBulkResaveSize until the next top-level call 
	// to OnEndLoadPackage.
	if (Context.RecursiveDepth == 0)
	{
		for (FName PackageName : DeferredRemove)
		{
			PackageBulkResaveSize.Remove(PackageName);
		}
		DeferredRemove.Reset();
	}
	DeferredRemove.Append(MoveTemp(PackageNames));
}

}
#endif // WITH_EDITOR
