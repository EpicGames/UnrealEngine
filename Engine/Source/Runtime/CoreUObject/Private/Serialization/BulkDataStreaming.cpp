// Copyright Epic Games, Inc. All Rights Reserved.
#include "Serialization/BulkData.h"
#include "Async/ManualResetEvent.h"
#include "Async/MappedFileHandle.h"
#include "Containers/ChunkedArray.h"
#include "HAL/CriticalSection.h"
#include "IO/IoDispatcher.h"
#include "IO/IoOffsetLength.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Timespan.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/MemoryReader.h"
#include "Templates/RefCounting.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceIoDispatcherBackend.h"

//////////////////////////////////////////////////////////////////////////////

TRACE_DECLARE_ATOMIC_INT_COUNTER(BulkDataBatchRequest_Count, TEXT("BulkData/BatchRequest/Count"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(BulkDataBatchRequest_PendingCount, TEXT("BulkData/BatchRequest/Pending"));

/**
 * When enabled calls to FChunkReadFileHandle::ReadRequest will validate that the request
 * is within the bulkdata payload bounds. Currently disabled as FFileCache still uses the
 * handle to represent the entire .ubulk file rather than the specific bulkdata payload.
 */
#define UE_ENABLE_BULKDATA_RANGE_TEST 0

//////////////////////////////////////////////////////////////////////////////

namespace UE::BulkData::Private
{

enum class EChunkRequestStatus : uint32
{
	None				= 0,
	Pending				= 1 << 0,
	Canceled			= 1 << 1,
	DataReady			= 1 << 2,
	CallbackTriggered	= 1 << 3,
};
ENUM_CLASS_FLAGS(EChunkRequestStatus);

class FChunkRequest
{
public:
	virtual ~FChunkRequest();
	
	void Issue(FIoChunkId ChunkId, FIoReadOptions Options, int32 Priority);

protected:
	FChunkRequest(FIoBuffer&& InBuffer);
	
	inline EChunkRequestStatus GetStatus() const
	{
		return static_cast<EChunkRequestStatus>(Status.load(std::memory_order_consume));
	}

	virtual void HandleChunkResult(TIoStatusOr<FIoBuffer>&& Result) = 0;
	bool WaitForChunkRequest(float TimeLimitSeconds = 0.0f);
	void CancelChunkRequest();
	int64 GetSizeResult() const { return SizeResult; }
	void UpdatePriority(const uint32 Pri) { Request.UpdatePriority(Pri); }
	
	FIoBuffer			Buffer;

private:

	UE::FManualResetEvent	DoneEvent;
	FIoRequest				Request;
	int64					SizeResult;
	std::atomic<uint32>		Status;
};

FChunkRequest::FChunkRequest(FIoBuffer&& InBuffer)
	: Buffer(MoveTemp(InBuffer))
	, SizeResult(-1)
	, Status{uint32(EChunkRequestStatus::None)}
{
}

FChunkRequest::~FChunkRequest()
{
	DoneEvent.Wait();
}

void FChunkRequest::Issue(FIoChunkId ChunkId, FIoReadOptions Options, int32 Priority)
{
	Status.store(uint32(EChunkRequestStatus::Pending), std::memory_order_release); 

	check(Options.GetSize() == Buffer.GetSize());
	Options.SetTargetVa(Buffer.GetData());

	FIoBatch IoBatch = FIoDispatcher::Get().NewBatch();
	Request = IoBatch.ReadWithCallback(ChunkId, Options, Priority, [this](TIoStatusOr<FIoBuffer> Result)
	{
		EChunkRequestStatus ReadyOrCanceled = EChunkRequestStatus::Canceled;

		if (Result.IsOk())
		{
			SizeResult = Result.ValueOrDie().GetSize();
			ReadyOrCanceled = EChunkRequestStatus::DataReady;
		}

		Status.store(uint32(ReadyOrCanceled), std::memory_order_release); 
		HandleChunkResult(MoveTemp(Result));
		Status.store(uint32(ReadyOrCanceled | EChunkRequestStatus::CallbackTriggered), std::memory_order_release); 

		DoneEvent.Notify();
	});

	IoBatch.Issue();
}

bool FChunkRequest::WaitForChunkRequest(float TimeLimitSeconds)
{
	checkf(GetStatus() != EChunkRequestStatus::None, TEXT("The request must be issued before waiting for completion"));

	return DoneEvent.WaitFor(TimeLimitSeconds <= 0.0f ? FMonotonicTimeSpan::Infinity() : FMonotonicTimeSpan::FromSeconds(TimeLimitSeconds));
}

void FChunkRequest::CancelChunkRequest()
{
	checkf(GetStatus() != EChunkRequestStatus::None, TEXT("The request must be issued before it can be canceled"));

	uint32 Expected = uint32(EChunkRequestStatus::Pending);
	if (Status.compare_exchange_strong(Expected, uint32(EChunkRequestStatus::Canceled)))
	{
		Request.Cancel();
	}
}

//////////////////////////////////////////////////////////////////////////////

class FChunkReadFileRequest final : public FChunkRequest, public IAsyncReadRequest
{
public:
	FChunkReadFileRequest(FAsyncFileCallBack* Callback, FIoBuffer&& InBuffer);
	virtual ~FChunkReadFileRequest();
	
	virtual void WaitCompletionImpl(float TimeLimitSeconds) override;

	virtual void CancelImpl() override;
	virtual void ReleaseMemoryOwnershipImpl() override;
	virtual void HandleChunkResult(TIoStatusOr<FIoBuffer>&& Result) override;
};

FChunkReadFileRequest::FChunkReadFileRequest(FAsyncFileCallBack* Callback, FIoBuffer&& InBuffer)
	: FChunkRequest(MoveTemp(InBuffer))
	, IAsyncReadRequest(Callback, false, nullptr)
{
	Memory = Buffer.GetData();
}

FChunkReadFileRequest::~FChunkReadFileRequest()
{
	WaitForChunkRequest();

	// Calling GetReadResult transfers ownership of the read buffer
	if (Memory == nullptr && Buffer.IsMemoryOwned())
	{
		const bool bReleased = Buffer.Release().IsOk();
		check(bReleased);
	}

	Memory = nullptr;
}
	
void FChunkReadFileRequest::WaitCompletionImpl(float TimeLimitSeconds)
{
	WaitForChunkRequest(TimeLimitSeconds);
}

void FChunkReadFileRequest::CancelImpl()
{
	bCanceled = true;
	CancelChunkRequest();
}

void FChunkReadFileRequest::ReleaseMemoryOwnershipImpl()
{
}

void FChunkReadFileRequest::HandleChunkResult(TIoStatusOr<FIoBuffer>&& Result)
{
	bCanceled = Result.Status().IsOk() == false;
	SetDataComplete();
	SetAllComplete();
}

//////////////////////////////////////////////////////////////////////////////

class FChunkFileSizeRequest : public IAsyncReadRequest
{
public:
	FChunkFileSizeRequest(const FIoChunkId& ChunkId, uint64 ChunkSize, FAsyncFileCallBack* Callback)
		: IAsyncReadRequest(Callback, true, nullptr)
	{
		if (ChunkSize > 0)
		{
			Size = static_cast<int64>(ChunkSize);
		}
		SetComplete();
	}

	virtual ~FChunkFileSizeRequest() = default;

private:

	virtual void WaitCompletionImpl(float TimeLimitSeconds) override
	{
		// Even though SetComplete called in the constructor and sets bCompleteAndCallbackCalled=true, we still need to implement WaitComplete as
		// the CompleteCallback can end up starting async tasks that can overtake the constructor execution and need to wait for the constructor to finish.
		while (!*(volatile bool*)&bCompleteAndCallbackCalled);
	}

	virtual void CancelImpl() override
	{
	}

	virtual void ReleaseMemoryOwnershipImpl() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////////

class FChunkReadFileHandle : public IAsyncReadFileHandle
{
public:
	FChunkReadFileHandle(const FIoChunkId& InChunkId, const FIoOffsetAndLength& InChunkRange, uint64 InChunkSize, uint64 InAvailableChunkSize) 
		: ChunkId(InChunkId)
		, ChunkRange(InChunkRange)
		, ChunkSize(InChunkSize)
		, AvailableChunkSize(InAvailableChunkSize)
	{
	}

	virtual ~FChunkReadFileHandle() = default;

	virtual IAsyncReadRequest* SizeRequest(FAsyncFileCallBack* CompleteCallback = nullptr) override;

	virtual IAsyncReadRequest* ReadRequest(
		int64 Offset,
		int64 BytesToRead,
		EAsyncIOPriorityAndFlags PriorityAndFlags = AIOP_Normal,
		FAsyncFileCallBack* CompleteCallback = nullptr,
		uint8* UserSuppliedMemory = nullptr) override;

private:
	FIoChunkId ChunkId;
	FIoOffsetAndLength ChunkRange;
	uint64 ChunkSize;
	uint64 AvailableChunkSize;
};

IAsyncReadRequest* FChunkReadFileHandle::SizeRequest(FAsyncFileCallBack* CompleteCallback)
{
	return new FChunkFileSizeRequest(ChunkId, ChunkSize, CompleteCallback);
}

IAsyncReadRequest* FChunkReadFileHandle::ReadRequest(
	int64 Offset, 
	int64 BytesToRead, 
	EAsyncIOPriorityAndFlags PriorityAndFlags,
	FAsyncFileCallBack* CompleteCallback,
	uint8* UserSuppliedMemory)
{
#if UE_ENABLE_BULKDATA_RANGE_TEST
	const bool bIsOutsideBulkDataRange =
		(Offset < static_cast<int64>(ChunkRange.GetOffset())) ||
		((Offset + BytesToRead) > static_cast<int64>(ChunkRange.GetOffset() + AvailableChunkSize));


	UE_CLOGF(bIsOutsideBulkDataRange, LogSerialization, Warning,
		"Reading outside of bulk data range, RequestRange='%lld, %lld', BulkDataRange='%llu, %llu', ChunkId='%ls'",
		Offset, BytesToRead, ChunkRange.GetOffset(), ChunkRange.GetLength(), *LexToString(ChunkId));
#endif //UE_ENABLE_BULKDATA_RANGE_TEST

	FIoBuffer Buffer = UserSuppliedMemory ? FIoBuffer(FIoBuffer::Wrap, UserSuppliedMemory, BytesToRead) : FIoBuffer(BytesToRead);
	FChunkReadFileRequest* Request = new FChunkReadFileRequest(CompleteCallback, MoveTemp(Buffer));

	Request->Issue(ChunkId, FIoReadOptions(Offset, BytesToRead), ConvertToIoDispatcherPriority(PriorityAndFlags));

	return Request;
}

//////////////////////////////////////////////////////////////////////////////

class FChunkBulkDataRequest final : public FChunkRequest, public IBulkDataIORequest
{
public:
	FChunkBulkDataRequest(FBulkDataIORequestCallBack* InCallback, FIoBuffer&& InBuffer);
	
	virtual ~FChunkBulkDataRequest() = default;
	
	inline virtual bool PollCompletion() const override
	{
		checkf(GetStatus() != EChunkRequestStatus::None, TEXT("The request must be issued before polling for completion"));
		return EnumHasAnyFlags(GetStatus(), EChunkRequestStatus::CallbackTriggered);
	}

	virtual bool WaitCompletion(float TimeLimitSeconds = 0.0f) override
	{
		checkf(GetStatus() != EChunkRequestStatus::None, TEXT("The request must be issued before waiting for completion"));
		return WaitForChunkRequest(TimeLimitSeconds);
	}

	virtual uint8* GetReadResults() override;
	
	inline virtual int64 GetSize() const override
	{
		checkf(GetStatus() != EChunkRequestStatus::None, TEXT("The request must be issued before polling for size"));
		return EnumHasAnyFlags(GetStatus(), EChunkRequestStatus::DataReady) ? GetSizeResult() : -1;
	}

	virtual void Cancel() override
	{
		CancelChunkRequest();
	}
	
private:

	virtual void HandleChunkResult(TIoStatusOr<FIoBuffer>&& Result) override
	{
		if (Callback)
		{
			const bool bCanceled = Result.IsOk() == false;
			Callback(bCanceled, this);
		}
	}

	FBulkDataIORequestCallBack Callback;
};

FChunkBulkDataRequest::FChunkBulkDataRequest(FBulkDataIORequestCallBack* InCallback, FIoBuffer&& InBuffer)
	: FChunkRequest(MoveTemp(InBuffer))
{
	if (InCallback)
	{
		Callback = *InCallback;
	}
}

uint8* FChunkBulkDataRequest::GetReadResults()
{
	uint8* ReadResult = nullptr;

	if (EnumHasAnyFlags(GetStatus(), EChunkRequestStatus::DataReady))
	{
		if (Buffer.IsMemoryOwned())
		{
			ReadResult = Buffer.Release().ConsumeValueOrDie();
		}
		else
		{
			ReadResult = Buffer.GetData();
		}
	}

	return ReadResult;
}

//////////////////////////////////////////////////////////////////////////////

bool OpenReadBulkData(
	const FBulkMetaData& BulkMeta,
	const FIoChunkId& BulkChunkId,
	int64 Offset,
	int64 Size,
	EAsyncIOPriorityAndFlags Priority,
	TFunction<void(FArchive& Ar)>&& Read)
{
	if (BulkChunkId.IsValid() == false)
	{
		return false;
	}

	FIoBatch Batch = FIoDispatcher::Get().NewBatch();
	FIoRequest Request = Batch.Read(BulkChunkId, FIoReadOptions(Offset, Size), ConvertToIoDispatcherPriority(Priority));
	FEventRef Event;
	Batch.IssueAndTriggerEvent(Event.Get());
	Event->Wait();

	if (const FIoBuffer* Buffer = Request.GetResult())
	{
		FMemoryReaderView Ar(Buffer->GetView());
		Read(Ar);

		return true;
	}

	return false;
}

//////////////////////////////////////////////////////////////////////////////

TUniquePtr<IAsyncReadFileHandle> OpenAsyncReadBulkData(
	const FBulkMetaData& BulkMeta,
	const FIoChunkId& BulkChunkId,
	uint64 ChunkSize,
	uint64 AvailableChunkSize)
{
	if (BulkChunkId.IsValid() == false)
	{
		return TUniquePtr<IAsyncReadFileHandle>();
	}

	return MakeUnique<FChunkReadFileHandle>(BulkChunkId, BulkMeta.GetOffsetAndLength(), ChunkSize, AvailableChunkSize);
}

TUniquePtr<IAsyncReadFileHandle> OpenAsyncReadBulkData(const FBulkMetaData& BulkMeta, const FIoChunkId& BulkChunkId)
{
	if (BulkChunkId.IsValid() == false)
	{
		return TUniquePtr<IAsyncReadFileHandle>();
	}

	TIoStatusOr<uint64> Status = FIoDispatcher::Get().GetSizeForChunk(BulkChunkId); 
	const uint64 ChunkSize = Status.IsOk() ? Status.ValueOrDie() : 0;
	return MakeUnique<FChunkReadFileHandle>(BulkChunkId, BulkMeta.GetOffsetAndLength(), ChunkSize, ChunkSize);
}

//////////////////////////////////////////////////////////////////////////////

TUniquePtr<IBulkDataIORequest> CreateStreamingRequest(
	const FBulkMetaData& BulkMeta,
	const FIoChunkId& BulkChunkId,
	int64 Offset,
	int64 Size,
	EAsyncIOPriorityAndFlags Priority,
	FBulkDataIORequestCallBack* Callback,
	uint8* UserSuppliedMemory)
{
	if (BulkChunkId.IsValid() == false)
	{
		return TUniquePtr<IBulkDataIORequest>();
	}

	FIoBuffer Buffer = UserSuppliedMemory ? FIoBuffer(FIoBuffer::Wrap, UserSuppliedMemory, Size) : FIoBuffer(Size);
	FChunkBulkDataRequest* Request = new FChunkBulkDataRequest(Callback, MoveTemp(Buffer));
	Request->Issue(BulkChunkId, FIoReadOptions(Offset, Size), ConvertToIoDispatcherPriority(Priority));
	
	return TUniquePtr<IBulkDataIORequest>(Request);
}

//////////////////////////////////////////////////////////////////////////////

bool TryMemoryMapBulkData(
	const FBulkMetaData& BulkMeta,
	const FIoChunkId& BulkChunkId,
	int64 Offset,
	int64 Size,
	FIoMappedRegion& OutRegion)
{
	TIoStatusOr<FIoMappedRegion> Status = FIoDispatcher::Get().OpenMapped(BulkChunkId, FIoReadOptions(Offset, Size));

	if (Status.IsOk())
	{
		OutRegion = Status.ConsumeValueOrDie();
		return true;
	}

	return false;
}

} // namespace UE::BulkData

//////////////////////////////////////////////////////////////////////////////

FBulkDataRequest::IHandle::~IHandle()
{
}

//////////////////////////////////////////////////////////////////////////////

class FHandleBase : public FBulkDataRequest::IHandle
{
public:
	FHandleBase() = default;
	FHandleBase(FBulkDataRequest::EStatus InStatus)
		: Status(uint32(InStatus))
	{ }

	virtual void AddRef() override final
	{
		RefCount.fetch_add(1, std::memory_order_relaxed);
	}

	virtual void Release() override final
	{
		if (RefCount.fetch_sub(1, std::memory_order_acq_rel) == 1)
		{
			delete this;
		}
	}

	virtual uint32 GetRefCount() const override final
	{
		return uint32(RefCount.load(std::memory_order_relaxed));
	}

	virtual FBulkDataBatchRequest::EStatus GetStatus() const override final
	{
		return FBulkDataBatchRequest::EStatus(Status.load(std::memory_order_consume));
	}

	void SetStatus(FBulkDataBatchRequest::EStatus InStatus)
	{
		Status.store(uint32(InStatus), std::memory_order_release);
	}

	virtual bool Cancel() override
	{
		return false;
	}

	virtual bool Wait(uint32 Milliseconds) override
	{
		return false;
	}

	virtual void UpdatePriority(const EAsyncIOPriorityAndFlags Priority) override
	{
	}

private:
	std::atomic<int32>	RefCount{0};
	std::atomic<uint32> Status{0};
};

//////////////////////////////////////////////////////////////////////////////

static FBulkDataRequest::EStatus GetStatusFromIoErrorCode(const EIoErrorCode ErrorCode)
{
	if (ErrorCode == EIoErrorCode::Unknown)
	{
		return FBulkDataRequest::EStatus::Pending;
	}
	else if (ErrorCode == EIoErrorCode::Ok)
	{
		return FBulkDataRequest::EStatus::Ok;
	}
	else if (ErrorCode == EIoErrorCode::Cancelled)
	{
		return FBulkDataRequest::EStatus::Cancelled;
	}
	else
	{
		return FBulkDataRequest::EStatus::Error;
	}
}

class FChunkBatchReadRequest final : public FBulkDataRequest::IHandle 
{
public:
	FChunkBatchReadRequest() = default;
	FChunkBatchReadRequest(FBulkDataRequest::IHandle* InBatch)
		: Batch(InBatch)
	{ }

	virtual void AddRef() override
	{
		Batch->AddRef();
	}

	virtual void Release() override
	{
		Batch->Release();
	}

	virtual uint32 GetRefCount() const override
	{
		return Batch->GetRefCount();
	}

	virtual FBulkDataRequest::EStatus GetStatus() const override
	{
		return GetStatusFromIoErrorCode(IoHandle.Status().GetErrorCode());
	}

	virtual bool Cancel() override
	{
		if (IoHandle.Status().GetErrorCode() == EIoErrorCode::Unknown)
		{
			IoHandle.Cancel();
			return true;
		}
		return false;
	}

	virtual bool Wait(uint32 Milliseconds) override
	{
		checkNoEntry(); // Bulk read requests is currently not awaitable
		return true;
	}

	virtual void UpdatePriority(const EAsyncIOPriorityAndFlags Priority) override
	{
		if (IoHandle.Status().GetErrorCode() <= EIoErrorCode::Unknown )
		{
			const int32 IoPriority = ConvertToIoDispatcherPriority(Priority);
			IoHandle.UpdatePriority(IoPriority);
		}
	}

	FBulkDataRequest::IHandle* Batch = nullptr;
	FIoRequest IoHandle;
};

class FBulkDataBatchRequest::FBatchHandle
	: public FHandleBase
{
public:
	FBatchHandle(int32 BatchMaxCount)
	{
		if (BatchMaxCount > 0)
		{
			Requests.Reserve(BatchMaxCount);
		}

		TRACE_COUNTER_INCREMENT(BulkDataBatchRequest_Count);
	}
	
	virtual ~FBatchHandle() override
	{
		Cancel();
		Wait(MAX_uint32);
		
		TRACE_COUNTER_DECREMENT(BulkDataBatchRequest_Count);
	}

	virtual bool Cancel() override final
	{
		if (FBulkDataRequest::EStatus::None == GetStatus())
		{
			CompleteBatch(FBulkDataRequest::EStatus::Cancelled);
			return true;
		}
		else if (FBulkDataRequest::EStatus::Pending == GetStatus())
		{
			for (FChunkBatchReadRequest& Request : Requests)
			{
				Request.Cancel();
			}

			return true;
		}

		return false;
	}
	
	virtual bool Wait(uint32 Milliseconds) override final
	{
		check(GetStatus() != FBulkDataRequest::EStatus::None);
		return DoneEvent.WaitFor(UE::FMonotonicTimeSpan::FromMilliseconds(Milliseconds));
	}

	virtual void UpdatePriority(const EAsyncIOPriorityAndFlags Priority) override final
	{
		for (FChunkBatchReadRequest& ReadRequest : Requests)
		{
			ReadRequest.UpdatePriority(Priority);
		}
	}

	void Read(
		const FIoChunkId& BulkChunkId,
		const FIoReadOptions& Options,
		EAsyncIOPriorityAndFlags Priority,
		FIoReadCallback&& Callback,
		FBulkDataBatchReadRequest* OutRequest) 
	{
		const int32 IoPriority = ConvertToIoDispatcherPriority(Priority);

		FChunkBatchReadRequest* Request = new (Requests) FChunkBatchReadRequest(this);
		Request->IoHandle = IoBatch.ReadWithCallback(BulkChunkId, Options, IoPriority, MoveTemp(Callback));

		if (OutRequest)
		{
			*OutRequest = FBulkDataBatchReadRequest(Request);
		}
	}

	void Issue(FBulkDataRequest::FCompletionCallback&& Callback) 
	{
		CompletionCallback = MoveTemp(Callback);

		if (Requests.IsEmpty())
		{
			CompleteBatch(FBulkDataRequest::EStatus::Ok);
			return;
		}

		TRACE_COUNTER_INCREMENT(BulkDataBatchRequest_PendingCount);

		SetStatus(FBulkDataRequest::EStatus::Pending);

		IoBatch.IssueWithCallback([this]()
		{
			FBulkDataRequest::EStatus BatchStatus = FBulkDataRequest::EStatus::Ok;

			for (FChunkBatchReadRequest& Request : Requests)
			{
				if (EIoErrorCode ErrorCode = Request.IoHandle.Status().GetErrorCode(); ErrorCode != EIoErrorCode::Ok)
				{
					BatchStatus = ErrorCode == EIoErrorCode::Cancelled ? FBulkDataRequest::EStatus::Cancelled : FBulkDataRequest::EStatus::Error;
					break;
				}
			}

			CompleteBatch(BatchStatus);

			TRACE_COUNTER_DECREMENT(BulkDataBatchRequest_PendingCount);
		});
	}

private:
	void CompleteBatch(FBulkDataRequest::EStatus CompletionStatus)
	{
		if (CompletionCallback)
		{
			CompletionCallback(CompletionStatus);
		}

		SetStatus(CompletionStatus);
		DoneEvent.Notify();
	}

	static constexpr int32 TargetBytesPerChunk = sizeof(FChunkBatchReadRequest) * 8;
	using FRequests = TChunkedArray<FChunkBatchReadRequest, TargetBytesPerChunk>;

	FIoBatch				IoBatch;
	FRequests				Requests; //TODO: Optimize if there's only a single read request 
	UE::FManualResetEvent	DoneEvent;
	FBulkDataRequest::FCompletionCallback CompletionCallback;
};

//////////////////////////////////////////////////////////////////////////////

void FBulkDataBatchRequest::Wait()
{
	if (Handle.IsValid())
	{
		Handle->Wait(MAX_uint32);
	}
}

bool FBulkDataBatchRequest::WaitFor(uint32 Milliseconds)
{
	if (Handle.IsValid())
	{
		return Handle->Wait(Milliseconds);
	}

	return false;
}

bool FBulkDataBatchRequest::WaitFor(const FTimespan& WaitTime)
{
	return WaitFor((uint32)FMath::Clamp<int64>(WaitTime.GetTicks() / ETimespan::TicksPerMillisecond, 0, MAX_uint32));
}

void FBulkDataBatchRequest::UpdatePriority(const EAsyncIOPriorityAndFlags Priority)
{
	if (Handle.IsValid() && Handle->GetStatus() <= EStatus::Pending) 
	{
		Handle->UpdatePriority(Priority);
	}
}

//////////////////////////////////////////////////////////////////////////////

FBulkDataBatchRequest::FBuilder::FBuilder(int32 MaxCount)
	: BatchMax(MaxCount)
{
}

FBulkDataBatchRequest::FBuilder::~FBuilder()
{
}

FBulkDataBatchRequest::FBatchHandle& FBulkDataBatchRequest::FBuilder::GetBatch()
{
	if (Batch.IsValid() == false)
	{
		Batch = new FBulkDataBatchRequest::FBatchHandle(BatchMax);
	}

	return *Batch;
}

void FBulkDataBatchRequest::FBuilder::IssueBatch(FBulkDataBatchRequest* OutRequest, FCompletionCallback&& Callback)
{
	check(Batch.IsValid());
	checkf(OutRequest != nullptr || Batch->GetRefCount() > 1, TEXT("At least one request handle needs to be used when creating a batch request"));

	TRefCountPtr<FBulkDataBatchRequest::FBatchHandle> NewBatch = MoveTemp(Batch);
	NewBatch->Issue(MoveTemp(Callback));

	if (OutRequest)
	{
		*OutRequest = FBulkDataBatchRequest(NewBatch.GetReference());
	}
}

FBulkDataBatchRequest::FBatchBuilder::FBatchBuilder(int32 MaxCount)
	: FBuilder(MaxCount)
{
}

bool FBulkDataBatchRequest::FBatchBuilder::IsEmpty() const
{
	return BatchCount == 0;
}

FBulkDataBatchRequest::FBatchBuilder& FBulkDataBatchRequest::FBatchBuilder::Read(FBulkData& BulkData, EAsyncIOPriorityAndFlags Priority)
{
	if (BulkData.IsBulkDataLoaded())
	{
		++NumLoaded;
		return *this;
	}

	GetBatch().Read(
		BulkData.BulkChunkId,
		FIoReadOptions(BulkData.GetBulkDataOffsetInFile(), BulkData.GetBulkDataSize()),
		Priority,
		[BulkData = &BulkData](TIoStatusOr<FIoBuffer> Status)
		{
			if (Status.IsOk())
			{
				FIoBuffer Buffer = Status.ConsumeValueOrDie();
				void* Data = BulkData->ReallocateData(Buffer.GetSize());

				FMemoryReaderView Ar(Buffer.GetView(), true);
				BulkData->SerializeBulkData(Ar, Data, Buffer.GetSize(), EBulkDataFlags(BulkData->GetBulkDataFlags()));
			}
		},
		nullptr);

	++BatchCount;

	return *this;
}

FBulkDataBatchRequest::FBatchBuilder& FBulkDataBatchRequest::FBatchBuilder::Read(
	const FBulkData& BulkData,
	uint64 Offset,
	uint64 Size,
	EAsyncIOPriorityAndFlags Priority,
	FIoBuffer& Dst,
	FBulkDataBatchReadRequest* OutRequest)
{
	ensureMsgf(Size == MAX_uint64 || (Offset + Size) <= uint64(BulkData.GetBulkDataSize()),
		TEXT("%s: Trying to read past the end of the payload, Offset: %llu, ReadSize: %llu, Payload Size: %lld"),
		*BulkData.GetDebugName(),
		Size,
		Offset,
		BulkData.GetBulkDataSize());

	const uint64 ReadOffset = BulkData.GetBulkDataOffsetInFile() + Offset;
	const uint64 ReadSize	= FMath::Min(uint64(BulkData.GetBulkDataSize()), Size);

	checkf(Dst.GetSize() == 0 || Dst.GetSize() == ReadSize, TEXT("Buffer size (%llu) does not match the read size (%llu)"), Dst.GetSize(), Size);

	if (Dst.GetSize() == 0)
	{
		Dst = FIoBuffer(ReadSize);
	}

	GetBatch().Read(
		BulkData.BulkChunkId,
		FIoReadOptions(ReadOffset, ReadSize, Dst.GetData(), (Priority & AIOP_FLAG_HW_TARGET_MEMORY) ? EIoReadOptionsFlags::HardwareTargetBuffer : EIoReadOptionsFlags::None),
		Priority,
		FIoReadCallback(),
		OutRequest);
	
	++BatchCount;
	
	return *this;
}

void FBulkDataBatchRequest::FBatchBuilder::Issue(FCompletionCallback&& Callback, FBulkDataBatchRequest& OutRequest)
{
	if (NumLoaded > 0 && BatchCount == 0)
	{
		OutRequest = FBulkDataBatchRequest(new FHandleBase(EStatus::Ok));
		return;
	}

	IssueBatch(&OutRequest, MoveTemp(Callback));
}

void FBulkDataBatchRequest::FBatchBuilder::Issue(FBulkDataBatchRequest& OutRequest)
{
	Issue(FCompletionCallback(), OutRequest);
}

void FBulkDataBatchRequest::FBatchBuilder::Issue()
{
	check(NumLoaded > 0 || BatchCount > 0);

	if (NumLoaded > 0 && BatchCount == 0)
	{
		return;
	}

	IssueBatch(nullptr, FCompletionCallback());
}

FBulkDataBatchRequest::FScatterGatherBuilder::FScatterGatherBuilder(int32 MaxCount)
	: FBuilder(MaxCount)
{
	if (MaxCount > 0)
	{
		Requests.Reserve(MaxCount);
	}
}

FBulkDataBatchRequest::FScatterGatherBuilder& FBulkDataBatchRequest::FScatterGatherBuilder::Read(const FBulkData& BulkData, uint64 Offset, uint64 Size)
{
	check(Size == MAX_uint64 || Size <= uint64(BulkData.GetBulkDataSize()));

	const uint64 ReadOffset = BulkData.GetBulkDataOffsetInFile() + Offset;
	const uint64 ReadSize	= FMath::Min(uint64(BulkData.GetBulkDataSize()), Size);

	if (Requests.Num() > 0)
	{
		FRequest& Last = Requests.Last();

		const bool bContiguous =
			Last.Offset + Last.Size == ReadOffset &&
			Last.BulkData->GetBulkDataFlags() == BulkData.GetBulkDataFlags() &&
			Last.BulkData->BulkChunkId == BulkData.BulkChunkId;

		if (bContiguous)
		{
			Last.Size += ReadSize; 
			return *this;
		}
	}

	Requests.Add(FRequest {&BulkData, ReadOffset, ReadSize});

	return *this;
}

void FBulkDataBatchRequest::FScatterGatherBuilder::Issue(FIoBuffer& Dst, EAsyncIOPriorityAndFlags Priority, FCompletionCallback&& Callback, FBulkDataBatchRequest& OutRequest)
{
	check(Requests.IsEmpty() == false);

	uint64 TotalSize = 0;
	for (const FRequest& Request : Requests)
	{
		TotalSize += Request.Size;
	}

	check(Dst.GetSize() == 0 || Dst.GetSize() == TotalSize);

	if (Dst.GetSize() != TotalSize)
	{
		Dst = FIoBuffer(TotalSize);
	}

	FMutableMemoryView DstView = Dst.GetMutableView();
	for (const FRequest& Request : Requests)
	{
		GetBatch().Read(
			Request.BulkData->BulkChunkId,
			FIoReadOptions(Request.Offset, Request.Size, DstView.GetData()),
			Priority,
			FIoReadCallback(),
			nullptr);
		
		DstView.RightChopInline(Request.Size);
	}

	IssueBatch(&OutRequest, MoveTemp(Callback));
}

//////////////////////////////////////////////////////////////////////////////

#undef UE_ENABLE_BULKDATA_RANGE_TEST
