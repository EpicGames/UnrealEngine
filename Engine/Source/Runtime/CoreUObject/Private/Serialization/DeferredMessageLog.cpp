// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnAsyncLoading.cpp: Unreal async messsage log.
=============================================================================*/

#include "Serialization/DeferredMessageLog.h"
#include "Misc/ScopeLock.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"

TMap<FName, TArray<TSharedRef<FTokenizedMessage>>*> FDeferredMessageLog::Messages;
FCriticalSection FDeferredMessageLog::MessagesCritical;

FDeferredMessageLog::FDeferredMessageLog(const FName& InLogCategory)
: LogCategory(InLogCategory)
{
	FScopeLock MessagesLock(&MessagesCritical);
	TArray<TSharedRef<FTokenizedMessage>>** ExistingCategoryMessages = Messages.Find(LogCategory);
	if (!ExistingCategoryMessages)
	{
		TArray<TSharedRef<FTokenizedMessage>>* CategoryMessages = new TArray<TSharedRef<FTokenizedMessage>>();
		Messages.Add(LogCategory, CategoryMessages);
	}
}

void FDeferredMessageLog::AddMessage(TSharedRef<FTokenizedMessage>& Message)
{
	FScopeLock MessagesLock(&MessagesCritical);
	TArray<TSharedRef<FTokenizedMessage>>* CategoryMessages = Messages.FindRef(LogCategory);
	check(CategoryMessages);
	CategoryMessages->Add(Message);
}

TSharedRef<FTokenizedMessage> FDeferredMessageLog::Info(const FText& InMessage)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Info, InMessage);
	AddMessage(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FDeferredMessageLog::Warning(const FText& InMessage)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Warning, InMessage);
	AddMessage(Message);
	return Message;
}

TSharedRef<FTokenizedMessage> FDeferredMessageLog::Error(const FText& InMessage)
{
	TSharedRef<FTokenizedMessage> Message = FTokenizedMessage::Create(EMessageSeverity::Error, InMessage);
	AddMessage(Message);
	return Message;
}

void FDeferredMessageLog::Flush()
{
	FScopeLock MessagesLock(&MessagesCritical);
	for (auto& CategoryMessages : Messages)
	{
		if (CategoryMessages.Value->Num())
		{
			FMessageLog LoaderLog(CategoryMessages.Key);
			LoaderLog.AddMessages(*CategoryMessages.Value);
			CategoryMessages.Value->Empty(CategoryMessages.Value->Num());
		}
	}
}

void FDeferredMessageLog::Cleanup()
{
	for (auto& CategoryMessages : Messages)
	{
		delete CategoryMessages.Value;
	}
	Messages.Empty();
}
