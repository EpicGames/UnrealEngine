// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/DerivedData.h"

#include "Containers/ArrayView.h"
#include "Hash/xxhash.h"
#include "IO/IoDispatcher.h"
#include "Memory/SharedBuffer.h"
#include "Misc/StringBuilder.h"
#include "Misc/TVariant.h"
#include "String/BytesToHex.h"
#include <atomic>

#if WITH_EDITORONLY_DATA
#include "Compression/CompressedBuffer.h"
#include "Containers/SharedString.h"
#include "DerivedDataCache.h"
#include "DerivedDataCacheKey.h"
#include "DerivedDataRequestOwner.h"
#include "DerivedDataRequestTypes.h"
#include "DerivedDataValue.h"
#include "DerivedDataValueId.h"
#include "IO/IoHash.h"
#include "Memory/CompositeBuffer.h"
#include "UObject/LinkerSave.h"
#endif // WITH_EDITORONLY_DATA


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
// TEMPORARY SERIALIZATION AS BULK DATA
#if WITH_EDITORONLY_DATA
#include "Async/ManualResetEvent.h"
#include "Serialization/BulkData.h"
#endif // WITH_EDITORONLY_DATA
// TEMPORARY SERIALIZATION AS BULK DATA
///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

static_assert(sizeof(UE::FDerivedData) == 32 + sizeof(void*) * WITH_EDITORONLY_DATA);

namespace UE::DerivedData::Private { class FIoResponseDispatcher; }

namespace UE::DerivedData::Private
{

static FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FCookedData& CookedData)
{
	Builder << TEXTVIEW("Chunk: ID ");
	String::BytesToHexLower(MakeArrayView(CookedData.ChunkId), Builder);
	if (CookedData.ChunkOffset)
	{
		Builder << TEXTVIEW(" / Offset ") << CookedData.ChunkOffset;
	}
	if (CookedData.ChunkSize != MAX_uint64)
	{
		Builder << TEXTVIEW(" / Size ") << CookedData.ChunkSize;
	}
	return Builder;
}

bool FCookedData::ReferenceEquals(const FCookedData& Other) const
{
	return ChunkOffset == Other.ChunkOffset && ChunkSize == Other.ChunkSize && MakeArrayView(ChunkId) == Other.ChunkId;
}

uint32 FCookedData::ReferenceHash() const
{
	FXxHash64Builder Builder;
	Builder.Update(&ChunkOffset, sizeof(ChunkOffset));
	Builder.Update(&ChunkSize, sizeof(ChunkSize));
	Builder.Update(MakeMemoryView(ChunkId));
	return uint32(Builder.Finalize().Hash);
}

void FCookedData::Serialize(FArchive& Ar)
{
	Ar << ChunkOffset;
	Ar << ChunkSize;
	static_assert(sizeof(ChunkId) == sizeof(FIoChunkId));
	Ar.Serialize(ChunkId, sizeof(ChunkId));
	Ar << Flags;
}

} // UE::DerivedData::Private

#if WITH_EDITORONLY_DATA
namespace UE::DerivedData::Private
{

struct FCacheKeyWithId
{
	FCacheKey Key;
	FValueId Id;

	inline FCacheKeyWithId(const FCacheKey& InKey, const FValueId& InId)
		: Key(InKey)
		, Id(InId)
	{
	}

	inline bool operator==(const FCacheKeyWithId& Other) const
	{
		return Key == Other.Key && Id == Other.Id;
	}
};

struct FCompositeBufferWithHash
{
	FCompositeBuffer Buffer;
	FIoHash Hash;

	template <typename... ArgTypes>
	inline explicit FCompositeBufferWithHash(ArgTypes&&... Args)
		: Buffer(Forward<ArgTypes>(Args)...)
		, Hash(FIoHash::HashBuffer(Buffer))
	{
	}

	inline bool operator==(const FCompositeBufferWithHash& Other) const
	{
		return Hash == Other.Hash;
	}
};

class FEditorData
{
public:
	template <typename... ArgTypes>
	inline FEditorData(const FSharedString& InName, ArgTypes&&... Args)
		: Name(InName)
		, Data(Forward<ArgTypes>(Args)...)
	{
	}

	const FSharedString& GetName() const { return Name; }

	friend FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FEditorData& EditorData);

	bool ReferenceEquals(const FEditorData& Other) const;
	uint32 ReferenceHash() const;

	template <typename VisitorType>
	inline void Visit(VisitorType&& Visitor) const
	{
		::Visit(Visitor, Data);
	}

	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////
	// TEMPORARY SERIALIZATION AS BULK DATA
	FByteBulkData BulkData;
	// TEMPORARY SERIALIZATION AS BULK DATA
	///////////////////////////////////////////////////////////////////////////////////////////////////////////////////

private:
	FSharedString Name;
	TVariant<FCompositeBufferWithHash, FCompressedBuffer, FCacheKeyWithId> Data;
};

FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FEditorData& EditorData)
{
	struct FVisitor
	{
		inline void operator()(const FCompositeBufferWithHash& BufferWithHash) const
		{
			Builder
				<< TEXTVIEW("Buffer: Size ") << BufferWithHash.Buffer.GetSize()
				<< TEXTVIEW(" Hash ") << BufferWithHash.Hash;
		}

		inline void operator()(const FCompressedBuffer& Buffer) const
		{
			Builder
				<< TEXTVIEW("Buffer: CompressedSize ") << Buffer.GetCompressedSize()
				<< TEXTVIEW(" Size ") << Buffer.GetRawSize()
				<< TEXTVIEW(" Hash ") << Buffer.GetRawHash();
		}

		inline void operator()(const FCacheKeyWithId& CacheKeyWithId) const
		{
			Builder << TEXTVIEW("Cache: Key ") << CacheKeyWithId.Key;
			if (CacheKeyWithId.Id.IsValid())
			{
				Builder << TEXTVIEW(" ID ") << CacheKeyWithId.Id;
			}
		}

		FStringBuilderBase& Builder;
	};

	Visit(FVisitor{Builder}, EditorData.Data);
	if (!EditorData.Name.IsEmpty())
	{
		Builder << TEXTVIEW(" for ") << EditorData.Name;
	}
	return Builder;
}

bool FEditorData::ReferenceEquals(const FEditorData& Other) const
{
	if (Data.GetIndex() != Other.Data.GetIndex())
	{
		return false;
	}

	if (const FCompositeBufferWithHash* BufferWithHash = Data.TryGet<FCompositeBufferWithHash>())
	{
		return Other.Data.Get<FCompositeBufferWithHash>().Hash == BufferWithHash->Hash;
	}

	if (const FCompressedBuffer* Buffer = Data.TryGet<FCompressedBuffer>())
	{
		return Other.Data.Get<FCompressedBuffer>().GetRawHash() == Buffer->GetRawHash();
	}

	if (const FCacheKeyWithId* CacheKeyWithId = Data.TryGet<FCacheKeyWithId>())
	{
		return Other.Data.Get<FCacheKeyWithId>() == *CacheKeyWithId;
	}

	checkNoEntry();
	return false;
}

uint32 FEditorData::ReferenceHash() const
{
	struct FVisitor
	{
		inline void operator()(const FCompositeBufferWithHash& BufferWithHash)
		{
			Hash = GetTypeHash(BufferWithHash.Hash);
		}

		inline void operator()(const FCompressedBuffer& Buffer)
		{
			Hash = GetTypeHash(Buffer.GetRawHash());
		}

		inline void operator()(const FCacheKeyWithId& CacheKeyWithId)
		{
			Hash = HashCombineFast(GetTypeHash(CacheKeyWithId.Key), GetTypeHash(CacheKeyWithId.Id));
		}
		uint32 Hash = 0;
	};

	FVisitor Visitor;
	Visit(Visitor);
	return Visitor.Hash;
}

} // UE::DerivedData::Private
#endif // WITH_EDITORONLY_DATA

namespace UE
{

FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FDerivedData& Data)
{
#if WITH_EDITORONLY_DATA
	using namespace DerivedData::Private;
	if (Data.EditorData)
	{
		return Builder << *Data.EditorData;
	}
#endif

	if (Data.CookedData)
	{
		return Builder << Data.CookedData;
	}

	return Builder << TEXTVIEW("Null");
}

bool FDerivedData::ReferenceEquals(const FDerivedData& Other) const
{
#if WITH_EDITORONLY_DATA
	if (EditorData && Other.EditorData)
	{
		return EditorData->ReferenceEquals(*Other.EditorData);
	}
#endif
	return CookedData.ReferenceEquals(Other.CookedData);
}

uint32 FDerivedData::ReferenceHash() const
{
#if WITH_EDITORONLY_DATA
	if (EditorData)
	{
		return EditorData->ReferenceHash();
	}
#endif
	return CookedData.ReferenceHash();
}

void FDerivedData::Serialize(FArchive& Ar, UObject* Owner)
{
	using namespace DerivedData::Private;

	if (!Ar.IsPersistent() || Ar.IsObjectReferenceCollector() || Ar.ShouldSkipBulkData())
	{
		return;
	}

#if WITH_EDITORONLY_DATA
	const EDerivedDataFlags Flags = GetFlags();
	if (EditorData && EnumHasAnyFlags(Flags, EDerivedDataFlags::Required | EDerivedDataFlags::Optional))
	{
		checkf(Ar.IsSaving() && Ar.IsCooking(),
			TEXT("FEditorData for FDerivedData only supports saving to cooked packages. %s"), *WriteToString<256>(*this));
		FLinkerSave* Linker = Cast<FLinkerSave>(Ar.GetLinker());
		checkf(Linker, TEXT("Serializing FDerivedData requires a linker. %s"), *WriteToString<256>(*this));

		///////////////////////////////////////////////////////////////////////////////////////////////////////////////
		// TEMPORARY SERIALIZATION AS BULK DATA
#if 1
		FCookedData LocalCookedData = Linker->AddDerivedData(*this).CookedData;
		LocalCookedData.ChunkId[8] = 0;
		LocalCookedData.ChunkId[9] = 0;
		LocalCookedData.ChunkId[10] = 0;
		LocalCookedData.ChunkId[11] = uint8(EIoChunkType::BulkData);

		uint32 BulkDataFlags = BULKDATA_PayloadAtEndOfFile | BULKDATA_Size64Bit | BULKDATA_NoOffsetFixUp;
		// Required AND Optional is the equivalent of DuplicateNonOptional, which is implemented by writing appending
		// the bulk data twice, but that may have different offsets, and this stores one offset. Only save as Required.
		if (EnumHasAnyFlags(GetFlags(), EDerivedDataFlags::Optional) && !EnumHasAnyFlags(Flags, EDerivedDataFlags::Required))
		{
			BulkDataFlags |= BULKDATA_OptionalPayload;
			LocalCookedData.ChunkId[11] = uint8(EIoChunkType::OptionalBulkData);
		}
		// Bulk Data does not support memory-mapped optional data. Optional takes priority over memory-mapped here.
		else if (EnumHasAnyFlags(Flags, EDerivedDataFlags::MemoryMapped))
		{
			BulkDataFlags |= BULKDATA_MemoryMappedPayload;
			LocalCookedData.ChunkId[11] = uint8(EIoChunkType::MemoryMappedBulkData);
		}

		// Blocking read to populate the bulk data to be serialized later by the linker.
		{
			FManualResetEvent BatchComplete;
			FDerivedDataIoBatch Batch;
			FDerivedDataIoRequest Request = Batch.Read(*this);
			FDerivedDataIoResponse Response;
			Batch.Dispatch(Response, [&BatchComplete] { BatchComplete.Notify(); });
			BatchComplete.Wait();

			const FSharedBuffer Data = Response.GetData(Request);
			checkf(Data, TEXT("FDerivedData failed to save because data was not fetched. %s"), *WriteToString<256>(*this));

			FByteBulkData& BulkData = EditorData->BulkData;
			BulkData.Lock(LOCK_READ_WRITE);
			MakeMemoryView(BulkData.Realloc(Data.GetSize()), Data.GetSize()).CopyFrom(Data);
			BulkData.Unlock();
			BulkData.SetBulkDataFlags(BulkDataFlags | BULKDATA_Force_NOT_InlinePayload);
		}

		const int32 ElementSize = 1;
		const bool bAttemptFileMapping = false;
		const EFileRegionType FileRegionType = EFileRegionType::None;
		Ar.SerializeBulkData(EditorData->BulkData, FBulkDataSerializationParams {Owner, ElementSize, FileRegionType, bAttemptFileMapping});

		return LocalCookedData.Serialize(Ar);

		// TEMPORARY SERIALIZATION AS BULK DATA
		///////////////////////////////////////////////////////////////////////////////////////////////////////////////
#else
		return Linker->AddDerivedData(*this).CookedData.Serialize(Ar);
#endif
	}
#endif

	CookedData.Serialize(Ar);
}

#if WITH_EDITORONLY_DATA

FDerivedData::FDerivedData(const FSharedString& Name, const FSharedBuffer& Data)
	: EditorData(MakePimpl<DerivedData::Private::FEditorData, EPimplPtrMode::DeepCopy>(
		Name, TInPlaceType<DerivedData::Private::FCompositeBufferWithHash>(), Data))
{
	CookedData.Flags = EDerivedDataFlags::Required;
}

FDerivedData::FDerivedData(const FSharedString& Name, const FCompositeBuffer& Data)
	: EditorData(MakePimpl<DerivedData::Private::FEditorData, EPimplPtrMode::DeepCopy>(
		Name, TInPlaceType<DerivedData::Private::FCompositeBufferWithHash>(), Data))
{
	CookedData.Flags = EDerivedDataFlags::Required;
}

FDerivedData::FDerivedData(const FSharedString& Name, const FCompressedBuffer& Data)
	: EditorData(MakePimpl<DerivedData::Private::FEditorData, EPimplPtrMode::DeepCopy>(
		Name, TInPlaceType<FCompressedBuffer>(), Data))
{
	CookedData.Flags = EDerivedDataFlags::Required;
}

FDerivedData::FDerivedData(const FSharedString& Name, const DerivedData::FCacheKey& Key)
	: EditorData(MakePimpl<DerivedData::Private::FEditorData, EPimplPtrMode::DeepCopy>(
		Name, TInPlaceType<DerivedData::Private::FCacheKeyWithId>(), Key, DerivedData::FValueId::Null))
{
	CookedData.Flags = EDerivedDataFlags::Required;
}

FDerivedData::FDerivedData(
	const FSharedString& Name,
	const DerivedData::FCacheKey& Key,
	const DerivedData::FValueId& ValueId)
	: EditorData(MakePimpl<DerivedData::Private::FEditorData, EPimplPtrMode::DeepCopy>(
		Name, TInPlaceType<DerivedData::Private::FCacheKeyWithId>(), Key, ValueId))
{
	CookedData.Flags = EDerivedDataFlags::Required;
}

const FSharedString& FDerivedData::GetName() const
{
	return EditorData ? EditorData->GetName() : FSharedString::Empty;
}

void FDerivedData::SetFlags(EDerivedDataFlags InFlags)
{
	CookedData.Flags = InFlags;
}

#endif // WITH_EDITORONLY_DATA

} // UE

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::DerivedData::Private
{

#if WITH_EDITORONLY_DATA
static inline EDerivedDataIoStatus ConvertToIoStatus(EStatus Status)
{
	switch (Status)
	{
	case EStatus::Ok:
		return EDerivedDataIoStatus::Ok;
	case EStatus::Error:
		return EDerivedDataIoStatus::Error;
	case EStatus::Canceled:
		return EDerivedDataIoStatus::Canceled;
	default:
		return EDerivedDataIoStatus::Unknown;
	}
}

static inline EPriority ConvertToDerivedDataPriority(FDerivedDataIoPriority Priority)
{
	if (Priority == FDerivedDataIoPriority::Blocking())
	{
		return EPriority::Blocking;
	}
	if (FDerivedDataIoPriority::Highest().InterpolateTo(FDerivedDataIoPriority::High(), 0.8f) < Priority)
	{
		return EPriority::Highest;
	}
	if (FDerivedDataIoPriority::High().InterpolateTo(FDerivedDataIoPriority::Normal(), 0.6f) < Priority)
	{
		return EPriority::High;
	}
	if (FDerivedDataIoPriority::Normal().InterpolateTo(FDerivedDataIoPriority::Low(), 0.4f) < Priority)
	{
		return EPriority::Normal;
	}
	if (FDerivedDataIoPriority::Low().InterpolateTo(FDerivedDataIoPriority::Lowest(), 0.2f) < Priority)
	{
		return EPriority::Low;
	}
	else
	{
		return EPriority::Lowest;
	}
}
#endif

static inline int32 ConvertToIoDispatcherPriority(FDerivedDataIoPriority Priority)
{
	static_assert(IoDispatcherPriority_Min == FDerivedDataIoPriority::Lowest().Value);
	static_assert(IoDispatcherPriority_Max == FDerivedDataIoPriority::Blocking().Value);
	static_assert(IoDispatcherPriority_Medium == FDerivedDataIoPriority::Normal().Value);
	return Priority.Value;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

enum class EIoRequestType : uint8
{
	Read,
	Cache,
	Exists,
#if WITH_EDITORONLY_DATA
	Compress,
#endif
};

struct FIoCookedState
{
	FIoRequest Request;
	FCookedData CookedData;
	std::atomic<bool> bCanceled = false;

	inline explicit FIoCookedState(const FCookedData& InCookedData)
		: CookedData(InCookedData)
	{
	}
};

struct FIoEditorState
{
#if WITH_EDITORONLY_DATA
	FEditorData EditorData;
	FCacheKey CacheKey;
	FValueId CacheValueId;
	FIoHash Hash;
	FCompressedBuffer CompressedData;

	inline explicit FIoEditorState(const FEditorData& InEditorData)
		: EditorData(InEditorData)
	{
	}
#endif
};

struct FIoRequestState
{
	TVariant<TYPE_OF_NULLPTR, FIoCookedState, FIoEditorState> State;
	FDerivedDataIoOptions Options;
	FSharedBuffer Data;
	uint64 Size = 0;
	EIoRequestType Type = EIoRequestType::Read;
	std::atomic<EDerivedDataIoStatus> Status = EDerivedDataIoStatus::Unknown;

	inline void SetStatus(EDerivedDataIoStatus NewStatus)
	{
		Status.store(NewStatus, std::memory_order_release);
	}
};

class FIoResponse
{
public:
	static FDerivedDataIoRequest Queue(
		TPimplPtr<FIoResponse>& Self,
		const FDerivedData& Data,
		const FDerivedDataIoOptions& Options,
		EIoRequestType Type);

	static void Dispatch(
		TPimplPtr<FIoResponse>& Self,
		FDerivedDataIoResponse& OutResponse,
		FDerivedDataIoPriority Priority,
		FDerivedDataIoComplete&& OnComplete);

	static inline FIoRequestState* TryGetRequest(const TPimplPtr<FIoResponse>& Self, const FDerivedDataIoRequest Handle)
	{
		return Self && Self->Requests.IsValidIndex(Handle.Index) ? &Self->Requests[Handle.Index] : nullptr;
	}

	~FIoResponse();

	void SetPriority(FDerivedDataIoPriority Priority);
	bool Cancel();
	bool Poll() const;

	EDerivedDataIoStatus GetOverallStatus() const;

private:
	inline void BeginRequest() { RemainingRequests.fetch_add(1, std::memory_order_relaxed); }
	void EndRequest();

#if WITH_EDITORONLY_DATA
	FRequestOwner Owner{EPriority::Normal};
#endif
	TArray<FIoRequestState> Requests;
	std::atomic<uint32> RemainingRequests = 0;
	std::atomic<EDerivedDataIoStatus> OverallStatus = EDerivedDataIoStatus::Unknown;
	FDerivedDataIoComplete ResponseComplete;

	friend FIoResponseDispatcher;
};

class FIoResponseDispatcher
{
public:
	static void Dispatch(FIoResponse& Response, FDerivedDataIoPriority Priority);

private:
	FIoResponseDispatcher() = default;

	void DispatchCooked(FIoResponse& Response, FIoRequestState& Request, FIoCookedState& State);

	static FIoReadOptions MakeIoReadOptions(const FIoCookedState& State, const FDerivedDataIoOptions& Options);
	static void OnIoRequestComplete(FIoResponse& Response, FIoRequestState& Request, TIoStatusOr<FIoBuffer> Buffer);

#if WITH_EDITORONLY_DATA
	void DispatchEditor(FIoResponse& Response, FIoRequestState& Request, FIoEditorState& Editor, int32 RequestIndex);
	static void OnCacheRecordRequestComplete(FIoResponse& Response, FCacheGetResponse&& CacheResponse);
	static void OnCacheValueRequestComplete(FIoResponse& Response, FCacheGetValueResponse&& CacheResponse);
	static void OnCacheChunkRequestComplete(FIoResponse& Response, FCacheGetChunkResponse&& CacheResponse);

	TArray<FCacheGetRequest> CacheRecordRequests;
	TArray<FCacheGetValueRequest> CacheValueRequests;
	TArray<FCacheGetChunkRequest> CacheChunkRequests;
#endif

	FIoBatch Batch;
	FDerivedDataIoPriority Priority;
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDerivedDataIoRequest FIoResponse::Queue(
	TPimplPtr<FIoResponse>& Self,
	const FDerivedData& Data,
	const FDerivedDataIoOptions& Options,
	const EIoRequestType Type)
{
	if (!Self)
	{
		Self = MakePimpl<FIoResponse>();
	}

	const int32 Index = Self->Requests.AddDefaulted();
	FIoRequestState& RequestState = Self->Requests[Index];
	RequestState.Options = Options;
	RequestState.Type = Type;

	FDerivedDataIoRequest Handle;
	Handle.Index = Index;

#if WITH_EDITORONLY_DATA
	if (Data.EditorData)
	{
		RequestState.State.Emplace<FIoEditorState>(*Data.EditorData);
	}
	else
#endif
	if (Data.CookedData)
	{
		RequestState.State.Emplace<FIoCookedState>(Data.CookedData);
	}

	return Handle;
}

void FIoResponse::Dispatch(
	TPimplPtr<FIoResponse>& InResponse,
	FDerivedDataIoResponse& OutResponse,
	FDerivedDataIoPriority Priority,
	FDerivedDataIoComplete&& OnComplete)
{
	// An empty batch completes immediately.
	if (!InResponse)
	{
		OutResponse.Reset();
		if (OnComplete)
		{
			Invoke(OnComplete);
		}
	}

	// Assign OutResponse early because OnComplete may reference it.
	FIoResponse& Self = *InResponse;
	Self.ResponseComplete = MoveTemp(OnComplete);
	OutResponse.Response = MoveTemp(InResponse);

	// The Begin/End pair blocks completion until dispatch is complete.
	Self.BeginRequest();
	FIoResponseDispatcher::Dispatch(Self, Priority);
	Self.EndRequest();
}

FIoResponse::~FIoResponse()
{
	verifyf(Cancel(), TEXT("Requests must be complete before the response is destroyed but it has %u remaining."),
		RemainingRequests.load(std::memory_order_relaxed));
}

void FIoResponse::SetPriority(FDerivedDataIoPriority Priority)
{
#if WITH_EDITORONLY_DATA
	Owner.SetPriority(ConvertToDerivedDataPriority(Priority));
#endif

	const int32 IoDispatcherPriority = ConvertToIoDispatcherPriority(Priority);
	for (FIoRequestState& Request : Requests)
	{
		if (FIoCookedState* CookedState = Request.State.TryGet<FIoCookedState>())
		{
			CookedState->Request.UpdatePriority(IoDispatcherPriority);
		}
	}
}

bool FIoResponse::Cancel()
{
	if (Poll())
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	// Request cancellation is synchronous but is expected to be very fast.
	Owner.Cancel();
#endif

	for (FIoRequestState& Request : Requests)
	{
		if (FIoCookedState* CookedState = Request.State.TryGet<FIoCookedState>())
		{
			// Request cancellation only once because every call wakes the dispatcher.
			if (!CookedState->bCanceled.exchange(true, std::memory_order_relaxed))
			{
				CookedState->Request.Cancel();
			}
		}
	}

	return Poll();
}

bool FIoResponse::Poll() const
{
	return OverallStatus.load(std::memory_order_relaxed) != EDerivedDataIoStatus::Unknown;
}

EDerivedDataIoStatus FIoResponse::GetOverallStatus() const
{
	return OverallStatus.load(std::memory_order_relaxed);
}

void FIoResponse::EndRequest()
{
	if (RemainingRequests.fetch_sub(1, std::memory_order_acq_rel) == 1)
	{
		// Calculate the overall status for the response.
		static_assert(EDerivedDataIoStatus::Ok < EDerivedDataIoStatus::Error);
		static_assert(EDerivedDataIoStatus::Error < EDerivedDataIoStatus::Canceled);
		static_assert(EDerivedDataIoStatus::Canceled < EDerivedDataIoStatus::Unknown);
		EDerivedDataIoStatus Status = EDerivedDataIoStatus::Ok;
		for (FIoRequestState& Request : Requests)
		{
			Status = FMath::Max(Status, Request.Status.load(std::memory_order_relaxed));
		}
		OverallStatus.store(Status, std::memory_order_relaxed);

		// Invoke the completion callback, but move it to the stack first because it may delete the response.
		if (FDerivedDataIoComplete OnComplete = MoveTemp(ResponseComplete))
		{
			Invoke(OnComplete);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FIoResponseDispatcher::Dispatch(FIoResponse& Response, FDerivedDataIoPriority Priority)
{
#if WITH_EDITORONLY_DATA
	FRequestBarrier Barrier(Response.Owner);
#endif

	FIoResponseDispatcher Dispatcher;
	Dispatcher.Priority = Priority;

	int32 RequestIndex = -1;
	for (FIoRequestState& Request : Response.Requests)
	{
		++RequestIndex;
		if (FIoCookedState* CookedState = Request.State.TryGet<FIoCookedState>())
		{
			Dispatcher.DispatchCooked(Response, Request, *CookedState);
		}
	#if WITH_EDITORONLY_DATA
		else if (FIoEditorState* EditorState = Request.State.TryGet<FIoEditorState>())
		{
			Dispatcher.DispatchEditor(Response, Request, *EditorState, RequestIndex);
		}
	#endif
		else
		{
			Request.SetStatus(EDerivedDataIoStatus::Error);
		}
	}

	Dispatcher.Batch.Issue();

#if WITH_EDITORONLY_DATA
	if (!Dispatcher.CacheRecordRequests.IsEmpty())
	{
		GetCache().Get(Dispatcher.CacheRecordRequests, Response.Owner,
			[Response = &Response](FCacheGetResponse&& CacheResponse)
			{
				OnCacheRecordRequestComplete(*Response, MoveTemp(CacheResponse));
			});
	}

	if (!Dispatcher.CacheValueRequests.IsEmpty())
	{
		GetCache().GetValue(Dispatcher.CacheValueRequests, Response.Owner,
			[Response = &Response](FCacheGetValueResponse&& CacheResponse)
			{
				OnCacheValueRequestComplete(*Response, MoveTemp(CacheResponse));
			});
	}

	if (!Dispatcher.CacheChunkRequests.IsEmpty())
	{
		GetCache().GetChunks(Dispatcher.CacheChunkRequests, Response.Owner,
			[Response = &Response](FCacheGetChunkResponse&& CacheResponse)
			{
				OnCacheChunkRequestComplete(*Response, MoveTemp(CacheResponse));
			});
	}
#endif
}

void FIoResponseDispatcher::DispatchCooked(FIoResponse& Response, FIoRequestState& Request, FIoCookedState& State)
{
	FIoChunkId ChunkId;
	ChunkId.Set(MakeMemoryView(State.CookedData.ChunkId));
	if (Request.Type == EIoRequestType::Read)
	{
		Response.BeginRequest();
		State.Request = Batch.ReadWithCallback(
			ChunkId,
			MakeIoReadOptions(State, Request.Options),
			ConvertToIoDispatcherPriority(Priority),
			[Response = &Response, &Request](TIoStatusOr<FIoBuffer> Buffer)
			{
				OnIoRequestComplete(*Response, Request, MoveTemp(Buffer));
			});
	}
	else
	{
		TIoStatusOr<uint64> Size = FIoDispatcher::Get().GetSizeForChunk(ChunkId);
		if (Size.IsOk())
		{
			const uint64 TotalSize = Size.ValueOrDie();
			const uint64 RequestOffset = Request.Options.GetOffset();
			const uint64 AvailableSize = RequestOffset <= TotalSize ? TotalSize - RequestOffset : 0;
			Request.Size = FMath::Min(Request.Options.GetSize(), AvailableSize);
			Request.SetStatus(EDerivedDataIoStatus::Ok);
		}
		else
		{
			Request.SetStatus(EDerivedDataIoStatus::Error);
		}
	}
}

FIoReadOptions FIoResponseDispatcher::MakeIoReadOptions(
	const FIoCookedState& State,
	const FDerivedDataIoOptions& Options)
{
	const FCookedData& CookedData = State.CookedData;
	const uint64 LocalOffset = Options.GetOffset();
	const uint64 TotalOffset = CookedData.ChunkOffset + LocalOffset;

	FIoReadOptions ReadOptions;
	ReadOptions.SetTargetVa(Options.GetTarget());

	if (Options.GetSize() == MAX_uint64)
	{
		if (CookedData.ChunkSize == MAX_uint64)
		{
			ReadOptions.SetRange(TotalOffset, MAX_uint64);
		}
		else
		{
			ReadOptions.SetRange(TotalOffset, LocalOffset <= CookedData.ChunkSize ? CookedData.ChunkSize - LocalOffset : 0);
		}
	}
	else
	{
		ReadOptions.SetRange(TotalOffset, Options.GetSize());
	}

	return ReadOptions;
}

void FIoResponseDispatcher::OnIoRequestComplete(
	FIoResponse& Response,
	FIoRequestState& Request,
	TIoStatusOr<FIoBuffer> StatusOrBuffer)
{
	FIoCookedState& State = Request.State.Get<FIoCookedState>();

	if (StatusOrBuffer.IsOk())
	{
		FIoBuffer Data = StatusOrBuffer.ConsumeValueOrDie();
		const uint64 DataSize = Data.GetSize();

		// Return a view of the target when one was provided, otherwise take ownership of the buffer.
		if (Request.Options.GetTarget())
		{
			Request.Data = FSharedBuffer::MakeView(Request.Options.GetTarget(), DataSize);
		}
		else
		{
			Request.Data = FSharedBuffer::TakeOwnership(Data.Release().ConsumeValueOrDie(), DataSize, FMemory::Free);
		}

		Request.Size = DataSize;

		Request.SetStatus(EDerivedDataIoStatus::Ok);
	}
	else
	{
		const bool bCanceled = StatusOrBuffer.Status().GetErrorCode() == EIoErrorCode::Cancelled;
		Request.SetStatus(bCanceled ? EDerivedDataIoStatus::Canceled : EDerivedDataIoStatus::Error);
	}

	State.Request.Release();
	Response.EndRequest();
}

#if WITH_EDITORONLY_DATA

void FIoResponseDispatcher::DispatchEditor(
	FIoResponse& Response,
	FIoRequestState& Request,
	FIoEditorState& Editor,
	int32 RequestIndex)
{
	struct FVisitor
	{
		void operator()(const FCompositeBufferWithHash& BufferWithHash) const
		{
			const uint64 TotalSize = BufferWithHash.Buffer.GetSize();
			Editor->Hash = BufferWithHash.Hash;

			const uint64 RequestOffset = Request->Options.GetOffset();
			const uint64 AvailableSize = RequestOffset <= TotalSize ? TotalSize - RequestOffset : 0;
			const uint64 RequestSize = FMath::Min(Request->Options.GetSize(), AvailableSize);
			Request->Size = RequestSize;

			if (Request->Type == EIoRequestType::Read)
			{
				auto Execute = [Response = Response, Request = Request, &BufferWithHash, RequestSize]
				{
					if (void* Target = Request->Options.GetTarget())
					{
						const FMutableMemoryView TargetView(Target, RequestSize);
						BufferWithHash.Buffer.CopyTo(TargetView, Request->Options.GetOffset());
						Request->Data = FSharedBuffer::MakeView(TargetView);
					}
					else
					{
						Request->Data = BufferWithHash.Buffer.Mid(Request->Options.GetOffset(), RequestSize).ToShared();
					}
					Request->SetStatus(EDerivedDataIoStatus::Ok);
					Response->EndRequest();
				};

				// Execute small copy tasks inline to avoid task overhead.
				Response->BeginRequest();
				if (RequestSize <= 64 * 1024)
				{
					Execute();
				}
				else
				{
					FRequestBarrier Barrier(Response->Owner);
					Response->Owner.LaunchTask(TEXT("DerivedDataCopy"), MoveTemp(Execute));
				}
			}
			else if (Request->Type == EIoRequestType::Compress)
			{
				Response->BeginRequest();
				FRequestBarrier Barrier(Response->Owner);
				Response->Owner.LaunchTask(TEXT("DerivedDataCompress"), [Response = Response, Request = Request, &BufferWithHash]
				{
					Request->State.Get<FIoEditorState>().CompressedData = FValue::Compress(BufferWithHash.Buffer).GetData();
					Request->SetStatus(EDerivedDataIoStatus::Ok);
					Response->EndRequest();
				});
			}
			else
			{
				Request->SetStatus(EDerivedDataIoStatus::Ok);
			}
		}

		void operator()(const FCompressedBuffer& Buffer) const
		{
			const uint64 TotalSize = Buffer.GetRawSize();
			Editor->Hash = Buffer.GetRawHash();

			const uint64 RequestOffset = Request->Options.GetOffset();
			const uint64 AvailableSize = RequestOffset <= TotalSize ? TotalSize - RequestOffset : 0;
			const uint64 RequestSize = FMath::Min(Request->Options.GetSize(), AvailableSize);
			Request->Size = RequestSize;

			if (Request->Type == EIoRequestType::Read)
			{
				auto Execute = [Response = Response, Request = Request, &Buffer, RequestSize]
				{
					FCompressedBufferReader Reader(Buffer);
					if (void* Target = Request->Options.GetTarget())
					{
						const FMutableMemoryView TargetView(Target, RequestSize);
						if (Reader.TryDecompressTo(TargetView, Request->Options.GetOffset()))
						{
							Request->Data = FSharedBuffer::MakeView(TargetView);
						}
					}
					else
					{
						Request->Data = Reader.Decompress(Request->Options.GetOffset(), RequestSize);
					}
					Request->SetStatus(Request->Data ? EDerivedDataIoStatus::Ok : EDerivedDataIoStatus::Error);
					Response->EndRequest();
				};

				// Execute small decompression tasks inline to avoid task overhead.
				Response->BeginRequest();
				if (RequestSize <= 16 * 1024)
				{
					Execute();
				}
				else
				{
					FRequestBarrier Barrier(Response->Owner);
					Response->Owner.LaunchTask(TEXT("DerivedDataDecompress"), MoveTemp(Execute));
				}
			}
			else if (Request->Type == EIoRequestType::Compress)
			{
				Request->State.Get<FIoEditorState>().CompressedData = Buffer;
				Request->SetStatus(EDerivedDataIoStatus::Ok);
			}
			else
			{
				Request->SetStatus(EDerivedDataIoStatus::Ok);
			}
		}

		void operator()(const FCacheKeyWithId& CacheKeyWithId) const
		{
			Editor->CacheKey = CacheKeyWithId.Key;
			Editor->CacheValueId = CacheKeyWithId.Id;

			if (Request->Type == EIoRequestType::Compress)
			{
				if (CacheKeyWithId.Id.IsValid())
				{
					// Requesting multiple values from the same record can be optimized by making
					// one request for the record with a policy that requests each of the values,
					// rather than one record request for each value.

					FCacheRecordPolicyBuilder Policy(ECachePolicy::None | ECachePolicy::SkipMeta);
					Policy.AddValuePolicy(CacheKeyWithId.Id, ECachePolicy::Default);

					FCacheGetRequest& CacheRequest = Dispatcher->CacheRecordRequests.AddDefaulted_GetRef();
					CacheRequest.Name = Editor->EditorData.GetName();
					CacheRequest.Key = CacheKeyWithId.Key;
					CacheRequest.Policy = Policy.Build();
					CacheRequest.UserData = uint64(RequestIndex);
				}
				else
				{
					FCacheGetValueRequest& CacheRequest = Dispatcher->CacheValueRequests.AddDefaulted_GetRef();
					CacheRequest.Name = Editor->EditorData.GetName();
					CacheRequest.Key = CacheKeyWithId.Key;
					CacheRequest.UserData = uint64(RequestIndex);
				}
			}
			else
			{
				FCacheGetChunkRequest& CacheRequest = Dispatcher->CacheChunkRequests.AddDefaulted_GetRef();
				CacheRequest.Name = Editor->EditorData.GetName();
				CacheRequest.Key = CacheKeyWithId.Key;
				CacheRequest.Id = CacheKeyWithId.Id;
				CacheRequest.RawOffset = Request->Options.GetOffset();
				CacheRequest.RawSize = Request->Options.GetSize();
				CacheRequest.Policy =
					Request->Type == EIoRequestType::Read  ? (ECachePolicy::Default) :
					Request->Type == EIoRequestType::Cache ? (ECachePolicy::Default | ECachePolicy::SkipData) :
															 (ECachePolicy::Query   | ECachePolicy::SkipData);
				CacheRequest.UserData = uint64(RequestIndex);
			}

			Response->BeginRequest();
		}

		FIoResponseDispatcher* Dispatcher;
		FIoResponse* Response;
		FIoRequestState* Request;
		FIoEditorState* Editor;
		int32 RequestIndex;
	};

	FVisitor Visitor{this, &Response, &Request, &Editor, RequestIndex};
	Editor.EditorData.Visit(Visitor);
}

void FIoResponseDispatcher::OnCacheRecordRequestComplete(FIoResponse& Response, FCacheGetResponse&& CacheResponse)
{
	FIoRequestState& Request = Response.Requests[int32(CacheResponse.UserData)];
	FIoEditorState& Editor = Request.State.Get<FIoEditorState>();
	const FValue& Value = CacheResponse.Record.GetValue(Editor.CacheValueId);
	Editor.CompressedData = Value.GetData();
	Editor.Hash = Value.GetRawHash();
	Request.Size = Value.GetRawSize();
	Request.SetStatus(ConvertToIoStatus(CacheResponse.Status));
	Response.EndRequest();
}

void FIoResponseDispatcher::OnCacheValueRequestComplete(FIoResponse& Response, FCacheGetValueResponse&& CacheResponse)
{
	FIoRequestState& Request = Response.Requests[int32(CacheResponse.UserData)];
	FIoEditorState& Editor = Request.State.Get<FIoEditorState>();
	Editor.CompressedData = CacheResponse.Value.GetData();
	Editor.Hash = CacheResponse.Value.GetRawHash();
	Request.Size = CacheResponse.Value.GetRawSize();
	Request.SetStatus(ConvertToIoStatus(CacheResponse.Status));
	Response.EndRequest();
}

void FIoResponseDispatcher::OnCacheChunkRequestComplete(FIoResponse& Response, FCacheGetChunkResponse&& CacheResponse)
{
	FIoRequestState& Request = Response.Requests[int32(CacheResponse.UserData)];
	FIoEditorState& Editor = Request.State.Get<FIoEditorState>();
	Editor.Hash = CacheResponse.RawHash;
	Request.Size = CacheResponse.RawSize;
	Request.Data = MoveTemp(CacheResponse.RawData);
	Request.SetStatus(ConvertToIoStatus(CacheResponse.Status));
	Response.EndRequest();
}

#endif // WITH_EDITORONLY_DATA

} // UE::DerivedData::Private

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE
{

void FDerivedDataIoResponse::SetPriority(FDerivedDataIoPriority Priority)
{
	if (Response)
	{
		Response->SetPriority(Priority);
	}
}

bool FDerivedDataIoResponse::Cancel()
{
	return Response ? Response->Cancel() : true;
}

bool FDerivedDataIoResponse::Poll() const
{
	return Response ? Response->Poll() : true;
}

EDerivedDataIoStatus FDerivedDataIoResponse::GetOverallStatus() const
{
	return Response ? Response->GetOverallStatus() : EDerivedDataIoStatus::Ok;
}

EDerivedDataIoStatus FDerivedDataIoResponse::GetStatus(FDerivedDataIoRequest Handle) const
{
	using namespace DerivedData::Private;
	if (FIoRequestState* Request = FIoResponse::TryGetRequest(Response, Handle))
	{
		return Request->Status.load(std::memory_order_relaxed);
	}
	return EDerivedDataIoStatus::Error;
}

FSharedBuffer FDerivedDataIoResponse::GetData(FDerivedDataIoRequest Handle) const
{
	using namespace DerivedData::Private;
	if (FIoRequestState* Request = FIoResponse::TryGetRequest(Response, Handle))
	{
		if (Request->Status.load(std::memory_order_acquire) == EDerivedDataIoStatus::Ok)
		{
			return Request->Data;
		}
	}
	return FSharedBuffer();
}

uint64 FDerivedDataIoResponse::GetSize(FDerivedDataIoRequest Handle) const
{
	using namespace DerivedData::Private;
	if (FIoRequestState* Request = FIoResponse::TryGetRequest(Response, Handle))
	{
		if (Request->Status.load(std::memory_order_acquire) == EDerivedDataIoStatus::Ok)
		{
			return Request->Size;
		}
	}
	return 0;
}

#if WITH_EDITORONLY_DATA
const FIoHash* FDerivedDataIoResponse::GetHash(FDerivedDataIoRequest Handle) const
{
	using namespace DerivedData::Private;
	if (FIoRequestState* Request = FIoResponse::TryGetRequest(Response, Handle))
	{
		if (Request->Status.load(std::memory_order_acquire) == EDerivedDataIoStatus::Ok)
		{
			if (FIoEditorState* EditorState = Request->State.TryGet<FIoEditorState>())
			{
				return &EditorState->Hash;
			}
		}
	}
	return nullptr;
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
const DerivedData::FCacheKey* FDerivedDataIoResponse::GetCacheKey(FDerivedDataIoRequest Handle) const
{
	using namespace DerivedData;
	using namespace DerivedData::Private;
	if (FIoRequestState* Request = FIoResponse::TryGetRequest(Response, Handle))
	{
		if (Request->Status.load(std::memory_order_acquire) == EDerivedDataIoStatus::Ok)
		{
			if (FIoEditorState* EditorState = Request->State.TryGet<FIoEditorState>())
			{
				if (EditorState->CacheKey != FCacheKey::Empty)
				{
					return &EditorState->CacheKey;
				}
			}
		}
	}
	return nullptr;
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
const DerivedData::FValueId* FDerivedDataIoResponse::GetCacheValueId(FDerivedDataIoRequest Handle) const
{
	using namespace DerivedData;
	using namespace DerivedData::Private;
	if (FIoRequestState* Request = FIoResponse::TryGetRequest(Response, Handle))
	{
		if (Request->Status.load(std::memory_order_acquire) == EDerivedDataIoStatus::Ok)
		{
			if (FIoEditorState* EditorState = Request->State.TryGet<FIoEditorState>())
			{
				if (EditorState->CacheValueId != FValueId::Null)
				{
					return &EditorState->CacheValueId;
				}
			}
		}
	}
	return nullptr;
}
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITORONLY_DATA
const FCompressedBuffer* FDerivedDataIoResponse::GetCompressedData(FDerivedDataIoRequest Handle) const
{
	using namespace DerivedData::Private;
	if (FIoRequestState* Request = FIoResponse::TryGetRequest(Response, Handle))
	{
		if (Request->Status.load(std::memory_order_acquire) == EDerivedDataIoStatus::Ok)
		{
			if (FIoEditorState* EditorState = Request->State.TryGet<FIoEditorState>())
			{
				if (EditorState->CompressedData)
				{
					return &EditorState->CompressedData;
				}
			}
		}
	}
	return nullptr;
}
#endif // WITH_EDITORONLY_DATA

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FDerivedDataIoRequest FDerivedDataIoBatch::Read(const FDerivedData& Data, const FDerivedDataIoOptions& Options)
{
	using namespace DerivedData::Private;
	return FIoResponse::Queue(Response, Data, Options, EIoRequestType::Read);
}

FDerivedDataIoRequest FDerivedDataIoBatch::Cache(const FDerivedData& Data, const FDerivedDataIoOptions& Options)
{
	using namespace DerivedData::Private;
	checkf(!Options.GetTarget(), TEXT("Target is not supported by Cache. %s"), *WriteToString<256>(Data));
	return FIoResponse::Queue(Response, Data, Options, EIoRequestType::Cache);
}

FDerivedDataIoRequest FDerivedDataIoBatch::Exists(const FDerivedData& Data, const FDerivedDataIoOptions& Options)
{
	using namespace DerivedData::Private;
	checkf(!Options.GetTarget(), TEXT("Target is not supported by Exists. %s"), *WriteToString<256>(Data));
	return FIoResponse::Queue(Response, Data, Options, EIoRequestType::Exists);
}

#if WITH_EDITORONLY_DATA
FDerivedDataIoRequest FDerivedDataIoBatch::Compress(const FDerivedData& Data)
{
	using namespace DerivedData::Private;
	return FIoResponse::Queue(Response, Data, {}, EIoRequestType::Compress);
}
#endif // WITH_EDITORONLY_DATA

void FDerivedDataIoBatch::Dispatch(FDerivedDataIoResponse& OutResponse)
{
	Dispatch(OutResponse, {}, {});
}

void FDerivedDataIoBatch::Dispatch(FDerivedDataIoResponse& OutResponse, FDerivedDataIoPriority Priority)
{
	Dispatch(OutResponse, Priority, {});
}

void FDerivedDataIoBatch::Dispatch(FDerivedDataIoResponse& OutResponse, FDerivedDataIoComplete&& OnComplete)
{
	Dispatch(OutResponse, {}, MoveTemp(OnComplete));
}

void FDerivedDataIoBatch::Dispatch(
	FDerivedDataIoResponse& OutResponse,
	FDerivedDataIoPriority Priority,
	FDerivedDataIoComplete&& OnComplete)
{
	using namespace DerivedData::Private;
	FIoResponse::Dispatch(Response, OutResponse, Priority, MoveTemp(OnComplete));
}

} // UE
