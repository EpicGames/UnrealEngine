// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "Serialization/DuplicatedObject.h"
#include "Serialization/DuplicatedDataReader.h"
#include "UObject/UObjectThreadContext.h"
#include "Internationalization/TextPackageNamespaceUtil.h"

/*----------------------------------------------------------------------------
	FDuplicateDataReader.
----------------------------------------------------------------------------*/
/**
 * Constructor
 *
 * @param	InDuplicatedObjects		map of original object to copy of that object
 * @param	InObjectData			object data to read from
 */
FDuplicateDataReader::FDuplicateDataReader(FDuplicatedObjectAnnotation& InDuplicatedObjects ,const FLargeMemoryData& InObjectData, uint32 InPortFlags, UObject* InDestOuter )
	: DuplicatedObjectAnnotation(InDuplicatedObjects)
	, ObjectData(InObjectData)
	, Offset(0)
{
	this->SetIsLoading(true);
	this->SetIsPersistent(true);
	this->SetUseUnversionedPropertySerialization(((InPortFlags & PPF_DuplicateForPIE) == 0));
	this->ArNoIntraPropertyDelta = true;
	ArPortFlags |= PPF_Duplicate | InPortFlags;

#if USE_STABLE_LOCALIZATION_KEYS
	if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
	{
		SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(InDestOuter));
	}
#endif // USE_STABLE_LOCALIZATION_KEYS
}

void FDuplicateDataReader::SetDuplicatedCells(Verse::FDuplicationContext& InDuplicatedCells)
{
	DuplicatedCells = &InDuplicatedCells;
}

void FDuplicateDataReader::SerializeFail()
{
	UE_LOGF(LogObj, Fatal, "FDuplicateDataReader Overread. SerializedObject = %ls SerializedProperty = %ls", *GetFullNameSafe(FUObjectThreadContext::Get().GetSerializeContext()->SerializedObject), *GetFullNameSafe(GetSerializedProperty()));
}

FArchive& FDuplicateDataReader::operator<<(FName& N)
{
	FNameEntryId ComparisonIndex;
	FNameEntryId DisplayIndex;
	int32 Number;
	ByteOrderSerialize(&ComparisonIndex, sizeof(ComparisonIndex));
	ByteOrderSerialize(&DisplayIndex, sizeof(DisplayIndex));
	ByteOrderSerialize(&Number, sizeof(Number));
	// copy over the name with a name made from the name index and number
	N = FName(ComparisonIndex, DisplayIndex, Number);
	return *this;
}

FArchive& FDuplicateDataReader::operator<<( UObject*& Object )
{
	UObject*	SourceObject = Object;
	Serialize(&SourceObject,sizeof(UObject*));

	FDuplicatedObject ObjectInfo = SourceObject ? DuplicatedObjectAnnotation.GetAnnotation(SourceObject) : FDuplicatedObject();
	if( !ObjectInfo.IsDefault() )
	{
		Object = ObjectInfo.DuplicatedObject.GetEvenIfUnreachable();
	}
	else
	{
		Object = SourceObject;
	}

	return *this;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FArchive& FDuplicateDataReader::operator<<( Verse::VCell*& Cell )
{
	Verse::VCell* SourceCell = Cell;
	Serialize(&SourceCell, sizeof(Verse::VCell*));
	Cell = DuplicatedCells->ReadCellReference(SourceCell);
	return *this;
}
#endif

FArchive& FDuplicateDataReader::operator<<(FObjectPtr& ObjectPtr)
{
	FObjectHandle& Handle  = ObjectPtr.GetHandleRef();
	Serialize(&Handle, sizeof(FObjectHandle));

	if (!IsObjectHandleNull(Handle) && IsObjectHandleResolved(Handle))
	{
		UObject* SourceObject = UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(Handle);
		FDuplicatedObject ObjectInfo = DuplicatedObjectAnnotation.GetAnnotation(SourceObject);
		if (!ObjectInfo.IsDefault())
		{
			ObjectPtr = ObjectInfo.DuplicatedObject.GetEvenIfUnreachable();
		}
	}
	return *this;
}

FArchive& FDuplicateDataReader::operator<<(FLazyObjectPtr& LazyObjectPtr)
{
	FArchive& Ar = *this;
	FUniqueObjectGuid ID;
	Ar << ID;

	if (Ar.GetPortFlags()&PPF_DuplicateForPIE)
	{
		// Remap unique ID if necessary
		ID = ID.FixupForPIE();
	}
	LazyObjectPtr = ID;
	return Ar;
}

FArchive& FDuplicateDataReader::operator<<(FSoftObjectPath& SoftObjectPath)
{
	FArchiveUObject::operator<<(SoftObjectPath);
			
	// Remap soft reference if necessary
	UObject* SourceObject = SoftObjectPath.ResolveObject();
	FDuplicatedObject ObjectInfo = SourceObject ? DuplicatedObjectAnnotation.GetAnnotation(SourceObject) : FDuplicatedObject();
	if (!ObjectInfo.IsDefault())
	{
		SoftObjectPath = FSoftObjectPath::GetOrCreateIDForObject(ObjectInfo.DuplicatedObject.GetEvenIfUnreachable());
	}
	
	return *this;
}

//FArchive& FDuplicateDataReader::operator<<(FField*& Field)
//{
//
//}