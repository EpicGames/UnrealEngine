// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "VerseVM/Inline/VVMCellInline.h"
#include "Serialization/DuplicatedObject.h"
#include "Serialization/DuplicatedDataWriter.h"

/*----------------------------------------------------------------------------
	FDuplicateDataWriter.
----------------------------------------------------------------------------*/
/**
 * Constructor
 *
 * @param	InDuplicatedObjects		will contain the original object -> copy mappings
 * @param	InObjectData			will store the serialized data
 * @param	SourceObject			the object to copy
 * @param	DestObject				the object to copy to
 * @param	InFlagMask				the flags that should be copied when the object is duplicated
 * @param	InApplyFlags			the flags that should always be set on the duplicated objects (regardless of whether they're set on the source)
 * @param	InInstanceGraph			the instancing graph to use when creating the duplicate objects.
 */
FDuplicateDataWriter::FDuplicateDataWriter(FDuplicatedObjectAnnotation& InDuplicatedObjects, FLargeMemoryData& InObjectData, UObject* SourceObject,
	UObject* DestObject, EObjectFlags InFlagMask, EObjectFlags InApplyFlags, EInternalObjectFlags InInternalFlagMask, EInternalObjectFlags InApplyInternalFlags, FObjectInstancingGraph* InInstanceGraph, uint32 InPortFlags,
	bool InAssignExternalPackages)
: DuplicatedObjectAnnotation(InDuplicatedObjects)
, ObjectData(InObjectData)
, Offset(0)
, FlagMask(InFlagMask)
, ApplyFlags(InApplyFlags)
, InternalFlagMask(InInternalFlagMask)
, ApplyInternalFlags(InApplyInternalFlags)
, bAssignExternalPackages(InAssignExternalPackages)
, InstanceGraph(InInstanceGraph)
{
	this->SetIsSaving(true);
	this->SetIsPersistent(true);
	this->SetUseUnversionedPropertySerialization(((InPortFlags & PPF_DuplicateForPIE) == 0));
	this->ArNoIntraPropertyDelta = true;
	ArAllowLazyLoading	= false;
	ArPortFlags |= PPF_Duplicate | InPortFlags;

	AddDuplicate(SourceObject,DestObject);
}

void FDuplicateDataWriter::SetDuplicatedCells(Verse::FDuplicationContext& InDuplicatedCells)
{
	DuplicatedCells = &InDuplicatedCells;
}

/**
 * I/O function
 */
FArchive& FDuplicateDataWriter::operator<<(FName& N)
{
	FNameEntryId ComparisonIndex = N.GetComparisonIndex();
	FNameEntryId DisplayIndex = N.GetDisplayIndex();
	int32 Number = N.GetNumber();
	ByteOrderSerialize(&ComparisonIndex, sizeof(ComparisonIndex));
	ByteOrderSerialize(&DisplayIndex, sizeof(DisplayIndex));
	ByteOrderSerialize(&Number, sizeof(Number));
	return *this;
}

FArchive& FDuplicateDataWriter::operator<<(UObject*& Object)
{
	if (Object &&
		!Object->HasAnyFlags(RF_DuplicateTransient) &&
		(!Object->HasAnyFlags(RF_NonPIEDuplicateTransient) || HasAnyPortFlags(PPF_DuplicateForPIE)))
	{
		GetDuplicatedObject(Object);

		// store the pointer to this object
		Serialize(&Object, sizeof(UObject*));
	}
	else
	{
		UObject* NullObject = nullptr;
		Serialize(&NullObject, sizeof(UObject*));
	}
	return *this;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FArchive& FDuplicateDataWriter::operator<<(Verse::VCell*& Cell)
{
	DuplicatedCells->WriteCellReference(Cell);
	Serialize(&Cell, sizeof(Verse::VCell*));
	return *this;
}
#endif


FArchive& FDuplicateDataWriter::operator<<(FObjectPtr& ObjectPtr)
{
	if (ObjectPtr.IsResolved())
	{
		UObject* Object = ObjectPtr.Get();
		*this << Object;
	}
	else
	{
		FObjectHandle& Handle = ObjectPtr.GetHandleRef();
		Serialize(&Handle, sizeof(FObjectHandle));
	}
	return *this;
}

FArchive& FDuplicateDataWriter::operator<<(FLazyObjectPtr& LazyObjectPtr)
{
	if ( (GetPortFlags() & PPF_DuplicateForPIE) == 0 )
	{
		if (UObject* DuplicatedObject = GetDuplicatedObject(LazyObjectPtr.Get(), false))
		{
			FLazyObjectPtr Ptr(DuplicatedObject);
			return *this << Ptr.GetUniqueID();
		}
	}

	FUniqueObjectGuid ID = LazyObjectPtr.GetUniqueID();
	return *this << ID;
}

void FDuplicateDataWriter::AddDuplicate(UObject* SourceObject, UObject* DupObject)
{
	if ( DupObject && !DupObject->IsTemplate() )
	{
		// Make sure the duplicated object is prepared to postload
		DupObject->SetFlags(RF_NeedPostLoad|RF_NeedPostLoadSubobjects);
	}

	// Check for an existing duplicate of the object; if found, use that one instead of creating a new one.
	FDuplicatedObject Info = DuplicatedObjectAnnotation.GetAnnotation( SourceObject );
	if ( Info.IsDefault() )
	{
		DuplicatedObjectAnnotation.AddAnnotation( SourceObject, FDuplicatedObject( DupObject ) );
	}
	else
	{
		Info.DuplicatedObject = DupObject;
	}


	UnserializedObjects.Add(SourceObject);
}

/**
 * Returns a pointer to the duplicate of a given object, creating the duplicate object if necessary.
 *
 * @param	Object	the object to find a duplicate for
 *
 * @return	a pointer to the duplicate of the specified object
 */
UObject* FDuplicateDataWriter::GetDuplicatedObject(UObject* Object, bool bCreateIfMissing)
{
	UObject* Result = nullptr;
	if (IsValid(Object))
	{
		// Check for an existing duplicate of the object.
		FDuplicatedObject DupObjectInfo = DuplicatedObjectAnnotation.GetAnnotation( Object );
		if( !DupObjectInfo.IsDefault() )
		{
			Result = DupObjectInfo.DuplicatedObject.GetEvenIfUnreachable();
		}
		else if (bCreateIfMissing)
		{
			// Check to see if the object's outer is being duplicated.
			UObject* DupOuter = GetDuplicatedObject(Object->GetOuter());
			if(DupOuter != nullptr)
			{
				// The object's outer is being duplicated, create a duplicate of this object.
				FStaticConstructObjectParameters Params(Object->GetClass());
				Params.Outer = DupOuter;
				Params.Name = Object->GetFName();
				Params.SetFlags = ApplyFlags | Object->GetMaskedFlags(FlagMask);
				Params.InternalSetFlags = ApplyInternalFlags | (Object->GetInternalFlags() & InternalFlagMask);
				Params.Template = Object->GetArchetype();
				Params.bCopyTransientsFromClassDefaults = true;
				Params.InstanceGraph = InstanceGraph;
				Result = StaticConstructObject_Internal(Params);
				
				// If we assign external package to duplicated object, fetch the package
				Result->SetExternalPackage(bAssignExternalPackages ? Cast<UPackage>(GetDuplicatedObject(Object->GetExternalPackage(), false)) : nullptr);
				
				AddDuplicate(Object, Result);
			}
		}
	}

	return Result;
}

FArchive& FDuplicateDataWriter::operator<<(FField*& Field)
{
	//if (Field &&
	//	!Field->HasAnyFlags(RF_DuplicateTransient) &&
	//	(!Field->HasAnyFlags(RF_NonPIEDuplicateTransient) || HasAnyPortFlags(PPF_DuplicateForPIE)))
	//{
	//}
	//else
	//{
	//	UObject* NullObject = nullptr;
	//	Serialize(&NullObject, sizeof(UObject*));
	//}
	return *this;
}