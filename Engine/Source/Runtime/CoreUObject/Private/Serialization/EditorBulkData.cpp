// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/EditorBulkData.h"

#include "Async/UniqueLock.h"
#include "Compression/OodleDataCompression.h"
#include "Experimental/Async/MultiUniqueLock.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageSegment.h"
#include "Misc/ScopeLock.h"
#include "Misc/SecureHash.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Serialization/BulkData.h"
#include "Serialization/BulkDataRegistry.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerSave.h"
#include "UObject/Object.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/ObjectVersion.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectGlobals.h"
#include "Virtualization/VirtualizationSystem.h"
#include "Virtualization/VirtualizationTypes.h"

//#if WITH_EDITORONLY_DATA

/** When enabled the non-virtualized bulkdata objects will attach to the FLinkerLoader for the package that they are loaded from */
#if WITH_EDITOR
	#define UE_ALLOW_LINKERLOADER_ATTACHMENT 1
#else
	#define UE_ALLOW_LINKERLOADER_ATTACHMENT 0
#endif //WITH_EDITOR

/** When enabled we will fatal log if we detect corrupted data rather than logging an error and returning a null FCompressedBuffer/FSharedBuffer. */
#define UE_CORRUPTED_PAYLOAD_IS_FATAL 0

#if UE_CORRUPTED_PAYLOAD_IS_FATAL
	#define UE_CORRUPTED_DATA_SEVERITY Fatal
#else
	#define UE_CORRUPTED_DATA_SEVERITY Error
#endif // VBD_CORRUPTED_PAYLOAD_IS_FATAL

TRACE_DECLARE_ATOMIC_INT_COUNTER(EditorBulkData_PayloadDataLoaded, TEXT("EditorBulkData/PayloadDataLoaded"));
TRACE_DECLARE_ATOMIC_INT_COUNTER(EditorBulkData_PayloadDataPulled, TEXT("EditorBulkData/PayloadDataPulled"));

namespace UE::Serialization
{
/** This is an experimental code path and is not expected to be used in production! */
static TAutoConsoleVariable<bool> CVarShouldLoadFromSidecar(
	TEXT("Serialization.LoadFromSidecar"),
	false,
	TEXT("When true FEditorBulkData will load from the sidecar file"));

/** 
 * Prefer loading from the package trailer (load the trailer, parse the look up, then load the payload) over 
 * using the in built OffsetInFile member to load from the package file directly.
 */
static TAutoConsoleVariable<bool> CVarShouldLoadFromTrailer(
	TEXT("Serialization.LoadFromTrailer"),
	false,
	TEXT("When true FEditorBulkData will load payloads via the package trailer rather than the package itself"));

static TAutoConsoleVariable<bool> CVarShouldAllowSidecarSyncing(
	TEXT("Serialization.AllowSidecarSyncing"),
	false,
	TEXT("When true FEditorBulkData will attempt to sync it's .upayload file via sourcecontrol if the first attempt to load from it fails"));

/** When enabled the bulkdata object will try pushing the payload when saved to disk as part of a package.
  * This is legacy behavior and likely to be removed
  */
static constexpr bool bAllowVirtualizationOnSave = false;

static TAutoConsoleVariable<bool> CVarShouldRehydrateOnSave(
	TEXT("Serialization.RehydrateOnSave"),
	false,
	TEXT("When true FVirtualizedUntypedBulkData virtualized payloads will by hydrated and stored locally when saved to a package"));

/** Wrapper around the config file option [Core.System.Experimental]EnablePackageSidecarSaving */
static bool ShouldSaveToPackageSidecar()
{
	static const struct FSaveToPackageSidecar
	{
		bool bEnabled = false;

		FSaveToPackageSidecar()
		{
			GConfig->GetBool(TEXT("Core.System.Experimental"), TEXT("EnablePackageSidecarSaving"), bEnabled, GEngineIni);
		}
	} ConfigSetting;

	return ConfigSetting.bEnabled;
}

/** 
 * Returns true if the project wants to log if an editor bulkdata is cooked or uncooked and returns
 * false if the project does not care. The engine ini file defaults to false 
 */
static bool ShouldLogCookedStatus()
{
	static const struct FShouldLogCookedStatus
	{
		bool bEnabled = false;

		FShouldLogCookedStatus()
		{
			GConfig->GetBool(TEXT("EditorBulkData"), TEXT("LogCookedStatus"), bEnabled, GEditorIni);
		}
	} ConfigSetting;

	return ConfigSetting.bEnabled;
}

/** Utility for logging extended error messages when we fail to open a package for reading */
static void LogPackageOpenFailureMessage(const FPackagePath& PackagePath, EPackageSegment PackageSegment)
{
	// TODO: Check the various error paths here again!
	const uint32 SystemError = FPlatformMisc::GetLastError();
	// If we have a system error we can give a more informative error message but don't output it if the error is zero as 
	// this can lead to very confusing error messages.
	if (SystemError != 0)
	{
		TCHAR SystemErrorMsg[2048] = { 0 };
		FPlatformMisc::GetSystemErrorMessage(SystemErrorMsg, UE_ARRAY_COUNT(SystemErrorMsg), SystemError);
		UE_LOGF(LogSerialization, Error, "Could not open the file '%ls' for reading due to system error: '%ls' (%d))", *PackagePath.GetLocalFullPath(PackageSegment), SystemErrorMsg, SystemError);
	}
	else
	{
		UE_LOGF(LogSerialization, Error, "Could not open (%ls) to read FEditorBulkData with an unknown error", *PackagePath.GetLocalFullPath(PackageSegment));
	}
}

/** 
 * Utility used to validate the contents of a recently loaded payload.
 * If the given payload is null, then we assume that the load failed and errors would've been raised else
 * where in code and there is no need to validate the contents.
 * If the contents are validated we check the loaded result against the members of a bulkdata object to 
 * see if they match.
 */
static bool IsDataValid(const FEditorBulkData& BulkData, const FCompressedBuffer& Payload)
{
	if (!Payload.IsNull())
	{
		if (!BulkData.HasPlaceholderPayloadId() && BulkData.GetPayloadId() != FIoHash(Payload.GetRawHash()))
		{
			return false;
		}

		if (Payload.GetRawSize() != BulkData.GetPayloadSize())
		{
			return false;
		}
	}

	return true;
}

/** Utility for finding the FLinkerLoad associated with a given UObject */
const FLinkerLoad* GetLinkerLoadFromOwner(UObject* Owner)
{
	if (Owner != nullptr)
	{
		UPackage* Package = Owner->GetOutermost();
		checkf(Package != nullptr, TEXT("Owner was not a valid UPackage!"));

		return FLinkerLoad::FindExistingLinkerForPackage(Package);
	}
	else
	{
		return nullptr;
	}
}

/** Utility for finding the package path associated with a given UObject */
static FPackagePath GetPackagePathFromOwner(UObject* Owner)
{
	const FLinkerLoad* Linker = GetLinkerLoadFromOwner(Owner);

	if (Linker != nullptr)
	{
		return Linker->GetPackagePath();
	}
	else
	{
		return FPackagePath();
	}
}

bool GCVarLegacyLinkerDiscovery = false;
static FAutoConsoleVariableRef CVar_LegacyLinkerDiscovery(
	TEXT("Serialization.LegacyLinkerDiscovery"),
	GCVarLegacyLinkerDiscovery,
	TEXT("When enabled we revert to the legacy behavior of trying find the FLinkerLoad from the owning UObject rather than the FArchive")
);

/** Utility for finding the FPackageTrailer associated from either a UObject or FArchive */
static const FPackageTrailer* FindPackageTrailer(UObject* Owner, FArchive& Ar)
{
	const FLinkerLoad* Linker = GCVarLegacyLinkerDiscovery ? GetLinkerLoadFromOwner(Owner) : Cast<FLinkerLoad>(Ar.GetLinker());

	if (Linker != nullptr)
	{
		return Linker->GetPackageTrailer();
	}
	else
	{
		return nullptr;
	}
}

/** Utility for finding the package path associated from either a UObject or FArchive */
static FPackagePath FindPackagePath(UObject* Owner, FArchive& Ar)
{
	const FLinkerLoad* Linker = GCVarLegacyLinkerDiscovery ? GetLinkerLoadFromOwner(Owner) : Cast<FLinkerLoad>(Ar.GetLinker());

	if (Linker != nullptr)
	{
		return Linker->GetPackagePath();
	}
	else
	{
		return FPackagePath();
	}
}

/** Utility for finding a valid debug name to print from the owning UObject */
FString GetDebugNameFromOwner(UObject* Owner)
{
	if (Owner != nullptr)
	{
		return Owner->GetFullName();
	}
	else
	{
		return FString("Unknown");
	}
}

/** 
 * Utility to check if we need to generate a new unique identifier for the editor bulkdata or not.
 * The current use case is if we are loading the editor bulkdata to a transient package, which can 
 * occur when some assets duplicate a number of UObjects from a template package that will eventually
 * be added to themselves. The duplication will load the editor bulkdata to a transient package before
 * it is re-parented.
 */
bool ShouldGenerateNewIdentifier(FArchive& Ar, UObject* Owner)
{
	UPackage* Package = Owner != nullptr ? Owner->GetPackage() : nullptr;
	if (Package != nullptr && Package->HasAnyFlags(RF_Transient))
	{
		return true;
	}

	const FLinkerLoad* LinkerLoad = Cast<FLinkerLoad>(Ar.GetLinker());

	if (Ar.HasAllPortFlags(PPF_Duplicate) || (LinkerLoad && LinkerLoad->GetInstancingContext().IsInstanced()))
	{
		return true;
	}

	return false;
}

/** Utility for hashing a payload, will return a default FIoHash if the payload is invalid or of zero length */
static FIoHash HashPayload(const FSharedBuffer& InPayload)
{
	if (InPayload.GetSize() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(EditorBulkData.HashPayload);

		return FIoHash::HashBuffer(InPayload);
	}
	else
	{
		return FIoHash();
	}
}

/** Returns the FIoHash of a FGuid */
static FIoHash GuidToIoHash(const FGuid& Guid)
{
	if (Guid.IsValid())
	{
		// Hash each element individually rather than making assumptions about
		// the internal layout of FGuid and treating it as a contiguous buffer.
		// Slightly slower, but safer.
		FBlake3 Hash;

		Hash.Update(&Guid[0], sizeof(uint32));
		Hash.Update(&Guid[1], sizeof(uint32));
		Hash.Update(&Guid[2], sizeof(uint32));
		Hash.Update(&Guid[3], sizeof(uint32));

		return FIoHash(Hash.Finalize());
	}
	else
	{
		return FIoHash();
	}
}

FGuid IoHashToGuid(const FIoHash& Hash)
{
	// We use the first 16 bytes of the FIoHash to create the guid, there is
	// no specific reason why these were chosen, we could take any pattern or combination
	// of bytes.
	// Note that if the input hash is invalid (all zeros) then the FGuid returned will
	// also be considered as invalid
	uint32* HashBytes = (uint32*)Hash.GetBytes();
	return FGuid(HashBytes[0], HashBytes[1], HashBytes[2], HashBytes[3]);
}

/** Utility for updating an existing entry in an Archive before returning the archive to it's original seek position */
template<typename DataType>
void UpdateArchiveData(FArchive& Ar, int64 DataPosition, DataType& Data)
{
	const int64 OriginalPosition = Ar.Tell();

	Ar.Seek(DataPosition);
	Ar << Data;

	Ar.Seek(OriginalPosition);
}

FArchive& operator<<(FArchive& Ar, FSharedBuffer& Buffer)
{
	// Note that there is a difference between a null FSharedBuffer and a zero length
	// shared buffer and they are not interchangeable! If we have a null buffer we 
	// write an invalid value for the buffer length so that we know to set the buffer
	// to null when loaded and not create a valid zero length buffer instead.

	if (Ar.IsLoading())
	{
		Buffer.Reset();

		int64 BufferLength;
		Ar << BufferLength;

		if (BufferLength >= 0)
		{
			if (const int64 SpaceRemaining = Ar.TotalSize() - Ar.Tell(); SpaceRemaining >= BufferLength)
			{
				FUniqueBuffer MutableBuffer = FUniqueBuffer::Alloc(BufferLength);
				Ar.Serialize(MutableBuffer.GetData(), BufferLength);

				Buffer = MutableBuffer.MoveToShared();
			}
			else
			{
				Ar.SetError();
				UE_LOGF(LogSerialization, Error, "Not enough data remaining in archive (%ls) to load the payload. Bytes required: %lld vs bytes remaining %lld", *Ar.GetArchiveName(), BufferLength, SpaceRemaining);
			}
		}
	}
	else if (Ar.IsSaving())
	{
		if (!Buffer.IsNull())
		{
			int64 BufferLength = (int64)Buffer.GetSize();
			Ar << BufferLength;

			// Need to remove const due to FArchive API
			Ar.Serialize(const_cast<void*>(Buffer.GetData()), BufferLength);
		}
		else
		{
			int64 InvalidLength = INDEX_NONE;
			Ar << InvalidLength;
		}
	}

	return Ar;
}

/** Utility for accessing IVirtualizationSourceControlUtilities from the modular feature system. */
UE::Virtualization::Experimental::IVirtualizationSourceControlUtilities* GetSourceControlInterface()
{
	return (UE::Virtualization::Experimental::IVirtualizationSourceControlUtilities*)IModularFeatures::Get().GetModularFeatureImplementation(FName("VirtualizationSourceControlUtilities"), 0);
}

namespace Private
{

FCompressionSettings::FCompressionSettings()
	: Compressor(ECompressedBufferCompressor::NotSet)
	, CompressionLevel(ECompressedBufferCompressionLevel::None)
	, bIsSet(false)
{

}

FCompressionSettings::FCompressionSettings(const FCompressedBuffer& Buffer)
{
	// Note that if the buffer is using a non-oodle format we consider it
	// as not set.
	uint64 BlockSize;
	if (!Buffer.TryGetCompressParameters(Compressor, CompressionLevel, BlockSize))
	{
		Reset();
	}
	else
	{
		bIsSet = true;
	}
}

bool FCompressionSettings::operator==(const FCompressionSettings& Other) const
{
	return	Compressor == Other.Compressor &&
			CompressionLevel == Other.CompressionLevel &&
			bIsSet == Other.bIsSet;
}

bool FCompressionSettings::operator != (const FCompressionSettings& Other) const
{
	return !(*this == Other);
}

void FCompressionSettings::Reset()
{
	Compressor = ECompressedBufferCompressor::NotSet;
	CompressionLevel = ECompressedBufferCompressionLevel::None;
	bIsSet = false;
}

void FCompressionSettings::Set(ECompressedBufferCompressor InCompressor, ECompressedBufferCompressionLevel InCompressionLevel)
{
	Compressor = InCompressor;
	CompressionLevel = InCompressionLevel;
	bIsSet = true;
}

void FCompressionSettings::SetToDefault()
{
	Compressor = ECompressedBufferCompressor::Kraken;
	CompressionLevel = ECompressedBufferCompressionLevel::Fast;
	bIsSet = true;
}

void FCompressionSettings::SetToDisabled()
{
	Compressor = ECompressedBufferCompressor::NotSet;
	CompressionLevel = ECompressedBufferCompressionLevel::None;
	bIsSet = true;
}

bool FCompressionSettings::IsSet() const
{
	return bIsSet;
}

bool FCompressionSettings::IsCompressed() const
{
	return bIsSet == true && CompressionLevel != ECompressedBufferCompressionLevel::None;
}

ECompressedBufferCompressor FCompressionSettings::GetCompressor() const
{
	return Compressor;
}

ECompressedBufferCompressionLevel FCompressionSettings::GetCompressionLevel()
{
	return CompressionLevel;
}

} // namespace Private

FEditorBulkData::~FEditorBulkData()
{
	if (HasAttachedArchive())
	{
		AttachedAr->DetachBulkData(this, false);
		AttachedAr = nullptr;
	}

	// If we have registered with the BulkData registry, keep our ownership of our guid registered,
	// and possibly loadable from our on-disk location, but let the BulkDataRegistry know that we
	// no longer have an in-memory copy of the data.
	OnExitMemory();
}

FEditorBulkData::FEditorBulkData(FEditorBulkData&& Other)
{
	*this = MoveTemp(Other);
}

FEditorBulkData& FEditorBulkData::operator=(FEditorBulkData&& Other)
{
	UE::TMultiUniqueLock<FRecursiveMutex> _({&Mutex, &Other.Mutex });

	// The same as the default move constructor, except we need to handle registration and unregistration
	Unregister();
	Other.Unregister();

	if (HasAttachedArchive())
	{
		AttachedAr->DetachBulkData(this, false);
	}

	BulkDataId = MoveTemp(Other.BulkDataId);
	PayloadContentId = MoveTemp(Other.PayloadContentId);
	Payload = MoveTemp(Other.Payload);
	PayloadSize = MoveTemp(Other.PayloadSize);
	AttachedAr = Other.AttachedAr;
	OffsetInFile = MoveTemp(Other.OffsetInFile);
	PackagePath = MoveTemp(Other.PackagePath);
	Flags = MoveTemp(Other.Flags);
	CompressionSettings = MoveTemp(Other.CompressionSettings);

	if (HasAttachedArchive())
	{
		AttachedAr->AttachBulkData(this);
	}

	Other.Reset();

	Register(nullptr, TEXT("operator=(FEditorBulkData &&)"), false /* bAllowUpdateId */);

	return *this;
}

FEditorBulkData::FEditorBulkData(const FEditorBulkData& Other)
{
	*this = Other;
}

FEditorBulkData& FEditorBulkData::operator=(const FEditorBulkData& Other)
{
	UE::TMultiUniqueLock<FRecursiveMutex> _({ &Mutex, &Other.Mutex });

	// Torn-off BulkDatas remain torn-off even when being copied into from a non-torn-off BulkData
	// Remaining torn-off is a work-around necessary for FTextureSource::CopyTornOff to avoid registering a new
	// guid before setting the new BulkData to torn-off. The caller can call Reset to clear the torn-off flag.
	bool bTornOff = false;
	if (EnumHasAnyFlags(Flags, EFlags::IsTornOff))
	{
		check(!EnumHasAnyFlags(Flags, EFlags::HasRegistered));
		BulkDataId = Other.BulkDataId;
		bTornOff = true;
	}
	else
	{
		Unregister();
		if (EnumHasAnyFlags(Other.Flags, EFlags::IsTornOff))
		{
			BulkDataId = Other.BulkDataId;
			bTornOff = true;
		}
		else if (!BulkDataId.IsValid() && Other.BulkDataId.IsValid())
		{
			BulkDataId = FGuid::NewGuid();
		}
	}

	if (HasAttachedArchive())
	{
		AttachedAr->DetachBulkData(this, false);
	}

	PayloadContentId = Other.PayloadContentId;
	Payload = Other.Payload;
	PayloadSize = Other.PayloadSize;
	OffsetInFile = Other.OffsetInFile;
	PackagePath = Other.PackagePath;
	Flags = Other.Flags;
	CompressionSettings = Other.CompressionSettings;

	if (HasAttachedArchive())
	{
		AttachedAr->AttachBulkData(this);
	}

	EnumRemoveFlags(Flags, TransientFlags);

	if (bTornOff)
	{
		EnumAddFlags(Flags, EFlags::IsTornOff);
	}
	else
	{
		Register(nullptr, TEXT("operator=(const FEditorBulkData&)"), false /* bAllowUpdateId */);
	}
	return *this;
}

FEditorBulkData::FEditorBulkData(const FEditorBulkData& Other, ETornOff)
{
	UE::TMultiUniqueLock<FRecursiveMutex> _({ &Mutex, &Other.Mutex });

	EnumAddFlags(Flags, EFlags::IsTornOff);
	*this = Other; // We rely on operator= preserving the torn-off flag
}

void FEditorBulkData::TearOff()
{
	UE::TUniqueLock _(Mutex);

	Unregister();
	EnumAddFlags(Flags, EFlags::IsTornOff);
}

void FEditorBulkData::UpdateRegistrationOwner(UObject* Owner)
{
	UE::TUniqueLock _(Mutex);

	UpdateRegistrationData(Owner, TEXT("UpdateRegistrationOwner"), false /* bAllowUpdateId */);
}

static FGuid CreateUniqueGuid(const FGuid& NonUniqueGuid, const UObject* Owner, const TCHAR* DebugName)
{
	if (NonUniqueGuid.IsValid() && Owner)
	{
		TStringBuilder<256> PathName;
		Owner->GetPathName(nullptr, PathName);
		FBlake3 Builder;
		Builder.Update(&NonUniqueGuid, sizeof(NonUniqueGuid));
		Builder.Update(PathName.GetData(), PathName.Len() * sizeof(*PathName.GetData()));
		FBlake3Hash Hash = Builder.Finalize();
		// We use the first 16 bytes of the FIoHash to create the guid, there is
		// no specific reason why these were chosen, we could take any pattern or combination
		// of bytes.
		uint32* HashBytes = (uint32*)Hash.GetBytes();
		return FGuid(HashBytes[0], HashBytes[1], HashBytes[2], HashBytes[3]);
	}
	else
	{
		UE_LOGF(LogSerialization, Warning,
			"CreateFromBulkData received an invalid FGuid. A temporary one will be generated until the package is next re-saved! Package: '%ls'",
			DebugName);
		return FGuid::NewGuid();
	}
}

void FEditorBulkData::Register(UObject* Owner, const TCHAR* ErrorLogCaller, bool bAllowUpdateId)
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::Register);

	if (BulkDataId.IsValid() && PayloadSize > 0 && !EnumHasAnyFlags(Flags, EFlags::IsTornOff))
	{
		UPackage* OwnerPackage = Owner ? Owner->GetPackage() : nullptr;
		UE::BulkDataRegistry::ERegisterResult Result = IBulkDataRegistry::Get().TryRegister(OwnerPackage, *this);
		if (Result == UE::BulkDataRegistry::ERegisterResult::Success)
		{
			EnumAddFlags(Flags, EFlags::HasRegistered);
		}
		else
		{
			UE::BulkDataRegistry::ERegisterResult SecondResult = UE::BulkDataRegistry::ERegisterResult::InvalidResultCode;
			if (Result == UE::BulkDataRegistry::ERegisterResult::AlreadyExists && bAllowUpdateId)
			{
				FGuid OldBulkDataId(BulkDataId);
				BulkDataId = CreateUniqueGuid(OldBulkDataId, Owner, TEXT("BulkDataRegistryGuidCollision"));
				SecondResult = IBulkDataRegistry::Get().TryRegister(OwnerPackage, *this);
				if (SecondResult == UE::BulkDataRegistry::ERegisterResult::Success)
				{
					EnumAddFlags(Flags, EFlags::HasRegistered);
					LogRegisterError(Result, Owner, OldBulkDataId, ErrorLogCaller, true/* bHandledByCreateUniqueGuid */);
				}
				else
				{
					BulkDataId = OldBulkDataId;
				}
			}
			if (SecondResult != UE::BulkDataRegistry::ERegisterResult::Success)
			{
				LogRegisterError(Result, Owner, BulkDataId, ErrorLogCaller, false /* bHandledByCreateUniqueGuid */);
			}
		}
	}
#endif
}

void FEditorBulkData::OnExitMemory()
{
#if WITH_EDITOR
	if (EnumHasAnyFlags(Flags, EFlags::HasRegistered))
	{
		check(!EnumHasAnyFlags(Flags, EFlags::IsTornOff));
		IBulkDataRegistry::Get().OnExitMemory(*this);
	}
#endif
}

void FEditorBulkData::Unregister()
{
#if WITH_EDITOR
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::Unregister);

	if (EnumHasAnyFlags(Flags, EFlags::HasRegistered))
	{
		check(!EnumHasAnyFlags(Flags, EFlags::IsTornOff));
		IBulkDataRegistry::Get().Unregister(*this);
		EnumRemoveFlags(Flags, EFlags::HasRegistered);
	}
#endif
}

void FEditorBulkData::UpdateRegistrationData(UObject* Owner, const TCHAR* ErrorLogCaller, bool bAllowUpdateId)
{
#if WITH_EDITOR
	UPackage* OwnerPackage = Owner ? Owner->GetPackage() : nullptr;
	bool bShouldRegister = BulkDataId.IsValid() && PayloadSize > 0 && !EnumHasAnyFlags(Flags, EFlags::IsTornOff);
	if (EnumHasAnyFlags(Flags, EFlags::HasRegistered))
	{
		if (bShouldRegister)
		{
			IBulkDataRegistry::Get().UpdateRegistrationData(OwnerPackage, *this);
		}
		else
		{
			Unregister();
		}
	}
	else
	{
		if (bShouldRegister)
		{
			Register(Owner, ErrorLogCaller, bAllowUpdateId);
		}
	}
#endif
}

void FEditorBulkData::LogRegisterError(UE::BulkDataRegistry::ERegisterResult Value, UObject* Owner,
	const FGuid& FailedBulkDataId, const TCHAR* CallerName, bool bHandledByCreateUniqueGuid) const
{
#if WITH_EDITOR
	if (Value == UE::BulkDataRegistry::ERegisterResult::Success)
	{
		return;
	}
	bool bMessageLogged = false;
	FString OwnerPathName = Owner ? Owner->GetPathName() : TEXT("<unknown>");
	UPackage* OwnerPackage = Owner ? Owner->GetPackage() : nullptr;
	CallerName = CallerName ? CallerName : TEXT("<unknown>");
	if (Value == UE::BulkDataRegistry::ERegisterResult::AlreadyExists)
	{
		FEditorBulkData OtherBulkData;
		FName OtherOwnerPackageFName;
		bool bOtherExists = IBulkDataRegistry::Get().TryGetBulkData(FailedBulkDataId, &OtherBulkData, &OtherOwnerPackageFName);
		FString OtherOwnerPackageName = !OtherOwnerPackageFName.IsNone() ? OtherOwnerPackageFName.ToString() : TEXT("<unknown>");

		if (bHandledByCreateUniqueGuid)
		{
			if (UE::SavePackageUtilities::OnAddResaveOnDemandPackage.IsBound() && OwnerPackage)
			{
				static FName BulkDataDuplicatesSystemName("BulkDataDuplicates");
				UE::SavePackageUtilities::OnAddResaveOnDemandPackage.Execute(BulkDataDuplicatesSystemName, OwnerPackage->GetFName());
				UE_LOGF(LogBulkDataRegistry, Display, "AddPackageToResave %ls: BulkData %ls collided with an ID in package %ls.",
					*OwnerPathName, *FailedBulkDataId.ToString(), *OtherOwnerPackageName);
			}
			else
			{
				FString SilenceWarningMessage;
				if (OwnerPackage)
				{
					// More notes on silencing the warning:
					// 1) Resaveondemand does not yet work for startup packages. If the warning is coming from startup packages, run ResavePackagesCommandlet with
					// argument "-package=<PackageName>" for each warned package.
					// 2) The guid will only be updated on resave if both packages contributing to the collision are loaded during the resave.
					SilenceWarningMessage = FString::Printf(TEXT(" To silence this warning, run the ResavePackagesCommandlet with ")
						TEXT("\"-autocheckout -resaveondemand=bulkdataduplicates -SaveAll -Package=%s -Package=%s\"."),
						*FPackageName::ObjectPathToPackageName(OwnerPathName), *OtherOwnerPackageName);
				}
				// Suppress the warning by default for now. Once we have eliminated all sources of duplication, turn it back on, but allow
				// projects to suppress the warning even then because they might not be able to resave packages during an integration
				bool bSuppressWarning = true; 
				GConfig->GetBool(TEXT("CookSettings"), TEXT("BulkDataRegistrySuppressDuplicateWarning"), bSuppressWarning, GEditorIni);
				FString LogMessage = FString::Printf(TEXT("%s updated BulkData %s on load because it collided with an ID in package %s.%s"),
					*OwnerPathName, *FailedBulkDataId.ToString(), *OtherOwnerPackageName, *SilenceWarningMessage);
				if (bSuppressWarning)
				{
					UE_LOGF(LogBulkDataRegistry, Verbose, "%ls", *LogMessage);
				}
				else
				{
					UE_LOGF(LogBulkDataRegistry, Warning, "%ls", *LogMessage);
				}
			}
			bMessageLogged = true;
		}
		else if (bOtherExists)
		{
			UE_LOGF(LogBulkDataRegistry, Verbose,
				"%ls failed to register BulkData %ls from %ls in the BulkDataRegistry: it already exists with owner %ls.\n" "            Payload: %ls\n" "    ExistingPayload: %ls",
				*OwnerPathName, *FailedBulkDataId.ToString(), CallerName, *OtherOwnerPackageName,
				*WriteToString<64>(GetPayloadId()), *WriteToString<64>(OtherBulkData.GetPayloadId()));
			bMessageLogged = true;
		}
	}
	if (!bMessageLogged)
	{
		UE_LOGF(LogBulkDataRegistry, Warning,
			"%ls failed to register BulkData %ls from %ls in the BulkDataRegistry: %ls.",
			*OwnerPathName, *FailedBulkDataId.ToString(), CallerName, LexToString(Value));
	}
#endif
}


void FEditorBulkData::CreateFromBulkData(FBulkData& InBulkData, const FGuid& InGuid, UObject* Owner)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::CreateFromBulkData);

	UE::TUniqueLock _(Mutex);

	Reset();

#if UE_ALLOW_LINKERLOADER_ATTACHMENT
	check(!HasAttachedArchive());
	AttachedAr = InBulkData.AttachedAr;
	if (HasAttachedArchive())
	{
		AttachedAr->AttachBulkData(this);
	}
#endif //VBD_ALLOW_LINKERLOADER_ATTACHMENT

	// We only need to set up the bulkdata/content identifiers if we have a valid payload
	bool bWasKeyGuidDerived = false;
	if (InBulkData.GetBulkDataSize() > 0)
	{
		BulkDataId = CreateUniqueGuid(InGuid, Owner, *InBulkData.GetDebugName());
		PayloadContentId = GuidToIoHash(BulkDataId);
		bWasKeyGuidDerived = true;
	}
	
	PayloadSize = InBulkData.GetBulkDataSize();
	
	PackagePath = GetPackagePathFromOwner(Owner); 
	
	OffsetInFile = InBulkData.GetBulkDataOffsetInFile();

	// Mark that we are actually referencing a payload stored in an old bulkdata
	// format.
	EnumAddFlags(Flags, EFlags::ReferencesLegacyFile);

	if (InBulkData.IsStoredCompressedOnDisk())
	{
		EnumAddFlags(Flags, EFlags::LegacyFileIsCompressed);
	}
	else 
	{
		EnumAddFlags(Flags, EFlags::DisablePayloadCompression);
	}
	if (bWasKeyGuidDerived)
	{
		EnumAddFlags(Flags, EFlags::LegacyKeyWasGuidDerived);
	}
	// bAllowUpdateId=true because we can encounter duplicates even with the newly
	// created BulkDataId, if the user duplicated (before the fix for duplication) the owner
	// of this bulkdata, without ever resaving this bulkdata's package
	Register(Owner, TEXT("CreateFromBulkData"), true /* bAllowUpdateId */);
}

void FEditorBulkData::CreateLegacyUniqueIdentifier(UObject* Owner)
{
	UE::TUniqueLock _(Mutex);

	if (BulkDataId.IsValid())
	{
		Unregister();
		BulkDataId = CreateUniqueGuid(BulkDataId, Owner, TEXT("Unknown"));
		// bAllowUpdateId=true because we can encounter duplicates even with the newly
		// created BulkDataId, if the user duplicated (before the fix for duplication) the owner
		// of this bulkdata, without ever resaving this bulkdata's package
		Register(Owner, TEXT("CreateLegacyUniqueIdentifier"), true /* bAllowUpdateId */);
	}
}

void FEditorBulkData::Serialize(FArchive& Ar, UObject* Owner, bool bAllowRegister)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::Serialize);
	
	UE::TUniqueLock _(Mutex);

	if (Ar.IsTransacting())
	{
		// We've made changes to this serialization path without versioning them, because we assume
		// IsTransacting means !IsPersistent and therefore all loaded data was saved with the current binary.
		check(!Ar.IsPersistent()); 
		// Do not process the transaction if the owner is mid loading (see FBulkData::Serialize)
		bool bNeedsTransaction = Ar.IsSaving() && (!Owner || !Owner->HasAnyFlags(RF_NeedLoad));

		Ar << bNeedsTransaction;

		if (bNeedsTransaction)
		{
			EFlags FlagsToPreserve = EFlags::None;

			if (Ar.IsLoading())
			{
				Unregister();

				FlagsToPreserve = Flags & TransientFlags;
			}

			Ar << Flags;
			// Transactions do not save/load the BulkDataId; it is specific to an instance and is
			// unchanged by modifications to the payload. Allowing it to save/load would allow it to be
			// duplicated if an editor operation plays back the transaction buffer multiple times.
			Ar << PayloadContentId;
			Ar << PayloadSize;
			Ar << PackagePath;
			Ar << OffsetInFile;

			// TODO: We could consider compressing the payload so it takes up less space in the 
			// undo stack or even consider storing as a tmp file on disk rather than keeping it
			// in memory or some other caching system.
			// Serializing full 8k texture payloads to memory on each metadata change will empty
			// the undo stack very quickly.

			const uint32 TagValue = 0x12345678;
			uint32 Tag1 = TagValue;
			Ar << Tag1;
			check( Tag1 == TagValue );

			if (Ar.IsSaving())
			{
				if ( PayloadSize == 0 )
				{
					// nothing to do
				}
				else if (Payload.IsNull() && !IsDataVirtualized())
				{
					// We load the data from disk to serialize it to the undo history
					// since don't know that the data on disk will stay the same

					FCompressedBuffer CompressedPayload = GetDataInternal();
					bool bIsCompressedBuffer = true;
					Ar << bIsCompressedBuffer;
					Ar << CompressedPayload;
				}
				else
				{
					bool bIsCompressedBuffer = false;
					Ar << bIsCompressedBuffer;
					Ar << Payload;
				}
			}
			else
			{
				Flags |= FlagsToPreserve;

				if (PayloadSize > 0 && !BulkDataId.IsValid())
				{
					BulkDataId = FGuid::NewGuid();
				}
				
				if ( PayloadSize == 0 )
				{
					// nothing to do
					Payload.Reset();
				}
				else
				{
					bool bIsCompressedBuffer = false;
					Ar << bIsCompressedBuffer;

					if ( bIsCompressedBuffer )
					{
						FCompressedBuffer CompressedPayload;
						Ar << CompressedPayload;
						
						// there's no way to just set a CompressedBuffer for Payload, so we have to Decompress
						//	it might be nice if Payload could be either FSharedBuffer or FCompressedBuffer
						//	then we could just store it without decompressing

						if ( CanUnloadData() )
						{
							//	don't bother to Decompress if it will just be discarded immediately by UnloadData below
							Payload.Reset();
						}
						else
						{
							Payload = CompressedPayload.Decompress();
						}
					}
					else
					{
						Ar << Payload;
					}
				}

				// the call to UnloadData below will then often discard the Payload we just read !?
				// ? is it guaranteed that the Payload we just loaded matches what we will get back from PackagePath after UnloadData ?

				Register(Owner, TEXT("Serialize/Transacting"), false /* bAllowUpdateId */);
			}
			
			uint32 Tag2 = TagValue;
			Ar << Tag2;
			check( Tag2 == TagValue );

			// Try to unload the payload if possible, usually because we loaded it during the transaction in the
			// first place and we don't want to keep it in memory anymore.
			// 
			// BulkData currently does not ever cache the decompressed Payload when it is possible to reload from disk.
			// that is, on typical BulkData, CanUnloadData is always false because the data is either on disk (so Payload is null)
			//	or not on disk
			// we may have gotten out of sync with that standard state here due to serialization,
			//	so go ahead and UnloadData now.
			UnloadData();
		}
	}
	else if (Ar.IsPersistent() && !Ar.IsObjectReferenceCollector() && !Ar.ShouldSkipBulkData())
	{
		FLinkerSave* LinkerSave = nullptr;
		bool bKeepFileDataByReference = false;
		if (Ar.IsSaving())
		{
			LinkerSave = Cast<FLinkerSave>(Ar.GetLinker());
			// If we're doing a save that can refer to bulk data by reference, and our legacy data format supports it,
			// keep any legacy data we have referenced rather than stored, to save space and avoid spending time loading it.
			bKeepFileDataByReference = LinkerSave != nullptr && LinkerSave->bProceduralSave && !Ar.IsCooking();
			if (!bKeepFileDataByReference)
			{
				UpdateKeyIfNeeded();
			}

			if (bAllowVirtualizationOnSave)
			{
				const bool bCanAttemptVirtualization = LinkerSave != nullptr;
				if (bCanAttemptVirtualization)
				{
					FPackagePath LinkerPackagePath;
					FPackagePath::TryFromPackageName(LinkerSave->LinkerRoot->GetName(), LinkerPackagePath);

					PushData(LinkerPackagePath); // Note this can change various members if we are going from non-virtualized to virtualized
				}
			}
		}
		else
		{
			Unregister();
		}

		// Store the position in the archive of the flags in case we need to update it later
		const int64 SavedFlagsPos = Ar.Tell();
		Ar << Flags;
		if (Ar.IsLoading())
		{
			EnumRemoveFlags(Flags, TransientFlags);
		}

		// TODO: Can probably remove these checks before UE5 release
		check(!Ar.IsSaving() || GetPayloadSize() == 0 || BulkDataId.IsValid() || EnumHasAnyFlags(Flags, EFlags::IsCooked)); // Sanity check to stop us saving out bad data
		check(!Ar.IsSaving() || GetPayloadSize() == 0 || !PayloadContentId.IsZero() || EnumHasAnyFlags(Flags, EFlags::IsCooked)); // Sanity check to stop us saving out bad data
		
		Ar << BulkDataId;
		Ar << PayloadContentId;
		Ar << PayloadSize;

		// TODO: Can probably remove these checks before UE5 release
		check(!Ar.IsLoading() || GetPayloadSize() == 0 || BulkDataId.IsValid() || EnumHasAnyFlags(Flags, EFlags::IsCooked)); // Sanity check to stop us loading in bad data
		check(!Ar.IsLoading() || GetPayloadSize() == 0 || !PayloadContentId.IsZero() || EnumHasAnyFlags(Flags, EFlags::IsCooked)); // Sanity check to stop us loading in bad data

		if (Ar.IsSaving())
		{
			EFlags UpdatedFlags = BuildFlagsForSerialization(Ar, bKeepFileDataByReference);

			// Write out required extra data if we're saving by reference
			bool bWriteOutPayload = true;
				
			if (EnumHasAnyFlags(UpdatedFlags, EFlags::IsCooked))
			{
				// If we are cooking, we currently aren't saving any payload, since they aren't supported at runtime
				// TODO: add iostore support for editor bulkdata
				bWriteOutPayload = false;
			}
			else if (IsReferencingByPackagePath(UpdatedFlags))
			{
				// Write out required extra data if we're saving by reference
				if (!IsStoredInPackageTrailer(UpdatedFlags))
				{
					Ar << OffsetInFile;
				}

				bWriteOutPayload = false;
			}
			else
			{
				bWriteOutPayload = !IsDataVirtualized(UpdatedFlags);
			}

			if (bWriteOutPayload)
			{
				// Need to load the payload so that we can write it out
				FCompressedBuffer PayloadToSerialize = GetDataInternal();
				
				if (!TryPayloadValidationForSaving(PayloadToSerialize, LinkerSave))
				{
					Ar.SetError();
					return;
				}

				RecompressForSerialization(PayloadToSerialize, UpdatedFlags);

				// If we are expecting a valid payload but fail to find one something critical has broken so assert now
				// to prevent potentially bad data being saved to disk.
				checkf(PayloadToSerialize || GetPayloadSize() == 0, TEXT("Failed to acquire the payload for saving!"));

				// If we have a valid linker then we will defer serialization of the payload so that it will
				// be placed at the end of the output file so we don't have to seek past the payload on load.
				// If we do not have a linker OR the linker is in text format then we should just serialize
				// the payload directly to the archive.
				if (LinkerSave != nullptr && !LinkerSave->IsTextFormat())
				{	
					if (IsStoredInPackageTrailer(UpdatedFlags))
					{
						// New path that will save the payload to the package trailer
						SerializeToPackageTrailer(*LinkerSave, PayloadToSerialize, UpdatedFlags, Owner);	
					}
					else 
					{
						// Legacy path, will save the payload data to the end of the package
						SerializeToLegacyPath(*LinkerSave, PayloadToSerialize, UpdatedFlags, Owner);
					}	
				}
				else
				{
					// Not saving to a package so serialize inline into the archive
					check(IsStoredInPackageTrailer(UpdatedFlags) == false);

					const int64 OffsetPos = Ar.Tell();

					// Write out a dummy value that we will write over once the payload is serialized
					int64 PlaceholderValue = INDEX_NONE;
					Ar << PlaceholderValue; // OffsetInFile

					int64 DataStartOffset = Ar.Tell();

					SerializeData(Ar, PayloadToSerialize, UpdatedFlags);
					
					UpdateArchiveData(Ar, OffsetPos, DataStartOffset);
				}
			}
			
			if (IsStoredInPackageTrailer(UpdatedFlags) && IsDataVirtualized(UpdatedFlags))
			{
				LinkerSave->PackageTrailerBuilder->AddVirtualizedPayload(PayloadContentId, PayloadSize);
			}

			/** Beyond this point UpdatedFlags will be modified to avoid serializing some flags */
			/** So any code actually using UpdatedFlags should come before this section of code */

			ValidatePackageTrailerBuilder(LinkerSave, PayloadContentId, UpdatedFlags);

			// Remove the IsVirtualized flag if we are storing the payload in a package trailer before we serialize the flags
			// to the package export data. We will determine if a payload is virtualized or not by checking the package trailer.
			if (IsStoredInPackageTrailer(UpdatedFlags))
			{
				EnumRemoveFlags(UpdatedFlags, EFlags::IsVirtualized);
			}

			// Replace the flags we wrote out earlier with the updated, final values
			UpdateArchiveData(Ar, SavedFlagsPos, UpdatedFlags);

			if (CanUnloadData())
			{
				this->CompressionSettings.Reset();
				Payload.Reset();
			}
		}
		else if (Ar.IsLoading())
		{
			if (BulkDataId.IsValid() && ShouldGenerateNewIdentifier(Ar, Owner))
			{
				BulkDataId = FGuid::NewGuid();
			}

			OffsetInFile = INDEX_NONE;
			PackagePath.Empty();

			const FPackageTrailer* Trailer = FindPackageTrailer(Owner, Ar);

			if (IsStoredInPackageTrailer())
			{
				checkf(Trailer != nullptr, TEXT("Payload was stored in a package trailer, but there no trailer loaded. [%s]"), *GetDebugNameFromOwner(Owner));
				// Cache the offset from the trailer (if we move the loading of the payload to the trailer 
				// at a later point then we can skip this)
				OffsetInFile = Trailer->FindPayloadOffsetInFile(PayloadContentId);

				// Attempt to catch data that saved with the virtualization flag when it's package has a trailers.
				// It is unlikely this will ever trigger in the wild but keeping the code path for now to be safe.
				// TODO: Consider removing in 5.2+
				if (IsDataVirtualized() && Trailer->FindPayloadStatus(PayloadContentId) != EPayloadStatus::StoredVirtualized)
				{
					UE_LOGF(LogSerialization, Warning, "Payload was saved with an invalid flag and required fixing. Please re-save the package! [%ls]", *GetDebugNameFromOwner(Owner));
					EnumRemoveFlags(Flags, EFlags::IsVirtualized);
				}
			}
			else
			{
				// This check is for older virtualized formats that might be seen in older test projects.
				// But we only care if the archive has a linker! (loading from a package)
				// TODO: Consider removing in 5.2+
				UE_CLOGF(IsDataVirtualized() && Ar.GetLinker() != nullptr, LogSerialization, Warning, "Payload is virtualized in an older format and should be re-saved! [%ls]", *GetDebugNameFromOwner(Owner));
				if (!IsDataVirtualized() && !EnumHasAnyFlags(Flags, EFlags::IsCooked))
				{
					Ar << OffsetInFile;
				}
			}

			// This cannot be inside the above ::IsStoredInPackageTrailer branch due to the original prototype assets using the trailer without the StoredInPackageTrailer flag
			if (Trailer != nullptr && Trailer->FindPayloadStatus(PayloadContentId) == EPayloadStatus::StoredVirtualized)
			{
				// As the virtualization process happens outside of serialization we need
				// to check with the trailer to see if the payload is virtualized or not
				EnumAddFlags(Flags, EFlags::IsVirtualized);
				OffsetInFile = INDEX_NONE;
			}

			checkf(!(IsDataVirtualized() && IsReferencingByPackagePath()), TEXT("Payload cannot be both virtualized and a reference"));
			checkf(!IsDataVirtualized() || OffsetInFile == INDEX_NONE, TEXT("Virtualized payloads should have an invalid offset"));
			
			if (!IsDataVirtualized())
			{
				// If we can lazy load then find the PackagePath, otherwise we will want to serialize immediately.
				FArchive* CacheableArchive = Ar.GetCacheableArchive();
				if (Ar.IsAllowingLazyLoading() && CacheableArchive != nullptr)
				{
					PackagePath = FindPackagePath(Owner, Ar);
				}
				else
				{
					PackagePath.Empty();
				}
					
				if (!PackagePath.IsEmpty() && CacheableArchive != nullptr)
				{
#if UE_ALLOW_LINKERLOADER_ATTACHMENT
					if (HasAttachedArchive())
					{
						// TODO: Remove this when doing UE-159339
						AttachedAr->DetachBulkData(this, false);
					}

					AttachedAr = CacheableArchive;
					AttachedAr->AttachBulkData(this);
#endif //VBD_ALLOW_LINKERLOADER_ATTACHMENT
				}
				else if (!EnumHasAnyFlags(Flags, EFlags::IsCooked))
				{
					checkf(Ar.Tell() == OffsetInFile, TEXT("Attempting to load an inline payload but the offset does not match"));

					// If the package path is invalid or the archive is not cacheable then we
					// cannot rely on loading the payload at a future point on demand so we need 
					// to load the data immediately.
					FCompressedBuffer CompressedPayload;
					SerializeData(Ar, CompressedPayload, Flags);
					
					// Only decompress if there is actual data, otherwise we might as well just 
					// store the payload as an empty FSharedBuffer.
					if (CompressedPayload.GetRawSize() > 0)
					{
						// an example of when this is hit is in UTexture::Serialize for object duplication
						Payload = CompressedPayload.Decompress();
					}
					else
					{
						Payload.Reset();
					}
				}
			}

			if (bAllowRegister)
			{
				Register(Owner, TEXT("Serialize/Persistent"), true /* bAllowUpdateId */);
			}
		}
	}
}

void FEditorBulkData::SerializeForRegistry(FArchive& Ar)
{
	UE::TUniqueLock _(Mutex);

	if (Ar.IsSaving())
	{
		check(CanSaveForRegistry());
		EFlags FlagsForSerialize = Flags;
		EnumRemoveFlags(FlagsForSerialize, TransientFlags);
		Ar << FlagsForSerialize;
	}
	else
	{
		Ar << Flags;
		EnumRemoveFlags(Flags, TransientFlags);
		EnumAddFlags(Flags, EFlags::IsTornOff);
	}

	Ar << BulkDataId;
	Ar << PayloadContentId;
	Ar << PayloadSize;
	if (Ar.IsSaving())
	{
		FString PackageName = PackagePath.GetPackageName();
		Ar << PackageName;
	}
	else
	{
		FString PackageName;
		Ar << PackageName;
		if (PackageName.IsEmpty())
		{
			PackagePath.Empty();
		}
		else
		{
			ensure(FPackagePath::TryFromPackageName(PackageName, PackagePath));
		}
	}
	Ar << OffsetInFile;
}

bool FEditorBulkData::CanSaveForRegistry() const
{
	UE::TUniqueLock _(Mutex);

	return BulkDataId.IsValid() && PayloadSize > 0 && !IsMemoryOnlyPayload()
		&& EnumHasAnyFlags(Flags, EFlags::IsTornOff) && !EnumHasAnyFlags(Flags, EFlags::HasRegistered);
}


FCompressedBuffer FEditorBulkData::LoadFromDisk() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::LoadFromDisk);

	if (PackagePath.IsEmpty())
	{
		// Bulkdata objects without a valid package path should not get this far when attempting to access a payload!
		UE_LOGF(LogSerialization, Error, "Cannot load a payload as the package path is empty!");
		return FCompressedBuffer();
	}

	if (HasPayloadSidecarFile() && CVarShouldLoadFromSidecar.GetValueOnAnyThread())
	{
		return LoadFromSidecarFile();
	}
	else if (!CanLoadDataFromDisk())
	{
		if (IsReferencingOldBulkData())
		{
			UE_LOGF(LogSerialization, Error, 
				"Cannot attempt to load the payload '%ls' from '%ls' as the package is no longer attached to the file on disk and the payload is in an old style bulkdata structure", 
				*LexToString(PayloadContentId), 
				*PackagePath.GetLocalFullPath(EPackageSegment::Header));

			return FCompressedBuffer();
		}

		if (!IsStoredInPackageTrailer())
		{
			UE_LOGF(LogSerialization, Error, 
				"Cannot attempt to load the payload '%ls' from '%ls' as the package is no longer attached to the file on disk and the payload is not stored in a package trailer",
				*LexToString(PayloadContentId),
				*PackagePath.GetLocalFullPath(EPackageSegment::Header));

			return FCompressedBuffer();
		}

		FCompressedBuffer CompressedPayload = LoadFromPackageTrailer();
		if (!CompressedPayload.IsNull())
		{
			return CompressedPayload;
		}
		else
		{
			UE_LOGF(LogSerialization, Error,
				"Cannot attempt to load the payload '%ls' from '%ls' as the package is no longer attached to the file on disk and the payload was not found in the package trailer",
				*LexToString(PayloadContentId),
				*PackagePath.GetLocalFullPath(EPackageSegment::Header));

			return FCompressedBuffer();
		}
		
	}
	else
	{
		if (CVarShouldLoadFromTrailer.GetValueOnAnyThread() && IsStoredInPackageTrailer())
		{
			return LoadFromPackageTrailer();
		}
		else
		{
			return LoadFromPackageFile();
		}
	}
}

FCompressedBuffer FEditorBulkData::LoadFromPackageFile() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::LoadFromPackageFile);

	UE_LOGF(LogSerialization, Verbose, "Attempting to load payload from the package file '%ls'", *PackagePath.GetLocalFullPath(EPackageSegment::Header));

	// Open a reader to the file
	TUniquePtr<FArchive> BulkArchive;
	if (!IsReferencingByPackagePath())
	{
		FOpenPackageResult Result = IPackageResourceManager::Get().OpenReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header);
		if (Result.Format == EPackageFormat::Binary)
		{
			BulkArchive = MoveTemp(Result.Archive);
		}
	}
	else
	{
		// *this may have been loaded from the EditorDomain, but saved with a reference to the bulk data in the
		// Workspace Domain file. This was only possible if PackageSegment == Header; we checked that when serializing to the EditorDomain
		// In that case, we need to use OpenReadExternalResource to access the Workspace Domain file
		// In the cases where *this was loaded from the WorkspaceDomain, OpenReadExternalResource and OpenReadPackage are identical.
		BulkArchive = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, PackagePath.GetPackageName());
	}

	if (!BulkArchive.IsValid())
	{
		LogPackageOpenFailureMessage(PackagePath, EPackageSegment::Header);
		return FCompressedBuffer();
	}

	checkf(OffsetInFile != INDEX_NONE, TEXT("Attempting to load from the package file '%s' with an invalid OffsetInFile!"), *PackagePath.GetLocalFullPath(EPackageSegment::Header));
	// Move the correct location of the data in the file
	BulkArchive->Seek(OffsetInFile);

	// Now we can actually serialize it
	FCompressedBuffer PayloadFromDisk;
	SerializeData(*BulkArchive, PayloadFromDisk, Flags);

	// If PayloadId is placeholder, compute the actual now that we have the payload available to hash.
	UpdateKeyIfNeeded(PayloadFromDisk);

	return PayloadFromDisk;
}

FCompressedBuffer FEditorBulkData::LoadFromPackageTrailer() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::LoadFromPackageTrailer);

	UE_LOGF(LogSerialization, Verbose, "Attempting to load a payload via the package trailer from file '%ls'", *PackagePath.GetLocalFullPath(EPackageSegment::Header));

	// TODO: Could just get the trailer from the owning FLinkerLoad if still attached

	// Open a reader to the file
	TUniquePtr<FArchive> BulkArchive;
	if (!IsReferencingByPackagePath())
	{
		FOpenPackageResult Result = IPackageResourceManager::Get().OpenReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header);
		if (Result.Format == EPackageFormat::Binary)
		{
			BulkArchive = MoveTemp(Result.Archive);
		}
	}
	else
	{
		// *this may have been loaded from the EditorDomain, but saved with a reference to the bulk data in the
		// Workspace Domain file. This was only possible if PackageSegment == Header; we checked that when serializing to the EditorDomain
		// In that case, we need to use OpenReadExternalResource to access the Workspace Domain file
		// In the cases where *this was loaded from the WorkspaceDomain, OpenReadExternalResource and OpenReadPackage are identical.
		BulkArchive = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, PackagePath.GetPackageName());
	}

	if (!BulkArchive.IsValid())
	{
		LogPackageOpenFailureMessage(PackagePath, EPackageSegment::Header);
		return FCompressedBuffer();
	}

	BulkArchive->Seek(BulkArchive->TotalSize());

	FPackageTrailer Trailer;
	if (Trailer.TryLoadBackwards(*BulkArchive))
	{
		FCompressedBuffer CompressedPayload = Trailer.LoadLocalPayload(PayloadContentId, *BulkArchive);

		UE_CLOGF(CompressedPayload.IsNull(), LogSerialization, Error, "Could not find the payload '%ls' in the package trailer of file '%ls'",
			*LexToString(PayloadContentId ), 
			*PackagePath.GetLocalFullPath(EPackageSegment::Header));

		return CompressedPayload;
	}
	else
	{
		UE_LOGF(LogSerialization, Error, "Could not read the package trailer from file '%ls'", *PackagePath.GetLocalFullPath(EPackageSegment::Header));
		return FCompressedBuffer();
	}
}

FCompressedBuffer FEditorBulkData::LoadFromSidecarFileInternal(ErrorVerbosity Verbosity) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::LoadFromSidecarFileInternal);

	FOpenPackageResult Result = IPackageResourceManager::Get().OpenReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::PayloadSidecar);
	if (Result.Archive.IsValid() && Result.Format == EPackageFormat::Binary)
	{
		FPackageTrailer Trailer;
		if (!FPackageTrailer::TryLoadFromArchive(*Result.Archive, Trailer))
		{
			UE_CLOGF(Verbosity > ErrorVerbosity::None, LogSerialization, Error, "Unable to parse FPackageTrailer in '%ls'", *PackagePath.GetLocalFullPath(EPackageSegment::PayloadSidecar));
			return FCompressedBuffer();
		}

		FCompressedBuffer CompressedPayload = Trailer.LoadLocalPayload(PayloadContentId, *Result.Archive);

		UE_CLOGF(CompressedPayload.IsNull(), LogSerialization, Error, "Could not find the payload '%ls' in '%ls'",
			*LexToString(PayloadContentId),
			*PackagePath.GetLocalFullPath(EPackageSegment::PayloadSidecar));

		return CompressedPayload;
	}
	else if(Verbosity > ErrorVerbosity::None)
	{
		LogPackageOpenFailureMessage(PackagePath, EPackageSegment::PayloadSidecar);
	}

	return FCompressedBuffer();
}

FCompressedBuffer FEditorBulkData::LoadFromSidecarFile() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::LoadFromSidecarFile);

	UE_LOGF(LogSerialization, Verbose, "Attempting to load payload from the sidecar file: '%ls'",
		*PackagePath.GetLocalFullPath(EPackageSegment::PayloadSidecar));

	if (CVarShouldAllowSidecarSyncing.GetValueOnAnyThread())
	{
		FCompressedBuffer PayloadFromDisk = LoadFromSidecarFileInternal(ErrorVerbosity::None);
		if (PayloadFromDisk.IsNull())
		{
			UE_LOGF(LogSerialization, Verbose, "Initial load from sidecar failed, attempting to sync the file: '%ls'",
				*PackagePath.GetLocalFullPath(EPackageSegment::PayloadSidecar));

			if (UE::Virtualization::Experimental::IVirtualizationSourceControlUtilities* SourceControlInterface = GetSourceControlInterface())
			{
				// SyncPayloadSidecarFile should log failure cases, so there is no need for us to add log messages here
				if (SourceControlInterface->SyncPayloadSidecarFile(PackagePath))
				{
					PayloadFromDisk = LoadFromSidecarFileInternal(ErrorVerbosity::All);
				}
			}
			else
			{
				UE_LOGF(LogSerialization, Error, "Failed to find IVirtualizationSourceControlUtilities, unable to try and sync: '%ls'",
					*PackagePath.GetLocalFullPath(EPackageSegment::PayloadSidecar));
			}
		}

		return PayloadFromDisk;
	}
	else
	{
		return LoadFromSidecarFileInternal(ErrorVerbosity::All);
	}
}

bool FEditorBulkData::SerializeData(FArchive& Ar, FCompressedBuffer& InPayload, const EFlags PayloadFlags) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::SerializeData);

	if (Ar.IsSaving())
	{
		Ar << InPayload;
		return true;
	}
	else if (Ar.IsLoading() && !IsReferencingOldBulkData(PayloadFlags))
	{
		Ar << InPayload;
		return InPayload.IsNull();
	}
	else if (Ar.IsLoading()) 
	{
		// Loading from old bulkdata format
		const int64 Size = GetPayloadSize();

		FUniqueBuffer LoadPayload;
		if (EnumHasAnyFlags(PayloadFlags, EFlags::LegacyFileIsCompressed))
		{
			LoadPayload = FUniqueBuffer::Alloc(Size);
			Ar.SerializeCompressed(LoadPayload.GetData(), Size, NAME_Zlib, COMPRESS_NoFlags, false);
		}
		else if(const int64 SpaceRemaining = Ar.TotalSize() - Ar.Tell(); SpaceRemaining >= Size)
		{
			LoadPayload = FUniqueBuffer::Alloc(Size);
			Ar.Serialize(LoadPayload.GetData(), Size);
		}
		else
		{
			Ar.SetError();
			UE_LOGF(LogSerialization, Error, "Not enough data remaining in archive (%ls) to load the payload. Bytes required: %lld vs bytes remaining %lld", *Ar.GetArchiveName(), Size, SpaceRemaining);

			return false;
		}

		InPayload = FCompressedBuffer::Compress(LoadPayload.MoveToShared(), ECompressedBufferCompressor::NotSet, ECompressedBufferCompressionLevel::None);

		return true;
	}
	else
	{
		return false;
	}
}

void FEditorBulkData::PushData(const FPackagePath& InPackagePath)
{
	checkf(IsDataVirtualized() == false || Payload.IsNull(), TEXT("Cannot have a valid payload in memory if the payload is virtualized!")); // Sanity check

	// We only need to push if the payload if it actually has data and it is not 
	// currently virtualized (either we have an updated payload in memory or the 
	// payload is currently non-virtualized and stored on disk)
	
	UE::Virtualization::IVirtualizationSystem& VirtualizationSystem = UE::Virtualization::IVirtualizationSystem::Get();
	if (!IsDataVirtualized() && GetPayloadSize() > 0 && VirtualizationSystem.IsEnabled())
	{ 
		TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::PushData);

		// We should only need to load from disk at this point if we are going from
		// a non-virtualized payload to a virtualized one. If the bulkdata is merely being
		// edited then we should have the payload in memory already and are just accessing a
		// reference to it.

		UpdateKeyIfNeeded();
		FCompressedBuffer PayloadToPush = GetDataInternal();
		// TODO: If the push fails we will end up potentially re-compressing this payload for
		// serialization, we need a better way to save the results of 'RecompressForSerialization'
		RecompressForSerialization(PayloadToPush, Flags);

		// TODO: We could make the storage type a config option?
		if (VirtualizationSystem.PushData(PayloadContentId, PayloadToPush, InPackagePath.GetPackageName(), UE::Virtualization::EStorageType::Cache))
		{
			EnumAddFlags(Flags, EFlags::IsVirtualized);
			EnumRemoveFlags(Flags, EFlags::ReferencesLegacyFile | EFlags::ReferencesWorkspaceDomain | EFlags::LegacyFileIsCompressed);
			check(!EnumHasAnyFlags(Flags, EFlags::LegacyKeyWasGuidDerived)); // Removed by UpdateKeyIfNeeded

			// Clear members associated with non-virtualized data and release the in-memory
			// buffer.
			PackagePath.Empty();
			OffsetInFile = INDEX_NONE;

			// Update our information in the registry
			UpdateRegistrationData(nullptr, TEXT("PushData"), false /* bAllowUpdateId */);
		}
	}	
}

FCompressedBuffer FEditorBulkData::PullData() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::PullData);

	// Utility lambda used to generate a postfix for VA failure messages based on if the project
	// wants to know the cooked status of the editor bulkdata.
	auto PostFixStatus = [](EFlags InFlags)
		{
			if (ShouldLogCookedStatus())
			{
				return EnumHasAnyFlags(InFlags, EFlags::IsCooked) ? TEXT(" [Cooked]") : TEXT(" [Uncooked]");
			}
			else
			{
				return TEXT("");
			}
		};

	UE::Virtualization::IVirtualizationSystem& System = UE::Virtualization::IVirtualizationSystem::Get();
	if (System.IsEnabled())
	{
		FCompressedBuffer PulledPayload = UE::Virtualization::IVirtualizationSystem::Get().PullData(PayloadContentId);

		checkf(!PulledPayload || PayloadSize == PulledPayload.GetRawSize(),
			TEXT("Mismatch between serialized length (%" INT64_FMT ") and virtualized data length (%" UINT64_FMT ")"),
			PayloadSize,
			PulledPayload.GetRawSize());

		UE_CLOGF(PulledPayload.IsNull(), LogSerialization, Error, "Failed to pull payload '%ls'%ls",
			*LexToString(PayloadContentId),
			PostFixStatus(Flags));

		return PulledPayload;
	}
	else
	{
		UE_LOGF(LogSerialization, Error, "Failed to pull payload '%ls' as the virtualization system is disabled%ls",
			*LexToString(PayloadContentId), 
			PostFixStatus(Flags));

		return FCompressedBuffer();
	}	
}

bool FEditorBulkData::CanUnloadData() const
{
	// Technically if we have a valid path but are detached we can still try  to load 
	// the payload from disk via ::LoadFromPackageTrailer but we are not guaranteed success
	// because the package file may have changed to one without the payload in it at all.
	// Due to this we do not allow the payload to be unloaded if we are detached.
	return IsDataVirtualized() || CanLoadDataFromDisk();
}

bool FEditorBulkData::CanLoadDataFromDisk() const
{
	return !PackagePath.IsEmpty() && !EnumHasAnyFlags(Flags, EFlags::WasDetached);
}

bool FEditorBulkData::IsMemoryOnlyPayload() const
{
	UE::TUniqueLock _(Mutex);
	return !Payload.IsNull() && !IsDataVirtualized() && PackagePath.IsEmpty();
}

void FEditorBulkData::Reset()
{
	UE::TUniqueLock _(Mutex);

	// Unregister rather than allowing the Registry to keep our record, since we are changing the payload
	Unregister();
	// Note that we do not reset the BulkDataId
	if (HasAttachedArchive())
	{
		AttachedAr->DetachBulkData(this, false);
		AttachedAr = nullptr;
	}

	PayloadContentId.Reset();
	Payload.Reset();
	PayloadSize = 0;
	OffsetInFile = INDEX_NONE;
	PackagePath.Empty();
	Flags = EFlags::None;

	CompressionSettings.Reset();
}

void FEditorBulkData::UnloadData()
{
	UE::TUniqueLock _(Mutex);

	// This only does anything in unusual cases.
	// In standard use, if a BulkData has disk backing (so CanUnloadData), then the in-memory Payload is always null
	//	 because decompression does NOT cache the decompressed result in Payload.
	// If there is no disk backing (so CanUnloadData is false), then Payload is not null.
	// In both cases, UnloadData is a nop.
	// The only time it does anything is if you had in-memory payload, then created a disk backing,
	//	so CanUnloadData transitions from false to true,
	//  then the next call to UnloadData will act to drop the in-memory payload.

	if ( ! Payload.IsNull() && CanUnloadData())
	{
		Payload.Reset();
	}
}

void FEditorBulkData::DetachFromDisk(FArchive* Ar, bool bEnsurePayloadIsLoaded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::DetachFromDisk);

	UE::TUniqueLock _(Mutex);

	check(Ar != nullptr);
	check(Ar == AttachedAr || HasAttachedArchive() == false || AttachedAr->IsProxyOf(Ar));

	// If bEnsurePayloadIsLoaded is true, then we should assume that a change to the 
	// package file is imminent and we should load the payload into memory if possible.
	// If it is false then either we are not expecting a change to the package file or
	// we are not expected that the payload will be used again and can unload the 
	// payload. If the payload is requested later we will try to load it off disk via
	// the package trailer, but there is no guarantee that it will be present in the
	// package file.
	if (!IsDataVirtualized() && !PackagePath.IsEmpty())
	{
		if (Payload.IsNull() && bEnsurePayloadIsLoaded)
		{
			FCompressedBuffer CompressedPayload = GetDataInternal();
			TRACE_CPUPROFILER_EVENT_SCOPE(DecompressPayload);
			Payload = CompressedPayload.Decompress();
		}

		OffsetInFile = INDEX_NONE;

		if (EnumHasAnyFlags(Flags, EFlags::HasRegistered))
		{
			if (bEnsurePayloadIsLoaded)
			{
				// We are e.g. renaming a package and should switch registration over to our in-memory Payload
				UpdateRegistrationData(nullptr, TEXT("DetachFromDisk"), false /* bAllowUpdateId */);
			}
			else
			{
				// The package is unloading; we should instruct the bulkdataregistry to keep our old packagepath information,
				// but let it know that we no longer have an in-memory copy of the data.
				OnExitMemory();
			}
		}
	}

	AttachedAr = nullptr;	

	EnumAddFlags(Flags, EFlags::WasDetached);
}

FGuid FEditorBulkData::GetIdentifier() const
{
	UE::TUniqueLock _(Mutex);

	checkf(GetPayloadSize() == 0 || BulkDataId.IsValid(), TEXT("If bulkdata has a valid payload then it should have a valid BulkDataId"));
	return BulkDataId;
}

const FIoHash& FEditorBulkData::GetPayloadId() const
{
	UE::TUniqueLock _(Mutex);

	return PayloadContentId;
}

void FEditorBulkData::SerializeToLegacyPath(FLinkerSave& LinkerSave, FCompressedBuffer PayloadToSerialize, EFlags UpdatedFlags, UObject* Owner)
{
	const int64 OffsetPos = LinkerSave.Tell();

	// Write out a dummy value that we will write over once the payload is serialized
	int64 PlaceholderValue = INDEX_NONE;
	LinkerSave << PlaceholderValue; // OffsetInFile

	// The lambda is mutable so that PayloadToSerialize is not const (due to FArchive api not accepting const values)
	auto SerializePayload = [this, OffsetPos, PayloadToSerialize, UpdatedFlags, Owner](FLinkerSave& LinkerSave, FArchive& ExportsArchive, FArchive& DataArchive, int64 DataStartOffset) mutable
	{
		checkf(ExportsArchive.IsCooking() == false, TEXT("FEditorBulkData payload should not be written during a cook"));

		SerializeData(DataArchive, PayloadToSerialize, UpdatedFlags);

		UpdateArchiveData(ExportsArchive, OffsetPos, DataStartOffset);

		// If we are saving the package to disk (we have access to FLinkerSave and its filepath is valid) 
		// then we should register a callback to be received once the package has actually been saved to 
		// disk so that we can update the object's members to be redirected to the saved file.
		if (!LinkerSave.GetFilename().IsEmpty())
		{
			// At some point saving to the sidecar file will be mutually exclusive with saving to the asset file, at that point
			// we can split these code paths entirely for clarity. (might need to update ::BuildFlagsForSerialization at that point too!)
			if (ShouldSaveToPackageSidecar())
			{
				FLinkerSave::FSidecarStorageInfo& SidecarData = LinkerSave.SidecarDataToAppend.AddZeroed_GetRef();
				SidecarData.Identifier = PayloadContentId;
				SidecarData.Payload = PayloadToSerialize;
			}

			auto OnSavePackage = [this, DataStartOffset, UpdatedFlags, Owner](const FPackagePath& InPackagePath, FObjectPostSaveContext ObjectSaveContext)
			{
				if (!ObjectSaveContext.IsUpdatingLoadedPath())
				{
					return;
				}

				// Mark the bulkdata as detached so that we don't try to load the payload from the newly saved package
				// or unload the current payload as under the existing system it is not safe to load from a package
				// that we are not attached to.
				EnumAddFlags(this->Flags, EFlags::WasDetached);

				this->PackagePath = InPackagePath;
				check(!this->PackagePath.IsEmpty()); // LinkerSave guarantees a valid PackagePath if we're updating loaded path
				this->OffsetInFile = DataStartOffset;
				this->Flags = (this->Flags & TransientFlags) | (UpdatedFlags & ~TransientFlags);

				if (CanUnloadData())
				{
					this->CompressionSettings.Reset();
					this->Payload.Reset();
				}

				// Update our information in the registry
				UpdateRegistrationData(Owner, TEXT("SerializeToLegacyPath"), false /* bAllowUpdateId */);
			};

			LinkerSave.PostSaveCallbacks.Add(MoveTemp(OnSavePackage));
		}
	};

	auto AdditionalDataCallback = [SerializePayload = MoveTemp(SerializePayload)](FLinkerSave& ExportsArchive, FArchive& DataArchive, int64 DataStartOffset) mutable
	{
		SerializePayload(ExportsArchive, ExportsArchive, DataArchive, DataStartOffset);
	};

	LinkerSave.AdditionalDataToAppend.Add(MoveTemp(AdditionalDataCallback));	// -V595 PVS believes that LinkerSave can potentially be nullptr at 
																				// this point however we test LinkerSave != nullptr to enter this branch.
}

void FEditorBulkData::SerializeToPackageTrailer(FLinkerSave& LinkerSave, FCompressedBuffer PayloadToSerialize, EFlags UpdatedFlags, UObject* Owner)
{
	auto OnPayloadWritten = [this, UpdatedFlags, Owner](FLinkerSave& LinkerSave, const FPackageTrailer& Trailer) mutable
	{
		checkf(LinkerSave.IsCooking() == false, TEXT("FEditorBulkData payload should not be written to a package trailer during a cook"));

		int64 PayloadOffset = Trailer.FindPayloadOffsetInFile(PayloadContentId);

		// If we are saving the package to disk (we have access to FLinkerSave and its filepath is valid) 
		// then we should register a callback to be received once the package has actually been saved to 
		// disk so that we can update the object's members to be redirected to the saved file.
		if (!LinkerSave.GetFilename().IsEmpty())
		{
			auto OnSavePackage = [this, PayloadOffset, UpdatedFlags, Owner](const FPackagePath& InPackagePath, FObjectPostSaveContext ObjectSaveContext)
			{
				if (!ObjectSaveContext.IsUpdatingLoadedPath())
				{
					return;
				}

				this->OffsetInFile = PayloadOffset;
				this->Flags = (this->Flags & TransientFlags) | (UpdatedFlags & ~TransientFlags);

				// If the payload is valid we might want to fix up the package path or virtualization flags now
				// that the package has saved.
				if (!this->PayloadContentId.IsZero())
				{
					if (PayloadOffset != INDEX_NONE)
					{
						// Mark the bulkdata as detached so that we don't try to load the payload from the newly saved package
						// or unload the current payload as under the existing system it is not safe to load from a package
						// that we are not attached to.
						EnumAddFlags(this->Flags, EFlags::WasDetached);

						this->PackagePath = InPackagePath;
						check(!this->PackagePath.IsEmpty()); // LinkerSave guarantees a valid PackagePath if we're updating loaded path
					}
					else
					{
						// If the payload offset we are given is INDEX_NONE it means that the payload was discarded as there was an identical
						// virtualized entry and the virtualized version takes priority. Since we know that the payload is in the 
						// virtualization system at this point we can set the virtualization flag allowing us to unload any existing payload
						// in memory to help reduce bloat.
						EnumAddFlags(this->Flags, EFlags::IsVirtualized);
					}
				}

				if (CanUnloadData())
				{
					this->CompressionSettings.Reset();
					this->Payload.Reset();
				}

				// Update our information in the registry
				UpdateRegistrationData(Owner, TEXT("SerializeToPackageTrailer"), false /* bAllowUpdateId */);
			};

			LinkerSave.PostSaveCallbacks.Add(MoveTemp(OnSavePackage));
		}
	};

	UE::Virtualization::EPayloadFilterReason PayloadFilter = UE::Virtualization::EPayloadFilterReason::None;
	if (UE::Virtualization::IVirtualizationSystem::IsInitialized())
	{
		PayloadFilter = UE::Virtualization::IVirtualizationSystem::Get().FilterPayload(Owner);
	}

	LinkerSave.PackageTrailerBuilder->AddPayload(PayloadContentId, MoveTemp(PayloadToSerialize), PayloadFilter, MoveTemp(OnPayloadWritten));
}

void FEditorBulkData::UpdatePayloadImpl(FSharedBuffer&& InPayload, const FIoHash& InPayloadID, UObject* Owner)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::UpdatePayloadImpl);

	UE::TUniqueLock _(Mutex);

	// Unregister before calling DetachBulkData; DetachFromDisk calls OnExitMemory which is incorrect since
	// we are changing data rather than leaving memory
	Unregister();
	if (HasAttachedArchive())
	{
		AttachedAr->DetachBulkData(this, false);
		AttachedAr = nullptr;
	}

	// We only take the payload if it has a length to avoid potentially holding onto a
	// 0 byte allocation in the FSharedBuffer
	if (InPayload.GetSize() > 0)
	{ 
		Payload = MoveTemp(InPayload).MakeOwned();
	}
	else
	{
		Payload.Reset();
	}

	PayloadSize = (int64)Payload.GetSize();
	PayloadContentId = InPayloadID;

	EnumRemoveFlags(Flags,	EFlags::IsVirtualized |
							EFlags::ReferencesLegacyFile |
							EFlags::ReferencesWorkspaceDomain |
							EFlags::LegacyFileIsCompressed |
							EFlags::LegacyKeyWasGuidDerived);

	PackagePath.Empty();
	OffsetInFile = INDEX_NONE;

	if (PayloadSize > 0 && !BulkDataId.IsValid())
	{
		BulkDataId = FGuid::NewGuid();
	}
	UpdateRegistrationData(Owner, TEXT("UpdatePayloadImpl"), false /* bAllowUpdateId */);
}

FCompressedBuffer FEditorBulkData::GetDataInternal() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::GetDataInternal);

	// Early out there isn't any data to actually load
	if (GetPayloadSize() == 0)
	{
		return FCompressedBuffer();
	}

	// Check if we already have the data in memory
	if (Payload)
	{
		// Note that this doesn't actually compress the data!
		//	it reinterprets Payload as an FCompressedBuffer, doesn't have to alloc or memcpy, just points at it
		return FCompressedBuffer::Compress(Payload, ECompressedBufferCompressor::NotSet, ECompressedBufferCompressionLevel::None);
	}

	if (IsDataVirtualized())
	{
		FCompressedBuffer CompressedPayload = PullData();
		
		checkf(Payload.IsNull(), TEXT("Pulling data somehow assigned it to the bulk data object!")); //Make sure that we did not assign the buffer internally

		UE_CLOGF(!IsDataValid(*this, CompressedPayload), LogSerialization, UE_CORRUPTED_DATA_SEVERITY, "Virtualized payload '%ls' is corrupt! Check the backend storage.", *LexToString(PayloadContentId));
		
		TRACE_COUNTER_ADD(EditorBulkData_PayloadDataPulled, (int64)CompressedPayload.GetCompressedSize());
		return CompressedPayload;
	}
	else
	{
		FCompressedBuffer CompressedPayload = LoadFromDisk();
		
		check(Payload.IsNull()); //Make sure that we did not assign the buffer internally

		UE_CLOGF(CompressedPayload.IsNull(), LogSerialization, Error, "Failed to load payload '%ls", *LexToString(PayloadContentId));
		UE_CLOGF(!IsDataValid(*this, CompressedPayload), LogSerialization, UE_CORRUPTED_DATA_SEVERITY, "Payload '%ls' loaded from package '%ls' is corrupt! Check the package file on disk.",
			*LexToString(PayloadContentId),
			*PackagePath.GetDebugName());

		TRACE_COUNTER_ADD(EditorBulkData_PayloadDataLoaded, (int64)CompressedPayload.GetCompressedSize());
		return CompressedPayload;
	}
}

bool FEditorBulkData::IsPayloadLoaded() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::IsPayloadLoaded);
	// this function is very simple, but can still be a long time scope
	//	 because acquiring the mutex blocks if the main thread is serializing the bulkdata to the undo transaction buffer
	UE::TUniqueLock _(Mutex);
	return !Payload.IsNull();
}

bool FEditorBulkData::DoesPayloadNeedLoading() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::DoesPayloadNeedLoading);
	// this function is very simple, but can still be a long time scope
	//	 because acquiring the mutex blocks if the main thread is serializing the bulkdata to the undo transaction buffer

	UE::TUniqueLock _(Mutex);
	return Payload.IsNull() && PayloadSize > 0;
}

TFuture<FSharedBuffer> FEditorBulkData::GetPayload() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::GetPayload);
	// this function is very simple, but can still be a long time scope
	//	 because acquiring the mutex blocks if the main thread is serializing the bulkdata to the undo transaction buffer

	UE::TUniqueLock _(Mutex);

	TPromise<FSharedBuffer> Promise;
	
	if (GetPayloadSize() == 0)
	{
		// Early out for 0 sized payloads
		Promise.SetValue(FSharedBuffer());
	}
	else if (Payload)
	{
		// Avoid a unnecessary compression and decompression if we already have the uncompressed payload
		Promise.SetValue(Payload);
	}
	else
	{
		FCompressedBuffer CompressedPayload = GetDataInternal();

		FSharedBuffer DecompressedPayload = CompressedPayload.Decompress();

		// NOTE: DecompressedPayload is *NOT* cached in the Payload variable
		//	so UnloadData will do nothing
		//	and we will reload from disk if GetPayload() is called twice in a row
		
		// TODO: Not actually async yet!
		Promise.SetValue(DecompressedPayload);
	}

	return Promise.GetFuture();
}

TFuture<FCompressedBuffer>FEditorBulkData::GetCompressedPayload() const
{
	UE::TUniqueLock _(Mutex);

	TPromise<FCompressedBuffer> Promise;

	FCompressedBuffer CompressedPayload = GetDataInternal();

	// TODO: Not actually async yet!
	Promise.SetValue(MoveTemp(CompressedPayload));

	return Promise.GetFuture();
}

void FEditorBulkData::UpdatePayload(FSharedBuffer InPayload, UObject* Owner)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::UpdatePayload (FSharedBuffer));
	FIoHash NewPayloadId = HashPayload(InPayload);

	UpdatePayloadImpl(MoveTemp(InPayload), NewPayloadId, Owner);
}

void FEditorBulkData::UpdatePayload(FCompressedBuffer InPayload, UObject* Owner)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEditorBulkData::UpdatePayload(FCompressedBuffer));

	FIoHash NewPayloadId = InPayload.GetRawSize() > 0 ? InPayload.GetRawHash() : FIoHash();
	FSharedBuffer NewPayload = InPayload.DecompressToComposite().ToShared();

	UpdatePayloadImpl(MoveTemp(NewPayload), NewPayloadId, Owner);
}

FEditorBulkData::FSharedBufferWithID::FSharedBufferWithID(FSharedBuffer InPayload)
	: Payload(MoveTemp(InPayload))
	, PayloadId(HashPayload(Payload))
{
}

void FEditorBulkData::UpdatePayload(FSharedBufferWithID InPayload, UObject* Owner)
{
	UpdatePayloadImpl(MoveTemp(InPayload.Payload), InPayload.PayloadId, Owner);
}

void FEditorBulkData::SetCompressionOptions(ECompressionOptions Option)
{
	UE::TUniqueLock _(Mutex);

	switch (Option)
	{
	case ECompressionOptions::Disabled:
		CompressionSettings.SetToDisabled();
		break;
	case ECompressionOptions::Default:
		CompressionSettings.Reset();
		break;
	default:
		checkNoEntry();
	}

	if (CompressionSettings.GetCompressionLevel() == ECompressedBufferCompressionLevel::None)
	{
		EnumAddFlags(Flags, EFlags::DisablePayloadCompression);
	}
	else
	{
		EnumRemoveFlags(Flags, EFlags::DisablePayloadCompression);
	}
}

void FEditorBulkData::SetCompressionOptions(ECompressedBufferCompressor Compressor, ECompressedBufferCompressionLevel CompressionLevel)
{
	UE::TUniqueLock _(Mutex);

	CompressionSettings.Set(Compressor, CompressionLevel);

	if (CompressionSettings.GetCompressionLevel() == ECompressedBufferCompressionLevel::None)
	{
		EnumAddFlags(Flags, EFlags::DisablePayloadCompression);
	}
	else
	{
		EnumRemoveFlags(Flags, EFlags::DisablePayloadCompression);
	}
}

void FEditorBulkData::GetBulkDataVersions(FArchive& InlineArchive, FPackageFileVersion& OutUEVersion,
	int32& OutLicenseeUEVersion, FCustomVersionContainer& OutCustomVersions) const
{
	TUniquePtr<FArchive> ExternalArchive;

	{
		UE::TUniqueLock _(Mutex);
		if (EnumHasAnyFlags(Flags, EFlags::ReferencesWorkspaceDomain))
		{
			// Read the version data out of the separate package file
			ExternalArchive = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, PackagePath.GetPackageName());
		}
	}

	if (ExternalArchive.IsValid())
	{
		FPackageFileSummary PackageFileSummary;
		*ExternalArchive << PackageFileSummary;
		if (PackageFileSummary.Tag == PACKAGE_FILE_TAG && !ExternalArchive->IsError())
		{
			OutUEVersion = PackageFileSummary.GetFileVersionUE();
			OutLicenseeUEVersion = PackageFileSummary.GetFileVersionLicenseeUE();
			OutCustomVersions = PackageFileSummary.GetCustomVersionContainer();

			return;
		}
	}
	
	OutUEVersion = InlineArchive.UEVer();
	OutLicenseeUEVersion = InlineArchive.LicenseeUEVer();
	OutCustomVersions = InlineArchive.GetCustomVersions();
}

void FEditorBulkData::UpdatePayloadId()
{
	UE::TUniqueLock _(Mutex);

	UpdateKeyIfNeeded();
}

bool FEditorBulkData::LocationMatches(const FEditorBulkData& Other) const
{
	UE::TMultiUniqueLock<FRecursiveMutex> _({ &Mutex, &Other.Mutex });

	if (GetIdentifier() != Other.GetIdentifier())
	{
		// Different identifiers return false, even if location is the same
		return false;
	}
	if (GetPayloadSize() == 0 || Other.GetPayloadSize() == 0)
	{
		// True if both of them have an empty payload, false if only one does
		return GetPayloadSize() == 0 && Other.GetPayloadSize() == 0;
	}
	else if (IsDataVirtualized() || Other.IsDataVirtualized())
	{
		// Data is virtualized and will be looked up by PayloadContentId; use PayloadContentId
		return IsDataVirtualized() && Other.IsDataVirtualized() &&
			PayloadContentId == Other.PayloadContentId;
	}
	else if (!PackagePath.IsEmpty() || !Other.PackagePath.IsEmpty())
	{
		if (Other.PackagePath != PackagePath)
		{
			return false;
		}
		if ((HasPayloadSidecarFile() && CVarShouldLoadFromSidecar.GetValueOnAnyThread()) ||
			CVarShouldLoadFromTrailer.GetValueOnAnyThread())
		{
			// Loading from the SidecarFile or PackageTrailer uses the PayloadContentId as
			// the identifier to look up the location in the file
			return PayloadContentId == Other.PayloadContentId;
		}
		else
		{
			// Loading from PackagePath without sidecar or trailer uses the OffsetInFile
			return OffsetInFile == Other.OffsetInFile;
		}
	}
	else
	{
		// BulkData is memory-only and therefore PayloadContentId is available, use PayloadContentId
		return PayloadContentId == Other.PayloadContentId;
	}
}

void FEditorBulkData::UpdateKeyIfNeeded()
{
	// If this was created from old BulkData then the key is generated from an older FGuid, we
	// should recalculate it based off the payload to keep the key consistent in the future.
	if (EnumHasAnyFlags(Flags, EFlags::LegacyKeyWasGuidDerived))
	{
		checkf(IsDataVirtualized() == false, TEXT("Cannot have a virtualized payload if loaded from legacy BulkData")); // Sanity check

		// Call GetDataInternal to load the payload from disk; LoadFromPackageFile will call the version of
		// UpdateKeyIfNeeded that takes the decompressed Payload parameter.
		FSharedBuffer LocalPayload = GetDataInternal().Decompress();

		if (EnumHasAnyFlags(Flags, EFlags::LegacyKeyWasGuidDerived))
		{
			ensureMsgf(false, TEXT("EditorBulkData logic error: LegacyKeyWasGuidDerived was not removed by GetDataInternal; this is unexpected and a minor performance problem."));
			UpdateKeyIfNeeded(FCompressedBuffer::Compress(LocalPayload,
				ECompressedBufferCompressor::NotSet, ECompressedBufferCompressionLevel::None));
			check(!EnumHasAnyFlags(Flags, EFlags::LegacyKeyWasGuidDerived)); // UpdateKeyIfNeeded(FCompressedBuffer) guarantees the conversion, so a fatal assert is warranted
		}

		// Store as the in memory payload, since this method is only called during saving 
		// we know it will get cleared anyway.
		Payload = LocalPayload;
	}
}

void FEditorBulkData::UpdateKeyIfNeeded(FCompressedBuffer CompressedPayload) const
{
	if (EnumHasAnyFlags(Flags, EFlags::LegacyKeyWasGuidDerived))
	{
		// PayloadContentId and its associated flag are mutable for legacy BulkDatas.
		// We do a const-cast instead of marking them mutable because they are immutable for non-legacy.
		const_cast<FEditorBulkData&>(*this).PayloadContentId = CompressedPayload.GetRawHash();

		EnumRemoveFlags(const_cast<FEditorBulkData&>(*this).Flags, EFlags::LegacyKeyWasGuidDerived);
#if WITH_EDITOR
		if (EnumHasAnyFlags(Flags, EFlags::HasRegistered))
		{
			IBulkDataRegistry::Get().UpdatePlaceholderPayloadId(*this);
		}
#endif
	}
}

void FEditorBulkData::RecompressForSerialization(FCompressedBuffer& InOutPayload, EFlags PayloadFlags) const
{
	Private::FCompressionSettings CurrentSettings(InOutPayload);
	Private::FCompressionSettings TargetSettings;

	if (EnumHasAnyFlags(PayloadFlags, EFlags::DisablePayloadCompression))
	{
		// If the disable payload compression flag is set, then we should not compress the payload
		TargetSettings.SetToDisabled(); 
	}
	else if (CompressionSettings.IsSet())
	{
		// If we have pending compression settings then we can apply them to the payload
		TargetSettings = CompressionSettings;
	}
	else if(!CurrentSettings.IsCompressed()) 
	{
		// If we have no settings to apply to the payload and the payload is currently uncompressed then we
		// should use the default compression settings.
		TargetSettings.SetToDefault();
	}
	else
	{
		// If we have no settings to apply to the payload but the payload is already compressed then we can
		// just keep the existing settings, what ever they are.
		TargetSettings = CurrentSettings;
	}
	
	// Now we will re-compress the input payload if the current compression settings differ from the desired settings
	if (TargetSettings != CurrentSettings)
	{
		FCompositeBuffer DecompressedBuffer = InOutPayload.DecompressToComposite();

		// If the buffer actually decompressed we can have both the compressed and the uncompressed version of the
		// payload in memory. Compressing it will create a third version so before doing that we should reset
		// the original compressed buffer in case that we can release it to reduce higher water mark pressure.
		InOutPayload.Reset();

		InOutPayload = FCompressedBuffer::Compress(DecompressedBuffer, TargetSettings.GetCompressor(), TargetSettings.GetCompressionLevel());
	}
}

FEditorBulkData::EFlags FEditorBulkData::BuildFlagsForSerialization(FArchive& Ar, bool bKeepFileDataByReference) const
{
	if (Ar.IsSaving())
	{
		EFlags UpdatedFlags = Flags;

		const FLinkerSave* LinkerSave = Cast<FLinkerSave>(Ar.GetLinker());

		// Now update any changes to the flags that we might need to make when serializing.
		// Note that these changes are not applied to the current object UNLESS we are saving
		// the package, in which case the newly modified flags will be applied once we confirm
		// that the package has saved.

		bool bIsReferencingByPackagePath = IsReferencingByPackagePath(UpdatedFlags);
		bool bCanKeepFileDataByReference = bIsReferencingByPackagePath || !PackagePath.IsEmpty();
		if (bKeepFileDataByReference && bCanKeepFileDataByReference)
		{
			if (!bIsReferencingByPackagePath)
			{
				EnumAddFlags(UpdatedFlags, EFlags::ReferencesWorkspaceDomain);
			}
			EnumRemoveFlags(UpdatedFlags, EFlags::HasPayloadSidecarFile | EFlags::IsVirtualized);
		}
		else
		{
			EnumRemoveFlags(UpdatedFlags, EFlags::ReferencesLegacyFile | EFlags::ReferencesWorkspaceDomain | 
				EFlags::LegacyFileIsCompressed | EFlags::LegacyKeyWasGuidDerived);
		
			if (LinkerSave != nullptr && !LinkerSave->GetFilename().IsEmpty() && ShouldSaveToPackageSidecar())
			{
				EnumAddFlags(UpdatedFlags, EFlags::HasPayloadSidecarFile);
				EnumRemoveFlags(UpdatedFlags, EFlags::IsVirtualized);
			}
			else
			{
				EnumRemoveFlags(UpdatedFlags, EFlags::HasPayloadSidecarFile);

				// Check to see if we need to rehydrate the payload on save, this is true if
				// a) We are saving to a package (LinkerSave is not null)
				// b) Either the save package call we made with the  ESaveFlags::SAVE_RehydratePayloads flag set OR
				// the cvar Serialization.RehydrateOnSave is true
				// c) We are not saving a reference to a different domain. If we are saving a reference then the goal
				// is to avoid including the payload in the current target domain if possible, rehydrating for this
				// domain would prevent this.
				if ( LinkerSave != nullptr && 
					(LinkerSave->bRehydratePayloads || CVarShouldRehydrateOnSave.GetValueOnAnyThread()) &&
					!bKeepFileDataByReference)
				{
					EnumRemoveFlags(UpdatedFlags, EFlags::IsVirtualized);
				}
			}
		}

		// Currently we do not support storing local payloads to a trailer if it is being built for reference
		// access (i.e. for the editor domain) and if this is detected we should force the legacy serialization
		// path for this payload.
		const bool bForceLegacyPath = bKeepFileDataByReference && bCanKeepFileDataByReference == false && !IsDataVirtualized(UpdatedFlags);

		if (ShouldUseLegacySerialization(LinkerSave) == true || bForceLegacyPath == true)
		{
			EnumRemoveFlags(UpdatedFlags, EFlags::StoredInPackageTrailer);
		}
		else
		{
			EnumAddFlags(UpdatedFlags, EFlags::StoredInPackageTrailer);
		}

		// If we are cooking add the cooked flag to the serialized stream
		// Editor bulk data might be cooked when using editor optional data at runtime
		if (Ar.IsCooking())
		{
			EnumAddFlags(UpdatedFlags, EFlags::IsCooked);
		}
		
		// Transient flags are never serialized
		EnumRemoveFlags(UpdatedFlags, TransientFlags);

		return UpdatedFlags;
	}
	else
	{
		return Flags;
	}
}

bool FEditorBulkData::TryPayloadValidationForSaving(const FCompressedBuffer& PayloadForSaving, FLinkerSave* LinkerSave) const
{
	if (!IsDataValid(*this, PayloadForSaving) || (GetPayloadSize() > 0 && PayloadForSaving.IsNull()))
	{
		const FString ErrorMessage = GetCorruptedPayloadErrorMsgForSave(LinkerSave).ToString();

		ensureMsgf(false, TEXT("%s"), *ErrorMessage);

		if (LinkerSave != nullptr && LinkerSave->GetOutputDevice() != nullptr)
		{
			LinkerSave->GetOutputDevice()->Logf(ELogVerbosity::Error, TEXT("%s"), *ErrorMessage);
		}
		else
		{
			UE_LOGF(LogSerialization, Error, "%ls", *ErrorMessage);
		}

		return false;
	}
	else
	{
		return true;
	}
}

FText FEditorBulkData::GetCorruptedPayloadErrorMsgForSave(FLinkerSave* Linker) const
{
	const FText GuidID = FText::FromString(GetIdentifier().ToString());

	if (Linker != nullptr)
	{
		// We know the package we are saving to.
		const FText PackageName = FText::FromString(Linker->LinkerRoot->GetName());
		
		return FText::Format(	NSLOCTEXT("Core", "Serialization_InvalidPayloadToPkg", "Attempting to save bulkdata {0} with an invalid payload to package '{1}'. The package probably needs to be reverted/recreated to fix this."), 						
								GuidID, PackageName);
	}
	else if(!PackagePath.IsEmpty())
	{
		// We don't know where we are saving to, but we do know the package where the payload came from.
		const FText PackageName = FText::FromString(PackagePath.GetDebugName());

		return FText::Format(	NSLOCTEXT("Core", "Serialization_InvalidPayloadFromPkg", "Attempting to save bulkdata {0} with an invalid payload from package '{1}'. The package probably needs to be reverted/recreated to fix this."),
								GuidID, PackageName);
	}
	else
	{
		// We don't know where the payload came from or where it is being saved to.
		return FText::Format(	NSLOCTEXT("Core", "Serialization_InvalidPayloadPath", "Attempting to save bulkdata {0} with an invalid payload, source unknown"),
								GuidID);
	}
}

void FEditorBulkData::ValidatePackageTrailerBuilder(const FLinkerSave* LinkerSave, const FIoHash& Id, EFlags PayloadFlags)
{
	if (IsStoredInPackageTrailer(PayloadFlags) && !Id.IsZero())
	{
		check(LinkerSave != nullptr);
		check(LinkerSave->PackageTrailerBuilder.IsValid());
		checkf(!(IsDataVirtualized(PayloadFlags) && IsReferencingByPackagePath(PayloadFlags)), TEXT("Payload cannot be both virtualized and a reference"));

		if (IsReferencingByPackagePath(PayloadFlags))
		{
			check(LinkerSave->PackageTrailerBuilder->IsReferencedPayloadEntry(Id));
		}
		else if (IsDataVirtualized(PayloadFlags))
		{
			check(LinkerSave->PackageTrailerBuilder->IsVirtualizedPayloadEntry(Id));
		}
		else
		{
			check(LinkerSave->PackageTrailerBuilder->IsLocalPayloadEntry(Id));
		}
	}
}

bool FEditorBulkData::ShouldUseLegacySerialization(const FLinkerSave* LinkerSave) const
{
	if (LinkerSave == nullptr)
	{
		return true;
	}

	if (ShouldSaveToPackageSidecar())
	{
		return true;
	}

	return !LinkerSave->PackageTrailerBuilder.IsValid();
}

} // namespace UE::Serialization

#undef UE_CORRUPTED_DATA_SEVERITY
#undef UE_CORRUPTED_PAYLOAD_IS_FATAL
#undef UE_ALLOW_LINKERLOADER_ATTACHMENT

//#endif //WITH_EDITORONLY_DATA
