// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/FindReferencersArchive.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Class.h"

/*----------------------------------------------------------------------------
	FFindReferencersArchive.
----------------------------------------------------------------------------*/
/**
 * Constructor
 *
 * @param	PotentialReferencer		the object to serialize which may contain references to our target objects
 * @param	InTargetObjects			array of objects to search for references to
 * @param	bFindAlsoWeakReferences should we also look into weak references?
 */
FFindReferencersArchive::FFindReferencersArchive(UObject* InPotentialReferencer, TArrayView<UObject*> InTargetObjects, bool bFindAlsoWeakReferences)
{
	// use the optimized RefLink to skip over properties which don't contain object references
	ArIsObjectReferenceCollector = true;

	// look also for weak references if user asked for it (we won't modify them, but there is no archive option for that)
	ArIsModifyingWeakAndStrongReferences = bFindAlsoWeakReferences;

	// ALL objects reference their outers...it's just log spam here
	ArIgnoreOuterRef = true;

	// initialize the map
	TargetObjects.Reserve(InTargetObjects.Num());
	for (int32 ObjIndex = 0; ObjIndex < InTargetObjects.Num(); ObjIndex++)
	{
		UObject* TargetObject = InTargetObjects[ObjIndex];
		if (TargetObject != nullptr)
		{
			TargetObjects.AddObject(TargetObject);
		}
	}
	TargetObjects.Freeze();

	PotentialReferencer = nullptr;
	ResetPotentialReferencer(InPotentialReferencer);
}

void FFindReferencersArchive::ResetPotentialReferencer(UObject* InPotentialReferencer)
{
	if (PotentialReferencer)
	{
		// Reset all reference counts
		TargetObjects.ResetRefCounts();
	}

	PotentialReferencer = InPotentialReferencer;

	if (PotentialReferencer)
	{
		// now start the search
		PotentialReferencer->Serialize(*this);

		// search for references coming from AddReferencedObjects
		class FArchiveProxyCollector : public FReferenceCollector
		{
			/** Archive we are a proxy for */
			FArchive& Archive;
		public:
			FArchiveProxyCollector(FArchive& InArchive)
				: Archive(InArchive)
			{
			}
			virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
			{
				Archive << Object;
			}
			virtual void HandleObjectReferences(UObject** InObjects, const int32 ObjectNum, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
			{
				for (int32 ObjectIndex = 0; ObjectIndex < ObjectNum; ++ObjectIndex)
				{
					UObject*& Object = InObjects[ObjectIndex];
					Archive << Object;
				}
			}
			virtual bool IsIgnoringArchetypeRef() const override
			{
				return false;
			}
			virtual bool IsIgnoringTransient() const override
			{
				return false;
			}
		} ArchiveProxyCollector(*this);
		PotentialReferencer->GetClass()->CallAddReferencedObjects(PotentialReferencer, ArchiveProxyCollector);
	}
}

/**
 * Retrieves the number of references from PotentialReferencer to the object specified.
 *
 * @param	TargetObject	the object to might be referenced
 * @param	out_ReferencingProperties
 *							receives the list of properties which were holding references to TargetObject
 *
 * @return	the number of references to TargetObject which were encountered when PotentialReferencer
 *			was serialized.
 */
int32 FFindReferencersArchive::GetReferenceCount( UObject* TargetObject, TArray<FProperty*>* out_ReferencingProperties/*=nullptr*/ ) const
{
	int32 Result = 0;
	if ( TargetObject != nullptr && PotentialReferencer != TargetObject )
	{
		const int32* pCount = TargetObjects.TryGetRefCountPtr(TargetObject);
		if ( pCount != nullptr && ( *pCount ) > 0 )
		{
			Result = *pCount;
			if ( out_ReferencingProperties != nullptr )
			{
				TArray<FProperty*> PropertiesReferencingObj;
				ReferenceMap.MultiFind(TargetObject, PropertiesReferencingObj);

				out_ReferencingProperties->Empty(PropertiesReferencingObj.Num());
				for ( int32 PropIndex = PropertiesReferencingObj.Num() - 1; PropIndex >= 0; PropIndex-- )
				{
					out_ReferencingProperties->Add(PropertiesReferencingObj[PropIndex]);
				}
			}
		}
	}

	return Result;
}

/**
 * Retrieves the number of references from PotentialReferencer list of TargetObjects
 *
 * @param	out_ReferenceCounts		receives the number of references to each of the TargetObjects
 *
 * @return	the number of objects which were referenced by PotentialReferencer.
 */
int32 FFindReferencersArchive::GetReferenceCounts( TMap<class UObject*, int32>& out_ReferenceCounts ) const
{
	out_ReferenceCounts.Empty();
	return AppendReferenceCounts(out_ReferenceCounts);
}

int32 FFindReferencersArchive::AppendReferenceCounts(TMap<class UObject*, int32>& out_ReferenceCounts) const
{
	for (int32 Index = 0; Index < TargetObjects.RefCountNum(); ++Index)
	{
		if (TargetObjects.GetRefCount(Index) > 0 && TargetObjects.GetObject(Index) != PotentialReferencer)
		{
			out_ReferenceCounts.Add(TargetObjects.GetObject(Index), TargetObjects.GetRefCount(Index));
		}
	}

	return out_ReferenceCounts.Num();
}

/**
 * Retrieves the number of references from PotentialReferencer list of TargetObjects
 *
 * @param	out_ReferenceCounts			receives the number of references to each of the TargetObjects
 * @param	out_ReferencingProperties	receives the map of properties holding references to each referenced object.
 *
 * @return	the number of objects which were referenced by PotentialReferencer.
 */
int32 FFindReferencersArchive::GetReferenceCounts( TMap<class UObject*, int32>& out_ReferenceCounts, TMultiMap<class UObject*, class FProperty*>& out_ReferencingProperties ) const
{
	GetReferenceCounts(out_ReferenceCounts);
	if ( out_ReferenceCounts.Num() > 0 )
	{
		out_ReferencingProperties.Empty();
		for ( TMap<UObject*,int32>::TIterator It(out_ReferenceCounts); It; ++It )
		{
			UObject* Object = It.Key();

			TArray<FProperty*> PropertiesReferencingObj;
			ReferenceMap.MultiFind(Object, PropertiesReferencingObj);

			for ( int32 PropIndex = PropertiesReferencingObj.Num() - 1; PropIndex >= 0; PropIndex-- )
			{
				out_ReferencingProperties.Add(Object, PropertiesReferencingObj[PropIndex]);
			}
		}
	}

	return out_ReferenceCounts.Num();
}

/**
 * Serializer - if Obj is one of the objects we're looking for, increments the reference count for that object
 */
FArchive& FFindReferencersArchive::operator<<( UObject*& Obj )
{
	if ( Obj != nullptr && Obj != PotentialReferencer )
	{
		int32* pReferenceCount = TargetObjects.GetRefCountPtr(Obj);
		if ( pReferenceCount != nullptr )
		{
			// if this object was serialized via a FProperty, add it to the list
			if (GetSerializedProperty() != nullptr)
			{
				ReferenceMap.AddUnique(Obj, GetSerializedProperty());
			}

			// now increment the reference count for this target object
			(*pReferenceCount)++;
		}
	}

	return *this;
}
