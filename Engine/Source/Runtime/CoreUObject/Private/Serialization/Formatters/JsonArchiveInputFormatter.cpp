// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/Formatters/JsonArchiveInputFormatter.h"
#include "Serialization/JsonReader.h"
#include "Containers/StringView.h"
#include "Dom/JsonObject.h"
#include "Misc/Base64.h"
#include "Misc/CString.h"
#include "Misc/SecureHash.h"
#include "Serialization/JsonSerializer.h"
#include "Templates/TypeCompatibleBytes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectPtr.h"
#include <limits>

#if WITH_TEXT_ARCHIVE_SUPPORT

FJsonArchiveInputFormatter::FJsonArchiveInputFormatter(FArchive& InInner, TFunction<UObject* (const FPackageIndex)> InResolveObject) 
	: Inner(InInner)
	, ResolveObject(InResolveObject)
{
	Inner.SetIsTextFormat(true);
	Inner.ArAllowLazyLoading = false;

	TSharedPtr< FJsonObject > RootObject;
	TSharedRef< TJsonReader<UTF8CHAR> > Reader = TJsonReaderFactory<UTF8CHAR>::Create(&InInner);
	ensure(FJsonSerializer::Deserialize(Reader, RootObject, FJsonSerializer::EFlags::StoreNumbersAsStrings));
	ValueStack.Add(MakeShared<FJsonValueObject>(RootObject));

	ValueStack.Reserve(64);
	ArrayValuesRemainingStack.Reserve(64);
}

FJsonArchiveInputFormatter::FJsonArchiveInputFormatter(FArchive& InInner, TFunction<UObject* (const FPackageIndex)>&& InResolveObject, TSharedPtr<FJsonValue>&& RootObject) 
	: Inner(InInner)
	, ResolveObject(MoveTemp(InResolveObject))
{
	ValueStack.Reserve(64);
	ValueStack.Push(MoveTemp(RootObject));
}

FJsonArchiveInputFormatter::~FJsonArchiveInputFormatter()
{
}

FArchive& FJsonArchiveInputFormatter::GetUnderlyingArchive()
{
	return Inner;
}

FStructuredArchiveFormatter* FJsonArchiveInputFormatter::CreateSubtreeReader()
{
	FJsonArchiveInputFormatter* Cloned = new FJsonArchiveInputFormatter(this->Inner, CopyTemp(this->ResolveObject), CopyTemp(ValueStack.Top()));
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Cloned->SetResolveCell(CopyTemp(this->ResolveCell));
#endif
	return Cloned;
}

bool FJsonArchiveInputFormatter::HasDocumentTree() const
{
	return true;
}

void FJsonArchiveInputFormatter::EnterRecord()
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();
	ObjectStack.Emplace(Value->AsObject(), ValueStack.Num());
}

void FJsonArchiveInputFormatter::LeaveRecord()
{
	check(ValueStack.Num() == ObjectStack.Top().ValueCountOnCreation);
	ObjectStack.Pop();
}

void FJsonArchiveInputFormatter::EnterField(FArchiveFieldName Name)
{
	TSharedPtr<FJsonObject>& NewObject = ObjectStack.Top().JsonObject;
	TSharedPtr<FJsonValue> Field = NewObject->TryGetField(EscapeFieldName(Name.Name));
	check(Field.IsValid());
	ValueStack.Add(Field);
}

void FJsonArchiveInputFormatter::LeaveField()
{
	ValueStack.Pop();
}

bool FJsonArchiveInputFormatter::TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving)
{
	TSharedPtr<FJsonObject>& NewObject = ObjectStack.Top().JsonObject;
	TSharedPtr<FJsonValue> Field = NewObject->TryGetField(EscapeFieldName(Name.Name));
	if (Field.IsValid())
	{
		ValueStack.Add(Field);
		return true;
	}
	return false;
}

void FJsonArchiveInputFormatter::EnterArray(int32& NumElements)
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();

	const TArray<TSharedPtr<FJsonValue>>& ArrayValue = Value->AsArray();
	for(int Idx = ArrayValue.Num() - 1; Idx >= 0; Idx--)
	{
		ValueStack.Add(ArrayValue[Idx]);
	}

	NumElements = ArrayValue.Num();
	ArrayValuesRemainingStack.Add(NumElements);
}

void FJsonArchiveInputFormatter::LeaveArray()
{
	check(ArrayValuesRemainingStack.Num() > 0);
	int32 Remaining = ArrayValuesRemainingStack.Top();
	ArrayValuesRemainingStack.Pop();
	check(Remaining >= 0);
	check(Remaining <= ValueStack.Num());
	while (Remaining-- > 0)
	{
		ValueStack.Pop();
	}
}

void FJsonArchiveInputFormatter::EnterArrayElement()
{
	check(ArrayValuesRemainingStack.Num() > 0);
	check(ArrayValuesRemainingStack.Top() > 0);
}

void FJsonArchiveInputFormatter::LeaveArrayElement()
{
	ValueStack.Pop();
	ArrayValuesRemainingStack.Top()--;
}

void FJsonArchiveInputFormatter::EnterStream()
{
	int32 NumElements = 0;
	EnterArray(NumElements);
}

void FJsonArchiveInputFormatter::LeaveStream()
{
	LeaveArray();
}

void FJsonArchiveInputFormatter::EnterStreamElement()
{
}

void FJsonArchiveInputFormatter::LeaveStreamElement()
{
	LeaveArrayElement();
}

void FJsonArchiveInputFormatter::EnterMap(int32& NumElements)
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();
	TSharedPtr<FJsonObject> Object = Value->AsObject();
	MapIteratorStack.Add(Object->Values.CreateConstIterator());
	NumElements = Object->Values.Num();
}

void FJsonArchiveInputFormatter::LeaveMap()
{
	MapIteratorStack.Pop();
}

void FJsonArchiveInputFormatter::EnterMapElement(FString& OutName)
{
	OutName = UnescapeFieldName(*MapIteratorStack.Top()->Key);
	ValueStack.Add(MapIteratorStack.Top()->Value);
}

void FJsonArchiveInputFormatter::LeaveMapElement()
{
	ValueStack.Pop();
	++MapIteratorStack.Top();
}

void FJsonArchiveInputFormatter::EnterAttributedValue()
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();
	const TSharedPtr<FJsonObject>* ObjectPtr = nullptr;
	if (Value->TryGetObject(ObjectPtr))
	{
		FJsonObject& ObjectRef = **ObjectPtr;

		TSharedPtr<FJsonValue> Field = ObjectRef.TryGetField(EscapeFieldName(TEXT("_Value")));
		if (Field.IsValid())
		{
			ObjectStack.Emplace(*ObjectPtr, ValueStack.Num());
			return;
		}
	}

	ObjectStack.Emplace(nullptr, ValueStack.Num());
}

void FJsonArchiveInputFormatter::EnterAttribute(FArchiveFieldName AttributeName)
{
	TSharedPtr<FJsonObject>& NewObject = ObjectStack.Top().JsonObject;
	TSharedPtr<FJsonValue> Field = NewObject->TryGetField(EscapeFieldName(*FString::Printf(TEXT("_%s"), AttributeName.Name)));
	check(Field.IsValid());
	ValueStack.Add(Field);
}

void FJsonArchiveInputFormatter::EnterAttributedValueValue()
{
	if (TSharedPtr<FJsonObject>& Object = ObjectStack.Top().JsonObject)
	{
		TSharedPtr<FJsonValue> Field = Object->TryGetField(EscapeFieldName(TEXT("_Value")));
		checkSlow(Field);
		ValueStack.Add(Field);
	}
	else
	{
		ValueStack.Add(CopyTemp(ValueStack.Top()));
	}
}

bool FJsonArchiveInputFormatter::TryEnterAttributedValueValue()
{
	TSharedPtr<FJsonValue>& Value = ValueStack.Top();
	const TSharedPtr<FJsonObject>* ObjectPtr = nullptr;
	if (Value->TryGetObject(ObjectPtr))
	{
		FJsonObject& ObjectRef = **ObjectPtr;

		TSharedPtr<FJsonValue> Field = ObjectRef.TryGetField(EscapeFieldName(TEXT("_Value")));
		if (Field.IsValid())
		{
			ObjectStack.Emplace(*ObjectPtr, ValueStack.Num());
			ValueStack.Add(Field);
			return true;
		}
	}

	return false;
}

void FJsonArchiveInputFormatter::LeaveAttribute()
{
	ValueStack.Pop();
}

void FJsonArchiveInputFormatter::LeaveAttributedValue()
{
	check(ValueStack.Num() == ObjectStack.Top().ValueCountOnCreation);
	ObjectStack.Pop();
}

bool FJsonArchiveInputFormatter::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving)
{
	if (TSharedPtr<FJsonObject>& Object = ObjectStack.Top().JsonObject)
	{
		if (TSharedPtr<FJsonValue> Field = Object->TryGetField(EscapeFieldName(TEXT("_Value"))))
		{
			TSharedPtr<FJsonValue> Attribute = Object->TryGetField(EscapeFieldName(*FString::Printf(TEXT("_%s"), AttributeName.Name)));
			if (Attribute.IsValid())
			{
				ValueStack.Add(Attribute);
				return true;
			}
		}
	}
	return false;
}

void FJsonArchiveInputFormatter::Serialize(uint8& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatter::Serialize(uint16& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatter::Serialize(uint32& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatter::Serialize(uint64& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatter::Serialize(int8& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatter::Serialize(int16& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatter::Serialize(int32& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

void FJsonArchiveInputFormatter::Serialize(int64& Value)
{
	verify(ValueStack.Top()->TryGetNumber(Value));
}

namespace
{
template<typename FloatType, typename IntType>
bool ParseFPSpecial(const FString& String, FloatType& OutValue, IntType SignMask, IntType ExponentMask, IntType SignificandMask)
{
	FStringView ResidualString{String};
	
	// Parse the Number: prefix
	FStringView SpecialStringPrefix = TEXT("Number:");
	if (!ResidualString.StartsWith(SpecialStringPrefix))
	{
		return false;
	}
	ResidualString.MidInline(SpecialStringPrefix.Len());

	// Parse the sign.
	if (!ResidualString.Len())
	{
		return false;
	}
	if (ResidualString[0] != TEXT('+') && ResidualString[0] != TEXT('-'))
	{
		return false;
	}
	const bool bIsNegative = ResidualString[0] == TEXT('-');
	ResidualString.MidInline(1);

	if(ResidualString == TEXT("inf"))
	{
		OutValue = bIsNegative
			? -std::numeric_limits<FloatType>::infinity()
			: +std::numeric_limits<FloatType>::infinity();
		return true;
	}
	else if(ResidualString.StartsWith(TEXT("nan:0x")))
	{
		ResidualString.MidInline(6);
		
		// Assume that the string view is null terminated for the sake of FCString::Strtoui64.
		check(*ResidualString.end() == 0);
		
		// Parse the significand.
		TCHAR* SignificandEnd = nullptr;
		uint64 Significand64 = FCString::Strtoui64(ResidualString.begin(), &SignificandEnd, 16);
		if (SignificandEnd != ResidualString.end()
			|| Significand64 == 0
			|| Significand64 > static_cast<uint64>(SignificandMask))
		{
			return false;
		}
		const IntType Significand = static_cast<IntType>(Significand64);

		// Construct the output NaN from its components.
		IntType ValueAsInt = ExponentMask | Significand;
		if (bIsNegative)
		{
			ValueAsInt |= SignMask;
		}
		OutValue = BitCast<FloatType>(ValueAsInt);
		return true;
	}
	else
	{
		return false;
	}
}
}

void FJsonArchiveInputFormatter::Serialize(float& Value)
{
	if(!ValueStack.Top()->TryGetNumber(Value))
	{
		FString SpecialString;
		verify(ValueStack.Top()->TryGetString(SpecialString));

		if(!ParseFPSpecial<float, uint32>(SpecialString, Value, 0x80000000, 0x7f800000, 0x007fffff))
		{
			Value = 0.0f;
			checkf(false, TEXT("Expected float special but got string \"%s\""), *SpecialString);
		}
	}
}

void FJsonArchiveInputFormatter::Serialize(double& Value)
{
	if(!ValueStack.Top()->TryGetNumber(Value))
	{
		FString SpecialString;
		verify(ValueStack.Top()->TryGetString(SpecialString));
		
		if(!ParseFPSpecial<double, uint64>(SpecialString, Value, 0x8000000000000000, 0x7ff0000000000000, 0x000fffffffffffff))
		{
			Value = 0.0f;
			checkf(false, TEXT("Expected double special but got string \"%s\""), *SpecialString);
		}
	}
}

void FJsonArchiveInputFormatter::Serialize(bool& Value)
{
	Value = ValueStack.Top()->AsBool();
}

void FJsonArchiveInputFormatter::Serialize(UTF32CHAR& Value)
{
	verify(ValueStack.Top()->TryGetNumber(*(uint32*)&Value));
}

void FJsonArchiveInputFormatter::Serialize(FString& Value)
{
	// If the string we serialized was empty, this value will be a null object rather than a string, so we have to
#if DO_CHECK
	bool bSuccess =
#endif
		ValueStack.Top()->TryGetString(Value);
#if DO_CHECK
	check(bSuccess || ValueStack.Top()->IsNull());
#endif
	Value.RemoveFromStart(TEXT("String:"));
}

void FJsonArchiveInputFormatter::Serialize(FName& Value)
{
	FString StringValue = ValueStack.Top()->AsString();
	Value = FName(*StringValue);
}

void FJsonArchiveInputFormatter::Serialize(UObject*& Value)
{
	FString StringValue;
	if (ValueStack.Top()->TryGetString(StringValue))
	{
		FPackageIndex ObjectIndex;
		LexFromString(ObjectIndex, *StringValue);
		Value = ResolveObject(ObjectIndex);
	}
	else
	{
		Value = nullptr;
	}
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FJsonArchiveInputFormatter::Serialize(Verse::VCell*& Value)
{
	FString StringValue;
	if (ValueStack.Top()->TryGetString(StringValue))
	{
		FPackageIndex CellIndex;
		LexFromString(CellIndex, *StringValue);
		Value = ResolveCell(CellIndex);
	}
	else
	{
		Value = nullptr;
	}
}
#endif

void FJsonArchiveInputFormatter::Serialize(FText& Value)
{
	FStructuredArchive ChildArchive(*this);
	FText::SerializeText(ChildArchive.Open(), Value);
	ChildArchive.Close();
}

void FJsonArchiveInputFormatter::Serialize(FWeakObjectPtr& Value)
{
	UObject* Object = nullptr;
	Serialize(Object);
	Value = Object;
}

void FJsonArchiveInputFormatter::Serialize(FSoftObjectPtr& Value)
{
	FSoftObjectPath Path;
	Serialize(Path);
	Value = Path;
}

void FJsonArchiveInputFormatter::Serialize(FSoftObjectPath& Value)
{
	FString StringValue;
	const auto& Prefix = TEXT("Object:");
	if (ValueStack.Top()->TryGetString(StringValue) && StringValue.StartsWith(Prefix))
	{
		Value.SetPath(*StringValue + UE_ARRAY_COUNT(Prefix) - 1);
	}
	else
	{
		Value.Reset();
	}
}

void FJsonArchiveInputFormatter::Serialize(FLazyObjectPtr& Value)
{
	FString StringValue;
	const auto& Prefix = TEXT("Lazy:");
	if (ValueStack.Top()->TryGetString(StringValue) && StringValue.StartsWith(Prefix))
	{
		FUniqueObjectGuid Guid;
		Guid.FromString(*StringValue + UE_ARRAY_COUNT(Prefix) - 1);
		Value = Guid;
	}
	else
	{
		Value.Reset();
	}
}

void FJsonArchiveInputFormatter::Serialize(FObjectPtr& Value)
{
	UObject* Object;
	Serialize(Object);
	Value = Object;
}

void FJsonArchiveInputFormatter::Serialize(TArray<uint8>& Data)
{
	const FJsonValue& Value = *ValueStack.Top();
	if(Value.Type == EJson::String)
	{
		// Single line string
		FString RawData = ValueStack.Top()->AsString();
		ensure(RawData.RemoveFromStart(TEXT("Base64:")));
		FBase64::Decode(RawData, Data);
	}
	else if(Value.Type == EJson::Object)
	{
		// Multi-line string
		FJsonObject& Object = *Value.AsObject();

		// Read the digest
		TSharedPtr<FJsonValue> DigestField = Object.TryGetField(TEXT("Digest"));
		checkf(DigestField.IsValid() && DigestField->Type == EJson::String, TEXT("Missing or invalid 'Digest' field for raw data"));
		FString Digest = DigestField->AsString();

		// Read the base64 array
		TSharedPtr<FJsonValue> Base64Field = Object.TryGetField(TEXT("Base64"));
		checkf(Base64Field.IsValid() && Base64Field->Type == EJson::Array, TEXT("Missing or invalid 'Base64' field for raw data"));
		const TArray<TSharedPtr<FJsonValue>>& Base64Array = Base64Field->AsArray();

		// Parse the digest
		uint8 ExpectedDigest[FSHA1::DigestSize];
		verify(FString::ToHexBlob(DigestField->AsString(), ExpectedDigest, sizeof(ExpectedDigest)));

		// Get the size of the encoded data
		uint32 DecodedSize = 0;
		for(const TSharedPtr<FJsonValue>& Base64Line : Base64Array)
		{
			DecodedSize += FBase64::GetDecodedDataSize(Base64Line->AsString());
		}

		// Read the encoded data
		Data.SetNum(DecodedSize);

		// Read each line
		uint32 DecodedPos = 0;
		for(const TSharedPtr<FJsonValue>& Base64Line : Base64Array)
		{
			FString Base64String = Base64Line->AsString();
			verify(FBase64::Decode(*Base64String, Base64String.Len(), Data.GetData() + DecodedPos));
			DecodedPos += FBase64::GetDecodedDataSize(Base64String);
		}

		// Make sure the digest matches
		uint8 ActualDigest[FSHA1::DigestSize];
		FSHA1::HashBuffer(Data.GetData(), Data.Num(), ActualDigest);
		checkf(FMemory::Memcmp(ExpectedDigest, ActualDigest, FSHA1::DigestSize) == 0, TEXT("Digest does not match for raw data. Check that this file was merged correctly."));
	}
	else
	{
		checkf(false, TEXT("Invalid value type for raw data"));
	}
}

void FJsonArchiveInputFormatter::Serialize(void* Data, uint64 DataSize)
{
	TArray<uint8> Buffer;
	Serialize(Buffer);
	check(Buffer.Num() == DataSize);
	if (DataSize)
	{
		memcpy(Data, Buffer.GetData(), DataSize);
	}
}

FString FJsonArchiveInputFormatter::EscapeFieldName(const TCHAR* Name)
{
	if(FCString::Stricmp(Name, TEXT("Base64")) == 0 || FCString::Stricmp(Name, TEXT("Digest")) == 0 || Name[0] == '_')
	{
		return FString::Printf(TEXT("_%s"), Name);
	}
	else
	{
		return Name;
	}
}

FString FJsonArchiveInputFormatter::UnescapeFieldName(const TCHAR* Name)
{
	if(Name[0] == '_')
	{
		return Name + 1;
	}
	else
	{
		return Name;
	}
}

EArchiveValueType FJsonArchiveInputFormatter::GetValueType(const FJsonValue& Value)
{
	switch (Value.Type)
	{
	case EJson::String:
		if (Value.AsString().StartsWith(TEXT("Object:")))
		{
			return EArchiveValueType::Object;
		}
		else if (Value.AsString().StartsWith(TEXT("Base64:")))
		{
			return EArchiveValueType::RawData;
		}
		else
		{
			return EArchiveValueType::String;
		}
	case EJson::Number:
		{
			double Number = Value.AsNumber();
			int64 NumberInt64 = (int64)Number;
			if((double)NumberInt64 == Number)
			{
				if((int16)NumberInt64 == NumberInt64)
				{
					if((int8)NumberInt64 == NumberInt64)
					{
						return EArchiveValueType::Int8;
					}
					else
					{
						return EArchiveValueType::Int16;
					}
				}
				else
				{
					if((int32)NumberInt64 == NumberInt64)
					{
						return EArchiveValueType::Int32;
					}
					else
					{
						return EArchiveValueType::Int64;
					}
				}
			}
			else
			{
				if((double)(float)Number == Number)
				{
					return EArchiveValueType::Float;
				}
				else
				{
					return EArchiveValueType::Double;
				}
			}
		}
	case EJson::Boolean:
		return EArchiveValueType::Bool;
	case EJson::Array:
		return EArchiveValueType::Array;
	case EJson::Object:
		if(Value.AsObject()->HasField(TEXT("Base64")))
		{
			return EArchiveValueType::RawData;
		}
		else
		{
			return EArchiveValueType::Record;
		}
	case EJson::Null:
		return EArchiveValueType::Object;
	default:
		checkf(false, TEXT("Unhandled value type in JSON archive (%d)"), (int)Value.Type);
		return EArchiveValueType::None;
	}
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FJsonArchiveInputFormatter::SetResolveCell(TFunction<Verse::VCell* (const FPackageIndex)>&& InResolveCell)
{
	ResolveCell = MoveTemp(InResolveCell);
}
#endif

#endif
