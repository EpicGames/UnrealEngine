// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/MemoryWriter.h"
#include "Misc/Base64.h"
#include "Misc/SecureHash.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectPtr.h"
#include <cinttypes>

#if WITH_TEXT_ARCHIVE_SUPPORT

FJsonArchiveOutputFormatter::FJsonArchiveOutputFormatter(FArchive& InInner) 
	: Inner(InInner)
	, Newline(LINE_TERMINATOR_ANSI, UE_ARRAY_COUNT(LINE_TERMINATOR_ANSI) - 1)
{
	Inner.SetIsTextFormat(true);
}

FJsonArchiveOutputFormatter::~FJsonArchiveOutputFormatter()
{
}

FArchive& FJsonArchiveOutputFormatter::GetUnderlyingArchive()
{
	return Inner;
}

bool FJsonArchiveOutputFormatter::HasDocumentTree() const
{
	return true;
}

void FJsonArchiveOutputFormatter::EnterRecord()
{
	WriteOptionalComma();
	WriteOptionalNewline();
	Write("{");
	Newline.Add('\t');
	bNeedsNewline = true;
	TextStartPosStack.Push(Inner.Tell());
}

void FJsonArchiveOutputFormatter::LeaveRecord()
{
	Newline.Pop(EAllowShrinking::No);
	if (TextStartPosStack.Pop() == Inner.Tell())
	{
		bNeedsNewline = false;
	}
	WriteOptionalNewline();
	Write("}");
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatter::EnterField(FArchiveFieldName Name)
{
	WriteOptionalComma();
	WriteOptionalNewline();
	WriteFieldName(Name.Name);
}

void FJsonArchiveOutputFormatter::LeaveField()
{
	bNeedsComma = true;
	bNeedsNewline = true;
}

bool FJsonArchiveOutputFormatter::TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving)
{
	if (bEnterWhenSaving)
	{
		EnterField(Name);
	}
	return bEnterWhenSaving;
}

void FJsonArchiveOutputFormatter::EnterArray(int32& NumElements)
{
	EnterStream();
}

void FJsonArchiveOutputFormatter::LeaveArray()
{
	LeaveStream();
}

void FJsonArchiveOutputFormatter::EnterArrayElement()
{
	EnterStreamElement();
}

void FJsonArchiveOutputFormatter::LeaveArrayElement()
{
	LeaveStreamElement();
}

void FJsonArchiveOutputFormatter::EnterStream()
{
	WriteOptionalComma();
	WriteOptionalNewline();
	Write("[");
	Newline.Add('\t');
	bNeedsNewline = true;
	TextStartPosStack.Push(Inner.Tell());
}

void FJsonArchiveOutputFormatter::LeaveStream()
{
	Newline.Pop(EAllowShrinking::No);
	if (TextStartPosStack.Pop() == Inner.Tell())
	{
		bNeedsNewline = false;
	}
	WriteOptionalNewline();
	Write("]");
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatter::EnterStreamElement()
{
	WriteOptionalComma();
	WriteOptionalNewline();
}

void FJsonArchiveOutputFormatter::LeaveStreamElement()
{
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatter::EnterMap(int32& NumElements)
{
	EnterRecord();
}

void FJsonArchiveOutputFormatter::LeaveMap()
{
	LeaveRecord();
}

void FJsonArchiveOutputFormatter::EnterMapElement(FString& Name)
{
	EnterField(FArchiveFieldName(*Name));
}

void FJsonArchiveOutputFormatter::LeaveMapElement()
{
	LeaveField();
}

void FJsonArchiveOutputFormatter::EnterAttributedValue()
{
	NumAttributesStack.Push(0);
}

void FJsonArchiveOutputFormatter::EnterAttribute(FArchiveFieldName AttributeName)
{
	WriteOptionalComma();
	WriteOptionalNewline();
	WriteOptionalAttributedBlockOpening();
	WriteOptionalComma();
	WriteOptionalNewline();
	checkf(FCString::Strcmp(AttributeName.Name, TEXT("Value")) != 0, TEXT("Attributes called 'Value' are reserved by the implementation"));
	WriteFieldName(*FString::Printf(TEXT("_%s"), AttributeName.Name));
	++NumAttributesStack.Top();
}

void FJsonArchiveOutputFormatter::LeaveAttribute()
{
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatter::LeaveAttributedValue()
{
	WriteOptionalAttributedBlockClosing();
	NumAttributesStack.Pop();
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonArchiveOutputFormatter::EnterAttributedValueValue()
{
	WriteOptionalComma();
	WriteOptionalNewline();
	WriteOptionalAttributedBlockValue();
}

bool FJsonArchiveOutputFormatter::TryEnterAttributedValueValue()
{
	return false;
}

bool FJsonArchiveOutputFormatter::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving)
{
	if (bEnterWhenSaving)
	{
		EnterAttribute(AttributeName);
	}
	return bEnterWhenSaving;
}

void FJsonArchiveOutputFormatter::Serialize(uint8& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(uint16& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(uint32& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(uint64& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(int8& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(int16& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(int32& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(int64& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(float& Value)
{
	if(FPlatformMath::IsFinite(Value))
	{
		FString String = FString::Printf(TEXT("%.17g"), Value);
#if DO_GUARD_SLOW
		float RoundTripped;
		LexFromString(RoundTripped, *String);
		check(RoundTripped == Value);
#endif
		WriteValue(String);
	}
	else if(FPlatformMath::IsNaN(Value))
	{
		const uint32 ValueAsInt = BitCast<uint32>(Value);
		const bool bIsNegative = !!(ValueAsInt & 0x80000000);
		const uint32 Significand = ValueAsInt & 0x007fffff;
		WriteValue(FString::Printf(TEXT("\"Number:%snan:0x%" PRIx32 "\""), bIsNegative ? TEXT("-") : TEXT("+"), Significand));
	}
	else
	{
		WriteValue(Value < 0.0f ? TEXT("\"Number:-inf\"") : TEXT("\"Number:+inf\""));
	}
}

void FJsonArchiveOutputFormatter::Serialize(double& Value)
{
	if(FPlatformMath::IsFinite(Value))
	{
		FString String = FString::Printf(TEXT("%.17g"), Value);
#if DO_GUARD_SLOW
		double RoundTripped;
		LexFromString(RoundTripped, *String);
		check(RoundTripped == Value);
#endif
		WriteValue(String);
	}
	else if(FPlatformMath::IsNaN(Value))
	{
		const uint64 ValueAsInt = BitCast<uint64>(Value);
		const bool bIsNegative = !!(ValueAsInt & 0x8000000000000000);
		const uint64 Significand = ValueAsInt & 0x000fffffffffffff;
		WriteValue(FString::Printf(TEXT("\"Number:%snan:0x%" PRIx64 "\""), bIsNegative ? TEXT("-") : TEXT("+"), Significand));
	}
	else
	{
		WriteValue(Value < 0.0 ? TEXT("\"Number:-inf\"") : TEXT("\"Number:+inf\""));
	}
}

void FJsonArchiveOutputFormatter::Serialize(bool& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonArchiveOutputFormatter::Serialize(UTF32CHAR& Value)
{
	WriteValue(LexToString(*(uint32*)&Value));
}

void FJsonArchiveOutputFormatter::Serialize(FString& Value)
{
	// Insert a "String:" prefix to prevent incorrect interpretation as another explicit type
	if (Value.StartsWith(TEXT("Object:")) || Value.StartsWith(TEXT("String:")) || Value.StartsWith(TEXT("Base64:")))
	{
		SerializeStringInternal(FString::Printf(TEXT("String:%s"), *Value));
	}
	else
	{
		SerializeStringInternal(Value);
	}
}

void FJsonArchiveOutputFormatter::Serialize(FName& Value)
{
	SerializeStringInternal(*Value.ToString());
}

void FJsonArchiveOutputFormatter::Serialize(UObject*& Value)
{
	if (Value != nullptr && IsObjectAllowed(Value))
	{
		FPackageIndex ObjectIndex = ObjectIndicesMap->FindChecked(Value);
		SerializeStringInternal(LexToString(ObjectIndex));
	}
	else
	{
		WriteValue(TEXT("null"));
	}
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FJsonArchiveOutputFormatter::Serialize(Verse::VCell*& Value)
{
	if (Value != nullptr && IsCellAllowed(Value))
	{
		FPackageIndex CellIndex = CellIndicesMap->FindChecked(Value);
		SerializeStringInternal(LexToString(CellIndex));
	}
	else
	{
		WriteValue(TEXT("null"));
	}
}
#endif

void FJsonArchiveOutputFormatter::Serialize(FText& Value)
{
	FStructuredArchive ChildArchive(*this);
	FText::SerializeText(ChildArchive.Open(), Value);
	ChildArchive.Close();
}

void FJsonArchiveOutputFormatter::Serialize(FWeakObjectPtr& Value)
{
	UObject* Ptr = Value.IsValid() ? Value.Get() : nullptr;
	Serialize(Ptr);
}

void FJsonArchiveOutputFormatter::Serialize(FSoftObjectPtr& Value)
{
	FSoftObjectPath Path = Value.ToSoftObjectPath();
	Serialize(Path);
}

void FJsonArchiveOutputFormatter::Serialize(FSoftObjectPath& Value)
{
	if (Value.IsValid())
	{
		SerializeStringInternal(FString::Printf(TEXT("Object:%s"), *Value.ToString()));
	}
	else
	{
		WriteValue(TEXT("null"));
	}
}

void FJsonArchiveOutputFormatter::Serialize(FLazyObjectPtr& Value)
{
	if (Value.IsValid() && IsObjectAllowed(Value.Get()))
	{
		SerializeStringInternal(FString::Printf(TEXT("Lazy:%s"), *Value.GetUniqueID().ToString()));
	}
	else
	{
		WriteValue(TEXT("null"));
	}
}

void FJsonArchiveOutputFormatter::Serialize(FObjectPtr& Value)
{
	UObject* Object = Value.Get();
	Serialize(Object);
}

void FJsonArchiveOutputFormatter::Serialize(TArray<uint8>& Data)
{
	Serialize(Data.GetData(), Data.Num());
}

void FJsonArchiveOutputFormatter::Serialize(void* Data, uint64 DataSize)
{
	static const int32 MaxLineChars = 120;
	static const int32 MaxLineBytes = FBase64::GetMaxDecodedDataSize(MaxLineChars);

	if(DataSize < MaxLineBytes)
	{
		// Encode the data on a single line. No need for hashing; intra-line merge conflicts are rare.
		WriteValue(FString::Printf(TEXT("\"Base64:%s\""), *FBase64::Encode((const uint8*)Data, static_cast<uint32>(DataSize))));
	}
	else
	{
		// Encode the data as a record containing a digest and array of base-64 encoded lines
		EnterRecord();
		Inner.Serialize(Newline.GetData(), Newline.Num());

		// Compute a SHA digest for the raw data, so we can check if it's corrupted
		uint8 Digest[FSHA1::DigestSize];
		FSHA1::HashBuffer(Data, DataSize, Digest);

		// Convert the hash to a string
		ANSICHAR DigestString[(FSHA1::DigestSize * 2) + 1];
		for(int32 Idx = 0; Idx < UE_ARRAY_COUNT(Digest); Idx++)
		{
			static const ANSICHAR HexDigits[] = "0123456789abcdef";
			DigestString[(Idx * 2) + 0] = HexDigits[Digest[Idx] >> 4];
			DigestString[(Idx * 2) + 1] = HexDigits[Digest[Idx] & 15];
		}
		DigestString[UE_ARRAY_COUNT(DigestString) - 1] = 0;

		// Write the digest
		Write("\"Digest\": \"");
		Write(DigestString);
		Write("\",");
		Inner.Serialize(Newline.GetData(), Newline.Num());

		// Write the base64 data
		Write("\"Base64\": ");
		for(uint64 DataPos = 0; DataPos < DataSize; DataPos += MaxLineBytes)
		{
			Write((DataPos > 0)? ',' : '[');
			Inner.Serialize(Newline.GetData(), Newline.Num());
			Write("\t\"");

			ANSICHAR LineData[MaxLineChars + 1];
			uint64 NumLineChars = FBase64::Encode((const uint8*)Data + DataPos, FMath::Min<uint32>(IntCastChecked<uint32>(DataSize - DataPos), MaxLineBytes), LineData);
			Inner.Serialize(LineData, NumLineChars);

			Write("\"");
		}

		// Close the array
		Inner.Serialize(Newline.GetData(), Newline.Num());
		Write(']');
		bNeedsNewline = true;

		// Close the record
		LeaveRecord();
	}
}

void FJsonArchiveOutputFormatter::Write(ANSICHAR Character)
{
	Inner.Serialize((void*)&Character, 1);
}

void FJsonArchiveOutputFormatter::Write(const ANSICHAR* Text)
{
	Inner.Serialize((void*)Text, TCString<ANSICHAR>::Strlen(Text));
}

void FJsonArchiveOutputFormatter::Write(const FString& Text)
{
	Write(TCHAR_TO_UTF8(*Text));
}

void FJsonArchiveOutputFormatter::WriteFieldName(const TCHAR* Name)
{
	if(FCString::Stricmp(Name, TEXT("Base64")) == 0 || FCString::Stricmp(Name, TEXT("Digest")) == 0)
	{
		Write(FString::Printf(TEXT("\"_%s\": "), Name));
	}
	else if(Name[0] == '_')
	{
		Write(FString::Printf(TEXT("\"_%s\": "), Name));
	}
	else
	{
		Write(FString::Printf(TEXT("\"%s\": "), Name));
	}
}

void FJsonArchiveOutputFormatter::WriteValue(const FString& Text)
{
	Write(Text);
}

void FJsonArchiveOutputFormatter::WriteOptionalComma()
{
	if (bNeedsComma)
	{
		Write(',');
		bNeedsComma = false;
	}
}

void FJsonArchiveOutputFormatter::WriteOptionalNewline()
{
	if (bNeedsNewline)
	{
		Inner.Serialize(Newline.GetData(), Newline.Num());
		bNeedsNewline = false;
	}
}

void FJsonArchiveOutputFormatter::WriteOptionalAttributedBlockOpening()
{
	if (NumAttributesStack.Top() == 0)
	{
		Write('{');
		Newline.Add('\t');
		bNeedsNewline = true;
	}
}

void FJsonArchiveOutputFormatter::WriteOptionalAttributedBlockValue()
{
	if (NumAttributesStack.Top() != 0)
	{
		WriteFieldName(TEXT("_Value"));
	}
}

void FJsonArchiveOutputFormatter::WriteOptionalAttributedBlockClosing()
{
	if (NumAttributesStack.Top() != 0)
	{
		Newline.Pop(EAllowShrinking::No);
		WriteOptionalNewline();
		Write("}");
		bNeedsComma                  = true;
		bNeedsNewline                = true;
	}
}

void FJsonArchiveOutputFormatter::SerializeStringInternal(const FString& String)
{
	FString Result = "\"";

	// Escape the string characters
	for (int32 Idx = 0; Idx < String.Len(); Idx++)
	{
		switch (String[Idx])
		{
		case '\"':
			Result += "\\\"";
			break;
		case '\\':
			Result += "\\\\";
			break;
		case '\b':
			Result += "\\b";
			break;
		case '\f':
			Result += "\\f";
			break;
		case '\n':
			Result += "\\n";
			break;
		case '\r':
			Result += "\\r";
			break;
		case '\t':
			Result += "\\t";
			break;
		default:
			if (String[Idx] <= 0x1f || String[Idx] >= 0x7f)
			{
				Result += FString::Printf(TEXT("\\u%04x"), String[Idx]);
			}
			else
			{
				Result.AppendChar(String[Idx]);
			}
			break;
		}
	}
	Result += TEXT("\"");

	WriteValue(Result);
}

bool FJsonArchiveOutputFormatter::IsObjectAllowed(UObject* InObject) const
{
	return ObjectIndicesMap && ObjectIndicesMap->Contains(InObject);
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
bool FJsonArchiveOutputFormatter::IsCellAllowed(Verse::VCell* InCell) const
{
	return CellIndicesMap && CellIndicesMap->Contains(InCell);
}
#endif

#endif
