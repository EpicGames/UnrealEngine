// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ImportTypeHierarchy.h"
#include "HAL/IConsoleManager.h"

namespace UE::Serialization::Private
{

FArchive& operator<<(FArchive& Ar, FTypeResource& TypeResource)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << TypeResource;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FTypeResource& TypeResource)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	Record << SA_VALUE(TEXT("TypeName"), TypeResource.TypeName);
	Record << SA_VALUE(TEXT("PackageName"), TypeResource.PackageName);
	Record << SA_VALUE(TEXT("ClassName"), TypeResource.ClassName);
	Record << SA_VALUE(TEXT("ClassPackageName"), TypeResource.ClassPackageName);
}

FArchive& operator<<(FArchive& Ar, FImportTypeHierarchy& I)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << I;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FImportTypeHierarchy& I)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	int32 SuperTypeCount = I.SuperTypes.Num();
	Record << SA_VALUE(TEXT("SuperTypeCount"), SuperTypeCount);

	if (!ensure(SuperTypeCount >= 0))
	{
		SuperTypeCount = 0;
	}

	FArchive& BaseArchive = Slot.GetUnderlyingArchive();
	if (BaseArchive.IsLoading())
	{
		I.SuperTypes.SetNum(SuperTypeCount);
	}

	FStructuredArchive::FStream SuperTypeStream = Record.EnterStream(TEXT("SuperTypes"));
	for (int32 i = 0; i < SuperTypeCount; ++i)
	{
		if (BaseArchive.IsCriticalError())
		{
			return;
		}

		FStructuredArchiveRecord NameRecord = SuperTypeStream.EnterElement().EnterRecord();

		NameRecord << SA_VALUE(TEXT("SuperType"), I.SuperTypes[i]);
	}
}

} // namespace UE::Serialization::Private
