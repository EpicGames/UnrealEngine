// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/LoadTimeTrace.h"

#if LOADTIMEPROFILERTRACE_ENABLED

#include "LoadTimeTracePrivate.h"
#include "Trace/Trace.inl"
#include "Misc/CString.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformTLS.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/CommandLine.h"

UE_TRACE_EVENT_BEGIN(LoadTime, StartAsyncLoading, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, SuspendAsyncLoading)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, ResumeAsyncLoading)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, PackageSummary)
	UE_TRACE_EVENT_FIELD(const void*, AsyncPackage)
	UE_TRACE_EVENT_FIELD(uint32, TotalHeaderSize)
	UE_TRACE_EVENT_FIELD(uint32, ImportCount)
	UE_TRACE_EVENT_FIELD(uint32, ExportCount)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, Name)
	UE_TRACE_EVENT_FIELD(int32, Priority) // added in UE 5.4
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, BeginProcessSummary)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(const void*, AsyncPackage)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, EndProcessSummary)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, BeginCreateExport)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(const void*, AsyncPackage)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, EndCreateExport)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(const UObject*, Object)
	UE_TRACE_EVENT_FIELD(const UClass*, Class)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, BeginSerializeExport)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(const UObject*, Object)
	UE_TRACE_EVENT_FIELD(uint64, SerialSize)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, EndSerializeExport)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, BeginPostLoad)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, EndPostLoad)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, BeginPostLoadObject)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(const UObject*, Object)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, EndPostLoadObject)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, BeginRequest)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, RequestId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, EndRequest)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint64, RequestId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, NewAsyncPackage)
	UE_TRACE_EVENT_FIELD(const void*, AsyncPackage)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, DestroyAsyncPackage)
	UE_TRACE_EVENT_FIELD(const void*, AsyncPackage)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, NewLinker)
	UE_TRACE_EVENT_FIELD(const void*, Linker)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, DestroyLinker)
	UE_TRACE_EVENT_FIELD(const void*, Linker)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, AsyncPackageRequestAssociation)
	UE_TRACE_EVENT_FIELD(const void*, AsyncPackage)
	UE_TRACE_EVENT_FIELD(uint64, RequestId)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, AsyncPackageLinkerAssociation)
	UE_TRACE_EVENT_FIELD(const void*, AsyncPackage)
	UE_TRACE_EVENT_FIELD(const void*, Linker)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, AsyncPackageImportDependency)
	UE_TRACE_EVENT_FIELD(const void*, AsyncPackage)
	UE_TRACE_EVENT_FIELD(const void*, ImportedAsyncPackage)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(LoadTime, ClassInfo, NoSync|Important)
	UE_TRACE_EVENT_FIELD(const UClass*, Class)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

void FLoadTimeProfilerTracePrivate::OutputStartAsyncLoading()
{
	UE_TRACE_LOG(LoadTime, StartAsyncLoading, LoadTimeChannel)
		<< StartAsyncLoading.Cycle(FPlatformTime::Cycles64());
}

PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
void FLoadTimeProfilerTracePrivate::OutputSuspendAsyncLoading()
{
	UE_TRACE_LOG(LoadTime, SuspendAsyncLoading, LoadTimeChannel)
		<< SuspendAsyncLoading.Cycle(FPlatformTime::Cycles64());
}

void FLoadTimeProfilerTracePrivate::OutputResumeAsyncLoading()
{
	UE_TRACE_LOG(LoadTime, ResumeAsyncLoading, LoadTimeChannel)
		<< ResumeAsyncLoading.Cycle(FPlatformTime::Cycles64());
}
PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS

void FLoadTimeProfilerTracePrivate::OutputBeginRequest(uint64 RequestId)
{
	UE_TRACE_LOG(LoadTime, BeginRequest, LoadTimeChannel)
		<< BeginRequest.Cycle(FPlatformTime::Cycles64())
		<< BeginRequest.RequestId(RequestId);
}

void FLoadTimeProfilerTracePrivate::OutputEndRequest(uint64 RequestId)
{
	UE_TRACE_LOG(LoadTime, EndRequest, LoadTimeChannel)
		<< EndRequest.Cycle(FPlatformTime::Cycles64())
		<< EndRequest.RequestId(RequestId);
}

void FLoadTimeProfilerTracePrivate::OutputNewAsyncPackage(const void* AsyncPackage)
{
	UE_TRACE_LOG(LoadTime, NewAsyncPackage, LoadTimeChannel)
		<< NewAsyncPackage.AsyncPackage(AsyncPackage);
}

void FLoadTimeProfilerTracePrivate::OutputDestroyAsyncPackage(const void* AsyncPackage)
{
	UE_TRACE_LOG(LoadTime, DestroyAsyncPackage, LoadTimeChannel)
		<< DestroyAsyncPackage.AsyncPackage(AsyncPackage);
}

void FLoadTimeProfilerTracePrivate::OutputNewLinker(const void* Linker)
{
	UE_TRACE_LOG(LoadTime, NewLinker, LoadTimeChannel)
		<< NewLinker.Linker(Linker);
}

void FLoadTimeProfilerTracePrivate::OutputDestroyLinker(const void* Linker)
{
	UE_TRACE_LOG(LoadTime, DestroyLinker, LoadTimeChannel)
		<< DestroyLinker.Linker(Linker);
}

void FLoadTimeProfilerTracePrivate::OutputPackageSummary(const void* AsyncPackage, const FName& PackageName, uint32 TotalHeaderSize, uint32 ImportCount, uint32 ExportCount, int Priority)
{
	TCHAR Buffer[FName::StringBufferSize];
	uint32 NameLength = PackageName.ToString(Buffer);

	UE_TRACE_LOG(LoadTime, PackageSummary, LoadTimeChannel)
		<< PackageSummary.AsyncPackage(AsyncPackage)
		<< PackageSummary.TotalHeaderSize(TotalHeaderSize)
		<< PackageSummary.ImportCount(ImportCount)
		<< PackageSummary.ExportCount(ExportCount)
		<< PackageSummary.Name(Buffer, NameLength)
		<< PackageSummary.Priority(Priority);
}

void FLoadTimeProfilerTracePrivate::OutputAsyncPackageRequestAssociation(const void* AsyncPackage, uint64 RequestId)
{
	UE_TRACE_LOG(LoadTime, AsyncPackageRequestAssociation, LoadTimeChannel)
		<< AsyncPackageRequestAssociation.AsyncPackage(AsyncPackage)
		<< AsyncPackageRequestAssociation.RequestId(RequestId);
}

void FLoadTimeProfilerTracePrivate::OutputAsyncPackageLinkerAssociation(const void* AsyncPackage, const void* Linker)
{
	UE_TRACE_LOG(LoadTime, AsyncPackageLinkerAssociation, LoadTimeChannel)
		<< AsyncPackageLinkerAssociation.AsyncPackage(AsyncPackage)
		<< AsyncPackageLinkerAssociation.Linker(Linker);
}

void FLoadTimeProfilerTracePrivate::OutputAsyncPackageImportDependency(const void* Package, const void* ImportedPackage)
{
	UE_TRACE_LOG(LoadTime, AsyncPackageImportDependency, LoadTimeChannel)
		<< AsyncPackageImportDependency.AsyncPackage(Package)
		<< AsyncPackageImportDependency.ImportedAsyncPackage(ImportedPackage);
}

void FLoadTimeProfilerTracePrivate::OutputClassInfo(const UClass* Class, const FName& Name)
{
	TCHAR Buffer[FName::StringBufferSize];
	uint16 NameLen = static_cast<uint16>(Name.ToString(Buffer));
	UE_TRACE_LOG(LoadTime, ClassInfo, LoadTimeChannel, NameLen * sizeof(ANSICHAR))
		<< ClassInfo.Class(Class)
		<< ClassInfo.Name(Buffer, NameLen);
}

void FLoadTimeProfilerTracePrivate::OutputClassInfo(const UClass* Class, const TCHAR* Name)
{
	uint16 NameLen = uint16(FCString::Strlen(Name));
	UE_TRACE_LOG(LoadTime, ClassInfo, LoadTimeChannel, NameLen * sizeof(ANSICHAR))
		<< ClassInfo.Class(Class)
		<< ClassInfo.Name(Name, NameLen);
}

void FLoadTimeProfilerTracePrivate::OutputBeginProcessSummary(const void* AsyncPackage)
{
	UE_TRACE_LOG(LoadTime, BeginProcessSummary, LoadTimeChannel)
		<< BeginProcessSummary.Cycle(FPlatformTime::Cycles64())
		<< BeginProcessSummary.AsyncPackage(AsyncPackage);
}

void FLoadTimeProfilerTracePrivate::OutputEndProcessSummary()
{
	UE_TRACE_LOG(LoadTime, EndProcessSummary, LoadTimeChannel)
		<< EndProcessSummary.Cycle(FPlatformTime::Cycles64());
}

FLoadTimeProfilerTracePrivate::FProcessSummaryScope::FProcessSummaryScope(const void* AsyncPackage)
{
	FLoadTimeProfilerTracePrivate::OutputBeginProcessSummary(AsyncPackage);
}

FLoadTimeProfilerTracePrivate::FProcessSummaryScope::~FProcessSummaryScope()
{
	FLoadTimeProfilerTracePrivate::OutputEndProcessSummary();
}

FLoadTimeProfilerTracePrivate::FCreateExportScope::FCreateExportScope(const void* AsyncPackage, const UObject* const* InObject)
	: Object(InObject)
{
	UE_TRACE_LOG(LoadTime, BeginCreateExport, LoadTimeChannel)
		<< BeginCreateExport.Cycle(FPlatformTime::Cycles64())
		<< BeginCreateExport.AsyncPackage(AsyncPackage);
}

FLoadTimeProfilerTracePrivate::FCreateExportScope::~FCreateExportScope()
{
	UE_TRACE_LOG(LoadTime, EndCreateExport, LoadTimeChannel)
		<< EndCreateExport.Cycle(FPlatformTime::Cycles64())
		<< EndCreateExport.Object(*Object)
		<< EndCreateExport.Class(*Object ? (*Object)->GetClass() : nullptr);
}

FLoadTimeProfilerTracePrivate::FSerializeExportScope::FSerializeExportScope(const UObject* Object, uint64 SerialSize)
{
	UE_TRACE_LOG(LoadTime, BeginSerializeExport, LoadTimeChannel)
		<< BeginSerializeExport.Cycle(FPlatformTime::Cycles64())
		<< BeginSerializeExport.Object(Object)
		<< BeginSerializeExport.SerialSize(SerialSize);
}

FLoadTimeProfilerTracePrivate::FSerializeExportScope::~FSerializeExportScope()
{
	UE_TRACE_LOG(LoadTime, EndSerializeExport, LoadTimeChannel)
		<< EndSerializeExport.Cycle(FPlatformTime::Cycles64());
}

FLoadTimeProfilerTracePrivate::FPostLoadScope::FPostLoadScope()
{
	UE_TRACE_LOG(LoadTime, BeginPostLoad, LoadTimeChannel);
}

FLoadTimeProfilerTracePrivate::FPostLoadScope::~FPostLoadScope()
{
	UE_TRACE_LOG(LoadTime, EndPostLoad, LoadTimeChannel);
}

FLoadTimeProfilerTracePrivate::FPostLoadObjectScope::FPostLoadObjectScope(const UObject* Object)
{
	UE_TRACE_LOG(LoadTime, BeginPostLoadObject, LoadTimeChannel)
		<< BeginPostLoadObject.Cycle(FPlatformTime::Cycles64())
		<< BeginPostLoadObject.Object(Object);
}

FLoadTimeProfilerTracePrivate::FPostLoadObjectScope::~FPostLoadObjectScope()
{
	UE_TRACE_LOG(LoadTime, EndPostLoadObject, LoadTimeChannel)
		<< EndPostLoadObject.Cycle(FPlatformTime::Cycles64());
}

#endif
