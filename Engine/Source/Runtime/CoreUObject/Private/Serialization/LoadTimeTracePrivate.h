// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/LoadTimeTrace.h"

#if LOADTIMEPROFILERTRACE_ENABLED

class UObject;
class FName;

struct FLoadTimeProfilerTracePrivate
{
	static void OutputStartAsyncLoading();
	static void OutputSuspendAsyncLoading();
	static void OutputResumeAsyncLoading();
	static void OutputNewAsyncPackage(const void* AsyncPackage);
	static void OutputDestroyAsyncPackage(const void* AsyncPackage);
	static void OutputNewLinker(const void* Linker);
	static void OutputDestroyLinker(const void* Linker);
	static void OutputBeginRequest(uint64 RequestId);
	static void OutputEndRequest(uint64 RequestId);
	static void OutputPackageSummary(const void* AsyncPackage, const FName& PackageName, uint32 TotalHeaderSize, uint32 ImportCount, uint32 ExportCount, int32 Priority);
	static void OutputAsyncPackageImportDependency(const void* Package, const void* ImportedPackage);
	static void OutputAsyncPackageRequestAssociation(const void* AsyncPackage, uint64 RequestId);
	static void OutputAsyncPackageLinkerAssociation(const void* AsyncPackage, const void* Linker);
	static void OutputClassInfo(const UClass* Class, const FName& Name);
	static void OutputClassInfo(const UClass* Class, const TCHAR* Name);
	static void OutputBeginProcessSummary(const void* AsyncPackage);
	static void OutputEndProcessSummary();

	struct FProcessSummaryScope
	{
		FProcessSummaryScope(const void* AsyncPackage);
		~FProcessSummaryScope();
	};

	struct FCreateExportScope
	{
		FCreateExportScope(const void* AsyncPackage, const UObject* const* InObject);
		~FCreateExportScope();

	private:
		const UObject* const* Object;
	};

	struct FSerializeExportScope
	{
		FSerializeExportScope(const UObject* Object, uint64 SerialSize);
		~FSerializeExportScope();
	};

	struct FPostLoadScope
	{
		FPostLoadScope();
		~FPostLoadScope();
	};

	struct FPostLoadObjectScope
	{
		FPostLoadObjectScope(const UObject* Object);
		~FPostLoadObjectScope();
	};
};

#define TRACE_LOADTIME_START_ASYNC_LOADING() \
	FLoadTimeProfilerTracePrivate::OutputStartAsyncLoading();

#define TRACE_LOADTIME_SUSPEND_ASYNC_LOADING() \
	FLoadTimeProfilerTracePrivate::OutputSuspendAsyncLoading();

#define TRACE_LOADTIME_RESUME_ASYNC_LOADING() \
	FLoadTimeProfilerTracePrivate::OutputResumeAsyncLoading();

#define TRACE_LOADTIME_BEGIN_REQUEST(RequestId) \
	FLoadTimeProfilerTracePrivate::OutputBeginRequest(RequestId);

#define TRACE_LOADTIME_END_REQUEST(RequestId) \
	FLoadTimeProfilerTracePrivate::OutputEndRequest(RequestId);

#define TRACE_LOADTIME_NEW_ASYNC_PACKAGE(AsyncPackage) \
	FLoadTimeProfilerTracePrivate::OutputNewAsyncPackage(AsyncPackage)

#define TRACE_LOADTIME_DESTROY_ASYNC_PACKAGE(AsyncPackage) \
	FLoadTimeProfilerTracePrivate::OutputDestroyAsyncPackage(AsyncPackage);

#define TRACE_LOADTIME_NEW_LINKER(Linker) \
	FLoadTimeProfilerTracePrivate::OutputNewLinker(Linker)

#define TRACE_LOADTIME_DESTROY_LINKER(Linker) \
	FLoadTimeProfilerTracePrivate::OutputDestroyLinker(Linker);

#define TRACE_LOADTIME_PACKAGE_SUMMARY(AsyncPackage, PackageName, TotalHeaderSize, ImportCount, ExportCount, Priority) \
	FLoadTimeProfilerTracePrivate::OutputPackageSummary(AsyncPackage, PackageName, TotalHeaderSize, ImportCount, ExportCount, Priority);

#define TRACE_LOADTIME_ASYNC_PACKAGE_REQUEST_ASSOCIATION(AsyncPackage, RequestId) \
	FLoadTimeProfilerTracePrivate::OutputAsyncPackageRequestAssociation(AsyncPackage, RequestId);

#define TRACE_LOADTIME_ASYNC_PACKAGE_LINKER_ASSOCIATION(AsyncPackage, Linker) \
	FLoadTimeProfilerTracePrivate::OutputAsyncPackageLinkerAssociation(AsyncPackage, Linker);

#define TRACE_LOADTIME_ASYNC_PACKAGE_IMPORT_DEPENDENCY(AsyncPackage, ImportedAsyncPackage) \
	FLoadTimeProfilerTracePrivate::OutputAsyncPackageImportDependency(AsyncPackage, ImportedAsyncPackage);

#define TRACE_LOADTIME_PROCESS_SUMMARY_SCOPE(AsyncPackage) \
	FLoadTimeProfilerTracePrivate::FProcessSummaryScope __LoadTimeTraceProcessSummaryScope(AsyncPackage);

#define TRACE_LOADTIME_BEGIN_PROCESS_SUMMARY(AsyncPackage) \
	FLoadTimeProfilerTracePrivate::OutputBeginProcessSummary(AsyncPackage);

#define TRACE_LOADTIME_END_PROCESS_SUMMARY \
	FLoadTimeProfilerTracePrivate::OutputEndProcessSummary();

#define TRACE_LOADTIME_CREATE_EXPORT_SCOPE(AsyncPackage, Object) \
	FLoadTimeProfilerTracePrivate::FCreateExportScope __LoadTimeTraceCreateExportScope(AsyncPackage, Object);

#define TRACE_LOADTIME_SERIALIZE_EXPORT_SCOPE(Object, SerialSize) \
	FLoadTimeProfilerTracePrivate::FSerializeExportScope __LoadTimeTraceSerializeExportScope(Object, SerialSize);

#define TRACE_LOADTIME_POSTLOAD_SCOPE \
	FLoadTimeProfilerTracePrivate::FPostLoadScope __LoadTimeTracePostLoadScope;

#define TRACE_LOADTIME_POSTLOAD_OBJECT_SCOPE(Object) \
	FLoadTimeProfilerTracePrivate::FPostLoadObjectScope __LoadTimeTracePostLoadObjectScope(Object);

#define TRACE_LOADTIME_CLASS_INFO(Class, Name) \
	FLoadTimeProfilerTracePrivate::OutputClassInfo(Class, Name);

#else

#define TRACE_LOADTIME_START_ASYNC_LOADING(...)
#define TRACE_LOADTIME_SUSPEND_ASYNC_LOADING(...)
#define TRACE_LOADTIME_RESUME_ASYNC_LOADING(...)
#define TRACE_LOADTIME_BEGIN_REQUEST(...)
#define TRACE_LOADTIME_END_REQUEST(...)
#define TRACE_LOADTIME_NEW_ASYNC_PACKAGE(...)
#define TRACE_LOADTIME_DESTROY_ASYNC_PACKAGE(...)
#define TRACE_LOADTIME_NEW_LINKER(...)
#define TRACE_LOADTIME_DESTROY_LINKER(...)
#define TRACE_LOADTIME_PACKAGE_SUMMARY(...)
#define TRACE_LOADTIME_ASYNC_PACKAGE_REQUEST_ASSOCIATION(...)
#define TRACE_LOADTIME_ASYNC_PACKAGE_LINKER_ASSOCIATION(...)
#define TRACE_LOADTIME_ASYNC_PACKAGE_IMPORT_DEPENDENCY(...)
#define TRACE_LOADTIME_PROCESS_SUMMARY_SCOPE(...)
#define TRACE_LOADTIME_BEGIN_PROCESS_SUMMARY(...)
#define TRACE_LOADTIME_END_PROCESS_SUMMARY
#define TRACE_LOADTIME_CREATE_EXPORT_SCOPE(...)
#define TRACE_LOADTIME_SERIALIZE_EXPORT_SCOPE(...)
#define TRACE_LOADTIME_POSTLOAD_SCOPE
#define TRACE_LOADTIME_POSTLOAD_OBJECT_SCOPE(...)
#define TRACE_LOADTIME_CLASS_INFO(...)

#endif
