// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Serialization/Archive.h"

class FNullArchive : public FArchive
{
	//override FObjectPtr to do nothing
	virtual FArchive& operator<<(struct FObjectPtr& Value) override
	{
		return *this;
	}
};