// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ObjectReader.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

///////////////////////////////////////////////////////
// FObjectReader

FArchive& FObjectReader::operator<<(FName& N)
{
	FNameEntryId ComparisonIndex;
	FNameEntryId DisplayIndex;
	int32 Number;
	ByteOrderSerialize(&ComparisonIndex, sizeof(ComparisonIndex));
	ByteOrderSerialize(&DisplayIndex, sizeof(DisplayIndex));
	ByteOrderSerialize(&Number, sizeof(Number));
	// copy over the name with a name made from the name index and number
	N = FName(ComparisonIndex, DisplayIndex, Number);
	return *this;
}

FArchive& FObjectReader::operator<<(UObject*& Res)
{
	ByteOrderSerialize(&Res, sizeof(Res));
	return *this;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FArchive& FObjectReader::operator<<(Verse::VCell*& Res)
{
	ByteOrderSerialize(&Res, sizeof(Res));
	return *this;
}
#endif

FArchive& FObjectReader::operator<<(FObjectPtr& Value)
{
	ByteOrderSerialize(&Value, sizeof(Value));
	return *this;
}

FArchive& FObjectReader::operator<<(FLazyObjectPtr& Value)
{
	FArchive& Ar = *this;
	FUniqueObjectGuid ID;
	Ar << ID;

	Value = ID;
	return Ar;
}

FArchive& FObjectReader::operator<<(FSoftObjectPtr& Value)
{
	Value.ResetWeakPtr();
	return *this << Value.GetUniqueID();
}

FArchive& FObjectReader::operator<<(FSoftObjectPath& Value)
{
	Value.SerializePath(*this);
	return *this;
}

FArchive& FObjectReader::operator<<(FWeakObjectPtr& Value)
{
	return FArchiveUObject::SerializeWeakObjectPtr(*this, Value);
}

FString FObjectReader::GetArchiveName() const
{
	return TEXT("FObjectReader");
}
