// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ObjectWriter.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"

///////////////////////////////////////////////////////
// FObjectWriter

FArchive& FObjectWriter::operator<<( class FName& N )
{
	FNameEntryId ComparisonIndex = N.GetComparisonIndex();
	FNameEntryId DisplayIndex = N.GetDisplayIndex();
	int32 Number = N.GetNumber();
	ByteOrderSerialize(&ComparisonIndex, sizeof(ComparisonIndex));
	ByteOrderSerialize(&DisplayIndex, sizeof(DisplayIndex));
	ByteOrderSerialize(&Number, sizeof(Number));
	return *this;
}

FArchive& FObjectWriter::operator<<( class UObject*& Res )
{
	ByteOrderSerialize(&Res, sizeof(Res));
	return *this;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FArchive& FObjectWriter::operator<<( Verse::VCell*& Res )
{
	ByteOrderSerialize(&Res, sizeof(Res));
	return *this;
}
#endif

FArchive& FObjectWriter::operator<<(FObjectPtr& Value)
{
	ByteOrderSerialize(&Value, sizeof(Value));
	return *this;
}

FArchive& FObjectWriter::operator<<(FLazyObjectPtr& Value)
{
	FUniqueObjectGuid ID = Value.GetUniqueID();
	return *this << ID;
}

FArchive& FObjectWriter::operator<<(FSoftObjectPtr& Value)
{
	return *this << Value.GetUniqueID();
}

FArchive& FObjectWriter::operator<<(FSoftObjectPath& Value)
{
	Value.SerializePath(*this);
	return *this;
}

FArchive& FObjectWriter::operator<<(FWeakObjectPtr& Value)
{
	return FArchiveUObject::SerializeWeakObjectPtr(*this, Value);
}

FString FObjectWriter::GetArchiveName() const
{
	return TEXT("FObjectWriter");
}
