// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/PackageStore.h"

#include "AssetRegistry/AssetData.h"
#include "Async/ParallelFor.h"
#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "Containers/StripedMap.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreGlobalsInternal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformFileManager.h"
#include "IO/PackageId.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/PackageName.h"
#include "Serialization/Archive.h"
#include "Serialization/AsyncPackageLoader.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Templates/Greater.h"


DEFINE_LOG_CATEGORY_STATIC(LogPackageStore, Log, All);

const TCHAR* LexToString(EPackageStoreEntryStatus PackageLoader)
{
	switch (PackageLoader)
	{
	case EPackageStoreEntryStatus::None: return TEXT("None");
	case EPackageStoreEntryStatus::Missing: return TEXT("Missing");
	case EPackageStoreEntryStatus::NotInstalled: return TEXT("NotInstalled");
	case EPackageStoreEntryStatus::Pending: return TEXT("Pending");
	case EPackageStoreEntryStatus::Ok: return TEXT("Ok");
	default:
		checkNoEntry();
		return TEXT("Not Implemented");
	}
};

///////////////////////////////////////
// FLinkerLoadPackageStoreBackend
///////////////////////////////////////

class FLinkerLoadPackageStoreBackend final : public IPackageStoreBackend
{
public:
	~FLinkerLoadPackageStoreBackend()
	{
		if (OnAssetRemovedHandle.IsValid())
		{
			if (IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr())
			{
				AssetRegistry->UnregisterOnAssetsRemovedDelegate(OnAssetRemovedHandle);
			}
		}
	}

	void OnAssetsRemoved(TConstArrayView<FAssetData> RemovedAssets)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLinkerLoadPackageStoreBackend::OnAssetsRemoved);
		const int32 MinBatchSize = 64;
		ParallelFor(TEXT("FLinkerLoadPackageStoreBackend::OnAssetsRemoved_PF"), RemovedAssets.Num(), MinBatchSize, [this, &RemovedAssets](int Index)
			{
				const FAssetData& AssetData = RemovedAssets[Index];
				FPackageId PackageId = FPackageId::FromName(AssetData.PackageName);
				PackageStoreCache.Remove(PackageId);
			});
	}

	virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext>) override
	{
		// We need to defer registration for the AssetRegistry delegate since the registry might not be ready yet
		FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([this]()
			{
				if (IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr())
				{
					AssetRegistry->RegisterOnAssetsRemovedDelegate([this](TConstArrayView<FAssetData> RemovedAssets) 
						{
							OnAssetsRemoved(RemovedAssets);
						} , OnAssetRemovedHandle);
				}
			});
	}

	virtual EPackageLoader GetSupportedLoaders() override
	{
		return EPackageLoader::LinkerLoad;
	}

	virtual void BeginRead() override
	{
	}

	virtual void EndRead() override
	{
	}

	EPackageStoreEntryStatus GetPackageStoreEntryEx(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry, FPackagePath* OutPackagePath)
	{
		EPackageStoreEntryStatus Status = EPackageStoreEntryStatus::Missing;
		bool bCacheHit = PackageStoreCache.FindAndApply(PackageId,
			[&](const FPackageStoreEntry& Entry)
			{
				if (Entry.LoaderType == EPackageLoader::LinkerLoad)
				{
					OutPackageStoreEntry = Entry;
					// can't get the package path without a name! (which is valid for cooked flows)
					if (OutPackagePath && !PackageName.IsNone())
					{
						*OutPackagePath = FPackagePath::FromPackageNameUnchecked(PackageName);
						OutPackagePath->SetHeaderExtension(OutPackageStoreEntry.PackageExtension);
					}
					Status = EPackageStoreEntryStatus::Ok;
				}
				else
				{
					Status = EPackageStoreEntryStatus::Missing;
				}
			});
		if (bCacheHit)
		{
			return Status;
		}

		if (PackageName.IsNone())
		{
			return EPackageStoreEntryStatus::Missing;
		}

#if WITH_EDITORONLY_DATA
		// In editor, set MatchCaseOnDisk=true so that we set the capitalization of the Package's FName to match the
		// capitalization on disk. Different capitalizations can arise from imports of the package that were somehow
		// constructed with a different captialization (most often because the disk captialization changed).
		// We need the captialization to match so that source control operations in case-significant source control
		// depots succeed, and to avoid indetermism in the cook.
		constexpr bool bMatchCaseOnDisk = true;
#else
		constexpr bool bMatchCaseOnDisk = false;
#endif

		FPackagePath LocalPackagePath;
		FPackagePath* PackagePath = OutPackagePath ? OutPackagePath : &LocalPackagePath;

		*PackagePath = FPackagePath::FromPackageNameUnchecked(PackageName);
		// now, check to see if the package exists in the local file system, and if it does, correct the case as needed
		if (FPackageName::DoesPackageExistEx(*PackagePath, FPackageName::EPackageLocationFilter::FileSystem,
			bMatchCaseOnDisk, PackagePath) != FPackageName::EPackageLocationFilter::None)
		{
			OutPackageStoreEntry.LoaderType = EPackageLoader::LinkerLoad;
			OutPackageStoreEntry.PackageExtension = PackagePath->GetHeaderExtension();
			check(OutPackageStoreEntry.PackageExtension != EPackageExtension::Unspecified);

#if WITH_EDITORONLY_DATA
			OutPackageStoreEntry.LinkerLoadCaseCorrectedPackageName = PackagePath->GetPackageFName();
#endif

			PackageStoreCache.Add(PackageId, OutPackageStoreEntry);
			return EPackageStoreEntryStatus::Ok;
		}

		// Disable caching missing entries for now. They can be loaded too early, before the plugin containing the asset
		// is mounted, and might become findable by FPackageName::DoesPackageExistEx later.
		//PackageStoreCache.Add(PackageId, FPackageStoreEntry());
		return EPackageStoreEntryStatus::Missing;
	}

	virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry) override
	{
		return GetPackageStoreEntryEx(PackageId, PackageName, OutPackageStoreEntry, nullptr);
	}

	virtual bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) override
	{
		return false;
	}

private:
	TStripedMap<32, FPackageId, FPackageStoreEntry> PackageStoreCache;
	FDelegateHandle OnAssetRemovedHandle;
};


///////////////////////////////////////
// FHybridPackageStoreBackend
///////////////////////////////////////


namespace
{
	TSet<FPackageId> ForcedLoosePackages;
}

FHybridPackageStoreBackend::FHybridPackageStoreBackend(TSharedPtr<IPackageStoreBackend> InLoosePackageStore, TSharedPtr<IPackageStoreBackend> InCookedPackageStore)
: LoosePackageStore(InLoosePackageStore)
, CookedPackageStore(InCookedPackageStore)
{
}

EPackageLoader FHybridPackageStoreBackend::GetSupportedLoaders()
{
	return EPackageLoader::LinkerLoad | EPackageLoader::Zen;
}

void FHybridPackageStoreBackend::OnMounted(TSharedRef<const FPackageStoreBackendContext>)
{

}

void FHybridPackageStoreBackend::BeginRead()
{
	LoosePackageStore->BeginRead();
	CookedPackageStore->BeginRead();
}

void FHybridPackageStoreBackend::EndRead()
{
	LoosePackageStore->EndRead();
	CookedPackageStore->EndRead();
}

bool FHybridPackageStoreBackend::GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId)
{
	return CookedPackageStore->GetPackageRedirectInfo(PackageId, OutSourcePackageName, OutRedirectedToPackageId);
}

EPackageStoreEntryStatus FHybridPackageStoreBackend::GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(HybridCookedEditor_GetPackageStoreEntry);

	EPackageStoreEntryStatus InnerStatus = EPackageStoreEntryStatus::Missing;

	bool bAllowZenLoad = true;
	// check if we are forcing this package to load uncooked
	if (ForcedLoosePackages.Contains(PackageId))
	{
		bAllowZenLoad = false;
	}
	else
	{
		// get the readonly flag (without slowly touching the disk for each package)
		FAssetPackageData AssetPackageData;
		UE::AssetRegistry::EExists Result = UE::AssetRegistry::EExists::Unknown;
		IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr();
		if (ensureMsgf(AssetRegistry, TEXT("It's expected we always have an AssetRegistry after Engine init when using Hybrid PackageStore")))
		{
			Result = AssetRegistry->TryGetAssetPackageData(PackageName, AssetPackageData);
		}

		// if we did find it in the AR, and it's writeable, assume user wants local changes, so load loose
		//
		// if we don't know the state yet, then instead of just assuming the worst and loading the loose version,
		// use the process of querying the loose packagestore, but also get the path, and then we can check the 
		// physical disk for readonly state, and use that to determine if we should load cooked or not (as if 
		// we had found the ReadOnly state in the AR, as above)
		//
		// otherwise, (it's readonly, or doesn't exist locally), we want to load cooked
		if (Result == UE::AssetRegistry::EExists::Exists && !AssetPackageData.IsReadOnly())
		{
			bAllowZenLoad = false;
		}
		if (Result == UE::AssetRegistry::EExists::Unknown)
		{
			FPackageStoreEntry TempLooseEntry;
			FPackagePath TempPath;
			TSharedPtr<FLinkerLoadPackageStoreBackend> LinkerLoadStore = StaticCastSharedPtr<FLinkerLoadPackageStoreBackend>(LoosePackageStore);
			// check if we can find this package on disk, getting the store entry, while also getting the FPackagePath.
			// this way, we can just return the PackageEntry directly
			if (LinkerLoadStore->GetPackageStoreEntryEx(PackageId, PackageName, TempLooseEntry, &TempPath) == EPackageStoreEntryStatus::Ok)
			{
				// now that we have determined the local path to the file, check if it's read only (if a valid path was returned)
				if (!TempPath.IsEmpty())
				{
					const bool bIsReadOnly = PackageName == NAME_None || FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*TempPath.GetLocalFullPath());
					if (!bIsReadOnly)
					{
						// if it's writeable, then we want to use the LinkerLoad version, so just return it
						OutPackageStoreEntry = TempLooseEntry;

						// we are done!
						InnerStatus = EPackageStoreEntryStatus::Ok;
						bAllowZenLoad = false;
					}
				}
			}
		}

		UE_LOGF(LogPackageStore, Verbose, "Hybrid Cooked Editor Discovery: Package: %ls, AssetPackageData.Exists: %d, bAllowZen: %d", 
			*PackageName.ToString(), (int)Result, bAllowZenLoad);
	}

	if (bAllowZenLoad)
	{
		InnerStatus = CookedPackageStore->GetPackageStoreEntry(PackageId, PackageName, OutPackageStoreEntry);
	}

	if (InnerStatus == EPackageStoreEntryStatus::Missing)
	{
		// reset it in case the cooked store partially filled this out
		OutPackageStoreEntry = {};
		InnerStatus = LoosePackageStore->GetPackageStoreEntry(PackageId, PackageName, OutPackageStoreEntry);
	}

	return InnerStatus;
}

void FHybridPackageStoreBackend::ForceLoadPackageAsLoose(FPackageId PackageId)
{
	ForcedLoosePackages.Add(PackageId);
}


///////////////////////////////////////
// FPackageStore
///////////////////////////////////////


FArchive& operator<<(FArchive& Ar, FPackageStoreEntryResource& PackageStoreEntry)
{
	uint32 Flags = static_cast<uint32>(PackageStoreEntry.Flags);

	Ar << Flags;
	Ar << PackageStoreEntry.PackageName;
	Ar << PackageStoreEntry.ImportedPackageIds;
	Ar << PackageStoreEntry.OptionalSegmentImportedPackageIds;
	Ar << PackageStoreEntry.SoftPackageReferences;

	if (Ar.IsLoading())
	{
		PackageStoreEntry.PackageId = FPackageId::FromName(PackageStoreEntry.PackageName);
		PackageStoreEntry.Flags = static_cast<EPackageStoreEntryFlags>(Flags);
	}

	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FPackageStoreEntryResource& PackageStoreEntry)
{
	Writer.BeginObject();

	Writer << "flags" << static_cast<uint32>(PackageStoreEntry.Flags);
	Writer << "packagename" << PackageStoreEntry.PackageName.ToString();

	if (PackageStoreEntry.ImportedPackageIds.Num())
	{
		Writer.BeginArray("importedpackageids");
		for (const FPackageId& ImportedPackageId : PackageStoreEntry.ImportedPackageIds)
		{
			Writer << ImportedPackageId.Value();
		}
		Writer.EndArray();
	}

	if (PackageStoreEntry.ShaderMapHashes.Num())
	{
		Writer.BeginArray("shadermaphashes");
		for (const FShaderHash& ShaderMapHash : PackageStoreEntry.ShaderMapHashes)
		{
			Writer << ShaderMapHash.ToString();
		}
		Writer.EndArray();
	}

	if (PackageStoreEntry.OptionalSegmentImportedPackageIds.Num())
	{
		Writer.BeginArray("optionalsegmentimportedpackageids");
		for (const FPackageId& ImportedPackageId : PackageStoreEntry.OptionalSegmentImportedPackageIds)
		{
			Writer << ImportedPackageId.Value();
		}
		Writer.EndArray();
	}

	if (PackageStoreEntry.SoftPackageReferences.Num())
	{
		Writer.BeginArray("softpackagereferences");
		for (const FPackageId& SoftRef : PackageStoreEntry.SoftPackageReferences)
		{
			Writer << SoftRef.Value();
		}
		Writer.EndArray();
	}

	Writer.EndObject();

	return Writer;
}

FPackageStoreEntryResource FPackageStoreEntryResource::FromCbObject(FCbObjectView Obj)
{
	FPackageStoreEntryResource Entry;

	Entry.Flags				= static_cast<EPackageStoreEntryFlags>(Obj["flags"].AsUInt32());
	Entry.PackageName		= FName(Obj["packagename"].AsString());
	Entry.PackageId			= FPackageId::FromName(Entry.PackageName);
	
	if (Obj["importedpackageids"])
	{
		for (FCbFieldView ArrayField : Obj["importedpackageids"])
		{
			Entry.ImportedPackageIds.Add(FPackageId::FromValue(ArrayField .AsUInt64()));
		}
	}
	
	if (Obj["shadermaphashes"])
	{
		for (FCbFieldView& ArrayField : Obj["shadermaphashes"].AsArrayView())
		{
			FShaderHash& ShaderMapHash = Entry.ShaderMapHashes.AddDefaulted_GetRef();
			ShaderMapHash.FromString(FUTF8ToTCHAR(ArrayField.AsString()));
		}
	}

	if (Obj["optionalsegmentimportedpackageids"])
	{
		for (FCbFieldView ArrayField : Obj["optionalsegmentimportedpackageids"])
		{
			Entry.OptionalSegmentImportedPackageIds.Add(FPackageId::FromValue(ArrayField.AsUInt64()));
		}
	}

	if (Obj["softpackagereferences"])
	{
		for (FCbFieldView ArrayField : Obj["softpackagereferences"])
		{
			Entry.SoftPackageReferences.Add(FPackageId::FromValue(ArrayField.AsUInt64()));
		}
	}

	return Entry;
}

FPackageStoreEntryResource FPackageStoreEntryResource::CreateEmptyPackage(FName PackageName, bool bHasCookError)
{
	FPackageStoreEntryResource Entry;
	Entry.PackageName = PackageName;
	Entry.PackageId = FPackageId::FromName(PackageName);
	Entry.Flags = bHasCookError ? EPackageStoreEntryFlags::HasCookError : EPackageStoreEntryFlags::None;
	return Entry;
}

thread_local int32 FPackageStore::ThreadReadCount = 0;

FPackageStoreReadScope::FPackageStoreReadScope(FPackageStore& InPackageStore)
	: PackageStore(InPackageStore)
{
	if (!PackageStore.ThreadReadCount)
	{
		for (const FPackageStore::FBackendAndPriority& Backend : PackageStore.Backends)
		{
			Backend.Value->BeginRead();
		}
	}
	++PackageStore.ThreadReadCount;
}

FPackageStoreReadScope::~FPackageStoreReadScope()
{
	check(PackageStore.ThreadReadCount > 0);
	if (--PackageStore.ThreadReadCount == 0)
	{
		for (const FPackageStore::FBackendAndPriority& Backend : PackageStore.Backends)
		{
			Backend.Value->EndRead();
		}
	}
}

FPackageStore::FPackageStore()
	: BackendContext(MakeShared<FPackageStoreBackendContext>())
{
	auto SetupLinkerLoadBackend = [this]()
	{
		// if loose file loading is enabled, make the package store that manages looking on disk for loose packages (cooked or uncooked)
		if (FAsyncLoadingThreadSettings::Get().bLooseFileLoadingEnabled)
		{
			LinkerLoadBackend = MakeShared<FLinkerLoadPackageStoreBackend>();
			if (LinkerLoadBackend)
			{
				bool bLooseTakesPrecedence = false;
				// non-hybrid editors prefer loose over cooked, but editor -game would prefer cooked, if there was a way for
				// it to load cooked data without hybred
				if (GIsEditor)
				{
					bLooseTakesPrecedence = true;
				}
				// since most package stores are prio 0, we use -1 or 1 to put the loose store above or below the default,
				// depending on which order we prefer
				Mount(LinkerLoadBackend.ToSharedRef(), bLooseTakesPrecedence ? 1 : -1);
			}
		}
	};

	if (GConfig && GConfig->IsReadyForUse())
	{
		SetupLinkerLoadBackend();
	}
	else
	{
		// delay setting up the loose loader untyil GConfig exists
		FCoreDelegates::TSConfigReadyForUse().AddLambda(SetupLinkerLoadBackend);
	}
}

FPackageStore& FPackageStore::Get()
{
	static FPackageStore Instance;
	return Instance;
}


void FPackageStore::Mount(TSharedRef<IPackageStoreBackend> Backend, int32 Priority)
{
	check(IsInGameThread());

#if WITH_EDITOR
	// delay adding the hybrid backend until we get a second package store to wrap along with
	// the loose package store
	if (IsRunningHybridCookedEditor())
	{
		if (!HybridBackend)
		{
			// the order of mounting could be LinkerLoad,Zen or Zen,LinkerLoad, so check all combos
			TSharedPtr<IPackageStoreBackend> LinkerLoad = LinkerLoadBackend;
			if (!LinkerLoad && !!(Backend->GetSupportedLoaders() & EPackageLoader::LinkerLoad))
			{
				LinkerLoad = Backend;
			}
			TSharedPtr<IPackageStoreBackend> Zen;
			if (!!(Backend->GetSupportedLoaders() & EPackageLoader::Zen))
			{
				Zen = Backend;
			}
			if (!Zen && Backends.Num() > 0 && !!(Backends[0].Value->GetSupportedLoaders() & EPackageLoader::Zen))
			{
				Zen = Backends[0].Value;
			}

			if (LinkerLoad.IsValid() && Zen.IsValid())
			{
				// remove the loose backend
				Backends.RemoveAll([LinkerLoad, Zen](const FBackendAndPriority& Pair) { return Pair.Value == LinkerLoad || Pair.Value == Zen; });

				// make a new hybrid backend which wraps the existing loose and the new backend 
				HybridBackend = MakeShared<FHybridPackageStoreBackend>(LinkerLoad, Zen);
				Mount(HybridBackend.ToSharedRef(), Priority);

				// call callbacks 
				Backend->OnMounted(BackendContext);
				return;
			}
		}
	}
#endif

	int32 Index = Algo::LowerBoundBy(Backends, Priority, &FBackendAndPriority::Key, TGreater<>());
	Backends.Insert(MakeTuple(Priority, Backend), Index);
	Backend->OnMounted(BackendContext);
}

EPackageStoreEntryStatus FPackageStore::GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry)
{
	check(ThreadReadCount);
	for (const FBackendAndPriority& Backend : Backends)
	{
		EPackageStoreEntryStatus Status = Backend.Value->GetPackageStoreEntry(PackageId, PackageName, OutPackageStoreEntry);
		const bool bContinueSearch = Status == EPackageStoreEntryStatus::None || Status == EPackageStoreEntryStatus::Missing;

		if (!bContinueSearch)
		{
			checkSlow((Backend.Value->GetSupportedLoaders() & OutPackageStoreEntry.LoaderType));
			return Status;
		}
	}

	return EPackageStoreEntryStatus::Missing;
}

bool FPackageStore::GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId)
{
	check(ThreadReadCount);
	for (const FBackendAndPriority& Backend : Backends)
	{
		if (Backend.Value->GetPackageRedirectInfo(PackageId, OutSourcePackageName, OutRedirectedToPackageId))
		{
			return true;
		}
	}
	return false;
}

TConstArrayView<uint32> FPackageStore::GetSoftReferences(FPackageId PackageId, TConstArrayView<FPackageId>& OutPackageIds)
{
	check(ThreadReadCount);
	for (const FBackendAndPriority& Backend : Backends)
	{
		if (TConstArrayView<uint32> SoftRefs = Backend.Value->GetSoftReferences(PackageId, OutPackageIds); !SoftRefs.IsEmpty())
		{
			return SoftRefs;
		}
	}
	return TConstArrayView<uint32>();
}

FPackageStoreBackendContext::FPendingEntriesAddedEvent& FPackageStore::OnPendingEntriesAdded()
{
	return BackendContext->PendingEntriesAdded;
}

bool FPackageStore::HasAnyBackendsMounted() const
{
	return !Backends.IsEmpty();
}

void FPackageStore::IterateBackends(FVisitPackageStoreBackend&& Visit)
{
	check(ThreadReadCount);
	check(Visit != nullptr);

	for (const FBackendAndPriority& Backend : Backends)
	{
		if (Visit(*Backend.Value) == false)
		{
			break;
		}
	}
}
