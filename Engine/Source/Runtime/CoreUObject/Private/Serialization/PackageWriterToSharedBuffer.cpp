// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/PackageWriterToSharedBuffer.h"

#include "Misc/ScopeRWLock.h"
#include "Serialization/LargeMemoryWriter.h"

DEFINE_LOG_CATEGORY_STATIC(LogPackageWriter, Display, All);

static FSharedBuffer IoBufferToSharedBuffer(const FIoBuffer& InBuffer)
{
	InBuffer.EnsureOwned();
	const uint64 DataSize = InBuffer.DataSize();
	FIoBuffer MutableBuffer(InBuffer);
	uint8* DataPtr = MutableBuffer.Release().ValueOrDie();
	return FSharedBuffer::TakeOwnership(DataPtr, DataSize, FMemory::Free);
};

void FPackageWriterRecords::BeginPackage(FPackage* Record, const IPackageWriter::FBeginPackageInfo& Info)
{
	checkf(Record, TEXT("TPackageWriterToSharedBuffer ConstructRecord must return non-null"));
	{
		FWriteScopeLock MapScopeLock(MapLock);
		TUniquePtr<FPackage>& Existing = Map.FindOrAdd(Info.PackageName);
		checkf(!Existing, TEXT("IPackageWriter->BeginPackage must not be called twice without calling CommitPackage."));
		Existing.Reset(Record);
	}
	Record->Begin = Info;
}

void FPackageWriterRecords::WritePackageData(const IPackageWriter::FPackageInfo& Info,
	FLargeMemoryWriter& ExportsArchive, const TArray<FFileRegion>& FileRegions)
{
	FPackage& Record = FindRecordChecked(Info.PackageName);
	int64 DataSize = ExportsArchive.TotalSize();
	checkf(DataSize > 0, TEXT("IPackageWriter->WritePackageData must not be called with an empty ExportsArchive"));
	checkf(static_cast<uint64>(DataSize) >= Info.HeaderSize,
		TEXT("IPackageWriter->WritePackageData must not be called with HeaderSize > ExportsArchive.TotalSize"));
	if (Record.Packages.FindByPredicate(
		[&Info](const FWritePackage& Existing) { return Existing.Info.MultiOutputIndex == Info.MultiOutputIndex; }))
	{
		UE_LOGF(LogPackageWriter, Error,
			"PackageWriter->WritePackageData called more than once for package (Package: %ls). Ignoring second value.",
			*Info.PackageName.ToString());
		return;
	}

	FSharedBuffer Buffer = FSharedBuffer::TakeOwnership(ExportsArchive.ReleaseOwnership(), DataSize,
		FMemory::Free);
	Record.Packages.Insert(FWritePackage{ Info, MoveTemp(Buffer), FileRegions }, Info.MultiOutputIndex);
}

void FPackageWriterRecords::WriteBulkData(const IPackageWriter::FBulkDataInfo& Info, const FIoBuffer& BulkData,
	const TArray<FFileRegion>& FileRegions)
{
	FPackage& Record = FindRecordChecked(Info.PackageName);
	Record.BulkDatas.Add(FBulkData{ Info, IoBufferToSharedBuffer(BulkData), FileRegions });
}

void FPackageWriterRecords::WriteAdditionalFile(const IPackageWriter::FAdditionalFileInfo& Info,
	const FIoBuffer& FileData)
{
	FPackage& Record = FindRecordChecked(Info.PackageName);
	FAdditionalFile& AdditionalFile = Record.AdditionalFiles.Add_GetRef(FAdditionalFile{ Info, IoBufferToSharedBuffer(FileData) });
	FIoChunkId ChunkId = CreateExternalFileChunkId(AdditionalFile.Info.Filename);
	if (AdditionalFile.Info.ChunkId.IsValid() && AdditionalFile.Info.ChunkId != ChunkId)
	{
		UE_LOGF(LogPackageWriter, Warning, "PackageWriter->WriteAdditionalFile called with an unexpected chunkid: Should match CreateExternalFileChunkId (Package: %ls Filename: %ls)", *Info.PackageName.ToString(), *AdditionalFile.Info.Filename);
	}
	else
	{
		AdditionalFile.Info.ChunkId = ChunkId;
	}
}

void FPackageWriterRecords::WriteLinkerAdditionalData(const IPackageWriter::FLinkerAdditionalDataInfo& Info,
	const FIoBuffer& Data, const TArray<FFileRegion>& FileRegions)
{
	FPackage& Record = FindRecordChecked(Info.PackageName);
	Record.LinkerAdditionalDatas.Add(
		FLinkerAdditionalData{ Info, IoBufferToSharedBuffer(Data), FileRegions });
}

void FPackageWriterRecords::WritePackageTrailer(const IPackageWriter::FPackageTrailerInfo& Info, const FIoBuffer& Data)
{
	FPackage& Record = FindRecordChecked(Info.PackageName);
	if (Record.PackageTrailers.FindByPredicate([&Info](const FPackageTrailer& Existing) { return Existing.Info.MultiOutputIndex == Info.MultiOutputIndex; }))
	{
		UE_LOGF(LogPackageWriter, Error,
			"PackageWriter->WritePackageTrailer called more than once for package (Package: %ls). Ignoring second value.",
			*Info.PackageName.ToString());
		return;
	}

	Record.PackageTrailers.Emplace(FPackageTrailer { Info, IoBufferToSharedBuffer(Data) });
}

FPackageWriterRecords::FPackage& FPackageWriterRecords::FindRecordChecked(FName InPackageName) const
{
	const TUniquePtr<FPackage>* Record;
	{
		FReadScopeLock MapScopeLock(MapLock);
		Record = Map.Find(InPackageName);
	}
	checkf(Record && *Record,
		TEXT("IPackageWriter->BeginPackage must be called before any other functions on IPackageWriter"));
	return **Record;
}

TUniquePtr<FPackageWriterRecords::FPackage> FPackageWriterRecords::FindAndRemoveRecordChecked(FName InPackageName)
{
	TUniquePtr<FPackage> Record;
	{
		FWriteScopeLock MapScopeLock(MapLock);
		Map.RemoveAndCopyValue(InPackageName, Record);
	}
	checkf(Record, TEXT("IPackageWriter->BeginPackage must be called before any other functions on IPackageWriter"));
	return Record;
}

void FPackageWriterRecords::ValidateCommit(FPackage& Record, const IPackageWriter::FCommitPackageInfo& Info) const
{
	if (EnumHasAnyFlags(Info.WriteOptions, IPackageWriter::EWriteOptions::Write | IPackageWriter::EWriteOptions::ComputeHash))
	{
		checkf(Info.Status != IPackageWriter::ECommitStatus::Success || Record.Packages.Num() > 0,
			TEXT("IPackageWriter->WritePackageData must be called before Commit if the Package save was successful."));
		checkf(Info.Status != IPackageWriter::ECommitStatus::Success || Record.Packages.FindByPredicate([](const FPackageWriterRecords::FWritePackage& Package) { return Package.Info.MultiOutputIndex == 0; }),
			TEXT("SavePackage must provide output 0 when saving multioutput packages."));
		uint8 HasBulkDataType[IPackageWriter::FBulkDataInfo::NumTypes]{};
		for (FBulkData& BulkRecord : Record.BulkDatas)
		{
			HasBulkDataType[(int32)BulkRecord.Info.BulkDataType] |= 1 << BulkRecord.Info.MultiOutputIndex;
		}
	}
}
