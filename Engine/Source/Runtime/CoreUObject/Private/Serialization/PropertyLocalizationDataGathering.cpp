// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/PropertyLocalizationDataGathering.h"
#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectHash.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/TextProperty.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/PropertyOptional.h"
#include "UObject/SparseClassDataUtils.h"
#include "HAL/UnrealMemory.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Serialization/NullArchive.h"

FPropertyLocalizationDataGatherer::FPropertyLocalizationDataGatherer(TArray<FGatherableTextData>& InOutGatherableTextDataArray, const UPackage* const InPackage, EPropertyLocalizationGathererResultFlags& OutResultFlags)
	: GatherableTextDataArray(InOutGatherableTextDataArray)
	, Package(InPackage)
#if USE_STABLE_LOCALIZATION_KEYS
	, PackageNamespace(TextNamespaceUtil::GetPackageNamespace(InPackage))
#endif	// USE_STABLE_LOCALIZATION_KEYS
	, ResultFlags(OutResultFlags)
	, AllObjectsInPackage()
{
	// Build up the list of objects that are within our package - we won't follow object references to things outside of our package
	ForEachObjectWithPackage(Package, [this](UObject* Object)
	{
		AllObjectsInPackage.Add(Object);
		return true;
	}, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);

	// Iterate over each root object in the package
	// Note: This calls GetObjectsWithPackage rather than test the AllObjectsInPackage array, as external actors don't pass a "Object->GetOuter() == Package" test when trying to query for root objects
	{
		TArray<UObject*> RootObjectsInPackage;
		GetObjectsWithPackage(Package, RootObjectsInPackage, EGetObjectsFlags::None, RF_Transient, EInternalObjectFlags::Garbage);

		for (const UObject* Object : RootObjectsInPackage)
		{
			GatherLocalizationDataFromObjectWithCallbacks(Object, EPropertyLocalizationGathererTextFlags::None);
		}
	}

	// Iterate any bytecode containing objects
	for (const FObjectAndGatherFlags& BytecodeToGather : BytecodePendingGather)
	{
		if (!EnumHasAnyFlags(BytecodeToGather.GatherTextFlags, EPropertyLocalizationGathererTextFlags::SkipScriptData))
		{
			const UStruct* Struct = CastChecked<UStruct>(BytecodeToGather.Object);
			GatherScriptBytecode(Struct->GetPathName(), Struct->Script, !!(BytecodeToGather.GatherTextFlags & EPropertyLocalizationGathererTextFlags::ForceEditorOnlyScriptData));
		}
	}
	BytecodePendingGather.Reset();
}

bool FPropertyLocalizationDataGatherer::ShouldProcessObject(const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags) const
{
	if (Object->HasAnyFlags(RF_Transient))
	{
		// Transient objects aren't saved, so skip them as part of the gather
		return false;
	}

	// Skip objects that we've already processed to avoid repeated work and cyclic chains
	const bool bAlreadyProcessed = ProcessedObjects.Contains(FObjectAndGatherFlags(Object, GatherTextFlags));
	return !bAlreadyProcessed;
}

void FPropertyLocalizationDataGatherer::MarkObjectProcessed(const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	ProcessedObjects.Add(FObjectAndGatherFlags(Object, GatherTextFlags));
}

const FPropertyLocalizationDataGatherer::FGatherableFieldsForType& FPropertyLocalizationDataGatherer::GetGatherableFieldsForType(const UStruct* InType)
{
	if (const TUniquePtr<FGatherableFieldsForType>* GatherableFieldsForTypePtr = GatherableFieldsForTypes.Find(InType))
	{
		// Already cached
		check(GatherableFieldsForTypePtr->IsValid());
		return *GatherableFieldsForTypePtr->Get();
	}

	// Not cached - work out the gatherable fields for this type and cache the result
	// Note: This will also cache the result for the sub-structs within this type
	return CacheGatherableFieldsForType(InType);
}

const FPropertyLocalizationDataGatherer::FGatherableFieldsForType& FPropertyLocalizationDataGatherer::CacheGatherableFieldsForType(const UStruct* InType)
{
	check(InType);

	TUniquePtr<FGatherableFieldsForType> GatherableFieldsForType = MakeUnique<FGatherableFieldsForType>();

	// Include the parent fields (this will recursively cache any parent types)
	if (const UStruct* SuperType = InType->GetSuperStruct())
	{
		const FGatherableFieldsForType& GatherableFieldsForSuperType = GetGatherableFieldsForType(SuperType);
		*GatherableFieldsForType = GatherableFieldsForSuperType;
	}

	// See if we have a custom handler for this type
	if (const UClass* Class = Cast<UClass>(InType))
	{
		if (const FLocalizationDataObjectGatheringCallback* CustomCallback = GetTypeSpecificLocalizationDataObjectGatheringCallbacks().Find(Class))
		{
			GatherableFieldsForType->CustomObjectCallback = CustomCallback;
		}
	}
	if (const UScriptStruct* Struct = Cast<UScriptStruct>(InType))
	{
		if (const FLocalizationDataStructGatheringCallback* CustomCallback = GetTypeSpecificLocalizationDataStructGatheringCallbacks().Find(Struct))
		{
			GatherableFieldsForType->CustomStructCallback = CustomCallback;
		}
	}

	// Look for potential properties
	for (TFieldIterator<const FProperty> FieldIt(InType, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); FieldIt; ++FieldIt)
	{
		auto ProcessInnerProperty = [this, &GatherableFieldsForType](const FProperty* InProp, const FProperty* InTypeProp)
		{
			if (CanGatherFromInnerProperty(InProp))
			{
				checkSlow(!GatherableFieldsForType->Properties.Contains(InTypeProp));
				GatherableFieldsForType->Properties.Add(InTypeProp);
				return true;
			}
			return false;
		};

		const FProperty* PropertyField = *FieldIt;
		if (PropertyField)
		{
			if (!ProcessInnerProperty(PropertyField, PropertyField))
			{
				if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(PropertyField))
				{
					ProcessInnerProperty(ArrayProp->Inner, PropertyField);
				}
				if (const FMapProperty* MapProp = CastField<const FMapProperty>(PropertyField))
				{
					if (!ProcessInnerProperty(MapProp->KeyProp, PropertyField))
					{
						ProcessInnerProperty(MapProp->ValueProp, PropertyField);
					}
				}
				if (const FSetProperty* SetProp = CastField<const FSetProperty>(PropertyField))
				{
					ProcessInnerProperty(SetProp->ElementProp, PropertyField);
				}
				if (const FOptionalProperty* OptionalProp = CastField<const FOptionalProperty>(PropertyField))
				{
					ProcessInnerProperty(OptionalProp->GetValueProperty(), PropertyField);
				}
			}
		}
	}

	// Look for potential functions
	for (TFieldIterator<const UField> FieldIt(InType, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); FieldIt; ++FieldIt)
	{
		const UFunction* FunctionField = Cast<UFunction>(*FieldIt);
		if (FunctionField && FunctionField->Script.Num() > 0 && IsObjectValidForGather(FunctionField))
		{
			checkSlow(!GatherableFieldsForType->Functions.Contains(FunctionField));
			GatherableFieldsForType->Functions.Add(FunctionField);
		}
	}

	check(GatherableFieldsForType.IsValid());
	return *GatherableFieldsForTypes.Add(InType, MoveTemp(GatherableFieldsForType)).Get();
}

bool FPropertyLocalizationDataGatherer::CanGatherFromInnerProperty(const FProperty* InInnerProperty)
{
	if (InInnerProperty->IsA<FTextProperty>() || InInnerProperty->IsA<FObjectPropertyBase>())
	{
		return true;
	}

	if (const FStructProperty* StructInnerProp = CastField<const FStructProperty>(InInnerProperty))
	{
		if (StructInnerProp->Struct)
		{
			// Call the "Get" version as we may have already cached a result for this type
			return !GetGatherableFieldsForType(StructInnerProp->Struct).IsEmpty();
		}
	}

	return false;
}

void FPropertyLocalizationDataGatherer::GatherLocalizationDataFromObjectWithCallbacks(const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	EPropertyLocalizationGathererTextFlags ObjectGatherTextFlags = GatherTextFlags;
	if (Object->GetClass()->GetPackage()->HasAnyPackageFlags(PKG_EditorOnly) || Object->GetClass()->HasAnyClassFlags(CLASS_Optional))
	{
		ObjectGatherTextFlags |= EPropertyLocalizationGathererTextFlags::ForceEditorOnlyProperties;
	}

	const FGatherableFieldsForType& GatherableFieldsForType = GetGatherableFieldsForType(Object->GetClass());
	if (GatherableFieldsForType.CustomObjectCallback)
	{
		checkf(IsObjectValidForGather(Object), TEXT("Cannot gather for objects outside of the current package! Package: '%s'. Object: '%s'."), *Package->GetFullName(), *Object->GetFullName());

		if (ShouldProcessObject(Object, ObjectGatherTextFlags))
		{
			MarkObjectProcessed(Object, ObjectGatherTextFlags);
			(*GatherableFieldsForType.CustomObjectCallback)(Object, *this, ObjectGatherTextFlags);
		}
	}
	else if (ShouldProcessObject(Object, ObjectGatherTextFlags))
	{
		MarkObjectProcessed(Object, ObjectGatherTextFlags);
		GatherLocalizationDataFromObject(Object, ObjectGatherTextFlags);
	}
}

void FPropertyLocalizationDataGatherer::GatherLocalizationDataFromObject(const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	checkf(IsObjectValidForGather(Object), TEXT("Cannot gather for objects outside of the current package! Package: '%s'. Object: '%s'."), *Package->GetFullName(), *Object->GetFullName());

	const FString Path = Object->GetPathName();

	// Gather text from our fields.
	GatherLocalizationDataFromObjectFields(Path, Object, GatherTextFlags);

	// Also gather from the sparse data on UClass types.
	if (const UClass* Class = Cast<UClass>(Object))
	{
		const auto FilterTransientProperties = [](const FProperty* P)
			{
				return !P->HasAnyPropertyFlags(CPF_Transient);
			};
		if (UE::Reflection::DoesSparseClassDataOverrideArchetype(Class, FilterTransientProperties))
		{
			const UScriptStruct* SparseDataStruct = Class->GetSparseClassDataStruct();
			const void* SparseData = const_cast<UClass*>(Class)->GetOrCreateSparseClassData();
			const void* ArchetypeSparseData = Class->GetArchetypeForSparseClassData();
			GatherLocalizationDataFromStructWithCallbacks(Path + TEXT(".SparseClassData"), SparseDataStruct, SparseData, ArchetypeSparseData, GatherTextFlags);
		}
	}

	// Also gather from the script data on UStruct types.
	{
		if (!!(GatherTextFlags & EPropertyLocalizationGathererTextFlags::ForceHasScript))
		{
			ResultFlags |= EPropertyLocalizationGathererResultFlags::HasScript;
		}

		if (const UStruct* Struct = Cast<UStruct>(Object))
		{
			if (Struct->Script.Num() > 0)
			{
				BytecodePendingGather.Add(FObjectAndGatherFlags(Struct, GatherTextFlags));
			}
		}
	}

	// Gather from anything that has us as their outer, as not all objects are reachable via a property pointer.
	if (!(GatherTextFlags & EPropertyLocalizationGathererTextFlags::SkipSubObjects))
	{
		TArray<UObject*> InnerObjects;
		GetObjectsWithOuter(Object, InnerObjects, EGetObjectsFlags::None, RF_Transient, EInternalObjectFlags::Garbage);
		for (UObject* ChildObject : InnerObjects)
		{
			// if the child object as a package set, do not gather from it
			if (!ChildObject->GetExternalPackage())
			{
				GatherLocalizationDataFromObjectWithCallbacks(ChildObject, GatherTextFlags);
			}
		}
	}
}

void FPropertyLocalizationDataGatherer::GatherLocalizationDataFromObjectFields(const FString& PathToParent, const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	const UObject* ArchetypeObject = Object->GetArchetype();
	const FGatherableFieldsForType& GatherableFieldsForType = GetGatherableFieldsForType(Object->GetClass());

	// Gather text from the property data
	for (const FProperty* PropertyField : GatherableFieldsForType.Properties)
	{
		if (!Object->IsA(PropertyField->GetOwner<UClass>()))
		{
			continue;
		}
		const void* ValueAddress = PropertyField->ContainerPtrToValuePtr<void>(Object);
		const void* DefaultValueAddress = (ArchetypeObject && ArchetypeObject->IsA(PropertyField->GetOwnerClass())) ? PropertyField->ContainerPtrToValuePtr<void>(ArchetypeObject) : nullptr;
		GatherLocalizationDataFromChildTextProperties(PathToParent, PropertyField, ValueAddress, DefaultValueAddress, GatherTextFlags | (IsEditorOnlyProperty(PropertyField) ? EPropertyLocalizationGathererTextFlags::ForceEditorOnly : EPropertyLocalizationGathererTextFlags::None));
	}

	// Gather text from the script bytecode
	for (const UFunction* FunctionField : GatherableFieldsForType.Functions)
	{
		if (ShouldProcessObject(FunctionField, GatherTextFlags))
		{
			MarkObjectProcessed(FunctionField, GatherTextFlags);
			GatherLocalizationDataFromObject(FunctionField, GatherTextFlags);
		}
	}
}

void FPropertyLocalizationDataGatherer::GatherLocalizationDataFromStructWithCallbacks(const FString& PathToParent, const UScriptStruct* Struct, const void* StructData, const void* DefaultStructData, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	const FGatherableFieldsForType& GatherableFieldsForType = GetGatherableFieldsForType(Struct);
	if (GatherableFieldsForType.CustomStructCallback)
	{
		(*GatherableFieldsForType.CustomStructCallback)(PathToParent, Struct, StructData, DefaultStructData, *this, GatherTextFlags);
	}
	else
	{
		GatherLocalizationDataFromStruct(PathToParent, Struct, StructData, DefaultStructData, GatherTextFlags);
	}
}

void FPropertyLocalizationDataGatherer::GatherLocalizationDataFromStruct(const FString& PathToParent, const UScriptStruct* Struct, const void* StructData, const void* DefaultStructData, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	GatherLocalizationDataFromStructFields(PathToParent, Struct, StructData, DefaultStructData, GatherTextFlags);
}

void FPropertyLocalizationDataGatherer::GatherLocalizationDataFromStructFields(const FString& PathToParent, const UStruct* Struct, const void* StructData, const void* DefaultStructData, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	const FGatherableFieldsForType& GatherableFieldsForType = GetGatherableFieldsForType(Struct);

	// Gather text from the property data
	for (const FProperty* PropertyField : GatherableFieldsForType.Properties)
	{
		const void* ValueAddress = PropertyField->ContainerPtrToValuePtr<void>(StructData);
		const void* DefaultValueAddress = DefaultStructData ? PropertyField->ContainerPtrToValuePtr<void>(DefaultStructData) : nullptr;
		GatherLocalizationDataFromChildTextProperties(PathToParent, PropertyField, ValueAddress, DefaultValueAddress, GatherTextFlags | (IsEditorOnlyProperty(PropertyField) ? EPropertyLocalizationGathererTextFlags::ForceEditorOnly : EPropertyLocalizationGathererTextFlags::None));
	}

	// Gather text from the script bytecode
	for (const UFunction* FunctionField : GatherableFieldsForType.Functions)
	{
		if (ShouldProcessObject(FunctionField, GatherTextFlags))
		{
			MarkObjectProcessed(FunctionField, GatherTextFlags);
			GatherLocalizationDataFromObject(FunctionField, GatherTextFlags);
		}
	}
}

void FPropertyLocalizationDataGatherer::GatherLocalizationDataFromChildTextProperties(const FString& PathToParent, const FProperty* const Property, const void* const ValueAddress, const void* const DefaultValueAddress, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
{
	if (Property->HasAnyPropertyFlags(CPF_Transient))
	{
		// Transient properties aren't saved, so skip them as part of the gather
		return;
	}

	// If adding more type support here, also update CacheGatherableFieldsForType
	const FTextProperty* const TextProperty = CastField<const FTextProperty>(Property);
	const FArrayProperty* const ArrayProperty = CastField<const FArrayProperty>(Property);
	const FMapProperty* const MapProperty = CastField<const FMapProperty>(Property);
	const FSetProperty* const SetProperty = CastField<const FSetProperty>(Property);
	const FStructProperty* const StructProperty = CastField<const FStructProperty>(Property);
	const FObjectPropertyBase* const ObjectProperty = CastField<const FObjectPropertyBase>(Property);
	const FOptionalProperty* const OptionalProperty = CastField<const FOptionalProperty>(Property);

	const EPropertyLocalizationGathererTextFlags FixedChildPropertyGatherTextFlags = GatherTextFlags | (IsEditorOnlyProperty(Property) ? EPropertyLocalizationGathererTextFlags::ForceEditorOnly : EPropertyLocalizationGathererTextFlags::None);

	FString PathToElementRoot;
	{
		const FString PropertyNameStr = Property->GetName();

		PathToElementRoot.Reserve(PathToParent.Len() + PropertyNameStr.Len() + 1);
		PathToElementRoot += PathToParent;
		if (!PathToParent.IsEmpty())
		{
			PathToElementRoot += TEXT('.');
		}
		PathToElementRoot += PropertyNameStr;
	}

	// Handles both native, fixed-size arrays and plain old non-array properties.
	const bool IsFixedSizeArray = Property->ArrayDim > 1;
	for(int32 i = 0; i < Property->ArrayDim; ++i)
	{
		FString PathToFixedSizeArrayElement;
		if (IsFixedSizeArray)
		{
			PathToFixedSizeArrayElement.Reserve(PathToElementRoot.Len() + 10); // +10 for some slack for the number and braces
			PathToFixedSizeArrayElement += PathToElementRoot;
			PathToFixedSizeArrayElement += TEXT('[');
			PathToFixedSizeArrayElement.AppendInt(i);
			PathToFixedSizeArrayElement += TEXT(']');
		}

		const FString& PathToElement = IsFixedSizeArray ? PathToFixedSizeArrayElement : PathToElementRoot;
		const void* const ElementValueAddress = reinterpret_cast<const uint8*>(ValueAddress) + Property->GetElementSize() * i;
		const void* const DefaultElementValueAddress = DefaultValueAddress ? (reinterpret_cast<const uint8*>(DefaultValueAddress) + Property->GetElementSize() * i) : nullptr;

		EPropertyLocalizationGathererTextFlags ElementChildPropertyGatherTextFlags = FixedChildPropertyGatherTextFlags;
		if (!EnumHasAnyFlags(ElementChildPropertyGatherTextFlags, EPropertyLocalizationGathererTextFlags::ForceIsDefaultValue))
		{
			const bool bIsDefaultValue = DefaultElementValueAddress && Property->Identical(ElementValueAddress, DefaultElementValueAddress, PPF_None);
			if (bIsDefaultValue)
			{
				ElementChildPropertyGatherTextFlags |= EPropertyLocalizationGathererTextFlags::ForceIsDefaultValue;
			}
		}

		// Property is a text property.
		if (TextProperty)
		{
			const FText* const TextElementValueAddress = static_cast<const FText*>(ElementValueAddress);

			const bool bIsDefaultValue = EnumHasAnyFlags(ElementChildPropertyGatherTextFlags, EPropertyLocalizationGathererTextFlags::ForceIsDefaultValue);
			if (bIsDefaultValue)
			{
				MarkDefaultTextInstance(*TextElementValueAddress);
			}
			else if (!EnumHasAnyFlags(GatherTextFlags, EPropertyLocalizationGathererTextFlags::SkipProperties))
			{
				UPackage* const PropertyPackage = TextProperty->GetOutermost();
				if (FTextInspector::GetFlags(*TextElementValueAddress) & ETextFlag::ConvertedProperty)
				{
					PropertyPackage->MarkPackageDirty();
				}

				GatherTextInstance(*TextElementValueAddress, PathToElement, !!(GatherTextFlags & EPropertyLocalizationGathererTextFlags::ForceEditorOnlyProperties) || IsEditorOnlyProperty(TextProperty));
			}
		}
		// Property is a DYNAMIC array property.
		else if (ArrayProperty)
		{
			// Iterate over all elements of the array.
			FScriptArrayHelper ScriptArrayHelper(ArrayProperty, ElementValueAddress);
			const int32 ElementCount = ScriptArrayHelper.Num();
			for(int32 j = 0; j < ElementCount; ++j)
			{
				FString PathToInnerElement;
				{
					PathToInnerElement.Reserve(PathToElement.Len() + 10); // +10 for some slack for the number and braces
					PathToInnerElement += PathToElement;
					PathToInnerElement += TEXT('(');
					PathToInnerElement.AppendInt(j);
					PathToInnerElement += TEXT(')');
				}

				const uint8* ElementPtr = ScriptArrayHelper.GetRawPtr(j);
				GatherLocalizationDataFromChildTextProperties(PathToInnerElement, ArrayProperty->Inner, ElementPtr, nullptr, ElementChildPropertyGatherTextFlags);
			}
		}
		// Property is a map property.
		else if (MapProperty)
		{
			const bool bGatherMapKey = CanGatherFromInnerProperty(MapProperty->KeyProp);
			const bool bGatherMapValue = CanGatherFromInnerProperty(MapProperty->ValueProp);

			// Iterate over all elements of the map.
			FScriptMapHelper ScriptMapHelper(MapProperty, ElementValueAddress);
			for (FScriptMapHelper::FIterator It(ScriptMapHelper); It; ++It)
			{
				const uint8* MapPairPtr = ScriptMapHelper.GetPairPtr(It);

				if (bGatherMapKey)
				{
					FString PathToInnerElement;
					{
						PathToInnerElement.Reserve(PathToElement.Len() + 20); // +20 for some slack for the number, braces, and description
						PathToInnerElement += PathToElement;
						PathToInnerElement += TEXT('(');
						PathToInnerElement.AppendInt(It.GetLogicalIndex());
						PathToInnerElement += TEXT(" - Key)");
					}

					const uint8* MapKeyPtr = MapPairPtr;
					GatherLocalizationDataFromChildTextProperties(PathToInnerElement, MapProperty->KeyProp, MapKeyPtr, nullptr, ElementChildPropertyGatherTextFlags);
				}

				if (bGatherMapValue)
				{
					FString PathToInnerElement;
					{
						PathToInnerElement.Reserve(PathToElement.Len() + 20); // +20 for some slack for the number, braces, and description
						PathToInnerElement += PathToElement;
						PathToInnerElement += TEXT('(');
						PathToInnerElement.AppendInt(It.GetLogicalIndex());
						PathToInnerElement += TEXT(" - Value)");
					}

					const uint8* MapValuePtr = MapPairPtr + MapProperty->MapLayout.ValueOffset;
					GatherLocalizationDataFromChildTextProperties(PathToInnerElement, MapProperty->ValueProp, MapValuePtr, nullptr, ElementChildPropertyGatherTextFlags);
				}
			}
		}
		// Property is a set property.
		else if (SetProperty)
		{
			// Iterate over all elements of the Set.
			FScriptSetHelper ScriptSetHelper(SetProperty, ElementValueAddress);
			for (FScriptSetHelper::FIterator It(ScriptSetHelper); It; ++It)
			{
				FString PathToInnerElement;
				{
					PathToInnerElement.Reserve(PathToElement.Len() + 10); // +10 for some slack for the number and braces
					PathToInnerElement += PathToElement;
					PathToInnerElement += TEXT('(');
					PathToInnerElement.AppendInt(It.GetLogicalIndex());
					PathToInnerElement += TEXT(')');
				}

				const uint8* ElementPtr = ScriptSetHelper.GetElementPtr(It);
				GatherLocalizationDataFromChildTextProperties(PathToInnerElement, SetProperty->ElementProp, ElementPtr, nullptr, ElementChildPropertyGatherTextFlags);
			}
		}
		// Property is a struct property.
		else if (StructProperty)
		{
			if (StructProperty->Struct)
			{
				GatherLocalizationDataFromStructWithCallbacks(PathToElement, StructProperty->Struct, ElementValueAddress, DefaultElementValueAddress, ElementChildPropertyGatherTextFlags);
			}
		}
		// Property is an object property.
		else if (ObjectProperty && !(GatherTextFlags & EPropertyLocalizationGathererTextFlags::SkipSubObjects))
		{
			// if the object is a FObjectPtr it might not be resolved
			// if unresolved there is no need to resolve it as IsObjectValidForGather would return false anyways
			const UObject* InnerObject = nullptr;
			if (auto ObjectPropertyPtr = CastField<FObjectProperty>(ObjectProperty))
			{
				const TObjectPtr<UObject>& ObjectPtr = ObjectPropertyPtr->GetPropertyValue(ElementValueAddress);
				if (ObjectPtr.IsResolved())
				{
					InnerObject = ObjectPtr.Get();
				}
			}
			else
			{
				InnerObject = ObjectProperty->GetObjectPropertyValue(ElementValueAddress);
			} 
			if (InnerObject && IsObjectValidForGather(InnerObject))
			{
				GatherLocalizationDataFromObjectWithCallbacks(InnerObject, FixedChildPropertyGatherTextFlags);
			}
		}
		else if (OptionalProperty)
		{
			if (const void* OptionalValueAddress = OptionalProperty->GetValuePointerForReadIfSet(ElementValueAddress))
			{
				GatherLocalizationDataFromChildTextProperties(
					FString::Printf(TEXT("%s?"), *PathToElement),
					OptionalProperty->GetValueProperty(),
					OptionalValueAddress,
					nullptr,
					ElementChildPropertyGatherTextFlags);
			}
		}
	}
}

void FPropertyLocalizationDataGatherer::GatherTextInstance(const FText& Text, const FString& Description, const bool bIsEditorOnly)
{
	auto AddGatheredText = [this, &Description](const FString& InNamespace, const FString& InKey, const FTextSourceData& InSourceData, const bool InIsEditorOnly, const FString& InDevNotes)
	{
		FGatherableTextData* GatherableTextData = GatherableTextDataArray.FindByPredicate([&](const FGatherableTextData& Candidate)
		{
			return Candidate.NamespaceName.Equals(InNamespace, ESearchCase::CaseSensitive)
				&& Candidate.SourceData.SourceString.Equals(InSourceData.SourceString, ESearchCase::CaseSensitive)
				&& Candidate.SourceData.SourceStringMetaData == InSourceData.SourceStringMetaData;
		});
		if (!GatherableTextData)
		{
			GatherableTextData = &GatherableTextDataArray[GatherableTextDataArray.AddDefaulted()];
			GatherableTextData->NamespaceName = InNamespace;
			GatherableTextData->SourceData = InSourceData;
		}

		// We might attempt to add the same text multiple times if we process the same object with slightly different flags - only add this source site once though.
		{
			static const FLocMetadataObject DefaultMetadataObject;
			const bool bFoundSourceSiteContext = GatherableTextData->SourceSiteContexts.ContainsByPredicate([&](const FTextSourceSiteContext& InSourceSiteContext) -> bool
			{
				return InSourceSiteContext.KeyName.Equals(InKey, ESearchCase::CaseSensitive)
					&& InSourceSiteContext.SiteDescription.Equals(Description, ESearchCase::CaseSensitive)
					&& InSourceSiteContext.IsEditorOnly == InIsEditorOnly
					&& InSourceSiteContext.IsOptional == false
					&& InSourceSiteContext.InfoMetaData == DefaultMetadataObject
					&& InSourceSiteContext.KeyMetaData == DefaultMetadataObject;
			});

			if (!bFoundSourceSiteContext)
			{
				FTextSourceSiteContext& SourceSiteContext = GatherableTextData->SourceSiteContexts[GatherableTextData->SourceSiteContexts.AddDefaulted()];
				SourceSiteContext.KeyName = InKey;
				SourceSiteContext.SiteDescription = Description;
				SourceSiteContext.IsEditorOnly = InIsEditorOnly;
				SourceSiteContext.IsOptional = false;
#if WITH_EDITORONLY_DATA
				SourceSiteContext.DevNotes = InDevNotes;
#endif
			}
		}
	};

	if (Text.IsInitializedFromString())
	{
		// We early out as a text initialized from a string does not have a namespace or key and thus can't be gathered. into the array
		ResultFlags |= EPropertyLocalizationGathererResultFlags::HasTextInitializedFromString;
		return;
	}

	FString Namespace;
	FString Key;
	if (!ExtractTextIdentity(Text, Namespace, Key, /*bCleanNamespace*/false))
	{
		return;
	}

	ResultFlags |= EPropertyLocalizationGathererResultFlags::HasText;

	FTextSourceData SourceData;
	{
		const FString* SourceString = FTextInspector::GetSourceString(Text);
		SourceData.SourceString = SourceString ? *SourceString : FString();
	}

	// Always include the text without its package localization ID
	const FString CleanNamespace = TextNamespaceUtil::StripPackageNamespace(Namespace);
#if WITH_EDITORONLY_DATA
	AddGatheredText(CleanNamespace, Key, SourceData, bIsEditorOnly, Text.GetDevNotes());
#else
	static const FString EmptyString = TEXT("");
	AddGatheredText(CleanNamespace, Key, SourceData, bIsEditorOnly, EmptyString);
#endif

#if USE_STABLE_LOCALIZATION_KEYS
	// Sanity check that the text we gathered has the expected package localization ID
	{
		const FString TextPackageNamespace = TextNamespaceUtil::ExtractPackageNamespace(Namespace);
		if (!TextPackageNamespace.IsEmpty() && !TextPackageNamespace.Equals(PackageNamespace, ESearchCase::CaseSensitive))
		{
			ResultFlags |= EPropertyLocalizationGathererResultFlags::HasTextWithInvalidPackageLocalizationID;
		}
	}
#endif // USE_STABLE_LOCALIZATION_KEYS
}

struct FGatherTextFromScriptBytecode
{
public:
	FGatherTextFromScriptBytecode(const TCHAR* InSourceDescription, const TArray<uint8>& InScript, FPropertyLocalizationDataGatherer& InPropertyLocalizationDataGatherer, const bool InTreatAsEditorOnlyData)
		: SourceDescription(InSourceDescription)
		, Script(const_cast<TArray<uint8>&>(InScript)) // We won't change the script, but we have to lie so that the code in ScriptSerialization.h will compile :(
		, PropertyLocalizationDataGatherer(InPropertyLocalizationDataGatherer)
		, bTreatAsEditorOnlyData(InTreatAsEditorOnlyData)
		, bIsParsingText(false)
	{
		const int32 ScriptSizeBytes = Script.Num();

		int32 iCode = 0;
		while (iCode < ScriptSizeBytes)
		{
			SerializeExpr(iCode, DummyArchive);
		}
	}

private:
	FLinker* GetLinker()
	{
		return nullptr;
	}

	EExprToken SerializeExpr(int32& iCode, FArchive& Ar)
	{
	#define XFERSTRING()		SerializeString(iCode, Ar)
	#define XFERUNICODESTRING() SerializeUnicodeString(iCode, Ar)
	#define XFERTEXT()			SerializeText(iCode, Ar)

	#define SERIALIZEEXPR_INC
	#define SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
	#include "UObject/ScriptSerialization.inl"
		return Expr;
	#undef SERIALIZEEXPR_INC
	#undef SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
	}

	void SerializeString(int32& iCode, FArchive& Ar)
	{
		if (bIsParsingText)
		{
			LastParsedString.Reset();

			do
			{
				LastParsedString += (char)(Script[iCode]);

				iCode += sizeof(uint8);
			}
			while (Script[iCode-1]);
		}
		else
		{
			do
			{
				iCode += sizeof(uint8);
			}
			while (Script[iCode-1]);
		}
	}

	void SerializeUnicodeString(int32& iCode, FArchive& Ar)
	{
		if (bIsParsingText)
		{
			LastParsedString.Reset();

			do
			{
				uint16 UnicodeChar = FPlatformMemory::ReadUnaligned<uint16>(&Script[iCode]);
				LastParsedString += (TCHAR)UnicodeChar;

				iCode += sizeof(uint16);
			}
			while (Script[iCode-1] || Script[iCode-2]);
		}
		else
		{
			do
			{
				iCode += sizeof(uint16);
			}
			while (Script[iCode-1] || Script[iCode-2]);
		}
	}

	void SerializeText(int32& iCode, FArchive& Ar)
	{
		// What kind of text are we dealing with?
		const EBlueprintTextLiteralType TextLiteralType = (EBlueprintTextLiteralType)Script[iCode++];

		switch (TextLiteralType)
		{
		case EBlueprintTextLiteralType::Empty:
			// Don't need to gather empty text
			break;

		case EBlueprintTextLiteralType::LocalizedText:
		case EBlueprintTextLiteralType::LocalizedTextWithNotes:
			{
				bIsParsingText = true;

				SerializeExpr(iCode, Ar);
				FString SourceString = MoveTemp(LastParsedString);

				SerializeExpr(iCode, Ar);
				const FString TextKey = MoveTemp(LastParsedString);

				SerializeExpr(iCode, Ar);
				const FString TextNamespace = MoveTemp(LastParsedString);
				
				FString DevNotes;
				if (TextLiteralType == EBlueprintTextLiteralType::LocalizedTextWithNotes)
				{
					SerializeExpr(iCode, Ar);
					DevNotes = MoveTemp(LastParsedString);
				}

				bIsParsingText = false;

#if WITH_EDITORONLY_DATA
				const FText TextInstance = FText::AsLocalizable_Advanced(TextNamespace, TextKey, MoveTemp(SourceString), DevNotes);
#else
				const FText TextInstance = FText::AsLocalizable_Advanced(TextNamespace, TextKey, MoveTemp(SourceString));
#endif
				if (!PropertyLocalizationDataGatherer.IsDefaultTextInstance(TextInstance))
				{
					PropertyLocalizationDataGatherer.GatherTextInstance(TextInstance, FString::Printf(TEXT("%s [Script Bytecode]"), SourceDescription), bTreatAsEditorOnlyData);
				}
			}
			break;

		case EBlueprintTextLiteralType::InvariantText:
			// Don't need to gather invariant text, but we do need to walk over the string in the buffer
			SerializeExpr(iCode, Ar);
			break;

		case EBlueprintTextLiteralType::LiteralString:
			// Don't need to gather literal strings, but we do need to walk over the string in the buffer
			SerializeExpr(iCode, Ar);
			break;

		case EBlueprintTextLiteralType::StringTableEntry:
			// Don't need to gather string table entries, but we do need to walk over the strings in the buffer
			iCode += sizeof(ScriptPointerType); // String Table asset (if any)
			SerializeExpr(iCode, Ar);
			SerializeExpr(iCode, Ar);
			break;

		default:
			checkf(false, TEXT("Unknown EBlueprintTextLiteralType! Please update FGatherTextFromScriptBytecode::SerializeText to handle this type of text."));
			break;
		}
	}

	const TCHAR* SourceDescription;
	TArray<uint8>& Script;
	FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer;
	bool bTreatAsEditorOnlyData;

	FNullArchive DummyArchive;
	bool bIsParsingText;
	FString LastParsedString;
};

void FPropertyLocalizationDataGatherer::GatherScriptBytecode(const FString& PathToScript, const TArray<uint8>& ScriptData, const bool bIsEditorOnly)
{
	if (ScriptData.Num() > 0)
	{
		ResultFlags |= EPropertyLocalizationGathererResultFlags::HasScript;
		FGatherTextFromScriptBytecode(*PathToScript, ScriptData, *this, bIsEditorOnly);
	}
}

bool FPropertyLocalizationDataGatherer::IsDefaultTextInstance(const FText& Text) const
{
	FString Namespace;
	FString Key;
	if (ExtractTextIdentity(Text, Namespace, Key, /*bCleanNamespace*/true))
	{
		return DefaultTextInstances.Contains(FTextId(MoveTemp(Namespace), MoveTemp(Key)));
	}
	return false;
}

void FPropertyLocalizationDataGatherer::MarkDefaultTextInstance(const FText& Text)
{
	FString Namespace;
	FString Key;
	if (ExtractTextIdentity(Text, Namespace, Key, /*bCleanNamespace*/true))
	{
		DefaultTextInstances.Add(FTextId(MoveTemp(Namespace), MoveTemp(Key)));
	}
}

bool FPropertyLocalizationDataGatherer::IsEditorOnlyProperty(const FProperty* Property)
{
	if (Property->HasAnyPropertyFlags(CPF_EditorOnly))
	{
		return true;
	}

#if WITH_METADATA
	{
		static const FName NAME_EditorOnlyLocalization = "EditorOnlyLocalization";
		if (Property->HasMetaData(NAME_EditorOnlyLocalization))
		{
			return true;
		}
	}
#endif
	
	return false;
}

bool FPropertyLocalizationDataGatherer::ExtractTextIdentity(const FText& Text, FString& OutNamespace, FString& OutKey, const bool bCleanNamespace)
{
	const FTextId TextId = FTextInspector::GetTextId(Text);
	if (!TextId.IsEmpty() && Text.ShouldGatherForLocalization())
	{
		TextId.GetNamespace().ToString(OutNamespace);
		TextId.GetKey().ToString(OutKey);
		if (bCleanNamespace)
		{
			OutNamespace = TextNamespaceUtil::StripPackageNamespace(OutNamespace);
		}
		return true;
	}
	return false;
}

FPropertyLocalizationDataGatherer::FLocalizationDataObjectGatheringCallbackMap& FPropertyLocalizationDataGatherer::GetTypeSpecificLocalizationDataObjectGatheringCallbacks()
{
	static FLocalizationDataObjectGatheringCallbackMap TypeSpecificLocalizationDataGatheringCallbacks;
	return TypeSpecificLocalizationDataGatheringCallbacks;
}

FPropertyLocalizationDataGatherer::FLocalizationDataStructGatheringCallbackMap& FPropertyLocalizationDataGatherer::GetTypeSpecificLocalizationDataStructGatheringCallbacks()
{
	static FLocalizationDataStructGatheringCallbackMap TypeSpecificLocalizationDataGatheringCallbacks;
	return TypeSpecificLocalizationDataGatheringCallbacks;
}
