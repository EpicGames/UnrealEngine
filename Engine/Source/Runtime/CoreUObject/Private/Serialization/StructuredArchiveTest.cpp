// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

#if WITH_LOW_LEVEL_TESTS && WITH_TEXT_ARCHIVE_SUPPORT

#include "Containers/StringView.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/ObjectAndNameAsStringProxyArchive.h"
#include "Templates/SubclassOf.h"
#include "TestHarness.h"

TEST_CASE("UE::CoreUObject::FStructuredArchive::TSubclassOf", "[Smoke]")
{
	// Get a UClass* as a TSubclassOf, and add it to a map that we can use as an indices map
	TSubclassOf<UObject> Obj = UObject::StaticClass();
	TMap<TObjectPtr<UObject>, FPackageIndex> Map;
	Map.Add(Obj.Get());

	// Create an output array and memory archive for testing
	TArray<uint8> Output;
	FMemoryWriter Ar(Output);

	// Create a proxy archive which converts UObject pointers
	FObjectAndNameAsStringProxyArchive Proxy(Ar, false);

	// Create a JSON formatter to write to the proxy and set the map on it to allow UObjects to be mapped to indices
	FJsonArchiveOutputFormatter Fmt(Proxy);
	Fmt.SetObjectIndicesMap(&Map);

	// Create structured archive and open the root object as a record
	FStructuredArchive StrAr(Fmt);
	FStructuredArchiveSlot Root = StrAr.Open();
	FStructuredArchiveRecord Record = Root.EnterRecord();

	// Serialize the TSubclassOf as a field of the record
	Record << SA_VALUE(TEXT("Class"), Obj);

	// Flush to memory
	StrAr.Close();

	// Test that we wrote out what we expected
	FAnsiStringView Result = ANSITEXTVIEW("{" LINE_TERMINATOR_ANSI "\t\"Class\": \"0\"" LINE_TERMINATOR_ANSI "}");
	CHECK_EQUAL(Output, TArray<uint8>((const uint8*)Result.GetData(), Result.Len()));
}

#endif // WITH_LOW_LEVEL_TESTS && WITH_TEXT_ARCHIVE_SUPPORT
