// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/TextReferenceCollector.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectHash.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

class FTextReferencesArchive : public FArchiveUObject
{
public:
	FTextReferencesArchive(const UPackage* const InPackage, const FTextReferenceCollector::EComparisonMode InComparisonMode, const FString& InTextNamespace, const FString& InTextKey, const FString& InTextSource, int32& OutCount)
		: ComparisonMode(InComparisonMode)
		, NamespaceToMatch(&InTextNamespace)
		, KeyToMatch(&InTextKey)
		, SourceToMatch(&InTextSource)
		, Count(&OutCount)
	{
		this->SetIsSaving(true);
		this->SetIsPersistent(false);
		ArShouldSkipBulkData = true; // Skips bulk data as we can't handle saving that!

		// Build up the list of objects that are within our package - we won't follow object references to things outside of our package
		{
			TArray<UObject*> AllObjectsInPackageArray;
			GetObjectsWithOuter(InPackage, AllObjectsInPackageArray, EGetObjectsFlags::IncludeNestedObjects, RF_Transient, EInternalObjectFlags::Garbage);

			AllObjectsInPackage.Reserve(AllObjectsInPackageArray.Num());
			for (UObject* Object : AllObjectsInPackageArray)
			{
				AllObjectsInPackage.Add(Object);
			}
		}

		TArray<UObject*> RootObjectsInPackage;
		GetObjectsWithOuter(InPackage, RootObjectsInPackage, EGetObjectsFlags::None, RF_Transient, EInternalObjectFlags::Garbage);

		// Iterate over each root object in the package
		for (UObject* Obj : RootObjectsInPackage)
		{
			ProcessObject(Obj);
		}
	}

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override
	{
		return FArchiveUObject::ShouldSkipProperty(InProperty)
			|| (InProperty->HasAnyPropertyFlags(CPF_Transient) && !IsSerializingDefaults());
	}

	void ProcessObject(UObject* Obj)
	{
		if (!Obj || !AllObjectsInPackage.Contains(Obj) || ProcessedObjects.Contains(Obj))
		{
			return;
		}

		ProcessedObjects.Add(Obj);

		// See if we have a custom handler for this type
		FTextReferenceCollector::FTextReferenceCollectorCallback* CustomCallback = nullptr;
		for (const UClass* Class = Obj->GetClass(); Class != nullptr; Class = Class->GetSuperClass())
		{
			CustomCallback = FTextReferenceCollector::GetTypeSpecificTextReferenceCollectorCallbacks().Find(Class);
			if (CustomCallback)
			{
				break;
			}
		}

		if (CustomCallback)
		{
			(*CustomCallback)(Obj, *this);
		}
		else
		{
			Obj->Serialize(*this);
		}
	}

	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		ProcessObject(Obj);
		return *this;
	}

	virtual FArchive& operator<<(FText& Value) override
	{
		const FString TextNamespace = FTextInspector::GetNamespace(Value).Get(FString());
		const FString TextKey = FTextInspector::GetKey(Value).Get(FString());
		if (TextNamespace.Equals(*NamespaceToMatch, ESearchCase::CaseSensitive) && TextKey.Equals(*KeyToMatch, ESearchCase::CaseSensitive))
		{
			switch (ComparisonMode)
			{
			case FTextReferenceCollector::EComparisonMode::MatchId:
				{
					++(*Count);
				}
				break;

			case FTextReferenceCollector::EComparisonMode::MatchSource:
				{
					const FString* TextSource = FTextInspector::GetSourceString(Value);
					if (TextSource && TextSource->Equals(*SourceToMatch, ESearchCase::CaseSensitive))
					{
						++(*Count);
					}
				}
				break;

			case FTextReferenceCollector::EComparisonMode::MismatchSource:
				{
					const FString* TextSource = FTextInspector::GetSourceString(Value);
					if (TextSource && !TextSource->Equals(*SourceToMatch, ESearchCase::CaseSensitive))
					{
						++(*Count);
					}
				}
				break;

			default:
				break;
			}
		}

		return *this;
	}

private:
	FTextReferenceCollector::EComparisonMode ComparisonMode;
	const FString* NamespaceToMatch;
	const FString* KeyToMatch;
	const FString* SourceToMatch;
	int32* Count;

	TSet<const UObject*> AllObjectsInPackage;
	TSet<const UObject*> ProcessedObjects;
};

FTextReferenceCollector::FTextReferenceCollector(const UPackage* const InPackage, const EComparisonMode InComparisonMode, const FString& InTextNamespace, const FString& InTextKey, const FString& InTextSource, int32& OutCount)
{
	FTextReferencesArchive(InPackage, InComparisonMode, InTextNamespace, InTextKey, InTextSource, OutCount);
}

FTextReferenceCollector::FTextReferenceCollectorCallbackMap& FTextReferenceCollector::GetTypeSpecificTextReferenceCollectorCallbacks()
{
	static FTextReferenceCollectorCallbackMap TypeSpecificTextReferenceCollectorCallbacks;
	return TypeSpecificTextReferenceCollectorCallbacks;
}
