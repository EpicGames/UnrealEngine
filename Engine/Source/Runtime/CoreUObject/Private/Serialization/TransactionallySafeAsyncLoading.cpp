// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/TransactionallySafeAsyncLoading.h"
#include "Serialization/AsyncPackageLoader.h"
#include "AutoRTFM.h"
#include "Containers/Map.h"
#include "UObject/LinkerInstancingContext.h"
#include "UObject/Package.h"
#include "Async/Mutex.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"

#if UE_AUTORTFM_ENABLED || defined(__INTELLISENSE__)

namespace
{
static bool bAutoRTFMUseNewAbort = true;
static FAutoConsoleVariableRef CVarAutoRTFMUseNewAbort(
	TEXT("AutoRTFM.UseNewAbort"),
	bAutoRTFMUseNewAbort,
	TEXT("If true, use new aborting approach for completion delegate."));
}

// A transactionally safe async package loader that wraps an underlying actual
// async package loader but allows it to be interacted with safely while also
// inside a transaction.
//
// The fundamental issue with async loading is, by its nature, it is touching
// deep bits of Unreal Engine. It's going to be creating a bunch of UObject's
// and thus touch all the deep and gnarly stuff that backs that system. This
// does not meld well with how we handle modifications to these *same* core
// bits of the engine from within AutoRTFM - namely by using transactionally
// safe locks that are held until the transaction completes. If we just tried
// to use the existing async package loader we'd deadlock because AutoRTFM
// would be holding locks that the async loader would be trying to take, and
// the AutoRTFM transaction could be blocked on the async loader while trying
// to flush load a package. Nasty!
//
// We get around this issue by making it so that if a user does anything that
// requires flushing the async loader (EG. if they are doing bad things like
// synchronous loads of packages...), we have to abort the entire transaction
// nest to release any locks we hold, flush the async loader, then retry the
// transaction. To get this to work we keep a cache of loaded packages so
// that we do not have to interact with the underlying async loader for a
// given package after we have successfully loaded that package.
//
// The two fundamental changes when in a transaction are:
// - When loading a package in the closed we check whether the package cache
//   already has the package, and if so we just return that package object. If
//   the package is not in the cache we instead just remember to load the
//   package when the transaction commits. Loading a package is an async action
//   so it is fine for us to just defer the async nature of it.
// - When flushing a previous request-id, we check if the request-id was one
//   that we know has already been flushed. If not, we need to abort and retry
//   the transaction with the flush happening in between the abort and the retry.
//
// There is an additional issue we've had to address, dealing with trashing packages and
// then issuing sync-loads for them in the same transaction. The problem is:
//
// - `FindObject` returns `A`.
// - We call `TrashPackage` on `A`.
// - We do `LoadPackage` which does abort.
// - We undo `TrashPackage` of `A`.
// - We then issue the `LoadPackage`, which will return the pointer to `A` because there
//   is already an object with the name.
// - We retry.
// - Repeat forever.
//
// What we actually need to happen is:
//
// - `FindObject` returns `A`.
// - We call `TrashPackage` on `A`.
// - We do `LoadPackage` which does abort.
// - We undo `TrashPackage` of `A`.
// - We then issue the `LoadPackage`, which needs to return a new package `B` that will
//   eventually get the same name as `A`.
// - We retry.
// - `FindObject` returns `A`.
// - We call `TrashPackage` on `A`.
// - We do `LoadPackage` which returns `B`.
//
// To get this to work we detect the case where we are aborting-and-retrying to do a
// sync-load for a package that already exists, and this case can *only* occur when
// a package has been trashed and then a new sync-load has been issued in the same
// transaction nest. We have to rename the old package temporarily, load a new package
// that'll get a new object backing it, then rename the old package back to the original
// package name. This ensures that we have both `A` (the original package we are going
// to trash) and `B` (the new package ready to take over once `A` is trashed and we do
// the sync-load) ready for the transactional retry.
class FTransactionallySafeAsyncPackageLoader final : public IAsyncPackageLoader
{
public:
	FTransactionallySafeAsyncPackageLoader(IAsyncPackageLoader* const InWrappedPackageLoader) : WrappedPackageLoader(InWrappedPackageLoader) {}

	virtual ~FTransactionallySafeAsyncPackageLoader()
	{
		delete WrappedPackageLoader;
	}

	void InitializeLoading() override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->InitializeLoading();
	}

	void ShutdownLoading() override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->ShutdownLoading();
	}

	void StartThread() override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->StartThread();
	}

	int32 LoadPackage(
			const FPackagePath& PackagePath,
			FName CustomPackageName,
			FLoadPackageAsyncDelegate InCompletionDelegate,
			EPackageFlags InPackageFlags,
			int32 InPIEInstanceID,
			int32 InPackagePriority,
			const FLinkerInstancingContext* InInstancingContext,
			uint32 InLoadFlags) override
	{
		if (CustomPackageName.IsNone())
		{
			CustomPackageName = PackagePath.GetPackageFName();
		}

		if (AutoRTFM::IsClosed())
		{
			return AutoRTFM::Open([&]
			{
				FString PackagePathAsString = PackagePath.GetDebugNameWithExtension();

				TOptional<FMapPayload> MapPayload;

				{
					UE::TScopeLock _(Mutex);

					// Check if we've cached the package previously.
					if (FMapPayload* const Result = PackageCache.Find(PackagePathAsString))
					{
						// Check if the result is still valid. It could become invalid if GC had collected
						// the underlying `UPackage` for instance.
						if (Result->IsStillValid())
						{
							// Need to copy it because as soon as we give up our mutex the package cache
							// could have been resized thus invalidating our underlying memory allocation.
							MapPayload = *Result;
						}
						else
						{
							UE_LOGF(LogStreaming, Display, "A previously loaded cached package `%ls` was garbage collected, and we are having to reload it.", *PackagePathAsString);
						}
					}
				}

				if (!MapPayload)
				{
					// We do not have the package cached and ready to be used in our transaction. So
					// we copy the required state into the open, and cache that so we can process it
					// later when the transaction has completed, or if we call `FlushLoading`, during
					// transaction retry.

					FPackagePath PackagePathClone = PackagePath;
					FName CustomPackageNameClone = FName(CustomPackageName.ToString());
					TOptional<FLinkerInstancingContext> LinkerInstancingContextClone;

					if (InInstancingContext)
					{
						LinkerInstancingContextClone = FLinkerInstancingContext::DuplicateContext(*InInstancingContext);
					}

					const bool bFirst = TransactionallyDeferredLoadPackages.IsEmpty();

					FTransactionallyDeferredLoadPackagePayload Payload;
					Payload.PackagePath = PackagePathClone;
					Payload.CustomPackageName = CustomPackageNameClone;
					Payload.CompletionDelegate = InCompletionDelegate;
					Payload.PackageFlags = InPackageFlags;
					Payload.PIEInstanceID = InPIEInstanceID;
					Payload.PackagePriority = InPackagePriority;
					Payload.InstancingContext = LinkerInstancingContextClone;
					Payload.LoadFlags = InLoadFlags;
					TransactionallyDeferredLoadPackages.Push(Payload);

					const int32 Index = TransactionallyDeferredLoadPackages.Num() - 1;

					if (LIKELY(bAutoRTFMUseNewAbort))
					{
						// We need to wipe out the completion delegate of our new deferred package load
						// entry if we abort. This is because that completion delegate could have
						// captured transactional local allocations that could be destroyed by the time
						// we came to destruct all the items in `TransactionallyDeferredLoadPackages`
						// when we empty it, causing use-after-frees.
						const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
						{
							AutoRTFM::OnAbort([this, Index]
							{
								TransactionallyDeferredLoadPackages[Index].CompletionDelegate.Unbind();
							});
						});
						ensure(Status == AutoRTFM::ETransactionStatus::Executing);
					}

					// If we are the first, we need to register our handlers so that
					// on-commit we issue the loads, and on-abort we purge the deferred
					// array of them.
					if (bFirst)
					{
						const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
						{
							AutoRTFM::OnCommit([this] { DrainTransactionallyDeferredLoadPackages(); });
							AutoRTFM::OnComplete([this]
							{
								if (!TransactionallyDeferredLoadPackages.IsEmpty())
								{
									for (const FTransactionallyDeferredLoadPackagePayload& Payload : TransactionallyDeferredLoadPackages)
									{
										UE_LOGF(LogStreaming, Display, "Ignoring aborted load for package '%ls'", *Payload.PackagePath.GetDebugNameWithExtension());
									}
									
									TransactionallyDeferredLoadPackages.Empty();
								}
							});
						});
						ensure(Status == AutoRTFM::ETransactionStatus::Executing);
					}

					return PackageCacheMiss;
				}
				else
				{
					// If we've got a package cache hit on a load that did not succeed we *only* want
					// to preserve this entry in the package cache until the transaction nest completes
					// Otherwise we could have a package load that fails, which a subsequent load could
					// have succeeded on (say the package became available on disk), and we'd never be
					// able to load that package.
					if (MapPayload->Result != EAsyncLoadingResult::Succeeded)
					{
						const bool bFirst = TransactionallyLoadedPackagesThatDidNotSucceed.IsEmpty();

						TransactionallyLoadedPackagesThatDidNotSucceed.Add(PackagePathAsString);

						if (bFirst)
						{
							AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
							{
								AutoRTFM::OnComplete([this]
								{
									UE::TScopeLock _(Mutex);

									for (const FString& PackagePathAsString : TransactionallyLoadedPackagesThatDidNotSucceed)
									{
										FMapPayload MapPayload;

										// This can return false if we've issued multiple async loads to the same failing
										// package within a transaction. The first would remove the element from the cache
										// then all subsequent wouldn't find a map payload.
										if (PackageCache.RemoveAndCopyValue(PackagePathAsString, MapPayload))
										{
											// We'll remove this later - but do an additional check that we definitely only
											// removed a package that did not succeed (as we intended).
											ensure(MapPayload.Result != EAsyncLoadingResult::Succeeded);
										}
									}

									TransactionallyLoadedPackagesThatDidNotSucceed.Empty();
								});
							});
							ensure(Status == AutoRTFM::ETransactionStatus::Executing);
						}
					}

					// Whether we cause an abort in `InCompletionDelegate`, all we are doing in the open
					// after this call is to return from the wrapped open, which will do the right thing
					// and continue with the abort. That is why it is safe to ignore the result from the
					// close.
					(void)AutoRTFM::Close([&]
					{
						ensure(MapPayload->IsStillValid());

						UPackage* const Package = MapPayload->GetPackage().Get();

						// Now that the package has come into actual use we rename it to the actual
						// package name that it originally was conceived with, making it discoverable
						// in `FindObject` calls.
						if (Package)
						{
							Package->Rename(*CustomPackageName.ToString(), nullptr, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty | REN_AllowPackageLinkerMismatch);
						}

						InCompletionDelegate.ExecuteIfBound(CustomPackageName, Package, MapPayload->Result);
					});

					return PackageCacheHit;
				}
			});
		}
		else
		{
			return LoadAndCachePackage(PackagePath,
				CustomPackageName,
				InCompletionDelegate,
				InPackageFlags,
				InPIEInstanceID,
				InPackagePriority,
				InInstancingContext,
				InLoadFlags);
		}
	}

	int32 LoadPackage(const FPackagePath& PackagePath, FLoadPackageAsyncOptionalParams OptionalParams) override
	{
		FLoadPackageAsyncDelegate CompletionDelegate;

		if (OptionalParams.CompletionDelegate.IsValid())
		{
			CompletionDelegate = *OptionalParams.CompletionDelegate.Get();
		}

		return LoadPackage(PackagePath,
			OptionalParams.CustomPackageName,
			CompletionDelegate,
			OptionalParams.PackageFlags,
			OptionalParams.PIEInstanceID,
			OptionalParams.PackagePriority,
			OptionalParams.InstancingContext,
			OptionalParams.LoadFlags);
	}

	EAsyncPackageState::Type ProcessLoading(bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit) override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		return WrappedPackageLoader->ProcessLoading(bUseTimeLimit, bUseFullTimeLimit, TimeLimit);
	}

	EAsyncPackageState::Type ProcessLoadingUntilComplete(TFunctionRef<bool()> CompletionPredicate, double TimeLimit) override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		return WrappedPackageLoader->ProcessLoadingUntilComplete(CompletionPredicate, TimeLimit);
	}

	void CancelLoading() override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->CancelLoading();
	}

	void SuspendLoading() override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->SuspendLoading();
	}

	void ResumeLoading() override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->ResumeLoading();
	}

	void FlushLoading(TConstArrayView<int32> RequestIds) override
	{
		if (AutoRTFM::IsClosed())
		{
			AutoRTFM::Open([&]
			{
				if (RequestIds.IsEmpty() && TransactionallyDeferredLoadPackages.IsEmpty())
				{
					const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
					{
						AutoRTFM::OnCommit([this]
						{
							WrappedPackageLoader->FlushLoading({});
						});
					});
					check(Status == AutoRTFM::ETransactionStatus::Executing);
					return;
				}

				const int32 LastMaxFlushedRequestId = GetLastMaxFlushedRequestId();

				bool bSkipRetry = true;
				for (const int32 RequestId : RequestIds)
				{
					if (PackageCacheHit == RequestId)
					{
						continue;
					}

					if ((RequestId <= LastMaxFlushedRequestId) && (PackageCacheMiss != RequestId))
					{
						continue;
					}

					bSkipRetry = false;
					break;
				}

				if (!bSkipRetry)
				{
					TArray<int32> RequestIdsClone;

					for (const int32 RequestId : RequestIds)
					{
						// Filter out our special return status' for package cache hit and miss.
						// The wrapped underlying async package loader does not understand them.
						if ((PackageCacheHit == RequestId) || (PackageCacheMiss == RequestId))
						{
							continue;
						}

						RequestIdsClone.Add(RequestId);
					}

					const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([&]
					{
						AutoRTFM::OnRetry([this, RequestIdsClone = ::MoveTemp(RequestIdsClone)]() mutable
						{
							UE_LOGF(LogStreaming, Display, "A call to `FlushLoading` that is flushing non-cached packages is causing Verse to abort and retry.");
							DrainTransactionallyDeferredLoadPackages(&RequestIdsClone, true);
							UpdateMaxFlushedRequestId(RequestIdsClone);
							WrappedPackageLoader->FlushLoading(RequestIdsClone);
							UE_LOGF(LogStreaming, Display, "`FlushLoading` has completed after Verse aborted, and we are now retrying.");
						});
						AutoRTFM::CascadingRetryTransaction();
					});
					check(Status == AutoRTFM::ETransactionStatus::AbortedByCascadingRetry);
				}
			});
		}
		else
		{
			AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
			UpdateMaxFlushedRequestId(RequestIds);

			if (INDEX_NONE != RequestIds.Find(PackageCacheMiss))
			{
				// If any request id was a package cache miss, it means we kicked
				// off an async load package request inside a transaction, and are
				// now trying to flush that from outside the transaction. Since we
				// don't have a mapping from the miss to the actual request id, we
				// just have to flush all packages.
				WrappedPackageLoader->FlushLoading({});
			}
			else if (INDEX_NONE != RequestIds.Find(PackageCacheHit))
			{
				// If any request id was a package cache hit, we just need to filter
				// it out from the list of request ids we are going to ask the actual
				// underlying wrapped async package loader to flush.
				TArray<int32> SubsetOfRequestIds;

				for (const int32 RequestId : RequestIds)
				{
					if (PackageCacheHit == RequestId)
					{
						continue;
					}

					SubsetOfRequestIds.Add(RequestId);
				}

				WrappedPackageLoader->FlushLoading(SubsetOfRequestIds);
			}
			else
			{
				WrappedPackageLoader->FlushLoading(RequestIds);
			}
		}
	}

	int32 GetNumQueuedPackages() override
	{
		return WrappedPackageLoader->GetNumQueuedPackages();
	}

	int32 GetNumAsyncPackages() override
	{
		return WrappedPackageLoader->GetNumAsyncPackages();
	}

	float GetAsyncLoadPercentage(const FName& PackageName) override
	{
		return WrappedPackageLoader->GetAsyncLoadPercentage(PackageName);
	}

	bool IsAsyncLoadingSuspended() override
	{
		return WrappedPackageLoader->IsAsyncLoadingSuspended();
	}

	bool IsInAsyncLoadThread() override
	{
		return WrappedPackageLoader->IsInAsyncLoadThread();
	}

	bool IsMultithreaded() override
	{
		return WrappedPackageLoader->IsMultithreaded();
	}

	bool IsAsyncLoadingPackages() override
	{
		return !TransactionallyDeferredLoadPackages.IsEmpty() || WrappedPackageLoader->IsAsyncLoadingPackages();
	}

	void NotifyConstructedDuringAsyncLoading(UObject* Object, bool bSubObject) override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->NotifyConstructedDuringAsyncLoading(Object, bSubObject);
	}

	void NotifyUnreachableObjects(const TArrayView<FUObjectItem*>& UnreachableObjects) override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->NotifyUnreachableObjects(UnreachableObjects);
	}

	void NotifyRegistrationEvent(FName PackageName, FName Name, ENotifyRegistrationType NotifyRegistrationType, ENotifyRegistrationPhase NotifyRegistrationPhase, FTypeConstructFunc* InRegister, bool InbDynamic, UObject* FinishedObject) override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->NotifyRegistrationEvent(PackageName, Name, NotifyRegistrationType, NotifyRegistrationPhase, InRegister, InbDynamic, FinishedObject);
	}

	void NotifyScriptVersePackage(Verse::VPackage* Package) override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->NotifyScriptVersePackage(Package);
	}

	void NotifyRegistrationComplete() override
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		WrappedPackageLoader->NotifyRegistrationComplete();
	}

	ELoaderType GetLoaderType() const override
	{
		return WrappedPackageLoader->GetLoaderType();
	}

private:
	static constexpr int32 PackageCacheHit = INT_MIN;
	static constexpr int32 PackageCacheMiss = PackageCacheHit + 1;

	static const char* const UnreachableMessage;
	IAsyncPackageLoader* const WrappedPackageLoader;

	struct FTransactionallyDeferredLoadPackagePayload final
	{
		FPackagePath PackagePath;
		FName CustomPackageName;
		FLoadPackageAsyncDelegate CompletionDelegate;
		EPackageFlags PackageFlags;
		int32 PIEInstanceID;
		int32 PackagePriority;
		TOptional<FLinkerInstancingContext> InstancingContext;
		uint32 LoadFlags;
	};

	TArray<FTransactionallyDeferredLoadPackagePayload> TransactionallyDeferredLoadPackages;
	TArray<FString> TransactionallyLoadedPackagesThatDidNotSucceed;

	struct FMapPayload final
	{
		FMapPayload() = default;
		FMapPayload(UPackage* const LoadedPackage, EAsyncLoadingResult::Type Result) :
			LoadedPackages({ MakeWeakObjectPtr(LoadedPackage) }), Result(Result) {}

		TArray<TWeakObjectPtr<UPackage>> LoadedPackages;
		EAsyncLoadingResult::Type Result = EAsyncLoadingResult::Failed;

		// Tells us whether a given map payload entry is still valid (EG. a cached loaded package didn't get GC'ed).
		bool IsStillValid() const
		{
			// If we succeeded in the load and at one point had a valid `LoadedPackage`, then check whether it
			// became null (meaning the GC did its thing).
			if (EAsyncLoadingResult::Succeeded == Result)
			{
				TWeakObjectPtr<UPackage> Package = GetPackage();

				// If the package has been GC'ed then it is not still valid.
				if (!Package.IsValid())
				{
					return false;
				}

				return true;
			}
			else
			{
				return true;
			}
		}

		TWeakObjectPtr<UPackage> GetPackage() const
		{
			if (LoadedPackages.IsEmpty())
			{
				return nullptr;
			}

			for (TWeakObjectPtr<UPackage> LoadedPackage : LoadedPackages)
			{
				if (LoadedPackage.IsValid())
				{
					const FName NameWithoutNumber(LoadedPackage->GetFName(), 0);

					// Skip trashed packages.
					const EName* const NameOrNull = NameWithoutNumber.ToEName();

					if (NameOrNull && (*NameOrNull == NAME_TrashedPackage))
					{
						continue;
					}

					// Out of an abundance of caution we'll leave the old trashing logic in
					// but ensure that it isn't actually hit for the time being.
					static FString TrashedPackage = FName(NAME_TrashedPackage).ToString();
					const bool bIsTrashedPackage = LoadedPackage->GetFName().ToString().StartsWith(TrashedPackage);

					if (ensureMsgf(!bIsTrashedPackage, TEXT("Loaded package named '%s'"), *LoadedPackage->GetFName().ToString()))
					{
						return LoadedPackage;
					}
				}
			}

			return nullptr;
		}
	};

	TMap<FString, FMapPayload> PackageCache;
	int32 MaxFlushedRequestId = -1;
	UE::FMutex Mutex;

	void UpdateMaxFlushedRequestId(const TConstArrayView<int32> RequestIds)
	{
		UE::TScopeLock _(Mutex);

		for (const int32 RequestId : RequestIds)
		{
			MaxFlushedRequestId = FMath::Max(MaxFlushedRequestId, RequestId);
		}
	}

	int32 GetLastMaxFlushedRequestId()
	{
		UE::TScopeLock _(Mutex);
		return MaxFlushedRequestId;
	}

	int32 LoadAndCachePackage(
			const FPackagePath& PackagePath,
			FName CustomPackageName,
			FLoadPackageAsyncDelegate InCompletionDelegate,
			EPackageFlags InPackageFlags,
			int32 InPIEInstanceID,
			int32 InPackagePriority,
			const FLinkerInstancingContext* InInstancingContext,
			uint32 InLoadFlags)
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);

		FString PackagePathStr = PackagePath.GetDebugNameWithExtension();

		FLoadPackageAsyncDelegate WrapperDelegate;
		WrapperDelegate.BindLambda([this, PackagePathStr, CustomPackageName, CompletionDelegate = ::MoveTemp(InCompletionDelegate)](
			const FName& Name, UPackage* const Package, EAsyncLoadingResult::Type Type)
		{
			const bool bRetrying = AutoRTFM::IsRetrying();

			// If we are retrying the transaction we will cache everything but cancelled loads.
			const bool bShouldCacheIfRetrying = bRetrying && (EAsyncLoadingResult::Canceled != Type);

			// But if we are not (EG. we are outside a transaction nest), we only cache succeeded loads.
			const bool bShouldCacheIfNotRetrying = !bRetrying && (EAsyncLoadingResult::Succeeded == Type);

			if (bShouldCacheIfRetrying || bShouldCacheIfNotRetrying)
			{
				UE::TScopeLock _(Mutex);

				if (FMapPayload* const Payload = PackageCache.Find(PackagePathStr))
				{
					// If we are not retrying it means we've done a package load outside of a transaction,
					// so we want to just forget all previous loaded packages, and start from a blank array.
					if (!bRetrying)
					{
						Payload->LoadedPackages.Empty();
					}

					Payload->LoadedPackages.Add(Package);
				}
				else
				{
					PackageCache.FindOrAdd(PackagePathStr) = FMapPayload(Package, Type);
				}
			}

			CompletionDelegate.ExecuteIfBound(Name, Package, Type);
		});

		return WrappedPackageLoader->LoadPackage(PackagePath,
			CustomPackageName,
			WrapperDelegate,
			InPackageFlags,
			InPIEInstanceID,
			InPackagePriority,
			InInstancingContext,
			InLoadFlags);
	}

	void DrainTransactionallyDeferredLoadPackages(TArray<int32>* RequestIds = nullptr, const bool bLogPackagePaths = false)
	{
		AutoRTFM::UnreachableIfTransactional(UnreachableMessage);
		const bool bIsRetrying = AutoRTFM::IsRetrying();

		for (const FTransactionallyDeferredLoadPackagePayload& Payload : TransactionallyDeferredLoadPackages)
		{
			const int32 RequestId = LoadAndCachePackage(Payload.PackagePath,
				bIsRetrying ? MakeUniqueObjectName(nullptr, UPackage::StaticClass(), NAME_AutoRTFMUnusedPackage) : Payload.CustomPackageName,
				bIsRetrying ? FLoadPackageAsyncDelegate() : Payload.CompletionDelegate,
				Payload.PackageFlags,
				Payload.PIEInstanceID,
				Payload.PackagePriority,
				Payload.InstancingContext.GetPtrOrNull(),
				Payload.LoadFlags);

			if (RequestIds)
			{
				RequestIds->Add(RequestId);
			}
		}

		if (bLogPackagePaths)
		{
			for (const FTransactionallyDeferredLoadPackagePayload& Payload : TransactionallyDeferredLoadPackages)
			{
				UE_LOGF(LogStreaming, Display, "Loading and caching '%ls' in the transactionally-safe async loader.", *Payload.PackagePath.GetDebugNameWithExtension());
			}
		}

		TransactionallyDeferredLoadPackages.Empty();
	}
};

const char* const FTransactionallySafeAsyncPackageLoader::UnreachableMessage = "Cannot call function within a transaction!";

#endif // UE_AUTORTFM_ENABLED || defined(__INTELLISENSE__)

IAsyncPackageLoader* MakeTransactionallySafeAsyncPackageLoader(IAsyncPackageLoader* const InWrappedPackageLoader)
{
#if UE_AUTORTFM_ENABLED
	return new FTransactionallySafeAsyncPackageLoader(InWrappedPackageLoader);
#else
	checkf(UE_AUTORTFM_ENABLED, TEXT("MakeTransactionallySafeAsyncPackageLoader() called without AutoRTFM instrumentation enabled for CoreUObject"));
	return InWrappedPackageLoader;
#endif
}
