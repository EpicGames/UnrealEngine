// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/UnversionedPropertySerialization.h"
#include "AutoRTFM.h"
#include "Serialization/UnversionedPropertySerializationTest.h"
#include "Hash/Blake3.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/ConfigAccessTracking.h"
#include "Misc/ScopeRWLock.h"
#include "UObject/OverridableManager.h"
#include "UObject/PropertyOptional.h"
#include "UObject/RemoteObjectSerialization.h"
#include "UObject/UnrealType.h"

#if WITH_EDITORONLY_DATA
#include "Algo/Sort.h"
#include "Misc/FileHelper.h"
#include "UObject/MetaData.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "UObject/PropertyStateTracking.h"
#endif

#include <atomic>

// Caches a property array per UStruct to avoid link-walking and touching FProperty data.
//
// As a reference point, this uses ~6MB of memory in an internal project and makes all
// unversioned property loading except non-numeric SerializeItem() calls about 2x faster.
#ifndef CACHE_UNVERSIONED_PROPERTY_SCHEMA
// x64 platforms tend to have more memory...
#	define CACHE_UNVERSIONED_PROPERTY_SCHEMA PLATFORM_CPU_X86_FAMILY
#endif

// Helper to pass around appropriate default value types depending on CACHE_UNVERSIONED_PROPERTY_SCHEMA
struct FDefaultStruct
{
	const uint8* Data;
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
	int32 StructStart;
	int32 StructEnd;

	FDefaultStruct(const uint8* InData, const UStruct* InStruct)
		: Data(InData)
	{
		StructStart = 0;
		StructEnd = 0;
		if (InData)
		{
			if (const UClass* StructAsClass = Cast<const UClass>(InStruct))
			{
				StructStart = StructAsClass->GetPropertiesStartOffset();
				StructEnd = InStruct->GetPropertiesSize() + StructStart;
			}
			else
			{
				StructEnd = InStruct->GetPropertiesSize();
			}
		}
	}
#else
	const UStruct* Struct;

	FDefaultStruct(const uint8* InData, const UStruct* InStruct) : Data(InData), Struct(InStruct) {}
#endif
};

// Serializes a FProperty at a specific static array index
//
// Extracts and caches relevant FProperty state when using
// CACHE_UNVERSIONED_PROPERTY_SCHEMA to improve data locality.
// Otherwise, extracts only the needed state on demand.
class FUnversionedPropertySerializer
{
public:
	FUnversionedPropertySerializer(FProperty* InProperty, int32 InArrayIndex)
		: Property(InProperty)
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
		, Offset(Property->GetOffset_ForInternal() + Property->GetElementSize() * InArrayIndex)
		, bSerializeAsInteger(CanSerializeAsInteger(Property))
		, bIsOptional(IsOptional(Property->GetClass()->GetCastFlags()))
#if WITH_EDITORONLY_DATA
		, bHasRequiredParam(Property->HasAnyPropertyFlags(CPF_RequiredParm))
#endif
		, IntType(GetIntType(Property->GetMinAlignment()))
#else
		, ArrayIndex(InArrayIndex)
#endif
	{
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
		uint32 IntNum = GetIntNum(InProperty, IntType);
		FastZeroIntNum = (CanSerializeAsZero(InProperty, IntType) & (IntNum < 256)) ? static_cast<uint8>(IntNum) : uint8(0);
#endif
	}

	FProperty* GetProperty() const
	{
		return Property;
	}

	void* GetValue(uint8* Data) const
	{
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
		return Data + Offset;
#else
		return Property->ContainerPtrToValuePtr<uint8>(Data, ArrayIndex);
#endif
	}

	const void* GetValue(const uint8* Data) const
	{
		return GetValue(const_cast<uint8*>(Data));
	}

	const void* GetDefaultValue(FDefaultStruct Defaults) const
	{
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
		return Offset >= Defaults.StructStart && Offset < Defaults.StructEnd ? Defaults.Data + Offset : nullptr;
#else
		return Property->ContainerPtrToValuePtrForDefaults<uint8>(Defaults.Struct, Defaults.Data);
#endif
	}

	FORCEINLINE void Serialize(FStructuredArchive::FSlot Slot, uint8* Data, FDefaultStruct Defaults) const
	{
#if !CACHE_UNVERSIONED_PROPERTY_SCHEMA
		bool bSerializeAsInteger = CanSerializeAsInteger(Property);
#endif

		if (bSerializeAsInteger)
		{
#if !CACHE_UNVERSIONED_PROPERTY_SCHEMA
			EIntegerType IntType = GetIntType(Property->GetMinAlignment());
#endif
			SerializeAsInteger(Slot, GetValue(Data), IntType);
		}
		else
		{
			// Note that each bitfield stores a redundant byte -- the zero mask could tell us if the bit is 0 or 1.
			//
			// Removing this redundant byte and the FBoolProperty::SerializeItem() calls would add some complexity,
			// but only reduce data size by ~0.5% in the data set I used to guide trade-offs. It would not make much 
			// difference for loading performance either. --jtorp
			Property->SerializeItem(Slot, GetValue(Data), GetDefaultValue(Defaults));
		}
	}

	FORCEINLINE void LoadZero(uint8* Data) const
	{
		void* ValueData = GetValue(Data);

#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
		// Cached FastZeroIntNum is only uint8 and not sufficient for large unset optionals
		if (FastZeroIntNum == 0)
		{
			checkf(bIsOptional && Property->GetElementSize() >= 256, TEXT("Only large unset optionals should hit this loading path"));
			FMemory::Memzero(ValueData, Property->GetElementSize());
			return;
		}
#else
		EIntegerType IntType = GetIntType(Property->GetMinAlignment());
		uint32 FastZeroIntNum = GetIntNum(Property, IntType);
#endif

		switch (IntType)
		{
			case EIntegerType::Uint8 : MemZeroRange<uint8 >(ValueData, FastZeroIntNum); break;
			case EIntegerType::Uint16: MemZeroRange<uint16>(ValueData, FastZeroIntNum); break;
			case EIntegerType::Uint32: MemZeroRange<uint32>(ValueData, FastZeroIntNum); break;
			case EIntegerType::Uint64: MemZeroRange<uint64>(ValueData, FastZeroIntNum); break;
			default: UE_ASSUME(0);
		}
	}

	bool ShouldSaveAsZero(const uint8* Data) const
	{
#if !CACHE_UNVERSIONED_PROPERTY_SCHEMA
		EIntegerType IntType = GetIntType(Property->GetMinAlignment());
		bool bIsOptional = IsOptional(Property->GetClass()->GetCastFlags());
		uint32 FastZeroIntNum = CanSerializeAsZero(Property, IntType) ? GetIntNum(Property, IntType) : uint32(0);
#endif

		if (bIsOptional)
		{
			return FastZeroIntNum > 0 && !static_cast<const FOptionalProperty*>(Property)->IsSet(GetValue(Data));
		}
		else if (FastZeroIntNum == 1) // Can likely be simplified and faster using a switch() statement like LoadZero()
		{
			return IsIntZero(GetValue(Data), IntType);
		}
		else
		{
			return FastZeroIntNum > 0 && IsIntRangeZero(GetValue(Data), FastZeroIntNum, IntType);
		}
	}

	bool IsDefault(const uint8* Data, FDefaultStruct Defaults, uint32 PortFlags) const
	{
		return Property->Identical(GetValue(Data), GetDefaultValue(Defaults), PortFlags);
	}


	bool HasRequiredParam() const
	{
#if WITH_EDITORONLY_DATA && CACHE_UNVERSIONED_PROPERTY_SCHEMA
		return bHasRequiredParam;
#else
		return Property->HasAnyPropertyFlags(CPF_RequiredParm);
#endif
	}

private:
	enum class EIntegerType : uint8 { Uint8, Uint16, Uint32, Uint64 };

	static bool IsOptional(uint64 CastFlags)
	{
		return !!(CastFlags & CASTCLASS_FOptionalProperty);
	}

	static uint32 GetIntNum(const FProperty* Property, EIntegerType IntType)
	{
		return Property->GetElementSize() / GetSizeOf(IntType);
	}

	static bool CanSerializeAsZero(const FProperty* Property, EIntegerType IntType)
	{
		static constexpr uint32 MaxZeroComparisons = 16;

		uint64 CastFlags = Property->GetClass()->GetCastFlags();

		if ((CastFlags & (CASTCLASS_FStructProperty | CASTCLASS_FBoolProperty)) == 0)
		{
			checkf(GetIntNum(Property, IntType) < MaxZeroComparisons || IsOptional(CastFlags),  TEXT("Unexpectedly large property type encountered %s"), *Property->GetName());

			// We can only zero-serialize properties that:
			// - Don't need a destructor
			// - And can be zero-initialized
			// This is because if the property is zero-serialized, loading it will simply memzero the previous value.
			return Property->HasAllPropertyFlags(CPF_ZeroConstructor | CPF_NoDestructor);
		}
		else if ((CastFlags & CASTCLASS_FBoolProperty) != 0)
		{
			return static_cast<const FBoolProperty*>(Property)->IsNativeBool();
		}
		else
		{
			bool bIsAtomic = !!(static_cast<const FStructProperty*>(Property)->Struct->StructFlags & STRUCT_Atomic);
			return bIsAtomic && GetIntNum(Property, IntType) < MaxZeroComparisons;
		}
	}

	static bool CanSerializeAsInteger(FProperty* Property)
	{
		uint64 CastFlags = Property->GetClass()->GetCastFlags();

		if ((CastFlags & CASTCLASS_FBoolProperty) != 0)
		{
			return static_cast<const FBoolProperty*>(Property)->IsNativeBool();
		}
		
		return (CastFlags & (CASTCLASS_FNumericProperty | CASTCLASS_FEnumProperty)) != 0;
	}

	template<typename T>
	static void SerializeAs(FStructuredArchive::FSlot Slot, void* Value)
	{
		Slot << *reinterpret_cast<T*>(Value);
	}

	static void SerializeAsInteger(FStructuredArchive::FSlot Slot, void* Value, EIntegerType IntType)
	{
		 switch (IntType)
		 {
			 case  EIntegerType::Uint8:	Slot << *reinterpret_cast< uint8*>(Value); break;
			 case EIntegerType::Uint16: Slot << *reinterpret_cast<uint16*>(Value); break;
			 case EIntegerType::Uint32: Slot << *reinterpret_cast<uint32*>(Value); break;
			 case EIntegerType::Uint64: Slot << *reinterpret_cast<uint64*>(Value); break;
			 default: UE_ASSUME(0);
		 }
	}

	template<typename T>
	static bool IsZero(const void* Value)
	{
		return *reinterpret_cast<const T*>(Value) == 0;
	}

	static bool IsIntZero(const void* Value, EIntegerType IntType)
	{
		static constexpr bool (*Functions[4])(const void*) = { &IsZero<uint8>, &IsZero<uint16>, &IsZero<uint32>, &IsZero<uint64> };
		return (*Functions[static_cast<uint32>(IntType)])(Value);
	}

	template<typename T>
	static bool IsRangeZero(const void* Value, uint32 Num)
	{
		T Out = 0;
		for (uint32 Idx = 0; Idx < Num; ++Idx)
		{
			Out |= reinterpret_cast<const T*>(Value)[Idx];
		}
		return Out == 0;
	}

	static bool IsIntRangeZero(const void* Value, uint32 Num, EIntegerType IntType)
	{
		static constexpr bool (*Functions[4])(const void*, uint32 Num) = { &IsRangeZero<uint8>, &IsRangeZero<uint16>, &IsRangeZero<uint32>, &IsRangeZero<uint64> };
		return (*Functions[static_cast<uint32>(IntType)])(Value, Num);	
	}

	template<typename T>
	FORCEINLINE static void MemZeroRange(void* Value, uint32 Num)
	{
		UE_ASSUME(Num > 0);
		for (uint32 Idx = 0; Idx < Num; ++Idx)
		{
			reinterpret_cast<T*>(Value)[Idx] = 0;
		}
	}

	static uint32 GetSizeOf(EIntegerType Type)
	{
		return 1u << static_cast<uint32>(Type);
	}

	// @param Bytes must be 1, 2, 4 or 8
	static constexpr uint32 Log2For1248(uint32 Bytes)
	{
		return (Bytes >> 1) - (Bytes >> 3);
	}

	static EIntegerType GetIntType(uint32 Alignment)
	{
		if (Alignment >= 8)
		{
			return EIntegerType::Uint64;
		}
		
		return static_cast<EIntegerType>(Log2For1248(Alignment));
	}

	FProperty* Property;
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
	int32 Offset;
	bool bSerializeAsInteger;
	bool bIsOptional : 1;
#if WITH_EDITORONLY_DATA
	bool bHasRequiredParam : 1;
#endif
	EIntegerType IntType;
	uint8 FastZeroIntNum;
#else
	uint32 ArrayIndex;
#endif
};

#if CACHE_UNVERSIONED_PROPERTY_SCHEMA

// Serialization is based on indices into this property array
struct FUnversionedStructSchema
{
#if WITH_EDITORONLY_DATA
	FBlake3Hash SchemaHash;
#endif
	uint32 Num;
	FUnversionedPropertySerializer Serializers[0];

	FORCEINLINE static FUnversionedStructSchema* Create(const UStruct* Struct, bool bSkipEditorOnly)
	{
		CA_ASSUME(Struct != nullptr);
#if WITH_EDITORONLY_DATA
		FBlake3 HashBuilder;
		AppendClassMetaData(Struct, HashBuilder);
#endif
		TArray<FUnversionedPropertySerializer, TInlineAllocator<256>> Serializers;
		for (FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
#if WITH_EDITORONLY_DATA
			if (!bSkipEditorOnly || !Property->IsEditorOnlyProperty())
#endif
			{
#if WITH_EDITORONLY_DATA
				Property->AppendSchemaHash(HashBuilder, bSkipEditorOnly);
#endif
				for (int32 ArrayIdx = 0, ArrayDim = Property->ArrayDim; ArrayIdx < ArrayDim; ++ArrayIdx)
				{
					Serializers.Add(FUnversionedPropertySerializer(Property, ArrayIdx));
				}
			}
		}

		uint32 Bytes = sizeof(FUnversionedStructSchema) + Serializers.Num() * sizeof(FUnversionedPropertySerializer);
		FUnversionedStructSchema* Schema = reinterpret_cast<FUnversionedStructSchema*>(FMemory::Malloc(Bytes, alignof(FUnversionedPropertySerializer)));
		
#if WITH_EDITORONLY_DATA
		const UClass* StructAsClass = Cast<const UClass>(Struct);
		if (StructAsClass)
		{
			FAppendToClassSchemaContext Context(&HashBuilder);
			StructAsClass->CallAppendToClassSchema(Context);
		}
		Schema->SchemaHash = HashBuilder.Finalize();
#endif
		Schema->Num = Serializers.Num();
		FMemory::Memcpy(Schema->Serializers, Serializers.GetData(), Serializers.Num() * sizeof(FUnversionedPropertySerializer));

		return Schema;
	}

	FORCEINLINE static void Delete(FUnversionedStructSchema* Schema)
	{
		if (Schema)
		{
			Schema->~FUnversionedStructSchema();
			FMemory::Free(Schema);
		}
	}

#if WITH_EDITORONLY_DATA
	static void AppendObjectPath(const UObject* Object, FBlake3& HashBuilder)
	{
		for (const UObject* Outer = Object; Outer; Outer = Outer->GetOuter())
		{
			AppendHash(HashBuilder, Outer->GetFName());
		}
	}

	static void AppendClassMetaData(const UStruct* Struct, FBlake3& HashBuilder)
	{
		// Append the full name of the struct. To improve performance, append the FNames of its outers individually
		// rather than calculating the full name as a string.
		const UObject* Outermost = nullptr;
		for (const UObject* Outer = Struct; Outer; Outer = Outer->GetOuter())
		{
			AppendHash(HashBuilder, Outer->GetFName());
			Outermost = Outer;
		}
		// Append some properties of the struct that can impact instances' serialization or errors.
		const UClass* StructAsClass = Cast<const UClass>(Struct);
		if (StructAsClass)
		{
			AppendObjectPath(StructAsClass->ClassWithin, HashBuilder);
		}
		// Append metadata properties of the struct since they influence builds.
		const TMap<FName, FString>* MetaData = nullptr;
		if (Struct != UPackage::StaticClass())
		{
			if (const UPackage* Package = Cast<UPackage>(Outermost))
			{
				// Package is const, but there is no const accessor for GetMetaData, so we const-cast it.
				FMetaData& MetaDataObject = const_cast<UPackage*>(Package)->GetMetaData();
				MetaData = MetaDataObject.GetMapForObject(Struct);
			}
		}
		if (MetaData)
		{
			TArray<FName, TInlineAllocator<16>> MetaDataNames;
			MetaData->GenerateKeyArray(MetaDataNames);
			Algo::Sort(MetaDataNames, FNameLexicalLess());
			for (FName MetaDataName : MetaDataNames)
			{
				AppendHash(HashBuilder, MetaDataName);
				const FString& Value = MetaData->FindChecked(MetaDataName);
				HashBuilder.Update(*Value, Value.Len() * sizeof(**Value));
			}
		}
	}

	static FBlake3Hash CalculateSchemaHash(UStruct* Struct, bool bSkipEditorOnly)
	{
		FBlake3 HashBuilder;
		for (FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			if (!bSkipEditorOnly || !Property->IsEditorOnlyProperty())
			{
				Property->AppendSchemaHash(HashBuilder, bSkipEditorOnly);
			}
		}
		return HashBuilder.Finalize();
	}
#endif
};

static std::atomic<const FUnversionedStructSchema*>& GetUnversionedSchema(const UStruct* Struct, bool bSkipEditorOnly)
{
#if WITH_EDITORONLY_DATA
	return bSkipEditorOnly ? Struct->UnversionedGameSchema : Struct->UnversionedEditorSchema;
#else
	return Struct->UnversionedGameSchema;
#endif
}

const FUnversionedStructSchema& GetOrCreateUnversionedSchema(const UStruct* Struct, bool bSkipEditorOnly)
{
	std::atomic<const FUnversionedStructSchema*>* ExistingSchemaPtr = &GetUnversionedSchema(Struct, bSkipEditorOnly);
	if (const FUnversionedStructSchema* ExistingSchema = ExistingSchemaPtr->load(std::memory_order_acquire))
	{
		return *ExistingSchema;
	}

	FUnversionedStructSchema* CreatedSchema = FUnversionedStructSchema::Create(Struct, bSkipEditorOnly);

	auto DedupSchema = [ExistingSchemaPtr, CreatedSchema]() -> const FUnversionedStructSchema*
	{
		const FUnversionedStructSchema* ExistingSchema = nullptr;
		if (!ExistingSchemaPtr->compare_exchange_strong(ExistingSchema, CreatedSchema, std::memory_order_acq_rel, std::memory_order_acquire))
		{
			FUnversionedStructSchema::Delete(CreatedSchema);
			check(ExistingSchema);
			return ExistingSchema;
		}
		return CreatedSchema;
	};

	if (AutoRTFM::IsClosed())
	{
		// AutoRTFM cannot undo atomic swaps, so skip the caching if called from a transaction.
		// and get back to try to do the CAS on commit. 
		// This implies that we might destroy the CreatedSchema that got used during the transaction
		// but as long as nothing holds a pointer to it, all should be fine. If not, ASAN will catch it.
		// OnCommit -> Dedup created schema using CAS.
		// OnAbort  -> Created schema should get deleted.
		// So we should not have any leaks remaining in any scenarios.
		// We also assume that GC doesn't run during transactions and that the UStruct will remain valid until the commit.
		AutoRTFM::OnCommit([DedupSchema]() { DedupSchema(); });

		// For now return our created schema and we'll dedup it on commit.
		return *CreatedSchema;
	}

	return *DedupSchema();
}

using FUnversionedSchemaIterator = const FUnversionedPropertySerializer*;

#else

struct FLinkWalkingSchemaIterator
{
	FProperty* Property = nullptr;
	uint32 ArrayIndex = 0;
	bool bSkipEditorOnly = true;

	FLinkWalkingSchemaIterator() {}

	FORCEINLINE explicit FLinkWalkingSchemaIterator(FProperty* FirstProperty, bool bInSkipEditorOnly)
		: Property(SkipEditorOnlyProperties(FirstProperty, bInSkipEditorOnly))
		, bSkipEditorOnly(bInSkipEditorOnly)
	{
	}

	FORCEINLINE void operator++()
	{
		check(Property);

		if (ArrayIndex + 1 == Property->ArrayDim)
		{
			Property = SkipEditorOnlyProperties(Property->PropertyLinkNext, bSkipEditorOnly);
			ArrayIndex = 0;
		}
		else
		{
			++ArrayIndex;
		}
	}

	FORCEINLINE void operator+=(uint32 Num)
	{
		while (Num--)
		{
			this->operator++();
		}
	}

	FUnversionedPropertySerializer operator*() const
	{
		return FUnversionedPropertySerializer(Property, ArrayIndex);
	}

	bool operator!=(FLinkWalkingSchemaIterator Rhs) const
	{
		return (Property != Rhs.Property) | (ArrayIndex != Rhs.ArrayIndex);
	}

	FORCEINLINE static FProperty* SkipEditorOnlyProperties(FProperty* Property, bool bSkipEditorOnly)
	{
#if WITH_EDITORONLY_DATA
		if (bSkipEditorOnly)
		{
			while (Property && Property->IsEditorOnlyProperty())
			{
				Property = Property->PropertyLinkNext;
			}
		}
#endif
		return Property;
	}
};

using FUnversionedSchemaIterator = FLinkWalkingSchemaIterator;

#endif // CACHE_UNVERSIONED_PROPERTY_SCHEMA

struct FUnversionedSchemaRange
{
	FORCEINLINE explicit FUnversionedSchemaRange(const UStruct* Struct, bool bSkipEditorOnly)
	{
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
		const FUnversionedStructSchema& Schema = GetOrCreateUnversionedSchema(Struct, bSkipEditorOnly);
		Begin = Schema.Serializers;
		End = Schema.Serializers + Schema.Num;	
#else
		Begin = FUnversionedSchemaIterator(Struct->PropertyLink, bSkipEditorOnly);
		// End is default-initialized
#endif
	}
	
	FUnversionedSchemaIterator begin() const { return Begin; }
	FUnversionedSchemaIterator end() const { return End; }

	FUnversionedSchemaIterator Begin;
	FUnversionedSchemaIterator End;
};

// List of serialized property indices and which of them are non-zero.
//
// Serialized as a stream of 16-bit skip-x keep-y fragments and a zero bitmask.
class FUnversionedHeader
{
public:
	void Save(FStructuredArchive::FStream Stream) const
	{
		for (FFragment Fragment : Fragments)
		{
			uint16 Packed = Fragment.Pack();
			Stream.EnterElement() << Packed;
		}

		if (ZeroMask.Num() > 0)
		{
			SaveZeroMaskData(Stream, ZeroMask.Num(), ZeroMask.GetData());
		}
	}

	void Load(FStructuredArchive::FStream Stream)
	{
		FFragment Fragment;
		uint32 ZeroMaskNum = 0;
		uint32 UnmaskedNum = 0;
		do
		{
			uint16 Packed;
			Stream.EnterElement() << Packed;
			Fragment = FFragment::Unpack(Packed);

			Fragments.Add(Fragment);

			(Fragment.bHasAnyZeroes ? ZeroMaskNum : UnmaskedNum) += Fragment.ValueNum;
		}
		while (!Fragment.bIsLast);

		if (ZeroMaskNum > 0)
		{
			ZeroMask.SetNumUninitialized(ZeroMaskNum);
			LoadZeroMaskData(Stream, ZeroMaskNum, ZeroMask.GetData());
			bHasNonZeroValues = UnmaskedNum > 0 || ZeroMask.Find(false) != INDEX_NONE;
		}
		else
		{
			bHasNonZeroValues = UnmaskedNum > 0;
		}
	}
	
	bool HasValues() const
	{
		return bHasNonZeroValues | (ZeroMask.Num() > 0);
	}

	bool HasNonZeroValues() const
	{
		return bHasNonZeroValues;
	}

protected:
	struct FFragment
	{
		static constexpr uint32 SkipMax = 127;
		static constexpr uint32 ValueMax = 127;

		uint8 SkipNum = 0; // Number of properties to skip before values
		bool bHasAnyZeroes = false;
		uint8 ValueNum = 0;  // Number of subsequent property values stored
		bool bIsLast = 0; // Is this the last fragment of the header?
	
		static constexpr uint32 SkipNumMask = 0x007fu;
		static constexpr uint32 HasZeroMask = 0x0080u;
		static constexpr uint32 ValueNumShift = 9u;
		static constexpr uint32 IsLastMask  = 0x0100u;

		uint16 Pack() const
		{
			return SkipNum | (uint16)(bHasAnyZeroes ? HasZeroMask : 0) | (uint16)(ValueNum << ValueNumShift) | (uint16)(bIsLast ? IsLastMask : 0);
		}

		static FFragment Unpack(uint16 Int)
		{
			FFragment Fragment;
			Fragment.SkipNum = static_cast<uint8>(Int & SkipNumMask);
			Fragment.bHasAnyZeroes = (Int & HasZeroMask) != 0;
			Fragment.ValueNum = static_cast<uint8>(Int >> ValueNumShift);
			Fragment.bIsLast = (Int & IsLastMask) != 0;
			return Fragment;
		}
	};

	using FZeroMask = TBitArray<TInlineAllocator<8>>;

	alignas (PLATFORM_CACHE_LINE_SIZE) TArray<FFragment, TInlineAllocator<32>> Fragments;
	bool bHasNonZeroValues = false;
	FZeroMask ZeroMask;

	void SaveZeroMaskData(FStructuredArchive::FStream Stream, uint32 NumBits, const uint32* Data) const
	{
		check(NumBits > 0);

		uint32 LastWordMask = ~0u >> ((32u - NumBits) % 32u); //-V1064
		if (NumBits <= 8)
		{
			uint8 Word = static_cast<uint8>(*Data & LastWordMask);
			checkf(Word != 0, TEXT("Zero mask shouldn't be saved when no bits are set"));
			Stream << Word;
		}
		else if (NumBits <= 16)
		{
			uint16 Word = static_cast<uint16>(*Data & LastWordMask);
			checkf(Word != 0, TEXT("Zero mask shouldn't be saved when no bits are set"));
			Stream << Word;
		}
		else
		{
			uint32 NumWords = FMath::DivideAndRoundUp(NumBits, 32u);

			for (uint32 WordIdx = 0; WordIdx < NumWords - 1; ++WordIdx)
			{
				uint32 Word = Data[WordIdx];
				Stream << Word;
			}

			uint32 LastWord = Data[NumWords - 1] & LastWordMask;
			Stream << LastWord;
		}
	}

	void LoadZeroMaskData(FStructuredArchive::FStream Stream, uint32 NumBits, uint32* Data)
	{
		if (NumBits <= 8)
		{
			uint8 Int;
			Stream << Int;
			*Data = Int;
		}
		else if (NumBits <= 16)
		{
			uint16 Int;
			Stream << Int;
			*Data = Int;
		}
		else
		{
			for (uint32 Idx = 0, Num = FMath::DivideAndRoundUp(NumBits, 32u); Idx < Num; ++Idx)
			{
				Stream << Data[Idx];
			}
		}
	}

public:
	class FIterator
	{
	public:
		FORCEINLINE FIterator(const FUnversionedHeader& Header, const FUnversionedSchemaRange& Schema)
			: SchemaIt(Schema.Begin)
			, ZeroMask(Header.ZeroMask)
			, FragmentIt(Header.Fragments.GetData())
			, bDone(!Header.HasValues())
#if DO_CHECK || USING_CODE_ANALYSIS
			, SchemaEnd(Schema.End)
#endif
		{
			if (!bDone)
			{
				Skip();
			}
		}

		void Next()
		{
			++SchemaIt;
			--RemainingFragmentValues;
			ZeroMaskIndex += FragmentIt->bHasAnyZeroes;

			if (RemainingFragmentValues == 0)
			{
				if (FragmentIt->bIsLast)
				{
					bDone = true;
				}
				else
				{
					++FragmentIt;
					Skip();
				}
			}
		}

		explicit operator bool() const
		{
			return !bDone;
		}

		FUnversionedPropertySerializer GetSerializer() const
		{
#if DO_CHECK || USING_CODE_ANALYSIS
			check(SchemaIt != SchemaEnd);
#endif
			return *SchemaIt;
		}

		bool IsNonZero() const
		{
			return !FragmentIt->bHasAnyZeroes || !ZeroMask[ZeroMaskIndex];
		}

	private:
		FUnversionedSchemaIterator SchemaIt;
		const FZeroMask& ZeroMask;
		const FFragment* FragmentIt = nullptr;
		bool bDone = false;
		uint32 ZeroMaskIndex = 0;
		uint32 RemainingFragmentValues = 0;
#if DO_CHECK || USING_CODE_ANALYSIS
		FUnversionedSchemaIterator SchemaEnd;
#endif

		void Skip()
		{
			SchemaIt += FragmentIt->SkipNum;
			
			while (FragmentIt->ValueNum == 0)
			{
				check(!FragmentIt->bIsLast);
				++FragmentIt;
				SchemaIt += FragmentIt->SkipNum;
			}

			RemainingFragmentValues = FragmentIt->ValueNum;
		}
	};
};

class FUnversionedHeaderBuilder : public FUnversionedHeader
{
public:
	FUnversionedHeaderBuilder()
	{
		Fragments.AddDefaulted();
	}

	void IncludeProperty(bool bIsZero)
	{
		if (Fragments.Last().ValueNum == FFragment::ValueMax)
		{
			TrimZeroMask(Fragments.Last());
			Fragments.AddDefaulted();
		}
		
		++Fragments.Last().ValueNum;
		Fragments.Last().bHasAnyZeroes |= bIsZero;
		ZeroMask.Add(bIsZero);
		bHasNonZeroValues |= !bIsZero;
	}

	void ExcludeProperty()
	{
		if (Fragments.Last().ValueNum || Fragments.Last().SkipNum == FFragment::SkipMax)
		{
			TrimZeroMask(Fragments.Last());
			Fragments.AddDefaulted();
		}

		++Fragments.Last().SkipNum;
	}

	void Finalize()
	{
		TrimZeroMask(Fragments.Last());

		// Trim trailing skips
		while (Fragments.Num() > 1 && Fragments.Last().ValueNum == 0)
		{
			checkf(!Fragments.Last().bHasAnyZeroes, TEXT("No values implies no zero-values"));			
			Fragments.Pop(EAllowShrinking::No);
		}

		Fragments.Last().bIsLast = true;
	}

private:
	void TrimZeroMask(const FFragment& Fragment)
	{
		if (!Fragment.bHasAnyZeroes)
		{
			ZeroMask.RemoveAt(ZeroMask.Num() - Fragment.ValueNum, Fragment.ValueNum);
		}
	}
};

static bool CanUseUnversionedPropertySerialization()
{
	bool bTemp;
	static bool bAllow = GConfig->GetBool(TEXT("Core.System"), TEXT("CanUseUnversionedPropertySerialization"), bTemp, GEngineIni) && bTemp;
	return bAllow;
}

static bool CanUseUnversionedPropertySerialization(const FConfigFile& TargetIni)
{
	bool bAllow;
	return TargetIni.GetBool(TEXT("Core.System"), TEXT("CanUseUnversionedPropertySerialization"), /* out */ bAllow) && bAllow;
}

static bool CanUseUnversionedPropertySerializationForServerOnly(const FConfigFile& TargetIni)
{
	bool bAllow;
	return TargetIni.GetBool(TEXT("Core.System"), TEXT("CanUseUnversionedPropertySerializationForServerOnly"), /* out */ bAllow) && bAllow;
}

static struct
{
	FRWLock Lock;
	TMap<uint32, bool> PlatformValues;
}
GUPSIniValueCache;

bool CanUseUnversionedPropertySerialization(const ITargetPlatform* Target)
{
	UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;

	if (!Target)
	{
		// Use current platform settings
		return CanUseUnversionedPropertySerialization();
	}

	const bool bIsServerOnly = Target->IsServerOnly();
	const int32 IsServerOnlyBit = bIsServerOnly << 31;
	const int32 TargetID = IsServerOnlyBit | Target->GetPlatformOrdinal();

	FWriteScopeLock Scope(GUPSIniValueCache.Lock);

	if (const bool* CachedValue = GUPSIniValueCache.PlatformValues.Find(TargetID))
	{
		return *CachedValue;
	}

	const FConfigFile& TargetIni = *FConfigCacheIni::ForPlatform(*Target->IniPlatformName())->FindConfigFile(GEngineIni);
	bool bTargetValue = CanUseUnversionedPropertySerialization(TargetIni);
	if (bIsServerOnly)
	{
		bTargetValue = bTargetValue && CanUseUnversionedPropertySerializationForServerOnly(TargetIni);
	}

			
	GUPSIniValueCache.PlatformValues.Add(TargetID, bTargetValue);

	return bTargetValue;
}

void DestroyUnversionedSchema(const UStruct* Struct)
{
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
	FUnversionedStructSchema::Delete(const_cast<FUnversionedStructSchema*>(Struct->UnversionedGameSchema.load(std::memory_order_acquire)));
	Struct->UnversionedGameSchema = nullptr;
#if WITH_EDITORONLY_DATA
	FUnversionedStructSchema::Delete(const_cast<FUnversionedStructSchema*>(Struct->UnversionedEditorSchema.load(std::memory_order_acquire)));
	Struct->UnversionedEditorSchema = nullptr;
#endif
#endif
}

#if WITH_EDITORONLY_DATA
static bool SkipEditorOnlyFields(FArchive& Ar) { return Ar.IsFilterEditorOnly(); }
#else
static constexpr bool SkipEditorOnlyFields(FArchive& Ar) { return true; }
#endif

void SerializeUnversionedProperties(const UStruct* Struct, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* DefaultsData)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord StructRecord = Slot.EnterRecord();

	if (UnderlyingArchive.IsLoading())
	{
		check(CanUseUnversionedPropertySerialization());

		FUnversionedHeader Header;
		Header.Load(StructRecord.EnterStream(TEXT("Header")));

		if (Header.HasValues())
		{
			FUnversionedSchemaRange Schema(Struct, SkipEditorOnlyFields(UnderlyingArchive));

			if (Header.HasNonZeroValues())
			{
				FDefaultStruct Defaults(DefaultsData, DefaultsStruct);

#if WITH_EDITORONLY_DATA
				bool bTrackInitializedProperties = false;

				// We cannot mark all properties as initialized when loading 'dense' property values.
				if (UnderlyingArchive.DoDelta() && !UnderlyingArchive.IsTransacting())
				{
					bTrackInitializedProperties = UE::FInitializedPropertyValueState(Struct, Data).IsTracking();
				}
#endif

				FStructuredArchive::FStream ValueStream = StructRecord.EnterStream(TEXT("Values"));
				for (FUnversionedHeader::FIterator It(Header, Schema); It; It.Next())
				{
					if (It.IsNonZero())
					{
#if WITH_EDITOR || UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING// Skip this scope to save time in the runtime; it is only needed for reference collection in editor
						FSerializedPropertyScope SerializedProperty(UnderlyingArchive, It.GetSerializer().GetProperty());
#endif

#if WITH_EDITORONLY_DATA
						if (bTrackInitializedProperties && It.GetSerializer().HasRequiredParam())
						{
							UE::FInitializedPropertyValueState(Struct, Data).Set(It.GetSerializer().GetProperty());
						}
#endif

						It.GetSerializer().Serialize(ValueStream.EnterElement(), Data, Defaults);
					}
					else
					{
						It.GetSerializer().LoadZero(Data);
					}
				}
			}
			else
			{
				for (FUnversionedHeader::FIterator It(Header, Schema); It; It.Next())
				{
					check(!It.IsNonZero());
					It.GetSerializer().LoadZero(Data);
				}	
			}
		}

	}
	else
	{
		FUnversionedPropertyTestRunner TestRunner({Struct, Data, DefaultsStruct, DefaultsData});
		FUnversionedPropertyTestCollector TestCollector;

		// Decide which properties to save and build header based on schema property indices
		const bool bDense = !UnderlyingArchive.DoDelta() || UnderlyingArchive.IsTransacting() || (!DefaultsData && !dynamic_cast<const UClass*>(Struct));
		FDefaultStruct Defaults(DefaultsData, DefaultsStruct);

		FUnversionedSchemaRange Schema(Struct, SkipEditorOnlyFields(UnderlyingArchive));
		FUnversionedHeaderBuilder Header;
		for (FUnversionedPropertySerializer Serializer : Schema)
		{
			if (Serializer.GetProperty()->ShouldSerializeValue(UnderlyingArchive) &&
				(bDense || !Serializer.IsDefault(Data, Defaults, UnderlyingArchive.GetPortFlags())))
			{
				Header.IncludeProperty(Serializer.ShouldSaveAsZero(Data));
				TestCollector.RecordSavedProperty(Serializer.GetProperty());
			}
			else
			{
				Header.ExcludeProperty();
			}
		}
		Header.Finalize();

		// Save header and non-zero values 
		Header.Save(StructRecord.EnterStream(TEXT("Header")));
		if (Header.HasNonZeroValues())
		{
			FStructuredArchive::FStream ValueStream = StructRecord.EnterStream(TEXT("Values"));
			for (FUnversionedHeader::FIterator It(Header, Schema); It; It.Next())
			{
				if (It.IsNonZero())
				{
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, It.GetSerializer().GetProperty());
					It.GetSerializer().Serialize(ValueStream.EnterElement(), Data, Defaults);
				}
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
const FBlake3Hash& GetSchemaHash(const UStruct* Struct, bool bSkipEditorOnly)
{
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
	return GetOrCreateUnversionedSchema(Struct, bSkipEditorOnly).SchemaHash;
#else
	static FBlake3Hash Placeholder;
	return Placeholder;
#endif
}

COREUOBJECT_API void DumpClassSchemas(const TCHAR* Str, FOutputDevice& Ar)
{
#if CACHE_UNVERSIONED_PROPERTY_SCHEMA
	TArray<FString> Lines;
	TArray<UStruct*> Structs;
	for (TObjectIterator<UStruct> It; It; ++It)
	{
		Structs.Add(*It);
	}
	Algo::Sort(Structs, [](UStruct* A, UStruct* B)
		{
			FNameBuilder AName;
			FNameBuilder BName;
			A->GetPathName(nullptr, AName);
			B->GetPathName(nullptr, BName);
			return FStringView(AName).Compare(FStringView(BName), ESearchCase::IgnoreCase) < 0;
		});
	bool bSkipEditorOnly = false;
	FParse::Bool(Str, TEXT("-SkipEditorOnly="), bSkipEditorOnly);
	for (UStruct* Struct : Structs)
	{
		const FBlake3Hash& ExistingHash = Struct->GetSchemaHash(bSkipEditorOnly);
		FBlake3Hash NewHash = FUnversionedStructSchema::CalculateSchemaHash(Struct, bSkipEditorOnly);
		ensureMsgf(ExistingHash == NewHash, TEXT("Hash mismatch for %s. Stored hash=%s, Current hash=%s"),
			*Struct->GetFullName(nullptr), *LexToString(ExistingHash), *LexToString(NewHash));
		Lines.Add(FString::Printf(TEXT("%-80s, %s"), *Struct->GetPathName(nullptr), *LexToString(ExistingHash)));
	}
	FString DumpFilename;
	FParse::Value(Str, TEXT("-FILE="), DumpFilename);
	if (!DumpFilename.IsEmpty())
	{
		FFileHelper::SaveStringArrayToFile(Lines, *DumpFilename);
	}
#endif
}

void FAppendToClassSchemaContext::Update(const void* Data, uint64 Size)
{
	FBlake3& Blake3Hasher = *(reinterpret_cast<FBlake3*>(Hasher));
	Blake3Hasher.Update(Data, Size);
}

#endif
