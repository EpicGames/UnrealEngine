// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/UnrealType.h"

#include "Serialization/UnversionedPropertySerializationInternal.h"

struct FBlake3Hash;

// Serialize sparse unversioned properties for a particular struct
void SerializeUnversionedProperties(const UStruct* Struct, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults);

void DestroyUnversionedSchema(const UStruct* Struct);

#if WITH_EDITORONLY_DATA
/**
 * Read the SchemaHash for the given struct, with our without editor-only properties.
 * This hash covers the Struct's data for both versioned and unversioned serialization.
 */
const FBlake3Hash& GetSchemaHash(const UStruct* Struct, bool bSkipEditorOnly);
#endif