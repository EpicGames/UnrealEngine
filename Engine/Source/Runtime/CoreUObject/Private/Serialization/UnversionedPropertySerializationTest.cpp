// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/UnversionedPropertySerializationTest.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/GuardValueAccessors.h"
#include "UObject/UnrealType.h"

#if UE_ENABLE_UNVERSIONED_PROPERTY_TEST

namespace PropertySerializationStats
{
	static TAtomic<uint64> Structs;
	static TAtomic<uint64> VersionedBytes;
	static TAtomic<uint64> UnversionedBytes;
	static TAtomic<uint64> UselessBytes;

#if ENABLE_COOK_STATS
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		AddStat(TEXT("UnversionedProperties"), 
				FCookStatsManager::CreateKeyValueArray(
					TEXT("SavedStructs"), Structs.Load(),
					TEXT("SavedMB"), static_cast<uint32>(UnversionedBytes.Load() >> 20),
					TEXT("EquivalentTaggedMB"), static_cast<uint32>(VersionedBytes.Load() >> 20),
					TEXT("CompressionRatio"), static_cast<float>(VersionedBytes.Load()) / static_cast<float>(UnversionedBytes.Load()),
					TEXT("BitfieldWasteKB"), static_cast<uint32>(UselessBytes.Load()) >> 10));
	});
#endif
}

// Serializes a UStruct to memory using both unversioned and versioned tagged property serialization,
// then creates two struct instances, loads the data back and compares that they are identical.
struct FUnversionedPropertyTest : public FUnversionedPropertyTestInput
{
	explicit FUnversionedPropertyTest(const FUnversionedPropertyTestInput& Input) : FUnversionedPropertyTestInput(Input) {}

	class FTestLinker : public FArchiveProxy
	{
	public:
		using FArchiveProxy::FArchiveProxy;

		virtual FArchive& operator<<(FName& Value) override
		{
			uint32 UnstableInt = Value.GetDisplayIndex().ToUnstableInt();
			int32 Number = Value.GetNumber();
			InnerArchive << UnstableInt << Number;

			if (IsLoading())
			{
				Value = FName::CreateFromDisplayId(FNameEntryId::FromUnstableInt(UnstableInt), Number);
			}

			return *this;
		}

		
		virtual FArchive& operator<<(FObjectPtr& Value) override
		{
			InnerArchive << reinterpret_cast<UPTRINT&>(Value);
			return *this;
		}

		virtual FArchive& operator<<(UObject*& Value) override
		{
			InnerArchive << reinterpret_cast<UPTRINT&>(Value);
			return *this;
		}

		virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); }
		virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return FArchiveUObject::SerializeSoftObjectPtr(*this, Value); }
		virtual FArchive& operator<<(FSoftObjectPath& Value) override { return FArchiveUObject::SerializeSoftObjectPath(*this, Value); }
		virtual FArchive& operator<<(FWeakObjectPtr& Value) override { return FArchiveUObject::SerializeWeakObjectPtr(*this, Value); }
	};

	enum class EPath { Versioned, Unversioned };

	static const TCHAR* ToString(EPath Path)
	{
		return Path == EPath::Unversioned ? TEXT("unversioned") : TEXT("versioned");
	}

	struct FSaveResult
	{
		TArray<uint8> Data;
		TArray<FProperty*> Properties;
		EPath Path;
	};

	static thread_local FSaveResult* TlsSaveResult;

	FSaveResult Save(EPath Path)
	{
		FSaveResult Result;
		Result.Path = Path;
		
		FMemoryWriter Writer(Result.Data);
		Writer.SetUseUnversionedPropertySerialization(Path == EPath::Unversioned);
		FTestLinker Linker(Writer);
		FBinaryArchiveFormatter Formatter(Linker);
		FStructuredArchive StructuredArchive(Formatter);
		FStructuredArchive::FSlot Slot = StructuredArchive.Open();

		TlsSaveResult = &Result;
		Struct->SerializeTaggedProperties(Slot, OriginalInstance, DefaultsStruct, Defaults);
		check(TlsSaveResult == nullptr);

		return Result;
	}

	struct FTestInstance
	{
		explicit FTestInstance(const UStruct* InType)
			: Type(InType)
		{
			Instance = FMemory::Malloc(Type->GetStructureSize(), Type->GetMinAlignment());
			Type->InitializeStruct(Instance);
		}

		FTestInstance(FTestInstance&& Other)
			: Type(Other.Type)
			, Instance(Other.Instance)
		{
			Other.Instance = nullptr;
		}

		~FTestInstance()
		{
			if (Instance)
			{
				Type->DestroyStruct(Instance);
				FMemory::Free(Instance);
			}
		}

		const UStruct* Type;
		void* Instance;
	};

	FTestInstance Load(const FSaveResult& Saved)
	{
		FMemoryReader Reader(Saved.Data);
		Reader.SetUseUnversionedPropertySerialization(Saved.Path == EPath::Unversioned);
		FTestLinker Linker(Reader);
		FBinaryArchiveFormatter Formatter(Linker);
		FStructuredArchive StructuredArchive(Formatter);
		FStructuredArchive::FSlot Slot = StructuredArchive.Open();

		TGuardValueAccessors<bool> IsSavingPackageGuard([]() { return UE::IsSavingPackage(); }, UE::SetIsSavingPackage, false);
		FTestInstance Result(Struct);
		// Call UStruct::SerializeTaggedProperties() directly to bypass
		// UUserDefinedStruct::SerializeTaggedProperties() for test loading,
		// since that is what the test saving does.
		Struct->UStruct::SerializeTaggedProperties(Slot, (uint8*)Result.Instance, DefaultsStruct, Defaults);

		checkf(Reader.Tell() == Saved.Data.Num(), TEXT("Failed to consume all %s saved property data"), ToString(Saved.Path));

		return Result;
	}

	static constexpr uint32 EqualsPortFlags = 0;

	struct FPropertyDiff
	{
		const FProperty* Property;
		const void* A;
		const void* B;
		const TCHAR* MismatchKind;

		FString GetType() const
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				return StructProperty->Struct->GetName();
			}

			return Property->GetClass()->GetName();
		}
	};

	// FProperty::Identical() flavor suited to comparing loaded instances
	static bool Equals(const FProperty* Property, const void* A, const void* B, FPropertyDiff& OutDiff)
	{
		if (Property->GetPropertyFlags() & (CPF_EditorOnly | CPF_Transient | CPF_NonNullable))
		{
			return true;
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			return Equals(StructProperty, A, B, OutDiff);
		}
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			return Equals(ArrayProperty, A, B, OutDiff);
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
		{
			return Equals(SetProperty, A, B, OutDiff);
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
		{
			return Equals(MapProperty, A, B, OutDiff);
		}
		else if (!Property->Identical(A, B, EqualsPortFlags))
		{
			OutDiff = {Property, A, B, TEXT("Identical()")};
			return false;
		}
		
		return true;
	}

	static bool Equals(const FArrayProperty* Property, const void* A, const void* B, FPropertyDiff& OutDiff)
	{
		FScriptArrayHelper HelperA(Property, A);
		FScriptArrayHelper HelperB(Property, B);

		if (HelperA.Num() != HelperB.Num())
		{
			OutDiff = {Property, A, B, TEXT("Num()")};
			return false;
		}

		for (int32 Idx = 0, Num = HelperA.Num(); Idx < Num; ++Idx)
		{
			if (!Equals(Property->Inner, HelperA.GetRawPtr(Idx), HelperB.GetRawPtr(Idx), OutDiff))
			{
				return false;
			}
		}

		return true;
	}

	static const uint8* FindElementPtr(const FScriptSetHelper& Helper, const uint8* Element, FPropertyDiff& OutDiff)
	{
		const FProperty* ElemProp = Helper.GetElementProperty();
		int32 Index = Helper.Set->FindIndex(Element, Helper.SetLayout,
											[ElemProp](const void* Element) { return ElemProp->GetValueTypeHash(Element); },
											[ElemProp, &OutDiff](const void* A, const void* B) { return Equals(ElemProp, A, B, OutDiff); });
		return Index >= 0 ? Helper.GetElementPtr(Index) : nullptr;
	}
	
	static bool Equals(const FSetProperty* Property, const void* A, const void* B, FPropertyDiff& OutDiff)
	{
		FScriptSetHelper HelperA(Property, A);
		FScriptSetHelper HelperB(Property, B);

		if (HelperA.Num() != HelperB.Num())
		{
			OutDiff = {Property, A, B, TEXT("Num()")};
			return false;
		}

		for (FScriptSetHelper::FIterator It(HelperA); It; ++It)
		{
			const uint8* ElemA = HelperA.GetElementPtr(It);
			const uint8* ElemB = FindElementPtr(HelperB, ElemA, OutDiff);

			if (!ElemB)
			{
				return false;
			}
		}

		return true;
	}

	static const uint8* FindPairPtr(const FScriptMapHelper& Helper, const uint8* Key, FPropertyDiff& OutDiff)
	{
		const FProperty* KeyProp = Helper.GetKeyProperty();
		int32 Index = Helper.HeapMap->FindPairIndex(Key, Helper.MapLayout,
												[KeyProp](const void* Key) { return KeyProp->GetValueTypeHash(Key); },
												[KeyProp, &OutDiff](const void* A, const void* B) { return Equals(KeyProp, A, B, OutDiff); });
		return Index >= 0 ? Helper.GetPairPtr(Index) : nullptr;
	}
	
	static bool Equals(const FMapProperty* Property, const void* A, const void* B, FPropertyDiff& OutDiff)
	{
		FScriptMapHelper HelperA(Property, A);
		FScriptMapHelper HelperB(Property, B);
		const FProperty* ValueProp = HelperA.GetValueProperty();
		int32 ValueOffset = HelperA.MapLayout.ValueOffset;

		if (HelperA.Num() != HelperB.Num())
		{
			OutDiff = {Property, A, B, TEXT("Num()")};
			return false;
		}
		
		for (FScriptMapHelper::FIterator It(HelperA); It; ++It)
		{
			const uint8* PairA = HelperA.GetPairPtr(It);
			const uint8* PairB = FindPairPtr(HelperB, PairA, OutDiff);

			if (!PairB)
			{
				return false;
			}

			if (!Equals(ValueProp, PairA + ValueOffset, PairB + ValueOffset, OutDiff))
			{
				return false;
			}
		}

		return true;
	}

	static bool Equals(const FStructProperty* Property, const void* A, const void* B, FPropertyDiff& OutDiff)
	{
		UScriptStruct* Struct = Property->Struct;
		if (Struct->StructFlags & STRUCT_IdenticalNative)
		{
			bool bResult = false;
			if (Struct->GetCppStructOps()->Identical(A, B, EqualsPortFlags, bResult))
			{
				if (!bResult)
				{
					OutDiff = {Property, A, B, TEXT("native operator==")};
					return false;
				}
				return true;
			}
		}

		// Skip deprecated fields
		for (TFieldIterator<FProperty> It(Struct, EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated); It; ++It)
		{
			for (int32 Idx = 0, MaxIdx = It->ArrayDim; Idx < MaxIdx; ++Idx)
			{
				if (!Equals_InContainer(*It, A, B, Idx, OutDiff))
				{
					return false;
				}
			}
		}
		return true;
	}

	static bool Equals_InContainer(const FProperty* Property, const void* A, const void* B, uint32 Idx, FPropertyDiff& OutDiff)
	{
		return Equals(Property, Property->ContainerPtrToValuePtr<void>(A, Idx), Property->ContainerPtrToValuePtr<void>(B, Idx), OutDiff);
	}

	static FString GetValueAsText(const FProperty* Property, uint32 ArrayIdx, const void* Instance)
	{
		FString Value;
		Property->ExportText_InContainer(ArrayIdx, Value, Instance, nullptr, nullptr, 0);
		return MoveTemp(Value);
	}

	static FString GetValueAsLimitedText(const FProperty* Property, uint32 ArrayIdx, const void* Instance)
	{
		FString Value = GetValueAsText(Property, ArrayIdx, Instance);
		if (Value.Len() > 100)
		{
			Value.LeftInline(100);
			Value += TEXT(" ... shortened");
		}
		return MoveTemp(Value);
	}

	void CheckEqual(FProperty* Property, void* VersionedInstance, void* UnversionedInstance)
	{
		for (int32 Idx = 0, Num = Property->ArrayDim; Idx < Num; ++Idx)
		{
			FPropertyDiff VersionedUnversionedDiff = {};
			if (!Equals_InContainer(Property, VersionedInstance, UnversionedInstance, Idx, VersionedUnversionedDiff))
			{
				FPropertyDiff VersionedOriginalDiff = {};
				FPropertyDiff UnversionedOriginalDiff = {};
				bool VersionedOk = Equals_InContainer(Property, VersionedInstance, OriginalInstance, Idx, VersionedOriginalDiff);
				bool UnversionedOk = Equals_InContainer(Property, UnversionedInstance, OriginalInstance, Idx, UnversionedOriginalDiff);
				const TCHAR* OkPaths = VersionedOk ? (UnversionedOk ? TEXT("Both paths") : TEXT("Versioned path"))
													: (UnversionedOk ? TEXT("Unversioned path") : TEXT("Neither path"));

				FString	VersionedValue = GetValueAsLimitedText(VersionedUnversionedDiff.Property, Idx, VersionedUnversionedDiff.A);
				FString	UnversionedValue = GetValueAsLimitedText(VersionedUnversionedDiff.Property, Idx, VersionedUnversionedDiff.B);
				const FPropertyDiff& OriginalDiff = VersionedOk ? UnversionedOriginalDiff : VersionedOriginalDiff;
				FString	OriginalValue = OriginalDiff.Property == VersionedUnversionedDiff.Property ? GetValueAsText(OriginalDiff.Property, Idx, OriginalDiff.B) : "(missing)";

				if (FPlatformMisc::IsDebuggerPresent())
				{
					// These strings might be too long to fit in the assert message.
					FString EntireVersionedValue = GetValueAsText(Property, Idx, VersionedInstance);
					FString EntireUnversionedValue = GetValueAsText(Property, Idx, UnversionedInstance);
					FString EntireOriginalValue = GetValueAsText(Property, Idx, OriginalInstance);

					// Put a breakpoint here if you need to debug UPS and/or TPS roundtripping

					FSaveResult VersionedSaved2 = Save(EPath::Versioned);
					FSaveResult UnversionedSaved2 = Save(EPath::Unversioned);

					FTestInstance VersionedLoaded2 = Load(VersionedSaved2);
					FTestInstance UnversionedLoaded2 = Load(UnversionedSaved2);
				}		

				checkf(false, TEXT("The %s %s.%s roundtripped differently in versioned / tagged vs unversioned property serialization. "
					"%s loaded an instance equal to the original. "
					"Inner mismatch in %s for the %s %s with UPS/TPS/Original values %s/%s/%s"), 
					*Property->GetClass()->GetName(), *Struct->GetName(), *Property->GetName(), OkPaths,
					VersionedUnversionedDiff.MismatchKind, *VersionedUnversionedDiff.GetType(),
					VersionedUnversionedDiff.Property ? *VersionedUnversionedDiff.Property->GetName() : TEXT("(null)"),
					*VersionedValue, *UnversionedValue, *OriginalValue);
			}
		}		
	}

	static TArray<FProperty*> ExcludeEditorOnlyProperties(const TArray<FProperty*>& Properties)
	{
		TArray<FProperty*> Out;
		Out.Reserve(Properties.Num());

		for (FProperty* Property : Properties)
		{
			if (!Property->IsEditorOnlyProperty())
			{
				Out.Add(Property);
			}
		}

		return Out;
	}

	void Run()
	{
		FSaveResult VersionedSaved = Save(EPath::Versioned);
		FSaveResult UnversionedSaved = Save(EPath::Unversioned);	

		check(VersionedSaved.Properties == UnversionedSaved.Properties || ExcludeEditorOnlyProperties(VersionedSaved.Properties) == UnversionedSaved.Properties);

		FTestInstance VersionedLoaded = Load(VersionedSaved);
		FTestInstance UnversionedLoaded = Load(UnversionedSaved);

		for (FProperty* Property : UnversionedSaved.Properties)
		{
			CheckEqual(Property, VersionedLoaded.Instance, UnversionedLoaded.Instance);
			PropertySerializationStats::UselessBytes += Property->IsA<FBoolProperty>() && !CastField<FBoolProperty>(Property)->IsNativeBool();
		}
		
		++PropertySerializationStats::Structs;
		PropertySerializationStats::VersionedBytes += VersionedSaved.Data.Num();
		PropertySerializationStats::UnversionedBytes += UnversionedSaved.Data.Num();
	}
};

thread_local FUnversionedPropertyTest::FSaveResult* FUnversionedPropertyTest::TlsSaveResult;
thread_local bool FUnversionedPropertyTestRunner::bTlsTesting;

void RunUnversionedPropertyTest(const FUnversionedPropertyTestInput& Input)
{
	FUnversionedPropertyTest Test(Input);
	Test.Run();
}

FUnversionedPropertyTestCollector::FUnversionedPropertyTestCollector()
{
	if (FUnversionedPropertyTest::FSaveResult* Result = FUnversionedPropertyTest::TlsSaveResult)
	{
		Out = &Result->Properties;

		// Nested SerializeTaggedProperties() call should not record nested properties
		FUnversionedPropertyTest::TlsSaveResult = nullptr;
	}
	else
	{
		Out = nullptr;
	}
}

#endif