// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ConfigCacheIni.h"

#define UE_ENABLE_UNVERSIONED_PROPERTY_TEST WITH_EDITORONLY_DATA

struct FUnversionedPropertyTestInput
{
	const UStruct* Struct;
	uint8* OriginalInstance;
	const UStruct* DefaultsStruct;
	const uint8* Defaults;
};

#if UE_ENABLE_UNVERSIONED_PROPERTY_TEST

void RunUnversionedPropertyTest(const FUnversionedPropertyTestInput& Input);

// Avoids starting tests recursively when we test serializing nested structs or struct containers.
// The test is both started from and recursively calls SerializeTaggedProperties().
struct FUnversionedPropertyTestRunner
{
	static thread_local bool bTlsTesting;
	bool bStartedTest = false;

	explicit FUnversionedPropertyTestRunner(const FUnversionedPropertyTestInput& Input)
	{
		bool bTemp;
		static bool bEnabled = GConfig->GetBool(TEXT("Core.System"), TEXT("TestUnversionedPropertySerializationWhenCooking"), bTemp, GEngineIni) && bTemp;
			
		if (bEnabled && !bTlsTesting)
		{
			bTlsTesting = true;
			bStartedTest = true;
			RunUnversionedPropertyTest(Input);
		}
	}

	~FUnversionedPropertyTestRunner()
	{
		if (bStartedTest)
		{
			bTlsTesting = false;
		}
	}
};

// Intrusive test helper that records which properties were saved
struct FUnversionedPropertyTestCollector
{
	TArray<FProperty*>* Out;

	FUnversionedPropertyTestCollector();
	
	void RecordSavedProperty(FProperty* Property)
	{
		if (Out)
		{
			Out->Add(Property);
		}
	}
};

#else  // !UE_ENABLE_UNVERSIONED_PROPERTY_TEST

struct FUnversionedPropertyTestRunner
{	
	explicit FUnversionedPropertyTestRunner(const FUnversionedPropertyTestInput& Input) {}
};

struct FUnversionedPropertyTestCollector
{
	void RecordSavedProperty(FProperty* Property) {}
};

#endif