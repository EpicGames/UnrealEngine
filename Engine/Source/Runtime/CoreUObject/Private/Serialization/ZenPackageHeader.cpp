// Copyright Epic Games, Inc. All Rights Reserved.

#include "Serialization/ZenPackageHeader.h"

#include "Serialization/MemoryReader.h"

FZenPackageHeader FZenPackageHeader::MakeView(FMemoryView Memory)
{
	FString Error;
	FZenPackageHeader Result = MakeView(Memory, Error);
	if (!Error.IsEmpty())
	{
		UE_LOGF(LogStreaming, Fatal, "%ls", *Error);
	}
	return Result;
}

FZenPackageHeader FZenPackageHeader::MakeView(FMemoryView Memory, FString& OutError)
{
	OutError.Reset();

	FZenPackageHeader PackageHeader;
	const uint8* PackageHeaderDataPtr = reinterpret_cast<const uint8*>(Memory.GetData());
	const FZenPackageSummary* PackageSummary = reinterpret_cast<const FZenPackageSummary*>(PackageHeaderDataPtr);
	PackageHeader.PackageSummary = PackageSummary;
	checkf(PackageSummary->HeaderSize <= Memory.GetSize(), TEXT("Corrupted package: HeaderSize=%u exceeds MemorySize=%llu"), PackageSummary->HeaderSize, Memory.GetSize());

	TArrayView<const uint8> PackageHeaderDataView(PackageHeaderDataPtr + sizeof(FZenPackageSummary), PackageHeader.PackageSummary->HeaderSize - sizeof(FZenPackageSummary));
	FMemoryReaderView PackageHeaderDataReader(PackageHeaderDataView);
	if (PackageHeader.PackageSummary->bHasVersioningInfo)
	{
		PackageHeader.VersioningInfo.Emplace();
		PackageHeaderDataReader << PackageHeader.VersioningInfo.GetValue();
	}
	const FZenPackageVersioningInfo* LocalVersioningInfo = PackageHeader.VersioningInfo.GetPtrOrNull();

	FZenPackageCellOffsets CellOffsets;
	if (LocalVersioningInfo == nullptr || LocalVersioningInfo->PackageVersion >= EUnrealEngineObjectUE5Version::VERSE_CELLS)
	{
		PackageHeaderDataReader << CellOffsets.CellImportMapOffset;
		PackageHeaderDataReader << CellOffsets.CellExportMapOffset;
	}
	else
	{
		CellOffsets.CellImportMapOffset = PackageSummary->ExportBundleEntriesOffset;
		CellOffsets.CellExportMapOffset = PackageSummary->ExportBundleEntriesOffset;
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LoadPackageNameMap);
		PackageHeader.NameMap.Load(PackageHeaderDataReader, FMappedName::EType::Package);
	}
	PackageHeader.PackageName = PackageHeader.NameMap.GetName(PackageHeader.PackageSummary->Name);
	
	if (LocalVersioningInfo == nullptr || LocalVersioningInfo->PackageVersion >= EUnrealEngineObjectUE5Version::DATA_RESOURCES)
	{
		int64 BulkDataMapSize = 0;
		uint64 BulkDataPad = 0;
		PackageHeaderDataReader << BulkDataPad;
		uint8 PadBytes[sizeof(uint64)] = {};
		PackageHeaderDataReader.Serialize(PadBytes, BulkDataPad);
		PackageHeaderDataReader << BulkDataMapSize;
		const uint8* BulkDataMapData = PackageHeaderDataPtr + sizeof(FZenPackageSummary) + PackageHeaderDataReader.Tell();
		PackageHeader.BulkDataMap = MakeArrayView(reinterpret_cast<const FBulkDataMapEntry*>(BulkDataMapData), BulkDataMapSize / sizeof(FBulkDataMapEntry));
	}

	PackageHeader.ImportedPublicExportHashes = TArrayView<const uint64>(
		reinterpret_cast<const uint64*>(PackageHeaderDataPtr + PackageSummary->ImportedPublicExportHashesOffset),
		(PackageSummary->ImportMapOffset - PackageSummary->ImportedPublicExportHashesOffset) / sizeof(uint64));
	PackageHeader.ImportMap = TArrayView<const FPackageObjectIndex>(
		reinterpret_cast<const FPackageObjectIndex*>(PackageHeaderDataPtr + PackageSummary->ImportMapOffset),
		(PackageSummary->ExportMapOffset - PackageSummary->ImportMapOffset) / sizeof(FPackageObjectIndex));
	PackageHeader.ExportMap = TArrayView<const FExportMapEntry>(
		reinterpret_cast<const FExportMapEntry*>(PackageHeaderDataPtr + PackageSummary->ExportMapOffset),
		(CellOffsets.CellImportMapOffset - PackageSummary->ExportMapOffset) / sizeof(FExportMapEntry));
	PackageHeader.CellImportMap = TArrayView<const FPackageObjectIndex>(
		reinterpret_cast<const FPackageObjectIndex*>(PackageHeaderDataPtr + CellOffsets.CellImportMapOffset),
		(CellOffsets.CellExportMapOffset - CellOffsets.CellImportMapOffset) / sizeof(FPackageObjectIndex));
	PackageHeader.CellExportMap = TArrayView<const FCellExportMapEntry>(
		reinterpret_cast<const FCellExportMapEntry*>(PackageHeaderDataPtr + CellOffsets.CellExportMapOffset),
		(PackageSummary->ExportBundleEntriesOffset - CellOffsets.CellExportMapOffset) / sizeof(FCellExportMapEntry));
	PackageHeader.ExportCount = PackageHeader.ExportMap.Num();

	const uint64 ExportBundleEntriesSize = PackageHeader.PackageSummary->DependencyBundleHeadersOffset - PackageHeader.PackageSummary->ExportBundleEntriesOffset;
	const int32 ExportBundleEntriesCount = static_cast<int32>(ExportBundleEntriesSize / sizeof(FExportBundleEntry));

	if (ExportBundleEntriesCount != (PackageHeader.ExportCount + PackageHeader.CellExportMap.Num()) * FExportBundleEntry::ExportCommandType_Count)
	{
		OutError = FString::Printf(TEXT("Corrupt Zen header in package %s"), *PackageHeader.PackageName.ToString());
		return PackageHeader;
	}

	PackageHeader.ExportBundleEntries = TArrayView<const FExportBundleEntry>(
		reinterpret_cast<const FExportBundleEntry*>(PackageHeaderDataPtr + PackageHeader.PackageSummary->ExportBundleEntriesOffset),
		ExportBundleEntriesCount);
	PackageHeader.DependencyBundleHeaders = TArrayView<const FDependencyBundleHeader>(
		reinterpret_cast<const FDependencyBundleHeader*>(PackageHeaderDataPtr + PackageSummary->DependencyBundleHeadersOffset),
		(PackageSummary->DependencyBundleEntriesOffset - PackageSummary->DependencyBundleHeadersOffset) / sizeof(FDependencyBundleHeader));
	PackageHeader.DependencyBundleEntries = TArrayView<const FDependencyBundleEntry>(
		reinterpret_cast<const FDependencyBundleEntry*>(PackageHeaderDataPtr + PackageSummary->DependencyBundleEntriesOffset),
		(PackageSummary->ImportedPackageNamesOffset - PackageSummary->DependencyBundleEntriesOffset) / sizeof(FDependencyBundleEntry));

#if WITH_EDITOR
	// Only read the imported package names in editor builds where we need to interact with uncooked packages
	TArrayView<const uint8> ImportedPackageNamesDataView(PackageHeaderDataPtr + PackageSummary->ImportedPackageNamesOffset, PackageSummary->HeaderSize - PackageSummary->ImportedPackageNamesOffset);
	FMemoryReaderView ImportedPackageNamesDataReader(ImportedPackageNamesDataView);
	FZenPackageImportedPackageNamesContainer Container;
	ImportedPackageNamesDataReader << Container;
	PackageHeader.ImportedPackageNames = MoveTemp(Container.Names);
#endif

	return PackageHeader;
}

void FZenPackageHeader::Reset()
{
	PackageSummary = nullptr;
	ImportedPublicExportHashes = TArrayView<const uint64>();
	ImportMap = TArrayView<const FPackageObjectIndex>();
	ExportMap = TArrayView<const FExportMapEntry>();
	CellImportMap = TArrayView<const FPackageObjectIndex>();
	CellExportMap = TArrayView<const FCellExportMapEntry>();
	BulkDataMap = TArrayView<const FBulkDataMapEntry>();
	ExportBundleEntries = TArrayView<const FExportBundleEntry>();
	DependencyBundleHeaders = TArrayView<const FDependencyBundleHeader>();
	DependencyBundleEntries = TArrayView<const FDependencyBundleEntry>();
}
