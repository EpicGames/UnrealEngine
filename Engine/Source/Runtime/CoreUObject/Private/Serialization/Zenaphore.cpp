// Copyright Epic Games, Inc. All Rights Reserved.

#include "Zenaphore.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"

FZenaphore::FZenaphore()
{
	Event = FPlatformProcess::GetSynchEventFromPool(true);
}

FZenaphore::~FZenaphore()
{
	FPlatformProcess::ReturnSynchEventToPool(Event);
}

void FZenaphore::NotifyInternal(FZenaphoreWaiterNode* Waiter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ZenaphoreTrigger);
	check(Waiter);
	FScopeLock Lock(&Mutex);
	Waiter->bTriggered = true;
	Event->Trigger();
}

void FZenaphore::NotifyOne()
{
	for (;;)
	{
		FZenaphoreWaiterNode* Waiter = HeadWaiter.Load();
		if (!Waiter)
		{
			return;
		}
		if (HeadWaiter.CompareExchange(Waiter, Waiter->Next))
		{
			NotifyInternal(Waiter);
			return;
		}
	}
}

void FZenaphore::NotifyAll()
{
	for (;;)
	{
		FZenaphoreWaiterNode* Waiter = HeadWaiter.Load();
		if (!Waiter)
		{
			return;
		}
		if (HeadWaiter.CompareExchange(Waiter, Waiter->Next))
		{
			NotifyInternal(Waiter);
		}
	}
}

FZenaphoreWaiter::~FZenaphoreWaiter()
{
	if (SpinCount)
	{
		WaitInternal();
	}
}

void FZenaphoreWaiter::WaitInternal()
{
	for (;;)
	{
		Outer.Event->Wait(INT32_MAX, true);
		FScopeLock Lock(&Outer.Mutex);
		if (WaiterNode.bTriggered)
		{
			Outer.Event->Reset();
			return;
		}
	}
}

void FZenaphoreWaiter::Wait()
{
	if (SpinCount == 0)
	{
		FZenaphoreWaiterNode* OldHeadWaiter = nullptr;
		WaiterNode.bTriggered = false;
		WaiterNode.Next = nullptr;
		while (!Outer.HeadWaiter.CompareExchange(OldHeadWaiter, &WaiterNode))
		{
			WaiterNode.Next = OldHeadWaiter;
		}
		++SpinCount;
	}
	else
	{
		WaitInternal();
		SpinCount = 0;
	}
}
