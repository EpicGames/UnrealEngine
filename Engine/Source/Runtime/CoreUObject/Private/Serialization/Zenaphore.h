// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FZenaphore;

struct FZenaphoreWaiterNode
{
	FZenaphoreWaiterNode* Next = nullptr;
	bool bTriggered = false;
};

class FZenaphoreWaiter
{
public:
	FZenaphoreWaiter(FZenaphore& Outer, const TCHAR* WaitCpuScopeName)
		: Outer(Outer)
	{
	}

	~FZenaphoreWaiter();

	void Wait();

private:
	friend class FZenaphore;

	void WaitInternal();

	FZenaphore& Outer;
	FZenaphoreWaiterNode WaiterNode;
	int32 SpinCount = 0;
};

class FZenaphore
{
public:
	FZenaphore();
	~FZenaphore();
	void NotifyOne();
	void NotifyAll();

private:
	friend class FZenaphoreWaiter;

	void NotifyInternal(FZenaphoreWaiterNode* Waiter);

	FEvent* Event;
	FCriticalSection Mutex;
	TAtomic<FZenaphoreWaiterNode*> HeadWaiter { nullptr };
};

