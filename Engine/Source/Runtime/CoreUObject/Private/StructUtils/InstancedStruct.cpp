// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/InstancedStruct.h"

#include "Misc/PackageName.h"
#include "StructUtils/StructView.h"
#include "Serialization/CustomVersion.h"
#include "UObject/Linker.h"
#include "UObject/CoreRedirects.h"

#if WITH_EDITOR
#include "StructUtils/StructReinstancer.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedStruct)


struct FInstancedStructCustomVersion
{
	enum EType
	{
		// Before any version changes were made
		CustomVersionAdded = 0,

		// -----<new versions can be added above this line>-----
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const FGuid GUID;
	FCustomVersionRegistration Registration;

	FInstancedStructCustomVersion()
	: GUID{0xE21E1CAA, 0xAF47425E, 0x89BF6AD4, 0x4C44A8BB}
	, Registration(GUID, FInstancedStructCustomVersion::LatestVersion, TEXT("InstancedStructCustomVersion"))
	{}
};
static FInstancedStructCustomVersion GInstancedStructCustomVersion;

FInstancedStruct::FNetSerializeInstancedStruct FInstancedStruct::NetSerializeScriptStructDelegate;

FInstancedStruct::FInstancedStruct()
{
}

FInstancedStruct::FInstancedStruct(const UScriptStruct* InScriptStruct)
{
	InitializeAs(InScriptStruct, nullptr);
}

FInstancedStruct::FInstancedStruct(const FConstStructView InOther)
{
	InitializeAs(InOther.GetScriptStruct(), InOther.GetMemory());
}

FInstancedStruct& FInstancedStruct::operator=(const FConstStructView InOther)
{
	if (FConstStructView(*this) != InOther)
	{
		InitializeAs(InOther.GetScriptStruct(), InOther.GetMemory());
	}
	return *this;
}

void FInstancedStruct::InitializeAs(const UScriptStruct* InScriptStruct, const uint8* InStructMemory /*= nullptr*/)
{
	const UScriptStruct* CurrentScriptStruct = GetScriptStruct();
	if (InScriptStruct && InScriptStruct == CurrentScriptStruct)
	{
		// Struct type already matches...
		if (InStructMemory)
		{
			// ... apply the given state
			CurrentScriptStruct->CopyScriptStruct(GetMutableMemory(), InStructMemory);
		}
		else
		{
			// ... return the struct to its default state
			CurrentScriptStruct->ClearScriptStruct(GetMutableMemory());
		}
	}
	else
	{
		// Struct type mismatch; reset and reinitialize
		Reset();

		// InScriptStruct == nullptr signifies an empty, unset FInstancedStruct instance
		if (InScriptStruct)
		{
			const int32 MinAlignment = InScriptStruct->GetMinAlignment();
			const int32 RequiredSize = InScriptStruct->GetStructureSize();
			uint8* Memory = static_cast<uint8*>(FMemory::Malloc(FMath::Max(1, RequiredSize), MinAlignment));
			SetStructData(InScriptStruct, Memory);

			InScriptStruct->InitializeStruct(GetMutableMemory());

			if (InStructMemory)
			{
				InScriptStruct->CopyScriptStruct(GetMutableMemory(), InStructMemory);
			}
		}
	}
}

void FInstancedStruct::Reset()
{
	if (uint8* Memory = GetMutableMemory())
	{
		check(StructMemory != nullptr);

		// We check if the struct is still valid, otherwise `static` Instanced Structs can crash here during teardown
		if (ScriptStruct != nullptr && !ScriptStruct->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed | RF_MirroredGarbage))
		{
			ScriptStruct->DestroyStruct(GetMutableMemory());
		}
		FMemory::Free(Memory);
	}
	ResetStructData();
}

bool FInstancedStruct::Serialize(FArchive& Ar, UStruct* DefaultsStruct, const void* Defaults)
{
	Ar.UsingCustomVersion(GInstancedStructCustomVersion.GUID);

	const bool bValidDefaults = DefaultsStruct && DefaultsStruct->IsChildOf(FInstancedStruct::StaticStruct());
	const FInstancedStruct* DefaultInstancedStruct = bValidDefaults ? static_cast<const FInstancedStruct*>(Defaults) : nullptr;

	auto SerializeWithDefaults = [Self = this, &Ar, DefaultInstancedStruct](UScriptStruct* ScriptStruct)
		{
			const bool bDuplicate = (0 != (Ar.GetPortFlags() & PPF_Duplicate));
			const bool bSavingLoading = (Ar.IsSaving() || Ar.IsLoading());
			const bool bUseDefaults = !bDuplicate
				&& bSavingLoading
				&& !Ar.IsCooking()
				&& !Ar.WantBinaryPropertySerialization();

			if (bUseDefaults)
			{
				const bool bSameStruct = DefaultInstancedStruct && DefaultInstancedStruct->GetScriptStruct() == ScriptStruct;
				if (bSameStruct)
				{
					// Use the parent Defaults
					ScriptStruct->SerializeItem(Ar, Self->GetMutableMemory(), DefaultInstancedStruct->GetMemory());
				}
				else
				{
					// Create a temporary struct on the stack. The struct has default values.
					const int32 MinAlignment = ScriptStruct->GetMinAlignment();
					const int32 RequiredSize = ScriptStruct->GetStructureSize();
					TArray<uint8, TInlineAllocator<64>> TmpMemoryBuffer;
					TmpMemoryBuffer.SetNumUninitialized(FMath::Max(1, RequiredSize) + FMath::Max(0, MinAlignment - (int32)MIN_ALIGNMENT));
					void* TempMemory = Align(TmpMemoryBuffer.GetData(), MinAlignment);
					ScriptStruct->InitializeStruct(TempMemory);
					ScriptStruct->SerializeItem(Ar, Self->GetMutableMemory(), TempMemory);
					ScriptStruct->DestroyStruct(TempMemory);
				}
			}
			else
			{
				// Serialize without defaults
				ScriptStruct->SerializeItem(Ar, Self->GetMutableMemory(), nullptr);
			}
		};

	if (Ar.IsLoading())
	{
		const int32 CustomVersion = Ar.CustomVer(GInstancedStructCustomVersion.GUID);
		if (CustomVersion < FInstancedStructCustomVersion::CustomVersionAdded)
		{
			// The old format had "header+version" in editor builds, and just "version" otherwise.
			// If the first thing we read is the old header, consume it, if not go back and assume that we have just the version.
			const int64 HeaderOffset = Ar.Tell();
			uint32 Header = 0;
			Ar << Header;

			constexpr uint32 LegacyEditorHeader = 0xABABABAB;
			if (Header != LegacyEditorHeader)
			{
				Ar.Seek(HeaderOffset);
			}

			uint8 Version = 0;
			Ar << Version;
		}
	}

	if (Ar.IsCountingMemory() || Ar.IsModifyingWeakAndStrongReferences() || Ar.IsObjectReferenceCollector())
	{
		// Report type
		Ar << ScriptStruct;

		// Report value
		if (ScriptStruct != nullptr && ensureMsgf(GetMutableMemory() != nullptr, TEXT("A valid script struct should always have allocated memory")))
		{
			SerializeWithDefaults(ConstCast(ScriptStruct));
		}
	}
	else if (Ar.IsLoading())
	{
		// UScriptStruct type
		TObjectPtr<UScriptStruct> SerializedScriptStruct = nullptr;
		Ar << SerializedScriptStruct;
		if (SerializedScriptStruct)
		{
#if WITH_EDITOR
			// During reinstantiation, the serialized struct can be the the duplicated struct.
			//Replace it with the new version.
			if (const UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance();
				Reinstancer && Reinstancer->IsSerializingForReinstantiation())
			{
				if (const UScriptStruct* CompiledStruct = Reinstancer->GetCompiledReinstantingStruct(SerializedScriptStruct))
				{
					SerializedScriptStruct = const_cast<UScriptStruct*>(CompiledStruct);
				}
			}
#endif

			Ar.Preload(SerializedScriptStruct);
		}

		// Seed from the default's memory only on an exact type match. That memory is sized for the
		// default's struct, so copying a larger subclass layout out of it would read past the allocation.
		if (DefaultInstancedStruct && SerializedScriptStruct && SerializedScriptStruct == DefaultInstancedStruct->GetScriptStruct())
		{
			InitializeAs(SerializedScriptStruct, DefaultInstancedStruct->GetMemory());
		}
		else
		{
			InitializeAs(SerializedScriptStruct);
		}

		// Size of the serialized memory
		int32 SerialSize = 0; 
		Ar << SerialSize;

		// Serialized memory
		if (ScriptStruct == nullptr && SerialSize > 0)
		{
			// A null struct indicates an old struct or an unsupported one for the current target.
			// In this case we manually seek in the archive to skip its serialized content. 
			// We don't want to rely on TaggedSerialization that will mark an error in the archive that
			// may cause other serialization to fail (e.g. FArchive& operator<<(FArchive& Ar, TArray& A))
			UE_LOGF(LogCore, Warning, "Unable to find serialized UScriptStruct -> Advance %u bytes in the archive and reset to empty FInstancedStruct. SerializedProperty:%ls LinkerRoot:%ls",
				SerialSize,
				*GetPathNameSafe(Ar.GetSerializedProperty()),
				Ar.GetLinker() ? *GetPathNameSafe(Ar.GetLinker()->LinkerRoot) : TEXT("NoLinker"));
			Ar.Seek(Ar.Tell() + SerialSize);
		}
		else if (ScriptStruct != nullptr && ensureMsgf(GetMutableMemory() != nullptr, TEXT("A valid script struct should always have allocated memory")))
		{
			SerializeWithDefaults(ConstCast(ScriptStruct));
		}
	}
	else
	{
		// Saving + inspector archives (e.g. FArchiveCrc32 used by UE::StructUtils::GetStructInstanceCrc32,
		// or the CityHash64-based archive used by GetStructHash64). The serial-size backpatch is gated on
		// the archive being seekable - non-seekable archives (FArchive::Tell defaults to INDEX_NONE) don't
		// need it. Without this branch the FInstancedStruct payload would not contribute to the hash, and
		// shared fragments differing only in their FInstancedStruct member would be wrongly aliased.
		Ar << ScriptStruct;

		// Size of the serialized memory (reserve location)
		const int64 SizeOffset = Ar.Tell(); // Position to write the actual size after struct serialization
		int32 SerialSize = 0;
		if (SizeOffset != INDEX_NONE)
		{
			Ar << SerialSize;
		}

		// Serialized memory
		const int64 InitialOffset = Ar.Tell(); // Position before struct serialization to compute its serial size
		if (ScriptStruct != nullptr && ensureMsgf(GetMutableMemory() != nullptr, TEXT("A valid script struct should always have allocated memory")))
		{
			SerializeWithDefaults(ConstCast(ScriptStruct));
		}

		if (SizeOffset != INDEX_NONE)
		{
			const int64 FinalOffset = Ar.Tell(); // Keep current offset to reset the archive pos after write the serial size

			// Size of the serialized memory
			Ar.Seek(SizeOffset);	// Go back in the archive to write the actual size
			SerialSize = static_cast<int32>(FinalOffset - InitialOffset);
			Ar << SerialSize;
			Ar.Seek(FinalOffset);	// Reset archive to its position
		}
	}

	return true;
}

bool FInstancedStruct::ExportTextItem(FString& ValueStr, FInstancedStruct const& DefaultValue, class UObject* Parent, int32 PortFlags, class UObject* ExportRootScope) const
{
	if (const UScriptStruct* StructTypePtr = GetScriptStruct())
	{
		ValueStr += StructTypePtr->GetPathName();
		// Force the default to Value to disable delta serialization because we reset the memory in import text.
		StructTypePtr->ExportText(ValueStr, GetMemory(), GetMemory(), Parent, PortFlags, ExportRootScope);
	}
	else
	{
		ValueStr += TEXT("None");
	}
	return true;
}

bool FInstancedStruct::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive /*= nullptr*/)
{
	FNameBuilder StructPathName;
	if (FCString::Strcmp(Buffer, TEXT("()")) == 0)
	{
		// UHT uses "()" as a general "empty struct" marker, so allow importing that as an alias for "None"
		Buffer += 2;
	}
	else if (const TCHAR* Result = FPropertyHelpers::ReadToken(Buffer, StructPathName, /*bDottedNames*/true))
	{
		Buffer = Result;
	}
	else
	{
		return false;
	}

	if (StructPathName.Len() == 0 || FCString::Stricmp(StructPathName.ToString(), TEXT("None")) == 0)
	{
		InitializeAs(nullptr);
	}
	else
	{
		// Redirect the struct name if required
		FCoreRedirectObjectName OldName = FCoreRedirectObjectName(StructPathName.ToString());
		FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Struct, OldName, ECoreRedirectMatchFlags::AllowPartialMatch);
		if(OldName != NewName)
		{
			StructPathName.Reset();
			StructPathName.Append(NewName.ToString());
		}

		// Extra validation when pasting, as we may be importing random data from the clipboard and this avoids LoadObject warning log spam
		if ((PortFlags & PPF_Copy) && !FPackageName::IsValidObjectPath(StructPathName.ToView()))
		{
			return false;
		}

		// Make sure the struct is actually loaded before trying to import the text (this boils down to FindObject if the struct is already loaded).
		// This is needed for user defined structs, BP pin values, config, copy/paste, where there's no guarantee that the referenced struct has actually been loaded yet.
		UScriptStruct* StructTypePtr = LoadObject<UScriptStruct>(nullptr, StructPathName.ToString());
		if (!StructTypePtr)
		{
			return false;
		}

		InitializeAs(StructTypePtr);
		if (const TCHAR* Result = StructTypePtr->ImportText(Buffer, GetMutableMemory(), Parent, PortFlags, ErrorText, [StructTypePtr]() { return StructTypePtr->GetName(); }))
		{
			Buffer = Result;
		}
		else
		{
			return false;
		}
	}

	return true;
}

bool FInstancedStruct::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName NAME_StructVariant = "StructVariant";
	if (Tag.GetType().IsStruct(NAME_StructVariant))
	{
		auto SerializeStructVariant = [this, &Slot]()
		{
			FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
			FStructuredArchive::FRecord Record = Slot.EnterRecord();

			// Serialize the struct type
			UScriptStruct* StructTypePtr = nullptr;
			Record << SA_VALUE(TEXT("StructType"), StructTypePtr);
			if (StructTypePtr)
			{
				UnderlyingArchive.Preload(StructTypePtr);
			}
			InitializeAs(StructTypePtr);

			auto SerializeStructInstance = [this, StructTypePtr, &Record]()
			{
				if (StructTypePtr)
				{
					StructTypePtr->SerializeItem(Record.EnterField(TEXT("StructInstance")), GetMutableMemory(), nullptr);
				}
			};

			// Serialize the struct instance, potentially tagging it with its serialized size 
			// in-case the struct is deleted later and we need to step over the instance data
			if (!UnderlyingArchive.IsTextFormat())
			{
				// Read the serialized size
				int64 StructInstanceSerializedSize = 0;
				UnderlyingArchive << StructInstanceSerializedSize;

				// Serialize the struct instance
				const int64 StructInstanceStartOffset = UnderlyingArchive.Tell();
				SerializeStructInstance();
				const int64 StructInstanceEndOffset = UnderlyingArchive.Tell();

				// Ensure we're at the correct location after serializing the instance data
				const int64 ExpectedStructInstanceEndOffset = StructInstanceStartOffset + StructInstanceSerializedSize;
				if (StructInstanceEndOffset != ExpectedStructInstanceEndOffset)
				{
					if (StructTypePtr)
					{
						// We only expect a mismatch here if the underlying struct is no longer available!
						UnderlyingArchive.SetCriticalError();
						UE_LOGF(LogCore, Error, "FStructVariant expected to read %lld bytes for struct %ls but read %lld bytes!", StructInstanceSerializedSize, *StructTypePtr->GetName(), StructInstanceEndOffset - StructInstanceStartOffset);
					}
					UnderlyingArchive.Seek(ExpectedStructInstanceEndOffset);
				}
			}
			else
			{
				SerializeStructInstance();
			}
		};

		SerializeStructVariant();
		return true;
	}
	
	if (Tag.Type == NAME_StructProperty)
	{
		FPropertyTag InnerPropertyTag;
		InnerPropertyTag.SetType(Tag.GetType().GetParameter(0));
		InnerPropertyTag.Name = Tag.Name;
		InnerPropertyTag.ArrayIndex = 0;

		const FString StructName = *WriteToString<64>(Tag.GetType().GetParameter(0).GetName());
		if (UScriptStruct* StructTypePtr = FindFirstObject<UScriptStruct>(*StructName
			, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous))
		{
			InitializeAs(StructTypePtr);
			StructTypePtr->SerializeItem(Slot, GetMutableMemory(), /*Defaults*/nullptr);

			return true;
		}
	}

	return false;
}

void FInstancedStruct::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	if (ScriptStruct)
	{
		OutDeps.Add(ConstCast(ScriptStruct));

		// Report direct dependencies of the instanced struct
		if (UScriptStruct::ICppStructOps* CppStructOps = GetScriptStruct()->GetCppStructOps())
		{
			CppStructOps->GetPreloadDependencies(GetMutableMemory(), OutDeps);
		}

		// Report indirect dependencies of the instanced struct
		// The iterator will recursively loop through all structs in structs/containers too
		for (TPropertyValueIterator<FStructProperty> It(GetScriptStruct(), GetMutableMemory()); It; ++It)
		{
			const UScriptStruct* StructType = It.Key()->Struct;
			if (UScriptStruct::ICppStructOps* CppStructOps = StructType->GetCppStructOps())
			{
				void* StructDataPtr = const_cast<void*>(It.Value());
				CppStructOps->GetPreloadDependencies(StructDataPtr, OutDeps);
			}
		}
	}
}

bool FInstancedStruct::Identical(const FInstancedStruct* Other, uint32 PortFlags) const
{
	const UScriptStruct* StructTypePtr = GetScriptStruct();
	if (!Other || StructTypePtr != Other->GetScriptStruct())
	{
		return false;
	}

	if (StructTypePtr)
	{
#if WITH_EDITOR
		// Struct can't be identical when re-instancing. The struct has to serialize to get the new values.
		if (const UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance())
		{
			if (Reinstancer->IsSerializingForReinstantiation()
				&& Reinstancer->IsReinstanting(StructTypePtr))
			{
				return false;
			}
		}
#endif

		return StructTypePtr->CompareScriptStruct(GetMemory(), Other->GetMemory(), PortFlags);
	}

	return true;
}

void FInstancedStruct::AddStructReferencedObjects(FReferenceCollector& Collector)
{
#if WITH_EDITOR
	// Reference collector is used to visit all instances of instanced structs and replace their contents.
	if (UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance())
	{
		if (ScriptStruct && Reinstancer->IsReinstanting(ScriptStruct))
		{
			Reinstancer->MarkAsRequiresReinstantiation();
			// It is not necessary to change the struct to a duplicated one, but it helps with tracking mistakes and prevents memory stomps.
			if (const UScriptStruct* Duplicated = Reinstancer->GetDuplicatedReinstantingStruct(ScriptStruct))
			{
				ScriptStruct = Duplicated;
			}
		}
	}
#endif // WITH_EDITOR
	
	if (ScriptStruct != nullptr)
	{
		Collector.AddReferencedObject(ScriptStruct);
		Collector.AddPropertyReferencesWithStructARO(ScriptStruct, GetMutableMemory());
	}
}

#if WITH_EDITOR
void FInstancedStruct::ReplaceScriptStructInternal(const UScriptStruct* NewStruct)
{
	ScriptStruct = NewStruct;
}
#endif // WITH_EDITOR

bool FInstancedStruct::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	uint8 bValidData = Ar.IsSaving() ? IsValid() : 0;
	Ar.SerializeBits(&bValidData, 1);

	if (bValidData)
	{
		if (Ar.IsLoading())
		{
			TObjectPtr<UScriptStruct> SerializedScriptStruct = nullptr;

			Ar << SerializedScriptStruct;

			// Initialize only if the type changes.
			if (ScriptStruct != SerializedScriptStruct)
			{
				InitializeAs(SerializedScriptStruct);
			}

			if (!IsValid())
			{
				UE_LOGF(LogCore, Error, "FInstancedStruct::NetSerialize: Bad script struct serialized, cannot recover.");
				Ar.SetError();
				bOutSuccess = false;
			}
		}
		else if (Ar.IsSaving())
		{
			check(::IsValid(ScriptStruct));
			Ar << ScriptStruct;
		}

		// Check ScriptStruct here, as loading might have failed. 
		if (ScriptStruct)
		{
			if (ScriptStruct->StructFlags & STRUCT_NetSerializeNative)
			{
				ScriptStruct->GetCppStructOps()->NetSerialize(Ar, Map, bOutSuccess, GetMutableMemory());
			}
			else if (ensureMsgf(NetSerializeScriptStructDelegate.IsBound(),
				TEXT("NetSerialization delegate is required for structs not using the native serialization.")))
			{
				bOutSuccess = NetSerializeScriptStructDelegate.Execute(*this, Ar, Map);
			}
		}
	}
	else
	{
		if (Ar.IsLoading())
		{
			Reset();
		}
		bOutSuccess = true;
	}

	return true;
}

bool FInstancedStruct::FindInnerPropertyInstance(FName PropertyName, const FProperty*& OutProp, const void*& OutData) const
{
	if (!ScriptStruct || !StructMemory)
	{
		return false;
	}
	
	for (const FProperty* Prop : TFieldRange<FProperty>(ScriptStruct))
	{
		if( Prop->GetAuthoredName() == PropertyName )
		{
			OutProp = Prop;
			OutData = StructMemory;
			return true;
		}
	}

	return false;
}

EPropertyVisitorControlFlow FInstancedStruct::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const
{
	if (!ScriptStruct || !StructMemory)
	{
		return EPropertyVisitorControlFlow::StepOver;
	}

	FPropertyVisitorContext SubContext = Context.VisitPropertyData(StructMemory);
	return ScriptStruct->Visit(SubContext, InFunc);
}

void* FInstancedStruct::ResolveVisitedPathInfo(const FPropertyVisitorInfo& Info) const
{
	if (!ScriptStruct || !StructMemory)
	{
		return nullptr;
	}

	return ScriptStruct->ResolveVisitedPathInfo(StructMemory, Info);
}
