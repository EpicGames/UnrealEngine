// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/InstancedStructContainer.h"
#include "UObject/Linker.h"

#if WITH_EDITOR
#include "StructUtils/StructReinstancer.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedStructContainer)

FInstancedStructContainer::FInstancedStructContainer()
{
}

FInstancedStructContainer::FInstancedStructContainer(const FInstancedStructContainer& InOther)
{
	Append(InOther);
}

FInstancedStructContainer::FInstancedStructContainer(FInstancedStructContainer&& InOther)
	: Memory(InOther.Memory)
	, AllocatedSize(InOther.AllocatedSize)
	, NumItems(InOther.NumItems)
{
	InOther.Memory = nullptr;
	InOther.AllocatedSize = 0;
	InOther.NumItems = 0;
}

FInstancedStructContainer& FInstancedStructContainer::operator=(const FInstancedStructContainer& InOther)
{
	if (this != &InOther)
	{
		Reset();
		Append(InOther);
	}
	return *this;
}

FInstancedStructContainer& FInstancedStructContainer::operator=(FInstancedStructContainer&& InOther)
{
	if (this != &InOther)
	{
		Empty();
			
		Memory = InOther.Memory;
		AllocatedSize = InOther.AllocatedSize;
		NumItems = InOther.NumItems;
			
		InOther.Memory = nullptr;
		InOther.AllocatedSize = 0;
		InOther.NumItems = 0;
	}
	return *this;
}

FInstancedStructContainer& FInstancedStructContainer::operator=(TConstArrayView<FInstancedStruct> InItems)
{
	Reset();
	TArray<FConstStructView> Views;
	Views.Reserve(InItems.Num());
	for (const FInstancedStruct& Value : InItems)
	{
		Views.Add(Value);
	}
	InsertAt(0, Views);
	return *this;
}

FInstancedStructContainer& FInstancedStructContainer::operator=(TConstArrayView<FStructView> InItems)
{
	Reset();
	InsertAt(0, InItems);
	return *this;
}

FInstancedStructContainer& FInstancedStructContainer::operator=(TConstArrayView<FConstStructView> InItems)
{
	Reset();
	InsertAt(0, InItems);
	return *this;
}


void FInstancedStructContainer::Append(const FInstancedStructContainer& Other)
{
	TArray<FConstStructView> Views;
	Views.Reserve(Other.Num());
	for (int32 Index = 0; Index < Other.Num(); Index++)
	{
		Views.Add(Other[Index]);
	}
	InsertAt(NumItems, Views);
}

void FInstancedStructContainer::Append(TConstArrayView<FInstancedStruct> NewItemValues)
{
	TArray<FConstStructView> Views;
	Views.Reserve(NewItemValues.Num());
	for (const FInstancedStruct& Value : NewItemValues)
	{
		Views.Add(Value);
	}
	InsertAt(NumItems, Views);
}

void FInstancedStructContainer::Append(TConstArrayView<FConstStructView> NewItemValues)
{
	InsertAt(NumItems, NewItemValues);
}

void FInstancedStructContainer::InsertAt(const int32 InsertAtIndex, const FInstancedStructContainer& Other)
{
	TArray<FConstStructView> Views;
	Views.Reserve(Other.Num());
	for (int32 Index = 0; Index < Other.Num(); Index++)
	{
		Views.Add(Other[Index]);
	}
	InsertAt(InsertAtIndex, Views);
}

void FInstancedStructContainer::InsertAt(const int32 InsertAtIndex, TConstArrayView<FInstancedStruct> NewItemValues)
{
	TArray<FConstStructView> Views;
	Views.Reserve(NewItemValues.Num());
	for (const FInstancedStruct& Value : NewItemValues)
	{
		Views.Add(Value);
	}
	InsertAt(InsertAtIndex, Views);
}

void FInstancedStructContainer::InsertAt(const int32 InsertAtIndex, TConstArrayView<FStructView> NewItemValues)
{
	TArray<FConstStructView> Views;
	Views.Reserve(NewItemValues.Num());
	for (const FStructView Value : NewItemValues)
	{
		Views.Add(Value);
	}
	InsertAt(InsertAtIndex, Views);
}

void FInstancedStructContainer::InsertAt(const int32 InsertAtIndex, TConstArrayView<FConstStructView> ValuesToInsert)
{
	checkSlow((ValuesToInsert.Num() >= 0) & (InsertAtIndex >= 0) & (InsertAtIndex <= NumItems));

	const int32 NumAddedItems = ValuesToInsert.Num();
	const int32 NumHeadItems = InsertAtIndex;
	const int32 NumTailItems = NumItems - InsertAtIndex;

	TArray<FItem> NewItems;
	NewItems.Reserve(ValuesToInsert.Num());

	// This alignment is used to align the tail values if needed.
	int32 OldMinAlignment = DefaultMinAlignment;
	for (int32 Index = 0; Index < NumItems; Index++)
	{
		OldMinAlignment = FMath::Max(OldMinAlignment, GetItem(Index).GetMinAlignment());
	}

	int32 MinAlignment = OldMinAlignment;
	int32 Offset = 0;

	// If there are items before the insert location, start the new layout from the end of the last item.
	if (NumHeadItems > 0)
	{
		Offset = GetItem(NumHeadItems - 1).GetEndOffset();
	}
	
	// Layout new items.
	for (const FConstStructView Value : ValuesToInsert)
	{
		FItem& NewItem = NewItems.Emplace_GetRef(Value.GetScriptStruct(), 0);
		MinAlignment = FMath::Max(MinAlignment, NewItem.GetMinAlignment());
		Offset = Align(Offset, NewItem.GetMinAlignment());
		NewItem.Offset = Offset;
		Offset += NewItem.GetStructureSize();
	}

	// Move the layout of the tail items. Align them so that we can use one copy to relocate items.
	int32 NewTailOffset = 0;
	int32 OldTailOffset = 0;
	int32 TailSize = 0;
	if (NumTailItems > 0)
	{
		// Align the rest of the items based on the old min alignment, no full layout, just shift the items. 
		Offset = Align(Offset, OldMinAlignment);

		// Calculate the moved size.
		NewTailOffset = Offset;
		OldTailOffset = GetItem(InsertAtIndex).Offset;
		TailSize = GetItem(NumItems-1).GetEndOffset() - OldTailOffset;

		// Offset items
		const int32 DeltaOffset = Offset - GetItem(InsertAtIndex).Offset;
		for (int32 Index = InsertAtIndex; Index < NumItems; Index++)
		{
			FItem& OldItem = GetItem(Index);
			OldItem.Offset += DeltaOffset;
		}

		// Update offset based on last item.
		Offset = GetItem(NumItems - 1).GetEndOffset();
	}

	const int32 NewNumItems = NumItems + NewItems.Num(); 
	
	// Space required for the item index.
	constexpr int32 ItemAlignment = alignof(FItem);
	MinAlignment = FMath::Max(MinAlignment, ItemAlignment);
	Offset += NewNumItems * ItemStride;
	Offset = Align(Offset, ItemAlignment); // Align last, since the items go to the end.

	// Allocate more memory if needed
	const int32 NewAllocatedSize = Offset;
	ReserveBytes(NewAllocatedSize, MinAlignment);

	// Set the item index.
	NumItems = NewNumItems;

	// Move tail items if needed.
	if (NumTailItems > 0)
	{
		FMemory::Memmove(Memory + NewTailOffset, Memory + OldTailOffset, TailSize);

		for (int32 Index = NumItems - 1; Index >= NumItems - NumTailItems; Index--)
		{
			GetItem(Index) = GetItem(Index - NumAddedItems);
		}
	}

	// Set new items.
	for (int32 Index = 0; Index < NewItems.Num(); Index++)
	{
		GetItem(InsertAtIndex + Index) = NewItems[Index];
	}
	
	// Initialize new item values.
	for (int32 Index = 0; Index < ValuesToInsert.Num(); Index++)
	{
		FConstStructView NewItemValue = ValuesToInsert[Index];
		const FItem& Item = GetItem(InsertAtIndex + Index);
		if (Item.ScriptStruct != nullptr)
		{ 
			check(Item.ScriptStruct == NewItemValue.GetScriptStruct()); 
			Item.ScriptStruct->InitializeStruct(Memory + Item.Offset);
			if (NewItemValue.GetMemory() != nullptr)
			{
				Item.ScriptStruct->CopyScriptStruct(Memory + Item.Offset, NewItemValue.GetMemory());
			}
		}
	}
}

void FInstancedStructContainer::RemoveAt(const int32 RemoveAtIndex, const int32 Count)
{
	checkSlow((Count >= 0) & (RemoveAtIndex >= 0) & (RemoveAtIndex + Count <= NumItems));

	// Destruct the removed items.
	for (int32 Index = 0; Index < Count; Index++)
	{
		const FItem& Item = GetItem(RemoveAtIndex + Index);
		if (Item.ScriptStruct != nullptr)
		{
			Item.ScriptStruct->DestroyStruct(Memory + Item.Offset);
		}
	}

	// Move tail items if any.
	const int32 FirstTailItem = RemoveAtIndex + Count;
	const int32 ItemsToMove = NumItems - FirstTailItem;
	if (ItemsToMove > 0)
	{
		// Calculate alignment of the tail items.
		int32 MinAlignment = DefaultMinAlignment;
		for (int32 Index = FirstTailItem; Index < NumItems; Index++)
		{
			MinAlignment = FMath::Max(MinAlignment, GetItem(Index).GetMinAlignment());
		}

		// Calculate the tail items new start location.
		int32 Offset = RemoveAtIndex > 0 ? GetItem(RemoveAtIndex - 1).GetEndOffset() : 0;
		Offset = Align(Offset, MinAlignment);

		// Move items values.
		const int32 NewTailOffset = Offset;
		const int32 OldTailOffset = GetItem(FirstTailItem).Offset;
		const int32 TailSize = GetItem(NumItems-1).GetEndOffset() - OldTailOffset;
		if (TailSize > 0)
		{
			FMemory::Memmove(Memory + NewTailOffset, Memory + OldTailOffset, TailSize);
		}

		// Move and update items.
		const int32 DeltaOffset = NewTailOffset - OldTailOffset;
		for (int32 Index = FirstTailItem; Index < NumItems; Index++)
		{
			FItem& NewItem = GetItem(Index - Count);
			const FItem& OldItem = GetItem(Index);
			NewItem.ScriptStruct = OldItem.ScriptStruct;
			NewItem.Offset = OldItem.Offset + DeltaOffset;
		}
	}

	NumItems -= Count;
}

void FInstancedStructContainer::ReserveBytes(const int32 NumBytes, const int32 MinAlignment)
{
	if (NumBytes > AllocatedSize || !IsAligned(Memory, MinAlignment))
	{
		const int32 NewSize = FMath::Max(AllocatedSize, NumBytes); 
		uint8* NewMemory = static_cast<uint8*>(FMemory::Malloc(NewSize, MinAlignment));
		if (NumItems > 0)
		{
			check(Memory != nullptr);
			// Copy item values (beginning of the buffer).
			const int32 ValuesSize = GetItem(NumItems - 1).GetEndOffset();
			check(ValuesSize <= AllocatedSize);
			FMemory::Memcpy(NewMemory, Memory, ValuesSize);
			// Copy items (at the end of the buffer).
			const int32 ItemsSize = NumItems * ItemStride;
			check(ItemsSize <= AllocatedSize);
			FMemory::Memcpy(NewMemory + NumBytes - ItemsSize, Memory + AllocatedSize - ItemsSize, ItemsSize);
		}
		
		FMemory::Free(Memory);
	
		Memory = NewMemory;
		AllocatedSize = NewSize;
	}
}

void FInstancedStructContainer::SetNum(const int32 NewNum)
{
	checkSlow(NewNum >= 0 && NewNum <= NumItems);

	for (int32 Index = NewNum; Index < NumItems; Index++)
	{
		const FItem& Item = GetItem(Index);
		if (Item.ScriptStruct != nullptr)
		{
			Item.ScriptStruct->DestroyStruct(Memory + Item.Offset);
		}
	}
	NumItems = NewNum;
}

void FInstancedStructContainer::Reset()
{
	// Destruct items
	for (int32 Index = 0; Index < NumItems; Index++)
	{
		const FItem& Item = GetItem(Index);
		if (Item.ScriptStruct != nullptr)
		{
			Item.ScriptStruct->DestroyStruct(Memory + Item.Offset);
		}
	}
	NumItems = 0;
}

void FInstancedStructContainer::Empty()
{
	Reset();

	// Free memory
	FMemory::Free(Memory);
	Memory = nullptr;
	AllocatedSize = 0;
}

void FInstancedStructContainer::AddStructReferencedObjects(FReferenceCollector& Collector)
{
#if WITH_EDITOR
	// Reference collector is used to visit all instances of instanced structs and replace their contents.
	if (UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance())
	{
		for (int32 Index = 0; Index < NumItems; ++Index)
		{
			TObjectPtr<const UScriptStruct>& ScriptStruct = GetItem(Index).ScriptStruct;
			if (ScriptStruct && Reinstancer->IsReinstanting(ScriptStruct))
			{
				Reinstancer->MarkAsRequiresReinstantiation();
				// It is not necessary to change the struct to a duplicated one, but it helps with tracking mistakes and prevents memory stomps.
				if (const UScriptStruct* Duplicated = Reinstancer->GetDuplicatedReinstantingStruct(ScriptStruct))
				{
					ScriptStruct = Duplicated;
				}
			}
		}
	}
#endif // WITH_EDITOR
	
	for (int32 Index = 0, Num = NumItems; Index < Num; Index++)
	{
		FItem& Item = GetItem(Index);
		if (Item.ScriptStruct != nullptr)
		{
			Collector.AddReferencedObject(Item.ScriptStruct);
			Collector.AddPropertyReferencesWithStructARO(Item.ScriptStruct, Memory + Item.Offset);
		}
	}
}

bool FInstancedStructContainer::Identical(const FInstancedStructContainer* Other, uint32 PortFlags) const
{
	if (Other == nullptr)
	{
		return false;
	}

	// Identical if both are uninitialized.
	if (!IsValid() && !Other->IsValid())
	{
		return true;
	}

	// Not identical if one is valid and other is not.
	if (IsValid() != Other->IsValid())
	{
		return false;
	}

	// Not identical if different layouts.
	if (NumItems != Other->NumItems)
	{
		return false;
	}

	bool bResult = true;

#if WITH_EDITOR
	const UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance();
#endif

	// Check that the struct contents are identical.
	for (int32 Index = 0; Index < Num(); Index++)
	{ 
		const FItem& Item = GetItem(Index);
		const FItem& OtherItem = Other->GetItem(Index);
		if (Item.ScriptStruct != OtherItem.ScriptStruct)
		{
			bResult = false;
			break;
		}
		if (Item.ScriptStruct != nullptr && OtherItem.ScriptStruct != nullptr)
		{
#if WITH_EDITOR
			// Struct can't be identical when re-instancing. The struct has to serialize to get the new values.
			if (Reinstancer
				&& Reinstancer->IsSerializingForReinstantiation()
				&& Reinstancer->IsReinstanting(Item.ScriptStruct))
			{
				bResult = false;
				break;
			}
#endif
			const uint8* ItemMemory = Memory + Item.Offset;
			const uint8* OtherItemMemory = Other->Memory + OtherItem.Offset;
			
			if (Item.ScriptStruct->CompareScriptStruct(ItemMemory, OtherItemMemory, PortFlags) == false)
			{
				bResult = false;
				break;
			}
		}
	}
	
	return bResult;
}

bool FInstancedStructContainer::Serialize(FArchive& Ar, UStruct* DefaultsStruct, const void* Defaults)
{
	const bool bValidDefaults = DefaultsStruct && DefaultsStruct->IsChildOf(FInstancedStructContainer::StaticStruct());
	const FInstancedStructContainer* DefaultInstancedStructContainer = bValidDefaults ? static_cast<const FInstancedStructContainer*>(Defaults) : nullptr;

	enum class EVersion : uint8
	{
		InitialVersion = 0,
		// -----<new versions can be added above this line>-----
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	EVersion Version = EVersion::LatestVersion;
	Ar << Version;

	if (Version > EVersion::LatestVersion)
	{
		UE_LOGF(LogCore, Error, "Invalid Version: %hhu", int(Version));
		Ar.SetError();
		return false;
	}

	int32 NumItemsSerialized = NumItems;
	Ar << NumItemsSerialized;

	auto SerializeWithDefaults = [Self = this, &Ar, DefaultInstancedStructContainer](const FItem& Item, int32 Index)
		{
			const bool bDuplicate = (0 != (Ar.GetPortFlags() & PPF_Duplicate));
			const bool bSavingLoading = (Ar.IsSaving() || Ar.IsLoading());
			const bool bUseDefaults = !bDuplicate
				&& bSavingLoading
				&& !Ar.IsCooking()
				&& !Ar.WantBinaryPropertySerialization();

			if (bUseDefaults)
			{
				const bool bSameStruct = DefaultInstancedStructContainer
					&& DefaultInstancedStructContainer->IsValidIndex(Index)
					&& DefaultInstancedStructContainer->GetItem(Index).ScriptStruct == Item.ScriptStruct;
				if (bSameStruct)
				{
					// Use the parent Defaults
					const void* DefaultMemory = DefaultInstancedStructContainer->Memory + DefaultInstancedStructContainer->GetItem(Index).Offset;
					
					if (Ar.IsLoading())
					{
						ConstCast(Item.ScriptStruct)->CopyScriptStruct(Self->Memory + Item.Offset, DefaultMemory);
					}
					
					ConstCast(Item.ScriptStruct)->SerializeItem(Ar, Self->Memory + Item.Offset, DefaultMemory);
				}
				else
				{
					// Create a temporary struct on the stack. The struct has default values.
					const int32 MinAlignment = Item.ScriptStruct->GetMinAlignment();
					const int32 RequiredSize = Item.ScriptStruct->GetStructureSize();
					TArray<uint8, TInlineAllocator<64>> TmpMemoryBuffer;
					TmpMemoryBuffer.SetNumUninitialized(FMath::Max(1, RequiredSize) + FMath::Max(0, MinAlignment - (int32)MIN_ALIGNMENT));
					void* TempMemory = Align(TmpMemoryBuffer.GetData(), MinAlignment);
					ConstCast(Item.ScriptStruct)->InitializeStruct(TempMemory);
					ConstCast(Item.ScriptStruct)->SerializeItem(Ar, Self->Memory + Item.Offset, TempMemory);
					ConstCast(Item.ScriptStruct)->DestroyStruct(TempMemory);
				}
			}
			else
			{
				// Serialize without defaults
				ConstCast(Item.ScriptStruct)->SerializeItem(Ar, Self->Memory + Item.Offset, nullptr);
			}
		};

	if (NumItemsSerialized > 0)
	{
		if (Ar.IsCountingMemory() || Ar.IsModifyingWeakAndStrongReferences() || Ar.IsObjectReferenceCollector())
		{
			// Report item types
			for (int32 Index = 0; Index < NumItems; Index++)
			{
				const FItem& Item = GetItem(Index);
				Ar << ConstCast(Item.ScriptStruct);
			}

			// Report item values
			for (int32 Index = 0; Index < NumItemsSerialized; Index++)
			{
				const FItem& Item = GetItem(Index);
				if (Item.ScriptStruct != nullptr)
				{
					SerializeWithDefaults(Item, Index);
				}
			}
		}
		else if (Ar.IsLoading())
		{
#if WITH_EDITOR
			const UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance();
#endif

			// Load item types 
			TArray<FConstStructView> Views;
			Views.Reserve(NumItemsSerialized);
			for (int32 Index = 0; Index < NumItemsSerialized; Index++)
			{
				UScriptStruct* SerializedScriptStruct = nullptr;
				Ar << SerializedScriptStruct;
				if (SerializedScriptStruct)
				{
#if WITH_EDITOR
					// During reinstantiation, the serialized struct can be the the duplicated struct.
					//Replace it with the new version.
					if (Reinstancer && Reinstancer->IsSerializingForReinstantiation())
					{
						if (const UScriptStruct* CompiledStruct = Reinstancer->GetCompiledReinstantingStruct(SerializedScriptStruct))
						{
							SerializedScriptStruct = const_cast<UScriptStruct*>(CompiledStruct);
						}
					}
#endif
					Ar.Preload(SerializedScriptStruct);
				}
				Views.Add(FConstStructView(SerializedScriptStruct, nullptr));
			}

			// Allocate and init items.
			Empty();
			InsertAt(0, Views);

			check(NumItems == NumItemsSerialized);
			
			// Load item values
			for (int32 Index = 0; Index < NumItemsSerialized; Index++)
			{
				const FItem& Item = GetItem(Index);

				// Size of the serialized memory
				int32 SerialSize = 0; 
				Ar << SerialSize;

				// Serialized memory
				if (Item.ScriptStruct == nullptr && SerialSize > 0)
				{
					// A null struct indicates an old struct or an unsupported one for the current target.
					// In this case we manually seek in the archive to skip its serialized content. 
					// We don't want to rely on TaggedSerialization that will mark an error in the archive that
					// may cause other serialization to fail (e.g. FArchive& operator<<(FArchive& Ar, TArray& A))
					UE_LOGF(LogCore, Warning, "Unable to find serialized UScriptStruct -> Advance %u bytes in the archive and reset to empty FInstancedStructContainer. LinkerRoot:%ls", SerialSize, Ar.GetLinker() ? *GetPathNameSafe(Ar.GetLinker()->LinkerRoot) : TEXT("NoLinker"));
					Ar.Seek(Ar.Tell() + SerialSize);
				}
				else if (Item.ScriptStruct != nullptr)
				{
					SerializeWithDefaults(Item, Index);
				}
			}
		}
		else
		{
			// Saving + inspector archives (e.g. FArchiveCrc32 used by UE::StructUtils::GetStructInstanceCrc32,
			// or the CityHash64-based archive used by GetStructHash64). The serial-size backpatch is gated
			// on the archive being seekable - non-seekable archives (FArchive::Tell defaults to INDEX_NONE)
			// don't need it. Without this branch the contained items would not contribute to the hash.

			// Save item types
			for (int32 Index = 0; Index < NumItems; Index++)
			{
				const FItem& Item = GetItem(Index);
				Ar << ConstCast(Item.ScriptStruct);
			}

			// Save item values
			for (int32 Index = 0; Index < NumItemsSerialized; Index++)
			{
				const FItem& Item = GetItem(Index);

				// Size of the serialized memory (reserve location)
				const int64 SizeOffset = Ar.Tell(); // Position to write the actual size after struct serialization
				int32 SerialSize = 0;
				if (SizeOffset != INDEX_NONE)
				{
					Ar << SerialSize;
				}

				// Serialized memory
				const int64 InitialOffset = Ar.Tell(); // Position before struct serialization to compute its serial size
				if (Item.ScriptStruct != nullptr)
				{
					SerializeWithDefaults(Item, Index);
				}

				if (SizeOffset != INDEX_NONE)
				{
					const int64 FinalOffset = Ar.Tell(); // Keep current offset to reset the archive pos after write the serial size

					// Size of the serialized memory
					Ar.Seek(SizeOffset);	// Go back in the archive to write the actual size
					SerialSize = static_cast<int32>(FinalOffset - InitialOffset);
					Ar << SerialSize;
					Ar.Seek(FinalOffset);	// Reset archive to its position
				}
			}
		}
	}

	return true;
}

void FInstancedStructContainer::GetPreloadDependencies(TArray<UObject*>& OutDeps) const
{
	for (int32 Index = 0; Index < NumItems; Index++)
	{
		FItem& Item = GetItem(Index);
		if (Item.ScriptStruct)
		{
			OutDeps.Add(ConstCast(Item.ScriptStruct));

			// Report direct dependencies of the instanced struct
			if (UScriptStruct::ICppStructOps* CppStructOps = Item.ScriptStruct->GetCppStructOps())
			{
				CppStructOps->GetPreloadDependencies(Memory + Item.Offset, OutDeps);
			}

			// Report indirect dependencies of the instanced struct
			// The iterator will recursively loop through all structs in structs/containers too
			for (TPropertyValueIterator<FStructProperty> It(Item.ScriptStruct, Memory + Item.Offset); It; ++It)
			{
				const UScriptStruct* StructType = It.Key()->Struct;
				if (UScriptStruct::ICppStructOps* CppStructOps = StructType->GetCppStructOps())
				{
					void* StructDataPtr = const_cast<void*>(It.Value());
					CppStructOps->GetPreloadDependencies(StructDataPtr, OutDeps);
				}
			}
		}
	}
}

bool FInstancedStructContainer::ExportTextItem(FString& ValueStr, FInstancedStructContainer const& DefaultValue,
	UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	ValueStr += TEXT('(');
	for (int32 Index = 0; Index < NumItems; ++Index)
	{
		if (Index > 0)
		{
			ValueStr += TEXT(',');
		}

		FItem& Item = GetItem(Index);

		// Serialize in the format of FInstancedStruct::ExportTextItem
		if (Item.ScriptStruct)
		{
			ValueStr += Item.ScriptStruct->GetPathName();

			Item.ScriptStruct->ExportText(
				ValueStr,
				Memory + Item.Offset,
				DefaultValue.IsValidIndex(Index) && Item.ScriptStruct == DefaultValue.GetItem(Index).ScriptStruct
					? DefaultValue.Memory + DefaultValue.GetItem(Index).Offset
					: nullptr,
				Parent,
				PortFlags,
				ExportRootScope
			);
		}
		else
		{
			ValueStr += TEXT("None");
		}
	}

	ValueStr += TEXT(')');
	return true;
}

bool FInstancedStructContainer::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent,
	FOutputDevice* ErrorText, FArchive* InSerializingArchive)
{
	// We can't predict the size of the container during string read, so let's work in a temporary array
	// instead of constantly resizing the container
	TArray<FInstancedStruct, TInlineAllocator<64>> RawStructs;

	if (*Buffer != TEXT('('))
	{
		ErrorText->Logf(
			ELogVerbosity::Warning,
			TEXT("FInstancedStructContainer: Missing opening \'(\' while importing property values.")
		);

		return false;
	}
	Buffer++;

	bool bFirst = true;
	while (*Buffer != TEXT(')'))
	{
		if (!bFirst)
		{
			if (*Buffer == TEXT(','))
			{
				Buffer++;
			}
			else
			{
				ErrorText->Logf(ELogVerbosity::Warning, TEXT("FInstancedStructContainer: Missing \',\' after item."));
				return false;
			}
		}

		// We can do this because we've exported exactly as FInstancedStruct::ExportTextItem does
		FInstancedStruct& InstancedStruct = RawStructs.AddDefaulted_GetRef();
		const bool bSuccess = InstancedStruct.ImportTextItem(Buffer, PortFlags, Parent, ErrorText, InSerializingArchive);
		if (!bSuccess)
		{
			return false;
		}

		bFirst = false;
	}

	*this = RawStructs;
	return true;
}
