// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/PropertyBag.h"

#include "HAL/IConsoleManager.h"
#include "Hash/xxhash.h"
#include "Misc/NotNull.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "StructUtils/PropertyBagMapRef.h"
#include "UObject/EnumProperty.h"
#include "UObject/Package.h"
#include "UObject/TextProperty.h"

#if WITH_EDITOR
#include "StructUtils/StructReinstancer.h"
#include "StructUtils/UserDefinedStruct.h"
#endif // WITH_EDITOR

static TAutoConsoleVariable<bool> CVarEnforceValidName(
	TEXT("StructUtils.EnforceValidPropertyName"),
	false,
	TEXT("Enforce valid naming on new and renamed property bag properties."));

#if WITH_EDITOR
static TAutoConsoleVariable<bool> CVarValidatePropertyBagDescriptionEquality(
	TEXT("StructUtils.ValidatePropertyBagDescriptionEquality"),
	false,
	TEXT("Do a full check on description equality. Otherwise, rely on the unique Guid."));
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyBag)

namespace FPropertyBagCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		BeforeCustomVersionWasAdded = 0,

		// Added support for array types
		ContainerTypes = 1,
		NestedContainerTypes = 2,
		MetaClass = 3,
		PropertyFlags = 4,
		KeyTypes = 5,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
};


const FGuid FPropertyBagCustomVersion::GUID(0x134A157E, 0xD5E249A3, 0x8D4E843C, 0x98FE9E31);

// Register the custom version with core
FCustomVersionRegistration GPropertyBagCustomVersion(FPropertyBagCustomVersion::GUID, FPropertyBagCustomVersion::LatestVersion, TEXT("PropertyBagCustomVersion"));


namespace UE::StructUtils::Private
{

	bool CanCastTo(const UStruct* From, const UStruct* To)
	{
		return From != nullptr && To != nullptr && From->IsChildOf(To);
	}

	bool IsTypeHashable(const EPropertyBagPropertyType InType, const UObject* InTypeObject)
	{
		switch (InType)
		{
		case EPropertyBagPropertyType::Bool:
		case EPropertyBagPropertyType::Text:
			return false;
		case EPropertyBagPropertyType::Struct:
		{
			const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InTypeObject);
			if (!ScriptStruct)
			{
				return false;
			}

			return ScriptStruct->GetCppStructOps() && ScriptStruct->GetCppStructOps()->HasGetTypeHash();
		}
		default:
			return true;
		}
	}

	bool IsNumericIntegralType(const EPropertyBagPropertyType ValueType)
	{
		switch (ValueType)
		{
		case EPropertyBagPropertyType::Int8: return true;
		case EPropertyBagPropertyType::Byte: return true;
		case EPropertyBagPropertyType::Int16: return true;
		case EPropertyBagPropertyType::UInt16: return true;
		case EPropertyBagPropertyType::Int32: return true;
		case EPropertyBagPropertyType::UInt32: return true;
		case EPropertyBagPropertyType::Int64: return true;
		case EPropertyBagPropertyType::UInt64: return true;
		default: return false;
		}
	}

	FPropertyBagContainerTypes GetContainerTypesFromProperty(const FProperty* InSourceProperty)
	{
		FPropertyBagContainerTypes ContainerTypes;

		while (InSourceProperty)
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InSourceProperty))
			{
				if (ContainerTypes.Add(EPropertyBagContainerType::Array))
				{
					InSourceProperty = ArrayProperty->Inner;
					continue;
				}
				else // we reached the nested containers limit
				{
					ContainerTypes.Reset();
				}
			}
			if (const FSetProperty* SetProperty = CastField<FSetProperty>(InSourceProperty))
			{
				if (ContainerTypes.Add(EPropertyBagContainerType::Set))
				{
					InSourceProperty = SetProperty->ElementProp;
					continue;
				}
				else // we reached the nested containers limit
				{
					ContainerTypes.Reset();
				}
			}
			if (const FMapProperty* MapProperty = CastField<FMapProperty>(InSourceProperty))
			{
				if (ContainerTypes.Add(EPropertyBagContainerType::Map))
				{
					InSourceProperty = MapProperty->ValueProp;
					continue;
				}
				else // we reached the nested containers limit
				{
					ContainerTypes.Reset();
				}
			}
			
			InSourceProperty = nullptr;
		}

		return ContainerTypes;
	}

	EPropertyBagPropertyType GetValueTypeFromProperty(const FProperty* InSourceProperty)
	{
		if (CastField<FBoolProperty>(InSourceProperty))
		{
			return EPropertyBagPropertyType::Bool;
		}
		if (CastField<FNumericProperty>(InSourceProperty))
		{
			if (const FInt8Property* Int8Property = CastField<FInt8Property>(InSourceProperty))
			{
				return EPropertyBagPropertyType::Int8;
			}
			if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InSourceProperty))
			{
				return ByteProperty->IsEnum() ? EPropertyBagPropertyType::Enum : EPropertyBagPropertyType::Byte;
			}
			if (CastField<FInt16Property>(InSourceProperty))
			{
				return EPropertyBagPropertyType::Int16;
			}
			if (CastField<FUInt16Property>(InSourceProperty))
			{
				return EPropertyBagPropertyType::UInt16;
			}
			if (CastField<FIntProperty>(InSourceProperty))
			{
				return EPropertyBagPropertyType::Int32;
			}
			if (CastField<FUInt32Property>(InSourceProperty))
			{
				return EPropertyBagPropertyType::UInt32;
			}
			if (CastField<FInt64Property>(InSourceProperty))
			{
				return EPropertyBagPropertyType::Int64;
			}
			if (CastField<FUInt64Property>(InSourceProperty))
			{
				return EPropertyBagPropertyType::UInt64;
			}
			if (CastField<FFloatProperty>(InSourceProperty))
			{
				return EPropertyBagPropertyType::Float;
			}
			if (CastField<FDoubleProperty>(InSourceProperty))
			{
				return EPropertyBagPropertyType::Double;
			}
			return EPropertyBagPropertyType::None;
		}
		if (CastField<FNameProperty>(InSourceProperty))
		{
			return EPropertyBagPropertyType::Name;
		}
		if (CastField<FStrProperty>(InSourceProperty))
		{
			return EPropertyBagPropertyType::String;
		}
		if (CastField<FTextProperty>(InSourceProperty))
		{
			return EPropertyBagPropertyType::Text;
		}
		if (CastField<FEnumProperty>(InSourceProperty))
		{
			return EPropertyBagPropertyType::Enum;
		}
		if (CastField<FStructProperty>(InSourceProperty))
		{
			return EPropertyBagPropertyType::Struct;
		}
		if (CastField<FObjectProperty>(InSourceProperty))
		{
			if (CastField<FClassProperty>(InSourceProperty))
			{
				return EPropertyBagPropertyType::Class;
			}

			return EPropertyBagPropertyType::Object;
		}
		if (CastField<FSoftObjectProperty>(InSourceProperty))
		{
			if (CastField<FSoftClassProperty>(InSourceProperty))
			{
				return EPropertyBagPropertyType::SoftClass;
			}

			return EPropertyBagPropertyType::SoftObject;
		}

		// Handle array property
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InSourceProperty))
		{
			return GetValueTypeFromProperty(ArrayProperty->Inner);	
		}

		if (const FSetProperty* SetProperty = CastField<FSetProperty>(InSourceProperty))
		{
			return GetValueTypeFromProperty(SetProperty->ElementProp);
		}

		if (const FMapProperty* MapProperty = CastField<FMapProperty>(InSourceProperty))
		{
			return GetValueTypeFromProperty(MapProperty->ValueProp);
		}

		return EPropertyBagPropertyType::None;
	}

	const UObject* GetValueTypeObjectFromProperty(const FProperty* InSourceProperty)
	{
		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InSourceProperty))
		{
			if (ByteProperty->IsEnum())
			{
				return ByteProperty->Enum;
			}
		}
		if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(InSourceProperty))
		{
			return EnumProp->GetEnum();
		}
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(InSourceProperty))
		{
			return StructProperty->Struct;
		}
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InSourceProperty))
		{
			if (const FClassProperty* ClassProperty = CastField<FClassProperty>(InSourceProperty))
			{
				return ClassProperty->MetaClass;
			}

			return ObjectProperty->PropertyClass;
		}
		if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(InSourceProperty))
		{
			if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(InSourceProperty))
			{
				return SoftClassProperty->MetaClass;
			}

			return SoftObjectProperty->PropertyClass;
		}
		
		// Handle array property
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InSourceProperty))
		{
			return GetValueTypeObjectFromProperty(ArrayProperty->Inner);
		}

		if (const FSetProperty* SetProperty = CastField<FSetProperty>(InSourceProperty))
		{
			return GetValueTypeObjectFromProperty(SetProperty->ElementProp);
		}

		if (const FMapProperty* MapProperty = CastField<FMapProperty>(InSourceProperty))
		{
			return GetValueTypeObjectFromProperty(MapProperty->ValueProp);
		}

		return nullptr;
	}

	EPropertyBagPropertyType GetKeyTypeFromProperty(const FProperty* InSourceProperty)
	{
		if (const FMapProperty* MapProperty = CastField<FMapProperty>(InSourceProperty))
		{
			return GetValueTypeFromProperty(MapProperty->KeyProp);
		}

		return EPropertyBagPropertyType::None;
	}

	const UObject* GetKeyTypeObjectFromProperty(const FProperty* InSourceProperty)
	{
		if (const FMapProperty* MapProperty = CastField<FMapProperty>(InSourceProperty))
		{
			return GetValueTypeObjectFromProperty(MapProperty->KeyProp);
		}

		return nullptr;
	}

	EPropertyBagPropertyType EnumUnderlyingTypeToPropertyBagPropertyType(const UEnum::EUnderlyingType UnderlyingType)
	{
		switch (UnderlyingType)
		{
			case UEnum::EUnderlyingType::uint8: return EPropertyBagPropertyType::Byte;
			case UEnum::EUnderlyingType::int8: return EPropertyBagPropertyType::Int8;
			case UEnum::EUnderlyingType::uint16: return EPropertyBagPropertyType::UInt16;
			case UEnum::EUnderlyingType::int16: return EPropertyBagPropertyType::Int16;
			case UEnum::EUnderlyingType::uint32: return EPropertyBagPropertyType::UInt32;
			case UEnum::EUnderlyingType::int32: return EPropertyBagPropertyType::Int32;
			case UEnum::EUnderlyingType::uint64: return EPropertyBagPropertyType::UInt64;
			case UEnum::EUnderlyingType::int64: return EPropertyBagPropertyType::Int64;
		}
		return EPropertyBagPropertyType::None;
	}

	FNumericProperty* CreateIntegerPropertyFromDesc(const EPropertyBagPropertyType ValueType, const FFieldVariant PropertyScope, const FName PropertyName)
	{
		FNumericProperty* Prop = nullptr;
		switch (ValueType)
		{
		case EPropertyBagPropertyType::Int8:
			Prop = new FInt8Property(PropertyScope, PropertyName);
			break;
		case EPropertyBagPropertyType::Byte:
			Prop = new FByteProperty(PropertyScope, PropertyName);
			break;
		case EPropertyBagPropertyType::Int16:
			Prop = new FInt16Property(PropertyScope, PropertyName);
			break;
		case EPropertyBagPropertyType::UInt16:
			Prop = new FUInt16Property(PropertyScope, PropertyName);
			break;
		case EPropertyBagPropertyType::Int32:
			Prop = new FIntProperty(PropertyScope, PropertyName);
			break;
		case EPropertyBagPropertyType::UInt32:
			Prop = new FUInt32Property(PropertyScope, PropertyName);
			break;
		case EPropertyBagPropertyType::Int64:
			Prop = new FInt64Property(PropertyScope, PropertyName);
			break;
		case EPropertyBagPropertyType::UInt64:
			Prop = new FUInt64Property(PropertyScope, PropertyName);
			break;
		}

		if (Prop != nullptr)
		{
			Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		return Prop;
	}

	FProperty* CreatePropertyFromDesc(const FPropertyBagPropertyDesc& Desc, const FFieldVariant PropertyScope)
	{
		// Handle array and nested containers properties
		if (Desc.ContainerTypes.Num() > 0)
		{
			FProperty* Prop = nullptr; // the first created container will fill the return value, nested ones will fill the inner

			// support for nested containers, i.e. : TArray<TArray<float>>
			FFieldVariant PropertyOwner = PropertyScope;
			FProperty** ValuePropertyPtr = &Prop;

			// Create the container list
			for (EPropertyBagContainerType BagContainerType : Desc.ContainerTypes)
			{
				switch(BagContainerType)
				{
				case EPropertyBagContainerType::Array:
					{
						// create an array property as a container for the tail
						FArrayProperty* ArrayProperty = new FArrayProperty(PropertyOwner, Desc.Name);
						*ValuePropertyPtr = ArrayProperty;
						ValuePropertyPtr = &ArrayProperty->Inner;
						PropertyOwner = ArrayProperty;
						break;
					}
				case EPropertyBagContainerType::Set:
					{
						// create a set property as a container for the tail
						FSetProperty* SetProperty = new FSetProperty(PropertyOwner, Desc.Name);
						*ValuePropertyPtr = SetProperty;
						ValuePropertyPtr = &SetProperty->ElementProp;
						PropertyOwner = SetProperty;
						break;
					}
				case EPropertyBagContainerType::Map:
					{
						// create a map property as a container for the tail
						FMapProperty* MapProperty = new FMapProperty(PropertyOwner, Desc.Name);
						*ValuePropertyPtr = MapProperty;
						ValuePropertyPtr = &MapProperty->ValueProp;
						PropertyOwner = MapProperty;

						// create the key property (container types are not supported for keys)
						FPropertyBagPropertyDesc KeyDesc;
						KeyDesc.ValueType = Desc.KeyType;
						KeyDesc.ValueTypeObject = Desc.KeyTypeObject;
						KeyDesc.Name = Desc.Name;
						MapProperty->KeyProp = CreatePropertyFromDesc(KeyDesc, PropertyOwner);
						break;
					}
				default:
					ensureMsgf(false, TEXT("Unsupported container type %s"), *UEnum::GetValueAsString(BagContainerType));
					break;
				}
			}

			// finally create the tail type
			FPropertyBagPropertyDesc InnerDesc = Desc;
			InnerDesc.ContainerTypes.Reset();
			*ValuePropertyPtr = CreatePropertyFromDesc(InnerDesc, PropertyOwner);
				
			return Prop;
		}
		
		switch (Desc.ValueType)
		{
		case EPropertyBagPropertyType::Bool:
			{
				FBoolProperty* Prop = new FBoolProperty(PropertyScope, Desc.Name);
				Prop->SetBoolSize(sizeof(bool), true); // Enable native access (init the whole byte, rather than just first bit)
				return Prop;
		}
		case EPropertyBagPropertyType::Int8:
		case EPropertyBagPropertyType::Byte:
		case EPropertyBagPropertyType::Int16:
		case EPropertyBagPropertyType::UInt16:
		case EPropertyBagPropertyType::Int32:
		case EPropertyBagPropertyType::UInt32:
		case EPropertyBagPropertyType::Int64:
		case EPropertyBagPropertyType::UInt64:
			return CreateIntegerPropertyFromDesc(Desc.ValueType, PropertyScope, Desc.Name);
		case EPropertyBagPropertyType::Float:
			{
				FFloatProperty* Prop = new FFloatProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPropertyBagPropertyType::Double:
			{
				FDoubleProperty* Prop = new FDoubleProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPropertyBagPropertyType::Name:
			{
				FNameProperty* Prop = new FNameProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPropertyBagPropertyType::String:
			{
				FStrProperty* Prop = new FStrProperty(PropertyScope, Desc.Name);
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
		case EPropertyBagPropertyType::Text:
			{
				FTextProperty* Prop = new FTextProperty(PropertyScope, Desc.Name);
				return Prop;
			}
		case EPropertyBagPropertyType::Enum:
			if (const UEnum* Enum = Cast<UEnum>(Desc.ValueTypeObject))
			{
				// Only generate FEnumProperty for cpp enum class like UHT and Blueprint.
				if (Enum->GetCppForm() == UEnum::ECppForm::EnumClass)
				{
					const EPropertyBagPropertyType UnderlyingType = EnumUnderlyingTypeToPropertyBagPropertyType(Enum->GetUnderlyingType());
					if (IsNumericIntegralType(UnderlyingType))
					{
						FEnumProperty* Prop = new FEnumProperty(PropertyScope, Desc.Name);
						Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
						Prop->SetEnum(const_cast<UEnum*>(Enum));
						static FName NAME_UnderlyingType = "UnderlyingType";
						FNumericProperty* UnderlyingProp = CreateIntegerPropertyFromDesc(UnderlyingType, Prop, NAME_UnderlyingType);
						checkf(UnderlyingProp, TEXT("The type was checked in IsNumericIntegralType."));
						Prop->AddCppProperty(UnderlyingProp);
						return Prop;
					}
				}
				else
				{
					FByteProperty* Prop = new FByteProperty(PropertyScope, Desc.Name);
					Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
					Prop->Enum = const_cast<UEnum*>(Enum);
					return Prop;
				}
			}
			break;
		case EPropertyBagPropertyType::Struct:
			if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Desc.ValueTypeObject))
			{
				FStructProperty* Prop = new FStructProperty(PropertyScope, Desc.Name);
				Prop->Struct = const_cast<UScriptStruct*>(ScriptStruct);

				if (ScriptStruct->GetCppStructOps() && ScriptStruct->GetCppStructOps()->HasGetTypeHash())
				{
					Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				}
				
				if (ScriptStruct->StructFlags & STRUCT_HasInstancedReference)
				{
					Prop->SetPropertyFlags(CPF_ContainsInstancedReference);
				}
				
				return Prop;
			}
			break;
		case EPropertyBagPropertyType::Class:
			if (const UClass* Class = Cast<UClass>(Desc.ValueTypeObject))
			{
				FClassProperty* Prop = new FClassProperty(PropertyScope, Desc.Name);
#if WITH_EDITORONLY_DATA
				Prop->SetMetaClass(Desc.MetaClass ? Desc.MetaClass.Get() : const_cast<UClass*>(Class));
#else
				Prop->SetMetaClass(const_cast<UClass*>(Class));
#endif
				Prop->PropertyClass = UClass::StaticClass();
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
			break;
		case EPropertyBagPropertyType::SoftClass:
			if (const UClass* Class = Cast<UClass>(Desc.ValueTypeObject))
			{
				FSoftClassProperty* Prop = new FSoftClassProperty(PropertyScope, Desc.Name);
#if WITH_EDITORONLY_DATA
				Prop->SetMetaClass(Desc.MetaClass ? Desc.MetaClass.Get() : const_cast<UClass*>(Class));
#else
				Prop->SetMetaClass(const_cast<UClass*>(Class));
#endif
				Prop->PropertyClass = UClass::StaticClass();
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
			break;
		case EPropertyBagPropertyType::Object:
			if (const UClass* Class = Cast<UClass>(Desc.ValueTypeObject))
			{
				FObjectProperty* Prop = new FObjectProperty(PropertyScope, Desc.Name);
				if (Class->HasAnyClassFlags(CLASS_DefaultToInstanced))
				{
					Prop->SetPropertyFlags(CPF_InstancedReference);
				}
				Prop->SetPropertyClass(const_cast<UClass*>(Class));
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash | CPF_TObjectPtr);
				return Prop;
			}
			break;
		case EPropertyBagPropertyType::SoftObject:
			if (const UClass* Class = Cast<UClass>(Desc.ValueTypeObject))
			{
				FSoftObjectProperty* Prop = new FSoftObjectProperty(PropertyScope, Desc.Name);
				if (Class->HasAnyClassFlags(CLASS_DefaultToInstanced))
				{
					Prop->SetPropertyFlags(CPF_InstancedReference);
				}
				Prop->SetPropertyClass(const_cast<UClass*>(Class));
				Prop->SetPropertyFlags(CPF_HasGetValueTypeHash);
				return Prop;
			}
			break;
		default:
			ensureMsgf(false, TEXT("Unhandled type %s"), *UEnum::GetValueAsString(Desc.ValueType));
		}

		return nullptr;
	}

	// Helper functions to get and set property values

	//----------------------------------------------------------------//
	//  Getters
	//----------------------------------------------------------------//

	EPropertyBagResult GetPropertyAsInt64(const FPropertyBagPropertyDesc* Desc, const void* Address, int64& OutValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		TValueOrError<int64, void> Result = UE::StructUtils::GetPropertyValueAsInt64(Desc->CachedProperty, Address);
		if (Result.HasValue())
		{
			OutValue = Result.GetValue();
			return EPropertyBagResult::Success;
		}
		return EPropertyBagResult::TypeMismatch;
	}

	EPropertyBagResult GetPropertyAsUInt64(const FPropertyBagPropertyDesc* Desc, const void* Address, uint64& OutValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		TValueOrError<uint64, void> Result = UE::StructUtils::GetPropertyValueAsUInt64(Desc->CachedProperty, Address);
		if (Result.HasValue())
		{
			OutValue = Result.GetValue();
			return EPropertyBagResult::Success;
		}
		return EPropertyBagResult::TypeMismatch;
	}

	EPropertyBagResult GetPropertyAsDouble(const FPropertyBagPropertyDesc* Desc, const void* Address, double& OutValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		TValueOrError<double, void> Result = UE::StructUtils::GetPropertyValueAsDouble(Desc->CachedProperty, Address);
		if (Result.HasValue())
		{
			OutValue = Result.GetValue();
			return EPropertyBagResult::Success;
		}
		return EPropertyBagResult::TypeMismatch;
	}

	// Generic property getter. Used for FName, FString, FText. 
	template<typename T, typename PropT>
	EPropertyBagResult GetPropertyValue(const FPropertyBagPropertyDesc* Desc, const void* Address, T& OutValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
    	{
    		return EPropertyBagResult::PropertyNotFound;
    	}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		if (!Desc->CachedProperty->IsA<PropT>())
		{
			return EPropertyBagResult::TypeMismatch;
		}
		
		const PropT* Property = CastFieldChecked<PropT>(Desc->CachedProperty);
		OutValue = Property->GetPropertyValue(Address);

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult GetPropertyValueAsEnum(const FPropertyBagPropertyDesc* Desc, const UEnum* RequestedEnum, const void* Address, int64& OutValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
    	{
    		return EPropertyBagResult::PropertyNotFound;
    	}
		if (Desc->ValueType != EPropertyBagPropertyType::Enum)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Desc->CachedProperty))
		{
			const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
			check(UnderlyingProperty);
	
			if (RequestedEnum != EnumProperty->GetEnum())
			{
				return EPropertyBagResult::TypeMismatch;
			}
	
			OutValue = UnderlyingProperty->GetSignedIntPropertyValue(Address);
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Desc->CachedProperty))
		{
			if (RequestedEnum != ByteProperty->Enum)
			{
				return EPropertyBagResult::TypeMismatch;
			}
			OutValue = ByteProperty->GetPropertyValue(Address);
		}
		else
		{
			return EPropertyBagResult::TypeMismatch;
		}

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult GetPropertyValueAsStruct(const FPropertyBagPropertyDesc* Desc, const UScriptStruct* RequestedStruct, const void* Address, FStructView& OutValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Desc->ValueType != EPropertyBagPropertyType::Struct)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Desc->CachedProperty);
		check(StructProperty->Struct);

		if (RequestedStruct != nullptr && CanCastTo(StructProperty->Struct, RequestedStruct) == false)
		{
			return EPropertyBagResult::TypeMismatch;
		}
	
		OutValue = FStructView(StructProperty->Struct, (uint8*)Address);

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult GetPropertyValueAsObject(const FPropertyBagPropertyDesc* Desc, const UClass* RequestedClass, const void* Address, UObject*& OutValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Desc->ValueType != EPropertyBagPropertyType::Object
			&& Desc->ValueType != EPropertyBagPropertyType::SoftObject
			&& Desc->ValueType != EPropertyBagPropertyType::Class
			&& Desc->ValueType != EPropertyBagPropertyType::SoftClass)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const FObjectPropertyBase* ObjectProperty = CastFieldChecked<FObjectPropertyBase>(Desc->CachedProperty);
		check(ObjectProperty->PropertyClass);

		if (RequestedClass != nullptr && CanCastTo(ObjectProperty->PropertyClass, RequestedClass) == false)
		{
			return EPropertyBagResult::TypeMismatch;
		}
	
		OutValue = ObjectProperty->GetObjectPropertyValue(Address);

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult GetPropertyValueAsSoftPath(const FPropertyBagPropertyDesc* Desc, const void* Address, FSoftObjectPath& OutValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Desc->ValueType != EPropertyBagPropertyType::SoftObject
			&& Desc->ValueType != EPropertyBagPropertyType::SoftClass)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const FSoftObjectProperty* SoftObjectProperty = CastFieldChecked<FSoftObjectProperty>(Desc->CachedProperty);
		check(SoftObjectProperty->PropertyClass);

		OutValue = SoftObjectProperty->GetPropertyValue(Address).ToSoftObjectPath();

		return EPropertyBagResult::Success;
	}

	//----------------------------------------------------------------//
	//  Setters
	//----------------------------------------------------------------//

	EPropertyBagResult SetPropertyFromInt64(const FPropertyBagPropertyDesc* Desc, void* Address, const int64 InValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		
		const bool bResult = UE::StructUtils::SetPropertyValueFromInt64(Desc->CachedProperty, Address, InValue);
		return bResult ? EPropertyBagResult::Success : EPropertyBagResult::TypeMismatch;
	}

	EPropertyBagResult SetPropertyFromUInt64(const FPropertyBagPropertyDesc* Desc, void* Address, const uint64 InValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const bool bResult = UE::StructUtils::SetPropertyValueFromUInt64(Desc->CachedProperty, Address, InValue);
		return bResult ? EPropertyBagResult::Success : EPropertyBagResult::TypeMismatch;
	}

	EPropertyBagResult SetPropertyFromDouble(const FPropertyBagPropertyDesc* Desc, void* Address, const double InValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const bool bResult = UE::StructUtils::SetPropertyValueFromDouble(Desc->CachedProperty, Address, InValue);
		return bResult ? EPropertyBagResult::Success : EPropertyBagResult::TypeMismatch;
	}

	// Generic property setter. Used for FName, FString, FText 
	template<typename T, typename PropT>
	EPropertyBagResult SetPropertyValue(const FPropertyBagPropertyDesc* Desc, void* Address, const T& InValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		if (!Desc->CachedProperty->IsA<PropT>())
		{
			return EPropertyBagResult::TypeMismatch;
		}
		
		const PropT* Property = CastFieldChecked<PropT>(Desc->CachedProperty);
		Property->SetPropertyValue(Address, InValue);

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult SetPropertyValueAsEnum(const FPropertyBagPropertyDesc* Desc, void* Address, const int64 InValue, const UEnum* Enum)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Desc->ValueType != EPropertyBagPropertyType::Enum)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Desc->CachedProperty))
		{
			const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
			check(UnderlyingProperty);

			if (Enum != EnumProperty->GetEnum())
			{
				return EPropertyBagResult::TypeMismatch;
			}
	
			UnderlyingProperty->SetIntPropertyValue(Address, InValue);
	
			return EPropertyBagResult::Success;
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Desc->CachedProperty))
		{
			if (Enum != ByteProperty->Enum)
			{
				return EPropertyBagResult::TypeMismatch;
			}
			ByteProperty->SetPropertyValue(Address, static_cast<uint8>(InValue));
			return EPropertyBagResult::Success;
		}
		else
		{
			return EPropertyBagResult::TypeMismatch;
		}
	}

	EPropertyBagResult SetPropertyValueAsStruct(const FPropertyBagPropertyDesc* Desc, void* Address, const FConstStructView InValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Desc->ValueType != EPropertyBagPropertyType::Struct)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Desc->CachedProperty);
		check(StructProperty->Struct);

		if (InValue.IsValid())
		{
			if (InValue.GetScriptStruct() != StructProperty->Struct)
			{
				return EPropertyBagResult::TypeMismatch;
			}
			StructProperty->Struct->CopyScriptStruct(Address, InValue.GetMemory());
		}
		else
		{
			StructProperty->Struct->ClearScriptStruct(Address);
		}

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult SetPropertyValueAsObject(const FPropertyBagPropertyDesc* Desc, void* Address, UObject* InValue)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Desc->ValueType != EPropertyBagPropertyType::Object
			&& Desc->ValueType != EPropertyBagPropertyType::SoftObject
			&& Desc->ValueType != EPropertyBagPropertyType::Class
			&& Desc->ValueType != EPropertyBagPropertyType::SoftClass)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		
		const FObjectPropertyBase* ObjectProperty = CastFieldChecked<FObjectPropertyBase>(Desc->CachedProperty);
		check(ObjectProperty->PropertyClass);
		check(Desc->ValueTypeObject);

		if (Desc->ValueType == EPropertyBagPropertyType::Object
			|| Desc->ValueType == EPropertyBagPropertyType::SoftObject)
		{
			if (InValue && CanCastTo(InValue->GetClass(),  ObjectProperty->PropertyClass) == false)
			{
				return EPropertyBagResult::TypeMismatch;
			}
		}
		else
		{
			check(Desc->ValueType == EPropertyBagPropertyType::Class
				|| Desc->ValueType == EPropertyBagPropertyType::SoftClass);
			const UClass* Class = Cast<UClass>(InValue);
			const UClass* PropClass = nullptr;

			if (const FClassProperty* ClassProperty = CastFieldChecked<FClassProperty>(Desc->CachedProperty))
			{
				PropClass = ClassProperty->MetaClass;
			}
			else if (const FSoftClassProperty* SoftClassProperty = CastFieldChecked<FSoftClassProperty>(Desc->CachedProperty))
			{
				PropClass = SoftClassProperty->MetaClass;
			}
			
			if (!Class || !PropClass || !Class->IsChildOf(PropClass))
			{
				return EPropertyBagResult::TypeMismatch;
			}
		}

		ObjectProperty->SetObjectPropertyValue(Address, InValue);

		return EPropertyBagResult::Success;
	}

	EPropertyBagResult SetPropertyValueAsSoftPath(const FPropertyBagPropertyDesc* Desc, void* Address, const FSoftObjectPath& InPath, const UObject* InObject = nullptr)
	{
		if (Desc == nullptr || Desc->CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}
		if (Desc->ValueType != EPropertyBagPropertyType::SoftObject &&
			Desc->ValueType != EPropertyBagPropertyType::SoftClass)
		{
			return EPropertyBagResult::TypeMismatch;
		}
		if (Address == nullptr)
		{
			return EPropertyBagResult::OutOfBounds;
		}
		if (Desc->ContainerTypes.Num() > 0)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		const FSoftObjectProperty* SoftObjectProperty = CastFieldChecked<FSoftObjectProperty>(Desc->CachedProperty);
		check(SoftObjectProperty->PropertyClass);
		check(Desc->ValueTypeObject);

		// Can perform extra validation when source object is provided
		if (InObject)
		{
			if (Desc->ValueType == EPropertyBagPropertyType::SoftObject)
			{
				if (CanCastTo(InObject->GetClass(), SoftObjectProperty->PropertyClass) == false)
				{
					return EPropertyBagResult::TypeMismatch;
				}
			}
			else
			{
				check(Desc->ValueType == EPropertyBagPropertyType::SoftClass);
				const UClass* Class = CastChecked<UClass>(InObject);
				const UClass* PropClass = nullptr;

				if (const FSoftClassProperty* SoftClassProperty = CastFieldChecked<FSoftClassProperty>(Desc->CachedProperty))
				{
					PropClass = SoftClassProperty->MetaClass;
				}
			
				if (!Class || !PropClass || !Class->IsChildOf(PropClass))
				{
					return EPropertyBagResult::TypeMismatch;
				}
			}
		}

		SoftObjectProperty->SetPropertyValue(Address, FSoftObjectPtr(InPath));

		return EPropertyBagResult::Success;
	}

	/**
	 * Copies properties from Source to Target property bag. The bag layouts do not need to match.
	 * 
	 * The provided FindTargetPropertyFunction should return the matching property in the Target 
	 * property bag, or nullptr if there isn't one.
	 */
	void CopyMatchingValues(
		const FConstStructView Source, 
		FStructView Target, 
		TFunctionRef<const FPropertyBagPropertyDesc*(TNotNull<const UPropertyBag*> /* TargetBagStruct */, const FPropertyBagPropertyDesc& /* SourceDesc */)> FindTargetPropertyFunction)
	{
		if (!Source.IsValid() || !Target.IsValid())
		{
			return;
		}

		const UPropertyBag* SourceBagStruct = Cast<const UPropertyBag>(Source.GetScriptStruct());
		const UPropertyBag* TargetBagStruct = Cast<const UPropertyBag>(Target.GetScriptStruct());

		if (!SourceBagStruct || !TargetBagStruct)
		{
			return;
		}

		// Iterate over source and copy to target if possible. Source is expected to usually have less items.
		for (const FPropertyBagPropertyDesc& SourceDesc : SourceBagStruct->GetPropertyDescs())
		{
			const FPropertyBagPropertyDesc* PotentialTargetDesc = FindTargetPropertyFunction(TargetBagStruct, SourceDesc);
			if (PotentialTargetDesc == nullptr
				|| PotentialTargetDesc->CachedProperty == nullptr
				|| SourceDesc.CachedProperty == nullptr)
			{
				continue;
			}

			const FPropertyBagPropertyDesc& TargetDesc = *PotentialTargetDesc;
			void* TargetAddress = Target.GetMemory() + TargetDesc.CachedProperty->GetOffset_ForInternal();
			const void* SourceAddress = Source.GetMemory() + SourceDesc.CachedProperty->GetOffset_ForInternal();
			
			if (TargetDesc.CompatibleType(SourceDesc))
			{
				TargetDesc.CachedProperty->CopyCompleteValue(TargetAddress, SourceAddress);
			}
			else if (TargetDesc.ContainerTypes.Num() == 0
					&& SourceDesc.ContainerTypes.Num() == 0)
			{
				if (TargetDesc.IsNumericType() && SourceDesc.IsNumericType())
				{
					// Try to convert numeric types.
					if (TargetDesc.IsNumericFloatType())
					{
						double Value = 0;
						if (GetPropertyAsDouble(&SourceDesc, SourceAddress, Value) == EPropertyBagResult::Success)
						{
							SetPropertyFromDouble(&TargetDesc, TargetAddress, Value);
						}
					}
					else
					{
						if (TargetDesc.IsUnsignedNumericType())
						{
							uint64 Value = 0;
							if (GetPropertyAsUInt64(&SourceDesc, SourceAddress, Value) == EPropertyBagResult::Success)
							{
								SetPropertyFromUInt64(&TargetDesc, TargetAddress, Value);
							}
						}
						else
						{
							int64 Value = 0;
							if (GetPropertyAsInt64(&SourceDesc, SourceAddress, Value) == EPropertyBagResult::Success)
							{
								SetPropertyFromInt64(&TargetDesc, TargetAddress, Value);
							}
						}
					}
				}
				else if ((TargetDesc.IsObjectType() && SourceDesc.IsObjectType())
					|| (TargetDesc.IsClassType() && SourceDesc.IsClassType()))
				{
					// Try convert between compatible objects and classes.
					const UClass* TargetObjectClass = Cast<const UClass>(TargetDesc.ValueTypeObject);
					const UClass* SourceObjectClass = Cast<const UClass>(SourceDesc.ValueTypeObject);
					if (CanCastTo(SourceObjectClass, TargetObjectClass))
					{
						const FObjectPropertyBase* TargetProp = CastFieldChecked<FObjectPropertyBase>(TargetDesc.CachedProperty);
						const FObjectPropertyBase* SourceProp = CastFieldChecked<FObjectPropertyBase>(SourceDesc.CachedProperty);
						TargetProp->SetObjectPropertyValue(TargetAddress, SourceProp->GetObjectPropertyValue(SourceAddress));
					}
				}
			}
		}
	}

	void CopyMatchingValuesByID(const FConstStructView Source, FStructView Target)
	{
		CopyMatchingValues(
			Source, 
			Target, 
			[](TNotNull<const UPropertyBag*> TargetBagStruct, const FPropertyBagPropertyDesc& SourceDesc)
			{
				return TargetBagStruct->FindPropertyDescByID(SourceDesc.ID);
			});
	}

	void CopyMatchingValuesByIDWithOverrides(const FConstStructView Source, FStructView Target, TConstArrayView<FGuid> Overrides)
	{
		CopyMatchingValues(
			Source, 
			Target, 
			[&Overrides](TNotNull<const UPropertyBag*> TargetBagStruct, const FPropertyBagPropertyDesc& SourceDesc) -> const FPropertyBagPropertyDesc*
			{
				if (Overrides.Contains(SourceDesc.ID))
				{
					return TargetBagStruct->FindPropertyDescByID(SourceDesc.ID);
				}

				return nullptr;
			});
	}

	void RemovePropertyByName(TArray<FPropertyBagPropertyDesc>& Descs, const FName PropertyName, const int32 StartIndex = 0)
	{
		// Remove properties which dont have unique name.
		for (int32 Index = StartIndex; Index < Descs.Num(); Index++)
		{
			if (Descs[Index].Name == PropertyName)
			{
				Descs.RemoveAt(Index);
				Index--;
			}
		}
	}

	EPropertyBagAlterationResult RenameProperty(TArray<FPropertyBagPropertyDesc>& Descs, const FName SourcePropertyName, const FName TargetPropertyName)
	{
		FName NewTargetPropertyName = TargetPropertyName;
		if (CVarEnforceValidName.GetValueOnAnyThread())
		{
			if (SourcePropertyName.IsNone() || TargetPropertyName.IsNone())
			{
				return EPropertyBagAlterationResult::PropertyNameEmpty;
			}
			else if (!FInstancedPropertyBag::IsPropertyNameValid(TargetPropertyName))
			{
				return EPropertyBagAlterationResult::PropertyNameInvalidCharacters;
			}
		}
		else
		{
			NewTargetPropertyName = FInstancedPropertyBag::SanitizePropertyName(NewTargetPropertyName);
		}

		int32 RenamedPropertyIndex = INDEX_NONE;
		// Rename the first instance of PropertyName to NewName
		for (int32 Index = 0; Index < Descs.Num(); ++Index)
		{
			if (Descs[Index].Name == NewTargetPropertyName)
			{
				return EPropertyBagAlterationResult::TargetPropertyAlreadyExists;
			}
			else if (Descs[Index].Name == SourcePropertyName)
			{
				if (!ensure(RenamedPropertyIndex == INDEX_NONE))
				{
					// There are duplicate source properties
					return EPropertyBagAlterationResult::InternalError;
				}

				RenamedPropertyIndex = Index;
			}
		}

		if (RenamedPropertyIndex != INDEX_NONE)
		{
			Descs[RenamedPropertyIndex].Name = NewTargetPropertyName;
			return EPropertyBagAlterationResult::Success;
		}
		else
		{
			return EPropertyBagAlterationResult::SourcePropertyNotFound;
		}
	}

	EPropertyBagAlterationResult ReorderProperty(TArray<FPropertyBagPropertyDesc>& Descs, const int32 SourcePropertyIndex, const int32 TargetPropertyIndex, const bool bInsertBefore)
	{
		// If the source and the target are the same, the reordering needs no operation.
		if (SourcePropertyIndex == TargetPropertyIndex)
		{
			return EPropertyBagAlterationResult::NoOperation;
		}
		else if (!Descs.IsValidIndex(SourcePropertyIndex))
		{
			return EPropertyBagAlterationResult::SourcePropertyNotFound;
		}
		else if (!Descs.IsValidIndex(TargetPropertyIndex))
		{
			return EPropertyBagAlterationResult::TargetPropertyNotFound;
		}

		FPropertyBagPropertyDesc SourceDesc = MoveTemp(Descs[SourcePropertyIndex]);
		Descs.RemoveAt(SourcePropertyIndex);

		int32 NewTargetPropertyIndex = TargetPropertyIndex;
		if (SourcePropertyIndex < TargetPropertyIndex)
		{
			--NewTargetPropertyIndex;
		}

		if (!bInsertBefore)
		{
			++NewTargetPropertyIndex;
		}

		Descs.Insert(MoveTemp(SourceDesc), NewTargetPropertyIndex);

		return EPropertyBagAlterationResult::Success;
	}

	namespace Constants
	{
		/**
		 * Minimal set of invalid characters that must be removed for a name to be valid. This allows convenience
		 * characters (ex. spaces) in display names for better readability, but stricter sanitization can and should
		 * be applied later.
		 * Derived from INVALID_OBJECTNAME_CHARACTERS.
		 */
		constexpr TCHAR MinimalInvalidNameCharacters[] = TEXT("\"'/.:&!?~\\\n\r\t@#(){}[]<>=;^%$`");
		constexpr TCHAR InvalidNameCharacters[] = TEXT(" \"',/.:|&!?~\\\n\r\t@#(){}[]<>=;^%$`+*");
	}
} // UE::StructUtils::Private


//----------------------------------------------------------------//
//  FPropertyBagContainerTypes
//----------------------------------------------------------------//
EPropertyBagContainerType FPropertyBagContainerTypes::PopHead()
{
	EPropertyBagContainerType Head = EPropertyBagContainerType::None;
		
	if (NumContainers > 0)
	{
		Swap(Head, Types[0]);

		uint8 Index = NumContainers - 1;
		while (Index > 0)
		{
			Types[Index - 1] = Types[Index];
			Types[Index] = EPropertyBagContainerType::None;
			Index--;
		}
		NumContainers--;
	}

	return Head;
}

void FPropertyBagContainerTypes::Serialize(FArchive& Ar)
{
	Ar << NumContainers;
	for (int i = 0; i < NumContainers; ++i)
	{
		Ar << Types[i];
	}
}

bool FPropertyBagContainerTypes::operator == (const FPropertyBagContainerTypes& Other) const
{
	if (NumContainers != Other.NumContainers)
	{
		return false;
	}

	for (int i = 0; i < NumContainers; ++i)
	{
		if (Types[i] != Other.Types[i])
		{
			return false;
		}
	}

	return true;
}

//----------------------------------------------------------------//
//  FPropertyBagPropertyDesc
//----------------------------------------------------------------//

void FPropertyBagPropertyDescMetaData::Serialize(FArchive& Ar)
{
	Ar << Key;
	Ar << Value;
}

FPropertyBagPropertyDesc::FPropertyBagPropertyDesc(const FName InName, const FProperty* InSourceProperty)
	: Name(InName)
{
	ValueType = UE::StructUtils::Private::GetValueTypeFromProperty(InSourceProperty);
	ValueTypeObject = UE::StructUtils::Private::GetValueTypeObjectFromProperty(InSourceProperty);

	KeyType = UE::StructUtils::Private::GetKeyTypeFromProperty(InSourceProperty);
	KeyTypeObject = UE::StructUtils::Private::GetKeyTypeObjectFromProperty(InSourceProperty);

	// @todo : improve error handling - if we reach the nested containers limit, the Desc will be invalid (empty container types)
	ContainerTypes = UE::StructUtils::Private::GetContainerTypesFromProperty(InSourceProperty);

#if WITH_EDITORONLY_DATA
	if (const FClassProperty* ClassProperty = CastField<FClassProperty>(InSourceProperty))
	{
		MetaClass = ClassProperty->MetaClass;
	}
	else if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(InSourceProperty))
	{
		MetaClass = SoftClassProperty->MetaClass;
	}

	if (const TMap<FName, FString>* SourcePropertyMetaData = InSourceProperty->GetMetaDataMap())
	{
		for (const TPair<FName, FString>& MetaDataPair : *SourcePropertyMetaData)
		{
			MetaData.Add({ MetaDataPair.Key, MetaDataPair.Value });
		}
	}
#endif
}

FArchive& operator<<(FArchive& Ar, FPropertyBagPropertyDesc& Bag)
{
	Ar << Bag.ValueTypeObject;
	Ar << Bag.ID;
	Ar << Bag.Name;
	Ar << Bag.ValueType;

	if (Ar.CustomVer(FPropertyBagCustomVersion::GUID) >= FPropertyBagCustomVersion::ContainerTypes)
	{
		if (Ar.IsLoading() && Ar.CustomVer(FPropertyBagCustomVersion::GUID) < FPropertyBagCustomVersion::NestedContainerTypes)
		{
			EPropertyBagContainerType TmpContainerType = EPropertyBagContainerType::None;
			Ar << TmpContainerType;

			if (TmpContainerType != EPropertyBagContainerType::None)
			{
				Bag.ContainerTypes.Add(TmpContainerType);
			}
		}
		else
		{
			Ar << Bag.ContainerTypes;
		}
	}

	bool bHasMetaData = false;
#if WITH_EDITORONLY_DATA
	if (Ar.IsSaving() && !Ar.IsCooking())
	{
		bHasMetaData = Bag.MetaData.Num() > 0 || Bag.MetaClass;
	}
#endif
	Ar << bHasMetaData;

	if (bHasMetaData)
	{
#if WITH_EDITORONLY_DATA
		Ar << Bag.MetaData;

		if (Ar.CustomVer(FPropertyBagCustomVersion::GUID) >= FPropertyBagCustomVersion::MetaClass)
		{
			Ar << Bag.MetaClass;
		}
#else
		TArray<FPropertyBagPropertyDescMetaData> TempMetaData; 
		Ar << TempMetaData;

		if (Ar.CustomVer(FPropertyBagCustomVersion::GUID) >= FPropertyBagCustomVersion::MetaClass)
		{
			TObjectPtr<UClass> TempMetaClass = nullptr;
			Ar << TempMetaClass;
		}
#endif
	}

	if (Ar.CustomVer(FPropertyBagCustomVersion::GUID) >= FPropertyBagCustomVersion::PropertyFlags)
	{
		Ar << Bag.PropertyFlags;
	}

	if (Ar.CustomVer(FPropertyBagCustomVersion::GUID) >= FPropertyBagCustomVersion::KeyTypes)
	{
		Ar << Bag.KeyType;
		Ar << Bag.KeyTypeObject;
	}
	
	return Ar;
}

bool FPropertyBagPropertyDesc::IsNumericType() const
{
	switch (ValueType)
	{
	case EPropertyBagPropertyType::Bool: return true;
	case EPropertyBagPropertyType::Int8: return true;
	case EPropertyBagPropertyType::Byte: return true;
	case EPropertyBagPropertyType::Int16: return true;
	case EPropertyBagPropertyType::UInt16: return true;
	case EPropertyBagPropertyType::Int32: return true;
	case EPropertyBagPropertyType::UInt32: return true;
	case EPropertyBagPropertyType::Int64: return true;
	case EPropertyBagPropertyType::UInt64: return true;
	case EPropertyBagPropertyType::Float: return true;
	case EPropertyBagPropertyType::Double: return true;
	default: return false;
	}
}

bool FPropertyBagPropertyDesc::IsUnsignedNumericType() const
{
	switch (ValueType)
	{
	case EPropertyBagPropertyType::Byte: return true;
	case EPropertyBagPropertyType::UInt16: return true;
	case EPropertyBagPropertyType::UInt32: return true;
	case EPropertyBagPropertyType::UInt64: return true;
	default: return false;
	}
}

bool FPropertyBagPropertyDesc::IsNumericIntegralType() const
{
	return UE::StructUtils::Private::IsNumericIntegralType(ValueType);
}

bool FPropertyBagPropertyDesc::IsNumericFloatType() const
{
	switch (ValueType)
	{
	case EPropertyBagPropertyType::Float: return true;
	case EPropertyBagPropertyType::Double: return true;
	default: return false;
	}
}

bool FPropertyBagPropertyDesc::IsObjectType() const
{
	switch (ValueType)
	{
	case EPropertyBagPropertyType::Object: return true;
	case EPropertyBagPropertyType::SoftObject: return true;
	default: return false;
	}
}

bool FPropertyBagPropertyDesc::IsClassType() const
{
	switch (ValueType)
	{
	case EPropertyBagPropertyType::Class: return true;
	case EPropertyBagPropertyType::SoftClass: return true;
	default: return false;
	}
}

bool FPropertyBagPropertyDesc::operator==(const FPropertyBagPropertyDesc& OtherDesc) const
{
	if (ID.IsValid() && ID == OtherDesc.ID)
	{
#if WITH_EDITOR
		if (CVarValidatePropertyBagDescriptionEquality.GetValueOnAnyThread())
		{
			check(CachedProperty == OtherDesc.CachedProperty);
			check(ValueType == OtherDesc.ValueType);
			check(ValueTypeObject == OtherDesc.ValueTypeObject);
			check(Name == OtherDesc.Name);
			check(ContainerTypes == OtherDesc.ContainerTypes);
			check(PropertyFlags == OtherDesc.PropertyFlags);
			check(CachedIndex == OtherDesc.CachedIndex);
			check(MetaData == OtherDesc.MetaData);
			check(MetaClass == OtherDesc.MetaClass);
		}
#endif // WITH_EDITOR

		return true;
	}
	else
	{
		return false;
	}
}

bool FPropertyBagPropertyDesc::CompatibleType(const FPropertyBagPropertyDesc& Other) const
{
	// Containers must match
	if (ContainerTypes != Other.ContainerTypes)
	{
		return false;
	}

	// Values must match.
	if (ValueType != Other.ValueType)
	{
		return false;
	}

	// Struct and enum must have same value type class
	if (ValueType == EPropertyBagPropertyType::Enum || ValueType == EPropertyBagPropertyType::Struct)
	{
		if (ValueTypeObject != Other.ValueTypeObject)
		{
			return false;
		}
	}

	// Objects should be castable.
	if (ValueType == EPropertyBagPropertyType::Object)
	{
		const UClass* ObjectClass = Cast<const UClass>(ValueTypeObject);
		const UClass* OtherObjectClass = Cast<const UClass>(Other.ValueTypeObject);
		if (!UE::StructUtils::Private::CanCastTo(OtherObjectClass, ObjectClass))
		{
			return false;
		}
	}

	// Keys must match.
	if (KeyType != Other.KeyType)
	{
		return false;
	}

	// Struct and enum keys must have same key type class
	if (KeyType == EPropertyBagPropertyType::Enum || KeyType == EPropertyBagPropertyType::Struct)
	{
		return KeyTypeObject == Other.KeyTypeObject;
	}

	// Object keys should be castable.
	if (KeyType == EPropertyBagPropertyType::Object)
	{
		const UClass* ObjectClass = Cast<const UClass>(KeyTypeObject);
		const UClass* OtherObjectClass = Cast<const UClass>(Other.KeyTypeObject);
		return UE::StructUtils::Private::CanCastTo(OtherObjectClass, ObjectClass);
	}

	return true;
}

#if WITH_EDITOR
bool FPropertyBagPropertyDesc::HasMetaData(FName SpecifierName) const
{
	return MetaData.ContainsByPredicate([SpecifierName](const FPropertyBagPropertyDescMetaData& MetaData)
	{
		return MetaData.Key == SpecifierName;
	});
}

void FPropertyBagPropertyDesc::SetMetaData(FName SpecifierName, const FString& InValue)
{
	if (FPropertyBagPropertyDescMetaData* Specifier =
		MetaData.FindByPredicate([SpecifierName](const FPropertyBagPropertyDescMetaData& InMetaData) { return InMetaData.Key == SpecifierName; }))
	{
		Specifier->Value = InValue;
	}
	else
	{
		MetaData.Emplace(FPropertyBagPropertyDescMetaData(SpecifierName, InValue));
	}
}

FString FPropertyBagPropertyDesc::GetMetaData(FName SpecifierName) const
{
	if (const FPropertyBagPropertyDescMetaData* Specifier = MetaData.FindByPredicate(
		[SpecifierName](const FPropertyBagPropertyDescMetaData& MetaData) { return MetaData.Key == SpecifierName; }))
	{
		return *Specifier->Value;
	}
	else
	{
		return {};
	}
}

void FPropertyBagPropertyDesc::RemoveMetadata(FName SpecifierName)
{
	MetaData.RemoveAll([SpecifierName](const FPropertyBagPropertyDescMetaData& MetaData)
	{
		return MetaData.Key == SpecifierName;
	});
}
#endif // WITH_EDITOR

//----------------------------------------------------------------//
//  FInstancedPropertyBag
//----------------------------------------------------------------//

void FInstancedPropertyBag::InitializeFromBagStruct(const UPropertyBag* NewBagStruct)
{
	Value.InitializeAs(NewBagStruct);
}

void FInstancedPropertyBag::CopyMatchingValuesByID(const FInstancedPropertyBag& Other, TOptional<TConstArrayView<FGuid>> OptionalPropertyIdsSubset)
{
	if (OptionalPropertyIdsSubset.IsSet())
	{
		UE::StructUtils::Private::CopyMatchingValuesByIDWithOverrides(Other.Value, Value, OptionalPropertyIdsSubset.GetValue());
	}
	else
	{
		UE::StructUtils::Private::CopyMatchingValuesByID(Other.Value, Value);
	}
}

void FInstancedPropertyBag::CopyMatchingValuesByName(const FInstancedPropertyBag& Other, TOptional<TConstArrayView<FName>> OptionalPropertyNamesSubset)
{
	UE::StructUtils::Private::CopyMatchingValues(
		Other.Value,
		Value,
		[&OptionalPropertyNamesSubset](TNotNull<const UPropertyBag*> TargetBagStruct, const FPropertyBagPropertyDesc& SourceDesc) -> const FPropertyBagPropertyDesc*
		{
			if (OptionalPropertyNamesSubset.IsSet() && !OptionalPropertyNamesSubset->Contains(SourceDesc.Name))
			{
				return nullptr;
			}
			else
			{
				return TargetBagStruct->FindPropertyDescByName(SourceDesc.Name);
			}
		});
}

int32 FInstancedPropertyBag::GetNumPropertiesInBag() const
{
	if (const UPropertyBag* BagStruct = GetPropertyBagStruct())
	{
		return BagStruct->PropertyDescs.Num();
	}

	return 0;
}

EPropertyBagAlterationResult FInstancedPropertyBag::AddProperties(const TConstArrayView<FPropertyBagPropertyDesc> NewDescs, const bool bOverwrite)
{
	TArray<FPropertyBagPropertyDesc> Descs;
	if (const UPropertyBag* CurrentBagStruct = GetPropertyBagStruct())
	{
		Descs = CurrentBagStruct->GetPropertyDescs();
	}

	for (const FPropertyBagPropertyDesc& NewDesc : NewDescs)
	{
		if (CVarEnforceValidName.GetValueOnAnyThread())
		{
			if (NewDesc.Name.IsNone())
			{
				return EPropertyBagAlterationResult::PropertyNameEmpty;
			}
			else if (!IsPropertyNameValid(NewDesc.Name))
			{
				return EPropertyBagAlterationResult::PropertyNameInvalidCharacters;
			}
		}

		if (!UE::StructUtils::Private::IsTypeHashable(NewDesc.KeyType, NewDesc.KeyTypeObject))
		{
			return EPropertyBagAlterationResult::PropertyKeyTypeNotHashable;
		}

		FPropertyBagPropertyDesc* ExistingProperty = Descs.FindByPredicate([&NewDesc](const FPropertyBagPropertyDesc& Desc) { return Desc.Name == NewDesc.Name; });
		if (ExistingProperty != nullptr)
		{
			if (bOverwrite)
			{
				const FGuid ExistingID = ExistingProperty->ID;
				*ExistingProperty = NewDesc;
				ExistingProperty->ID = ExistingID;
			}
			else
			{
				return EPropertyBagAlterationResult::TargetPropertyAlreadyExists;
			}
		}
		else if (NewDesc.ValueType != EPropertyBagPropertyType::None)
		{
			FPropertyBagPropertyDesc& Desc = Descs.Add_GetRef(NewDesc);
			Desc.Name = SanitizePropertyName(NewDesc.Name, /*ReplacementChar=*/TEXT('_'));
		}
	}

	const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(Descs);
	MigrateToNewBagStruct(NewBagStruct);

	return EPropertyBagAlterationResult::Success;
}

EPropertyBagAlterationResult FInstancedPropertyBag::AddProperty(const FName InName, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject, const bool bOverwrite)
{
	return AddProperties({ FPropertyBagPropertyDesc(InName, InValueType, InValueTypeObject) }, bOverwrite);
}

EPropertyBagAlterationResult FInstancedPropertyBag::AddContainerProperty(const FName InName, const EPropertyBagContainerType InContainerType, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject, const bool bOverwrite, const EPropertyBagPropertyType InKeyType, const UObject* InKeyTypeObject)
{
	return AddProperties({ FPropertyBagPropertyDesc(InName, InContainerType, InValueType, InValueTypeObject, CPF_Edit, InKeyType, InKeyTypeObject) }, bOverwrite);
}

EPropertyBagAlterationResult FInstancedPropertyBag::AddContainerProperty(const FName InName, const FPropertyBagContainerTypes InContainerTypes, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject, const bool bOverwrite, const EPropertyBagPropertyType InKeyType, const UObject* InKeyTypeObject)
{
	return AddProperties({ FPropertyBagPropertyDesc(InName, InContainerTypes, InValueType, InValueTypeObject, CPF_Edit, InKeyType, InKeyTypeObject) }, bOverwrite);
}

EPropertyBagAlterationResult FInstancedPropertyBag::AddMapProperty(const FName InName, const EPropertyBagPropertyType InKeyType, const EPropertyBagPropertyType InValueType, const UObject* InKeyTypeObject, const UObject* InValueTypeObject, const EPropertyBagContainerType InNestedContainerType, bool bOverwrite)
{
	return AddProperties({ FPropertyBagPropertyDesc(InName, { EPropertyBagContainerType::Map, InNestedContainerType }, InValueType, InValueTypeObject, CPF_Edit, InKeyType, InKeyTypeObject)}, bOverwrite);
}

EPropertyBagAlterationResult FInstancedPropertyBag::AddProperty(const FName InName, const FProperty* InSourceProperty, const bool bOverwrite)
{
	return AddProperties({ FPropertyBagPropertyDesc(InName, InSourceProperty ) }, bOverwrite);
}

EPropertyBagAlterationResult FInstancedPropertyBag::DuplicateProperty(const FName InName, FName* InOutNewName)
{
	if (const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(InName))
	{
		FName NewName = InOutNewName ? *InOutNewName : NAME_None;
		if (NewName.IsNone())
		{
			NewName = InName;

			do
			{
				// We start from _2 (so Number=3)
				NewName.SetNumber(FMath::Max(NewName.GetNumber() + 1, 3));
			}
			while (FindPropertyDescByName(NewName));

			if (InOutNewName)
			{
				*InOutNewName = NewName;
			}
		}

		FPropertyBagPropertyDesc NewDesc = *Desc;
		NewDesc.ID = FGuid();
		NewDesc.Name = NewName;

		const EPropertyBagAlterationResult Result = AddProperties({ NewDesc }, false);
		if (Result != EPropertyBagAlterationResult::Success)
		{
			return Result;
		}

		const FPropertyBagPropertyDesc* OriginalDesc = FindPropertyDescByName(InName);
		const FPropertyBagPropertyDesc* AddedDesc = FindPropertyDescByName(NewName);
		check(OriginalDesc && AddedDesc && OriginalDesc->CachedProperty && AddedDesc->CachedProperty);

		void* TargetAddress = Value.GetMutableMemory() + AddedDesc->CachedProperty->GetOffset_ForInternal();
		void const* SourceAddress = Value.GetMemory() + OriginalDesc->CachedProperty->GetOffset_ForInternal();
		AddedDesc->CachedProperty->CopyCompleteValue(TargetAddress, SourceAddress);

		return EPropertyBagAlterationResult::Success;
	}

	return EPropertyBagAlterationResult::SourcePropertyNotFound;
}

EPropertyBagResult FInstancedPropertyBag::ReplaceAllPropertiesAndValues(const TConstArrayView<FPropertyBagPropertyDesc> InDescs, const TConstArrayView<TConstArrayView<uint8>> InValues)
{
	if (InDescs.Num() != InValues.Num())
	{
		return EPropertyBagResult::OutOfBounds;
	}

	Reset();

	const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(InDescs);
	InitializeFromBagStruct(NewBagStruct);

	for (int32 Index = 0; Index < NewBagStruct->PropertyDescs.Num(); ++Index)
	{
		const FPropertyBagPropertyDesc& Desc = NewBagStruct->PropertyDescs[Index];
		const TConstArrayView<uint8>& NewValue = InValues[Index];

		if (NewValue.Num() == Desc.CachedProperty->GetSize())
		{
			void* TargetAddress = Value.GetMutableMemory() + Desc.CachedProperty->GetOffset_ForInternal();
			void const* SourceAddress = NewValue.GetData();
			Desc.CachedProperty->CopyCompleteValue(TargetAddress, SourceAddress);
		}
		else
		{
			return EPropertyBagResult::TypeMismatch;
		}
	}

	return EPropertyBagResult::Success;
}

EPropertyBagAlterationResult FInstancedPropertyBag::RemovePropertiesByName(const TConstArrayView<FName> PropertiesToRemove)
{
	TArray<FPropertyBagPropertyDesc> Descs;
	if (const UPropertyBag* CurrentBagStruct = GetPropertyBagStruct())
	{
		Descs = CurrentBagStruct->GetPropertyDescs();
	}

	for (const FName Name : PropertiesToRemove)
	{
		UE::StructUtils::Private::RemovePropertyByName(Descs, Name);
	}

	const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(Descs);
	MigrateToNewBagStruct(NewBagStruct);

	return EPropertyBagAlterationResult::Success;
}

EPropertyBagAlterationResult FInstancedPropertyBag::RemovePropertyByName(const FName PropertyToRemove)
{
	return RemovePropertiesByName({PropertyToRemove});
}

EPropertyBagAlterationResult FInstancedPropertyBag::RenameProperty(const FName PropertyToRename, const FName NewName)
{
	TArray<FPropertyBagPropertyDesc> Descs;
	if (const UPropertyBag* CurrentBagStruct = GetPropertyBagStruct())
	{
		Descs = CurrentBagStruct->GetPropertyDescs();
	}

	const EPropertyBagAlterationResult Result = UE::StructUtils::Private::RenameProperty(Descs, PropertyToRename, NewName);
	if (Result == EPropertyBagAlterationResult::Success)
	{
		const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(Descs);
		MigrateToNewBagStruct(NewBagStruct);
	}

	return Result;
}

EPropertyBagAlterationResult FInstancedPropertyBag::ReorderProperty(const FName SourcePropertyName, const FName TargetPropertyName, const bool bInsertBefore)
{
	int32 SourceIndex = INDEX_NONE;
	int32 TargetIndex = INDEX_NONE;
	if (const UPropertyBag* CurrentBagStruct = GetPropertyBagStruct())
	{
		const FPropertyBagPropertyDesc* SourceDesc = CurrentBagStruct->FindPropertyDescByName(SourcePropertyName);
		SourceIndex = SourceDesc ? SourceDesc->GetCachedIndex() : INDEX_NONE;
		const FPropertyBagPropertyDesc* TargetDesc = CurrentBagStruct->FindPropertyDescByName(TargetPropertyName);
		TargetIndex = TargetDesc ? TargetDesc->GetCachedIndex() : INDEX_NONE;
	}

	return ReorderProperty(SourceIndex, TargetIndex, bInsertBefore);
}

EPropertyBagAlterationResult FInstancedPropertyBag::ReorderProperty(const int32 SourcePropertyIndex, const int32 TargetPropertyIndex, const bool bInsertBefore)
{
	TArray<FPropertyBagPropertyDesc> Descs;
	if (const UPropertyBag* CurrentBagStruct = GetPropertyBagStruct())
	{
		Descs = CurrentBagStruct->GetPropertyDescs();
	}

	const EPropertyBagAlterationResult Result = UE::StructUtils::Private::ReorderProperty(Descs, SourcePropertyIndex, TargetPropertyIndex, bInsertBefore);
	if (Result == EPropertyBagAlterationResult::Success)
	{
		const UPropertyBag* NewBagStruct = UPropertyBag::GetOrCreateFromDescs(Descs);
		MigrateToNewBagStruct(NewBagStruct);
	}

	return Result;
}

void FInstancedPropertyBag::MigrateToNewBagStruct(const UPropertyBag* NewBagStruct)
{
	FInstancedStruct NewValue(NewBagStruct);

	UE::StructUtils::Private::CopyMatchingValuesByID(Value, NewValue);
	
	Value = MoveTemp(NewValue);
}

void FInstancedPropertyBag::MigrateToNewBagInstance(const FInstancedPropertyBag& NewBagInstance)
{
	FInstancedStruct NewValue(NewBagInstance.Value);

	UE::StructUtils::Private::CopyMatchingValuesByID(Value, NewValue);
	
	Value = MoveTemp(NewValue);
}

void FInstancedPropertyBag::MigrateToNewBagInstanceWithOverrides(const FInstancedPropertyBag& NewBagInstance, TConstArrayView<FGuid> OverriddenPropertyIDs)
{
	FInstancedStruct NewValue(NewBagInstance.Value);

	UE::StructUtils::Private::CopyMatchingValuesByIDWithOverrides(Value, NewValue, OverriddenPropertyIDs);
	
	Value = MoveTemp(NewValue);
}

const UPropertyBag* FInstancedPropertyBag::GetPropertyBagStruct() const
{
	return Value.IsValid() ? Cast<const UPropertyBag>(Value.GetScriptStruct()) : nullptr;
}

const FPropertyBagPropertyDesc* FInstancedPropertyBag::FindPropertyDescByID(const FGuid ID) const
{
	if (const UPropertyBag* BagStruct = GetPropertyBagStruct())
	{
		return BagStruct->FindPropertyDescByID(ID);
	}
	return nullptr;
}
	
const FPropertyBagPropertyDesc* FInstancedPropertyBag::FindPropertyDescByName(const FName Name) const
{
	if (const UPropertyBag* BagStruct = GetPropertyBagStruct())
	{
		return BagStruct->FindPropertyDescByName(Name);
	}
	return nullptr;
}

bool FInstancedPropertyBag::OwnsPropertyDesc(const FPropertyBagPropertyDesc& Desc) const
{
	if (const UPropertyBag* BagStruct = GetPropertyBagStruct())
	{
		return BagStruct->OwnsPropertyDesc(Desc);
	}
	return false;
}

bool FInstancedPropertyBag::HasSameLayout(const FInstancedPropertyBag& Other) const
{
	const UPropertyBag* BagStruct = GetPropertyBagStruct();
	const UPropertyBag* OtherBagStruct = Other.GetPropertyBagStruct();

	// Note: FStructUtils::TheSameLayout will check nullptr & type per property, not just layout. We do not check name however
	return FStructUtils::TheSameLayout(BagStruct, OtherBagStruct);
}

const void* FInstancedPropertyBag::GetValueAddress(const FPropertyBagPropertyDesc* Desc) const
{
	if (Desc == nullptr || !Value.IsValid())
	{
		return nullptr;
	}
	return Value.GetMemory() + Desc->CachedProperty->GetOffset_ForInternal();
}

void* FInstancedPropertyBag::GetMutableValueAddress(const FPropertyBagPropertyDesc* Desc)
{
	if (Desc == nullptr || !Value.IsValid())
	{
		return nullptr;
	}
	return Value.GetMutableMemory() + Desc->CachedProperty->GetOffset_ForInternal();
}

FInstancedStruct FInstancedPropertyBag::Detach()
{
	return FInstancedStruct(MoveTemp(Value));
}

TValueOrError<bool, EPropertyBagResult> FInstancedPropertyBag::GetValueBool(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	int64 ReturnValue = 0;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsInt64(Desc, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue != 0);
}

template<typename T>
TValueOrError<T, EPropertyBagResult> FInstancedPropertyBag::GetNumericalValue(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if constexpr (std::is_floating_point_v<T>)
	{
		double ReturnValue = 0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsDouble(Desc, GetValueAddress(Desc), ReturnValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(ReturnValue));
	}
	else if constexpr (std::is_unsigned_v<T>)
	{
		uint64 ReturnValue = 0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsUInt64(Desc, GetValueAddress(Desc), ReturnValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(ReturnValue));
	}
	else
	{
		int64 ReturnValue = 0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsInt64(Desc, GetValueAddress(Desc), ReturnValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(ReturnValue));
	}
}

TValueOrError<int8, EPropertyBagResult> FInstancedPropertyBag::GetValueInt8(const FName Name) const
{
	return GetNumericalValue<int8>(Name);
}

TValueOrError<uint8, EPropertyBagResult> FInstancedPropertyBag::GetValueByte(const FName Name) const
{
	return GetNumericalValue<uint8>(Name);
}

TValueOrError<int16, EPropertyBagResult> FInstancedPropertyBag::GetValueInt16(const FName Name) const
{
	return GetNumericalValue<int16>(Name);
}

TValueOrError<uint16, EPropertyBagResult> FInstancedPropertyBag::GetValueUInt16(const FName Name) const
{
	return GetNumericalValue<uint16>(Name);
}

TValueOrError<int32, EPropertyBagResult> FInstancedPropertyBag::GetValueInt32(const FName Name) const
{
	return GetNumericalValue<int32>(Name);
}

TValueOrError<uint32, EPropertyBagResult> FInstancedPropertyBag::GetValueUInt32(const FName Name) const
{
	return GetNumericalValue<uint32>(Name);
}

TValueOrError<int64, EPropertyBagResult> FInstancedPropertyBag::GetValueInt64(const FName Name) const
{
	return GetNumericalValue<int64>(Name);
}

TValueOrError<uint64, EPropertyBagResult> FInstancedPropertyBag::GetValueUInt64(const FName Name) const
{
	return GetNumericalValue<uint64>(Name);
}

TValueOrError<float, EPropertyBagResult> FInstancedPropertyBag::GetValueFloat(const FName Name) const
{
	return GetNumericalValue<float>(Name);
}

TValueOrError<double, EPropertyBagResult> FInstancedPropertyBag::GetValueDouble(const FName Name) const
{
	return GetNumericalValue<double>(Name);
}

TValueOrError<FName, EPropertyBagResult> FInstancedPropertyBag::GetValueName(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	FName ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FName, FNameProperty>(Desc, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<FString, EPropertyBagResult> FInstancedPropertyBag::GetValueString(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	FString ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FString, FStrProperty>(Desc, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<FText, EPropertyBagResult> FInstancedPropertyBag::GetValueText(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	FText ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FText, FTextProperty>(Desc, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<uint8, EPropertyBagResult> FInstancedPropertyBag::GetValueEnum(const FName Name, const UEnum* RequestedEnum) const
{
	TValueOrError<int64, EPropertyBagResult> Result = GetValueEnumInt64(Name, RequestedEnum);
	if (Result.HasValue())
	{
		return MakeValue(static_cast<uint8>(Result.GetValue()));
	}
	return MakeError(Result.GetError());
}

TValueOrError<int64, EPropertyBagResult> FInstancedPropertyBag::GetValueEnumInt64(const FName Name, const UEnum* RequestedEnum) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	int64 ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsEnum(Desc, RequestedEnum, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<FStructView, EPropertyBagResult> FInstancedPropertyBag::GetValueStruct(const FName Name, const UScriptStruct* RequestedStruct) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	FStructView ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsStruct(Desc, RequestedStruct, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<UObject*, EPropertyBagResult> FInstancedPropertyBag::GetValueObject(const FName Name, const UClass* RequestedClass) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	UObject* ReturnValue = nullptr;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsObject(Desc, RequestedClass, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<UClass*, EPropertyBagResult> FInstancedPropertyBag::GetValueClass(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	UObject* ReturnValue = nullptr;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsObject(Desc, nullptr, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	UClass* Class = Cast<UClass>(ReturnValue);
	if (Class == nullptr && ReturnValue != nullptr)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}
	return MakeValue(Class);
}

TValueOrError<FSoftObjectPath, EPropertyBagResult> FInstancedPropertyBag::GetValueSoftPath(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	FSoftObjectPath ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsSoftPath(Desc, GetValueAddress(Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}

	return MakeValue(std::move(ReturnValue));
}

TValueOrError<FString, EPropertyBagResult> FInstancedPropertyBag::GetValueSerializedString(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	
	const FProperty* Property = Desc->CachedProperty;
	check(Property);

	const void* ValueAddress = GetValueAddress(Desc);
	FString OutStringValue;
	if (!Property->ExportText_Direct(OutStringValue, ValueAddress, ValueAddress, nullptr, PPF_None))
	{
		UE_LOGF(LogCore, Warning, "PropertyBag: Getting the serialized value of the property '%ls' failed.", *Desc->Name.ToString());
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	return MakeValue(OutStringValue);
}

TValueOrError<bool, EPropertyBagResult> FInstancedPropertyBag::GetValueBool(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	int64 ReturnValue = 0;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsInt64(&Desc, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue != 0);
}

template<typename T>
TValueOrError<T, EPropertyBagResult> FInstancedPropertyBag::GetNumericalValue(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	if constexpr (std::is_floating_point<T>::value)
	{
		double ReturnValue = 0.0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsDouble(&Desc, GetValueAddress(&Desc), ReturnValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(ReturnValue));
	}
	else if constexpr (std::is_unsigned<T>::value)
	{
		uint64 ReturnValue = 0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsUInt64(&Desc, GetValueAddress(&Desc), ReturnValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(ReturnValue));
	}
	else
	{
		int64 ReturnValue = 0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsInt64(&Desc, GetValueAddress(&Desc), ReturnValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(ReturnValue));
	}
}

TValueOrError<int8, EPropertyBagResult> FInstancedPropertyBag::GetValueInt8(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<int8>(Desc);
}

TValueOrError<uint8, EPropertyBagResult> FInstancedPropertyBag::GetValueByte(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<uint8>(Desc);
}

TValueOrError<int16, EPropertyBagResult> FInstancedPropertyBag::GetValueInt16(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<int16>(Desc);
}

TValueOrError<uint16, EPropertyBagResult> FInstancedPropertyBag::GetValueUInt16(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<uint16>(Desc);
}

TValueOrError<int32, EPropertyBagResult> FInstancedPropertyBag::GetValueInt32(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<int32>(Desc);
}

TValueOrError<uint32, EPropertyBagResult> FInstancedPropertyBag::GetValueUInt32(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<uint32>(Desc);
}

TValueOrError<int64, EPropertyBagResult> FInstancedPropertyBag::GetValueInt64(const FPropertyBagPropertyDesc& Desc) const
{

	return GetNumericalValue<int64>(Desc);
}

TValueOrError<uint64, EPropertyBagResult> FInstancedPropertyBag::GetValueUInt64(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<uint64>(Desc);
}

TValueOrError<float, EPropertyBagResult> FInstancedPropertyBag::GetValueFloat(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<float>(Desc);
}

TValueOrError<double, EPropertyBagResult> FInstancedPropertyBag::GetValueDouble(const FPropertyBagPropertyDesc& Desc) const
{
	return GetNumericalValue<double>(Desc);
}

TValueOrError<FName, EPropertyBagResult> FInstancedPropertyBag::GetValueName(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	FName ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FName, FNameProperty>(&Desc, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<FString, EPropertyBagResult> FInstancedPropertyBag::GetValueString(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	FString ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FString, FStrProperty>(&Desc, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<FText, EPropertyBagResult> FInstancedPropertyBag::GetValueText(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	FText ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FText, FTextProperty>(&Desc, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<uint8, EPropertyBagResult> FInstancedPropertyBag::GetValueEnum(const FPropertyBagPropertyDesc& Desc, const UEnum* RequestedEnum) const
{
	TValueOrError<int64, EPropertyBagResult> Result = GetValueEnumInt64(Desc, RequestedEnum);
	if (Result.HasValue())
	{
		return MakeValue(static_cast<uint8>(Result.GetValue()));
	}
	return MakeError(Result.GetError());
}

TValueOrError<int64, EPropertyBagResult> FInstancedPropertyBag::GetValueEnumInt64(const FPropertyBagPropertyDesc& Desc, const UEnum* RequestedEnum) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	int64 ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsEnum(&Desc, RequestedEnum, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<FStructView, EPropertyBagResult> FInstancedPropertyBag::GetValueStruct(const FPropertyBagPropertyDesc& Desc, const UScriptStruct* RequestedStruct) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	FStructView ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsStruct(&Desc, RequestedStruct, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<UObject*, EPropertyBagResult> FInstancedPropertyBag::GetValueObject(const FPropertyBagPropertyDesc& Desc, const UClass* RequestedClass) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	UObject* ReturnValue = nullptr;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsObject(&Desc, RequestedClass, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(ReturnValue);
}

TValueOrError<UClass*, EPropertyBagResult> FInstancedPropertyBag::GetValueClass(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	UObject* ReturnValue = nullptr;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsObject(&Desc, nullptr, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	UClass* Class = Cast<UClass>(ReturnValue);
	if (Class == nullptr && ReturnValue != nullptr)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}
	return MakeValue(Class);
}

TValueOrError<FSoftObjectPath, EPropertyBagResult> FInstancedPropertyBag::GetValueSoftPath(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	FSoftObjectPath ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsSoftPath(&Desc, GetValueAddress(&Desc), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}

	return MakeValue(std::move(ReturnValue));
}

TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> FInstancedPropertyBag::GetMutableArrayRef(const FPropertyBagPropertyDesc& Desc)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc.CachedProperty);

	if (Desc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Array)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(&Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	return MakeValue(FPropertyBagArrayRef(Desc, Address));
}

TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> FInstancedPropertyBag::GetArrayRef(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc.CachedProperty);

	if (Desc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Array)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(&Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	return MakeValue(FPropertyBagArrayRef(Desc, Address));
}

TValueOrError<FPropertyBagSetRef, EPropertyBagResult> FInstancedPropertyBag::GetMutableSetRef(const FPropertyBagPropertyDesc& Desc)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc.CachedProperty);

	if (Desc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Set)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(&Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	return MakeValue(FPropertyBagSetRef(Desc, Address));
}

TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> FInstancedPropertyBag::GetSetRef(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc.CachedProperty);

	if (Desc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Set)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(&Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	return MakeValue(FPropertyBagSetRef(Desc, Address));
}

TValueOrError<FPropertyBagMapRef, EPropertyBagResult> FInstancedPropertyBag::GetMutableMapRef(const FPropertyBagPropertyDesc& Desc)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc.CachedProperty);

	if (Desc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Map)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(&Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	return MakeValue(FPropertyBagMapRef(Desc, Address));
}

TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> FInstancedPropertyBag::GetMapRef(const FPropertyBagPropertyDesc& Desc) const
{
	if (!OwnsPropertyDesc(Desc))
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc.CachedProperty);

	if (Desc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Map)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(&Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	return MakeValue(FPropertyBagMapRef(Desc, Address));
}

EPropertyBagResult FInstancedPropertyBag::SetValueBool(const FName Name, const bool bInValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromInt64(Desc, GetMutableValueAddress(Desc), bInValue ? 1 : 0);
}

EPropertyBagResult FInstancedPropertyBag::SetValueInt8(const FName Name, const int8 InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromInt64(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueByte(const FName Name, const uint8 InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromUInt64(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueInt16(const FName Name, const int16 InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromInt64(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueUInt16(const FName Name, const uint16 InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromUInt64(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueInt32(const FName Name, const int32 InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromInt64(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueUInt32(const FName Name, const uint32 InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromUInt64(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueInt64(const FName Name, const int64 InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromInt64(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueUInt64(const FName Name, const uint64 InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromUInt64(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueFloat(const FName Name, const float InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromDouble(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueDouble(const FName Name, const double InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyFromDouble(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueName(const FName Name, const FName InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValue<FName, FNameProperty>(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueString(const FName Name, const FString& InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValue<FString, FStrProperty>(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueText(const FName Name, const FText& InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValue<FText, FTextProperty>(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueEnum(const FName Name, const int64 InValue, const UEnum* Enum)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValueAsEnum(Desc, GetMutableValueAddress(Desc), InValue, Enum);
}

EPropertyBagResult FInstancedPropertyBag::SetValueStruct(const FName Name, FConstStructView InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValueAsStruct(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueObject(const FName Name, UObject* InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValueAsObject(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueClass(const FName Name, UClass* InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValueAsObject(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueSoftPath(const FName Name, const FSoftObjectPath& InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValueAsSoftPath(Desc, GetMutableValueAddress(Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueSoftPath(const FName Name, const UObject* InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	return UE::StructUtils::Private::SetPropertyValueAsSoftPath(Desc, GetMutableValueAddress(Desc), InValue, InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueSerializedString(const FName Name, const FString& InValue)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	
	const FProperty* Property = Desc->CachedProperty;
	check(Property);

	if (!Property->ImportText_Direct(*InValue, GetMutableValueAddress(Desc), nullptr, PPF_None))
	{
		UE_LOGF(LogCore, Warning, "PropertyBag: Setting the value of the property '%ls' failed because the string representation provided was not accepted.", *Desc->Name.ToString());
		return EPropertyBagResult::TypeMismatch;
	}

	return EPropertyBagResult::Success;
}

EPropertyBagResult FInstancedPropertyBag::SetValue(const FName Name, const FProperty* InSourceProperty, const void* InSourceContainerAddress)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr || InSourceProperty == nullptr || InSourceContainerAddress == nullptr)
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	check(Desc->CachedProperty);

	void* TargetAddress = Value.GetMutableMemory() + Desc->CachedProperty->GetOffset_ForInternal();
	void const* SourceAddress = InSourceProperty->ContainerPtrToValuePtr<void>(InSourceContainerAddress);

	if (InSourceProperty->GetClass() == Desc->CachedProperty->GetClass())
	{
		Desc->CachedProperty->CopyCompleteValue(TargetAddress, SourceAddress);
	}
	else
	{
		return EPropertyBagResult::TypeMismatch;
	}

	return EPropertyBagResult::Success;
}

EPropertyBagResult FInstancedPropertyBag::SetValueBool(const FPropertyBagPropertyDesc& Desc, const bool bInValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromInt64(&Desc, GetMutableValueAddress(&Desc), bInValue ? 1 : 0);
}

EPropertyBagResult FInstancedPropertyBag::SetValueInt8(const FPropertyBagPropertyDesc& Desc, const int8 InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromInt64(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueByte(const FPropertyBagPropertyDesc& Desc, const uint8 InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromUInt64(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueInt16(const FPropertyBagPropertyDesc& Desc, const int16 InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromInt64(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueUInt16(const FPropertyBagPropertyDesc& Desc, const uint16 InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromUInt64(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueInt32(const FPropertyBagPropertyDesc& Desc, const int32 InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromInt64(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueUInt32(const FPropertyBagPropertyDesc& Desc, const uint32 InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromUInt64(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueInt64(const FPropertyBagPropertyDesc& Desc, const int64 InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromInt64(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueUInt64(const FPropertyBagPropertyDesc& Desc, const uint64 InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromUInt64(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueFloat(const FPropertyBagPropertyDesc& Desc, const float InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromDouble(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueDouble(const FPropertyBagPropertyDesc& Desc, const double InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyFromDouble(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueName(const FPropertyBagPropertyDesc& Desc, const FName InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValue<FName, FNameProperty>(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueString(const FPropertyBagPropertyDesc& Desc, const FString& InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValue<FString, FStrProperty>(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueText(const FPropertyBagPropertyDesc& Desc, const FText& InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValue<FText, FTextProperty>(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueEnum(const FPropertyBagPropertyDesc& Desc, const int64 InValue, const UEnum* Enum)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValueAsEnum(&Desc, GetMutableValueAddress(&Desc), InValue, Enum);
}

EPropertyBagResult FInstancedPropertyBag::SetValueStruct(const FPropertyBagPropertyDesc& Desc, FConstStructView InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValueAsStruct(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueObject(const FPropertyBagPropertyDesc& Desc, UObject* InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValueAsObject(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueClass(const FPropertyBagPropertyDesc& Desc, UClass* InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValueAsObject(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueSoftPath(const FPropertyBagPropertyDesc& Desc, const FSoftObjectPath& InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValueAsSoftPath(&Desc, GetMutableValueAddress(&Desc), InValue);
}

EPropertyBagResult FInstancedPropertyBag::SetValueSoftPath(const FPropertyBagPropertyDesc& Desc, const UObject* InValue)
{
	if (!OwnsPropertyDesc(Desc))
	{
		return EPropertyBagResult::PropertyNotFound;
	}
	return UE::StructUtils::Private::SetPropertyValueAsSoftPath(&Desc, GetMutableValueAddress(&Desc), InValue, InValue);
}

TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> FInstancedPropertyBag::GetMutableArrayRef(const FName Name)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc->CachedProperty);

	if (Desc->ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Array)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	
	return MakeValue(FPropertyBagArrayRef(*Desc, Address));
}

TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> FInstancedPropertyBag::GetArrayRef(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc->CachedProperty);

	if (Desc->ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Array)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	
	return MakeValue(FPropertyBagArrayRef(*Desc, Address));
}

TValueOrError<FPropertyBagSetRef, EPropertyBagResult> FInstancedPropertyBag::GetMutableSetRef(const FName Name)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc->CachedProperty);

	if (Desc->ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Set)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	
	return MakeValue(FPropertyBagSetRef(*Desc, Address));
}


TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> FInstancedPropertyBag::GetSetRef(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc->CachedProperty);

	if (Desc->ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Set)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	
	return MakeValue(FPropertyBagSetRef(*Desc, Address));
}

TValueOrError<FPropertyBagMapRef, EPropertyBagResult> FInstancedPropertyBag::GetMutableMapRef(const FName Name)
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc->CachedProperty);

	if (Desc->ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Map)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	return MakeValue(FPropertyBagMapRef(*Desc, Address));
}

TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> FInstancedPropertyBag::GetMapRef(const FName Name) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByName(Name);
	if (Desc == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	check(Desc->CachedProperty);

	if (Desc->ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Map)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	const void* Address = GetValueAddress(Desc);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}

	return MakeValue(FPropertyBagMapRef(*Desc, Address));
}

bool FInstancedPropertyBag::Identical(const FInstancedPropertyBag* Other, const uint32 PortFlags) const
{
#if WITH_EDITOR
	// Struct can't be identical when re-instancing. The struct has to serialize to get the new values.
	UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance();
	if (Other && Reinstancer && Reinstancer->IsSerializingForReinstantiation())
	{
		if (const UPropertyBag* BagStruct = GetPropertyBagStruct())
		{
			auto IsReinstanting = [Reinstancer](const UScriptStruct* Struct)
				{
					return Struct && Reinstancer->IsReinstanting(Struct);
				};
			TArrayView<const FPropertyBagPropertyDesc> PropertyDescs = BagStruct->PropertyDescs;
			for (const FPropertyBagPropertyDesc& Desc : PropertyDescs)
			{
				if (Desc.ValueType == EPropertyBagPropertyType::Struct)
				{
					if (IsReinstanting(Cast<const UScriptStruct>(Desc.ValueTypeObject)))
					{
						return false;
					}
				}
				if (Desc.KeyType == EPropertyBagPropertyType::Struct)
				{
					if (IsReinstanting(Cast<const UScriptStruct>(Desc.KeyTypeObject)))
					{
						return false;
					}
				}
			}
		}
	}
#endif // WITH_EDITOR

	return Other && Value.Identical(&Other->Value, PortFlags);
}

bool FInstancedPropertyBag::Serialize(FArchive& Ar)
{
	// Obsolete, use custom version instead.
	enum class EVersion : uint8
	{
		InitialVersion = 0,
		SerializeStructSize,
		// -----<new versions can be added above this line>-----
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
	EVersion Version = EVersion::LatestVersion;

	Ar.UsingCustomVersion(FPropertyBagCustomVersion::GUID);

	if (Ar.CustomVer(FPropertyBagCustomVersion::GUID) < FPropertyBagCustomVersion::ContainerTypes)
	{
		Ar << Version;
	}
	
	UPropertyBag* BagStruct = Cast<UPropertyBag>(const_cast<UScriptStruct*>(Value.GetScriptStruct()));
	bool bHasData = (BagStruct != nullptr);
	
	Ar << bHasData;
	
	if (bHasData)
	{
		// The script struct class is not serialized, the properties are serialized and type is created based on that.
		if (Ar.IsLoading())
		{
			TArray<FPropertyBagPropertyDesc> PropertyDescs;
			Ar << PropertyDescs;

#if WITH_EDITOR
			const UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance();
#endif

			for (FPropertyBagPropertyDesc& PropDesc : PropertyDescs)
			{
				if (PropDesc.ValueTypeObject)
				{
#if WITH_EDITOR
					if (Reinstancer && PropDesc.ValueType == EPropertyBagPropertyType::Struct)
					{
						if (const UScriptStruct* ValueStruct = Cast<const UScriptStruct>(PropDesc.ValueTypeObject);
							ValueStruct && Reinstancer->IsSerializingForReinstantiation())
						{
							if (const UScriptStruct* CompiledStruct = Reinstancer->GetCompiledReinstantingStruct(ValueStruct))
							{
								PropDesc.ValueTypeObject = CompiledStruct;
							}
						}
					}
#endif
					Ar.Preload(const_cast<UObject*>(PropDesc.ValueTypeObject.Get()));
				}
				if (PropDesc.KeyTypeObject)
				{
#if WITH_EDITOR
					if (Reinstancer && PropDesc.KeyType == EPropertyBagPropertyType::Struct)
					{
						if (const UScriptStruct* ValueStruct = Cast<const UScriptStruct>(PropDesc.KeyTypeObject);
							ValueStruct && Reinstancer->IsSerializingForReinstantiation())
						{
							if (const UScriptStruct* CompiledStruct = Reinstancer->GetCompiledReinstantingStruct(ValueStruct))
							{
								PropDesc.KeyTypeObject = CompiledStruct;
							}
						}
					}
#endif
					Ar.Preload(const_cast<UObject*>(PropDesc.KeyTypeObject.Get()));
				}
			}

			BagStruct = const_cast<UPropertyBag*>(UPropertyBag::GetOrCreateFromDescs(PropertyDescs));
			Value.InitializeAs(BagStruct);

			// Size of the serialized memory
			int32 SerialSize = 0; 
			if (Version >= EVersion::SerializeStructSize)
			{
				Ar << SerialSize;
			}
			
			// BagStruct can be null if it contains structs, classes or enums that could not be found.
			if (BagStruct != nullptr && Value.GetMutableMemory() != nullptr)
			{
				BagStruct->SerializeItem(Ar, Value.GetMutableMemory(), /*Defaults*/nullptr);
			}
			else
			{
				UE_LOGF(LogCore, Warning, "Unable to create serialized UPropertyBag -> Advance %u bytes in the archive and reset to empty FInstancedPropertyBag", SerialSize);
				Ar.Seek(Ar.Tell() + SerialSize);
			}
		}
		else if (Ar.IsSaving() || Ar.IsObjectReferenceCollector() || Ar.IsCountingMemory())
		{
			check(BagStruct);
			
			TArray<FPropertyBagPropertyDesc> PropertyDescs = BagStruct->PropertyDescs;
			Ar << PropertyDescs;

			const int64 SizeOffset = Ar.Tell(); // Position to write the actual size after struct serialization
			int32 SerialSize = 0;
			// Size of the serialized memory (reserve location)
			Ar << SerialSize;

			const int64 InitialOffset = Ar.Tell(); // Position before struct serialization to compute its serial size

			check(Value.GetMutableMemory() != nullptr);
			BagStruct->SerializeItem(Ar, Value.GetMutableMemory(), /*Defaults*/nullptr);
		
			const int64 FinalOffset = Ar.Tell(); // Keep current offset to reset the archive pos after write the serial size

			// Size of the serialized memory
			Ar.Seek(SizeOffset);	// Go back in the archive to write the actual size
			SerialSize = static_cast<int32>(FinalOffset - InitialOffset);
			Ar << SerialSize;
			Ar.Seek(FinalOffset);	// Reset archive to its position

#if WITH_EDITOR
			// Destroy the memory. Making sure we are not using the memory with an invalid struct.
			if (UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance())
			{
				if (Ar.IsSaving() && Reinstancer->IsSerializingForReinstantiation())
				{
					for (const FPropertyBagPropertyDesc& Desc : PropertyDescs)
					{
						if (Desc.ValueType == EPropertyBagPropertyType::Struct)
						{
							if (const UScriptStruct* ValueStruct = Cast<const UScriptStruct>(Desc.ValueTypeObject);
								ValueStruct && Reinstancer->IsReinstanting(ValueStruct))
							{
								Value.InitializeAs(nullptr);
								break;
							}
						}
						if (Desc.KeyType == EPropertyBagPropertyType::Struct)
						{
							if (const UScriptStruct* KeyStruct = Cast<const UScriptStruct>(Desc.KeyTypeObject);
								KeyStruct&& Reinstancer->IsReinstanting(KeyStruct))
							{
								Value.InitializeAs(nullptr);
								break;
							}
						}
					}
				}
			}
#endif
		}
		else
		{
			// Inspector archives that are neither Saving nor Loading nor any of the modes handled
			// by the previous branch (e.g. FArchiveCrc32 used by UE::StructUtils::GetStructInstanceCrc32,
			// CityHash64-based archive used by GetStructHash64). Without this branch the property bag
			// payload would not contribute to the hash.
			//
			// The generated UPropertyBag is interned by name `[Prefix_]PropertyBag_<BagHash>`, where
			// BagHash is derived from the (sanitized) descs (see UPropertyBag::GetOrCreateFromDescs).
			// Within a process, the FName uniquely identifies the interned bag, so serializing it gives
			// passive archives the schema disambiguation they need (UseBinarySerialization is true here,
			// which routes BagStruct->SerializeItem to SerializeBin - properties walked by offset, no
			// names in the stream) without paying for the full descs array.
			check(BagStruct);

			FName BagStructName = BagStruct->GetFName();
			Ar << BagStructName;

			if (Value.GetMutableMemory() != nullptr)
			{
				BagStruct->SerializeItem(Ar, Value.GetMutableMemory(), /*Defaults*/nullptr);
			}
		}
	}
	else
	{
		if (Ar.IsLoading())
		{
			// If loading and there was no data saved in the archive, make sure the value is empty.
			Reset();
		}
	}
	
	return true;
}

void FInstancedPropertyBag::AddStructReferencedObjects(FReferenceCollector& Collector)
{
#if WITH_EDITOR
	// Reference collector is used to visit all instances of instanced structs the the like when a user defined struct is reinstanced.
	if (UE::StructUtils::FStructReinstancer* Reinstancer = UE::StructUtils::FStructReinstancer::GetInstance())
	{
		if (UPropertyBag* BagStruct = Cast<UPropertyBag>(const_cast<UScriptStruct*>(Value.GetScriptStruct())))
		{
			auto MarkAsRequiresReinstantiation = [Reinstancer](TObjectPtr<const UObject>& TypeObject)
				{
					const UScriptStruct* Struct = Cast<const UScriptStruct>(TypeObject);
					if (Struct && Reinstancer->IsReinstanting(Struct))
					{
						Reinstancer->MarkAsRequiresReinstantiation();
						// It is not necessary to change the struct to a duplicated one, but it helps with tracking mistakes and prevents memory stomps.
						if (const UScriptStruct* Duplicated = Reinstancer->GetDuplicatedReinstantingStruct(Struct))
						{
							TypeObject = Duplicated;
						}
					}
				};
			TArrayView<FPropertyBagPropertyDesc> PropertyDescs = BagStruct->PropertyDescs;
			for (FPropertyBagPropertyDesc& Desc : PropertyDescs)
			{
				if (Desc.ValueType == EPropertyBagPropertyType::Struct)
				{
					MarkAsRequiresReinstantiation(Desc.ValueTypeObject);
				}
				if (Desc.KeyType == EPropertyBagPropertyType::Struct)
				{
					MarkAsRequiresReinstantiation(Desc.KeyTypeObject);
				}
			}
		}
	}
#endif // WITH_EDITOR
}

void FInstancedPropertyBag::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	if (const UPropertyBag* BagStruct = GetPropertyBagStruct())
	{
		for (const FPropertyBagPropertyDesc& Desc : BagStruct->PropertyDescs)
		{
			if (Desc.ValueTypeObject)
			{
				OutDeps.Add(const_cast<UObject*>(Desc.ValueTypeObject.Get()));
			}
			if (Desc.KeyTypeObject)
			{
				OutDeps.Add(const_cast<UObject*>(Desc.KeyTypeObject.Get()));
			}
		}

		// Report indirect dependencies of the instanced property bag struct
		// The iterator will recursively loop through all structs in structs/containers too
		for (TPropertyValueIterator<FStructProperty> It(BagStruct, Value.GetMutableMemory()); It; ++It)
		{
			const UScriptStruct* StructType = It.Key()->Struct;
			if (UScriptStruct::ICppStructOps* CppStructOps = StructType->GetCppStructOps())
			{
				void* StructDataPtr = const_cast<void*>(It.Value());
				CppStructOps->GetPreloadDependencies(StructDataPtr, OutDeps);
			}
		}
	}
}

bool FInstancedPropertyBag::IsPropertyNameValid(const FString& Name)
{
	return !Name.IsEmpty() && FName::IsValidXName(Name, UE::StructUtils::Private::Constants::MinimalInvalidNameCharacters);
}

bool FInstancedPropertyBag::IsPropertyNameValid(const FName Name)
{
	return IsPropertyNameValid(Name.ToString());
}

FName FInstancedPropertyBag::SanitizePropertyName(const FString& Name, const TCHAR ReplacementChar)
{
	if (Name.IsEmpty())
	{
		return FName("Property");
	}

	FString SanitizedName = Name;
	constexpr TStringView InvalidChars = UE::StructUtils::Private::Constants::InvalidNameCharacters;

	// Validate the replacement character is not on the invalid list.
	checkf(!InvalidChars.Contains(FString(1, &ReplacementChar)), TEXT("Invalid replacement character."));

	for (const TCHAR Char : InvalidChars)
	{
		SanitizedName.ReplaceCharInline(Char, ReplacementChar);
	}

	return FName(SanitizedName);
}

FName FInstancedPropertyBag::SanitizePropertyName(const FName Name, const TCHAR ReplacementChar)
{
	return SanitizePropertyName(Name.ToString(), ReplacementChar);
}

//----------------------------------------------------------------//
//  FPropertyBagArrayRef
//----------------------------------------------------------------//

TValueOrError<bool, EPropertyBagResult> FPropertyBagArrayRef::GetValueBool(const int32 Index) const
{
	int64 IntValue = 0;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsInt64(&ValueDesc, GetAddress(Index), IntValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(IntValue != 0);
}

template<typename T>
TValueOrError<T, EPropertyBagResult> FPropertyBagArrayRef::GetNumericalValue(const int32 Index) const
{
	if constexpr (std::is_floating_point_v<T>)
	{
		double DblValue = 0.0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsDouble(&ValueDesc, GetAddress(Index), DblValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(DblValue));
	}
	else if constexpr (std::is_unsigned_v<T>)
	{
		uint64 IntValue = 0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsUInt64(&ValueDesc, GetAddress(Index), IntValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(IntValue));
	}
	else
	{
		int64 IntValue = 0;
		const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyAsInt64(&ValueDesc, GetAddress(Index), IntValue);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}
		return MakeValue(static_cast<T>(IntValue));
	}
}

TValueOrError<int8, EPropertyBagResult> FPropertyBagArrayRef::GetValueInt8(const int32 Index) const
{
	return GetNumericalValue<int8>(Index);
}

TValueOrError<uint8, EPropertyBagResult> FPropertyBagArrayRef::GetValueByte(const int32 Index) const
{
	return GetNumericalValue<uint8>(Index);
}

TValueOrError<int16, EPropertyBagResult> FPropertyBagArrayRef::GetValueInt16(const int32 Index) const
{
	return GetNumericalValue<int16>(Index);
}

TValueOrError<uint16, EPropertyBagResult> FPropertyBagArrayRef::GetValueUInt16(const int32 Index) const
{
	return GetNumericalValue<uint16>(Index);
}

TValueOrError<int32, EPropertyBagResult> FPropertyBagArrayRef::GetValueInt32(const int32 Index) const
{
	return GetNumericalValue<int32>(Index);
}

TValueOrError<uint32, EPropertyBagResult> FPropertyBagArrayRef::GetValueUInt32(const int32 Index) const
{
	return GetNumericalValue<uint32>(Index);
}

TValueOrError<int64, EPropertyBagResult> FPropertyBagArrayRef::GetValueInt64(const int32 Index) const
{
	return GetNumericalValue<int64>(Index);
}

TValueOrError<uint64, EPropertyBagResult> FPropertyBagArrayRef::GetValueUInt64(const int32 Index) const
{
	return GetNumericalValue<uint64>(Index);
}

TValueOrError<float, EPropertyBagResult> FPropertyBagArrayRef::GetValueFloat(const int32 Index) const
{
	return GetNumericalValue<float>(Index);
}

TValueOrError<double, EPropertyBagResult> FPropertyBagArrayRef::GetValueDouble(const int32 Index) const
{
	return GetNumericalValue<double>(Index);
}

TValueOrError<FName, EPropertyBagResult> FPropertyBagArrayRef::GetValueName(const int32 Index) const
{
	FName Value;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FName, FNameProperty>(&ValueDesc, GetAddress(Index), Value);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(Value);
}

TValueOrError<FString, EPropertyBagResult> FPropertyBagArrayRef::GetValueString(const int32 Index) const
{
	FString Value;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FString, FStrProperty>(&ValueDesc, GetAddress(Index), Value);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(Value);
}

TValueOrError<FText, EPropertyBagResult> FPropertyBagArrayRef::GetValueText(const int32 Index) const
{
	FText Value;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValue<FText, FTextProperty>(&ValueDesc, GetAddress(Index), Value);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(Value);
}

TValueOrError<uint8, EPropertyBagResult> FPropertyBagArrayRef::GetValueEnum(const int32 Index, const UEnum* RequestedEnum) const
{
	int64 Value;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsEnum(&ValueDesc, RequestedEnum, GetAddress(Index), Value);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(static_cast<uint8>(Value));
}

TValueOrError<int64, EPropertyBagResult> FPropertyBagArrayRef::GetValueEnumInt64(const int32 Index, const UEnum* RequestedEnum) const
{
	int64 Value;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsEnum(&ValueDesc, RequestedEnum, GetAddress(Index), Value);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(Value);
}

TValueOrError<FStructView, EPropertyBagResult> FPropertyBagArrayRef::GetValueStruct(const int32 Index, const UScriptStruct* RequestedStruct) const
{
	FStructView Value;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsStruct(&ValueDesc, RequestedStruct, GetAddress(Index), Value);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(Value);
}

TValueOrError<UObject*, EPropertyBagResult> FPropertyBagArrayRef::GetValueObject(const int32 Index, const UClass* RequestedClass) const
{
	UObject* Value = nullptr;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsObject(&ValueDesc, RequestedClass, GetAddress(Index), Value);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(Value);
}

TValueOrError<UClass*, EPropertyBagResult> FPropertyBagArrayRef::GetValueClass(const int32 Index) const
{
	UObject* Value = nullptr;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsObject(&ValueDesc, nullptr, GetAddress(Index), Value);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	UClass* Class = Cast<UClass>(Value);
	if (Class == nullptr && Value != nullptr)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}
	return MakeValue(Class);
}

TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> FPropertyBagArrayRef::GetMutableNestedArrayRef(const int32 Index) const
{
	if (ValueDesc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Array)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	check(ValueDesc.CachedProperty);

	// Get the array address
	const void* Address = GetAddress(Index);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	
	// And create a FPropertyBagArrayRef with the dummy desc and the element address
	return MakeValue(FPropertyBagArrayRef(ValueDesc, Address));
}

TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> FPropertyBagArrayRef::GetNestedArrayRef(const int32 Index) const
{
	if (ValueDesc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Array)
	{
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	check(ValueDesc.CachedProperty);

	// Get the array address
	const void* Address = GetAddress(Index);
	if (Address == nullptr)
	{
		return MakeError(EPropertyBagResult::PropertyNotFound);
	}
	
	// And create a FPropertyBagArrayRef with the dummy desc and the element address
	return MakeValue(FPropertyBagArrayRef(ValueDesc, Address));
}

TValueOrError<FSoftObjectPath, EPropertyBagResult> FPropertyBagArrayRef::GetValueSoftPath(const int32 Index) const
{
	FSoftObjectPath ReturnValue;
	const EPropertyBagResult Result = UE::StructUtils::Private::GetPropertyValueAsSoftPath(&ValueDesc, GetAddress(Index), ReturnValue);
	if (Result != EPropertyBagResult::Success)
	{
		return MakeError(Result);
	}
	return MakeValue(std::move(ReturnValue));
}

EPropertyBagResult FPropertyBagArrayRef::SetValueBool(const int32 Index, const bool bInValue)
{
	return UE::StructUtils::Private::SetPropertyFromInt64(&ValueDesc, GetMutableAddress(Index), bInValue ? 1 : 0);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueInt8(const int32 Index, const int8 InValue)
{
	return UE::StructUtils::Private::SetPropertyFromInt64(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueByte(const int32 Index, const uint8 InValue)
{
	return UE::StructUtils::Private::SetPropertyFromUInt64(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueInt16(const int32 Index, const int16 InValue)
{
	return UE::StructUtils::Private::SetPropertyFromInt64(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueUInt16(const int32 Index, const uint16 InValue)
{
	return UE::StructUtils::Private::SetPropertyFromUInt64(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueInt32(const int32 Index, const int32 InValue)
{
	return UE::StructUtils::Private::SetPropertyFromInt64(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueUInt32(const int32 Index, const uint32 InValue)
{
	return UE::StructUtils::Private::SetPropertyFromUInt64(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueInt64(const int32 Index, const int64 InValue)
{
	return UE::StructUtils::Private::SetPropertyFromInt64(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueUInt64(const int32 Index, const uint64 InValue)
{
	return UE::StructUtils::Private::SetPropertyFromUInt64(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueFloat(const int32 Index, const float InValue)
{
	return UE::StructUtils::Private::SetPropertyFromDouble(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueDouble(const int32 Index, const double InValue)
{
	return UE::StructUtils::Private::SetPropertyFromDouble(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueName(const int32 Index, const FName InValue)
{
	return UE::StructUtils::Private::SetPropertyValue<FName, FNameProperty>(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueString(const int32 Index, const FString& InValue)
{
	return UE::StructUtils::Private::SetPropertyValue<FString, FStrProperty>(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueText(const int32 Index, const FText& InValue)
{
	return UE::StructUtils::Private::SetPropertyValue<FText, FTextProperty>(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueEnum(const int32 Index, const int64 InValue, const UEnum* Enum)
{
	return UE::StructUtils::Private::SetPropertyValueAsEnum(&ValueDesc, GetMutableAddress(Index), InValue, Enum);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueStruct(const int32 Index, FConstStructView InValue)
{
	return UE::StructUtils::Private::SetPropertyValueAsStruct(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueObject(const int32 Index, UObject* InValue)
{
	return UE::StructUtils::Private::SetPropertyValueAsObject(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueClass(const int32 Index, UClass* InValue)
{
	return UE::StructUtils::Private::SetPropertyValueAsObject(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueSoftPath(const int32 Index, const FSoftObjectPath& InValue)
{
	return UE::StructUtils::Private::SetPropertyValueAsSoftPath(&ValueDesc, GetMutableAddress(Index), InValue);
}

EPropertyBagResult FPropertyBagArrayRef::SetValueSoftPath(const int32 Index, const UObject* InValue)
{
	return UE::StructUtils::Private::SetPropertyValueAsSoftPath(&ValueDesc, GetMutableAddress(Index), InValue, InValue);
}

//----------------------------------------------------------------//
//  FPropertyBagSetRef
//----------------------------------------------------------------//

EPropertyBagResult FPropertyBagSetRef::AddValueBool(const bool bInValue)
{
	return ValueDesc.IsNumericType() ? Add(bInValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueInt8(const int8 InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueByte(const uint8 InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueInt16(const int16 InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueUInt16(const uint16 InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueInt32(const int32 InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueUInt32(const uint32 InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueInt64(const int64 InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueUInt64(const uint64 InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueFloat(const float InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueDouble(const double InValue)
{
	return ValueDesc.IsNumericType() ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueName(const FName InValue)
{
	return ValueDesc.ValueType == EPropertyBagPropertyType::Name ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueString(const FString& InValue)
{
	return ValueDesc.ValueType == EPropertyBagPropertyType::String ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueText(const FText& InValue)
{
	return ValueDesc.ValueType == EPropertyBagPropertyType::Text ? Add(InValue) : EPropertyBagResult::TypeMismatch;
}

EPropertyBagResult FPropertyBagSetRef::AddValueEnum(const int64 InValue, const UEnum* Enum)
{
	if (ValueDesc.ValueType != EPropertyBagPropertyType::Enum)
	{
		return EPropertyBagResult::TypeMismatch;
	}

	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(ValueDesc.CachedProperty))
	{
		const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
		check(UnderlyingProperty);

		if (Enum != EnumProperty->GetEnum())
		{
			return EPropertyBagResult::TypeMismatch;
		}

		return Add(InValue);
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(ValueDesc.CachedProperty))
	{
		if (Enum != ByteProperty->Enum)
		{
			return EPropertyBagResult::TypeMismatch;
		}

		return Add(InValue);
	}
	else
	{
		return EPropertyBagResult::TypeMismatch;
	}
}

EPropertyBagResult FPropertyBagSetRef::AddValueStruct(FConstStructView InValue)
{
	const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(ValueDesc.CachedProperty);
	check(StructProperty->Struct);

	if (InValue.IsValid())
	{
		if (InValue.GetScriptStruct() != StructProperty->Struct)
		{
			return EPropertyBagResult::TypeMismatch;
		}
	}

	return Add(InValue.GetMemory());
}

EPropertyBagResult FPropertyBagSetRef::AddValueObject(UObject* InValue)
{
	if (ValueDesc.ValueType != EPropertyBagPropertyType::Object
		&& ValueDesc.ValueType != EPropertyBagPropertyType::SoftObject
		&& ValueDesc.ValueType != EPropertyBagPropertyType::Class
		&& ValueDesc.ValueType != EPropertyBagPropertyType::SoftClass)
	{
		return EPropertyBagResult::TypeMismatch;
	}

	const FObjectPropertyBase* ObjectProperty = CastFieldChecked<FObjectPropertyBase>(ValueDesc.CachedProperty);
	check(ObjectProperty->PropertyClass);
	check(ValueDesc.ValueTypeObject);

	if (ValueDesc.ValueType == EPropertyBagPropertyType::Object
		|| ValueDesc.ValueType == EPropertyBagPropertyType::SoftObject)
	{
		if (InValue && UE::StructUtils::Private::CanCastTo(InValue->GetClass(), ObjectProperty->PropertyClass) == false)
		{
			return EPropertyBagResult::TypeMismatch;
		}
	}
	else
	{
		const UClass* Class = Cast<UClass>(InValue);
		const UClass* PropClass = nullptr;

		if (const FClassProperty* ClassProperty = CastFieldChecked<FClassProperty>(ValueDesc.CachedProperty))
		{
			PropClass = ClassProperty->MetaClass;
		}
		else if (const FSoftClassProperty* SoftClassProperty = CastFieldChecked<FSoftClassProperty>(ValueDesc.CachedProperty))
		{
			PropClass = SoftClassProperty->MetaClass;
		}

		if (!Class || !PropClass || !Class->IsChildOf(PropClass))
		{
			return EPropertyBagResult::TypeMismatch;
		}
	}

	Add(InValue);

	return EPropertyBagResult::Success;
}

EPropertyBagResult FPropertyBagSetRef::AddValueClass(UClass* InValue)
{
	return AddValueObject(InValue);
}

EPropertyBagResult FPropertyBagSetRef::AddValueSoftPath(const FSoftObjectPath& InValue)
{
	if (ValueDesc.ValueType != EPropertyBagPropertyType::SoftObject &&
		ValueDesc.ValueType != EPropertyBagPropertyType::SoftClass)
	{
		return EPropertyBagResult::TypeMismatch;
	}

	return Add(InValue);
}

//----------------------------------------------------------------//
//  UPropertyBag
//----------------------------------------------------------------//

namespace UE::StructUtils::Private
{
	// Lock to prevent concurrent access to lazily-constructed UPropertyBag objects in UPropertyBag::GetOrCreateFromDescs
	static FTransactionallySafeCriticalSection GPropertyBagLock;
}

TArray<FPropertyBagPropertyDesc> UPropertyBag::SanitizePropertyDescs(const TArrayView<const FPropertyBagPropertyDesc> PropertyDescs)
{
	TArray<FPropertyBagPropertyDesc> Result = TArray<FPropertyBagPropertyDesc>(PropertyDescs.GetData(), PropertyDescs.Num());

	// Fix missing structs, enums, and objects.
	auto CheckType = [](const FName PropertyName, EPropertyBagPropertyType InType, TObjectPtr<const UObject>& InOutTypeObject)
	{
		if (InType == EPropertyBagPropertyType::Struct)
		{
			if (InOutTypeObject == nullptr || InOutTypeObject->GetClass()->IsChildOf(UScriptStruct::StaticClass()) == false)
			{
				UE_LOGF(LogCore, Warning, "PropertyBag: Struct property '%ls' is missing type.", *PropertyName.ToString());
				InOutTypeObject = FPropertyBagMissingStruct::StaticStruct();
			}
		}
		else if (InType == EPropertyBagPropertyType::Enum)
		{
			if (InOutTypeObject == nullptr || InOutTypeObject->GetClass()->IsChildOf(UEnum::StaticClass()) == false)
			{
				UE_LOGF(LogCore, Warning, "PropertyBag: Enum property '%ls' is missing type.", *PropertyName.ToString());
				InOutTypeObject = StaticEnum<EPropertyBagMissingEnum>();
			}
		}
		else if (InType == EPropertyBagPropertyType::Object || InType == EPropertyBagPropertyType::SoftObject)
		{
			if (InOutTypeObject == nullptr)
			{
				UE_LOGF(LogCore, Warning, "PropertyBag: Object property '%ls' is missing type.", *PropertyName.ToString());
				InOutTypeObject = UPropertyBagMissingObject::StaticClass();
			}
		}
		else if (InType == EPropertyBagPropertyType::Class || InType == EPropertyBagPropertyType::SoftClass)
		{
			if (InOutTypeObject == nullptr || InOutTypeObject->GetClass()->IsChildOf(UClass::StaticClass()) == false)
			{
				UE_LOGF(LogCore, Warning, "PropertyBag: Class property '%ls' is missing type.", *PropertyName.ToString());
				InOutTypeObject = UPropertyBagMissingObject::StaticClass();
			}
		}
		else 
		{
			InOutTypeObject = nullptr;
		}	
	};

	for (FPropertyBagPropertyDesc& Desc : Result)
	{
		CheckType(Desc.Name, Desc.ValueType, Desc.ValueTypeObject);

		if (Desc.KeyType != EPropertyBagPropertyType::None)
		{
			CheckType(Desc.Name, Desc.KeyType, Desc.KeyTypeObject);
		}
	}

	// Remove properties with same name
	for (int32 Index = 0; Index < Result.Num() - 1; Index++)
	{
		UE::StructUtils::Private::RemovePropertyByName(Result, Result[Index].Name, Index + 1);
	}

	// Remove unsupported types
	Result.RemoveAll([](const FPropertyBagPropertyDesc& Desc) { return Desc.ValueType == EPropertyBagPropertyType::None; });

	return Result;
}

const UPropertyBag* UPropertyBag::GetOrCreateFromDescs(const TConstArrayView<FPropertyBagPropertyDesc> InPropertyDescs, const TCHAR* PrefixName)
{
	TArray<FPropertyBagPropertyDesc> PropertyDescs = SanitizePropertyDescs(InPropertyDescs);
	const uint64 BagHash = UE::StructUtils::CalcPropertyDescArrayHash(PropertyDescs);
	const FString ScriptStructName = PrefixName == nullptr
		? FString::Printf(TEXT("PropertyBag_%llx"), BagHash)
		: FString::Printf(TEXT("%s_%llx"), PrefixName, BagHash);

	// We need to linearize this entire function otherwise threads that create bags of identical layouts can view
	// partially-constructed objects 
	UE::TScopeLock ScopeLock(UE::StructUtils::Private::GPropertyBagLock);

	// we need to use StaticFindObjectFastInternal with ExclusiveInternalFlags = EInternalObjectFlags::None
	// here because objects with RF_NeedPostLoad cannot be found by regular FindObject calls as they will have
	// ExclusiveInternalFlags = EInternalObjectFlags::AsyncLoading when called from game thread.
	if (UObject* ExistingObject = StaticFindObjectFastInternal(UPropertyBag::StaticClass(), GetTransientPackage(), *ScriptStructName, EFindObjectFlags::ExactClass, RF_NoFlags, EInternalObjectFlags::None))
	{
		if (const UPropertyBag* ExistingBag = Cast<UPropertyBag>(ExistingObject))
		{
			return ExistingBag;
		}
	}

	UPropertyBag* NewBag = NewObject<UPropertyBag>(GetTransientPackage(), *ScriptStructName, RF_Standalone | RF_Transient);

	NewBag->PropertyDescs = MoveTemp(PropertyDescs);

	// Add properties (AddCppProperty() adds them backwards in the linked list)
	for (int32 DescIndex = NewBag->PropertyDescs.Num() - 1; DescIndex >= 0; DescIndex--)
	{
		FPropertyBagPropertyDesc& Desc = NewBag->PropertyDescs[DescIndex];

		// Fix IDs
		if (!Desc.ID.IsValid())
		{
			Desc.ID = FGuid::NewGuid();
		}

		if (FProperty* NewProperty = UE::StructUtils::Private::CreatePropertyFromDesc(Desc, NewBag))
		{
#if WITH_EDITORONLY_DATA
			// Add metadata
			for (const FPropertyBagPropertyDescMetaData& PropertyDescMetaData : Desc.MetaData)
			{
				NewProperty->SetMetaData(*PropertyDescMetaData.Key.ToString(), *PropertyDescMetaData.Value);
			}
#endif
			NewProperty->SetPropertyFlags((EPropertyFlags)Desc.PropertyFlags);
			NewBag->AddCppProperty(NewProperty);
			Desc.CachedProperty = NewProperty;
			Desc.CachedIndex = DescIndex;
		}
	}

	// @hack:
	// This method is called to prevent non-editor builds to not crash on IsChildOf().
	// The issues is that the UScriptStruct(const FObjectInitializer& ObjectInitializer) ctor (which is macro/UHT generated)
	// does not call ReinitializeBaseChainArray(), when the code is compiled with USTRUCT_ISCHILDOF_STRUCTARRAY.
	// Calling SetSuperStruct() forces the ReinitializeBaseChainArray() to be called.
	NewBag->SetSuperStruct(nullptr);
	
	NewBag->Bind();
	NewBag->StaticLink(/*RelinkExistingProperties*/true);

	return NewBag;
}

void UPropertyBag::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	Super::Link(Ar, bRelinkExistingProperties);
	
	// Every time an instance of this property bag is initialized we need to increment the ref count by calling `InitializeStruct`.
	// And when it is destroyed, it decrements the ref count by calling `DestroyStruct`.
	// Therefore, the property bag can never be zero constructed and must have a destructor.
	// Same logic applies to "plain old data", we can never be memcpy.
	StructFlags = static_cast<EStructFlags>(StructFlags & ~(STRUCT_ZeroConstructor | STRUCT_NoDestructor | STRUCT_IsPlainOldData));
}

#if WITH_EDITOR
bool UPropertyBag::ContainsUserDefinedStruct(const UUserDefinedStruct* UserDefinedStruct) const
{
	if (!UserDefinedStruct)
	{
		return false;
	}
	
	for (const FPropertyBagPropertyDesc& Desc : PropertyDescs)
	{
		if (Desc.ValueType == EPropertyBagPropertyType::Struct)
		{
			if (const UUserDefinedStruct* OwnedUserDefinedStruct = Cast<UUserDefinedStruct>(Desc.ValueTypeObject))
			{
				if (OwnedUserDefinedStruct == UserDefinedStruct
					|| OwnedUserDefinedStruct->PrimaryStruct == UserDefinedStruct
					|| OwnedUserDefinedStruct == UserDefinedStruct->PrimaryStruct
					|| (OwnedUserDefinedStruct->PrimaryStruct == UserDefinedStruct->PrimaryStruct
						&& OwnedUserDefinedStruct->PrimaryStruct != nullptr)
					)
				{
					return true;
				}
			}
		}
		if (Desc.KeyType == EPropertyBagPropertyType::Struct)
		{
			if (const UUserDefinedStruct* OwnedUserDefinedStruct = Cast<UUserDefinedStruct>(Desc.KeyTypeObject))
			{
				if (OwnedUserDefinedStruct == UserDefinedStruct
					|| OwnedUserDefinedStruct->PrimaryStruct == UserDefinedStruct
					|| OwnedUserDefinedStruct == UserDefinedStruct->PrimaryStruct
					|| (OwnedUserDefinedStruct->PrimaryStruct == UserDefinedStruct->PrimaryStruct
						&& OwnedUserDefinedStruct->PrimaryStruct != nullptr)
					)
				{
					return true;
				}
			}
		}
	}
	return false;
}
#endif // WITH_EDITOR

void UPropertyBag::DecrementRefCount() const
{
	// Do ref counting based on struct usage.
	// This ensures that if the UPropertyBag is still valid in the C++ destructor of
	// the last instance of the bag.
	UE_AUTORTFM_ONCOMMIT(this)
	{
		UPropertyBag* NonConstThis = const_cast<UPropertyBag*>(this);
		const int32 OldCount = NonConstThis->RefCount.fetch_sub(1, std::memory_order_acq_rel);
		if (OldCount == 1)
		{
			NonConstThis->RemoveFromRoot();
		}
		if (OldCount <= 0)
		{
			UE_LOGF(LogCore, Error, "PropertyBag: DestroyStruct is called when RefCount is %d.", OldCount);
		}
	};
}

void UPropertyBag::IncrementRefCount() const
{
	// Do ref counting based on struct usage.
	// This ensures that if the UPropertyBag is still valid in the C++ destructor of
	// the last instance of the bag.
	UPropertyBag* NonConstThis = const_cast<UPropertyBag*>(this);

	const int32 OldCount = AutoRTFM::Open([&]
	{
		return NonConstThis->RefCount.fetch_add(1, std::memory_order_acq_rel);
	});

	UE_AUTORTFM_ONABORT(NonConstThis)
	{
		NonConstThis->RefCount.fetch_sub(1, std::memory_order_acq_rel);
	};

	if (OldCount == 0)
	{
		NonConstThis->AddToRoot();
	}
}

void UPropertyBag::InitializeStruct(void* Dest, int32 ArrayDim) const
{
	Super::InitializeStruct(Dest, ArrayDim);

	IncrementRefCount();
}

void UPropertyBag::DestroyStruct(void* Dest, int32 ArrayDim) const
{
	Super::DestroyStruct(Dest, ArrayDim);

	DecrementRefCount();
}

void UPropertyBag::FinishDestroy()
{
	const int32 Count = RefCount.load(); 
	if (Count > 0 && !IsEngineExitRequested())
	{
		UE_LOGF(LogCore, Error, "PropertyBag: Expecting RefCount to be zero on destructor, but it is %d.", Count);
	}
	
	Super::FinishDestroy();
}

const FPropertyBagPropertyDesc* UPropertyBag::FindPropertyDescByID(const FGuid ID) const
{
	return PropertyDescs.FindByPredicate([&ID](const FPropertyBagPropertyDesc& Desc) { return Desc.ID == ID; });
}

const FPropertyBagPropertyDesc* UPropertyBag::FindPropertyDescByName(const FName Name) const
{
	return PropertyDescs.FindByPredicate([&Name](const FPropertyBagPropertyDesc& Desc) { return Desc.Name == Name; });
}

const FPropertyBagPropertyDesc* UPropertyBag::FindPropertyDescByPropertyName(const FName PropertyName) const
{
	return PropertyDescs.FindByPredicate([&PropertyName](const FPropertyBagPropertyDesc& Desc) { return Desc.CachedProperty && Desc.CachedProperty->GetFName() == PropertyName; });
}

const FPropertyBagPropertyDesc* UPropertyBag::FindPropertyDescByProperty(const FProperty* Property) const
{
	return Property ? PropertyDescs.FindByPredicate([&Property](const FPropertyBagPropertyDesc& Desc) { return Desc.CachedProperty == Property; }) : nullptr;
}

const FPropertyBagPropertyDesc* UPropertyBag::FindPropertyDescByIndex(const int32 Index) const
{
	return PropertyDescs.IsValidIndex(Index) ? &PropertyDescs[Index] : nullptr;
}

bool UPropertyBag::OwnsPropertyDesc(const FPropertyBagPropertyDesc& Desc) const
{
	const FPropertyBagPropertyDesc* StartDesc = PropertyDescs.GetData();
	const FPropertyBagPropertyDesc* EndDesc = StartDesc + PropertyDescs.Num();
	return &Desc >= StartDesc && &Desc < EndDesc;
}

FName UPropertyBag::FindPropertyNameFromGuid(const FGuid& PropertyID) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByID(PropertyID);
	return Desc ? Desc->Name : FName();
}

FGuid UPropertyBag::FindPropertyGuidFromName(const FName PropertyName) const
{
	const FPropertyBagPropertyDesc* Desc = FindPropertyDescByPropertyName(PropertyName);
	return Desc ? Desc->ID : FGuid();
}
