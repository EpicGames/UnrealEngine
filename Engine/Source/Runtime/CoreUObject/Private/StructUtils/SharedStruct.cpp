// Copyright Epic Games, Inc. All Rights Reserved.
#include "StructUtils/SharedStruct.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SharedStruct)

///////////////////////////////////////////////////////////////// FConstSharedStruct /////////////////////////////////////////////////////////////////

bool FConstSharedStruct::Identical(const FConstSharedStruct* Other, uint32 PortFlags) const
{
	// Only empty or strictly identical is considered equal
	return Other != nullptr
		&& GetMemory() == Other->GetMemory()
		&& GetScriptStruct() == Other->GetScriptStruct();
}

void FConstSharedStruct::AddStructReferencedObjects(class FReferenceCollector& Collector)
{
	if (auto* Struct = GetScriptStructPtr(); Struct && *Struct)
	{
		Collector.AddReferencedObjects(*Struct, const_cast<uint8*>(GetMemory()));
	}
}

///////////////////////////////////////////////////////////////// FSharedStruct /////////////////////////////////////////////////////////////////

bool FSharedStruct::Identical(const FSharedStruct* Other, uint32 PortFlags) const
{
	// Only empty or strictly identical is considered equal
	return Other != nullptr 
		&& GetMemory() == Other->GetMemory() 
		&& GetScriptStruct() == Other->GetScriptStruct();
}

void FSharedStruct::AddStructReferencedObjects(class FReferenceCollector& Collector) const
{
	if (auto* Struct = GetScriptStructPtr(); Struct && *Struct)
	{
		Collector.AddReferencedObjects(*Struct, const_cast<uint8*>(GetMemory()));
	}
}
