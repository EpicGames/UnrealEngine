// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/StructReinstancer.h"

#if WITH_EDITOR
#include "Logging/StructuredLog.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/UObjectIterator.h"

namespace UE::StructUtils::Private
{
	static thread_local FStructReinstancer* CurrentReinstancer = nullptr;
}

namespace UE::StructUtils
{

FStructReinstancer* FStructReinstancer::GetInstance()
{
	return Private::CurrentReinstancer;
}

void FStructReinstancer::AddStruct(TNotNull<const UScriptStruct*> Original, TNotNull<const UScriptStruct*> Duplicated)
{
	StructuresToReinstantiate.Emplace(Original, Duplicated, nullptr);
}

void FStructReinstancer::SetCompiledStruct(TNotNull<const UScriptStruct*> Original, TNotNull<const UScriptStruct*> NewStruct)
{
	FStruct* Found = StructuresToReinstantiate.FindByPredicate(
		[Original](const FStruct& Other)
		{
			return Other.Original == Original;
		});
	if (ensure(Found))
	{
		Found->New = NewStruct;
	}
}

void FStructReinstancer::CollectObjects()
{
	if (StructuresToReinstantiate.Num() == 0)
	{
		return;
	}

	// Call AddStructReferencedObjects on all object to find the InstancedStruct/PropertyBag/... that uses the struct.

	// Helper preference collector, does not collect anything, but makes sure AddStructReferencedObjects() gets called e.g. on instanced struct. 
	class FVisitorReferenceCollector : public FReferenceCollector
	{
	public:
		virtual bool IsIgnoringArchetypeRef() const override
		{
			return false;
		}
		virtual bool IsIgnoringTransient() const override
		{
			return false;
		}
		virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
		{
			// Empty
		}
	};
	struct FDataForReinstantiation
	{
		TNotNull<UObject*> Object;
		UObject* ClassGeneratedBy = nullptr;
		TNotNull<UStruct*> Struct;
		TNotNull<void*> Data;
		const UStruct* SuperStruct = nullptr;
		const void* SuperData = nullptr;
	};

	FVisitorReferenceCollector Collector;

	// bRequiresReinstantiation will be set (in AddStructReferencedObjects) when the struct is reached.
	check(bSerializingForReinstantiation == false);
	bRequiresReinstantiation = false;

	TArray<FDataForReinstantiation> ReinstantiationDatas;

	// Handle regular UObjects
	for (TObjectIterator<UObject> ObjectIt; ObjectIt; ++ObjectIt)
	{
		UObject* Object = *ObjectIt;

		Object->CallAddReferencedObjects(Collector);
		Collector.AddPropertyReferencesWithStructARO(Object->GetClass(), Object);

		if (bRequiresReinstantiation)
		{
			ReinstantiationDatas.Emplace(Object, Object->GetClass()->ClassGeneratedBy.Get(), Object->GetClass(), (void*)Object, Object->GetClass(), (const void*)Object->GetClass()->GetDefaultObject());
			bRequiresReinstantiation = false;
		}
	}

	// Handle CDOs and sparse class data 
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;

		// Handle Sparse class data
		if (void* SparseData = const_cast<void*>(Class->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull)))
		{
			UScriptStruct* SparseDataStruct = Class->GetSparseClassDataStruct();
			Collector.AddPropertyReferencesWithStructARO(SparseDataStruct, SparseData);

			if (bRequiresReinstantiation)
			{
				UClass* SuperClass = Class->GetSuperClass();
				const UScriptStruct* SuperSCDType = SuperClass ? SuperClass->GetSparseClassDataStruct() : nullptr;
				const void* SuperSCD = SuperClass ? SuperClass->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull) : nullptr;
				ReinstantiationDatas.Emplace(Class, Class->ClassGeneratedBy.Get(), SparseDataStruct, SparseData, SuperSCDType, (const void*)SuperSCD);
				bRequiresReinstantiation = false;
			}
		}

		// Handle CDO
		if (UObject* CDO = Class->GetDefaultObject())
		{
			Collector.AddPropertyReferencesWithStructARO(Class, CDO);

			if (bRequiresReinstantiation)
			{
				const UClass* SuperClass = Class->GetSuperClass();
				const void* SuperData = SuperClass ? SuperClass->GetDefaultObject() : nullptr;
				ReinstantiationDatas.Emplace(CDO, Class->ClassGeneratedBy.Get(), Class, (void*)CDO, SuperClass, (const void*)SuperData);
				bRequiresReinstantiation = false;
			}
		}
	}

	ensure(bRequiresReinstantiation == false);

	ObjectsToReinstantiate.Reset();
	ObjectsToReinstantiate.Reserve(ReinstantiationDatas.Num());
	for (FDataForReinstantiation& ReinstantiationData : ReinstantiationDatas)
	{
		bSerializingForReinstantiation = true;

		UE_LOGFMT(LogClass, Verbose, "Serialize object {Name}", ReinstantiationData.Object->GetFullName());

		TArray<uint8> Buffer;
		FObjectWriter Writer(Buffer);
		ReinstantiationData.Struct->SerializeTaggedProperties(
			Writer,
			(uint8*)((void*)ReinstantiationData.Data),
			ReinstantiationData.SuperStruct,
			(uint8*)ReinstantiationData.SuperData,
			ReinstantiationData.Object
		);

		bSerializingForReinstantiation = false;

		ObjectsToReinstantiate.Emplace(
			FObjectKey(ReinstantiationData.Object),
			FObjectKey(ReinstantiationData.ClassGeneratedBy),
			TObjectKey<UStruct>(ReinstantiationData.Struct),
			ReinstantiationData.Data,
			TObjectKey<const UStruct>(ReinstantiationData.SuperStruct),
			ReinstantiationData.SuperData,
			MoveTemp(Buffer)
		);
	}
}

void FStructReinstancer::ReinstanceObjects()
{
	// We want archetype to be re-instantiated before a child in case the child uses the archetype in the serializer.
	struct FSortedSavedData
	{
		UObject* Object;
		UObject* Archetype;
		int32 SavedDataIndex;
	};

	TArray<FSortedSavedData> SortedDatas;
	SortedDatas.Reserve(ObjectsToReinstantiate.Num());
	for (int32 Index = 0; Index < ObjectsToReinstantiate.Num(); ++Index)
	{
		if (UObject* SavedObject = ObjectsToReinstantiate[Index].Object.ResolveObjectPtr())
		{
			SortedDatas.Emplace(SavedObject, SavedObject->GetArchetype(), Index);
		}
	}

	SortedDatas.Sort([](const FSortedSavedData& A, const FSortedSavedData& B)
		{
			return A.Archetype == nullptr || A.Object == B.Archetype;
		});
	for (FSortedSavedData& SortedData : SortedDatas)
	{
		const FSavedDataForReinstantiation& SavedData = ObjectsToReinstantiate[SortedData.SavedDataIndex];
		UObject* SavedObject = SortedData.Object;
		UStruct* SavedStruct = SavedData.Struct.ResolveObjectPtr();
		if (ensure(SavedStruct))
		{
			if (const UClass* Class = Cast<const UClass>(SavedStruct); Class && Class->HasAnyClassFlags(CLASS_NewerVersionExists))
			{
				continue;
			}
			if (const UScriptStruct* Struct = Cast<const UScriptStruct>(SavedStruct); Struct && (Struct->StructFlags & STRUCT_NewerVersionExists) != 0)
			{
				continue;
			}

			const UStruct* SavedSuperStruct = SavedData.SuperStruct.ResolveObjectPtr();
			const void* SavedSuperObjectData = SavedSuperStruct ? SavedData.SuperObjectData : nullptr;

			bSerializingForReinstantiation = true;

			UE_LOGFMT(LogClass, Verbose, "Reinstance object {Name}", SavedObject->GetFullName());

			FObjectReader Reader(SavedData.Buffer);
			SavedStruct->SerializeTaggedProperties(
				Reader,
				(uint8*)(void*)SavedData.ObjectData,
				SavedSuperStruct,
				(const uint8*)SavedSuperObjectData);

			bSerializingForReinstantiation = false;
		}
	}
}

FStructReinstancerScope::FStructReinstancerScope()
{
	OldStructReinstancer = Private::CurrentReinstancer;
	Private::CurrentReinstancer = new FStructReinstancer();
}

FStructReinstancerScope::~FStructReinstancerScope()
{
	delete Private::CurrentReinstancer;
	Private::CurrentReinstancer = OldStructReinstancer;
}

}
#endif // WITH_EDITOR
