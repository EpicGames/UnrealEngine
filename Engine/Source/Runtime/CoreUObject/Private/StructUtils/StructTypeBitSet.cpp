// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/StructTypeBitSet.h"
#include "Serialization/Archive.h"


DEFINE_LOG_CATEGORY_STATIC(LogStructUtils, Warning, All);

namespace FBitSetSerializationDictionary
{
	TMap<uint32 /*CurrentStoredTypesHash*/, const FStructTracker* /*Tracker*/> TrackerMap;
	TMap<uint32 /*NotUpToDateStoredTypesHash*/, TArray<int32> /*Mapping*/> BitMappings;

	void RegisterHash(const uint32 SerializationHash, const FStructTracker& Tracker)
	{
		if (const FStructTracker** StoredTracer = TrackerMap.Find(SerializationHash))
		{
			CA_SUPPRESS(6269); // warning C6269: Possibly incorrect order of operations.
			// disabling the warning since it's wrong. 
			// adding:
			//		const FStructTracker* StoredTracerPtr = *StoredTracer;
			// and then using StoredTracerPtr clears out the warning.
			ensureMsgf(((*StoredTracer) == nullptr) || ((*StoredTracer) == &Tracker)
				, TEXT("Hash conflict when registering a FStructTracker instance"));
		}
		TrackerMap.Add(SerializationHash, &Tracker);
	}

	const FStructTracker** GetTracker(const uint32 SerializationHash)
	{
		return TrackerMap.Find(SerializationHash);
	}

	TArray<int32>& GetOrCreateBitMapping(const uint32 SerializationHash)
	{
		return BitMappings.FindOrAdd(SerializationHash);
	}

	void UnregisterTracker(const FStructTracker& Tracker)
	{
		for (auto It = TrackerMap.CreateIterator(); It; ++It)
		{
			if (It->Value == &Tracker)
			{
				// note that we're not bailing out once the entry is removes, since
				// Trackers are being registered for all the hashes calculated as 
				// the new types to track get added
				It.RemoveCurrent();
			}
		}
	}
}


//-----------------------------------------------------------------------------
// FStructTracker
//-----------------------------------------------------------------------------
FStructTracker::FStructTracker(const FBaseStructGetter& InBaseStructGetter)
	: BaseStructGetter(InBaseStructGetter)
	, TypeVerification([this](const UStruct* InStruct)
		{
			return InStruct != nullptr && InStruct->IsChildOf(GetBaseType());
		})
{
}

FStructTracker::FStructTracker(const UStruct* InBaseType, const FTypeValidation& InTypeVerification)
	: TypeVerification(InTypeVerification 
		? InTypeVerification
		: [this](const UStruct* Struct)
			{
				return Struct && this->GetBaseType() && Struct->IsChildOf(this->BaseType);
			})
	, BaseType(InBaseType)
{
}

FStructTracker::~FStructTracker()
{
	FBitSetSerializationDictionary::UnregisterTracker(*this);
}

FORCEINLINE const UStruct* FStructTracker::GetBaseType() const
{
	return BaseType ? BaseType : (BaseStructGetter ? (BaseType = BaseStructGetter()) : nullptr);
}

int32 FStructTracker::FindOrAddStructTypeIndex(TNotNull<const UStruct*> InStructType)
{
	// Get existing index...
	const uint32 Hash = PointerHash(InStructType);
	const FSetElementId ElementId = StructTypeToIndexSet.FindIdByHash(Hash, Hash);

	if (LIKELY(ElementId.IsValidId()))
	{
		return ElementId.AsInteger();
	}
	return RegisterImplementation(InStructType, /*bCheckExisting=*/false);
}

int32 FStructTracker::RegisterImplementation(TNotNull<const UStruct*> InStructType, const bool bCheckExisting)
{
	if (!ensureMsgf(TypeVerification(InStructType)
		, TEXT("Unable to register %s singe it fails the validation test.")
		, *InStructType->GetName()))
	{
		return INDEX_NONE;
	}

	const uint32 Hash = PointerHash(InStructType);
	if (bCheckExisting)
	{
		const FSetElementId ElementId = StructTypeToIndexSet.FindIdByHash(Hash, Hash);
		if (ElementId.IsValidId())
		{
			return ElementId.AsInteger();
		}
	}

	// .. or create a new one
	const FSetElementId ElementId = StructTypeToIndexSet.AddByHash(Hash, Hash);
	check(ElementId.IsValidId());

	const int32 NewIndex = ElementId.AsInteger();
	check(StructTypesList.Num() == NewIndex);
	StructTypesList.Add(InStructType);

	OnTypeRegistered.Broadcast(InStructType, NewIndex);

	if (bIsSerializable)
	{
		// first-time SerializationHash initialization
		if (SerializationHash == 0)
		{
			ensure(StructTypesList.Num() == 1);
			const UStruct* LocalBaseType = GetBaseType();
			CA_ASSUME(LocalBaseType);
			SerializationHash = GetTypeHash(LocalBaseType->GetFullName());
		}
		SerializationHash = HashCombine(SerializationHash, GetTypeHash(InStructType->GetFullName()));

		// it's worth pointing out that we're registering a given tracker for all the hashes created along the way.
		// This will help with loading bitsets from serialized data.
		FBitSetSerializationDictionary::RegisterHash(SerializationHash, *this);
	}

#if WITH_STRUCTUTILS_DEBUG
	DebugStructTypeNamesList.Add(InStructType->GetFName());
	ensure(StructTypeToIndexSet.Num() == DebugStructTypeNamesList.Num());
#endif // WITH_STRUCTUTILS_DEBUG

	return NewIndex;
}


int32 FStructTracker::FindStructTypeIndex(TNotNull<const UStruct*> InStructType) const
{
	const uint32 Hash = PointerHash(InStructType);
	const FSetElementId ElementId = StructTypeToIndexSet.FindIdByHash(Hash, Hash);

	return ElementId.IsValidId() ? ElementId.AsInteger() : INDEX_NONE;
}

void FStructTracker::Serialize(FArchive& Ar, FStructTypeBitSet::FBitSetContainer& StructTypesBitArray)
{	
	enum class EVersion : uint8
	{
		Initial,
		Last,
		Current = Last - 1
	};

	if (!bIsSerializable && Ar.IsSaving())
	{
		// nothing to do if trying to save. If loading it means it's some old data and we at least need to consume it.
		return;
	}

	uint8 Version = static_cast<uint8>(EVersion::Current);
	Ar << Version;

	if (Ar.IsSaving())
	{
		Ar << SerializationHash;
		
		const int64 SizeOffset = Ar.Tell(); 
		int32 SerialSize = 0;
		Ar << SerialSize;

		// Serialized memory
		const int64 InitialDataOffset = Ar.Tell();

		// store information on the base type, so that we can verify we're trying to load the right data later on
		FTopLevelAssetPath BaseStructPath(GetBaseType());
		Ar << BaseStructPath;

		int32 StructTypesListNum = StructTypesList.Num();
		Ar << StructTypesListNum;
	
		for (const TWeakObjectPtr<const UStruct>& StructType : StructTypesList)
		{
			FTopLevelAssetPath StructPath(StructType.Get());
			Ar << StructPath;
		}

		const int64 FinalDataOffset = Ar.Tell();

		// Size of the serialized memory
		Ar.Seek(SizeOffset);	
		SerialSize = static_cast<int32>(FinalDataOffset - InitialDataOffset);
		Ar << SerialSize;
		// switch back to the end position
		Ar.Seek(FinalDataOffset);

		// serializing the actual bits
		Ar << StructTypesBitArray;
	}
	else if (Ar.IsLoading())
	{
		uint32 LoadedSerializationHash = 0;
		Ar << LoadedSerializationHash;
		
		int32 SerialSize = 0;
		Ar << SerialSize;

		const FStructTracker** KnownTracker = FBitSetSerializationDictionary::GetTracker(LoadedSerializationHash);

		if (KnownTracker && *KnownTracker == this)
		{
			// can skip the whole Tracker saved data
			Ar.Seek(Ar.Tell() + SerialSize);
			Ar << StructTypesBitArray;
		}
		else
		{
			FTopLevelAssetPath BaseStructPath;
			Ar << BaseStructPath;
			// first, verify that the base type of the tracker matches - otherwise we can be accidentally storing 
			// the wrong types creating the Mapping (via FindOrAddStructTypeIndex below)
			UStruct* SerializedBaseStructType = FindObject<UStruct>(BaseStructPath);

			if (SerializedBaseStructType == GetBaseType())
			{
				// create a translator 
				TArray<int32>& BitMapping = FBitSetSerializationDictionary::GetOrCreateBitMapping(LoadedSerializationHash);
				if (BitMapping.Num() == 0)
				{
					int32 StructTypesListNum = 0;
					Ar << StructTypesListNum;

					BitMapping.AddUninitialized(StructTypesListNum);

					for (int32 Index = 0; Index < StructTypesListNum; ++Index)
					{
						FTopLevelAssetPath TypePath;
						Ar << TypePath;

						if (UStruct* StructType = FindObject<UStruct>(TypePath))
						{
							checkSlow(StructType->IsChildOf(GetBaseType()));
							BitMapping[Index] = FindOrAddStructTypeIndex(StructType);
						}
						else
						{
							BitMapping[Index] = INDEX_NONE;
						}
					}
				}
				else
				{
					Ar.Seek(Ar.Tell() + SerialSize);
				}

				// it's where we read in data saved with a different order
				TBitArray<> TempStructTypesBitArray;
				Ar << TempStructTypesBitArray;
			
				StructTypesBitArray.Init(false, StructTypeToIndexSet.Num());
				for (TBitArray<>::FConstIterator It(TempStructTypesBitArray); It; ++It)
				{
					if (It.GetValue())
					{
						const int32 TranslatedIndex = BitMapping[It.GetIndex()];
						StructTypesBitArray.AddAtIndex(TranslatedIndex);
					}
				}
			}
			else
			{
				UE_LOGF(LogStructUtils, Error, "Trying to load mismatching BitSet data. Current base class: %ls, read base class: %ls"
					, *GetNameSafe(GetBaseType()), *BaseStructPath.ToString());
			}
		}
	}
}

#if WITH_STRUCTUTILS_DEBUG
const UStruct* FStructTracker::DebugFindTypeByPartialName(const FString& PartialName) const
{
	for (int32 TypeIndex = 0; TypeIndex < DebugStructTypeNamesList.Num(); ++TypeIndex)
	{
		if (DebugStructTypeNamesList[TypeIndex].ToString().Contains(PartialName))
		{
			return StructTypesList[TypeIndex].Get();
		}
	}
	return nullptr;
}
#endif // WITH_STRUCTUTILS_DEBUG
