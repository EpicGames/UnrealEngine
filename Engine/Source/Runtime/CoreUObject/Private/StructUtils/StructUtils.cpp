// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/StructUtils.h"

#include "Hash/xxhash.h"
#include "StructUtils/PropertyBag.h"
#include "StructUtils/UserDefinedStruct.h"

namespace UE::StructUtils::Private
{
	void AppendPathName(FStringBuilderBase& StringBuilder, const UObject* Object)
	{
		if (Object)
		{
			Object->GetPathName(nullptr, StringBuilder);
		}
	}

	void AppendValueObjectHash(FXxHash64Builder& HashBuilder, FStringBuilderBase& StringBuilder, const UObject* InValueObject)
	{
		AppendPathName(StringBuilder, InValueObject);
	#if WITH_EDITOR
		// for user defined structures we need to hash each property. after changing a user defined structure
		// we may find the same property bag again if the hash is not sufficient, even though the memory layout
		// has changed. hashing the path name of the user defined structure is not enough.
		if(const UUserDefinedStruct* UserDefinedStruct = Cast<UUserDefinedStruct>(InValueObject))
		{
			for(const FField* Property = UserDefinedStruct->ChildProperties; Property != nullptr; Property = Property->Next)
			{
				const uint32 FieldHash = GetTypeHash(Property);
				HashBuilder.Update(&FieldHash, sizeof(FieldHash));
				if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					if(StructProperty->Struct)
					{
						AppendValueObjectHash(HashBuilder, StringBuilder, StructProperty->Struct);
					}
				}
			}
		}
		// for property bags we recurse
		else if(const UPropertyBag* PropertyBag = Cast<UPropertyBag>(InValueObject))
		{
			for (const FPropertyBagPropertyDesc& Desc : PropertyBag->GetPropertyDescs())
			{
				UE::StructUtils::AppendPropertyDescs(HashBuilder, Desc);
			}
		}
	#endif
	}
}

void UE::StructUtils::AppendPropertyDescs(FXxHash64Builder& HashBuilder, const FPropertyBagPropertyDesc& Desc)
{
	TStringBuilder<256> StringBuilder;
	HashBuilder.Update(&Desc.ID, sizeof(Desc.ID));
	StringBuilder << Desc.Name;
	HashBuilder.Update(&Desc.ValueType, sizeof(Desc.ValueType));
	HashBuilder.Update(&Desc.ContainerTypes, sizeof(Desc.ContainerTypes));
	HashBuilder.Update(&Desc.PropertyFlags, sizeof(Desc.PropertyFlags));
	HashBuilder.Update(&Desc.KeyType, sizeof(Desc.KeyType));
#if WITH_EDITORONLY_DATA
	for (const FPropertyBagPropertyDescMetaData& MetaData : Desc.MetaData)
	{
		StringBuilder << MetaData.Key;
		HashBuilder.Update(*MetaData.Value, MetaData.Value.Len() * sizeof(FString::ElementType));
	}
#endif
	Private::AppendValueObjectHash(HashBuilder, StringBuilder, Desc.ValueTypeObject);
	Private::AppendValueObjectHash(HashBuilder, StringBuilder, Desc.KeyTypeObject);
	HashBuilder.Update(StringBuilder.ToString(), StringBuilder.Len() * sizeof(TStringBuilder<256>::ElementType));
}

uint64 UE::StructUtils::CalcPropertyDescArrayHash(const TConstArrayView<FPropertyBagPropertyDesc> Descs)
{
	FXxHash64Builder HashBuilder;
	for (const FPropertyBagPropertyDesc& Desc : Descs)
	{
		AppendPropertyDescs(HashBuilder, Desc);
	}
	return HashBuilder.Finalize().Hash;
}

TValueOrError<int64, void> UE::StructUtils::GetPropertyValueAsInt64(TNotNull<const FProperty*> InProperty, TNotNull<const void*> Address)
{
	if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(InProperty))
	{
		if (NumericProperty->IsInteger())
		{
			return MakeValue(NumericProperty->GetSignedIntPropertyValue(Address));
		}
		else if (NumericProperty->IsFloatingPoint())
		{
			return MakeValue((int64)NumericProperty->GetFloatingPointPropertyValue(Address));
		}

	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		const int64 Value = BoolProperty->GetPropertyValue(Address) ? 1 : 0;
		return MakeValue(Value);
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
		if (UnderlyingProperty->IsInteger())
		{
			return MakeValue(UnderlyingProperty->GetSignedIntPropertyValue(Address));
		}
	}
	return MakeError();
}

TValueOrError<uint64, void> UE::StructUtils::GetPropertyValueAsUInt64(TNotNull<const FProperty*> InProperty, TNotNull<const void*> Address)
{
	if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(InProperty))
	{
		if (NumericProperty->IsInteger())
		{
			return MakeValue(NumericProperty->GetUnsignedIntPropertyValue(Address));
		}
		else if (NumericProperty->IsFloatingPoint())
		{
			return MakeValue((uint64)NumericProperty->GetFloatingPointPropertyValue(Address));
		}
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		const uint64 Value = BoolProperty->GetPropertyValue(Address) ? 1 : 0;
		return MakeValue(Value);
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
		if (UnderlyingProperty->IsInteger())
		{
			return MakeValue(UnderlyingProperty->GetUnsignedIntPropertyValue(Address));
		}
	}
	return MakeError();
}

TValueOrError<double, void> UE::StructUtils::GetPropertyValueAsDouble(TNotNull<const FProperty*> InProperty, TNotNull<const void*> Address)
{
	if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(InProperty))
	{
		if (NumericProperty->IsInteger())
		{
			return MakeValue((double)NumericProperty->GetSignedIntPropertyValue(Address));
		}
		else if (NumericProperty->IsFloatingPoint())
		{
			return MakeValue(NumericProperty->GetFloatingPointPropertyValue(Address));
		}
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		const double Value = BoolProperty->GetPropertyValue(Address) ? 1.0 : 0.0;
		return MakeValue(Value);
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
		if (UnderlyingProperty->IsInteger())
		{
			return MakeValue((double)UnderlyingProperty->GetSignedIntPropertyValue(Address));
		}
	}
	return MakeError();
}

bool UE::StructUtils::SetPropertyValueFromInt64(TNotNull<const FProperty*> InProperty, TNotNull<void*> Address, const int64 Value)
{
	if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(InProperty))
	{
		if (NumericProperty->IsInteger())
		{
			NumericProperty->SetIntPropertyValue(Address, Value);
			return true;
		}
		else if (NumericProperty->IsFloatingPoint())
		{
			NumericProperty->SetFloatingPointPropertyValue(Address, (double)Value);
			return true;
		}
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		const bool bValue = Value != 0;
		BoolProperty->SetPropertyValue(Address, bValue);
		return true;
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
		if (UnderlyingProperty->IsInteger())
		{
			UnderlyingProperty->SetIntPropertyValue(Address, Value);
			return true;
		}
	}
	return false;
}

bool UE::StructUtils::SetPropertyValueFromUInt64(TNotNull<const FProperty*> InProperty, TNotNull<void*> Address, uint64 Value)
{
	if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(InProperty))
	{
		if (NumericProperty->IsInteger())
		{
			NumericProperty->SetIntPropertyValue(Address, Value);
			return true;
		}
		else if (NumericProperty->IsFloatingPoint())
		{
			NumericProperty->SetFloatingPointPropertyValue(Address, (double)Value);
			return true;
		}
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		const bool bValue = Value != 0;
		BoolProperty->SetPropertyValue(Address, bValue);
		return true;
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
		if (UnderlyingProperty->IsInteger())
		{
			UnderlyingProperty->SetIntPropertyValue(Address, Value);
			return true;
		}
	}
	return false;
}

bool UE::StructUtils::SetPropertyValueFromDouble(TNotNull<const FProperty*> InProperty, TNotNull<void*> Address, double Value)
{
	if (const FNumericProperty* NumericProperty = CastField<const FNumericProperty>(InProperty))
	{
		if (NumericProperty->IsInteger())
		{
			NumericProperty->SetIntPropertyValue(Address, FMath::RoundToInt64(Value));
			return true;
		}
		else if (NumericProperty->IsFloatingPoint())
		{
			NumericProperty->SetFloatingPointPropertyValue(Address, Value);
			return true;
		}
	}
	else if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		const bool bValue = !(FMath::IsNearlyZero(Value));
		BoolProperty->SetPropertyValue(Address, bValue);
		return true;
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		const FNumericProperty* UnderlyingProperty = EnumProperty->GetUnderlyingProperty();
		if (UnderlyingProperty->IsInteger())
		{
			UnderlyingProperty->SetIntPropertyValue(Address, FMath::RoundToInt64(Value));
			return true;
		}
	}
	return false;
}
