// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/StructUtilsTypes.h"
#include "Hash/CityHash.h"
#include "Serialization/ArchiveCrc32.h"
#include "StructUtils/PropertyBag.h"
#include "StructUtils/StructView.h"
#include "StructUtils/SharedStruct.h"
#include "StructUtils/UserDefinedStruct.h"
#include "UObject/ObjectKey.h"


namespace UE::StructUtils
{
	/**
	 * Calculates a checksum from the input provided to the archive
	 * with customizable mode regarding UObject serialization.
	 */
	class FHashBuilder : public FArchiveCrc32
	{
	public:
		enum class EObjectHashingMode
		{
			/** Objects with same name will produce the same hash */
			ObjectPathBasedCRC,

			/** Generated hash will be unique per object instance */
			ObjectPtrBasedCRC
		};

		explicit FHashBuilder(uint32 InCRC, EObjectHashingMode ObjectHashingMode)
			: FArchiveCrc32(InCRC, /*RootObject*/nullptr)
			, ObjectHashingMode(ObjectHashingMode)
		{
		}

		//~ Begin FArchive Interface
		virtual FArchive& operator<<(UObject*& Object) override
		{
			if (ObjectHashingMode == EObjectHashingMode::ObjectPtrBasedCRC)
			{
				uint32 ObjectKeyHash = GetTypeHash(FObjectKey(Object));
				Serialize(&ObjectKeyHash, sizeof(ObjectKeyHash));
				return *this;
			}
			
			return FArchiveCrc32::operator<<(Object);
		}

		virtual FString GetArchiveName() const override
		{
			return TEXT("StructUtils_FHashBuilder");
		}
		//~ End FArchive Interface
	private:
		EObjectHashingMode ObjectHashingMode = EObjectHashingMode::ObjectPathBasedCRC;
	};

	uint32 GetStructInstanceCrc32(const UScriptStruct& ScriptStruct, const uint8* StructMemory, const uint32 CRC /*= 0*/)
	{
		FHashBuilder Ar(HashCombine(CRC, PointerHash(&ScriptStruct)), FHashBuilder::EObjectHashingMode::ObjectPtrBasedCRC);
		if (StructMemory)
		{
			UScriptStruct& NonConstScriptStruct = const_cast<UScriptStruct&>(ScriptStruct);
			NonConstScriptStruct.SerializeItem(Ar, const_cast<uint8*>(StructMemory), nullptr);
		}
		return Ar.GetCrc();
	}

	uint32 GetStructCrc32(const UScriptStruct& ScriptStruct, const uint8* StructMemory, const uint32 CRC /*= 0*/)
	{
		FHashBuilder Ar(HashCombine(CRC, PointerHash(&ScriptStruct)), FHashBuilder::EObjectHashingMode::ObjectPathBasedCRC);
		if (StructMemory)
		{
			UScriptStruct& NonConstScriptStruct = const_cast<UScriptStruct&>(ScriptStruct);
			NonConstScriptStruct.SerializeItem(Ar, const_cast<uint8*>(StructMemory), nullptr);
		}
		return Ar.GetCrc();
	}

	template<typename T>
	uint32 GetStructCrc32Helper(const T& Struct, const FHashBuilder::EObjectHashingMode ObjectHashingMode, const uint32 CRC /*= 0*/)
	{
		if (const UScriptStruct* ScriptStruct = Struct.GetScriptStruct())
		{
			if (ObjectHashingMode == FHashBuilder::EObjectHashingMode::ObjectPathBasedCRC)
			{
				return GetStructCrc32(*ScriptStruct, Struct.GetMemory(), CRC);
			}

			return GetStructInstanceCrc32(*ScriptStruct, Struct.GetMemory(), CRC);
		}
		return 0;
	}

	uint32 GetStructCrc32(const FStructView& StructView, const uint32 CRC /*= 0*/)
	{
		return GetStructCrc32Helper(StructView, FHashBuilder::EObjectHashingMode::ObjectPathBasedCRC, CRC);
	}

	uint32 GetStructCrc32(const FConstStructView& StructView, const uint32 CRC /*= 0*/)
	{
		return GetStructCrc32Helper(StructView, FHashBuilder::EObjectHashingMode::ObjectPathBasedCRC, CRC);
	}

	uint32 GetStructCrc32(const FSharedStruct& SharedView, const uint32 CRC /*= 0*/)
	{
		return GetStructCrc32Helper(SharedView, FHashBuilder::EObjectHashingMode::ObjectPathBasedCRC, CRC);
	}

	uint32 GetStructCrc32(const FConstSharedStruct& SharedView, const uint32 CRC /*= 0*/)
	{
		return GetStructCrc32Helper(SharedView, FHashBuilder::EObjectHashingMode::ObjectPathBasedCRC, CRC);
	}

	uint32 GetStructInstanceCrc32(const FStructView& StructView, const uint32 CRC /*= 0*/)
	{
		return GetStructCrc32Helper(StructView, FHashBuilder::EObjectHashingMode::ObjectPtrBasedCRC, CRC);
	}

	uint32 GetStructInstanceCrc32(const FConstStructView& StructView, const uint32 CRC /*= 0*/)
	{
		return GetStructCrc32Helper(StructView, FHashBuilder::EObjectHashingMode::ObjectPtrBasedCRC, CRC);
	}

	uint32 GetStructInstanceCrc32(const FSharedStruct& SharedView, const uint32 CRC /*= 0*/)
	{
		return GetStructCrc32Helper(SharedView, FHashBuilder::EObjectHashingMode::ObjectPtrBasedCRC, CRC);
	}

	uint32 GetStructInstanceCrc32(const FConstSharedStruct& SharedView, const uint32 CRC /*= 0*/)
	{
		return GetStructCrc32Helper(SharedView, FHashBuilder::EObjectHashingMode::ObjectPtrBasedCRC, CRC);
	}

	class FArchiveCityHash64 : public FArchiveUObject
	{
	public:
		explicit FArchiveCityHash64(const uint64 InHash = 0)
			: Hash(InHash)
		{}

		/**
		 * @return The hash computed so far.
		 */
		uint64 GetHash() const { return Hash; }

		//~ Begin FArchive Interface
		virtual void Serialize(void* Data, int64 Num) override
		{
			const char* BytePointer = static_cast<const char*>(Data);
			while (Num > 0)
			{
				const int32 BytesToHash = static_cast<int32>(FMath::Min(Num, static_cast<int64>(MAX_int32)));

				Hash = CityHash64WithSeed(BytePointer, BytesToHash, Hash);

				Num -= BytesToHash;
				BytePointer += BytesToHash;
			}
		}

		virtual FArchive& operator<<(class FName& Name) override
		{
			Hash = HashString(Name.ToString(), Hash);
			return *this;
		}

		virtual FArchive& operator<<(class UObject*& Object) override
		{
			Hash = HashString(GetPathNameSafe(Object), Hash);
			return *this;
		}

		virtual FString GetArchiveName() const override { return TEXT("FArchiveCityHash64"); }
		//~ End FArchive Interface

		FORCEINLINE static uint64 HashString(const FString& InString, const uint64 InHash = 0)
		{
			return CityHash64WithSeed(reinterpret_cast<const char*>(*InString), static_cast<uint32>(InString.GetAllocatedSize()), InHash);
		}

	private:
		uint64 Hash;
	};

	uint64 GetStructHash64(const UScriptStruct& ScriptStruct, const uint8* StructMemory)
	{
		const uint64 BaseHash = FArchiveCityHash64::HashString(ScriptStruct.GetPathName());

		const UScriptStruct::ICppStructOps* CppStructOps = ScriptStruct.GetCppStructOps();
		if (CppStructOps && CppStructOps->HasGetTypeHash())
		{
			const uint32 StructHash = ScriptStruct.GetStructTypeHash(StructMemory);
			return CityHash128to64(Uint128_64(BaseHash, StructHash));
		}
		else if (StructMemory)
		{
			FArchiveCityHash64 Ar(BaseHash);
			UScriptStruct& NonConstScriptStruct = const_cast<UScriptStruct&>(ScriptStruct);
			NonConstScriptStruct.SerializeItem(Ar, const_cast<uint8*>(StructMemory), nullptr);

			return Ar.GetHash();
		}
		
		return BaseHash;
	}

	template<typename T>
	uint64 GetStructHash64Helper(const T& Struct)
	{
		if (const UScriptStruct* ScriptStruct = Struct.GetScriptStruct())
		{
			return GetStructHash64(*ScriptStruct, Struct.GetMemory());
		}
		return 0;
	}

	uint64 GetStructHash64(const FStructView& StructView)
	{
		return GetStructHash64Helper(StructView);
	}

	uint64 GetStructHash64(const FConstStructView& StructView)
	{
		return GetStructHash64Helper(StructView);
	}

	uint64 GetStructHash64(const FSharedStruct& SharedView)
	{
		return GetStructHash64Helper(SharedView);
	}

	uint64 GetStructHash64(const FConstSharedStruct& SharedView)
	{
		return GetStructHash64Helper(SharedView);
	}

	bool TheSameLayout(const FInstancedPropertyBag& A, const UScriptStruct* B, bool bCheckPropertiesNames)
	{
		// Both invalid or Both empty.
		if (!A.IsValid())
		{
			return B == nullptr || B->PropertyLink == nullptr;
		}

		return FStructUtils::TheSameLayout(A.GetPropertyBagStruct(), B, bCheckPropertiesNames);
	}
}
