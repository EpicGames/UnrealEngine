// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/UserDefinedStruct.h"
#include "Templates/SubclassOf.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/FrameworkObjectVersion.h"
#include "Misc/PackageName.h"
#include "Blueprint/BlueprintSupport.h"

#if WITH_EDITORONLY_DATA
#include "UObject/CookedMetaData.h"
#include "StructUtils/UserDefinedStructEditorUtils.h"
#endif //WITH_EDITORONLY_DATA

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserDefinedStruct)

FUserStructOnScopeIgnoreDefaults::FUserStructOnScopeIgnoreDefaults(const UUserDefinedStruct* InUserStruct)
{
	// Can't call super constructor because we need to call our overridden initialize
	ScriptStruct = InUserStruct;
	SampleStructMemory = nullptr;
	OwnsMemory = false;
	Initialize();
}

FUserStructOnScopeIgnoreDefaults::FUserStructOnScopeIgnoreDefaults(const UUserDefinedStruct* InUserStruct, uint8* InData) : FStructOnScope(InUserStruct, InData)
{
}

void FUserStructOnScopeIgnoreDefaults::Recreate(const UUserDefinedStruct* InUserStruct)
{
	Destroy();
	ScriptStruct = InUserStruct;
	Initialize();
}

void FUserStructOnScopeIgnoreDefaults::Initialize()
{
	if (const UStruct* ScriptStructPtr = ScriptStruct.Get())
	{
		SampleStructMemory = (uint8*)FMemory::Malloc(ScriptStructPtr->GetStructureSize() ? ScriptStructPtr->GetStructureSize() : 1);
		((UUserDefinedStruct*)ScriptStruct.Get())->InitializeStructIgnoreDefaults(SampleStructMemory);
		OwnsMemory = true;
	}
}

UUserDefinedStruct::UUserDefinedStruct(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DefaultStructInstance.SetPackage(GetOutermost());
}

void UUserDefinedStruct::PrepareCppStructOps()
{
	// User structs can never have struct ops, this stops it from incorrectly assigning a native struct of the same raw name to this user struct
	CppStructOps = nullptr;
	bPrepareCppStructOpsCompleted = true;
}

FName UUserDefinedStruct::FindPropertyNameFromGuid(const FGuid& PropertyID) const
{
	for (const UE::StructUtils::FUserDefinedPropertyID& Property : PropertyIDs)
	{
		if (Property.ID == PropertyID)
		{
			return Property.Name;
		}
	}
	return FName();
}

FGuid UUserDefinedStruct::FindPropertyGuidFromName(const FName PropertyName) const
{
	for (const UE::StructUtils::FUserDefinedPropertyID& Property : PropertyIDs)
	{
		if (Property.Name == PropertyName)
		{
			return Property.ID;
		}
	}
	return FGuid();
}

void UUserDefinedStruct::Serialize(FStructuredArchive::FRecord Record)
{
	Super::Serialize(Record);
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	UnderlyingArchive.UsingCustomVersion(FFrameworkObjectVersion::GUID);

	if (UnderlyingArchive.CustomVer(FFrameworkObjectVersion::GUID) >= FFrameworkObjectVersion::UserDefinedStructsStoreDefaultInstance)
	{
		if (EUserDefinedStructureStatus::UDSS_UpToDate == Status && !(UnderlyingArchive.GetPortFlags() & PPF_Duplicate))
		{
			// If we're saving or loading new data, serialize our defaults
			if (!DefaultStructInstance.IsValid() && UnderlyingArchive.IsLoading())
			{
				DefaultStructInstance.Recreate(this);
			}

			uint8* StructData = DefaultStructInstance.GetStructMemory();

			FScopedPlaceholderRawContainerTracker TrackStruct(StructData);
			SerializeItem(Record.EnterField(TEXT("Data")), StructData, nullptr);

			// Now that defaults have been loaded we can inspect our properties
			// and default values and set the StructFlags accordingly:
			if(UnderlyingArchive.IsLoading())
			{
				UpdateStructFlags();
			}
		}
	}

#if WITH_EDITOR
	if (UnderlyingArchive.IsLoading())
	{
		if (UnderlyingArchive.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::UserDefinedStructsBlueprintVisible)
		{
			for (TFieldIterator<FProperty> PropIt(this); PropIt; ++PropIt)
			{
				PropIt->PropertyFlags |= CPF_BlueprintVisible;
			}
		}

		if (EUserDefinedStructureStatus::UDSS_UpToDate == Status)
		{
			// We need to force the editor data to be preload in case anyone needs to extract variable
			// information at editor time about the user structure.
			if (EditorData != nullptr)
			{
				UnderlyingArchive.Preload(EditorData);
				if (!(UnderlyingArchive.GetPortFlags() & PPF_Duplicate))
				{
					if (UUserDefinedStructEditorDataBase* UDSEditorData = Cast<UUserDefinedStructEditorDataBase>(EditorData))
					{
						if (!DefaultStructInstance.IsValid())
						{
							UDSEditorData->RecreateDefaultInstance();
						}
						else
						{
							UDSEditorData->ReinitializeDefaultInstance();
						}
					}
				}
			}

			// While duplicating, the struct is not linked and is therefore invalid.
			if (!(UnderlyingArchive.GetPortFlags() & PPF_Duplicate))
			{
				const FUserDefinedStructEditorUtils::EStructureError Result = FUserDefinedStructEditorUtils::IsStructureValid(this, NULL, &ErrorMessage);
				if (FUserDefinedStructEditorUtils::EStructureError::Ok != Result)
				{
					Status = EUserDefinedStructureStatus::UDSS_Error;
					UE_LOGF(LogClass, Log, "UUserDefinedStruct.Serialize '%ls' validation: %ls", *GetName(), *ErrorMessage);
				}
			}
		}
	}
#endif
}

#if WITH_EDITOR

void UUserDefinedStruct::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if (!bDuplicateForPIE)
	{
		Guid = FGuid::NewGuid();
	}
	if (!bDuplicateForPIE && (GetOuter() != GetTransientPackage()))
	{
		FUserDefinedStructEditorUtils::OnStructureChanged(this);
	}
}

void UUserDefinedStruct::PostLoad()
{
	Super::PostLoad();

	ValidateGuid();
}

void UUserDefinedStruct::PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext)
{
	Super::PreSaveRoot(ObjectSaveContext);

	if (ObjectSaveContext.IsCooking() && (ObjectSaveContext.GetSaveFlags() & SAVE_Optional))
	{
		UStructCookedMetaData* CookedMetaData = NewCookedMetaData();
		CookedMetaData->CacheMetaData(this);

		if (!CookedMetaData->HasMetaData())
		{
			PurgeCookedMetaData();
		}
	}
	else
	{
		PurgeCookedMetaData();
	}
}

void UUserDefinedStruct::PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
	Super::PostSaveRoot(ObjectSaveContext);

	PurgeCookedMetaData();
}

void UUserDefinedStruct::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	if (const UUserDefinedStructEditorDataBase* StructEditorData = Cast<const UUserDefinedStructEditorDataBase>(EditorData))
	{
		Context.AddTag(FAssetRegistryTag(TEXT("Tooltip"), StructEditorData->GetTooltip(), FAssetRegistryTag::TT_Hidden));
	}
}

void UUserDefinedStruct::ValidateGuid()
{
	// Backward compatibility:
	// The guid is created in an deterministic way using existing name.
	if (!Guid.IsValid() && (GetFName() != NAME_None))
	{
		const FString HashString = GetFName().ToString();
		ensure(HashString.Len());

		const uint32 BufferLength = HashString.Len() * sizeof(HashString[0]);
		uint32 HashBuffer[5];
		FSHA1::HashBuffer(*HashString, BufferLength, reinterpret_cast<uint8*>(HashBuffer));
		Guid = FGuid(HashBuffer[1], HashBuffer[2], HashBuffer[3], HashBuffer[4]);
	}
}

void UUserDefinedStruct::OnChanged()
{
	ChangedEvent.Broadcast(this);
}

#endif	// WITH_EDITOR

FProperty* UUserDefinedStruct::CustomFindProperty(const FName Name) const
{
#if WITH_EDITOR
	// If we have the editor data, check that first as it's more up to date
	if (const UUserDefinedStructEditorDataBase* StructEditorData = Cast<const UUserDefinedStructEditorDataBase>(EditorData))
	{
		if (FProperty* EditorProperty = StructEditorData->FindProperty(this, Name))
		{
			return EditorProperty;
		}
	}
#endif // WITH_EDITOR

	// Check the authored names for each field
	FString NameString = Name.ToString();
	for (FProperty* CurrentProp : TFieldRange<FProperty>(this))
	{
		if (GetAuthoredNameForField(CurrentProp) == NameString)
		{
			return CurrentProp;
		}
	}
	return nullptr;
}

FString UUserDefinedStruct::GetAuthoredNameForField(const FField* Field) const
{
	const FProperty* Property = CastField<FProperty>(Field);
	if (!Property)
	{
		return Super::GetAuthoredNameForField(Field);
	}

#if WITH_EDITOR
	if (const UUserDefinedStructEditorDataBase* StructEditorData = Cast<const UUserDefinedStructEditorDataBase>(EditorData))
	{
		const FString EditorName = StructEditorData->GetFriendlyNameForProperty(this, Property);
		if (!EditorName.IsEmpty())
		{
			return EditorName;
		}
	}
#endif	// WITH_EDITOR

	const int32 GuidStrLen = 32;
	const int32 MinimalPostfixlen = GuidStrLen + 3;
	const FString OriginalName = Property->GetName();
	if (OriginalName.Len() > MinimalPostfixlen)
	{
		FString DisplayName = OriginalName.LeftChop(GuidStrLen + 1);
		int FirstCharToRemove = -1;
		const bool bCharFound = DisplayName.FindLastChar(TCHAR('_'), FirstCharToRemove);
		if (bCharFound && (FirstCharToRemove > 0))
		{
			return DisplayName.Mid(0, FirstCharToRemove);
		}
	}
	return OriginalName;
}

void UUserDefinedStruct::InitializeStructIgnoreDefaults(void* Dest, int32 ArrayDim) const
{
	Super::InitializeStruct(Dest, ArrayDim);
}

void UUserDefinedStruct::InitializeStruct(void* Dest, int32 ArrayDim) const
{
	check(Dest);

	const uint8* DefaultInstance = GetDefaultInstance();
	if ((StructFlags & STRUCT_IsPlainOldData) == 0 || !DefaultInstance)
	{
		InitializeStructIgnoreDefaults(Dest, ArrayDim);
	}

	if (DefaultInstance)
	{
		int32 Stride = GetStructureSize();

		for (int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++)
		{
			void* DestStruct = (uint8*)Dest + (Stride * ArrayIndex);
			CopyScriptStruct(DestStruct, DefaultInstance);

#if WITH_EDITOR
			// When copying into another struct we need to register this raw struct pointer so any deferred dependencies will get fixed later
			FScopedPlaceholderRawContainerTracker TrackStruct(DestStruct);
			FBlueprintSupport::RegisterDeferredDependenciesInStruct(this, DestStruct);
#endif
		}	
	}
}

void UUserDefinedStruct::SerializeTaggedProperties(FStructuredArchive::FSlot Slot, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad) const 
{
	bool bTemporarilyEnableDelta = false;
	FArchive& Ar = Slot.GetUnderlyingArchive();

#if WITH_EDITOR
	// In the editor the default structure may change while the editor is running, so we need to always delta serialize
	const UUserDefinedStruct* UDDefaultsStruct = Cast<UUserDefinedStruct>(DefaultsStruct);

	const bool bDuplicate = (0 != (Ar.GetPortFlags() & PPF_Duplicate));

	// When saving delta, we want the difference between current data and true structure's default values.
	// So if we don't have defaults we need to use the struct defaults
	const bool bUseNewDefaults = !Defaults
		&& UDDefaultsStruct
		&& (UDDefaultsStruct->GetDefaultInstance() != Data)
		&& !bDuplicate
		&& (Ar.IsSaving() || Ar.IsLoading())
		&& !Ar.IsCooking();

	if (bUseNewDefaults)
	{
		Defaults = UDDefaultsStruct->GetDefaultInstance();
	}

	// Temporarily enable delta serialization if this is a CPFUO 
	bTemporarilyEnableDelta = bUseNewDefaults && Ar.IsIgnoringArchetypeRef() && Ar.IsIgnoringClassRef() && !Ar.DoDelta();
	if (bTemporarilyEnableDelta)
	{
		Ar.ArNoDelta = false;
	}
#endif // WITH_EDITOR

	Super::SerializeTaggedProperties(Slot, Data, DefaultsStruct, Defaults, BreakRecursionIfFullyLoad);

	if (bTemporarilyEnableDelta)
	{
		Ar.ArNoDelta = true;
	}
}

uint32 UUserDefinedStruct::GetStructTypeHash(const void* Src) const
{
	return GetUserDefinedStructTypeHash(Src, this);
}

void UUserDefinedStruct::RecursivelyPreload()
{
	FLinkerLoad* Linker = GetLinker();
	if( Linker && (nullptr == PropertyLink) )
	{
		TArray<UObject*> AllChildMembers;
		GetObjectsWithOuter(this, AllChildMembers);
		for (int32 Index = 0; Index < AllChildMembers.Num(); ++Index)
		{
			UObject* Member = AllChildMembers[Index];
			Linker->Preload(Member);
		}

		Linker->Preload(this);
		if (nullptr == PropertyLink)
		{
			StaticLink(true);
		}
	}
}

FGuid UUserDefinedStruct::GetCustomGuid() const
{
	return Guid;
}

FString GetPathPostfix(const UObject* ForObject)
{
	FString FullAssetName = ForObject->GetOutermost()->GetPathName();
	FString AssetName = FPackageName::GetLongPackageAssetName(FullAssetName);
	// append a hash of the path, this uniquely identifies assets with the same name, but different folders:
	FullAssetName.RemoveFromEnd(AssetName);
	return FString::Printf(TEXT("%u"), GetTypeHash(FullAssetName));
}

FString UUserDefinedStruct::GetStructCPPName(uint32 CPPExportFlags) const
{
	if (CPPExportFlags & CPPF_BlueprintCppBackend)
	{
		return ::UnicodeToCPPIdentifier(*GetName(), false, GetPrefixCPP()) + GetPathPostfix(this);
	}
	else
	{
		return Super::GetStructCPPName(CPPExportFlags);
	}
}

uint32 UUserDefinedStruct::GetUserDefinedStructTypeHash(const void* Src, const UScriptStruct* Type)
{
	// Ugliness to maximize entropy on first call to HashCombine... Alternatively we could
	// do special logic on the first iteration of the below loops (unwind loops or similar):
	const auto ConditionalCombineHash = [](uint32 AccumulatedHash, uint32 CurrentHash) -> uint32
	{
		if (AccumulatedHash != 0)
		{
			return HashCombine(AccumulatedHash, CurrentHash);
		}
		else
		{
			return CurrentHash;
		}
	};

	uint32 ValueHash = 0;
	// combining bool values and hashing them together, small range enums could get stuffed into here as well,
	// but FBoolProperty does not actually provide GetValueTypeHash (and probably shouldn't). For structs
	// with more than 64 boolean values we lose some information, but that is acceptable, just slightly 
	// increasing risk of hash collision:
	bool bHasBoolValues = false;
	uint64 BoolValues = 0;
	// for blueprint defined structs we can just loop and hash the individual properties:
	for (TFieldIterator<const FProperty> It(Type); It; ++It)
	{
		uint32 CurrentHash = 0;
		if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(*It))
		{
			for (int32 I = 0; I < It->ArrayDim; ++I)
			{
				BoolValues = (BoolValues << 1) + ( BoolProperty->GetPropertyValue_InContainer(Src, I) ? 1 : 0 );
			}
		}
		else if (ensure(It->HasAllPropertyFlags(CPF_HasGetValueTypeHash)))
		{
			for (int32 I = 0; I < It->ArrayDim; ++I)
			{
				uint32 Hash = It->GetValueTypeHash(It->ContainerPtrToValuePtr<void>(Src, I));
				CurrentHash = ConditionalCombineHash(CurrentHash, Hash);
			}
		}

		ValueHash = ConditionalCombineHash(ValueHash, CurrentHash);
	}

	if (bHasBoolValues)
	{
		ValueHash = ConditionalCombineHash(ValueHash, GetTypeHash(BoolValues));
	}

	return ValueHash;
}

const uint8* UUserDefinedStruct::GetDefaultInstance() const
{
	if (ensureMsgf(DefaultStructInstance.IsValid(), TEXT("DefaultStructInstance is invalid for '%s'."), *GetPathName()))
	{
		const UStruct* DefaultStruct = DefaultStructInstance.GetStruct();
		ensureMsgf(DefaultStruct == this, TEXT("DefaultStructInstance struct mismatch: '%s' vs. '%s'"), *GetPathNameSafe(DefaultStruct),
			*GetPathName());
	}
	return DefaultStructInstance.GetStructMemory();
}

void UUserDefinedStruct::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UUserDefinedStruct* This = CastChecked<UUserDefinedStruct>(InThis);
	ensure(!This->DefaultStructInstance.IsValid() || This->DefaultStructInstance.GetStruct() == This);

	if (uint8* StructData = This->DefaultStructInstance.GetStructMemory())
	{
		Collector.AddPropertyReferences(This, StructData, This);
	}

	Super::AddReferencedObjects(This, Collector);
}
void UUserDefinedStruct::UpdateStructFlags()
{
	// Adapted from PrepareCppStructOps, where we 'discover' zero constructability
	// for native types:
	bool bIsZeroConstruct = true;
	{
		uint8* StructData = DefaultStructInstance.GetStructMemory();
		if (StructData)
		{
			int32 Size = GetStructureSize();
			for (int32 Index = 0; Index < Size; Index++)
			{
				if (StructData[Index])
				{
					bIsZeroConstruct = false;
					break;
				}
			}
		}

		if(bIsZeroConstruct)
		{	
			for (TFieldIterator<FProperty> It(this); It; ++It)
			{
				FProperty* Property = *It;
				if (Property && !Property->HasAnyPropertyFlags(CPF_ZeroConstructor))
				{
					bIsZeroConstruct = false;
					break;
				}
			}
		}
	}

	// IsPOD/NoDtor could be derived earlier than bIsZeroConstruct because they do not depend on 
	// the structs default values, but it is convenient to calculate them all in one place:
	bool bIsPOD = true;
	{
		for (TFieldIterator<FProperty> It(this); It; ++It)
		{
			FProperty* Property = *It;
			if (Property && !Property->HasAnyPropertyFlags(CPF_IsPlainOldData))
			{
				bIsPOD = false;
				break;
			}
		}
	}

	bool bHasNoDtor = bIsPOD;
	{
		if(!bHasNoDtor)
		{
			// we're not POD, but we still may have no destructor, check properties:
			bHasNoDtor = true;
			for (TFieldIterator<FProperty> It(this); It; ++It)
			{
				FProperty* Property = *It;
				if (Property && !Property->HasAnyPropertyFlags(CPF_NoDestructor))
				{
					bHasNoDtor = false;
					break;
				}
			}
		}
	}
	
	StructFlags = EStructFlags(StructFlags | STRUCT_ZeroConstructor | STRUCT_IsPlainOldData | STRUCT_NoDestructor);
	if(!bIsZeroConstruct)
	{
		StructFlags = EStructFlags(StructFlags & ~STRUCT_ZeroConstructor);
	}
	if(!bIsPOD)
	{
		StructFlags = EStructFlags(StructFlags & ~STRUCT_IsPlainOldData);
	}
	if(!bHasNoDtor)
	{
		StructFlags = EStructFlags(StructFlags & ~STRUCT_NoDestructor);
	}

}

#if WITH_EDITORONLY_DATA
TSubclassOf<UStructCookedMetaData> UUserDefinedStruct::GetCookedMetaDataClass() const
{
	return UStructCookedMetaData::StaticClass();
}

UStructCookedMetaData* UUserDefinedStruct::NewCookedMetaData()
{
	if (!CachedCookedMetaDataPtr)
	{
		CachedCookedMetaDataPtr = CookedMetaDataUtil::NewCookedMetaData<UStructCookedMetaData>(this, "CookedStructMetaData", GetCookedMetaDataClass());
	}
	return CachedCookedMetaDataPtr;
}

const UStructCookedMetaData* UUserDefinedStruct::FindCookedMetaData()
{
	if (!CachedCookedMetaDataPtr)
	{
		CachedCookedMetaDataPtr = CookedMetaDataUtil::FindCookedMetaData<UStructCookedMetaData>(this, TEXT("CookedStructMetaData"));
	}
	return CachedCookedMetaDataPtr;
}

void UUserDefinedStruct::PurgeCookedMetaData()
{
	if (CachedCookedMetaDataPtr)
	{
		CookedMetaDataUtil::PurgeCookedMetaData<UStructCookedMetaData>(CachedCookedMetaDataPtr);
	}
}
#endif // WITH_EDITORONLY_DATA
