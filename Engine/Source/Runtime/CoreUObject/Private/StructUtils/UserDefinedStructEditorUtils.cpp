// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructUtils/UserDefinedStructEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UserDefinedStructEditorUtils)

#if WITH_EDITOR

#define LOCTEXT_NAMESPACE "Structure"

//////////////////////////////////////////////////////////////////////////
// FUserDefinedStructEditorUtils

FUserDefinedStructEditorUtils::FOnUserDefinedStructChanged FUserDefinedStructEditorUtils::OnUserDefinedStructChanged;

namespace
{
	bool IsObjPropertyValid(const FProperty* Property)
	{
		if (const FInterfaceProperty* InterfaceProperty = CastField<const FInterfaceProperty>(Property))
		{
			return InterfaceProperty->InterfaceClass != nullptr;
		}
		
		if (const FArrayProperty* ArrayProperty = CastField<const FArrayProperty>(Property))
		{
			return ArrayProperty->Inner && IsObjPropertyValid(ArrayProperty->Inner);
		}
		
		if (const FObjectProperty* ObjectProperty = CastField<const FObjectProperty>(Property))
		{
			return ObjectProperty->PropertyClass != nullptr;
		}
		return true;
	}
}


void FUserDefinedStructEditorUtils::OnStructureChanged(UUserDefinedStruct* Struct)
{
	if (Struct)
	{
		OnUserDefinedStructChanged.ExecuteIfBound(Struct);
	}
}

FUserDefinedStructEditorUtils::EStructureError FUserDefinedStructEditorUtils::IsStructureValid(const UScriptStruct* Struct, const UStruct* RecursionParent, FString* OutMsg)
{
	check(Struct);
	if (Struct == RecursionParent)
	{
		if (OutMsg)
		{
			*OutMsg = FText::Format(LOCTEXT("StructureRecursionFmt", "Recursion: Recursion: Struct cannot have itself or a nested struct member referencing itself as a member variable. Struct '{0}', recursive parent '{1}'"), 
				 FText::FromString(Struct->GetFullName()), FText::FromString(RecursionParent->GetFullName())).ToString();
		}
		return EStructureError::Recursion;
	}

	const UScriptStruct* FallbackStruct = GetFallbackStruct();
	if (Struct == FallbackStruct)
	{
		if (OutMsg)
		{
			*OutMsg = LOCTEXT("StructureUnknown", "Struct unknown (deleted?)").ToString();
		}
		return EStructureError::FallbackStruct;
	}

	if (Struct->GetStructureSize() <= 0)
	{
		if (OutMsg)
		{
			*OutMsg = FText::Format(LOCTEXT("StructureSizeIsZeroFmt", "Struct '{0}' is empty"), FText::FromString(Struct->GetFullName())).ToString();
		}
		return EStructureError::EmptyStructure;
	}

	if (const UUserDefinedStruct* UDStruct = Cast<const UUserDefinedStruct>(Struct))
	{
		if (UDStruct->Status != EUserDefinedStructureStatus::UDSS_UpToDate)
		{
			if (OutMsg)
			{
				*OutMsg = FText::Format(LOCTEXT("StructureNotCompiledFmt", "Struct '{0}' is not compiled"), FText::FromString(Struct->GetFullName())).ToString();
			}
			return EStructureError::NotCompiled;
		}

		for (const FProperty* P = Struct->PropertyLink; P; P = P->PropertyLinkNext)
		{
			const FStructProperty* StructProp = CastField<const FStructProperty>(P);
			if (nullptr == StructProp)
			{
				if (const FArrayProperty* ArrayProp = CastField<const FArrayProperty>(P))
				{
					StructProp = CastField<const FStructProperty>(ArrayProp->Inner);
				}
			}

			if (StructProp)
			{
				if ((NULL == StructProp->Struct) || (FallbackStruct == StructProp->Struct))
				{
					if (OutMsg)
					{
						*OutMsg = FText::Format(
							LOCTEXT("StructureUnknownPropertyFmt", "Struct unknown (deleted?). Parent '{0}' Property: '{1}'"),
							FText::FromString(Struct->GetFullName()),
							FText::FromString(StructProp->GetName())
						).ToString();
					}
					return EStructureError::FallbackStruct;
				}

				FString OutMsgInner;
				const EStructureError Result = IsStructureValid(
					StructProp->Struct,
					RecursionParent ? RecursionParent : Struct,
					OutMsg ? &OutMsgInner : nullptr);
				if (EStructureError::Ok != Result)
				{
					if (OutMsg)
					{
						*OutMsg = FText::Format(
							LOCTEXT("StructurePropertyErrorTemplateFmt", "Struct '{0}' Property '{1}' Error ( {2} )"),
							FText::FromString(Struct->GetFullName()),
							FText::FromString(StructProp->GetName()),
							FText::FromString(OutMsgInner)
						).ToString();
					}
					return Result;
				}
			}

			// The structure is loaded (from .uasset) without recompilation. All properties should be verified.
			if (!IsObjPropertyValid(P))
			{
				if (OutMsg)
				{
					*OutMsg = FText::Format(
						LOCTEXT("StructureUnknownObjectPropertyFmt", "Invalid object property. Structure '{0}' Property: '{1}'"),
						FText::FromString(Struct->GetFullName()),
						FText::FromString(P->GetName())
					).ToString();
				}
				return EStructureError::NotCompiled;
			}
		}
	}

	return EStructureError::Ok;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
