// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/Casts.h"

DECLARE_LOG_CATEGORY_EXTERN(LogCasts, Log, All);
DEFINE_LOG_CATEGORY(LogCasts);

void CastLogError(const TCHAR* FromType, const TCHAR* ToType)
{
	UE_LOGF(LogCasts, Fatal, "Cast of %ls to %ls failed", FromType, ToType);
    for(;;);
}
