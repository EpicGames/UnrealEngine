// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ANY_PACKAGE macro for legacy support
=============================================================================*/

#pragma once

// HEADER_UNIT_SKIP - Deprecated

#include "CoreTypes.h"

UE_DEPRECATED_HEADER(5.7, "AnyPackagePrivate.h has been deprecated and should no longer be included")
