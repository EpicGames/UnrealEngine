// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/AssetRegistryTagsContext.h"

#include "Logging/LogMacros.h"
#include "Misc/StringBuilder.h"
#include "UObject/NameTypes.h"

FAssetRegistryTagsContextData::FAssetRegistryTagsContextData(const UObject* CallTarget,
	EAssetRegistryTagsCaller InCaller)
: Object(CallTarget)
, Caller(InCaller)
{
}

FAssetRegistryTagsContext::FAssetRegistryTagsContext(FAssetRegistryTagsContextData& InData)
	: Data(InData)
{
}

FAssetRegistryTagsContext::FAssetRegistryTagsContext(const FAssetRegistryTagsContext& Other)
	: Data(Other.Data)
{
}

const UObject* FAssetRegistryTagsContext::GetObject() const
{
	return Data.Object;
}

EAssetRegistryTagsCaller FAssetRegistryTagsContext::GetCaller() const
{
	return Data.Caller;
}

bool FAssetRegistryTagsContext::IsFullUpdate() const
{
	return Data.Caller == EAssetRegistryTagsCaller::FullUpdate || Data.bFullUpdateRequested;
}

bool FAssetRegistryTagsContext::IsSaving() const
{
	return Data.Caller == EAssetRegistryTagsCaller::SavePackage;
}

bool FAssetRegistryTagsContext::IsProceduralSave() const
{
	return Data.bProceduralSave;
}

bool FAssetRegistryTagsContext::IsCooking() const
{
	return Data.TargetPlatform != nullptr;
}

const ITargetPlatform* FAssetRegistryTagsContext::GetTargetPlatform() const
{
	return Data.TargetPlatform;
}

bool FAssetRegistryTagsContext::IsCookByTheBook() const
{
	return Data.CookType == UE::Cook::ECookType::ByTheBook;
}

bool FAssetRegistryTagsContext::IsCookOnTheFly() const
{
	return Data.CookType == UE::Cook::ECookType::OnTheFly;
}

bool FAssetRegistryTagsContext::IsCookTypeUnknown() const
{
	return Data.CookType == UE::Cook::ECookType::Unknown;
}

UE::Cook::ECookType FAssetRegistryTagsContext::GetCookType() const
{
	return Data.CookType;
}

UE::Cook::ECookingDLC FAssetRegistryTagsContext::GetCookingDLC() const
{
	return Data.CookingDLC;
}

UObject::FAssetRegistryTag& FAssetRegistryTagsContext::FindOrAddTag(FName TagName, bool* bOutAlreadyExists)
{
	return FindOrAddTagInternal(TagName, FStringView(), bOutAlreadyExists);
}

UObject::FAssetRegistryTag* FAssetRegistryTagsContext::FindTag(FName TagName)
{
	return FindTagInternal(TagName, FStringView());
}

bool FAssetRegistryTagsContext::ContainsTag(FName TagName) const
{
	return ContainsTagInternal(TagName, FStringView());
}

void FAssetRegistryTagsContext::AddTag(UObject::FAssetRegistryTag TagResult)
{
	AddTagInternal(MoveTemp(TagResult), FStringView());
}

void FAssetRegistryTagsContext::RemoveTag(FName TagName)
{
	RemoveTagInternal(TagName, FStringView());
}

int32 FAssetRegistryTagsContext::GetNumTags() const
{
	return Data.Tags.Num();
}

void FAssetRegistryTagsContext::EnumerateTags(TFunctionRef<void(const UObject::FAssetRegistryTag&)> Visitor) const
{
	for (const TPair<FName, UObject::FAssetRegistryTag>& Pair : Data.Tags)
	{
		Visitor(Pair.Value);
	}
}

bool FAssetRegistryTagsContext::WantsBundleResult() const
{
	return Data.bWantsBundleResult;
}

const FAssetBundleData* FAssetRegistryTagsContext::GetBundleResult()
{
	return Data.BundleResult;
}

void FAssetRegistryTagsContext::SetBundleResult(const FAssetBundleData* InBundleResult)
{
	Data.BundleResult = InBundleResult;
}

bool FAssetRegistryTagsContext::WantsCookTags() const
{
	return Data.bWantsCookTags;
}

UObject::FAssetRegistryTag& FAssetRegistryTagsContext::FindOrAddCookTag(FName TagName, bool* bOutAlreadyExists)
{
	if (!WantsCookTags())
	{
		static UObject::FAssetRegistryTag PlaceholderTag;
		PlaceholderTag.Name = TagName;
		return PlaceholderTag;
	}
	return FindOrAddTagInternal(TagName, UE::AssetRegistry::CookTagPrefix, bOutAlreadyExists);
}

UObject::FAssetRegistryTag* FAssetRegistryTagsContext::FindCookTag(FName TagName)
{
	if (!WantsCookTags())
	{
		return nullptr;
	}
	return FindTagInternal(TagName, UE::AssetRegistry::CookTagPrefix);
}

bool FAssetRegistryTagsContext::ContainsCookTag(FName TagName) const
{
	if (!WantsCookTags())
	{
		return false;
	}
	return ContainsTagInternal(TagName, UE::AssetRegistry::CookTagPrefix);
}

void FAssetRegistryTagsContext::AddCookTag(UObject::FAssetRegistryTag TagResult)
{
	if (!WantsCookTags())
	{
		return;
	}
	AddTagInternal(MoveTemp(TagResult), UE::AssetRegistry::CookTagPrefix);
}

void FAssetRegistryTagsContext::RemoveCookTag(FName TagName)
{
	if (!WantsCookTags())
	{
		return;
	}
	RemoveTagInternal(TagName, UE::AssetRegistry::CookTagPrefix);
}

bool FAssetRegistryTagsContext::TryValidateTagName(FName TagName) const
{
	using namespace UE::AssetRegistry;

	if (!ensureMsgf(!TagName.IsNone(), TEXT("Invalid empty TagName passed to FAssetRegistryTagsContext.")))
	{
		return false;
	}
	TStringBuilder<64> TagNameStr(InPlace, TagName);
	bool bProtectedKeyword = FStringView(TagNameStr).StartsWith(CookTagPrefix, ESearchCase::IgnoreCase);
	if (!ensureMsgf(!bProtectedKeyword,
		TEXT("Invalid TagName %s that starts with protected keyword %.*s passed to FAssetRegistryTagsContext."),
		*TagNameStr, CookTagPrefix.Len(), CookTagPrefix.GetData()))
	{
		return false;
	}
	return true;
}

FName FAssetRegistryTagsContext::TransformTagName(FName TagName, FStringView PrefixToAdd, bool bCreateNameIfDoesNotExist) const
{
	if (!TryValidateTagName(TagName))
	{
		return NAME_None;
	}
	if (!PrefixToAdd.IsEmpty())
	{
		TStringBuilder<64> TagNameStr(InPlace, PrefixToAdd, TagName);
		TagName = FName(TagNameStr, bCreateNameIfDoesNotExist ? FNAME_Add : FNAME_Find);
	}
	return TagName;
}

UObject::FAssetRegistryTag& FAssetRegistryTagsContext::FindOrAddTagInternal(FName TagName, FStringView PrefixToAdd, bool* bOutAlreadyExists)
{
	TagName = TransformTagName(TagName, PrefixToAdd);
	if (TagName.IsNone())
	{
		static UObject::FAssetRegistryTag ErrorResult;
		ErrorResult.Name = TagName;
		if (bOutAlreadyExists)
		{
			*bOutAlreadyExists = false;
		}
		return ErrorResult;
	}

	UObject::FAssetRegistryTag& Result = Data.Tags.FindOrAdd(TagName);
	if (Result.Name != TagName)
	{
		Result.Name = TagName;
		if (bOutAlreadyExists)
		{
			*bOutAlreadyExists = false;
		}
	}
	else
	{
		if (bOutAlreadyExists)
		{
			*bOutAlreadyExists = true;
		}
	}
	return Result;
}

UObject::FAssetRegistryTag* FAssetRegistryTagsContext::FindTagInternal(FName TagName, FStringView PrefixToAdd)
{
	TagName = TransformTagName(TagName, PrefixToAdd, false);
	if (TagName.IsNone())
	{
		return nullptr;
	}

	return Data.Tags.Find(TagName);
}

bool FAssetRegistryTagsContext::ContainsTagInternal(FName TagName, FStringView PrefixToAdd) const
{
	TagName = TransformTagName(TagName, PrefixToAdd, false);
	if (TagName.IsNone())
	{
		return false;
	}

	return Data.Tags.Contains(TagName);
}

void FAssetRegistryTagsContext::AddTagInternal(UObject::FAssetRegistryTag&& TagResult, FStringView PrefixToAdd)
{
	FName TagName = TransformTagName(TagResult.Name, PrefixToAdd);
	if (TagName.IsNone())
	{
		return;
	}
	UObject::FAssetRegistryTag& AddedTag = Data.Tags.FindOrAdd(TagName);
	AddedTag = MoveTemp(TagResult);
	AddedTag.Name = TagName;
}

void FAssetRegistryTagsContext::RemoveTagInternal(FName TagName, FStringView PrefixToAdd)
{
	TagName = TransformTagName(TagName, PrefixToAdd, false);
	if (TagName.IsNone())
	{
		return;
	}
	Data.Tags.Remove(TagName);
}
