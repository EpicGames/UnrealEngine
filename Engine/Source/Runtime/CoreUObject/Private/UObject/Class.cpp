// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/Class.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/ScopeLock.h"
#include "Serialization/MemoryWriter.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/EnumRange.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceConsole.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/AutomationTest.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/StringBuilder.h"
#include "Misc/OutputDeviceNull.h"
#include "UObject/CoreNet.h"
#include "Modules/ModuleManager.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/StrProperty.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/MetaData.h"
#include "Templates/Casts.h"
#include "Templates/MemoryOps.h"
#include "UObject/DebugSerializationFlags.h"
#include "UObject/PropertyTag.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/Reload.h"
#include "UObject/Stack.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectResource.h"
#include "UObject/LinkerSave.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/Interface.h"
#include "UObject/UObjectPartials.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/LinkerPlaceholderFunction.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPathNameTree.h"
#include "UObject/PropertyStateTracking.h"
#include "UObject/StructOnScope.h"
#include "UObject/StructScriptLoader.h"
#include "UObject/PropertyHelper.h"
#include "UObject/CoreRedirects.h"
#include "UObject/ObjectMacros.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/TestUninitializedScriptStructMembersTest.h"
#include "Internationalization/PolyglotTextData.h"
#include "Serialization/ArchiveScriptReferenceCollector.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "Serialization/NullArchive.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/GarbageCollection.h"
#include "UObject/UObjectThreadContext.h"
#include "Serialization/LoadTimeTracePrivate.h"
#include "Serialization/UnversionedPropertySerialization.h"
#include "Serialization/UnversionedPropertySerializationTest.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/PropertyProxyArchive.h"
#include "UObject/FieldPath.h"
#include "HAL/ThreadSafeCounter.h"
#include "Math/InterpCurvePoint.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/TopLevelAssetPath.h"
#include "AssetRegistry/AssetData.h"
#include "HAL/PlatformStackWalk.h"
#include "String/Find.h"
#include "String/ParseTokens.h"
#include "AutoRTFM.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Serialization/TestUndeclaredScriptStructObjectReferences.h"
#include "UObject/OverridableManager.h"
#include "UObject/OverriddenPropertySet.h"
#include "Misc/UObjectTestUtils.h"
#include "UObject/UObjectConstructInternal.h"
#include "UObject/RegisterCompiledInObjects.h"

// This flag enables some expensive class tree validation that is meant to catch mutations of
// the class tree outside of SetSuperStruct. It has been disabled because loading blueprints
// does a lot of mutation of the class tree, and the validation checks impact iteration time.

#include UE_INLINE_GENERATED_CPP_BY_NAME(Class)
#define DO_CLASS_TREE_VALIDATION 0

DEFINE_LOG_CATEGORY(LogScriptSerialization);
DEFINE_LOG_CATEGORY(LogClass);

LLM_DEFINE_TAG(UObject_UClass);

#if defined(_MSC_VER) && _MSC_VER == 1900
	#ifdef PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
		PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
	#endif
#endif

// If we end up pushing class flags out beyond a uint32, there are various places
// casting it to uint32 that need to be fixed up (mostly printfs but also some serialization code)
static_assert(sizeof(__underlying_type(EClassFlags)) == sizeof(uint32), "expecting ClassFlags enum to fit in a uint32");

enum class EStructStateFlags : uint16
{
	None = 0,
	PropertyDataAvailable = 1 << 0
};

ENUM_CLASS_FLAGS(EStructStateFlags)

//////////////////////////////////////////////////////////////////////////

namespace { UE_CALL_ONCE(UE::GC::RegisterSlowImplementation, &UClass::AddReferencedObjects, UE::GC::EAROFlags::Unbalanced); }

FThreadSafeBool& InternalSafeGetTokenStreamDirtyFlag()
{
	static FThreadSafeBool TokenStreamDirty(true);
	return TokenStreamDirty;
}

#if !UE_WITH_CONSTINIT_UOBJECT
/**
 * Shared function called from the various InitializePrivateStaticClass functions generated my the IMPLEMENT_CLASS macro.
 */
COREUOBJECT_API void InitializePrivateStaticClass(
	class UClass* (*TClass_StaticClassFn)(),
	class UClass* TClass_Super_StaticClass,
	class UClass* TClass_PrivateStaticClass,
	class UClass* TClass_WithinClass_StaticClass,
	const TCHAR* PackageName,
	const TCHAR* Name
	)
{
	TRACE_LOADTIME_CLASS_INFO(TClass_PrivateStaticClass, Name);

	/* No recursive ::StaticClass calls allowed. Setup extras. */
	if (TClass_Super_StaticClass != TClass_PrivateStaticClass)
	{
		TClass_PrivateStaticClass->SetSuperStruct(TClass_Super_StaticClass);
	}
	else
	{
		TClass_PrivateStaticClass->SetSuperStruct(NULL);
	}
	TClass_PrivateStaticClass->ClassWithin = TClass_WithinClass_StaticClass;

	// Register the class's dependencies, then itself.
	TClass_PrivateStaticClass->RegisterDependencies();
	{
		// Defer
		TClass_PrivateStaticClass->Register(TClass_StaticClassFn, PackageName, Name);
	}
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

void FNativeFunctionRegistrar::RegisterFunction(class UClass* Class, const ANSICHAR* InName, FNativeFuncPtr InPointer)
{
	Class->AddNativeFunction(InName, InPointer);
}

void FNativeFunctionRegistrar::RegisterFunction(class UClass* Class, const WIDECHAR* InName, FNativeFuncPtr InPointer)
{
	Class->AddNativeFunction(InName, InPointer);
}

#if !UE_WITH_CONSTINIT_UOBJECT
void FNativeFunctionRegistrar::RegisterFunctions(class UClass* Class, TConstArrayView<UE::CodeGen::FClassNativeFunction> InFunctions)
{
	for (const UE::CodeGen::FClassNativeFunction& Function : InFunctions)
	{
		Class->AddNativeFunction(UTF8_TO_TCHAR(Function.NameUTF8), Function.Pointer);
	}

}
void FNativeFunctionRegistrar::RegisterFunctions(class UClass* Class, const FNameNativePtrPair* InArray, int32 NumFunctions)
{
	for (; NumFunctions; ++InArray, --NumFunctions)
	{
		Class->AddNativeFunction(UTF8_TO_TCHAR(InArray->NameUTF8), InArray->Pointer);
	}
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

/*-----------------------------------------------------------------------------
	UField implementation.
-----------------------------------------------------------------------------*/

UField::UField( EStaticConstructor, EObjectFlags InFlags )
: UObject( EC_StaticConstructor, InFlags )
, Next( NULL )
{}

UClass* UField::GetOwnerClass() const
{
	UClass* OwnerClass = NULL;
	UObject* TestObject = const_cast<UField*>(this);

	while ((TestObject != NULL) && (OwnerClass == NULL))
	{
		OwnerClass = dynamic_cast<UClass*>(TestObject);
		TestObject = TestObject->GetOuter();
	}

	return OwnerClass;
}

UStruct* UField::GetOwnerStruct() const
{
	const UObject* Obj = this;
	do
	{
		if (const UStruct* Result = dynamic_cast<const UStruct*>(Obj))
		{
			return const_cast<UStruct*>(Result);
		}

		Obj = Obj->GetOuter();
	}
	while (Obj);

	return nullptr;
}

FString UField::GetAuthoredName() const
{
	UStruct* Struct = GetOwnerStruct();
	if (Struct)
	{
		return Struct->GetAuthoredNameForField(this);
	}
	return FString();
}

void UField::Bind()
{
}

void UField::PostLoad()
{
	Super::PostLoad();
	Bind();
}

bool UField::NeedsLoadForClient() const
{
	// Overridden to avoid calling the expensive generic version, which only ensures that our class is not excluded, which it never can be
	return true;
}

bool UField::NeedsLoadForServer() const
{
	return true;
}

void UField::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::RemoveUField_Next)
	{
		Ar << Next;
	}
}

void UField::AddCppProperty(FProperty* Property)
{
	UE_LOGF(LogClass, Fatal,"UField::AddCppProperty");
}

#if WITH_EDITORONLY_DATA

struct FDisplayNameHelper
{
	static FString Get(const UObject& Object)
	{
		const UClass* Class = Cast<const UClass>(&Object);
		if (Class && !Class->HasAnyClassFlags(CLASS_Native))
		{
			FString Name = Object.GetName();
			Name.RemoveFromEnd(TEXT("_C"));
			Name.RemoveFromStart(TEXT("SKEL_"));
			return Name;
		}

		//if (auto Property = dynamic_cast<const FProperty*>(&Object))
		//{
		//	return Property->GetAuthoredName();
		//}

		return Object.GetName();
	}
};

/**
 * Finds the localized display name or native display name as a fallback.
 *
 * @return The display name for this object.
 */
FText UField::GetDisplayNameText() const
{
	static const FTextKey Namespace = TEXT("UObjectDisplayNames");
	static const FName NAME_DisplayName("DisplayName");

	const FString Key = GetFullGroupName(false);

	FString NativeDisplayName = GetMetaData(NAME_DisplayName);
	if (NativeDisplayName.IsEmpty())
	{
		NativeDisplayName = FName::NameToDisplayString(FDisplayNameHelper::Get(*this), false);
	}

	return FText::AsLocalizable_Advanced(Namespace, Key, MoveTemp(NativeDisplayName));
}

/**
 * Finds the localized tooltip or native tooltip as a fallback.
 *
 * @return The tooltip for this object.
 */
FText UField::GetToolTipText(bool bShortTooltip) const
{
	bool bFoundShortTooltip = false;
	static const FName NAME_Tooltip("Tooltip");
	static const FName NAME_ShortTooltip("ShortTooltip");
	FText LocalizedToolTip;
	FString NativeToolTip;

	if (bShortTooltip)
	{
		NativeToolTip = GetMetaData(NAME_ShortTooltip);
		if (NativeToolTip.IsEmpty())
		{
			NativeToolTip = GetMetaData(NAME_Tooltip);
		}
		else
		{
			bFoundShortTooltip = true;
		}
	}
	else
	{
		NativeToolTip = GetMetaData(NAME_Tooltip);
	}

	const FString Namespace = bFoundShortTooltip ? TEXT("UObjectShortTooltips") : TEXT("UObjectToolTips");
	const FString Key = GetFullGroupName(false);
	if ( !FText::FindTextInLiveTable_Advanced( Namespace, Key, /*OUT*/LocalizedToolTip, &NativeToolTip ) )
	{
		if (NativeToolTip.IsEmpty())
		{
			return GetDisplayNameText();
		}
		else if (!bShortTooltip && IsNative())
		{
			FormatNativeToolTip(NativeToolTip, true);
		}
		LocalizedToolTip = FText::AsLocalizable_Advanced(Namespace, Key, MoveTemp(NativeToolTip));
	}

	return LocalizedToolTip;
}

void UField::FormatNativeToolTip(FString& ToolTipString, bool bRemoveExtraSections)
{
	// First do doxygen replace
	static const FString DoxygenSee("@see");
	static const FString TooltipSee("See:");
	ToolTipString.ReplaceInline(*DoxygenSee, *TooltipSee);

	bool bCurrentLineIsEmpty = true;
	int32 EmptyLineCount = 0;
	int32 LastContentIndex = INDEX_NONE;
	const int32 ToolTipLength = ToolTipString.Len();

	// Start looking for empty lines and whitespace to strip
	for (int32 StrIndex = 0; StrIndex < ToolTipLength; StrIndex++)
	{
		TCHAR CurrentChar = ToolTipString[StrIndex];

		if (!FChar::IsWhitespace(CurrentChar))
		{
			if (FChar::IsPunct(CurrentChar))
			{
				// Punctuation is considered content if it's on a line with alphanumeric text
				if (!bCurrentLineIsEmpty)
				{
					LastContentIndex = StrIndex;
				}
			}
			else
			{
				// This is something alphanumeric, this is always content and mark line as not empty
				bCurrentLineIsEmpty = false;
				LastContentIndex = StrIndex;
			}
		}
		else if (CurrentChar == TEXT('\n'))
		{
			if (bCurrentLineIsEmpty)
			{
				EmptyLineCount++;
				if (bRemoveExtraSections && EmptyLineCount >= 2)
				{
					// If we get two empty or punctuation/separator lines in a row, cut off the string if requested
					break;
				}
			}
			else
			{
				EmptyLineCount = 0;
			}

			bCurrentLineIsEmpty = true;
		}
	}

	// Trim string to last content character, this strips trailing whitespace as well as extra sections if needed
	if (LastContentIndex >= 0 && LastContentIndex != ToolTipLength - 1)
	{
		ToolTipString.RemoveAt(LastContentIndex + 1, ToolTipLength - (LastContentIndex + 1));
	}
}

#endif // WITH_EDITORONLY_DATA

#if WITH_METADATA

/**
 * Determines if the property has any metadata associated with the key
 *
 * @param Key The key to lookup in the metadata
 * @return true if there is a (possibly blank) value associated with this key
 */
const FString* UField::FindMetaData(const TCHAR* Key) const
{
	UPackage* Package = GetOutermost();
	check(Package);

	FMetaData& MetaData = Package->GetMetaData();

	return MetaData.FindValue(this, Key);
}

const FString* UField::FindMetaData(const FName& Key) const
{
	UPackage* Package = GetOutermost();
	check(Package);

	FMetaData& MetaData = Package->GetMetaData();

	return MetaData.FindValue(this, Key);
}

/**
 * Find the metadata value associated with the key
 *
 * @param Key The key to lookup in the metadata
 * @return The value associated with the key
*/
const FString& UField::GetMetaData(const TCHAR* Key) const
{
	UPackage* Package = GetOutermost();
	check(Package);

	FMetaData& MetaData = Package->GetMetaData();

	const FString& MetaDataString = MetaData.GetValue(this, Key);

	return MetaDataString;
}

const FString& UField::GetMetaData(const FName& Key) const
{
	UPackage* Package = GetOutermost();
	check(Package);

	FMetaData& MetaData = Package->GetMetaData();

	const FString& MetaDataString = MetaData.GetValue(this, Key);

	return MetaDataString;
}

FText UField::GetMetaDataText(const TCHAR* MetaDataKey, const FTextKey LocalizationNamespace, const FTextKey LocalizationKey) const
{
	FString DefaultMetaData;

	if(const FString* FoundMetaData = FindMetaData( MetaDataKey ))
	{
		DefaultMetaData = *FoundMetaData;
	}

	// If attempting to grab the DisplayName metadata, we must correct the source string and output it as a DisplayString for lookup
	if( DefaultMetaData.IsEmpty() && FCString::Stricmp(MetaDataKey, TEXT("DisplayName")) == 0 )
	{
		DefaultMetaData = FName::NameToDisplayString(GetAuthoredName(), false);
	}

	FText LocalizedMetaData;
	if (!DefaultMetaData.IsEmpty())
	{
		LocalizedMetaData = FText::AsLocalizable_Advanced(LocalizationNamespace, LocalizationKey, MoveTemp(DefaultMetaData));
	}
	return LocalizedMetaData;
}

FText UField::GetMetaDataText(const FName& MetaDataKey, const FTextKey LocalizationNamespace, const FTextKey LocalizationKey) const
{
	FString DefaultMetaData;

	if (const FString* FoundMetaData = FindMetaData( MetaDataKey ))
	{
		DefaultMetaData = *FoundMetaData;
	}

	// If attempting to grab the DisplayName metadata, we must correct the source string and output it as a DisplayString for lookup
	if( DefaultMetaData.IsEmpty() && MetaDataKey == TEXT("DisplayName") )
	{
		DefaultMetaData = FName::NameToDisplayString(GetAuthoredName(), false);
	}

	FText LocalizedMetaData;
	if (!DefaultMetaData.IsEmpty())
	{
		LocalizedMetaData = FText::AsLocalizable_Advanced(LocalizationNamespace, LocalizationKey, MoveTemp(DefaultMetaData));
	}
	return LocalizedMetaData;
}

/**
 * Sets the metadata value associated with the key
 *
 * @param Key The key to lookup in the metadata
 * @return The value associated with the key
 */
void UField::SetMetaData(const TCHAR* Key, const TCHAR* InValue)
{
	UPackage* Package = GetOutermost();
	check(Package);

	Package->GetMetaData().SetValue(this, Key, InValue);
}

void UField::SetMetaData(const FName& Key, const TCHAR* InValue)
{
	UPackage* Package = GetOutermost();
	check(Package);

	Package->GetMetaData().SetValue(this, Key, InValue);
}

UClass* UField::GetClassMetaData(const TCHAR* Key) const
{
	const FString& ClassName = GetMetaData(Key);
	UClass* const FoundObject = UClass::TryFindTypeSlow<UClass>(ClassName);
	return FoundObject;
}

UClass* UField::GetClassMetaData(const FName& Key) const
{
	const FString& ClassName = GetMetaData(Key);
	UClass* const FoundObject = UClass::TryFindTypeSlow<UClass>(ClassName);
	return FoundObject;
}

void UField::RemoveMetaData(const TCHAR* Key)
{
	UPackage* Package = GetOutermost();
	check(Package);
	return Package->GetMetaData().RemoveValue(this, Key);
}

void UField::RemoveMetaData(const FName& Key)
{
	UPackage* Package = GetOutermost();
	check(Package);
	return Package->GetMetaData().RemoveValue(this, Key);
}

#endif // WITH_METADATA

bool UField::HasAnyCastFlags(const uint64 InCastFlags) const
{
	return !!(GetClass()->ClassCastFlags & InCastFlags);
}

bool UField::HasAllCastFlags(const uint64 InCastFlags) const
{
	return (GetClass()->ClassCastFlags & InCastFlags) == InCastFlags;
}

#if WITH_EDITORONLY_DATA
FField* UField::GetAssociatedFField()
{
	return nullptr;
}

void UField::SetAssociatedFField(FField* InField)
{
	check(false); // unsupported for this type
}
#endif // WITH_EDITORONLY_DATA



/*-----------------------------------------------------------------------------
	UStruct implementation.
-----------------------------------------------------------------------------*/

static bool PropertyTypeContainsStructOrEnum(UE::FPropertyTypeName Type)
{
	const FName Name = Type.GetName();
	if ((Name == NAME_StructProperty) ||
		(Name == NAME_EnumProperty) ||
		(Name == NAME_ByteProperty && Type.GetParameterCount() > 0))
	{
		return true;
	}
	const int32 Count = Type.GetParameterCount();
	for (int32 Index = 1; Index < Count; ++Index)
	{
		if (PropertyTypeContainsStructOrEnum(Type.GetParameter(Index)))
		{
			return true;
		}
	}
	// Handle parameter 0 at the end to give the compiler freedom to make this a tail call.
	return Count > 0 && PropertyTypeContainsStructOrEnum(Type.GetParameter(0));
}

#if WITH_EDITORONLY_DATA
static int32 GetNextFieldPathSerialNumber()
{
	static FThreadSafeCounter GlobalSerialNumberCounter;
	return GlobalSerialNumberCounter.Increment();
}
#endif // WITH_EDITORONLY_DATA

//
// Constructors.
//
UStruct::UStruct(EStaticConstructor, int32 InSize, int32 InMinAlignment, EObjectFlags InFlags)
	: UField(EC_StaticConstructor, InFlags)
	, SuperStruct(nullptr)
	, Children(nullptr)
	, ChildProperties(nullptr)
	, PropertiesSize(InSize)
	, MinAlignment(IntCastChecked<int16>(InMinAlignment))
	, PropertyLink(nullptr)
	, RefLink(nullptr)
	, DestructorLink(nullptr)
	, PostConstructLink(nullptr)
	, UnresolvedScriptProperties(nullptr)
{
#if WITH_EDITORONLY_DATA
	FieldPathSerialNumber = GetNextFieldPathSerialNumber();
	bHasAssetRegistrySearchableProperties = false;
#endif // WITH_EDITORONLY_DATA
}

UStruct::UStruct(UStruct* InSuperStruct, SIZE_T ParamsSize, SIZE_T Alignment)
	: UField(FObjectInitializer::Get())
	, SuperStruct(InSuperStruct)
	, Children(nullptr)
	, ChildProperties(nullptr)
	, PropertiesSize(ParamsSize ? IntCastChecked<int32>(ParamsSize) : (InSuperStruct ? InSuperStruct->GetPropertiesSize() : 0))
	, MinAlignment(IntCastChecked<int16>(Alignment ? Alignment : (FMath::Max(InSuperStruct ? InSuperStruct->GetMinAlignment() : 1, 1))))
	, PropertyLink(nullptr)
	, RefLink(nullptr)
	, DestructorLink(nullptr)
	, PostConstructLink(nullptr)
	, UnresolvedScriptProperties(nullptr)
{
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	this->ReinitializeBaseChainArray();
#endif
#if WITH_EDITORONLY_DATA
	FieldPathSerialNumber = GetNextFieldPathSerialNumber();
	bHasAssetRegistrySearchableProperties = false;
#endif // WITH_EDITORONLY_DATA
}

UStruct::UStruct(const FObjectInitializer& ObjectInitializer, UStruct* InSuperStruct, SIZE_T ParamsSize, SIZE_T Alignment)
	: UField(ObjectInitializer)
	, SuperStruct(InSuperStruct)
	, Children(nullptr)
	, ChildProperties(nullptr)
	, PropertiesSize(ParamsSize ? IntCastChecked<int32>(ParamsSize) : (InSuperStruct ? InSuperStruct->GetPropertiesSize() : 0))
	, MinAlignment(IntCastChecked<int16>(Alignment ? Alignment : (FMath::Max(InSuperStruct ? InSuperStruct->GetMinAlignment() : 1, 1))))
	, PropertyLink(nullptr)
	, RefLink(nullptr)
	, DestructorLink(nullptr)
	, PostConstructLink(nullptr)
	, UnresolvedScriptProperties(nullptr)
{
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	this->ReinitializeBaseChainArray();
#endif
#if WITH_EDITORONLY_DATA
	FieldPathSerialNumber = GetNextFieldPathSerialNumber();
	bHasAssetRegistrySearchableProperties = false;
#endif // WITH_EDITORONLY_DATA
}

#if !UE_WITH_CONSTINIT_UOBJECT
/**
 * Force any base classes to be registered first, then call BaseRegister
 */
void UStruct::RegisterDependencies()
{
	Super::RegisterDependencies();
	if (SuperStruct != NULL)
	{
		SuperStruct->RegisterDependencies();
	}
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

void UStruct::AddCppProperty(FProperty* Property)
{
	Property->Next = ChildProperties;
	ChildProperties = Property;
}

void UStruct::StaticLink(bool bRelinkExistingProperties)
{
	FNullArchive ArDummy;
	Link(ArDummy, bRelinkExistingProperties);
}

void UStruct::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	OutDeps.Add(SuperStruct);

	for (UField* Field = Children; Field; Field = Field->Next)
	{
		if (!Cast<UFunction>(Field))
		{
			OutDeps.Add(Field);
		}
	}

	for (FField* Field = ChildProperties; Field; Field = Field->Next)
	{
		Field->GetPreloadDependencies(OutDeps);
	}
}

void UStruct::CollectBytecodeReferencedObjects(TArray<UObject*>& OutReferencedObjects)
{
	// Collect references from byte code but exclude ourselves from the refs lists
	FArchiveScriptReferenceCollector ObjRefCollector(OutReferencedObjects, this);

	int32 BytecodeIndex = 0;
	while (BytecodeIndex < Script.Num())
	{
		SerializeExpr(BytecodeIndex, ObjRefCollector);
	}
}

void UStruct::CollectPropertyReferencedObjects(TArray<UObject*>& OutReferencedObjects)
{
	FPropertyReferenceCollector PropertyReferenceCollector(this, OutReferencedObjects);
	for (FField* CurrentField = ChildProperties; CurrentField; CurrentField = CurrentField->Next)
	{
		CurrentField->AddReferencedObjects(PropertyReferenceCollector);
	}
}

void UStruct::CollectBytecodeAndPropertyReferencedObjects()
{
	ScriptAndPropertyObjectReferences.Empty();
	CollectBytecodeReferencedObjects(MutableView(ScriptAndPropertyObjectReferences));
	CollectPropertyReferencedObjects(MutableView(ScriptAndPropertyObjectReferences));
}

void UStruct::CollectBytecodeAndPropertyReferencedObjectsRecursively()
{
	CollectBytecodeAndPropertyReferencedObjects();

	for (UField* Field = Children; Field; Field = Field->Next)
	{
		if (UStruct* ChildStruct = Cast<UStruct>(Field))
		{
			ChildStruct->CollectBytecodeAndPropertyReferencedObjectsRecursively();
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EPropertyVisitorControlFlow UStruct::Visit(void* InData, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorPath& /*Path*/, const FPropertyVisitorData& /*Data*/)> InFunc) const
{
	return Visit(InData, [&InFunc](const FPropertyVisitorContext& Context) -> EPropertyVisitorControlFlow
	{
		return InFunc(Context.Path, Context.Data);
	});
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

EPropertyVisitorControlFlow UStruct::Visit(void* InData, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc, FPropertyVisitorContext::EScope InScope) const
{
	FPropertyVisitorPath Path;
	FPropertyVisitorData Data(InData, /*ParentStructData*/nullptr);
	FPropertyVisitorContext Context(Path, Data, InScope);
	return Visit(Context, InFunc);
}

EPropertyVisitorControlFlow UStruct::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const
{
	EPropertyVisitorControlFlow RetVal = EPropertyVisitorControlFlow::StepOver;
	const bool bObjectRefsOnly = Context.Scope == FPropertyVisitorContext::EScope::ObjectRefs;
	for (const FProperty* Property = bObjectRefsOnly ? RefLink : PropertyLink; Property; Property = bObjectRefsOnly ? Property->NextRef : Property->PropertyLinkNext)
	{
		RetVal = PropertyVisitorHelpers::VisitProperty(this, Property, Context, InFunc);
		if (RetVal == EPropertyVisitorControlFlow::Stop)
		{
			return EPropertyVisitorControlFlow::Stop;
		}
		if (RetVal == EPropertyVisitorControlFlow::StepOut)
		{
			return EPropertyVisitorControlFlow::StepOver;
		}
	}
	return RetVal;
}

void* UStruct::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	if (Info.ParentStructType) // We don't care if this matches, but we do care that it's set as it identifies a UStruct info
	{
		if (const UStruct* PropertyOwnerStruct = Info.Property->GetOwnerStruct();
			PropertyOwnerStruct && IsChildOf(PropertyOwnerStruct))
		{
			return Info.Property->ContainerPtrToValuePtr<void>(Data, Info.PropertyInfo == EPropertyVisitorInfoType::StaticArrayIndex ? Info.Index : 0);
		}
	}

	return nullptr;
}

void UStruct::PreloadChildren(FArchive& Ar)
{
	for (UField* Field = Children; Field; Field = Field->Next)
	{
		Ar.Preload(Field);
	}
}

void UStruct::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
#if WITH_EDITORONLY_DATA
	// In the editor structures can be relinked to accomdate changes elsewhere in the struct hierarchy (mostly
	// due to blueprint compilation, but also from package reload, live coding, and other blueprint like systems
	// such as property bags, control rig, or third party scripting solutions)
	DestroyUnversionedSchema(this);
#endif

	if (bRelinkExistingProperties)
	{
		// Preload everything before we calculate size, as the preload may end up recursively linking things
		UStruct* InheritanceSuper = GetInheritanceSuper();
		if (Ar.IsLoading())
		{
			if (InheritanceSuper)
			{
				Ar.Preload(InheritanceSuper);
			}

			PreloadChildren(Ar);

#if WITH_EDITORONLY_DATA
			ConvertUFieldsToFFields();
#endif // WITH_EDITORONLY_DATA
		}

	#if WITH_EDITORONLY_DATA
		TotalFieldCount = 0;
	#endif
		PropertiesSize = 0;
		MinAlignment = 1;

		if (InheritanceSuper)
		{
		#if WITH_EDITORONLY_DATA
			TotalFieldCount = InheritanceSuper->TotalFieldCount;
		#endif
			PropertiesSize = InheritanceSuper->GetPropertiesSize();
			MinAlignment = IntCastChecked<int16>(InheritanceSuper->GetMinAlignment());
		}

		for (FField* Field = ChildProperties; Field; Field = Field->Next)
		{
			if (Field->GetOwner<UObject>() != this)
			{
				break;
			}

			if (FProperty* Property = CastField<FProperty>(Field))
			{
			#if !WITH_EDITORONLY_DATA
				// If we don't have the editor, make sure we aren't trying to link properties that are editor only.
				check(!Property->IsEditorOnlyProperty());
			#endif // WITH_EDITORONLY_DATA
				ensureMsgf(Property->GetOwner<UObject>() == this, TEXT("Linking '%s'. Property '%s' has outer '%s'"),
					*GetFullName(), *Property->GetName(), *Property->GetOwnerVariant().GetFullName());

				PropertiesSize = Property->Link(Ar);

			#if WITH_EDITORONLY_DATA
				Property->SetIndexInOwner(TotalFieldCount);
				TotalFieldCount += Property->ArrayDim;
			#endif

				MinAlignment = IntCastChecked<int16>(FMath::Max(MinAlignment, Property->GetMinAlignment()));
			}
		}

		bool bHandledWithCppStructOps = false;
		if (GetClass()->IsChildOf(UScriptStruct::StaticClass()))
		{
			// check for internal struct recursion via arrays
			for (FField* Field = ChildProperties; Field; Field = Field->Next)
			{
				FArrayProperty* ArrayProp = CastField<FArrayProperty>(Field);
				if (ArrayProp != NULL)
				{
					FStructProperty* StructProp = CastField<FStructProperty>(ArrayProp->Inner);
					if (StructProp != NULL && StructProp->Struct == this)
					{
						//we won't support this, too complicated
						UE_LOGF(LogClass, Fatal, "'Struct recursion via arrays is unsupported for properties.");
					}
				}
			}

			UScriptStruct& ScriptStruct = dynamic_cast<UScriptStruct&>(*this);
			ScriptStruct.PrepareCppStructOps();

			if (UScriptStruct::ICppStructOps* CppStructOps = ScriptStruct.GetCppStructOps())
			{
				MinAlignment = IntCastChecked<int16>(CppStructOps->GetAlignment());
				PropertiesSize = CppStructOps->GetSize();
				bHandledWithCppStructOps = true;
			}
		}
	}
	else
	{
	#if WITH_EDITORONLY_DATA
		TotalFieldCount = 0;
		if (UStruct* InheritanceSuper = GetInheritanceSuper())
		{
			TotalFieldCount = InheritanceSuper->TotalFieldCount;
		}
	#endif

		for (FField* Field = ChildProperties; (Field != NULL) && (Field->GetOwner<UObject>() == this); Field = Field->Next)
		{
			if (FProperty* Property = CastField<FProperty>(Field))
			{
				Property->LinkWithoutChangingOffset(Ar);

			#if WITH_EDITORONLY_DATA
				Property->SetIndexInOwner(TotalFieldCount);
				TotalFieldCount += Property->ArrayDim;
			#endif
			}
		}
	}

	if (GetOutermost()->GetFName() == GLongCoreUObjectPackageName)
	{
		FName ToTest = GetFName();
		if ( ToTest == NAME_Matrix )
		{
			check(MinAlignment == alignof(FMatrix));
			check(PropertiesSize == sizeof(FMatrix));
		}
		else if ( ToTest == NAME_Plane )
		{
			check(MinAlignment == alignof(FPlane));
			check(PropertiesSize == sizeof(FPlane));
		}
		else if ( ToTest == NAME_Vector4 )
		{
			check(MinAlignment == alignof(FVector4));
			check(PropertiesSize == sizeof(FVector4));
		}
		else if ( ToTest == NAME_Quat )
		{
			check(MinAlignment == alignof(FQuat));
			check(PropertiesSize == sizeof(FQuat));
		}
		else if ( ToTest == NAME_Double )
		{
			check(MinAlignment == alignof(double));
			check(PropertiesSize == sizeof(double));
		}
		else if ( ToTest == NAME_Color )
		{
			check(MinAlignment == alignof(FColor));
			check(PropertiesSize == sizeof(FColor));
#if !PLATFORM_LITTLE_ENDIAN
			// Object.h declares FColor as BGRA which doesn't match up with what we'd like to use on
			// Xenon to match up directly with the D3D representation of D3DCOLOR. We manually fiddle
			// with the property offsets to get everything to line up.
			// In any case, on big-endian systems we want to byte-swap this.
			//@todo cooking: this should be moved into the data cooking step.
			{
				FProperty*	ColorComponentEntries[4];
				uint32		ColorComponentIndex = 0;

				for( UField* Field=Children; Field && Field->GetOuter()==this; Field=Field->Next )
				{
					FProperty* Property = CastFieldChecked<FProperty>( Field );
					ColorComponentEntries[ColorComponentIndex++] = Property;
				}
				check( ColorComponentIndex == 4 );

				Exchange( ColorComponentEntries[0]->Offset, ColorComponentEntries[3]->Offset );
				Exchange( ColorComponentEntries[1]->Offset, ColorComponentEntries[2]->Offset );
			}
#endif

		}
	}


	// Link the references, structs, and arrays for optimized cleanup.
	// Note: Could optimize further by adding FProperty::NeedsDynamicRefCleanup, excluding things like arrays of ints.
	UEProperty_Private::FPropertyListBuilderPropertyLink PropertyLinkBuilder(&PropertyLink);
	UEProperty_Private::FPropertyListBuilderDestructorLink DestructorLinkBuilder(&DestructorLink);
	UEProperty_Private::FPropertyListBuilderRefLink RefLinkBuilder(&RefLink);
	UEProperty_Private::FPropertyListBuilderPostConstructLink PostConstructLinkBuilder(&PostConstructLink);

	TArray<const FStructProperty*> EncounteredStructProps;
	for (TFieldIterator<FProperty> It(this); It; ++It)
	{
		FProperty* Property = *It;

		// Determine which lists we add the property to.
		//
		// The relationship between a property and the owning class is more complex that might first
		// appear.
		//
		// For UHT generated types, UFunction are owned by a UClass.  UScriptStructs are not
		// owned by a class and will default to needing a LinkDestructor. For blueprint types,
		// UScriptStructs can be owned and will get link flags from the owning UClass.
		EStructPropertyLinkFlags PropertyLinkFlags = EStructPropertyLinkFlags::LinkDestructor;
		{
			if (UClass* OwnerClass = Property->GetOwnerClass())
			{
				PropertyLinkFlags = OwnerClass->GetPropertyLinkFlags(this, Property);
			}
			if (Property->ContainsFinishDestroy(EncounteredStructProps))
			{
				PropertyLinkFlags |= EStructPropertyLinkFlags::LinkDestructor;
			}
			else if (Property->HasAnyPropertyFlags(CPF_IsPlainOldData | CPF_NoDestructor))
			{
				PropertyLinkFlags &= ~EStructPropertyLinkFlags::LinkDestructor;
			}
		}

		// Ref link contains any properties which contain object references including types with user-defined serializers which don't explicitly specify whether they
		// contain object references
		if (Property->ContainsObjectReference(EncounteredStructProps, EPropertyObjectReferenceType_Any))
		{
			RefLinkBuilder.AppendNoTerminate(*Property);
		}

		// These properties will be destructed via the property implementation.  Things in a struct that need a destructor will still be in here,
		// even though in many cases they will also be destroyed by a native destructor on the whole struct
		if ((PropertyLinkFlags & (EStructPropertyLinkFlags::LinkDestructor | EStructPropertyLinkFlags::NeverLinkDestructor_Internal)) == EStructPropertyLinkFlags::LinkDestructor)
		{
			DestructorLinkBuilder.AppendNoTerminate(*Property);
		}

		// Link references to properties that require their values to be initialized and/or copied from CDO post-construction. Note that this includes all non-native-class-owned properties.
		if (EnumHasAnyFlags(PropertyLinkFlags, EStructPropertyLinkFlags::LinkPostConstruct))
		{
			PostConstructLinkBuilder.AppendNoTerminate(*Property);
		}

#if WITH_EDITORONLY_DATA
		// Set the bHasAssetRegistrySearchableProperties flag.
		// Note that we're also iterating over super class properties here so this flag is being automatically inherited
		bHasAssetRegistrySearchableProperties |= Property->HasAnyPropertyFlags(CPF_AssetRegistrySearchable);
#endif

		PropertyLinkBuilder.AppendNoTerminate(*Property);
	}

	PropertyLinkBuilder.NullTerminate();
	DestructorLinkBuilder.NullTerminate();
	RefLinkBuilder.NullTerminate();
	PostConstructLinkBuilder.NullTerminate();

	{
		// Now collect all references from FProperties to UObjects and store them in GC-exposed array for fast access
		CollectPropertyReferencedObjects(MutableView(ScriptAndPropertyObjectReferences));

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
		// The old (non-EDL) FLinkerLoad code paths create placeholder objects
		// for classes and functions. We have to babysit these, just as we do
		// for bytecode references (reusing the AddReferencingScriptExpr fn).
		// Long term we should not use placeholder objects like this:
		for(int32 ReferenceIndex = ScriptAndPropertyObjectReferences.Num() - 1; ReferenceIndex >= 0; --ReferenceIndex)
		{
			if (ScriptAndPropertyObjectReferences[ReferenceIndex])
			{
				if (ULinkerPlaceholderClass* PlaceholderObj = Cast<ULinkerPlaceholderClass>(ScriptAndPropertyObjectReferences[ReferenceIndex]))
				{
					// let the placeholder track the reference to it:
					PlaceholderObj->AddReferencingScriptExpr(reinterpret_cast<UClass**>(&ScriptAndPropertyObjectReferences[ReferenceIndex]));
				}
				// I don't currently see how placeholder functions could be present in this list, but that's
				// a dangerous assumption.
				ensure(!(ScriptAndPropertyObjectReferences[ReferenceIndex]->IsA<ULinkerPlaceholderFunction>()));
			}
			else
			{
				// It's possible that in the process of recompilation one of the refernces got GC'd leaving a null ptr in the array
				ScriptAndPropertyObjectReferences.RemoveAt(ReferenceIndex);
			}
		}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	}

#if WITH_EDITORONLY_DATA
	// Discard old wrapper objects used by property grids
	for (UPropertyWrapper* Wrapper : PropertyWrappers)
	{
		Wrapper->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional| REN_AllowPackageLinkerMismatch);
		Wrapper->RemoveFromRoot();
	}
	PropertyWrappers.Empty();
#endif

	StructStateFlags.fetch_or(static_cast<uint16>(EStructStateFlags::PropertyDataAvailable), std::memory_order_release);
}

void UStruct::InitializeStruct(void* InDest, int32 ArrayDim/* = 1*/) const
{
	uint8 *Dest = (uint8*)InDest;
	check(Dest);

	int32 Stride = GetStructureSize();

	//@todo UE optimize
	FMemory::Memzero(Dest, 1 * Stride);

	for (FProperty* Property = PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (ensure(Property->IsInContainer(Stride)))
		{
			for (int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++)
			{
				Property->InitializeValue_InContainer(Dest + ArrayIndex * Stride);
			}
		}
		else
		{
			break;
		}
	}
}

void UStruct::DestroyStruct(void* Dest, int32 ArrayDim) const
{
	uint8 *Data = (uint8*)Dest;
	int32 Stride = GetStructureSize();

	bool bHitBase = false;
	for (FProperty* P = DestructorLink; P  && !bHitBase; P = P->DestructorLinkNext)
	{
		if (!P->HasAnyPropertyFlags(CPF_NoDestructor))
		{
			if (P->IsInContainer(Stride))
			{
				for ( int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++ )
				{
					P->DestroyValue_InContainer(Data + ArrayIndex * Stride);
				}
			}
		}
		else
		{
			bHitBase = true;
		}
	}
}

#if DO_CHECK

bool UStruct::DebugIsPropertyChainReady() const
{
	return HasAllStructStateFlags(EStructStateFlags::PropertyDataAvailable);
}

#endif // DO_CHECK

//
// Serialize all of the class's data that belongs in a particular
// bin and resides in Data.
//
void UStruct::SerializeBin(FStructuredArchive::FSlot Slot, TNotNull<void*> Data) const
{
#if WITH_EDITORONLY_DATA
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	const bool bSaveSerializedPropertyPath = IsA<UClass>() && SerializeContext && !SerializeContext->SerializedPropertyPath.IsEmpty();
	UE::FPropertyPathName PrevSerializedPropertyPath;

	if (bSaveSerializedPropertyPath)
	{
		PrevSerializedPropertyPath = MoveTemp(SerializeContext->SerializedPropertyPath);
		SerializeContext->SerializedPropertyPath.Reset();
	}
#endif

	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	FStructuredArchive::FStream PropertyStream = Slot.EnterStream();

	// RefLink contains Strong, Weak and Soft object references including FSoftObjectPath
	// If objects wish to serialize other properties they should not set ArIsObjectReferenceCollector
	if (UnderlyingArchive.IsObjectReferenceCollector())
	{
		// The FProperty instance might start in the middle of a cache line
		static constexpr uint32 ExtraPrefetchBytes = PLATFORM_CACHE_LINE_SIZE - /* min alignment */ 16;
		// Prefetch vtable, PropertyFlags and NextRef. NextRef comes last.
		static constexpr uint32 PropertyPrefetchBytes = offsetof(FProperty, NextRef) + ExtraPrefetchBytes;
		FPlatformMisc::PrefetchBlock(RefLink, PropertyPrefetchBytes);
		for( FProperty* RefLinkProperty=RefLink; RefLinkProperty!=NULL; RefLinkProperty=RefLinkProperty->NextRef )
		{
			FPlatformMisc::PrefetchBlock(RefLinkProperty->NextRef, PropertyPrefetchBytes);
			RefLinkProperty->SerializeBinProperty(PropertyStream.EnterElement(), Data );
		}
	}
	else if( UnderlyingArchive.ArUseCustomPropertyList )
	{
		const FCustomPropertyListNode* CustomPropertyList = UnderlyingArchive.ArCustomPropertyList;
		for (auto PropertyNode = CustomPropertyList; PropertyNode; PropertyNode = PropertyNode->PropertyListNext)
		{
			FProperty* Property = PropertyNode->Property;
			if( Property )
			{
				// Temporarily set to the sub property list, in case we're serializing a UStruct property.
				UnderlyingArchive.ArCustomPropertyList = PropertyNode->SubPropertyList;

				Property->SerializeBinProperty(PropertyStream.EnterElement(), Data, PropertyNode->ArrayIndex);

				// Restore the original property list.
				UnderlyingArchive.ArCustomPropertyList = CustomPropertyList;
			}
		}
	}
	else
	{
		for (FProperty* Property = PropertyLink; Property != NULL; Property = Property->PropertyLinkNext)
		{
			Property->SerializeBinProperty(PropertyStream.EnterElement(), Data);
		}
	}

#if WITH_EDITORONLY_DATA
	if (bSaveSerializedPropertyPath)
	{
		SerializeContext->SerializedPropertyPath = MoveTemp(PrevSerializedPropertyPath);
	}
#endif
}

void UStruct::SerializeBinEx( FStructuredArchive::FSlot Slot, TNotNull<void*> Data, void const* DefaultData, UStruct* DefaultStruct ) const
{
	if ( !DefaultData || !DefaultStruct )
	{
		SerializeBin(Slot, Data);
		return;
	}

#if WITH_EDITORONLY_DATA
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	const bool bSaveSerializedPropertyPath = IsA<UClass>() && SerializeContext && !SerializeContext->SerializedPropertyPath.IsEmpty();
	UE::FPropertyPathName PrevSerializedPropertyPath;

	if (bSaveSerializedPropertyPath)
	{
		PrevSerializedPropertyPath = MoveTemp(SerializeContext->SerializedPropertyPath);
		SerializeContext->SerializedPropertyPath.Reset();
	}
#endif

	for( TFieldIterator<FProperty> It(this); It; ++It )
	{
		It->SerializeNonMatchingBinProperty(Slot, Data, DefaultData, DefaultStruct);
	}

#if WITH_EDITORONLY_DATA
	if (bSaveSerializedPropertyPath)
	{
		SerializeContext->SerializedPropertyPath = MoveTemp(PrevSerializedPropertyPath);
	}
#endif
}

void UStruct::LoadTaggedPropertiesFromText(FStructuredArchive::FSlot Slot, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	const bool bUseRedirects = !FPlatformProperties::RequiresCookedData() || UnderlyingArchive.IsSaveGame();
	int32 NumProperties = 0;
	FStructuredArchiveMap PropertiesMap = Slot.EnterMap(NumProperties);

	for (int32 PropertyIndex = 0; PropertyIndex < NumProperties; ++PropertyIndex)
	{
		FString PropertyNameString;
		FStructuredArchiveSlot PropertySlot = PropertiesMap.EnterElement(PropertyNameString);
		FName PropertyName = *PropertyNameString;

		// If this property has a guid attached then we need to resolve it to the right name before we start loading
		TOptional<FStructuredArchiveSlot> PropertyGuidSlot = PropertySlot.TryEnterAttribute(TEXT("PropertyGuid"), false);
		if (PropertyGuidSlot.IsSet())
		{
			FGuid PropertyGuid;
			PropertyGuidSlot.GetValue() << PropertyGuid;
			if (PropertyGuid.IsValid())
			{
				FName NewName = FindPropertyNameFromGuid(PropertyGuid);
				if (NewName != NAME_None)
				{
					PropertyName = NewName;
				}
			}
		}

		// Resolve any redirects if necessary
		if (bUseRedirects && !UnderlyingArchive.HasAnyPortFlags(PPF_DuplicateForPIE | PPF_Duplicate))
		{
			for (UStruct* CheckStruct : GetOwnerStruct()->GetSuperStructIterator())
			{
				FName NewTagName = FProperty::FindRedirectedPropertyName(CheckStruct, PropertyName);
				if (!NewTagName.IsNone())
				{
					PropertyName = NewTagName;
					break;
				}
			}
		}

		// Now we know what the property name is, we can try and load it
		FProperty* Property = FindPropertyByName(PropertyName);

		if (Property == nullptr)
		{
			Property = CustomFindProperty(PropertyName);
		}

		if (Property && Property->ShouldSerializeValue(UnderlyingArchive))
		{
			// Static arrays of tagged properties are special cases where the slot is always an array with no tag data attached. We currently have no TryEnterArray we can't
			// react based on what is in the file (yet) so we'll just have to assume that nobody converts a property from an array to a single value and go with whatever
			// the code property tells us.
			TOptional<FStructuredArchiveArray> SlotArray;
			int32 NumItems = Property->ArrayDim;
			if (Property->ArrayDim > 1)
			{
				int32 NumAvailableItems = 0;
				SlotArray.Emplace(PropertySlot.EnterArray(NumAvailableItems));
				NumItems = FMath::Min(Property->ArrayDim, NumAvailableItems);
			}

			for (int32 ItemIndex = 0; ItemIndex < NumItems; ++ItemIndex)
			{
				TOptional<FStructuredArchiveSlot> ItemSlot;
				if (SlotArray.IsSet())
				{
					ItemSlot.Emplace(SlotArray->EnterElement());
				}
				else
				{
					ItemSlot.Emplace(PropertySlot);
				}

				FPropertyTag Tag;
				ItemSlot.GetValue() << Tag;
				Tag.SetProperty(Property);
				Tag.ArrayIndex = ItemIndex;
				Tag.Name = PropertyName;

				if (Tag.SerializeType == EPropertyTagSerializeType::Skipped)
				{
				#if WITH_EDITORONLY_DATA
					if (SerializeContext->bTrackInitializedProperties && (Tag.OverrideOperation != EOverriddenPropertyOperation::SubObjectsShadowing))
					{
						UE::FInitializedPropertyValueState(this, Data).Set(Property, ItemIndex);
					}
				#endif
					continue;
				}

				if (bUseRedirects)
				{
					if (UE::FPropertyTypeName NewTypeName = ApplyRedirectsToPropertyType(Tag.GetType(), Property); !NewTypeName.IsEmpty())
					{
						Tag.SetType(NewTypeName);
					}
				}

				if (BreakRecursionIfFullyLoad && BreakRecursionIfFullyLoad->HasAllFlags(RF_LoadCompleted))
				{
					continue;
				}

				bool bSerialized = false;

				switch (Property->ConvertFromType(Tag, ItemSlot.GetValue(), Data, DefaultsStruct, Defaults))
				{
				case EConvertFromTypeResult::Converted:
				case EConvertFromTypeResult::Serialized:
					bSerialized = true;
					break;

				case EConvertFromTypeResult::UseSerializeItem:
					if (const FName PropID = Property->GetID(); Tag.Type != PropID)
					{
						UE_LOGF(LogClass, Warning, "Type mismatch in %ls of %ls - Previous (%ls) Current(%ls) in package: %ls",
							*WriteToString<32>(Tag.Name), *WriteToString<32>(GetFName()),
							*WriteToString<32>(Tag.Type), *WriteToString<32>(PropID),
							*UnderlyingArchive.GetArchiveName());
					}
					else
					{
						uint8* DestAddress = Property->ContainerPtrToValuePtr<uint8>(Data, Tag.ArrayIndex);
						const uint8* DefaultsFromParent = Property->ContainerPtrToValuePtrForDefaults<uint8>(DefaultsStruct, Defaults, Tag.ArrayIndex);

						// This property is ok.
						Tag.SerializeTaggedProperty(ItemSlot.GetValue(), Property, DestAddress, DefaultsFromParent);
						bSerialized = true;
					}
					break;

				case EConvertFromTypeResult::CannotConvert:
					break;

				default:
					check(false);
				}

			#if WITH_EDITORONLY_DATA
				if (bSerialized)
				{
					if ((Tag.OverrideOperation != EOverriddenPropertyOperation::SubObjectsShadowing))
					{
						if (SerializeContext->bTrackInitializedProperties)
						{
							UE::FInitializedPropertyValueState(this, Data).Set(Property, ItemIndex);
						}
						if (SerializeContext->bTrackSerializedProperties)
						{
							UE::FSerializedPropertyValueState(this, Data).Set(Property, ItemIndex);
						}
					}
				}
			#endif
			}
		}
	}
}

void UStruct::SerializeTaggedProperties(FStructuredArchive::FSlot Slot, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad) const
{
#if WITH_EDITORONLY_DATA
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	const bool bSaveSerializedPropertyPath = IsA<UClass>() && SerializeContext && !SerializeContext->SerializedPropertyPath.IsEmpty();
	UE::FPropertyPathName PrevSerializedPropertyPath;

	if (bSaveSerializedPropertyPath)
	{
		PrevSerializedPropertyPath = MoveTemp(SerializeContext->SerializedPropertyPath);
		SerializeContext->SerializedPropertyPath.Reset();
	}
#endif

	if (Slot.GetArchiveState().UseUnversionedPropertySerialization())
	{
		SerializeUnversionedProperties(this, Slot, Data, DefaultsStruct, Defaults);
	}
	else
	{
		SerializeVersionedTaggedProperties(Slot, Data, DefaultsStruct, Defaults, BreakRecursionIfFullyLoad);
	}

#if WITH_EDITORONLY_DATA
	if (bSaveSerializedPropertyPath)
	{
		SerializeContext->SerializedPropertyPath = MoveTemp(PrevSerializedPropertyPath);
	}
#endif
}

#if WITH_EDITORONLY_DATA
const FBlake3Hash& UStruct::GetSchemaHash(bool bSkipEditorOnly) const
{
	return ::GetSchemaHash(this, bSkipEditorOnly);
}
#endif


/**
 *  Enum flags that indicate that additional data may be serialized prior to actual tagged property serialization
 *	Those extensions are used to store additional function to control how TPS will resolved. i.e use overridable serialization for example
 *	Registered flag should be serialized in ascending order
 *  @Note: do not use lightly
 */
enum class EClassSerializationControlExtension : uint8
{
	NoExtension					= 0x00,
	ReserveForFutureUse			= 0x01, // Can be use to add a next group of extension

	////////////////////////////////////////////////
	// First extension group
	OverridableSerializationInformation	= 0x02,

	//
	// Add more extension for the first group here
	//
};
ENUM_CLASS_FLAGS(EClassSerializationControlExtension);

namespace UE::OS
{
	static bool GUseForCooking = false;
	static FAutoConsoleVariableRef CVarOSUseForCooking(TEXT("OS.UseForCooking"), GUseForCooking, TEXT("Use overridable serialization for cooking else fall back on delta serialization"));
} // UE::OS

struct FSerializationControlExtensionContext
{
	FArchive& UnderlyingArchive;
	uint8* Data = nullptr;
	bool bEnableOverridableSerialization = false;
	FOverriddenPropertySet* OverriddenProperties = nullptr;

	EClassSerializationControlExtension InitializeSerializationControlExtensions()
	{
		EClassSerializationControlExtension SerializationExtension = EClassSerializationControlExtension::NoExtension;

		// Let's not enable OS(Overridable Serialization) when cooking and fall back on DS(Delta Serialization).
		// That way the result will be similar if the server is using TPS(Tag Property Serialization) and the client is using UPS(Unversion Property Serialization)
		// This fixes the problem CDO divergences in network layer with the server having -0.0 with OS and the client having 0.0 using delta serialization
		// as mathematically (-0.0 == 0.0) but is memcmp(&-0.0, &0.0) is false
		if (UE::OS::GUseForCooking || !UnderlyingArchive.IsCooking())
		{
			// Overridable serialization information initialization
			if (FOverriddenPropertySet* ObjectOverriddenProperties = FOverridableManager::Get().GetOverriddenProperties((UObject*)Data))
			{
				SerializationExtension |= EClassSerializationControlExtension::OverridableSerializationInformation;
				bEnableOverridableSerialization = true;
				OverriddenProperties = ObjectOverriddenProperties;
			}
		}

		return SerializationExtension;
	}
};

bool UStruct::HasAllStructStateFlags(EStructStateFlags TestStructStateFlags) const
{
	return EnumHasAllFlags(static_cast<EStructStateFlags>(StructStateFlags.load(std::memory_order_acquire)), TestStructStateFlags);
}

void UStruct::SerializeVersionedTaggedProperties(FStructuredArchive::FSlot Slot, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad) const
{
	using namespace UE;

	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	//SCOPED_LOADTIMER(SerializeTaggedPropertiesTime);

	// Setup serialization control data extensions, this is serialized only on root i.e. UObject and not structs!
	const bool bIsUClass = IsA<UClass>();
	FSerializationControlExtensionContext ControlContext{ UnderlyingArchive, Data };
	if (bIsUClass && UnderlyingArchive.UEVer() >= EUnrealEngineObjectUE5Version::PROPERTY_TAG_EXTENSION_AND_OVERRIDABLE_SERIALIZATION)
	{
		EClassSerializationControlExtension SerializationControl = EClassSerializationControlExtension::NoExtension;
		if (UnderlyingArchive.IsSaving())
		{
			SerializationControl = ControlContext.InitializeSerializationControlExtensions();
		}

		Slot << SA_ATTRIBUTE(TEXT("SerializationControlExtensions"), SerializationControl);

		// Overridable serialization information serialization
		if (EnumHasAnyFlags(SerializationControl, EClassSerializationControlExtension::OverridableSerializationInformation))
		{
			checkf(!UnderlyingArchive.ArUseCustomPropertyList, TEXT("Overridable serialization does not support custom property list"))
			EOverriddenPropertyOperation Operation = UnderlyingArchive.IsSaving() ? ControlContext.OverriddenProperties->GetOverriddenPropertyOperation((FArchiveSerializedPropertyChain*)nullptr, (FProperty*)nullptr) : EOverriddenPropertyOperation::None;
			Slot << SA_ATTRIBUTE(TEXT("OverridableOperation"), Operation);

			if (UnderlyingArchive.IsLoading())
			{
				ControlContext.bEnableOverridableSerialization = true;
				// Overridden values are saved independently in transaction, so do no need to restore them here.
				if (!UnderlyingArchive.IsTransacting())
				{
					ControlContext.OverriddenProperties = FOverridableManager::Get().RestoreOverrideOperation((UObject*)NotNullGet(Data), Operation, /*bNeedsSubobjectTemplateInstantiation*/true, UnderlyingArchive.ArMergeOverrides);
				}
			}
		}
	}

	// Scope that enables the overridable serialization for this object
	FEnableOverridableSerializationScope OverridableSerializationScope(ControlContext.bEnableOverridableSerialization, ControlContext.OverriddenProperties);

	// Determine if this struct supports optional property guid's (UBlueprintGeneratedClasses Only)
	const bool bArePropertyGuidsAvailable = (UnderlyingArchive.UEVer() >= VER_UE4_PROPERTY_GUID_IN_PROPERTY_TAG) && (!FPlatformProperties::RequiresCookedData() || UnderlyingArchive.IsSaveGame()) && ArePropertyGuidsAvailable();
	const bool bUseRedirects = (!FPlatformProperties::RequiresCookedData() || UnderlyingArchive.IsSaveGame()) && !UnderlyingArchive.IsLoadingFromCookedPackage();

	if (UnderlyingArchive.IsLoading())
	{
#if WITH_TEXT_ARCHIVE_SUPPORT
		if (UnderlyingArchive.IsTextFormat())
		{
			LoadTaggedPropertiesFromText(Slot, Data, DefaultsStruct, Defaults, BreakRecursionIfFullyLoad);
		}
		else
#endif // WITH_TEXT_ARCHIVE_SUPPORT
		{
			// Track whether the unknown property tree has been looked up.
			TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree;
			bool bSearchedForUnknownPropertyTree = false;

			// Load tagged properties.
			FStructuredArchive::FStream PropertiesStream = Slot.EnterStream();

			// This code assumes that properties are loaded in the same order they are saved in. This removes a n^2 search
			// and makes it an O(n) when properties are saved in the same order as they are loaded (default case). In the
			// case that a property was reordered the code falls back to a slower search.
			FProperty*	Property = PropertyLink;
			bool		bAdvanceProperty	= false;
			int32		RemainingArrayDim	= Property ? Property->ArrayDim : 0;

			// Load all stored properties, potentially skipping unknown ones.
			while (!UnderlyingArchive.IsCriticalError())
			{
				FStructuredArchive::FRecord PropertyRecord = PropertiesStream.EnterElement().EnterRecord();

				FPropertyTag Tag;
				PropertyRecord << SA_VALUE(TEXT("Tag"), Tag);

				if (Tag.Name.IsNone())
				{
					break;
				}

				// Move to the next property to be serialized
				if (bAdvanceProperty && --RemainingArrayDim <= 0)
				{
					Property = Property->PropertyLinkNext;
					// Skip over properties that don't need to be serialized.
					while (Property && !Property->ShouldSerializeValue(UnderlyingArchive))
					{
						Property = Property->PropertyLinkNext;
					}
					RemainingArrayDim = Property ? Property->ArrayDim : 0;
				}
				bAdvanceProperty = false;

				// Optionally resolve properties using Guid Property tags if the class supports it.
				if (bArePropertyGuidsAvailable && Tag.HasPropertyGuid)
				{
					// Use property guids from blueprint generated classes to redirect serialised data.
					FName Result = FindPropertyNameFromGuid(Tag.PropertyGuid);
					if (Result != NAME_None && Tag.Name != Result)
					{
						Tag.Name = Result;
					}
				}
				// If this property is not the one we expect (e.g. skipped as it matches the default value), do the brute force search.
				if (Property == nullptr || Property->GetFName() != Tag.Name)
				{
					// No need to check redirects on platforms where everything is cooked. Always check for save games
					if (bUseRedirects && !UnderlyingArchive.HasAnyPortFlags(PPF_DuplicateForPIE | PPF_Duplicate))
					{
						for (UStruct* CheckStruct : GetOwnerStruct()->GetSuperStructIterator())
						{
							FName NewTagName = FProperty::FindRedirectedPropertyName(CheckStruct, Tag.Name);
							if (!NewTagName.IsNone())
							{
								Tag.Name = NewTagName;
								break;
							}
						}
					}

					FProperty* CurrentProperty = Property;
					// Search forward...
					for (; Property; Property = Property->PropertyLinkNext)
					{
						if (Property->GetFName() == Tag.Name)
						{
							break;
						}
					}
					// ... and then search from the beginning till we reach the current property if it's not found.
					if (Property == nullptr)
					{
						for (Property = PropertyLink; Property && Property != CurrentProperty; Property = Property->PropertyLinkNext)
						{
							if (Property->GetFName() == Tag.Name)
							{
								break;
							}
						}

						if (Property == CurrentProperty)
						{
							// Property wasn't found.
							Property = nullptr;
						}
					}

					RemainingArrayDim = Property ? Property->ArrayDim : 0;
				}

				const int64 StartOfProperty = UnderlyingArchive.Tell();

				if (!Property)
				{
					Property = CustomFindProperty(Tag.Name);
				}

				if (Tag.SerializeType == EPropertyTagSerializeType::Skipped)
				{
				#if WITH_EDITORONLY_DATA
					if (SerializeContext->bTrackInitializedProperties && Property && (Tag.OverrideOperation != EOverriddenPropertyOperation::SubObjectsShadowing))
					{
						UE::FInitializedPropertyValueState(this, Data).Set(Property, Tag.ArrayIndex);
					}
				#endif
					continue;
				}

				if (bUseRedirects)
				{
					if (UE::FPropertyTypeName NewTypeName = ApplyRedirectsToPropertyType(Tag.GetType(), Property); !NewTypeName.IsEmpty())
					{
						Tag.SetType(NewTypeName);
					}
				}

			#if WITH_EDITORONLY_DATA
				// Try to match the type when impersonating because there can be multiple properties with the same name.
				if (UNLIKELY(SerializeContext->bImpersonateProperties && Property && !Property->CanSerializeFromTypeName(Tag.GetType())))
				{
					for (FProperty* PropertyMatch = PropertyLink; PropertyMatch; PropertyMatch = PropertyMatch->PropertyLinkNext)
					{
						if (PropertyMatch->GetFName() == Tag.Name && PropertyMatch->CanSerializeFromTypeName(Tag.GetType()))
						{
							Property = PropertyMatch;
							break;
						}
					}
				}

				TOptional<UE::FSerializedPropertyPathScope> SerializedPropertyPath;
				if (SerializeContext->bTrackSerializedPropertyPath)
				{
					const FName Name = Property ? Property->GetFName() : Tag.Name;
					const int32 Index = Tag.ArrayIndex > 0 || (Property && Property->ArrayDim > 1) ? Tag.ArrayIndex : INDEX_NONE;
					const UE::FPropertyPathNameSegment Segment{Name, Tag.GetType(), Index};
					SerializedPropertyPath.Emplace(SerializeContext, Segment);
				}
			#endif

				Tag.SetProperty(Property);

				bool bTryStoreUnknownPropertyPath = false;

				if (Property)
				{
	#if WITH_EDITOR
					if (BreakRecursionIfFullyLoad && BreakRecursionIfFullyLoad->HasAllFlags(RF_LoadCompleted))
					{
					}
					else
	#endif // WITH_EDITOR
					// editoronly properties should be skipped if we don't have editor only data to read into
					if ((Property->PropertyFlags & CPF_EditorOnly) && !FPlatformProperties::HasEditorOnlyData() && !GForceLoadEditorOnly)
					{
					}
					// check for valid array index
					else if (Tag.ArrayIndex >= Property->ArrayDim || Tag.ArrayIndex < 0)
					{
						UE_LOGF(LogClass, Warning, "Array bound exceeded in %ls of %ls - %d exceeds [0-%d] in package: %ls",
							*WriteToString<32>(Tag.Name), *WriteToString<32>(GetFName()), Tag.ArrayIndex, Property->ArrayDim - 1, *UnderlyingArchive.GetArchiveName());
					}
					else if (!Property->ShouldSerializeValue(UnderlyingArchive))
					{
						UE_CLOGF((UnderlyingArchive.IsPersistent() && FPlatformProperties::RequiresCookedData()), LogClass, Warning, "Skipping saved property %ls of %ls since it is no longer serializable for asset:  %ls. (Maybe resave asset?)", *Tag.Name.ToString(), *GetName(), *UnderlyingArchive.GetArchiveName());
					}
#if WITH_EDITORONLY_DATA
					// Handle the sub object shadowing serialization,
					// do not skip this property if loading into a unknown property or into a placeholder
					else if (Tag.OverrideOperation == EOverriddenPropertyOperation::SubObjectsShadowing && 
						(!SerializeContext->bImpersonateProperties ))
					{
						UnderlyingArchive.Seek(StartOfProperty + Tag.Size);
						bAdvanceProperty = true;
					}
#endif // WITH_EDITORONLY_DATA
					else
					{
						FStructuredArchive::FSlot ValueSlot = PropertyRecord.EnterField(TEXT("Value"));

						switch (Property->ConvertFromType(Tag, ValueSlot, Data, DefaultsStruct, Defaults))
						{
							case EConvertFromTypeResult::Converted:
								bAdvanceProperty = true;
								bTryStoreUnknownPropertyPath = true;
								break;

							case EConvertFromTypeResult::Serialized:
								bAdvanceProperty = !UnderlyingArchive.IsCriticalError();
								break;

							case EConvertFromTypeResult::UseSerializeItem:
								if (const FName PropID = Property->GetID(); Tag.Type != PropID)
								{
									UE_LOGF(LogClass, Warning, "Type mismatch in %ls of %ls - Previous (%ls) Current(%ls) in package: %ls",
										*WriteToString<32>(Tag.Name), *WriteToString<32>(GetFName()),
										*WriteToString<32>(Tag.Type), *WriteToString<32>(PropID),
										*UnderlyingArchive.GetArchiveName());
									bTryStoreUnknownPropertyPath = true;
								}
								else
								{
									uint8* DestAddress = Property->ContainerPtrToValuePtr<uint8>(Data, Tag.ArrayIndex);
									const uint8* DefaultsFromParent = Property->ContainerPtrToValuePtrForDefaults<uint8>(DefaultsStruct, Defaults, Tag.ArrayIndex);

									// This property is ok.
									Tag.SerializeTaggedProperty(ValueSlot, Property, DestAddress, DefaultsFromParent);
									bAdvanceProperty = !UnderlyingArchive.IsCriticalError();
								}
								break;

							case EConvertFromTypeResult::CannotConvert:
								bTryStoreUnknownPropertyPath = true;
								break;

							default:
								checkNoEntry();
								break;
						}
					}
				}
				else
				{
					bTryStoreUnknownPropertyPath = true;
				}

			#if WITH_EDITORONLY_DATA
				// Track the path for an unknown property and serialize it to track any unknown property within it.
				if (UNLIKELY(bTryStoreUnknownPropertyPath))
				{
					if (!bSearchedForUnknownPropertyTree)
					{
						bSearchedForUnknownPropertyTree = true;
						if (SerializeContext->bTrackUnknownProperties)
						{
							if (UObject* Object = SerializeContext->SerializedObject)
							{
								UnknownPropertyTree = FUnknownPropertyTree(Object).FindOrCreate();
							}
						}
					}

					if (UnknownPropertyTree)
					{
						UnknownPropertyTree->Add(SerializeContext->SerializedPropertyPath).SetTag(Tag);

						const bool bSerializeValue = PropertyTypeContainsStructOrEnum(Tag.GetType());
						const bool bRestoreOverrideOperation = Tag.OverrideOperation != EOverriddenPropertyOperation:: None && !Property && !UnderlyingArchive.IsTransacting();

						// Try to construct a field from the property tag for serialization and overrides.
						if (bSerializeValue || bRestoreOverrideOperation)
						{
							TUniquePtr<FField> TempField(FField::TryConstruct(Tag.Type, {}, Tag.Name));
							if (FProperty* TempProperty = CastField<FProperty>(TempField.Get()); TempProperty && TempProperty->LoadTypeName(Tag.GetType(), &Tag))
							{
								TempProperty->Link(UnderlyingArchive);

								// Serialize the value if it contains a struct or enum because only those may contain an unknown property.
								if (bSerializeValue)
								{
									TGuardValue<bool> ScopedLoadingUnknownPropertyTree(SerializeContext->bIsLoadingToUnknownPropertyTree, true);
									UnderlyingArchive.Seek(StartOfProperty);
									FStructuredArchive::FSlot ValueSlotCopy = PropertyRecord.EnterField(TEXT("Value"));
									void* TempData = TempProperty->AllocateAndInitializeValue();
									Tag.SerializeTaggedProperty(ValueSlotCopy, TempProperty, (uint8*)TempData, nullptr);
									TempProperty->DestroyAndFreeValue(TempData);
								}

								// Restore overrides for unknown properties.
								if (bRestoreOverrideOperation)
								{
									if (FOverriddenPropertySet* OverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties())
									{
										OverriddenProperties->ConditionallyRestoreOverriddenPropertyOperation(Tag.OverrideOperation, UnderlyingArchive.GetSerializedPropertyChain(), TempProperty);
									}
								}
							}
						}
					}
				}
			#endif // WITH_EDITORONLY_DATA

				int64 Loaded = UnderlyingArchive.Tell() - StartOfProperty;

				if (bAdvanceProperty)
				{
					checkf(Tag.Size == Loaded,
						TEXT("Size mismatch in %s of %s of type %s. Loaded %" INT64_FMT " bytes but expected %d. Package: %s. Property: '%s'. Type: '%s'."),
						*Tag.Name.ToString(), *GetName(), *WriteToString<64>(Tag.GetType()), Loaded, Tag.Size, *UnderlyingArchive.GetArchiveName(),
						Property ? *Property->GetName() : TEXT(""), *GetFullName());

					// The operation was set part of the tag, now that we know the associated property, restore the overridden operation on the object
					// No need to rebuild the overridden state in transaction as it was serialized is one chunk
					if (!UnderlyingArchive.IsTransacting())
					{
						if (FOverriddenPropertySet* OverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties())
						{
							OverriddenProperties->ConditionallyRestoreOverriddenPropertyOperation(Tag.OverrideOperation, UnderlyingArchive.GetSerializedPropertyChain(), Property);
						}
					}

				#if WITH_EDITORONLY_DATA
					if ((Tag.OverrideOperation != EOverriddenPropertyOperation::SubObjectsShadowing))
					{
						if (SerializeContext->bTrackInitializedProperties && Property)
						{
							UE::FInitializedPropertyValueState(this, Data).Set(Property, Tag.ArrayIndex);
						}
						if (SerializeContext->bTrackSerializedProperties && Property)
						{
							UE::FSerializedPropertyValueState(this, Data).Set(Property, Tag.ArrayIndex);
						}
					}
				#endif
				}
				else if (Tag.Size != Loaded)
				{
#if UE_INTERNAL_LINKER_BOUNDS
					if(!UnderlyingArchive.IsCriticalError())
#endif // UE_INTERNAL_LINKER_BOUNDS
					{
						checkf(Tag.Size > 0, TEXT("Invalid property tag size %d in %s of %s of type %s. Package: %s. Property: '%s'. Type: '%s'."),
							Tag.Size, *Tag.Name.ToString(), *GetName(), *WriteToString<64>(Tag.GetType()), *UnderlyingArchive.GetArchiveName(),
							Property ? *Property->GetName() : TEXT(""), *GetFullName());
						UnderlyingArchive.Seek(StartOfProperty + Tag.Size);
					}
				}
			}
		}
	}
	else
	{
		FUnversionedPropertyTestCollector TestCollector;

		FStructuredArchive::FRecord PropertiesRecord = Slot.EnterRecord();

		check(UnderlyingArchive.IsSaving() || UnderlyingArchive.IsCountingMemory() || UnderlyingArchive.IsObjectReferenceCollector());
		checkf(!UnderlyingArchive.ArUseCustomPropertyList,
				TEXT("Custom property lists only work with binary serialization, not tagged property serialization. "
					 "Attempted for struct '%s' and archive '%s'. "), *GetFName().ToString(), *UnderlyingArchive.GetArchiveName());

		const UScriptStruct* DefaultsScriptStruct = dynamic_cast<const UScriptStruct*>(DefaultsStruct);

		/** If true, it means that we want to serialize all properties of this struct if any properties differ from defaults */
		bool bUseAtomicSerialization = false;
		if (DefaultsScriptStruct)
		{
			bUseAtomicSerialization = DefaultsScriptStruct->ShouldSerializeAtomically(UnderlyingArchive);
		}

		// Save tagged properties.
		const bool bDoDeltaSerialization = UnderlyingArchive.DoDelta() && !UnderlyingArchive.IsTransacting() && (Defaults || bIsUClass);

		// Iterate over properties in the order they were linked and serialize them.
		const FCustomPropertyListNode* CustomPropertyNode = UnderlyingArchive.ArUseCustomPropertyList ? UnderlyingArchive.ArCustomPropertyList : nullptr;
		for (FProperty* Property = UnderlyingArchive.ArUseCustomPropertyList ? (CustomPropertyNode ? CustomPropertyNode->Property : nullptr) : PropertyLink;
			Property;
			Property = UnderlyingArchive.ArUseCustomPropertyList ? FCustomPropertyListNode::GetNextPropertyAndAdvance(CustomPropertyNode) : Property->PropertyLinkNext)
		{
			if (Property->ShouldSerializeValue(UnderlyingArchive))
			{
				const int32 LoopMin = CustomPropertyNode ? CustomPropertyNode->ArrayIndex : 0;
				const int32 LoopMax = CustomPropertyNode ? LoopMin + 1 : Property->ArrayDim;

				TOptional<FStructuredArchive::FArray> StaticArrayContainer;
				if (((LoopMax - 1) > LoopMin) && UnderlyingArchive.IsTextFormat())
				{
					int32 NumItems = LoopMax - LoopMin;
					StaticArrayContainer.Emplace(PropertiesRecord.EnterArray(*Property->GetName(), NumItems));
				}

				for (int32 Idx = LoopMin; Idx < LoopMax; Idx++)
				{
					uint8* DataPtr = Property->ContainerPtrToValuePtr<uint8>(Data, Idx);
					const uint8* DefaultValue = Property->ContainerPtrToValuePtrForDefaults<uint8>(DefaultsStruct, Defaults, Idx);
					const bool bSerializeValue = (StaticArrayContainer.IsSet() || CustomPropertyNode || !bDoDeltaSerialization ||
						(FOverridableSerializationLogic::IsEnabled() && FOverridableSerializationLogic::GetOverriddenPropertyOperation(UnderlyingArchive, Property, DataPtr, DefaultValue) != EOverriddenPropertyOperation::None) ||
						(!FOverridableSerializationLogic::IsEnabled() && !Property->Identical(DataPtr, DefaultValue, UnderlyingArchive.GetPortFlags())));
				#if WITH_EDITORONLY_DATA
					const bool bInitializedValue = !SerializeContext->bTrackInitializedProperties || UE::FInitializedPropertyValueState(this, Data).IsSet(Property, Idx);
					if (bInitializedValue && (bSerializeValue || SerializeContext->bTrackInitializedProperties))
				#else
					if (bSerializeValue)
				#endif
					{
						if (bUseAtomicSerialization)
						{
							DefaultValue = nullptr;
						}
#if WITH_EDITOR
						static const FName NAME_PropertySerialize = FName("PropertySerialize");
						FArchive::FScopeAddDebugData P(UnderlyingArchive, NAME_PropertySerialize);
						FArchive::FScopeAddDebugData S(UnderlyingArchive, Property->GetFName());
#endif
						TestCollector.RecordSavedProperty(Property);

						FPropertyTag Tag(Property, Idx, DataPtr);
						// If available use the property guid from BlueprintGeneratedClasses, provided we aren't cooking data.
						if (bArePropertyGuidsAvailable && !UnderlyingArchive.IsCooking())
						{
							const FGuid PropertyGuid = FindPropertyGuidFromName(Tag.Name);
							Tag.SetPropertyGuid(PropertyGuid);
						}

						if (!bSerializeValue)
						{
							Tag.SerializeType = EPropertyTagSerializeType::Skipped;
						}

						TStringBuilder<256> TagName;
						Tag.Name.ToString(TagName);
						FStructuredArchive::FSlot PropertySlot = StaticArrayContainer.IsSet() ? StaticArrayContainer->EnterElement() : PropertiesRecord.EnterField(TagName.ToString());

						PropertySlot << Tag;

						if (!bSerializeValue)
						{
							PropertySlot.EnterStream(); // Save an empty value for text format archives.
							continue;
						}

						// need to know how much data this call to SerializeTaggedProperty consumes, so mark where we are
						int64 DataOffset = UnderlyingArchive.Tell();

						// if using it, save the current custom property list and switch to its sub property list (in case of UStruct serialization)
						const FCustomPropertyListNode* SavedCustomPropertyList = nullptr;
						if (UnderlyingArchive.ArUseCustomPropertyList && CustomPropertyNode)
						{
							SavedCustomPropertyList = UnderlyingArchive.ArCustomPropertyList;
							UnderlyingArchive.ArCustomPropertyList = CustomPropertyNode->SubPropertyList;
						}

						Tag.SerializeTaggedProperty(PropertySlot, Property, DataPtr, DefaultValue);

						// restore the original custom property list after serializing
						if (SavedCustomPropertyList)
						{
							UnderlyingArchive.ArCustomPropertyList = SavedCustomPropertyList;
						}

						// set the tag's size
						Tag.Size = IntCastChecked<int32>(UnderlyingArchive.Tell() - DataOffset);

						if (Tag.Size > 0 && !UnderlyingArchive.IsTextFormat())
						{
							// mark our current location
							DataOffset = UnderlyingArchive.Tell();

							// go back and re-serialize the size now that we know it
							UnderlyingArchive.Seek(Tag.SizeOffset);
							UnderlyingArchive << Tag.Size;

							// return to the current location
							UnderlyingArchive.Seek(DataOffset);
						}
					}
				}
			}
		}

		if (!UnderlyingArchive.IsTextFormat())
		{
			// Add an empty FName that serves as a null-terminator
			FName NoneTerminator;
			UnderlyingArchive << NoneTerminator;
		}
	}
}
void UStruct::FinishDestroy()
{
	DestroyUnversionedSchema(this);
	Script.Empty();
	Super::FinishDestroy();
}

/** Helper function that destroys properties from the privided linked list and nulls the list head pointer */
inline void DestroyPropertyLinkedList(FField*& PropertiesToDestroy)
{
	for (FField* FieldToDestroy = PropertiesToDestroy; FieldToDestroy; )
	{
		FField* NextField = FieldToDestroy->Next;
		delete FieldToDestroy;
		FieldToDestroy = NextField;
	}
	PropertiesToDestroy = nullptr;
}

void UStruct::DestroyChildPropertiesAndResetPropertyLinks()
{
	DestroyPropertyLinkedList(ChildProperties);
	PropertyLink = nullptr;
	RefLink = nullptr;
	DestructorLink = nullptr;
	PostConstructLink = nullptr;
#if WITH_EDITORONLY_DATA
	FieldPathSerialNumber = GetNextFieldPathSerialNumber();
	bHasAssetRegistrySearchableProperties = false;
#endif // WITH_EDITORONLY_DATA
	DestroyUnversionedSchema(this);
}

UStruct::~UStruct()
{
	// Destroy all properties owned by this struct
	// This needs to happen after FinishDestroy which calls DestroyNonNativeProperties
	// Also, Blueprint generated classes can have DestroyNonNativeProperties called on them after their FinishDestroy has been called
	// so properties can only be deleted in the destructor
	DestroyPropertyLinkedList(ChildProperties);
	DeleteUnresolvedScriptProperties();

	DestructItem(&Script);
}

IMPLEMENT_FSTRUCTUREDARCHIVE_SERIALIZER(UStruct);

#if WITH_EDITORONLY_DATA
void UStruct::ConvertUFieldsToFFields()
{
	TArray<FField*> NewChildProperties;
	UField* OldField = Children;
	UField* PreviousUnconvertedField = nullptr;

	// First convert all properties and store them in a temp array
	while (OldField)
	{
		if (OldField->IsA<UProperty>())
		{
			FField* NewField = OldField->GetAssociatedFField();
			if (!NewField)
			{
				NewField = FField::CreateFromUField(OldField);
				OldField->SetAssociatedFField(NewField);
				check(NewField);
			}
			NewChildProperties.Add(NewField);
			// Remove this field from the linked list
			if (PreviousUnconvertedField)
			{
				PreviousUnconvertedField->Next = OldField->Next;
			}
			else
			{
				Children = OldField->Next;
			}

            // Rename will remove the renamed object's linker when moving to a new package so invalidate the export beforehand
			FLinkerLoad::InvalidateExport(OldField);
			// Move the old UProperty to the transient package and rename it to something unique
			OldField->Rename(*MakeUniqueObjectName(GetTransientPackage(), OldField->GetClass()).ToString(), GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			OldField->RemoveFromRoot();
		}
		else
		{
			// Update the previous unconverted field
			if (PreviousUnconvertedField)
			{
				PreviousUnconvertedField->Next = OldField;
			}
			else
			{
				Children = OldField;
			}
			PreviousUnconvertedField = OldField;
		}
		OldField = OldField->Next;
	}
	// Now add them to the linked list in the reverse order to preserve their actual order (adding to the list reverses the order)
	for (int32 ChildPropertyIndex = NewChildProperties.Num() - 1; ChildPropertyIndex >= 0; --ChildPropertyIndex)
	{
		FField* NewField = NewChildProperties[ChildPropertyIndex];
		check(NewField->Next == nullptr);
		NewField->Next = ChildProperties;
		ChildProperties = NewField;
	}
}
#endif // WITH_EDITORONLY_DATA

void UStruct::SerializeProperties(FArchive& Ar)
{
	int32 PropertyCount = 0;

	if (Ar.IsSaving())
	{
		// Count properties
		for (FField* Field = ChildProperties; Field; Field = Field->Next)
		{
			bool bSaveProperty = true;
#if WITH_EDITORONLY_DATA
			FProperty* Property = CastField<FProperty>(Field);
			if (Property)
			{
				bSaveProperty = !(Ar.IsFilterEditorOnly() && Property->IsEditorOnlyProperty());
			}
#endif // WITH_EDITORONLY_DATA
			if (bSaveProperty)
			{
				PropertyCount++;
			}
		}
	}

	Ar << PropertyCount;

	if (Ar.IsLoading())
	{
		// Not using SerializeSingleField here to avoid unnecessary checks for each property
		TArray<FField*> LoadedProperties;
		LoadedProperties.Reserve(PropertyCount);
		for (int32 PropertyIndex = 0; PropertyIndex < PropertyCount; ++PropertyIndex)
		{
			FName PropertyTypeName;
			Ar << PropertyTypeName;
			FField* Prop = FField::Construct(PropertyTypeName, this, NAME_None);
			check(Prop);
			Prop->Serialize(Ar);
			LoadedProperties.Add(Prop);
		}
		for (int32 PropertyIndex = LoadedProperties.Num() - 1; PropertyIndex >= 0; --PropertyIndex)
		{
			FField* Prop = LoadedProperties[PropertyIndex];
			Prop->Next = ChildProperties;
			ChildProperties = Prop;
		}
	}
	else
	{
		int32 VerifySerializedFieldsCount = 0;
		for (FField* Field = ChildProperties; Field; Field = Field->Next)
		{
			bool bSaveProperty = true;
#if WITH_EDITORONLY_DATA
			FProperty* Property = CastField<FProperty>(Field);
			if (Property)
			{
				bSaveProperty = !(Ar.IsFilterEditorOnly() && Property->IsEditorOnlyProperty());
			}
#endif // WITH_EDITORONLY_DATA
			if (bSaveProperty)
			{
				FName PropertyTypeName = Field->GetClass()->GetFName();
				Ar << PropertyTypeName;
				Field->Serialize(Ar);
				VerifySerializedFieldsCount++;
			}
		}
		check(!Ar.IsSaving() || VerifySerializedFieldsCount == PropertyCount);
	}
}

void UStruct::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	UStruct* SuperStructBefore = GetSuperStruct();
#endif

	Ar << SuperStruct;

#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	if (Ar.IsLoading())
	{
		this->ReinitializeBaseChainArray();
	}
	// Handle that fact that FArchive takes UObject*s by reference, and archives can just blat
	// over our SuperStruct with impunity.
	else if (SuperStructBefore)
	{
		UStruct* SuperStructAfter = GetSuperStruct();
		if (SuperStructBefore != SuperStructAfter)
		{
			this->ReinitializeBaseChainArray();
		}
	}
#endif

	Ar.UsingCustomVersion(FFrameworkObjectVersion::GUID);
	Ar.UsingCustomVersion(FCoreObjectVersion::GUID);

	if (Ar.CustomVer(FFrameworkObjectVersion::GUID) < FFrameworkObjectVersion::RemoveUField_Next)
	{
		Ar << Children;
	}
	else
	{
		TArray<UField*> ChildArray;
		if (Ar.IsLoading())
		{
			Ar << ChildArray;

			// skip null fields
			Children = nullptr;
			if (ChildArray.Num() > 0)
			{
				UField* CurrentChild = nullptr;
 				int32 Index = 0;
				for (; Index < ChildArray.Num(); ++Index)
				{
					if (ChildArray[Index])
					{
						Children = ChildArray[Index];
						CurrentChild = Children;
						break;
					}
				}
				if (CurrentChild)
				{
					for (Index+=1; Index < ChildArray.Num(); ++Index)
					{
						if (ChildArray[Index])
						{
							CurrentChild->Next = ChildArray[Index];
							CurrentChild = ChildArray[Index];
						}
					}

					CurrentChild->Next = nullptr;
				}
			}
		}
		else
		{
			UField* Child = Children;
			while (Child)
			{
				ChildArray.Add(Child);
				Child = Child->Next;
			}
			Ar << ChildArray;
		}
	}

	if (Ar.CustomVer(FCoreObjectVersion::GUID) >= FCoreObjectVersion::FProperties)
	{
		SerializeProperties(Ar);
	}

	if (Ar.IsLoading())
	{
		FStructScriptLoader ScriptLoadHelper(/*TargetScriptContainer =*/this, Ar);
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
		bool const bAllowDeferredScriptSerialization = true;
#else  // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
		bool const bAllowDeferredScriptSerialization = false;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

		// NOTE: if bAllowDeferredScriptSerialization is set to true, then this
		//       could temporarily skip script serialization (as it could
		//       introduce unwanted dependency loads at this time)
		ScriptLoadHelper.LoadStructWithScript(this, Ar, bAllowDeferredScriptSerialization);

		if (!dynamic_cast<UClass*>(this) && !(Ar.GetPortFlags() & PPF_Duplicate)) // classes are linked in the UClass serializer, which just called me
		{
			// Link the properties.
			Link(Ar, true);
		}
	}
	else
	{
		int32 ScriptBytecodeSize = Script.Num();
		int64 ScriptStorageSizeOffset = INDEX_NONE;

		if (Ar.IsSaving())
		{
			Ar << ScriptBytecodeSize;

			int32 ScriptStorageSize = 0;
			// drop a zero here.  will seek back later and re-write it when we know it
			ScriptStorageSizeOffset = Ar.Tell();
			Ar << ScriptStorageSize;
		}

		// Skip serialization if we're duplicating classes for reinstancing, since we only need the memory layout
		if (!GIsDuplicatingClassForReinstancing)
		{

			// no bytecode patch for this struct - serialize normally [i.e. from disk]
			int32 iCode = 0;
			int64 const BytecodeStartOffset = Ar.Tell();

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// ScriptSHAHash is no longer supported; delete this if block when deleting it.
			if (Ar.IsPersistent() && Ar.GetLinker())
			{
				// make sure this is a ULinkerSave
				FLinkerSave* LinkerSave = CastChecked<FLinkerSave>(Ar.GetLinker());

				// remember how we were saving
				FArchive* SavedSaver = LinkerSave->Saver;

				// force writing to a buffer
				TArray<uint8> TempScript;
				FMemoryWriter MemWriter(TempScript, Ar.IsPersistent());
				LinkerSave->Saver = &MemWriter;

				{
					FPropertyProxyArchive PropertyAr(Ar, iCode, this);
					// now, use the linker to save the byte code, but writing to memory
					while (iCode < ScriptBytecodeSize)
					{
						SerializeExpr(iCode, PropertyAr);
					}
				}

				// restore the saver
				LinkerSave->Saver = SavedSaver;

				// now write out the memory bytes
				Ar.Serialize(TempScript.GetData(), TempScript.Num());

				// and update the SHA (does nothing if not currently calculating SHA)
				LinkerSave->UpdateScriptSHAKey(TempScript);
			}
			else
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				FPropertyProxyArchive PropertyAr(Ar, iCode, this);
				while (iCode < ScriptBytecodeSize)
				{
					SerializeExpr(iCode, PropertyAr);
				}
			}

			if (iCode != ScriptBytecodeSize)
			{
				UE_LOGF(LogClass, Fatal, "Script serialization mismatch: Got %i, expected %i", iCode, ScriptBytecodeSize);
			}

			if (Ar.IsSaving())
			{
				int64 const BytecodeEndOffset = Ar.Tell();

				// go back and write on-disk size
				Ar.Seek(ScriptStorageSizeOffset);
				int32 ScriptStorageSize = IntCastChecked<int32>(BytecodeEndOffset - BytecodeStartOffset);
				Ar << ScriptStorageSize;

				// back to where we were
				Ar.Seek(BytecodeEndOffset);
			}
		} // if !GIsDuplicatingClassForReinstancing
	}

	// For consistency between serialization and TFastReferenceCollector based reference collection
	if (Ar.IsObjectReferenceCollector() && !Ar.IsPersistent())
	{
		Ar << ScriptAndPropertyObjectReferences;
	}
}

void UStruct::PostLoad()
{
	Super::PostLoad();

	// Finally try to resolve all script properties that couldn't be resolved at load time
	if (UnresolvedScriptProperties)
	{
		for (TPair<TFieldPath<FField>, int32>& MissingProperty : *UnresolvedScriptProperties)
		{
			FField* ResolvedProperty = MissingProperty.Key.Get(this);
			if (ResolvedProperty)
			{
				check((int32)Script.Num() >= (int32)(MissingProperty.Value + sizeof(FField*)));
				FField** TargetScriptPropertyPtr = (FField**)(Script.GetData() + MissingProperty.Value);
				*TargetScriptPropertyPtr = ResolvedProperty;

				// Collect UObjects referenced by this property so that its owner doesn't get GC'd leaving a stale FProperty reference in bytecode
				{
					auto ScriptAndPropertyObjectReferencesView = MutableView(ScriptAndPropertyObjectReferences);
					FPropertyReferenceCollector Collector(this, ScriptAndPropertyObjectReferencesView);
					ResolvedProperty->AddReferencedObjects(Collector);
				}
			}
			else if (!MissingProperty.Key.IsPathToFieldEmpty())
			{
				UE_LOGF(LogClass, Warning, "Failed to resolve bytecode referenced field from path: %ls when loading %ls", *MissingProperty.Key.ToString(), *GetFullName());
			}
		}
		DeleteUnresolvedScriptProperties();
	}
}

void UStruct::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	UStruct* This = CastChecked<UStruct>(InThis);
#if WITH_EDITOR
	if( GIsEditor )
	{
		// Required by the unified GC when running in the editor
		Collector.AddReferencedObject( This->SuperStruct, This );
		Collector.AddReferencedObject( This->Children, This );
		Collector.AddReferencedObjects(This->ScriptAndPropertyObjectReferences, This);
	}
#endif
#if WITH_EDITORONLY_DATA
	Collector.AddReferencedObjects(This->PropertyWrappers, This);
#endif
	Super::AddReferencedObjects( This, Collector );
}

void UStruct::SetSuperStruct(UStruct* NewSuperStruct)
{
	SuperStruct = NewSuperStruct;
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	this->ReinitializeBaseChainArray();
#endif
}

FString UStruct::GetAuthoredNameForField(const UField* Field) const
{
	if (Field)
	{
		return Field->GetName();
	}
	return FString();
}

FString UStruct::GetAuthoredNameForField(const FField* Field) const
{
	if (Field)
	{
		return Field->GetName();
	}
	return FString();
}

#if WITH_METADATA
bool UStruct::GetBoolMetaDataHierarchical(const FName& Key) const
{
	bool bResult = false;
	for(const UStruct* TestStruct : GetSuperStructIterator())
	{
		if( TestStruct->HasMetaData(Key) )
		{
			bResult = TestStruct->GetBoolMetaData(Key);
			break;
		}
	}
	return bResult;
}

bool UStruct::GetStringMetaDataHierarchical(const FName& Key, FString* OutValue) const
{
	for (const UStruct* TestStruct : GetSuperStructIterator())
	{
		if (const FString* FoundMetaData = TestStruct->FindMetaData(Key))
		{
			if (OutValue != nullptr)
			{
				*OutValue = *FoundMetaData;
			}

			return true;
		}
	}

	return false;
}

const UStruct* UStruct::HasMetaDataHierarchical(const FName& Key) const
{
	for (const UStruct* TestStruct : GetSuperStructIterator())
	{
		if (TestStruct->HasMetaData(Key))
		{
			return TestStruct;
		}
	}

	return nullptr;
}

#endif // WITH_METADATA

#if WITH_EDITORONLY_DATA

bool UStruct::ActivateTrackingPropertyValueFlag(EPropertyValueFlags Flags, void* Data) const
{
	return false;
}

bool UStruct::IsTrackingPropertyValueFlag(EPropertyValueFlags Flags, const void* Data) const
{
	return false;
}

bool UStruct::HasPropertyValueFlag(EPropertyValueFlags Flags, const void* Data, const FProperty* Property, int32 ArrayIndex) const
{
	return true;
}

void UStruct::SetPropertyValueFlag(EPropertyValueFlags Flags, bool bValue, void* Data, const FProperty* Property, int32 ArrayIndex) const
{
}

void UStruct::ResetPropertyValueFlags(EPropertyValueFlags Flags, void* Data) const
{
}

void UStruct::SerializePropertyValueFlags(EPropertyValueFlags Flags, void* Data, FStructuredArchiveRecord Record, FArchiveFieldName Name) const
{
}

#endif // WITH_EDITORONLY_DATA

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	/**
	 * If we're loading, then the value of the script's UObject* expression
	 * could be pointing at a ULinkerPlaceholderClass (used by the linker to
	 * fight cyclic dependency issues on load). So here, if that's the case, we
	 * have the placeholder track this ref (so it'll replace it once the real
	 * class is loaded).
	 *
	 * @param  ScriptPtr    Reference to the point in the bytecode buffer, where a UObject* has been stored (for us to check).
	 */
	static void HandlePlaceholderScriptRef(void* ScriptPtr)
	{
		ScriptPointerType  Temp = FPlatformMemory::ReadUnaligned<ScriptPointerType>(ScriptPtr);
		UObject*& ExprPtrRef = (UObject*&)Temp;

		TObjectPtr<UObject> ObjPtr(ExprPtrRef);
		if (!ObjPtr.IsResolved())
		{
			return;
		}
		else if (ULinkerPlaceholderClass* PlaceholderObj = Cast<ULinkerPlaceholderClass>(ExprPtrRef))
		{
			PlaceholderObj->AddReferencingScriptExpr((UClass**)(&ExprPtrRef));
		}
		else if (ULinkerPlaceholderFunction* PlaceholderFunc = Cast<ULinkerPlaceholderFunction>(ExprPtrRef))
		{
			PlaceholderFunc->AddReferencingScriptExpr((UFunction**)(&ExprPtrRef));
		}
	}

	#define FIXUP_EXPR_OBJECT_POINTER(Type) \
	{ \
		if (!Ar.IsSaving()) \
		{ \
			int32 const ExprIndex = iCode - sizeof(ScriptPointerType); \
			HandlePlaceholderScriptRef(&Script[ExprIndex]); \
		} \
	}
#endif // #if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

EExprToken UStruct::SerializeExpr( int32& iCode, FArchive& Ar )
{
#define SERIALIZEEXPR_INC
#define SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
#include "UObject/ScriptSerialization.inl"
	return Expr;
#undef SERIALIZEEXPR_INC
#undef SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
}

FTopLevelAssetPath UStruct::GetStructPathName() const
{
	// Some day this check may actually be relevant
	checkf(GetOuter() == GetPackage(), TEXT("Only top level objects are supported by FTopLevelAssetPath. This object is a subobject: \"%s\""), *GetPathName());
	return FTopLevelAssetPath(GetOuter()->GetFName(), GetFName());
}

void UStruct::InstanceSubobjectTemplates(TNotNull<void*> Data, const void* DefaultData, const UStruct* DefaultStruct, TNotNull<UObject*> Owner, FObjectInstancingGraph* InstanceGraph)
{
	const FObjectInitializer* ObjInitializer = FUObjectThreadContext::Get().TopInitializer();

	// Dynamic instancing allows the instance graph to infer when to instance subobjects instead of requiring the
	// explicit 'CPF_InstancedReference' flag. This also means the code below intentionally visits all properties
	// in the reflink chain as a result, unless they are otherwise explicitly excluded through the instance graph.
	const bool bUsesDynamicInstancing = Owner->GetClass()->ShouldUseDynamicSubobjectInstancing();
	for (FProperty* Property = RefLink; Property != nullptr; Property = Property->NextRef)
	{
		const bool bPropertySupportsInstancing =
			bUsesDynamicInstancing ||
			Property->ContainsInstancedObjectProperty() ||
			Property->HasAnyPropertyFlags(CPF_AllowSelfReference)
		;

		const bool bPropertyInExclusionList =
			(InstanceGraph && InstanceGraph->IsPropertyInSubobjectExclusionList(Property)) ||
			(ObjInitializer && ObjInitializer->IsPropertyInSubobjectExclusionList(Property))
		;

		if (bPropertySupportsInstancing && !bPropertyInExclusionList)
		{
			Property->InstanceSubobjects(Property->ContainerPtrToValuePtr<uint8>(Data), Property->ContainerPtrToValuePtrForDefaults<uint8>(DefaultStruct, DefaultData), Owner, InstanceGraph);
		}
	}
}

/**
* @return	true if this object is of the specified type.
*/
#if USTRUCT_FAST_ISCHILDOF_COMPARE_WITH_OUTERWALK || USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_OUTERWALK
bool UStruct::IsChildOf( const UStruct* SomeBase ) const
{
	if (SomeBase == nullptr)
	{
		return false;
	}

	bool bOldResult = false;
	// Avoid access tracking when traversing supers to reduce costs in cooking
	for (const UStruct* TempStruct : GetSuperStructIterator())
	{
		if ( TempStruct == SomeBase )
		{
			bOldResult = true;
			break;
		}
	}

#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	const bool bNewResult = IsChildOfUsingStructArray(*SomeBase);
#endif

#if USTRUCT_FAST_ISCHILDOF_COMPARE_WITH_OUTERWALK
	ensureMsgf(bOldResult == bNewResult, TEXT("New cast code failed"));
#endif

	return bOldResult;
}
#endif

/*-----------------------------------------------------------------------------
	UScriptStruct.
-----------------------------------------------------------------------------*/

// sample of how to customize structs
#if 0
USTRUCT()
struct ENGINE_API FTestStruct
{
	GENERATED_USTRUCT_BODY()

	UObject* RawObjectPtr = nullptr;
	TMap<int32, double> Doubles;
	FTestStruct()
	{
		Doubles.Add(1, 1.5);
		Doubles.Add(2, 2.5);
	}
	void AddStructReferencedObjects(class FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(RawObjectPtr);
	}
	bool Serialize(FArchive& Ar)
	{
		Ar << Doubles;
		return true;
	}
	bool operator==(FTestStruct const& Other) const
	{
		if (Doubles.Num() != Other.Doubles.Num())
		{
			return false;
		}
		for (TMap<int32, double>::TConstIterator It(Doubles); It; ++It)
		{
			double const* OtherVal = Other.Doubles.Find(It.Key());
			if (!OtherVal || *OtherVal != It.Value() )
			{
				return false;
			}
		}
		return true;
	}
	bool Identical(FTestStruct const& Other, uint32 PortFlags) const
	{
		return (*this) == Other;
	}
	void operator=(FTestStruct const& Other)
	{
		Doubles.Empty(Other.Doubles.Num());
		for (TMap<int32, double>::TConstIterator It(Other.Doubles); It; ++It)
		{
			Doubles.Add(It.Key(), It.Value());
		}
	}
	bool ExportTextItem(FString& ValueStr, FTestStruct const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
	{
		ValueStr += TEXT("(");
		for (TMap<int32, double>::TConstIterator It(Doubles); It; ++It)
		{
			ValueStr += FString::Printf( TEXT("(%d,%f)"),It.Key(), It.Value());
		}
		ValueStr += TEXT(")");
		return true;
	}
	bool ImportTextItem( const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText )
	{
		check(*Buffer == TEXT('('));
		Buffer++;
		Doubles.Empty();
		while (1)
		{
			const TCHAR* Start = Buffer;
			while (*Buffer && *Buffer != TEXT(','))
			{
				if (*Buffer == TEXT(')'))
				{
					break;
				}
				Buffer++;
			}
			if (*Buffer == TEXT(')'))
			{
				break;
			}
			int32 Key = FCString::Atoi(Start);
			if (*Buffer)
			{
				Buffer++;
			}
			Start = Buffer;
			while (*Buffer && *Buffer != TEXT(')'))
			{
				Buffer++;
			}
			double Value = FCString::Atod(Start);

			if (*Buffer)
			{
				Buffer++;
			}
			Doubles.Add(Key, Value);
		}
		if (*Buffer)
		{
			Buffer++;
		}
		return true;
	}
	bool SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FArchive& Ar)
	{
		// no example of this provided, doesn't make sense
		return false;
	}
};

template<>
struct TStructOpsTypeTraits<FTestStruct> : public TStructOpsTypeTraitsBase2<FTestStruct>
{
	enum
	{
		WithZeroConstructor = true,
		WithSerializer = true,
		WithPostSerialize = true,
		WithCopy = true,
		WithIdenticalViaEquality = true,
		//WithIdentical = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithAddStructReferencedObjects = true,
		WithSerializeFromMismatchedTag = true,
	};
};

#endif


/** Used to hold virtual methods to construct, destruct, etc native structs in a generic and dynamic fashion
 * singleton-style to avoid issues with static constructor order
**/
static TMap<FTopLevelAssetPath,UScriptStruct::ICppStructOps*>& GetDeferredCppStructOps()
{
	static struct TMapWithAutoCleanup : public TMap<FTopLevelAssetPath, UScriptStruct::ICppStructOps*>
	{
		~TMapWithAutoCleanup()
		{
			for (ElementSetType::TConstIterator It(Pairs); It; ++It)
			{
				delete It->Value;
			}
		}
	}
	DeferredCppStructOps;
	return DeferredCppStructOps;
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
bool FindConstructorUninitialized(UStruct* BaseClass,uint8* Data,uint8* Defaults)
{
	bool bAnyProblem = false;
	static TSet<FString> PrintedWarnings;
	for(FProperty* P=BaseClass->PropertyLink; P; P=P->PropertyLinkNext )
	{
		int32 Size = P->GetSize();
		bool bProblem = false;
		check(Size);
		FBoolProperty*     PB        = CastField<FBoolProperty>(P);
		FStructProperty*   PS        = CastField<FStructProperty>(P);
		FStrProperty*      PStr      = CastField<FStrProperty>(P);
		FArrayProperty*    PArray    = CastField<FArrayProperty>(P);
		FOptionalProperty* POptional = CastField<FOptionalProperty>(P);
		if(PStr)
		{
			// string that actually have data would be false positives, since they would point to the same string, but actually be different pointers
			// string is known to have a good default constructor
		}
		else if(PB)
		{
			check(Size == PB->GetElementSize());
			if( PB->GetPropertyValue_InContainer(Data) && !PB->GetPropertyValue_InContainer(Defaults) )
			{
				bProblem = true;
			}
		}
		else if (PS)
		{
			// these are legitimate exceptions
			if (PS->Struct->GetName() != TEXT("BitArray")
				&& PS->Struct->GetName() != TEXT("SparseArray")
				&& PS->Struct->GetName() != TEXT("Set")
				&& PS->Struct->GetName() != TEXT("Map")
				&& PS->Struct->GetName() != TEXT("MultiMap")
				&& PS->Struct->GetName() != TEXT("ShowFlags_Mirror")
				&& PS->Struct->GetName() != TEXT("Pointer")
				)
			{
				bProblem = FindConstructorUninitialized(PS->Struct, P->ContainerPtrToValuePtr<uint8>(Data), P->ContainerPtrToValuePtr<uint8>(Defaults));
			}
		}
		else if (PArray)
		{
			bProblem = !PArray->Identical_InContainer(Data, Defaults);
		}
		else if (POptional)
		{
			bProblem = !POptional->Identical_InContainer(Data, Defaults);
		}
		else
		{
			if (FMemory::Memcmp(P->ContainerPtrToValuePtr<uint8>(Data), P->ContainerPtrToValuePtr<uint8>(Defaults), Size) != 0)
			{
//				UE_LOGF(LogClass, Warning,"Mismatch %d %d",(int32)*(Data + P->Offset), (int32)*(Defaults + P->Offset));
				bProblem = true;
			}
		}
		if (bProblem)
		{
			FString Issue;
			if (PS)
			{
				Issue = TEXT("     From ");
				Issue += P->GetFullName();
			}
			else
			{
				Issue = BaseClass->GetPathName() + TEXT(",") + P->GetFullName();
			}
			if (!PrintedWarnings.Contains(Issue))
			{
				bAnyProblem = true;
				PrintedWarnings.Add(Issue);
				if (PS)
				{
					UE_LOGF(LogClass, Warning,"%ls",*Issue);
//					OutputDebugStringW(*FString::Printf(TEXT("%s\n"),*Issue));
				}
				else
				{
					UE_LOGF(LogClass, Warning,"Native constructor does not initialize all properties %ls (may need to recompile excutable with new headers)",*Issue);
//					OutputDebugStringW(*FString::Printf(TEXT("Native contructor does not initialize all properties %s\n"),*Issue));
				}
			}
		}
	}
	return bAnyProblem;
}
#endif


UScriptStruct::UScriptStruct( EStaticConstructor, int32 InSize, int32 InAlignment, EObjectFlags InFlags )
	: UStruct( EC_StaticConstructor, InSize, InAlignment, InFlags )
	, StructFlags(STRUCT_NoFlags)
	, bPrepareCppStructOpsCompleted(false)
	, CppStructOps(NULL)
{
}

UScriptStruct::UScriptStruct(const FObjectInitializer& ObjectInitializer, UScriptStruct* InSuperStruct, ICppStructOps* InCppStructOps, EStructFlags InStructFlags, SIZE_T ExplicitSize, SIZE_T ExplicitAlignment )
	: UStruct(ObjectInitializer, InSuperStruct, InCppStructOps ? InCppStructOps->GetSize() : ExplicitSize, InCppStructOps ? InCppStructOps->GetAlignment() : ExplicitAlignment )
	, StructFlags(EStructFlags(InStructFlags | (InCppStructOps ? STRUCT_Native : STRUCT_NoFlags)))
	, bPrepareCppStructOpsCompleted(false)
	, CppStructOps(InCppStructOps)
{
	PrepareCppStructOps(); // propgate flags, etc
}

UScriptStruct::UScriptStruct(const FObjectInitializer& ObjectInitializer)
	: UStruct(ObjectInitializer)
	, StructFlags(STRUCT_NoFlags)
	, bPrepareCppStructOpsCompleted(false)
	, CppStructOps(NULL)
{
}

/** Stash a CppStructOps for future use
	* @param Target Name of the struct
	* @param InCppStructOps Cpp ops for this struct
**/
void UScriptStruct::DeferCppStructOps(FTopLevelAssetPath Target, ICppStructOps* InCppStructOps)
{
	TMap<FTopLevelAssetPath, UScriptStruct::ICppStructOps*>& DeferredStructOps = GetDeferredCppStructOps();

	if (UScriptStruct::ICppStructOps* ExistingOps = DeferredStructOps.FindRef(Target))
	{
#if UE_MERGED_MODULES

		// When using merged modules, we'll manually unload deferred struct ops at module teardown.
		// Whenever we get to this, it means the using code is doing multiple initialization, which we'll prevent here.
		delete InCppStructOps;
		return;

#else

		IReload* Reload = GetActiveReloadInterface();
		if (Reload == nullptr)
		{
			check(ExistingOps != InCppStructOps); // if it was equal, then we would be re-adding a now stale pointer to the map
			delete ExistingOps;
		}
		else if (!Reload->GetEnableReinstancing(false))
		{
			delete InCppStructOps;
			return;
		}
		// in reload, we will just leak these...they may be in use.

#endif // UE_MERGED_MODULES
	}
	DeferredStructOps.Add(Target, InCppStructOps);
}

/**
 * On module unload, remove the entry from the stash
 * @param Target Name of the struct
 */
void UScriptStruct::RemoveDeferredCppStructOps(FTopLevelAssetPath Target)
{
	TMap<FTopLevelAssetPath, UScriptStruct::ICppStructOps*>& DeferredStructOps = GetDeferredCppStructOps();

	UScriptStruct::ICppStructOps* ExistingOps = nullptr;
	if (DeferredStructOps.RemoveAndCopyValue(Target, ExistingOps))
	{
		delete ExistingOps;
	}
}

/**
 * On module unload, remove all entries in a package from the stash
 * The package name should be the full package name, such as "/Script/Engine"
 * @param PackageName Name of the package
 */
void UScriptStruct::RemoveAllDeferredCppStructOps(FName PackageName)
{
	UE_LOGF(LogClass, Verbose, "RemoveAllDeferredCppStructOps: %ls", *PackageName.ToString());

	TMap<FTopLevelAssetPath, UScriptStruct::ICppStructOps*>& DeferredStructOps = GetDeferredCppStructOps();

	for (auto DeferredStructOpsIt = DeferredStructOps.CreateIterator(); DeferredStructOpsIt; ++DeferredStructOpsIt)
	{
		if (DeferredStructOpsIt->Key.GetPackageName() == PackageName)
		{
			UScriptStruct::ICppStructOps* ExistingOps = DeferredStructOpsIt->Value;
			check(ExistingOps);
			delete ExistingOps;
			DeferredStructOpsIt.RemoveCurrent();
		}
	}
}

FTopLevelAssetPath UScriptStruct::GetFlattenedStructPathName() const
{
	return FTopLevelAssetPath(GetPackage()->GetFName(), GetFName());
}

/** Look for the CppStructOps if we don't already have it and set the property size **/
void UScriptStruct::PrepareCppStructOps()
{
	if (bPrepareCppStructOpsCompleted)
	{
		return;
	}
	if (!CppStructOps)
	{
		// RobM: the use of GetFlattenedStructPathName() is a bit of a hack to make AnimBPs work as they create nested structs.
		// Theoretically we could just wrap the line below in if (StructFlags&STRUCT_Native) but the native flag is set after
		// CppStructOps struct is actually found (see below).
		// It's also confusing why PrepareCppStructOps is being called from UScriptStruct::Serialize when loading since serialized
		// structs are not native and they should not implement CppStructOps
		CppStructOps = GetDeferredCppStructOps().FindRef(GetFlattenedStructPathName());
		if (!CppStructOps)
		{
			if (StructFlags&STRUCT_Native)
			{
				UE_LOGF(LogClass, Fatal, "Couldn't bind to native struct %ls. Headers need to be rebuilt, or a noexport class is missing a IMPLEMENT_STRUCT.",*GetName());
			}
			check(!bPrepareCppStructOpsCompleted); // recursion is unacceptable
			bPrepareCppStructOpsCompleted = true;
			return;
		}
		StructFlags = EStructFlags(StructFlags | STRUCT_Native);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		// test that the constructor is initializing everything
		if (!CppStructOps->HasZeroConstructor() && !IsReloadActive()) // when reloading, we get bogus warnings
		{
			int32 Size = CppStructOps->GetSize();
			uint8* TestData00 = (uint8*)FMemory::Malloc(Size);
			FMemory::Memzero(TestData00,Size);
			CppStructOps->Construct(TestData00);
			uint8* TestDataFF = (uint8*)FMemory::Malloc(Size);
			FMemory::Memset(TestDataFF,0xff,Size);
			CppStructOps->Construct(TestDataFF);

			if (FMemory::Memcmp(TestData00,TestDataFF, Size) != 0)
			{
				FindConstructorUninitialized(this,TestData00,TestDataFF);
			}
			if (CppStructOps->HasDestructor())
			{
				CppStructOps->Destruct(TestData00);
				CppStructOps->Destruct(TestDataFF);
			}
			FMemory::Free(TestData00);
			FMemory::Free(TestDataFF);
		}
#endif
	}

	check(!(StructFlags & STRUCT_ComputedFlags));
	if (CppStructOps->HasSerializer() || CppStructOps->HasStructuredSerializer())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has a custom serializer.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_SerializeNative );
	}
	if (CppStructOps->HasPostSerialize())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls wants post serialize.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_PostSerializeNative );
	}
	if (CppStructOps->HasPostScriptConstruct())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls wants post script construct.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_PostScriptConstruct);
	}
	if (CppStructOps->HasNetSerializer())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has a custom net serializer.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_NetSerializeNative);

		if (CppStructOps->HasNetSharedSerialization())
		{
			UE_LOGF(LogClass, Verbose, "Native struct %ls can share net serialization.",*GetName());
			StructFlags = EStructFlags(StructFlags | STRUCT_NetSharedSerialization);
		}
	}
	if (CppStructOps->HasNetDeltaSerializer())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has a custom net delta serializer.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_NetDeltaSerializeNative);
	}

	if (CppStructOps->IsPlainOldData())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls is plain old data.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_IsPlainOldData | STRUCT_NoDestructor);
	}
	else
	{
		if (CppStructOps->HasCopy())
		{
			UE_LOGF(LogClass, Verbose, "Native struct %ls has a native copy.",*GetName());
			StructFlags = EStructFlags(StructFlags | STRUCT_CopyNative);
		}
		if (CppStructOps->HasMoveAssign())
		{
			UE_LOGF(LogClass, Verbose, "Native struct %ls has a native move assign.",*GetName());
			StructFlags = EStructFlags(StructFlags | STRUCT_MoveAssignNative);

			// Move-only types are not yet supported. MoveAssignScriptStruct falls back to
			// CopyScriptStruct for code paths that haven't been updated, so WithCopy must
			// also be set to ensure correct behavior in those fallback paths.
			checkf(CppStructOps->HasCopy(), TEXT("Native struct %s has WithMoveAssign but not WithCopy. "
				"Move-only types are not yet supported; WithCopy must also be set."), *GetName());
		}
		if (!CppStructOps->HasDestructor())
		{
			UE_LOGF(LogClass, Verbose, "Native struct %ls has no destructor.",*GetName());
			StructFlags = EStructFlags(StructFlags | STRUCT_NoDestructor);
		}
	}
	if (CppStructOps->HasZeroConstructor())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has zero construction.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_ZeroConstructor);
	}
	if (CppStructOps->IsPlainOldData() && !CppStructOps->HasZeroConstructor())
	{
		// hmm, it is safe to see if this can be zero constructed, lets try
		int32 Size = CppStructOps->GetSize();
		uint8* TestData00 = (uint8*)FMemory::Malloc(Size);
		FMemory::Memzero(TestData00,Size);
		CppStructOps->Construct(TestData00);
		CppStructOps->Construct(TestData00); // slightly more like to catch "internal counters" if we do this twice
		bool IsZeroConstruct = true;
		for (int32 Index = 0; Index < Size && IsZeroConstruct; Index++)
		{
			if (TestData00[Index])
			{
				IsZeroConstruct = false;
			}
		}
		FMemory::Free(TestData00);
		if (IsZeroConstruct)
		{
			UE_LOGF(LogClass, Verbose, "Native struct %ls has DISCOVERED zero construction. Size = %d",*GetName(), Size);
			StructFlags = EStructFlags(StructFlags | STRUCT_ZeroConstructor);
		}
	}
	if (CppStructOps->HasIdentical())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has native identical.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_IdenticalNative);
	}
	if (CppStructOps->HasAddStructReferencedObjects())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has native AddStructReferencedObjects.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_AddStructReferencedObjects);
	}
	if (CppStructOps->HasExportTextItem())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has native ExportTextItem.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_ExportTextItemNative);
	}
	if (CppStructOps->HasImportTextItem())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has native ImportTextItem.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_ImportTextItemNative);
	}
	if (CppStructOps->HasSerializeFromMismatchedTag() || CppStructOps->HasStructuredSerializeFromMismatchedTag())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has native SerializeFromMismatchedTag.",*GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_SerializeFromMismatchedTag);
	}
#if WITH_EDITOR
	if (CppStructOps->HasCanEditChange())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has native CanEditChange.", *GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_CanEditChange);
	}
	if (CppStructOps->HasPostLoad())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has native PostLoad.", *GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_PostLoad);
	}
#endif

	if (CppStructOps->HasVisitor())
	{
		UE_LOGF(LogClass, Verbose, "Native struct %ls has native property Visit.", *GetName());
		StructFlags = EStructFlags(StructFlags | STRUCT_Visitor);
	}

	check(!bPrepareCppStructOpsCompleted); // recursion is unacceptable
	bPrepareCppStructOpsCompleted = true;
}

IMPLEMENT_FSTRUCTUREDARCHIVE_SERIALIZER(UScriptStruct);

void UScriptStruct::Serialize( FArchive& Ar )
{
	Super::Serialize(Ar);

	// Serialize only the non-computed struct flags.
	const uint32 ComputedStructFlags = uint32(StructFlags) & STRUCT_ComputedFlags;
	const uint32 NonComputedStructFlags = uint32(StructFlags) & ~STRUCT_ComputedFlags;
	uint32 SavedStructFlags = NonComputedStructFlags;
	Ar << SavedStructFlags;

	if (Ar.IsLoading())
	{
		StructFlags = EStructFlags(SavedStructFlags);

		ClearCppStructOps(); // we want to be sure to do this from scratch
		PrepareCppStructOps();

		// If there was no matching CppStructOps found, restore the computed struct flags that were reset by ClearCppStructOps.
		if (!CppStructOps)
		{
			StructFlags = EStructFlags((uint32(StructFlags) & ~STRUCT_ComputedFlags) | ComputedStructFlags);
		}
	}
}

bool UScriptStruct::UseBinarySerialization(const FArchive& Ar) const
{
	return !(Ar.IsLoading() || Ar.IsSaving())
		|| Ar.WantBinaryPropertySerialization()
		|| (0 != (StructFlags & STRUCT_Immutable));
}

void UScriptStruct::SerializeItem(FArchive& Ar, TNotNull<void*> Value, void const* Defaults)
{
	SerializeItem(FStructuredArchiveFromArchive(Ar).GetSlot(), Value, Defaults);
}

void UScriptStruct::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	const bool bUseBinarySerialization = UseBinarySerialization(UnderlyingArchive);
	const bool bUseNativeSerialization = UseNativeSerialization();

	// Preload struct before serialization tracking to not double count time.
	if (bUseBinarySerialization || bUseNativeSerialization)
	{
		UnderlyingArchive.Preload(this);
	}

	bool bItemSerialized = false;
	if (bUseNativeSerialization)
	{
		// Native serialization is not compatible with custom property lists.
		FGuardValue_Bitfield(Slot.GetUnderlyingArchive().ArUseCustomPropertyList, false);

		UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
		check(TheCppStructOps); // else should not have STRUCT_SerializeNative

		if (TheCppStructOps->HasStructuredSerializer())
		{
			bItemSerialized = TheCppStructOps->Serialize(Slot, Value, this, Defaults);
		}
		else
		{
#if WITH_TEXT_ARCHIVE_SUPPORT
			if (Slot.GetUnderlyingArchive().IsTextFormat())
			{
				FArchiveUObjectFromStructuredArchive Adapter(Slot);
				FArchive& Ar = Adapter.GetArchive();
				bItemSerialized = TheCppStructOps->Serialize(Ar, Value, this, Defaults);
				if (bItemSerialized && !Slot.IsFilled())
				{
					// The struct said that serialization succeeded but it didn't actually write anything.
					Slot.EnterRecord();
				}
				Adapter.Close();
			}
			else
#endif
			{
				bItemSerialized = TheCppStructOps->Serialize(Slot.GetUnderlyingArchive(), Value, this, Defaults);
				if (bItemSerialized && !Slot.IsFilled())
				{
					// The struct said that serialization succeeded but it didn't actually write anything.
					Slot.EnterRecord();
				}
			}
		}
	}

	if (!bItemSerialized)
	{
		if (bUseBinarySerialization)
		{
			// Struct is already preloaded above.
			if (!UnderlyingArchive.IsPersistent() && UnderlyingArchive.GetPortFlags() != 0 && !ShouldSerializeAtomically(UnderlyingArchive) && !UnderlyingArchive.ArUseCustomPropertyList)
			{
				SerializeBinEx(Slot, Value, Defaults, this);
			}
			else
			{
				SerializeBin(Slot, Value);
			}
		}
		else
		{
			SerializeTaggedProperties(Slot, (uint8*)NotNullGet(Value), this, (uint8*)Defaults);
		}
	}

	if (StructFlags & STRUCT_PostSerializeNative)
	{
		UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
		check(TheCppStructOps); // else should not have STRUCT_PostSerializeNative
		TheCppStructOps->PostSerialize(UnderlyingArchive, Value);
	}
}

const TCHAR* UScriptStruct::ImportText(const TCHAR* InBuffer, TNotNull<void*> Value, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText, const FString& StructName, bool bAllowNativeOverride) const
{
	return ImportText(InBuffer, Value, OwnerObject, PortFlags, ErrorText, [&StructName](){return StructName;}, bAllowNativeOverride);
}

const TCHAR* UScriptStruct::ImportText(const TCHAR* InBuffer, TNotNull<void*> Value, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText, const TFunctionRef<FString()>& StructNameGetter, bool bAllowNativeOverride) const
{
	FOutputDeviceNull NullErrorText;
	if (!ErrorText)
	{
		ErrorText = &NullErrorText;
	}

	if (bAllowNativeOverride && StructFlags & STRUCT_ImportTextItemNative)
	{
		UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
		check(TheCppStructOps); // else should not have STRUCT_ImportTextItemNative
		if (TheCppStructOps->ImportTextItem(InBuffer, Value, PortFlags, OwnerObject, ErrorText))
		{
			return InBuffer;
		}
	}

	TArray<FDefinedProperty> DefinedProperties;
	// this keeps track of the number of errors we've logged, so that we can add new lines when logging more than one error
	int32 ErrorCount = 0;
	const TCHAR* Buffer = InBuffer;
	if (*Buffer++ == TCHAR('('))
	{
		// Parse all properties.
		while (*Buffer != TCHAR(')'))
		{
			// parse and import the value
			Buffer = FProperty::ImportSingleProperty(Buffer, Value, this, OwnerObject, PortFlags | PPF_Delimited, ErrorText, DefinedProperties);

			// skip any remaining text before the next property value
			SkipWhitespace(Buffer);
			int32 SubCount = 0;
			while (*Buffer && *Buffer != TCHAR('\r') && *Buffer != TCHAR('\n') &&
				(SubCount > 0 || *Buffer != TCHAR(')')) && (SubCount > 0 || *Buffer != TCHAR(',')))
			{
				SkipWhitespace(Buffer);
				if (*Buffer == TCHAR('\"'))
				{
					do
					{
						Buffer++;
					} while (*Buffer && *Buffer != TCHAR('\"') && *Buffer != TCHAR('\n') && *Buffer != TCHAR('\r'));

					if (*Buffer != TCHAR('\"'))
					{
						ErrorText->Logf(TEXT("%sImportText (%s): Bad quoted string at: %s"), ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), *StructNameGetter(), Buffer);
						return nullptr;
					}
				}
				else if (*Buffer == TCHAR('('))
				{
					SubCount++;
				}
				else if (*Buffer == TCHAR(')'))
				{
					SubCount--;
					if (SubCount < 0)
					{
						ErrorText->Logf(TEXT("%sImportText (%s): Too many closing parenthesis in: %s"), ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), *StructNameGetter(), InBuffer);
						return nullptr;
					}
				}
				Buffer++;
			}
			if (SubCount > 0)
			{
				ErrorText->Logf(TEXT("%sImportText(%s): Not enough closing parenthesis in: %s"), ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), *StructNameGetter(), InBuffer);
				return nullptr;
			}

			// Skip comma.
			if (*Buffer == TCHAR(','))
			{
				// Skip comma.
				Buffer++;
			}
			else if (*Buffer != TCHAR(')'))
			{
				ErrorText->Logf(TEXT("%sImportText (%s): Missing closing parenthesis: %s"), ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), *StructNameGetter(), InBuffer);
				return nullptr;
			}

			SkipWhitespace(Buffer);
		}

		// Skip trailing ')'.
		Buffer++;
	}
	else
	{
		ErrorText->Logf(TEXT("%sImportText (%s): Missing opening parenthesis: %s"), ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), *StructNameGetter(), InBuffer); //-V547
		return nullptr;
	}
	return Buffer;
}

void UScriptStruct::ExportText(FString& ValueStr, TNotNull<const void*> Value, const void* Defaults, UObject* OwnerObject, int32 PortFlags, UObject* ExportRootScope, bool bAllowNativeOverride) const
{
	if (bAllowNativeOverride && StructFlags & STRUCT_ExportTextItemNative)
	{
		UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
		check(TheCppStructOps); // else should not have STRUCT_ExportTextItemNative
		if (TheCppStructOps->ExportTextItem(ValueStr, Value, Defaults, OwnerObject, PortFlags, ExportRootScope))
		{
			return;
		}
	}

	int32 Count = 0;

	// if this struct is configured to be serialized as a unit, it must be exported as a unit as well.
	if ((StructFlags & STRUCT_Atomic) != 0)
	{
		// change Defaults to match Value so that ExportText always exports this item
		Defaults = Value;
	}

	const bool bT3DOverrideEnabled = FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization);

	FString InnerValue;
	InnerValue.Reserve(512);
	for (TFieldIterator<FProperty> It(this); It; ++It)
	{
		if (It->ShouldPort(PortFlags))
		{
			for (int32 Index = 0; Index < It->ArrayDim; Index++)
			{
				const bool bStaticArray = It->ArrayDim > 1;
				FOverridableTextPortPropertyPathScope ScopePath(*It, bStaticArray ? Index : INDEX_NONE, bStaticArray ? EPropertyVisitorInfoType::StaticArrayIndex : EPropertyVisitorInfoType::None);

				if (It->ExportText_InContainer(Index, InnerValue, Value, Defaults, OwnerObject, PPF_Delimited | PortFlags, ExportRootScope))
				{
					Count++;
					if (Count == 1)
					{
						ValueStr += TCHAR('(');
					}
					else if ((PortFlags & PPF_BlueprintDebugView) == 0)
					{
						ValueStr += TCHAR(',');
					}
					else
					{
						ValueStr += TEXT(",\n");
					}

					// Add property name
					ValueStr += (PortFlags & (PPF_ExternalEditor | PPF_BlueprintDebugView)) != 0 ? It->GetAuthoredName() : It->GetName();

					if (bStaticArray)
					{
						ValueStr += FString::Printf(TEXT("[%i]"), Index);
					}

					if (bT3DOverrideEnabled)
					{
						if(const FOverriddenPropertySet* OverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties())
						{
							const FPropertyVisitorPath* Path = FOverridableSerializationLogic::GetOverriddenPortTextPropertyPath();
							checkf(Path, TEXT("Expecting a path"));
							FArchiveSerializedPropertyChain Chain = Path->ToSerializedPropertyChain();

							const EOverriddenPropertyOperation Operation = OverriddenProperties->GetOverriddenPropertyOperation(&Chain, /*Property*/nullptr);
							if (Operation != EOverriddenPropertyOperation::None)
							{
								ValueStr += FString::Printf(TEXT("<%s>"), *GetOverriddenOperationString(Operation));
							}
						}
					}

					ValueStr += TCHAR('=');;
					ValueStr += InnerValue;
				}
				InnerValue.Reset();
			}
		}
	}

	if (Count > 0)
	{
		ValueStr += TEXT(")");
	}
	else
	{
		ValueStr += TEXT("()");
	}
}

void UScriptStruct::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	Super::Link(Ar, bRelinkExistingProperties);
	SetStructTrashed(false);
	if (!HasDefaults()) // if you have CppStructOps, then that is authoritative, otherwise we look at the properties
	{
		StructFlags = EStructFlags(StructFlags | STRUCT_ZeroConstructor | STRUCT_NoDestructor | STRUCT_IsPlainOldData);
		for( FProperty* Property = PropertyLink; Property; Property = Property->PropertyLinkNext )
		{
			if (!Property->HasAnyPropertyFlags(CPF_ZeroConstructor))
			{
				StructFlags = EStructFlags(StructFlags & ~STRUCT_ZeroConstructor);
			}
			if (!Property->HasAnyPropertyFlags(CPF_NoDestructor))
			{
				StructFlags = EStructFlags(StructFlags & ~STRUCT_NoDestructor);
			}
			if (!Property->HasAnyPropertyFlags(CPF_IsPlainOldData))
			{
				StructFlags = EStructFlags(StructFlags & ~STRUCT_IsPlainOldData);
			}
		}
		if (StructFlags & STRUCT_IsPlainOldData)
		{
			UE_LOGF(LogClass, Verbose, "Non-Native struct %ls is plain old data.",*GetName());
		}
		if (StructFlags & STRUCT_NoDestructor)
		{
			UE_LOGF(LogClass, Verbose, "Non-Native struct %ls has no destructor.",*GetName());
		}
		if (StructFlags & STRUCT_ZeroConstructor)
		{
			UE_LOGF(LogClass, Verbose, "Non-Native struct %ls has zero construction.",*GetName());
		}
	}
}

bool UScriptStruct::CompareScriptStruct(const void* A, const void* B, uint32 PortFlags) const
{
	check(A);

	if (nullptr == B) // if the comparand is NULL, we just call this no-match
	{
		return false;
	}

	if (StructFlags & STRUCT_IdenticalNative)
	{
		UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
		check(TheCppStructOps);
		bool bResult = false;
		if (TheCppStructOps->Identical(A, B, PortFlags, bResult))
		{
			return bResult;
		}
	}

	for( TFieldIterator<FProperty> It(this); It; ++It )
	{
		for( int32 i=0; i<It->ArrayDim; i++ )
		{
			if( !It->Identical_InContainer(A,B,i,PortFlags) )
			{
				return false;
			}
		}
	}
	return true;
}


void UScriptStruct::CopyScriptStruct(void* InDest, void const* InSrc, int32 ArrayDim) const
{
	uint8 *Dest = (uint8*)InDest;
	check(Dest);
	uint8 const* Src = (uint8 const*)InSrc;
	check(Src);

	int32 Stride = GetStructureSize();

	if (StructFlags & STRUCT_CopyNative)
	{
		check(!(StructFlags & STRUCT_IsPlainOldData)); // should not have both
		UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
		check(TheCppStructOps);
		check(Stride == TheCppStructOps->GetSize() && PropertiesSize <= Stride);
		if (TheCppStructOps->Copy(Dest, Src, ArrayDim))
		{
			return;
		}
	}
	if (StructFlags & STRUCT_IsPlainOldData)
	{
		FMemory::Memcpy(Dest, Src, ArrayDim * Stride);
	}
	else
	{
		for( TFieldIterator<FProperty> It(this); It; ++It )
		{
			for (int32 Index = 0; Index < ArrayDim; Index++)
			{
				It->CopyCompleteValue_InContainer((uint8*)Dest + Index * Stride,(uint8*)Src + Index * Stride);
			}
		}
	}
}

void UScriptStruct::MoveAssignScriptStruct(void* InDest, void* InSrc, int32 ArrayDim) const
{
	uint8* Dest = (uint8*)InDest;
	check(Dest);
	uint8* Src = (uint8*)InSrc;
	check(Src);

	int32 Stride = GetStructureSize();

	if (StructFlags & STRUCT_MoveAssignNative)
	{
		check(!(StructFlags & STRUCT_IsPlainOldData)); // should not have both
		UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
		check(TheCppStructOps);
		check(Stride == TheCppStructOps->GetSize() && PropertiesSize == Stride);
		if (TheCppStructOps->MoveAssign(Dest, Src, ArrayDim))
		{
			return;
		}
	}

	// Fallback: use copy. For POD this is memcpy; for non-POD this is copy assignment.
	// Source is not modified — caller's destructor handles cleanup as before.
	CopyScriptStruct(Dest, Src, ArrayDim);
}

uint32 UScriptStruct::GetStructTypeHash(const void* Src) const
{
	// Calling GetStructTypeHash on struct types that doesn't provide a native
	// GetTypeHash implementation is an error that neither the C++ compiler nor the BP
	// compiler permit. Still, old reflection data could be loaded that invalidly uses
	// unhashable types.

	// If any the ensure or check in this function fires the fix is to implement GetTypeHash
	// or erase the data. USetProperties and UMapProperties that are loaded from disk
	// will clear themselves when they detect this error (see FSetProperty and
	// FMapProperty::ConvertFromType).

	UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
	return TheCppStructOps->GetStructTypeHash(Src);
}

void UScriptStruct::InitializeStruct(void* InDest, int32 ArrayDim) const
{
	uint8 *Dest = (uint8*)InDest;
	check(Dest);

	int32 Stride = GetStructureSize();

	//@todo UE optimize
	FMemory::Memzero(Dest, ArrayDim * Stride);

	int32 InitializedSize = 0;
	UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
	if (TheCppStructOps != NULL)
	{
		if (!TheCppStructOps->HasZeroConstructor())
		{
			for (int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++)
			{
				void* PropertyDest = Dest + ArrayIndex * Stride;
				checkf(IsAligned(PropertyDest, TheCppStructOps->GetAlignment()),
					TEXT("Destination address for property does not match requirement of %d byte alignment for %s"),
					TheCppStructOps->GetAlignment(),
					*GetPathNameSafe(this));
				TheCppStructOps->Construct(PropertyDest);
			}
		}

		InitializedSize = TheCppStructOps->GetSize();
		// here we want to make sure C++ and the property system agree on the size
		check(Stride == InitializedSize && PropertiesSize <= InitializedSize);
	}

	if (PropertiesSize > InitializedSize)
	{
		bool bHitBase = false;
		for (FProperty* Property = PropertyLink; Property && !bHitBase; Property = Property->PropertyLinkNext)
		{
			if (!Property->IsInContainer(InitializedSize))
			{
				for (int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++)
				{
					Property->InitializeValue_InContainer(Dest + ArrayIndex * Stride);
				}
			}
			else
			{
				bHitBase = true;
			}
		}
	}
}

void UScriptStruct::InitializeDefaultValue(uint8* InStructData) const
{
	InitializeStruct(InStructData);
}

bool UScriptStruct::FindInnerPropertyInstance(FName PropertyName, const void* Data, const FProperty*& OutProp, const void*& OutData) const
{
	if (const UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps())
	{
        if (TheCppStructOps->HasFindInnerPropertyInstance())
        {
            return TheCppStructOps->FindInnerPropertyInstance(PropertyName, Data, OutProp, OutData);
        }
	}

	return false;
}

EPropertyVisitorControlFlow UScriptStruct::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const
{
	const EPropertyVisitorControlFlow RetVal = UStruct::Visit(Context, InFunc);
	if (RetVal == EPropertyVisitorControlFlow::Stop)
	{
		return EPropertyVisitorControlFlow::Stop;
	}
	if (RetVal == EPropertyVisitorControlFlow::StepOut)
	{
		return EPropertyVisitorControlFlow::StepOver;
	}
	if (StructFlags & STRUCT_Visitor)
	{
		checkf(CppStructOps && CppStructOps->HasVisitor(), TEXT("Expecting to have a visitor implementation when STRUCT_Visitor is set"));
		return CppStructOps->Visit(Context, InFunc);
	}
	return RetVal;
}

void* UScriptStruct::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	if (void* InnerData = Super::ResolveVisitedPathInfo(Data, Info))
	{
		return InnerData;
	}

	if (StructFlags & STRUCT_Visitor)
	{
		checkf(CppStructOps && CppStructOps->HasVisitor(), TEXT("Expecting to have a visitor implementation when STRUCT_Visitor is set"));
		return CppStructOps->ResolveVisitedPathInfo(Data, Info);
	}

	return nullptr;
}

void UScriptStruct::ClearScriptStruct(void* Dest, int32 ArrayDim) const
{
	uint8 *Data = (uint8*)Dest;
	int32 Stride = GetStructureSize();

	int32 ClearedSize = 0;
	UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
	if (TheCppStructOps)
	{
		for ( int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++ )
		{
			uint8* PropertyData = Data + ArrayIndex * Stride;
			if (TheCppStructOps->HasDestructor())
			{
				TheCppStructOps->Destruct(PropertyData);
			}
			if (TheCppStructOps->HasZeroConstructor())
			{
				FMemory::Memzero(PropertyData, Stride);
			}
			else
			{
				TheCppStructOps->Construct(PropertyData);
			}
		}
		ClearedSize = TheCppStructOps->GetSize();
		// here we want to make sure C++ and the property system agree on the size
		check(Stride == ClearedSize && PropertiesSize <= ClearedSize);
	}
	if ( PropertiesSize > ClearedSize )
	{
		bool bHitBase = false;
		for ( FProperty* Property = PropertyLink; Property && !bHitBase; Property = Property->PropertyLinkNext )
		{
			if (!Property->IsInContainer(ClearedSize))
			{
				for ( int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++ )
				{
					for ( int32 PropArrayIndex = 0; PropArrayIndex < Property->ArrayDim; PropArrayIndex++ )
					{
						Property->ClearValue_InContainer(Data + ArrayIndex * Stride, PropArrayIndex);
					}
				}
			}
			else
			{
				bHitBase = true;
			}
		}
	}

}

void UScriptStruct::DestroyStruct(void* Dest, int32 ArrayDim) const
{
	if (StructFlags & (STRUCT_IsPlainOldData | STRUCT_NoDestructor))
	{
		return; // POD types don't need destructors
	}
	uint8 *Data = (uint8*)Dest;
	int32 Stride = GetStructureSize();
	int32 ClearedSize = 0;

	UScriptStruct::ICppStructOps* TheCppStructOps = GetCppStructOps();
	if (TheCppStructOps)
	{
		if (TheCppStructOps->HasDestructor())
		{
			for ( int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++ )
			{
				uint8* PropertyData = (uint8*)Dest + ArrayIndex * Stride;
				TheCppStructOps->Destruct(PropertyData);
			}
		}
		ClearedSize = TheCppStructOps->GetSize();
		// here we want to make sure C++ and the property system agree on the size
		checkf(Stride == ClearedSize && PropertiesSize <= ClearedSize, TEXT("C++ and the property system struct size mismatch for %s (C++ Size: %d, Aligned Property Size: %d, Unaligned Property Size: %d)"),
			*GetPathName(), ClearedSize, Stride, PropertiesSize);
	}

	if (PropertiesSize > ClearedSize)
	{
		bool bHitBase = false;
		for (FProperty* P = DestructorLink; P  && !bHitBase; P = P->DestructorLinkNext)
		{
			if (!P->IsInContainer(ClearedSize))
			{
				if (!P->HasAnyPropertyFlags(CPF_NoDestructor))
				{
					for ( int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++ )
					{
						P->DestroyValue_InContainer(Data + ArrayIndex * Stride);
					}
				}
			}
			else
			{
				bHitBase = true;
			}
		}
	}
}

bool UScriptStruct::IsStructTrashed() const
{
	return !!(StructFlags & STRUCT_Trashed);
}

void UScriptStruct::SetStructTrashed(bool bIsTrash)
{
	if (bIsTrash)
	{
		StructFlags = EStructFlags(StructFlags | STRUCT_Trashed);
	}
	else
	{
		StructFlags = EStructFlags(StructFlags & ~STRUCT_Trashed);
	}
}

void UScriptStruct::RecursivelyPreload() {}

FGuid UScriptStruct::GetCustomGuid() const
{
	return FGuid();
}

FString UScriptStruct::GetStructCPPName(uint32 CPPExportFlags/*=0*/) const
{
	return FString::Printf(TEXT("F%s"), *GetName());
}

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)

enum class EScriptStructTestCtorSyntax
{
	NoInit = 0,
	CompilerZeroed = 1
};

struct FScriptStructTestWrapper
{
public:

	FScriptStructTestWrapper(UScriptStruct* InStruct, uint8 InInitValue = 0xFD, EScriptStructTestCtorSyntax InConstrutorSyntax = EScriptStructTestCtorSyntax::NoInit)
		: ScriptStruct(InStruct)
		, TempBuffer(nullptr)
		, InitValue(InInitValue)
		, ConstrutorSyntax(InConstrutorSyntax)
		, bTempBufferAttemptedCreate(false)
	{
	}

	~FScriptStructTestWrapper()
	{
		if (TempBuffer != nullptr)
		{
			// Destroy it
			ScriptStruct->DestroyStruct(TempBuffer);
			FMemory::Free(TempBuffer);
		}
	}

	static bool CanRunTests(UScriptStruct* Struct)
	{
		return (Struct != nullptr) && Struct->IsNative() && (!Struct->GetCppStructOps() || !Struct->GetCppStructOps()->HasZeroConstructor());
	}

	uint8* GetData()
	{
		if (!bTempBufferAttemptedCreate)
		{
			AttemptCreateTempBuffer();
		}
		return TempBuffer;
	}
private:

	void AttemptCreateTempBuffer()
	{
		if (ScriptStruct->IsNative())
		{
			UScriptStruct::ICppStructOps* StructOps = ScriptStruct->GetCppStructOps();

			// Make one
			if ((StructOps != nullptr) && StructOps->HasZeroConstructor())
			{
				// These structs have basically promised to be used safely, not going to audit them
			}
			else
			{
				// Allocate space for the struct
				int32 RequiredAllocSize = ScriptStruct->GetStructureSize();
				if (StructOps)
				{
					const int32 CppAllocSize = Align(StructOps->GetSize(), ScriptStruct->GetMinAlignment());
					if (!ensure(RequiredAllocSize >= CppAllocSize))
					{
						UE_LOGF(LogClass, Warning, "Struct %ls%ls has Cpp alloc size = %d > ScriptStruct->GetStructureSize() = %d, this could result in allocations that are too small to fit the structure.",
							ScriptStruct->GetPrefixCPP(), *ScriptStruct->GetName(), CppAllocSize, RequiredAllocSize);
						// We'd probably crash below, so use a larger allocated size.
						RequiredAllocSize = StructOps->GetSize();
					}
				}
				TempBuffer = (uint8*)FMemory::Malloc(RequiredAllocSize, ScriptStruct->GetMinAlignment());

				// The following section is a partial duplication of ScriptStruct->InitializeStruct, except we initialize with 0xFD instead of 0x00
				FMemory::Memset(TempBuffer, InitValue, RequiredAllocSize);

				int32 InitializedSize = 0;
				if (StructOps != nullptr)
				{
					if (ConstrutorSyntax == EScriptStructTestCtorSyntax::NoInit)
					{
						StructOps->ConstructForTests(TempBuffer);
					}
					else
					{
						StructOps->Construct(TempBuffer);
					}
					InitializedSize = StructOps->GetSize();
				}

				if (ScriptStruct->PropertiesSize > InitializedSize)
				{
					bool bHitBase = false;
					for (FProperty* Property = ScriptStruct->PropertyLink; Property && !bHitBase; Property = Property->PropertyLinkNext)
					{
						if (!Property->IsInContainer(InitializedSize))
						{
							Property->InitializeValue_InContainer(TempBuffer);
						}
						else
						{
							bHitBase = true;
						}
					}
				}

				if (ScriptStruct->StructFlags & STRUCT_PostScriptConstruct)
				{
					check(StructOps);
					StructOps->PostScriptConstruct(TempBuffer);
				}
			}
		}

		bTempBufferAttemptedCreate = true;
	}

	UScriptStruct* ScriptStruct;
	uint8* TempBuffer;
	uint8 InitValue;
	EScriptStructTestCtorSyntax ConstrutorSyntax;
	bool bTempBufferAttemptedCreate;
};

static void FindUninitializedScriptStructMembers(UScriptStruct* ScriptStruct, EScriptStructTestCtorSyntax ConstructorSyntax, TSet<const FProperty*>& OutUninitializedProperties)
{
	FScriptStructTestWrapper WrapperFE(ScriptStruct, 0xFE, ConstructorSyntax);
	FScriptStructTestWrapper Wrapper00(ScriptStruct, 0x00, ConstructorSyntax);
	FScriptStructTestWrapper WrapperAA(ScriptStruct, 0xAA, ConstructorSyntax);
	FScriptStructTestWrapper Wrapper55(ScriptStruct, 0x55, ConstructorSyntax);

	const void* BadPointer = (void*)0xFEFEFEFEFEFEFEFEull;

	for (const FProperty* Property : TFieldRange<FProperty>(ScriptStruct, EFieldIteratorFlags::ExcludeSuper))
	{
#if	WITH_EDITORONLY_DATA
		static const FName NAME_IgnoreForMemberInitializationTest("IgnoreForMemberInitializationTest");
		if (Property->HasMetaData(NAME_IgnoreForMemberInitializationTest))
		{
			continue;
		}
#endif // WITH_EDITORONLY_DATA

		if (const FObjectProperty* ObjectPtrProperty = CastField<const FObjectProperty>(Property))
		{
			//using reinterpret_cast to avoid any methods of TObjectPtr being invoked
			const TObjectPtr<UObject>* PropValue = ObjectPtrProperty->GetPropertyValuePtr_InContainer(WrapperFE.GetData());
			const void* const* RawValue = reinterpret_cast<const void* const*>(PropValue);
			if (*RawValue == BadPointer)
			{
				OutUninitializedProperties.Add(Property);
			}
		}
		else if (const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property))
		{
			// Check any reflected pointer properties to make sure they got initialized
			const UObject* PropValue = ObjectProperty->GetObjectPropertyValue_InContainer(WrapperFE.GetData());
			if (PropValue == BadPointer)
			{
				OutUninitializedProperties.Add(Property);
			}
		}
		else if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(Property))
		{
			// Check for uninitialized boolean properties (done separately to deal with byte-wide booleans that would evaluate to true with either 0x55 or 0xAA)
			const bool bValue0 = BoolProperty->GetPropertyValue_InContainer(Wrapper00.GetData());
			const bool bValue1 = BoolProperty->GetPropertyValue_InContainer(WrapperFE.GetData());

			if (bValue0 != bValue1)
			{
				OutUninitializedProperties.Add(Property);
			}
		}
		else if (Property->IsA(FNameProperty::StaticClass()))
		{
			// Skip some other types that will crash in equality with garbage data
			//@TODO: Shouldn't need to skip FName, it's got a default ctor that initializes correctly...
		}
		else
		{
			bool bShouldInspect = true;
			if (Property->IsA(FStructProperty::StaticClass()))
			{
				// Skip user defined structs since we will consider those structs directly.
				// Calling again here will just result in false positives
				const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
				bShouldInspect = (StructProperty->Struct->StructFlags & STRUCT_NoExport) != 0;
			}

			if (bShouldInspect)
			{
				// Catch all remaining properties

				// Uncomment the following line to aid finding crash sources encountered while running this test. A crash usually indicates an uninitialized pointer
				// UE_LOGF(LogClass, Log, "Testing %ls%ls::%ls for proper initialization", ScriptStruct->GetPrefixCPP(), *ScriptStruct->GetName(), *Property->GetNameCPP());
				if (!Property->Identical_InContainer(WrapperAA.GetData(), Wrapper55.GetData()))
				{
					OutUninitializedProperties.Add(Property);
				}
			}
		}
	}
}

static FString GetFieldLocation(const UField* Field)
{
	check(Field);
	UPackage* ScriptPackage = Field->GetOutermost();
	FString StructLocation = FString::Printf(TEXT(" Module:%s"), *FPackageName::GetShortName(ScriptPackage->GetName()));
#if WITH_EDITORONLY_DATA
	static const FName NAME_ModuleRelativePath("ModuleRelativePath");
	const FString& ModuleRelativeIncludePath = Field->GetMetaData(NAME_ModuleRelativePath);
	if (!ModuleRelativeIncludePath.IsEmpty())
	{
		StructLocation += FString::Printf(TEXT(" File:%s"), *ModuleRelativeIncludePath);
	}
#endif // WITH_EDITORONLY_DATA
	return StructLocation;
};

int32 FStructUtils::AttemptToFindUninitializedScriptStructMembers()
{
	struct FExecuteIfExceedsTimeLimit
	{
		FExecuteIfExceedsTimeLimit(double InTimeLimit, TFunction<void(double)> InFunc)
			: StartTime(FPlatformTime::Seconds()), TimeLimit(InTimeLimit), Func(InFunc) {}

		~FExecuteIfExceedsTimeLimit()
		{
			double Time = FPlatformTime::Seconds() - StartTime;
			if (Time > TimeLimit)
			{
				Func(Time);
			}
		}
		double StartTime;
		double TimeLimit;
		TFunction<void(double)> Func;
	};

	FExecuteIfExceedsTimeLimit TotalTimeLimit(2.0, [](double Time) {
		UE_LOGF(LogClass, Display, "AttemptToFindUninitializedScriptStructMembers took more than 2s to complete. Time: %.2f", Time);
	});

	auto DetermineIfModuleIsEngine = [](const UScriptStruct* ScriptStruct) -> bool
	{
		UPackage* ScriptPackage = ScriptStruct->GetOutermost();
		const FName ScriptModuleName = FPackageName::GetShortFName(ScriptPackage->GetName());

		FModuleStatus ScriptModuleStatus;
		if (FModuleManager::Get().QueryModule(ScriptModuleName, /*out*/ ScriptModuleStatus))
		{
			const bool bIsProjectModule = ScriptModuleStatus.FilePath.StartsWith(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()));
			return !bIsProjectModule;
		}
		else
		{
			// Default to project if we can't determine it (shouldn't ever happen tho)
			return false;
		}
	};

	struct FScriptStructSettings
	{
		ELogVerbosity::Type ProjectVerbosity = ELogVerbosity::Display;
		ELogVerbosity::Type EngineVerbosity = ELogVerbosity::Display;
		ELogVerbosity::Type PointerVerbosity = ELogVerbosity::Warning;

		FScriptStructSettings()
		{
			FExecuteIfExceedsTimeLimit SettingsTimeLimit(0.05, [](double Time) {
				UE_LOGF(LogClass, Display, "AttemptToFindUninitializedScriptStructMembers took more than 50ms to construct the FScriptStructSettings. Time: %.1f", Time * 1000);
				});
			{
				FString ProjectSettingString;
				if (GConfig->GetString(TEXT("CoreUObject.UninitializedScriptStructMembersCheck"), TEXT("ProjectModuleReflectedUninitializedPropertyVerbosity"), ProjectSettingString, GEngineIni))
				{
					ProjectVerbosity = ParseLogVerbosityFromString(ProjectSettingString);
				}
			}

			{
				FString EngineSettingString;
				if (GConfig->GetString(TEXT("CoreUObject.UninitializedScriptStructMembersCheck"), TEXT("EngineModuleReflectedUninitializedPropertyVerbosity"), EngineSettingString, GEngineIni))
				{
					EngineVerbosity = ParseLogVerbosityFromString(EngineSettingString);
				}
			}

			//@TODO: Remove this eventually and change the default to Error (or maybe even Fatal)
			{
				FString PointerSettingString;
				if (GConfig->GetString(TEXT("CoreUObject.UninitializedScriptStructMembersCheck"), TEXT("ObjectReferenceReflectedUninitializedPropertyVerbosity"), PointerSettingString, GEngineIni))
				{
					PointerVerbosity = ParseLogVerbosityFromString(PointerSettingString);
				}
			}
		}
	};

	static FScriptStructSettings Settings;

	int32 UninitializedScriptStructMemberCount = 0;
	int32 UninitializedObjectPropertyCount = 0;
	UScriptStruct* TestUninitializedScriptStructMembersTestStruct = StaticStruct<FTestUninitializedScriptStructMembersTest>();
	check(TestUninitializedScriptStructMembersTestStruct != nullptr);

	{
		const void* BadPointer = (void*)0xFEFEFEFEFEFEFEFEull;

		// First test if the tests aren't broken
		FScriptStructTestWrapper WrapperFE(TestUninitializedScriptStructMembersTestStruct, 0xFE);
		const FObjectProperty* UninitializedProperty = CastFieldChecked<const FObjectProperty>(TestUninitializedScriptStructMembersTestStruct->FindPropertyByName(TEXT("UninitializedObjectReference")));
		const FObjectProperty* InitializedProperty = CastFieldChecked<const FObjectProperty>(TestUninitializedScriptStructMembersTestStruct->FindPropertyByName(TEXT("InitializedObjectReference")));

		//using reinterpret_cast to avoid any methods of TObjectPtr being invoked
		const TObjectPtr<UObject>* UninitializedPropValue = UninitializedProperty->GetPropertyValuePtr_InContainer(WrapperFE.GetData());
		const void* const* RawValue = reinterpret_cast<const void* const*>(UninitializedPropValue);
		if (*RawValue != BadPointer)
		{
			UE_LOGF(LogClass, Error, "ObjectProperty %ls%ls::%ls seems to be initialized properly but it shouldn't be. Verify that AttemptToFindUninitializedScriptStructMembers() is working properly",
				TestUninitializedScriptStructMembersTestStruct->GetPrefixCPP(), *TestUninitializedScriptStructMembersTestStruct->GetName(), *UninitializedProperty->GetNameCPP());
		}
		const TObjectPtr<UObject>* InitializedPropValue = InitializedProperty->GetPropertyValuePtr_InContainer(WrapperFE.GetData());
		RawValue = reinterpret_cast<const void* const*>(InitializedPropValue);
		if (*RawValue != nullptr)
		{
			UE_LOGF(LogClass, Error, "ObjectProperty %ls%ls::%ls seems to be not initialized properly but it should be. Verify that AttemptToFindUninitializedScriptStructMembers() is working properly",
				TestUninitializedScriptStructMembersTestStruct->GetPrefixCPP(), *TestUninitializedScriptStructMembersTestStruct->GetName(), *InitializedProperty->GetNameCPP());
		}
	}

	double PreScriptStructTime = FPlatformTime::Seconds() - TotalTimeLimit.StartTime;
	if (PreScriptStructTime > 0.05)
	{
		UE_LOGF(LogClass, Display, "AttemptToFindUninitializedScriptStructMembers took more than 50ms before starting to check ScriptStructs. Time(ms):%.1f", PreScriptStructTime * 1000);
	}

	TSet<const FProperty*> UninitializedPropertiesNoInit;
	TSet<const FProperty*> UninitializedPropertiesZeroed;
	for (TObjectIterator<UScriptStruct> ScriptIt; ScriptIt; ++ScriptIt)
	{
		UScriptStruct* ScriptStruct = *ScriptIt;
		FExecuteIfExceedsTimeLimit StructTimeLimit(0.001, [ScriptStruct](double Time) {
			UE_LOGF(LogClass, Display, "AttemptToFindUninitializedScriptStructMembers took more than 1ms to process ScriptStruct %ls. Time(ms): %.1f", ScriptStruct ? *ScriptStruct->GetPathName() : TEXT("None"), Time * 1000);
			});

		if (!FScriptStructTestWrapper::CanRunTests(ScriptStruct) || ScriptStruct == TestUninitializedScriptStructMembersTestStruct)
		{
			continue;
		}

		UninitializedPropertiesNoInit.Reset();
		UninitializedPropertiesZeroed.Reset();

		// Test the struct by constructing it with 'new FMyStruct();' syntax first. The compiler should zero all members in this case if the
		// struct doesn't have a custom default constructor defined
		FindUninitializedScriptStructMembers(ScriptStruct, EScriptStructTestCtorSyntax::CompilerZeroed, UninitializedPropertiesZeroed);
		// Test the struct by constructing it with 'new FStruct;' syntax in which case the compiler doesn't zero the properties automatically
		FindUninitializedScriptStructMembers(ScriptStruct, EScriptStructTestCtorSyntax::NoInit, UninitializedPropertiesNoInit);

		if (UninitializedPropertiesNoInit.Num() == 0 && UninitializedPropertiesZeroed.Num() == 0)
		{
			continue;
		}

		ELogVerbosity::Type StructVerbosity = DetermineIfModuleIsEngine(ScriptStruct) ? Settings.EngineVerbosity : Settings.ProjectVerbosity;
		ELogVerbosity::Type PointerVerbosity = FMath::Min(StructVerbosity, Settings.PointerVerbosity);
		auto LogUninitializedProperty =
			[&UninitializedScriptStructMemberCount, &UninitializedObjectPropertyCount, ScriptStruct, StructVerbosity, PointerVerbosity]
			(const FProperty* Property, const TCHAR* MessageText)
		{
			++UninitializedScriptStructMemberCount;
			ELogVerbosity::Type Verbosity = StructVerbosity;
			if (Property->IsA<FObjectPropertyBase>())
			{
				++UninitializedObjectPropertyCount;
				Verbosity = PointerVerbosity;
			}
#if !NO_LOGGING
			// LogClass: Error: SetProperty FStructSerializerSetTestStruct::StructSet is not initialized properly even though its struct probably has a custom default constructor. Module:Serialization
			FMsg::Logf(__FILE__, __LINE__, LogClass.GetCategoryName(), Verbosity, TEXT("%s %s%s::%s %s.%s"),
				*Property->GetClass()->GetName(),
				ScriptStruct->GetPrefixCPP(),
				*ScriptStruct->GetName(),
				*Property->GetNameCPP(),
				MessageText,
				*GetFieldLocation(ScriptStruct));
#endif
		};

		for (const FProperty* Property : UninitializedPropertiesZeroed)
		{
			LogUninitializedProperty(
				Property,
				TEXT("is not initialized properly even though its struct probably has a custom default constructor. "
					"Non deterministic fields should use UPROPERTY(Meta = (IgnoreForMemberInitializationTest)) to avoid errors from this test"));
		}

		for (const FProperty* Property : UninitializedPropertiesNoInit)
		{
			if (UninitializedPropertiesZeroed.Contains(Property))
			{
				continue;
			}
			LogUninitializedProperty(Property, TEXT("is not initialized properly"));
		}
	}

	if (UninitializedScriptStructMemberCount > 0)
	{
		UE_LOGF(LogClass, Display, "%i Uninitialized script struct members found including %d object properties", UninitializedScriptStructMemberCount, UninitializedObjectPropertyCount);
	}

	return UninitializedScriptStructMemberCount;
}

#include "HAL/IConsoleManager.h"

FAutoConsoleCommandWithWorldAndArgs GCmdListBadScriptStructs(
	TEXT("CoreUObject.AttemptToFindUninitializedScriptStructMembers"),
	TEXT("Finds USTRUCT() structs that fail to initialize reflected member variables"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
		[](const TArray<FString>& Params, UWorld* World)
		{
			FStructUtils::AttemptToFindUninitializedScriptStructMembers();
		}));

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestAttemptToFindUninitializedScriptStructMembers, FAutomationTestUObjectClassBase, "UObject.Class AttemptToFindUninitializedScriptStructMembers", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::SmokeFilter)
bool FAutomationTestAttemptToFindUninitializedScriptStructMembers::RunTest(const FString& Parameters)
{
	// This test fails when running tests under UHT because there is no TestUninitializedScriptStructMembersTest, so just skip it in that config.
	if (UObjectInitialized())
	{
		FStructUtils::AttemptToFindUninitializedScriptStructMembers();
		return !HasAnyErrors();
	}
	else
	{
		return true;
	}
}

#if WITH_TESTS
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FTestUninitializedStructMemberTest, "System::CoreUObject::ScriptStruct::UninitializedMemberTest", "[CoreUObject]")
{
	const void* BadPointer = (void*)0xFEFEFEFEFEFEFEFEull;
	UScriptStruct* TestUninitializedScriptStructMembersTestStruct = StaticStruct<FTestUninitializedScriptStructMembersTest>();

	// First test if the tests aren't broken
	FScriptStructTestWrapper WrapperFE(TestUninitializedScriptStructMembersTestStruct, 0xFE);
	const FObjectProperty* UninitializedProperty = CastFieldChecked<const FObjectProperty>(TestUninitializedScriptStructMembersTestStruct->FindPropertyByName(TEXT("UninitializedObjectReference")));
	const FObjectProperty* InitializedProperty = CastFieldChecked<const FObjectProperty>(TestUninitializedScriptStructMembersTestStruct->FindPropertyByName(TEXT("InitializedObjectReference")));

	//using reinterpret_cast to avoid any methods of TObjectPtr being invoked
	const TObjectPtr<UObject>* UninitializedPropValue = UninitializedProperty->GetPropertyValuePtr_InContainer(WrapperFE.GetData());
	const void* const* RawValue = reinterpret_cast<const void* const*>(UninitializedPropValue);
	CHECK_MESSAGE("FTestUninitializedScriptStructMembersTest::UninitializedObjectReference should be BadPointer value", *RawValue == BadPointer);

	const TObjectPtr<UObject>* InitializedPropValue = InitializedProperty->GetPropertyValuePtr_InContainer(WrapperFE.GetData());
	RawValue = reinterpret_cast<const void* const*>(InitializedPropValue);
	CHECK_MESSAGE("FTestUninitializedScriptStructMembersTest::InitializedObjectReference should be null", *RawValue == nullptr);
}
#endif

#if WITH_METADATA
/**
* Checks if MetaData value contains short type name
* @param MetaDataKey MetaData key name
* @param MetaDataValue Value stored under MetaData key
* @param OutParsedMetaDataValue If short type name is stored in MetaData value this parameter will contain the short type name
* @returns true if MetaData value contains short type name
**/
static bool CheckIfMetaDataValueIsShortTypeName(FName MetaDataKey, const FString& MetaDataValue, FString& OutParsedMetaDataValue)
{
	// Some keys require additional parsing
	static const FName NAME_RequiredAssetDataTags("RequiredAssetDataTags");

	if (MetaDataKey == NAME_RequiredAssetDataTags) // @see SPropertyEditorAsset::InitializeAssetDataTags
	{
		TArray<FString> RequiredAssetDataTagsAndValues;
		MetaDataValue.ParseIntoArray(RequiredAssetDataTagsAndValues, TEXT(","), true);

		for (const FString& TagAndOptionalValueString : RequiredAssetDataTagsAndValues)
		{
			TArray<FString> TagAndOptionalValue;
			TagAndOptionalValueString.ParseIntoArray(TagAndOptionalValue, TEXT("="), true);
			if (TagAndOptionalValue.Num() == 2 && TagAndOptionalValue[0] == TEXT("RowStructure"))
			{
				if (FPackageName::IsShortPackageName(TagAndOptionalValue[1]))
				{
					OutParsedMetaDataValue = TagAndOptionalValue[1];
					return true;
				}
			}
		}
	}
	else if (FPackageName::IsShortPackageName(MetaDataValue))
	{
		OutParsedMetaDataValue = MetaDataValue;
		return true;
	}
	return false;
}

/** Tries to find a path name for short type name **/
static FString GetSuggestedPathNameForTypeShortName(const FString& ShortName)
{
	UField* FoundType = FindFirstObject<UField>(*ShortName);
	if (FoundType)
	{
		return FString::Printf(TEXT("Suggested pathname: \"%s\"."), *FoundType->GetPathName());
	}
	else
	{
		return FString("No valid type found in memory.");
	}
}

/** Logs a field that defines MetaData entry with a short type name **/
template <typename T>
static void LogMetaDataShortTypeName(const T* Field, FName MetaDataKey, const FString& MetaDataValue)
{
#if !NO_LOGGING
	static struct FLogShortTypeNameInMetaDataCheckSettings
	{
		ELogVerbosity::Type LogVerbosity = ELogVerbosity::Warning;

		FLogShortTypeNameInMetaDataCheckSettings()
		{
			{
				FString VerbositySettingString;
				if (GConfig->GetString(TEXT("CoreUObject.ShortTypeNameInMetaDataCheck"), TEXT("LogVerbosity"), VerbositySettingString, GEngineIni))
				{
					LogVerbosity = ParseLogVerbosityFromString(VerbositySettingString);
				}
			}
		}
	} Settings;

	if constexpr (std::is_base_of_v<T, UField>)
	{
		if (const UFunction* Func = Cast<UFunction>(Field))
		{
			UStruct* FuncOwner = CastChecked<UStruct>(Func->GetOuter());
			FMsg::Logf(__FILE__, __LINE__, LogClass.GetCategoryName(), Settings.LogVerbosity, TEXT("Function %s%s::%s defines MetaData key \"%s\" which contains short type name \"%s\". %s%s"),
				FuncOwner->GetPrefixCPP(),
				*FuncOwner->GetName(),
				*Func->GetName(),
				*MetaDataKey.ToString(),
				*MetaDataValue,
				*GetSuggestedPathNameForTypeShortName(MetaDataValue),
				*GetFieldLocation(Func));
		}
		else
		{
			FMsg::Logf(__FILE__, __LINE__, LogClass.GetCategoryName(), Settings.LogVerbosity, TEXT("%s %s%s defines MetaData key \"%s\" which contains short type name \"%s\". %s%s"),
				*Field->GetClass()->GetName(),
				Field->template IsA<UStruct>() ? CastChecked<UStruct>(Field)->GetPrefixCPP() : TEXT(""),
				*Field->GetName(),
				*MetaDataKey.ToString(),
				*MetaDataValue,
				*GetSuggestedPathNameForTypeShortName(MetaDataValue),
				*GetFieldLocation(Field));
		}
	}
	else if constexpr (std::is_same_v<T, FProperty>)
	{
		UStruct* OwnerStruct = Field->GetOwnerStruct();
		FMsg::Logf(__FILE__, __LINE__, LogClass.GetCategoryName(), Settings.LogVerbosity, TEXT("Property %s %s%s::%s defines MetaData key \"%s\" which contains short type name \"%s\". %s%s"),
			*Field->GetClass()->GetName(),
			OwnerStruct->GetPrefixCPP(),
			*OwnerStruct->GetName(),
			*Field->GetName(),
			*MetaDataKey.ToString(),
			*MetaDataValue,
			*GetSuggestedPathNameForTypeShortName(MetaDataValue),
			*GetFieldLocation(OwnerStruct));
	}
#endif //# !NO_LOGGING
}

/** Checks MetaData values for known MetaData keys to see if they contain short type names **/
template <typename T>
static int32 FindFieldMetaDataShortTypeNames(const T* Field, const TArray<FName>& MetaDataKeys)
{
	int32 ShortTypeNameCount = 0;
	for (FName MetaDataKey : MetaDataKeys)
	{
		const FString* MetaData = Field->FindMetaData(MetaDataKey);
		if (MetaData && !MetaData->IsEmpty())
		{
			FString ParsedMetaDataValue;
			int32 CommaIndex = -1;
			// Some keys can store multiple type names separated with a comma
			if (MetaData->FindChar(TCHAR(','), CommaIndex))
			{
				TArray<FString> MetaDataValues;
				MetaData->ParseIntoArrayWS(MetaDataValues, TEXT(","), true);
				for (const FString& MetaDataValue : MetaDataValues)
				{
					if (CheckIfMetaDataValueIsShortTypeName(MetaDataKey, MetaDataValue, ParsedMetaDataValue))
					{
						LogMetaDataShortTypeName(Field, MetaDataKey, ParsedMetaDataValue);
						ShortTypeNameCount++;
					}
				}
			}
			else if (CheckIfMetaDataValueIsShortTypeName(MetaDataKey, *MetaData, ParsedMetaDataValue))
			{
				LogMetaDataShortTypeName(Field, MetaDataKey, ParsedMetaDataValue);
				ShortTypeNameCount++;
			}
		}
	}
	return ShortTypeNameCount;
}

int32 FStructUtils::AttemptToFindShortTypeNamesInMetaData()
{
	static struct FShortTypeNameMetaDataSettings
	{
		TArray<FName> MetaDataKeys;
		FShortTypeNameMetaDataSettings()
		{
			TArray<FString> ConfigMetaDataKeys;
			if (GConfig->GetArray(TEXT("CoreUObject.ShortTypeNameInMetaDataCheck"), TEXT("MetaDataKeys"), ConfigMetaDataKeys, GEngineIni))
			{
				MetaDataKeys.Reserve(ConfigMetaDataKeys.Num());
				for (const FString& Key : ConfigMetaDataKeys)
				{
					MetaDataKeys.Add(FName(*Key));
				}
			}
		}
	} Settings;

	int32 ShortTypeNamesCount = 0;

	if (Settings.MetaDataKeys.Num())
	{
		for (TObjectIterator<UField> FieldIt; FieldIt; ++FieldIt)
		{
			// Check MetaData stored with classes / enums / structs
			ShortTypeNamesCount += FindFieldMetaDataShortTypeNames(*FieldIt, Settings.MetaDataKeys);

			// Check struct members but no need to check function parameters
			UStruct* StructWithMembers = Cast<UStruct>(*FieldIt);
			if (StructWithMembers && !StructWithMembers->IsA<UFunction>())
			{
				for (TFieldIterator<FProperty> PropertyIt(StructWithMembers, EFieldIterationFlags::None); PropertyIt; ++PropertyIt)
				{
					ShortTypeNamesCount += FindFieldMetaDataShortTypeNames(*PropertyIt, Settings.MetaDataKeys);
				}
			}
		}
	}

	if (ShortTypeNamesCount > 0)
	{
		UE_LOGF(LogClass, Display, "%i short type names in reflected types' MetaData", ShortTypeNamesCount);
	}

	return ShortTypeNamesCount;
}

FAutoConsoleCommandWithWorldAndArgs GCmdListShortTypeNamesInMetaData(
	TEXT("CoreUObject.AttemptToFindShortTypeNamesInMetaData"),
	TEXT("Finds short type names stored in known MetaData entries"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
		[](const TArray<FString>& Params, UWorld* World)
		{
			FStructUtils::AttemptToFindShortTypeNamesInMetaData();
		}));

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestAttemptToFindShortTypeNamesInMetaData, FAutomationTestUObjectClassBase, "UObject.Class AttemptToFindShortTypeNamesInMetaData", EAutomationTestFlags::EditorContext | EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ServerContext | EAutomationTestFlags::EngineFilter)
bool FAutomationTestAttemptToFindShortTypeNamesInMetaData::RunTest(const FString& Parameters)
{
	// This test is not necessary when running under UHT so just skip it in that config.
	return FStructUtils::AttemptToFindShortTypeNamesInMetaData() == 0;
}

#endif // WITH_METADATA

// bExactCheck - Check for places where structs serialize a different set of object references to what they declare (Conservative is always an error)
// otherwise - Check for places where structs serialize object references they don't declare (Conservative allows serializing any reference types)
static bool FindUndeclaredObjectReferencesInStructSerializers(FAutomationTestBase& Test, bool bExactCheck)
{
	struct FTestArchive : public FArchiveUObject
	{
		FTestArchive()
		{
			// Not persistent, loading, saving, etc
			ArIsObjectReferenceCollector = true;
		}

		EPropertyObjectReferenceType References = EPropertyObjectReferenceType::None;

		virtual FArchive& operator<<(FLazyObjectPtr& Value) override
		{
			References |= EPropertyObjectReferenceType::Weak;
			return *this;
		}
		virtual FArchive& operator<<(FObjectPtr& Value) override
		{
			References |= EPropertyObjectReferenceType::Strong;
			return *this;
		}
		virtual FArchive& operator<<(FSoftObjectPtr& Value) override
		{
			References |= EPropertyObjectReferenceType::Weak | EPropertyObjectReferenceType::Soft;
			return *this;
		}
		virtual FArchive& operator<<(FSoftObjectPath& Value) override
		{
			References |= EPropertyObjectReferenceType::Soft;
			return *this;
		}
		virtual FArchive& operator<<(FWeakObjectPtr& Value) override
		{
			References |= EPropertyObjectReferenceType::Weak;
			return *this;
		}
	};

	bool bFoundProblems = false;

	for (TObjectIterator<UScriptStruct> ScriptIt; ScriptIt; ++ScriptIt)
	{
		UScriptStruct* Struct = *ScriptIt;
		if ((Struct->StructFlags & STRUCT_SerializeNative) == 0)
		{
			continue;
		}

		auto LogUndeclaredObjectReference = [&bFoundProblems, &Test, Struct](ELogVerbosity::Type Verbosity, const TCHAR* RefType)
		{
			Test.AddError(FString::Printf(TEXT("Struct %s%s serializes an object reference of type EPropertyObjectReferenceType::%s but does not declare it"),
				Struct->GetPrefixCPP(),
				*Struct->GetName(),
				RefType
				));
			bFoundProblems = true;
		};
		auto LogOverdeclaredObjectReference = [&bFoundProblems, &Test, Struct](ELogVerbosity::Type Verbosity, const TCHAR* RefType)
		{
			Test.AddError(FString::Printf(TEXT("Struct %s%s declares an object reference of type EPropertyObjectReferenceType::%s but does not serialize it"),
				Struct->GetPrefixCPP(),
				*Struct->GetName(),
				RefType
				));
			bFoundProblems = true;
		};
		EPropertyObjectReferenceType DeclaredReferences = Struct->GetCppStructOps()->GetCapabilities().HasSerializerObjectReferences;
		EPropertyObjectReferenceType PropertyReferences = EPropertyObjectReferenceType::None;
		for (EPropertyObjectReferenceType Type : { EPropertyObjectReferenceType::Strong, EPropertyObjectReferenceType::Weak, EPropertyObjectReferenceType::Soft })
		{
			FProperty* Property = Struct->PropertyLink;
			TArray<const FStructProperty*> EncounteredStructProps;
			while (Property && !EnumHasAllFlags(PropertyReferences, Type))
			{
				if (Property->ContainsObjectReference(EncounteredStructProps, Type))
				{
					PropertyReferences |= Type;
				}
				Property = Property->PropertyLinkNext;
			}
		}

		FStructOnScope TempStruct(Struct);

		FTestArchive Ar;
		Struct->SerializeItem(Ar, TempStruct.GetStructMemory(), nullptr);

		if(!bExactCheck && !EnumHasAllFlags(DeclaredReferences, EPropertyObjectReferenceType::Conservative))
		{
			for (EPropertyObjectReferenceType Type : { EPropertyObjectReferenceType::Strong, EPropertyObjectReferenceType::Weak, EPropertyObjectReferenceType::Soft })
			{
				if (EnumHasAllFlags(Ar.References, Type) && !EnumHasAllFlags(DeclaredReferences | PropertyReferences, Type))
				{
					LogUndeclaredObjectReference(ELogVerbosity::Warning, LexToString(Type));
				}
			}
		}
		if(bExactCheck && EnumHasAllFlags(DeclaredReferences, EPropertyObjectReferenceType::Conservative))
		{
			FString SerializedTypes;
			for (EPropertyObjectReferenceType TypedFlag : MakeFlagsRange(Ar.References))
			{
				if (SerializedTypes.IsEmpty())
				{
					SerializedTypes = LexToString(TypedFlag);
				}
				else
				{
					SerializedTypes.Appendf(TEXT(" | %s"),LexToString(TypedFlag));
				}
			}
			if (SerializedTypes.IsEmpty())
			{
				SerializedTypes = LexToString(EPropertyObjectReferenceType::None);
			}
			Test.AddError(FString::Printf(TEXT("Struct %s%s declares its object references as EPropertyObjectReferenceType::Conservative which is never exact. Actual types serialized: %s"),
				Struct->GetPrefixCPP(),
				*Struct->GetName(),
				*SerializedTypes
			));
			bFoundProblems = true;
		}
		else if (bExactCheck)
		{
			for (EPropertyObjectReferenceType Type : { EPropertyObjectReferenceType::Strong, EPropertyObjectReferenceType::Weak, EPropertyObjectReferenceType::Soft })
			{
				if (EnumHasAllFlags(DeclaredReferences, Type) && !EnumHasAllFlags(Ar.References, Type))
				{
					LogOverdeclaredObjectReference(ELogVerbosity::Warning, LexToString(Type));
				}
			}
		}
	}
	return !bFoundProblems;
}

// Test for finding structs which misreport their object properties for native serialization
IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestFindUndeclaredObjectReferencesInStructSerializers, FAutomationTestUObjectClassBase, "UObject.Class FindUndeclaredObjectReferencesInStructSerializers", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::SmokeFilter)
bool FAutomationTestFindUndeclaredObjectReferencesInStructSerializers::RunTest(const FString& Parameters)
{
	return FindUndeclaredObjectReferencesInStructSerializers(*this, false);
}

// Test for finding structs which unoptimally their object properties for native serialization - not a smoke test

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FAutomationTestFindInexactObjectReferencesInStructSerializers, FAutomationTestUObjectClassBase, "UObject.Class FindInexactObjectReferencesInStructSerializers", EAutomationTestFlags::EditorContext | EAutomationTestFlags::ClientContext | EAutomationTestFlags::ServerContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::RequiresUser)
bool FAutomationTestFindInexactObjectReferencesInStructSerializers::RunTest(const FString& Parameters)
{
	return FindUndeclaredObjectReferencesInStructSerializers(*this, true);
}

#endif // !(UE_BUILD_TEST || UE_BUILD_SHIPPING)

bool FTestUndeclaredScriptStructObjectReferencesTest::Serialize(FArchive& Ar)
{
	Ar << StrongObjectPointer << SoftObjectPointer << SoftObjectPath << WeakObjectPointer;
	return true;
}


/*-----------------------------------------------------------------------------
	UClass implementation.
-----------------------------------------------------------------------------*/

/** Default C++ class type information, used for all new UClass objects. */
static const FCppClassTypeInfoStatic DefaultCppClassTypeInfoStatic = { false };

void UClass::PostInitProperties()
{
	Super::PostInitProperties();
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		if (!CppClassStaticFunctions.IsInitialized())
		{
			CppClassStaticFunctions = UOBJECT_CPPCLASS_STATICFUNCTIONS_FORCLASS(UClass);
		}
	}
}

UObject* UClass::GetDefaultSubobjectByName(FName ToFind)
{
	UObject* DefaultObj = GetDefaultObject();
	UObject* DefaultSubobject = nullptr;
	if (DefaultObj)
	{
		DefaultSubobject = DefaultObj->GetDefaultSubobjectByName(ToFind);
	}
	return DefaultSubobject;
}

void UClass::GetDefaultObjectSubobjects(TArray<UObject*>& OutDefaultSubobjects)
{
	UObject* DefaultObj = GetDefaultObject();
	if (DefaultObj)
	{
		DefaultObj->GetDefaultSubobjects(OutDefaultSubobjects);
	}
	else
	{
		OutDefaultSubobjects.Empty();
	}
}

/**
 * Callback used to allow an object to register its direct object references that are not already covered by
 * the token stream.
 *
 * @param ObjectArray	array to add referenced objects to via AddReferencedObject
 */
void UClass::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UClass* This = CastChecked<UClass>(InThis);
	for (FImplementedInterface& Inter : This->Interfaces)
	{
		Collector.AddStableReference( &Inter.Class );
	}

	for (auto& Pair : This->FuncMap)
	{
		Collector.AddStableReference( &Pair.Value );
	}

	Collector.AddStableReference( &This->ClassWithin );

#if WITH_EDITORONLY_DATA
	Collector.AddStableReference( &This->ClassGeneratedBy );
#endif

	if ( !Collector.IsIgnoringArchetypeRef() )
	{
		Collector.AddStableReference( &This->ClassDefaultObject );
	}
	else if( This->ClassDefaultObject != NULL)
	{
		// Get the ARO function pointer from the CDO class (virtual functions using static function pointers).
		This->CallAddReferencedObjects(This->ClassDefaultObject, Collector);
	}

#if UE_WITH_IMMUTABLE_CDO
	Collector.AddStableReference(&This->ImmutableDefaultObject);
#endif

	Collector.AddStableReference( &This->SparseClassDataStruct );

	// Add sparse class data
	if (This->SparseClassDataStruct && This->SparseClassData)
	{
		Collector.AddPropertyReferencesWithStructARO(This->SparseClassDataStruct, This->SparseClassData, This);
	}

	Super::AddReferencedObjects( This, Collector );
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

/**
 * Helper class used to save and restore information across a StaticAllocateObject over the top of an existing UClass.
 */
class FRestoreClassInfo: public FRestoreForUObjectOverwrite
{
	/** Keep a copy of the pointer, which isn't supposed to change **/
	UClass*			Target;
	/** Saved ClassWithin **/
	UClass*			Within;
#if WITH_EDITORONLY_DATA
	/** Saved ClassGeneratedBy */
	UObject*		GeneratedBy;
#endif
	/** Saved ClassDefaultObject **/
	UObject*		DefaultObject;
	/** Saved ClassFlags **/
	EClassFlags		Flags;
	/** Saved ClassCastFlags **/
	EClassCastFlags	CastFlags;
	/** Saved ClassConstructor **/
	UClass::ClassConstructorType Constructor;
	/** Saved ClassVTableHelperCtorCaller **/
	UClass::ClassVTableHelperCtorCallerType ClassVTableHelperCtorCaller;
	/** Saved CppClassStaticFunctions **/
	FUObjectCppClassStaticFunctions CppClassStaticFunctions;
	/** Saved NativeFunctionLookupTable. */
	TArray<FNativeFunctionLookup> NativeFunctionLookupTable;
public:

	/**
	 * Constructor: remember the info for the class so that we can restore it after we've called
	 * FMemory::Memzero() on the object's memory address, which results in the non-intrinsic classes losing
	 * this data
	 */
	FRestoreClassInfo(UClass *Save) :
		Target(Save),
		Within(Save->ClassWithin),
#if WITH_EDITORONLY_DATA
		GeneratedBy(Save->ClassGeneratedBy),
#endif
		DefaultObject(Save->GetDefaultsCount() ? Save->GetDefaultObject() : NULL),
		Flags(Save->ClassFlags & CLASS_Abstract),
		CastFlags(Save->ClassCastFlags),
		Constructor(Save->ClassConstructor),
		ClassVTableHelperCtorCaller(Save->ClassVTableHelperCtorCaller),
		CppClassStaticFunctions(Save->CppClassStaticFunctions),
		NativeFunctionLookupTable(Save->NativeFunctionLookupTable)
	{
	}
	/** Called once the new object has been reinitialized
	**/
	virtual void Restore() const
	{
		Target->ClassWithin = Within;
#if WITH_EDITORONLY_DATA
		Target->ClassGeneratedBy = GeneratedBy;
#endif
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Target->ClassDefaultObject = DefaultObject;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Target->ClassFlags |= Flags;
		Target->ClassCastFlags |= CastFlags;
		Target->ClassConstructor = Constructor;
		Target->ClassVTableHelperCtorCaller = ClassVTableHelperCtorCaller;
		Target->CppClassStaticFunctions = CppClassStaticFunctions;
		Target->NativeFunctionLookupTable = NativeFunctionLookupTable;
	}
};

/**
 * Save information for StaticAllocateObject in the case of overwriting an existing object.
 * StaticAllocateObject will call delete on the result after calling Restore()
 *
 * @return An FRestoreForUObjectOverwrite that can restore the object or NULL if this is not necessary.
 */
FRestoreForUObjectOverwrite* UClass::GetRestoreForUObjectOverwrite()
{
	return new FRestoreClassInfo(this);
}

/**
	* Get the default object from the class, creating it if missing, if requested or under a few other circumstances
	* @return		the CDO for this class
**/
UObject* UClass::CreateDefaultObject()
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if ( ClassDefaultObject == NULL )
	{
#if UE_WITH_CONSTINIT_UOBJECT
		checkfSlow(!HasAnyFlags(RF_NeedInitialization), TEXT("Class %s trying to create its default object when it still needs initialization"), *GetPathName());
#endif // UE_WITH_CONSTINIT_UOBJECT
		ensureMsgf(!bLayoutChanging, TEXT("Class named %s creating its CDO while changing its layout"), *GetName());

		UClass* ParentClass = GetSuperClass();
		UObject* ParentDefaultObject = NULL;
		if ( ParentClass != NULL )
		{
			UObjectForceRegistration(ParentClass);
			ParentDefaultObject = ParentClass->GetDefaultObject(); // Force the default object to be constructed if it isn't already
			check(GConfig);
		}

		if ( (ParentDefaultObject != NULL) || (this == UObject::StaticClass()) )
		{
			// If this is a class that can be regenerated, it is potentially not completely loaded.  Preload and Link here to ensure we properly zero memory and read in properties for the CDO
			if( HasAnyClassFlags(CLASS_CompiledFromBlueprint) && (PropertyLink == NULL) && !GIsDuplicatingClassForReinstancing)
			{
				auto ClassLinker = GetLinker();
				if (ClassLinker)
				{
					UField* FieldIt = Children;
					while (FieldIt && (FieldIt->GetOuter() == this))
					{
						// If we've had cyclic dependencies between classes here, we might need to preload to ensure that we load the rest of the property chain
						FieldIt->ConditionalPreload();
						FieldIt = FieldIt->Next;
					}

					StaticLink(true);
				}
			}

			// in the case of cyclic dependencies, the above Preload() calls could end up
			// invoking this method themselves... that means that once we're done with
			// all the Preload() calls we have to make sure ClassDefaultObject is still
			// NULL (so we don't invalidate one that has already been setup)
			if (ClassDefaultObject == NULL)
			{
				// RF_ArchetypeObject flag is often redundant to RF_ClassDefaultObject, but we need to tag
				// the CDO as RF_ArchetypeObject in order to propagate that flag to any default sub objects.
				UObject* NewClassDefaultObject = StaticAllocateObject(this, GetOuter(), NAME_None, EObjectFlags(RF_Public | RF_ClassDefaultObject | RF_ArchetypeObject));
				// Make sure all memory writes that happened during static allocate objects are visible
				// to a thread that would try to access it once stored in ClassDefaultObject.
				// This is particularly important for asset registry that can try to read RF_NeedInitialization
				// do determine if the class is ready.
				std::atomic_thread_fence(std::memory_order_release);
				// TSAN doesn't recognize fence yet, so use annotations.
				// Any pointer would do as the fence is global, but needs to be paired with TSAN_AFTER
				// on the same pointer before consuming any object properties.
				TSAN_BEFORE(NewClassDefaultObject);
				ClassDefaultObject = NewClassDefaultObject;
				check(ClassDefaultObject);
				// Register the offsets of any sparse delegates this class introduces with the sparse delegate storage
				for (TFieldIterator<FMulticastSparseDelegateProperty> SparseDelegateIt(this, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::ExcludeDeprecated); SparseDelegateIt; ++SparseDelegateIt)
				{
					const FSparseDelegate& SparseDelegate = SparseDelegateIt->GetPropertyValue_InContainer(ClassDefaultObject);
					USparseDelegateFunction* SparseDelegateFunction = CastChecked<USparseDelegateFunction>(SparseDelegateIt->SignatureFunction);
					FSparseDelegateStorage::RegisterDelegateOffset(ClassDefaultObject, SparseDelegateFunction->DelegateName, (size_t)&SparseDelegate - (size_t)ClassDefaultObject.Get());
				}
				EObjectInitializerOptions InitOptions = EObjectInitializerOptions::None;
				if (!HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
				{
					// Blueprint CDOs have their properties always initialized.
					InitOptions |= EObjectInitializerOptions::InitializeProperties;
				}
				(*ClassConstructor)(FObjectInitializer(ClassDefaultObject, ParentDefaultObject, InitOptions));
				if (GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn) && !GetOutermost()->HasAnyPackageFlags(PKG_RuntimeGenerated))
				{
					NotifyRegistrationEvent(GetOutermost()->GetFName(), GetDefaultObjectName(), ENotifyRegistrationType::NRT_ClassCDO, ENotifyRegistrationPhase::NRP_Finished, nullptr, false, ClassDefaultObject);
				}
				ClassDefaultObject->PostCDOContruct();
			}
		}
	}
	return ClassDefaultObject;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

class FFeedbackContextImportDefaults final : public FFeedbackContext
{
	/** Context information for warning and error messages */
	FContextSupplier*	Context;

public:
	FContextSupplier* GetContext() const override { return Context; }
	void SetContext(FContextSupplier* InContext) override { Context = InContext; }
};

FFeedbackContext& UClass::GetDefaultPropertiesFeedbackContext()
{
	static FFeedbackContextImportDefaults FeedbackContextImportDefaults;
	return FeedbackContextImportDefaults;
}

/**
* Get the name of the CDO for the this class
* @return The name of the CDO
*/
FName UClass::GetDefaultObjectName() const
{
	FNameBuilder DefaultNameBuilder;
	DefaultNameBuilder.Append(DEFAULT_OBJECT_PREFIX);
	GetFName().AppendString(DefaultNameBuilder);

	return FName(*DefaultNameBuilder);
}

//
// Register the native class.
//
void UClass::DeferredRegister(UClass *UClassStaticClass, const TCHAR* PackageName, const TCHAR* Name
#if UE_WITH_REMOTE_OBJECT_HANDLE
	, FRemoteObjectId RemoteId
#endif
)
{
	Super::DeferredRegister(UClassStaticClass, PackageName, Name
#if UE_WITH_REMOTE_OBJECT_HANDLE
		, RemoteId
#endif
	);

	// Propagate inherited flags.
	UClass* SuperClass = GetSuperClass();
	if (SuperClass != NULL)
	{
		ClassFlags |= (SuperClass->ClassFlags & CLASS_Inherit);
		ClassCastFlags |= SuperClass->ClassCastFlags;
	}
}

bool UClass::Rename( const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags )
{
	bool bSuccess = Super::Rename( InName, NewOuter, Flags );

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// If we have a default object, rename that to the same package as the class, and rename so it still matches the class name (Default__ClassName)
	if(bSuccess && (ClassDefaultObject != NULL))
	{
		ClassDefaultObject->Rename(*GetDefaultObjectName().ToString(), NewOuter, Flags);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Now actually rename the class
	return bSuccess;
}

void UClass::TagSubobjects(EObjectFlags NewFlags)
{
	Super::TagSubobjects(NewFlags);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ClassDefaultObject && !ClassDefaultObject->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS) && !ClassDefaultObject->IsRooted())
	{
		ClassDefaultObject->SetFlags(NewFlags);
		ClassDefaultObject->TagSubobjects(NewFlags);
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

/**
 * Find the class's native constructor.
 */
void UClass::Bind()
{
	UStruct::Bind();

	if( !ClassConstructor && IsNative() )
	{
		UE_LOGF(LogClass, Fatal, "Can't bind to native class %ls", *GetPathName() );
	}

	UClass* SuperClass = GetSuperClass();
	if (SuperClass && (ClassConstructor == nullptr || !CppClassStaticFunctions.IsInitialized()
		|| ClassVTableHelperCtorCaller == nullptr
		))
	{
		// Chase down constructor in parent class.
		SuperClass->Bind();
		if (!ClassConstructor)
		{
			ClassConstructor = SuperClass->ClassConstructor;
		}
		if (!ClassVTableHelperCtorCaller)
		{
			ClassVTableHelperCtorCaller = SuperClass->ClassVTableHelperCtorCaller;
		}
		if (!CppClassStaticFunctions.IsInitialized())
		{
			CppClassStaticFunctions = SuperClass->CppClassStaticFunctions;
		}

		// propagate flags.
		// we don't propagate the inherit flags, that is more of a header generator thing
		ClassCastFlags |= SuperClass->ClassCastFlags;
	}
	//if( !Class && SuperClass )
	//{
	//}
	if( !ClassConstructor )
	{
		UE_LOGF(LogClass, Fatal, "Can't find ClassConstructor for class %ls", *GetPathName() );
	}
}


/**
 * Returns the struct/ class prefix used for the C++ declaration of this struct/ class.
 * Classes deriving from AActor have an 'A' prefix and other UObject classes an 'U' prefix.
 *
 * @return Prefix character used for C++ declaration of this struct/ class.
 */
const TCHAR* UClass::GetPrefixCPP() const
{
	const UClass* TheClass	= this;
	bool	bIsActorClass	= false;
	bool	bIsDeprecated	= TheClass->HasAnyClassFlags(CLASS_Deprecated);
	while( TheClass && !bIsActorClass )
	{
		bIsActorClass	= TheClass->GetFName() == NAME_Actor;
		TheClass		= TheClass->GetSuperClass();
	}

	if( bIsActorClass )
	{
		if( bIsDeprecated )
		{
			return TEXT("ADEPRECATED_");
		}
		else
		{
			return TEXT("A");
		}
	}
	else
	{
		if( bIsDeprecated )
		{
			return TEXT("UDEPRECATED_");
		}
		else
		{
			return TEXT("U");
		}
	}
}

FString UClass::GetDescription() const
{
	FString Description;

#if WITH_EDITOR
	// See if display name meta data has been specified
	Description = GetDisplayNameText().ToString();
	if (Description.Len())
	{
		return Description;
	}
#endif

	// Look up the the classes name in the legacy int file and return the class name if there is no match.
	//Description = Localize( TEXT("Objects"), *GetName(), *(FInternationalization::Get().GetCurrentCulture()->GetName()), true );
	//if (Description.Len())
	//{
	//	return Description;
	//}

	// Otherwise just return the class name
	return FString( GetName() );
}

//	UClass UObject implementation.

void UClass::FinishDestroy()
{
	// Empty arrays.
	//warning: Must be emptied explicitly in order for intrinsic classes
	// to not show memory leakage on exit.
	NetFields.Empty();
	ClassReps.Empty();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ClassDefaultObject = nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CleanupSparseClassData();
	SparseClassDataStruct = nullptr;

#if WITH_EDITORONLY_DATA
	// If for whatever reason there's still properties that have not been destroyed in PurgeClass, destroy them now
	DestroyPropertiesPendingDestruction();
#endif // WITH_EDITORONLY_DATA

	Super::FinishDestroy();
}

void UClass::PostLoad()
{
	check(ClassWithin);
	Super::PostLoad();

	// Postload super.
	if( GetSuperClass() )
	{
		GetSuperClass()->ConditionalPostLoad();
	}

	if (!HasAnyClassFlags(CLASS_Native))
	{
		ClassFlags &= ~CLASS_ReplicationDataIsSetUp;
	}
}

FString UClass::GetDesc()
{
	return GetName();
}

void UClass::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

namespace UE::CoreUObject::Private::AssetRegistryTags
{
#if WITH_EDITOR
	static const FName ParentClassName        = "ParentClass";
	static const FName ModuleNameName         = "ModuleName";
	static const FName ModuleRelativePathName = "ModuleRelativePath";
#endif
}

void UClass::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	using namespace UE::CoreUObject::Private;
	
	Super::GetAssetRegistryTags(Context);

#if WITH_EDITOR
	const UClass* const ParentClass = GetSuperClass();
	Context.AddTag( FAssetRegistryTag(AssetRegistryTags::ParentClassName, ((ParentClass) ? ParentClass->GetPathName() : FString()), FAssetRegistryTag::TT_Alphabetical) );

	const UPackage* const ClassPackage = GetOuterUPackage();
	Context.AddTag( FAssetRegistryTag(AssetRegistryTags::ModuleNameName, ((ClassPackage) ? FPackageName::GetShortFName(ClassPackage->GetFName()) : NAME_None).ToString(), FAssetRegistryTag::TT_Alphabetical) );

	const FString& ClassModuleRelativeIncludePath = GetMetaData(AssetRegistryTags::ModuleRelativePathName);
	Context.AddTag( FAssetRegistryTag(AssetRegistryTags::ModuleRelativePathName, ClassModuleRelativeIncludePath, FAssetRegistryTag::TT_Alphabetical) );
#endif
}

#if WITH_EDITOR

void UClass::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	using namespace UE::CoreUObject::Private;
	
	Super::GetAssetRegistryTagMetadata(OutMetadata);

	OutMetadata.Emplace(AssetRegistryTags::ParentClassName, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UClass", "ParentClass", "Parent Class"))
		.SetTooltip(NSLOCTEXT("UClass", "ParentClassTooltip", "The parent class of this type."))
	);

	OutMetadata.Emplace(AssetRegistryTags::ModuleNameName, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UClass", "ModuleName", "Module Name"))
		.SetTooltip(NSLOCTEXT("UClass", "ModuleNameTooltip", "The module name of this type."))
	);

	OutMetadata.Emplace(AssetRegistryTags::ModuleRelativePathName, FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UClass", "ModuleRelativePath", "Relative Module Path"))
		.SetTooltip(NSLOCTEXT("UClass", "ModuleRelativePathTooltip", "The relative module path of this type."))
	);
}

void UClass::ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const
{
	//@TODO this is not actually thread safe and should be guarded to avoid execution in parallel with BP compilation
	// the ensure should help catch this via TSAN. UE-209240
	// Unfortunately, we cannot simply limit the PostLoadAssetRegistryTags fixup to native classes because the base
	// UObject::ThreadedPostLoadAssetRegistryTags iterates over the properties of non-native types and performs fixup.
	ensure(!bLayoutChanging);

	Super::ThreadedPostLoadAssetRegistryTagsOverride(Context);

	static const FName ParentClassFName("ParentClass");
	FString ParentClassTagValue = Context.GetAssetData().GetTagValueRef<FString>(ParentClassFName);
	if (!ParentClassTagValue.IsEmpty() && FPackageName::IsShortPackageName(ParentClassTagValue))
	{
		FTopLevelAssetPath ParentClassPathName = UClass::TryConvertShortTypeNameToPathName<UStruct>(ParentClassTagValue, ELogVerbosity::Warning, TEXT("UClass::ThreadedPostLoadAssetRegistryTagsOverride"));
		if (!ParentClassPathName.IsNull())
		{
			Context.AddTagToUpdate(FAssetRegistryTag(ParentClassFName, ParentClassPathName.ToString(), FAssetRegistryTag::TT_Alphabetical));
		}
	}
}
#endif // WITH_EDITOR

void UClass::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);

	if (SparseClassDataStruct)
	{
		OutDeps.Add(SparseClassDataStruct);

		const void* SparseClassDataToUse = GetSparseClassData(EGetSparseClassDataMethod::ArchetypeIfNull);
		if (UScriptStruct::ICppStructOps* CppStructOps = SparseClassDataStruct->GetCppStructOps())
		{
			CppStructOps->GetPreloadDependencies(const_cast<void*>(SparseClassDataToUse), OutDeps);
		}
		// The iterator will recursively loop through all structs in structs/containers too.
		for (TPropertyValueIterator<FStructProperty> It(SparseClassDataStruct, SparseClassDataToUse); It; ++It)
		{
			const UScriptStruct* StructType = It.Key()->Struct;
			if (UScriptStruct::ICppStructOps* CppStructOps = StructType->GetCppStructOps())
			{
				void* StructDataPtr = const_cast<void*>(It.Value());
				CppStructOps->GetPreloadDependencies(StructDataPtr, OutDeps);
			}
		}
	}
}

void UClass::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	check(!bRelinkExistingProperties || !(ClassFlags & CLASS_Intrinsic));
	Super::Link(Ar, bRelinkExistingProperties);
}

EStructPropertyLinkFlags UClass::GetPropertyLinkFlags(UStruct* ContainerStruct, FProperty* Property) const
{
	// If we aren't a native type or we have a VM script, then we will need destructor and post construct
	if (!ContainerStruct->Script.IsEmpty() || !HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic) || Property->HasAnyPropertyFlags(CPF_ForcePostConstructLink))
	{
		return EStructPropertyLinkFlags::LinkDestructor | EStructPropertyLinkFlags::LinkPostConstruct;
	}

	// For native types, config properties also need post construct
	if (Property->HasAnyPropertyFlags(CPF_Config) && !HasAnyClassFlags(CLASS_PerObjectConfig))
	{
		return EStructPropertyLinkFlags::LinkPostConstruct;
	}

	// All other properties of native types assumes constructor/destructor will handle everything
	return EStructPropertyLinkFlags::None;
}

#if VALIDATE_CLASS_REPS
namespace UE::Net::Private
{
	static bool GValidateReplicatedProperties = !UE_BUILD_SHIPPING;
	static FAutoConsoleVariable CVarValidateReplicatedPropertyRegistration(TEXT("net.ValidateReplicatedPropertyRegistration"), GValidateReplicatedProperties, TEXT("Warns if NetDeltaSerialize properties are not top-level"));

	void ValidateNetDeltaSerializeProperties(const TArray<FRepRecord>& ClassReps)
	{
		// Find NetDeltaSerialize properties which are not top-level/parent properties (e.g. nested in another struct)
		for (const FRepRecord& RepRecord : ClassReps)
		{
			const FProperty* BaseProp = RepRecord.Property;
			TArray<const UStruct*, TInlineAllocator<16>> RecursiveStructList;

			enum class ECheckStructType : uint8
			{
				TopLevel,	// Parent/top-level - NetDeltaSerialize is valid
				Recursive	// Recursive (within other structs/arrays) - NetDeltaSerialize is not valid
			};

			auto CheckStructRecursive = [&BaseProp, &RecursiveStructList]
			(auto CheckStructRecursive, const FProperty* InProp, ECheckStructType CheckType)
				-> void
				{
					auto BadStruct = [&BaseProp](const FStructProperty* StructProp)
						{
							UE_LOGF(LogClass, Warning, "Property %ls contained nested NetDeltaSerialize struct '%ls', "
								"when this is not supported. Only use NetDeltaSerialize structs at class level. "
								"Struct will replicate using non-delta serialization.",
								ToCStr(BaseProp->GetPathName()), ToCStr(StructProp->GetPathName()));
						};

					const FStructProperty* ParamStructProp = nullptr;
					bool bNonDeltaArrayStruct = false;

					if (!EnumHasAnyFlags(InProp->PropertyFlags, CPF_RepSkip))
					{
						if (const FStructProperty* StructProp = CastField<FStructProperty>(InProp))
						{
							ParamStructProp = StructProp;
						}
						else if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(InProp))
						{
							if (const FStructProperty* ArrayStructProp = CastField<FStructProperty>(ArrayProp->Inner))
							{
								if (EnumHasAnyFlags(ArrayStructProp->Struct->StructFlags, STRUCT_NetDeltaSerializeNative))
								{
									BadStruct(ArrayStructProp);
								}
								else
								{
									ParamStructProp = ArrayStructProp;
									bNonDeltaArrayStruct = true;
								}
							}
						}
					}


					const bool bTopLevelNonDeltaStruct = ParamStructProp != nullptr && CheckType == ECheckStructType::TopLevel &&
						(bNonDeltaArrayStruct || !EnumHasAnyFlags(ParamStructProp->Struct->StructFlags, STRUCT_NetDeltaSerializeNative));

					const bool bFreshRecursiveStruct = ParamStructProp != nullptr && CheckType == ECheckStructType::Recursive &&
						!RecursiveStructList.Contains(ParamStructProp->Struct);

					if (bTopLevelNonDeltaStruct || bFreshRecursiveStruct || bNonDeltaArrayStruct)
					{
						RecursiveStructList.Add(ParamStructProp->Struct);

						for (TFieldIterator<FProperty> It(ParamStructProp->Struct); It; ++It)
						{
							if (!EnumHasAnyFlags(It->PropertyFlags, CPF_RepSkip))
							{
								const FStructProperty* CurLevelStructProp = nullptr;

								if (const FStructProperty* StructProp = CastField<FStructProperty>(*It))
								{
									CurLevelStructProp = StructProp;
								}
								else if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(*It))
								{
									if (const FStructProperty* ArrayStructProp = CastField<FStructProperty>(ArrayProp->Inner))
									{
										CurLevelStructProp = ArrayStructProp;
									}
								}

								if (CurLevelStructProp != nullptr)
								{
									if (EnumHasAnyFlags(CurLevelStructProp->Struct->StructFlags, STRUCT_NetDeltaSerializeNative))
									{
										BadStruct(CurLevelStructProp);
									}
									else
									{
										CheckStructRecursive(CheckStructRecursive, CurLevelStructProp, ECheckStructType::Recursive);
									}
								}
							}
						}
					}
				};

			CheckStructRecursive(CheckStructRecursive, BaseProp, ECheckStructType::TopLevel);
		}
	}

} // end namespace UE::Net::Private
#endif

void UClass::SetUpRuntimeReplicationData()
{
	// Nothing to do if we already built the replication data
	if (HasAnyClassFlags(CLASS_ReplicationDataIsSetUp))
	{
		return;
	}

	NetFields.Empty();

	if (UClass* SuperClass = GetSuperClass())
	{
		SuperClass->SetUpRuntimeReplicationData();
		ClassReps = SuperClass->ClassReps;
		FirstOwnedClassRep = ClassReps.Num();
	}
	else
	{
		ClassReps.Empty();
		FirstOwnedClassRep = 0;
	}

	// Track properties so me can ensure they are sorted by offsets at the end
	TArray<FProperty*> NetProperties;
	for (TFieldIterator<FField> It(this, EFieldIteratorFlags::ExcludeSuper); It; ++It)
	{
		if (FProperty* Prop = CastField<FProperty>(*It))
		{
			if ((Prop->PropertyFlags & CPF_Net) && Prop->GetOwner<UObject>() == this)
			{
				NetProperties.Add(Prop);
			}
		}
	}

	for(TFieldIterator<UField> It(this,EFieldIteratorFlags::ExcludeSuper); It; ++It)
	{
		if (UFunction * Func = Cast<UFunction>(*It))
		{
			// When loading reflection data (e.g. from blueprints), we may have references to placeholder functions, or reflection data
			// in children may be out of date. In that case we cannot enforce this check, but that is ok because reflection data will
			// be regenerated by compile on load anyway:
			const bool bCanCheck = (!GIsEditor && !IsRunningCommandlet()) || !Func->HasAnyFlags(RF_WasLoaded);
			check(!bCanCheck || (!Func->GetSuperFunction() || (Func->GetSuperFunction()->FunctionFlags&FUNC_NetFuncFlags) == (Func->FunctionFlags&FUNC_NetFuncFlags)));
			if ((Func->FunctionFlags&FUNC_Net) && !Func->GetSuperFunction())
			{
				NetFields.Add(Func);
			}
		}
	}

	const bool bIsNativeClass = HasAnyClassFlags(CLASS_Native);
	if (!bIsNativeClass)
	{
		// Sort NetProperties so that their ClassReps are sorted by memory offset
		struct FComparePropertyOffsets
		{
			FORCEINLINE bool operator()(FProperty* A, FProperty* B) const
			{
				// Ensure stable sort
				if (A->GetOffset_ForGC() == B->GetOffset_ForGC())
				{
					return A->GetName() < B->GetName();
				}

				return A->GetOffset_ForGC() < B->GetOffset_ForGC();
			}
		};

		Algo::Sort(NetProperties, FComparePropertyOffsets());
	}

	ClassReps.Reserve(ClassReps.Num() + NetProperties.Num());
	for (int32 i = 0; i < NetProperties.Num(); i++)
	{
		NetProperties[i]->RepIndex = (uint16)ClassReps.Num();
		for (int32 j = 0; j < NetProperties[i]->ArrayDim; j++)
		{
			ClassReps.Emplace(NetProperties[i], j);
		}
	}
	check(ClassReps.Num() <= 65535);

	NetFields.Shrink();

	Algo::SortBy(NetFields, &UField::GetFName, FNameLexicalLess());

	ClassFlags |= CLASS_ReplicationDataIsSetUp;

#if VALIDATE_CLASS_REPS
	if (UE::Net::Private::GValidateReplicatedProperties)
	{
		if (bIsNativeClass)
		{
			GetDefaultObject()->ValidateGeneratedRepEnums(ClassReps);
		}
		UE::Net::Private::ValidateNetDeltaSerializeProperties(ClassReps);
	}
#endif
}

void UClass::InternalCreateDefaultObjectWrapper() const
{
	UE_TRACK_REFERENCING_PACKAGE_SCOPED(this, PackageAccessTrackingOps::NAME_CreateDefaultObject);
	const_cast<UClass*>(this)->CreateDefaultObject();
}

#if UE_WITH_IMMUTABLE_CDO
void UClass::InternalCreateImmutableDefaultObject() const
{
	UObject* DefaultObject = GetDefaultObject();
	checkf(DefaultObject, TEXT("Unable to create immutable default object because the default object couldn't be created for class %s"), *GetPathName());
	checkf(ImmutableDefaultObject == nullptr, TEXT("Attempting to create another instance of mutable default object for %s"), *GetPathName());

	UObject* ImmutableDefault = nullptr;
	const bool bIsClassNative = HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic);
	{
		checkf(DefaultObject->GetClass() != DefaultObject->GetOuter(), TEXT("Attempting to create an immutable CDO on top of the original CDO"));
		FStaticConstructObjectParameters ConstructParams(DefaultObject->GetClass());
		ConstructParams.Outer = DefaultObject->GetClass();
		ConstructParams.Name = DefaultObject->GetFName();
		ConstructParams.SetFlags = RF_ClassDefaultObject | RF_ImmutableDefaultObject | RF_ArchetypeObject | RF_Transient;
		ImmutableDefault = StaticConstructObject_Internal(ConstructParams);
	}
	if (!HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
	{
		// Non-native CDOs need to be duplicated to copy all of their non-native properties
		check(DefaultObject->GetClass() != DefaultObject->GetOuter());
		FObjectDuplicationParameters DuplicationParameters(DefaultObject, DefaultObject->GetClass());
		DuplicationParameters.DestName = DefaultObject->GetFName();
		DuplicationParameters.ApplyFlags = RF_ArchetypeObject | RF_Transient;
		DuplicationParameters.DuplicationSeed.Add(DefaultObject, ImmutableDefault);
		ImmutableDefault = StaticDuplicateObjectEx(DuplicationParameters);
	}
	checkf(ImmutableDefault, TEXT("Unable to create immutable default object for %s"), *GetPathName());
	ImmutableDefault->PostCDOContruct();
	FPermanentObjectPoolExtents PermanentPool;
	if (GUObjectArray.IsDisregardForGC(this) || PermanentPool.Contains(this))
	{
		ImmutableDefault->AddToRoot();
	}
	const_cast<UClass*>(this)->ImmutableDefaultObject = ImmutableDefault;
}
#endif

void UClass::SetDefaultObject(UObject* InClassDefaultObject)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if UE_WITH_IMMUTABLE_CDO
	if (InClassDefaultObject != ClassDefaultObject && ImmutableDefaultObject)
	{
		ImmutableDefaultObject->ClearFlags(RF_ClassDefaultObject | RF_ImmutableDefaultObject | RF_ArchetypeObject);
		ImmutableDefaultObject->RemoveFromRoot();
		ImmutableDefaultObject->Rename(*MakeUniqueObjectName(GetTransientPackage(), this).ToString(), GetTransientPackage(), REN_DoNotDirty | REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		ImmutableDefaultObject = nullptr;
	}
#endif
	ClassDefaultObject = InClassDefaultObject;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UClass::PostLoadDefaultObject(UObject* Object)
{
	Object->PostLoad();
#if UE_WITH_IMMUTABLE_CDO
	// Construct the immutable version of the CDO immediately after the CDO has been loaded
	GetImmutableDefaultObject();
#endif
}
/**
* Helper function for determining if the given class is compatible with structured archive serialization
*/
bool UClass::IsSafeToSerializeToStructuredArchives(UClass* InClass)
{
	while (InClass)
	{
		if (!InClass->HasAnyClassFlags(CLASS_MatchedSerializers))
		{
			return false;
		}
		InClass = InClass->GetSuperClass();
	}
	return true;
}

#if WITH_EDITOR
FTopLevelAssetPath UClass::GetReinstancedClassPathName() const
{
	if (HasAnyClassFlags(CLASS_NewerVersionExists))
	{
		return GetReinstancedClassPathName_Impl();
	}

	return nullptr;
}
#endif

#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY

	FStructBaseChain::~FStructBaseChain()
	{
		delete [] StructBaseChainArray;
	}

	void FStructBaseChain::ReinitializeBaseChainArray()
	{
		delete [] StructBaseChainArray;

		int32 Depth = 0;
		for (UStruct* Ptr : static_cast<UStruct*>(this)->GetSuperStructIterator())
		{
			++Depth;
		}

		const FStructBaseChain** Bases = new const FStructBaseChain*[Depth];
		{
			const FStructBaseChain** Base = Bases + Depth;
			for (UStruct* Ptr : static_cast<UStruct*>(this)->GetSuperStructIterator())
			{
				*--Base = Ptr;
			}
		}

		StructBaseChainArray = Bases;
		NumStructBasesInChainMinusOne = Depth - 1;
	}

#endif

void UClass::SetSuperStruct(UStruct* NewSuperStruct)
{
	UnhashObject(this);
	ClearFunctionMapsCaches();
	Super::SetSuperStruct(NewSuperStruct);

	if (!GetSparseClassDataStruct())
	{
		if (UScriptStruct* SparseClassDataStructArchetype = GetSparseClassDataArchetypeStruct())
		{
			SetSparseClassDataStruct(SparseClassDataStructArchetype);
		}
	}

	HashObject(this);
}

bool UClass::IsStructTrashed() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return Children == nullptr && ChildProperties == nullptr && ClassDefaultObject == nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#if WITH_EDITORONLY_DATA

int32 CalculatePropertyIndex(const UStruct* Struct, const FProperty* Property, int32 ArrayIndex)
{
	checkf(Struct->IsChildOf(Property->GetOwnerStruct()),
		TEXT("Property %s in %s is not related to the provided type %s."),
		*Property->GetAuthoredName(), *Property->GetOwnerStruct()->GetPathName(), *Struct->GetPathName());

	const int32 IndexInOwner = Property->GetIndexInOwner();
	checkf(IndexInOwner >= 0,
		TEXT("Property %s in %s does not have an index within its owner."),
		*Property->GetAuthoredName(), *Struct->GetPathName());

	checkf(ArrayIndex < Property->ArrayDim,
		TEXT("Property %s in %s has out of range array index %d with capacity %d."),
		*Property->GetAuthoredName(), *Struct->GetPathName(), ArrayIndex, Property->ArrayDim);

	return IndexInOwner + ArrayIndex;
}

struct FPropertyValueFlagsAnnotationData
{
	TBitArray<> FlagByIndex;
};

struct FPropertyValueFlagsAnnotation
{
	bool IsDefault() const
	{
		return !Data.IsValid();
	}

	TSharedPtr<FPropertyValueFlagsAnnotationData> Data;
};

using FPropertyValueFlagsAnnotationStore = FUObjectAnnotationSparse<FPropertyValueFlagsAnnotation, /*bAutoRemove*/ true>;

template <EPropertyValueFlags Flags>
UE_AUTORTFM_ALWAYS_OPEN FPropertyValueFlagsAnnotationStore& GetPropertyValueFlagsAnnotations()
{
	static FPropertyValueFlagsAnnotationStore Annotations;
	return Annotations;
}

static FPropertyValueFlagsAnnotationStore& GetPropertyValueFlagsAnnotations(EPropertyValueFlags Flags)
{
	switch (Flags)
	{
	default:
		checkNoEntry();
		[[fallthrough]];
	case EPropertyValueFlags::Initialized:
		return GetPropertyValueFlagsAnnotations<EPropertyValueFlags::Initialized>();
	case EPropertyValueFlags::Serialized:
		return GetPropertyValueFlagsAnnotations<EPropertyValueFlags::Serialized>();
	}
}

bool UClass::ActivateTrackingPropertyValueFlag(EPropertyValueFlags Flags, void* Data) const
{
	UObjectBase* Object = static_cast<UObjectBase*>(Data);
	FPropertyValueFlagsAnnotationStore& Store = GetPropertyValueFlagsAnnotations(Flags);
	TSharedPtr<FPropertyValueFlagsAnnotationData> AnnotationData = Store.GetAnnotation(Object).Data;
	if (!AnnotationData)
	{
		AnnotationData = MakeShared<FPropertyValueFlagsAnnotationData>();
		Store.AddAnnotation(Object, {AnnotationData});
	}
	return true;
}

bool UClass::IsTrackingPropertyValueFlag(EPropertyValueFlags Flags, const void* Data) const
{
	const UObjectBase* Object = static_cast<const UObjectBase*>(Data);
	return GetPropertyValueFlagsAnnotations(Flags).GetAnnotation(Object).Data.IsValid();
}

bool UClass::HasPropertyValueFlag(EPropertyValueFlags Flags, const void* Data, const FProperty* Property, int32 ArrayIndex) const
{
	const UObjectBase* Object = static_cast<const UObjectBase*>(Data);
	if (TSharedPtr<FPropertyValueFlagsAnnotationData> AnnotationData = GetPropertyValueFlagsAnnotations(Flags).GetAnnotation(Object).Data)
	{
		const int32 PropertyIndex = CalculatePropertyIndex(this, Property, ArrayIndex);
		if (AnnotationData->FlagByIndex.IsValidIndex(PropertyIndex))
		{
			return AnnotationData->FlagByIndex[PropertyIndex];
		}
		// Default to uninitialized when tracking is active and a flag has not been stored.
		return false;
	}
	// Default to initialized when tracking is inactive.
	return true;
}

void UClass::SetPropertyValueFlag(EPropertyValueFlags Flags, bool bValue, void* Data, const FProperty* Property, int32 ArrayIndex) const
{
	UObjectBase* Object = static_cast<UObjectBase*>(Data);
	if (TSharedPtr<FPropertyValueFlagsAnnotationData> AnnotationData = GetPropertyValueFlagsAnnotations(Flags).GetAnnotation(Object).Data)
	{
		const int32 PropertyIndex = CalculatePropertyIndex(this, Property, ArrayIndex);
		if (AnnotationData->FlagByIndex.Num() <= PropertyIndex)
		{
			checkf(PropertyIndex < TotalFieldCount,
				TEXT("Property %s in %s has out of range index %d with capacity for %d."),
				*Property->GetAuthoredName(), *GetPathName(), PropertyIndex, TotalFieldCount);
			AnnotationData->FlagByIndex.SetNum(TotalFieldCount, /*bValue*/ false);
		}
		AnnotationData->FlagByIndex[PropertyIndex] = bValue;
	}
}

void UClass::ResetPropertyValueFlags(EPropertyValueFlags Flags, void* Data) const
{
	UObjectBase* Object = static_cast<UObjectBase*>(Data);
	if (TSharedPtr<FPropertyValueFlagsAnnotationData> AnnotationData = GetPropertyValueFlagsAnnotations(Flags).GetAnnotation(Object).Data)
	{
		AnnotationData->FlagByIndex.SetRange(0, AnnotationData->FlagByIndex.Num(), /*bValue*/ false);
	}
}

void UClass::SerializePropertyValueFlags(EPropertyValueFlags Flags, void* Data, FStructuredArchiveRecord Record, FArchiveFieldName Name) const
{
	UObjectBase* Object = static_cast<UObjectBase*>(Data);
	FPropertyValueFlagsAnnotationStore& Store = GetPropertyValueFlagsAnnotations(Flags);
	TSharedPtr<FPropertyValueFlagsAnnotationData> AnnotationData = Store.GetAnnotation(Object).Data;
	if (TOptional<FStructuredArchiveSlot> Slot = Record.TryEnterField(Name, AnnotationData.IsValid()))
	{
		if (!AnnotationData)
		{
			AnnotationData = MakeShared<FPropertyValueFlagsAnnotationData>();
			Store.AddAnnotation(Object, {AnnotationData});
		}
		*Slot << AnnotationData->FlagByIndex;
	}
}

#endif // WITH_EDITORONLY_DATA

void UClass::Serialize( FArchive& Ar )
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if ( Ar.IsLoading() || (Ar.IsModifyingWeakAndStrongReferences() && !Ar.IsSaving()) )
	{
		// Rehash since SuperStruct will be serialized in UStruct::Serialize
		UnhashObject(this);
	}

	Super::Serialize( Ar );

	if ( Ar.IsLoading() || (Ar.IsModifyingWeakAndStrongReferences() && !Ar.IsSaving()) )
	{
		HashObject(this);
	}

	Ar.ThisContainsCode();

	// serialize the function map
	//@TODO: UCREMOVAL: Should we just regenerate the FuncMap post load, instead of serializing it?
	{
	 	FUClassFuncScopeWriteLock ScopeLock(FuncMapLock);
		Ar << FuncMap;
	}

	// Class flags first.
	if (Ar.IsSaving())
	{
		uint32 SavedClassFlags = ClassFlags;
		SavedClassFlags &= ~CLASS_ShouldNeverBeLoaded;
		Ar << SavedClassFlags;
	}
	else if (Ar.IsLoading())
	{
		Ar << (uint32&)ClassFlags;
		ClassFlags &= ~CLASS_ShouldNeverBeLoaded;

		checkf(!HasAnyClassFlags(CLASS_Native), TEXT("Class %s loaded with CLASS_Native....we should not be loading any native classes."), *GetFullName());
		checkf(!HasAnyClassFlags(CLASS_Intrinsic), TEXT("Class %s loaded with CLASS_Intrinsic....we should not be loading any intrinsic classes."), *GetFullName());
	}
	else
	{
		Ar << (uint32&)ClassFlags;
	}
	if (Ar.UEVer() < VER_UE4_CLASS_NOTPLACEABLE_ADDED)
	{
		// We need to invert the CLASS_NotPlaceable flag here because it used to mean CLASS_Placeable
		ClassFlags ^= CLASS_NotPlaceable;

		// We can't import a class which is placeable and has a not-placeable base, so we need to check for that here.
		if (ensure(HasAnyClassFlags(CLASS_NotPlaceable) || !GetSuperClass()->HasAnyClassFlags(CLASS_NotPlaceable)))
		{
			// It's good!
		}
		else
		{
			// We'll just make it non-placeable to ensure loading works, even if there's an off-chance that it's already been placed
			ClassFlags |= CLASS_NotPlaceable;
		}
	}

	// Variables.
	Ar << ClassWithin;
	Ar << ClassConfigName;

	int32 NumInterfaces = 0;
	int64 InterfacesStart = 0L;
	if(Ar.IsLoading())
	{
		// Always start with no interfaces
		Interfaces.Empty();

		// In older versions, interface classes were serialized before linking. In case of cyclic dependencies, we need to skip over the serialized array and defer the load until after Link() is called below.
		if(Ar.UEVer() < VER_UE4_UCLASS_SERIALIZE_INTERFACES_AFTER_LINKING && !GIsDuplicatingClassForReinstancing)
		{
			// Get our current position
			InterfacesStart = Ar.Tell();

			// Load the length of the Interfaces array
			Ar << NumInterfaces;

			// Seek past the Interfaces array
			struct FSerializedInterfaceReference
			{
				FPackageIndex Class;
				int32 PointerOffset;
				bool bImplementedByK2;
			};
			Ar.Seek(InterfacesStart + sizeof(NumInterfaces) + NumInterfaces * sizeof(FSerializedInterfaceReference));
		}
	}

	if (!Ar.IsIgnoringClassGeneratedByRef())
	{
#if !WITH_EDITORONLY_DATA
		// Dummy variable to keep archive consistency
		UObject* ClassGeneratedBy = nullptr;
#endif
		Ar << ClassGeneratedBy;
	}

	if (Ar.IsLoading() && !(Ar.GetPortFlags() & PPF_Duplicate))
	{
		Link(Ar, true);
	}

	if(Ar.IsLoading())
	{
		// Save current position
		int64 CurrentOffset = Ar.Tell();

		// In older versions, we need to seek backwards to the start of the interfaces array
		if(Ar.UEVer() < VER_UE4_UCLASS_SERIALIZE_INTERFACES_AFTER_LINKING && !GIsDuplicatingClassForReinstancing)
		{
			Ar.Seek(InterfacesStart);
		}

		// Load serialized interface classes
		TArray<FImplementedInterface> SerializedInterfaces;
		Ar << SerializedInterfaces;

		// Apply loaded interfaces only if we have not already set them (i.e. during compile-on-load)
		if(Interfaces.Num() == 0 && SerializedInterfaces.Num() > 0)
		{
			Interfaces = SerializedInterfaces;
		}

		// In older versions, seek back to our current position after linking
		if(Ar.UEVer() < VER_UE4_UCLASS_SERIALIZE_INTERFACES_AFTER_LINKING && !GIsDuplicatingClassForReinstancing)
		{
			Ar.Seek(CurrentOffset);
		}
	}
	else
	{
		Ar << Interfaces;
	}

	bool bDeprecatedForceScriptOrder = false;
	Ar << bDeprecatedForceScriptOrder;

	FName Dummy = NAME_None;
	Ar << Dummy;

	if (Ar.UEVer() >= VER_UE4_ADD_COOKED_TO_UCLASS)
	{
		if (Ar.IsSaving())
		{
			bool bCookedAsBool = bCooked || Ar.IsCooking();
			Ar << bCookedAsBool;
		}
		if (Ar.IsLoading())
		{
			bool bCookedAsBool = false;
			Ar << bCookedAsBool;
			bCooked = bCookedAsBool;
		}
	}

	// Defaults.

	// mark the archive as serializing defaults
	Ar.StartSerializingDefaults();

	if( Ar.IsLoading() )
	{
		check((Ar.GetPortFlags() & PPF_Duplicate) || (GetStructureSize() >= sizeof(UObject)));
		check(!GetSuperClass() || !GetSuperClass()->HasAnyFlags(RF_NeedLoad));

		// record the current CDO, as it stands, so we can compare against it
		// after we've serialized in the new CDO (to detect if, as a side-effect
		// of the serialization, a different CDO was generated)
		UObject* const OldCDO = ClassDefaultObject;

		// serialize in the CDO, but first store it here (in a temporary var) so
		// we can check to see if it should be the authoritative CDO (a newer
		// CDO could be generated as a side-effect of this serialization)
		//
		// @TODO: for USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING, do we need to
		//        defer this serialization (should we just save off the tagged
		//        serialization data for later use)?
		UObject* PerspectiveNewCDO = NULL;
		Ar << PerspectiveNewCDO;

		// Blueprint class regeneration could cause the class's CDO to be set.
		// The CDO (<<) serialization call (above) probably will invoke class
		// regeneration, and as a side-effect the CDO could already be set by
		// the time it returns. So we only want to set the CDO here (to what was
		// serialized in) if it hasn't already changed (else, the serialized
		// version could be stale). See: TTP #343166
		if (ClassDefaultObject == OldCDO)
		{
			ClassDefaultObject = PerspectiveNewCDO;
		}
		// if we reach this point, then the CDO was regenerated as a side-effect
		// of the serialization... let's log if the regenerated CDO (what's
		// already been set) is not the same as what was returned from the
		// serialization (could mean the CDO was regenerated multiple times?)
		else if (PerspectiveNewCDO != ClassDefaultObject)
		{
			UE_LOGF(LogClass, Log, "CDO was changed while class serialization.\n\tOld: '%ls'\n\tSerialized: '%ls'\n\tActual: '%ls'"
				, OldCDO ? *OldCDO->GetFullName() : TEXT("NULL")
				, PerspectiveNewCDO ? *PerspectiveNewCDO->GetFullName() : TEXT("NULL")
				, ClassDefaultObject ? *ClassDefaultObject->GetFullName() : TEXT("NULL"));
		}
		ClassUnique = 0;
	}
	else
	{
		check(!ClassDefaultObject || GetDefaultsCount()==GetPropertiesSize());

		// only serialize the class default object if the archive allows serialization of ObjectArchetype
		// otherwise, serialize the properties that the ClassDefaultObject references
		// The logic behind this is the assumption that the reason for not serializing the ObjectArchetype
		// is because we are performing some actions on objects of this class and we don't want to perform
		// that action on the ClassDefaultObject.  However, we do want to perform that action on objects that
		// the ClassDefaultObject is referencing, so we'll serialize it's properties instead of serializing
		// the object itself
		if ( !Ar.IsIgnoringArchetypeRef() )
		{
			Ar << ClassDefaultObject;
		}
		else if( (ClassDefaultObject != nullptr && !Ar.HasAnyPortFlags(PPF_DuplicateForPIE|PPF_Duplicate)) || ClassDefaultObject != nullptr )
		{
			ClassDefaultObject->Serialize(Ar);
		}
	}

	if (!Ar.IsLoading() && !Ar.IsSaving())
	{
		Ar << SparseClassDataStruct;

		if (SparseClassDataStruct && SparseClassData)
		{
			SerializeSparseClassData(FStructuredArchiveFromArchive(Ar).GetSlot());
		}
	}

	// mark the archive we that we are no longer serializing defaults
	Ar.StopSerializingDefaults();

	if( Ar.IsLoading() )
	{
		if (ClassDefaultObject == nullptr)
		{
			check(GConfig);
			if (Ar.IsLoadingFromCookedPackage())
			{
				ClassDefaultObject = GetDefaultObject();
				// we do this later anyway, once we find it and set it in the export table.
				// ClassDefaultObject->SetFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects);
			}
			else if( !Ar.HasAnyPortFlags(PPF_DuplicateForPIE|PPF_Duplicate) )
			{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
				// Blueprint CDO serialization is deferred (see FLinkerLoad::CreateExport).
				const FLinkerLoad* LinkerLoad = Cast<FLinkerLoad>(Ar.GetLinker());
				const bool bHasDeferredCDOSerialization = HasAnyClassFlags(CLASS_CompiledFromBlueprint)
					&& GetClass()->HasAnyClassFlags(CLASS_NeedsDeferredDependencyLoading)
					&& LinkerLoad && LinkerLoad->IsBlueprintFinalizationPending();
				if (!bHasDeferredCDOSerialization)
#endif	// USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
				{
					UE_LOGF(LogClass, Error, "CDO for class %ls did not load!", *GetPathName());
					ensure(ClassDefaultObject != nullptr);
					ClassDefaultObject = GetDefaultObject();
					Ar.ForceBlueprintFinalization();
				}
			}
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool UClass::ImplementsInterface( const class UClass* SomeInterface ) const
{
	if (SomeInterface != NULL && SomeInterface->HasAnyClassFlags(CLASS_Interface) && SomeInterface != UInterface::StaticClass())
	{
		for (const UClass* CurrentClass = this; CurrentClass; CurrentClass = CurrentClass->GetSuperClass())
		{
			// SomeInterface might be a base interface of our implemented interface
			for (TArray<FImplementedInterface>::TConstIterator It(CurrentClass->Interfaces); It; ++It)
			{
				const UClass* InterfaceClass = It->Class;
				if (InterfaceClass && InterfaceClass->IsChildOf(SomeInterface))
				{
					return true;
				}
			}
		}
	}

	return false;
}

/** serializes the passed in object as this class's default object using the given archive
 * @param Object the object to serialize as default
 * @param Ar the archive to serialize from
 */
void UClass::SerializeDefaultObject(UObject* Object, FStructuredArchive::FSlot Slot)
{
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

#if WITH_EDITORONLY_DATA
	if (SerializeContext->SerializedObject == Object)
	{
		SerializeContext->SerializedObjectScriptStartOffset = UnderlyingArchive.Tell();
	}
#endif
	UnderlyingArchive.MarkScriptSerializationStart(Object);
	UnderlyingArchive.StartSerializingDefaults();

	if( ((UnderlyingArchive.IsLoading() || UnderlyingArchive.IsSaving()) && !UnderlyingArchive.WantBinaryPropertySerialization()) )
	{
	    // class default objects do not always have a vtable when saved
		// so use script serialization as opposed to native serialization to
	    // guarantee that all property data is loaded into the correct location
	    SerializeTaggedProperties(Slot, (uint8*)Object, GetSuperClass(), (uint8*)Object->GetArchetype());
	}
	else if (UnderlyingArchive.GetPortFlags() != 0 )
	{
		SerializeBinEx(Slot, Object, Object->GetArchetype(), GetSuperClass() );
	}
	else
	{
		SerializeBin(Slot, Object);
	}
	UnderlyingArchive.StopSerializingDefaults();
	UnderlyingArchive.MarkScriptSerializationEnd(Object);
#if WITH_EDITORONLY_DATA
	if (SerializeContext->SerializedObject == Object)
	{
		SerializeContext->SerializedObjectScriptEndOffset = UnderlyingArchive.Tell();
	}
#endif
}

void UClass::SerializeSparseClassData(FStructuredArchive::FSlot Slot)
{
	if (!SparseClassDataStruct)
	{
		Slot.EnterRecord();
		return;
	}

	// tell the archive that it's allowed to load data for transient properties
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	// Preload sparse class data struct if required
	if(SparseClassDataStruct->HasAnyFlags(RF_NeedLoad))
	{
		UnderlyingArchive.Preload(SparseClassDataStruct);
	}

	// make sure we always have sparse class a sparse class data struct to read from/write to
	GetOrCreateSparseClassData();

	if (((UnderlyingArchive.IsLoading() || UnderlyingArchive.IsSaving()) && !UnderlyingArchive.WantBinaryPropertySerialization()))
	{
		// class default objects do not always have a vtable when saved
		// so use script serialization as opposed to native serialization to
		// guarantee that all property data is loaded into the correct location
		SparseClassDataStruct->SerializeItem(Slot, SparseClassData, GetArchetypeForSparseClassData());
	}
	else if (UnderlyingArchive.GetPortFlags() != 0)
	{
		SparseClassDataStruct->SerializeBinEx(Slot, SparseClassData, GetArchetypeForSparseClassData(), GetSparseClassDataArchetypeStruct());
	}
	else
	{
		SparseClassDataStruct->SerializeBin(Slot, SparseClassData);
	}
}


FArchive& operator<<(FArchive& Ar, FImplementedInterface& A)
{
	Ar << A.Class;
	Ar << A.PointerOffset;
	Ar << A.bImplementedByK2;

	return Ar;
}

const void* UClass::GetArchetypeForSparseClassData(EGetSparseClassDataMethod GetMethod) const
{
	UClass* SuperClass = GetSuperClass();
	return SuperClass ? SuperClass->GetSparseClassData(GetMethod) : nullptr;
}

UScriptStruct* UClass::GetSparseClassDataArchetypeStruct() const
{
	UClass* SuperClass = GetSuperClass();
	return SuperClass ? SuperClass->GetSparseClassDataStruct() : nullptr;
}

UObject* UClass::GetArchetypeForCDO() const
{
	UClass* SuperClass = GetSuperClass();
	return SuperClass ? SuperClass->GetDefaultObject() : nullptr;
}

void UClass::PurgeClass(bool bRecompilingOnLoad)
{
	ClassConstructor = nullptr;
	ClassVTableHelperCtorCaller = nullptr;
	ClassFlags = CLASS_None;
	ClassCastFlags = CASTCLASS_None;
	ClassUnique = 0;
	ClassReps.Empty();
	NetFields.Empty();

#if WITH_EDITOR
	if (!bRecompilingOnLoad)
	{
		// this is not safe to do at COL time. The meta data is not loaded yet, so if we attempt to load it, we recursively load the package and that will fail
		RemoveMetaData("HideCategories");
		RemoveMetaData("ShowCategories");
		RemoveMetaData("HideFunctions");
		RemoveMetaData("AutoExpandCategories");
		RemoveMetaData("AutoCollapseCategories");
		RemoveMetaData("PrioritizeCategories");
		RemoveMetaData("ClassGroupNames");
	}
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ClassDefaultObject = nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	Interfaces.Empty();
	NativeFunctionLookupTable.Empty();
	SetSuperStruct(nullptr);
	Children = nullptr;
	Script.Empty();
	MinAlignment = 0;
	RefLink = nullptr;
	PropertyLink = nullptr;
	DestructorLink = nullptr;
	CppClassStaticFunctions.Reset();
	Partials = nullptr;
	ScriptAndPropertyObjectReferences.Empty();
	DeleteUnresolvedScriptProperties();

	{
		FUClassFuncScopeWriteLock ScopeLock(FuncMapLock);
		FuncMap.Empty();
	}
	ClearFunctionMapsCaches();
	PropertyLink = nullptr;

#if WITH_EDITORONLY_DATA
	{
		for (UPropertyWrapper* Wrapper : PropertyWrappers)
		{
			Wrapper->SetProperty(nullptr);
		}
		PropertyWrappers.Empty();
	}

	// When compiling properties can't be immediately destroyed because we need
	// to fix up references to these properties. The caller of PurgeClass is
	// expected to call DestroyPropertiesPendingDestruction
	FField* LastField = ChildProperties;
	if (LastField)
	{
		while (LastField->Next)
		{
			LastField = LastField->Next;
		}
		check(LastField->Next == nullptr);
		LastField->Next = PropertiesPendingDestruction;
		PropertiesPendingDestruction = ChildProperties;
		ChildProperties = nullptr;
	}
	// Update the serial number so that FFieldPaths that point to properties of this struct know they need to resolve themselves again
	FieldPathSerialNumber = GetNextFieldPathSerialNumber();
	bHasAssetRegistrySearchableProperties = false;
#else
	{
		// Destroy all properties owned by this struct
		DestroyPropertyLinkedList(ChildProperties);
	}
#endif // WITH_EDITORONLY_DATA

	DestroyUnversionedSchema(this);
}

#if WITH_EDITORONLY_DATA
void UClass::DestroyPropertiesPendingDestruction()
{
	DestroyPropertyLinkedList(PropertiesPendingDestruction);
}
#endif // WITH_EDITORONLY_DATA

UClass* UClass::FindCommonBase(UClass* InClassA, UClass* InClassB)
{
	check(InClassA);
	UClass* CommonClass = InClassA;
	while (InClassB && !InClassB->IsChildOf(CommonClass))
	{
		CommonClass = CommonClass->GetSuperClass();

		if( !CommonClass )
			break;
	}
	return CommonClass;
}

UClass* UClass::FindCommonBase(const TArray<UClass*>& InClasses)
{
	check(InClasses.Num() > 0);
	auto ClassIter = InClasses.CreateConstIterator();
	UClass* CommonClass = *ClassIter;
	ClassIter++;

	for (; ClassIter; ++ClassIter)
	{
		CommonClass = UClass::FindCommonBase(CommonClass, *ClassIter);
	}
	return CommonClass;
}

bool UClass::IsFunctionImplementedInScript(FName InFunctionName) const
{
	// Implemented in classes such as UBlueprintGeneratedClass
	return false;
}

bool UClass::HasProperty(const FProperty* InProperty) const
{
	if (InProperty->GetOwner<UObject>())
	{
		UClass* PropertiesClass = InProperty->GetOwner<UClass>();
		if (PropertiesClass)
		{
			return IsChildOf(PropertiesClass);
		}
	}

	return false;
}


/*-----------------------------------------------------------------------------
	UClass constructors.
-----------------------------------------------------------------------------*/

/**
 * Internal constructor.
 */
UClass::UClass(const FObjectInitializer& ObjectInitializer)
:	UStruct( ObjectInitializer )
,	ClassUnique(0)
,	bCooked(false)
,	bLayoutChanging(false)
,	ClassFlags(CLASS_None)
,	ClassCastFlags(CASTCLASS_None)
,	ClassWithin( UObject::StaticClass() )
#if WITH_EDITORONLY_DATA
,	ClassGeneratedBy(nullptr)
,	PropertiesPendingDestruction(nullptr)
#endif
PRAGMA_DISABLE_DEPRECATION_WARNINGS
,	ClassDefaultObject(nullptr)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
,	SparseClassData(nullptr)
,	SparseClassDataStruct(nullptr)
{
	// If you add properties here, please update the other constructors and PurgeClass()

	SetCppTypeInfoStatic(&DefaultCppClassTypeInfoStatic);
	TRACE_LOADTIME_CLASS_INFO(this, GetFName());
}

/**
 * Create a new UClass given its superclass.
 */
UClass::UClass(const FObjectInitializer& ObjectInitializer, UClass* InBaseClass)
:	UStruct(ObjectInitializer, InBaseClass)
,	ClassUnique(0)
,	bCooked(false)
,	bLayoutChanging(false)
,	ClassFlags(CLASS_None)
,	ClassCastFlags(CASTCLASS_None)
,	ClassWithin(UObject::StaticClass())
#if WITH_EDITORONLY_DATA
,	ClassGeneratedBy(nullptr)
,	PropertiesPendingDestruction(nullptr)
#endif
PRAGMA_DISABLE_DEPRECATION_WARNINGS
,	ClassDefaultObject(nullptr)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
,	SparseClassData(nullptr)
,	SparseClassDataStruct(nullptr)
{
	// If you add properties here, please update the other constructors and PurgeClass()

	SetCppTypeInfoStatic(&DefaultCppClassTypeInfoStatic);

	UClass* ParentClass = GetSuperClass();
	if (ParentClass)
	{
		ClassWithin = ParentClass->ClassWithin;
		Bind();

		// if this is a native class, we may have defined a StaticConfigName() which overrides
		// the one from the parent class, so get our config name from there
		if (IsNative())
		{
			ClassConfigName = StaticConfigName();
		}
		else
		{
			// otherwise, inherit our parent class's config name
			ClassConfigName = ParentClass->ClassConfigName;
		}
	}
}

/**
 * Called when statically linked.
 */
UClass::UClass
(
	EStaticConstructor,
	FName			InName,
	uint32			InSize,
	uint32			InAlignment,
	EClassFlags		InClassFlags,
	EClassCastFlags	InClassCastFlags,
	const TCHAR*    InConfigName,
	EObjectFlags	InFlags,
	ClassConstructorType InClassConstructor,
	ClassVTableHelperCtorCallerType InClassVTableHelperCtorCaller,
	FUObjectCppClassStaticFunctions&& InCppClassStaticFunctions
)
:	UStruct					( EC_StaticConstructor, InSize, InAlignment, InFlags )
,	ClassConstructor		( InClassConstructor )
,	ClassVTableHelperCtorCaller(InClassVTableHelperCtorCaller)
,	CppClassStaticFunctions(MoveTemp(InCppClassStaticFunctions))
,	ClassUnique				( 0 )
,	bCooked					( false )
,	bLayoutChanging			( false )
,	ClassFlags				( InClassFlags | CLASS_Native )
,	ClassCastFlags			( InClassCastFlags )
,	ClassWithin				( nullptr )
#if WITH_EDITORONLY_DATA
,	ClassGeneratedBy		( nullptr )
,	PropertiesPendingDestruction( nullptr )
#endif
,	ClassConfigName			(InConfigName)
,	NetFields				()
PRAGMA_DISABLE_DEPRECATION_WARNINGS
,	ClassDefaultObject		( nullptr )
PRAGMA_ENABLE_DEPRECATION_WARNINGS
,	SparseClassData			( nullptr )
,	SparseClassDataStruct	( nullptr )
{
	// If you add properties here, please update the other constructors and PurgeClass()

	SetCppTypeInfoStatic(&DefaultCppClassTypeInfoStatic);
}

UClass::~UClass()
{
	DestructItem(&Interfaces);
	DestructItem(&NativeFunctionLookupTable);
}

void* UClass::CreateSparseClassData()
{
	check(SparseClassData == nullptr);

	if (SparseClassDataStruct)
	{
		SparseClassData = FMemory::Malloc(SparseClassDataStruct->GetStructureSize(), SparseClassDataStruct->GetMinAlignment());
		SparseClassDataStruct->InitializeStruct(SparseClassData);
	}
	if (SparseClassData)
	{
		// initialize per class data from the archetype if we have one
		const void* SparseArchetypeData = GetArchetypeForSparseClassData();
		UScriptStruct* SparseClassDataArchetypeStruct = GetSparseClassDataArchetypeStruct();

		if (SparseArchetypeData)
		{
			// our saved SparseClassDataStruct typically will be a child of
			// its parent's SCD, but sometimes (e.g. because a parent introduced
			// a new SCDStruct) it won't be, and will actually be a child of
			// *our* SCDStruct:
			if (SparseClassDataStruct->IsChildOf(SparseClassDataArchetypeStruct))
			{
				SparseClassDataArchetypeStruct->CopyScriptStruct(SparseClassData, SparseArchetypeData);
			}
			else if (SparseClassDataArchetypeStruct->IsChildOf(SparseClassDataStruct))
			{
				SparseClassDataStruct->CopyScriptStruct(SparseClassData, SparseArchetypeData);
			}
			else
			{
				UE_LOGF(LogClass, Warning, "SparseClassData %ls for class %ls archetype is of unrelated type %ls",
					*SparseClassDataStruct->GetPathName(),
					*GetPathName(),
					*SparseClassDataArchetypeStruct->GetPathName());
			}
		}

		FCoreUObjectDelegates::OnPostInitSparseClassData.Broadcast(this, SparseClassDataStruct, SparseClassData);
	}

	return SparseClassData;
}

void UClass::CleanupSparseClassData()
{
	if (SparseClassData)
	{
		SparseClassDataStruct->DestroyStruct(SparseClassData);
		FMemory::Free(SparseClassData);
		SparseClassData = nullptr;
	}
}

const void* UClass::GetSparseClassData(const EGetSparseClassDataMethod GetMethod)
{
	if (SparseClassData)
	{
		return SparseClassData;
	}

	switch (GetMethod)
	{
	case EGetSparseClassDataMethod::CreateIfNull:
		return CreateSparseClassData();

	case EGetSparseClassDataMethod::ArchetypeIfNull:
		// Use the archetype data only when it's the expected type, otherwise the result may be cast to an incorrect type by the caller
		return SparseClassDataStruct == GetSparseClassDataArchetypeStruct()
			? GetArchetypeForSparseClassData(GetMethod)
			: CreateSparseClassData();

	case EGetSparseClassDataMethod::DeferIfNull:
		// Use the archetype data only when it's the expected type, otherwise the result may be cast to an incorrect type by the caller
		return SparseClassDataStruct == GetSparseClassDataArchetypeStruct()
			? GetArchetypeForSparseClassData(GetMethod)
			: nullptr;

	case EGetSparseClassDataMethod::ReturnIfNull:
		return nullptr;

	default:
		checkNoEntry();
		return nullptr;
	}
}

UScriptStruct* UClass::GetSparseClassDataStruct() const
{
	// this info is specified on the object via code generation so we use it instead of looking at the UClass
	return SparseClassDataStruct;
}

void UClass::SetSparseClassDataStruct(UScriptStruct* InSparseClassDataStruct)
{
	if (SparseClassDataStruct != InSparseClassDataStruct)
	{
		// Passing nullptr as InSparseClassDataStruct is a valid way to clear sparse class data
		if (SparseClassDataStruct)
		{
			// Find all subclasses and point the SuperClass of their SparseClassDataStruct to point to this new SparseClassDataStruct.
			// We have to do this when compilation creates a new SparseClassDataStruct that has already loaded subclasses.
			// Most class-owned subobjects are regenerated at compile time (properties,functions,etc) or mirrored in the UBlueprint
			// (see FKismetCompilerContext::SaveSubObjectsFromCleanAndSanitizeClass) and copied over to the new class inside of
			// FKismetCompilerContext::FinishCompilingClass (SimpleConstructionScript,etc). But SparseClassDataStruct is not covered
			// by either of those cases, so we instead fixup subclasses' SparseClassDataStructs' superclass pointers here.
			TArray<UClass*> SubClasses;
			GetDerivedClasses(this, SubClasses, true /* bRecursive */);
			for (UClass* SubClass : SubClasses)
			{
				UScriptStruct* SubClassSparseClassDataStruct = SubClass->GetSparseClassDataStruct();
				if (SubClassSparseClassDataStruct && SubClassSparseClassDataStruct->GetSuperStruct() == SparseClassDataStruct)
				{
					// As we are potentialy completely changing the struct layout, we need to cleanup any data we have
					// before the superstruct link gets set
					SubClass->CleanupSparseClassData();
					SubClassSparseClassDataStruct->SetSuperStruct(InSparseClassDataStruct);
				}
			}
		}
		// the old type and new type may not match when we do a reload so get rid of the old data
		CleanupSparseClassData();

		SparseClassDataStruct = InSparseClassDataStruct;
	}
}

void UClass::ClearSparseClassDataStruct(bool bInRecomplingOnLoad)
{
	if (SparseClassDataStruct != nullptr)
	{
		CleanupSparseClassData();
		SparseClassDataStruct = nullptr;
	}
}


#if WITH_RELOAD

bool UClass::HotReloadPrivateStaticClass(
	uint32			InSize,
	EClassFlags		InClassFlags,
	EClassCastFlags	InClassCastFlags,
	const TCHAR*    InConfigName,
	ClassConstructorType InClassConstructor,
	ClassVTableHelperCtorCallerType InClassVTableHelperCtorCaller,
	FUObjectCppClassStaticFunctions&& InCppClassStaticFunctions,
	class UClass* TClass_Super_StaticClass,
	class UClass* TClass_WithinClass_StaticClass
	)
{
	if (InSize != PropertiesSize)
	{
		UClass::GetDefaultPropertiesFeedbackContext().Logf(ELogVerbosity::Warning, TEXT("Property size mismatch. Will not update class %s (was %d, new %d)."), *GetName(), PropertiesSize, InSize);
		return false;
	}
	//We could do this later, but might as well get it before we start corrupting the object
	UObject* CDO = GetDefaultObject();
	void* OldVTable = *(void**)CDO;


	//@todo safe? ClassFlags = InClassFlags | CLASS_Native;
	//@todo safe? ClassCastFlags = InClassCastFlags;
	//@todo safe? ClassConfigName = InConfigName;
	ClassConstructorType OldClassConstructor = ClassConstructor;
	ClassConstructor = InClassConstructor;
	ClassVTableHelperCtorCaller = InClassVTableHelperCtorCaller;
	CppClassStaticFunctions = InCppClassStaticFunctions; // Not MoveTemp; it is used again below
	/* No recursive ::StaticClass calls allowed. Setup extras. */
	/* @todo safe?
	if (TClass_Super_StaticClass != this)
	{
		SetSuperStruct(TClass_Super_StaticClass);
	}
	else
	{
		SetSuperStruct(NULL);
	}
	ClassWithin = TClass_WithinClass_StaticClass;
	*/

	UE_LOGF(LogClass, Verbose, "Attempting to change VTable for class %ls.",*GetName());
	ClassWithin = UPackage::StaticClass();  // We are just avoiding error checks with this...we don't care about this temp object other than to get the vtable.

	static struct FUseVTableConstructorsCache
	{
		FUseVTableConstructorsCache()
		{
			bUseVTableConstructors = false;
			GConfig->GetBool(TEXT("Core.System"), TEXT("UseVTableConstructors"), bUseVTableConstructors, GEngineIni);
		}

		bool bUseVTableConstructors;
	} UseVTableConstructorsCache;

	UObject* TempObjectForVTable = nullptr;
	{
		TGuardValue<bool> Guard(GIsRetrievingVTablePtr, true);

		// Mark we're in the constructor now.
		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		TScopeCounter<int32> InConstructor(ThreadContext.IsInConstructor);

		FVTableHelper Helper;
		TempObjectForVTable = ClassVTableHelperCtorCaller(Helper);
		TempObjectForVTable->AtomicallyClearInternalFlags(EInternalObjectFlags::PendingConstruction);
	}

	if( !TempObjectForVTable->IsRooted() )
	{
		TempObjectForVTable->MarkAsGarbage();
	}
	else
	{
		UE_LOGF(LogClass, Warning, "Reload:  Was not expecting temporary object '%ls' for class '%ls' to become rooted during construction.  This object cannot be marked pending kill.", *TempObjectForVTable->GetFName().ToString(), *this->GetName() );
	}

	ClassWithin = TClass_WithinClass_StaticClass;

	void* NewVTable = *(void**)TempObjectForVTable;
	if (NewVTable != OldVTable)
	{
		int32 Count = 0;
		int32 CountClass = 0;
		for ( FRawObjectIterator It; It; ++It )
		{
			UObject* Target = static_cast<UObject*>(It->GetObject());
			if (OldVTable == *(void**)Target)
			{
				*(void**)Target = NewVTable;
				Count++;
			}
			else if (dynamic_cast<UClass*>(Target))
			{
				UClass *Class = CastChecked<UClass>(Target);
				if (Class->ClassConstructor == OldClassConstructor)
				{
					Class->ClassConstructor = ClassConstructor;
					Class->ClassVTableHelperCtorCaller = ClassVTableHelperCtorCaller;
					Class->CppClassStaticFunctions = InCppClassStaticFunctions; // Not MoveTemp; it is used in later loop iterations
					CountClass++;
				}
			}
		}
		UE_LOGF(LogClass, Verbose, "Updated the vtable for %d live objects and %d blueprint classes.  %016llx -> %016llx", Count, CountClass, PTRINT(OldVTable), PTRINT(NewVTable));
	}
	else
	{
		UE_LOGF(LogClass, Error, "VTable for class %ls did not change?",*GetName());
	}

	return true;
}

bool UClass::ReplaceNativeFunction(FName InFName, FNativeFuncPtr InPointer, bool bAddToFunctionRemapTable)
{
	// Find the function in the class's native function lookup table.
	for (int32 FunctionIndex = 0; FunctionIndex < NativeFunctionLookupTable.Num(); ++FunctionIndex)
	{
		FNativeFunctionLookup& NativeFunctionLookup = NativeFunctionLookupTable[FunctionIndex];
		if (NativeFunctionLookup.Name == InFName)
		{
			if (bAddToFunctionRemapTable)
			{
				ReloadNotifyFunctionRemap(InPointer, NativeFunctionLookup.Pointer);
			}
			NativeFunctionLookup.Pointer = InPointer;
			return true;
		}
	}
	return false;
}

#endif

UClass* UClass::GetAuthoritativeClass()
{
#if WITH_RELOAD && WITH_ENGINE
	if (IsReloadActive())
	{
		const TMap<UClass*, UClass*>& ReinstancedClasses = GetClassesToReinstanceForHotReload();
		if (UClass* const* FoundMapping = ReinstancedClasses.Find(this))
		{
			return *FoundMapping ? *FoundMapping : this;
		}
	}
#endif

	return this;
}

void UClass::AddNativeFunction(const ANSICHAR* InName, FNativeFuncPtr InPointer)
{
	FName InFName(InName);
#if WITH_RELOAD
	if (IsReloadActive())
	{
		// Find the function in the class's native function lookup table.
		if (ReplaceNativeFunction(InFName, InPointer, true))
		{
			return;
		}
		else
		{
			// function was not found, so it's new
			UE_LOGF(LogClass, Log, "Function %ls is new or belongs to a modified class.", *InFName.ToString());
		}
	}
#endif
	NativeFunctionLookupTable.Emplace(InFName,InPointer);
}

void UClass::AddNativeFunction(const WIDECHAR* InName, FNativeFuncPtr InPointer)
{
	FName InFName(InName);
#if WITH_RELOAD
	if (IsReloadActive())
	{
		// Find the function in the class's native function lookup table.
		if (ReplaceNativeFunction(InFName, InPointer, true))
		{
			return;
		}
		else
		{
			// function was not found, so it's new
			UE_LOGF(LogClass, Log, "Function %ls is new or belongs to a modified class.", *InFName.ToString());
		}
	}
#endif
	NativeFunctionLookupTable.Emplace(InFName, InPointer);
}

#if !UE_WITH_CONSTINIT_UOBJECT
void UClass::CreateLinkAndAddChildFunctionsToMap(const FClassFunctionLinkInfo* Functions, uint32 NumFunctions)
{
	for (; NumFunctions; --NumFunctions, ++Functions)
	{
		const char* FuncNameUTF8 = Functions->FuncNameUTF8;
		UFunction*  Func         = Functions->CreateFuncPtr(ETypeConstructPhase::Outer);

		Func->Next = Children;
		Children = Func;

		AddFunctionToFunctionMap(Func, FName((const UTF8CHAR*)FuncNameUTF8));
	}
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

void UClass::ClearFunctionMapsCaches()
{
	FUClassFuncScopeWriteLock ScopeLock(AllFunctionsCacheLock);
	AllFunctionsCache.Empty();
}

UFunction* UClass::FindFunctionByName(FName InName, EIncludeSuperFlag::Type IncludeSuper) const
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UClass);

	UFunction* Result = nullptr;

	UE_AUTORTFM_OPEN
	{
		UClass* SuperClass = GetSuperClass();
		if (IncludeSuper == EIncludeSuperFlag::ExcludeSuper || ( Interfaces.Num() == 0 && SuperClass == nullptr ) )
		{
			// Trivial case: just look up in this class's function map and don't involve the cache
			FUClassFuncScopeReadLock ScopeLock(FuncMapLock);
			Result = FuncMap.FindRef(InName);
		}
		else
		{
			// Check the cache
			bool bFoundInCache = false;
			{
				FUClassFuncScopeReadLock ScopeLock(AllFunctionsCacheLock);
				if (UFunction** SuperResult = AllFunctionsCache.Find(InName))
				{
					Result = *SuperResult;
					bFoundInCache = true;
				}
			}

			if (!bFoundInCache)
			{
				// Try this class's FuncMap first
				{
					FUClassFuncScopeReadLock ScopeLock(FuncMapLock);
					Result = FuncMap.FindRef(InName);
				}

				if (Result)
				{
					// Cache the result
					FUClassFuncScopeWriteLock ScopeLock(AllFunctionsCacheLock);
					AllFunctionsCache.Add(InName, Result);
				}
				else
				{
					// Check superclass and interfaces
					if (Interfaces.Num() > 0)
					{
						for (const FImplementedInterface& Inter : Interfaces)
						{
							Result = Inter.Class ? Inter.Class->FindFunctionByName(InName) : nullptr;
							if (Result)
							{
								break;
							}
						}
					}

					UFunction* SuperResult = nullptr;
					if (SuperClass != nullptr)
					{
						SuperResult = SuperClass->FindFunctionByName(InName);
					}

					// Check for multiple inheritance: If a superclass implements the same interface, use its implementation instead
					if (!Result || (SuperResult && SuperResult->GetOwnerClass()->ImplementsInterface(Result->GetOwnerClass())))
					{
						Result = SuperResult;
					}

					{
						// Do a final check to make sure the function still doesn't exist in this class before we add it to the cache, in case the function was added by another thread since we last checked
						// This avoids us writing null (or a superclass func with the same name) to the cache if the function was just added
						FUClassFuncScopeReadLock ScopeLockFuncMap(FuncMapLock);
						if (FuncMap.FindRef(InName) == nullptr)
						{
							// Cache the result (even if it's nullptr)
							FUClassFuncScopeWriteLock ScopeLock(AllFunctionsCacheLock);
							AllFunctionsCache.Add(InName, Result);
						}
					}
				}
			}
		}
	};

	return Result;
}

void UClass::AssembleReferenceTokenStreams()
{
	SCOPED_BOOT_TIMING("AssembleReferenceTokenStreams (can be optimized)");
	// Iterate over all class objects and force the default objects to be created. Additionally also
	// assembles the token reference stream at this point. This is required for class objects that are
	// not taken into account for garbage collection but have instances that are.
	for (FRawObjectIterator It(false); It; ++It) // GetDefaultObject can create a new class, that need to be handled as well, so we cannot use TObjectIterator
	{
		if (UClass* Class = Cast<UClass>((UObject*)(It->GetObject())))
		{
			// Force the default object to be created (except when we're in the middle of exit purge -
			// this may happen if we exited PreInit early because of error).
			//
			// Keep from handling script generated classes here, as those systems handle CDO
			// instantiation themselves.
			if (!GExitPurge && !Class->HasAnyFlags(RF_BeingRegenerated))
			{
				Class->GetDefaultObject(); // Force the default object to be constructed if it isn't already
			}
			// Assemble reference token stream for garbage collection/ RTGC.
			if (!Class->HasAnyFlags(RF_ClassDefaultObject) && !Class->HasAnyClassFlags(CLASS_TokenStreamAssembled))
			{
				Class->AssembleReferenceTokenStream();
			}
		}
	}
}

const FString UClass::GetConfigName() const
{
	// Look up in the known configs
	if (GConfig->IsKnownConfigName(ClassConfigName))
	{
		return ClassConfigName.ToString();
	}
	else if( ClassConfigName == NAME_Editor )
	{
		return GEditorIni;
	}
	else if ( ClassConfigName == NAME_EditorSettings )
	{
		return GEditorSettingsIni;
	}
	else if ( ClassConfigName == NAME_EditorLayout )
	{
		return GEditorLayoutIni;
	}
	else if ( ClassConfigName == NAME_EditorKeyBindings )
	{
		return GEditorKeyBindingsIni;
	}
	else if( ClassConfigName == NAME_None )
	{
		UE_LOGF(LogClass, Fatal,"UObject::GetConfigName() called on class with config name 'None'. Class flags = 0x%08X", (uint32)ClassFlags );
		return TEXT("");
	}
	else
	{
		// if the branch was already loaded, we have the IniPath right there
		FConfigBranch* Branch = GConfig->FindBranch(*ClassConfigName.ToString(), FString());
		if (Branch != nullptr)
		{
			return Branch->IniPath;
		}

		// generate the class ini name, and make sure it's up to date
		FString ConfigGameName;
		FConfigContext::ReadIntoGConfig().Load(*ClassConfigName.ToString(), ConfigGameName);
		return ConfigGameName;
	}
}

UField* UClass::TryFindTypeSlow(UClass* TypeClass, FStringView InPathNameOrShortName, EFindFirstObjectOptions InOptions)
{
	checkf(TypeClass && TypeClass->IsChildOf(UField::StaticClass()), TEXT("TryFindType requires a valid TypeClass parameter which is a subclass of UField. \"%s\" provided."), *GetFullNameSafe(TypeClass));

	UField* FoundType = nullptr;
	// We check against "None" as it's not uncommon to pass an FName converted to string as InPathNameOrShortName
	if (!InPathNameOrShortName.IsEmpty() && InPathNameOrShortName != TEXT("None"))
	{
		if (!FPackageName::IsShortPackageName(InPathNameOrShortName))
		{
			FoundType = (UField*)StaticFindObject(TypeClass, nullptr, InPathNameOrShortName, !!(InOptions & EFindFirstObjectOptions::ExactClass) ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
		}
		else
		{
			// RobM: I can't decide if this should be an ensure. Maybe in the future?
			TStringBuilder<1024> Callstack;
			ANSICHAR Buffer[1024];
			uint64 StackFrames[10];
			uint32 NumStackFrames = FPlatformStackWalk::CaptureStackBackTrace(StackFrames, UE_ARRAY_COUNT(StackFrames));
			const uint32 IgnoreStackCount = 1; // Ignore the call to CaptureStackBackTrace itself
			for (uint32 Idx = IgnoreStackCount; Idx < NumStackFrames && Idx < UE_ARRAY_COUNT(StackFrames); Idx++)
			{
				Buffer[0] = '\0';
				const ANSICHAR* TrimmedBuffer = Buffer;

				// Trim the address/module only if we resolve the symbol
				const bool bFoundSymbol = FPlatformStackWalk::ProgramCounterToHumanReadableString(Idx, StackFrames[Idx], Buffer, sizeof(Buffer));
				if (bFoundSymbol)
				{
					const ANSICHAR* BufferAfterModuleAndAddress = FCStringAnsi::Strstr(Buffer, "!");
					if (BufferAfterModuleAndAddress)
					{
						TrimmedBuffer = BufferAfterModuleAndAddress + 1;
					}
				}

				Callstack.Append(TrimmedBuffer);
				Callstack.Append(TEXT("\r\n"));
			}

			FoundType = (UField*)StaticFindFirstObject(TypeClass, InPathNameOrShortName, InOptions | EFindFirstObjectOptions::EnsureIfAmbiguous | EFindFirstObjectOptions::NativeFirst, ELogVerbosity::Error, TEXT("TryFindType"));

			UE_LOGF(LogClass, Warning, "Short type name \"%.*ls\" provided for TryFindType. Please convert it to a path name (suggested: \"%ls\"). Callstack:\r\n\r\n%ls", InPathNameOrShortName.Len(), InPathNameOrShortName.GetData(), *GetPathNameSafe(FoundType), Callstack.ToString());
		}
	}
	return FoundType;
}

UField* UClass::TryFindTypeSlowSafe(UClass* TypeClass, FStringView InPathNameOrShortName, EFindFirstObjectOptions InOptions)
{
	if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
	{
		return TryFindTypeSlow(TypeClass, InPathNameOrShortName, InOptions);
	}
	return nullptr;
}

FTopLevelAssetPath UClass::TryConvertShortTypeNameToPathName(UClass* TypeClass, FStringView InShortTypeName, ELogVerbosity::Type AmbiguousMessageVerbosity /*= ELogVerbosity::NoLogging*/, const TCHAR* AmbiguousClassMessage /*= nullptr*/)
{
	checkf(TypeClass && TypeClass->IsChildOf(UField::StaticClass()), TEXT("TryConvertShortTypeNameToPathName requires a valid TypeClass parameter which is a subclass of UField. \"%s\" provided."), *GetFullNameSafe(TypeClass));

	if (InShortTypeName.IsEmpty())
	{
		return FTopLevelAssetPath();
	}
	else if (!FPackageName::IsShortPackageName(InShortTypeName))
	{
		return FTopLevelAssetPath(InShortTypeName);
	}
	else
	{
		FTopLevelAssetPath Result;
		EFindFirstObjectOptions Options = EFindFirstObjectOptions::NativeFirst;
		if (AmbiguousMessageVerbosity != ELogVerbosity::NoLogging && AmbiguousMessageVerbosity <= ELogVerbosity::Error)
		{
			Options |= EFindFirstObjectOptions::EnsureIfAmbiguous;
		}
		UField* FoundType = (UField*)StaticFindFirstObject(TypeClass, InShortTypeName, Options, AmbiguousMessageVerbosity, AmbiguousClassMessage);
		if (FoundType)
		{
			// UField does not define a GetFieldPathName so do what GetStructPathName does (this is the only place that needs it)
			checkf(FoundType->GetOuter() == FoundType->GetOutermost(), TEXT("Trying to construct FTopLevelAssetPath for nested type: \"%s\""), *FoundType->GetPathName());
			Result = FTopLevelAssetPath(FoundType->GetOuter()->GetFName(), FoundType->GetFName());
		}
		return Result;
	}
}

bool UClass::TryFixShortClassNameExportPath(FString& InOutExportPathToFix,
	ELogVerbosity::Type AmbiguousMessageVerbosity /*= ELogVerbosity::NoLogging*/,
	const TCHAR* AmbiguousClassMessage /*= nullptr*/, bool bClearOnError /* = false */)
{
	FString ClassName;
	FString ObjectPath;
	if (FPackageName::ParseExportTextPath(*InOutExportPathToFix, &ClassName, &ObjectPath))
	{
		if (FPackageName::IsShortPackageName(ClassName))
		{
			FTopLevelAssetPath ClassPathName = UClass::TryConvertShortTypeNameToPathName<UClass>(ClassName, AmbiguousMessageVerbosity, AmbiguousClassMessage);
			if (!ClassPathName.IsNull() || bClearOnError)
			{
				InOutExportPathToFix = !ClassPathName.IsNull() ? FObjectPropertyBase::GetExportPath(ClassPathName, ObjectPath) : FString();
				return true;
			}
		}
	}
	return false;
}

FString UClass::ConvertPathNameToShortTypeName(FStringView InPathName)
{
	return FString(FPackageName::ObjectPathToObjectName(InPathName));
}

FString UClass::ConvertFullNameToShortTypeFullName(FStringView InFullName)
{
	FStringView ClassPath, PackageName, ObjectName, SubObjectName;
	FPackageName::SplitFullObjectPath(InFullName, ClassPath, PackageName, ObjectName, SubObjectName);
	if (ClassPath.IsEmpty() || PackageName.IsEmpty())
	{
		// Not a valid FullName, return it untransformed
		return FString(InFullName);
	}
	FStringView ClassObjectName = FPackageName::ObjectPathToObjectName(ClassPath);
	if (ClassObjectName.Len() == ClassPath.Len())
	{
		// Already a ShortTypeFullName, return it untransformed
		return FString(InFullName);
	}

	FString Result;
	Result.Reserve(InFullName.Len());
	Result += ClassObjectName;
	Result += TEXT(" ");
	Result += PackageName;
	if (ObjectName.Len())
	{
		Result += TEXT(".");
		Result += ObjectName;
		if (SubObjectName.Len())
		{
			Result += SUBOBJECT_DELIMITER;
			Result += SubObjectName;
		}
	}
	return Result;
}

bool UClass::IsShortTypeName(FStringView InClassPath)
{
	if (InClassPath.Len() == 0)
	{
		// Arbitrary
		return true;
	}
	// If it starts with '/' it is definitely not a ShortTypeName
	// If it doesn't start with '/', most of the time it is a ShortTypeName because this function is usually called on either PathNames or ShortTypeNames
	if (InClassPath[0] == '/')
	{
		return false;
	}
	return UE::String::FindFirstOfAnyChar(InClassPath, TEXT("./\\" SUBOBJECT_DELIMITER_ANSI)) == INDEX_NONE;
}


#if WITH_EDITOR
void UClass::GetHideFunctions(TArray<FString>& OutHideFunctions) const
{
	static const FName NAME_HideFunctions("HideFunctions");
	if (const FString* HideFunctions = FindMetaData(NAME_HideFunctions))
	{
		HideFunctions->ParseIntoArray(OutHideFunctions, TEXT(" "), true);
	}
}

bool UClass::IsFunctionHidden(const TCHAR* InFunction) const
{
	static const FName NAME_HideFunctions("HideFunctions");
	if (const FString* HideFunctions = FindMetaData(NAME_HideFunctions))
	{
		return !!FCString::StrfindDelim(**HideFunctions, InFunction, TEXT(" "));
	}
	return false;
}

void UClass::GetAutoExpandCategories(TArray<FString>& OutAutoExpandCategories) const
{
	static const FName NAME_AutoExpandCategories("AutoExpandCategories");
	if (const FString* AutoExpandCategories = FindMetaData(NAME_AutoExpandCategories))
	{
		AutoExpandCategories->ParseIntoArray(OutAutoExpandCategories, TEXT(" "), true);
	}
}

bool UClass::IsAutoExpandCategory(const TCHAR* InCategory) const
{
	static const FName NAME_AutoExpandCategories("AutoExpandCategories");
	if (const FString* AutoExpandCategories = FindMetaData(NAME_AutoExpandCategories))
	{
		return !!FCString::StrfindDelim(**AutoExpandCategories, InCategory, TEXT(" "));
	}
	return false;
}

void UClass::GetPrioritizeCategories(TArray<FString>& OutPrioritizedCategories) const
{
	static const FName NAME_PrioritizeCategories("PrioritizeCategories");
	if (const FString* PrioritizeCategories = FindMetaData(NAME_PrioritizeCategories))
	{
		PrioritizeCategories->ParseIntoArray(OutPrioritizedCategories, TEXT(" "), true);
	}
}

bool UClass::IsPrioritizeCategory(const TCHAR* InCategory) const
{
	static const FName NAME_PrioritizeCategories("PrioritizeCategories");
	if (const FString* PrioritizeCategories = FindMetaData(NAME_PrioritizeCategories))
	{
		return !!FCString::StrfindDelim(**PrioritizeCategories, InCategory, TEXT(" "));
	}
	return false;
}

void UClass::GetAutoCollapseCategories(TArray<FString>& OutAutoCollapseCategories) const
{
	static const FName NAME_AutoCollapseCategories("AutoCollapseCategories");
	if (const FString* AutoCollapseCategories = FindMetaData(NAME_AutoCollapseCategories))
	{
		AutoCollapseCategories->ParseIntoArray(OutAutoCollapseCategories, TEXT(" "), true);
	}
}

bool UClass::IsAutoCollapseCategory(const TCHAR* InCategory) const
{
	static const FName NAME_AutoCollapseCategories("AutoCollapseCategories");
	if (const FString* AutoCollapseCategories = FindMetaData(NAME_AutoCollapseCategories))
	{
		return !!FCString::StrfindDelim(**AutoCollapseCategories, InCategory, TEXT(" "));
	}
	return false;
}

void UClass::GetClassGroupNames(TArray<FString>& OutClassGroupNames) const
{
	static const FName NAME_ClassGroupNames("ClassGroupNames");
	if (const FString* ClassGroupNames = FindMetaData(NAME_ClassGroupNames))
	{
		ClassGroupNames->ParseIntoArray(OutClassGroupNames, TEXT(" "), true);
	}
}

bool UClass::IsClassGroupName(const TCHAR* InGroupName) const
{
	static const FName NAME_ClassGroupNames("ClassGroupNames");
	if (const FString* ClassGroupNames = FindMetaData(NAME_ClassGroupNames))
	{
		return !!FCString::StrfindDelim(**ClassGroupNames, InGroupName, TEXT(" "));
	}
	return false;
}

#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
void UClass::CallAppendToClassSchema(FAppendToClassSchemaContext& Context) const
{
	// Call the C++ Append function for the class and its superclasses
	check(CppClassStaticFunctions.GetAppendToClassSchema());
	CppClassStaticFunctions.GetAppendToClassSchema()(Context);

	// Add on the ini append values for the class and its superclasses.
	FString AppendText;
	const UClass* SuperClassIterator = this;
	check(GConfig);
	while (SuperClassIterator != nullptr)
	{
		if (GConfig->GetString(TEXT("AppendToClassSchema"), *SuperClassIterator->GetPathName(),
			AppendText, GEditorIni))
		{
			AppendText.TrimStartAndEndInline();
			if (!AppendText.IsEmpty())
			{
				Context.Update(*AppendText, sizeof(**AppendText) * AppendText.Len());
			}
		}
		SuperClassIterator = SuperClassIterator->GetSuperClass();
	}
}

bool UClass::CanCreateInstanceDataObject() const
{
	return false;
}
#endif

#if !UE_WITH_CONSTINIT_UOBJECT
void GetPrivateStaticClassBody(
	const TCHAR* PackageName,
	const TCHAR* Name,
	UClass*& ReturnClass,
	void(*RegisterNativeFunc)(),
	uint32 InSize,
	uint32 InAlignment,
	EClassFlags InClassFlags,
	EClassCastFlags InClassCastFlags,
	const TCHAR* InConfigName,
	UClass::ClassConstructorType InClassConstructor,
	UClass::ClassVTableHelperCtorCallerType InClassVTableHelperCtorCaller,
	FUObjectCppClassStaticFunctions&& InCppClassStaticFunctions,
	UClass::StaticClassFunctionType InSuperClassFn,
	UClass::StaticClassFunctionType InWithinClassFn
	)
{
	UECodeGen_Private::ConstructUClassNoInitHelper<UClass>(
		PackageName,
		Name,
		ReturnClass,
		RegisterNativeFunc,
		InSize,
		InAlignment,
		InClassFlags,
		InClassCastFlags,
		InConfigName,
		InClassConstructor,
		InClassVTableHelperCtorCaller,
		MoveTemp(InCppClassStaticFunctions),
		InSuperClassFn,
		InWithinClassFn,
		[](UClass*) {});
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

/*-----------------------------------------------------------------------------
	UFunction.
-----------------------------------------------------------------------------*/

UFunction::UFunction(const FObjectInitializer& ObjectInitializer, UFunction* InSuperFunction, EFunctionFlags InFunctionFlags, SIZE_T ParamsSize )
: UStruct( ObjectInitializer, InSuperFunction, ParamsSize )
, FunctionFlags(InFunctionFlags)
, RPCId(0)
, RPCResponseId(0)
, FirstPropertyToInit(nullptr)
#if UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
, EventGraphFunction(nullptr)
, EventGraphCallOffset(0)
#endif
#if WITH_LIVE_CODING
, SingletonPtr(nullptr)
#endif
{
}

UFunction::UFunction(UFunction* InSuperFunction, EFunctionFlags InFunctionFlags, SIZE_T ParamsSize)
	: UStruct(InSuperFunction, ParamsSize)
	, FunctionFlags(InFunctionFlags)
	, RPCId(0)
	, RPCResponseId(0)
	, FirstPropertyToInit(NULL)
#if WITH_LIVE_CODING
	, SingletonPtr(nullptr)
#endif
{
}


void UFunction::InitializeDerivedMembers()
{
	NumParms = 0;
	ParmsSize = 0;
	ReturnValueOffset = MAX_uint16;

	UEProperty_Private::FPropertyListBuilderPostConstructLink ConstructLink(&FirstPropertyToInit);
	for (FProperty* Property = CastField<FProperty>(ChildProperties); Property; Property = CastField<FProperty>(Property->Next))
	{
		if (Property->PropertyFlags & CPF_Parm)
		{
			NumParms++;
			ParmsSize = IntCastChecked<uint16>(Property->GetOffset_ForUFunction() + Property->GetSize());
			if (Property->PropertyFlags & CPF_ReturnParm)
			{
				ReturnValueOffset = IntCastChecked<uint16>(Property->GetOffset_ForUFunction());
			}
		}
		else if ((FunctionFlags & FUNC_HasDefaults) == 0)
		{
			// we're done with parms and we've not been tagged as FUNC_HasDefaults, so we can abort
			// this potentially costly loop:
			break;
		}
		else if (!Property->HasAnyPropertyFlags(CPF_ZeroConstructor))
		{
			ConstructLink.AppendTerminated(*Property);
		}
	}
}

void UFunction::Invoke(UObject* Obj, FFrame& Stack, RESULT_DECL)
{
	checkSlow(Func);

	UClass* OuterClass = (UClass*)GetOuter();
	if (OuterClass->IsChildOf(UInterface::StaticClass()))
	{
		Obj = (UObject*)Obj->GetInterfaceAddress(OuterClass);
	}

	TGuardValue<UFunction*> NativeFuncGuard(Stack.CurrentNativeFunction, this);
	return (*Func)(Obj, Stack, RESULT_PARAM);
}

void UFunction::Serialize( FArchive& Ar )
{
#if WITH_EDITOR
	const static FName NAME_UFunction("UFunction");
	FArchive::FScopeAddDebugData S(Ar, NAME_UFunction);
	FArchive::FScopeAddDebugData Q(Ar, GetFName());
#endif

	Super::Serialize( Ar );

	Ar.ThisContainsCode();

	Ar << FunctionFlags;

	// Replication info.
	if (FunctionFlags & FUNC_Net)
	{
		// Unused
		int16 RepOffset = 0;
		Ar << RepOffset;
	}

#if !UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
	// We need to serialize these values even if the feature is disabled, in order to keep the serialization stream in sync
	UFunction* EventGraphFunction = nullptr;
	int32 EventGraphCallOffset = 0;
#endif

	if (Ar.UEVer() >= VER_UE4_SERIALIZE_BLUEPRINT_EVENTGRAPH_FASTCALLS_IN_UFUNCTION)
	{
		Ar << EventGraphFunction;
		Ar << EventGraphCallOffset;
	}

	// Precomputation.
	if ((Ar.GetPortFlags() & PPF_Duplicate) != 0)
	{
		Ar << NumParms;
		Ar << ParmsSize;
		Ar << ReturnValueOffset;
		Ar << FirstPropertyToInit;
	}
	else
	{
		if (Ar.IsLoading())
		{
			InitializeDerivedMembers();
		}
	}
}

void UFunction::PostLoad()
{
	Super::PostLoad();

	UClass* const OwningClass = GetOuterUClass();
	if (OwningClass && HasAnyFunctionFlags(FUNC_Net))
	{
		OwningClass->ClassFlags &= ~CLASS_ReplicationDataIsSetUp;
	}

	// fix deprecated state.
	if (HasAllFunctionFlags(FUNC_Const | FUNC_Static))
	{
		// static functions can't be marked const
		FunctionFlags &= ~FUNC_Const;
	}
}

FProperty* UFunction::GetReturnProperty() const
{
	for( TFieldIterator<FProperty> It(this); It && (It->PropertyFlags & CPF_Parm); ++It )
	{
		if( It->PropertyFlags & CPF_ReturnParm )
		{
			return *It;
		}
	}
	return NULL;
}

void UFunction::Bind()
{
	UClass* OwnerClass = GetOwnerClass();

	// if this isn't a native function, or this function belongs to a native interface class (which has no C++ version),
	// use ProcessInternal (call into script VM only) as the function pointer for this function
	if (!HasAnyFunctionFlags(FUNC_Native))
	{
		// Use processing function.
		Func = &UObject::ProcessInternal;
	}
	else
	{
#if UE_WITH_CONSTINIT_UOBJECT
		decltype(Func) OldFunc = Func;
#endif // UE_WITH_CONSTINIT_UOBJECT

		// Find the function in the class's native function lookup table.
		FName Name = GetFName();
		FNativeFunctionLookup* Found = OwnerClass->NativeFunctionLookupTable.FindByPredicate([=](const FNativeFunctionLookup& NativeFunctionLookup){ return Name == NativeFunctionLookup.Name; });
		if (Found)
		{
			Func = Found->Pointer;
		}
#if USE_COMPILED_IN_NATIVES
		else if (!HasAnyFunctionFlags(FUNC_NetRequest))
		{
			UE_LOGF(LogClass, Warning,"Failed to bind native function %ls.%ls",*OwnerClass->GetName(),*GetName());
		}
#endif
#if UE_WITH_CONSTINIT_UOBJECT
		if (HasAnyInternalFlags(EInternalObjectFlags::Native))
		{
			check(Func == OldFunc);
		}
#endif // UE_WITH_CONSTINIT_UOBJECT
	}
}

void UFunction::Link(FArchive& Ar, bool bRelinkExistingProperties)
{
	Super::Link(Ar, bRelinkExistingProperties);

	InitializeDerivedMembers();
}

bool UFunction::IsSignatureCompatibleWith(const UFunction* OtherFunction) const
{
	const uint64 IgnoreFlags = UFunction::GetDefaultIgnoredSignatureCompatibilityFlags();

	return IsSignatureCompatibleWith(OtherFunction, IgnoreFlags);
}

bool FStructUtils::ArePropertiesTheSame(const FProperty* A, const FProperty* B, bool bCheckPropertiesNames)
{
	if (A == B)
	{
		return true;
	}

	if (!A || !B) //one of properties is null
	{
		return false;
	}

	if (bCheckPropertiesNames && (A->GetFName() != B->GetFName()))
	{
		return false;
	}

	if (A->GetSize() != B->GetSize())
	{
		return false;
	}

	if (A->GetOffset_ForGC() != B->GetOffset_ForGC())
	{
		return false;
	}

	if (!A->SameType(B))
	{
		return false;
	}

	return true;
}

bool FStructUtils::TheSameLayout(const UStruct* StructA, const UStruct* StructB, bool bCheckPropertiesNames)
{
	if (StructA == StructB)
	{
		return true;
	}

	bool bResult = false;
	if (StructA
		&& StructB
		&& (StructA->GetPropertiesSize() == StructB->GetPropertiesSize())
		&& (StructA->GetMinAlignment() == StructB->GetMinAlignment()))
	{
		const FProperty* PropertyA = StructA->PropertyLink;
		const FProperty* PropertyB = StructB->PropertyLink;

		bResult = true;
		while (bResult && (PropertyA != PropertyB))
		{
			bResult = ArePropertiesTheSame(PropertyA, PropertyB, bCheckPropertiesNames);
			PropertyA = PropertyA ? PropertyA->PropertyLinkNext : NULL;
			PropertyB = PropertyB ? PropertyB->PropertyLinkNext : NULL;
		}

		if(bResult)
		{
			// If structs are actually classes, their 'layout' is affected by their sparse class data too
			const UClass* ClassA = Cast<UClass>(StructA);
			const UClass* ClassB = Cast<UClass>(StructB);
			if(ClassA && ClassB)
			{
				bResult = TheSameLayout(ClassA->GetSparseClassDataStruct(), ClassB->GetSparseClassDataStruct(), bCheckPropertiesNames);
			}
		}
	}
	return bResult;
}

UStruct* FStructUtils::FindStructureInPackageChecked(const TCHAR* StructName, const TCHAR* PackageName)
{
	const FName StructPackageFName(PackageName);
	if (StructPackageFName != NAME_None)
	{
		static TMap<FName, UPackage*> StaticStructPackageMap;

		UPackage* StructPackage;
		UPackage** StructPackagePtr = StaticStructPackageMap.Find(StructPackageFName);
		if (StructPackagePtr != nullptr)
		{
			StructPackage = *StructPackagePtr;
		}
		else
		{
			StructPackage = StaticStructPackageMap.Add(StructPackageFName, FindObjectChecked<UPackage>(nullptr, PackageName));
		}

		return FindObjectChecked<UStruct>(StructPackage, StructName);
	}
	else
	{
		return CastChecked<UStruct>(UClass::TryFindTypeSlow<UStruct>(StructName));
	}
}

bool UFunction::IsSignatureCompatibleWith(const UFunction* OtherFunction, uint64 IgnoreFlags) const
{
	// Early out if they're exactly the same function
	if (this == OtherFunction)
	{
		return true;
	}

	// Run thru the parameter property chains to compare each property
	TFieldIterator<FProperty> IteratorA(this);
	TFieldIterator<FProperty> IteratorB(OtherFunction);

	while (IteratorA && (IteratorA->PropertyFlags & CPF_Parm))
	{
		if (IteratorB && (IteratorB->PropertyFlags & CPF_Parm))
		{
			// Compare the two properties to make sure their types are identical
			// Note: currently this requires both to be strictly identical and wouldn't allow functions that differ only by how derived a class is,
			// which might be desirable when binding delegates, assuming there is directionality in the SignatureIsCompatibleWith call
			FProperty* PropA = *IteratorA;
			FProperty* PropB = *IteratorB;

			// Check the flags as well
			const uint64 PropertyMash = PropA->PropertyFlags ^ PropB->PropertyFlags;
			if (!FStructUtils::ArePropertiesTheSame(PropA, PropB, false) || ((PropertyMash & ~IgnoreFlags) != 0))
			{
				// Type mismatch between an argument of A and B
				return false;
			}
		}
		else
		{
			// B ran out of arguments before A did
			return false;
		}
		++IteratorA;
		++IteratorB;
	}

	// They matched all the way thru A's properties, but it could still be a mismatch if B has remaining parameters
	return !(IteratorB && (IteratorB->PropertyFlags & CPF_Parm));
}

static UScriptStruct* StaticGetBaseStructureInternal(FName Name)
{
	static UPackage* CoreUObjectPkg = FindObjectChecked<UPackage>(nullptr, TEXT("/Script/CoreUObject"));

	UScriptStruct* Result = (UScriptStruct*)StaticFindObjectFastInternal(UScriptStruct::StaticClass(), CoreUObjectPkg, Name, EFindObjectFlags::None, RF_NoFlags, EInternalObjectFlags::None);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (!Result)
	{
		UE_LOGF(LogClass, Fatal, "Failed to find native struct '%ls.%ls'", *CoreUObjectPkg->GetName(), *Name.ToString());
	}
#endif
	return Result;
}

UScriptStruct* TBaseStructure<FIntPoint>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("IntPoint"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FIntVector>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("IntVector"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInt64Vector2>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("Int64Vector2"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FIntVector4>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("IntVector4"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FLinearColor>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("LinearColor"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FColor>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("Color"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FRandomStream>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("RandomStream"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FGuid>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("Guid"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FFallbackStruct>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("FallbackStruct"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurveFloat>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurveFloat"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurveVector2D>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurveVector2D"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurveVector>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurveVector"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurveQuat>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurveQuat"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurveTwoVectors>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurveTwoVectors"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurveLinearColor>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurveLinearColor"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurvePointFloat>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurvePointFloat"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurvePointVector2D>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurvePointVector2D"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurvePointVector>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurvePointVector"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurvePointQuat>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurvePointQuat"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurvePointTwoVectors>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurvePointTwoVectors"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInterpCurvePointLinearColor>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("InterpCurvePointLinearColor"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FFloatRangeBound>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("FloatRangeBound"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FFloatRange>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("FloatRange"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FDoubleRangeBound>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("DoubleRangeBound"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FDoubleRange>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("DoubleRange"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInt32RangeBound>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("Int32RangeBound"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInt32Range>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("Int32Range"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FFloatInterval>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("FloatInterval"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FDoubleInterval>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("DoubleInterval"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FInt32Interval>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("Int32Interval"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FPolyglotTextData>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("PolyglotTextData"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FDateTime>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("DateTime"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FFrameNumber>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("FrameNumber"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FFrameTime>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("FrameTime"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FFrameRate>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("FrameRate"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FRemoteObjectId>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("RemoteObjectId"));
	return ScriptStruct;
}
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", RemoteObjectId);

UScriptStruct* TBaseStructure<FRemoteObjectTables>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("RemoteObjectTables"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FRemoteObjectPathName>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("RemoteObjectPathName"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FPackedRemoteObjectPathName>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("PackedRemoteObjectPathName"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FRemoteObjectBytes>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("RemoteObjectBytes"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FPackedRemoteObjectHeader>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("PackedRemoteObjectHeader"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FRemoteObjectData>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("RemoteObjectData"));
	return ScriptStruct;
}

UScriptStruct* TBaseStructure<FRemoteServerId>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("RemoteServerId"));
	return ScriptStruct;
}
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", RemoteServerId);

UScriptStruct* TBaseStructure<FRemoteObjectReference>::Get()
{
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(TEXT("RemoteObjectReference"));
	return ScriptStruct;
}
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", RemoteObjectReference);

#define UE_DEFINE_CORE_VARIANT_TYPE(VARIANT, CORE)												\
UScriptStruct* TBaseStructure<F##CORE>::Get()												\
{																								\
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(NAME_##CORE);			\
	return ScriptStruct;																		\
}																								\
UScriptStruct* TVariantStructure<F##VARIANT##f>::Get()											\
{																								\
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(NAME_##VARIANT##f);		\
	return ScriptStruct;																		\
}																								\
UScriptStruct* TVariantStructure<F##VARIANT##d>::Get()											\
{																								\
	static UScriptStruct* ScriptStruct = StaticGetBaseStructureInternal(NAME_##VARIANT##d);		\
	return ScriptStruct;																		\
}

UE_DEFINE_CORE_VARIANT_TYPE(Vector2,	Vector2D);
UE_DEFINE_CORE_VARIANT_TYPE(Vector3,	Vector);
UE_DEFINE_CORE_VARIANT_TYPE(Vector4,	Vector4);
UE_DEFINE_CORE_VARIANT_TYPE(Plane4,		Plane);
UE_DEFINE_CORE_VARIANT_TYPE(Quat4,		Quat);
UE_DEFINE_CORE_VARIANT_TYPE(Rotator3,	Rotator);
UE_DEFINE_CORE_VARIANT_TYPE(Transform3,	Transform);
UE_DEFINE_CORE_VARIANT_TYPE(Matrix44,	Matrix);
UE_DEFINE_CORE_VARIANT_TYPE(Box2,		Box2D);
UE_DEFINE_CORE_VARIANT_TYPE(Ray3,		Ray);
UE_DEFINE_CORE_VARIANT_TYPE(Sphere3,	Sphere);

#undef UE_DEFINE_CORE_VARIANT_TYPE

UDelegateFunction::UDelegateFunction(const FObjectInitializer& ObjectInitializer, UFunction* InSuperFunction, EFunctionFlags InFunctionFlags, SIZE_T ParamsSize)
	: UFunction(ObjectInitializer, InSuperFunction, InFunctionFlags, ParamsSize)
{

}

UDelegateFunction::UDelegateFunction(UFunction* InSuperFunction, EFunctionFlags InFunctionFlags, SIZE_T ParamsSize)
	: UFunction(InSuperFunction, InFunctionFlags, ParamsSize)
{

}

USparseDelegateFunction::USparseDelegateFunction(const FObjectInitializer& ObjectInitializer, UFunction* InSuperFunction, EFunctionFlags InFunctionFlags, SIZE_T ParamsSize)
	: UDelegateFunction(ObjectInitializer, InSuperFunction, InFunctionFlags, ParamsSize)
{

}

USparseDelegateFunction::USparseDelegateFunction(UFunction* InSuperFunction, EFunctionFlags InFunctionFlags, SIZE_T ParamsSize)
	: UDelegateFunction(InSuperFunction, InFunctionFlags, ParamsSize)
{

}

void USparseDelegateFunction::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << OwningClassName;
	Ar << DelegateName;
}

#if defined(_MSC_VER) && _MSC_VER == 1900
	#ifdef PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS
		PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS
	#endif
#endif
