// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#if UE_WITH_CONSTINIT_UOBJECT 

#include "Concepts/DerivedFrom.h"
#include "Misc/TVariant.h"
#include "UObject/CompiledInObjectPtr.h"
#include "UObject/RegisterCompiledInObjects.h"
#include "UObject/UObjectGlobals.h"

class FCompiledInObjectRegistry
{
    UE_NONCOPYABLE(FCompiledInObjectRegistry);
    FCompiledInObjectRegistry() = default;
    ~FCompiledInObjectRegistry() = default;

    // Matches list markers in TList noting which objects have been constructed to which step 
    enum class EConstructionStep
    {
        AddObjects,
        ConstructObjects,
        ConstructDefaultObjects,
    };

    template<typename T> 
    struct TList
    {
        // Linked list of objects to construct. New items are added here, before items that may have
        // already been partially processed which are indicated by the following pointers.
        T* Head = nullptr;
        // First block of objects that have already been processed by AddAndHashObjects
        T* AlreadyAdded = nullptr;
        // First block of objects that have already been processed by FinishConstructingObjects
        T* AlreadyConstructed = nullptr;
        // First block of objects that have already been processed by CreateClassDefaultObjects
        T* AlreadyConstructedDefaultObjects = nullptr;

        /** Remove all objects after all construction work is complete. */
        void EmptyObjects();

        [[nodiscard]] T* AdvanceConstruction(EConstructionStep Step)
        {
            T** Link = nullptr;
            switch (Step)
            {
                case EConstructionStep::AddObjects: 
                    Link = &AlreadyAdded;
                    break;
                case EConstructionStep::ConstructObjects: 
                    Link = &AlreadyConstructed;
                    break;
                case EConstructionStep::ConstructDefaultObjects: 
                    Link = &AlreadyConstructedDefaultObjects;
                    break;
                default: check(false); 
            }

            T* Ret = *Link;
            *Link = Head;
            return Ret;
        }

        /** Return whether there are any registrants that not been processed by AddAndHashObjects or FinishConstructingObjects. */
        [[nodiscard]] bool HasObjectsPendingConstruction() const
        {
            return Head && (Head != AlreadyAdded || Head != AlreadyConstructed);
        }

        /** Return whether there are any registrants that have not done all phases of construction. */
        [[nodiscard]] bool HasPendingObjects() const
        {
            return Head && (Head != AlreadyAdded || Head != AlreadyConstructed || Head != AlreadyConstructedDefaultObjects);
        }
    };

    // Used to mark a point in the lists to iterate up to when advancing construction
    using FIterationMarker = TTuple<const FRegisterCompiledInObjects*, const FRegisterIntrinsicClass*>;

    // UHT-generated objects in various states of construction
    TList<FRegisterCompiledInObjects> GeneratedObjects;
    // Intrinsic classes in various states of construction
    TList<FRegisterIntrinsicClass> IntrinsicClasses;

public:
    static FCompiledInObjectRegistry& Get()
    {
        static FCompiledInObjectRegistry Singleton;
        return Singleton;
    }

    /** Add the given set of objects to the linked list of items to process. */
    void AddObjects(FRegisterCompiledInObjects* Info);

    /** Add the given intrinsic class to the linked list of items to process. */
    void AddIntrinsicClass(FRegisterIntrinsicClass* Info);

    /** Remove all objects after all construction work is complete. */
    void EmptyObjects();

    /** Return whether there are any registrants that not been processed by AddAndHashObjects or FinishConstructingObjects. */
    [[nodiscard]] bool HasObjectsPendingConstruction() const;

    /** Return whether there are any registrants that have done all phases of construction. */
    [[nodiscard]] bool HasPendingObjects() const;

    /** 
     * Initialize the name and index of all pending objects and add them to the object hash.
     * Also links cross-module pointers if necessary.
     */
    void AddAndHashObjects();

    /** 
     * Finalize construction of all pending compiled in objects but not class default objects
     */
    void FinishConstructingObjects();

    /** 
     * Create class default objects for all pending classes 
     * Note that InModuleName does not filter which objects are constructed at this point, it is just forwarded
     * to FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate. Per-module initialization for modular builds
     * and hot reload is not yet implemented. 
     */
    void CreateClassDefaultObjects(FName InModuleName);

    /** Assemble reference token stream for all registered classes. */
    void AssembleReferenceTokenStream();

private:
	// Get the links in the lists of generated and intrinsic objects which have already completed the given construction step.
    [[nodiscard]] FIterationMarker AdvanceConstruction(EConstructionStep Step);
    
    // Iterate all objects before the given links and call the functors on the FRegisterCompiledInObjects and FRegisterIntrinsicClass structs
    template<typename... FUNCTORS>
    void Iterate(FIterationMarker Stop, FUNCTORS&&... FS);

    // Iterate all objects before the given links, calling the functor on all non-null entries
    template<typename FUNCTOR>
    void IterateObjects(FIterationMarker Stop, FUNCTOR&& F);

    // Iterate all objects of type UScriptStruct before the given links, calling the functor on all non-null entries
    template<typename FUNCTOR>
    void IterateScriptStructs(FIterationMarker Stop, FUNCTOR&& F);

    // Iterate all objects of type UClass before the given links, calling the functor on all non-null entries
    template<typename FUNCTOR>
    void IterateClasses(FIterationMarker Stop, FUNCTOR&& F);

    // Iterate all intrinsic classes before the given links, calling the functor on the intrinsic class and its static constructor function
    template<typename FUNCTOR>
    void IterateIntrinsicClasses(FIterationMarker Stop, FUNCTOR&& F);

#if WITH_METADATA
	static TConstArrayView<UE::CodeGen::ConstInit::FMetaData> GetCompiledInMetaData(UEnum* InEnum);
	static TConstArrayView<UE::CodeGen::ConstInit::FMetaData> GetCompiledInMetaData(UStruct* InStruct);
	static void AddMetaData(UObject* Object, TConstArrayView<UE::CodeGen::ConstInit::FMetaData> InMetaData);
#endif
	static void AddMetaData(UStruct* Object);
	static void AddMetaData(UEnum* Object);

};

#endif // UE_WITH_CONSTINIT_UOBJECT 