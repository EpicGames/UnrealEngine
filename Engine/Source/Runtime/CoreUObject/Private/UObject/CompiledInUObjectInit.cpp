// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/CompiledInObjectPtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyOptional.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectPartials.h"

#if UE_WITH_CONSTINIT_UOBJECT

#include "Logging/StructuredLog.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Misc/TVariant.h"
#include "Serialization/AsyncLoadingEvents.h"
#include "Serialization/LoadTimeTrace.h"
#include "Templates/Overload.h"
#include "UObject/CompiledInObjectRegistry.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectPrivate.h"
#include "VerseVM/VVMVerseClass.h"
#include "VerseVM/VVMVerseEnum.h"
#include "VerseVM/VVMVerseFunction.h"
#include "VerseVM/VVMVerseStruct.h"

// Initialization functions called during boot.

void UClassRegisterAllCompiledInClasses();
void ProcessNewlyLoadedUObjects(FName InModuleName, bool bCanProcessNewlyLoadedObjects);


// Requirements
static_assert(!WITH_LIVE_CODING, "UE_WITH_CONSTINIT_UOBJECT and WITH_LIVE_CODING cannot currently be set simultaneously.");
static_assert(!WITH_HOT_RELOAD, "UE_WITH_CONSTINIT_UOBJECT and WITH_HOT_RELOAD cannot currently be set simultaneously.");
static_assert(!WITH_RELOAD, "UE_WITH_CONSTINIT_UOBJECT and WITH_RELOAD cannot currently be set simultaneously.");
static_assert(!UE_WITH_REMOTE_OBJECT_HANDLE, "UE_WITH_CONSTINIT_UOBJECT and UE_WITH_REMOTE_OBJECT_HANDLE cannot currently be set simultaneously.");
static_assert(!USE_PER_MODULE_UOBJECT_BOOTSTRAP, "UE_WITH_CONSTINIT_UOBJECT and USE_PER_MODULE_UOBJECT_BOOTSTRAP cannot currently be set simultaneously.");



void UClassRegisterAllCompiledInClasses()
{
	SCOPED_BOOT_TIMING("UClassRegisterAllCompiledInClasses");
	LLM_SCOPE(ELLMTag::UObject);
	UE_LOGFMT(LogInit, Verbose, "UClassRegisterAllCompiledInClasses");

	// Don't seem to actually need to do anything here - Objects are already constructed, but we don't
	// want to hash or index them yet because necessary config has not been initialized.
}

void UObjectForceRegistration(UObjectBase* Object, bool bCheckForModuleRelease)
{
	// No-op for constinit uobjects case
}

void UObjectProcessRegistrants()
{
	SCOPED_BOOT_TIMING("UObjectProcessRegistrants");
	LLM_SCOPE(ELLMTag::UObject);
	check(UObjectInitialized());

	FCompiledInObjectRegistry& Registry = FCompiledInObjectRegistry::Get();
	Registry.AddAndHashObjects();
}

void UE::CoreUObject::ConstructCompiledInObjects()
{
	SCOPED_BOOT_TIMING("UE::CoreUObject::ConstructCompiledInObjects");
	LLM_SCOPE(ELLMTag::UObject);
	check(UObjectInitialized());

	FCompiledInObjectRegistry& Registry = FCompiledInObjectRegistry::Get();
	while (Registry.HasObjectsPendingConstruction())
	{
		Registry.AddAndHashObjects();
		Registry.FinishConstructingObjects();
	}
}

void ProcessNewlyLoadedUObjects(FName InModuleName, bool bCanProcessNewlyLoadedObjects)
{
	if (!bCanProcessNewlyLoadedObjects)
	{
#if !IS_MONOLITHIC
		// Even if we're delayed, do minimal construction to fixup names and encoded pointers because some
		// module loads like to call StaticClass before they really ought to
		if (UObjectInitialized())
		{
			FCompiledInObjectRegistry::Get().AddAndHashObjects();
		}
#endif
		FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.Broadcast(InModuleName, ECompiledInUObjectsRegisteredStatus::Delayed);
		return;
	}
	SCOPED_BOOT_TIMING("ProcessNewlyLoadedUObjects");
	LLM_SCOPE(ELLMTag::UObject);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ProcessNewlyLoadedUObjects"), STAT_ProcessNewlyLoadedUObjects, STATGROUP_ObjectVerbose);

	check(UObjectInitialized());
	FCompiledInObjectRegistry& Registry = FCompiledInObjectRegistry::Get();

	while (Registry.HasPendingObjects())
	{
		Registry.AddAndHashObjects();
		Registry.FinishConstructingObjects();
		Registry.CreateClassDefaultObjects(InModuleName);
	}
	if (!GIsInitialLoad)
	{
		Registry.AssembleReferenceTokenStream();
	}
	Registry.EmptyObjects();

	FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.Broadcast(InModuleName, ECompiledInUObjectsRegisteredStatus::PostCDO);
}


static void InitializeConstInitProperties(UStruct* Struct);

namespace UE::CoreUObject::Private
{
#if !IS_MONOLITHIC
	class FCompiledInObjectLinker
	{
		TConstArrayView<TConstArrayView<UObject*>> LinkedModules;
	public:
		FCompiledInObjectLinker(TConstArrayView<TConstArrayView<UObject*>> InLinkedModules)
			: LinkedModules(InLinkedModules)
		{
		}

		// Handler for all TObjectPtr-like things
		template<typename T>
		void LinkCompiledInObjectPointer(T& InPtr)
		{
			LinkCompiledInObjectPointer(ObjectPtr_Private::Friend::GetHandleRef(InPtr));
		}
		
		void LinkCompiledInObjectPointer(FObjectHandle& InHandle)
		{
			if (InHandle.CompiledInPtr.IsEncodedRef())
			{
				UObject* Object = FindCompiledInObject(InHandle.CompiledInPtr);
				check(Object);
				InHandle.Pointer = Object;
			}
		}

		void LinkCompiledInObjectPointer(UE::CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain>& InPtr)
		{
			if (InPtr.IsEncodedRef())
			{
				UStruct* Object = FindCompiledInObject<UStruct>(InPtr.CompiledInPtr);
				check(Object);
				InPtr.Pointer = UE::Private::AsStructBaseChain(Object);
			}
		}

		template<UE::CDerivedFrom<UObject> T>
		void LinkCompiledInObjectPointer(UE::CodeGen::ConstInit::TCompiledInObjectPtr<T>& InPtr)
		{
			if (InPtr.IsEncodedRef())
			{
				T* Object = FindCompiledInObject<T>(InPtr.CompiledInPtr);
				check(Object);
				InPtr.Pointer = Object;
			}
		}

		/**
		 * Look up a compiled in object by its encoded pointer and return it as the given type.
		 * Type checking is NOT performed, a reference to an object of the correct type is expected to have been
		 * encoded by UHT.
		 */
		template<UE::CDerivedFrom<UObject> T> 
		T* FindCompiledInObject(UE::CodeGen::ConstInit::FCompiledInObjectPtr InPtr)
		{
			return reinterpret_cast<T*>(FindCompiledInObject(InPtr));
		}

		/** Look up a compiled in object by its encoded pointer and return it */
		UObject* FindCompiledInObject(UE::CodeGen::ConstInit::FCompiledInObjectPtr InPtr)
		{
			const UE::CodeGen::ConstInit::FCompiledInObjectReference Decoded = InPtr.GetDecoded();
			return LinkedModules[Decoded.ModuleId][Decoded.ObjectId];
		}
	};
#endif // !IS_MONOLITHIC
}

template<typename T>
void FCompiledInObjectRegistry::TList<T>::EmptyObjects()
{
	checkfSlow(Head == nullptr || (Head == AlreadyAdded && Head == AlreadyConstructed && Head == AlreadyConstructedDefaultObjects), 
		TEXT("Attempting to empty deferred registry when we don't appear to have constructed all objects"));
	// Just drop the whole list - if the structs are recompiled and call Link again we will replace the Next ptr
	Head = nullptr;
	AlreadyAdded = nullptr;
	AlreadyConstructed = nullptr;
	AlreadyConstructedDefaultObjects = nullptr;
}

void FCompiledInObjectRegistry::EmptyObjects()
{
	GeneratedObjects.EmptyObjects();
	IntrinsicClasses.EmptyObjects();
}

bool FCompiledInObjectRegistry::HasObjectsPendingConstruction() const
{
	return GeneratedObjects.HasObjectsPendingConstruction() || IntrinsicClasses.HasObjectsPendingConstruction();
}

bool FCompiledInObjectRegistry::HasPendingObjects() const
{
	return GeneratedObjects.HasPendingObjects() || IntrinsicClasses.HasPendingObjects();
}

FCompiledInObjectRegistry::FIterationMarker FCompiledInObjectRegistry::AdvanceConstruction(EConstructionStep Step)
{
	return MakeTuple(GeneratedObjects.AdvanceConstruction(Step), IntrinsicClasses.AdvanceConstruction(Step));
}

template<typename... FUNCTORS>
void FCompiledInObjectRegistry::Iterate(FIterationMarker Stop, FUNCTORS&&... FS)
{
	auto Dispatch = UE::Overload(Forward<FUNCTORS>(FS)...);

	if constexpr (std::is_invocable_v<decltype(Dispatch), const FRegisterIntrinsicClass&>)
	{
		for (const FRegisterIntrinsicClass* It = IntrinsicClasses.Head; It != Stop.Get<const FRegisterIntrinsicClass*>(); It = It->ListNext)
		{
			Dispatch(*It);
		}
	}

	if constexpr (std::is_invocable_v<decltype(Dispatch), const FRegisterCompiledInObjects&>)
	{
		for (const FRegisterCompiledInObjects* It = GeneratedObjects.Head; It != Stop.Get<const FRegisterCompiledInObjects*>(); It = It->ListNext)
		{
			Dispatch(*It);	
		}
	}
}

template<typename FUNCTOR>
void FCompiledInObjectRegistry::IterateObjects(FIterationMarker Stop, FUNCTOR&& F)
{
	Iterate(Stop, 
		[&F](const FRegisterCompiledInObjects& Generated)
		{
			for (UPackage* Package : Generated.Objects.Packages)
			{
				F(Package);
			}
			for (UClass* Class : Generated.Objects.GetClasses())
			{
				if(Class)
				{
					F(Class);
				}
			}
			for (UScriptStruct* Struct : Generated.Objects.GetStructs())
			{
				if (Struct)
				{
					F(Struct);
				}
			}
			for (UEnum* Enum : Generated.Objects.GetEnums())
			{
				if (Enum)
				{
					F(Enum);
				}
			}
			for (UObject* Obj : Generated.Objects.Others)
			{
				if (Obj)
				{
					F(Obj);
				}
			}
		},
		[&F](const FRegisterIntrinsicClass& Intrinsic)
		{
			F(Intrinsic.Class);
		}
	);
}

template<typename FUNCTOR>
void FCompiledInObjectRegistry::IterateScriptStructs(FIterationMarker Stop, FUNCTOR&& F)
{
	Iterate(Stop, 
		[&F](const FRegisterCompiledInObjects& Generated)
		{
			for (UScriptStruct* Struct : Generated.Objects.GetStructs())
			{
				if (Struct)
				{ 
					F(Struct);
				}
			}
			for (UObject* Obj : Generated.Objects.Others)
			{
				if(UScriptStruct* Struct = Cast<UScriptStruct>(Obj))
				{
					F(Struct);
				}
			}
		}
	);
}

template<typename FUNCTOR>
void FCompiledInObjectRegistry::IterateClasses(FIterationMarker Stop, FUNCTOR&& F)
{
	Iterate(Stop, 
		[&F](const FRegisterCompiledInObjects& Generated)
		{
			for (UClass* Class : Generated.Objects.GetClasses())
			{
				if (Class)
				{
					F(Class);
				}
			}
		},
		[&F](const FRegisterIntrinsicClass& Intrinsic)
		{
			F(Intrinsic.Class);
		}
	);
}

template<typename FUNCTOR>
void FCompiledInObjectRegistry::IterateIntrinsicClasses(FIterationMarker Stop, FUNCTOR&& F)
{
	Iterate(Stop, 
		[&F](const FRegisterIntrinsicClass& Intrinsic) 
		{
			F(Intrinsic.Class, Intrinsic.Constructor);
		});
}

/** Initialize the name and index of all pending objects and add them to the object hash. */
void FCompiledInObjectRegistry::AddAndHashObjects()
{
	SCOPED_BOOT_TIMING("FDeferredRegistry::AddAndHashPendingRegistrants");
	FIterationMarker Stop = AdvanceConstruction(EConstructionStep::AddObjects);

	{
		TArray<UClass*> PartialClasses;
		Iterate(Stop, 
			[&](const FRegisterCompiledInObjects& Generated)
			{
				for (UE::CoreUObject::Private::FPartialClass* Partial : Generated.Objects.Partials)
				{
					UClass* PartialClass = Partial->Class;
					if (!PartialClass->Partials)
					{
						PartialClasses.Add(PartialClass);
					}
					Partial->LinkToClass(*PartialClass);
				}
			}
		);
		Algo::Sort(PartialClasses, [](UClass* A, UClass* B) { return B->IsChildOf(A); });
		for (UClass* PartialClass : PartialClasses)
		{
			UE::CoreUObject::Private::ConstructUClassPartials(*PartialClass);
		}
	}

	// Fixup object pointers first - in order to hash a class we will need e.g. the super pointer
	// This is only necessary for generated objects and not intrinsic ones
#if !IS_MONOLITHIC
	Iterate(Stop,
		[this](const FRegisterCompiledInObjects& Reg)
		{
			UE::CoreUObject::Private::FCompiledInObjectLinker Linker(Reg.Objects.LinkedModules);
			for (UPackage* Package : Reg.Objects.Packages)
			{
				Package->LinkCompiledInPointerFields(Linker);
			}
			for (UClass* Class : Reg.Objects.GetClasses())
			{
				if (Class)
				{ 
					UE::CoreUObject::Private::ConstructUClassPartials(*Class);
					Class->LinkCompiledInPointerFields(Linker);
					for (UField* Field = Class->Children; Field; Field = Field->Next)
					{
						Field->LinkCompiledInPointerFields(Linker);
					}
				}
			}
			for (UScriptStruct* Struct: Reg.Objects.GetStructs())
			{
				if (Struct)
				{ 
					Struct->LinkCompiledInPointerFields(Linker);
					for (UField* Field = Struct->Children; Field; Field = Field->Next)
					{
						Field->LinkCompiledInPointerFields(Linker);
					}
				}
			}
			for (UEnum* Enum : Reg.Objects.GetEnums())
			{
				if (Enum)
				{
					Enum->LinkCompiledInPointerFields(Linker);
				}
			}
			for (UObject* Obj : Reg.Objects.Others)
			{
				if (Obj)
				{
					Obj->LinkCompiledInPointerFields(Linker);
				}
			}
		}
	);
#endif // !IS_MONOLITHIC

	IterateObjects(Stop, [](UObject* Obj)
	{
		Obj->AddConstInitObject();
		// Initialize UStruct and UFunction objects
		if (UStruct* Struct = Cast<UStruct>(Obj))
		{
			InitializeConstInitProperties(Struct);
			for (UField* Field = Struct->Children; Field; Field = Field ->Next)
			{
				Field->AddConstInitObject();
				if (UStruct* InnerStruct = Cast<UStruct>(Field))
				{
					InitializeConstInitProperties(InnerStruct);
				}
			}
		}
	});

	IterateObjects(Stop, [](UObject* Obj)
	{
		// TODO: General purpose interface function for initializing compiled in FNames
		if (UClass* Class = Cast<UClass>(Obj))
		{
			UE::CoreUObject::Private::ConstructUClassPartials(*Class); // We need to call this again because there might be subclasses to partial classes that has no partials

			Class->ClassConfigName = FName(Class->CompiledInClassConfigName);

			// Create TArray of native functions from compiled-in data to maintain API
			// Note: this needs to be done before native function binding
			if (Class != UObject::StaticClass())
			{
				TConstArrayView<UE::CodeGen::FClassNativeFunction> CompiledInNativeFunctions = Class->CompiledInNativeFunctions;
				new (&Class->NativeFunctionLookupTable) TArray<FNativeFunctionLookup>(CompiledInNativeFunctions);

				// Partials have functions too that needs to be added to the NativeFunctionLookupTable
				for (UE::CoreUObject::Private::FPartialClass* It = Class->Partials; It ; It = It->NextPartial)
				{
					for (UField* Child = It->FirstChild; Child != It->EndChild; Child = Child->Next)
					{
						UFunction* Func = (UFunction*)Child;
						Class->AddNativeFunction(*Func->GetName(), Func->GetNativeFunc());
					}
				}
			}
		}
		else if (USparseDelegateFunction* SparseDelegate = Cast<USparseDelegateFunction>(Obj))
		{
			const UTF8CHAR* OwningClassName = SparseDelegate->CompiledInOwningClassName;
			const UTF8CHAR* DelegateName = SparseDelegate->CompiledInDelegateName;
			new (&SparseDelegate->OwningClassName) FName(OwningClassName); // Can this not be inferred from elsewhere...?
			new (&SparseDelegate->DelegateName) FName(DelegateName);
		}
	});
}

#if WITH_METADATA
void FCompiledInObjectRegistry::AddMetaData(UObject* Object, TConstArrayView<UE::CodeGen::ConstInit::FMetaData> InMetaData)
{
	if (InMetaData.Num())
	{
		FMetaData& MetaData = Object->GetPackage()->GetMetaData();
		for (const UE::CodeGen::ConstInit::FMetaData& MetaDataParam : InMetaData)
		{
			MetaData.SetValue(Object, UTF8_TO_TCHAR(reinterpret_cast<const UTF8CHAR*>(MetaDataParam.NameUTF8)), UTF8_TO_TCHAR(reinterpret_cast<const UTF8CHAR*>(MetaDataParam.ValueUTF8)));
		}
	}
}

TConstArrayView<UE::CodeGen::ConstInit::FMetaData> FCompiledInObjectRegistry::GetCompiledInMetaData(UStruct* InStruct)
{
	TConstArrayView<UE::CodeGen::ConstInit::FMetaData> MetaData = InStruct->CompiledInMetaData;
	static_assert(STRUCT_OFFSET(UStruct, CompiledInMetaData) == STRUCT_OFFSET(UStruct, Script));
	new (&InStruct->Script) TArray<uint8>(); // Activate other union member
	return MetaData;
}

TConstArrayView<UE::CodeGen::ConstInit::FMetaData> FCompiledInObjectRegistry::GetCompiledInMetaData(UEnum* InEnum)
{
	TConstArrayView<UE::CodeGen::ConstInit::FMetaData> MetaData = InEnum->CompiledInMetaData;
	InEnum->CompiledInMetaData = {}; // This is not aliased with anything 
	return MetaData;
}
#endif // WITH_METADATA

void FCompiledInObjectRegistry::AddMetaData(UStruct* Object)
{
#if WITH_METADATA
	AddMetaData(Object, GetCompiledInMetaData(Object));
#endif
}
void FCompiledInObjectRegistry::AddMetaData(UEnum* Object)
{
#if WITH_METADATA
	AddMetaData(Object, GetCompiledInMetaData(Object));
#endif
}

/** Finalize construction of all pending objects */
void FCompiledInObjectRegistry::FinishConstructingObjects()
{
	SCOPED_BOOT_TIMING("FDeferredRegistry::FinishConstructingPendingRegistrants");
	FIterationMarker Stop = AdvanceConstruction(EConstructionStep::ConstructObjects);
	// Prepare struct cpp ops before linking to try and remove dependency between FStructProperty and UScriptStruct
	IterateScriptStructs(Stop, [](UScriptStruct* Struct){
		Struct->PrepareCppStructOps();
	});

	auto ConstructScriptStruct = [](UScriptStruct* Struct)
	{
		// Get MetaData from union struct fields before linking
		AddMetaData(Struct);
		Struct->StaticLink();
		if (UVerseStruct* VerseStruct = Cast<UVerseStruct>(Struct))
		{
			VerseStruct->QualifiedName = VerseStruct->CompiledInQualifiedName;
		}
		NotifyRegistrationEvent(
			Struct->GetOutermost()->GetFName(),
			Struct->GetFName(),
			ENotifyRegistrationType::NRT_Struct,
			ENotifyRegistrationPhase::NRP_Finished,
			nullptr,
			false,
			Struct
		);
	};
	auto ConstructDelegate = [](UDelegateFunction* Delegate)
	{
		// Get MetaData from union struct fields before linking
		AddMetaData(Delegate);
		//	ConstructUFunctionInternal
		Delegate->Bind(); // Bind should no longer be necessary as constinit functions have their native function pointer assigned by UHT
		Delegate->StaticLink();
		if (Delegate->GetOuter() == Delegate->GetPackage())
		{
			// Notify loader of new top level noexport objects like UScriptStruct, UDelegateFunction and USparseDelegateFunction
			NotifyRegistrationEvent(
				Delegate->GetPackage()->GetFName(),
				Delegate->GetFName(),
				ENotifyRegistrationType::NRT_NoExportObject,
				ENotifyRegistrationPhase::NRP_Finished,
				nullptr,
				false,
				Delegate
			);
		}
	};
	auto ConstructFunction = [](UFunction* Function)
	{
		// Get MetaData from union struct fields before linking
		AddMetaData(Function);
		if (UVerseFunction* VerseFunction = Cast<UVerseFunction>(Function))
		{
			VerseFunction->AlternateName = VerseFunction->CompiledInAlternateNameUTF8;
		}
		Function->Bind(); // Bind should no longer be necessary as constinit functions have their native function pointer assigned by UHT
		Function->StaticLink();
	};
	auto ConstructEnum = [](UEnum* Enum)
	{
		// Get MetaData from union struct fields before linking
		AddMetaData(Enum);
		Enum->InitializeNames();
		if (UVerseEnum* VerseEnum = Cast<UVerseEnum>(Enum))
		{
			VerseEnum->QualifiedName = VerseEnum->CompiledInQualifiedName;
		}
		NotifyRegistrationEvent(
			Enum->GetOutermost()->GetFName(),
			Enum->GetFName(),
			ENotifyRegistrationType::NRT_Enum,
			ENotifyRegistrationPhase::NRP_Finished,
			nullptr,
			false,
			Enum
		);
	};
	auto ConstructObject = [&](UObject* Object)
	{
		if (UScriptStruct* Struct = Cast<UScriptStruct>(Object))
		{
			ConstructScriptStruct(Struct);
		}
		else if (UDelegateFunction* Delegate = Cast<UDelegateFunction>(Object))
		{
			ConstructDelegate(Delegate);
		}
		else if (UFunction* Function = Cast<UFunction>(Object))
		{
			ConstructFunction(Function);
		}
		else if (UEnum* Enum = Cast<UEnum>(Object))
		{
			ConstructEnum(Enum);
		}
		else 
		{
			checkf(false, TEXT("Unknown compiled-in object type to construct: %s"), *Object->GetFullName());
		}
	};
	{
		SCOPED_BOOT_TIMING("ConstructNonClassObjects");
		IterateObjects(Stop, UE::Overload(
			[](UPackage* Package) {},
			[](UClass* Class) {},
			ConstructObject,
			ConstructEnum,
			ConstructFunction,
			ConstructDelegate,
			ConstructScriptStruct
		));
	}

	SCOPED_BOOT_TIMING("ConstructClasses");

	// Execute manual construction code for intrinsic classes, which may create properties, before linking
	IterateIntrinsicClasses(Stop, [&](UClass* Class, void (*IntrinsicClassConstructor)()){
		IntrinsicClassConstructor();
	});
	IterateClasses(Stop ,[&](UClass* Class){
		AddMetaData(Class);
		if (UClass* SuperClass = Class->GetSuperClass())
		{
			checkfSlow(EnumHasAllFlags(Class->ClassFlags, (SuperClass->ClassFlags & CLASS_Inherit)), TEXT("Inheritable flags were not all propagated from %s to %s"), *SuperClass->GetPathName(), *Class->GetPathName());
			checkfSlow(EnumHasAllFlags(Class->ClassCastFlags, SuperClass->ClassCastFlags), TEXT("Class cast flags were no tall propagated from %s to %s"), *SuperClass->GetPathName(), *Class->GetPathName());
		}
		Class->ClassFlags |= CLASS_Constructed;

		// Make sure the reference token stream is empty since it will be reconstructed later on
		// This should not apply to intrinsic classes since they emit native references before AssembleReferenceTokenStream is called.
		if ((Class->ClassFlags & CLASS_Intrinsic) != CLASS_Intrinsic)
		{
			check((Class->ClassFlags & CLASS_TokenStreamAssembled) != CLASS_TokenStreamAssembled);
			Class->ReferenceSchema.Reset();
		}

		Class->InitFuncMap();
		{
			TConstArrayView<UE::CodeGen::ConstInit::FClassImplementedInterface> Interfaces = Class->CompiledInInterfaces;
			if (UVerseClass* VerseClass = Cast<UVerseClass>(Class))
			{
				for (const UE::CodeGen::ConstInit::FClassImplementedInterface& ImplementedInterface : Interfaces)
				{
					if (ImplementedInterface.bVerseDirectInterface)
					{
						if (UVerseClass* InterfaceClass = CastChecked<UVerseClass>((UClass*)ImplementedInterface.Class, ECastCheckedType::NullAllowed))
						{
							VerseClass->DirectInterfaces.Add(InterfaceClass);
						}
					}
				}
			}
			

			new (&Class->Interfaces) TArray<FImplementedInterface>();
			Class->Interfaces.Reserve(Interfaces.Num());
			for (const UE::CodeGen::ConstInit::FClassImplementedInterface& Interface : Interfaces)
			{
				Class->Interfaces.Emplace(Interface.Class, Interface.PointerOffset, Interface.bImplementedByK2);
			}
		}

		Class->StaticLink();

		if (UVerseClass* VerseClass = Cast<UVerseClass>(Class))
		{
			VerseClass->MangledPackageVersePath = VerseClass->CompiledInMangledPackageVersePath;
			VerseClass->PackageRelativeVersePath = VerseClass->CompiledInPackageRelativeVersePath;
		}

		// Initialize functions of classes after the class is constructed
		for (UField* Field = Class->Children; Field; Field = Field->Next)
		{
			ConstructObject(Field);
		}

		UE_LOGF(LogUObjectBootstrap, Verbose, "UObjectLoadAllCompiledInDefaultProperties After Registrant %ls %ls", *Class->GetPackage()->GetName(), *Class->GetName());
	});

}

void FCompiledInObjectRegistry::CreateClassDefaultObjects(FName InModuleName)
{
	TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("FCompiledInObjectRegistry::CreateClassDefaultObjects"));
	SCOPED_BOOT_TIMING("FCompiledInObjectRegistry::CreateClassDefaultObjects");

	FIterationMarker Stop = AdvanceConstruction(EConstructionStep::ConstructDefaultObjects);

	FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.Broadcast(InModuleName, ECompiledInUObjectsRegisteredStatus::PreCDO);

	static FName LongEnginePackageName("/Script/Engine");

	TArray<UClass*> NewClasses;
	TArray<UClass*> NewClassesInCoreUObject;
	TArray<UClass*> NewClassesInEngine;
	IterateClasses(Stop, [&](UClass* Class){
		FName PackageName = Class->GetOutermost()->GetFName();
		if (PackageName == GLongCoreUObjectPackageName)
		{
			NewClassesInCoreUObject.Add(Class);
		}
		else if (PackageName == LongEnginePackageName)
		{
			NewClassesInEngine.Add(Class);
		}
		else
		{
			NewClasses.Add(Class);
		}
	});

	// Sort classes by class hierarchy to avoid deep recursion in creating default objects - registration is reversed because of linked list add-at-head behavior
	auto SortClasses = [](UClass* A, UClass* B)
	{
		if (B->IsChildOf(A))
		{
			return true;
		}	
		return false;
	};
	Algo::Sort(NewClassesInCoreUObject, SortClasses);
	Algo::Sort(NewClassesInEngine, SortClasses);
	Algo::Sort(NewClasses, SortClasses);
	
	// notify async loader of all new classes before creating the class default objects
	for (const TArray<UClass*>* Array : {&NewClassesInCoreUObject, &NewClassesInEngine, &NewClasses })
	{
		SCOPED_BOOT_TIMING("NotifyClassFinishedRegistrationEvents");
		for (UClass* Class : *Array)
		{
			NotifyRegistrationEvent(
				Class->GetPackage()->GetFName(),
				Class->GetFName(),
				ENotifyRegistrationType::NRT_Class,
				ENotifyRegistrationPhase::NRP_Finished,
				nullptr,
				false,
				Class
			);
		}
	}

	if (!NewClassesInCoreUObject.IsEmpty())
	{
		SCOPED_BOOT_TIMING("CoreUObject Classes");
		for (UClass* Class : NewClassesInCoreUObject) // we do these first because we assume these never trigger loads
		{
			UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject Begin %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
			Class->GetDefaultObject();
			UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject End %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
		}
	}
	if (!NewClassesInEngine.IsEmpty())
	{
		SCOPED_BOOT_TIMING("Engine Classes");
		for (UClass* Class : NewClassesInEngine) // we do these second because we want to bring the engine up before the game
		{
			UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject Begin %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
			Class->GetDefaultObject();
			UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject End %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
		}
	}
	if (!NewClasses.IsEmpty())
	{
		SCOPED_BOOT_TIMING("Other Classes");
		for (UClass* Class : NewClasses)
		{
			UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject Begin %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
			Class->GetDefaultObject();
			UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject End %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
		}
	}
	FFeedbackContext& ErrorsFC = UClass::GetDefaultPropertiesFeedbackContext();
	if (ErrorsFC.GetNumErrors() || ErrorsFC.GetNumWarnings())
	{
		TArray<FString> AllErrorsAndWarnings;
		ErrorsFC.GetErrorsAndWarningsAndEmpty(AllErrorsAndWarnings);

		FString AllInOne;
		UE_LOGF(LogUObjectBootstrap, Warning, "-------------- Default Property warnings and errors:");
		for (const FString& ErrorOrWarning : AllErrorsAndWarnings)
		{
			UE_LOGF(LogUObjectBootstrap, Warning, "%ls", *ErrorOrWarning);
			AllInOne += ErrorOrWarning;
			AllInOne += TEXT("\n");
		}
		FMessageDialog::Open(EAppMsgType::Ok, FText::Format( NSLOCTEXT("Core", "DefaultPropertyWarningAndErrors", "Default Property warnings and errors:\n{0}"), FText::FromString( AllInOne ) ) );
	}
}

void FCompiledInObjectRegistry::AssembleReferenceTokenStream()
{
	IterateClasses(FIterationMarker{nullptr, nullptr}, [](UClass* Class)
	{
		// Assemble reference token stream for garbage collection/ RTGC.
		if (!Class->HasAnyFlags(RF_ClassDefaultObject) && !Class->HasAnyClassFlags(CLASS_TokenStreamAssembled))
		{
			Class->AssembleReferenceTokenStream();
		}
	});
}





void FRegisterCompiledInObjects::Register()
{
	FCompiledInObjectRegistry::Get().AddObjects(this);
}

void FRegisterIntrinsicClass::Register()
{
	FCompiledInObjectRegistry::Get().AddIntrinsicClass(this);
}

void FCompiledInObjectRegistry::AddObjects(FRegisterCompiledInObjects* Info)
{
	// Insert new objects at the head of the list - AlreadyAdded/AlreadyConstructed keeps track of what we've already processed
	Info->ListNext = GeneratedObjects.Head;
	GeneratedObjects.Head = Info;
}

void FCompiledInObjectRegistry::AddIntrinsicClass(FRegisterIntrinsicClass* Info)
{
	// Insert new objects at the head of the list - AlreadyAdded/AlreadyConstructed keeps track of what we've already processed
	Info->ListNext = IntrinsicClasses.Head;
	IntrinsicClasses.Head = Info;
}


void FField::InitializeConstInitField(FField* InOwner)
{
#if DO_GUARD_SLOW
	bool bFound = false;
	for (FField* ExistingOwner = Owner.ToFieldUnsafe(); ExistingOwner; ExistingOwner = ExistingOwner->Owner.ToFieldUnsafe())
	{
		if (ExistingOwner == InOwner)
		{
			bFound = true;
			break;
		}
	}
	checkSlow(bFound);
#endif
	Owner = FFieldVariant(Owner.ToFieldUnsafe());
	checkSlow(!Owner.IsUObject());
	checkSlow((void*)NameTempUTF8 != (void*)GetFieldClassPrivate());
	NamePrivate = NameTempUTF8;
	ClassPrivate = GetFieldClassPrivate();
	checkSlow(ClassPrivate != nullptr);
}

void FField::InitializeConstInitField(UObject* InOwner)
{
	checkSlow(Owner.GetRawPointer() == InOwner);
	Owner = FFieldVariant(Owner.ToUObjectUnsafe());
	checkSlow(Owner.IsUObject());
	checkSlow((void*)NameTempUTF8 != (void*)GetFieldClassPrivate());
	NamePrivate = NameTempUTF8;
	ClassPrivate = GetFieldClassPrivate();
	checkSlow(ClassPrivate != nullptr);
}

static void InitializeConstInitProperties(UStruct* Struct)
{
	checkSlow(Struct->IsA<UObject>());
	TArray<FField*> InnerFields;
	for (FProperty* Property = static_cast<FProperty*>(Struct->ChildProperties); Property; Property = static_cast<FProperty*>(Property->Next))
	{
		checkfSlow(Property->InternalGetOwnerAsUObjectUnsafe() == Struct, TEXT("ChildProperties linked list should only contain fields from a single struct in a hierarchy"));
		Property->InitializeConstInitProperty(Struct);
		InnerFields.Reset();
		Property->GetInnerFields(InnerFields);
		for (FField* Inner : InnerFields)
		{
			static_cast<FProperty*>(Inner)->InitializeConstInitProperty(Property);
		}
	}
}

void FProperty::InitializeConstInitProperty(UStruct* InStructOwner)
{
	InitializeConstInitField(InStructOwner);
	if (RepNotifyFuncNameUTF8)
	{
		RepNotifyFunc = FName(RepNotifyFuncNameUTF8);
		DestructorLinkNext = nullptr;
	}
#if WITH_METADATA
	InitializeMetaData();
#endif
}

void FProperty::InitializeConstInitProperty(FProperty* InPropertyOwner)
{
	InitializeConstInitField(InPropertyOwner);
	if (RepNotifyFuncNameUTF8)
	{
		RepNotifyFunc = FName(RepNotifyFuncNameUTF8);
		DestructorLinkNext = nullptr;
	}
#if WITH_METADATA
	InitializeMetaData();
#endif
}

#if WITH_METADATA
void FProperty::InitializeMetaData()
{
	TConstArrayView<UE::CodeGen::ConstInit::FMetaData> MetaDataArray = MakeArrayView(MetaDataParams, NumMetaDataParams);
	// Overwrite fields that were unioned with metadata params
	PropertyLinkNext = nullptr;
#if WITH_EDITORONLY_DATA
	IndexInOwner = INDEX_NONE;
#endif
	for (const UE::CodeGen::ConstInit::FMetaData& MetaDataData : MetaDataArray)
	{
		SetMetaData(FName(MetaDataData.NameUTF8), UTF8_TO_TCHAR(MetaDataData.ValueUTF8));
	}
}
#endif // WITH_METADATA

// Perform basic initialization of an object that was constructed at compile time
//	Initialize its name from a static string stored at construction
//	Add the object to the object array / object hash
// 	Clear flags that were set at construction/by AddObject
void UObjectBase::AddConstInitObject()
{
	FName Name = GetUninitializedName();
	AddObject(Name, EInternalObjectFlags::None);
	ObjectFlags &= ~RF_NeedInitialization;
	GUObjectArray.IndexToObject(InternalIndex)->ClearFlags(EInternalObjectFlags::PendingConstruction);
	check(!GUObjectArray.IsDisregardForGC(this) || GUObjectArray.IndexToObject(InternalIndex)->IsRootSet());
	checkSlow(ClassPrivate);
	checkSlow(!(OuterPrivate) != !(ClassPrivate->IsChildOf<UPackage>())); // logical xor
}

#if !IS_MONOLITHIC
void UObjectBase::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Linker.LinkCompiledInObjectPointer(ClassPrivate);
	Linker.LinkCompiledInObjectPointer(OuterPrivate);
}

void UField::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);
	Linker.LinkCompiledInObjectPointer(Next);
}

void UStruct::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);
	Linker.LinkCompiledInObjectPointer(SuperStruct);
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	for (int32 i=0; i <= NumStructBasesInChainMinusOne; ++i)
	{
		UE::CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain>& CompiledInPtr = CompiledInBases[i];
		Linker.LinkCompiledInObjectPointer(CompiledInPtr);
	}
#endif
	for (FField* Prop = ChildProperties; Prop; Prop = Prop->Next)
	{
		Prop->LinkCompiledInPointerFields(Linker);
	}
}

void UClass::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);
	Linker.LinkCompiledInObjectPointer(ClassWithin);
	Linker.LinkCompiledInObjectPointer(SparseClassDataStruct);
	for (UE::CodeGen::ConstInit::FClassImplementedInterface& Interface : CompiledInInterfaces)
	{
		Linker.LinkCompiledInObjectPointer(Interface.Class);
	}
}

void FField::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
}

void FByteProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(Enum);
}

void FObjectPropertyBase::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(PropertyClass);
}

void FClassProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(MetaClass);
}

void FSoftClassProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(MetaClass);
}

void FInterfaceProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(InterfaceClass);
}

void FStructProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(Struct);
}

void FDelegateProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(SignatureFunction);
}

void FMulticastDelegateProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(SignatureFunction);
}

void FEnumProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Linker.LinkCompiledInObjectPointer(Enum);
	UnderlyingProp->LinkCompiledInPointerFields(Linker);
}

void FMapProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	KeyProp->LinkCompiledInPointerFields(Linker);
	ValueProp->LinkCompiledInPointerFields(Linker);
}

void FSetProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	ElementProp->LinkCompiledInPointerFields(Linker);
}

void FArrayProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	Inner->LinkCompiledInPointerFields(Linker);
}

void FFieldPathProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	FName FieldClassName(PropertyClassName);
	PropertyClass = FFieldClass::GetNameToFieldClassMap().FindRef(FieldClassName);
}

void FOptionalProperty::LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker)
{
	Super::LinkCompiledInPointerFields(Linker);	
	ValueProperty->LinkCompiledInPointerFields(Linker);
}
#endif // !IS_MONOLITHIC

void UClass::InitFuncMap()
{
	// Called during UObject init, should be no lock contention
	{
		FUClassFuncScopeWriteLock ScopeLock(FuncMapLock);
		for (UField* Field = Children; Field; Field = Field->Next)
		{
			if (UFunction* Function = Cast<UFunction>(Field))
			{
				FuncMap.Add(Function->GetFName(), Function);
			}
		}
	}

#if DO_CHECK
	{
		FUClassFuncScopeWriteLock ScopeLock(AllFunctionsCacheLock);
		check(AllFunctionsCache.IsEmpty());
	}
#endif
}

#endif // UE_WITH_CONSTINIT_UOBJECT 