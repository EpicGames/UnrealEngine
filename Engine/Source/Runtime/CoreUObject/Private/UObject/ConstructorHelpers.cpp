// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ConstructorHelpers.h"
#include "UObject/Package.h"

namespace ConstructorHelpersInternal
{

	template<>
	UPackage* FindOrLoadObject<UPackage>(FString& PathName, uint32 LoadFlags)
	{
		// If there is a dot, remove it.
		int32 PackageDelimPos = INDEX_NONE;
		PathName.FindChar(TCHAR('.'), PackageDelimPos);
		if (PackageDelimPos != INDEX_NONE)
		{
			PathName.RemoveAt(PackageDelimPos, EAllowShrinking::No);
		}

		// Find the package in memory. 
		UPackage* PackagePtr = FindPackage(nullptr, *PathName);
		if (!PackagePtr)
		{
			// If it is not in memory, try to load it.
			PackagePtr = LoadPackage(nullptr, *PathName, LoadFlags);
		}
		if (PackagePtr)
		{
			PackagePtr->AddToRoot();
		}
		return PackagePtr;
	}

}