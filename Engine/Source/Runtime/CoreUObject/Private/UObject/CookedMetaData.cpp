// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/CookedMetaData.h"
#include "UObject/Package.h"
#include "UObject/MetaData.h"
#include "UObject/PropertyOptional.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CookedMetaData)

namespace CookedMetaDataUtil::Internal
{

void PrepareCookedMetaDataForPurge(UObject* CookedMetaDataPtr)
{
	// Skip the rename for cooked packages, as IO store cannot currently handle renames
	if (!CookedMetaDataPtr->GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		FNameBuilder BaseMetaDataName(CookedMetaDataPtr->GetFName());
		BaseMetaDataName << TEXT("_PURGED");
		CookedMetaDataPtr->Rename(FNameBuilder(MakeUniqueObjectName(CookedMetaDataPtr->GetOuter(), CookedMetaDataPtr->GetClass(), FName(BaseMetaDataName))).ToString(), nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
	}

	CookedMetaDataPtr->ClearFlags(RF_Standalone | RF_Public);
}

template <typename CookedMetaDataOuterType, typename CookedMetaDataType>
void PreloadCookedMetaData(CookedMetaDataType* CookedMetaDataPtr)
{
#if WITH_METADATA
	checkf(CookedMetaDataPtr->GetPackage()->HasAnyPackageFlags(PKG_Cooked), TEXT("Cooked meta-data should only be loaded for a cooked package!"));

	if (CookedMetaDataOuterType* Owner = CastChecked<CookedMetaDataOuterType>(CookedMetaDataPtr->GetOuter()))
	{
		Owner->ConditionalPreload();
		CookedMetaDataPtr->ApplyMetaData(Owner);
		PrepareCookedMetaDataForPurge(CookedMetaDataPtr);
	}
#endif // WITH_METADATA
}

void GetInnerFields(const FField* OuterField, TArray<FField*, TInlineAllocator<2>>& OutInnerFields)
{
	OutInnerFields.Reset();

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(OuterField))
	{
		OutInnerFields.Emplace(ArrayProperty->Inner);
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(OuterField))
	{
		OutInnerFields.Emplace(SetProperty->ElementProp);
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(OuterField))
	{
		OutInnerFields.Emplace(MapProperty->KeyProp);
		OutInnerFields.Emplace(MapProperty->ValueProp);
	}
	else if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(OuterField))
	{
		OutInnerFields.Emplace(OptionalProperty->GetValueProperty());
	}
}

} // namespace CookedMetaDataUtil::Internal

FFieldCookedMetaDataKey::FFieldCookedMetaDataKey()
{
	// Shouldn't get more than 2 levels deep on inner field paths, so reserve that up front.
	// Note: We can't use an inline allocator for that b/c this is reflected via UPROPERTY().
	FieldPath.Reserve(2);
}

bool FObjectCookedMetaDataStore::HasMetaData() const
{
	return ObjectMetaData.Num() > 0;
}

void FObjectCookedMetaDataStore::CacheMetaData(const UObject* SourceObject)
{
	ObjectMetaData.Reset();

#if WITH_METADATA
	if (UPackage* SourcePackage = SourceObject->GetPackage())
	{
		const FMetaData& SourceMetaData = SourcePackage->GetMetaData();
		if (const TMap<FName, FString>* SourceObjectMetaData = SourceMetaData.ObjectMetaDataMap.Find(SourceObject))
		{
			ObjectMetaData = *SourceObjectMetaData;
		}
	}
#endif // WITH_METADATA
}

void FObjectCookedMetaDataStore::ApplyMetaData(UObject* TargetObject) const
{
#if WITH_METADATA
	if (UPackage* TargetPackage = TargetObject->GetPackage())
	{
		FMetaData& TargetMetaData = TargetPackage->GetMetaData();
		TargetMetaData.ObjectMetaDataMap.FindOrAdd(TargetObject).Append(ObjectMetaData);
	}
#endif // WITH_METADATA
}


bool FFieldCookedMetaDataStore::HasMetaData() const
{
	return FieldMetaData.Num() > 0 || SubFieldMetaData.Num() > 0;
}

void FFieldCookedMetaDataStore::CacheMetaData(const FField* SourceField)
{
	FieldMetaData.Reset();
	SubFieldMetaData.Reset();

#if WITH_METADATA
	FFieldCookedMetaDataKey SubFieldMapKey;
	CacheFieldMetaDataInternal(SourceField, FieldMetaData, SubFieldMapKey);
#endif // WITH_METADATA
}

void FFieldCookedMetaDataStore::ApplyMetaData(FField* TargetField) const
{
#if WITH_METADATA
	FFieldCookedMetaDataKey SubFieldMapKey;
	ApplyFieldMetaDataInternal(TargetField, FieldMetaData, SubFieldMapKey);
#endif // WITH_METADATA
}

void FFieldCookedMetaDataStore::CacheFieldMetaDataInternal(const FField* SourceField, TMap<FName, FString>& TargetFieldMetaData, FFieldCookedMetaDataKey& SubFieldMapKey)
{
#if WITH_METADATA
	if (const TMap<FName, FString>* SourceFieldMetaData = SourceField->GetMetaDataMap())
	{
		TargetFieldMetaData = *SourceFieldMetaData;
	}

	TArray<FField*, TInlineAllocator<2>> InnerFields;
	CookedMetaDataUtil::Internal::GetInnerFields(SourceField, InnerFields);
	for (FField* InnerField : InnerFields)
	{
		const TMap<FName, FString>* InnerFieldMetaData = InnerField->GetMetaDataMap();
		if (!InnerFieldMetaData
			|| InnerFieldMetaData->IsEmpty())
		{
			continue;
		}

		FFieldCookedMetaDataKey InnerFieldMapKey(SubFieldMapKey);
		InnerFieldMapKey.FieldPath.Emplace(InnerField->GetFName());
		CacheFieldMetaDataInternal(InnerField, SubFieldMetaData.Emplace(InnerFieldMapKey).MetaData, InnerFieldMapKey);
	}
#endif // WITH_METADATA
}

void FFieldCookedMetaDataStore::ApplyFieldMetaDataInternal(FField* TargetField, const TMap<FName, FString>& SourceFieldMetaData, FFieldCookedMetaDataKey& SubFieldMapKey) const
{
#if WITH_METADATA
	TargetField->AppendMetaData(SourceFieldMetaData);

	TArray<FField*, TInlineAllocator<2>> InnerFields;
	CookedMetaDataUtil::Internal::GetInnerFields(TargetField, InnerFields);
	for (FField* InnerField : InnerFields)
	{
		FFieldCookedMetaDataKey InnerFieldMapKey(SubFieldMapKey);
		InnerFieldMapKey.FieldPath.Emplace(InnerField->GetFName());
		if (const FFieldCookedMetaDataValue* InnerFieldMetaDataValue = SubFieldMetaData.Find(InnerFieldMapKey))
		{
			ApplyFieldMetaDataInternal(InnerField, InnerFieldMetaDataValue->MetaData, InnerFieldMapKey);
		}
	}
#endif // WITH_METADATA
}

bool FStructCookedMetaDataStore::HasMetaData() const
{
	return ObjectMetaData.HasMetaData()
		|| PropertiesMetaData.Num() > 0;
}

void FStructCookedMetaDataStore::CacheMetaData(const UStruct* SourceStruct)
{
	ObjectMetaData.CacheMetaData(SourceStruct);

	for (TFieldIterator<const FProperty> SourcePropertyIt(SourceStruct, EFieldIterationFlags::None); SourcePropertyIt; ++SourcePropertyIt)
	{
		FFieldCookedMetaDataStore SourcePropertyMetaData;
		SourcePropertyMetaData.CacheMetaData(*SourcePropertyIt);

		if (SourcePropertyMetaData.HasMetaData())
		{
			PropertiesMetaData.Add(SourcePropertyIt->GetFName(), MoveTemp(SourcePropertyMetaData));
		}
	}
}

void FStructCookedMetaDataStore::ApplyMetaData(UStruct* TargetStruct) const
{
	ObjectMetaData.ApplyMetaData(TargetStruct);

	for (TFieldIterator<FProperty> TargetPropertyIt(TargetStruct, EFieldIterationFlags::None); TargetPropertyIt; ++TargetPropertyIt)
	{
		if (const FFieldCookedMetaDataStore* TargetPropertyMetaData = PropertiesMetaData.Find(TargetPropertyIt->GetFName()))
		{
			TargetPropertyMetaData->ApplyMetaData(*TargetPropertyIt);
		}
	}
}

void UEnumCookedMetaData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		CookedMetaDataUtil::Internal::PreloadCookedMetaData<UEnum, UEnumCookedMetaData>(this);
	}
}

bool UEnumCookedMetaData::HasMetaData() const
{
	return EnumMetaData.HasMetaData();
}

void UEnumCookedMetaData::CacheMetaData(const UEnum* SourceEnum)
{
	EnumMetaData.CacheMetaData(SourceEnum);
}

void UEnumCookedMetaData::ApplyMetaData(UEnum* TargetEnum) const
{
	EnumMetaData.ApplyMetaData(TargetEnum);
}

void UStructCookedMetaData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		CookedMetaDataUtil::Internal::PreloadCookedMetaData<UScriptStruct, UStructCookedMetaData>(this);
	}
}

bool UStructCookedMetaData::HasMetaData() const
{
	return StructMetaData.HasMetaData();
}

void UStructCookedMetaData::CacheMetaData(const UScriptStruct* SourceStruct)
{
	StructMetaData.CacheMetaData(SourceStruct);
}

void UStructCookedMetaData::ApplyMetaData(UScriptStruct* TargetStruct) const
{
	StructMetaData.ApplyMetaData(TargetStruct);
}


void UClassCookedMetaData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		CookedMetaDataUtil::Internal::PreloadCookedMetaData<UClass, UClassCookedMetaData>(this);
	}
}

bool UClassCookedMetaData::HasMetaData() const
{
	return ClassMetaData.HasMetaData()
		|| FunctionsMetaData.Num() > 0;
}

void UClassCookedMetaData::CacheMetaData(const UClass* SourceClass)
{
	ClassMetaData.CacheMetaData(SourceClass);

	for (TFieldIterator<const UFunction> SourceFunctionIt(SourceClass, EFieldIterationFlags::None); SourceFunctionIt; ++SourceFunctionIt)
	{
		FStructCookedMetaDataStore SourceFunctionMetaData;
		SourceFunctionMetaData.CacheMetaData(*SourceFunctionIt);

		if (SourceFunctionMetaData.HasMetaData())
		{
			FunctionsMetaData.Add(SourceFunctionIt->GetFName(), MoveTemp(SourceFunctionMetaData));
		}
	}
}

void UClassCookedMetaData::ApplyMetaData(UClass* TargetClass) const
{
	ClassMetaData.ApplyMetaData(TargetClass);

	for (TFieldIterator<UFunction> TargetFunctionIt(TargetClass, EFieldIterationFlags::None); TargetFunctionIt; ++TargetFunctionIt)
	{
		if (const FStructCookedMetaDataStore* TargetFunctionMetaData = FunctionsMetaData.Find(TargetFunctionIt->GetFName()))
		{
			TargetFunctionMetaData->ApplyMetaData(*TargetFunctionIt);
		}
	}
}
