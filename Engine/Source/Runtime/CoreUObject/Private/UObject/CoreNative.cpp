// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/CoreNative.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "Misc/PackageName.h"
#include "Misc/RuntimeErrors.h"
#include "UObject/Stack.h"
#include "UObject/OverridableManager.h"
#include "VerseVM/VVMVerseHangDetection.h"

#if WITH_EDITORONLY_DATA
#include "UObject/InstanceDataObjectUtils.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "StructUtils/InstancedStruct.h"
#endif	// WITH_EDITORONLY_DATA

void UClassRegisterAllCompiledInClasses();
bool IsInAsyncLoadingThreadCoreUObjectInternal();
bool IsAsyncLoadingCoreUObjectInternal();
void SuspendAsyncLoadingInternal();
void ResumeAsyncLoadingInternal();
bool IsAsyncLoadingSuspendedInternal();
bool IsAsyncLoadingMultithreadedCoreUObjectInternal();
ELoaderType GetLoaderTypeInternal();

// CoreUObject module. Handles UObject system pre-init (registers init function with Core callbacks).
class FCoreUObjectModule : public FDefaultModuleImpl
{
public:
	static void RouteRuntimeMessageToBP(ELogVerbosity::Type Verbosity, const ANSICHAR* FileName, int32 LineNumber, const FText& Message)
	{
#if UE_RAISE_RUNTIME_ERRORS && !NO_LOGGING
		check((Verbosity == ELogVerbosity::Error) || (Verbosity == ELogVerbosity::Warning));
		FMsg::Logf_Internal(FileName, LineNumber, LogScript.GetCategoryName(), Verbosity, TEXT("%s(%d): Runtime %s: \"%s\""), ANSI_TO_TCHAR(FileName), LineNumber, (Verbosity == ELogVerbosity::Error) ? TEXT("Error") : TEXT("Warning"), *Message.ToString());
#endif
		FFrame::KismetExecutionMessage(*Message.ToString(), Verbosity);
	}

#if UE_TRACK_ENSURE_OVERHEAD_DURING_SCRIPTS
	static void HandleEnsureFailed(const ANSICHAR* /*Expr*/, const ANSICHAR* /*File*/, int32 /*Line*/, const TCHAR* /*Msg*/, const TCHAR* /*CombinedErrorMsg*/)
	{
		VerseHangDetection::EndOverheadScope();
	}
#endif // UE_TRACK_ENSURE_OVERHEAD_DURING_SCRIPTS

	virtual void StartupModule() override
	{
		// Register all classes that have been loaded so far. This is required for CVars to work.		
		UClassRegisterAllCompiledInClasses();

		void InitUObject();
		FCoreDelegates::OnInit.AddStatic(InitUObject);

		// Substitute Core version of async loading functions with CoreUObject ones.
		IsInAsyncLoadingThread = &IsInAsyncLoadingThreadCoreUObjectInternal;
		IsAsyncLoading = &IsAsyncLoadingCoreUObjectInternal;
		SuspendAsyncLoading = &SuspendAsyncLoadingInternal;
		ResumeAsyncLoading = &ResumeAsyncLoadingInternal;
		IsAsyncLoadingSuspended = &IsAsyncLoadingSuspendedInternal;
		IsAsyncLoadingMultithreaded = &IsAsyncLoadingMultithreadedCoreUObjectInternal;
		GetLoaderType = &GetLoaderTypeInternal;

#if WITH_EDITORONLY_DATA
		FCoreDelegates::GetOnPostEngineInit().AddStatic(&RegisterCustomLocalizationDataGathering);
#endif

#if UE_TRACK_ENSURE_OVERHEAD_DURING_SCRIPTS
		// Register ensure handler for time tracking during script execution.
		FCoreDelegates::OnEnsureFailed.AddStatic(&FCoreUObjectModule::HandleEnsureFailed);
#endif // UE_TRACK_ENSURE_OVERHEAD_DURING_SCRIPTS

		// Register the script callstack callback to the runtime error logging
#if UE_RAISE_RUNTIME_ERRORS
		FRuntimeErrors::OnRuntimeIssueLogged.BindStatic(&FCoreUObjectModule::RouteRuntimeMessageToBP);
#endif

		// Make sure that additional content mount points can be registered after CoreUObject loads
		FPackageName::OnCoreUObjectInitialized();

#if DO_BLUEPRINT_GUARD
		FFrame::InitPrintScriptCallstack();
#endif
	}

#if WITH_EDITORONLY_DATA
	static void RegisterCustomLocalizationDataGathering()
	{
		{
			static const FAutoRegisterLocalizationDataGatheringCallback _(TBaseStructure<FInstancedStruct>::Get(),
				[](const FString& PathToParent, const UScriptStruct* Struct, const void* StructData, const void* DefaultStructData, FPropertyLocalizationDataGatherer& PropertyLocalizationDataGatherer, const EPropertyLocalizationGathererTextFlags GatherTextFlags)
			{
				const FInstancedStruct* ThisInstance = static_cast<const FInstancedStruct*>(StructData);
				const FInstancedStruct* DefaultInstance = static_cast<const FInstancedStruct*>(DefaultStructData);

				PropertyLocalizationDataGatherer.GatherLocalizationDataFromStruct(PathToParent, Struct, StructData, DefaultStructData, GatherTextFlags);

				if (const UScriptStruct* StructTypePtr = ThisInstance->GetScriptStruct())
				{
					const uint8* DefaultInstanceMemory = nullptr;
					if (DefaultInstance)
					{
						// Types must match
						if (StructTypePtr == DefaultInstance->GetScriptStruct())
						{
							DefaultInstanceMemory = DefaultInstance->GetMemory();
						}
					}

					PropertyLocalizationDataGatherer.GatherLocalizationDataFromStructWithCallbacks(PathToParent + TEXT(".StructInstance"), StructTypePtr, ThisInstance->GetMemory(), DefaultInstanceMemory, GatherTextFlags);
				}
			});
		}
	}
#endif
};
IMPLEMENT_MODULE( FCoreUObjectModule, CoreUObject );

// if we are not using compiled in natives, we still need this as a base class for intrinsics
#if !USE_COMPILED_IN_NATIVES
COREUOBJECT_API UClass* Z_Construct_UClass_UObject(ETypeConstructPhase);
extern FClassRegistrationInfo Z_Registration_Info_UClass_UObject;
UClass* Z_Construct_UClass_UObject(ETypeConstructPhase)
{
	if (!Z_Registration_Info_UClass_UObject.OuterSingleton)
	{
		Z_Registration_Info_UClass_UObject.OuterSingleton = UObject::StaticClass();
		UObjectForceRegistration(Z_Registration_Info_UClass_UObject.OuterSingleton);
		Z_Registration_Info_UClass_UObject.OuterSingleton->StaticLink();
	}
	check(Z_Registration_Info_UClass_UObject.OuterSingleton->GetClass());
	return Z_Registration_Info_UClass_UObject.OuterSingleton;
}
IMPLEMENT_CLASS(UObject, 0);
#endif

static TAutoConsoleVariable<bool> CVarEnableLinkerPatchingForReinstancedExports(
	TEXT("s.EnableLinkerPatchingForReinstancedExports"),
	true,
	TEXT("Enable/disable linker patching of existing exports that get moved aside during deferred subobject instancing on load.")
);

/*-----------------------------------------------------------------------------
	FObjectInstancingGraph.
-----------------------------------------------------------------------------*/

FObjectInstancingGraph::FObjectInstancingGraph(bool bDisableInstancing)
	: FObjectInstancingGraph(bDisableInstancing ? EObjectInstancingGraphOptions::DisableInstancing : EObjectInstancingGraphOptions::None)	
{
}

FObjectInstancingGraph::FObjectInstancingGraph(EObjectInstancingGraphOptions InOptions)
	: InstancingOptions(InOptions)
{
}

FObjectInstancingGraph::FObjectInstancingGraph( UObject* DestinationSubobjectRoot, EObjectInstancingGraphOptions InOptions)
	: InstancingOptions(InOptions)
{
	SetDestinationRoot(DestinationSubobjectRoot);
}

void FObjectInstancingGraph::SetDestinationRoot(UObject* DestinationSubobjectRoot, UObject* InSourceRoot /*= nullptr*/)
{
	DestinationRoot = DestinationSubobjectRoot;
	check(DestinationRoot);

	SourceRoot = InSourceRoot ? InSourceRoot : DestinationRoot->GetArchetype();
	check(SourceRoot);

	// add the subobject roots to the Source -> Destination mapping
	SourceToDestinationMap.Add(SourceRoot, DestinationRoot);
	DestinationToSourceMap.Add(DestinationRoot, SourceRoot);

	bCreatingArchetype = DestinationSubobjectRoot->HasAnyFlags(RF_ArchetypeObject);
	if (DestinationSubobjectRoot->GetPackage()->HasAnyPackageFlags(PKG_Cooked))
	{
		//We are never updating archetypes when loading cooked packages,
		//and we can't safely run the reconstruct logic with UObject destruction from the async loading thread.
		//Make sure to never reconstruct found existing destination subobjects in cooked packages,
		//they should always have been created from the correct up-to-date template already.
		bCreatingArchetype = false;
	}

	#if WITH_EDITORONLY_DATA
	if (UE::IsInstanceDataObject(DestinationRoot))
	{
		const UClass* DestinationRootClass = DestinationRoot->GetClass();

		for (const FProperty* Property = DestinationRootClass->RefLink; Property; Property = Property->NextRef)
		{
			if (UE::IsPropertyLoose(Property))
			{
				AddPropertyToSubobjectExclusionList(Property);
			}
		}
	}
	#endif
}

UObject* FObjectInstancingGraph::GetDestinationObject(UObject* SourceObject)
{
	check(SourceObject);
	return SourceToDestinationMap.FindRef(SourceObject);
}

UObject* FObjectInstancingGraph::GetInstancedSubobject( UObject* SourceSubobject, UObject* CurrentValue, UObject* CurrentObject, EInstancePropertyValueFlags Flags )
{
	checkSlow(SourceSubobject);

	const bool bDoNotCreateNewInstance = !!(Flags & EInstancePropertyValueFlags::DoNotCreateNewInstance);
	const bool bAllowSelfReference     = !!(Flags & EInstancePropertyValueFlags::AllowSelfReference);

	UObject* InstancedSubobject = INVALID_OBJECT;

	if ( SourceSubobject != nullptr && CurrentValue != nullptr && !CurrentValue->IsIn(CurrentObject))
	{
		const bool bAllowedSelfReference = bAllowSelfReference && SourceSubobject == SourceRoot;

		bool bShouldInstance = bAllowedSelfReference || SourceSubobject->IsIn(SourceRoot);
		if ( !bShouldInstance && CurrentValue->GetOuter() == CurrentObject->GetArchetype() )
		{
			// this code is intended to catch cases where SourceRoot contains subobjects assigned to instanced object properties, where the subobject's class
			// contains subobjects, and the class of the subobject is outside of the inheritance hierarchy of the SourceRoot, for example, a weapon
			// class which contains UIObject subobject definitions in its defaultproperties, where the property referencing the UIObjects is marked instanced.
			bShouldInstance = true;

			// if this case is triggered, ensure that the CurrentValue of the subobject property is still pointing to the template subobject.
			check(SourceSubobject == CurrentValue);
		}

		if ( bShouldInstance )
		{
			// If the CurrentValue is within the SourceRoot, lets use it to instantiate as it must have come from the merge result of the serialization
			const bool bIsInstantiatingSubObjectForOverridableSerialization = SourceSubobject && FOverridableManager::Get().NeedSubObjectTemplateInstantiation(SourceSubobject);
			if (bIsInstantiatingSubObjectForOverridableSerialization && SourceSubobject != CurrentValue && CurrentValue->IsIn(SourceRoot))
			{
				SourceSubobject = CurrentValue;
			}

			// search for the unique subobject instance that corresponds to this subobject template
			InstancedSubobject = GetDestinationObject(SourceSubobject);
			if ( InstancedSubobject == nullptr )
			{
				if (bDoNotCreateNewInstance)
				{
					InstancedSubobject = INVALID_OBJECT; // leave it unchanged
				}
				else
				{
					// if the Outer for the subobject currently assigned to this property is the same as the object that we're instancing subobjects for,
					// the subobject does not need to be instanced; otherwise, there are two possiblities:
					// 1. CurrentValue is a template and needs to be instanced
					// 2. CurrentValue is an instanced subobject, in which case it should already be in InstanceGraph, UNLESS the subobject was created
					//		at runtime (editinline export properties, for example).  If that is the case, CurrentValue will be an instance that is not linked
					//		to the subobject template referenced by CurrentObject's archetype, and in this case, we also don't want to re-instance the subobject template

					const bool bIsRuntimeInstance = CurrentValue != SourceSubobject && CurrentValue->GetOuter() == CurrentObject;
					if (bIsRuntimeInstance )
					{
						InstancedSubobject = CurrentValue; 
					}
					else
					{
						// If the subobject template is relevant in this context(client vs server vs editor), instance it.
						const bool bShouldLoadForClient = SourceSubobject->NeedsLoadForClient();
						const bool bShouldLoadForServer = SourceSubobject->NeedsLoadForServer();
						const bool bShouldLoadForEditor = ( GIsEditor && ( bShouldLoadForClient || !CurrentObject->RootPackageHasAnyFlags(PKG_PlayInEditor) ) );
						const bool bShouldLoadForStandalone = (!GIsEditor && !GIsClient && !GIsServer);

						if ( (GIsClient && bShouldLoadForClient) || (GIsServer && bShouldLoadForServer) || bShouldLoadForEditor || bShouldLoadForStandalone )
						{
							// this is the first time the instance corresponding to SourceSubobject has been requested

							// get the object instance corresponding to the source subobject's Outer - this is the object that
							// will be used as the Outer for the destination subobject
							UObject* SubobjectOuter = GetDestinationObject(SourceSubobject->GetOuter());

							// In the event we're templated off a deep nested UObject hierarchy, with several links to objects nested in the object
							// graph, it's entirely possible that we'll encounter UObjects that we haven't yet discovered and instanced a copy of their
							// outer.  In that case - we need to go ahead and instance that outer.
							if ( SubobjectOuter == nullptr )
							{
								SubobjectOuter = GetInstancedSubobject(SourceSubobject->GetOuter(), SourceSubobject->GetOuter(), CurrentObject, Flags);

								checkf(SubobjectOuter && SubobjectOuter != INVALID_OBJECT, TEXT("No corresponding destination object found for '%s' while attempting to instance subobject '%s'"), *SourceSubobject->GetOuter()->GetFullName(), *SourceSubobject->GetFullName());

								// Constructing the outer may have also instanced this subobject - avoid reconstructing it again with the same archetype.
								InstancedSubobject = GetDestinationObject(SourceSubobject);
								if (InstancedSubobject != nullptr)
								{
									return InstancedSubobject;
								}
							}

							FName SubobjectName = SourceSubobject->GetFName();

							// If a property serialized a reference to an instanced subobject and it is not type-compatible with the default value that the
							// serializing (owner) object should be referencing at this point on load, the serialized object needs to be verified against
							// the value the instancing graph will use during subobject instancing, which gets deferred until PostLoadSubobjects() on load.
							UObject* ExistingObject = nullptr;
							if (IsLoadingObject()
								&& CurrentValue == SourceSubobject
								&& SubobjectOuter->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint)
								&& (FOverridableManager::Get().IsEnabled(SubobjectOuter)
									|| SubobjectOuter->GetClass()->ShouldUseDynamicSubobjectInstancing()))
							{
								// Look for an existing instance with the same name at the outer scope. If we find one, we can infer that it's a subobject
								// that has not been explicitly overridden, but that was serialized on save as a default-instanced subobject. In that case,
								// we need to check the object's type against the type that we're about to instance below. If there's a mismatch, rename
								// the existing object out of the way, as subobject recycling will assert in that case. Any serialized data will be lost.
								ExistingObject = StaticFindObjectFast(nullptr, SubobjectOuter, SubobjectName);
								if (ExistingObject)
								{
									if (!ExistingObject->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad)
										&& SourceSubobject->IsA(ExistingObject->GetClass()))
									{
										if (SourceSubobject->GetClass() == ExistingObject->GetClass())
										{
											InstancedSubobject = ExistingObject;
										}
										else
										{
											// Use the serialized instance data to construct a new object of the correct source type.
											SourceSubobject = ExistingObject;
										}
									}
									else
									{
										// Keep the existing base name and scope in case some other property was overridden and serialized this reference.
										// If nothing else winds up referencing this object after load, the export will be garbage collected (which would
										// happen whether or not we renamed the object below - that is being done in order to free up the name for instancing).
										// Also, don't reset the linker as this object may still be pending load/PostLoad; this is handled by linker patching below.
										FName NewName = MakeUniqueObjectName(SubobjectOuter, ExistingObject->GetClass(), SubobjectName);
										ExistingObject->Rename(*NewName.ToString(), nullptr, REN_DoNotDirty | REN_NonTransactional | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
									}
								}
							}

							// Don't search for the existing subobjects on Blueprint-generated classes. What we'll find is a subobject
							// created by the constructor which may not have all of its fields initialized to the correct value (which
							// should be coming from a blueprint).
							if (!SubobjectOuter->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
							{
								InstancedSubobject = StaticFindObjectFast(nullptr, SubobjectOuter, SubobjectName);
							}

							if (InstancedSubobject && IsCreatingArchetype() && !InstancedSubobject->HasAnyFlags(RF_LoadCompleted))
							{
								// since we are updating an archetype, this needs to reconstruct as that is the mechanism used to copy properties
								// it will destroy the existing object and overwrite it
								InstancedSubobject = nullptr;
							}

							if (!InstancedSubobject)
							{
								// finally, create the subobject instance
								FStaticConstructObjectParameters Params(SourceSubobject->GetClass());
								Params.Outer = SubobjectOuter;
								Params.Name = SubobjectName;

								// Copy transient flags from template to avoid saving subobjects that shouldn't be saved
								EObjectFlags SourceFlagsToCopy = SourceSubobject->GetFlags() & (EObjectFlags)(RF_Transient | RF_TextExportTransient);
								Params.SetFlags = SubobjectOuter->GetMaskedFlags(RF_PropagateToSubObjects) | SourceFlagsToCopy;
								Params.Template = SourceSubobject;
								Params.bCopyTransientsFromClassDefaults = true;
								Params.InstanceGraph = this;
								InstancedSubobject = StaticConstructObject_Internal(Params);
							}

							// Patch the subobject into the export table when replacing an existing object while loading
							if (IsLoadingObject() && ExistingObject
								&& ExistingObject != InstancedSubobject
								&& CVarEnableLinkerPatchingForReinstancedExports.GetValueOnAnyThread())
							{
								if (FLinkerLoad* Linker = ExistingObject->GetLinker())
								{
									// This should only be happening on the game thread or for outer objects that are still in the construction loading phase.
									check(IsInGameThread() || (IsInAsyncLoadingThread() && SubobjectOuter->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1)));

									// Acquire the linker's mutex first in case this is an async load
									UE::TUniqueLock LinkerLock(Linker->Mutex);
									Linker->PRIVATE_PatchNewObjectIntoExport(ExistingObject->GetLinkerIndex(), InstancedSubobject);
								}
							}
						}
					}
				}
			}
			else if ( IsLoadingObject() && InstancedSubobject->GetClass()->HasAnyClassFlags(CLASS_HasInstancedReference) )
			{
				/* When loading an object from disk, in some cases we have a subobject which has a reference to another subobject in DestinationObject which
					wasn't serialized and hasn't yet been instanced.  For example, the PointLight class declared two component templates:

						Begin DrawLightRadiusComponent0
						End
						Components.Add(DrawLightRadiusComponent0)

						Begin MyPointLightComponent
							SomeProperty=DrawLightRadiusComponent
						End
						LightComponent=MyPointLightComponent

					The components array will be processed by UClass::InstanceSubobjectTemplates after the LightComponent property is processed.  If the instance
					of DrawLightRadiusComponent0 that was created during the last session (i.e. when this object was saved) was identical to the component template
					from the PointLight class's defaultproperties, and the instance of MyPointLightComponent was serialized, then the MyPointLightComponent instance will
					exist in the InstanceGraph, but the instance of DrawLightRadiusComponent0 will not.  To handle this case and make sure that the SomeProperty variable of
					the MyPointLightComponent instance is correctly set to the value of the DrawLightRadiusComponent0 instance that will be created as a result of calling
					InstanceSubobjectTemplates on the PointLight actor from ConditionalPostLoad, we must call ConditionalPostLoad on each existing component instance that we
					encounter, while we still have access to all of the component instances owned by the PointLight.
				*/
				InstancedSubobject->ConditionalPostLoadSubobjects(this);
			}
		}
	}

	return InstancedSubobject;
}


UObject* FObjectInstancingGraph::InstancePropertyValue(UObject* SubObjectTemplate, TNotNull<UObject*> CurrentValue, TNotNull<UObject*> Owner, EInstancePropertyValueFlags Flags)
{
	bool bCausesInstancing   = !!(Flags & EInstancePropertyValueFlags::CausesInstancing);
	bool bAllowSelfReference = !!(Flags & EInstancePropertyValueFlags::AllowSelfReference);

	UObject* CurrentSourceRoot = SourceRoot;

	if (CurrentValue->GetClass()->HasAnyClassFlags(CLASS_DefaultToInstanced))
	{
		bCausesInstancing = true; // these are always instanced no matter what
	}
	else if (!bCausesInstancing && !bAllowSelfReference && Owner->GetClass()->ShouldUseDynamicSubobjectInstancing())
	{
		// Dynamic instancing means that we'll analyze the current value to determine how instancing should proceed. At
		// construction time for example, the current value will have been initialized to the value from the default data
		// object, so in that case we'll compare that value against the owner's archetype to see if it should be instanced.
		// As an optimization, we only need to do this even if the Owner is the same as destination root during reinstantiation.
		if (Owner != DestinationRoot || GIsReinstancing)
		{
			// In this case, we're initializing an owner as a subgraph within the context of its outer destination root. If
			// the owner archetype does not exist within the outer object's source hierarchy, we need to adjust the source
			// root that we're using for analysis to match the archetype of the owner of the subgraph that we're instancing.
			if (UObject** SourceSubobjectPtr = DestinationToSourceMap.Find(Owner))
			{
				// We only need to adjust the source root if it falls outside of the current source object hierarchy.
				// This allows for subobjects to reference their outer chains back to their instance hierarchy's root.
				UObject* SourceSubobject = *SourceSubobjectPtr;
				if (!SourceSubobject->IsIn(SourceRoot))
				{
					while (SourceToDestinationMap.Contains(SourceSubobject))
					{
						CurrentSourceRoot = SourceSubobject;
						SourceSubobject = SourceSubobject->GetOuter();
					}
				}
			}
		}

		if (CurrentValue == CurrentSourceRoot)
		{
			// In this case, the current value was initialized to reference the source graph's root. The instancing
			// graph will already contain the mapping from the archetype source to the root instance, so we'll allow
			// instancing to proceed, where it will resolve the archetype and return the current root as the new value.
			bAllowSelfReference = true;

			// Must also set the flag here b/c it's required later on (unlike the instancing flag)
			Flags |= EInstancePropertyValueFlags::AllowSelfReference;
		}
		else if (CurrentValue->IsIn(CurrentSourceRoot))
		{
			// In this case, the current value was initialized from the default data to reference an archetype that's
			// exists within the source object graph. The instancing graph may not yet contain a mapping for this
			// archetype, so we allow instancing to proceed, where it will either construct a new instance under the
			// current owner, or return a reference to the instance that was already created for the source archetype.
			bCausesInstancing = true;
		}
		else if (CurrentValue->HasAllFlags(RF_ArchetypeObject)
				 && SubObjectTemplate && CurrentValue->GetOuter() == SubObjectTemplate->GetOuter()->GetClass()->GetDefaultObject())
		{
			// In this case, the source may have been instanced from class default data (e.g. a transient reference).
			// It is similar to the case above in that if the instancing graph does not yet contain a mapping for this
			// archetype, we'll allow instancing to proceed and construct a new instance under the current owner. But
			// in this scenario, the current value is referencing an archetype that's based outside of its source graph.
			bCausesInstancing = true;
		}
	}

	if (!IsSubobjectInstancingEnabled() || // if instancing is off
		(!bCausesInstancing && // or if this class isn't forced to be instanced and this var did not have the instance keyword
		!bAllowSelfReference)) // and this isn't a delegate
	{
		return CurrentValue; // not instancing
	}

	if (!!(InstancingOptions & EObjectInstancingGraphOptions::InstanceTemplatesOnly) && // if we're allowed to instance templates only
		!CurrentValue->IsTemplate()) // and the current value is not a template
	{
		return CurrentValue; // not instancing
	}

	UObject* NewValue = CurrentValue;

	// if the object we're instancing the subobjects for (Owner) has the current subobject's outer in its archetype chain, and its archetype has a nullptr value
	// for this subobject property it means that the archetype didn't instance its subobject, so we shouldn't either.

	if (SubObjectTemplate == nullptr && Owner->IsBasedOnArchetype(CurrentValue->GetOuter()))
	{
		NewValue = nullptr;
	}
	else
	{
		TGuardValue<UObject*> ScopedSourceRoot(SourceRoot, CurrentSourceRoot);

		if (SubObjectTemplate == nullptr )
		{
			// should only be here if our archetype doesn't contain this subobject property
			SubObjectTemplate = CurrentValue;
		}

		UObject* MaybeNewValue = GetInstancedSubobject(SubObjectTemplate, CurrentValue, Owner, Flags);
		if ( MaybeNewValue != INVALID_OBJECT )
		{
			NewValue = MaybeNewValue;
		}
	}
	return NewValue;
}

void FObjectInstancingGraph::AddNewObject(UObject* ObjectInstance, UObject* InArchetype /*= nullptr*/)
{
	if (HasDestinationRoot())
	{
		AddNewInstance(ObjectInstance, InArchetype);
	}
	else
	{
		SetDestinationRoot(ObjectInstance, InArchetype);
	}
}

void FObjectInstancingGraph::AddNewInstance(UObject* ObjectInstance, UObject* InArchetype /*= nullptr*/)
{
	checkf(HasDestinationRoot(), TEXT("Expecting the destination root to be set before calling this method"));

	if (ObjectInstance != nullptr)
	{
		UObject* SourceObject = InArchetype ? InArchetype : ObjectInstance->GetArchetype();
		check(SourceObject);

		SourceToDestinationMap.Add(SourceObject, ObjectInstance);
		DestinationToSourceMap.Add(ObjectInstance, SourceObject);
	}
}

void FObjectInstancingGraph::PrepopulateWithInstance(TNotNull<UObject*> ObjectInstance, TNotNull<UObject*> InArchetype)
{
	checkf(!HasDestinationRoot(), TEXT("Expecting destination root to NOT be set before calling method"));

	SourceToDestinationMap.Add(InArchetype, ObjectInstance);
	DestinationToSourceMap.Add(ObjectInstance, InArchetype);
}

void FObjectInstancingGraph::RemoveInstanceFromPrepopulate(TNotNull<UObject*> ObjectInstance)
{
	checkf(!HasDestinationRoot(), TEXT("Expecting destination root to NOT be set before calling method"));

	UObject** FoundArchetype = DestinationToSourceMap.Find(ObjectInstance);

	if (FoundArchetype)
	{
		check(*FoundArchetype);
		SourceToDestinationMap.Remove(*FoundArchetype);
		DestinationToSourceMap.Remove(ObjectInstance);
	}
}

void FObjectInstancingGraph::RetrieveObjectInstances( UObject* SearchOuter, TArray<UObject*>& out_Objects )
{
	if ( HasDestinationRoot() && SearchOuter != nullptr && (SearchOuter == DestinationRoot || SearchOuter->IsIn(DestinationRoot)) )
	{
		for ( TMap<UObject*,UObject*>::TIterator It(SourceToDestinationMap); It; ++It )
		{
			UObject* InstancedObject = It.Value();
			if ( InstancedObject->GetOuter() == SearchOuter )
			{
				out_Objects.AddUnique(InstancedObject);
			}
		}
	}
}
