// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnCoreNet.cpp: Core networking support.
=============================================================================*/

#include "UObject/CoreNet.h"
#include "UObject/PropertyOptional.h"
#include "UObject/UnrealType.h"
#include "UObject/RemoteObjectTransfer.h"
#include "Misc/EngineNetworkCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CoreNet)

DEFINE_LOG_CATEGORY_STATIC(LogCoreNet, Log, All);

/*-----------------------------------------------------------------------------
	FClassNetCache implementation.
-----------------------------------------------------------------------------*/

FClassNetCache::FClassNetCache()
{
}

FClassNetCache::FClassNetCache( const UClass* InClass ) : Class( InClass )
{
}

void FClassNetCacheMgr::SortProperties( TArray< FProperty* >& Properties ) const
{
	// Sort NetProperties so that their ClassReps are sorted by memory offset
	struct FCompareUFieldOffsets
	{
		FORCEINLINE bool operator()(FProperty* A, FProperty* B) const
		{
			// Ensure stable sort
			if (A->GetOffset_ForGC() == B->GetOffset_ForGC())
			{
				return A->GetName() < B->GetName();
			}

			return A->GetOffset_ForGC() < B->GetOffset_ForGC();
		}
	};

	Algo::Sort(Properties, FCompareUFieldOffsets());
}

uint32 FClassNetCacheMgr::SortedStructFieldsChecksum( const UStruct* Struct, uint32 Checksum ) const
{
	// Generate a list that we can sort, to make sure we process these deterministically
	TArray< FProperty * > Fields;

	for ( TFieldIterator< FProperty > It( Struct ); It; ++It )
	{
		if ( It->PropertyFlags & CPF_RepSkip )
		{
			continue;
		}

		Fields.Add( *It );
	}

	// Sort them
	SortProperties( Fields );

	// Evolve the checksum on the sorted list
	for ( auto Field : Fields )
	{
		Checksum = GetPropertyChecksum( Field, Checksum, true );
	}

	return Checksum;
}

uint32 FClassNetCacheMgr::GetPropertyChecksum( const FProperty* Property, uint32 Checksum, const bool bIncludeChildren ) const
{
	if ( bDebugChecksum )
	{
		UE_LOGF( LogCoreNet, Warning, "%ls%ls [%ls] [%u] [%u]", FCString::Spc( 2 * DebugChecksumIndent ), *Property->GetName().ToLower(), *Property->GetClass()->GetName().ToLower(), Property->ArrayDim, Checksum );
	}

	Checksum = FCrc::StrCrc32( *Property->GetName().ToLower(), Checksum );							// Evolve checksum on name
	Checksum = FCrc::StrCrc32( *Property->GetCPPType( nullptr, 0 ).ToLower(), Checksum );			// Evolve by property type
	Checksum = FCrc::StrCrc32( *FString::Printf( TEXT( "%u" ), Property->ArrayDim ), Checksum );	// Evolve checksum on array dim (to detect when static arrays change size)

	if ( bIncludeChildren )
	{
		const FArrayProperty* ArrayProperty = CastField< FArrayProperty >( Property );

		// Evolve checksum on array inner
		if ( ArrayProperty != NULL )
		{
			return GetPropertyChecksum( ArrayProperty->Inner, Checksum, bIncludeChildren );
		}

		const FStructProperty* StructProperty = CastField< FStructProperty >( Property );

		// Evolve checksum on property struct fields
		if ( StructProperty != NULL )
		{
			if ( bDebugChecksum )
			{
				UE_LOGF( LogCoreNet, Warning, "%ls [%ls] [%u]", FCString::Spc( 2 * DebugChecksumIndent ), *StructProperty->Struct->GetName().ToLower(), Checksum );
			}

			// Evolve checksum on struct name
			Checksum = FCrc::StrCrc32( *StructProperty->Struct->GetName().ToLower(), Checksum );

			const_cast< FClassNetCacheMgr* >( this )->DebugChecksumIndent++;

			Checksum = SortedStructFieldsChecksum( StructProperty->Struct, Checksum );

			const_cast< FClassNetCacheMgr* >( this )->DebugChecksumIndent--;
		}

		// Evolve checksum on optional inner
		if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
		{
			return GetPropertyChecksum(OptionalProperty->GetValueProperty(), Checksum, bIncludeChildren);
		}
	}

	return Checksum;
}

uint32 FClassNetCacheMgr::GetFunctionChecksum( const UFunction* Function, uint32 Checksum ) const
{
	// Evolve checksum on function name
	Checksum = FCrc::StrCrc32( *Function->GetName().ToLower(), Checksum );

	// Evolve the checksum on function flags
	Checksum = FCrc::StrCrc32( *FString::Printf( TEXT( "%u" ), (uint32)Function->FunctionFlags ), Checksum );

#if 0	// This is disabled now that we have backwards compatibility for RPC parameters working in replays 
	TArray< FProperty * > Parms;

	for ( TFieldIterator< FProperty > It( Function ); It && ( It->PropertyFlags & ( CPF_Parm | CPF_ReturnParm ) ) == CPF_Parm; ++It )
	{
		Parms.Add( *It );
	}

	// Sort parameters by offset/name
	SortProperties( Parms );

	// Evolve checksum on sorted function parameters
	for ( FProperty* Parm : Parms )
	{
		Checksum = GetPropertyChecksum( Parm, Checksum, true );
	}
#endif

	return Checksum;
}

uint32 FClassNetCacheMgr::GetFieldChecksum( const UField* Field, uint32 Checksum ) const
{
	if ( Cast< UFunction >( Field ) != NULL )
	{
		return GetFunctionChecksum( (UFunction*)Field, Checksum );
	}

	UE_LOGF( LogCoreNet, Warning, "GetFieldChecksum: Unknown field: %ls", *Field->GetName() );

	return Checksum;
}

const FClassNetCache* FClassNetCacheMgr::GetClassNetCache( UClass* Class )
{
	FClassNetCache* Result = ClassFieldIndices.FindRef( Class );

	if ( !Result )
	{
		Class->SetUpRuntimeReplicationData();

		Result					= ClassFieldIndices.Add( Class, new FClassNetCache( Class ) );
		Result->Super			= NULL;
		Result->FieldsBase		= 0;
		Result->ClassChecksum	= 0;

		if ( Class->GetSuperClass() )
		{
			Result->Super			= GetClassNetCache( Class->GetSuperClass() );
			Result->FieldsBase		= Result->Super->GetMaxIndex();
			Result->ClassChecksum	= Result->Super->ClassChecksum;
		}

		Result->Fields.Empty(Class->NetFields.Num());

		TArray<FProperty*> Properties;
		Properties.Empty(Class->ClassReps.Num() - Class->FirstOwnedClassRep);

		for (int32 i = Class->FirstOwnedClassRep; i < Class->ClassReps.Num(); i++)
		{
			// Add each net field to cache, and assign index/checksum
			FProperty* Property = Class->ClassReps[i].Property;
			check(Property);
			Properties.Add(Property);

			// Get individual checksum
			const uint32 Checksum = GetPropertyChecksum(Property, 0, false);

			// Get index
			const int32 ThisIndex = Result->GetMaxIndex();

			// Add to cached fields on this class
			Result->Fields.Add(FFieldNetCache(Property, ThisIndex, Checksum));
			
			// Skip over static array properties.
			i += (Property->ArrayDim - 1);
		}

		for( int32 i = 0; i < Class->NetFields.Num(); i++ )
		{
			// Add each net field to cache, and assign index/checksum
			UField* Field = Class->NetFields[i];

			// Get individual checksum
			const uint32 Checksum = GetFieldChecksum( Field, 0 );

			// Get index
			const int32 ThisIndex = Result->GetMaxIndex();

			// Add to cached fields on this class
			Result->Fields.Add( FFieldNetCache( Field, ThisIndex, Checksum ) );
		}

		Result->Fields.Shrink();

		// Add fields to the appropriate hash maps
		for ( TArray< FFieldNetCache >::TIterator It( Result->Fields ); It; ++It )
		{
			Result->FieldMap.Add( It->Field.GetRawPointer(), &*It );

			if ( Result->FieldChecksumMap.Contains( It->FieldChecksum ) )
			{
				UE_LOGF( LogCoreNet, Error, "Duplicate checksum: %ls, %u", *It->Field.GetFullName(), It->FieldChecksum );
			}

			Result->FieldChecksumMap.Add( It->FieldChecksum, &*It );
		}

		const bool bIsNativeClass = Class->HasAnyClassFlags(CLASS_Native);
		if (!bIsNativeClass)
		{
			// Initialize class checksum (just use properties for this)
			SortProperties(Properties);
		}

		for ( auto Property : Properties )
		{
			Result->ClassChecksum = GetPropertyChecksum( Property, Result->ClassChecksum, true );
		}
	}
	return Result;
}

void FClassNetCacheMgr::ClearClassNetCache()
{
	for ( auto It = ClassFieldIndices.CreateIterator(); It; ++It)
	{
		delete It.Value();
	}

	ClassFieldIndices.Empty();
}

void FClassNetCacheMgr::CountBytes(FArchive& Ar) const
{
	ClassFieldIndices.CountBytes(Ar);
	for (const auto& ClassFieldPair : ClassFieldIndices)
	{
		if (ClassFieldPair.Value != nullptr)
		{
			Ar.CountBytes(sizeof(FClassNetCache), sizeof(FClassNetCache));
			ClassFieldPair.Value->CountBytes(Ar);
		}
	}
}

void FClassNetCache::CountBytes(FArchive& Ar) const
{
	Fields.CountBytes(Ar);
	FieldMap.CountBytes(Ar);
	FieldChecksumMap.CountBytes(Ar);
}

/*-----------------------------------------------------------------------------
	UPackageMap implementation.
-----------------------------------------------------------------------------*/

UPackageMap::UPackageMap() = default;
UPackageMap::UPackageMap(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

UPackageMap::~UPackageMap() = default;


bool UPackageMap::SerializeName(FArchive& Ar, FName& InName)
{
	return StaticSerializeName(Ar, InName);
}

bool UPackageMap::StaticSerializeName(FArchive& Ar, FName& InName)
{
	if (Ar.IsLoading())
	{
		uint8 bHardcoded = 0;
		Ar.SerializeBits(&bHardcoded, 1);
		if (bHardcoded)
		{
			// replicated by hardcoded index
			uint32 NameIndex;
			if (Ar.EngineNetVer() < FEngineNetworkCustomVersion::ChannelNames)
			{
				Ar.SerializeInt(NameIndex, MAX_NETWORKED_HARDCODED_NAME + 1);
			}
			else
			{
				Ar.SerializeIntPacked(NameIndex);
			}

			if (NameIndex < (uint32)EName::MaxHardcodedNameIndex)
			{
				InName = EName(NameIndex);
				// hardcoded names never have a Number
			}
			else
			{
				Ar.SetError();
			}
		}
		else
		{
			// replicated by string
			FString InString;
			int32 InNumber;
			Ar << InString << InNumber;
			InName = FName(*InString, InNumber);
		}
	}
	else if (Ar.IsSaving())
	{
		const EName* InEName = InName.ToEName();
		uint8 bHardcoded = InEName && ShouldReplicateAsInteger(*InEName, InName);
		Ar.SerializeBits(&bHardcoded, 1);
		if (bHardcoded && /* silence static analyzer */ InEName)
		{
			// send by hardcoded index
			checkSlow(InName.GetNumber() <= 0); // hardcoded names should never have a Number
			uint32 NameIndex = (uint32)*InEName;
			Ar.SerializeIntPacked(NameIndex);
		}
		else
		{
			// send by string
			FString OutString = InName.GetPlainNameString();
			int32 OutNumber = InName.GetNumber();
			Ar << OutString << OutNumber;
		}
	}
	return true;
}

void UPackageMap::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{
		TrackedUnmappedNetGuids.CountBytes(Ar);
		TrackedMappedDynamicNetGuids.CountBytes(Ar);
		Ar << DebugContextString;
	}
}

// ----------------------------------------------------------------

void SerializeChecksum(FArchive &Ar, uint32 x, bool ErrorOK)
{
	if (Ar.IsLoading() )
	{
		uint32 Magic = 0;
		Ar << Magic;
		if((!ErrorOK || !Ar.IsError()) && !ensure(Magic==x))
		{
			UE_LOGF(LogCoreNet, Warning, "%d == %d", Magic, x );
		}
		
	}
	else
	{
		uint32 Magic = x;
		Ar << Magic;
	}
}

void FPropertyRetirement::CountBytes(FArchive& Ar) const
{
	for (const FPropertyRetirement* NextRetirement = Next; NextRetirement; NextRetirement = NextRetirement->Next)
	{
		Ar.CountBytes(sizeof(FPropertyRetirement), sizeof(FPropertyRetirement));
	}
}

// ----------------------------------------------------------------
//	FNetBitWriter
// ----------------------------------------------------------------
FNetBitWriter::FNetBitWriter()
	: FBitWriter(0)
	, PackageMap(nullptr)
{}

FNetBitWriter::FNetBitWriter(int64 InMaxBits)
	: FBitWriter(InMaxBits, true)
	, PackageMap(nullptr)
{

}

FNetBitWriter::FNetBitWriter(UPackageMap* InPackageMap, int64 InMaxBits)
	: FBitWriter(InMaxBits, true)
	, PackageMap(InPackageMap)
{

}

FArchive& FNetBitWriter::operator<<(class FName& N)
{
	if (PackageMap)
	{
		PackageMap->SerializeName(*this, N);
	}
	else
	{
		UPackageMap::StaticSerializeName(*this, N);
	}

	return *this;
}

FArchive& FNetBitWriter::operator<<(UObject*& Object)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_CLOGF(PackageMap->IsUsingRemoteObjectReferences(), LogCoreNet, Warning, "Sending raw object reference when NetDriver expects remote references (%ls)", *GetFullNameSafe(Object));
#endif

	PackageMap->SerializeObject(*this, UObject::StaticClass(), Object);
	return *this;
}

FArchive& FNetBitWriter::operator<<(FSoftObjectPath& Value)
{
	// It's more efficient to serialize as a string then name+string
	FString Path = Value.ToString();
	*this << Path;

	return *this;
}

FArchive& FNetBitWriter::operator<<(FSoftObjectPtr& Value)
{
	// Keep in sync with FNetBitReader::operator<<(FSoftObjectPtr&)
	UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	if (EngineNetVer() >= FEngineNetworkCustomVersion::SoftObjectPtrNetGuids)
	{
		// Treat soft pointers to dynamic (non-stably named) objects as weak pointers.
		// This allows these objects to be resolved on clients and to only replicate a NetGUID.
		//
		// In theory stably named objects could be replicated as NetGUIDs also via the package map,
		// only exporting and replicating the path string once, but this would require a refactor
		// of the package map to handle "soft" references to packages and not automatically load them
		// when they're referenced.
		UObject* Object = Value.Get();
		const bool bUsePath = !Object || Object->IsFullNameStableForNetworking();

		WriteBit(bUsePath);

		if (bUsePath)
		{
			FArchiveUObject::SerializeSoftObjectPtr(*this, Value);
		}
		else
		{
			PackageMap->SerializeObject(*this, UObject::StaticClass(), Object, nullptr);
		}
	}
	else
	{
		FArchiveUObject::SerializeSoftObjectPtr(*this, Value);
	}

	return *this;
}

FArchive& FNetBitWriter::operator<<(FObjectPtr& Value)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (PackageMap->IsUsingRemoteObjectReferences())
	{
		FRemoteObjectReference RemoteReference(Value);
		bool bResult = false;
		RemoteReference.NetSerialize(*this, PackageMap, bResult);
		return *this;
	}
#endif
	return FArchiveUObject::SerializeObjectPtr(*this, Value);
}

FArchive& FNetBitWriter::operator<<(struct FWeakObjectPtr& WeakObjectPtr)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (PackageMap->IsUsingRemoteObjectReferences())
	{
		FRemoteObjectReference RemoteReference(WeakObjectPtr);
		bool bResult = false;
		RemoteReference.NetSerialize(*this, PackageMap, bResult);
		return *this;
	}
#endif
	return FArchiveUObject::SerializeWeakObjectPtr(*this, WeakObjectPtr);
}

void FNetBitWriter::CountMemory(FArchive& Ar) const
{
	FBitWriter::CountMemory(Ar);
	const SIZE_T MemberSize = sizeof(*this) - sizeof(FBitWriter);
	Ar.CountBytes(MemberSize, MemberSize);
}

// ----------------------------------------------------------------
//	FNetBitReader
// ----------------------------------------------------------------
FNetBitReader::FNetBitReader(UPackageMap* InPackageMap, const uint8* Src, int64 CountBits)
	: FBitReader(Src, CountBits)
	, PackageMap( InPackageMap )
{
}

FNetBitReader::FNetBitReader(const FNetBitReader&) = default;
FNetBitReader& FNetBitReader::operator=(const FNetBitReader&) = default;
FNetBitReader::FNetBitReader(FNetBitReader&&) = default;
FNetBitReader& FNetBitReader::operator=(FNetBitReader&&) = default;
FNetBitReader::~FNetBitReader() = default;

FArchive& FNetBitReader::operator<<(UObject*& Object)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_CLOGF(PackageMap->IsUsingRemoteObjectReferences(), LogCoreNet, Warning, "Received raw object reference when NetDriver expects remote references (%ls)", *GetFullNameSafe(Object));
#endif

	PackageMap->SerializeObject(*this, UObject::StaticClass(), Object);
	return *this;
}

FArchive& FNetBitReader::operator<<(class FName& N)
{
	if (PackageMap)
	{
		PackageMap->SerializeName(*this, N);
	}
	else
	{
		UPackageMap::StaticSerializeName(*this, N);
	}

	return *this;
}

FArchive& FNetBitReader::operator<<(FSoftObjectPath& Value)
{
	FString Path;
	*this << Path;

	Value.SetPath(MoveTemp(Path));

	return *this;
}

FArchive& FNetBitReader::operator<<(FSoftObjectPtr& Value)
{
	// Keep in sync with FNetBitWriter::operator<<(FSoftObjectPtr&)
	UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	if (EngineNetVer() >= FEngineNetworkCustomVersion::SoftObjectPtrNetGuids)
	{
		// Treat soft pointers to dynamic (non-stably named) objects as weak pointers.
		// This allows these objects to be resolved on clients and to only replicate a NetGUID.
		//
		// In theory stably named objects could be replicated as NetGUIDs also via the package map,
		// only exporting and replicating the path string once, but this would require a refactor
		// of the package map to handle "soft" references to packages and not automatically load them
		// when they're referenced.
		const bool bUsePath = ReadBit() != 0;

		UObject* Object = nullptr;

		if (bUsePath)
		{
			FArchiveUObject::SerializeSoftObjectPtr(*this, Value);
		}
		else
		{
			PackageMap->SerializeObject(*this, UObject::StaticClass(), Object, nullptr);
		}

		if (!bUsePath)
		{
			Value = Object;
		}
		Value.GetUniqueID().FixupForPIE();
	}
	else
	{
		FArchiveUObject::SerializeSoftObjectPtr(*this, Value);
	}

	return *this;
}

FArchive& FNetBitReader::operator<<(FObjectPtr& Value)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (PackageMap->IsUsingRemoteObjectReferences())
	{
		FRemoteObjectReference RemoteReference;
		bool bResult = false;
		RemoteReference.NetSerialize(*this, PackageMap, bResult);
		Value = RemoteReference.ToObjectPtr();
		return *this;
	}
#endif
	return FArchiveUObject::SerializeObjectPtr(*this, Value);
}

FArchive& FNetBitReader::operator<<(struct FWeakObjectPtr& WeakObjectPtr)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (PackageMap->IsUsingRemoteObjectReferences())
	{
		FRemoteObjectReference RemoteReference;
		bool bResult = false;
		RemoteReference.NetSerialize(*this, PackageMap, bResult);
		WeakObjectPtr = RemoteReference.ToWeakPtr();
		return *this;
	}
#endif
	return FArchiveUObject::SerializeWeakObjectPtr(*this, WeakObjectPtr);
}

void FNetBitReader::CountMemory(FArchive& Ar) const
{
	FBitReader::CountMemory(Ar);
	const SIZE_T MemberSize = sizeof(*this) - sizeof(FBitReader);
	Ar.CountBytes(MemberSize, MemberSize);
}

static const TCHAR* GLastRPCFailedReason = NULL;

void RPC_ResetLastFailedReason()
{
	GLastRPCFailedReason = NULL;
}
void RPC_ValidateFailed( const TCHAR* Reason )
{
	GLastRPCFailedReason = Reason;
}

const TCHAR* RPC_GetLastFailedReason()
{
	return GLastRPCFailedReason;
}

const TCHAR* LexToString(const EChannelCloseReason Value)
{
	switch (Value)
	{
	case EChannelCloseReason::Destroyed:
		return TEXT("Destroyed");
	case EChannelCloseReason::Dormancy:
		return TEXT("Dormancy");
	case EChannelCloseReason::LevelUnloaded:
		return TEXT("LevelUnloaded");
	case EChannelCloseReason::Relevancy:
		return TEXT("Relevancy");
	case EChannelCloseReason::TearOff:
		return TEXT("TearOff");
	case EChannelCloseReason::Migrated:
		return TEXT("Migrated");
	}

	return TEXT("Unknown");
}
