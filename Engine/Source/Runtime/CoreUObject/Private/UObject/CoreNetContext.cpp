// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/CoreNetContext.h"
#include "UObject/Class.h"

namespace UE::Net::Private
{

FCoreNetContext* FCoreNetContext::Get()
{
	static FCoreNetContext* InstancePtr;

	if (!InstancePtr)
	{
		// It's important that the initializer goes here to avoid the overhead of
		// "magic static" initialization on every call (mostly an issue with MSVC
		// because of their epoch-based initialization scheme which doesn't seem
		// to make any real sense on x86)

		static FCoreNetContext Instance;
		InstancePtr = &Instance;
	}

	return InstancePtr;
}

ERemoteFunctionMode FCoreNetContext::GetCurrentRemoteFunctionMode()
{
	return RemoteFunctionStack.IsEmpty() ? ERemoteFunctionMode::None : RemoteFunctionStack.Top();
}

FScopedRemoteRPCMode::FScopedRemoteRPCMode(const UFunction* Function, ERemoteFunctionMode Mode)
{
	// Only add to stack if the function is specified as "Remote"
	bAddToStack =
		((Function->FunctionFlags & FUNC_Net) != 0) &&
		((Function->FunctionFlags & (FUNC_NetClient | FUNC_NetServer | FUNC_NetMulticast)) == 0);

	if (bAddToStack)
	{
		check(FCoreNetContext::Get()->RemoteFunctionStack.Num() < FCoreNetContext::MaxRemoteStackSize);
		FCoreNetContext::Get()->RemoteFunctionStack.Push(Mode);
	}
}

FScopedRemoteRPCMode::~FScopedRemoteRPCMode()
{
	if (bAddToStack)
	{
		check(!FCoreNetContext::Get()->RemoteFunctionStack.IsEmpty());
		FCoreNetContext::Get()->RemoteFunctionStack.Pop();
	}
}

}