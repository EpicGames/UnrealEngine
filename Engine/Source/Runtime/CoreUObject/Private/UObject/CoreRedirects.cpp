// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/CoreRedirects.h"

#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UObject/CoreRedirects/CoreRedirectsContext.h"
#include "UObject/CoreRedirects/RedirectionSummary.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "UObject/PropertyHelper.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UnrealType.h"

#include "Algo/Compare.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Hash/Blake3.h"
#include "Logging/StructuredLog.h"
#include "Misc/AutomationTest.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/DeferredMessageLog.h"
#include "Templates/Casts.h"

#include "AutoRTFM.h"

#if WITH_EDITOR
#include "Misc/RedirectCollector.h"
#include "Misc/CommandLine.h"
#endif

DEFINE_LOG_CATEGORY(LogCoreRedirects);

#if !defined UE_WITH_CORE_REDIRECTS
#	define UE_WITH_CORE_REDIRECTS 1
#endif

#define COREREDIRECT_STATS 0
#if COREREDIRECT_STATS
static int sWildcardLookups = 0;
static int sWildcardPredictHit = 0;
static int sWildcardPredictMiss = 0;
FAutoConsoleCommand StaticLoadTrackerDump(TEXT("CoreRedirects.WildcardPrediction.StatsDump"), TEXT(""), FConsoleCommandDelegate::CreateLambda([]()
	{
		UE_LOGF(LogCoreRedirects, Log, "\n\n"
			"==========================================\n"
			"Dumping Wildcard Stats\n\n"
			"Lookups:\t\t\t\t%d\n"
			"Prediction Hit:\t\t\t%d\n"
			"Prediction Miss:\t\t%d\n"
			"==========================================",  sWildcardLookups, sWildcardPredictHit, sWildcardPredictMiss)
	}));

#define COREREDIRECT_STATS_UPDATE_PREDICTION_LOOKUP() do{ sWildcardLookups++; }while(false)
#define COREREDIRECT_STATS_UPDATE_PREDICTION_RESULT(bFound) do{ if(bFound){ sWildcardPredictHit++; } else { sWildcardPredictMiss++; }}while(false)
#else
#define COREREDIRECT_STATS_UPDATE_PREDICTION_LOOKUP()
#define COREREDIRECT_STATS_UPDATE_PREDICTION_RESULT(bFound)
#endif

/* 
* Small helper class to deal with the string representation of FNames while trying to minimize copies 
* without relying on internal details of FName. Due to the sheer volume of CoreRedirects, it's 
* significant to avoid the copies where possible. We maintain UTF8 encoding for FNames when dealing with Wildcard
* redirects since UTF8 overlays ASCII allowing pure ASCII strings to be compact, greatly improving PredictMatch efficiency.
* We also need a common encoding for how we store wildcard redirects and how we query them to ensure we do not generate
* false negatives in our fuzzy searchs using PredictMatch; UTF8 will allow us to still predict non-ASCII wildcards.
*/
namespace UE::CoreRedirects::Private
{
	struct FNameUtf8String
	{
		FNameUtf8String() : UTF8() {}
		~FNameUtf8String() {}
		FNameUtf8String(FName Name)
			: UTF8()
		{
			// FName will append "None" when FName.IsNone. Instead treat empty string as None
			if (!Name.IsNone())
			{
				Name.AppendString(UTF8);
			}
		}

		operator FUtf8StringView() const
		{ 
			return UTF8; 
		}

		bool IsNone() const
		{
			return Len() == 0;
		}

		const UTF8CHAR* GetData() const
		{
			return UTF8.GetData();
		}

		int32 Len() const
		{
			return UTF8.Len();
		}

		int32 ByteLength() const
		{
			return UTF8.Len() * sizeof(UTF8CHAR);
		}

		bool StartsWith(const FUtf8StringView InPrefix, ESearchCase::Type InSearchCase) const
		{
			return UTF8.ToView().StartsWith(InPrefix, InSearchCase);
		}

		bool EndsWith(const FUtf8StringView InSuffix, ESearchCase::Type InSearchCase) const
		{
			return UTF8.ToView().EndsWith(InSuffix, InSearchCase);
		}

		bool Contains(const FUtf8StringView InSubstring, ESearchCase::Type InSearchCase) const
		{
			return (INDEX_NONE != UE::String::FindFirst(UTF8, InSubstring, InSearchCase));
		}

		void ReplaceAt(int32 Pos, int32 RemoveLen, FNameUtf8String& Str)
		{
			UTF8.ReplaceAt(Pos, RemoveLen, Str);
		}

		void RightChopInline(int Len)
		{
			UTF8.RemoveAt(0, Len);
		}

		TUtf8StringBuilder<FName::StringBufferSize> UTF8;
	};

	struct FCoreRedirectObjectUtf8Name
	{
		FCoreRedirectObjectUtf8Name(const FCoreRedirectObjectName& InName)
			: ObjectName(InName.ObjectName)
			, OuterName(InName.OuterName)
			, PackageName(InName.PackageName)
		{
		}

		FNameUtf8String ObjectName;
		FNameUtf8String OuterName;
		FNameUtf8String PackageName;
	};

	struct FSubstringMatcher
	{
		static bool Matches(const FName LHS, const FNameUtf8String& RHSString, bool bPartialRHS)
		{
			const bool bLHSIsNone = LHS.IsNone();
			const bool bRHSIsNone = RHSString.IsNone();
			if (bLHSIsNone || bRHSIsNone)
			{
				return bLHSIsNone == bRHSIsNone || (!bRHSIsNone || bPartialRHS);
			}

			FNameUtf8String LHSString(LHS);
			return RHSString.Contains(LHSString, ESearchCase::IgnoreCase);
		}
	};

	struct FPrefixMatcher
	{
		static bool Matches(const FName LHS, const FNameUtf8String& RHSString, bool bPartialRHS)
		{
			const bool bLHSIsNone = LHS.IsNone();
			const bool bRHSIsNone = RHSString.IsNone();
			if (bLHSIsNone || bRHSIsNone)
			{
				return bLHSIsNone == bRHSIsNone || (!bRHSIsNone || bPartialRHS);
			}

			FNameUtf8String LHSString(LHS);
			return RHSString.StartsWith(LHSString, ESearchCase::IgnoreCase);
		}
	};

	struct FSuffixMatcher
	{
		static bool Matches(const FName LHS, const FNameUtf8String& RHSString, bool bPartialRHS)
		{
			const bool bLHSIsNone = LHS.IsNone();
			const bool bRHSIsNone = RHSString.IsNone();
			if (bLHSIsNone || bRHSIsNone)
			{
				return bLHSIsNone == bRHSIsNone || (!bRHSIsNone || bPartialRHS);
			}

			FNameUtf8String LHSString(LHS);
			return RHSString.EndsWith(LHSString, ESearchCase::IgnoreCase);
		}
	};

	template <typename Matcher>
	static bool MatchWildcardRedirect(const FCoreRedirectObjectName& InRedirect, 
		const FCoreRedirectObjectUtf8Name& InUtf8Name, bool bPartialRHS)
	{
		if (!Matcher::Matches(InRedirect.ObjectName, InUtf8Name.ObjectName, bPartialRHS))
		{
			return false;
		}
		if (!Matcher::Matches(InRedirect.OuterName, InUtf8Name.OuterName, bPartialRHS))
		{
			return false;
		}
		if (!Matcher::Matches(InRedirect.PackageName, InUtf8Name.PackageName, bPartialRHS))
		{
			return false;
		}
		return true;
	}

	static bool MatchSubstring(const FCoreRedirectObjectName& InSubstringRedirect, const FCoreRedirectObjectUtf8Name& InUtf8Name, bool bPartialRHS)
	{
		return MatchWildcardRedirect<FSubstringMatcher>(InSubstringRedirect, InUtf8Name, bPartialRHS);
	}

	static bool MatchPrefix(const FCoreRedirectObjectName& InPrefixRedirect, const FCoreRedirectObjectUtf8Name& InUtf8Name, bool bPartialRHS)
	{
		return MatchWildcardRedirect<FPrefixMatcher>(InPrefixRedirect, InUtf8Name, bPartialRHS);
	}

	static bool MatchSuffix(const FCoreRedirectObjectName& InSuffixRedirect, const FCoreRedirectObjectUtf8Name& InUtf8Name, bool bPartialRHS)
	{
		return MatchWildcardRedirect<FSuffixMatcher>(InSuffixRedirect, InUtf8Name, bPartialRHS);
	}


	bool RunAssetRedirectTests()
	{
		bool bSuccess = true;

		// Test
		TMap<FSoftObjectPath, FSoftObjectPath> Redirects;

		struct FTestDefinition
		{
			const TCHAR* Origin = nullptr;
			const TCHAR* Destination = nullptr;
			const TCHAR* TestDescription = nullptr;
			ECoreRedirectFlags RedirectFlags = ECoreRedirectFlags::None;
			bool ExpectTrue = true;
		};
		TArray<FTestDefinition> Tests;

		// Simple single redirection BasicName --> BasicNewName
		{
			FTestDefinition& BasicTest = Tests.AddDefaulted_GetRef();
			BasicTest.Origin = TEXT("/Game/BasicName.BasicName");
			BasicTest.Destination = TEXT("/Game/BasicNewName.BasicNewName");
			BasicTest.TestDescription = TEXT("basic asset redirect with Type_Object");
			BasicTest.RedirectFlags = ECoreRedirectFlags::Type_Object;
			BasicTest.ExpectTrue = true;

			{
				FSoftObjectPath SourcePath(BasicTest.Origin);
				FSoftObjectPath DestPath(BasicTest.Destination);
				Redirects.Add(SourcePath, DestPath);
			}
		}

		// Multi-asset package redirections. bp_orig_name + _C + Default --> bp_new_name + _C + Default
		{
			FTestDefinition& DefaultObjectTest = Tests.AddDefaulted_GetRef();
			DefaultObjectTest.Origin = TEXT("/Plugin/bp_orig_name.Default__bp_orig_name_C");
			DefaultObjectTest.Destination = TEXT("/Plugin/bp_new_name.Default__bp_new_name_C");
			DefaultObjectTest.TestDescription = TEXT("default object asset redirect with Type_Object");
			DefaultObjectTest.RedirectFlags = ECoreRedirectFlags::Type_Object;
			DefaultObjectTest.ExpectTrue = true;

			FTestDefinition& BPGCTest = Tests.AddDefaulted_GetRef();
			BPGCTest.Origin = TEXT("/Plugin/bp_orig_name.bp_orig_name_C");
			BPGCTest.Destination = TEXT("/Plugin/bp_new_name.bp_new_name_C");
			BPGCTest.TestDescription = TEXT("default BPGC asset redirect with Type_Class");
			BPGCTest.RedirectFlags = ECoreRedirectFlags::Type_Class;
			BPGCTest.ExpectTrue = true;

			FTestDefinition& InstanceOnlyTest = Tests.AddDefaulted_GetRef();
			InstanceOnlyTest.Origin = BPGCTest.Origin;
			InstanceOnlyTest.Destination = BPGCTest.Destination;
			InstanceOnlyTest.RedirectFlags = ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly;
			InstanceOnlyTest.TestDescription = TEXT("category instance only");
			InstanceOnlyTest.ExpectTrue = false;

			{
				FSoftObjectPath SourcePath(TEXT("/Plugin/bp_orig_name.Default__bp_orig_name_C"));
				FSoftObjectPath DestPath(TEXT("/Plugin/bp_new_name.Default__bp_new_name_C"));
				Redirects.Add(SourcePath, DestPath);
			}

			{
				FSoftObjectPath SourcePath(TEXT("/Plugin/bp_orig_name.bp_orig_name_C"));
				FSoftObjectPath DestPath(TEXT("/Plugin/bp_new_name.bp_new_name_C"));
				Redirects.Add(SourcePath, DestPath);
			}

			{
				FSoftObjectPath SourcePath(TEXT("/Plugin/bp_orig_name.bp_orig_name"));
				FSoftObjectPath DestPath(TEXT("/Plugin/bp_new_name.bp_new_name"));
				Redirects.Add(SourcePath, DestPath);
			}
		}

		FCoreRedirects::AddAssetRedirects(Redirects);

		if (!FCoreRedirects::ValidateAssetRedirects())
		{
			bSuccess = false;
			UE_LOGF(LogCoreRedirects, Error, "Failed asset redirect validation.");
		}

		for (auto& Test : Tests)
		{
			FCoreRedirectObjectName OldName(Test.Origin);
			FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(Test.RedirectFlags, OldName);
			if (NewName.ToString().Equals(Test.Destination) != Test.ExpectTrue)
			{
				bSuccess = false;
				UE_LOGF(LogCoreRedirects, Error, "Failed %ls. Source = %ls was unexpectedly redirected to %ls",
					Test.TestDescription, Test.Origin, *NewName.ToString());
			}
			if (Test.ExpectTrue)
			{
				TArray<FCoreRedirectObjectName> OldNames;
				if (!FCoreRedirects::FindPreviousNames(Test.RedirectFlags, NewName, OldNames))
				{
					bSuccess = false;
					UE_LOGF(LogCoreRedirects, Error, "Failed to FindPreviousNames for %ls", Test.Destination);
				}
				else
				{
					bool bContainsName = false;
					for (const FCoreRedirectObjectName& ReverseOldName : OldNames)
					{
						if (ReverseOldName.ToString().Compare(Test.Origin))
						{
							bContainsName = true;
							break;
						}
					}
					if (!bContainsName)
					{
						bSuccess = false;
						UE_LOGF(LogCoreRedirects, Error, "Failed to find expected previous name for %ls", Test.Destination);
					}
				}
			}
		}

		// Remove all redirects temporarily and verify no test finds a redirection
		FCoreRedirects::RemoveAllAssetRedirects();
		for (auto& Test : Tests)
		{
			FCoreRedirectObjectName OldName(Test.Origin);
			FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(Test.RedirectFlags, OldName);
			if (!NewName.ToString().Equals(Test.Origin))
			{
				bSuccess = false;
				UE_LOGF(LogCoreRedirects, Error, "Found unexpected redirect from %ls to %ls",
					Test.Origin, *NewName.ToString());
			}
		}

#if WITH_EDITOR
		// ensure that it's safe to add any redirector in GRedirectCollector
		{
			TArray<const FCoreRedirect> RedirectList;
			GRedirectCollector.EnumerateRedirectsUnderLock([&RedirectList](FRedirectCollector::FRedirectionData& Data)
				{
					RedirectList.Emplace(ECoreRedirectFlags::Type_Asset,
						Data.GetSource().ToString(),
						Data.GetFirstTarget().ToString());
				});

			FCoreRedirects::AddRedirectList(RedirectList, TEXT("GRedirectCollector"));
			if (!FCoreRedirects::ValidateAssetRedirects())
			{
				bSuccess = false;
				UE_LOGF(LogCoreRedirects, Error, "Failed asset redirect validation.");
			}
		}
		FCoreRedirects::RemoveAllAssetRedirects();
#endif

		// Ensure validation failure if chains exist
		{
			TMap<FSoftObjectPath, FSoftObjectPath> ChainRedirects;

			{
				FSoftObjectPath SourcePath(TEXT("/Game/Chain_FirstName.Chain_FirstName"));
				FSoftObjectPath DestPath(TEXT("/Game/Chain_SecondName.Chain_SecondName"));
				ChainRedirects.Add(SourcePath, DestPath);
			}

			{
				FSoftObjectPath SourcePath(TEXT("/Game/Chain_SecondName.Chain_SecondName"));
				FSoftObjectPath DestPath(TEXT("/Game/Chain_ThirdName.Chain_ThirdName"));
				ChainRedirects.Add(SourcePath, DestPath);
			}

			{
				FSoftObjectPath SourcePath(TEXT("/Game/Chain_ThirdName.Chain_ThirdName"));
				FSoftObjectPath DestPath(TEXT("/Game/Chain_FourthName.Chain_FourthName"));
				ChainRedirects.Add(SourcePath, DestPath);
			}

			FCoreRedirects::AddAssetRedirects(ChainRedirects);
			if (FCoreRedirects::ValidateAssetRedirects())
			{
				bSuccess = false;
				UE_LOGF(LogCoreRedirects, Error, "Failed to detect erroneous chained redirect in ValidateAssetRedirects()");
			}
			FCoreRedirects::RemoveAllAssetRedirects();
		}

		// Re-add the redirects so that they are in place for subsequent tests. This can help
		// detect unexpected interactions between asset redirects and legacy types of redirects
		FCoreRedirects::AddAssetRedirects(Redirects);

		return bSuccess;
	}
}

FCoreRedirectObjectName::FCoreRedirectObjectName(const FTopLevelAssetPath& TopLevelAssetPath)
	: FCoreRedirectObjectName(TopLevelAssetPath.GetAssetName(), NAME_None, TopLevelAssetPath.GetPackageName())
{
}

FCoreRedirectObjectName::FCoreRedirectObjectName(const FSoftObjectPath& SoftObjectPath)
{
	if (!SoftObjectPath.IsSubobject())
	{
		PackageName = SoftObjectPath.GetLongPackageFName();
		ObjectName = SoftObjectPath.GetAssetFName();
	}
	else
	{
		if (!ExpandNames(SoftObjectPath.ToString(), ObjectName, OuterName, PackageName))
		{
			Reset();
		}
	}
}

FCoreRedirectObjectName::FCoreRedirectObjectName(const FString& InString)
{
	if (!ExpandNames(InString, ObjectName, OuterName, PackageName))
	{
		Reset();
	}
}

FCoreRedirectObjectName::FCoreRedirectObjectName(const class UObject* Object)
{
	// This is more efficient than going to path string and back to FNames
	if (Object)
	{	
		UObject* Outer = Object->GetOuter();

		if (!Outer)
		{
			PackageName = Object->GetFName();
			// This is a package
		}
		else
		{
			FString OuterString;
			ObjectName = Object->GetFName();

			// Follow outer chain,
			while (Outer)
			{
				UObject* NextOuter = Outer->GetOuter();
				if (!NextOuter)
				{
					OuterName = FName(*OuterString);
					PackageName = Outer->GetFName();
					break;
				}
				if (!OuterString.IsEmpty())
				{
					OuterString.AppendChar(TEXT('.'));
				}
				OuterString.Append(Outer->GetName());

				Outer = NextOuter;
			}
		}
	}
}

FString FCoreRedirectObjectName::ToString() const
{
	return CombineNames(ObjectName, OuterName, PackageName);
}

void FCoreRedirectObjectName::Reset()
{
	ObjectName = OuterName = PackageName = NAME_None;
}

void FCoreRedirects::FWildcardData::Add(const FCoreRedirect& Redirect)
{
	if (Redirect.IsSubstringMatch())
	{
		Substrings.Add(Redirect);
	}
	else if (Redirect.IsPrefixMatch())
	{
		Prefixes.Add(Redirect);
	}
	else
	{
		check(Redirect.IsSuffixMatch());
		Suffixes.Add(Redirect);
	}
	AddPredictionWords(Redirect);
}

void FCoreRedirects::FWildcardData::AddPredictionWords(const FCoreRedirect& Redirect)
{
	using namespace UE::CoreRedirects::Private;

	const FName Names[3] = { Redirect.OldName.ObjectName, Redirect.OldName.OuterName, Redirect.OldName.PackageName };
	for (const FName& Name : Names)
	{
		if (Name.IsNone())
		{
			continue;
		}

		FNameUtf8String NameView(Name);

		// Since we only predict based on a small window of characters (8), 
		// try to use characters that will be more unique by removing common prefixes.
		// Note, this does not affect the actual wildcard matching, only
		// our prediction and more specifically our false positive rate.
		FUtf8StringView CommonPrefixes[] = 
		{ "/", "Script/", "Temp/", "Extra/", "Memory/", "Config/", "Game/", "Engine/", "Transient/", "Niagara/"};

		for (FUtf8StringView Prefix : CommonPrefixes)
		{
			// Only remove the prefix if doing so won't leave the string empty.
			// To consider: To keep entropy high, we might want to consider only performing the 
			// removal if we leave 'n' number of characers for our prediction.
			if (NameView.Len() > Prefix.Len() && NameView.StartsWith(Prefix, ESearchCase::IgnoreCase))
			{
				NameView.RightChopInline(Prefix.Len());
			}
		}

		// Predict match doesn't interpret character data, however in order to avoid false negatives
		// we need to ensure all PredictionWords are added in the same encoding we will be using
		// for comparisons in MatchSubstringApproximate
		PredictMatch.AddPredictionWord((uint8*) NameView.GetData(), (uint16) NameView.ByteLength());
	}
}

bool FCoreRedirects::FWildcardData::MatchSubstringApproximate(const UE::CoreRedirects::Private::FCoreRedirectObjectUtf8Name& InUtf8Name) const
{
	using namespace UE::CoreRedirects::Private;

	const FNameUtf8String* Names[3] = { &InUtf8Name.ObjectName, &InUtf8Name.OuterName, &InUtf8Name.PackageName };
	for (const FNameUtf8String* Name : Names)
	{
		if (Name->IsNone())
		{
			continue;
		}

		if (PredictMatch.MatchApproximate((const uint8*) Name->GetData(), (uint16) Name->ByteLength()))
		{
			return true;
		}
	}
	return false;
}

void FCoreRedirects::FWildcardData::Rebuild()
{
	PredictMatch.Reset();

	for (const FCoreRedirect& Redirect : Prefixes)
	{
		AddPredictionWords(Redirect);
	}

	for (const FCoreRedirect& Redirect : Suffixes)
	{
		AddPredictionWords(Redirect);
	}

	for (const FCoreRedirect& Redirect : Substrings)
	{
		AddPredictionWords(Redirect);
	}
}

bool FCoreRedirects::FWildcardData::Matches(ECoreRedirectFlags InFlags, const FCoreRedirectObjectName& InName, 
	ECoreRedirectMatchFlags InMatchFlags, TArray<const FCoreRedirect*>& OutFoundRedirects) const
{
	using namespace UE::CoreRedirects::Private;

	bool bFound = false;
	const bool bPartialRHS = EnumHasAnyFlags(InMatchFlags, ECoreRedirectMatchFlags::AllowPartialMatch);

	// Substring implies prefix and suffix matches. For wildcard matches we must search all three
	// types as each can be defined distinctly
	const bool bSubstringMatch = EnumHasAllFlags(InFlags, ECoreRedirectFlags::Option_MatchSubstring);
	const bool bPrefixMatch = EnumHasAllFlags(InFlags, ECoreRedirectFlags::Option_MatchPrefix);
	const bool bSuffixMatch = EnumHasAllFlags(InFlags, ECoreRedirectFlags::Option_MatchSuffix);

	// Creating the string form of InName is expensive so early out if we know we can
	if ((!bSubstringMatch && !bPrefixMatch && !bSuffixMatch) ||
		(Prefixes.IsEmpty() && Suffixes.IsEmpty() && Substrings.IsEmpty()))
	{
		return false;
	}

	const FCoreRedirectObjectUtf8Name ObjectUtf8Name(InName);
	
	COREREDIRECT_STATS_UPDATE_PREDICTION_LOOKUP();
	// Perform a fuzzy match against all known wildcard (prefix, suffix and substrings since they all overlap). 
	// We may have false positives so we must still perform a stronger check on a fuzzy match. 
	// However we can eliminate the common failure case quickly with a fast fuzzy match.
	if (MatchSubstringApproximate(ObjectUtf8Name))
	{
		// We found a possible wildcard match, so now do a slow match to find if we really match
		if (bPrefixMatch)
		{
			for (const FCoreRedirect& CheckRedirect : Prefixes)
			{
				if (MatchPrefix(CheckRedirect.OldName, ObjectUtf8Name, bPartialRHS))
				{
					bFound = true;
					OutFoundRedirects.Add(&CheckRedirect);
				}
			}
		}

		if (bSuffixMatch)
		{
			for (const FCoreRedirect& CheckRedirect : Suffixes)
			{
				if (MatchSuffix(CheckRedirect.OldName, ObjectUtf8Name, bPartialRHS))
				{
					bFound = true;
					OutFoundRedirects.Add(&CheckRedirect);
				}
			}
		}

		if (bSubstringMatch )
		{
			for (const FCoreRedirect& CheckRedirect : Substrings)
			{
				if (MatchSubstring(CheckRedirect.OldName, ObjectUtf8Name, bPartialRHS))
				{
					bFound = true;
					OutFoundRedirects.Add(&CheckRedirect);
				}
			}
		}
			
		COREREDIRECT_STATS_UPDATE_PREDICTION_RESULT(bFound);
	}

	return bFound;
}

bool FCoreRedirectObjectName::Matches(const FCoreRedirectObjectName& Other, EMatchFlags MatchFlags) const
{
	using namespace UE::CoreRedirects::Private;

	bool bPartialLHS = !EnumHasAnyFlags(MatchFlags, EMatchFlags::DisallowPartialLHSMatch);
	bool bPartialRHS = EnumHasAnyFlags(MatchFlags, EMatchFlags::AllowPartialRHSMatch);
	bool bSubstring = EnumHasAllFlags(MatchFlags, EMatchFlags::CheckSubString);
	bool bPrefix = EnumHasAllFlags(MatchFlags, EMatchFlags::CheckPrefix);
	bool bSuffix = EnumHasAllFlags(MatchFlags, EMatchFlags::CheckSuffix);

	// Substring implies prefix and suffix so we must check it first
	if (bSubstring)
	{
		return MatchSubstring(*this, Other, bPartialRHS);
	}
	else if(bPrefix)
	{
		return MatchPrefix(*this, Other, bPartialRHS);
	}
	else if (bSuffix)
	{
		return MatchSuffix(*this, Other, bPartialRHS);
	}

	auto FieldMatches = [bPartialLHS, bPartialRHS, bSubstring, bPrefix, bSuffix](FName LHS, FName RHS)
	{
		if (LHS == RHS)
			return true;

		const bool bLHSIsNone = LHS.IsNone();
		const bool bRHSIsNone = RHS.IsNone();
		if (bLHSIsNone || bRHSIsNone)
		{
			return (!bLHSIsNone || bPartialLHS) && (!bRHSIsNone || bPartialRHS);
		}

		return false;
	};

	if (!FieldMatches(ObjectName, Other.ObjectName))
	{
		return false;
	}
	if (!FieldMatches(OuterName, Other.OuterName))
	{
		return false;
	}
	if (!FieldMatches(PackageName, Other.PackageName))
	{
		return false;
	}
	return true;
}

template<typename Matcher>
static int32 WildcardMatchScore(const FCoreRedirectObjectName Redirect, const FCoreRedirectObjectName& Other, bool bPartialRHS)
{
	using namespace UE::CoreRedirects::Private;

	// Note, WildcardMatches default lower (1 vs 2) than direct matches intentionally to give priority to direct matches
	int32 MatchScore = 1;
	if (Redirect.ObjectName != NAME_None)
	{
		FNameUtf8String OtherName(Other.ObjectName);
		if (Matcher::Matches(Redirect.ObjectName, OtherName, bPartialRHS))
		{
			// Object name most important
			MatchScore += 16;
		}
		else
		{
			return 0;
		}
	}

	if (Redirect.OuterName != NAME_None)
	{
		FNameUtf8String OtherName(Other.OuterName);
		if (Matcher::Matches(Redirect.OuterName, OtherName, bPartialRHS))
		{
			MatchScore += 8;
		}
		else
		{
			return 0;
		}
	}

	if (Redirect.PackageName != NAME_None)
	{
		FNameUtf8String OtherName(Other.PackageName);
		if (Matcher::Matches(Redirect.PackageName, OtherName, bPartialRHS))
		{
			MatchScore += 4;
		}
		else
		{
			return 0;
		}
	}
	return MatchScore;
}

int32 FCoreRedirectObjectName::MatchScore(const FCoreRedirectObjectName& Other, ECoreRedirectFlags RedirectFlags, ECoreRedirectMatchFlags MatchFlags) const
{
	int32 MatchScore = 2;
	if (!EnumHasAnyFlags(RedirectFlags, ECoreRedirectFlags::Option_MatchWildcardMask))
	{
		if (ObjectName != NAME_None)
		{
			if (ObjectName == Other.ObjectName)
			{
				// Object name most important
				MatchScore += 16;
			}
			else
			{
				return 0;
			}
		}

		if (OuterName != NAME_None)
		{
			if (OuterName == Other.OuterName)
			{
				MatchScore += 8;
			}
			else
			{
				return 0;
			}
		}

		if (PackageName != NAME_None)
		{
			if (PackageName == Other.PackageName)
			{
				MatchScore += 4;
			}
			else
			{
				return 0;
			}
		}
	}
	else
	{
		using namespace UE::CoreRedirects::Private;
		const bool bPartialRHS = EnumHasAnyFlags(MatchFlags, ECoreRedirectMatchFlags::AllowPartialMatch);
		if (EnumHasAnyFlags(RedirectFlags, ECoreRedirectFlags::Option_MatchPrefix))
		{
			return WildcardMatchScore<FPrefixMatcher>(*this, Other, bPartialRHS);
		}
		else if (EnumHasAnyFlags(RedirectFlags, ECoreRedirectFlags::Option_MatchSuffix))
		{
			return WildcardMatchScore<FSuffixMatcher>(*this, Other, bPartialRHS);
		}
		else if (EnumHasAnyFlags(RedirectFlags, ECoreRedirectFlags::Option_MatchSubstring))
		{
			return WildcardMatchScore<FSubstringMatcher>(*this, Other, bPartialRHS);
		}
		else
		{
			checkNoEntry();
		}
	}

	return MatchScore;
}

void FCoreRedirectObjectName::UnionFieldsInline(const FCoreRedirectObjectName& Other)
{
	if (ObjectName.IsNone())
	{
		ObjectName = Other.ObjectName;
	}
	if (OuterName.IsNone())
	{
		OuterName = Other.OuterName;
	}
	if (PackageName.IsNone())
	{
		PackageName = Other.PackageName;
	}
}

FName FCoreRedirectObjectName::GetSearchKey(ECoreRedirectFlags Type) const
{
	if (EnumHasAnyFlags(Type, ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Type_Asset))
	{
		return PackageName;
	}

	return ObjectName;
}

bool FCoreRedirectObjectName::HasValidCharacters(ECoreRedirectFlags Type) const
{
	static FString InvalidObjectNameCharacters = ".\n\r\t"; // Object and field names in Blueprints are very permissive
	if (FPackageName::IsVersePackage(WriteToString<NAME_SIZE>(PackageName)))
	{
		// Verse allows the visible character ASCII set minus these CR, LF, TAB chars.
		// However CoreRedirects currently does not support names with '.' which verse does allow. 
		// Until support is added (UE-248198) we reject such names since they can't work correctly.
		// For redirecting verse names, a verse specific transformation must occur before using FCoreRedirects
		// which ensures '.' is no longer used as a non-delimiting character.
		static FString InvalidVerseCharacters = "\n\r\t";
		return  ObjectName.IsValidXName(InvalidObjectNameCharacters) &&
				OuterName.IsValidXName(InvalidVerseCharacters) &&
				PackageName.IsValidXName(InvalidVerseCharacters);
	}

	static FString InvalidRedirectNameCharacters = "\"',|&!~\n\r\t@#(){}[]=;^%$`";
	const bool bUsePermissiveObjectNames = EnumHasAnyFlags(Type, 
		ECoreRedirectFlags::Type_Object | ECoreRedirectFlags::Type_Property | ECoreRedirectFlags::Type_Function);

	return	ObjectName.IsValidXName(bUsePermissiveObjectNames ? InvalidObjectNameCharacters : InvalidRedirectNameCharacters) &&
			OuterName.IsValidXName(InvalidRedirectNameCharacters) &&
			PackageName.IsValidXName(InvalidRedirectNameCharacters);
}

bool FCoreRedirectObjectName::ExpandNames(const FStringView InStringView, FName& OutName, FName& OutOuter, FName &OutPackage)
{
	FStringView FullStringView = InStringView.TrimStartAndEnd();
	FullStringView.TrimStartAndEndInline();

	// Parse (/path.)?(outerchain.)?(name) where path and outerchain are optional
	// We also need to support (/path.)?(singleouter:)?(name) because the second delimiter in a chain is : for historical reasons
	int32 SlashIndex = INDEX_NONE;
	int32 FirstPeriodIndex = INDEX_NONE;
	int32 LastPeriodIndex = INDEX_NONE;
	int32 FirstColonIndex = INDEX_NONE;
	int32 LastColonIndex = INDEX_NONE;
	FullStringView.FindChar(TEXT('/'), SlashIndex);
	FullStringView.FindChar(TEXT('.'), FirstPeriodIndex);
	FullStringView.FindChar(TEXT(':'), FirstColonIndex);

	if (FirstColonIndex != INDEX_NONE && (FirstPeriodIndex == INDEX_NONE || FirstColonIndex < FirstPeriodIndex))
	{
		// If : is before . treat it as the first period
		FirstPeriodIndex = FirstColonIndex;
	}

	if (FirstPeriodIndex == INDEX_NONE)
	{
		// We cannot make FNames from strings of this size
		if (FullStringView.Len() >= NAME_SIZE)
		{
			return false;
		}

		// If start with /, fill in package name, otherwise name
		if (SlashIndex != INDEX_NONE)
		{
			OutPackage = FName(FullStringView);
		}
		else
		{
			OutName = FName(FullStringView);
		}
		return true;
	}

	FullStringView.FindLastChar(TEXT('.'), LastPeriodIndex);
	FullStringView.FindLastChar(TEXT(':'), LastColonIndex);

	if (LastColonIndex != INDEX_NONE && (LastPeriodIndex == INDEX_NONE || LastColonIndex > LastPeriodIndex))
	{
		// If : is after . treat it as the last period
		LastPeriodIndex = LastColonIndex;
	}

	if (SlashIndex == INDEX_NONE)
	{
		// We cannot make FNames from strings of this size
		if (LastPeriodIndex >= NAME_SIZE)
		{
			return false;
		}

		// No /, so start from beginning. There must be an outer if we got this far
		OutOuter = FName(FullStringView.Mid(0, LastPeriodIndex));
	}
	else
	{
		// We cannot make FNames from strings of this size
		if (FirstPeriodIndex >= NAME_SIZE)
		{
			return false;
		}

		OutPackage = FName(FullStringView.Left(FirstPeriodIndex));
		if (FirstPeriodIndex != LastPeriodIndex)
		{
			const int32 StartPos = FirstPeriodIndex + 1;
			const int32 Count = LastPeriodIndex - FirstPeriodIndex - 1;
			// We cannot make FNames from strings of this size
			if (Count >= NAME_SIZE)
			{
				return false;
			}

			// Extract Outer between periods
			OutOuter = FName(FullStringView.Mid(StartPos, Count));
		}
	}

	const int32 StartPos = LastPeriodIndex + 1;
	// We cannot make FNames from strings of this size
	if ((FullStringView.Len() - StartPos) >= NAME_SIZE)
	{
		return false;
	}
	OutName = FName(FullStringView.Mid(StartPos));

	return true;
}

FString FCoreRedirectObjectName::CombineNames(FName NewName, FName NewOuter, FName NewPackage)
{
	TArray<FString> CombineStrings;

	if (NewOuter != NAME_None)
	{
		// If Outer is simple, need to use : instead of . because : is used for second delimiter only
		FString OuterString = NewOuter.ToString();
		int32 DelimIndex = INDEX_NONE;

		if (OuterString.FindChar(TEXT('.'), DelimIndex) || OuterString.FindChar(TEXT(':'), DelimIndex))
		{
			if (NewPackage != NAME_None)
			{
				return FString::Printf(TEXT("%s.%s.%s"), *NewPackage.ToString(), *NewOuter.ToString(), *NewName.ToString());
			}
			else
			{
				return FString::Printf(TEXT("%s.%s"), *NewOuter.ToString(), *NewName.ToString());
			}
		}
		else
		{
			if (NewPackage != NAME_None)
			{
				return FString::Printf(TEXT("%s.%s:%s"), *NewPackage.ToString(), *NewOuter.ToString(), *NewName.ToString());
			}
			else
			{
				return FString::Printf(TEXT("%s:%s"), *NewOuter.ToString(), *NewName.ToString());
			}
		}
	}
	else if (NewPackage != NAME_None)
	{
		if (NewName != NAME_None)
		{
			return FString::Printf(TEXT("%s.%s"), *NewPackage.ToString(), *NewName.ToString());
		}
		return NewPackage.ToString();
	}
	return NewName.ToString();
}

namespace UE::CoreRedirects
{

static void SplitFirstComponent(FName Text, FStringBuilderBase& Buffer, FStringView& OutFirst, FStringView& OutRemainder)
{
	if (Text.IsNone())
	{
		OutFirst = FStringView();
		OutRemainder = FStringView();
		return;
	}

	Buffer.Reset();
	Buffer << Text;
	// Normalize the Text: treat SUBOBJECT_DELIMITER_CHAR as '.'
	for (TCHAR& C : Buffer)
	{
		if (C == SUBOBJECT_DELIMITER_CHAR)
		{
			C = '.';
		}
	}
	FStringView TextStr(Buffer);
	int32 FirstDotIndex;
	if (TextStr.FindChar('.', FirstDotIndex))
	{
		OutFirst = TextStr.Left(FirstDotIndex);
		OutRemainder = TextStr.RightChop(FirstDotIndex + 1);
	}
	else
	{
		OutFirst = TextStr;
		OutRemainder = FStringView();
	}
}

static void SplitLastComponent(FName Text, FStringBuilderBase& Buffer, FStringView& OutRemainder, FStringView& OutLast)
{
	if (Text.IsNone())
	{
		OutRemainder = FStringView();
		OutLast = FStringView();
		return;
	}

	Buffer.Reset();
	Buffer << Text;
	// Normalize the Text: treat SUBOBJECT_DELIMITER_CHAR as '.'
	for (TCHAR& C : Buffer)
	{
		if (C == SUBOBJECT_DELIMITER_CHAR)
		{
			C = '.';
		}
	}
	FStringView TextStr(Buffer);
	int32 LastDotIndex;
	if (TextStr.FindLastChar('.', LastDotIndex))
	{
		OutRemainder = TextStr.Left(LastDotIndex);
		OutLast = TextStr.RightChop(LastDotIndex + 1);
	}
	else
	{
		OutRemainder = FStringView();
		OutLast = TextStr;
	}
}

}

FCoreRedirectObjectName FCoreRedirectObjectName::AppendObjectName(const FCoreRedirectObjectName& Parent,
	FName ObjectName)
{
	using namespace UE::CoreRedirects;

	if (Parent.ObjectName.IsNone())
	{
		if (Parent.OuterName.IsNone())
		{
			if (Parent.PackageName.IsNone())
			{
				// Empty parent, return a packagename with the given ObjectName
				return FCoreRedirectObjectName(NAME_None, NAME_None, ObjectName);
			}
			else
			{
				// Child of a package
				return FCoreRedirectObjectName(ObjectName, NAME_None, Parent.PackageName);
			}
		}
		else
		{
			// An unexpected case; anything with an outer name should also had an objectname.
			// But if this case were valid, then the proper child would be to just set the objectname
			return FCoreRedirectObjectName(ObjectName, Parent.OuterName, Parent.PackageName);
		}
	}
	else
	{
		if (Parent.OuterName.IsNone())
		{
			if (Parent.PackageName.IsNone())
			{
				// An unexpected case; there should not be an objectname with no outer or packagename.
				// Treat this case is if the ObjectName were the packagename
				return FCoreRedirectObjectName(ObjectName, NAME_None, Parent.ObjectName);
			}
			else
			{
				// Parent is a toplevel child of a package, so the child has just the parent's objectname
				// as its outername.
				return FCoreRedirectObjectName(ObjectName, Parent.ObjectName, Parent.PackageName);
			}
		}
		else
		{
			if (Parent.PackageName.IsNone())
			{
				// An unexpected case; there should not be an objectname and outername with no packagename.
				// Treat this case is if the outermost of the outername were the packagename.
				TStringBuilder<NAME_SIZE> OuterNameBuffer;
				FStringView PackageName;
				FStringView OuterNameStr;
				SplitFirstComponent(Parent.OuterName, OuterNameBuffer, PackageName, OuterNameStr);
				if (PackageName.Len() == 0)
				{
					// Just a period by itself, treat this as OuterName == None.
					return AppendObjectName(FCoreRedirectObjectName(Parent.ObjectName, NAME_None, NAME_None), ObjectName);
				}
				FName OuterName = OuterNameStr.Len() > 0 ? FName(OuterNameStr) : NAME_None;
				return AppendObjectName(FCoreRedirectObjectName(Parent.ObjectName, OuterName, FName(PackageName)), ObjectName);
			}
			else
			{
				// Parent is Package.Outer.Object. Append Parent's Object to its Outer and set the child's ObjectName.
				TStringBuilder<NAME_SIZE> NewOuterNameBuffer(InPlace, Parent.OuterName);
				NewOuterNameBuffer << '.' << Parent.ObjectName;
				return FCoreRedirectObjectName(ObjectName, FName(NewOuterNameBuffer.ToView()), Parent.PackageName);
			}
		}
	}
}

FCoreRedirectObjectName FCoreRedirectObjectName::GetParent(const FCoreRedirectObjectName& Child)
{
	using namespace UE::CoreRedirects;

	if (Child.ObjectName.IsNone())
	{
		if (Child.OuterName.IsNone())
		{
			// Either An empty FCoreRedirectObjectName (Child.PackageName.IsNone()) or a package
			// Return an empty FCoreRedirectObjectName() for both of these cases.
			return FCoreRedirectObjectName();
		}
		else
		{
			if (Child.PackageName.IsNone())
			{
				// An unexpected case; anything with an outer name should also have an objectname and a packagename.
				// Treat this case is if the outermost of the outername were the packagename.
				TStringBuilder<NAME_SIZE> OuterNameBuffer;
				FStringView PackageName;
				FStringView OuterName;
				SplitFirstComponent(Child.OuterName, OuterNameBuffer, PackageName, OuterName);
				if (PackageName.Len() == 0)
				{
					// Just a period by itself, treat this as no outer name.
					return GetParent(FCoreRedirectObjectName(NAME_None, NAME_None, NAME_None));
				}
				else
				{
					FName NewOuter = OuterName.Len() > 0 ? FName(OuterName) : NAME_None;
					return GetParent(FCoreRedirectObjectName(NAME_None, NewOuter, FName(PackageName)));
				}
			}
			else
			{
				// An unexpected case; anything with an outer name should also have an objectname.
				// Treat this case is if the innermost of the outername were the objectname.
				TStringBuilder<NAME_SIZE> OuterNameBuffer;
				FStringView NewOuterNameStr;
				FStringView NewObjectName;
				SplitLastComponent(Child.OuterName, OuterNameBuffer, NewOuterNameStr, NewObjectName);
				if (NewObjectName.Len() == 0)
				{
					// Just a period by itself, treat this as no outer name.
					return GetParent(FCoreRedirectObjectName(NAME_None, NAME_None, Child.PackageName));
				}
				else
				{
					FName NewOuterName = NewOuterNameStr.Len() > 0 ? FName(NewOuterNameStr) : NAME_None;
					return GetParent(FCoreRedirectObjectName(FName(NewObjectName), NewOuterName, Child.PackageName));
				}
			}
		}
	}
	else
	{
		if (Child.OuterName.IsNone())
		{
			if (Child.PackageName.IsNone())
			{
				// An unexpected case; there should not be an objectname with no outer or packagename.
				// Treat this case is if the ObjectName were the packagename
				return GetParent(FCoreRedirectObjectName(NAME_None, NAME_None, Child.ObjectName));
			}
			else
			{
				// A toplevel object, parent is its package
				return FCoreRedirectObjectName(NAME_None, NAME_None, Child.PackageName);
			}
		}
		else
		{
			// A subobject, aka a child of an object that is not the package. Get the
			// FCoreRedirectObjectName of the outer by peeling the last dot-delimited name from the OuterName.
			TStringBuilder<NAME_SIZE> OuterNameBuffer;
			FStringView NewOuterNameStr;
			FStringView NewObjectName;
			SplitLastComponent(Child.OuterName, OuterNameBuffer, NewOuterNameStr, NewObjectName);
			if (NewObjectName.Len() == 0)
			{
				// Just a period by itself, treat this as no outer name.
				return GetParent(FCoreRedirectObjectName(Child.ObjectName, NAME_None, Child.PackageName));
			}
			else
			{
				FName NewOuterName = NewOuterNameStr.Len() > 0 ? FName(NewOuterNameStr) : NAME_None;
				return FCoreRedirectObjectName(FName(NewObjectName), NewOuterName, Child.PackageName);
			}
		}
	}
}

void FCoreRedirect::NormalizeNewName()
{
	// Fill in NewName as needed
	if (NewName.ObjectName == NAME_None)
	{
		NewName.ObjectName = OldName.ObjectName;
	}
	if (NewName.OuterName == NAME_None)
	{
		NewName.OuterName = OldName.OuterName;
	}
	if (NewName.PackageName == NAME_None)
	{
		NewName.PackageName = OldName.PackageName;
	}
}

const TCHAR* FCoreRedirect::ParseValueChanges(const TCHAR* Buffer)
{
	// If we export an empty array we export an empty string, so ensure that if we're passed an empty string
	// we interpret it as an empty array.
	if (*Buffer++ != TCHAR('('))
	{
		return nullptr;
	}

	SkipWhitespace(Buffer);
	if (*Buffer == TCHAR(')'))
	{
		return Buffer + 1;
	}

	for (;;)
	{
		SkipWhitespace(Buffer);
		if (*Buffer++ != TCHAR('('))
		{
			return nullptr;
		}

		// Parse the key and value
		FString KeyString, ValueString;
		Buffer = FPropertyHelpers::ReadToken(Buffer, KeyString, true);
		if (!Buffer)
		{
			return nullptr;
		}

		SkipWhitespace(Buffer);
		if (*Buffer++ != TCHAR(','))
		{
			return nullptr;
		}

		// Parse the value
		SkipWhitespace(Buffer);
		Buffer = FPropertyHelpers::ReadToken(Buffer, ValueString, true);
		if (!Buffer)
		{
			return nullptr;
		}

		SkipWhitespace(Buffer);
		if (*Buffer++ != TCHAR(')'))
		{
			return nullptr;
		}

		ValueChanges.Add(KeyString, ValueString);

		switch (*Buffer++)
		{
		case TCHAR(')'):
			return Buffer;

		case TCHAR(','):
			break;

		default:
			return nullptr;
		}
	}
}

static bool CheckRedirectFlagsMatch(ECoreRedirectFlags FlagsA, ECoreRedirectFlags FlagsB)
{
	// For type, check it includes the matching type
	const bool bTypesOverlap = !!((FlagsA & FlagsB) & ECoreRedirectFlags::Type_AllMask);

	// For category, the bits must be an exact match
	const bool bCategoriesMatch = (FlagsA & ECoreRedirectFlags::Category_AllMask) == (FlagsB & ECoreRedirectFlags::Category_AllMask);

	// Options are not considered in this function; Flags will match at this point regardless of their bits in Options_AllMask
	return bTypesOverlap && bCategoriesMatch;
}

bool FCoreRedirect::Matches(ECoreRedirectFlags InFlags, const FCoreRedirectObjectName& InName,
	ECoreRedirectMatchFlags MatchFlags) const
{
	// Check flags for Type/Category match
	if (!CheckRedirectFlagsMatch(InFlags, RedirectFlags))
	{
		return false;
	}

	if (EnumHasAnyFlags(InFlags, ECoreRedirectFlags::Type_Asset))
	{
		// Type_Asset matches should always be exact. They will either exactly match a package entry or an object entry
		MatchFlags |= ECoreRedirectMatchFlags::DisallowPartialLHSMatch;
	}

	return Matches(InName, MatchFlags);
}

bool FCoreRedirect::Matches(const FCoreRedirectObjectName& InName, ECoreRedirectMatchFlags MatchFlags) const
{
	FCoreRedirectObjectName::EMatchFlags NameMatchFlags = FCoreRedirectObjectName::EMatchFlags::None;

	if (EnumHasAllFlags(MatchFlags, ECoreRedirectMatchFlags::AllowPartialMatch))
	{
		NameMatchFlags |= FCoreRedirectObjectName::EMatchFlags::AllowPartialRHSMatch;
	}

	if (EnumHasAllFlags(MatchFlags, ECoreRedirectMatchFlags::DisallowPartialLHSMatch))
	{
		NameMatchFlags |= FCoreRedirectObjectName::EMatchFlags::DisallowPartialLHSMatch;
	}
	if (IsSubstringMatch())
	{
		NameMatchFlags |= FCoreRedirectObjectName::EMatchFlags::CheckSubString;
	}
	else if (IsPrefixMatch())
	{
		NameMatchFlags |= FCoreRedirectObjectName::EMatchFlags::CheckPrefix;
	}
	else if (IsSuffixMatch())
	{
		NameMatchFlags |= FCoreRedirectObjectName::EMatchFlags::CheckSuffix;
	}

	return OldName.Matches(InName, NameMatchFlags);
}

bool FCoreRedirect::HasValueChanges() const
{
	return ValueChanges.Num() > 0;
}

bool FCoreRedirect::IsSubstringMatch() const
{
	return EnumHasAllFlags(RedirectFlags, ECoreRedirectFlags::Option_MatchSubstring);
}

FCoreRedirectObjectName FCoreRedirect::RedirectName(const FCoreRedirectObjectName& OldObjectName, bool bIsKnownToMatch) const
{
	using namespace UE::CoreRedirects::Private;

	const bool bIsSubstringMatch = IsSubstringMatch();
	const bool bIsPrefixMatch = IsPrefixMatch();
	const bool bIsSuffixMatch = IsSuffixMatch();

	auto ConvertNames = [bIsSubstringMatch, bIsPrefixMatch, bIsSuffixMatch, bIsKnownToMatch](FName CurrentName,
		FName RedirectOldName, FName RedirectNewName)
		{
			if ((RedirectOldName != RedirectNewName) && !CurrentName.IsNone())
			{
				if (RedirectOldName.IsNone())
				{
					return RedirectNewName;
				}

				if (bIsSubstringMatch)
				{
					FNameUtf8String OutName(CurrentName);
					FNameUtf8String Substring(RedirectOldName);
					FNameUtf8String ReplacementName(RedirectNewName);
					int ReplacePos = UE::String::FindFirst(OutName, Substring, ESearchCase::IgnoreCase);

					if (!bIsKnownToMatch && ReplacePos == INDEX_NONE)
					{
						return CurrentName;
					}
					check(ReplacePos != INDEX_NONE);

					OutName.ReplaceAt(ReplacePos, Substring.Len(), ReplacementName);
					return FName(OutName);
				}
				else if (bIsPrefixMatch)
				{
					FNameUtf8String OutName(CurrentName);
					FNameUtf8String Prefix(RedirectOldName);
					FNameUtf8String ReplacementName(RedirectNewName);

					if (!bIsKnownToMatch && !OutName.StartsWith(Prefix, ESearchCase::IgnoreCase))
					{
						return CurrentName;
					}
					// bKnownToMatch implies FPrefixMatcher::Matches(CurrentName, RedirectOldName), 
					// so OutName.StartsWith(Prefix) is true, so we skip the stringcompare
					check(OutName.Len() >= Prefix.Len());

					OutName.ReplaceAt(0, Prefix.Len(), ReplacementName);
					return FName(OutName);
				}
				else if (bIsSuffixMatch)
				{
					FNameUtf8String OutName(CurrentName);
					FNameUtf8String Suffix(RedirectOldName);
					FNameUtf8String ReplacementName(RedirectNewName);

					if (!bIsKnownToMatch && !OutName.EndsWith(Suffix, ESearchCase::IgnoreCase))
					{
						return CurrentName;
					}
					// bKnownToMatch implies FSuffixMatcher::Matches(CurrentName, RedirectOldName), 
					// so OutName.EndsWith(Suffix) is true, so we skip the stringcompare
					check(OutName.Len() >= Suffix.Len());

					OutName.ReplaceAt(OutName.Len() - Suffix.Len(), Suffix.Len(), ReplacementName);
					return FName(OutName);
				}
				else
				{
					return RedirectNewName;
				}
			}
			return CurrentName;
		};

	// Convert names that are different and non empty
	FCoreRedirectObjectName ModifyName(OldObjectName);
	ModifyName.ObjectName = ConvertNames(OldObjectName.ObjectName, OldName.ObjectName, NewName.ObjectName);

	if (OldName.OuterName == NewName.OuterName)
	{
		// If package name and object name are specified, overwrite outer also since it was set to null explicitly
		if (OldName.OuterName.IsNone() && !NewName.PackageName.IsNone() && !NewName.ObjectName.IsNone() && !ModifyName.OuterName.IsNone())
		{
			ModifyName.OuterName = NewName.OuterName;
		}
	}
	else
	{
		ModifyName.OuterName = ConvertNames(OldObjectName.OuterName, OldName.OuterName, NewName.OuterName);
	}
	
	ModifyName.PackageName = ConvertNames(OldObjectName.PackageName, OldName.PackageName, NewName.PackageName);

	return ModifyName;
}

FCoreRedirectObjectName FCoreRedirect::RedirectName(const FCoreRedirectObjectName& OldObjectName) const
{
	return RedirectName(OldObjectName, false /* bIsKnownToMatch */);
}

bool FCoreRedirect::IdenticalMatchRules(const FCoreRedirect& Other) const
{
	// All types now use the full path
	return RedirectFlags == Other.RedirectFlags && OldName == Other.OldName;
}

void FCoreRedirect::AppendHash(FBlake3& Hasher) const
{
	Hasher.Update(&RedirectFlags, sizeof(RedirectFlags));
	OldName.AppendHash(Hasher);
	NewName.AppendHash(Hasher);
	OverrideClassName.AppendHash(Hasher);
	TArray<TPair<FString, FString>> ValueArray = ValueChanges.Array();
	Algo::Sort(ValueArray);
	for (const TPair<FString, FString>& Pair : ValueArray)
	{
		Hasher.Update(*Pair.Key, Pair.Key.Len() * sizeof((*Pair.Key)[0]));
		Hasher.Update(*Pair.Value, Pair.Value.Len() * sizeof((*Pair.Value)[0]));
	}
}

int FCoreRedirect::Compare(const FCoreRedirect& Other) const
{
	if (RedirectFlags != Other.RedirectFlags)
	{
		return RedirectFlags < Other.RedirectFlags ? -1 : 1;
	}
	int Compare = OldName.Compare(Other.OldName);
	if (Compare != 0)
	{
		return Compare;
	}
	Compare = NewName.Compare(Other.NewName);
	if (Compare != 0)
	{
		return Compare;
	}
	Compare = OverrideClassName.Compare(Other.OverrideClassName);
	if (Compare != 0)
	{
		return Compare;
	}
	Compare = Algo::CompareMap(ValueChanges, Other.ValueChanges);
	if (Compare != 0)
	{
		return Compare;
	}

	return 0;
}

int32 FCoreRedirect::GetTypeHash() const
{
	uint32 Hashes[] = {
		GetTypeHashHelper(RedirectFlags),
		GetTypeHashHelper(OldName),
		GetTypeHashHelper(NewName),
		GetTypeHashHelper(OverrideClassName),
	};

	uint32 SmooshedHash = 0;

	for (const uint32 Hash : Hashes)
	{
		SmooshedHash = HashCombineFast(SmooshedHash, Hash);
	}

	TArray<TPair<FString, FString>> ValueArray = ValueChanges.Array();
	Algo::Sort(ValueArray);

	for (const TPair<FString, FString>& Pair : ValueArray)
	{
		SmooshedHash = HashCombineFast(SmooshedHash, GetTypeHashHelper(Pair.Key));
		SmooshedHash = HashCombineFast(SmooshedHash, GetTypeHashHelper(Pair.Value));
	}

	return SmooshedHash;
}

void FCoreRedirectObjectName::AppendHash(FBlake3& Hasher) const
{
	FNameBuilder NameStr;
	int32 Marker = 0xabacadab;
	if (!PackageName.IsNone())
	{
		NameStr << PackageName;
		Hasher.Update(NameStr.GetData(), NameStr.Len() * sizeof(NameStr.GetData()[0]));
	}
	Hasher.Update(&Marker, sizeof(Marker));
	if (!OuterName.IsNone())
	{
		NameStr.Reset();
		NameStr << OuterName;
		Hasher.Update(NameStr.GetData(), NameStr.Len() * sizeof(NameStr.GetData()[0]));
	}
	Hasher.Update(&Marker, sizeof(Marker));
	if (!ObjectName.IsNone())
	{
		NameStr.Reset();
		NameStr << ObjectName;
		Hasher.Update(NameStr.GetData(), NameStr.Len() * sizeof(NameStr.GetData()[0]));
	}
	Hasher.Update(&Marker, sizeof(Marker));
}

int FCoreRedirectObjectName::Compare(const FCoreRedirectObjectName& Other) const
{
	if (PackageName != Other.PackageName)
	{
		return PackageName.Compare(Other.PackageName);
	}
	if (OuterName != Other.OuterName)
	{
		return OuterName.Compare(Other.OuterName);
	}
	if (ObjectName != Other.ObjectName)
	{
		return ObjectName.Compare(Other.ObjectName);
	}
	return 0;
}

FCoreRedirects::FRedirectTypeMap::FRedirectTypeMap(const FRedirectTypeMap& Other)
{
	FastIterable.Append(Other.FastIterable);

	Map.Reserve(FastIterable.Num());
	for (TPair<ECoreRedirectFlags, FRedirectNameMap>& Pair : FastIterable)
	{
		FRedirectNameMap& RedirectNameMap = Pair.Value;
		Map.Add(Pair.Key, &RedirectNameMap);

		if (RedirectNameMap.Wildcards)
		{
			FWildcardData* WildCardCopy = new FWildcardData();
			*WildCardCopy = *RedirectNameMap.Wildcards;
			RedirectNameMap.Wildcards = MakeShareable(WildCardCopy);
		}
	}
}

FCoreRedirects::FRedirectTypeMap& FCoreRedirects::FRedirectTypeMap::operator=(const FCoreRedirects::FRedirectTypeMap& Other)
{
	FastIterable.Empty(Other.FastIterable.Num());
	FastIterable.Append(Other.FastIterable);

	Map.Empty(FastIterable.Num());
	for (TPair<ECoreRedirectFlags, FRedirectNameMap>& Pair : FastIterable)
	{
		FRedirectNameMap& RedirectNameMap = Pair.Value;
		Map.Add(Pair.Key, &RedirectNameMap);

		if (RedirectNameMap.Wildcards)
		{
			FWildcardData* WildCardCopy = new FWildcardData();
			*WildCardCopy = *RedirectNameMap.Wildcards;
			RedirectNameMap.Wildcards = MakeShareable(WildCardCopy);
		}
	}
	return *this;
}

FCoreRedirects::FRedirectNameMap& FCoreRedirects::FRedirectTypeMap::FindOrAdd(ECoreRedirectFlags Key)
{
	FRedirectNameMap*& NameMap = Map.FindOrAdd(Key);
	if (NameMap)
	{
		return *NameMap;
	}

	TPair<ECoreRedirectFlags, FRedirectNameMap>* OldData = FastIterable.GetData();
	TPair<ECoreRedirectFlags, FRedirectNameMap>& NewPair = FastIterable.Emplace_GetRef(Key, FRedirectNameMap());

	// Check to ensure FastIterable did not reallocate after emplacement
	if (FastIterable.GetData() == OldData)
	{
		NameMap = &NewPair.Value;
	}
	else
	{
		Map.Reset();
		for (TPair<ECoreRedirectFlags, FRedirectNameMap>& Pair : FastIterable)
		{
			Map.Add(Pair.Key, &Pair.Value);
		}
	}

	if (EnumHasAnyFlags(Key, ECoreRedirectFlags::Option_MatchWildcardMask))
	{
		NewPair.Value.Wildcards = MakeShareable<FWildcardData>(new FWildcardData());
	}
	
	return NewPair.Value;
}

FCoreRedirects::FRedirectNameMap* FCoreRedirects::FRedirectTypeMap::Find(ECoreRedirectFlags Key)
{
	return Map.FindRef(Key);
}

void FCoreRedirects::FRedirectTypeMap::Empty()
{
	Map.Empty();
	FastIterable.Empty();
}

void FCoreRedirects::Initialize()
{
	ensureMsgf(IsInGameThread(), TEXT("FCoreRedirects can only be initialized on the game thread."));

	FCoreRedirectsContext& Context = FCoreRedirectsContext::GetGlobalContext();
	Context.InitializeContext();

	// Enable to run startup tests
#if 0
	{
		FScopeCoreRedirectsContext ScopeContext;
		ensure(RunTests());
	}
#endif
}

bool FCoreRedirects::RedirectNameAndValues(ECoreRedirectFlags Type, const FCoreRedirectObjectName& OldObjectName,
	FCoreRedirectObjectName& NewObjectName, const FCoreRedirect** FoundValueRedirect, ECoreRedirectMatchFlags MatchFlags)
{
	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	return RedirectNameAndValuesUnderReadLock(Type, OldObjectName, NewObjectName, FoundValueRedirect, MatchFlags, LockedContext);
}

bool FCoreRedirects::RedirectNameAndValuesUnderReadLock(ECoreRedirectFlags Type, const FCoreRedirectObjectName& OldObjectName,
	FCoreRedirectObjectName& NewObjectName, const FCoreRedirect** FoundValueRedirect, ECoreRedirectMatchFlags MatchFlags,
	FScopeCoreRedirectsReadLockedContext& LockedContext)
{
	// Iteratively apply redirects until a fixed point or a non-iterative redirect is reached.
	FCoreRedirectObjectName ObjectName = OldObjectName;
	for (int NumIterations = 0;;++NumIterations)
	{
		bool bShouldIterate = false;
		auto ProcessRedirect = [&FoundValueRedirect, OldObjectName, &bShouldIterate](const FCoreRedirect* Redirect, const FCoreRedirectObjectName& NewObjectName)
		{
			if (FoundValueRedirect && (Redirect->HasValueChanges() || Redirect->OverrideClassName.IsValid()))
			{
				if (*FoundValueRedirect)
				{
					if ((*FoundValueRedirect)->ValueChanges.OrderIndependentCompareEqual(Redirect->ValueChanges) == false)
					{
						UE_LOGF(LogCoreRedirects, Error, "RedirectNameAndValues(%ls) found multiple conflicting value redirects, %ls and %ls!",
							*OldObjectName.ToString(), *(*FoundValueRedirect)->OldName.ToString(), *Redirect->OldName.ToString());
					}
				}
				else
				{
					// Set value redirects for processing outside
					*FoundValueRedirect = Redirect;
				}
			}

			if (!!(Redirect->RedirectFlags & ECoreRedirectFlags::Option_Iterate))
			{
				bShouldIterate = true;
			}

			return Redirect->RedirectName(NewObjectName, true /* bIsKnownToMatch */);
		};

		NewObjectName = ObjectName;
		TArray<const FCoreRedirect*> FoundRedirects;
		if (GetMatchingRedirectsUnderReadLock(Type, ObjectName, FoundRedirects, MatchFlags, LockedContext))
		{
			if (FoundRedirects.Num() > 1)
			{
				// Sort them based on match
				FoundRedirects.Sort([&ObjectName, MatchFlags](const FCoreRedirect& A, const FCoreRedirect& B)
					{ return A.OldName.MatchScore(ObjectName, A.RedirectFlags, MatchFlags) > B.OldName.MatchScore(ObjectName, B.RedirectFlags, MatchFlags); });
				NewObjectName = ProcessRedirect(FoundRedirects[0], NewObjectName);

				// Apply in order
				for (int32 i = 1; i < FoundRedirects.Num(); i++)
				{
					const FCoreRedirect* Redirect = FoundRedirects[i];

					// Only apply if name match is still valid, if it already renamed part of it it may not apply any more. 
					// Don't want to check flags as those were checked in the gather step
					if (Redirect->Matches(NewObjectName, MatchFlags))
					{
						NewObjectName = ProcessRedirect(Redirect, NewObjectName);
					}
				}
			}
			else
			{
				const FCoreRedirect* Redirect = FoundRedirects[0];
				NewObjectName = ProcessRedirect(Redirect, NewObjectName);
			}
		}

		if (NewObjectName != ObjectName)
		{
			UE_CLOGF(LockedContext.Get().IsInDebugMode(), LogCoreRedirects, Verbose,
				"RedirectNameAndValues(%ls) replaced by %ls", *ObjectName.ToString(), *NewObjectName.ToString());
			ObjectName = NewObjectName;
		}
		else
		{
			bShouldIterate = false;
		}

		// Limit the maximum number of iterations to ensure we don't get stuck in an infinite loop here if somebody accidentally created a redirect cycle.
		constexpr int MaxIterations = 10;
		if (!ensureMsgf(NumIterations < MaxIterations, TEXT("RedirectNameAndValues stopping after maximum number of iterations: %s -> %s"), *OldObjectName.ToString(), *NewObjectName.ToString()))
		{
			bShouldIterate = false;
		}

		if (!bShouldIterate)
		{
			return NewObjectName != OldObjectName;
		}
	};
}

bool FCoreRedirects::ValidateAssetRedirectsUnderReadLock(FScopeCoreRedirectsReadLockedContext& LockedContext)
{
	bool bValidationSucceeded = true;
	FRedirectNameMap* RedirectNameMap = LockedContext.Get().GetRedirectTypeMap().Find(ECoreRedirectFlags::Type_Asset);
	if (RedirectNameMap != nullptr)
	{
		for (const TPair<FName, TArray<FCoreRedirect>>& Pair : RedirectNameMap->RedirectMap)
		{
			// Pairs are package --> redirects because the search key for Type_Asset is the package name
			for (const FCoreRedirect& Redirect : Pair.Value)
			{
				const FCoreRedirectObjectName& SearchName = Redirect.NewName;

				if (!(Redirect.RedirectFlags & ECoreRedirectFlags::Option_Iterate))
				{
					TArray<const FCoreRedirect*> MatchingRedirects;
					GetMatchingRedirects(ECoreRedirectFlags::Type_Asset, SearchName, MatchingRedirects);
					for (const FCoreRedirect* MatchingRedirect : MatchingRedirects)
					{
						UE_LOGF(LogCoreRedirects, Warning, "Found redirect from existing redirect. Chained redirects will not be followed. %ls --> %ls --> %ls",
							*Redirect.OldName.ToString(), *SearchName.ToString(), *MatchingRedirect->NewName.ToString());
						bValidationSucceeded = false;
					}
				}
			}
		}
	}

	return bValidationSucceeded;
}

FCoreRedirectObjectName FCoreRedirects::GetRedirectedName(ECoreRedirectFlags Type,
	const FCoreRedirectObjectName& OldObjectName, ECoreRedirectMatchFlags MatchFlags)
{
	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	FCoreRedirectObjectName NewObjectName;

	RedirectNameAndValuesUnderReadLock(Type, OldObjectName, NewObjectName, nullptr, MatchFlags, LockedContext);

	return NewObjectName;
}

const TMap<FString, FString>* FCoreRedirects::GetValueRedirects(ECoreRedirectFlags Type,
	const FCoreRedirectObjectName& OldObjectName, ECoreRedirectMatchFlags MatchFlags)
{
	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	FCoreRedirectObjectName NewObjectName;
	const FCoreRedirect* FoundRedirect = nullptr;

	RedirectNameAndValuesUnderReadLock(Type, OldObjectName, NewObjectName, &FoundRedirect, MatchFlags, LockedContext);

	if (FoundRedirect && FoundRedirect->ValueChanges.Num() > 0)
	{
		UE_CLOGF(LockedContext.Get().IsInDebugMode(), LogCoreRedirects, VeryVerbose, "GetValueRedirects found %d matches for %ls",
			FoundRedirect->ValueChanges.Num(), *OldObjectName.ToString());

		return &FoundRedirect->ValueChanges;
	}

	return nullptr;
}

bool FCoreRedirects::GetMatchingRedirects(ECoreRedirectFlags SearchFlags, const FCoreRedirectObjectName& OldObjectName,
	TArray<const FCoreRedirect*>& FoundRedirects, ECoreRedirectMatchFlags MatchFlags)
{
	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	return GetMatchingRedirectsUnderReadLock(SearchFlags, OldObjectName, FoundRedirects, MatchFlags, LockedContext);
}

bool FCoreRedirects::GetMatchingRedirectsUnderReadLock(ECoreRedirectFlags SearchFlags, const FCoreRedirectObjectName& OldObjectName,
	TArray<const FCoreRedirect*>& FoundRedirects, ECoreRedirectMatchFlags MatchFlags, FScopeCoreRedirectsReadLockedContext& LockedContext)
{
	// Look for all redirects that match the given names and flags
	bool bFound = false;

	// We always search Type_Asset as well as whatever is requested
	// That is because asset redirectors can redirect packages (implicitly) and any UObject type (explicitly)
	SearchFlags |= ECoreRedirectFlags::Type_Asset;

	// If we're not explicitly searching for packages, and not looking for removed things, and not searching for partial matches
	// based on ObjectName only, add the implicit (Type=Package,Category=None) redirects
	const bool bSearchPackageRedirects = !(SearchFlags & ECoreRedirectFlags::Type_Package) &&
		!(SearchFlags & ECoreRedirectFlags::Category_Removed) &&
		(!(MatchFlags & ECoreRedirectMatchFlags::AllowPartialMatch) || !OldObjectName.PackageName.IsNone());

	// Determine list of maps to look over, need to handle being passed multiple types in a bit mask
	for (const TPair<ECoreRedirectFlags, FRedirectNameMap>& Pair : LockedContext.Get().GetRedirectTypeMap())
	{
		ECoreRedirectFlags PairFlags = Pair.Key;

		// We need to check all maps that match the search or package flags
		if (CheckRedirectFlagsMatch(PairFlags, SearchFlags) || 
			(bSearchPackageRedirects && CheckRedirectFlagsMatch(PairFlags, ECoreRedirectFlags::Type_Package)))
		{
			if (EnumHasAnyFlags(PairFlags, ECoreRedirectFlags::Option_MatchWildcardMask))
			{
				const FWildcardData* Wildcards = Pair.Value.Wildcards.Get();
				check(Wildcards);
				bFound |= Wildcards->Matches(PairFlags, OldObjectName, MatchFlags, FoundRedirects);
			}
			else
			{
				const TArray<FCoreRedirect>* RedirectsForName = Pair.Value.RedirectMap.Find(OldObjectName.GetSearchKey(PairFlags));
				if (RedirectsForName)
				{
					for (const FCoreRedirect& CheckRedirect : *RedirectsForName)
					{
						if (CheckRedirect.Matches(PairFlags, OldObjectName, MatchFlags))
						{
							bFound = true;
							FoundRedirects.Add(&CheckRedirect);
						}
					}
				}
			}
		}
	}

	return bFound;
}

bool FCoreRedirects::FindPreviousNames(ECoreRedirectFlags SearchFlags, const FCoreRedirectObjectName& NewObjectName, TArray<FCoreRedirectObjectName>& PreviousNames)
{
	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());

	// Look for reverse direction redirects
	bool bFound = false;

	// If we're not explicitly searching for packages or looking for removed things, add the implicit (Type=Package,Category=None) redirects
	const bool bSearchPackageRedirects = !(SearchFlags & ECoreRedirectFlags::Type_Package) && !(SearchFlags & ECoreRedirectFlags::Category_Removed);

	// We always search Type_Asset as well as whatever is requested
	// That is because asset redirectors can redirect packages (implicitly) and any UObject type (explicitly)
	SearchFlags |= ECoreRedirectFlags::Type_Asset;

	auto TryReverseRedirect = [](const FCoreRedirect& Redirect, const FCoreRedirectObjectName& NewObjectName, TArray<FCoreRedirectObjectName>& PreviousNames)
		{
			FCoreRedirect ReverseRedirect = FCoreRedirect(Redirect);
			ReverseRedirect.OldName = Redirect.NewName;
			ReverseRedirect.NewName = Redirect.OldName;

			FCoreRedirectObjectName OldName = ReverseRedirect.RedirectName(NewObjectName, true /* bIsKnownToMatch */);

			if (OldName != NewObjectName)
			{
				PreviousNames.AddUnique(OldName);
				return true;
			}
			return false;
		};

	// Determine list of maps to look over, need to handle being passed multiple Flags in a bit mask
	for (const TPair<ECoreRedirectFlags, FRedirectNameMap>& Pair : LockedContext.Get().GetRedirectTypeMap())
	{
		ECoreRedirectFlags PairFlags = Pair.Key;

		// We need to check all maps that match the search or package flags
		if (CheckRedirectFlagsMatch(PairFlags, SearchFlags) || (bSearchPackageRedirects && CheckRedirectFlagsMatch(PairFlags, ECoreRedirectFlags::Type_Package)))
		{
			if (EnumHasAnyFlags(PairFlags, ECoreRedirectFlags::Option_MatchWildcardMask))
			{
				const FWildcardData* Wildcards = Pair.Value.Wildcards.Get();
				check(Wildcards);

				FCoreRedirectObjectName::EMatchFlags MatchFlags = FCoreRedirectObjectName::EMatchFlags::None;
				const TArray<FCoreRedirect>* WildcardRedirects = nullptr;

				if (EnumHasAllFlags(PairFlags, ECoreRedirectFlags::Option_MatchSubstring))
				{
					WildcardRedirects = &Wildcards->Substrings;
					MatchFlags |= FCoreRedirectObjectName::EMatchFlags::CheckSubString;
				}
				else if (EnumHasAllFlags(PairFlags, ECoreRedirectFlags::Option_MatchPrefix))
				{
					WildcardRedirects = &Wildcards->Prefixes;
					MatchFlags |= FCoreRedirectObjectName::EMatchFlags::CheckPrefix;
				}
				else if (EnumHasAllFlags(PairFlags, ECoreRedirectFlags::Option_MatchSuffix))
				{
					WildcardRedirects = &Wildcards->Suffixes;
					MatchFlags |= FCoreRedirectObjectName::EMatchFlags::CheckSuffix;
				}
				check(WildcardRedirects);

				for (const FCoreRedirect& Redirect : *WildcardRedirects)
				{
					if (Redirect.NewName.Matches(NewObjectName, MatchFlags))
					{
						bFound |= TryReverseRedirect(Redirect, NewObjectName, PreviousNames);
					}
				}
			}
			else
			{
				FCoreRedirectObjectName::EMatchFlags MatchFlags = 
					EnumHasAnyFlags(PairFlags, ECoreRedirectFlags::Type_Asset) 
						? FCoreRedirectObjectName::EMatchFlags::AllowPartialRHSMatch : FCoreRedirectObjectName::EMatchFlags::None;
				for (const TPair<FName, TArray<FCoreRedirect>>& RedirectPair : Pair.Value.RedirectMap)
				{
					for (const FCoreRedirect& Redirect : RedirectPair.Value)
					{
						if (Redirect.NewName.Matches(NewObjectName, MatchFlags))
						{
							bFound |= TryReverseRedirect(Redirect, NewObjectName, PreviousNames);
						}
					}
				}
			}
		}
	}

	UE_CLOGF(bFound && LockedContext.Get().IsInDebugMode(), LogCoreRedirects, VeryVerbose, "FindPreviousNames found %d previous names for %ls", PreviousNames.Num(), *NewObjectName.ToString());

	return bFound;
}

bool FCoreRedirects::IsKnownMissing(ECoreRedirectFlags Type, const FCoreRedirectObjectName& ObjectName)
{
	TArray<const FCoreRedirect*> FoundRedirects;
	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	return FCoreRedirects::GetMatchingRedirectsUnderReadLock(Type | ECoreRedirectFlags::Category_Removed, ObjectName, FoundRedirects, ECoreRedirectMatchFlags::None, LockedContext);
}

bool FCoreRedirects::AddKnownMissing(ECoreRedirectFlags Type, const FCoreRedirectObjectName& ObjectName, ECoreRedirectFlags Channel)
{
	ensureMsgf(IsInitialized(), TEXT("FCoreRedirects must be initialized on the game thread before use."));

	check((Channel & ~ECoreRedirectFlags::Option_MissingLoad) == ECoreRedirectFlags::None);
	FCoreRedirect NewRedirect(Type | ECoreRedirectFlags::Category_Removed | Channel, ObjectName, FCoreRedirectObjectName());

	return AddRedirectList(TArrayView<const FCoreRedirect>(&NewRedirect, 1), TEXT("AddKnownMissing"));
}

bool FCoreRedirects::RemoveKnownMissing(ECoreRedirectFlags Type, const FCoreRedirectObjectName& ObjectName, ECoreRedirectFlags Channel)
{
	check((Channel & ~ECoreRedirectFlags::Option_MissingLoad) == ECoreRedirectFlags::None);
	FCoreRedirect RedirectToRemove(Type | ECoreRedirectFlags::Category_Removed | Channel, ObjectName, FCoreRedirectObjectName());

	return RemoveRedirectList(TArrayView<const FCoreRedirect>(&RedirectToRemove, 1), TEXT("RemoveKnownMissing"));
}

void FCoreRedirects::ClearKnownMissing(ECoreRedirectFlags Type, ECoreRedirectFlags Channel)
{
	check((Channel & ~ECoreRedirectFlags::Option_MissingLoad) == ECoreRedirectFlags::None);
	ECoreRedirectFlags RedirectFlags = Type | ECoreRedirectFlags::Category_Removed | Channel;

	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	FRedirectNameMap* RedirectNameMap = LockedContext.Get().GetRedirectTypeMap().Find(RedirectFlags);
	if (RedirectNameMap)
	{
		RedirectNameMap->RedirectMap.Empty();
	}
}

#if WITH_EDITOR
void FCoreRedirects::GetHashOfRedirectsAffectingPackages(const TConstArrayView<FName>& PackageNames, TArray<FBlake3Hash>& Hashes)
{
	FCoreRedirectsContext& Context = FCoreRedirectsContext::GetThreadContext();
	if (!!(Context.GetFlags() & FCoreRedirectsContext::EFlags::UseRedirectionSummary))
	{
		Context.GetRedirectionSummary().GetHashAffectingPackages(PackageNames, Hashes);
	}
}

void FCoreRedirects::AppendHashOfGlobalRedirects(FBlake3& Hasher)
{
	FCoreRedirectsContext& Context = FCoreRedirectsContext::GetThreadContext();
	if (!!(Context.GetFlags() & FCoreRedirectsContext::EFlags::UseRedirectionSummary))
	{
		FRedirectionSummary& RedirectionSummary = Context.GetRedirectionSummary();
		RedirectionSummary.AppendHashGlobal(Hasher);

		static bool bDebugPrintGlobalCoreRedirects = FParse::Param(FCommandLine::Get(), TEXT("DebugPrintGlobalCoreRedirects"));
		if (bDebugPrintGlobalCoreRedirects)
		{
			RedirectionSummary.DebugPrintGlobalCoreRedirects();
		}
	}
}

void FCoreRedirects::GetObjectRedirectHashAffectingPackage(const FName& PackageName, FBlake3& Hasher)
{
	FCoreRedirectsContext& Context = FCoreRedirectsContext::GetThreadContext();
	if (!!(Context.GetFlags() & FCoreRedirectsContext::EFlags::UseRedirectionSummary))
	{
		FRedirectionSummary& RedirectionSummary = Context.GetRedirectionSummary();
		RedirectionSummary.GetObjectRedirectHashAffectingPackage(PackageName, Hasher);
	}
}

void FCoreRedirects::RecordAddedObjectRedirector(const FSoftObjectPath& Source, const FSoftObjectPath& Dest)
{
	FCoreRedirectsContext& Context = FCoreRedirectsContext::GetThreadContext();
	if (!!(Context.GetFlags() & FCoreRedirectsContext::EFlags::UseRedirectionSummary))
	{
		FCoreRedirect ConvertedToCoreRedirect(ECoreRedirectFlags::Type_Object,
			FCoreRedirectObjectName(Source), FCoreRedirectObjectName(Dest));
		Context.GetRedirectionSummary().AddObjectRedirector(ConvertedToCoreRedirect);
	}
}

void FCoreRedirects::RecordRemovedObjectRedirector(const FSoftObjectPath& Source, const FSoftObjectPath& Dest)
{
	FCoreRedirectsContext& Context = FCoreRedirectsContext::GetThreadContext();
	if (!!(Context.GetFlags() & FCoreRedirectsContext::EFlags::UseRedirectionSummary))
	{
		FCoreRedirect ConvertedToCoreRedirect(ECoreRedirectFlags::Type_Object,
			FCoreRedirectObjectName(Source), FCoreRedirectObjectName(Dest));
		Context.GetRedirectionSummary().RemoveObjectRedirector(ConvertedToCoreRedirect);
	}
}

FCoreRedirects::FGlobalCoreRedirectAddedEvent* FCoreRedirects::GetOnGlobalCoreRedirectAdded()
{
	FCoreRedirectsContext& Context = FCoreRedirectsContext::GetThreadContext();
	if (!!(Context.GetFlags() & FCoreRedirectsContext::EFlags::UseRedirectionSummary))
	{
		return &Context.GetRedirectionSummary().GetOnGlobalCoreRedirectAdded();
	}
	return nullptr;
}

#endif // #if WITH_EDITOR

bool FCoreRedirects::RunTests()
{
	bool bSuccess = true;

	UE_LOGF(LogCoreRedirects, Log, "Running FCoreRedirect Tests");

	TArray<FCoreRedirect> NewRedirects;

	NewRedirects.Emplace(ECoreRedirectFlags::Type_Property, TEXT("Property"), TEXT("Property2"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Property, TEXT("Class.Property"), TEXT("Property3"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Property, TEXT("/Game/PackageSpecific.Class.Property"), TEXT("Property4"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Property, TEXT("/Game/Package.Class.OtherProperty"), TEXT("/Game/Package.Class.OtherProperty2"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Class, TEXT("Class"), TEXT("Class2"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Class, TEXT("/Game/Package.Class"), TEXT("Class3"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, TEXT("/Game/Package.Class"), TEXT("/Game/Package.ClassInstance"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package, TEXT("/Game/Package"), TEXT("/Game/Package2"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchSubstring, TEXT("/oldgame"), TEXT("/newgame"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchSubstring, TEXT("/古いゲーム"), TEXT("/新しいゲーム"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchSubstring, TEXT("/混合部分文字列"), TEXT("/mixed_substring"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchPrefix, TEXT("/oldprefix"), TEXT("/newprefix"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchPrefix, TEXT("/古いプレフィックス"), TEXT("/新しいプレフィックス"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchPrefix, TEXT("/混合接頭辞"), TEXT("/mixed_prefix"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Object  | ECoreRedirectFlags::Option_MatchPrefix, TEXT("/old/object.prefix."), TEXT("/new/superobject.prefix2."));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchPrefix, TEXT("/PrefixOverlappingDirectMatch"), TEXT("/ShouldNeverHappen"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package, TEXT("/PrefixOverlappingDirectMatch/Path"), TEXT("/DirectMatchIsPreferred/NewPath"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchSuffix, TEXT("/oldsuffix"), TEXT("/newsuffix"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchSuffix, TEXT("/古い接尾辞"), TEXT("/新しい接尾辞"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchSuffix, TEXT("/混合接尾辞"), TEXT("/mixed_suffix"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Category_Removed, TEXT("/Game/RemovedPackage"), TEXT("/Game/RemovedPackage"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Category_Removed | ECoreRedirectFlags::Option_MissingLoad, TEXT("/Game/MissingLoadPackage"), TEXT("/Game/MissingLoadPackage"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchPrefix | ECoreRedirectFlags::Option_Iterate, TEXT("/oldoldprefix"), TEXT("/oldprefix"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchPrefix | ECoreRedirectFlags::Option_Iterate, TEXT("/oldoldoldprefix"), TEXT("/oldoldprefix"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchPrefix, TEXT("/noiterate_oldprefix"), TEXT("/oldprefix"));

	// Test ordering of package redirects and other redirects to ensure the package redirect is applied last so we can still redirect non-package names that might be using the old package name
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package,	TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectFirst"),			TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectFirst_New"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Struct,	TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectFirst.OldStruct1"),	TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectFirst_New.NewStruct1"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Struct, TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectLast.OldStruct1"), TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectLast_New.NewStruct1"));
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package, TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectLast"), TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectLast_New"));

	FCoreRedirect EnumRedirectWithValues(ECoreRedirectFlags::Type_Enum, TEXT("MaskTarget_PhysMesh"), TEXT("/Script/ClothingSystemRuntimeCommon.EWeightMapTargetCommon"));
	EnumRedirectWithValues.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::MaxDistance"), TEXT("EWeightMapTargetCommon::MaxDistance"));
	EnumRedirectWithValues.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::BackstopDistance"), TEXT("EWeightMapTargetCommon::BackstopDistance"));
	EnumRedirectWithValues.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::BackstopRadius"), TEXT("EWeightMapTargetCommon::BackstopRadius"));
	EnumRedirectWithValues.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::AnimDriveMultiplier"), TEXT("EWeightMapTargetCommon::AnimDriveStiffness"));
	NewRedirects.Add(EnumRedirectWithValues);

	// This value redirect extends the values from the EnumRedirectWithValues redirect above
	FCoreRedirect EnumValueRedirect(ECoreRedirectFlags::Type_Enum, TEXT("/Script/ClothingSystemRuntimeCommon.EWeightMapTargetCommon"), TEXT(""));
	EnumValueRedirect.ValueChanges.Add(TEXT("EWeightMapTargetCommon::AnimDriveMultiplier"), TEXT("EWeightMapTargetCommon::AnimDriveStiffness"));
	NewRedirects.Add(EnumValueRedirect);

	AddRedirectList(NewRedirects, TEXT("RunTests"));

	// Run the asset tests first so that their entries are present when we run the other tests
	// That way we can confirm that having asset type redirects doesn't break existing tests
	bSuccess = bSuccess && UE::CoreRedirects::Private::RunAssetRedirectTests();

	struct FRedirectTest
	{
		FCoreRedirect Expected;

		FRedirectTest(const FString& InOldName, const FString& InNewName, ECoreRedirectFlags InType)
			: Expected(InType, InOldName, InNewName)
		{}
	};

	TArray<FRedirectTest> Tests;

	// Package-specific property rename and package rename apply
	Tests.Emplace(TEXT("/Game/PackageSpecific.Class:Property"), TEXT("/Game/PackageSpecific.Class:Property4"), ECoreRedirectFlags::Type_Property);
	// Verify . works as well
	Tests.Emplace(TEXT("/Game/PackageSpecific.Class.Property"), TEXT("/Game/PackageSpecific.Class:Property4"), ECoreRedirectFlags::Type_Property);
	// Wrong type, no replacement
	Tests.Emplace(TEXT("/Game/PackageSpecific.Class:Property"), TEXT("/Game/PackageSpecific.Class:Property"), ECoreRedirectFlags::Type_Function);
	// Class-specific property rename and package rename apply
	Tests.Emplace(TEXT("/Game/Package.Class:Property"), TEXT("/Game/Package2.Class:Property3"), ECoreRedirectFlags::Type_Property);
	// Package-Specific class rename applies
	Tests.Emplace(TEXT("/Game/Package.Class"), TEXT("/Game/Package2.Class3"), ECoreRedirectFlags::Type_Class);
	// Generic class rename applies
	Tests.Emplace(TEXT("/Game/PackageOther.Class"), TEXT("/Game/PackageOther.Class2"), ECoreRedirectFlags::Type_Class);
	// Check instance option
	Tests.Emplace(TEXT("/Game/Package.Class"), TEXT("/Game/Package2.ClassInstance"), ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly);

	// String manipulation tests. While we use TCHAR for input, FNames will coerce storage to ASCII where possible 
	// so we validate we can support truly wide strings as well as mixed string manipulation operations.

	// Substring tests
	Tests.Emplace(TEXT("/oldgame/Package.DefaultClass"), TEXT("/newgame/Package.DefaultClass"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/古いゲーム/Package.DefaultClass"), TEXT("/新しいゲーム/Package.DefaultClass"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/混合部分文字列/Package.DefaultClass"), TEXT("/mixed_substring/Package.DefaultClass"), ECoreRedirectFlags::Type_Package);

	// Prefix tests
	Tests.Emplace(TEXT("/oldprefix_SomeGame/Package.DefaultClass"), TEXT("/newprefix_SomeGame/Package.DefaultClass"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/古いプレフィックス_SomeGame/Package.DefaultClass"), TEXT("/新しいプレフィックス_SomeGame/Package.DefaultClass"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/混合接頭辞_SomeGame/Package.DefaultClass"), TEXT("/mixed_prefix_SomeGame/Package.DefaultClass"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/old/object.prefix.subobjects.do.not.change"), TEXT("/new/superobject.prefix2.subobjects.do.not.change"), ECoreRedirectFlags::Type_Object);
	Tests.Emplace(TEXT("/PrefixOverlappingDirectMatch/Path.Remain.Unchanged"), TEXT("/DirectMatchIsPreferred/NewPath.Remain:Unchanged"), ECoreRedirectFlags::Type_Package);

	// Iteration tests
	Tests.Emplace(TEXT("/oldoldoldprefix_SomeGame/Package.DefaultClass"), TEXT("/newprefix_SomeGame/Package.DefaultClass"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/noiterate_oldprefix_SomeGame/Package.DefaultClass"), TEXT("/oldprefix_SomeGame/Package.DefaultClass"), ECoreRedirectFlags::Type_Package);

	// Suffix tests
	Tests.Emplace(TEXT("/Game/Package/oldsuffix"), TEXT("/Game/Package/newsuffix"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/Game/Package/古い接尾辞"), TEXT("/Game/Package/新しい接尾辞"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/Game/Package/混合接尾辞"), TEXT("/Game/Package/mixed_suffix"), ECoreRedirectFlags::Type_Package);

	// Redirect ordering tests
	Tests.Emplace(TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectFirst"), TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectFirst_New"), ECoreRedirectFlags::Type_Package);
	Tests.Emplace(TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectFirst.OldStruct1"), TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectFirst_New.NewStruct1"), ECoreRedirectFlags::Type_Struct);
	Tests.Emplace(TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectLast.OldStruct1"), TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectLast_New.NewStruct1"), ECoreRedirectFlags::Type_Struct);
	Tests.Emplace(TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectLast"), TEXT("/Script/PackageRedirectOrderingTest_PackageRedirectLast_New"), ECoreRedirectFlags::Type_Package);

	// Enum value tests
	FRedirectTest EnumValueTest(TEXT("/Script/ClothingSystemRuntimeCommon.MaskTarget_PhysMesh"), TEXT("/Script/ClothingSystemRuntimeCommon.EWeightMapTargetCommon"), ECoreRedirectFlags::Type_Enum);
	EnumValueTest.Expected.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::MaxDistance"), TEXT("EWeightMapTargetCommon::MaxDistance"));
	EnumValueTest.Expected.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::BackstopDistance"), TEXT("EWeightMapTargetCommon::BackstopDistance"));
	EnumValueTest.Expected.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::BackstopRadius"), TEXT("EWeightMapTargetCommon::BackstopRadius"));
	EnumValueTest.Expected.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::AnimDriveMultiplier"), TEXT("EWeightMapTargetCommon::AnimDriveStiffness"));
	Tests.Add(EnumValueTest);

	// Confirms we inherit all the mappings from the MaskTarget_PhysMesh -> EWeightMapTargetCommon mapping
	// plus the explicit addition to EWeightMapTargetCommon added via a separate value redirect
	FRedirectTest EnumInheritedAndAppendedValueTest(TEXT("/Script/ClothingSystemRuntimeCommon.EWeightMapTargetCommon"), TEXT("/Script/ClothingSystemRuntimeCommon.EWeightMapTargetCommon"), ECoreRedirectFlags::Type_Enum);
	EnumInheritedAndAppendedValueTest.Expected.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::MaxDistance"), TEXT("EWeightMapTargetCommon::MaxDistance"));
	EnumInheritedAndAppendedValueTest.Expected.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::BackstopDistance"), TEXT("EWeightMapTargetCommon::BackstopDistance"));
	EnumInheritedAndAppendedValueTest.Expected.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::BackstopRadius"), TEXT("EWeightMapTargetCommon::BackstopRadius"));
	EnumInheritedAndAppendedValueTest.Expected.ValueChanges.Add(TEXT("MaskTarget_PhysMesh::AnimDriveMultiplier"), TEXT("EWeightMapTargetCommon::AnimDriveStiffness"));
	EnumInheritedAndAppendedValueTest.Expected.ValueChanges.Add(TEXT("EWeightMapTargetCommon::AnimDriveMultiplier"), TEXT("EWeightMapTargetCommon::AnimDriveStiffness"));
	Tests.Add(EnumInheritedAndAppendedValueTest);

	for (FRedirectTest& Test : Tests)
	{
		FCoreRedirectObjectName ActualNewName;
		const FCoreRedirect* ValueRedirects = nullptr;
		RedirectNameAndValues(Test.Expected.RedirectFlags, Test.Expected.OldName, ActualNewName, &ValueRedirects);
		bool bTestSuccess = ActualNewName == Test.Expected.NewName;
		if (bTestSuccess && !Test.Expected.ValueChanges.IsEmpty())
		{
			bTestSuccess &= ValueRedirects != nullptr;
			if (ValueRedirects)
			{
				for (const TPair<FString, FString>& Pair : Test.Expected.ValueChanges)
				{
					const FString* ActualChangeValue = ValueRedirects->ValueChanges.Find(Pair.Key);
					bTestSuccess &= ActualChangeValue != nullptr;
					if (ActualChangeValue)
					{
						bTestSuccess &= Pair.Value == *ActualChangeValue;
					}
					if (!bTestSuccess)
					{
						break;
					}
				}
			}
		}
		if (!bTestSuccess)
		{
			bSuccess = false;
			TStringBuilder<256> ExpectedBuilder;
			TStringBuilder<256> ActualBuilder;
			auto PrintRedirect = [](FCoreRedirect& Redirect, TStringBuilderBase<TCHAR>& Builder)
				{
					Builder.Appendf(TEXT("%s -> %s"), *Redirect.OldName.ToString(), *Redirect.NewName.ToString());
					for (TPair<FString, FString>& Pair : Redirect.ValueChanges)
					{
						Builder.Appendf(TEXT("\n\t[%s -> %s]"), *Pair.Key, *Pair.Value);
					}
				};
			PrintRedirect(Test.Expected, ExpectedBuilder);
			FCoreRedirect ActualRedirect = ValueRedirects ? *ValueRedirects : FCoreRedirect(Test.Expected.RedirectFlags, Test.Expected.OldName, ActualNewName);
			PrintRedirect(ActualRedirect, ActualBuilder);
			UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed:\nExpected:\t%ls,\nActual:\t\t%ls", ExpectedBuilder.ToString(), ActualBuilder.ToString());
		}
	}

	// Check reverse lookup
	TArray<FCoreRedirectObjectName> OldNames;

	FindPreviousNames(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(TEXT("/Game/PackageOther.Class2")), OldNames);
	if (OldNames.Num() != 1 || OldNames[0].ToString() != TEXT("/Game/PackageOther.Class"))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: ReverseLookup (direct matching)!");
	}
	OldNames.Empty();

	// Check reverse lookup - substring
	FindPreviousNames(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/newgame/TestPackage")), OldNames);
	if (OldNames.Num() != 1 || OldNames[0].ToString() != TEXT("/oldgame/TestPackage"))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: ReverseLookup (substring matching)!");
	}
	OldNames.Empty();

	// Check reverse lookup - prefix
	FindPreviousNames(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/newprefix_SomeGame/TestPackage")), OldNames);
	if (OldNames.Num() != 1 || OldNames[0].ToString() != TEXT("/oldprefix_SomeGame/TestPackage"))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: ReverseLookup (prefix matching)!");
	}
	OldNames.Empty();

	// Check reverse lookup - suffix
	FindPreviousNames(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/TestGame/newsuffix")), OldNames);
	if (OldNames.Num() != 1 || OldNames[0].ToString() != TEXT("/TestGame/oldsuffix"))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: ReverseLookup (suffix matching)!");
	}
	OldNames.Empty();

	// ObjectNames have almost no restrictions but PackageNames are more restrictive
	if (FCoreRedirectObjectName(TEXT("/Foo/Foo.Foo:\nNew")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report Object redirects with \\n in the ObjectName as invalid");
	}
	if (FCoreRedirectObjectName(TEXT("/Foo/Foo.\nFoo:New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report Object redirects with \\n in the OuterName as invalid");
	}
	if (FCoreRedirectObjectName(TEXT("/Foo/\nFoo.Foo:New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report Object redirects with \\n in the PackageName as invalid");
	}
	if (FCoreRedirectObjectName(TEXT("/Foo/_Verse/Foo.Foo:\nNew")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report verse Object redirects with \\n in the ObjectName as invalid");
	}
	if (FCoreRedirectObjectName(TEXT("/Foo/_Verse/Foo.\nFoo:New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report verse Object redirects with \\n in the OuterName as invalid");
	}
	if (FCoreRedirectObjectName(TEXT("/Foo/_Verse/\nFoo.Foo:New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report verse Object redirects with \\n in the PackageName as invalid");
	}

	// Blueprints allow almost anything to be an ObjectName, but should complain about outers and package names with special characters
	if (!FCoreRedirectObjectName(TEXT("/Foo/Foo.Foo:$New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report Object redirects with $ in the ObjectName as valid");
	}
	if (FCoreRedirectObjectName(TEXT("/Foo/Foo.$Foo:New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report Object redirects with $ in the OuterName as invalid");
	}
	if (FCoreRedirectObjectName(TEXT("/Foo/$Foo.Foo:New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report Object redirects with $ in the PackageName as invalid");
	}
	// Verse is much more permissive and allows $ anywhere
	if (!FCoreRedirectObjectName(TEXT("/Foo/_Verse/Foo.Foo:$New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report verse Object redirects with $ in the ObjectName as valid");
	}
	if (!FCoreRedirectObjectName(TEXT("/Foo/_Verse/Foo.$Foo:New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report verse Object redirects with $ in the OuterName as valid");
	}
	if (!FCoreRedirectObjectName(TEXT("/Foo/_Verse/$Foo.Foo:New")).HasValidCharacters(ECoreRedirectFlags::Type_Object))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: HasValidCharacters should report verse Object redirects with $ in the PackageName as valid");
	}


	// Check removed
	if (!IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/RemovedPackage"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: /Game/RemovedPackage should be removed!");
	}

	if (!IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/MissingLoadPackage"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: /Game/MissingLoadPackage should be removed!");
	}

	if (IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedPackage"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: /Game/NotRemovedPackage should be removed!");
	}

	AddKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedMissingLoad")), ECoreRedirectFlags::Option_MissingLoad);

	if (!IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedMissingLoad"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: /Game/NotRemovedMissingLoad should be removed now!");
	}

	RemoveKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedMissingLoad")), ECoreRedirectFlags::None);

	if (!IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedMissingLoad"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: RemoveKnownMissing of /Game/NotRemovedMissingLoad but with bIsMissingLoad=false should not have removed the redirect!");
	}

	RemoveKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedMissingLoad")), ECoreRedirectFlags::Option_MissingLoad);

	if (IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedMissingLoad"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: /Game/NotRemovedMissingLoad should no longer be removed!");
	}

	AddKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedPackage")), ECoreRedirectFlags::None);

	if (!IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedPackage"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: /Game/NotRemovedPackage should be removed now!");
	}

	RemoveKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedPackage")), ECoreRedirectFlags::Option_MissingLoad);

	if (!IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedPackage"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: RemoveKnownMissing of /Game/NotRemovedPackage but with bIsMissingLoad=true should not have removed the redirect!");
	}

	RemoveKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedPackage")), ECoreRedirectFlags::None);

	if (IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(TEXT("/Game/NotRemovedPackage"))))
	{
		bSuccess = false;
		UE_LOGF(LogCoreRedirects, Error, "FCoreRedirect Test Failed: /Game/NotRemovedPackage should no longer be removed!");
	}

	UE_LOGF(LogCoreRedirects, Log, "FCoreRedirect Test %ls!", (bSuccess ? TEXT("Passed") : TEXT("Failed")));
	return bSuccess;
}

#if WITH_DEV_AUTOMATION_TESTS
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCoreRedirectObjectNameTest, "System.Core.Misc.CoreRedirects.FCoreRedirectObjectName", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FCoreRedirectObjectNameTest::RunTest(const FString& Parameters)
{
	FCoreRedirectObjectName Parent;
	FCoreRedirectObjectName Child;
	FCoreRedirectObjectName ExpectedChild;
	FCoreRedirectObjectName ExpectedParent;
	FName NamePackage("/Root/Package");
	FName NameA("A");
	FName NameB("B");
	FName NameC("C");
	FName NameD("D");
	FName NameADotB("A.B");
	FName NameADotBDotC("A.B.C");

	// AppendObjectName
	Parent = FCoreRedirectObjectName(NAME_None, NAME_None, NAME_None);
	ExpectedChild = FCoreRedirectObjectName(NAME_None, NAME_None, NamePackage);
	Child = FCoreRedirectObjectName::AppendObjectName(Parent, NamePackage);
	TestTrue(TEXT("AppendObjectName(Empty + Package)"), Child == ExpectedChild);

	Parent = FCoreRedirectObjectName(NAME_None, NAME_None, NamePackage);
	ExpectedChild = FCoreRedirectObjectName(NameA, NAME_None, NamePackage);
	Child = FCoreRedirectObjectName::AppendObjectName(Parent, NameA);
	TestTrue(TEXT("AppendObjectName(Package + A)"), Child == ExpectedChild);

	Parent = FCoreRedirectObjectName(NameA, NAME_None, NamePackage);
	ExpectedChild = FCoreRedirectObjectName(NameB, NameA, NamePackage);
	Child = FCoreRedirectObjectName::AppendObjectName(Parent, NameB);
	TestTrue(TEXT("AppendObjectName(Package.A + B)"), Child == ExpectedChild);

	Parent = FCoreRedirectObjectName(NameB, NameA, NamePackage);
	ExpectedChild = FCoreRedirectObjectName(NameC, NameADotB, NamePackage);
	Child = FCoreRedirectObjectName::AppendObjectName(Parent, NameC);
	TestTrue(TEXT("AppendObjectName(Package.A.B + C)"), Child == ExpectedChild);

	Parent = FCoreRedirectObjectName(NameC, NameADotB, NamePackage);
	ExpectedChild = FCoreRedirectObjectName(NameD, NameADotBDotC, NamePackage);
	Child = FCoreRedirectObjectName::AppendObjectName(Parent, NameD);
	TestTrue(TEXT("AppendObjectName(Package.A.B.C + D)"), Child == ExpectedChild);

	// Verify these edge cases do not crash, but we do not provide a contract for what they mean.
	// ObjectName with no PackageName
	Parent = FCoreRedirectObjectName(NameA, NAME_None, NAME_None);
	FCoreRedirectObjectName::AppendObjectName(Parent, NameB);

	// ObjectName and OuterName with no PackageName
	Parent = FCoreRedirectObjectName(NameB, NameA, NAME_None);
	FCoreRedirectObjectName::AppendObjectName(Parent, NameC);

	// OuterName with no ObjectName and no PackageName
	Parent = FCoreRedirectObjectName(NAME_None, NameA, NAME_None);
	FCoreRedirectObjectName::AppendObjectName(Parent, NameB);

	// OuterName and ObjectName with no PackageName
	Parent = FCoreRedirectObjectName(NameB, NameA, NAME_None);
	FCoreRedirectObjectName::AppendObjectName(Parent, NameC);

	// OuterName and PackageName with no ObjectName
	Parent = FCoreRedirectObjectName(NAME_None, NameA, NamePackage);
	FCoreRedirectObjectName::AppendObjectName(Parent, NameB);


	// GetParent
	Child = FCoreRedirectObjectName(NAME_None, NAME_None, NAME_None);
	ExpectedParent = FCoreRedirectObjectName(NAME_None, NAME_None, NAME_None);
	Parent = FCoreRedirectObjectName::GetParent(Child);
	TestTrue(TEXT("GetParent(Empty)"), Parent == ExpectedParent);

	Child = FCoreRedirectObjectName(NAME_None, NAME_None, NamePackage);
	ExpectedParent = FCoreRedirectObjectName(NAME_None, NAME_None, NAME_None);
	Parent = FCoreRedirectObjectName::GetParent(Child);
	TestTrue(TEXT("GetParent(Package)"), Parent == ExpectedParent);

	Child = FCoreRedirectObjectName(NameA, NAME_None, NamePackage);
	ExpectedParent = FCoreRedirectObjectName(NAME_None, NAME_None, NamePackage);
	Parent = FCoreRedirectObjectName::GetParent(Child);
	TestTrue(TEXT("GetParent(Package.A)"), Parent == ExpectedParent);

	Child = FCoreRedirectObjectName(NameC, NameADotB, NamePackage);
	ExpectedParent = FCoreRedirectObjectName(NameB, NameA, NamePackage);
	Parent = FCoreRedirectObjectName::GetParent(Child);
	TestTrue(TEXT("GetParent(Package.A.B)"), Parent == ExpectedParent);

	Child = FCoreRedirectObjectName(NameD, NameADotBDotC, NamePackage);
	ExpectedParent = FCoreRedirectObjectName(NameC, NameADotB, NamePackage);
	Parent = FCoreRedirectObjectName::GetParent(Child);
	TestTrue(TEXT("GetParent(Package.A.B.C)"), Parent == ExpectedParent);

	// Verify these edge cases do not crash, but we do not provide a contract for what they mean.
	// ObjectName with no PackageName
	Child = FCoreRedirectObjectName(NameA, NAME_None, NAME_None);
	FCoreRedirectObjectName::GetParent(Child);

	// ObjectName and OuterName with no PackageName
	Child = FCoreRedirectObjectName(NameB, NameA, NAME_None);
	FCoreRedirectObjectName::GetParent(Child);

	// OuterName with no ObjectName and no PackageName
	Child = FCoreRedirectObjectName(NAME_None, NameA, NAME_None);
	FCoreRedirectObjectName::GetParent(Child);

	// OuterName and ObjectName with no PackageName
	Child = FCoreRedirectObjectName(NameB, NameA, NAME_None);
	FCoreRedirectObjectName::GetParent(Child);

	// OuterName and PackageName with no ObjectName
	Child = FCoreRedirectObjectName(NAME_None, NameA, NamePackage);
	FCoreRedirectObjectName::GetParent(Child);

	return true;
}
#endif


void FCoreRedirects::AddAssetRedirects(const TMap<FSoftObjectPath, FSoftObjectPath>& InRedirects)
{
	if (InRedirects.IsEmpty())
	{
		return;
	}

	FScopeCoreRedirectsWriteLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	FRedirectNameMap* ExistingMap = &LockedContext.Get().GetRedirectTypeMap().FindOrAdd(ECoreRedirectFlags::Type_Asset);
	int32 NumAdded = 0;
	int32 NumSkipped = 0;
	for (const TPair<FSoftObjectPath, FSoftObjectPath>& Pair : InRedirects)
	{
		// Asset redirects are, by definition, not package redirects
		if (Pair.Key.GetLongPackageFName().IsNone() || Pair.Value.GetAssetFName().IsNone())
		{
			UE_LOGF(LogCoreRedirects, Warning, "Attempted to register asset redirector that was missing a package or object name. Redirector was from %ls to %ls",
				*Pair.Key.ToString(), *Pair.Value.ToString());
			NumSkipped++;
			continue;
		}

		// Asset redirects use the package as the lookup key but contain multiple redirects underneath
		// Conceptually, for each object redirector we add two core redirects: one for the package and one for the object itself
		// In practice, we can just add the package redirect the first time

		FCoreRedirect ObjectRedirector(ECoreRedirectFlags::Type_Asset, Pair.Key.ToString(), Pair.Value.ToString());
		TArray<FCoreRedirect>& ExistingRedirects = ExistingMap->RedirectMap.FindOrAdd(ObjectRedirector.GetSearchKey());

		if (ExistingRedirects.Num() == 0)
		{
			// This is a new redirector. Add two entries, one for the package first
			FCoreRedirect PackageRedirect(ECoreRedirectFlags::Type_Asset, 
				Pair.Key.GetLongPackageName(), Pair.Value.GetLongPackageName());

			ExistingRedirects.Add(PackageRedirect);
		}

		// Check that, among the existing redirects, this one won't be a duplicate. Don't bother checking the package entry
		bool bShouldAddRedirect = true;
		for (int32 RedirectIndex = 1; RedirectIndex < ExistingRedirects.Num(); RedirectIndex++)
		{
			if (Pair.Key.GetLongPackageFName() == ExistingRedirects[RedirectIndex].OldName.PackageName
				&& Pair.Key.GetAssetName() == ExistingRedirects[RedirectIndex].OldName.ObjectName)
			{
				const FCoreRedirectObjectName& ExistingTargetName = ExistingRedirects[0].NewName;

				UE_LOGFMT(LogCoreRedirects, Error, "Skipping new redirect target '{target}' due to existing map from '{source}' to '{dest}'",
					Pair.Value.ToString(), Pair.Key.ToString(), ExistingTargetName.ToString());

				bShouldAddRedirect = false;
				NumSkipped++;
				break;
			}
		}

		if (bShouldAddRedirect)
		{
			ExistingRedirects.Add(ObjectRedirector);
			NumAdded++;
		}
	}

	UE_LOGF(LogCoreRedirects, Display, 
		"Object redirects provided to FCoreRedirects: %d. Redirects add: %d. Redirects skipped: %d",
		InRedirects.Num(), NumAdded, NumSkipped);

	if (LockedContext.Get().IsInDebugMode())
	{
		ValidateAssetRedirects();
	}
}

void FCoreRedirects::RemoveAllAssetRedirects()
{
	FScopeCoreRedirectsWriteLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	FRedirectNameMap* ExistingMap = &LockedContext.Get().GetRedirectTypeMap().FindOrAdd(ECoreRedirectFlags::Type_Asset);
	ExistingMap->RedirectMap.Reset();
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FCoreRedirectTest, "System.Core.Misc.CoreRedirects", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FCoreRedirectTest::RunTest(const FString& Parameters)
{
	return FCoreRedirects::RunTests();
}

bool FCoreRedirects::ReadRedirectsFromIni(const FString& IniName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FCoreRedirects::ReadRedirectsFromIni");
	FScopeCoreRedirectsWriteLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	ensureMsgf(IsInitialized(), TEXT("FCoreRedirects must be initialized on the game thread before use."));

	if (GConfig)
	{
		const TCHAR* RedirectSectionName = TEXT("CoreRedirects");
		const FConfigSection* RedirectSection = GConfig->GetSection(RedirectSectionName, false, IniName);
		if (RedirectSection)
		{
			TArray<FCoreRedirect> NewRedirects;

			for (FConfigSection::TConstIterator It(*RedirectSection); It; ++It)
			{
				FString OldName, NewName, OverrideClassName;

				bool bInstanceOnly = false;
				bool bIterate = false;
				bool bRemoved = false;
				bool bMatchSubstring = false;
				bool bMatchWildcard = false;
				bool bSuppressIterateWarning = false;

				const FString& ValueString = It.Value().GetValue();

				FParse::Bool(*ValueString, TEXT("InstanceOnly="), bInstanceOnly);
				FParse::Bool(*ValueString, TEXT("Iterate="), bIterate);
				FParse::Bool(*ValueString, TEXT("Removed="), bRemoved);
				FParse::Bool(*ValueString, TEXT("MatchSubstring="), bMatchSubstring);
				FParse::Bool(*ValueString, TEXT("MatchWildcard="), bMatchWildcard);
				FParse::Bool(*ValueString, TEXT("SuppressIterateWarning="), bSuppressIterateWarning);

				FParse::Value(*ValueString, TEXT("OldName="), OldName);
				FParse::Value(*ValueString, TEXT("NewName="), NewName);

				FParse::Value(*ValueString, TEXT("OverrideClassName="), OverrideClassName);

				ECoreRedirectFlags* FlagPtr = LockedContext.Get().GetConfigKeyMap().Find(It.Key());

				// If valid type
				if (FlagPtr)
				{
					ECoreRedirectFlags NewFlags = *FlagPtr;

					if (bInstanceOnly)
					{
						NewFlags |= ECoreRedirectFlags::Category_InstanceOnly;
					}

					if (bRemoved)
					{
						NewFlags |= ECoreRedirectFlags::Category_Removed;
					}

					if (bIterate)
					{
						NewFlags |= ECoreRedirectFlags::Option_Iterate;

						if (!bSuppressIterateWarning)
						{
							UE_LOGF(LogCoreRedirects, Warning,
								"ReadRedirectsFromIni(%ls) Redirect %ls is iterative. Iterative redirects are expensive, and should be used with caution.",
								*IniName, *ValueString);
						}
					}

					if (bMatchWildcard || bMatchSubstring)
					{
						UE_CLOGF(bMatchSubstring, LogCoreRedirects, Warning, "ReadRedirectsFromIni(%ls) 'MatchSubstring=' is deprecated. "
							"Please prefer `MatchWildcard=' instead for redirect %ls.\n\t"
							"For more information refer to the documentation in Engine/Config/BaseEngine.ini.", *IniName, *ValueString);

						constexpr FStringView WildcardMarker(TEXTVIEW("..."));
						bool bMatchPrefix = OldName.EndsWith(WildcardMarker, ESearchCase::CaseSensitive);
						bool bMatchSuffix = OldName.StartsWith(WildcardMarker, ESearchCase::CaseSensitive);

						bMatchSubstring = bMatchSubstring || (bMatchPrefix && bMatchSuffix);

						// Count how many '...' there are to ensure the OldName is not malformed
						int WildcardCount = 0;
						int StartPos = 0;
						FStringView OldNameView(OldName);
						while ((StartPos = OldNameView.Find(WildcardMarker, StartPos)) != INDEX_NONE)
						{
							StartPos += WildcardMarker.Len();
							WildcardCount++;
						}

						if ((!bMatchPrefix && !bMatchSuffix && !bMatchSubstring)		  // No wildcards found and not handling MatchSubstring (no '...')
							|| (WildcardCount > ((int)bMatchPrefix + (int)bMatchSuffix))) // Ensure we don't have more wildcards than necessary
						{
							UE_LOGF(LogCoreRedirects, Error, "ReadRedirectsFromIni(%ls) failed to parse OldName for wildcard redirect %ls! "
								"OldName must be of the form 'PrefixName...', '...SubstringName...' or '...SuffixName'. For more information refer to the documentation in Engine/Config/BaseEngine.ini.", *IniName, *ValueString);
							continue;
						}

						if (bMatchPrefix)
						{
							NewFlags |= ECoreRedirectFlags::Option_MatchPrefix;
							OldName.LeftChopInline(WildcardMarker.Len(), EAllowShrinking::No);
						}

						if (bMatchSuffix)
						{
							NewFlags |= ECoreRedirectFlags::Option_MatchSuffix;
							OldName.RightChopInline(WildcardMarker.Len(), EAllowShrinking::No);
						}

						NewFlags |= bMatchSubstring ? ECoreRedirectFlags::Option_MatchSubstring : ECoreRedirectFlags::None;
					}

					FCoreRedirect* Redirect = &NewRedirects.Emplace_GetRef(NewFlags, FCoreRedirectObjectName(OldName), FCoreRedirectObjectName(NewName));

					if (!OverrideClassName.IsEmpty())
					{
						Redirect->OverrideClassName = FCoreRedirectObjectName(OverrideClassName);
					}

					int32 ValueChangesIndex = ValueString.Find(TEXT("ValueChanges="));
					if (ValueChangesIndex != INDEX_NONE)
					{
						// Look for first (
						ValueChangesIndex = ValueString.Find(TEXT("("), ESearchCase::CaseSensitive, ESearchDir::FromStart, ValueChangesIndex);

						FString ValueChangesString = ValueString.Mid(ValueChangesIndex);
						const TCHAR* Buffer = Redirect->ParseValueChanges(GetData(ValueChangesString));

						if (!Buffer)
						{
							UE_LOGF(LogCoreRedirects, Error, "ReadRedirectsFromIni(%ls) failed to parse ValueChanges for redirect %ls!", *IniName, *ValueString);

							// Remove added redirect
							Redirect = nullptr;
							NewRedirects.RemoveAt(NewRedirects.Num() - 1);
						}
					}
				}
				else
				{
					UE_LOGF(LogCoreRedirects, Error, "ReadRedirectsFromIni(%ls) failed to parse type for redirect %ls!", *IniName, *ValueString);
				}
			}

			// We no longer need the redirect config data in memory so remove it entirely
			GConfig->RemoveSectionFromBranch(RedirectSectionName, *IniName);

			return AddRedirectList(NewRedirects, IniName);
		}
		else
		{
			UE_LOGF(LogCoreRedirects, Verbose, "ReadRedirectsFromIni(%ls) did not find any redirects", *IniName);
		}
	}
	else
	{
		UE_LOGF(LogCoreRedirects, Warning, " **** CORE REDIRECTS UNABLE TO INITIALIZE! **** ");
	}
	return false;
}

bool FCoreRedirects::AddRedirectList(TArrayView<const FCoreRedirect> Redirects, const FString& SourceString)
{
	FScopeCoreRedirectsWriteLockedContext ScopeLock(FCoreRedirectsContext::GetThreadContext());
	ensureMsgf(IsInitialized(), TEXT("FCoreRedirects must be initialized on the game thread before use."));
	return AddRedirectListUnderWriteLock(Redirects, SourceString, ScopeLock);
}

bool FCoreRedirects::AddRedirectListUnderWriteLock(TArrayView<const FCoreRedirect> Redirects, const FString & SourceString, 
	FScopeCoreRedirectsWriteLockedContext& LockedContext)
{
	UE_LOGF(LogCoreRedirects, Verbose, "AddRedirect(%ls) adding %d redirects", *SourceString, Redirects.Num());

	if (LockedContext.Get().IsInDebugMode() && LockedContext.Get().HasValidated())
	{
		// Validate on apply because we finished our initial validation pass
		ValidateRedirectList(Redirects, SourceString);
	}

	bool bAddedAny = false;
	for (const FCoreRedirect& NewRedirect : Redirects)
	{
		if (!NewRedirect.OldName.IsValid() || !NewRedirect.NewName.IsValid())
		{
			UE_LOGF(LogCoreRedirects, Error, "AddRedirect(%ls) failed to add redirect from %ls to %ls with empty name!", *SourceString, *NewRedirect.OldName.ToString(), *NewRedirect.NewName.ToString());
			continue;
		}

		// Type_Asset redirects derive from UObjectRedirector instances on disk.
		// Therefore we can assume that the names in it are valid (because they exist)
		// This is as opposed to redirects that are manually created either in code or in 
		// ini files and which should be subject to additional validation.
		// Because the validation in HasValidCharacters does not perfectly align with other systems,
		// there can exist assets which are valid, are referenced by UObjectRedirector assets, but which
		// fail the check.
		if (!!(LockedContext.Get().GetFlags() & FCoreRedirectsContext::EFlags::ValidateAddedRedirects)
			&& !EnumHasAnyFlags(NewRedirect.RedirectFlags, ECoreRedirectFlags::Type_Asset))
		{
			if (!NewRedirect.OldName.HasValidCharacters(NewRedirect.RedirectFlags)
				|| !NewRedirect.NewName.HasValidCharacters(NewRedirect.RedirectFlags))
			{
				UE_LOGF(LogCoreRedirects, Error, "AddRedirect(%ls) failed to add redirect from %ls to %ls with invalid characters!", *SourceString, *NewRedirect.OldName.ToString(), *NewRedirect.NewName.ToString());
				continue;
			}
		}

		if (NewRedirect.IsWildcardMatch())
		{
			UE_LOGF(LogCoreRedirects, Verbose, "AddRedirect(%ls) has wildcard redirect %ls, these are very slow and should be resolved as soon as possible! Please refer to the documentation in Engine/Config/BaseEngine.ini.", *SourceString, *NewRedirect.OldName.ToString());
		}
		
		if (AddSingleRedirectUnderWriteLock(NewRedirect, SourceString, LockedContext))
		{
			bAddedAny = true;

			// If value redirect, add a value redirect from NewName->NewName as well, this will merge with existing ones as needed
			if (NewRedirect.OldName != NewRedirect.NewName && NewRedirect.HasValueChanges())
			{
				FCoreRedirect ValueRedirect = NewRedirect;
				ValueRedirect.OldName = ValueRedirect.NewName;

				AddSingleRedirectUnderWriteLock(ValueRedirect, SourceString, LockedContext);
			}
		}
	}

	return bAddedAny;
}

bool FCoreRedirects::AddSingleRedirectUnderWriteLock(const FCoreRedirect& NewRedirect, const FString& SourceString, 
	FScopeCoreRedirectsWriteLockedContext& LockedContext)
{
#if WITH_EDITOR
	static FString ClassName;
	static bool bDebugNullCook = FParse::Value(FCommandLine::Get(), TEXT("-DebugNullCook="), ClassName);
	bool bPrintLogsDebugNullCook = bDebugNullCook && NewRedirect.OldName.ObjectName.ToString() == ClassName;

	if (bPrintLogsDebugNullCook)
	{
		UE_LOGF(LogCoreRedirects, Display, "[DebugNullCook] Adding redirector %ls->%ls from %ls", *NewRedirect.OldName.ToString(),
			*NewRedirect.NewName.ToString(), *SourceString);
	}
#endif // #if WITH_EDITOR

	const bool bIsWildcardMatch = NewRedirect.IsWildcardMatch();
	FRedirectNameMap* ExistingNameMap;

	ExistingNameMap = &LockedContext.Get().GetRedirectTypeMap().FindOrAdd(NewRedirect.RedirectFlags);

	TArray<FCoreRedirect>* ExistingRedirects = nullptr;
	if (bIsWildcardMatch)
	{
		if (NewRedirect.IsSubstringMatch())
		{
			ExistingRedirects = &ExistingNameMap->Wildcards->Substrings;
		}
		else if (NewRedirect.IsPrefixMatch())
		{
			ExistingRedirects = &ExistingNameMap->Wildcards->Prefixes;
		}
		else if (NewRedirect.IsSuffixMatch())
		{
			ExistingRedirects = &ExistingNameMap->Wildcards->Suffixes;
		}
		check(ExistingRedirects);
	}
	else
	{
		ExistingRedirects = &ExistingNameMap->RedirectMap.FindOrAdd(NewRedirect.GetSearchKey());
	}

	// Check for duplicate
	bool bFoundDuplicate = false;
	for (FCoreRedirect& ExistingRedirect : *ExistingRedirects)
	{
		if (ExistingRedirect.IdenticalMatchRules(NewRedirect))
		{
			bFoundDuplicate = true;
			bool bIsSameNewName = ExistingRedirect.NewName == NewRedirect.NewName;
			bool bOneIsPartialOtherIsFull = false;
			if (!bIsSameNewName &&
				ExistingRedirect.OldName.Matches(NewRedirect.OldName, FCoreRedirectObjectName::EMatchFlags::AllowPartialRHSMatch) &&
				ExistingRedirect.NewName.Matches(NewRedirect.NewName, FCoreRedirectObjectName::EMatchFlags::AllowPartialRHSMatch))
			{
				bIsSameNewName = true;
				bOneIsPartialOtherIsFull = true;
			}

			if (bIsSameNewName)
			{
				// Merge fields from the two duplicate redirects
				bool bBothHaveValueChanges = ExistingRedirect.HasValueChanges() && NewRedirect.HasValueChanges();
				ExistingRedirect.OldName.UnionFieldsInline(NewRedirect.OldName);
				ExistingRedirect.NewName.UnionFieldsInline(NewRedirect.NewName);
				ExistingRedirect.ValueChanges.Append(NewRedirect.ValueChanges);
				if (bBothHaveValueChanges)
				{
					// No warning when there are values changes to merge
					UE_LOGF(LogCoreRedirects, Verbose, "AddRedirect(%ls) merging value redirects for %ls", *SourceString, *ExistingRedirect.NewName.ToString());
				}
				else if (bOneIsPartialOtherIsFull)
				{
					UE_LOGF(LogCoreRedirects, Warning, "AddRedirect(%ls) found duplicate redirects for %ls to %ls, one a FullPath and the other ObjectName-only." " This used to be required for StructRedirects but now you should remove the ObjectName-only redirect and keep the FullPath.",
						*SourceString, *ExistingRedirect.OldName.ToString(), *ExistingRedirect.NewName.ToString());
				}
				else
				{
					UE_LOGF(LogCoreRedirects, Verbose, "AddRedirect(%ls) ignoring duplicate redirects for %ls to %ls", *SourceString, *ExistingRedirect.OldName.ToString(), *ExistingRedirect.NewName.ToString());
				}
			}
			else
			{
				UE_LOGF(LogCoreRedirects, Error, "AddRedirect(%ls) found conflicting redirects for %ls! Old: %ls, New: %ls", *SourceString, *ExistingRedirect.OldName.ToString(), *ExistingRedirect.NewName.ToString(), *NewRedirect.NewName.ToString());
			}
			break;
		}
	}

	if (bFoundDuplicate)
	{
		return false;
	}

	if (bIsWildcardMatch)
	{
		ExistingNameMap->Wildcards->Add(NewRedirect);
	}
	else
	{
		ExistingRedirects->Add(NewRedirect);
	}

#if WITH_EDITOR
	if (!!(LockedContext.Get().GetFlags() & FCoreRedirectsContext::EFlags::UseRedirectionSummary))
	{
		LockedContext.Get().GetRedirectionSummary().Add(NewRedirect);
	}
#endif

	return true;
}

bool FCoreRedirects::RemoveRedirectList(TArrayView<const FCoreRedirect> Redirects, const FString& SourceString)
{
	FScopeCoreRedirectsWriteLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	UE_LOGF(LogCoreRedirects, Verbose, "RemoveRedirect(%ls) Removing %d redirects", *SourceString, Redirects.Num());

	bool bRemovedAny = false;
	for (const FCoreRedirect& RedirectToRemove : Redirects)
	{
		if (!RedirectToRemove.OldName.IsValid() || !RedirectToRemove.NewName.IsValid())
		{
			UE_LOGF(LogCoreRedirects, Error, "RemoveRedirect(%ls) failed to remove redirect from %ls to %ls with empty name!", *SourceString, *RedirectToRemove.OldName.ToString(), *RedirectToRemove.NewName.ToString());
			continue;
		}

		if (RedirectToRemove.HasValueChanges())
		{
			UE_LOGF(LogCoreRedirects, Error, "RemoveRedirect(%ls) failed to remove redirect from %ls to %ls as it contains value changes!", *SourceString, *RedirectToRemove.OldName.ToString(), *RedirectToRemove.NewName.ToString());
			continue;
		}

		if (!RedirectToRemove.OldName.HasValidCharacters(RedirectToRemove.RedirectFlags) ||
			!RedirectToRemove.NewName.HasValidCharacters(RedirectToRemove.RedirectFlags))
		{
			UE_LOGF(LogCoreRedirects, Error, "RemoveRedirect(%ls) failed to remove redirect from %ls to %ls with invalid characters!", *SourceString, *RedirectToRemove.OldName.ToString(), *RedirectToRemove.NewName.ToString());
			continue;
		}

		if (RedirectToRemove.NewName.PackageName != RedirectToRemove.OldName.PackageName && RedirectToRemove.OldName.OuterName != NAME_None)
		{
			UE_LOGF(LogCoreRedirects, Error, "RemoveRedirect(%ls) failed to remove redirect, it's not valid to modify package from %ls to %ls while specifying outer!", *SourceString, *RedirectToRemove.OldName.ToString(), *RedirectToRemove.NewName.ToString());
			continue;
		}

		if (RedirectToRemove.IsWildcardMatch())
		{
			UE_LOGF(LogCoreRedirects, Verbose, "RemoveRedirect(%ls) has wildcard redirect %ls, these are very slow and should be resolved as soon as possible! Please refer to the documentation in Engine/Config/BaseEngine.ini.", *SourceString, *RedirectToRemove.OldName.ToString());
		}

		bRemovedAny |= RemoveSingleRedirectUnderWriteLock(RedirectToRemove, SourceString, LockedContext);
	}

	return bRemovedAny;
}

bool FCoreRedirects::IsInitialized()
{
	return FCoreRedirectsContext::GetThreadContext().IsInitialized();
}

bool FCoreRedirects::IsInDebugMode()
{
	return FCoreRedirectsContext::GetThreadContext().IsInDebugMode();
}

bool FCoreRedirects::RemoveSingleRedirectUnderWriteLock(const FCoreRedirect& RedirectToRemove, const FString& SourceString,
	FScopeCoreRedirectsWriteLockedContext& LockedContext)
{
	const bool bIsWildcardMatch = RedirectToRemove.IsWildcardMatch();
	FRedirectNameMap* ExistingNameMap = LockedContext.Get().GetRedirectTypeMap().Find(RedirectToRemove.RedirectFlags);

	if (!ExistingNameMap)
	{
		return false;
	}

	TArray<FCoreRedirect>* ExistingRedirects = nullptr;
	if (bIsWildcardMatch)
	{
		if (RedirectToRemove.IsSubstringMatch())
		{
			ExistingRedirects = &ExistingNameMap->Wildcards->Substrings;
		}
		else if (RedirectToRemove.IsPrefixMatch())
		{
			ExistingRedirects = &ExistingNameMap->Wildcards->Prefixes;
		}
		else if (RedirectToRemove.IsSuffixMatch())
		{
			ExistingRedirects = &ExistingNameMap->Wildcards->Suffixes;
		}
	}
	else
	{
		ExistingRedirects = ExistingNameMap->RedirectMap.Find(RedirectToRemove.GetSearchKey());
	}

	if (!ExistingRedirects)
	{
		return false;
	}

	bool bRemovedRedirect = false;
	for (int32 ExistingRedirectIndex = 0; ExistingRedirectIndex < ExistingRedirects->Num(); ++ExistingRedirectIndex)
	{
		FCoreRedirect& ExistingRedirect = (*ExistingRedirects)[ExistingRedirectIndex];

		if (ExistingRedirect.IdenticalMatchRules(RedirectToRemove))
		{
			if (ExistingRedirect.NewName != RedirectToRemove.NewName)
			{
				// This isn't the redirect we were looking for... move on in case there's another match for our old name
				continue;
			}

			bRemovedRedirect = true;
			ExistingRedirects->RemoveAt(ExistingRedirectIndex);
			break;
		}
	}

	if (bRemovedRedirect)
	{
		if (bIsWildcardMatch)
		{
			// We removed a wildcard redirect so we need to regenerate our prediction 
			// tables to avoid unnecessary false positives
			ExistingNameMap->Wildcards->Rebuild();
		}

#if WITH_EDITOR
		if (!!(LockedContext.Get().GetFlags() & FCoreRedirectsContext::EFlags::UseRedirectionSummary))
		{
			LockedContext.Get().GetRedirectionSummary().Remove(RedirectToRemove);
		}
#endif
	}

	return bRemovedRedirect;
}

void FCoreRedirects::ValidateRedirectList(TArrayView<const FCoreRedirect> Redirects, const FString& SourceString)
{
	for (const FCoreRedirect& Redirect : Redirects)
	{
		if (Redirect.NewName.IsValid())
		{
			// If the new package is loaded but the new name isn't, this is very likely a bug
			// If the new package isn't loaded the redirect can't be validated either way, so report it for manual follow up
			UObject* NewPackage = FindObjectFast<UPackage>(nullptr, Redirect.NewName.PackageName);
			FString NewPath = Redirect.NewName.ToString();
			FString OldPath = Redirect.OldName.ToString();

			if (CheckRedirectFlagsMatch(Redirect.RedirectFlags, ECoreRedirectFlags::Type_Class))
			{
				if (Redirect.NewName.PackageName.IsNone())
				{
					UE_LOGF(LogCoreRedirects, Warning, "ValidateRedirect(%ls) has missing package for Class redirect from %ls to %ls!", *SourceString, *OldPath, *NewPath);
				}
				else
				{
					const UClass* FindClass = FindObject<const UClass>(FTopLevelAssetPath(NewPath));
					if (!FindClass)
					{
						if (NewPackage)
						{
							UE_LOGF(LogCoreRedirects, Error, "ValidateRedirect(%ls) failed to find destination Class for redirect from %ls to %ls with loaded package!", *SourceString, *OldPath, *NewPath);
						}
						else
						{
							UE_LOGF(LogCoreRedirects, Log, "ValidateRedirect(%ls) can't validate destination Class for redirect from %ls to %ls with unloaded package", *SourceString, *OldPath, *NewPath);
						}
					}
				}
			}

			if (CheckRedirectFlagsMatch(Redirect.RedirectFlags, ECoreRedirectFlags::Type_Struct))
			{
				if (Redirect.NewName.PackageName.IsNone())
				{
					UE_LOGF(LogCoreRedirects, Warning, "ValidateRedirect(%ls) has missing package for Struct redirect from %ls to %ls!", *SourceString, *OldPath, *NewPath);
				}
				else
				{
					const UScriptStruct* FindStruct = FindObject<const UScriptStruct>(FTopLevelAssetPath(NewPath));
					if (!FindStruct)
					{
						if (NewPackage)
						{
							UE_LOGF(LogCoreRedirects, Error, "ValidateRedirect(%ls) failed to find destination Struct for redirect from %ls to %ls with loaded package!", *SourceString, *OldPath, *NewPath);
						}
						else
						{
							UE_LOGF(LogCoreRedirects, Log, "ValidateRedirect(%ls) can't validate destination Struct for redirect from %ls to %ls with unloaded package", *SourceString, *OldPath, *NewPath);
						}
					}
				}
			}

			if (CheckRedirectFlagsMatch(Redirect.RedirectFlags, ECoreRedirectFlags::Type_Enum))
			{
				if (Redirect.NewName.PackageName.IsNone())
				{
					// If the name is the same that's fine and this is just for value redirects
					if (Redirect.NewName != Redirect.OldName)
					{
						UE_LOGF(LogCoreRedirects, Warning, "ValidateRedirect(%ls) has missing package for Enum redirect from %ls to %ls!", *SourceString, *OldPath, *NewPath);
					}
				}
				else
				{
					const UEnum* FindEnum = FindObject<const UEnum>(FTopLevelAssetPath(NewPath));
					if (!FindEnum)
					{
						if (NewPackage)
						{
							UE_LOGF(LogCoreRedirects, Error, "ValidateRedirect(%ls) failed to find destination Enum for redirect from %ls to %ls with loaded package!", *SourceString, *OldPath, *NewPath);
						}
						else
						{
							UE_LOGF(LogCoreRedirects, Log, "ValidateRedirect(%ls) can't validate destination Enum for redirect from %ls to %ls with unloaded package", *SourceString, *OldPath, *NewPath);
						}
					}
				}
			}
		}
	}
}

void FCoreRedirects::ValidateAllRedirects()
{
	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	LockedContext.Get().SetHasValidated();

	// Validate all existing redirects

	for (const TPair<ECoreRedirectFlags, FRedirectNameMap>& Pair : LockedContext.Get().GetRedirectTypeMap())
	{
		ECoreRedirectFlags PairFlags = Pair.Key;
		FString ListName = FString::Printf(TEXT("Type %d"), (int32)PairFlags);

		for (const TPair<FName, TArray<FCoreRedirect> >& ArrayPair : Pair.Value.RedirectMap)
		{
			ValidateRedirectList(ArrayPair.Value, ListName);
		}
	}

	ValidateAssetRedirectsUnderReadLock(LockedContext);
}

bool FCoreRedirects::ValidateAssetRedirects()
{
	FScopeCoreRedirectsReadLockedContext LockedContext(FCoreRedirectsContext::GetThreadContext());
	return ValidateAssetRedirectsUnderReadLock(LockedContext);
}

const TMap<FName, ECoreRedirectFlags>& FCoreRedirects::GetConfigKeyMap()
{
	// The config key map is only written to during initialization. That allows us to provide unguarded access after initialization
	ensureMsgf(IsInitialized(), TEXT("It is not legal to read the config key map until after FCoreRedirects has been initialized."));
	return FCoreRedirectsContext::GetThreadContext().GetConfigKeyMap();
}

ECoreRedirectFlags FCoreRedirects::GetFlagsForTypeName(FName PackageName, FName TypeName)
{
	if (PackageName == GLongCoreUObjectPackageName)
	{
		if (TypeName == NAME_Class || TypeName == NAME_VerseClass)
		{
			return ECoreRedirectFlags::Type_Class;
		}
		else if (TypeName == NAME_ScriptStruct || TypeName == NAME_VerseStruct)
		{
			return ECoreRedirectFlags::Type_Struct;
		}
		else if (TypeName == NAME_Enum || TypeName == NAME_VerseEnum)
		{
			return ECoreRedirectFlags::Type_Enum;
		}
		else if (TypeName == NAME_Package)
		{
			return ECoreRedirectFlags::Type_Package;
		}
		else if (TypeName == NAME_Function)
		{
			return ECoreRedirectFlags::Type_Function;
		}

		// If ending with property, it's a property
		if (TypeName.ToString().EndsWith(TEXT("Property")))
		{
			return ECoreRedirectFlags::Type_Property;
		}
	}

	// If ending with GeneratedClass this has to be a class subclass, some of these are in engine or plugins
	if (TypeName.ToString().EndsWith(TEXT("GeneratedClass")))
	{
		return ECoreRedirectFlags::Type_Class;
	}

	if (TypeName == NAME_UserDefinedEnum)
	{
		return ECoreRedirectFlags::Type_Enum;
	}

	return ECoreRedirectFlags::Type_Object;
}

ECoreRedirectFlags FCoreRedirects::GetFlagsForTypeClass(UClass *TypeClass)
{
	// Use Name version for consistency, if we can't figure it out from just the name it isn't safe
	return GetFlagsForTypeName(TypeClass->GetOutermost()->GetFName(), TypeClass->GetFName());
}

// We want to only load these redirects in editor builds, but Matinee needs them at runtime still 

#if UE_WITH_CORE_REDIRECTS

PRAGMA_PUSH_ATTRIBUTE_MINSIZE_FUNCTIONS

template <typename T>
FORCEINLINE static T& DiscardNoDiscard(T& Ref)
{
	return Ref;
}

static FORCENOINLINE FCoreRedirect& ClassRedirectImpl(TArray<FCoreRedirect>& Redirects, const TCHAR* OldName, const TCHAR* NewName)
{
	return DiscardNoDiscard(Redirects.Emplace_GetRef(ECoreRedirectFlags::Type_Class, OldName, NewName));
}

static FORCENOINLINE FCoreRedirect& ClassRedirectInstancesImpl(TArray<FCoreRedirect>& Redirects, const TCHAR* OldName, const TCHAR* NewName)
{
	return DiscardNoDiscard(Redirects.Emplace_GetRef(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, OldName, NewName));
}

static FORCENOINLINE FCoreRedirect& StructRedirectImpl(TArray<FCoreRedirect>& Redirects, const TCHAR* OldName, const TCHAR* NewName)
{
	return DiscardNoDiscard(Redirects.Emplace_GetRef(ECoreRedirectFlags::Type_Struct, OldName, NewName));
}

static FORCENOINLINE FCoreRedirect& EnumRedirectImpl(TArray<FCoreRedirect>& Redirects, const TCHAR* OldName, const TCHAR* NewName)
{
	return DiscardNoDiscard(Redirects.Emplace_GetRef(ECoreRedirectFlags::Type_Enum, OldName, NewName));
}

static FORCENOINLINE FCoreRedirect& PropertyRedirectImpl(TArray<FCoreRedirect>& Redirects, const TCHAR* OldName, const TCHAR* NewName)
{
	return DiscardNoDiscard(Redirects.Emplace_GetRef(ECoreRedirectFlags::Type_Property, OldName, NewName));
}

static FORCENOINLINE FCoreRedirect& FunctionRedirectImpl(TArray<FCoreRedirect>& Redirects, const TCHAR* OldName, const TCHAR* NewName)
{
	return DiscardNoDiscard(Redirects.Emplace_GetRef(ECoreRedirectFlags::Type_Function, OldName, NewName));
}

static FORCENOINLINE FCoreRedirect& PackageRedirectImpl(TArray<FCoreRedirect>& Redirects, const TCHAR* OldName, const TCHAR* NewName)
{
	return DiscardNoDiscard(Redirects.Emplace_GetRef(ECoreRedirectFlags::Type_Package, OldName, NewName));
}

// The compiler doesn't like having a massive string table in a single function so split it up
#define CLASS_REDIRECT(OldName, NewName) ClassRedirectImpl(Redirects, TEXT(OldName), TEXT(NewName))
#define CLASS_REDIRECT_INSTANCES(OldName, NewName) ClassRedirectInstancesImpl(Redirects, TEXT(OldName), TEXT(NewName))
#define STRUCT_REDIRECT(OldName, NewName) StructRedirectImpl(Redirects, TEXT(OldName), TEXT(NewName))
#define ENUM_REDIRECT(OldName, NewName) EnumRedirectImpl(Redirects, TEXT(OldName), TEXT(NewName))
#define PROPERTY_REDIRECT(OldName, NewName) PropertyRedirectImpl(Redirects, TEXT(OldName), TEXT(NewName))
#define FUNCTION_REDIRECT(OldName, NewName) FunctionRedirectImpl(Redirects, TEXT(OldName), TEXT(NewName))
#define PACKAGE_REDIRECT(OldName, NewName) PackageRedirectImpl(Redirects, TEXT(OldName), TEXT(NewName))

static void RegisterNativeRedirects40(TArray<FCoreRedirect>& Redirects)
{
	// CLASS_REDIRECT("AnimTreeInstance", "/Script/Engine.AnimInstance"); // Leaving some of these disabled as they can cause issues with the reverse class lookup in asset registry scans 
	CLASS_REDIRECT("AnimationCompressionAlgorithm", "/Script/Engine.AnimCompress");
	CLASS_REDIRECT("AnimationCompressionAlgorithm_BitwiseCompressOnly", "/Script/Engine.AnimCompress_BitwiseCompressOnly");
	CLASS_REDIRECT("AnimationCompressionAlgorithm_LeastDestructive", "/Script/Engine.AnimCompress_LeastDestructive");
	CLASS_REDIRECT("AnimationCompressionAlgorithm_PerTrackCompression", "/Script/Engine.AnimCompress_PerTrackCompression");
	CLASS_REDIRECT("AnimationCompressionAlgorithm_RemoveEverySecondKey", "/Script/Engine.AnimCompress_RemoveEverySecondKey");
	CLASS_REDIRECT("AnimationCompressionAlgorithm_RemoveLinearKeys", "/Script/Engine.AnimCompress_RemoveLinearKeys");
	CLASS_REDIRECT("AnimationCompressionAlgorithm_RemoveTrivialKeys", "/Script/Engine.AnimCompress_RemoveTrivialKeys");
	// CLASS_REDIRECT("BlueprintActorBase", "/Script/Engine.Actor");
	CLASS_REDIRECT("DefaultPawnMovement", "/Script/Engine.FloatingPawnMovement");
	CLASS_REDIRECT("DirectionalLightMovable", "/Script/Engine.DirectionalLight");
	CLASS_REDIRECT("DirectionalLightStatic", "/Script/Engine.DirectionalLight");
	CLASS_REDIRECT("DirectionalLightStationary", "/Script/Engine.DirectionalLight");
	CLASS_REDIRECT("DynamicBlockingVolume", "/Script/Engine.BlockingVolume");
	CLASS_REDIRECT("DynamicPhysicsVolume", "/Script/Engine.PhysicsVolume");
	CLASS_REDIRECT("DynamicTriggerVolume", "/Script/Engine.TriggerVolume");
	// CLASS_REDIRECT("GameInfo", "/Script/Engine.GameMode");
	// CLASS_REDIRECT("GameReplicationInfo", "/Script/Engine.GameState");
	CLASS_REDIRECT("InterpActor", "/Script/Engine.StaticMeshActor");
	CLASS_REDIRECT("K2Node_CallSuperFunction", "/Script/BlueprintGraph.K2Node_CallParentFunction");
	CLASS_REDIRECT("MaterialSpriteComponent", "/Script/Engine.MaterialBillboardComponent");
	CLASS_REDIRECT("MovementComp_Character", "/Script/Engine.CharacterMovementComponent");
	CLASS_REDIRECT("MovementComp_Projectile", "/Script/Engine.ProjectileMovementComponent");
	CLASS_REDIRECT("MovementComp_Rotating", "/Script/Engine.RotatingMovementComponent");
	CLASS_REDIRECT("NavAreaDefault", "/Script/NavigationSystem.NavArea_Default");
	CLASS_REDIRECT("NavAreaDefinition", "/Script/NavigationSystem.NavArea");
	CLASS_REDIRECT("NavAreaNull", "/Script/NavigationSystem.NavArea_Null");
	CLASS_REDIRECT("PhysicsActor", "/Script/Engine.StaticMeshActor");
	CLASS_REDIRECT("PhysicsBSJointActor", "/Script/Engine.PhysicsConstraintActor");
	CLASS_REDIRECT("PhysicsHingeActor", "/Script/Engine.PhysicsConstraintActor");
	CLASS_REDIRECT("PhysicsPrismaticActor", "/Script/Engine.PhysicsConstraintActor");
	// CLASS_REDIRECT("PlayerCamera", "/Script/Engine.PlayerCameraManager");
	// CLASS_REDIRECT("PlayerReplicationInfo", "/Script/Engine.PlayerState");
	CLASS_REDIRECT("PointLightMovable", "/Script/Engine.PointLight");
	CLASS_REDIRECT("PointLightStatic", "/Script/Engine.PointLight");
	CLASS_REDIRECT("PointLightStationary", "/Script/Engine.PointLight");
	CLASS_REDIRECT("RB_BSJointSetup", "/Script/Engine.PhysicsConstraintTemplate");
	CLASS_REDIRECT("RB_BodySetup", "/Script/Engine.BodySetup");
	CLASS_REDIRECT("RB_ConstraintActor", "/Script/Engine.PhysicsConstraintActor");
	CLASS_REDIRECT("RB_ConstraintComponent", "/Script/Engine.PhysicsConstraintComponent");
	CLASS_REDIRECT("RB_ConstraintSetup", "/Script/Engine.PhysicsConstraintTemplate");
	CLASS_REDIRECT("RB_Handle", "/Script/Engine.PhysicsHandleComponent");
	CLASS_REDIRECT("RB_HingeSetup", "/Script/Engine.PhysicsConstraintTemplate");
	CLASS_REDIRECT("RB_PrismaticSetup", "/Script/Engine.PhysicsConstraintTemplate");
	CLASS_REDIRECT("RB_RadialForceComponent", "/Script/Engine.RadialForceComponent");
	CLASS_REDIRECT("RB_SkelJointSetup", "/Script/Engine.PhysicsConstraintTemplate");
	CLASS_REDIRECT("RB_Thruster", "/Script/Engine.PhysicsThruster");
	CLASS_REDIRECT("RB_ThrusterComponent", "/Script/Engine.PhysicsThrusterComponent");
	CLASS_REDIRECT("SensingComponent", "/Script/AIModule.PawnSensingComponent");
	CLASS_REDIRECT("SingleAnimSkeletalActor", "/Script/Engine.SkeletalMeshActor");
	CLASS_REDIRECT("SingleAnimSkeletalComponent", "/Script/Engine.SkeletalMeshComponent");
	CLASS_REDIRECT("SkeletalMeshReplicatedComponent", "/Script/Engine.SkeletalMeshComponent");
	CLASS_REDIRECT("SkeletalPhysicsActor", "/Script/Engine.SkeletalMeshActor");
	CLASS_REDIRECT("SoundMode", "/Script/Engine.SoundMix");
	CLASS_REDIRECT("SpotLightMovable", "/Script/Engine.SpotLight");
	CLASS_REDIRECT("SpotLightStatic", "/Script/Engine.SpotLight");
	CLASS_REDIRECT("SpotLightStationary", "/Script/Engine.SpotLight");
	CLASS_REDIRECT("SpriteComponent", "/Script/Engine.BillboardComponent");
	CLASS_REDIRECT("StaticMeshReplicatedComponent", "/Script/Engine.StaticMeshComponent");
	CLASS_REDIRECT("VimBlueprint", "/Script/Engine.AnimBlueprint");
	CLASS_REDIRECT("VimGeneratedClass", "/Script/Engine.AnimBlueprintGeneratedClass");
	CLASS_REDIRECT("VimInstance", "/Script/Engine.AnimInstance");
	CLASS_REDIRECT("WorldInfo", "/Script/Engine.WorldSettings");
	CLASS_REDIRECT_INSTANCES("NavAreaMeta", "/Script/NavigationSystem.NavArea_Default");

	STRUCT_REDIRECT("VimDebugData", "/Script/Engine.AnimBlueprintDebugData");

	FUNCTION_REDIRECT("Actor.GetController", "Pawn.GetController");
	FUNCTION_REDIRECT("Actor.GetTouchingActors", "Actor.GetOverlappingActors");
	PROPERTY_REDIRECT("Actor.GetOverlappingActors.OutTouchingActors", "OverlappingActors");
	FUNCTION_REDIRECT("Actor.GetTouchingComponents", "Actor.GetOverlappingComponents");
	PROPERTY_REDIRECT("Actor.GetOverlappingComponents.TouchingComponents", "OverlappingComponents");
	FUNCTION_REDIRECT("Actor.HasTag", "Actor.ActorHasTag");
	FUNCTION_REDIRECT("Actor.ReceiveActorTouch", "Actor.ReceiveActorBeginOverlap");
	PROPERTY_REDIRECT("Actor.ReceiveActorBeginOverlap.Other", "OtherActor");
	FUNCTION_REDIRECT("Actor.ReceiveActorUntouch", "Actor.ReceiveActorEndOverlap");
	PROPERTY_REDIRECT("Actor.ReceiveActorEndOverlap.Other", "OtherActor");
	PROPERTY_REDIRECT("Actor.ReceiveHit.NormalForce", "NormalImpulse");
	FUNCTION_REDIRECT("Actor.SetActorHidden", "Actor.SetActorHiddenInGame");
	PROPERTY_REDIRECT("Actor.LifeSpan", "Actor.InitialLifeSpan");
	PROPERTY_REDIRECT("Actor.OnActorTouch", "OnActorBeginOverlap");
	PROPERTY_REDIRECT("Actor.OnActorUnTouch", "OnActorEndOverlap");

	FUNCTION_REDIRECT("AnimInstance.GetSequencePlayerLength", "GetAnimAssetPlayerLength");
	FUNCTION_REDIRECT("AnimInstance.GetSequencePlayerTimeFraction", "GetAnimAssetPlayerTimeFraction");
	FUNCTION_REDIRECT("AnimInstance.GetSequencePlayerTimeFromEnd", "GetAnimAssetPlayerTimeFromEnd");
	FUNCTION_REDIRECT("AnimInstance.GetSequencePlayerTimeFromEndFraction", "GetAnimAssetPlayerTimeFromEndFraction");
	FUNCTION_REDIRECT("AnimInstance.KismetInitializeAnimation", "AnimInstance.BlueprintInitializeAnimation");
	FUNCTION_REDIRECT("AnimInstance.KismetUpdateAnimation", "AnimInstance.BlueprintUpdateAnimation");
	PROPERTY_REDIRECT("AnimInstance.GetAnimAssetPlayerLength.Sequence", "AnimAsset");
	PROPERTY_REDIRECT("AnimInstance.GetAnimAssetPlayerTimeFraction.Sequence", "AnimAsset");
	PROPERTY_REDIRECT("AnimInstance.GetAnimAssetPlayerTimeFromEnd.Sequence", "AnimAsset");
	PROPERTY_REDIRECT("AnimInstance.GetAnimAssetPlayerTimeFromEndFraction.Sequence", "AnimAsset");
	PROPERTY_REDIRECT("AnimInstance.VimVertexAnims", "AnimInstance.VertexAnims");

	FUNCTION_REDIRECT("GameplayStatics.ClearSoundMode", "GameplayStatics.ClearSoundMixModifiers");
	FUNCTION_REDIRECT("GameplayStatics.GetGameInfo", "GetGameMode");
	FUNCTION_REDIRECT("GameplayStatics.GetGameReplicationInfo", "GetGameState");
	FUNCTION_REDIRECT("GameplayStatics.GetPlayerCamera", "GameplayStatics.GetPlayerCameraManager");
	FUNCTION_REDIRECT("GameplayStatics.K2_SetSoundMode", "GameplayStatics.SetBaseSoundMix");
	FUNCTION_REDIRECT("GameplayStatics.PopSoundMixModifier.InSoundMode", "InSoundMixModifier");
	FUNCTION_REDIRECT("GameplayStatics.PopSoundMode", "GameplayStatics.PopSoundMixModifier");
	FUNCTION_REDIRECT("GameplayStatics.PushSoundMixModifier.InSoundMode", "InSoundMixModifier");
	FUNCTION_REDIRECT("GameplayStatics.PushSoundMode", "GameplayStatics.PushSoundMixModifier");
	FUNCTION_REDIRECT("GameplayStatics.SetBaseSoundMix.InSoundMode", "InSoundMix");
	FUNCTION_REDIRECT("GameplayStatics.SetTimeDilation", "GameplayStatics.SetGlobalTimeDilation");

	FUNCTION_REDIRECT("KismetMaterialLibrary.CreateMaterialInstanceDynamic", "KismetMaterialLibrary.CreateDynamicMaterialInstance");
	FUNCTION_REDIRECT("KismetMaterialParameterCollectionLibrary.GetScalarParameterValue", "KismetMaterialLibrary.GetScalarParameterValue");
	FUNCTION_REDIRECT("KismetMaterialParameterCollectionLibrary.GetVectorParameterValue", "KismetMaterialLibrary.GetVectorParameterValue");
	FUNCTION_REDIRECT("KismetMaterialParameterCollectionLibrary.SetScalarParameterValue", "KismetMaterialLibrary.SetScalarParameterValue");
	FUNCTION_REDIRECT("KismetMaterialParameterCollectionLibrary.SetVectorParameterValue", "KismetMaterialLibrary.SetVectorParameterValue");

	FUNCTION_REDIRECT("KismetMathLibrary.BreakTransform.Translation", "Location");
	FUNCTION_REDIRECT("KismetMathLibrary.Conv_VectorToTransform.InTranslation", "InLocation");
	FUNCTION_REDIRECT("KismetMathLibrary.FRand", "RandomFloat");
	FUNCTION_REDIRECT("KismetMathLibrary.FRandFromStream", "RandomFloatFromStream");
	FUNCTION_REDIRECT("KismetMathLibrary.FRandRange", "RandomFloatInRange");
	FUNCTION_REDIRECT("KismetMathLibrary.FRandRangeFromStream", "RandomFloatInRangeFromStream");
	FUNCTION_REDIRECT("KismetMathLibrary.InverseTransformPosition", "KismetMathLibrary.InverseTransformLocation");
	PROPERTY_REDIRECT("KismetMathLibrary.InverseTransformLocation.Position", "Location");
	PROPERTY_REDIRECT("KismetMathLibrary.MakeTransform.Translation", "Location");
	FUNCTION_REDIRECT("KismetMathLibrary.Rand", "RandomInteger");
	FUNCTION_REDIRECT("KismetMathLibrary.RandBool", "RandomBool");
	FUNCTION_REDIRECT("KismetMathLibrary.RandBoolFromStream", "RandomBoolFromStream");
	FUNCTION_REDIRECT("KismetMathLibrary.RandFromStream", "RandomIntegerFromStream");
	FUNCTION_REDIRECT("KismetMathLibrary.RandRange", "RandomIntegerInRange");
	FUNCTION_REDIRECT("KismetMathLibrary.RandRangeFromStream", "RandomIntegerInRangeFromStream");
	FUNCTION_REDIRECT("KismetMathLibrary.RotRand", "RandomRotator");
	FUNCTION_REDIRECT("KismetMathLibrary.RotRandFromStream", "RandomRotatorFromStream");
	FUNCTION_REDIRECT("KismetMathLibrary.TransformPosition", "KismetMathLibrary.TransformLocation");
	PROPERTY_REDIRECT("KismetMathLibrary.TransformLocation.Position", "Location");
	FUNCTION_REDIRECT("KismetMathLibrary.VRand", "RandomUnitVector");
	FUNCTION_REDIRECT("KismetMathLibrary.VRandFromStream", "RandomUnitVectorFromStream");

	PROPERTY_REDIRECT("KismetSystemLibrary.CapsuleTraceMultiForObjects.ObjectsToTrace", "ObjectTypes");
	PROPERTY_REDIRECT("KismetSystemLibrary.CapsuleTraceSingleForObjects.ObjectsToTrace", "ObjectTypes");
	PROPERTY_REDIRECT("KismetSystemLibrary.LineTraceMultiForObjects.ObjectsToTrace", "ObjectTypes");
	PROPERTY_REDIRECT("KismetSystemLibrary.LineTraceSingleForObjects.ObjectsToTrace", "ObjectTypes");
	PROPERTY_REDIRECT("KismetSystemLibrary.PrintKismetWarning", "PrintWarning");
	PROPERTY_REDIRECT("KismetSystemLibrary.SphereTraceMultiForObjects.ObjectsToTrace", "ObjectTypes");
	PROPERTY_REDIRECT("KismetSystemLibrary.SphereTraceSingleForObjects.ObjectsToTrace", "ObjectTypes");

	FUNCTION_REDIRECT("AIController.ClearFocus", "AIController.K2_ClearFocus");
	FUNCTION_REDIRECT("AIController.SetFocalPoint", "AIController.K2_SetFocalPoint");
	FUNCTION_REDIRECT("AIController.SetFocus", "AIController.K2_SetFocus");
	FUNCTION_REDIRECT("ArrowComponent.SetArrowColor_New", "ArrowComponent.SetArrowColor");
	FUNCTION_REDIRECT("Character.Launch", "Character.LaunchCharacter");
	FUNCTION_REDIRECT("Controller.K2_GetActorRotation", "Controller.GetControlRotation");
	FUNCTION_REDIRECT("DecalActor.CreateMIDForDecal", "DecalActor.CreateDynamicMaterialInstance");
	FUNCTION_REDIRECT("DecalComponent.CreateMIDForDecal", "DecalComponent.CreateDynamicMaterialInstance");
	PROPERTY_REDIRECT("HUD.AddHitBox.InPos", "Position");
	PROPERTY_REDIRECT("HUD.AddHitBox.InPriority", "Priority");
	PROPERTY_REDIRECT("HUD.AddHitBox.InSize", "Size");
	PROPERTY_REDIRECT("HUD.AddHitBox.bInConsumesInput", "bConsumesInput");
	FUNCTION_REDIRECT("LevelScriptActor.BeginGame", "Actor.ReceiveBeginPlay");
	FUNCTION_REDIRECT("LevelScriptActor.LoadStreamLevel", "GameplayStatics.LoadStreamLevel");
	FUNCTION_REDIRECT("LevelScriptActor.OpenLevel", "GameplayStatics.OpenLevel");
	FUNCTION_REDIRECT("LevelScriptActor.UnloadStreamLevel", "GameplayStatics.UnloadStreamLevel");
	FUNCTION_REDIRECT("MovementComponent.ConstrainPositionToPlane", "MovementComponent.ConstrainLocationToPlane");
	PROPERTY_REDIRECT("MovementComponent.ConstrainLocationToPlane.Position", "Location");
	FUNCTION_REDIRECT("PlayerCameraManager.KismetUpdateCamera", "BlueprintUpdateCamera");
	FUNCTION_REDIRECT("PlayerController.AddLookUpInput", "PlayerController.AddPitchInput");
	FUNCTION_REDIRECT("PlayerController.AddTurnInput", "PlayerController.AddYawInput");
	PROPERTY_REDIRECT("PlayerController.DeprojectMousePositionToWorld.Direction", "WorldDirection");
	PROPERTY_REDIRECT("PlayerController.DeprojectMousePositionToWorld.WorldPosition", "WorldLocation");
	FUNCTION_REDIRECT("PrimitiveComponent.AddForceAtPosition", "PrimitiveComponent.AddForceAtLocation");
	PROPERTY_REDIRECT("PrimitiveComponent.AddForceAtLocation.Position", "Location");
	FUNCTION_REDIRECT("PrimitiveComponent.AddImpulseAtPosition", "PrimitiveComponent.AddImpulseAtLocation");
	PROPERTY_REDIRECT("PrimitiveComponent.AddImpulseAtLocation.Position", "Location");
	FUNCTION_REDIRECT("PrimitiveComponent.CreateAndSetMaterialInstanceDynamic", "PrimitiveComponent.CreateDynamicMaterialInstance");
	FUNCTION_REDIRECT("PrimitiveComponent.CreateAndSetMaterialInstanceDynamicFromMaterial", "PrimitiveComponent.CreateDynamicMaterialInstance");
	PROPERTY_REDIRECT("PrimitiveComponent.CreateDynamicMaterialInstance.Parent", "SourceMaterial");
	FUNCTION_REDIRECT("PrimitiveComponent.GetRBAngularVelocity", "GetPhysicsAngularVelocity");
	FUNCTION_REDIRECT("PrimitiveComponent.GetRBLinearVelocity", "GetPhysicsLinearVelocity");
	FUNCTION_REDIRECT("PrimitiveComponent.GetTouchingActors", "PrimitiveComponent.GetOverlappingActors");
	PROPERTY_REDIRECT("PrimitiveComponent.GetOverlappingActors.TouchingActors", "OverlappingActors");
	FUNCTION_REDIRECT("PrimitiveComponent.GetTouchingComponents", "PrimitiveComponent.GetOverlappingComponents");
	PROPERTY_REDIRECT("PrimitiveComponent.GetOverlappingComponents.TouchingComponents", "OverlappingComponents");
	FUNCTION_REDIRECT("PrimitiveComponent.KismetTraceComponent", "PrimitiveComponent.K2_LineTraceComponent");
	FUNCTION_REDIRECT("PrimitiveComponent.SetAllRBLinearVelocity", "SetAllPhysicsLinearVelocity");
	FUNCTION_REDIRECT("PrimitiveComponent.SetMovementChannel", "PrimitiveComponent.SetCollisionObjectType");
	FUNCTION_REDIRECT("PrimitiveComponent.SetRBAngularVelocity", "SetPhysicsAngularVelocity");
	FUNCTION_REDIRECT("PrimitiveComponent.SetRBLinearVelocity", "SetPhysicsLinearVelocity");
	FUNCTION_REDIRECT("ProjectileMovementComponent.StopMovement", "ProjectileMovementComponent.StopSimulating");
	FUNCTION_REDIRECT("SceneComponent.GetComponentToWorld", "K2_GetComponentToWorld");
	FUNCTION_REDIRECT("SingleAnimSkeletalComponent.GetPlayRate", "SkeletalMeshComponent.GetPlayRate");
	FUNCTION_REDIRECT("SingleAnimSkeletalComponent.GetPosition", "SkeletalMeshComponent.GetPosition");
	FUNCTION_REDIRECT("SingleAnimSkeletalComponent.IsPlaying", "SkeletalMeshComponent.IsPlaying");
	FUNCTION_REDIRECT("SingleAnimSkeletalComponent.Play", "SkeletalMeshComponent.Play");
	FUNCTION_REDIRECT("SingleAnimSkeletalComponent.SetAnim", "SkeletalMeshComponent.SetAnimation");
	FUNCTION_REDIRECT("SingleAnimSkeletalComponent.SetPlayRate", "SkeletalMeshComponent.SetPlayRate");
	FUNCTION_REDIRECT("SingleAnimSkeletalComponent.SetPosition", "SkeletalMeshComponent.SetPosition");
	FUNCTION_REDIRECT("SingleAnimSkeletalComponent.Stop", "SkeletalMeshComponent.Stop");
	FUNCTION_REDIRECT("SkinnedMeshComponent.MatchRefBone", "SkinnedMeshComponent.GetBoneIndex");

	PROPERTY_REDIRECT("AnimNotifyEvent.Time", "AnimNotifyEvent.DisplayTime");
	PROPERTY_REDIRECT("AnimSequence.BasePose", "AnimSequence.RetargetSource");
	PROPERTY_REDIRECT("AudioComponent.PitchMultiplierMax", "AudioComponent.PitchModulationMax");
	PROPERTY_REDIRECT("AudioComponent.PitchMultiplierMin", "AudioComponent.PitchModulationMin");
	PROPERTY_REDIRECT("AudioComponent.VolumeMultiplierMax", "AudioComponent.VolumeModulationMax");
	PROPERTY_REDIRECT("AudioComponent.VolumeMultiplierMin", "AudioComponent.VolumeModulationMin");
	PROPERTY_REDIRECT("BodyInstance.MovementChannel", "BodyInstance.ObjectType");
	PROPERTY_REDIRECT("BranchingPoint.Time", "BranchingPoint.DisplayTime");
	PROPERTY_REDIRECT("CapsuleComponent.CapsuleHeight", "CapsuleComponent.CapsuleHalfHeight");
	PROPERTY_REDIRECT("CharacterMovementComponent.AccelRate", "CharacterMovementComponent.MaxAcceleration");
	PROPERTY_REDIRECT("CharacterMovementComponent.BrakingDeceleration", "CharacterMovementComponent.BrakingDecelerationWalking");
	PROPERTY_REDIRECT("CharacterMovementComponent.CrouchHeight", "CharacterMovementComponent.CrouchedHalfHeight");
	PROPERTY_REDIRECT("CollisionResponseContainer.Dynamic", "CollisionResponseContainer.WorldDynamic");
	PROPERTY_REDIRECT("CollisionResponseContainer.RigidBody", "CollisionResponseContainer.PhysicsBody");
	PROPERTY_REDIRECT("CollisionResponseContainer.Static", "CollisionResponseContainer.WorldStatic");
	PROPERTY_REDIRECT("Controller.PlayerReplicationInfo", "Controller.PlayerState");
	PROPERTY_REDIRECT("DefaultPawn.DefaultPawnMovement", "DefaultPawn.MovementComponent");
	PROPERTY_REDIRECT("DirectionalLightComponent.MovableWholeSceneDynamicShadowRadius", "DirectionalLightComponent.DynamicShadowDistanceMovableLight");
	PROPERTY_REDIRECT("DirectionalLightComponent.StationaryWholeSceneDynamicShadowRadius", "DirectionalLightComponent.DynamicShadowDistanceStationaryLight");
	PROPERTY_REDIRECT("FloatingPawnMovement.AccelRate", "FloatingPawnMovement.Acceleration");
	PROPERTY_REDIRECT("FloatingPawnMovement.DecelRate", "FloatingPawnMovement.Deceleration");
	PROPERTY_REDIRECT("GameMode.GameReplicationInfoClass", "GameMode.GameStateClass");
	PROPERTY_REDIRECT("GameMode.PlayerReplicationInfoClass", "GameMode.PlayerStateClass");
	PROPERTY_REDIRECT("GameState.GameClass", "GameState.GameModeClass");
	PROPERTY_REDIRECT("K2Node_TransitionRuleGetter.AssociatedSequencePlayerNode", "K2Node_TransitionRuleGetter.AssociatedAnimAssetPlayerNode");
	PROPERTY_REDIRECT("LightComponent.InverseSquaredFalloff", "PointLightComponent.bUseInverseSquaredFalloff");
	PROPERTY_REDIRECT("LightComponentBase.Brightness", "LightComponentBase.Intensity");
	PROPERTY_REDIRECT("Material.RefractionBias", "Material.RefractionDepthBias");
	PROPERTY_REDIRECT("MaterialEditorInstanceConstant.RefractionBias", "MaterialEditorInstanceConstant.RefractionDepthBias");
	PROPERTY_REDIRECT("NavLinkProxy.NavLinks", "NavLinkProxy.PointLinks");
	PROPERTY_REDIRECT("NavLinkProxy.NavSegmentLinks", "NavLinkProxy.SegmentLinks");
	PROPERTY_REDIRECT("Pawn.ControllerClass", "Pawn.AIControllerClass");
	PROPERTY_REDIRECT("Pawn.PlayerReplicationInfo", "Pawn.PlayerState");
	PROPERTY_REDIRECT("PawnSensingComponent.SightCounterInterval", "PawnSensingComponent.SensingInterval");
	PROPERTY_REDIRECT("PawnSensingComponent.bWantsSeePlayerNotify", "PawnSensingComponent.bSeePawns");
	PROPERTY_REDIRECT("PlayerController.LookRightScale", "PlayerController.InputYawScale");
	PROPERTY_REDIRECT("PlayerController.LookUpScale", "PlayerController.InputPitchScale");
	PROPERTY_REDIRECT("PlayerController.InputYawScale", "PlayerController.InputYawScale_DEPRECATED");
	PROPERTY_REDIRECT("PlayerController.InputPitchScale", "PlayerController.InputPitchScale_DEPRECATED");
	PROPERTY_REDIRECT("PlayerController.InputRollScale", "PlayerController.InputRollScale_DEPRECATED");
	PROPERTY_REDIRECT("PlayerController.PlayerCamera", "PlayerController.PlayerCameraManager");
	PROPERTY_REDIRECT("PlayerController.PlayerCameraClass", "PlayerController.PlayerCameraManagerClass");
	PROPERTY_REDIRECT("PointLightComponent.Radius", "PointLightComponent.AttenuationRadius");
	PROPERTY_REDIRECT("PostProcessSettings.ExposureOffset", "PostProcessSettings.AutoExposureBias");
	PROPERTY_REDIRECT("PostProcessSettings.EyeAdaptationHighPercent", "PostProcessSettings.AutoExposureHighPercent");
	PROPERTY_REDIRECT("PostProcessSettings.EyeAdaptationLowPercent", "PostProcessSettings.AutoExposureLowPercent");
	PROPERTY_REDIRECT("PostProcessSettings.EyeAdaptationMaxBrightness", "PostProcessSettings.AutoExposureMaxBrightness");
	PROPERTY_REDIRECT("PostProcessSettings.EyeAdaptationMinBrightness", "PostProcessSettings.AutoExposureMinBrightness");
	PROPERTY_REDIRECT("PostProcessSettings.EyeAdaptionSpeedDown", "PostProcessSettings.AutoExposureSpeedDown");
	PROPERTY_REDIRECT("PostProcessSettings.EyeAdaptionSpeedUp", "PostProcessSettings.AutoExposureSpeedUp");
	PROPERTY_REDIRECT("PostProcessSettings.bOverride_ExposureOffset", "PostProcessSettings.bOverride_AutoExposureBias");
	PROPERTY_REDIRECT("PostProcessSettings.bOverride_EyeAdaptationHighPercent", "PostProcessSettings.bOverride_AutoExposureHighPercent");
	PROPERTY_REDIRECT("PostProcessSettings.bOverride_EyeAdaptationLowPercent", "PostProcessSettings.bOverride_AutoExposureLowPercent");
	PROPERTY_REDIRECT("PostProcessSettings.bOverride_EyeAdaptationMaxBrightness", "PostProcessSettings.bOverride_AutoExposureMaxBrightness");
	PROPERTY_REDIRECT("PostProcessSettings.bOverride_EyeAdaptationMinBrightness", "PostProcessSettings.bOverride_AutoExposureMinBrightness");
	PROPERTY_REDIRECT("PostProcessSettings.bOverride_EyeAdaptionSpeedDown", "PostProcessSettings.bOverride_AutoExposureSpeedDown");
	PROPERTY_REDIRECT("PostProcessSettings.bOverride_EyeAdaptionSpeedUp", "PostProcessSettings.bOverride_AutoExposureSpeedUp");
	PROPERTY_REDIRECT("SceneComponent.ModifyFrequency", "SceneComponent.Mobility");
	PROPERTY_REDIRECT("SceneComponent.RelativeTranslation", "SceneComponent.RelativeLocation");
	PROPERTY_REDIRECT("SceneComponent.bAbsoluteTranslation", "SceneComponent.bAbsoluteLocation");
	PROPERTY_REDIRECT("SceneComponent.bComputeBoundsOnceDuringCook", "SceneComponent.bComputeBoundsOnceForGame");
	PROPERTY_REDIRECT("SkeletalMeshComponent.AnimationBlueprint", "SkeletalMeshComponent.AnimBlueprintGeneratedClass");
	PROPERTY_REDIRECT("SlateBrush.TextureName", "SlateBrush.ResourceName");
	PROPERTY_REDIRECT("SlateBrush.TextureObject", "SlateBrush.ResourceObject");
	PROPERTY_REDIRECT("WorldSettings.DefaultGameType", "WorldSettings.DefaultGameMode");

	FCoreRedirect& PointLightComponent = CLASS_REDIRECT("PointLightComponent", "/Script/Engine.PointLightComponent");
	PointLightComponent.ValueChanges.Add(TEXT("PointLightComponent0"), TEXT("LightComponent0"));

	FCoreRedirect& DirectionalLightComponent = CLASS_REDIRECT("DirectionalLightComponent", "/Script/Engine.DirectionalLightComponent");
	DirectionalLightComponent.ValueChanges.Add(TEXT("DirectionalLightComponent0"), TEXT("LightComponent0"));

	FCoreRedirect& SpotLightComponent = CLASS_REDIRECT("SpotLightComponent", "/Script/Engine.SpotLightComponent");
	SpotLightComponent.ValueChanges.Add(TEXT("SpotLightComponent0"), TEXT("LightComponent0"));

	FCoreRedirect& ETransitionGetterType = ENUM_REDIRECT("ETransitionGetterType", "/Script/AnimGraph.ETransitionGetter");
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_ArbitraryState_GetBlendWeight"), TEXT("ETransitionGetter::ArbitraryState_GetBlendWeight"));
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_CurrentState_ElapsedTime"), TEXT("ETransitionGetter::CurrentState_ElapsedTime"));
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_CurrentState_GetBlendWeight"), TEXT("ETransitionGetter::CurrentState_GetBlendWeight"));
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_CurrentTransitionDuration"), TEXT("ETransitionGetter::CurrentTransitionDuration"));
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_SequencePlayer_GetCurrentTime"), TEXT("ETransitionGetter::AnimationAsset_GetCurrentTime"));
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_SequencePlayer_GetCurrentTimeFraction"), TEXT("ETransitionGetter::AnimationAsset_GetCurrentTimeFraction"));
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_SequencePlayer_GetLength"), TEXT("ETransitionGetter::AnimationAsset_GetLength"));
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_SequencePlayer_GetTimeFromEnd"), TEXT("ETransitionGetter::AnimationAsset_GetTimeFromEnd"));
	ETransitionGetterType.ValueChanges.Add(TEXT("TGT_SequencePlayer_GetTimeFromEndFraction"), TEXT("ETransitionGetter::AnimationAsset_GetTimeFromEndFraction"));

	FCoreRedirect& EModifyFrequency = ENUM_REDIRECT("EModifyFrequency", "/Script/Engine.EComponentMobility");
	EModifyFrequency.ValueChanges.Add(TEXT("MF_Dynamic"), TEXT("EComponentMobility::Movable"));
	EModifyFrequency.ValueChanges.Add(TEXT("MF_OccasionallyModified"), TEXT("EComponentMobility::Stationary"));
	EModifyFrequency.ValueChanges.Add(TEXT("MF_Static"), TEXT("EComponentMobility::Static"));

	FCoreRedirect& EAttachLocationType = ENUM_REDIRECT("EAttachLocationType", "/Script/Engine.EAttachLocation");
	EAttachLocationType.ValueChanges.Add(TEXT("EAttachLocationType_AbsoluteWorld"), TEXT("EAttachLocation::KeepWorldPosition"));
	EAttachLocationType.ValueChanges.Add(TEXT("EAttachLocationType_RelativeOffset"), TEXT("EAttachLocation::KeepRelativeOffset"));
	EAttachLocationType.ValueChanges.Add(TEXT("EAttachLocationType_SnapTo"), TEXT("EAttachLocation::SnapToTarget"));

	FCoreRedirect& EAxis = ENUM_REDIRECT("EAxis", "/Script/CoreUObject.EAxis");
	EAxis.ValueChanges.Add(TEXT("AXIS_BLANK"), TEXT("EAxis::None"));
	EAxis.ValueChanges.Add(TEXT("AXIS_NONE"), TEXT("EAxis::None"));
	EAxis.ValueChanges.Add(TEXT("AXIS_X"), TEXT("EAxis::X"));
	EAxis.ValueChanges.Add(TEXT("AXIS_Y"), TEXT("EAxis::Y"));
	EAxis.ValueChanges.Add(TEXT("AXIS_Z"), TEXT("EAxis::Z"));

	FCoreRedirect& EMaxConcurrentResolutionRule = ENUM_REDIRECT("EMaxConcurrentResolutionRule", "/Script/Engine.EMaxConcurrentResolutionRule");
	EMaxConcurrentResolutionRule.ValueChanges.Add(TEXT("EMaxConcurrentResolutionRule::StopFarthest"), TEXT("EMaxConcurrentResolutionRule::StopFarthestThenPreventNew"));

	FCoreRedirect& EParticleEventType = ENUM_REDIRECT("EParticleEventType", "/Script/Engine.EParticleEventType");
	EParticleEventType.ValueChanges.Add(TEXT("EPET_Kismet"), TEXT("EPET_Blueprint"));

	FCoreRedirect& ETranslucencyLightingMode = ENUM_REDIRECT("ETranslucencyLightingMode", "/Script/Engine.ETranslucencyLightingMode");
	ETranslucencyLightingMode.ValueChanges.Add(TEXT("TLM_PerPixel"), TEXT("TLM_VolumetricDirectional"));
	ETranslucencyLightingMode.ValueChanges.Add(TEXT("TLM_PerPixelNonDirectional"), TEXT("TLM_VolumetricNonDirectional"));
}

static void RegisterNativeRedirects46(TArray<FCoreRedirect>& Redirects)
{
	// 4.1-4.4 

	CLASS_REDIRECT("K2Node_CastToInterface", "/Script/BlueprintGraph.K2Node_DynamicCast");
	CLASS_REDIRECT("K2Node_MathExpression", "/Script/BlueprintGraph.K2Node_MathExpression");
	CLASS_REDIRECT("EmitterSpawnable", "/Script/Engine.Emitter");
	CLASS_REDIRECT("SlateWidgetStyleAsset", "/Script/SlateCore.SlateWidgetStyleAsset");
	CLASS_REDIRECT("SlateWidgetStyleContainerBase", "/Script/SlateCore.SlateWidgetStyleContainerBase");
	CLASS_REDIRECT("SmartNavLinkComponent", "/Script/NavigationSystem.NavLinkCustomComponent");
	CLASS_REDIRECT("WidgetBlueprint", "/Script/UMGEditor.WidgetBlueprint");

	PROPERTY_REDIRECT("AnimNotify.Received_Notify.AnimSeq", "Animation");
	PROPERTY_REDIRECT("AnimNotifyState.Received_NotifyBegin.AnimSeq", "Animation");
	PROPERTY_REDIRECT("AnimNotifyState.Received_NotifyEnd.AnimSeq", "Animation");
	PROPERTY_REDIRECT("AnimNotifyState.Received_NotifyTick.AnimSeq", "Animation");
	FUNCTION_REDIRECT("Character.IsJumping", "Character.IsJumpProvidingForce");
	PROPERTY_REDIRECT("CharacterMovementComponent.AddImpulse.InMomentum", "Impulse");
	PROPERTY_REDIRECT("CharacterMovementComponent.AddImpulse.bMassIndependent", "bVelocityChange");
	FUNCTION_REDIRECT("CharacterMovementComponent.AddMomentum", "CharacterMovementComponent.AddImpulse");
	FUNCTION_REDIRECT("Controller.GetControlledPawn", "Controller.K2_GetPawn");
	FUNCTION_REDIRECT("DefaultPawn.LookUp", "Pawn.AddControllerPitchInput");
	FUNCTION_REDIRECT("DefaultPawn.Turn", "Pawn.AddControllerYawInput");
	FUNCTION_REDIRECT("KismetSystemLibrary.EXPERIMENTAL_ShowGameCenterLeaderboard", "KismetSystemLibrary.ShowPlatformSpecificLeaderboardScreen");
	FUNCTION_REDIRECT("MovementComponent.GetMaxSpeedModifier", "MovementComponent.K2_GetMaxSpeedModifier");
	FUNCTION_REDIRECT("MovementComponent.GetModifiedMaxSpeed", "MovementComponent.K2_GetModifiedMaxSpeed");
	FUNCTION_REDIRECT("Pawn.AddLookUpInput", "Pawn.AddControllerPitchInput");
	FUNCTION_REDIRECT("Pawn.AddPitchInput", "Pawn.AddControllerPitchInput");
	FUNCTION_REDIRECT("Pawn.AddRollInput", "Pawn.AddControllerRollInput");
	FUNCTION_REDIRECT("Pawn.AddTurnInput", "Pawn.AddControllerYawInput");
	FUNCTION_REDIRECT("Pawn.AddYawInput", "Pawn.AddControllerYawInput");
	FUNCTION_REDIRECT("PawnMovementComponent.StopActiveMovement", "NavMovementComponent.StopActiveMovement");
	FUNCTION_REDIRECT("PointLightComponent.SetRadius", "PointLightComponent.SetAttenuationRadius");
	FUNCTION_REDIRECT("SkeletalMeshComponent.SetAnimBlueprint", "SkeletalMeshComponent.SetAnimInstanceClass");
	FUNCTION_REDIRECT("SkeletalMeshComponent.SetAnimClass", "SkeletalMeshComponent.SetAnimInstanceClass");
	PROPERTY_REDIRECT("SkeletalMeshComponent.SetAnimInstanceClass.NewBlueprint", "NewClass");

	PROPERTY_REDIRECT("StringClassReference.ClassName", "StringClassReference.AssetLongPathname");
	PROPERTY_REDIRECT("Material.LightingModel", "Material.ShadingModel");
	PROPERTY_REDIRECT("MaterialInstanceBasePropertyOverrides.LightingModel", "MaterialInstanceBasePropertyOverrides.ShadingModel");
	PROPERTY_REDIRECT("MaterialInstanceBasePropertyOverrides.bOverride_LightingModel", "MaterialInstanceBasePropertyOverrides.bOverride_ShadingModel");
	PROPERTY_REDIRECT("PassiveSoundMixModifier.VolumeThreshold", "PassiveSoundMixModifier.MinVolumeThreshold");
	PROPERTY_REDIRECT("PrimitiveComponent.CanBeCharacterBase", "PrimitiveComponent.CanCharacterStepUpOn");
	PROPERTY_REDIRECT("SkeletalMeshLODInfo.DisplayFactor", "SkeletalMeshLODInfo.ScreenSize");
	PROPERTY_REDIRECT("SplineMeshComponent.SplineXDir", "SplineMeshComponent.SplineUpDir");
	PROPERTY_REDIRECT("TextureFactory.LightingModel", "TextureFactory.ShadingModel");

	FCoreRedirect& EKinematicBonesUpdateToPhysics = ENUM_REDIRECT("EKinematicBonesUpdateToPhysics", "/Script/Engine.EKinematicBonesUpdateToPhysics");
	EKinematicBonesUpdateToPhysics.ValueChanges.Add(TEXT("EKinematicBonesUpdateToPhysics::SkipFixedAndSimulatingBones"), TEXT("EKinematicBonesUpdateToPhysics::SkipAllBones"));

	FCoreRedirect& EMaterialLightingModel = ENUM_REDIRECT("EMaterialLightingModel", "/Script/Engine.EMaterialShadingModel");
	EMaterialLightingModel.ValueChanges.Add(TEXT("MLM_DefaultLit"), TEXT("MSM_DefaultLit"));
	EMaterialLightingModel.ValueChanges.Add(TEXT("MLM_PreintegratedSkin"), TEXT("MSM_PreintegratedSkin"));
	EMaterialLightingModel.ValueChanges.Add(TEXT("MLM_Subsurface"), TEXT("MSM_Subsurface"));
	EMaterialLightingModel.ValueChanges.Add(TEXT("MLM_Unlit"), TEXT("MSM_Unlit"));

	FCoreRedirect& ESmartNavLinkDir = ENUM_REDIRECT("ESmartNavLinkDir", "/Script/Engine.ENavLinkDirection");
	ESmartNavLinkDir.ValueChanges.Add(TEXT("ESmartNavLinkDir::BothWays"), TEXT("ENavLinkDirection::BothWays"));
	ESmartNavLinkDir.ValueChanges.Add(TEXT("ESmartNavLinkDir::OneWay"), TEXT("ENavLinkDirection::LeftToRight"));

	// 4.5

	CLASS_REDIRECT("AIController", "/Script/AIModule.AIController");
	CLASS_REDIRECT("AIResourceInterface", "/Script/AIModule.AIResourceInterface");
	CLASS_REDIRECT("AISystem", "/Script/AIModule.AISystem");
	CLASS_REDIRECT("BTAuxiliaryNode", "/Script/AIModule.BTAuxiliaryNode");
	CLASS_REDIRECT("BTCompositeNode", "/Script/AIModule.BTCompositeNode");
	CLASS_REDIRECT("BTComposite_Selector", "/Script/AIModule.BTComposite_Selector");
	CLASS_REDIRECT("BTComposite_Sequence", "/Script/AIModule.BTComposite_Sequence");
	CLASS_REDIRECT("BTComposite_SimpleParallel", "/Script/AIModule.BTComposite_SimpleParallel");
	CLASS_REDIRECT("BTDecorator", "/Script/AIModule.BTDecorator");
	CLASS_REDIRECT("BTDecorator_Blackboard", "/Script/AIModule.BTDecorator_Blackboard");
	CLASS_REDIRECT("BTDecorator_BlackboardBase", "/Script/AIModule.BTDecorator_BlackboardBase");
	CLASS_REDIRECT("BTDecorator_BlueprintBase", "/Script/AIModule.BTDecorator_BlueprintBase");
	CLASS_REDIRECT("BTDecorator_CompareBBEntries", "/Script/AIModule.BTDecorator_CompareBBEntries");
	CLASS_REDIRECT("BTDecorator_ConeCheck", "/Script/AIModule.BTDecorator_ConeCheck");
	CLASS_REDIRECT("BTDecorator_Cooldown", "/Script/AIModule.BTDecorator_Cooldown");
	CLASS_REDIRECT("BTDecorator_DoesPathExist", "/Script/AIModule.BTDecorator_DoesPathExist");
	CLASS_REDIRECT("BTDecorator_ForceSuccess", "/Script/AIModule.BTDecorator_ForceSuccess");
	CLASS_REDIRECT("BTDecorator_KeepInCone", "/Script/AIModule.BTDecorator_KeepInCone");
	CLASS_REDIRECT("BTDecorator_Loop", "/Script/AIModule.BTDecorator_Loop");
	CLASS_REDIRECT("BTDecorator_Optional", "/Script/AIModule.BTDecorator_ForceSuccess");
	CLASS_REDIRECT("BTDecorator_ReachedMoveGoal", "/Script/AIModule.BTDecorator_ReachedMoveGoal");
	CLASS_REDIRECT("BTDecorator_TimeLimit", "/Script/AIModule.BTDecorator_TimeLimit");
	CLASS_REDIRECT("BTFunctionLibrary", "/Script/AIModule.BTFunctionLibrary");
	CLASS_REDIRECT("BTNode", "/Script/AIModule.BTNode");
	CLASS_REDIRECT("BTService", "/Script/AIModule.BTService");
	CLASS_REDIRECT("BTService_BlackboardBase", "/Script/AIModule.BTService_BlackboardBase");
	CLASS_REDIRECT("BTService_BlueprintBase", "/Script/AIModule.BTService_BlueprintBase");
	CLASS_REDIRECT("BTService_DefaultFocus", "/Script/AIModule.BTService_DefaultFocus");
	CLASS_REDIRECT("BTTaskNode", "/Script/AIModule.BTTaskNode");
	CLASS_REDIRECT("BTTask_BlackboardBase", "/Script/AIModule.BTTask_BlackboardBase");
	CLASS_REDIRECT("BTTask_BlueprintBase", "/Script/AIModule.BTTask_BlueprintBase");
	CLASS_REDIRECT("BTTask_MakeNoise", "/Script/AIModule.BTTask_MakeNoise");
	CLASS_REDIRECT("BTTask_MoveDirectlyToward", "/Script/AIModule.BTTask_MoveDirectlyToward");
	CLASS_REDIRECT("BTTask_MoveTo", "/Script/AIModule.BTTask_MoveTo");
	CLASS_REDIRECT("BTTask_PlaySound", "/Script/AIModule.BTTask_PlaySound");
	CLASS_REDIRECT("BTTask_RunBehavior", "/Script/AIModule.BTTask_RunBehavior");
	CLASS_REDIRECT("BTTask_RunEQSQuery", "/Script/AIModule.BTTask_RunEQSQuery");
	CLASS_REDIRECT("BTTask_Wait", "/Script/AIModule.BTTask_Wait");
	CLASS_REDIRECT("BehaviorTree", "/Script/AIModule.BehaviorTree");
	CLASS_REDIRECT("BehaviorTreeComponent", "/Script/AIModule.BehaviorTreeComponent");
	CLASS_REDIRECT("BehaviorTreeManager", "/Script/AIModule.BehaviorTreeManager");
	CLASS_REDIRECT("BehaviorTreeTypes", "/Script/AIModule.BehaviorTreeTypes");
	CLASS_REDIRECT("BlackboardComponent", "/Script/AIModule.BlackboardComponent");
	CLASS_REDIRECT("BlackboardData", "/Script/AIModule.BlackboardData");
	CLASS_REDIRECT("BlackboardKeyType", "/Script/AIModule.BlackboardKeyType");
	CLASS_REDIRECT("BlackboardKeyType_Bool", "/Script/AIModule.BlackboardKeyType_Bool");
	CLASS_REDIRECT("BlackboardKeyType_Class", "/Script/AIModule.BlackboardKeyType_Class");
	CLASS_REDIRECT("BlackboardKeyType_Enum", "/Script/AIModule.BlackboardKeyType_Enum");
	CLASS_REDIRECT("BlackboardKeyType_Float", "/Script/AIModule.BlackboardKeyType_Float");
	CLASS_REDIRECT("BlackboardKeyType_Int", "/Script/AIModule.BlackboardKeyType_Int");
	CLASS_REDIRECT("BlackboardKeyType_Name", "/Script/AIModule.BlackboardKeyType_Name");
	CLASS_REDIRECT("BlackboardKeyType_NativeEnum", "/Script/AIModule.BlackboardKeyType_NativeEnum");
	CLASS_REDIRECT("BlackboardKeyType_Object", "/Script/AIModule.BlackboardKeyType_Object");
	CLASS_REDIRECT("BlackboardKeyType_String", "/Script/AIModule.BlackboardKeyType_String");
	CLASS_REDIRECT("BlackboardKeyType_Vector", "/Script/AIModule.BlackboardKeyType_Vector");
	CLASS_REDIRECT("BrainComponent", "/Script/AIModule.BrainComponent");
	CLASS_REDIRECT("CrowdAgentInterface", "/Script/AIModule.CrowdAgentInterface");
	CLASS_REDIRECT("CrowdFollowingComponent", "/Script/AIModule.CrowdFollowingComponent");
	CLASS_REDIRECT("CrowdManager", "/Script/AIModule.CrowdManager");
	CLASS_REDIRECT("EQSQueryResultSourceInterface", "/Script/AIModule.EQSQueryResultSourceInterface");
	CLASS_REDIRECT("EQSRenderingComponent", "/Script/AIModule.EQSRenderingComponent");
	CLASS_REDIRECT("EQSTestingPawn", "/Script/AIModule.EQSTestingPawn");
	CLASS_REDIRECT("EnvQuery", "/Script/AIModule.EnvQuery");
	CLASS_REDIRECT("EnvQueryContext", "/Script/AIModule.EnvQueryContext");
	CLASS_REDIRECT("EnvQueryContext_BlueprintBase", "/Script/AIModule.EnvQueryContext_BlueprintBase");
	CLASS_REDIRECT("EnvQueryContext_Item", "/Script/AIModule.EnvQueryContext_Item");
	CLASS_REDIRECT("EnvQueryContext_Querier", "/Script/AIModule.EnvQueryContext_Querier");
	CLASS_REDIRECT("EnvQueryGenerator", "/Script/AIModule.EnvQueryGenerator");
	CLASS_REDIRECT("EnvQueryGenerator_Composite", "/Script/AIModule.EnvQueryGenerator_Composite");
	CLASS_REDIRECT("EnvQueryGenerator_OnCircle", "/Script/AIModule.EnvQueryGenerator_OnCircle");
	CLASS_REDIRECT("EnvQueryGenerator_PathingGrid", "/Script/AIModule.EnvQueryGenerator_PathingGrid");
	CLASS_REDIRECT("EnvQueryGenerator_ProjectedPoints", "/Script/AIModule.EnvQueryGenerator_ProjectedPoints");
	CLASS_REDIRECT("EnvQueryGenerator_SimpleGrid", "/Script/AIModule.EnvQueryGenerator_SimpleGrid");
	CLASS_REDIRECT("EnvQueryItemType", "/Script/AIModule.EnvQueryItemType");
	CLASS_REDIRECT("EnvQueryItemType_Actor", "/Script/AIModule.EnvQueryItemType_Actor");
	CLASS_REDIRECT("EnvQueryItemType_ActorBase", "/Script/AIModule.EnvQueryItemType_ActorBase");
	CLASS_REDIRECT("EnvQueryItemType_Direction", "/Script/AIModule.EnvQueryItemType_Direction");
	CLASS_REDIRECT("EnvQueryItemType_Point", "/Script/AIModule.EnvQueryItemType_Point");
	CLASS_REDIRECT("EnvQueryItemType_VectorBase", "/Script/AIModule.EnvQueryItemType_VectorBase");
	CLASS_REDIRECT("EnvQueryManager", "/Script/AIModule.EnvQueryManager");
	CLASS_REDIRECT("EnvQueryOption", "/Script/AIModule.EnvQueryOption");
	CLASS_REDIRECT("EnvQueryTest", "/Script/AIModule.EnvQueryTest");
	CLASS_REDIRECT("EnvQueryTest_Distance", "/Script/AIModule.EnvQueryTest_Distance");
	CLASS_REDIRECT("EnvQueryTest_Dot", "/Script/AIModule.EnvQueryTest_Dot");
	CLASS_REDIRECT("EnvQueryTest_Pathfinding", "/Script/AIModule.EnvQueryTest_Pathfinding");
	CLASS_REDIRECT("EnvQueryTest_Trace", "/Script/AIModule.EnvQueryTest_Trace");
	CLASS_REDIRECT("EnvQueryTypes", "/Script/AIModule.EnvQueryTypes");
	CLASS_REDIRECT("KismetAIAsyncTaskProxy", "/Script/AIModule.AIAsyncTaskBlueprintProxy");
	CLASS_REDIRECT("KismetAIHelperLibrary", "/Script/AIModule.AIBlueprintHelperLibrary");
	CLASS_REDIRECT("PathFollowingComponent", "/Script/AIModule.PathFollowingComponent");
	CLASS_REDIRECT("PawnSensingComponent", "/Script/AIModule.PawnSensingComponent");

	STRUCT_REDIRECT("SReply", "/Script/UMG.EventReply");

	PROPERTY_REDIRECT("Actor.AddTickPrerequisiteActor.DependentActor", "PrerequisiteActor");
	FUNCTION_REDIRECT("Actor.AttachRootComponentTo", "Actor.K2_AttachRootComponentTo");
	FUNCTION_REDIRECT("Actor.AttachRootComponentToActor", "Actor.K2_AttachRootComponentToActor");
	FUNCTION_REDIRECT("Actor.SetTickPrerequisite", "Actor.AddTickPrerequisiteActor");
	PROPERTY_REDIRECT("BTTask_MoveDirectlyToward.bForceMoveToLocation", "bDisablePathUpdateOnGoalLocationChange");
	PROPERTY_REDIRECT("KismetSystemLibrary.DrawDebugPlane.Loc", "Location");
	PROPERTY_REDIRECT("KismetSystemLibrary.DrawDebugPlane.P", "PlaneCoordinates");
	FUNCTION_REDIRECT("KismetSystemLibrary.EXPERIMENTAL_CloseAdBanner", "KismetSystemLibrary.ForceCloseAdBanner");
	FUNCTION_REDIRECT("KismetSystemLibrary.EXPERIMENTAL_HideAdBanner", "KismetSystemLibrary.HideAdBanner");
	FUNCTION_REDIRECT("KismetSystemLibrary.EXPERIMENTAL_ShowAdBanner", "KismetSystemLibrary.ShowAdBanner");
	FUNCTION_REDIRECT("LightComponent.SetBrightness", "LightComponent.SetIntensity");
	FUNCTION_REDIRECT("NavigationPath.GetPathLenght", "NavigationPath.GetPathLength");
	FUNCTION_REDIRECT("Pawn.GetMovementInputVector", "Pawn.K2_GetMovementInputVector");
	FUNCTION_REDIRECT("PawnMovementComponent.GetInputVector", "PawnMovementComponent.GetPendingInputVector");
	FUNCTION_REDIRECT("SceneComponent.AttachTo", "SceneComponent.K2_AttachTo");
	FUNCTION_REDIRECT("SkyLightComponent.SetBrightness", "SkyLightComponent.SetIntensity");

	// 4.6

	CLASS_REDIRECT("ControlPointMeshComponent", "/Script/Landscape.ControlPointMeshComponent");
	CLASS_REDIRECT("Landscape", "/Script/Landscape.Landscape");
	CLASS_REDIRECT("LandscapeComponent", "/Script/Landscape.LandscapeComponent");
	CLASS_REDIRECT("LandscapeGizmoActiveActor", "/Script/Landscape.LandscapeGizmoActiveActor");
	CLASS_REDIRECT("LandscapeGizmoActor", "/Script/Landscape.LandscapeGizmoActor");
	CLASS_REDIRECT("LandscapeGizmoRenderComponent", "/Script/Landscape.LandscapeGizmoRenderComponent");
	CLASS_REDIRECT("LandscapeHeightfieldCollisionComponent", "/Script/Landscape.LandscapeHeightfieldCollisionComponent");
	CLASS_REDIRECT("LandscapeInfo", "/Script/Landscape.LandscapeInfo");
	CLASS_REDIRECT("LandscapeInfoMap", "/Script/Landscape.LandscapeInfoMap");
	CLASS_REDIRECT("LandscapeLayerInfoObject", "/Script/Landscape.LandscapeLayerInfoObject");
	CLASS_REDIRECT("LandscapeMaterialInstanceConstant", "/Script/Landscape.LandscapeMaterialInstanceConstant");
	CLASS_REDIRECT("LandscapeMeshCollisionComponent", "/Script/Landscape.LandscapeMeshCollisionComponent");
	CLASS_REDIRECT("LandscapeProxy", "/Script/Landscape.LandscapeProxy");
	CLASS_REDIRECT("LandscapeSplineControlPoint", "/Script/Landscape.LandscapeSplineControlPoint");
	CLASS_REDIRECT("LandscapeSplineSegment", "/Script/Landscape.LandscapeSplineSegment");
	CLASS_REDIRECT("LandscapeSplinesComponent", "/Script/Landscape.LandscapeSplinesComponent");
	CLASS_REDIRECT("MaterialExpressionLandscapeLayerBlend", "/Script/Landscape.MaterialExpressionLandscapeLayerBlend");
	CLASS_REDIRECT("MaterialExpressionLandscapeLayerCoords", "/Script/Landscape.MaterialExpressionLandscapeLayerCoords");
	CLASS_REDIRECT("MaterialExpressionLandscapeLayerSwitch", "/Script/Landscape.MaterialExpressionLandscapeLayerSwitch");
	CLASS_REDIRECT("MaterialExpressionLandscapeLayerWeight", "/Script/Landscape.MaterialExpressionLandscapeLayerWeight");
	CLASS_REDIRECT("MaterialExpressionLandscapeVisibilityMask", "/Script/Landscape.MaterialExpressionLandscapeVisibilityMask");
	CLASS_REDIRECT("MaterialExpressionTerrainLayerCoords", "/Script/Landscape.MaterialExpressionLandscapeLayerCoords");
	CLASS_REDIRECT("MaterialExpressionTerrainLayerSwitch", "/Script/Landscape.MaterialExpressionLandscapeLayerSwitch");
	CLASS_REDIRECT("MaterialExpressionTerrainLayerWeight", "/Script/Landscape.MaterialExpressionLandscapeLayerWeight");
	CLASS_REDIRECT("ReverbVolume", "/Script/Engine.AudioVolume");
	CLASS_REDIRECT("ReverbVolumeToggleable", "/Script/Engine.AudioVolume");

	STRUCT_REDIRECT("KeyboardEvent", "/Script/SlateCore.KeyEvent");
	STRUCT_REDIRECT("KeyboardFocusEvent", "/Script/SlateCore.FocusEvent");

	FUNCTION_REDIRECT("Actor.AddActorLocalOffset", "Actor.K2_AddActorLocalOffset");
	FUNCTION_REDIRECT("Actor.AddActorLocalRotation", "Actor.K2_AddActorLocalRotation");
	FUNCTION_REDIRECT("Actor.AddActorLocalTransform", "Actor.K2_AddActorLocalTransform");
	FUNCTION_REDIRECT("Actor.AddActorLocalTranslation", "Actor.K2_AddActorLocalOffset");
	PROPERTY_REDIRECT("Actor.K2_AddActorLocalOffset.DeltaTranslation", "DeltaLocation");
	FUNCTION_REDIRECT("Actor.AddActorWorldOffset", "Actor.K2_AddActorWorldOffset");
	FUNCTION_REDIRECT("Actor.AddActorWorldRotation", "Actor.K2_AddActorWorldRotation");
	FUNCTION_REDIRECT("Actor.AddActorWorldTransform", "Actor.K2_AddActorWorldTransform");
	FUNCTION_REDIRECT("Actor.SetActorLocation", "Actor.K2_SetActorLocation");
	FUNCTION_REDIRECT("Actor.SetActorLocationAndRotation", "Actor.K2_SetActorLocationAndRotation");
	FUNCTION_REDIRECT("Actor.SetActorRelativeLocation", "Actor.K2_SetActorRelativeLocation");
	PROPERTY_REDIRECT("Actor.K2_SetActorRelativeLocation.NewRelativeTranslation", "NewRelativeLocation");
	FUNCTION_REDIRECT("Actor.SetActorRelativeRotation", "Actor.K2_SetActorRelativeRotation");
	FUNCTION_REDIRECT("Actor.SetActorRelativeTransform", "Actor.K2_SetActorRelativeTransform");
	FUNCTION_REDIRECT("Actor.SetActorRelativeTranslation", "Actor.K2_SetActorRelativeLocation");
	FUNCTION_REDIRECT("Actor.SetActorTransform", "Actor.K2_SetActorTransform");
	FUNCTION_REDIRECT("BTFunctionLibrary.GetBlackboard", "BTFunctionLibrary.GetOwnersBlackboard");
	FUNCTION_REDIRECT("KismetMathLibrary.NearlyEqual_RotatorRotator", "EqualEqual_RotatorRotator");
	FUNCTION_REDIRECT("KismetMathLibrary.NearlyEqual_VectorVector", "EqualEqual_VectorVector");
	FUNCTION_REDIRECT("KismetMathLibrary.ProjectOnTo", "ProjectVectorOnToVector");
	PROPERTY_REDIRECT("KismetMathLibrary.ProjectVectorOnToVector.X", "V");
	PROPERTY_REDIRECT("KismetMathLibrary.ProjectVectorOnToVector.Y", "Target");
	PROPERTY_REDIRECT("LightComponent.SetIntensity.NewBrightness", "NewIntensity");
	FUNCTION_REDIRECT("SceneComponent.AddLocalOffset", "SceneComponent.K2_AddLocalOffset");
	FUNCTION_REDIRECT("SceneComponent.AddLocalRotation", "SceneComponent.K2_AddLocalRotation");
	FUNCTION_REDIRECT("SceneComponent.AddLocalTransform", "SceneComponent.K2_AddLocalTransform");
	FUNCTION_REDIRECT("SceneComponent.AddLocalTranslation", "SceneComponent.K2_AddLocalOffset");
	PROPERTY_REDIRECT("SceneComponent.K2_AddLocalOffset.DeltaTranslation", "DeltaLocation");
	FUNCTION_REDIRECT("SceneComponent.AddRelativeLocation", "SceneComponent.K2_AddRelativeLocation");
	PROPERTY_REDIRECT("SceneComponent.K2_AddRelativeLocation.DeltaTranslation", "DeltaLocation");
	FUNCTION_REDIRECT("SceneComponent.AddRelativeRotation", "SceneComponent.K2_AddRelativeRotation");
	FUNCTION_REDIRECT("SceneComponent.AddRelativeTranslation", "SceneComponent.K2_AddRelativeLocation");
	FUNCTION_REDIRECT("SceneComponent.AddWorldOffset", "SceneComponent.K2_AddWorldOffset");
	FUNCTION_REDIRECT("SceneComponent.AddWorldRotation", "SceneComponent.K2_AddWorldRotation");
	FUNCTION_REDIRECT("SceneComponent.AddWorldTransform", "SceneComponent.K2_AddWorldTransform");
	FUNCTION_REDIRECT("SceneComponent.SetRelativeLocation", "SceneComponent.K2_SetRelativeLocation");
	PROPERTY_REDIRECT("SceneComponent.K2_SetRelativeLocation.NewTranslation", "NewLocation");
	FUNCTION_REDIRECT("SceneComponent.SetRelativeRotation", "SceneComponent.K2_SetRelativeRotation");
	FUNCTION_REDIRECT("SceneComponent.SetRelativeTransform", "SceneComponent.K2_SetRelativeTransform");
	FUNCTION_REDIRECT("SceneComponent.SetRelativeTranslation", "SceneComponent.K2_SetRelativeLocation");
	FUNCTION_REDIRECT("SceneComponent.SetWorldLocation", "SceneComponent.K2_SetWorldLocation");
	PROPERTY_REDIRECT("SceneComponent.K2_SetWorldLocation.NewTranslation", "NewLocation");
	FUNCTION_REDIRECT("SceneComponent.SetWorldRotation", "SceneComponent.K2_SetWorldRotation");
	FUNCTION_REDIRECT("SceneComponent.SetWorldTransform", "SceneComponent.K2_SetWorldTransform");
	FUNCTION_REDIRECT("SceneComponent.SetWorldTranslation", "SceneComponent.K2_SetWorldLocation");
	PROPERTY_REDIRECT("SkyLightComponent.SetIntensity.NewBrightness", "NewIntensity");
}

static void RegisterNativeRedirects49(TArray<FCoreRedirect>& Redirects)
{
	// 4.7

	CLASS_REDIRECT("EdGraphNode_Comment", "/Script/UnrealEd.EdGraphNode_Comment");
	CLASS_REDIRECT("K2Node_Comment", "/Script/UnrealEd.EdGraphNode_Comment");
	CLASS_REDIRECT("VimBlueprintFactory", "/Script/UnrealEd.AnimBlueprintFactory");

	FUNCTION_REDIRECT("Actor.SetTickEnabled", "Actor.SetActorTickEnabled");
	PROPERTY_REDIRECT("UserWidget.OnKeyboardFocusLost.InKeyboardFocusEvent", "InFocusEvent");
	PROPERTY_REDIRECT("UserWidget.OnControllerAnalogValueChanged.ControllerEvent", "InAnalogInputEvent");
	PROPERTY_REDIRECT("UserWidget.OnControllerButtonPressed.ControllerEvent", "InKeyEvent");
	PROPERTY_REDIRECT("UserWidget.OnControllerButtonReleased.ControllerEvent", "InKeyEvent");
	PROPERTY_REDIRECT("UserWidget.OnKeyDown.InKeyboardEvent", "InKeyEvent");
	PROPERTY_REDIRECT("UserWidget.OnKeyUp.InKeyboardEvent", "InKeyEvent");
	PROPERTY_REDIRECT("UserWidget.OnKeyboardFocusReceived.InKeyboardFocusEvent", "InFocusEvent");
	PROPERTY_REDIRECT("UserWidget.OnPreviewKeyDown.InKeyboardEvent", "InKeyEvent");
	
	PROPERTY_REDIRECT("MeshComponent.Materials", "MeshComponent.OverrideMaterials");
	PROPERTY_REDIRECT("Pawn.AutoPossess", "Pawn.AutoPossessPlayer");

	FCoreRedirect& ECollisionChannel = ENUM_REDIRECT("ECollisionChannel", "/Script/Engine.ECollisionChannel");
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_Default"), TEXT("ECC_Visibility"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_Dynamic"), TEXT("ECC_WorldDynamic"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_OverlapAll"), TEXT("ECC_OverlapAll_Deprecated"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_OverlapAllDynamic"), TEXT("ECC_OverlapAll_Deprecated"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_OverlapAllDynamic_Deprecated"), TEXT("ECC_OverlapAll_Deprecated"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_OverlapAllStatic"), TEXT("ECC_OverlapAll_Deprecated"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_OverlapAllStatic_Deprecated"), TEXT("ECC_OverlapAll_Deprecated"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_PawnMovement"), TEXT("ECC_Pawn"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_RigidBody"), TEXT("ECC_PhysicsBody"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_RigidBodyInteractable"), TEXT("ECC_PhysicsBody"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_TouchAll"), TEXT("ECC_OverlapAll_Deprecated"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_TouchAllDynamic"), TEXT("ECC_OverlapAll_Deprecated"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_TouchAllStatic"), TEXT("ECC_OverlapAll_Deprecated"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_VehicleMovement"), TEXT("ECC_Vehicle"));
	ECollisionChannel.ValueChanges.Add(TEXT("ECC_WorldTrace"), TEXT("ECC_WorldStatic"));

	// 4.8

	CLASS_REDIRECT("EditorGameAgnosticSettings", "/Script/UnrealEd.EditorSettings");
	CLASS_REDIRECT("FoliageType", "/Script/Foliage.FoliageType");
	CLASS_REDIRECT("FoliageType_InstancedStaticMesh", "/Script/Foliage.FoliageType_InstancedStaticMesh");
	CLASS_REDIRECT("InstancedFoliageActor", "/Script/Foliage.InstancedFoliageActor");
	CLASS_REDIRECT("InstancedFoliageSettings", "/Script/Foliage.FoliageType_InstancedStaticMesh");
	CLASS_REDIRECT("InteractiveFoliageComponent", "/Script/Foliage.InteractiveFoliageComponent");
	CLASS_REDIRECT("ProceduralFoliage", "/Script/Foliage.ProceduralFoliageSpawner");
	CLASS_REDIRECT("ProceduralFoliageActor", "/Script/Foliage.ProceduralFoliageVolume");
	
	STRUCT_REDIRECT("ProceduralFoliageTypeData", "/Script/Foliage.FoliageTypeObject");

	FCoreRedirect& EComponentCreationMethod = ENUM_REDIRECT("EComponentCreationMethod", "/Script/Engine.EComponentCreationMethod");
	EComponentCreationMethod.ValueChanges.Add(TEXT("EComponentCreationMethod::ConstructionScript"), TEXT("EComponentCreationMethod::SimpleConstructionScript"));

	FCoreRedirect& EConstraintTransform = ENUM_REDIRECT("EConstraintTransform", "/Script/Engine.EConstraintTransform");
	EConstraintTransform.ValueChanges.Add(TEXT("EConstraintTransform::Absoluate"), TEXT("EConstraintTransform::Absolute"));

	FCoreRedirect& ELockedAxis = ENUM_REDIRECT("ELockedAxis", "/Script/Engine.EDOFMode");
	ELockedAxis.ValueChanges.Add(TEXT("Custom"), TEXT("EDOFMode::CustomPlane"));
	ELockedAxis.ValueChanges.Add(TEXT("X"), TEXT("EDOFMode::YZPlane"));
	ELockedAxis.ValueChanges.Add(TEXT("Y"), TEXT("EDOFMode::XZPlane"));
	ELockedAxis.ValueChanges.Add(TEXT("Z"), TEXT("EDOFMode::XYPlane"));

	FCoreRedirect& EEndPlayReason = ENUM_REDIRECT("EEndPlayReason", "/Script/Engine.EEndPlayReason");
	EEndPlayReason.ValueChanges.Add(TEXT("EEndPlayReason::ActorDestroyed"), TEXT("EEndPlayReason::Destroyed"));

	FUNCTION_REDIRECT("ActorComponent.ReceiveInitializeComponent", "ActorComponent.ReceiveBeginPlay");
	FUNCTION_REDIRECT("ActorComponent.ReceiveUninitializeComponent", "ActorComponent.ReceiveEndPlay");

	PROPERTY_REDIRECT("CameraComponent.bUseControllerViewRotation", "CameraComponent.bUsePawnControlRotation");
	PROPERTY_REDIRECT("CameraComponent.bUsePawnViewRotation", "CameraComponent.bUsePawnControlRotation");
	PROPERTY_REDIRECT("CharacterMovementComponent.AirSpeed", "CharacterMovementComponent.MaxFlySpeed");
	PROPERTY_REDIRECT("CharacterMovementComponent.CrouchedSpeedPercent", "CharacterMovementComponent.CrouchedSpeedMultiplier");
	PROPERTY_REDIRECT("CharacterMovementComponent.GroundSpeed", "CharacterMovementComponent.MaxWalkSpeed");
	PROPERTY_REDIRECT("CharacterMovementComponent.JumpZ", "CharacterMovementComponent.JumpZVelocity");
	PROPERTY_REDIRECT("CharacterMovementComponent.WaterSpeed", "CharacterMovementComponent.MaxSwimSpeed");
	PROPERTY_REDIRECT("CharacterMovementComponent.bCrouchMovesCharacterDown", "CharacterMovementComponent.bCrouchMaintainsBaseLocation");
	PROPERTY_REDIRECT("CharacterMovementComponent.bOrientToMovement", "CharacterMovementComponent.bOrientRotationToMovement");
	PROPERTY_REDIRECT("FunctionalTest.GetAdditionalTestFinishedMessage", "FunctionalTest.OnAdditionalTestFinishedMessageRequest");
	PROPERTY_REDIRECT("FunctionalTest.WantsToRunAgain", "FunctionalTest.OnWantsReRunCheck");
	PROPERTY_REDIRECT("ProjectileMovementComponent.Speed", "ProjectileMovementComponent.InitialSpeed");
	PROPERTY_REDIRECT("SpringArmComponent.bUseControllerViewRotation", "SpringArmComponent.bUsePawnControlRotation");
	PROPERTY_REDIRECT("SpringArmComponent.bUsePawnViewRotation", "SpringArmComponent.bUsePawnControlRotation");
	PROPERTY_REDIRECT("BodyInstance.CustomLockedAxis", "BodyInstance.CustomDOFPlaneNormal");
	PROPERTY_REDIRECT("BodyInstance.LockedAxisMode", "BodyInstance.DOFMode");
	PROPERTY_REDIRECT("CharacterMovementComponent.NavMeshProjectionCapsuleHeightScaleDown", "CharacterMovementComponent.NavMeshProjectionHeightScaleDown");
	PROPERTY_REDIRECT("CharacterMovementComponent.NavMeshProjectionCapsuleHeightScaleUp", "CharacterMovementComponent.NavMeshProjectionHeightScaleUp");
	PROPERTY_REDIRECT("LandscapeSplineControlPoint.MeshComponent", "LandscapeSplineControlPoint.LocalMeshComponent");
	PROPERTY_REDIRECT("LandscapeSplineSegment.MeshComponents", "LandscapeSplineSegment.LocalMeshComponents");
	PROPERTY_REDIRECT("ProceduralFoliageComponent.Overlap", "ProceduralFoliageComponent.TileOverlap");
	PROPERTY_REDIRECT("ProceduralFoliageComponent.ProceduralFoliage", "ProceduralFoliageComponent.FoliageSpawner");
	PROPERTY_REDIRECT("ProceduralFoliageSpawner.Types", "ProceduralFoliageSpawner.FoliageTypes");
	PROPERTY_REDIRECT("SpriteGeometryCollection.Polygons", "SpriteGeometryCollection.Shapes");

	// 4.9

	CLASS_REDIRECT("EditorUserSettings", "/Script/UnrealEd.EditorPerProjectUserSettings");	
	CLASS_REDIRECT("MovieScene", "/Script/MovieScene.MovieScene");
	CLASS_REDIRECT("MovieScene3DTransformSection", "/Script/MovieSceneTracks.MovieScene3DTransformSection");
	CLASS_REDIRECT("MovieScene3DTransformTrack", "/Script/MovieSceneTracks.MovieScene3DTransformTrack");
	CLASS_REDIRECT("MovieSceneAudioSection", "/Script/MovieSceneTracks.MovieSceneAudioSection");
	CLASS_REDIRECT("MovieSceneAudioTrack", "/Script/MovieSceneTracks.MovieSceneAudioTrack");
	CLASS_REDIRECT("MovieSceneBoolTrack", "/Script/MovieSceneTracks.MovieSceneBoolTrack");
	CLASS_REDIRECT("MovieSceneByteSection", "/Script/MovieSceneTracks.MovieSceneByteSection");
	CLASS_REDIRECT("MovieSceneByteTrack", "/Script/MovieSceneTracks.MovieSceneByteTrack");
	CLASS_REDIRECT("MovieSceneColorSection", "/Script/MovieSceneTracks.MovieSceneColorSection");
	CLASS_REDIRECT("MovieSceneColorTrack", "/Script/MovieSceneTracks.MovieSceneColorTrack");
	CLASS_REDIRECT("MovieSceneFloatSection", "/Script/MovieSceneTracks.MovieSceneFloatSection");
	CLASS_REDIRECT("MovieSceneFloatTrack", "/Script/MovieSceneTracks.MovieSceneFloatTrack");
	CLASS_REDIRECT("MovieSceneParticleSection", "/Script/MovieSceneTracks.MovieSceneParticleSection");
	CLASS_REDIRECT("MovieSceneParticleTrack", "/Script/MovieSceneTracks.MovieSceneParticleTrack");
	CLASS_REDIRECT("MovieScenePropertyTrack", "/Script/MovieSceneTracks.MovieScenePropertyTrack");
	CLASS_REDIRECT("MovieSceneSection", "/Script/MovieScene.MovieSceneSection");
	CLASS_REDIRECT("MovieSceneTrack", "/Script/MovieScene.MovieSceneTrack");

	PACKAGE_REDIRECT("/Script/MovieSceneCore", "/Script/MovieScene");
	PACKAGE_REDIRECT("/Script/MovieSceneCoreTypes", "/Script/MovieSceneTracks");

	STRUCT_REDIRECT("Anchors", "/Script/Slate.Anchors");
	STRUCT_REDIRECT("AnimNode_BoneDrivenController", "/Script/AnimGraphRuntime.AnimNode_BoneDrivenController");
	STRUCT_REDIRECT("AnimNode_CopyBone", "/Script/AnimGraphRuntime.AnimNode_CopyBone");
	STRUCT_REDIRECT("AnimNode_HandIKRetargeting", "/Script/AnimGraphRuntime.AnimNode_HandIKRetargeting");
	STRUCT_REDIRECT("AnimNode_LookAt", "/Script/AnimGraphRuntime.AnimNode_LookAt");
	STRUCT_REDIRECT("AnimNode_ModifyBone", "/Script/AnimGraphRuntime.AnimNode_ModifyBone");
	STRUCT_REDIRECT("AnimNode_RotationMultiplier", "/Script/AnimGraphRuntime.AnimNode_RotationMultiplier");
	STRUCT_REDIRECT("AnimNode_SkeletalControlBase", "/Script/AnimGraphRuntime.AnimNode_SkeletalControlBase");
	STRUCT_REDIRECT("AnimNode_SpringBone", "/Script/AnimGraphRuntime.AnimNode_SpringBone");
	STRUCT_REDIRECT("AnimNode_Trail", "/Script/AnimGraphRuntime.AnimNode_Trail");
	STRUCT_REDIRECT("AnimNode_TwoBoneIK", "/Script/AnimGraphRuntime.AnimNode_TwoBoneIK");
	STRUCT_REDIRECT("MovieSceneEditorData", "/Script/MovieScene.MovieSceneEditorData");
	STRUCT_REDIRECT("MovieSceneObjectBinding", "/Script/MovieScene.MovieSceneBinding");
	STRUCT_REDIRECT("MovieScenePossessable", "/Script/MovieScene.MovieScenePossessable");
	STRUCT_REDIRECT("MovieSceneSpawnable", "/Script/MovieScene.MovieSceneSpawnable");
	STRUCT_REDIRECT("SpritePolygon", "/Script/Paper2D.SpriteGeometryShape");
	STRUCT_REDIRECT("SpritePolygonCollection", "/Script/Paper2D.SpriteGeometryCollection");

	FUNCTION_REDIRECT("GameplayStatics.PlayDialogueAttached", "GameplayStatics.SpawnDialogueAttached");
	FUNCTION_REDIRECT("GameplayStatics.PlaySoundAttached", "GameplayStatics.SpawnSoundAttached");
	FUNCTION_REDIRECT("KismetMathLibrary.BreakRot", "KismetMathLibrary.BreakRotator");
	FUNCTION_REDIRECT("KismetMathLibrary.MakeRot", "KismetMathLibrary.MakeRotator");
	FUNCTION_REDIRECT("KismetMathLibrary.MapRange", "KismetMathLibrary.MapRangeUnclamped");
	FUNCTION_REDIRECT("PrimitiveComponent.GetMoveIgnoreActors", "PrimitiveComponent.CopyArrayOfMoveIgnoreActors");
	FUNCTION_REDIRECT("SplineComponent.GetNumSplinePoints", "SplineComponent.GetNumberOfSplinePoints");
	FUNCTION_REDIRECT("VerticalBox.AddChildVerticalBox", "VerticalBox.AddChildToVerticalBox");
	
	PROPERTY_REDIRECT("ComponentKey.VariableGuid", "ComponentKey.AssociatedGuid");
	PROPERTY_REDIRECT("ComponentKey.VariableName", "ComponentKey.SCSVariableName");
	PROPERTY_REDIRECT("FoliageType.InitialMaxAge", "FoliageType.MaxInitialAge");
	PROPERTY_REDIRECT("FoliageType.bGrowsInShade", "FoliageType.bSpawnsInShade");
	PROPERTY_REDIRECT("MemberReference.MemberParentClass", "MemberReference.MemberParent");
	PROPERTY_REDIRECT("SimpleMemberReference.MemberParentClass", "SimpleMemberReference.MemberParent");
	PROPERTY_REDIRECT("SoundNodeModPlayer.SoundMod", "SoundNodeModPlayer.SoundModAssetPtr");
	PROPERTY_REDIRECT("SoundNodeWavePlayer.SoundWave", "SoundNodeWavePlayer.SoundWaveAssetPtr");

	ENUM_REDIRECT("ECheckBoxState", "/Script/SlateCore.ECheckBoxState");
	ENUM_REDIRECT("ESlateCheckBoxState", "/Script/SlateCore.ECheckBoxState");
	ENUM_REDIRECT("EAxisOption", "/Script/Engine.EAxisOption");
	ENUM_REDIRECT("EBoneAxis", "/Script/Engine.EBoneAxis");
	ENUM_REDIRECT("EBoneModificationMode", "/Script/AnimGraphRuntime.EBoneModificationMode");
	ENUM_REDIRECT("EComponentType", "/Script/Engine.EComponentType");
	ENUM_REDIRECT("EInterpolationBlend", "/Script/AnimGraphRuntime.EInterpolationBlend");
}

PRAGMA_POP_ATTRIBUTE_MINSIZE_FUNCTIONS

void FCoreRedirects::RegisterNativeRedirectsUnderWriteLock(FScopeCoreRedirectsWriteLockedContext& LockedContext)
{
	// Registering redirects here instead of in baseengine.ini is faster to parse and can clean up the ini, but is not required
	TArray<FCoreRedirect> Redirects;

	RegisterNativeRedirects40(Redirects);
	RegisterNativeRedirects46(Redirects);
	RegisterNativeRedirects49(Redirects);

	// 4.10 and later are in baseengine.ini

	AddRedirectListUnderWriteLock(Redirects, TEXT("RegisterNativeRedirects"), LockedContext);
}
#else
void FCoreRedirects::RegisterNativeRedirectsUnderWriteLock(const FCoreRedirectorScopeLock& HeldLock, FCoreRedirectsContext& Context)
{
}
#endif // UE_WITH_CORE_REDIRECTS
