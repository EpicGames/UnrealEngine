// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/CoreRedirects/CoreRedirectsContext.h"
#include "UObject/CoreRedirects.h"

#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"

namespace UE::CoreRedirects::Private
{
	FCoreRedirectsContext GlobalCoreRedirectContext;
	thread_local FCoreRedirectsContext* ThreadCoreRedirectContext = nullptr;
}

FCoreRedirectsContext::FCoreRedirectsContext()
	: bInitialized(false)
	, Flags(EFlags::Default)
	, bValidatedOnce(false)
{
}

FCoreRedirectsContext::FCoreRedirectsContext(const FCoreRedirectsContext& OtherContext)
	: bInitialized(OtherContext.bInitialized.load())
	, Flags(OtherContext.Flags.load())
	, bValidatedOnce(OtherContext.bValidatedOnce)
	, ConfigKeyMap(OtherContext.ConfigKeyMap)
	, RedirectTypeMap(OtherContext.RedirectTypeMap)
{
}

FCoreRedirectsContext& FCoreRedirectsContext::operator=(const FCoreRedirectsContext& Other)
{
	bInitialized.store(Other.bInitialized.load(std::memory_order_relaxed), std::memory_order_relaxed);
	Flags.store(Other.Flags.load(std::memory_order_relaxed), std::memory_order_relaxed);
	bValidatedOnce = Other.bValidatedOnce;
	ConfigKeyMap = Other.ConfigKeyMap;
	RedirectTypeMap = Other.RedirectTypeMap;

	return *this;
}

void FCoreRedirectsContext::InitializeContext()
{
	if (bInitialized.load(std::memory_order_relaxed))
	{
		return;
	}

	{
		FScopeCoreRedirectsWriteLockedContext WriteLockedContext(*this);

#if !UE_BUILD_SHIPPING
		if (FParse::Param(FCommandLine::Get(), TEXT("FullDebugCoreRedirects")))
		{
			// Enable debug mode and set to maximum verbosity
			Flags.store(Flags.load(std::memory_order_relaxed) | EFlags::DebugMode, std::memory_order_relaxed);
			UE_SET_LOG_VERBOSITY(LogCoreRedirects, VeryVerbose);
			FCoreDelegates::OnFEngineLoopInitComplete.AddStatic(FCoreRedirects::ValidateAllRedirects);
		}
		else if (FParse::Param(FCommandLine::Get(), TEXT("DebugCoreRedirects")))
		{
			// Enable debug mode and increase log levels but don't show every message
			Flags.store(Flags.load(std::memory_order_relaxed) | EFlags::DebugMode, std::memory_order_relaxed);
			UE_SET_LOG_VERBOSITY(LogCoreRedirects, Verbose);
			FCoreDelegates::OnFEngineLoopInitComplete.AddStatic(FCoreRedirects::ValidateAllRedirects);
		}
#endif

		// Setup map
		ConfigKeyMap.Add(TEXT("ObjectRedirects"), ECoreRedirectFlags::Type_Object);
		ConfigKeyMap.Add(TEXT("ClassRedirects"), ECoreRedirectFlags::Type_Class);
		ConfigKeyMap.Add(TEXT("StructRedirects"), ECoreRedirectFlags::Type_Struct);
		ConfigKeyMap.Add(TEXT("EnumRedirects"), ECoreRedirectFlags::Type_Enum);
		ConfigKeyMap.Add(TEXT("FunctionRedirects"), ECoreRedirectFlags::Type_Function);
		ConfigKeyMap.Add(TEXT("PropertyRedirects"), ECoreRedirectFlags::Type_Property);
		ConfigKeyMap.Add(TEXT("PackageRedirects"), ECoreRedirectFlags::Type_Package);
		ConfigKeyMap.Add(TEXT("AssetRedirects"), ECoreRedirectFlags::Type_Asset);

		FCoreRedirects::RegisterNativeRedirectsUnderWriteLock(WriteLockedContext);

		// Prepopulate RedirectTypeMap entries that some threads write to after the engine goes multi-threaded.
		// Most RedirectTypeMap entries are written to only from InitUObject's call to ReadRedirectsFromIni, and at that point the Engine is single-threaded.
		// Known missing packages and plugin loads can add entries to existing lists but will not add brand new types.
		// Taking advantage of this, we treat the list of Key/Value pairs of RedirectTypeMap as immutable and read from it without synchronization.
		// Note that the values for those written-during-multithreading entries need to be synchronized; it is only the list of Key/Value pairs that is immutable.
		RedirectTypeMap.FindOrAdd(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Category_Removed | ECoreRedirectFlags::Option_MissingLoad);

		bInitialized.store(true, std::memory_order_release);
	}
}

FCoreRedirectsContext& FCoreRedirectsContext::GetGlobalContext()
{
	using namespace UE::CoreRedirects::Private;
	return GlobalCoreRedirectContext; // -V558
}

FCoreRedirectsContext& FCoreRedirectsContext::GetThreadContext()
{
	using namespace UE::CoreRedirects::Private;
	return ThreadCoreRedirectContext ? *ThreadCoreRedirectContext : GetGlobalContext();
}

void FCoreRedirectsContext::SetThreadContext(FCoreRedirectsContext& NewContext)
{
	using namespace UE::CoreRedirects::Private;
	ThreadCoreRedirectContext = &NewContext;
}

FScopeCoreRedirectsContext::FScopeCoreRedirectsContext()
	: ContextToRestore(FCoreRedirectsContext::GetThreadContext())
{
	FCoreRedirectsContext::SetThreadContext(ScopeContext);
	ScopeContext.InitializeContext();
}

FScopeCoreRedirectsContext::FScopeCoreRedirectsContext(FCoreRedirectsContext& ContextToCopyFrom)
	: ScopeContext(ContextToCopyFrom)
	, ContextToRestore(FCoreRedirectsContext::GetThreadContext())
{
	FCoreRedirectsContext::SetThreadContext(ScopeContext);
	ScopeContext.InitializeContext();
}

FScopeCoreRedirectsContext::~FScopeCoreRedirectsContext()
{
	FCoreRedirectsContext::SetThreadContext(ContextToRestore);
}

uint32 FCoreRedirectsContext::FRWLockWithExclusiveRecursion::LoadWriteLockOwnerThreadIdRelaxed() const
{
	uint32 ThreadId;
	UE_AUTORTFM_OPEN
	{
		ThreadId = WriteLockOwnerThreadId.load(std::memory_order_relaxed);
	};
	return ThreadId;
}

void FCoreRedirectsContext::FRWLockWithExclusiveRecursion::StoreWriteLockOwnerThreadId(uint32 ThreadId, std::memory_order Order)
{
	if (AutoRTFM::IsClosed())
	{
		// If an abort occurs, we want to restore the variable's previous value, so we use
		// `RecordOpenWrite` here so that AutoRTFM can undo our work. (We aren't overly
		// concerned with atomicity as all loads are done in `memory_order_relaxed`.)
		UE_AUTORTFM_OPEN
		{
			AutoRTFM::RecordOpenWrite(&ThreadId);
			WriteLockOwnerThreadId.store(ThreadId, Order);
		};
	}
	else
	{
		WriteLockOwnerThreadId.store(ThreadId, Order);
	}
}

void FCoreRedirectsContext::FRWLockWithExclusiveRecursion::ReadLock()
{
	uint32 InitialLockOwner = LoadWriteLockOwnerThreadIdRelaxed();

	// Avoid calling GetCurrentThreadId in the common case where we start out unlocked
	// If we are unowned at the start, then there are three cases to consider:
	// 1. We are unlocked. In that case we'll pass the InitialLockOwner check and call ReadLock()
	// 2. We are already in a ReadLock() state. Taking the read lock again will cause us to deadlock. 
	// That is part of the contract for this lock so it's okay.
	// 3. We are write locked, in which case we'll enter the first block 
	if (InitialLockOwner != 0)
	{
		// If we are initially write locked, there are two possibilities:
		// 1. This thread owns the write lock. In that case, it's safe to just bump the recursion count
		// 2. Another thread owns the write lock, in which case we want to block with a ReadLock()
		if (InitialLockOwner == FPlatformTLS::GetCurrentThreadId())
		{
			// Allow recursive read locking by bumping the recursion count
			RecursionCount++;
		}
		else
		{
			InternalLock.ReadLock();
		}
	}
	else
	{
		// We may or may not still be locked at this point if another thread took a WriteLock
		// but it doesn't matter because in either case we want to take a ReadLock and potentially wait
		InternalLock.ReadLock();
	}
}

void FCoreRedirectsContext::FRWLockWithExclusiveRecursion::WriteLock()
{
	uint32 CurrentThreadId = FPlatformTLS::GetCurrentThreadId();
	if (LoadWriteLockOwnerThreadIdRelaxed() != CurrentThreadId)
	{
		InternalLock.WriteLock();
		StoreWriteLockOwnerThreadId(CurrentThreadId);
	}
	RecursionCount++;
}

void FCoreRedirectsContext::FRWLockWithExclusiveRecursion::WriteUnlock()
{
	ensure(LoadWriteLockOwnerThreadIdRelaxed() == FPlatformTLS::GetCurrentThreadId());
	ensure(RecursionCount > 0);
	RecursionCount--;
	if (RecursionCount == 0)
	{
		StoreWriteLockOwnerThreadId(0, std::memory_order_relaxed);
		InternalLock.WriteUnlock();
	}
}

void FCoreRedirectsContext::FRWLockWithExclusiveRecursion::ReadUnlock()
{
	if (RecursionCount > 0)
	{
		ensureMsgf(LoadWriteLockOwnerThreadIdRelaxed() == FPlatformTLS::GetCurrentThreadId(),
			TEXT("Called ReadUnlock() on a lock held exclusively by another thread."));
		RecursionCount--;
	}
	else
	{
		InternalLock.ReadUnlock();
	}
}