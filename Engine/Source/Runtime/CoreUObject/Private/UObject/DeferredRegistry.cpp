// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/DeferredRegistry.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "Templates/Casts.h"

#if WITH_RELOAD

void ReloadProcessObject(UScriptStruct* ScriptStruct, const TCHAR* RenamePrefix)
{
	// Make sure the old struct is not used by anything
	ScriptStruct->ClearFlags(RF_Standalone | RF_Public);
	ScriptStruct->RemoveFromRoot();
	const FName OldRename = MakeUniqueObjectName(GetTransientPackage(), ScriptStruct->GetClass(), *FString::Printf(TEXT("%s_%s"), RenamePrefix, *ScriptStruct->GetName()));
	ScriptStruct->Rename(*OldRename.ToString(), GetTransientPackage());
}

void ReloadProcessObject(UEnum* Enum, const TCHAR* RenamePrefix)
{	
	// Before renaming the enum (moving the enum to transient package) remove its enum value names from the enum Primary list as the list is populated with enum package name
	Enum->RemoveNamesFromPrimaryList();
	// Make sure the old enum is not used by anything
	Enum->ClearFlags(RF_Standalone | RF_Public);
	Enum->RemoveFromRoot();
	const FName OldRename = MakeUniqueObjectName(GetTransientPackage(), Enum->GetClass(), *FString::Printf(TEXT("%s_%s"), RenamePrefix, *Enum->GetName()));
	Enum->Rename(*OldRename.ToString(), GetTransientPackage());
}

void ReloadProcessObject(UClass* Class, const TCHAR* RenamePrefix)
{
	// Make sure enums de-register their names BEFORE we create the new class, otherwise there will be name conflicts.
	// This needs to be done before renaming the class (moving it to transient package) as enum names are stored with the package name
	TArray<UObject*> ClassSubobjects;
	GetObjectsWithOuter(Class, ClassSubobjects);
	for (auto ClassSubobject : ClassSubobjects)
	{
		if (auto Enum = dynamic_cast<UEnum*>(ClassSubobject))
		{
			Enum->RemoveNamesFromPrimaryList();
		}
	}

	// Rename the old class and move it to transient package
	Class->RemoveFromRoot();
	Class->ClearFlags(RF_Standalone | RF_Public);
	Class->GetDefaultObject()->RemoveFromRoot();
	Class->GetDefaultObject()->ClearFlags(RF_Standalone | RF_Public);
	const FName OldClassRename = MakeUniqueObjectName(GetTransientPackage(), Class->GetClass(), *FString::Printf(TEXT("%s_%s"), RenamePrefix, *Class->GetName()));
	Class->Rename(*OldClassRename.ToString(), GetTransientPackage());
	Class->SetFlags(RF_Transient);
	Class->AddToRoot();

	// Reset singletons for any child fields
#if WITH_LIVE_CODING
	for (UField* Field = Class->Children; Field != nullptr; Field = Field->Next)
	{
		UFunction* Function = Cast<UFunction>(Field);
		if (Function != nullptr && Function->SingletonPtr != nullptr)
		{
			*Function->SingletonPtr = nullptr;
		}
	}
#endif
}

void ReloadProcessObject(UPackage* Package, const TCHAR* RenamePrefix)
{
#if WITH_LIVE_CODING
	for (UFunction* Function : Package->GetReloadDelegates())
	{
		if (Function->SingletonPtr != nullptr)
		{
			*Function->SingletonPtr = nullptr;
		}
	}
#endif
}

#endif // WITH_RELOAD
