// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#if !UE_WITH_CONSTINIT_UOBJECT

#include "Modules/ModuleManager.h"
#include "UObject/Class.h"
#include "UObject/Reload.h"
#include "UObject/UObjectBase.h"

#if WITH_RELOAD
inline bool operator != (const FClassReloadVersionInfo& lhs, const FClassReloadVersionInfo& rhs)
{
	return lhs.Size != rhs.Size || lhs.Hash != rhs.Hash;
}

inline bool operator != (const FStructReloadVersionInfo& lhs, const FStructReloadVersionInfo& rhs)
{
	return lhs.Size != rhs.Size || lhs.Hash != rhs.Hash;
}

inline bool operator != (const FEnumReloadVersionInfo& lhs, const FEnumReloadVersionInfo& rhs)
{
	return lhs.Hash != rhs.Hash;
}

inline bool operator != (const FPackageReloadVersionInfo& lhs, const FPackageReloadVersionInfo& rhs)
{
	return lhs.BodyHash != rhs.BodyHash || lhs.DeclarationsHash != rhs.DeclarationsHash;
}

inline bool operator != (const FPartialReloadVersionInfo& lhs, const FPartialReloadVersionInfo& rhs)
{
	return lhs.Size != rhs.Size || lhs.Hash != rhs.Hash;
}

void ReloadProcessObject(UScriptStruct* Object, const TCHAR* RenamePrefix);
void ReloadProcessObject(UEnum* Object, const TCHAR* RenamePrefix);
void ReloadProcessObject(UClass* Object, const TCHAR* RenamePrefix);
void ReloadProcessObject(UPackage* Object, const TCHAR* RenamePrefix);
#endif

template <typename T>
class TDeferredRegistry
{
public:
	using TInfo = T;
	using TType = typename T::TType;
	using TVersion = typename T::TVersion;

	enum class AddResult
	{
		New,
		ExistingNoChange,
		ExistingChanged,
	};

private:

	using FPackageAndNameKey = TTuple<FName, FName>;

public:

	/**
	* Maintains information about a pending registration
	*/
	struct FRegistrant
	{
		TType* (*RegisterFn)(ETypeConstructPhase);
		const TCHAR* PackageName;
		const TCHAR* Name;
		TInfo* Info;
#if WITH_RELOAD
		TType* OldSingleton;
#endif

#if WITH_RELOAD
		bool bHasChanged;
#endif
	};

public:

	/// <summary>
	/// Adds the given registration information for the given object.  Objects are either classes, structs, or enumerations.
	/// </summary>
	/// <param name="InRegister">Function that can either returns a fully initialize instance of the object (outer) or an allocated but uninitialized instance of the object (This is used only by UClass)</param>
	/// <param name="InPackageName">Name of the package</param>
	/// <param name="InName">Name of the object</param>
	/// <param name="InInfo">Persistent information about the object</param>
	/// <param name="InVersion">Version information for this incarnation of the object</param>
	AddResult AddRegistration(TType* (*InRegister)(ETypeConstructPhase), const TCHAR* InPackageName, const TCHAR* InName, TInfo& InInfo, const TVersion& InVersion)
	{
#if WITH_RELOAD
		const FPackageAndNameKey Key = FPackageAndNameKey{ InPackageName, InName };
		TInfo** ExistingInfo = InfoMap.Find(Key);

		bool bHasChanged = !ExistingInfo || (*ExistingInfo)->ReloadVersionInfo != InVersion;
		InInfo.ReloadVersionInfo = InVersion;
		TType* OldSingleton = ExistingInfo ? ((*ExistingInfo)->InnerSingleton ? (*ExistingInfo)->InnerSingleton : (*ExistingInfo)->OuterSingleton) : nullptr;

		bool bAdd = true;
		if (ExistingInfo)
		{
			if (IReload* Reload = GetActiveReloadInterface())
			{
				bAdd = Reload->GetEnableReinstancing(bHasChanged);
			}
			if (bAdd)
			{
				if (!bHasChanged)
				{
					// With live coding, the existing might be the same as the new info.  
					// We still invoke the copy method to allow UClasses to clear the singletons.
					UpdateSingletons(InInfo, **ExistingInfo);
				}
				*ExistingInfo = &InInfo;
			}
		}
		else
		{
			InfoMap.Add(Key, &InInfo);
		}
		if (bAdd)
		{
			Registrations.Add(FRegistrant{ InRegister, InPackageName, InName, &InInfo, OldSingleton, bHasChanged });
		}
		return ExistingInfo == nullptr ? AddResult::New : (bHasChanged ? AddResult::ExistingChanged : AddResult::ExistingNoChange);
#else
		Registrations.Add(FRegistrant{ InRegister, InPackageName, InName, &InInfo });
		return AddResult::New;
#endif
	}

	/**
	* Find an existing registration for reload when the version information matches
	*/
#if WITH_RELOAD
	TType* FindMatchingObject(const TCHAR* InPackageName, const TCHAR* InName, const TVersion& InVersion)
	{
		const FPackageAndNameKey Key = FPackageAndNameKey{ InPackageName, InName };
		TInfo** ExistingInfo = InfoMap.Find(Key);
		if (ExistingInfo == nullptr)
		{
			return nullptr;
		}
		if ((*ExistingInfo)->ReloadVersionInfo != InVersion)
		{
			return nullptr;
		}
		return (*ExistingInfo)->InnerSingleton;
	}
#endif

	/**
	* Return the collection of registrations
	*/
	TArray<FRegistrant>& GetRegistrations()
	{
		return Registrations;
	}

	/**
	* Return true if we have registrations
	*/
	bool HasPendingRegistrations()
	{
		return ProcessedRegistrations < Registrations.Num();
	}

	/**
	* Create all the packages for the packages associated with the registrations
	*/
	void DoPendingPackageRegistrations()
	{
		for (int32 Index = ProcessedRegistrations, Num = Registrations.Num(); Index < Num; ++Index)
		{
			CreatePackage(Registrations[Index].PackageName);
		}
	}

	/**
	* Invoke the inner registration method for all the registrations
	*/
	void DoPendingInnerRegistrations(bool UpdateCounter)
	{
		int32 Num = Registrations.Num();
		for (int32 Index = ProcessedRegistrations; Index < Num; ++Index)
		{
			InnerRegister(Registrations[Index]);
		}

		if (UpdateCounter)
		{
			ProcessedRegistrations = Num;
		}
	}

	/**
	* Invoke the outer registration method for all the registrations
	*/
	void DoPendingOuterRegistrations(bool UpdateCounter)
	{
		int32 Num = Registrations.Num();
		for (int32 Index = ProcessedRegistrations; Index < Num; ++Index)
		{
			OuterRegister(Registrations[Index]);
		}

		if (UpdateCounter)
		{
			ProcessedRegistrations = Num;
		}
	}

	/**
	* Invoke the outer registration method for all the registrations and invoke the given function with the resulting object
	*/
	template <typename FuncType>
	void DoPendingOuterRegistrations(bool UpdateCounter, FuncType&& InOnRegistration)
	{
		int32 Num = Registrations.Num();
		for (int32 Index = ProcessedRegistrations; Index < Num; ++Index)
		{
			TType* Object = OuterRegister(Registrations[Index]);
			InOnRegistration(Registrations[Index].PackageName, *Object);
		}
		if (UpdateCounter)
		{
			ProcessedRegistrations = Num;
		}
	}

	/**
	* Invoke the register function for an inner registrant
	*/
	static TType* InnerRegister(const FRegistrant& Registrant)
	{
		return Registrant.RegisterFn(ETypeConstructPhase::Inner);
	}

	/**
	* Invoke the register function for an outer registrant
	*/
	static TType* OuterRegister(const FRegistrant& Registrant)
	{
		return Registrant.RegisterFn(ETypeConstructPhase::Outer);
	}

	/**
	* Process existing objects that have changed
	*/
	void ProcessChangedObjects(bool InvokeOuterRegisterFunction = false)
	{
#if WITH_RELOAD
		IReload* Reload = GetActiveReloadInterface();
		if (Reload != nullptr)
		{
			const TCHAR* Prefix = Reload->GetPrefix();
			for (const FRegistrant& Registrant : Registrations)
			{
				if (Registrant.bHasChanged && Registrant.OldSingleton != nullptr)
				{
					UE_LOGF(LogClass, Log, "%ls %ls Reload.", GetRegistryName(), *Registrant.OldSingleton->GetName());

					// Reset the cached class construct info.
					Registrant.Info->InnerSingleton = nullptr;
					Registrant.Info->OuterSingleton = nullptr;

					// Perform any other processing needed for existing objects
					ReloadProcessObject(Registrant.OldSingleton, Prefix);

					// If requested, invoke the registration function to make sure it has been recreated
					if (InvokeOuterRegisterFunction)
					{
						Registrant.RegisterFn(ETypeConstructPhase::Outer);
					}
				}
			}
		}
#endif
	}

	/**
	* Send notifications to the reload system
	*/
#if WITH_RELOAD
	void NotifyReload(IReload& Reload)
	{
		for (const FRegistrant& Registrant : Registrations)
		{
			TType* Old = Registrant.OldSingleton;
			TType* New = Registrant.bHasChanged ? OuterRegister(Registrant) : Old;
			Reload.NotifyChange(New, Old);
		}
	}
#endif

	/**
	* Clear the pending registrations
	*/
	void EmptyRegistrations()
	{
		TArray<FRegistrant> Local(MoveTemp(Registrations));
		ProcessedRegistrations = 0;
	}

	/**
	* When registering a new info, when the contents haven't changed, we copy singleton pointers 
	*/
	void UpdateSingletons(TInfo& NewInfo, const TInfo& OldInfo)
	{
		NewInfo.InnerSingleton = OldInfo.InnerSingleton;
		NewInfo.OuterSingleton = OldInfo.OuterSingleton;
	}

	/**
	* Return the registry singleton
	*/
	static TDeferredRegistry& Get()
	{
		static TDeferredRegistry Registry;
		return Registry;
	}

#if WITH_RELOAD
	const TCHAR* GetRegistryName();
#endif

private:
	TArray<FRegistrant> Registrations;
	int32 ProcessedRegistrations = 0;

#if WITH_RELOAD
	TMap<FPackageAndNameKey, TInfo*> InfoMap;
#endif
};

// Specialization for classes for the LLM_SCOPE
template <>
inline typename TDeferredRegistry<FClassRegistrationInfo>::TType* TDeferredRegistry<FClassRegistrationInfo>::InnerRegister(const TDeferredRegistry<FClassRegistrationInfo>::FRegistrant& Registrant)
{
	LLM_DEFINE_STATIC_TAG(UObject_TDeferredRegistry);
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_TDeferredRegistry);
	return Registrant.RegisterFn(ETypeConstructPhase::Inner);
}

// Specialization for class to not copy the existing singletons when reloading a class that hasn't changed.
// In the case of live coding, we must clear out the pointer for classes to make sure it invokes the registration code.
template <>
inline void TDeferredRegistry<FClassRegistrationInfo>::UpdateSingletons(FClassRegistrationInfo& NewInfo, const FClassRegistrationInfo& OldInfo)
{
	NewInfo.InnerSingleton = nullptr;
	NewInfo.OuterSingleton = nullptr;
}

#if WITH_RELOAD
template <> inline const TCHAR* TDeferredRegistry<FClassRegistrationInfo>::GetRegistryName() { return TEXT("UClass"); }
template <> inline const TCHAR* TDeferredRegistry<FEnumRegistrationInfo>::GetRegistryName() { return TEXT("UEnum"); }
template <> inline const TCHAR* TDeferredRegistry<FStructRegistrationInfo>::GetRegistryName() { return TEXT("UScriptStruct"); }
template <> inline const TCHAR* TDeferredRegistry<FPackageRegistrationInfo>::GetRegistryName() { return TEXT("UPackage"); }
#endif

using FClassDeferredRegistry = TDeferredRegistry<FClassRegistrationInfo>;
using FEnumDeferredRegistry = TDeferredRegistry<FEnumRegistrationInfo>;
using FStructDeferredRegistry = TDeferredRegistry<FStructRegistrationInfo>;
using FPackageDeferredRegistry = TDeferredRegistry<FPackageRegistrationInfo>;
using FPartialDeferredRegistry = TDeferredRegistry<FPartialRegistrationInfo>;

#endif // !UE_WITH_CONSTINIT_UOBJECT
