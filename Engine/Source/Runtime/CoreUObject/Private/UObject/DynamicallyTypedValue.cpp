// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/DynamicallyTypedValue.h"

UE::FDynamicallyTypedValueType& UE::FDynamicallyTypedValue::NullType()
{
	struct FNullType : FDynamicallyTypedValueType
	{
		FNullType(): FDynamicallyTypedValueType(0, 0, EContainsReferences::DoesNot) {}

		virtual void MarkReachable(FReferenceCollector& Collector) override {}
		virtual void MarkValueReachable(void* Data, FReferenceCollector& Collector) const override {}

		virtual void InitializeValue(void* Data) const override {}
		virtual void InitializeValueFrom(void* DestData, const void* SourceData, const UE::Verse::FRuntimeType& SourceType) const override {}
		virtual void InitializeValueFromCopy(void* DestData, const void* SourceData) const override {}
		virtual void DestroyValue(void* Data) const override {}

#if WITH_VERSE_VM
		virtual ::Verse::VValue ToVValue(::Verse::FAllocationContext Context, const void* Data) const override { return {}; }
#endif

		virtual void SerializeValue(FStructuredArchive::FSlot Slot, void* Data, const void* DefaultData) const override {}

		virtual uint32 GetValueHash(const void* Data) const override { return 0; }
		virtual bool AreIdentical(const void* DataA, const void* DataB) const override { return true; }

		virtual void ForEachSubobject(const void* Data, TFunctionRef<void(const UObject*)> ObjectFunc) const {};
	};

	static FNullType Singleton;
	return Singleton;
}