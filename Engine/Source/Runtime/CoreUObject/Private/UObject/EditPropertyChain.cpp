// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Templates/Casts.h"
#include "Containers/List.h"
#include "UObject/UnrealType.h"

/* ==========================================================================================================
	FEditPropertyChain
========================================================================================================== */
/**
 * Sets the ActivePropertyNode to the node associated with the property specified.
 *
 * @param	NewActiveProperty	the FProperty that is currently being evaluated by Pre/PostEditChange
 *
 * @return	true if the ActivePropertyNode was successfully changed to the node associated with the property
 *			specified.  false if there was no node corresponding to that property.
 */
bool FEditPropertyChain::SetActivePropertyNode( FProperty* NewActiveProperty )
{
	bool bResult = false;

	TDoubleLinkedListNode* PropertyNode = FindNode(NewActiveProperty);
	if ( PropertyNode != NULL )
	{
		ActivePropertyNode = PropertyNode;
		bResult = true;
	}

	return bResult;
}

/**
 * Sets the ActiveMemberPropertyNode to the node associated with the property specified.
 *
 * @param	NewActiveMemberProperty		the member FProperty which contains the property currently being evaluated
 *										by Pre/PostEditChange
 *
 * @return	true if the ActiveMemberPropertyNode was successfully changed to the node associated with the
 *			property specified.  false if there was no node corresponding to that property.
 */
bool FEditPropertyChain::SetActiveMemberPropertyNode( FProperty* NewActiveMemberProperty )
{
	bool bResult = false;

	TDoubleLinkedListNode* PropertyNode = FindNode(NewActiveMemberProperty);
	if ( PropertyNode != NULL )
	{
		ActiveMemberPropertyNode = PropertyNode;
		bResult = true;
	}

	return bResult;
}

bool FEditPropertyChain::IsArchetypeInstanceAffected( UObject* InInstance ) const
{
	return !bFilterAffectedInstances || AffectedInstances.Contains(InInstance);
}

/**
 * Returns the node corresponding to the currently active property.
 */
FEditPropertyChain::TDoubleLinkedListNode* FEditPropertyChain::GetActiveNode() const
{
	return ActivePropertyNode;
}

/**
 * Returns the node corresponding to the currently active property, or if the currently active property
 * is not a member variable (i.e. inside of a struct/array), the node corresponding to the member variable
 * which contains the currently active property.
 */
FEditPropertyChain::TDoubleLinkedListNode* FEditPropertyChain::GetActiveMemberNode() const
{
	return ActiveMemberPropertyNode;
}

/**
 * Updates the size reported by Num().  Child classes can use this function to conveniently
 * hook into list additions/removals.
 *
 * This version ensures that the ActivePropertyNode either points to a valid node, or NULL if this list is empty.
 *
 * @param	NewListSize		the new size for this list
 */
void FEditPropertyChain::SetListSize( int32 NewListSize )
{
	int32 PreviousListSize = Num();
	TDoubleLinkedList<FProperty*>::SetListSize(NewListSize);

	if ( Num() == 0 )
	{
		ActivePropertyNode = ActiveMemberPropertyNode = NULL;
	}
	else if ( PreviousListSize != NewListSize )
	{
		// if we have no active property node, set it to the tail of the list, which would be the property that was
		// actually changed by the user (assuming this FEditPropertyChain is being used by the code that handles changes
		// to property values in the editor)
		if ( ActivePropertyNode == NULL )
		{
			ActivePropertyNode = GetTail();
		}

		// now figure out which property the ActiveMemberPropertyNode should be pointing at
		if ( ActivePropertyNode != NULL )
		{
			// start at the currently active property
			TDoubleLinkedListNode* PropertyNode = ActivePropertyNode;

			// then iterate backwards through the chain, searching for the first property which is owned by a UClass - this is our member property
			for ( TIterator It(PropertyNode); It; --It )
			{
				// if we've found the member property, we can stop here
				if (It->GetOwner<UClass>() != nullptr)
				{
					PropertyNode = It.GetNode();
					break;
				}
			}

			ActiveMemberPropertyNode = PropertyNode;
		}
	}
}
