// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/ConfigCacheIni.h"
#include "String/ParseTokens.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/Package.h"
#include "UObject/MetaData.h"
#include "UObject/PropertyHelper.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/DevObjectVersion.h"
#include "UObject/CoreObjectVersion.h"
#include "UObject/CoreRedirects.h"
#include "UObject/LinkerLoad.h"

DEFINE_LOG_CATEGORY(LogEnum);

/*-----------------------------------------------------------------------------
	UEnum implementation.
-----------------------------------------------------------------------------*/

FRWLock UEnum::AllEnumNamesLock;
TMap<FName, TMap<FName, UEnum*> > UEnum::AllEnumNames;

UEnum::FNameData::~FNameData()
{
	Empty();
}

void UEnum::FNameData::DeleteNames()
{
	const bool bDeleteNames = (TaggedNames & TagDynamic) == TagDynamic;
	if (bDeleteNames)
	{
		FMemory::Free(reinterpret_cast<void*>(TaggedNames & MaskPointer));
	}
	TaggedNames = 0;
}

void UEnum::FNameData::DeleteValues()
{
	const bool bDeleteValues = (TaggedValues & TagDynamic) == TagDynamic;
	if (bDeleteValues)
	{
		FMemory::Free(reinterpret_cast<void*>(TaggedValues & MaskPointer));
	}
	TaggedValues = 0;
}

FName* UEnum::FNameData::AllocateNames(int32 InNum)
{
	checkfSlow(TaggedNames == 0 || ((TaggedNames & MaskPointer) != TagDynamic), TEXT("AllocateNames should only be called when names are unallocated or statically allocated"));
	FName* TmpNames = (FName*)FMemory::Malloc(sizeof(FName) * InNum);
	TaggedNames = (UPTRINT)TmpNames | TagDynamic;
	return TmpNames;
}
int64* UEnum::FNameData::AllocateValues(int32 InNum)
{
	checkfSlow(TaggedValues == 0 || ((TaggedValues & MaskPointer) != TagDynamic), TEXT("AllocateValues should only be called when values are unallocated or statically allocated"));
	int64* Values = (int64*)FMemory::Malloc(sizeof(int64) * InNum);
	TaggedValues = (UPTRINT)Values | TagDynamic;
	return Values;
}

void UEnum::FNameData::Empty()
{
	DeleteNames();
	DeleteValues();
	NumValues = 0;
}

#if UE_WITH_CONSTINIT_UOBJECT
void UEnum::FNameData::InitializeFromCompiledInData()
{
	checkfSlow((TaggedNames & MaskPointer) == TaggedNames, TEXT("InitializeFromCompiledInData should only be called when names are statically allocated"));
	const UTF8CHAR* const* StaticNames = StaticNamesUTF8;
	FName* TmpNames = AllocateNames(NumValues);
	// Values are already initialized with static data
	for (int32 i=0; i < NumValues; ++i)
	{
		TmpNames[i] = FName(StaticNames[i]);
	}
}
#endif // UE_WITH_CONSTINIT_UOBJECT

// Initialize from legacy serialization of names without values
void UEnum::FNameData::Initialize(const TArray<FName>& InNames)
{
	check(NumValues == 0);
	NumValues = InNames.Num();
	FName* TmpNames = AllocateNames(InNames.Num());
	int64* TmpValues = AllocateValues(InNames.Num());
	for (int32 i=0; i < NumValues; ++i)
	{
		TmpNames[i] = InNames[i];
		TmpValues[i] = i;
	}
}

// Initialize from legacy serialization of names with byte values
void UEnum::FNameData::Initialize(const TArray<TPair<FName, uint8>>& InNames)
{
	check(NumValues == 0);
	NumValues = InNames.Num();
	FName* TmpNames = AllocateNames(InNames.Num());
	int64* TmpValues = AllocateValues(InNames.Num());
	for (int32 i=0; i < NumValues; ++i)
	{
		TmpNames[i] = InNames[i].Key;
		TmpValues[i] = InNames[i].Value;
	}
}

// Initialize from names and int64 values
void UEnum::FNameData::Initialize(const TArray<TPair<FName, int64>>& InNames)
{
	check(NumValues == 0);
	NumValues = InNames.Num();
	FName* TmpNames = AllocateNames(InNames.Num());
	int64* TmpValues = AllocateValues(InNames.Num());
	for (int32 i=0; i < NumValues; ++i)
	{
		TmpNames[i] = InNames[i].Key;
		TmpValues[i] = InNames[i].Value;
	}
}

// Initialize from names and int64 values with a placeholder to generate a max element
void UEnum::FNameData::InitializeWithPlaceholder(const TArray<TPair<FName, int64>>& InNames, int64 MaxValue)
{
	check(NumValues == 0);
	NumValues = InNames.Num();
	FName* TmpNames = AllocateNames(InNames.Num() + 1);
	int64* TmpValues = AllocateValues(InNames.Num() + 1);
	for (int32 i=0; i < NumValues; ++i)
	{
		TmpNames[i] = InNames[i].Key;
		TmpValues[i] = InNames[i].Value;
	}
	TmpNames[NumValues] = FName();
	TmpValues[NumValues] = MaxValue;
}

// If necessary, replace values with sequential values counting up from zero
void UEnum::FNameData::MakeValuesSequentialFromZero()
{
	bool bChange = false;
	TConstArrayView<int64> Values = GetValues();
	for (int32 i=0; i < NumValues; ++i)
	{
		if (Values[i] != i)
		{
			bChange = true;
			break;
		}
	}

	if (bChange)
	{
		int64* TmpValues = nullptr;
		if ((TaggedValues & TagDynamic) == 0)
		{
			// If values are statically allocated, allocate a new array to overwrite
			TmpValues = AllocateValues(NumValues);
		}
		else
		{
			// If values are dynamically allocated, we aren't changing the count so reuse the allocation
			TmpValues = reinterpret_cast<int64*>(TaggedValues & MaskPointer);
		}
		for (int32 i=0; i < NumValues; ++i)
		{
			TmpValues[i] = i;
		}
	}
}

void UEnum::FNameData::UsePlaceholder(FName InName, int64 InValue)
{
	FName* FNames = reinterpret_cast<FName*>(TaggedNames & MaskPointer);
	FNames[NumValues] = InName;
	check(reinterpret_cast<int64*>(TaggedValues & MaskPointer)[NumValues] == InValue);
	++NumValues;
}

void UEnum::FNameData::DiscardPlaceholder()
{
	NumValues -= 1;
}

UEnum::UEnum(const FObjectInitializer& ObjectInitializer)
	: UField(ObjectInitializer)
	, CppType()
	, CppForm(ECppForm::Regular)
	, UnderlyingType(EUnderlyingType::int64)
	, EnumDisplayNameFn( nullptr )
{
}

UEnum::~UEnum()
{	
	DestructItem(&CppType);
}

void UEnum::Serialize( FArchive& Ar )
{
	Ar.UsingCustomVersion(FCoreObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Super::Serialize(Ar);
	if (Ar.IsLoading())
	{
#if UE_WITH_CONSTINIT_UOBJECT
		checkf(!HasAnyInternalFlags(EInternalObjectFlags::Native), TEXT("Shouldn't be loading data into a compiled-in enum %s"), *GetPathName());
#endif
		if (Ar.UEVer() < VER_UE4_TIGHTLY_PACKED_ENUMS)
		{
			TArray<FName> TempNames;
			Ar << TempNames;
			Names.Initialize(TempNames);
		}
		else if (Ar.CustomVer(FCoreObjectVersion::GUID) < FCoreObjectVersion::EnumProperties)
		{
			TArray<TPair<FName, uint8>> OldNames;
			Ar << OldNames;
			Names.Initialize(OldNames);
		}
		else
		{
			// Not using TArray serializer because serialization from FNameData doesn't use it
			TArray<TPair<FName, int64>> TempNames;
			int32 Num = 0;
			Ar << Num;
			TempNames.AddZeroed(Num);
			for (TPair<FName, int64>& Pair : TempNames)
			{
				Ar << Pair;
			}
			Names.Initialize(TempNames);
		}
	}
	else
	{
		int32 Num = Names.Num();
		Ar << Num;
		for (TPair<FName, int64> Pair : Names)
		{
			Ar << Pair;
		}
	}

	if (Ar.UEVer() < VER_UE4_ENUM_CLASS_SUPPORT)
	{
		bool bIsNamespace;
		Ar << bIsNamespace;
		CppForm = bIsNamespace ? ECppForm::Namespaced : ECppForm::Regular;
	}
	else
	{
		uint8 EnumTypeByte = (uint8)CppForm;
		Ar << EnumTypeByte;
		CppForm = (ECppForm)EnumTypeByte;
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::EnumUnderlyingType)
	{
		uint8 UnderlyingTypeByte = (uint8)UnderlyingType;
		Ar << UnderlyingTypeByte;
		UnderlyingType = (EUnderlyingType)UnderlyingTypeByte;
	}

	if (Ar.IsLoading() || Ar.IsSaving())
	{
		// We're duplicating this enum.
		if ((Ar.GetPortFlags() & PPF_Duplicate)
			// and we're loading it from already serialized base.
			&& Ar.IsLoading())
		{
			// Rename enum names to reflect new class.
			RenameNamesAfterDuplication();
		}
		AddNamesToPrimaryList();
	}
}

void UEnum::BeginDestroy()
{
	RemoveNamesFromPrimaryList();

	Super::BeginDestroy();
}

FString UEnum::GetBaseEnumNameOnDuplication() const
{
	// Last name is always fully qualified, in form EnumName::Prefix_MAX.
	FString BaseEnumName = GetNameByIndex(Names.Num() - 1).ToString();

	// Double check we have a fully qualified name.
	const int32 DoubleColonPos = BaseEnumName.Find(TEXT("::"), ESearchCase::CaseSensitive);
	check(DoubleColonPos != INDEX_NONE);

	// Get actual base name.
	BaseEnumName.LeftChopInline(BaseEnumName.Len() - DoubleColonPos, EAllowShrinking::No);

	return BaseEnumName;
}

void UEnum::RenameNamesAfterDuplication()
{
	if (Names.Num() != 0)
	{
		// Get name of base enum, from which we're duplicating.
		FString BaseEnumName = GetBaseEnumNameOnDuplication();

		// Get name of duplicated enum.
		FString ThisName = GetName();

		// Replace all usages of base class name to the duplicated one.
		for (FName& Name : Names.GetMutableNames())
		{
			FString NameString = Name.ToString();
			NameString.ReplaceInline(*BaseEnumName, *ThisName);
			Name = FName(*NameString);
		}
	}
}

int64 UEnum::ResolveEnumerator(FArchive& Ar, int64 EnumeratorIndex) const
{
	return EnumeratorIndex;
}

FString UEnum::GenerateFullEnumName(const TCHAR* InEnumName) const
{
	if (GetCppForm() == ECppForm::Regular || IsFullEnumName(InEnumName))
	{
		return InEnumName;
	}

	return FString::Printf(TEXT("%s::%s"), *GetName(), InEnumName);
}

FName UEnum::GetNameByIndex(int32 Index) const
{
	if (Names.IsValidIndex(Index))
	{
		return Names[Index].Key;
	}

	return NAME_None;
}

FName UEnum::GetNameByValue(int64 InValue) const
{
	for (const TPair<FName, int64>& Kvp : Names)
	{
		if (Kvp.Value == InValue)
		{
			return Kvp.Key;
		}
	}

	return NAME_None;
}

int32 UEnum::GetIndexByName(const FName InName, EGetByNameFlags Flags ) const
{
	ENameCase ComparisonMethod = !!(Flags & EGetByNameFlags::CaseSensitive) ? ENameCase::CaseSensitive : ENameCase::IgnoreCase;

	// First try the fast path
	const int32 Count = Names.Num();
	for (int32 Counter = 0; Counter < Count; ++Counter)
	{
		if (Names[Counter].Key.IsEqual(InName, ComparisonMethod))
		{
			return Counter;
		}
	}

	// Otherwise see if it is in the redirect table
	int32 Result = GetIndexByNameString(InName.ToString(), Flags);

	return Result;
}

int64 UEnum::GetValueByName(FName InName, EGetByNameFlags Flags) const
{
	// This handles redirects
	const int32 EnumIndex = GetIndexByName(InName, Flags);

	if (EnumIndex != INDEX_NONE)
	{
		return GetValueByIndex(EnumIndex);
	}

	return INDEX_NONE;
}

int64 UEnum::GetValueOrBitfieldFromString(FStringView InString, EGetByNameFlags LookupFlags) const
{
	bool  bSuccess = true;
	int64 Result   = 0;

	UE::String::ParseTokens(
		InString,
		TEXTVIEW("|"),
		[this, &bSuccess, &Result, LookupFlags](FStringView Flag)
		{
			if (!bSuccess)
			{
				return;
			}

			FName FlagName(Flag);

			int32 FlagIndex = GetIndexByName(FlagName, LookupFlags);
			if (FlagIndex != INDEX_NONE)
			{
				Result |= GetValueByName(FlagName);
			}
			else
			{
				bSuccess = false;
			}
		},
		UE::String::EParseTokensOptions::SkipEmpty | UE::String::EParseTokensOptions::Trim
	);

	return bSuccess ? Result : INDEX_NONE;
}

namespace UE
{
template <typename NamesCollection>
int64 GetMaxEnumValue(const NamesCollection& InNamesAndValues, EEnumFlags InFlags)
{
	int32 NamesNum = InNamesAndValues.Num();
	if (NamesNum == 0)
	{
		return 0;
	}

	if (EnumHasAnyFlags(InFlags, EEnumFlags::Flags))
	{
		// The max value of a set of flags is the combination of all the power-of-two flags
		int64 MaxFlag = 0;
		for (const TPair<FName, int64>& NameAndValue : InNamesAndValues)
		{
			int64 EnumeratorValue = NameAndValue.Value;
			if (FMath::IsPowerOfTwo(EnumeratorValue))
			{
				MaxFlag |= EnumeratorValue;
			}
		}

		return MaxFlag;
	}
	else
	{
		int64 MaxValue = InNamesAndValues[0].Value;
		for (int32 i = 1; i < NamesNum; ++i)
		{
			int64 CurrentValue = InNamesAndValues[i].Value;
			if (CurrentValue > MaxValue)
			{
				MaxValue = CurrentValue;
			}
		}

		return MaxValue;
	}

}
}

int64 UEnum::GetMaxEnumValue(const TArray<TPair<FName, int64>>& InNamesAndValues, EEnumFlags InFlags)
{
	return UE::GetMaxEnumValue(InNamesAndValues, InFlags);
}

int64 UEnum::GetMaxEnumValue() const
{
	return UE::GetMaxEnumValue(Names, EnumFlags);
}

bool UEnum::IsValidEnumValue(int64 InValue) const
{
	int32 NamesNum = Names.Num();
	for (int32 i = 0; i < NamesNum; ++i)
	{
		int64 CurrentValue = Names[i].Value;
		if (CurrentValue == InValue)
		{
			return true;
		}
	}

	return false;
}

bool UEnum::IsValidEnumValueOrBitfield(int64 InValue) const
{
	if (!EnumHasAnyFlags(EnumFlags, EEnumFlags::Flags))
	{
		return IsValidEnumValue(InValue);
	}

	// Remove the known flags from the value - ignore the last enumerator because it's MAX.
	// TODO: Update this when MAX is no longer generated for flag-based enums.
	int32 NamesNum  = Names.Num() - 1;
	int32 NameIndex = 0;
	for (;;)
	{
		// If all the flags have been removed then it must be a valid value
		if (InValue == 0)
		{
			return true;
		}

		// If all the flags in the enum have been tested and there are still left in the value, it wasn't a valid value
		if (NameIndex == NamesNum)
		{
			return false;
		}

		// Remove flag from value, but only if it's a power of two
		int64 EnumeratorValue = Names[NameIndex].Value;
		if (FMath::IsPowerOfTwo(EnumeratorValue))
		{
			InValue &= ~EnumeratorValue;
		}

		++NameIndex;
	}
}

bool UEnum::IsValidEnumName(FName InName) const
{
	int32 NamesNum = Names.Num();
	for (int32 i = 0; i < NamesNum; ++i)
	{
		FName CurrentName = Names[i].Key;
		if (CurrentName == InName)
		{
			return true;
		}
	}

	return false;
}

void UEnum::AddNamesToPrimaryList()
{
	FWriteScopeLock ScopeLock(AllEnumNamesLock);
	EnumPackage = GetPackage()->GetFName();
	TMap<FName, UEnum*>& PackageEnumValues = AllEnumNames.FindOrAdd(EnumPackage);
	for (FName Name : Names.GetNames())
	{
		UEnum* Enum = PackageEnumValues.FindRef(Name);
		if (Enum == nullptr || Enum->HasAnyFlags(RF_NewerVersionExists))
		{
			PackageEnumValues.Add(Name, this);
		}
		else if (Enum != this && Enum->GetOutermost() != GetTransientPackage())
		{
			UE_LOGF(LogEnum, Warning, "Enum name collision: '%ls' is in both '%ls' and '%ls'", *Name.ToString(), *GetPathName(), *Enum->GetPathName());
		}
	}
}

void UEnum::RemoveNamesFromPrimaryList()
{
	FWriteScopeLock ScopeLock(AllEnumNamesLock);
	TMap<FName, UEnum*>* PackageEnumValues = AllEnumNames.Find(EnumPackage);
	if (PackageEnumValues != nullptr)
	{
		for (FName Name : Names.GetNames())
		{
			UEnum* Enum = PackageEnumValues->FindRef(Name);
			if (Enum == this)
			{
				PackageEnumValues->Remove(Name);
			}
		}
		if (PackageEnumValues->Num() == 0)
		{
			AllEnumNames.Remove(EnumPackage);
		}
	}
}

UEnum* UEnum::LookupAllEnumNamesWithOptions(FName PackageName, EFindFirstObjectOptions Options, TFunctionRef<bool(FName)> CompareNameFunction)
{
	UEnum* TheEnum = nullptr;
	bool bLastFoundEnumWasNative = false;

	auto LookupAllEnumNamesInPackageWithOptions = [Options, &CompareNameFunction, &bLastFoundEnumWasNative](UEnum*& OutEnum, FName CurrentPackageName, const TMap<FName, UEnum*>& EnumNameToUEnumMap)
	{		
		bool bIsNativePackageOrDontCare = !(Options & EFindFirstObjectOptions::NativeFirst) || FPackageName::IsScriptPackage(FNameBuilder(CurrentPackageName));

		for (const TPair<FName, UEnum*>& ValueNameToEnumPair : EnumNameToUEnumMap)
		{
			if (CompareNameFunction(ValueNameToEnumPair.Key))
			{
				if (OutEnum)
				{
					if (!!(Options & EFindFirstObjectOptions::EnsureIfAmbiguous))
					{
						ensureAlwaysMsgf(false, TEXT("Ambiguous results in LookupAllEnumNamesWithOptions: first enum found \"%s\" but it could also be: \"%s\""),
							*OutEnum->GetPathName(), *ValueNameToEnumPair.Value->GetPathName());
					}

					if (bIsNativePackageOrDontCare && !bLastFoundEnumWasNative)
					{
						OutEnum = ValueNameToEnumPair.Value;
						bLastFoundEnumWasNative = bIsNativePackageOrDontCare;
					}
				}
				else
				{
					OutEnum = ValueNameToEnumPair.Value;
					bLastFoundEnumWasNative = bIsNativePackageOrDontCare;

					// If we don't want to check if the search is ambiguous and we don't care if the enum is native or not (or we do care and it's native), abort now
					if (!(Options & EFindFirstObjectOptions::EnsureIfAmbiguous) && bIsNativePackageOrDontCare)
					{
						break;
					}
				}
			}
		}
	};

	if (!PackageName.IsNone())
	{
		// Fast path, we only need to look in the specified package
		const TMap<FName, UEnum*>* EnumNameToUEnumMap = AllEnumNames.Find(PackageName);
		if (EnumNameToUEnumMap)
		{
			LookupAllEnumNamesInPackageWithOptions(TheEnum, PackageName, *EnumNameToUEnumMap);
		}
	}
	else
	{
		// Slow path, look through all existing packages
		for (const TPair<FName, TMap<FName, UEnum*>>& PackageToValuesPair : AllEnumNames)
		{
			LookupAllEnumNamesInPackageWithOptions(TheEnum, PackageToValuesPair.Key, PackageToValuesPair.Value);

			if (TheEnum && bLastFoundEnumWasNative && !(Options & EFindFirstObjectOptions::EnsureIfAmbiguous))
			{
				break;
			}
		}
	}

	return TheEnum;
}

int64 UEnum::LookupEnumName(FName PackageName, FName TestName, EFindFirstObjectOptions Options /*= EFindFirstObjectOptions::None*/, UEnum** OutFoundEnum /*= nullptr*/)
{
	FReadScopeLock ScopeLock(AllEnumNamesLock);
	UEnum* TheEnum = nullptr;
	if (!PackageName.IsNone() && Options == EFindFirstObjectOptions::None)
	{
		TMap<FName, UEnum*>* PackageEnumValues = AllEnumNames.Find(PackageName);
		if (PackageEnumValues)
		{
			TheEnum = PackageEnumValues->FindRef(TestName);
		}
	}
	else
	{
		if (Options == EFindFirstObjectOptions::None)
		{
			// If no options are specified we can just look the value name up with a TMap lookup for each of the packages until we find a match
			for (const TPair<FName, TMap<FName, UEnum*>>& PackageToValuesPair : AllEnumNames)
			{
				TheEnum = PackageToValuesPair.Value.FindRef(TestName);
				if (TheEnum)
				{
					break;
				}
			}
		}
		else
		{
			TheEnum = LookupAllEnumNamesWithOptions(PackageName, Options, [TestName](FName EnumValueName) { return TestName == EnumValueName; });
		}
	}

	if (OutFoundEnum != nullptr)
	{
		*OutFoundEnum = TheEnum;
	}
	return (TheEnum != nullptr) ? TheEnum->GetValueByName(TestName) : INDEX_NONE;
}

int64 UEnum::LookupEnumNameSlow(FName PackageName, const TCHAR* InTestShortName, EFindFirstObjectOptions Options /*= EFindFirstObjectOptions::None*/, UEnum** OutFoundEnum /*= nullptr*/)
{
	FReadScopeLock ScopeLock(AllEnumNamesLock);
	FName TestName(InTestShortName);
	FString TestShortName(FString("::") + InTestShortName);
	UEnum* TheEnum = LookupAllEnumNamesWithOptions(PackageName, Options, [TestName, &TestShortName](FName EnumValueName) { return TestName == EnumValueName || EnumValueName.ToString().Contains(TestShortName); });

	if (OutFoundEnum != nullptr)
	{
		*OutFoundEnum = TheEnum;
	}
	int64 Result = (TheEnum != nullptr) ? TheEnum->GetValueByName(InTestShortName) : INDEX_NONE;

	return Result;
}

FString UEnum::GenerateEnumPrefix() const
{
	FString Prefix;
	if (Names.Num() > 0)
	{
		Names[0].Key.ToString(Prefix);

		// For each item in the enumeration, trim the prefix as much as necessary to keep it a prefix.
		// This ensures that once all items have been processed, a common prefix will have been constructed.
		// This will be the longest common prefix since as little as possible is trimmed at each step.
		for (int32 NameIdx = 1; NameIdx < Names.Num(); NameIdx++)
		{
			FString EnumItemName = Names[NameIdx].Key.ToString();

			// Find the length of the longest common prefix of Prefix and EnumItemName.
			int32 PrefixIdx = 0;
			while (PrefixIdx < Prefix.Len() && PrefixIdx < EnumItemName.Len() && Prefix[PrefixIdx] == EnumItemName[PrefixIdx])
			{
				PrefixIdx++;
			}

			// Trim the prefix to the length of the common prefix.
			Prefix.LeftInline(PrefixIdx, EAllowShrinking::No);
		}

		// Find the index of the rightmost underscore in the prefix.
		int32 UnderscoreIdx = Prefix.Find(TEXT("_"), ESearchCase::CaseSensitive, ESearchDir::FromEnd);

		// If an underscore was found, trim the prefix so only the part before the rightmost underscore is included.
		if (UnderscoreIdx > 0)
		{
			Prefix.LeftInline(UnderscoreIdx, EAllowShrinking::No);
		}
		else
		{
			// no underscores in the common prefix - this probably indicates that the names
			// for this enum are not using Epic's notation, so just empty the prefix so that
			// the max item will use the full name of the enum
			Prefix.Empty();
		}
	}

	// If no common prefix was found, or if the enum does not contain any entries,
	// use the name of the enumeration instead.
	if (Prefix.Len() == 0)
	{
		Prefix = GetName();
	}
	return Prefix;
}

FString UEnum::GetNameStringByIndex(int32 InIndex) const
{
	if (Names.IsValidIndex(InIndex))
	{
		FName EnumEntryName = GetNameByIndex(InIndex);
		if (CppForm == ECppForm::Regular)
		{
			return EnumEntryName.ToString();
		}

		// Strip the namespace from the name.
		FString EnumNameString(EnumEntryName.ToString());
		int32 ScopeIndex = EnumNameString.Find(TEXT("::"), ESearchCase::CaseSensitive);
		if (ScopeIndex != INDEX_NONE)
		{
			return EnumNameString.Mid(ScopeIndex + 2);
		}
	}
	return FString();
}

FString UEnum::GetNameStringByValue(int64 Value) const
{
	int32 Index = GetIndexByValue(Value);
	return GetNameStringByIndex(Index);
}

namespace
{
	template <typename NameGetterType>
	FString CreateStringFromValueOrBitfield(const UEnum* Enum, int64 InValue, NameGetterType&& NameGetter)
	{
		if (!Enum->HasAnyEnumFlags(EEnumFlags::Flags) || InValue == 0)
		{
			return NameGetter(InValue);
		}

		FString BitfieldString;
		bool bWroteFirstFlag = false;
		while (InValue != 0)
		{
			int64 NextValue = 1ll << FMath::CountTrailingZeros64(InValue);
			InValue = InValue & ~NextValue;

			// We don't just want to use the NameValuePair.Key because we want to strip enum class prefixes
			BitfieldString.Appendf(TEXT("%s%s"), bWroteFirstFlag ? TEXT(" | ") : TEXT(""), *NameGetter(NextValue));
			bWroteFirstFlag = true;
		}
		return BitfieldString;
	}
}

FString UEnum::GetValueOrBitfieldAsString(int64 InValue) const
{
	return CreateStringFromValueOrBitfield(
		this,
		InValue,
		[this](int64 Value)
		{
			return GetNameStringByValue(Value);
		}
	);
}

FString UEnum::GetValueOrBitfieldAsAuthoredNameString(int64 InValue) const
{
	return CreateStringFromValueOrBitfield(
		this,
		InValue,
		[this](int64 Value)
		{
			return GetAuthoredNameStringByValue(Value);
		}
	);
}

FText UEnum::GetValueOrBitfieldAsDisplayNameText(int64 InValue) const
{
	return FText::FromString(
		CreateStringFromValueOrBitfield(
			this,
			InValue,
			[this](int64 Value)
			{
				return GetDisplayNameTextByValue(Value).ToString();
			}
		)
	);
}

bool UEnum::FindNameStringByValue(FString& Out, int64 InValue) const
{
	int32 Index = GetIndexByValue(InValue);
	if (Index == INDEX_NONE)
	{
		return false;
	}

	Out = GetNameStringByIndex(Index);
	return true;
}

FText UEnum::GetDisplayNameTextByIndex(int32 NameIndex) const
{
	FString RawName = GetNameStringByIndex(NameIndex);

	if (RawName.IsEmpty())
	{
		return FText::GetEmpty();
	}

#if WITH_EDITOR
	// In the editor, use metadata and localization to look up names
	static const FTextKey Namespace = TEXT("UObjectDisplayNames");
	const FString Key = GetFullGroupName(false) + TEXT(".") + RawName;

	FString NativeDisplayName;
	if (HasMetaData(TEXT("DisplayName"), NameIndex))
	{
		NativeDisplayName = GetMetaData(TEXT("DisplayName"), NameIndex);
	}
	else
	{
		const FString AuthoredName = GetAuthoredNameStringByIndex(NameIndex);
		NativeDisplayName = FName::NameToDisplayString(AuthoredName, false);
	}

	if (!NativeDisplayName.IsEmpty())
	{
		return FText::AsLocalizable_Advanced(Namespace, Key, MoveTemp(NativeDisplayName));
	}
#endif

	if (EnumDisplayNameFn)
	{
		return (*EnumDisplayNameFn)(NameIndex);
	}

	return FText::FromString(MoveTemp(RawName));
}

FText UEnum::GetDisplayNameTextByValue(int64 Value) const
{
	int32 Index = GetIndexByValue(Value);
	return GetDisplayNameTextByIndex(Index);
}

bool UEnum::FindDisplayNameTextByValue(FText& Out, int64 Value) const
{
	int32 Index = GetIndexByValue(Value);
	if (Index == INDEX_NONE)
	{
		return false;
	}

	Out = GetDisplayNameTextByIndex(Index);
	return true;
}

FString UEnum::GetAuthoredNameStringByIndex(int32 InIndex) const
{
	return GetNameStringByIndex(InIndex);
}

FString UEnum::GetAuthoredNameStringByValue(int64 Value) const
{
	int32 Index = GetIndexByValue(Value);
	return GetAuthoredNameStringByIndex(Index);
}

bool UEnum::FindAuthoredNameStringByValue(FString& Out, int64 Value) const
{
	int32 Index = GetIndexByValue(Value);
	if (Index == INDEX_NONE)
	{
		return false;
	}

	Out = GetAuthoredNameStringByIndex(Index);
	return true;
}

int32 UEnum::GetIndexByNameString(const FString& InSearchString, EGetByNameFlags Flags) const
{
	ENameCase         NameComparisonMethod   = !!(Flags & EGetByNameFlags::CaseSensitive) ? ENameCase  ::CaseSensitive : ENameCase  ::IgnoreCase;
	ESearchCase::Type StringComparisonMethod = !!(Flags & EGetByNameFlags::CaseSensitive) ? ESearchCase::CaseSensitive : ESearchCase::IgnoreCase;

	FString SearchEnumEntryString = InSearchString;
	FString ModifiedEnumEntryString;

	// Strip or add the namespace
	int32 DoubleColonIndex = SearchEnumEntryString.Find(TEXT("::"), ESearchCase::CaseSensitive);
	if (DoubleColonIndex == INDEX_NONE)
	{
		ModifiedEnumEntryString = GenerateFullEnumName(*SearchEnumEntryString);
	}
	else
	{
		ModifiedEnumEntryString = SearchEnumEntryString.RightChop(DoubleColonIndex + 2);
	}

	const TMap<FString, FString>* ValueChanges = FCoreRedirects::GetValueRedirects(ECoreRedirectFlags::Type_Enum, this);

	if (ValueChanges)
	{
		const FString* FoundNewEnumEntry = ValueChanges->Find(SearchEnumEntryString);
		if (FoundNewEnumEntry == nullptr)
		{
			FoundNewEnumEntry = ValueChanges->Find(ModifiedEnumEntryString);
		}

		if (FoundNewEnumEntry)
		{
			SearchEnumEntryString = **FoundNewEnumEntry;

			// Recompute modified name
			int32 NewDoubleColonIndex = SearchEnumEntryString.Find(TEXT("::"), ESearchCase::CaseSensitive);
			if (NewDoubleColonIndex == INDEX_NONE)
			{
				ModifiedEnumEntryString = GenerateFullEnumName(*SearchEnumEntryString);
			}
			else
			{
				ModifiedEnumEntryString = SearchEnumEntryString.RightChop(NewDoubleColonIndex + 2);
			}
		}
	}
	else if (DoubleColonIndex != INDEX_NONE)
	{
		// If we didn't find a value redirect and our original string was namespaced, we need to fix the namespace now as it may have changed due to enum type redirect
		SearchEnumEntryString = GenerateFullEnumName(*ModifiedEnumEntryString);
	}

	// Search for names both with and without namespace
	FName SearchName = FName(*SearchEnumEntryString, FNAME_Find);
	FName ModifiedName = FName(*ModifiedEnumEntryString, FNAME_Find);

	const int32 Count = Names.Num();
	for (int32 Counter = 0; Counter < Count; ++Counter)
	{
		if (Names[Counter].Key.IsEqual(SearchName, NameComparisonMethod) || Names[Counter].Key.IsEqual(ModifiedName, NameComparisonMethod))
		{
			return Counter;
		}
	}

	// Check authored name, but only if this is a subclass of Enum that might have implemented it 
	// and we've ascertained that there are no entries that match on the cheaper FName checks
	const bool bCheckAuthoredName = !!(Flags & EGetByNameFlags::CheckAuthoredName) && (GetClass() != UEnum::StaticClass());

	if (bCheckAuthoredName)
	{
		for (int32 Counter = 0; Counter < Count; ++Counter)
		{
			FString AuthoredName = GetAuthoredNameStringByIndex(Counter);

			if (AuthoredName.Equals(SearchEnumEntryString, StringComparisonMethod) || AuthoredName.Equals(ModifiedEnumEntryString, StringComparisonMethod))
			{
				return Counter;
			}
		}
	}

	if (!InSearchString.Equals(SearchEnumEntryString, StringComparisonMethod))
	{
		// There was an actual redirect, and we didn't find it
		UE_LOGF(LogEnum, Warning, "EnumRedirect for enum %ls maps '%ls' to invalid value '%ls'!", *GetName(), *InSearchString, *SearchEnumEntryString);
	}
	else if (!!(Flags & EGetByNameFlags::ErrorIfNotFound) && !InSearchString.IsEmpty() && !InSearchString.Equals(FName().ToString(), StringComparisonMethod))
	{
		// None is passed in by blueprints at various points, isn't an error. Any other failed resolve should be fixed
		UObject* SerializedObject = nullptr;
		if (FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext())
		{
			SerializedObject = LoadContext->SerializedObject;
		}
		const bool bIsNativeOrLoaded = (!HasAnyFlags(RF_WasLoaded) || HasAnyFlags(RF_LoadCompleted));
		UE_LOGF(LogEnum, Warning, "UEnum: In asset '%ls', there is an enum property of type '%ls' with an invalid value of '%ls' - %ls - %d", 
			*GetPathNameSafe(SerializedObject ? SerializedObject : FUObjectThreadContext::Get().ConstructedObject), 
			*GetName(), 
			*InSearchString,
			bIsNativeOrLoaded ? TEXT("loaded") : TEXT("not loaded"),
			Count);
	}

	return INDEX_NONE;
}

int64 UEnum::GetValueByNameString(const FString& SearchString, EGetByNameFlags Flags) const
{
	int32 Index = GetIndexByNameString(SearchString, Flags);

	if (Index != INDEX_NONE)
	{
		return GetValueByIndex(Index);
	}

	return INDEX_NONE;
}

bool UEnum::ContainsExistingMax() const
{
	if (GetIndexByName(*GenerateFullEnumName(TEXT("MAX")), EGetByNameFlags::CaseSensitive) != INDEX_NONE)
	{
		return true;
	}

	FName MaxEnumItem = *GenerateFullEnumName(*(GenerateEnumPrefix() + TEXT("_MAX")));
	if (GetIndexByName(MaxEnumItem, EGetByNameFlags::CaseSensitive) != INDEX_NONE)
	{
		return true;
	}

	return false;
}

#if UE_WITH_CONSTINIT_UOBJECT
void UEnum::InitializeNames()
{
	const UTF8CHAR* CppTypeCopy = CppTypeStaticUTF8;
	// Construct CppType in place to avoid attempting to free static string
	new (&CppType) FString(CppTypeCopy);

	if (Names.Num() == 0)
	{
		return;
	}

	Names.InitializeFromCompiledInData();

	checkfSlow(ContainsExistingMax(), TEXT("Enums from UHT should contain a max value in the code or have one generated by UHT"));
    // Check if otherwise the MAX identifier is already being used by another enum, if so we can't add it to the lookup of short enum names
	if (LookupEnumName(GetOutermost()->GetFName(), Names.GetNames().Last()) == INDEX_NONE)
	{
		AddNamesToPrimaryList();
	}
}
#endif // UE_WITH_CONSTINIT_UOBJECT

// Deprecated
bool UEnum::SetEnums(TArray<TPair<FName, int64>>& InNames, UEnum::ECppForm InCppForm, EEnumFlags InFlags, bool bAddMaxKeyIfMissing)
{
	return SetEnums(InNames, InCppForm, UEnum::EUnderlyingType::int64, InFlags, bAddMaxKeyIfMissing ? EAddMaxKeyIfMissing::Yes : EAddMaxKeyIfMissing::No);
}

bool UEnum::SetEnums(TArray<TPair<FName, int64>>& InNames, UEnum::ECppForm InCppForm, UEnum::EUnderlyingType InUnderlyingType, EEnumFlags InFlags, EAddMaxKeyIfMissing AddMaxKeyIfMissing)
{
#if UE_WITH_CONSTINIT_UOBJECT
	checkf(!HasAnyInternalFlags(EInternalObjectFlags::Native), TEXT("SetEnums should not be called for compiled-in constinit enum %s"), *GetPathName());
#endif
	if (Names.Num() > 0)
	{
		RemoveNamesFromPrimaryList();
	}
	Names.Empty();
	CppForm   = InCppForm;
	UnderlyingType = InUnderlyingType;
	EnumFlags = InFlags;

	if (AddMaxKeyIfMissing == EAddMaxKeyIfMissing::Yes)
	{
		const int64 MaxEnumValue = (InNames.Num() == 0) ? 0 : GetMaxEnumValue(InNames, InFlags) + 1;
		Names.InitializeWithPlaceholder(InNames, MaxEnumValue);

		// TODO: A MAX value for enum flags doesn't make sense, but keep generating it until we're comfortable that
		// it's no longer needed.
		if (!ContainsExistingMax() /*&& !EnumHasAnyFlags(EnumFlags, EEnumFlags::Flags)*/)
		{
			FName MaxEnumItem = *GenerateFullEnumName(*(GenerateEnumPrefix() + TEXT("_MAX")));
			if (LookupEnumName(GetOutermost()->GetFName(), MaxEnumItem) != INDEX_NONE)
			{
				// the MAX identifier is already being used by another enum
				return false;
			}

			Names.UsePlaceholder(MaxEnumItem, MaxEnumValue);
		}
	}
	else
	{
		Names.Initialize(InNames);
	}

	AddNamesToPrimaryList();

	return true;
}

#if WITH_EDITOR

FText UEnum::GetToolTipTextByIndex(int32 NameIndex) const
{
	FText LocalizedToolTip;
	FString NativeToolTip = GetMetaData( TEXT("ToolTip"), NameIndex );

	static const FTextKey Namespace = TEXT("UObjectToolTips");
	const FString Key = GetFullGroupName(false) + TEXT(".") + GetNameStringByIndex(NameIndex);
		
	if ( !FText::FindTextInLiveTable_Advanced( Namespace, Key, /*OUT*/LocalizedToolTip, &NativeToolTip ) )
	{
		static const FString DoxygenSee("@see");
		static const FString TooltipSee("See:");
		if (NativeToolTip.ReplaceInline(*DoxygenSee, *TooltipSee) > 0)
		{
			NativeToolTip.TrimEndInline();
		}

		LocalizedToolTip = FText::AsLocalizable_Advanced(Namespace, Key, MoveTemp(NativeToolTip));
	}

	return LocalizedToolTip;
}

#endif

#if WITH_METADATA

bool UEnum::HasMetaData( const TCHAR* Key, int32 NameIndex/*=INDEX_NONE*/ ) const
{
	bool bResult = false;

	UPackage* Package = GetOutermost();
	check(Package);

	FMetaData& MetaData = Package->GetMetaData();

	FString KeyString;

	// If an index was specified, search for metadata linked to a specified value
	if ( NameIndex != INDEX_NONE )
	{
		KeyString = GetNameStringByIndex(NameIndex);
		KeyString.AppendChar(TEXT('.'));
		KeyString.Append(Key);
	}
	// If no index was specified, search for metadata for the enum itself
	else
	{
		KeyString = Key;
	}
	bResult = MetaData.HasValue( this, *KeyString );
	
	return bResult;
}

FString UEnum::GetMetaData( const TCHAR* Key, int32 NameIndex/*=INDEX_NONE*/, bool bAllowRemap/*=true*/ ) const
{
	UPackage* Package = GetOutermost();
	check(Package);

	FMetaData& MetaData = Package->GetMetaData();

	FString KeyString;

	// If an index was specified, search for metadata linked to a specified value
	if ( NameIndex != INDEX_NONE )
	{
		check(Names.IsValidIndex(NameIndex));
		KeyString = GetNameStringByIndex(NameIndex) + TEXT(".") + Key;
	}
	// If no index was specified, search for metadata for the enum itself
	else
	{
		KeyString = Key;
	}

	FString ResultString = MetaData.GetValue( this, *KeyString );
	
	// look in the engine ini, in a section named after the metadata key we are looking for, and the enum's name (KeyString)
	if (bAllowRemap && ResultString.StartsWith(TEXT("ini:")))
	{
		if (!GConfig->GetString(TEXT("EnumRemap"), *KeyString, ResultString, GEngineIni))
		{
			// if this fails, then use what's after the ini:
			ResultString.MidInline(4, MAX_int32, EAllowShrinking::No);
		}
	}

	return ResultString;
}

void UEnum::SetMetaData( const TCHAR* Key, const TCHAR* InValue, int32 NameIndex ) const
{
	UPackage* Package = GetOutermost();
	check(Package);

	FMetaData& MetaData = Package->GetMetaData();

	FString KeyString;

	// If an index was specified, search for metadata linked to a specified value
	if ( NameIndex != INDEX_NONE )
	{
		check(Names.IsValidIndex(NameIndex));
		KeyString = GetNameStringByIndex(NameIndex) + TEXT(".") + Key;
	}
	// If no index was specified, search for metadata for the enum itself
	else
	{
		KeyString = Key;
	}

	MetaData.SetValue( this, *KeyString, InValue );
}

void UEnum::RemoveMetaData( const TCHAR* Key, int32 NameIndex/*=INDEX_NONE*/) const
{
	UPackage* Package = GetOutermost();
	check(Package);

	FMetaData& MetaData = Package->GetMetaData();

	FString KeyString;

	// If an index was specified, search for metadata linked to a specified value
	if ( NameIndex != INDEX_NONE )
	{
		check(Names.IsValidIndex(NameIndex));
		KeyString = GetNameStringByIndex(NameIndex) + TEXT(".") + Key;
	}
	// If no index was specified, search for metadata for the enum itself
	else
	{
		KeyString = Key;
	}

	MetaData.RemoveValue( this, *KeyString );
}

#endif // WITH_METADATA

int64 UEnum::ParseEnum(const TCHAR*& Str)
{
	FString Token;
	const TCHAR* ParsedStr = Str;
	if (FParse::AlnumToken(ParsedStr, Token))
	{
		FName TheName = FName(*Token, FNAME_Find);
		int64 Result = LookupEnumName(FName(), TheName);
		if (Result != INDEX_NONE)
		{
			Str = ParsedStr;
		}
		return Result;
	}
	else
	{
		return 0;
	}
}


#if WITH_DEV_AUTOMATION_TESTS 

#include "Misc/AutomationTest.h"
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FEnumBitfieldTest, "System.CoreUObject.EnumBitfields", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FEnumBitfieldTest::RunTest(const FString& Parameters)
{
	UPackage* NativePackage = CreatePackage(TEXT("/Script/TestEnumBitfieldsPackage"));
	NativePackage->SetPackageFlags(PKG_CompiledIn);

	{
		const FName BitfieldTestEnumName("EBitfieldTestEnum");
		TArray<TPair<FName, int64>> BitfieldTestEnumValueNames =
		{
		 { TEXT("EBitfieldTestEnum::TestValue0"), 0 },
		 { TEXT("EBitfieldTestEnum::TestValue1"), 1 << 0 },
		 { TEXT("EBitfieldTestEnum::TestValue2"), 1 << 1 },
		 { TEXT("EBitfieldTestEnum::TestValue4"), 1 << 2 },
		 { TEXT("EBitfieldTestEnum::TestValue2to63"), 1ll << 63 },
		};

		UEnum* NativeEnum = NewObject<UEnum>(NativePackage, BitfieldTestEnumName);
		NativeEnum->SetEnums(BitfieldTestEnumValueNames, UEnum::ECppForm::EnumClass, UEnum::EUnderlyingType::int64, EEnumFlags::None, UEnum::EAddMaxKeyIfMissing::Yes);
		NativeEnum->SetEnumFlags(EEnumFlags::Flags);

		FString BitfieldTestValueOf3 = NativeEnum->GetValueOrBitfieldAsString(3);
		TestEqual(TEXT("Test bitfield with value 3"), BitfieldTestValueOf3, FString("TestValue1 | TestValue2"));

		FString BitfieldTestValueOf1 = NativeEnum->GetValueOrBitfieldAsString(1);
		TestEqual(TEXT("Test bitfield with value 1"), BitfieldTestValueOf1, FString("TestValue1"));

		FString BitfieldTestValueOf0 = NativeEnum->GetValueOrBitfieldAsString(0);
		TestEqual(TEXT("Test bitfield with value 0"), BitfieldTestValueOf0, FString("TestValue0"));

		FString BitfieldTestValueOf2to63 = NativeEnum->GetValueOrBitfieldAsString(1ll << 63);
		TestEqual(TEXT("Test bitfield with value 2^63"), BitfieldTestValueOf2to63, FString("TestValue2to63"));

		FString BitfieldTestValueOf8 = NativeEnum->GetValueOrBitfieldAsString(1ll << 3);
		TestEqual(TEXT("Test bitfield with invalid value 8"), BitfieldTestValueOf8, FString());
	}

	{
		const FName BitfieldTestStandardEnumName("EBitfieldTestStandardEnum");
		TArray<TPair<FName, int64>> BitfieldTestStandardEnumValueNames =
		{
		 { TEXT("EBitfieldTestStandardEnum::TestValue0"), 0 },
		 { TEXT("EBitfieldTestStandardEnum::TestValue1"), 1 },
		 { TEXT("EBitfieldTestStandardEnum::TestValue2"), 2 },
		 { TEXT("EBitfieldTestStandardEnum::TestValue3"), 3 },
		 { TEXT("EBitfieldTestStandardEnum::TestValue2to63"), 1ll << 63 },
		};

		UEnum* NativeEnum = NewObject<UEnum>(NativePackage, BitfieldTestStandardEnumName);
		NativeEnum->SetEnums(BitfieldTestStandardEnumValueNames, UEnum::ECppForm::EnumClass, UEnum::EUnderlyingType::int64, EEnumFlags::None, UEnum::EAddMaxKeyIfMissing::Yes);

		FString BitfieldTestValueOf3 = NativeEnum->GetValueOrBitfieldAsString(3);
		TestEqual(TEXT("Test non bitfield with value 3"), BitfieldTestValueOf3, FString("TestValue3"));

		FString BitfieldTestValueOf1 = NativeEnum->GetValueOrBitfieldAsString(1);
		TestEqual(TEXT("Test non bitfield with value 1"), BitfieldTestValueOf1, FString("TestValue1"));

		FString BitfieldTestValueOf0 = NativeEnum->GetValueOrBitfieldAsString(0);
		TestEqual(TEXT("Test non bitfield with value 0"), BitfieldTestValueOf0, FString("TestValue0"));

		FString BitfieldTestValueOf2to63 = NativeEnum->GetValueOrBitfieldAsString(1ll << 63);
		TestEqual(TEXT("Test non bitfield with value 2^63"), BitfieldTestValueOf2to63, FString("TestValue2to63"));

		FString BitfieldTestValueOf8 = NativeEnum->GetValueOrBitfieldAsString(8);
		TestEqual(TEXT("Test non bitfield with bad value"), BitfieldTestValueOf8, FString());
	}
	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
