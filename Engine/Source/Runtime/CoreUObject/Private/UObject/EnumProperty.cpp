// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/EnumProperty.h"

#include "Algo/Find.h"
#include "Hash/Blake3.h"
#include "UObject/PropertyHelper.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectThreadContext.h"

#if WITH_EDITORONLY_DATA
#include "UObject/CoreRedirects.h"
#include "UObject/PropertyStateTracking.h"
#endif

namespace UE::CoreUObject::Private
{
	template <typename OldIntType>
	void ConvertIntValueToEnumProperty(OldIntType OldValue, FEnumProperty* EnumProp, FNumericProperty* UnderlyingProp, UEnum* Enum, void* Obj)
	{
		using LargeIntType = std::conditional_t<TIsSigned<OldIntType>::Value, int64, uint64>;

		LargeIntType NewValue = OldValue;
		if (!UnderlyingProp->CanHoldValue(NewValue) || !Enum->IsValidEnumValueOrBitfield(NewValue))
		{
			NewValue = Enum->HasAnyEnumFlags(EEnumFlags::Flags) ? 0 : Enum->GetMaxEnumValue();

			UE_LOGF(
				LogClass,
				Warning,
				"Failed to find valid enum value '%ls' for enum type '%ls' when converting property '%ls' during property loading - setting to '%ls'",
				*LexToString(OldValue),
				*Enum->GetName(),
				*EnumProp->GetName(),
				*Enum->GetNameByValue(NewValue).ToString()
			);
		}

		UnderlyingProp->SetIntPropertyValue(Obj, NewValue);
	}

	template <typename OldIntType>
	void ConvertIntToEnumProperty(FStructuredArchive::FSlot Slot, FEnumProperty* EnumProp, FNumericProperty* UnderlyingProp, UEnum* Enum, void* Obj)
	{
		OldIntType OldValue;
		Slot << OldValue;

		ConvertIntValueToEnumProperty(OldValue, EnumProp, UnderlyingProp, Enum, Obj);
	}

	const TCHAR* ImportEnumFromBuffer(UEnum* Enum, const FProperty* PropertyToSet, const FNumericProperty* UnderlyingProp, const TCHAR* PropertyClassName, const TCHAR* Buffer, void* ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, FOutputDevice* ErrorText)
	{
		if (!Buffer)
		{
			return nullptr;
		}

		bool    bIsEnumOfFlags = Enum->HasAnyEnumFlags(EEnumFlags::Flags);
		int64   EnumValue      = 0;
		FString Temp;
		for (;;)
		{
			SkipWhitespace(Buffer);
			Buffer = FPropertyHelpers::ReadToken(Buffer, Temp, /*bDottedNames=*/true);
			if (!Buffer)
			{
				break;
			}

			int32 EnumIndex = Enum->GetIndexByName(*Temp, EGetByNameFlags::CheckAuthoredName);
			if (!bIsEnumOfFlags && EnumIndex == INDEX_NONE && (Temp.IsNumeric() && !Algo::Find(Temp, TEXT('.'))))
			{
				int64 LexedEnumValue = INDEX_NONE;
				LexFromString(LexedEnumValue, *Temp);
				EnumIndex = Enum->GetIndexByValue(LexedEnumValue);
			}
			if (EnumIndex == INDEX_NONE)
			{
				Buffer = nullptr;
				break;
			}

			EnumValue |= Enum->GetValueByIndex(EnumIndex);

			if (!bIsEnumOfFlags)
			{
				break;
			}

			Temp.Reset();
			SkipWhitespace(Buffer);
			if (*Buffer != TEXT('|'))
			{
				break;
			}
			++Buffer;
		}

		if (!Buffer)
		{
			// Enum could not be created from value. This indicates a bad value so
			// return null so that the caller of ImportText can generate a more meaningful
			// warning/error
			UObject* SerializedObject = nullptr;
			if (FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext())
			{
				SerializedObject = LoadContext->SerializedObject;
			}
			const bool bIsNativeOrLoaded = (!Enum->HasAnyFlags(RF_WasLoaded) || Enum->HasAnyFlags(RF_LoadCompleted));
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("%s: In asset '%s', there is an enum property of type '%s' with an invalid value of '%s' - %s"), 
				PropertyClassName,
				*GetPathNameSafe(SerializedObject ? SerializedObject : FUObjectThreadContext::Get().ConstructedObject), 
				*Enum->GetName(), 
				*Temp,
				bIsNativeOrLoaded ? TEXT("loaded") : TEXT("not loaded"));
			return nullptr;
		}

		if (PropertyPointerType == EPropertyPointerType::Container && PropertyToSet->HasSetter())
		{
			PropertyToSet->SetValue_InContainer(ContainerOrPropertyPtr, &EnumValue);
		}
		else
		{
			UnderlyingProp->SetIntPropertyValue(PropertyToSet->PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType), EnumValue);
		}
		return Buffer;
	}

	void ExportEnumToBuffer(const UEnum* Enum, const FProperty* Prop, const FNumericProperty* NumericProp, FString& ValueStr, const void* PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope)
	{
		// This should be big enough to import all numeric enums
		alignas(int64) uint8 GetterResult[sizeof(int64)];
		void* ValuePtr = nullptr;

		if (PropertyPointerType == EPropertyPointerType::Container && Prop->HasGetter())
		{
			// Put the bytes returned by the getter into GetterResult (which is big and aligned enough to hold any numeric value)
			// - these bytes will be re-read by a call to NumericProp below.
			ValuePtr = &GetterResult;
			Prop->GetValue_InContainer(PropertyValueOrContainer, ValuePtr);
		}
		else
		{
			// Otherwise read directly from the property
			ValuePtr = Prop->PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType);
		}

		if (PortFlags & PPF_ConsoleVariable)
		{
			NumericProp->ExportText_Internal(ValueStr, ValuePtr, EPropertyPointerType::Direct, DefaultValue, Parent, PortFlags, ExportRootScope);
			return;
		}

		int64 Value = NumericProp->GetSignedIntPropertyValue(ValuePtr);

		// if the value is the max value (the autogenerated *_MAX value), export as "INVALID", unless we're exporting text for copy/paste (for copy/paste,
		// the property text value must actually match an entry in the enum's names array)
		if (!Enum->HasAnyEnumFlags(EEnumFlags::Flags) && (!Enum->IsValidEnumValue(Value) || (!(PortFlags & PPF_Copy) && Value == Enum->GetMaxEnumValue())))
		{
			ValueStr += TEXT("(INVALID)");
			return;
		}

		// We do not want to export the enum text for non-display uses, localization text is very dynamic and would cause issues on import
		if (PortFlags & PPF_PropertyWindow)
		{
			ValueStr += Enum->GetValueOrBitfieldAsDisplayNameText(Value).ToString();
		}
		else if (PortFlags & PPF_ExternalEditor)
		{
			ValueStr += Enum->GetValueOrBitfieldAsAuthoredNameString(Value);
		}
		else
		{
			ValueStr += Enum->GetValueOrBitfieldAsString(Value);
		}
	}
}

bool TryLoadEnumValueByName(FStructuredArchive::FSlot Slot, FArchive& UnderlyingArchive, UEnum* Enum, FName& OutEnumValueName, int64& OutEnumValue)
{
	Slot << OutEnumValueName;

	if (Enum)
	{
		// Make sure enum is properly populated
		Enum->ConditionalPreload();

		if (Enum->HasAnyEnumFlags(EEnumFlags::Flags))
		{
			if (OutEnumValueName != NAME_None)
			{
				OutEnumValue = Enum->GetValueOrBitfieldFromString(*OutEnumValueName.ToString());
				return OutEnumValue != INDEX_NONE;
			}
		}
		else
		{
			// There's no guarantee EnumValueName is still present in Enum, in which case Value will be set to the enum's max value.
			// On save, it will then be serialized as NAME_None.
			const int32 EnumIndex = Enum->GetIndexByName(OutEnumValueName, EGetByNameFlags::ErrorIfNotFound);
			if (EnumIndex == INDEX_NONE)
			{
				OutEnumValue = Enum->GetMaxEnumValue();
				return false;
			}
			else
			{
				OutEnumValue = Enum->GetValueByIndex(EnumIndex);
				return true;
			}
		}
	}

	OutEnumValue = 0;
	return false;
}

IMPLEMENT_FIELD(FEnumProperty)

FEnumProperty::FEnumProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)	
	, UnderlyingProp(nullptr)
	, Enum(nullptr)
{

}

FEnumProperty::FEnumProperty(FFieldVariant InOwner, const UECodeGen_Private::FEnumPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop, CPF_HasGetValueTypeHash)
{
	Enum = Prop.EnumFunc ? Prop.EnumFunc(ETypeConstructPhase::Inner) : nullptr;

	// This is expected to be set post-construction by AddCppProperty
	UnderlyingProp = nullptr;
}

#if WITH_EDITORONLY_DATA
FEnumProperty::FEnumProperty(UField* InField)
	: Super(InField)
{
	UEnumProperty* SourceProperty = CastChecked<UEnumProperty>(InField);
	Enum = SourceProperty->Enum;

	UnderlyingProp = CastField<FNumericProperty>(SourceProperty->UnderlyingProp->GetAssociatedFField());
	if (!UnderlyingProp)
	{
		UnderlyingProp = CastField<FNumericProperty>(CreateFromUField(SourceProperty->UnderlyingProp));
		SourceProperty->UnderlyingProp->SetAssociatedFField(UnderlyingProp);
	}
}
#endif // WITH_EDITORONLY_DATA

FEnumProperty::~FEnumProperty()
{
	delete UnderlyingProp;
	UnderlyingProp = nullptr;
}

void FEnumProperty::PostDuplicate(const FField& InField)
{
	const FEnumProperty& Source = static_cast<const FEnumProperty&>(InField);
	Enum = Source.Enum;
	UnderlyingProp = CastFieldChecked<FNumericProperty>(FField::Duplicate(Source.UnderlyingProp, this));
	Super::PostDuplicate(InField);
}

void FEnumProperty::AddCppProperty(FProperty* Inner)
{
	check(!UnderlyingProp);
	UnderlyingProp = CastFieldChecked<FNumericProperty>(Inner);
	check(UnderlyingProp->GetOwner<FEnumProperty>() == this);
	if (UnderlyingProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
	{
		PropertyFlags |= CPF_HasGetValueTypeHash;
	}
}

void FEnumProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	check(UnderlyingProp);

	if (Enum && UnderlyingArchive.UseToResolveEnumerators())
	{
		Slot.EnterStream();
		int64 IntValue = UnderlyingProp->GetSignedIntPropertyValue(Value);
		int64 ResolvedIndex = Enum->ResolveEnumerator(UnderlyingArchive, IntValue);
		UnderlyingProp->SetIntPropertyValue(Value, ResolvedIndex);
		return;
	}

	// Loading
	if (UnderlyingArchive.IsLoading())
	{
		FName EnumValueName;
		int64 EnumValue = 0;
		if (!TryLoadEnumValueByName(Slot, UnderlyingArchive, Enum, EnumValueName, EnumValue))
		{
		#if WITH_EDITORONLY_DATA
			FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
			if (UNLIKELY(SerializeContext->bTrackUnknownEnumNames))
			{
				UE::FUnknownEnumNames(SerializeContext->SerializedObject).Add(Enum, UE::FindOriginalType(this), EnumValueName);
			}
		#endif
		}
		UnderlyingProp->SetIntPropertyValue(Value, EnumValue);
	}
	// Saving
	else if (UnderlyingArchive.IsSaving())
	{
		FName EnumValueName;
		if (Enum)
		{
			const int64 IntValue = UnderlyingProp->GetSignedIntPropertyValue(Value);

			if (Enum->HasAnyEnumFlags(EEnumFlags::Flags))
			{
				if (IntValue != 0)
				{
					EnumValueName = *Enum->GetValueOrBitfieldAsString(IntValue);
				}
			}
			else
			{
				if (Enum->IsValidEnumValue(IntValue))
				{
					EnumValueName = Enum->GetNameByValue(IntValue);

				#if WITH_EDITORONLY_DATA
					// Fix up the type name when this property is impersonating another enum type.
					if (UE::FPropertyTypeName OriginalType = UE::FindOriginalType(this); !OriginalType.IsEmpty())
					{
						EnumValueName = FName(EnumValueName.ToString().Replace(*Enum->GetName(), *OriginalType.GetName().ToString()));
					}
				#endif
				}
			}
		}

		Slot << EnumValueName;
	}
	else
	{
		UnderlyingProp->SerializeItem(Slot, Value, Defaults);
	}
}

bool FEnumProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	if (Ar.EngineNetVer() < FEngineNetworkCustomVersion::FixEnumSerialization)
	{
		Ar.SerializeBits(Data, FMath::CeilLogTwo64(Enum->GetMaxEnumValue()));
	}
	else
	{
		Ar.SerializeBits(Data, GetMaxNetSerializeBits());
	}

	return true;
}

void FEnumProperty::Serialize( FArchive& Ar )
{
	Super::Serialize(Ar);
	Ar << Enum;
	if (Enum != nullptr)
	{
		Ar.Preload(Enum);
	}
	SerializeSingleField(Ar, UnderlyingProp, this);
}

void FEnumProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Enum);
	Super::AddReferencedObjects(Collector);
}

FString FEnumProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const
{
	check(Enum);
	check(UnderlyingProp);

	const bool bNonNativeEnum = Enum->GetClass() != UEnum::StaticClass(); // cannot use RF_Native flag, because in UHT the flag is not set

	if (!Enum->CppType.IsEmpty())
	{
		return Enum->CppType;
	}

	FString EnumName = Enum->GetName();

	// This would give the wrong result if it's a namespaced type and the CppType hasn't
	// been set, but we do this here in case existing code relies on it... somehow.
	if ((CPPExportFlags & CPPF_BlueprintCppBackend) && bNonNativeEnum)
	{
		ensure(Enum->CppType.IsEmpty());
		FString Result = ::UnicodeToCPPIdentifier(EnumName, false, TEXT("E__"));
		return Result;
	}

	return EnumName;
}

void FEnumProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (Enum == nullptr)
	{
		UE_LOGF(
			LogClass,
			Warning,
			"Member 'Enum' of %ls is nullptr, export operation would fail. This can occur when the enum class has been moved or deleted.",
			*GetFullName()
		);
		return;
	}

	FNumericProperty* LocalUnderlyingProp = UnderlyingProp;
	check(LocalUnderlyingProp);

	UE::CoreUObject::Private::ExportEnumToBuffer(Enum, this, UnderlyingProp, ValueStr, PropertyValueOrContainer, PropertyPointerType, DefaultValue, Parent, PortFlags, ExportRootScope);
}

const TCHAR* FEnumProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	check(Enum);
	check(UnderlyingProp);
	
	if (!(PortFlags & PPF_ConsoleVariable))
	{
		return UE::CoreUObject::Private::ImportEnumFromBuffer(Enum, this, UnderlyingProp, TEXT("FEnumProperty"), Buffer, ContainerOrPropertyPtr, PropertyPointerType, ErrorText);
	}

	// UnderlyingProp has a 0 offset so we need to make sure we convert the container pointer to the actual value pointer
	Buffer = UnderlyingProp->ImportText_Internal(Buffer, PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType), EPropertyPointerType::Direct, Parent, PortFlags, ErrorText);
	return Buffer;
}

FString FEnumProperty::GetCPPMacroType(FString& ExtendedTypeText) const
{
	ExtendedTypeText = Enum->GetName();
	return TEXT("ENUM");
}

void FEnumProperty::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	//OutDeps.Add(UnderlyingProp);
	OutDeps.Add(Enum);
}

void FEnumProperty::LinkInternal(FArchive& Ar)
{
	check(UnderlyingProp);

	UnderlyingProp->Link(Ar);

	this->SetElementSize(UnderlyingProp->GetElementSize());
	this->PropertyFlags |= CPF_IsPlainOldData | CPF_NoDestructor | CPF_ZeroConstructor;

	PropertyFlags |= (UnderlyingProp->PropertyFlags & CPF_HasGetValueTypeHash);
}

bool FEnumProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	return UnderlyingProp->Identical(A, B, PortFlags);
}

int32 FEnumProperty::GetMinAlignment() const
{
	return UnderlyingProp->GetMinAlignment();
}

bool FEnumProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && static_cast<const FEnumProperty*>(Other)->Enum == Enum;
}

EConvertFromTypeResult FEnumProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot , uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	const EName* TagType = Tag.Type.ToEName();
	if (!TagType || Tag.Type.GetNumber())
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}

	if (*TagType == NAME_EnumProperty)
	{
	#if WITH_EDITORONLY_DATA
		if (UNLIKELY(FUObjectThreadContext::Get().GetSerializeContext()->bTrackUnknownProperties && !CanSerializeFromTypeName(Tag.GetType())))
		{
			FName EnumValueName;
			int64 EnumValue = 0;
			TryLoadEnumValueByName(Slot, Slot.GetUnderlyingArchive(), Enum, EnumValueName, EnumValue);

			check(UnderlyingProp);
			UnderlyingProp->SetIntPropertyValue(ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex), EnumValue);

			return EConvertFromTypeResult::Converted;
		}
	#endif
		return EConvertFromTypeResult::UseSerializeItem;
	}

	if (!Enum || !UnderlyingProp)
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}

	switch (*TagType)
	{
	default:
		return EConvertFromTypeResult::UseSerializeItem;
	case NAME_ByteProperty:
		if (Tag.GetType().GetParameterCount() == 0)
		{
			// A nested property would lose its enum name on previous versions. Handle this case for backward compatibility reasons.
			if (GetOwner<FProperty>() && Slot.GetArchiveState().UEVer() < EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
			{
				UE::FPropertyTypeNameBuilder TypeBuilder;
				TypeBuilder.AddName(Tag.Type);
				TypeBuilder.BeginParameters();
				TypeBuilder.AddPath(Enum);
				TypeBuilder.EndParameters();

				FPropertyTag InnerPropertyTag;
				InnerPropertyTag.SetType(TypeBuilder.Build());
				InnerPropertyTag.Name = Tag.Name;
				InnerPropertyTag.ArrayIndex = 0;

				int64 PreviousValue = FNumericProperty::ReadEnumAsInt64(Slot, DefaultsStruct, InnerPropertyTag);
				UnderlyingProp->SetIntPropertyValue(ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex), PreviousValue);
			}
			else
			{
				// A byte property gained an enum.
				UE::CoreUObject::Private::ConvertIntToEnumProperty<uint8>(Slot, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
			}
		}
		else
		{
			FName EnumValueName;
			int64 EnumValue = 0;
			TryLoadEnumValueByName(Slot, Slot.GetUnderlyingArchive(), Enum, EnumValueName, EnumValue);
			UnderlyingProp->SetIntPropertyValue(ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex), EnumValue);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_Int8Property:
		UE::CoreUObject::Private::ConvertIntToEnumProperty<int8>(Slot, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
		return EConvertFromTypeResult::Converted;
	case NAME_Int16Property:
		UE::CoreUObject::Private::ConvertIntToEnumProperty<int16>(Slot, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
		return EConvertFromTypeResult::Converted;
	case NAME_IntProperty:
		UE::CoreUObject::Private::ConvertIntToEnumProperty<int32>(Slot, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
		return EConvertFromTypeResult::Converted;
	case NAME_Int64Property:
		UE::CoreUObject::Private::ConvertIntToEnumProperty<int64>(Slot, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
		return EConvertFromTypeResult::Converted;
	case NAME_UInt16Property:
		UE::CoreUObject::Private::ConvertIntToEnumProperty<uint16>(Slot, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
		return EConvertFromTypeResult::Converted;
	case NAME_UInt32Property:
		UE::CoreUObject::Private::ConvertIntToEnumProperty<uint32>(Slot, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
		return EConvertFromTypeResult::Converted;
	case NAME_UInt64Property:
		UE::CoreUObject::Private::ConvertIntToEnumProperty<uint64>(Slot, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
		return EConvertFromTypeResult::Converted;
	case NAME_BoolProperty:
		UE::CoreUObject::Private::ConvertIntValueToEnumProperty<uint8>(Tag.BoolVal, this, UnderlyingProp, Enum, ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex));
		return EConvertFromTypeResult::Converted;
	}
}

#if WITH_EDITORONLY_DATA
void FEnumProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (Enum)
	{
		FNameBuilder NameBuilder;
		Enum->GetPathName(nullptr, NameBuilder);
		Builder.Update(NameBuilder.GetData(), NameBuilder.Len() * sizeof(NameBuilder.GetData()[0]));
		int32 Num = Enum->NumEnums();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			AppendHash(Builder, Enum->GetNameByIndex(Index));
		}

		// All redirectors that should trigger a recook are handled either by schema hash, package digest, package redirector dependencies or 
		// global core redirect dependencies. But there is one case not handled : native package redirectors mapping an enum value to another. 
		// To detect when an enum value is redirected to another value, I add the FCoreRedirect to the schema hash. This will trigger
		// a recook when an enum property value might be changed because of a redirector.
		// It might trigger useless recooks as well since the redirection could be about a value not used by the current object but it's
		// better than not detecting those cases at all.
		// Adding enum value redirectors to the schema hash will also change the package digest of any package containing an object with this enum. 
		// This is also a case I want to detect. For example : package A depends on package B and package B has an object with an enum property. 
		// If the enum property value is redirected, then it could matter for package A. The change of package digest for B will trigger a recook of A.
		FCoreRedirectObjectName OldName(*NameBuilder);
		FCoreRedirectObjectName NewName;
		const FCoreRedirect* Redirect = nullptr;
		FCoreRedirects::RedirectNameAndValues(ECoreRedirectFlags::Type_Enum, OldName, NewName, &Redirect);

		if (Redirect)
		{
			Redirect->AppendHash(Builder);
		}
	}
}
#endif

uint32 FEnumProperty::GetValueTypeHashInternal(const void* Src) const
{
	check(UnderlyingProp);
	return UnderlyingProp->GetValueTypeHash(Src);
}

FField* FEnumProperty::GetInnerFieldByName(const FName& InName)
{
	if (UnderlyingProp && UnderlyingProp->GetFName() == InName)
	{
		return UnderlyingProp;
	}
	return nullptr;
}


void FEnumProperty::GetInnerFields(TArray<FField*>& OutFields)
{
	if (UnderlyingProp)
	{
		OutFields.Add(UnderlyingProp);
		UnderlyingProp->GetInnerFields(OutFields);
	}
}

uint64 FEnumProperty::GetMaxNetSerializeBits() const
{
	const uint64 MaxBits = GetElementSize() * 8;
	const uint64 DesiredBits = FMath::CeilLogTwo64(Enum->GetMaxEnumValue() + 1);
	
	return FMath::Min(DesiredBits, MaxBits);
}

bool FEnumProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	if (!Super::LoadTypeName(Type, Tag))
	{
		return false;
	}

	const UE::FPropertyTypeName TypePath = Type.GetParameter(0);
	UEnum* LocalEnum = UE::FindObjectByTypePath<UEnum>(TypePath);
	if (!LocalEnum)
	{
	#if WITH_EDITORONLY_DATA
		LocalEnum = StaticEnum<EFallbackEnum>();
		SetMetaData(UE::NAME_OriginalType, *WriteToString<256>(TypePath));
	#else
		return false;
	#endif
	}

	const UE::FPropertyTypeName UnderlyingType = Type.GetParameter(1);
	FField* Field = FField::TryConstruct(UnderlyingType.GetName(), this, GetFName());
	if (FNumericProperty* Property = CastField<FNumericProperty>(Field); Property && Property->LoadTypeName(UnderlyingType, Tag))
	{
		Enum = LocalEnum;
		UE_CLOGF(!Property->CanHoldValue(Enum->GetMaxEnumValue()), LogClass, Warning,
			"Enum '%ls' does not fit in a %ls loading property '%ls'.",
			*WriteToString<64>(Enum->GetFName()), *WriteToString<32>(Property->GetID()), *WriteToString<32>(GetFName()));
		AddCppProperty(Property);
		return true;
	}
	delete Field;
	return false;
}

void FEnumProperty::SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const
{
	Super::SaveTypeName(Type);

	if (const UEnum* LocalEnum = Enum)
	{
		check(UnderlyingProp);
		Type.BeginParameters();
	#if WITH_EDITORONLY_DATA
		if (const UE::FPropertyTypeName OriginalType = UE::FindOriginalType(this); !OriginalType.IsEmpty())
		{
			Type.AddType(OriginalType);
		}
		else
	#endif // WITH_EDITORONLY_DATA
		{
			Type.AddPath(LocalEnum);
		}
		UnderlyingProp->SaveTypeName(Type);
		Type.EndParameters();
	}
}

bool FEnumProperty::CanSerializeFromTypeName(UE::FPropertyTypeName Type) const
{
	if (!Super::CanSerializeFromTypeName(Type))
	{
		return false;
	}

	const UEnum* LocalEnum = Enum;
	if (!LocalEnum)
	{
		return false;
	}

	const FName EnumName = Type.GetParameterName(0);
	if (EnumName == LocalEnum->GetFName())
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	if (const UE::FPropertyTypeName OriginalType = UE::FindOriginalType(this); !OriginalType.IsEmpty())
	{
		return EnumName == OriginalType.GetName();
	}
#endif // WITH_EDITORONLY_DATA

	return false;
}
