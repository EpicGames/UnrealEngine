// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ExpandingChunkedList.h: Unreal realtime garbage collection internal helpers
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include <atomic>
#include "Misc/SpinLock.h"

template<typename T, int32 NumElementsPerChunk = 64>
class TExpandingChunkedList
{
	struct FChunk
	{
		/** Pointer to the next chunk */
		FChunk* Next = nullptr;
		/** Number of items in this chunk, this can be > NumElementsPerChunk if the chunk is full */
		int32 NumItems = 0;
		/** Chunk items */
		T Items[NumElementsPerChunk];
	};

	alignas(PLATFORM_CACHE_LINE_SIZE) mutable UE::FSpinLock Lock;

	/** Head of the chunk list */
	FChunk* Head = nullptr;

	static void DeleteChunks(FChunk* Head)
	{
		for (FChunk* Chunk = Head; Chunk;)
		{
			FChunk* NextChunk = Chunk->Next;
			FMemory::Free(Chunk);
			Chunk = NextChunk;
		}
	}

public:

	~TExpandingChunkedList()
	{
		Empty();
	}

	/**
	* Thread safe: Pushes a new item into the list
	**/
	void Push(T Item)
	{

		// If we have a chunk and aren't full, then use it
		Lock.Lock();
		FChunk* OldHead = Head;
		if (Head != nullptr && Head->NumItems < NumElementsPerChunk)
		{
			Head->Items[Head->NumItems++] = Item;
			Lock.Unlock();
			return;
		}

		// Outside of the lock, allocate the chunk
		Lock.Unlock();
		FChunk* NewChunk = new (FMemory::Malloc(sizeof(FChunk), PLATFORM_CACHE_LINE_SIZE)) FChunk();
		Lock.Lock();

		// If the head hs changed, then someone else has added a new node, try to add to that one
		if (Head != nullptr && OldHead != Head && Head->NumItems < NumElementsPerChunk)
		{
			Head->Items[Head->NumItems++] = Item;
			Lock.Unlock();
			FMemory::Free(NewChunk); //-V611 There is no destructor and the memory came from Malloc...
			return;
		}

		// Otherwise, we can just add out allocated node
		NewChunk->Next = Head;
		Head = NewChunk;
		Head->Items[Head->NumItems++] = Item;
		Lock.Unlock();
		return;
	}

	/**
	* Checks if the list is empty
	**/
	FORCEINLINE bool IsEmpty() const
	{
		Lock.Lock();
		bool bIsEmpty = Head == nullptr;
		Lock.Unlock();
		return bIsEmpty;
	}

	/**
	* Empties the list and frees its memory
	**/
	void Empty()
	{
		Lock.Lock();
		FChunk* Current = Head;
		Head = nullptr;
		Lock.Unlock();
		DeleteChunks(Current);
	}

	/**
	* Moves all items to the provided array and empties the list
	**/
	FORCEINLINE void PopAllAndEmpty(TArray<T>& OutArray)
	{
		Lock.Lock();
		FChunk* Current = Head;
		Head = nullptr;
		Lock.Unlock();

		for (FChunk* Chunk = Current; Chunk; Chunk = Chunk->Next)
		{
			OutArray.Append(Chunk->Items, Chunk->NumItems);
		}
		DeleteChunks(Current);
	}
};
