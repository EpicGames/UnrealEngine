// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Field.cpp: Defines FField property system fundamentals
=============================================================================*/

#include "UObject/Field.h"
#include "UObject/Class.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/ScopeLock.h"
#include "Misc/StringBuilder.h"
#include "Serialization/MemoryWriter.h"
#include "Misc/AutomationTest.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/OutputDeviceHelper.h"
#include "Misc/FeedbackContext.h"
#include "Misc/OutputDeviceConsole.h"
#include "Modules/ModuleManager.h"
#include "UObject/StrProperty.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/MetaData.h"
#include "Templates/Casts.h"
#include "UObject/DebugSerializationFlags.h"
#include "UObject/PropertyTag.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/Stack.h"
#include "Misc/PackageName.h"
#include "UObject/ObjectResource.h"
#include "UObject/Interface.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/LinkerPlaceholderFunction.h"
#include "UObject/StructScriptLoader.h"
#include "UObject/PropertyHelper.h"
#include "UObject/CoreRedirects.h"
#include "Serialization/ArchiveScriptReferenceCollector.h"
#include "UObject/FrameworkObjectVersion.h"
#include "UObject/WeakFieldPtr.h"
#include "Templates/SubclassOf.h"
#include "UObject/TextProperty.h"
#include "UObject/EnumProperty.h"

#if UE_WITH_CONSTINIT_UOBJECT
FRegisterFieldClass::FRegisterFieldClass(FFieldClass* InFieldClass, const TCHAR* InCppName)
{
#if !IS_MONOLITHIC
	InFieldClass->SuperClass = InFieldClass->GetSuperStaticClass();
#endif
	InFieldClass->Name = &InCppName[1];
	FFieldClass::GetAllFieldClasses().Add(InFieldClass);
	FFieldClass::GetNameToFieldClassMap().Add(InFieldClass->Name, InFieldClass);
}
#endif // UE_WITH_CONSTINIT_UOBJECT

FFieldClass::FFieldClass(const TCHAR* InCPPName, uint64 InId, EClassCastFlags InCastFlags, FFieldClass* InSuperClass, FConstructFunction InConstructFn)
	: ClassFlags(CLASS_None)
	, Id(InId)
	, CastFlags(InCastFlags)
	, SuperClass(InSuperClass)
	, DefaultObject(nullptr)
	, ConstructFn(InConstructFn)
{
	check(InCPPName);
	// Skip the 'F' prefix for the name
	check(InCPPName[0] == 'F');
	Name = ++InCPPName;

	GetAllFieldClasses().Add(this);
	GetNameToFieldClassMap().Add(Name, this);
}

FFieldClass::~FFieldClass()
{
	delete DefaultObject;
	DefaultObject = nullptr;
}

TArray<FFieldClass*>& FFieldClass::GetAllFieldClasses()
{
	static TArray<FFieldClass*> AllClasses;
	return AllClasses;
}
TMap<FName, FFieldClass*>& FFieldClass::GetNameToFieldClassMap()
{
	static TMap<FName, FFieldClass*> NameToFieldClassMap;
	return NameToFieldClassMap;
}

FField* FFieldClass::ConstructDefaultObject()
{
	FField* NewDefault = Construct(UClass::StaticClass()->GetOutermost(), *FString::Printf(TEXT("Default__%s"), *GetName()));
	return NewDefault;
}

FString FFieldClass::GetDescription() const
{
	return GetName();
}

FText FFieldClass::GetDisplayNameText() const
{
	return FText::FromString(GetName());
}

FArchive& operator << (FArchive& Ar, FFieldClass*& InOutFieldClass)
{
	FName ClassName = InOutFieldClass ? InOutFieldClass->GetFName() : NAME_None;
	Ar << ClassName;
	if (Ar.IsLoading())
	{
		if (ClassName != NAME_None)
		{
			InOutFieldClass = FFieldClass::GetNameToFieldClassMap().FindRef(ClassName);
		}
		else
		{
			InOutFieldClass = nullptr;
		}
	}
	return Ar;
}

FFieldVariant FFieldVariant::GetOwnerVariant() const
{
	if (IsUObject())
	{
		return ToUObjectUnsafe()->GetOuter();
	}
	else
	{
		return ToFieldUnsafe()->GetOwnerVariant();
	}
}

bool FFieldVariant::IsA(const UClass* InClass) const
{
	return IsUObject() && ToUObjectUnsafe() && ToUObjectUnsafe()->IsA(InClass);
}
bool FFieldVariant::IsA(const FFieldClass* InClass) const
{
	return !IsUObject() && ToFieldUnsafe() && ToFieldUnsafe()->IsA(InClass);
}

UClass* FFieldVariant::GetOwnerClass() const
{
	check(Container.Object);
	if (IsUObject())
	{
		return CastChecked<UField>(ToUObjectUnsafe())->GetOwnerClass();
	}
	else
	{
		return ToFieldUnsafe()->GetOwnerClass();
	}
}

FString FFieldVariant::GetFullName() const
{
	if (IsUObject())
	{
		return ToUObjectUnsafe()->GetFullName();
	}
	else
	{
		return ToFieldUnsafe()->GetFullName();
	}
}

FString FFieldVariant::GetPathName() const
{
	if (IsUObject())
	{
		return ToUObjectUnsafe()->GetPathName();
	}
	else
	{
		return ToFieldUnsafe()->GetPathName();
	}
}

FString FFieldVariant::GetName() const
{
	if (IsUObject())
	{
		return ToUObjectUnsafe()->GetName();
	}
	else
	{
		return ToFieldUnsafe()->GetName();
	}
}

FName FFieldVariant::GetFName() const
{
	if (IsUObject())
	{
		return ToUObjectUnsafe()->GetFName();
	}
	else
	{
		return ToFieldUnsafe()->GetFName();
	}
}

FString FFieldVariant::GetClassName() const
{
	check(Container.Object);
	if (IsUObject())
	{
		return ToUObjectUnsafe()->GetClass()->GetName();
	}
	else
	{
		return ToFieldUnsafe()->GetClass()->GetName();
	}
}

bool FFieldVariant::IsNative() const
{
	check(Container.Object);
	if (IsUObject())
	{
		return ToUObjectUnsafe()->IsNative();
	}
	else
	{
		return ToFieldUnsafe()->IsNative();
	}
}

UPackage* FFieldVariant::GetOutermost() const
{
	check(Container.Object);
	if (IsUObject())
	{
		return ToUObjectUnsafe()->GetOutermost();
	}
	else
	{
		return ToFieldUnsafe()->GetOutermost();
	}
}

bool FFieldVariant::IsValidLowLevel() const
{
	check(Container.Object);
	if (IsUObject())
	{
		return ToUObjectUnsafe()->IsValidLowLevel();
	}
	else
	{
		return !!ToFieldUnsafe();
	}
}

#if WITH_METADATA
bool FFieldVariant::HasMetaData(const FName& Key) const
{
	check(Container.Object);
	if (IsUObject())
	{
		return CastChecked<UField>(ToUObjectUnsafe())->HasMetaData(Key);
	}
	else
	{
		return ToFieldUnsafe()->HasMetaData(Key);
	}
}
#endif // WITH_METADATA

/*-----------------------------------------------------------------------------
FField implementation.
-----------------------------------------------------------------------------*/

FField* FField::Construct(const FFieldVariant& InOwner, const FName& InName)
{
	// Can't construct an abstract type
	return nullptr;
}

#if UE_WITH_CONSTINIT_UOBJECT
constinit TNoDestroy<TFieldClass<FField>> FField::ConstInitClass{NoDestroyConstEval};
#endif // UE_WITH_CONSTINIT_UOBJECT

FFieldClass* FField::StaticClass()
{
#if UE_WITH_CONSTINIT_UOBJECT
	return &ConstInitClass;
#else
	static FFieldClass StaticFieldClass(TEXT("FField"), FField::StaticClassCastFlagsPrivate(), FField::StaticClassCastFlags(), nullptr, &FField::Construct);
	return &StaticFieldClass;
#endif
}

FField::FField(EInternal InInernal, FFieldClass* InClass)
	: ClassPrivate(InClass)
	, Owner(nullptr)
	, Next(nullptr)
#if WITH_METADATA
	, MetaDataMap(nullptr)
#endif // WITH_METADATA
{
	checkSlow(ClassPrivate);
}

FField::FField(FFieldVariant InOwner, const FName& InName)
	: Owner(InOwner)
	, Next(nullptr)
	, NamePrivate(InName)
#if WITH_METADATA
	, MetaDataMap(nullptr)
#endif // WITH_METADATA
{
}

FField::~FField()
{
#if WITH_METADATA
	if (MetaDataMap)
	{
		delete MetaDataMap;
		MetaDataMap = nullptr;
	}
#endif // WITH_METADATA
}

#if WITH_EDITORONLY_DATA
FField::FField(UField* InField)
	: Next(nullptr)
	, NamePrivate(InField->GetFName())
	, MetaDataMap(nullptr)
{
	check(InField);
	InField->ConditionalPreload();
	// Associate this FField with the UField we're constructing from so that next time something tries to convert it,
	// it can already grab the cached new FField
	InField->SetAssociatedFField(this);

	UObject* OriginalOuter = InField->GetOuter();
	if (UProperty* OuterProperty = Cast<UProperty>(OriginalOuter))
	{
		FField* NewOwnerField = OuterProperty->GetAssociatedFField();
		if (!NewOwnerField)
		{
			NewOwnerField = CreateFromUField(OuterProperty);
			OuterProperty->SetAssociatedFField(NewOwnerField);
		}
		Owner = NewOwnerField;
	}
	else
	{
		Owner = OriginalOuter;
	}

	TMap<FName, FString>* FieldMetaDataMap = FMetaData::GetMapForObject(InField);
	if (FieldMetaDataMap && FieldMetaDataMap->Num())
	{
		MetaDataMap = new TMap<FName, FString>(*FieldMetaDataMap);
	}
}
#endif // WITH_EDITORONLY_DATA

UClass* FField::GetOwnerClass() const
{
	UField* OwnerUField = GetOwnerUField();
	if (OwnerUField)
	{
		UClass* OwnerClass = Cast<UClass>(OwnerUField);
		if (OwnerClass)
		{
			return OwnerClass;
		}
		else
		{
			return OwnerUField->GetOwnerClass();
		}
	}
	else
	{
		return nullptr;
	}
}

UStruct* FField::GetOwnerStruct() const
{
	UObject* Obj = GetOwnerUObject();
	do
	{
		if (UStruct* Result = Cast<UStruct>(Obj))
		{
			return Result;
		}
		Obj = Obj->GetOuter();
	} while (Obj);

	return nullptr;
}

UField* FField::GetOwnerUField() const
{
	UObject* Obj = GetOwnerUObject();
	return CastChecked<UField>(Obj);
}

UPackage* FField::GetOutermost() const
{
	UObject* OwnerUObject = GetOwnerUObject();
	check(OwnerUObject);
	return OwnerUObject->GetOutermost();
}

void FField::Serialize(FArchive& Ar)
{
	Ar << NamePrivate;

	if (!Ar.IsFilterEditorOnly())
	{
		uint32 FlagsPrivateDummy = 0;
		Ar << FlagsPrivateDummy;
	}

#if WITH_METADATA	
	if (!Ar.IsCooking())
	{
		UPackage* Package = GetOutermost();
		if (!Package || !Package->HasAnyPackageFlags(PKG_Cooked))
		{
			bool bHasMetaData = false;
			if (Ar.IsLoading())
			{
				Ar << bHasMetaData;

			}
			else
			{
				bHasMetaData = MetaDataMap && MetaDataMap->Num();
				Ar << bHasMetaData;
			}
			if (bHasMetaData)
			{
				if (!MetaDataMap)
				{
					MetaDataMap = new TMap<FName, FString>();
				}
				Ar << *MetaDataMap;
			}			
		}		
	}
#endif // WITH_METADATA
}

void FField::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{

}

void FField::AddReferencedObjects(FReferenceCollector& Collector)
{
	TObjectPtr<UObject> OwnerUObject = ObjectPtrWrap(Owner.ToUObject());
	if (OwnerUObject)
	{
		Collector.AddReferencedObject(OwnerUObject);
		if (!OwnerUObject)
		{
			Owner = FFieldVariant{};
		}
	}
}

bool FField::IsRooted() const
{
	bool bIsRooted = false;
	UObject* OwnerObject = GetOwnerUObject();
	while (OwnerObject)
	{
		if (OwnerObject->IsRooted())
		{
			bIsRooted = true;
			break;
		}
		else
		{
			OwnerObject = OwnerObject->GetOuter();
		}
	}
	return bIsRooted;
}

bool FField::IsNative() const
{
	UObject* OwnerObject = GetOwnerUObject();
	if (OwnerObject)
	{
		return OwnerObject->IsNative();
	}
	check(false); // we shouldn't ever get here but if we do it's fine to remove this check
	return true;
}

bool FField::IsValidLowLevel() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return IsThisNotNull(this, "FField::IsValidLowLevel");
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FField::IsIn(const UObject* InOwner) const
{
	check(InOwner);
	UObject* OwnerObject = GetOwnerUObject();
	if (OwnerObject)
	{
		if (OwnerObject == InOwner)
		{
			return true;
		}
		else
		{
			return OwnerObject->IsIn(InOwner);
		}
	}
	return false;
}

bool FField::IsIn(const FField* InOwner) const
{
	for (FField* OwnerField = GetOwner<FField>(); OwnerField; OwnerField = OwnerField->GetOwner<FField>())
	{
		if (OwnerField == InOwner)
		{
			return true;
		}
	}
	return false;
}

FLinkerLoad* FField::GetLinker() const
{
	UObject* OwnerObject = GetOwnerUObject();
	return OwnerObject ? OwnerObject->GetLinker() : nullptr;
}

void FField::AddCppProperty(FProperty* Property)
{
	UE_LOGF(LogClass, Fatal, "FField::AddCppProperty");
}

FString FField::GetPathName(const UObject* StopOuter /*= nullptr*/) const
{
	TStringBuilder<256> ResultString;
	GetPathName(StopOuter, ResultString);
	return FString(FStringView(ResultString));
}

void FField::GetPathName(const UObject* StopOuter, FStringBuilderBase& ResultString) const
{
	TArray<FName, TInlineAllocator<16>> ParentFields;
	for (FFieldVariant TempOwner = Owner; TempOwner.IsValid(); TempOwner = TempOwner.GetOwnerVariant())
	{		
		if (!TempOwner.IsUObject())
		{
			FField* FieldOwner = TempOwner.ToField();
			ParentFields.Add(FieldOwner->GetFName());
		}
		else
		{
			UObject* ObjectOwner = TempOwner.ToUObject();
			ObjectOwner->GetPathName(StopOuter, ResultString);
			ResultString << SUBOBJECT_DELIMITER_CHAR;
			break;
		}
	}

	for (int FieldIndex = ParentFields.Num() - 1; FieldIndex >= 0; --FieldIndex)
	{
		ParentFields[FieldIndex].AppendString(ResultString);
		ResultString << TEXT(".");
	}
	GetFName().AppendString(ResultString);
}

FString FField::GetFullName() const
{
	FString FullName = GetClass()->GetName();
	FullName += TEXT(" ");
	FullName += GetPathName();
	return FullName;
}

UObject* FField::GetTypedOwner(UClass* Target) const
{
	UObject* Result = nullptr;
	for (UObject* NextOuter = GetOwnerUObject(); Result == nullptr && NextOuter != nullptr; NextOuter = NextOuter->GetOuter())
	{
		if (NextOuter->IsA(Target))
		{
			Result = NextOuter;
		}
	}
	return Result;
}

FString FField::GetAuthoredName() const
{
	UStruct* Struct = GetOwnerStruct();
	if (Struct)
	{
		return Struct->GetAuthoredNameForField(this);
	}
	return FString();
}

void FField::Rename(const FName& NewName)
{
	NamePrivate = NewName;
	// @todo: What about FFieldPath now?
}

FField* FField::GetTypedOwner(FFieldClass* Target) const
{
	FField* Result = nullptr;
	for (FField* NextOuter = GetOwner<FField>(); Result == nullptr && NextOuter != nullptr; NextOuter = NextOuter->GetOwner<FField>())
	{
		if (NextOuter->IsA(Target))
		{
			Result = NextOuter;
		}
	} 
	return Result;
}

#if WITH_METADATA
const FString* FField::FindMetaData(const TCHAR* Key) const
{
	return FindMetaData(FName(Key, FNAME_Find));
}

const FString* FField::FindMetaData(const FName& Key) const
{
	return (MetaDataMap ? MetaDataMap->Find(Key) : nullptr);
}

/**
* Find the metadata value associated with the key
*
* @param Key The key to lookup in the metadata
* @return The value associated with the key
*/
const FString& FField::GetMetaData(const TCHAR* Key) const
{
	return GetMetaData(FName(Key, FNAME_Find));
}

const FString& FField::GetMetaData(const FName& Key) const
{
	// if not found, return a static empty string
	static FString EmptyString;

	// every key needs to be valid and meta data needs to exist
	if (Key == NAME_None || !MetaDataMap)
	{
		return EmptyString;
	}

	// look for the property
	const FString* ValuePtr = MetaDataMap->Find(Key);

	// if we didn't find it, return NULL
	if (!ValuePtr)
	{
		return EmptyString;
	}

	// if we found it, return the pointer to the character data
	return *ValuePtr;
}

const FText FField::GetMetaDataText(const TCHAR* MetaDataKey, const FTextKey LocalizationNamespace, const FTextKey LocalizationKey) const
{
	FString DefaultMetaData;

	if (const FString* FoundMetaData = FindMetaData(MetaDataKey))
	{
		DefaultMetaData = *FoundMetaData;
	}

	// If attempting to grab the DisplayName metadata, we must correct the source string and output it as a DisplayString for lookup
	if (DefaultMetaData.IsEmpty() && FString(MetaDataKey) == TEXT("DisplayName"))
	{
		DefaultMetaData = FName::NameToDisplayString(GetAuthoredName(), IsA(FBoolProperty::StaticClass()));
	}

	FText LocalizedMetaData;
	if (!DefaultMetaData.IsEmpty())
	{
		LocalizedMetaData = FText::AsLocalizable_Advanced(LocalizationNamespace, LocalizationKey, MoveTemp(DefaultMetaData));
	}
	return LocalizedMetaData;
}

const FText FField::GetMetaDataText(const FName& MetaDataKey, const FTextKey LocalizationNamespace, const FTextKey LocalizationKey) const
{
	FString DefaultMetaData;

	if (const FString* FoundMetaData = FindMetaData(MetaDataKey))
	{
		DefaultMetaData = *FoundMetaData;
	}

	// If attempting to grab the DisplayName metadata, we must correct the source string and output it as a DisplayString for lookup
	if (DefaultMetaData.IsEmpty() && MetaDataKey == TEXT("DisplayName"))
	{
		DefaultMetaData = FName::NameToDisplayString(GetAuthoredName(), IsA(FBoolProperty::StaticClass()));
	}

	FText LocalizedMetaData;
	if (!DefaultMetaData.IsEmpty())
	{
		LocalizedMetaData = FText::AsLocalizable_Advanced(LocalizationNamespace, LocalizationKey, MoveTemp(DefaultMetaData));
	}
	return LocalizedMetaData;
}

/**
* Sets the metadata value associated with the key
*
* @param Key The key to lookup in the metadata
* @return The value associated with the key
*/
void FField::SetMetaData(const TCHAR* Key, const TCHAR* InValue)
{
	SetMetaData(FName(Key), FString(InValue));
}

void FField::SetMetaData(const TCHAR* Key, FString&& InValue)
{
	SetMetaData(FName(Key), MoveTemp(InValue));
}

void FField::SetMetaData(const FName& Key, const TCHAR* InValue)
{
	SetMetaData(Key, FString(InValue));
}

void FField::SetMetaData(const FName& Key, FString&& InValue)
{
	check(Key != NAME_None);
	if (!MetaDataMap)
	{
		MetaDataMap = new TMap<FName, FString>();
	}
	MetaDataMap->Add(Key, MoveTemp(InValue));
}

UClass* FField::GetClassMetaData(const TCHAR* Key) const
{
	const FString& ClassName = GetMetaData(Key);
	UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(ClassName);
	return FoundClass;
}

UClass* FField::GetClassMetaData(const FName& Key) const
{
	const FString& ClassName = GetMetaData(Key);
	UClass* FoundClass = UClass::TryFindTypeSlow<UClass>(ClassName);;
	return FoundClass;
}

void FField::RemoveMetaData(const TCHAR* Key)
{
	RemoveMetaData(FName(Key));
}

void FField::RemoveMetaData(const FName& Key)
{
	check(Key != NAME_None);
	if (MetaDataMap)
	{
		MetaDataMap->Remove(Key);
	}
}

const TMap<FName, FString>* FField::GetMetaDataMap() const
{
	return MetaDataMap;
}

void FField::AppendMetaData(const TMap<FName, FString>& MetaDataMapToAppend)
{
	if (MetaDataMapToAppend.Num() > 0)
	{
		if (MetaDataMap)
		{
			MetaDataMap->Append(MetaDataMapToAppend);
		}
		else
		{
			MetaDataMap = new TMap<FName, FString>(MetaDataMapToAppend);
		}
	}
}

void FField::CopyMetaData(const FField* InSourceField, FField* InDestField)
{
	check(InSourceField);
	check(InDestField);
	if (InSourceField->MetaDataMap)
	{
		if (!InDestField->MetaDataMap)
		{
			InDestField->MetaDataMap = new TMap<FName, FString>();
		}
		*InDestField->MetaDataMap = *InSourceField->MetaDataMap;
	}
	else if (InDestField->MetaDataMap)
	{
		delete InDestField->MetaDataMap;
		InDestField->MetaDataMap = nullptr;
	}
}

#endif // WITH_METADATA

void FField::PostDuplicate(const FField& InField)
{

}

FField* FField::Duplicate(const FField* InField, FFieldVariant DestOwner, const FName DestName)
{
	check(InField);
	FField* NewField = InField->GetClass()->Construct(DestOwner, (DestName == NAME_None) ? InField->GetFName() : DestName);
	NewField->PostDuplicate(*InField);
	return NewField;
}

FField* FField::Construct(const FName& FieldTypeName, const FFieldVariant& InOwner, const FName& InName)
{
	FFieldClass** FieldClassPtr = FFieldClass::GetNameToFieldClassMap().Find(FieldTypeName);
	checkf(FieldClassPtr, TEXT("Field type %s does not exist"), *FieldTypeName.ToString());
	FField* Instance = (*FieldClassPtr)->Construct(InOwner, InName);
	return Instance;
}

FField* FField::TryConstruct(const FName& FieldTypeName, const FFieldVariant& InOwner, const FName& InName)
{
	if (FFieldClass* FieldClassPtr = FFieldClass::GetNameToFieldClassMap().FindRef(FieldTypeName))
	{
		return FieldClassPtr->Construct(InOwner, InName);
	}
	return nullptr;
}

FName FField::GenerateFFieldName(FFieldVariant InOwner /** Unused yet */, FFieldClass* InClass)
{
	check(InClass);
	return FName(*InClass->GetName(), InClass->GetNextUniqueNameIndex());
}

#if WITH_EDITORONLY_DATA
FString FField::GetFullGroupName(bool bStartWithOuter) const
{
	if (bStartWithOuter)
	{
		if (Owner.IsValid())
		{
			if (Owner.IsUObject())
			{
				return Owner.ToUObject()->GetPathName(Owner.ToUObject()->GetOutermost());
			}
			else
			{
				return Owner.ToField()->GetPathName(GetOutermost());
			}
		}
		else
		{
			return FString();
		}
	}
	else
	{
		return GetPathName(GetOutermost());
	}
}

struct FFieldDisplayNameHelper
{
	static FString Get(const FField& Object)
	{
		if (const FProperty* Property = CastField<FProperty>(&Object))
		{
			if (auto OwnerStruct = Property->GetOwnerStruct())
			{
				return OwnerStruct->GetAuthoredNameForField(Property);
			}
		}

		return Object.GetName();
	}
};

/**
* Finds the localized display name or native display name as a fallback.
*
* @return The display name for this object.
*/
FText FField::GetDisplayNameText() const
{
	static const FTextKey Namespace = TEXT("UObjectDisplayNames");
	static const FName NAME_DisplayName("DisplayName");

	const FString Key = GetFullGroupName(false);

	FString NativeDisplayName;
	if (const FString* FoundMetaData = FindMetaData(NAME_DisplayName))
	{
		NativeDisplayName = *FoundMetaData;
	}
	else
	{
		NativeDisplayName = FName::NameToDisplayString(FFieldDisplayNameHelper::Get(*this), IsA<FBoolProperty>());
	}

	return FText::AsLocalizable_Advanced(Namespace, Key, MoveTemp(NativeDisplayName));
}

/**
* Finds the localized tooltip or native tooltip as a fallback.
*
* @return The tooltip for this object.
*/
FText FField::GetToolTipText(bool bShortTooltip) const
{
	bool bFoundShortTooltip = false;
	static const FName NAME_Tooltip("Tooltip");
	static const FName NAME_ShortTooltip("ShortTooltip");
	FText LocalizedToolTip;
	FString NativeToolTip;

	if (bShortTooltip)
	{
		NativeToolTip = GetMetaData(NAME_ShortTooltip);
		if (NativeToolTip.IsEmpty())
		{
			NativeToolTip = GetMetaData(NAME_Tooltip);
		}
		else
		{
			bFoundShortTooltip = true;
		}
	}
	else
	{
		NativeToolTip = GetMetaData(NAME_Tooltip);
	}

	const FString Namespace = bFoundShortTooltip ? TEXT("UObjectShortTooltips") : TEXT("UObjectToolTips");
	const FString Key = GetFullGroupName(false);
	if (!FText::FindTextInLiveTable_Advanced(Namespace, Key, /*OUT*/LocalizedToolTip, &NativeToolTip))
	{
		if (!NativeToolTip.IsEmpty())
		{
			static const FString DoxygenSee("@see");
			static const FString TooltipSee("See:");
			if (NativeToolTip.ReplaceInline(*DoxygenSee, *TooltipSee) > 0)
			{
				NativeToolTip.TrimEndInline();
			}
		}
		LocalizedToolTip = FText::AsLocalizable_Advanced(Namespace, Key, MoveTemp(NativeToolTip));
	}

	return LocalizedToolTip;
}

FField::FOnConvertCustomUFieldToFField& FField::GetConvertCustomUFieldToFFieldDelegate()
{
	static FOnConvertCustomUFieldToFField ConvertCustomUFieldToFFieldDelegate;
	return ConvertCustomUFieldToFFieldDelegate;
}

FField* FField::CreateFromUField(UField* InField)
{
	FField* NewField = nullptr;
	UClass* UFieldClass = InField->GetClass();

	if (UFieldClass == UByteProperty::StaticClass())
	{
		NewField = new FByteProperty(InField);
	}
	else if (UFieldClass == UInt8Property::StaticClass())
	{
		NewField = new FInt8Property(InField);
	}
	else if (UFieldClass == UInt16Property::StaticClass())
	{
		NewField = new FInt16Property(InField);
	}
	else if (UFieldClass == UIntProperty::StaticClass())
	{
		NewField = new FIntProperty(InField);
	}
	else if (UFieldClass == UInt64Property::StaticClass())
	{
		NewField = new FInt64Property(InField);
	}
	else if (UFieldClass == UUInt16Property::StaticClass())
	{
		NewField = new FUInt16Property(InField);
	}
	else if (UFieldClass == UUInt32Property::StaticClass())
	{
		NewField = new FUInt32Property(InField);
	}
	else if (UFieldClass == UUInt64Property::StaticClass())
	{
		NewField = new FUInt64Property(InField);
	}
	else if (UFieldClass == UFloatProperty::StaticClass())
	{
		NewField = new FFloatProperty(InField);
	}
	else if (UFieldClass == UDoubleProperty::StaticClass())
	{
		NewField = new FDoubleProperty(InField);
	}
	else if (UFieldClass == UBoolProperty::StaticClass())
	{
		NewField = new FBoolProperty(InField);
	}
	else if (UFieldClass == UObjectProperty::StaticClass())
	{
		FObjectProperty* ObjectProperty = new FObjectProperty(InField);
		NewField = ObjectProperty;
		if (FLinkerLoad::IsImportLazyLoadEnabled())
		{
			ObjectProperty->SetPropertyFlags(CPF_TObjectPtrWrapper);
		}
	}
	else if (UFieldClass == UWeakObjectProperty::StaticClass())
	{
		NewField = new FWeakObjectProperty(InField);
	}
	else if (UFieldClass == ULazyObjectProperty::StaticClass())
	{
		NewField = new FLazyObjectProperty(InField);
	}
	else if (UFieldClass == USoftObjectProperty::StaticClass())
	{
		NewField = new FSoftObjectProperty(InField);
	}
	else if (UFieldClass == UClassProperty::StaticClass())
	{
		NewField = new FClassProperty(InField);
	}
	else if (UFieldClass == USoftClassProperty::StaticClass())
	{
		NewField = new FSoftClassProperty(InField);
	}
	else if (UFieldClass == UInterfaceProperty::StaticClass())
	{
		NewField = new FInterfaceProperty(InField);
	}
	else if (UFieldClass == UNameProperty::StaticClass())
	{
		NewField = new FNameProperty(InField);
	}
	else if (UFieldClass == UStrProperty::StaticClass())
	{
		NewField = new FStrProperty(InField);
	}
	else if (UFieldClass == UArrayProperty::StaticClass())
	{
		NewField = new FArrayProperty(InField);
	}
	else if (UFieldClass == UMapProperty::StaticClass())
	{
		NewField = new FMapProperty(InField);
	}
	else if (UFieldClass == USetProperty::StaticClass())
	{
		NewField = new FSetProperty(InField);
	}
	else if (UFieldClass == UStructProperty::StaticClass())
	{
		NewField = new FStructProperty(InField);
	}
	else if (UFieldClass == UDelegateProperty::StaticClass())
	{
		NewField = new FDelegateProperty(InField);
	}
	else if (UFieldClass == UMulticastInlineDelegateProperty::StaticClass())
	{
		NewField = new FMulticastInlineDelegateProperty(InField);
	}
	else if (UFieldClass == UMulticastSparseDelegateProperty::StaticClass())
	{
		NewField = new FMulticastSparseDelegateProperty(InField);
	}
	else if (UFieldClass == UEnumProperty::StaticClass())
	{
		NewField = new FEnumProperty(InField);
	}
	else if (UFieldClass == UTextProperty::StaticClass())
	{
		NewField = new FTextProperty(InField);
	}
	else
	{
		FFieldClass** FieldClassPtr = FFieldClass::GetNameToFieldClassMap().Find(UFieldClass->GetFName());
		checkf(FieldClassPtr, TEXT("Cannot create an FField from %s. The class is either abstract or not implemented."), *InField->GetFullName());
		GetConvertCustomUFieldToFFieldDelegate().Broadcast(*FieldClassPtr, InField, NewField);
		checkf(NewField, TEXT("Cannot create an FField from %s. The class conversion function is not implemented or not bound to FField::GetConvertCustomUFieldToFField() delegate."), *InField->GetFullName());
	}

	return NewField;
}
#endif // WITH_EDITORONLY_DATA

FString GetFullNameSafe(const FField* InField)
{
	if (InField)
	{
		return InField->GetFullName();
	}
	else
	{
		return TEXT("none");
	}
}

FString GetPathNameSafe(const FField* InField, const UObject* StopOuter /*= nullptr*/)
{
	if (InField)
	{
		return InField->GetPathName(StopOuter);
	}
	else
	{
		return TEXT("none");
	}
}

FField* FindFPropertyByPath(const TCHAR* InFieldPath)
{ 
	// Expected format: FullPackageName.OwnerName:Field
	FField* FoundField = nullptr;
	TCHAR PathBuffer[NAME_SIZE];
	int32 LastSubobjectDelimiterIndex = -1;

	for (int32 Index = 0; InFieldPath[Index] != '\0'; ++Index)
	{
		if (InFieldPath[Index] == SUBOBJECT_DELIMITER_CHAR)
		{
			LastSubobjectDelimiterIndex = Index;
		}
	}
	if (LastSubobjectDelimiterIndex >= 0)
	{
		// Get the UObject part
		FCString::Strncpy(PathBuffer, InFieldPath, LastSubobjectDelimiterIndex + 1);

		// And the FField part
		InFieldPath += (LastSubobjectDelimiterIndex + 1);

		UStruct* Owner = FindObject<UStruct>(nullptr, PathBuffer);
		if (Owner)
		{
#if DO_CHECK
			for (const TCHAR* TestChar = InFieldPath; *TestChar != '\0'; TestChar++)
			{
				checkf(*TestChar != ':' && *TestChar != '.', TEXT("FindFieldByPath can't resolve nested properties: %s"), InFieldPath);
			}
#endif
			FoundField = FindFProperty<FField>(Owner, InFieldPath);
		}
	}
	return FoundField;
}

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

TEST_CASE_NAMED(FFieldVariantTest, "System::CoreUObject::FFieldVariant", "[CoreUObject]")
{
	FFieldVariant NullObjectVariant((UObject*)nullptr);
	CHECK_MESSAGE("Null object coerces to false", !NullObjectVariant);
	FFieldVariant NullFieldVariant((FField*)nullptr);
	CHECK_MESSAGE("Null field coerces to false", !NullFieldVariant);

	// Fails, impact of changing behavior is unknown at this time
	// CHECK_EQUALS("Nulls equal to each other", NullObjectVariant, NullFieldVariant);
}
#endif

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FFieldCastTest, "UObject.Field Cast", EAutomationTestFlags::EditorContext | EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ServerContext | EAutomationTestFlags::SmokeFilter)

bool FFieldCastTest::RunTest(const FString& Parameters)
{
	FBoolProperty* DefaultBoolProperty = static_cast<FBoolProperty*>(FBoolProperty::StaticClass()->GetDefaultObject());
	FIntProperty* DefaultIntProperty = static_cast<FIntProperty*>(FIntProperty::StaticClass()->GetDefaultObject());
	FNumericProperty* DefaultNumericProperty = static_cast<FNumericProperty*>(FNumericProperty::StaticClass()->GetDefaultObject());
	FProperty* BaseProperty = nullptr;

	AddErrorIfFalse(CastField<FBoolProperty>(DefaultBoolProperty) == DefaultBoolProperty, TEXT("DefaultBoolProperty could not be CastField to a FBoolProperty."));
	BaseProperty = DefaultBoolProperty;
	AddErrorIfFalse(CastField<FBoolProperty>(BaseProperty) == DefaultBoolProperty, TEXT("Property could not be CastField to a FBoolProperty."));

	AddErrorIfFalse(CastField<FBoolProperty>(DefaultIntProperty) == nullptr, TEXT("DefaultIntProperty was CastField to a FBoolProperty."));
	BaseProperty = DefaultIntProperty;
	AddErrorIfFalse(CastField<FBoolProperty>(BaseProperty) == nullptr, TEXT("DefaultIntProperty was CastField to a FBoolProperty."));

	AddErrorIfFalse(CastField<FBoolProperty>(DefaultNumericProperty) == nullptr, TEXT("DefaultNumericProperty was CastField to a FBoolProperty."));
	BaseProperty = DefaultNumericProperty;
	AddErrorIfFalse(CastField<FBoolProperty>(BaseProperty) == nullptr, TEXT("BaseProperty was CastField to a FBoolProperty."));

	AddErrorIfFalse(CastField<FNumericProperty>(DefaultIntProperty) == DefaultIntProperty, TEXT("DefaultIntProperty could not be CastField to a FNumericProperty."));
	BaseProperty = DefaultIntProperty;
	AddErrorIfFalse(CastField<FNumericProperty>(BaseProperty) == DefaultIntProperty, TEXT("BaseProperty could not be CastField to a FNumericProperty."));

	BaseProperty = nullptr;
	AddErrorIfFalse(CastField<FNumericProperty>(BaseProperty) == nullptr, TEXT("nullptr was CastField to a FNumericProperty."));

	return true;
}

#endif
