// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FieldPath.cpp: Pointer to UObject asset, keeps extra information so that it is works even if the asset is not in memory
=============================================================================*/

#include "UObject/FieldPath.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "UObject/UObjectArray.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/PropertyHelper.h"
#include "Algo/Reverse.h"

/** Helper function used for logging the path to property */
static FString PathToString(const TArray<FName>& InPath)
{
	// See FFieldPath::Generate for formatting specifics
	// Generate path from the outermost (last item) to the property (first item)

	// Initilize with the number of delimiters we're going to add
	int32 PathLength = InPath.Num() ? (InPath.Num() - 1) : 0; 
	for (FName PathSegment : InPath)
	{
		PathLength += PathSegment.GetStringLength();
	}
	FString Result;
	if (PathLength)
	{
		// Allocate once and construct the path string
		Result.Reserve(PathLength);
		Result += InPath.Last().ToString();

		if (InPath.Num() > 1)
		{
			// Path may be either a full path from the package to the property or just from the outer property to the inner property
			// If it's a full path, the delimiter char for the next object (owner struct) will be a '.' otherwise we assume it's all subobjects
			if (Result.Len() && Result[0] == '/')
			{
				Result += '.';
			}
			else
			{
				Result += SUBOBJECT_DELIMITER_CHAR;
			}

			// Add the remainder of the path (it's all subobjects from this point)
			for (int32 PathIndex = InPath.Num() - 2; PathIndex >= 0; --PathIndex)
			{
				Result += InPath[PathIndex].ToString();
				if (PathIndex > 0)
				{
					Result += SUBOBJECT_DELIMITER_CHAR;
				}
			}
		}
	}

	return Result;
}

#if WITH_EDITORONLY_DATA
FFieldPath::FFieldPath(UField* InField, const FName& InPropertyTypeName)
{
	if (InField)
	{
		// Must be constructed from the equivalent UField class
		check(InField->GetClass()->GetFName() == InPropertyTypeName);
		GenerateFromUField(InField);
	}
}
#endif

void FFieldPath::Generate(FField* InField)
{
	Reset();
	if (InField)
	{
		// Add field names from the innermost to the outermost, stop at the owner struct
		UStruct* Owner = InField->GetOwnerStruct();
		check(Owner); //  A field that has no owner is not allowed in FFieldPath
		for (FFieldVariant Iter(InField); Iter.IsValid(); Iter = Iter.GetOwnerVariant())
		{
			UStruct* MaybeOwner = Iter.Get<UStruct>();
			if (MaybeOwner == Owner)
			{
				break;
			}
			else
			{
				Path.Add(Iter.GetFName());
			}
		}
		ResolvedOwner = Owner;
		ResolvedField = InField;
#if WITH_EDITORONLY_DATA
		FieldPathSerialNumber = Owner->FieldPathSerialNumber;
		InitialFieldClass = InField->GetClass();
#endif // WITH_EDITORONLY_DATA
	}
}

void FFieldPath::Generate(const TCHAR* InFieldPathString)
{
	// Expected format is: FullPackageName.Subobject[:Subobject:...]:FieldName
	check(InFieldPathString);
	Reset();

	TArray<FName> Result;
	{
		
		TCHAR NameBuffer[NAME_SIZE];
		int32 NameIndex = 0;

		// Construct names
		while (true)
		{
			if (*InFieldPathString == TEXT('.') || *InFieldPathString == SUBOBJECT_DELIMITER_CHAR || *InFieldPathString == TEXT('\0'))
			{
				NameBuffer[NameIndex] = TEXT('\0');
				if (NameIndex > 0)
				{
					Result.Add(NameBuffer);
					NameIndex = 0;
				}
				if (*InFieldPathString == TEXT('\0'))
				{
					break;
				}
			}
			else
			{
				NameBuffer[NameIndex++] = *InFieldPathString;
			}
			++InFieldPathString;
		}
	}

	if (Result.Num() > 1)
	{
		// Reverse the order so that it's innermost to outermost
		Algo::Reverse(Result);
	}
	else if (Result.Num() == 1 && Result[0] == NAME_None)
	{
		Result.Empty();
	}

	Path = MoveTemp(Result);
	ResolveField();
}

UStruct* FFieldPath::ConvertFromFullPath(FLinkerLoad* InLinker)
{
	// First try the StaticFindObject approach
	UStruct* Owner = TryToResolveOwnerFromStruct();
	if (!Owner)
	{
		// UClass::Serialize() unhashes the class currently being serialized so SFO will not work on it
		// If possible try to find the owner through the current serialize context
		if (InLinker)
		{
			Owner = TryToResolveOwnerFromLinker(InLinker);
		}
	}
	// It's possible the full path points to a renamed asset
	UE_CLOGF(!Owner && Path.Num(), LogProperty, Verbose, "Failed resolve owner when converting from full property path \"%ls\"", *PathToString(Path));
	return Owner;
}

/** Helper function that checks if two paths have identical trailing sequences */
static bool HasCommonTrailingSequence(const TArray<FName>& PathA, const TArray<FName>& PathB)
{
	bool bTrailingSeuenceIdentical = true;
	const int32 MaxNumToCheck = FMath::Min(PathA.Num(), PathB.Num());
	for (int32 PathIndex = 0; PathIndex < MaxNumToCheck; ++PathIndex)
	{
		if (PathA[PathA.Num() - PathIndex - 1] != PathB[PathB.Num() - PathIndex - 1])
		{
			bTrailingSeuenceIdentical = false;
			break;
		}
	}
	return bTrailingSeuenceIdentical;
}

UStruct* FFieldPath::TryToResolveOwnerFromLinker(FLinkerLoad* InLinker) const
{
	UStruct* OwnerStruct = nullptr;
	check(InLinker);
	FUObjectSerializeContext* Context = FUObjectThreadContext::Get().GetSerializeContext();
	if (Context && Context->SerializedObject && Context->SerializedObject->IsA<UStruct>())
	{
		TArray<FName> StructPath;
		for (UObject* Obj = Context->SerializedObject; Obj; Obj = Obj->GetOuter())
		{
			StructPath.Add(Obj->GetFName());
		}
		// Check if our Path contains StructToPath (the assumption is that Path has more elements than struct path) otherwise we know the struct can't hold our property
		if (StructPath.Num() < Path.Num() && HasCommonTrailingSequence(StructPath, Path))
		{
			int32 OwnerPathIndex = Path.Num() - StructPath.Num();
			OwnerStruct = CastChecked<UStruct>(Context->SerializedObject);
			ResolvedOwner = OwnerStruct;

			// Remove portion of the path responsible for storing the owner path
			FFieldPath* MutableThis = const_cast<FFieldPath*>(this);
			MutableThis->Path.RemoveAt(OwnerPathIndex, Path.Num() - OwnerPathIndex);
		}
	}
	return OwnerStruct;
}

UStruct* FFieldPath::TryToResolveOwnerFromStruct(UStruct* InCurrentStruct /*= nullptr*/, EPathResolveType InResolveType /*= FFieldPath::UseStructIfOuterNotFound*/) const
{
	// Resolve from the outermost to the innermost UObject
	UObject* LastOuter = nullptr;
	int32 LastOuterIndex = -1;
	for (int32 PathIndex = Path.Num() - 1; PathIndex > 0; --PathIndex)
	{
		UObject* Outer = StaticFindObjectFastSafe(UObject::StaticClass(), LastOuter, Path[PathIndex]);

		if (InCurrentStruct && PathIndex == (Path.Num() - 1))
		{
			UObject* CurrentOutermost = InCurrentStruct->GetOutermost();

			if ((InResolveType == FFieldPath::UseStructIfOuterNotFound && !Outer) || // Outer is not found so try to use the provided struct Outer
				(InResolveType == FFieldPath::UseStructAlways && CurrentOutermost != Outer) // Prioritize the provided struct Outer over the resolved one
				)
			{
				Outer = CurrentOutermost;
			}
		}
		if (!Outer)
		{
			break;
		}
		LastOuterIndex = PathIndex;
		LastOuter = Outer;
	}
	UStruct* Owner = Cast<UStruct>(LastOuter);

	if (!Owner && LastOuter)
	{
		// The outer resolve uses names, so follow a redirector chain if needed
		while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(LastOuter))
		{
			LastOuter = Redirector->DestinationObject;
			Owner = Cast<UStruct>(LastOuter);
		}
	}

	if (Owner)
	{
		ResolvedOwner = Owner;

		// Remove portion of the path responsible for storing the owner path
		FFieldPath* MutableThis = const_cast<FFieldPath*>(this);
		MutableThis->Path.RemoveAt(LastOuterIndex, MutableThis->Path.Num() - LastOuterIndex);
	}
	return Owner;
}

FField* FFieldPath::TryToResolvePath(UStruct* InCurrentStruct, FFieldPath::EPathResolveType InResolveType /*= FFieldPath::UseStructIfOuterNotFound*/) const
{	
	FField* Result = nullptr;
	UStruct* Owner = ResolvedOwner.Get();
	if (!Owner)
	{
		// We're probably dealing with an old path format where the Path array contained the full path to the field
		Owner = TryToResolveOwnerFromStruct(InCurrentStruct, InResolveType);
	}
	// At this point the owner should've been fully resolved
	if (Owner && Path.Num() > 0)
	{
		int32 PathIndex = Path.Num() - 1;
		Result = FindFProperty<FField>(Owner, Path[PathIndex]);
		if (!Result)
		{
			const FName RedirectedName = FProperty::FindRedirectedPropertyName(Owner, Path[PathIndex]);
			if (!RedirectedName.IsNone())
			{
				Result = FindFProperty<FField>(Owner, RedirectedName);
			}
		}
		PathIndex--;
		while (Result && PathIndex >= 0)
		{
			// Nested property
			Result = Result->GetInnerFieldByName(Path[PathIndex--]);
		}
	}
	return Result;
}

FString FFieldPath::ToString() const
{
	FString Result;
	if (UStruct* Owner = ResolvedOwner.Get())
	{
		if (FField* LocalResolvedField = GetTyped(FField::StaticClass()))
		{
			Result = LocalResolvedField->GetPathName();
		}
		else
		{
			Result = Owner->GetPathName();
			Result += SUBOBJECT_DELIMITER_CHAR;
			Result += PathToString(Path);
		}
	}
	else
	{
		// Revert back to old path format where the package and UStruct owner were also specified
		Result = PathToString(Path);
	}

	return Result;
}

FArchive& operator<<(FArchive& Ar, FFieldPath& InOutPropertyPath)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	if (Ar.IsSaving())
	{		
		UStruct* Owner = InOutPropertyPath.ResolvedOwner.Get();
		bool bFilterEditorOnlyProperty = false;
		if (Owner && Ar.IsFilterEditorOnly())
		{
			FProperty* ResolvedProperty = CastField<FProperty>(InOutPropertyPath.GetTyped(FProperty::StaticClass()));
			if (ResolvedProperty && ResolvedProperty->HasAnyPropertyFlags(CPF_EditorOnly))
			{
				bFilterEditorOnlyProperty = true;
			}
		}
		if (!Owner || bFilterEditorOnlyProperty)
		{
			// If there's no owner, make sure we don't serialize potentially unresolved path from the actual field
			// because if we don't save the owner, we won't be able to resolve it anyway.
			// Possible scenario: the owner was GC'd and the path is no longer valid
			TArray<FName> EmptyPath;
			UStruct* NullOwner = nullptr;
			Ar << EmptyPath;
			Ar << NullOwner;
			UE_CLOGF(InOutPropertyPath.Path.Num(), LogProperty, Verbose, "Null owner but property path is not empty when saving \"%ls\"", *PathToString(InOutPropertyPath.Path));
		}
		else
		{
			Ar << InOutPropertyPath.Path;
			Ar << Owner;
		}
		checkf(Owner == InOutPropertyPath.ResolvedOwner.Get(), TEXT("FFieldPath owner has changed when saving, this is not allowed (Path: \"%s\", new owner: \"%s\")"),
			*InOutPropertyPath.ToString(), *GetPathNameSafe(Owner));
	}
	else
	{		
		Ar << InOutPropertyPath.Path;
		// The old serialization format could save 'None' paths, they should be just empty
		if (InOutPropertyPath.Path.Num() == 1 && InOutPropertyPath.Path[0] == NAME_None)
		{
			InOutPropertyPath.Path.Empty();
		}
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::FFieldPathOwnerSerialization || Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::FFieldPathOwnerSerialization)
		{
			TObjectPtr<UStruct> SerializedOwner = InOutPropertyPath.ResolvedOwner.Get();
			Ar << SerializedOwner;
			InOutPropertyPath.ResolvedOwner = SerializedOwner;
			if (!SerializedOwner)
			{
				UE_CLOGF(InOutPropertyPath.Path.Num(), LogProperty, Verbose, "Serialized null owner for property \"%ls\"", *PathToString(InOutPropertyPath.Path));
				// At this point it makes no sense to keep the remainder of the path around as it will produce unnecessary warnings
				// Possible scenario: owning struct was not cooked or deleted and the asset was not loaded
				InOutPropertyPath.Path.Empty();
			}
		}
		else if (InOutPropertyPath.Path.Num())
		{
			// Convert from the old format: resolve the owner now and remove its path from the field path
			FLinkerLoad* Linker = Cast<FLinkerLoad>(Ar.GetLinker());
			UStruct* Owner = InOutPropertyPath.ConvertFromFullPath(Linker);

			InOutPropertyPath.ResolvedOwner = Owner;

			// This usually happens when the old path format serialized a path and then the owner struct's package got renamed or moved
			// There's code to handle that in bot UClass and UAnimBlueprintGeneratedClass
			UE_CLOGF(!Owner, LogProperty, Verbose, "Failed to resolve property owner from Path \"%ls\"", *PathToString(InOutPropertyPath.Path));
		}
		else
		{
			InOutPropertyPath.ResolvedOwner = nullptr;
		}

		if (!Ar.IsObjectReferenceCollector())
		{
			InOutPropertyPath.ClearCachedField();
		}
	}

	return Ar;
}

#if WITH_EDITORONLY_DATA
void FFieldPath::GenerateFromUField(UField* InField)
{
	Reset();
	for (UObject* Obj = InField; Obj; Obj = Obj->GetOuter())
	{
		UStruct* MaybeOwner = Cast<UStruct>(Obj);
		if (MaybeOwner)
		{
			ResolvedOwner = MaybeOwner;
			break;
		}
		else
		{
			Path.Add(Obj->GetFName());
		}
	}
}

bool FFieldPath::IsFieldPathSerialNumberIdentical(UStruct* InStruct) const
{
	return FieldPathSerialNumber == InStruct->FieldPathSerialNumber;
}

int32 FFieldPath::GetFieldPathSerialNumber(UStruct* InStruct) const
{
	return InStruct->FieldPathSerialNumber;
}
#endif // WITH_EDITORONLY_DATA
