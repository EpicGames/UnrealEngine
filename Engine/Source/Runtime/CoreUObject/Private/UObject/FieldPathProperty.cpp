// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/FieldPathProperty.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/Package.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ConstructorHelpers.h"
#include "Misc/Parse.h"
#include "UObject/CoreRedirects.h"
#include "UObject/PropertyHelper.h"

IMPLEMENT_FIELD(FFieldPathProperty)

FFieldPathProperty::FFieldPathProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, PropertyClass(nullptr)
{
}

FFieldPathProperty::FFieldPathProperty(FFieldVariant InOwner, const UECodeGen_Private::FFieldPathPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
	PropertyClass = Prop.PropertyClassFunc();
}

#if WITH_EDITORONLY_DATA
FFieldPathProperty::FFieldPathProperty(UField* InField)
	: Super(InField)
	, PropertyClass(nullptr)
{
	check(InField);
	PropertyClass = FFieldClass::GetNameToFieldClassMap().FindRef(InField->GetClass()->GetFName());
}
#endif // WITH_EDITORONLY_DATA

EConvertFromTypeResult FFieldPathProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	// Convert UProperty object to TFieldPath
	if (Tag.Type == NAME_ObjectProperty)
	{
		FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
		check(UnderlyingArchive.IsLoading() && UnderlyingArchive.IsPersistent());
		FLinker* Linker = UnderlyingArchive.GetLinker();
		check(Linker);

		FFieldPath ConvertedValue;

		FPackageIndex Index;
		UnderlyingArchive << Index;

		bool bExport = Index.IsExport();
		while (!Index.IsNull())
		{
			const FObjectResource& Res = Linker->ImpExp(Index);
			ConvertedValue.Path.Add(Res.ObjectName);
			Index = Res.OuterIndex;
		}
		if (bExport)
		{
			check(Linker->LinkerRoot);
			ConvertedValue.Path.Add(Linker->LinkerRoot->GetFName());
		}
		if (ConvertedValue.Path.Num())
		{
			ConvertedValue.ConvertFromFullPath(Cast<FLinkerLoad>(Linker));
		}

		SetPropertyValue_InContainer(Data, ConvertedValue, Tag.ArrayIndex);
		return EConvertFromTypeResult::Converted;
	}

	return EConvertFromTypeResult::UseSerializeItem;
}

bool FFieldPathProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	if (!B)
	{
		return false;
	}
	const FFieldPath& ValueA = GetPropertyValue(A);
	const FFieldPath& ValueB = GetPropertyValue(B);
	return ValueA == ValueB;
}

uint32 FFieldPathProperty::GetValueTypeHashInternal(const void* Src) const
{
	return GetTypeHash(GetPropertyValue(Src));
}

void FFieldPathProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FFieldPath* FieldPtr = GetPropertyValuePtr(Value);
	Slot << *FieldPtr;
}

void FFieldPathProperty::ExportText_Internal( FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope ) const
{
	FFieldPath Value;
	
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		GetValue_InContainer(PropertyValueOrContainer, &Value);
	}
	else
	{
		Value = GetPropertyValue(PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType));
	}

	if (PortFlags & PPF_PropertyWindow)
	{
		if (PortFlags & PPF_Delimited)
		{
			ValueStr += TEXT("\"");
			ValueStr += Value.ToString();
			ValueStr += TEXT("\"");
		}
		else
		{
			ValueStr += Value.ToString();
		}
	}
	else
	{
		ValueStr += Value.ToString();
	}
}

const TCHAR* FFieldPathProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	check(Buffer);
	FFieldPath ImportedPath;
	FString PathName;

	if (!(PortFlags & PPF_Delimited))
	{
		PathName = Buffer;
		// in order to indicate that the value was successfully imported, advance the buffer past the last character that was imported
		Buffer += PathName.Len();
	}
	else
	{		
		// Advance to the next delimiter (comma) or the end of the buffer
		int32 SeparatorIndex = 0;
		while (Buffer[SeparatorIndex] != '\0' && Buffer[SeparatorIndex] != ',' && Buffer[SeparatorIndex] != ')')
		{
			++SeparatorIndex;
		}
		// Copy the value string
		PathName = FString::ConstructFromPtrSize(Buffer, SeparatorIndex);
		// Advance the buffer to let the calling function know we succeeded
		Buffer += SeparatorIndex;
	}

	if (PathName.Len())
	{
		// Strip the class name if present, we don't need it
		ConstructorHelpers::StripObjectClass(PathName);
		if (PathName[0] == '\"')
		{
			FString UnquotedPathName;
			if (!FParse::QuotedString(*PathName, UnquotedPathName))
			{
				ErrorText->Logf(ELogVerbosity::Warning, TEXT("FieldPathProperty: Bad quoted string: %s"), *PathName);
				return nullptr;
			}
			PathName = MoveTemp(UnquotedPathName);
		}

		// Attempt to apply core redirects to imported path name
		PathName = RedirectFieldPathName(PathName);

		// Resolve FFieldPath from imported and fixed up path name
		ImportedPath.Generate(*PathName);

		// Set the FFieldPath value within container
		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			SetValue_InContainer(ContainerOrPropertyPtr, ImportedPath);
		}
		else
		{
			FFieldPath* PathPtr = GetPropertyValuePtr(PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType));
			*PathPtr = ImportedPath;
		}
	}	

	return Buffer;
}

void FFieldPathProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << PropertyClass;
}

FString FFieldPathProperty::GetCPPMacroType(FString& ExtendedTypeText) const
{
	check(PropertyClass);
	ExtendedTypeText = FString::Printf(TEXT("TFieldPath<F%s>"), *PropertyClass->GetName());
	return TEXT("TFIELDPATH");
}

FString FFieldPathProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const
{
	checkSlow(PropertyClass);
	if (ExtendedTypeText != nullptr)
	{
		FString& InnerTypeText = *ExtendedTypeText;
		InnerTypeText = TEXT("<F");
		InnerTypeText += PropertyClass->GetName();
		InnerTypeText += TEXT(">");
	}
	return TEXT("TFieldPath");
}

bool FFieldPathProperty::SupportsNetSharedSerialization() const
{
	return false;
}

FString FFieldPathProperty::RedirectFieldPathName(const FString& InPathName)
{
	// Apply redirectors to imported PathName value. First deconstruct it into package, class and field names.
	// PathName format: /Script/MyGameModule.MyClass:MyField
	FString OldPackageName;
	FString OldClassName;
	FString OldFieldName;
	FString Tmp;
	if (InPathName.Split(TEXT("."), &OldPackageName, &Tmp) && Tmp.Split(TEXT(":"), &OldClassName, &OldFieldName))
	{
		// Check for full property path redirect
		{
			const FCoreRedirectObjectName OldRedirectName = FCoreRedirectObjectName(InPathName);
			constexpr ECoreRedirectFlags RedirectFlags = ECoreRedirectFlags::Type_Property; 
			FCoreRedirectObjectName NewObjectName;
			const FCoreRedirect* FoundValueRedirect = nullptr;
			if (FCoreRedirects::RedirectNameAndValues(RedirectFlags, OldRedirectName, NewObjectName, &FoundValueRedirect))
			{
				const FString NewPathName = NewObjectName.ToString();
				UE_LOGF(LogCoreRedirects, Verbose, "FFieldPathProperty: Redirected '%ls' -> '%ls'", *InPathName, *NewPathName);
				return NewPathName;
			}
		}
		
		// Check for outer-only redirect
		{
			const FCoreRedirectObjectName OldRedirectName(*OldClassName, NAME_None, *OldPackageName);
			constexpr ECoreRedirectFlags RedirectFlags = ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Type_Struct;
			FCoreRedirectObjectName NewObjectName;
			const FCoreRedirect* FoundValueRedirect = nullptr;
			if (FCoreRedirects::RedirectNameAndValues(RedirectFlags, OldRedirectName, NewObjectName, &FoundValueRedirect))
			{
				const FString NewPathName = FPackageName::ObjectPathCombine(NewObjectName.ToString(), OldFieldName);
				UE_LOGF(LogCoreRedirects, Verbose, "FFieldPathProperty: Redirected '%ls' -> '%ls'", *InPathName, *NewPathName);
				return NewPathName;
			}
		}
	}
	
	return InPathName;
}
