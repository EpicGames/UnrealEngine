// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/FindStronglyConnected.h"
#include "UObject/GarbageCollection.h"
#include "UObject/UObjectIterator.h"
#include "Serialization/ArchiveFindAllRefs.h"

void FFindStronglyConnected::FindAllCycles()
{
	UE_LOGF(LogObj, Log, "Finding Edges");
	for( FThreadSafeObjectIterator It; It; ++It )
	{
		UObject* Object = *It;
		{
			AllObjects.Add(Object);

			FArchiveFindAllRefs ArFind(Object);

			for (int32 Index = 0; Index < ArFind.References.Num(); Index++)
			{
				AllEdges.Add(Object, ArFind.References[Index]);
				if (AllEdges.Num() % 25000 == 0)
				{
					UE_LOGF(LogObj, Log, "Finding Edges %d", AllEdges.Num());
				}
			}
		}
	}
	UE_LOGF(LogObj, Log, "Finding Edges Done %d", AllEdges.Num());

	UE_LOGF(LogObj, Log, "Finding permanent objects");
	TArray<UObject*> Fringe;
	for (int32 Index = 0; Index < AllObjects.Num(); Index++)
	{
		UObject* Object = AllObjects[Index];
		if (Object->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS))
		{
			PermanentObjects.Add(Object);
			Fringe.Add(Object);
		}
	}
	while (Fringe.Num())
	{
		TArray<UObject*> LastFringe;
		Exchange(LastFringe,Fringe);
		for (int32 Index = 0; Index < LastFringe.Num(); Index++)
		{
			UObject* Object = LastFringe[Index];
			TArray<UObject*> Refs;
			AllEdges.MultiFind(Object, Refs);

			for (int32 IndexRef = 0; IndexRef < Refs.Num(); IndexRef++)
			{
				UObject* RefObject = Refs[IndexRef];
				if (!PermanentObjects.Contains(RefObject))
				{
					PermanentObjects.Add(RefObject);
					Fringe.Add(RefObject);
				}
			}
		}
	}
	for (int32 Index = 0; Index < AllObjects.Num(); Index++)
	{
		UObject* Object = AllObjects[Index];
		if (!PermanentObjects.Contains(Object))
		{
			TempObjects.Add(Object);
		}
	}
	for (TMultiMap<UObject*, UObject*>::TIterator It(AllEdges); It; ++It)
	{
		if (!PermanentObjects.Contains(It.Key()) && !PermanentObjects.Contains(It.Value()))
		{
			Edges.Add(It.Key(),It.Value());
		}
	}
	UE_LOGF(LogObj, Log, "Finding cycles");

	for (int32 Index = 0; Index < TempObjects.Num(); Index++)
	{
		StrongConnect(TempObjects[Index]);
	}
	UE_LOGF(LogObj, Log, "Finding simple cycles");
	Stack.Empty();
	NodeIndex.Empty();
	PrimaryIndex = 1;

	for (int32 Index = 0; Index < Components.Num(); Index++)
	{
		TArray<UObject*>& Dest = SimpleCycles[SimpleCycles.Add(TArray<UObject*>())];
		FindSimpleCycleForComponent(Dest, Components[Index]);
	}
}

void FFindStronglyConnected::FindSimpleCycleForComponent( TArray<UObject*>& Dest, const TArray<UObject*>& Component )
{
	if (Component.Num() < 3)
	{
		Dest = Component; // if it is only one or two items, the cycle is the component
		return;
	}
	FindSimpleCycleForComponentInner(Dest, Component, Component[0]);
	Stack.Empty();
}

bool FFindStronglyConnected::FindSimpleCycleForComponentInner( TArray<UObject*>& Dest, const TArray<UObject*>& Component, UObject* Node )
{
	Stack.Push(Node);
	TArray<UObject*> Refs;
	Edges.MultiFind(Node, Refs);

	for (int32 Index = 0; Index < Refs.Num(); Index++)
	{
		UObject* Other = Refs[Index];
		if (!Component.Contains(Other))
		{
			continue;
		}
		if (Stack.Contains(Other))
		{
			while (1)
			{
				UObject* Out = Stack.Pop(EAllowShrinking::No);
				Dest.Add(Out);
				if (Out == Other)
				{
					return true;
				}
			}
		}
		if (FindSimpleCycleForComponentInner(Dest, Component, Other))
		{
			return true;
		}
	}
	check(0);
	return false;
}


void FFindStronglyConnected::StrongConnect( UObject* Node )
{
	if (NodeIndex.Find(Node))
	{
		return;
	}
	StrongConnectInner(Node);
}

FFindStronglyConnected::NodeInfo* FFindStronglyConnected::StrongConnectInner( UObject* Node )
{
	NodeInfo NewNode;
	NewNode.IndexValue = PrimaryIndex;
	NewNode.LowIndex = PrimaryIndex;
	NewNode.InStack = true;
	PrimaryIndex++;
	Stack.Push(Node);
	NodeInfo* CurrentIndex = &NodeIndex.Add(Node, NewNode);

	TArray<UObject*> Refs;
	Edges.MultiFind(Node, Refs);

	for (int32 Index = 0; Index < Refs.Num(); Index++)
	{
		UObject* Other = Refs[Index];
		NodeInfo* OtherIndexVal = NodeIndex.Find(Other);
		if (!OtherIndexVal)
		{
			OtherIndexVal = StrongConnectInner(Other);
			CurrentIndex = NodeIndex.Find(Node); // this could have reallocated in the recursive call
			CurrentIndex->LowIndex = FMath::Min<int32>(CurrentIndex->LowIndex, OtherIndexVal->LowIndex);
		}
		else if (OtherIndexVal->InStack)
		{
			CurrentIndex->LowIndex = FMath::Min<int32>(CurrentIndex->LowIndex, OtherIndexVal->IndexValue);
		}

	}
	if (CurrentIndex->IndexValue == CurrentIndex->LowIndex)
	{
		TArray<UObject*>& Dest = Components[Components.Add(TArray<UObject*>())];
		while (1)
		{
			UObject* Out = Stack.Pop(EAllowShrinking::No);
			NodeInfo* OutVal = NodeIndex.Find(Out);
			OutVal->InStack = false;
			Dest.Add(Out);
			if (Out == Node)
			{
				break;
			}
		}
	}
	return CurrentIndex;
}

