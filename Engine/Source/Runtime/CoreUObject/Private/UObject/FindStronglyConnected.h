// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Data structure and algorithm to find all cycles in a UObject directed graph **/
struct FFindStronglyConnected
{
	struct NodeInfo
	{
		int32 IndexValue;
		int32 LowIndex;
		bool InStack;
	};

	TMultiMap<UObject*, UObject*> AllEdges;
	TMultiMap<UObject*, UObject*> Edges;
	TArray<UObject*> AllObjects;
	TSet<UObject*> PermanentObjects;
	TArray<UObject*> TempObjects;
	TMap<UObject*, NodeInfo> NodeIndex;
	int32 PrimaryIndex;
	TArray<UObject *> Stack;

	TArray<TArray<UObject *> > Components;
	TArray<TArray<UObject *> > SimpleCycles;

	FFindStronglyConnected()
		: PrimaryIndex(1)
	{
	}

	/** Find all cycles in the uobject reference graph **/
	void FindAllCycles();
	
	void FindSimpleCycleForComponent(TArray<UObject*>& Dest, const TArray<UObject*>& Component);

	void StrongConnect(UObject* Node);
private:
	
	bool FindSimpleCycleForComponentInner(TArray<UObject*>& Dest, const TArray<UObject*>& Component, UObject* Node);

	NodeInfo* StrongConnectInner(UObject* Node);
};
