// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GCObjectInfo.cpp: Info about object participating in Garbage Collection code.
=============================================================================*/

#include "UObject/GCObjectInfo.h"
#include "UObject/Class.h"
#include "Templates/Casts.h"

UObject* FGCObjectInfo::TryResolveObject()
{
	if (this == Class)
	{
		// Recursion base case, the class of class is class 
		return UClass::StaticClass();
	}

	if (Outer == nullptr)
	{
		// We are a package, those are the only objects that are allowed to have no outer
		return StaticFindObjectFast(UObject::StaticClass(), nullptr, Name, EFindObjectFlags::None);
	}

	UClass* ResolvedClass = Cast<UClass>(Class->TryResolveObject());
	if (!ResolvedClass)
	{
		return nullptr;
	}
	
	if (UObject* OuterObj = Outer->TryResolveObject())
	{
		return StaticFindObjectFast(ResolvedClass, OuterObj, Name, EFindObjectFlags::ExactClass);
	}
	return nullptr;
}

void FGCObjectInfo::GetPathName(FStringBuilderBase& ResultString) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IsThisNotNull(this, "FGCObjectInfo::GetPathName"))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		if (Outer)
		{
			Outer->GetPathName(ResultString);

			// SUBOBJECT_DELIMITER_CHAR is used to indicate that this object's outer is not a UPackage
			if (Outer->Class->Name != NAME_Package && Outer->Outer->Class->Name == NAME_Package)
			{
				ResultString << SUBOBJECT_DELIMITER_CHAR;
			}
			else
			{
				ResultString << TEXT('.');
			}
		}
		Name.AppendString(ResultString);
	}
	else
	{
		ResultString << TEXT("None");
	}
}

FString FGCObjectInfo::GetPathName() const
{
	TStringBuilder<256> ResultBuilder;
	GetPathName(ResultBuilder);
	return ResultBuilder.ToString();
}

FString FGCObjectInfo::GetFullName() const
{
	return FString::Printf(TEXT("%s %s"), *GetClassName(), *GetPathName());
}

FGCObjectInfo* FGCObjectInfo::FindOrAddInfoHelper(const UObject* InObject, TMap<const UObject*, FGCObjectInfo*>& InOutObjectToInfoMap)
{
	if (FGCObjectInfo** ExistingObjInfo = InOutObjectToInfoMap.Find(InObject))
	{
		return *ExistingObjInfo;
	}

	FGCObjectInfo* NewInfo = new FGCObjectInfo(InObject);
	InOutObjectToInfoMap.Add(InObject, NewInfo);

	NewInfo->Class = FindOrAddInfoHelper(InObject->GetClass(), InOutObjectToInfoMap);
	NewInfo->Outer = InObject->GetOuter() ? FindOrAddInfoHelper(InObject->GetOuter(), InOutObjectToInfoMap) : nullptr;

	return NewInfo;
};