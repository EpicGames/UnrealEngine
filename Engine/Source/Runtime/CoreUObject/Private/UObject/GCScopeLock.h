// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Garbage Collection scope lock. 
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"
#include "HAL/ThreadSafeBool.h"
#include "Async/SharedMutex.h"
#include "Async/SharedLock.h"
#include "Async/UniqueLock.h"
#include "HAL/Event.h"


/**
* Garbage Collection synchronization objects
* Will not lock other threads if GC is not running.
* Has the ability to only lock for GC if no other locks are present.
*/
class FGCCSyncObject
{
	/** Non zero if any of the non-game threads is blocking GC */
	FThreadSafeCounter AsyncCounter;
	/** Non zero if GC is running */
	FThreadSafeCounter GCCounter;
	/** Non zero if GC wants to run but is blocked by some other thread \
	    This flag is not automatically enforced on the async threads, instead
			threads have to manually implement support for it. */
	TAtomic<int32> GCWantsToRunCounter {0};
	/** Shared mutex for thread safe operations */
	UE::FSharedMutex SharedMutex;
	/** Event used to block non-game threads when GC is running */
	FEvent* GCUnlockedEvent;

public:

	FGCCSyncObject();
	~FGCCSyncObject();

	/** Creates the singleton object */
	static void Create();

	/** Gets the singleton object */
	static FGCCSyncObject& Get();

	/** Lock on non-game thread. Will block if GC is running. */
	void LockAsync()
	{
		if (!IsInGameThread())
		{
			// Wait until GC is done if it was running when entering this function
			bool bLocked = false;
			do
			{
				if (GCCounter.GetValue() > 0)
				{
					GCUnlockedEvent->Wait();
				}
				{
					// We only need a read lock here since AsyncCounter is already thread-safe
					// with other threads that want to lock but we need to guard against
					// the GC trying to take the write lock.
					UE::TSharedLock SharedLock(SharedMutex);
					if (GCCounter.GetValue() == 0)
					{
						AsyncCounter.Increment();
						bLocked = true;
					}
				}
			} while (!bLocked);
		}
	}

	/** Attempt async Lock **/
	bool TryLockAsync(bool bShouldLock = !IsInGameThread())
	{
		bool bLocked = true;
		if (bShouldLock)
		{
			bLocked = false;
			if (GCCounter.GetValue() == 0)
			{
				UE::TSharedLock SharedLock(SharedMutex);
				if (GCCounter.GetValue() == 0)
				{
					AsyncCounter.Increment();
					bLocked = true;
				}
			}
		}

		return bLocked;
	}

	/** Release lock from non-game thread */
	void UnlockAsync(bool bShouldUnlock = !IsInGameThread())
	{
		if (bShouldUnlock)
		{
			AsyncCounter.Decrement();
		}
	}
	/** Lock for GC. Will block if any other thread has locked. */
	void GCLock()
	{
		// Signal other threads that GC wants to run
		SetGCIsWaiting();

		// Wait until all other threads are done if they're currently holding the lock
		bool bLocked = false;
		do
		{
			FPlatformProcess::ConditionalSleep([&]()
			{
				return AsyncCounter.GetValue() == 0;
			});
			{
				// Guard against any reader locks owned by other threads.
				UE::TUniqueLock ExclusiveLock(SharedMutex);
				if (AsyncCounter.GetValue() == 0)
				{
					GCUnlockedEvent->Reset();
					int32 GCCounterValue = GCCounter.Increment();
					check(GCCounterValue == 1); // GCLock doesn't support recursive locks
					// At this point GC can run so remove the signal that it's waiting
					FPlatformMisc::MemoryBarrier();
					ResetGCIsWaiting();
					bLocked = true;
				}
			}
		} while (!bLocked);
	}
	/** Checks if any async thread has a lock */
	bool IsAsyncLocked() const
	{
		return AsyncCounter.GetValue() != 0;
	}
	/** Checks if GC has a lock */
	bool IsGCLocked() const
	{
		return GCCounter.GetValue() != 0;
	}
	/** Lock for GC. Will not block and return false if any other thread has already locked. */
	bool TryGCLock()
	{
		bool bSuccess = false;
		if (AsyncCounter.GetValue() != 0)
		{
			return bSuccess;
		}
		// Guard against any reader locks owned by other threads.
		UE::TUniqueLock ExclusiveLock(SharedMutex);
		// If any other thread is currently locking we just exit
		if (AsyncCounter.GetValue() == 0)
		{
			GCUnlockedEvent->Reset();
			int32 GCCounterValue = GCCounter.Increment();
			check(GCCounterValue == 1); // GCLock doesn't support recursive locks
			bSuccess = true;
		}
		return bSuccess;
	}
	/** Unlock GC */
	void GCUnlock()
	{
		GCUnlockedEvent->Trigger();
		GCCounter.Decrement();
	}

	/** Manually mark GC state as 'waiting to run' */
	void SetGCIsWaiting()
	{
		GCWantsToRunCounter++;
	}

	/** Manually reset GC 'waiting to run' state*/
	void ResetGCIsWaiting()
	{
		GCWantsToRunCounter.Store(0);
	}

	/** True if GC wants to run on the game thread but is maybe blocked by some other thread */
	FORCEINLINE bool IsGCWaiting() const
	{
		return !!GCWantsToRunCounter.Load(EMemoryOrder::Relaxed);
	}
};
