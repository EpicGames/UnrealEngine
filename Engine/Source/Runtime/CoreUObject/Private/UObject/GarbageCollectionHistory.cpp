// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GarbageCollectionHistory.cpp: Unreal object garbage collection history code.
=============================================================================*/

#include "UObject/GarbageCollectionHistory.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/TimeGuard.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectBase.h"
#include "UObject/Object.h"
#include "UObject/UnrealType.h"
#include "UObject/GarbageCollectionInternal.h"
#include "UObject/GCObject.h"
#include "UObject/GCScopeLock.h"
#include "UObject/GCVerseCellInfo.h"
#include "HAL/ExceptionHandling.h"


/*-----------------------------------------------------------------------------
   Garbage collection history code.
-----------------------------------------------------------------------------*/

#if ENABLE_GC_HISTORY

static FAutoConsoleCommand CmdCalculateGCHistorySize(
	TEXT("gc.CalculateHistorySize"),
	TEXT(""),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			int64 TotalSize = FGCHistory::Get().GetAllocatedSize();
			UE_LOGF(LogGarbage, Display, "Memory allocated for GC history: %lldb", TotalSize);
		})
);

static FAutoConsoleCommand CmdSetHistorySize(
	TEXT("gc.HistorySize"),
	TEXT(""),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			if (Args.Num())
			{
				int32 HistorySize = FMath::Max(0, FCString::Atoi(*Args[0]));
				FGCHistory::Get().SetHistorySize(HistorySize);
				UE_LOGF(LogGarbage, Display, "GC history size changed to: %d", HistorySize);
			}
			else
			{
				UE_LOGF(LogGarbage, Display, "GC history size currently set to: %d", FGCHistory::Get().GetHistorySize());
			}			
		})
);

FGCHistory::~FGCHistory()
{
	Cleanup();
}

FGCHistory& FGCHistory::Get()
{
	static FGCHistory Singleton;
	return Singleton;
}

void FGCHistory::Cleanup(FGCSnapshot& InSnapshot)
{
	for (TPair<FReferenceToken, TArray<FGCDirectReference>*>& DirectReferenceInfos : InSnapshot.DirectReferences)
	{
		delete DirectReferenceInfos.Value;
	}
	InSnapshot.DirectReferences.Reset();
	for (TPair<const UObject*, FGCObjectInfo*>& ObjectToInfoPair : InSnapshot.ObjectToInfoMap)
	{
		delete ObjectToInfoPair.Value;
	}
	InSnapshot.ObjectToInfoMap.Reset();
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	for (TPair<const Verse::VCell*, FGCVerseCellInfo*>& VerseCellToInfoPair : InSnapshot.VerseCellToInfoMap)
	{
		delete VerseCellToInfoPair.Value;
	}
	InSnapshot.VerseCellToInfoMap.Reset();
#endif
}

void FGCHistory::Cleanup()
{
	for (FGCSnapshot& Snapshot : Snapshots)
	{
		Cleanup(Snapshot);
	}
	GCBarrier = nullptr;
	Snapshots.Empty();
	MostRecentSnapshotIndex = -1;	
}

void FGCHistory::SetHistorySize(int32 HistorySize)
{
	if (HistorySize != Snapshots.Num())
	{
		Cleanup();
		
		if (HistorySize > 0)
		{
			Snapshots.AddDefaulted(HistorySize);
		}		
	}
	if (HistorySize > 0)
	{
		if (!GCBarrier)
		{
			GCBarrier = NewObject<UGCBarrier>(GetTransientPackage(), TEXT("GC_Barrier"), RF_Transient);
			GCBarrier->AddToRoot();
		}
	}
	else if (GCBarrier)
	{
		GCBarrier->RemoveFromRoot();
	}
}

void FGCHistory::MergeArrayStructHistory(TMap<FReferenceToken, TArray<FGCDirectReference>*>& History, FGCSnapshot& Snapshot)
{
	for (TPair<FReferenceToken, TArray<FGCDirectReference>*>& ReferencePair : History)
	{
		FReferenceToken Referencer = GetInfoReferenceToken(ReferencePair.Key, Snapshot);
		TArray<FGCDirectReference>*& DirectReferencesInfoArray = Snapshot.DirectReferences.FindOrAdd(Referencer);
		if (!DirectReferencesInfoArray)
		{
			DirectReferencesInfoArray = new TArray<FGCDirectReference>();
			DirectReferencesInfoArray->Reserve(ReferencePair.Value->Num());
		}
		for (FGCDirectReference& DirectReference : *ReferencePair.Value)
		{
			DirectReferencesInfoArray->Add(FGCDirectReference(GetInfoReferenceToken(DirectReference.Reference, Snapshot), DirectReference.ReferencerName));
		}
	}
}

FReferenceToken FGCHistory::GetInfoReferenceToken(FReferenceToken InToken, FGCSnapshot& Snapshot)
{
	switch (InToken.GetType())
	{
	case EReferenceTokenType::Object:
		return FReferenceToken(FGCObjectInfo::FindOrAddInfoHelper(InToken.AsObject(), Snapshot.ObjectToInfoMap));
	case EReferenceTokenType::VerseCell:
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		return FReferenceToken(FGCVerseCellInfo::FindOrAddInfoHelper(InToken.AsVerseCell(), Snapshot.VerseCellToInfoMap));
#else
		unimplemented();
		return InToken;
#endif
	default:
		return InToken;
	}
}

void FGCHistory::Update(TConstArrayView<TUniquePtr<UE::GC::FWorkerContext>> Contexts)
{
	if (IsActive())
	{
		// Advance to the next snapshot in the ring buffer
		MostRecentSnapshotIndex = (MostRecentSnapshotIndex + 1) % Snapshots.Num();
		FGCSnapshot& CurrentSnapshot = Snapshots[MostRecentSnapshotIndex];
		if (CurrentSnapshot.DirectReferences.Num())
		{
			// Already taken so cleanup to be able to overwrite
			Cleanup(CurrentSnapshot);
		}

		// Copy information gather in all of the GC struct arrays
		for (const TUniquePtr<UE::GC::FWorkerContext>& Context : Contexts)
		{
			MergeArrayStructHistory(Context->History, CurrentSnapshot);
		}
	}
}

FGCSnapshot* FGCHistory::GetSnapshot(int32 HistoryLevel)
{
	// Negative values make sense as input and are also accepted
	HistoryLevel = FMath::Abs(HistoryLevel);
	if (HistoryLevel >= Snapshots.Num())
	{
		// Clamp to the max available history size
		HistoryLevel = (Snapshots.Num() - 1);
	}
	int32 SnapshotIndex = MostRecentSnapshotIndex - HistoryLevel;
	if (SnapshotIndex < 0)
	{
		// Wrap around the ring buffer
		SnapshotIndex += Snapshots.Num();
	}
	if (SnapshotIndex >= 0 && SnapshotIndex < Snapshots.Num() && Snapshots[SnapshotIndex].ObjectToInfoMap.Num())
	{
		return &Snapshots[SnapshotIndex];
	}
	else
	{
		// This means that the requested history level hasn't been filled yet or history recording is disabled
		return nullptr;
	}
}

int64 FGCHistory::GetAllocatedSize() const
{
	int64 TotalSize = Snapshots.GetAllocatedSize();
	for (const FGCSnapshot& Snapshot : Snapshots)
	{
		TotalSize += Snapshot.GetAllocatedSize();
	}
	return TotalSize;
}

int64 FGCSnapshot::GetAllocatedSize() const
{
	int64 TotalSize = ObjectToInfoMap.GetAllocatedSize() + ObjectToInfoMap.Num() * sizeof(FGCObjectInfo);
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	TotalSize += VerseCellToInfoMap.GetAllocatedSize() + VerseCellToInfoMap.Num() * sizeof(FGCVerseCellInfo);
#endif

	TotalSize += DirectReferences.GetAllocatedSize();
	for (const TPair<FReferenceToken, TArray<FGCDirectReference>*>& ReferencePair : DirectReferences)
	{
		TotalSize += ReferencePair.Value->GetAllocatedSize();
	}
	return TotalSize;
}

#endif // ENABLE_GC_HISTORY
