// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GarbageCollectionInternal.h: Unreal realtime garbage collection internal helpers
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/Object.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GCScopeLock.h"

#include "GarbageCollectionInternal.generated.h"

extern FGCCSyncObject* GGCSingleton;

/** Returns true if GC wants to run */
FORCEINLINE bool IsGarbageCollectionWaiting()
{
	return GGCSingleton->IsGCWaiting();
}

/** Dummy class to represent GC barrier inside of GC history object graph */
UCLASS(Transient)
class UGCBarrier : public UObject
{
    GENERATED_BODY()
public:
};