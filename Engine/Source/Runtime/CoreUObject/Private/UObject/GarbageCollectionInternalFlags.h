// Copyright Epic Games, Inc. All Rights Reserved.

/*==============================================================================================
	GarbageCollectionInternalFlags.h: Unreal realtime garbage collection internal flags helpers
===============================================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Object.h"
#include "UObject/UObjectArray.h"

namespace UE::GC::Private
{

/**
* Access to internal garbage collector rachability flags. Only GC and GC related functions can use these.
* NOTHING except GC should be manipulating reachability flags (including EInternalObjectFlags::Unreachable).
* EInternalObjectFlags::Unreachable is the ONLY reachability flag that can be safely READ by non-GC functions.
* Reading ReachableObjectFlag and MaybeUnreachableObjectFlag outside of GC is NOT THREAD SAFE.
*/
class FGCFlags
{
	/** Current EInternalObjectFlags value representing a reachable object */
	static EInternalObjectFlags ReachableObjectFlag;

	/** Current EInternalObjectFlags value representing a maybe unreachable object */
	static EInternalObjectFlags MaybeUnreachableObjectFlag;

public:

	FORCEINLINE static void SetUnreachable(FUObjectItem* ObjectItem)
	{
		ObjectItem->AtomicallySetFlag_ForGC(EInternalObjectFlags::Unreachable);
	}

	FORCEINLINE static void SetReachable_ForGC(FUObjectItem* ObjectItem)
	{
		ObjectItem->AtomicallySetFlag_ForGC(ReachableObjectFlag);
	}

	FORCEINLINE static bool IsReachable_ForGC(const FUObjectItem* ObjectItem)
	{
		return !!(ObjectItem->GetFlagsInternal() & int32(ReachableObjectFlag));
	}

	FORCEINLINE static void SetMaybeUnreachable_ForGC(FUObjectItem* ObjectItem)
	{
		ObjectItem->AtomicallyClearFlag_ForGC(ReachableObjectFlag);
		ObjectItem->AtomicallySetFlag_ForGC(MaybeUnreachableObjectFlag);
	}

	FORCEINLINE static void ClearMaybeUnreachable_ForGC(FUObjectItem* ObjectItem)
	{
		ObjectItem->AtomicallyClearFlag_ForGC(MaybeUnreachableObjectFlag);
		ObjectItem->AtomicallySetFlag_ForGC(ReachableObjectFlag);
	}

	FORCEINLINE static bool IsMaybeUnreachable_ForGC(const FUObjectItem* ObjectItem)
	{
		return !!(ObjectItem->GetFlagsInternal() & int32(MaybeUnreachableObjectFlag));
	}

	FORCEINLINE static bool IsMaybeUnreachable_ForGC(const UObject* Object)
	{
		const FUObjectItem* ObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(GUObjectArray.ObjectToIndex(Object));
		return IsMaybeUnreachable_ForGC(ObjectItem);
	}

	FORCEINLINE static bool ThisThreadAtomicallyClearedRFUnreachable(FUObjectItem* ObjectItem)
	{
		return ObjectItem->AtomicallyClearFlag_ForGC(EInternalObjectFlags::Unreachable);
	}

	FORCEINLINE static bool ThisThreadAtomicallySetFlag_ForGC(FUObjectItem* ObjectItem, EInternalObjectFlags Flag)
	{
		return ObjectItem->AtomicallySetFlag_ForGC(Flag);
	}

	FORCEINLINE static bool ThisThreadAtomicallyClearedFlag_ForGC(FUObjectItem* ObjectItem, EInternalObjectFlags Flag)
	{
		return ObjectItem->AtomicallyClearFlag_ForGC(Flag);
	}

	FORCEINLINE static void FastMarkAsReachableInterlocked_ForGC(FUObjectItem* ObjectItem)
	{
		FPlatformAtomics::InterlockedAnd(&ObjectItem->FlagsAndRefCount, ~(int64(MaybeUnreachableObjectFlag) << 32));
		FPlatformAtomics::InterlockedOr(&ObjectItem->FlagsAndRefCount, int64(ReachableObjectFlag) << 32);
	}

	FORCEINLINE static void FastMarkAsReachableAndClearReachableInClusterInterlocked_ForGC(FUObjectItem* ObjectItem)
	{
		FPlatformAtomics::InterlockedAnd(&ObjectItem->FlagsAndRefCount, ~(int64(MaybeUnreachableObjectFlag | EInternalObjectFlags::ReachableInCluster) << 32));
		FPlatformAtomics::InterlockedOr(&ObjectItem->FlagsAndRefCount, int64(ReachableObjectFlag) << 32);
	}

	FORCEINLINE static bool MarkAsReachableInterlocked_ForGC(FUObjectItem* ObjectItem)
	{
		const int32 FlagToClear = int32(MaybeUnreachableObjectFlag);
		if (ObjectItem->GetFlagsInternal() & FlagToClear)
		{
			int64 Old = FPlatformAtomics::InterlockedAnd(&ObjectItem->FlagsAndRefCount, ~((int64)FlagToClear << 32));
			FPlatformAtomics::InterlockedOr(&ObjectItem->FlagsAndRefCount, int64(ReachableObjectFlag) << 32);
			return FUObjectItem::GetFlagsInternal(Old) & FlagToClear;
		}
		return false;
	}

	FORCEINLINE static constexpr ::size_t OffsetOfFlags_ForGC()
	{
		return offsetof(FUObjectItem, FlagsAndRefCount);
	}

	FORCEINLINE static void SwapReachableAndMaybeUnreachable()
	{
		// It's important to lock the global UObjectArray so that the flag swap doesn't occur while a new object is being created
		// as we set the ReachableObject flag on all newly created objects
		GUObjectArray.LockInternalArray();
		Swap(ReachableObjectFlag, MaybeUnreachableObjectFlag);
		GUObjectArray.UnlockInternalArray();
	}

	FORCEINLINE static EInternalObjectFlags GetReachableFlagValue_ForGC()
	{
		return ReachableObjectFlag;
	}

	FORCEINLINE static EInternalObjectFlags GetMaybeUnreachableFlagValue_ForGC()
	{
		return MaybeUnreachableObjectFlag;
	}

	FORCEINLINE static bool IsIncrementalGatherUnreachableSupported()
	{
		return false;
	}
};

} // namespace UE::GC::Private