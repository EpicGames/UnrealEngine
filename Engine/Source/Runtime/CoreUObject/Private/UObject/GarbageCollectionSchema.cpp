// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/GarbageCollectionSchema.h"

#include "Misc/AsciiSet.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/VVMVerseStruct.h"
#endif

namespace UE::GC {

//////////////////////////////////////////////////////////////////////////

static constexpr uint32 ToBit(EMemberType Type)		{ return 1u << static_cast<uint32>(Type); }
static constexpr uint32 InnerSchemaTypes =			ToBit(EMemberType::StructArray) | ToBit(EMemberType::StructSet) | ToBit(EMemberType::FreezableStructArray) | ToBit(EMemberType::Optional);
static constexpr uint32 AROTypes =					ToBit(EMemberType::ARO) | ToBit(EMemberType::SlowARO) | ToBit(EMemberType::MemberARO);
static constexpr uint32 AROPointerTypes =			ToBit(EMemberType::ARO) | ToBit(EMemberType::MemberARO);
static constexpr uint32 ExtraWordTypes =			InnerSchemaTypes | AROPointerTypes | ToBit(EMemberType::StridedArray);
static constexpr uint32 StopTypes =					ToBit(EMemberType::Stop) | ToBit(EMemberType::ARO) | ToBit(EMemberType::SlowARO);
static constexpr uint32 GeneratedTypes =			StopTypes | ToBit(EMemberType::Jump) | ToBit(EMemberType::StridedArray);
static constexpr uint32 MemberTypeCount =			uint8(EMemberType::Count);

template<uint32 Types>
static constexpr bool IsIn(EMemberType Type)
{
	return !!(Types & ToBit(Type));
}

static uint8 ToInt(EMemberType Type)
{
	check((uint8)Type < MemberTypeCount);
	return (uint8)Type;
}

FName ToName(EMemberType Type)
{
	struct FMemberTypeNames
	{
		FMemberTypeNames()
		{
			Names[(uint8)EMemberType::Stop] =							"Stop";
			Names[(uint8)EMemberType::Jump] =							"Jump";
			Names[(uint8)EMemberType::Reference] =						"Reference";
			Names[(uint8)EMemberType::ReferenceArray] =					"ReferenceArray";
			Names[(uint8)EMemberType::StructArray] =					"StructArray";
			Names[(uint8)EMemberType::StridedArray] =					"StridedArray";
			Names[(uint8)EMemberType::StructSet] =						"StructSet";
			Names[(uint8)EMemberType::FieldPath] =						"FieldPath";
			Names[(uint8)EMemberType::FieldPathArray] =					"FieldPathArray";
			Names[(uint8)EMemberType::FreezableReferenceArray] =		"FreezableReferenceArray";
			Names[(uint8)EMemberType::FreezableStructArray] =			"FreezableStructArray";
			Names[(uint8)EMemberType::Optional] =						"Optional";
			Names[(uint8)EMemberType::DynamicallyTypedValue] =			"DynamicallyTypedValue";
			Names[(uint8)EMemberType::ARO] =							"ARO";
			Names[(uint8)EMemberType::SlowARO] =						"SlowARO";
			Names[(uint8)EMemberType::MemberARO] =						"MemberARO";
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
			Names[(uint8)EMemberType::VerseValue] =						"VerseValue";
			Names[(uint8)EMemberType::VerseValueArray] =				"VerseValueArray";
#endif
		}

		FName Names[MemberTypeCount];
	};

	static FMemberTypeNames MemberTypes;
	check(!MemberTypes.Names[ToInt(Type)].IsNone());
	return MemberTypes.Names[ToInt(Type)];
}

//////////////////////////////////////////////////////////////////////////

static_assert(sizeof(FMemberWord) == 8 && alignof(FMemberWord) == 8); //-V547

static bool IsSet(FMemberWord Word)
{
	return !!reinterpret_cast<const uint64&>(Word);
}

static FMemberWord ToWord(ObjectAROFn In)
{
	FMemberWord Out;
	Out.ObjectARO = In;
	return Out;
}

//////////////////////////////////////////////////////////////////////////

FSchemaHeader& FSchemaView::GetHeader()
{
	check(GetWords());
	return *(reinterpret_cast<FSchemaHeader*>(Handle & ~OriginBit) - 1);
}

FSchemaHeader* FSchemaView::TryGetHeader()
{
	return IsEmpty() ? nullptr : &GetHeader();
}

//////////////////////////////////////////////////////////////////////////

template<typename Fn>
void VisitInnerSchemas(FSchemaView Schema, Fn Visitor)
{
	check(!Schema.IsEmpty());
	const FMemberWord* WordIt = Schema.GetWords();
	while (true)
	{
		FMemberWord Word = *WordIt++;
		for (Private::FMemberUnpacked Member : Word.Members)
		{
			if (IsIn<StopTypes>(Member.Type))
			{
				return;
			}
			else if (IsIn<ExtraWordTypes>(Member.Type))
			{
				FMemberWord ExtraWord = *WordIt++;
				if (IsIn<InnerSchemaTypes>(Member.Type))
				{
					Visitor(ExtraWord.InnerSchema);
				}
			}
		}
	}		
}

static void AddSchemaReference(FSchemaView Schema)
{
	Schema.GetHeader().RefCount.fetch_add(1);
}

static void DropSchemaReference(FSchemaView Schema)
{
	FSchemaHeader& Header = Schema.GetHeader();
	if (Header.RefCount.fetch_sub(1) == 1)
	{
		VisitInnerSchemas(Schema, &DropSchemaReference);
		FMemory::Free(&Header);
	}
}

static void TryAddSchemaReference(FSchemaView Schema)
{
	if (!Schema.IsEmpty())
	{
		AddSchemaReference(Schema);
	}
}

static void TryDropSchemaReference(FSchemaView Schema)
{
	if (!Schema.IsEmpty())
	{
		DropSchemaReference(Schema);
	}
}

static void TryAddSchemaReference(FMemberDeclaration& Member)
{
	if (IsIn<InnerSchemaTypes>(Member.Type))
	{
		AddSchemaReference(Member.ExtraWord.InnerSchema);
	}
}

static void TryDropSchemaReference(FMemberDeclaration& Member)
{
	if (IsIn<InnerSchemaTypes>(Member.Type))
	{
		DropSchemaReference(Member.ExtraWord.InnerSchema);
	}
}

static void ReplaceSchemaReference(FSchemaView& Out, FSchemaView New)
{
	if (Out.GetWords() != New.GetWords())
	{
		TryAddSchemaReference(New);
		TryDropSchemaReference(Out);
		Out = New;
	}
}

FSchemaOwner::FSchemaOwner(FSchemaView Schema)
: SchemaView(reinterpret_cast<uint64&>(Schema))
{
	static_assert(sizeof(FSchemaOwner) == sizeof(FSchemaView) && alignof(FSchemaOwner) == alignof(FSchemaView) && offsetof(FSchemaOwner, SchemaView) == 0);
	TryAddSchemaReference(Schema);
}

void FSchemaOwner::Set(FSchemaView In)
{
	ReplaceSchemaReference(reinterpret_cast<FSchemaView&>(SchemaView), In);
}

void FSchemaOwner::Reset()
{
	TryDropSchemaReference(Get());
	SchemaView = 0;
}

//////////////////////////////////////////////////////////////////////////

static FName ToName(EMemberlessId Id)
{
	static_assert(EMemberlessId::Collector == static_cast<EMemberlessId>(1u));
	static_assert(EMemberlessId::Max == EMemberlessId::InitialReference);

	static FName Names[] =
	{
		FName(EName::Error),
		FName("Collector"), // EMemberlessId::Collector
		FName("Class"), // EMemberlessId::Class
		FName("Outer"), // EMemberlessId::Outer
		FName("ExternalPackage"), // EMemberlessId::ExternalPackage
		FName("ClassOuter"), // EMemberlessId::ClassOuter
		FName("InitialReference"), // EMemberlessId::InitialReference
	};

	check(static_cast<uint32>(Id) < UE_ARRAY_COUNT(Names));
	return Names[static_cast<uint32>(Id)];
}

void FMemberId::StaticAssert()
{
	static_assert((1u << MemberlessIdBits) >= static_cast<uint32>(EMemberlessId::Max), "Need to bump MemberlessIdBits");
}

FMemberInfo GetMemberDebugInfo(FSchemaView Schema, FMemberId Id)
{
	if (Id.IsMemberless())
	{
		// Technically offsetof(UObjectBase, ClassPrivate/OuterPrivate) could be exposed via some new API or friend declaration
		return {-1, ToName(Id.AsMemberless()) };	
	}

	int32 Offset = -1;
	int32 DebugNameIdx = -1;
	const uint32 FindIdx = Id.GetIndex();
	uint32 MemberIdx = 0;
	uint32 JumpedWords = 0;
	uint32 NumJumps = 0;
	const FMemberWord* WordIt = Schema.GetWords();
	while (true)
	{
		for (Private::FMemberUnpacked Member : WordIt++->Members)
		{
			WordIt += IsIn<ExtraWordTypes>(Member.Type);

			if (MemberIdx++ == FindIdx)
			{
				if (IsIn<StopTypes | ToBit(EMemberType::Jump)>(Member.Type))
				{
					return { IntCastChecked<int32>(Member.WordOffset), ToName(Member.Type) };
				}

				Offset = IntCastChecked<int32>(8 * (JumpedWords + Member.WordOffset));

#if UE_GC_DEBUGNAMES // Must visit all remaining words to find start of debug names
				DebugNameIdx = FindIdx - NumJumps;
#else
				return { Offset, ToName(Member.Type) };
#endif
			}
			else if (IsIn<StopTypes>(Member.Type))
			{
				return { Offset, DebugNameIdx >= 0 ? ((FName*)WordIt)[DebugNameIdx] : EName::Error };
			}
			else if (Member.Type == EMemberType::Jump)
			{
				JumpedWords += (Member.WordOffset + 1) * FMemberPacked::OffsetRange;
				++NumJumps;
			}
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FSchemaBuilder::FSchemaBuilder(uint32 InStride, std::initializer_list<FMemberDeclaration> InMembers)
	: Members(InMembers)
	, StructStride(InStride) 
{
	checkf(StructStride % 8 == 0, TEXT("Structs containing TObjectPtr/UObject* references must be 8-byte aligned"));

	for (FMemberDeclaration& Member : Members)
	{
		TryAddSchemaReference(Member);
	}
}

FSchemaBuilder::~FSchemaBuilder()
{
	check(StructStride % 8 == 0); // Builder memory stomp
	check(!BuiltSchema || BuiltSchema->Get().IsEmpty() || FSchemaView(BuiltSchema->Get()).GetHeader().RefCount.load() > 0); // Schema ref-count bug or header memory stomp
	check(!BuiltSchema || BuiltSchema->Get().IsEmpty() || FSchemaView(BuiltSchema->Get()).GetHeader().StructStride == StructStride); // Header memory stomp

	for (FMemberDeclaration& Member : Members)
	{
		TryDropSchemaReference(Member);
	}
}

void FSchemaBuilder::Add(FMemberDeclaration Member)
{
	checkf(IsIn<ExtraWordTypes>(Member.Type) == IsSet(Member.ExtraWord), TEXT("FSchemaBuilder::Add while IsIn(Type) doesn't match IsSet(ExtraWord). DebugName:%s"), *Member.DebugName.ToString());
	checkf(IsIn<~GeneratedTypes>(Member.Type), TEXT("Member type %d is generated by Build() and shouldn't be declared"), int32{EnumToUnderlyingType(Member.Type)});
	check(!BuiltSchema.IsSet());
	
	Members.Add(Member);
	TryAddSchemaReference(Member);
}

static void CopyDebugNames(const FName* In, TArrayView<FMemberDeclaration> Out)
{
	for (FMemberDeclaration& Member : Out)
	{
#if UE_GC_DEBUGNAMES
		Member.DebugName = *In++;
#else
		Member.DebugName = ToName(Member.Type);
#endif
	}
}

void FSchemaBuilder::Append(FSchemaView SuperSchema, int32 OffsetForProperties)
{
	if (SuperSchema.IsEmpty())
	{
		return;
	}
	
	uint32 JumpedWords = 0;
	const int32 OldNum = Members.Num();
	const FMemberWord* WordIt = SuperSchema.GetWords();
	while (true)
	{
		FMemberWord Word = *WordIt++;
		for (Private::FMemberUnpacked Member : Word.Members)
		{
			FMemberWord ExtraWord = IsIn<ExtraWordTypes>(Member.Type) ? *WordIt++ : FMemberWord{};

			if (IsIn<StopTypes>(Member.Type))
			{
				CopyDebugNames((const FName*)WordIt, MakeArrayView(Members).RightChop(OldNum));
				return;
			}
			else if (Member.Type == EMemberType::Jump)
			{
				JumpedWords += (Member.WordOffset + 1) * FMemberPacked::OffsetRange;
			}
			else
			{
				Members.Add({ FName(), OffsetForProperties + (JumpedWords + Member.WordOffset) * 8, Member.Type, ExtraWord});
				TryAddSchemaReference(Members.Last());
			}
		}
	}
}

// Build()-helper that packs declarations into words, generates unnamed jumps and collects debug names
struct FSchemaPacker
{
	FSchemaPacker(TArrayView<FMemberDeclaration> Members)
	{
		WordIdx = Words.AddZeroed();
		for (const FMemberDeclaration& Member : Members)
		{
			Pack(Member);
		}
	}
	
	TArray<FMemberWord, TInlineAllocator<16>> Words;
	TArray<FName, TInlineAllocator<64>> DebugNames;

private:
	static constexpr uint32 OffsetRange = FMemberPacked::OffsetRange;

	int32 WordIdx = 0;
	int32 QuadIdx = 0;
	uint64 JumpWordPos = 0;

	void Pack(const FMemberDeclaration& Member)
	{
		uint16 RelativeWordOffset = IsIn<StopTypes>(Member.Type) ? IntCastChecked<uint16>(Member.Offset) : GenerateJumps(Member.Offset);

		if (TOptional<FStridedLayout> StridedLayout = TrySimplifyStructArray(Member))
		{
			AddNamedMember(EMemberType::StridedArray, RelativeWordOffset, Member.DebugName);
			Words.AddZeroed_GetRef().StridedLayout = StridedLayout.GetValue();
		}
		else
		{
			AddNamedMember(Member.Type, RelativeWordOffset, Member.DebugName);

			if (IsIn<ExtraWordTypes>(Member.Type))
			{
				Words.Add(Member.ExtraWord);
			}
		}
		
	}

	uint16 GenerateJumps(uint32 ByteOffset)
	{
		check(ByteOffset % 8 == 0);
		check(ByteOffset / 8 >= JumpWordPos);

		const uint32 WordOffsetFromJumpPos = ByteOffset / 8 - IntCastChecked<uint32>(JumpWordPos);

		// 1 jump unit is OffsetRange words, a max jump is OffsetRange * OffsetRange * 8B = 32MB with 11 offset bits
		const uint32 JumpUnits = WordOffsetFromJumpPos / OffsetRange;
		for (uint32 MaxJumps = JumpUnits / OffsetRange; MaxJumps; --MaxJumps)
		{
			GenerateJump(OffsetRange);
		}
		if (uint32 LastJumpUnits = JumpUnits % OffsetRange)
		{
			GenerateJump(LastJumpUnits);
		}
		
		check(ByteOffset / 8 >= JumpWordPos);
		check(ByteOffset / 8 <  JumpWordPos + OffsetRange);

		return IntCastChecked<uint16>(WordOffsetFromJumpPos % OffsetRange);
	}

	void GenerateJump(uint32 JumpUnit)
	{
		check(JumpUnit > 0 && JumpUnit < OffsetRange);
		// -1 since we always jump at least 1 JumpUnit
		AddMember(EMemberType::Jump, IntCastChecked<uint16>(JumpUnit - 1));
		JumpWordPos += JumpUnit * OffsetRange;
	}
	
	void AddNamedMember(EMemberType Type, uint16 Offset, FName DebugName)
	{
		AddMember(Type, Offset);
#if UE_GC_DEBUGNAMES
		DebugNames.Add(DebugName);
#endif
	}

	void AddMember(EMemberType Type, uint16 Offset)
	{
		static_assert(MemberTypeCount <= 1u << FMemberPacked::TypeBits);
		check(Offset < OffsetRange);

		if (QuadIdx == 4)
		{
			WordIdx = Words.AddZeroed();
			QuadIdx = 0;
		}
		
		Words[WordIdx].Members[QuadIdx] = { ToInt(Type), Offset };
		++QuadIdx;
	}

	static TOptional<FStridedLayout> TrySimplifyStructArray(const FMemberDeclaration& Member)
	{
		const FSchemaView& InnerSchema = Member.ExtraWord.InnerSchema;
		if (Member.Type == EMemberType::StructArray &&
			InnerSchema.GetWords()[0].Members[0].Type == ToInt(EMemberType::Reference) &&
			InnerSchema.GetWords()[0].Members[1].Type == ToInt(EMemberType::Stop) &&
			InnerSchema.GetStructStride() / 8u <= std::numeric_limits<decltype(FStridedLayout::WordStride)>::max())
		{
			return FStridedLayout{ InnerSchema.GetWords()[0].Members[0].WordOffset, decltype(FStridedLayout::WordStride)(InnerSchema.GetStructStride() / 8u) };
		}

		return NullOpt;
	}
};


int32 FindSlowImplementation(ObjectAROFn ARO);

static FMemberDeclaration GenerateTerminator(ObjectAROFn ARO)
{
	if (!ARO)
	{
		return { ToName(EMemberType::Stop), 0, EMemberType::Stop };
	}
	else if (int32 SlowAROIdx = FindSlowImplementation(ARO); SlowAROIdx != INDEX_NONE)
	{
		return { ToName(EMemberType::SlowARO), static_cast<uint32>(SlowAROIdx), EMemberType::SlowARO };
	}
	else
	{
		return { ToName(EMemberType::ARO), 0, EMemberType::ARO, ToWord(ARO) };
	}
}

FSchemaView FSchemaBuilder::Build(ObjectAROFn ARO)
{
	check(StructStride % 8 == 0); // Memory stomp or use-after-free
	if (BuiltSchema.IsSet())
	{
		return BuiltSchema.GetValue().Get();
	}
	else if (Members.IsEmpty() && ARO == nullptr)
	{
		return BuiltSchema.Emplace().Get();
	}

	// Sort and terminate
	Algo::SortBy(Members, &FMemberDeclaration::Offset);
	Members.Add(GenerateTerminator(ARO));

	// Pack into words and generate jump instructions
	FSchemaPacker Packed(Members);
	check(Packed.DebugNames.IsEmpty() == !UE_GC_DEBUGNAMES);

	// Allocate and initialize schema data
	const SIZE_T Bytes = Align(sizeof(FSchemaHeader), sizeof(FMemberWord)) + sizeof(FMemberWord) * Packed.Words.Num() + sizeof(FName) * Packed.DebugNames.Num();
	FSchemaHeader* Header = new (FMemory::Malloc(Bytes)) FSchemaHeader {StructStride};
	FMemberWord* FirstWord = (FMemberWord*)(Header + 1);
	FMemory::Memcpy(FirstWord, Packed.Words.GetData(), sizeof(FMemberWord) * Packed.Words.Num());
	FMemory::Memcpy(FirstWord + Packed.Words.Num(), Packed.DebugNames.GetData(), sizeof(FName) * Packed.DebugNames.Num());
	
	// Add references to nested schemas and construct schema owner
	VisitInnerSchemas(FSchemaView(FirstWord), [](FSchemaView Inner){ AddSchemaReference(Inner); });
	return BuiltSchema.Emplace(FSchemaView(FirstWord)).Get();
}

//////////////////////////////////////////////////////////////////////////

TMap<UClass*, FSchemaOwner> GIntrinsicClassSchemas;

void DeclareIntrinsicSchema(UClass* Class, FSchemaView Schema)
{
	FSchemaOwner& Owner = GIntrinsicClassSchemas.FindOrAdd(Class);
	checkf(Owner.Get().IsEmpty(), TEXT("Declared multiple schemas for same class"));
	Owner.Set(Schema);
}

FSchemaView GetIntrinsicSchema(UClass* Class)
{
	FSchemaOwner* Owner = GIntrinsicClassSchemas.Find(Class);
	return Owner ? Owner->Get() : FSchemaView();
}

//////////////////////////////////////////////////////////////////////////

FString FPropertyStack::GetPropertyPath() const
{
	TStringBuilder<128> Result;
	const FProperty* PreviousProperty = nullptr;
	const TCHAR DelimiterChar = TEXT('.');

	for (const FProperty* Property : Props)
	{
		if (PreviousProperty)
		{
			if (Property->GetOwner<FProperty>() == PreviousProperty && Property->GetFName() == PreviousProperty->GetFName())
			{
				// Skipping inner properties (inside of containers) if their name matches their owner name - TArrayName.TArrayName doesn't have much value
				// but we do want to keep TMapName.TMapName_Key
				continue;
			}
			Result += DelimiterChar;
		}
		CA_ASSUME(Property);
		Result += Property->GetName();
		PreviousProperty = Property;
	}
	return Result.ToString();
}

bool FPropertyStack::ConvertPathToProperties(UClass* ObjectClass, FName InPropertyPath, TArray<FProperty*>& OutProperties)
{
	FNameBuilder Buffer(InPropertyPath);
	return ConvertPathToProperties(ObjectClass, Buffer.ToView(), OutProperties);
}

bool FPropertyStack::ConvertPathToProperties(UClass* ObjectClass, FStringView PropertyNameOrPath, TArray<FProperty*>& OutProperties)
{
	constexpr FAsciiSet Delimiters(".");
	bool bFullPathConstructed = true;

	FStringView PropertyName = FAsciiSet::FindPrefixWithout(PropertyNameOrPath, Delimiters);
	if (PropertyName.Len() == PropertyNameOrPath.Len())
	{
		// 99% of the time we're be dealing with just a single property
		if (FProperty* FoundProperty = ObjectClass->FindPropertyByName(FName(PropertyNameOrPath)))
		{
			OutProperties.Add(FoundProperty);
		}
		else
		{
			bFullPathConstructed = false;
		}
	}
	else
	{
		// Try and find the first property as we can't start processing the rest of the path without it
		if (FProperty* FoundProperty = ObjectClass->FindPropertyByName(FName(PropertyName)))
		{
			FStringView Remaining = FAsciiSet::TrimPrefixWith(PropertyNameOrPath.RightChop(PropertyName.Len()), Delimiters);
			OutProperties.Add(FoundProperty);
			do
			{
				// Determine the next property name
				PropertyName = FAsciiSet::FindPrefixWithout(Remaining, Delimiters);
				Remaining = FAsciiSet::TrimPrefixWith(Remaining.RightChop(PropertyName.Len()), Delimiters);
				if (FStructProperty* StructProp = CastField<FStructProperty>(FoundProperty))
				{
					// If the previous property was a struct property, the next one belongs to the struct the previous property represented
					FoundProperty = StructProp->Struct->FindPropertyByName(FName(PropertyName));
				}
				else
				{
					// In all other case (though in reality it should only be a TMap) find the inner property
					FoundProperty = CastField<FProperty>(FoundProperty->GetInnerFieldByName(FName(PropertyName)));
				}

				if (FoundProperty)
				{
					OutProperties.Add(FoundProperty);
				}
				else
				{
					bFullPathConstructed = false;
				}
			} while (Remaining.Len() > 0 && bFullPathConstructed);
		}
		else
		{
			bFullPathConstructed = false;
		}
	}
	return bFullPathConstructed;
}

//////////////////////////////////////////////////////////////////////////
#if !UE_BUILD_SHIPPING

struct FSchemaStats;

struct FStructMemberStats
{
	EMemberType Type = EMemberType::Stop;
	uint32 WordOffset = 0;
	FSchemaView Schema;
	FSchemaStats* Stat = nullptr;

	template<typename Fn>
	void VisitInstances(const uint8* Instance, Fn Visitor) const
	{
		const uint8* Container = Instance + 8 * WordOffset;
		switch (Type)
		{ 
			case EMemberType::StructArray:			VisitArray(*(const FScriptArray*)Container, Visitor);
			break;
			case EMemberType::StructSet:			VisitSet(*(const FScriptSet*)Container, Visitor);
			break;
			case EMemberType::FreezableStructArray:	VisitArray(*(const FFreezableScriptArray*)Container, Visitor);
			break;
			case EMemberType::Optional:				VisitElements(Container, *(const bool*)(Container + Schema.GetStructStride()), Visitor);
			break;
			default:								check(false); 
			break;
		}
	}

private:
	template<typename Fn>
	void VisitElements(const uint8* It, int32 Num, Fn Visitor) const
	{
		const uint32 Stride = Schema.GetStructStride();
		for (const uint8* End = It + Num * Stride; It != End; It += Stride)
		{
			Visitor(It);
		}
	}
	
	template<class ArrayType, typename Fn>
	void VisitArray(const ArrayType& Array, Fn Visitor) const
	{
		VisitElements((const uint8*)Array.GetData(), Array.Num(), Visitor);
	}

	template<typename Fn>
	void VisitSet(const FScriptSet& Set, Fn Visitor) const
	{
		const uint8* Data = Private::GetSetData(Set);
		const uint32 Stride = Schema.GetStructStride();
		for (int32 Idx = 0, Num = Set.GetMaxIndex(); Idx < Num; ++Idx)
		{
			if (Set.IsValidIndex(Idx))
			{
				Visitor(Data + Idx * Stride);
			}
		}
	}
};

struct FSchemaStats
{
	uint32 Counters[MemberTypeCount] = {};
	uint32 NumMembers = 0;
	uint32 NumExtraWords = 0;
	TArray<FStructMemberStats> InnerStructs;

	FSchemaStats() = default;
	explicit FSchemaStats(FSchemaView Schema, TMap<const void*, FSchemaStats>& /* in-out */ StructStats)
	: InnerStructs(MakeInnerStats(Schema, StructStats))
	{
		const FMemberWord* WordIt = Schema.GetWords();
		while (true)
		{
			for (Private::FMemberUnpacked Member : WordIt++->Members)
			{
				++NumMembers;
				++Counters[ToInt(Member.Type)];
				NumExtraWords += IsIn<ExtraWordTypes>(Member.Type);
				WordIt += IsIn<ExtraWordTypes>(Member.Type);

				if (IsIn<StopTypes>(Member.Type))
				{
					return;
				}
			}
		}
	}
	
	void Aggregate(const FSchemaStats& In)
	{
		for (uint32 Idx = 0; Idx < MemberTypeCount; ++Idx)
		{
			Counters[Idx] += In.Counters[Idx];
		}
		NumMembers += In.NumMembers;
		NumExtraWords += In.NumExtraWords;
	}

	static TArray<FStructMemberStats> MakeInnerStats(FSchemaView Schema, TMap<const void*, FSchemaStats>& /* in-out */ StructStats)
	{
		TArray<FStructMemberStats> Out;

		check(!Schema.IsEmpty());
		uint32 JumpedWords = 0;
		const FMemberWord* WordIt = Schema.GetWords();
		while (true)
		{
			for (Private::FMemberUnpacked Member : WordIt++->Members)
			{
				if (IsIn<StopTypes>(Member.Type))
				{
					return Out;
				}
				else if (Member.Type == EMemberType::Jump)
				{
					JumpedWords += (Member.WordOffset + 1) * FMemberPacked::OffsetRange;
				}
				else if (IsIn<ExtraWordTypes>(Member.Type))
				{
					FMemberWord ExtraWord = *WordIt++;
					if (IsIn<InnerSchemaTypes>(Member.Type))
					{
						FSchemaView InnerSchema = ExtraWord.InnerSchema;
						Out.Add({Member.Type, JumpedWords + Member.WordOffset, InnerSchema});

						if (!StructStats.Contains(InnerSchema.GetWords()))
						{
							StructStats.Emplace(InnerSchema.GetWords(), FSchemaStats(InnerSchema, StructStats));
						}
					}
				}
			}
		}
	}
	
	void LinkInnerStats(TMap<const void*, FSchemaStats>& StructStats)
	{
		for (FStructMemberStats& InnerStruct : InnerStructs)
		{
			if (InnerStruct.Stat == nullptr)
			{
				InnerStruct.Stat = &StructStats.FindChecked(InnerStruct.Schema.GetWords());
				InnerStruct.Stat->LinkInnerStats(StructStats);
			}
		}
	}

	uint32 NumWords() const
	{
		return NumExtraWords + Align(NumMembers, 4) / 4;
	}
};

using FLog2Distribution = uint32[32];
enum class EDumpUnit {Bytes, Number};

static void DumpDistribution(FOutputDevice& Out, const TCHAR* Title, const FLog2Distribution& Bins, uint32 Total, EDumpUnit Unit)
{
	if (Bins[0] == Total)
	{
		return;
	}

	uint32 End = 32;
	for (; End > 0 && Bins[End - 1] == 0; --End);

	uint32 Begin = 0;
	for (; Begin < End && Bins[Begin] == 0; ++Begin);

	Out.Logf(TEXT(" _____ %-24s _____"), Title);

	for (uint32 Idx = Begin, Sum = 0; Idx < End; ++Idx)
	{
		uint32 Num = Bins[Idx];
		Sum += Num;
		uint32 Max = Idx == 0 ? 0 : 1u << (Idx - 1);
		float Percentage = (100.f * (float)Num) / (float)Total;
		float Cumulative = (100.f * (float)Sum) / (float)Total;
		if (Unit == EDumpUnit::Number)
		{
			Out.Logf(TEXT(" %4d %5.1f%% %5.1f%%  %d"), Max, Percentage, Cumulative, Num);
		}
		else
		{
			uint32 bK = Idx > 10;
			Out.Logf(TEXT(" %3d%c %5.1f%% %5.1f%%  %d"), Max >> bK*10, "BK"[bK], Percentage, Cumulative, Num);
		}
	}
}
static void DumpSchemaStats(FOutputDevice& Out, const FSchemaStats& Stats)
{
	static constexpr TCHAR FormatStr[] = TEXT(" %18s %5.1f%% %5.1f%%  %d");

	uint32 Sum = 0;
	for (uint32 Idx = 0; Idx < MemberTypeCount; ++Idx)
	{
		if (uint32 Num = Stats.Counters[Idx])
		{
			Sum += Num;
			float Percentage = (100.f * (float)Num) / (float)Stats.NumMembers;
			float Cumulative = (100.f * (float)Sum) / (float)Stats.NumMembers;
			Out.Logf(FormatStr, *ToName(EMemberType(Idx)).ToString(), Percentage, Cumulative, Num);
		}
	}
}

static uint32 GetBinIdx(uint32 N) { return FMath::CeilLogTwo(N) + !!N; }

struct FInstanceStats
{
	uint32 Num = 0;
	FSchemaStats Sum;
	FLog2Distribution SizeBins = {};
	FLog2Distribution NumMemberBins = {};
	FLog2Distribution MemberBins[MemberTypeCount] = {};

	void CountInstance(const uint8* Instance, uint32 InstanceSize, const FSchemaStats& TypeStat)
	{
		Sum.Aggregate(TypeStat);
		++SizeBins[GetBinIdx(InstanceSize)];
		++NumMemberBins[GetBinIdx(TypeStat.NumMembers)];
		++Num;

		for (uint32 Idx = 0; Idx < MemberTypeCount; ++Idx)
		{
			++MemberBins[Idx][GetBinIdx(TypeStat.Counters[Idx])];
		}
	}

	void CountInnerStructInstances(const uint8* Instance, TConstArrayView<FStructMemberStats> InnerStructs)
	{
		for (const FStructMemberStats& InnerStruct : InnerStructs)
		{
			check(InnerStruct.Stat);
			InnerStruct.VisitInstances(Instance, [this, &InnerStruct] (const uint8* InnerInstance) 
			{ 
				CountInstance(InnerInstance, InnerStruct.Schema.GetStructStride(), *InnerStruct.Stat);
				CountInnerStructInstances(InnerInstance, InnerStruct.Stat->InnerStructs);
			});				
		}
	}
};

void DumpSchemaStats(FOutputDevice& Out)
{
	struct FTotals
	{
		uint32 NumMembers = 0;
		TMap<const void*, FSchemaStats> TypeStats;
		FInstanceStats InstanceStats;
	};
	FTotals Classes;
	FTotals Structs;
	uint32 NumClasses = 0;

	auto GatherStats = [&Classes, &Structs, &NumClasses](FSchemaView Schema)
		{
			++NumClasses;
			FSchemaStats& ClassStat = Classes.TypeStats.FindOrAdd(Schema.GetWords());
			if (ClassStat.NumMembers == 0 && !Schema.IsEmpty())
			{
				ClassStat = FSchemaStats(Schema, Structs.TypeStats);
				Classes.NumMembers += ClassStat.NumMembers;
			}
		};

	for (TObjectIterator<UClass> It; It; ++It)
	{
		GatherStats(It->ReferenceSchema.Get());
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	for (TObjectIterator<UVerseStruct> It; It; ++It)
	{
		GatherStats(It->ReferenceSchema.Get());
	}
#endif

	// Link up stats pointers after all stats have been created
	for (TPair<const void*, FSchemaStats>& Pair : Classes.TypeStats)
	{
		Pair.Value.LinkInnerStats(Structs.TypeStats);
	}

	for (TPair<const void*, FSchemaStats>& Pair : Structs.TypeStats)
	{
		Structs.NumMembers += Pair.Value.NumMembers;
	}

	// Aggregate instance stats
	for (TObjectIterator<UObject> It; It; ++It)
	{
		const uint8* Object = reinterpret_cast<const uint8*>(*It);
		const FSchemaStats& ClassStat = Classes.TypeStats.FindChecked(It->GetClass()->ReferenceSchema.Get().GetWords());
		Classes.InstanceStats.CountInstance(Object, It->GetClass()->GetPropertiesSize(), ClassStat);
		Structs.InstanceStats.CountInnerStructInstances(Object, ClassStat.InnerStructs);
	}

	// Log stats
	Out.Logf(TEXT("----------------------- GC schema statistics -----------------------"));
	Out.Logf(TEXT(" %d schemas shared by %d classes contain %d members"), Classes.TypeStats.Num(), NumClasses, Classes.NumMembers);
	Out.Logf(TEXT(" %d inner struct schemas contain %d members"), Structs.TypeStats.Num(), Structs.NumMembers);
	Out.Logf(TEXT(" Sweeping %d objects and %d struct instances"),	Classes.InstanceStats.Num, Structs.InstanceStats.Num);
	Out.Logf(TEXT(" will process %d class and %d struct members"),	Classes.InstanceStats.Sum.NumMembers, Structs.InstanceStats.Sum.NumMembers);
	Out.Logf(TEXT("---------------------- Object distributions ------------------------"));
	DumpDistribution(Out, TEXT("Sizes"),							Classes.InstanceStats.SizeBins,			Classes.InstanceStats.Num, EDumpUnit::Bytes);
	DumpDistribution(Out, TEXT("Total members"),					Classes.InstanceStats.NumMemberBins,	Classes.InstanceStats.Num, EDumpUnit::Number);
	for (uint32 Idx = 0; Idx < MemberTypeCount; ++Idx)
	{
		DumpDistribution(Out, *ToName(EMemberType(Idx)).ToString(), Classes.InstanceStats.MemberBins[Idx],	Classes.InstanceStats.Num, EDumpUnit::Number);
	}
	Out.Logf(TEXT("------------------ Struct instance distributions -------------------"));
	DumpDistribution(Out, TEXT("Sizes"),							Structs.InstanceStats.SizeBins,			Structs.InstanceStats.Num, EDumpUnit::Bytes);
	DumpDistribution(Out, TEXT("Total members"),					Structs.InstanceStats.NumMemberBins,	Structs.InstanceStats.Num, EDumpUnit::Number);
	for (uint32 Idx = 0; Idx < MemberTypeCount; ++Idx)
	{
		DumpDistribution(Out, *ToName(EMemberType(Idx)).ToString(),	Structs.InstanceStats.MemberBins[Idx],	Structs.InstanceStats.Num, EDumpUnit::Number);
	}
	Out.Logf(TEXT("--------------------- Class members to process ---------------------"));
	DumpSchemaStats(Out, Classes.InstanceStats.Sum);
	Out.Logf(TEXT("--------------------- Struct members to process --------------------"));
	DumpSchemaStats(Out, Structs.InstanceStats.Sum);
	Out.Logf(TEXT("--------------------------------------------------------------------"));
}

static FAutoConsoleCommandWithOutputDevice GDumpSchemaStats(TEXT("gc.DumpSchemaStats"), TEXT("Print GC schema statistics"), FConsoleCommandWithOutputDeviceDelegate::CreateStatic(&DumpSchemaStats));

uint32 CountSchemas(uint32& OutNumWords)
{
	// Create all schema stats
	TMap<const void*, FSchemaStats> ClassStats;
	TMap<const void*, FSchemaStats> StructStats;

	auto CountSchema = [&ClassStats, &StructStats](FSchemaView Schema)
		{
			if (!Schema.IsEmpty())
			{
				FSchemaStats& Stat = ClassStats.FindOrAdd(Schema.GetWords());
				if (Stat.NumMembers == 0)
				{
					Stat = FSchemaStats(Schema, StructStats);
				}
			}
		};

	for (TObjectIterator<UClass> It; It; ++It)
	{
		CountSchema(It->ReferenceSchema.Get());
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	for (TObjectIterator<UVerseStruct> It; It; ++It)
	{
		CountSchema(It->ReferenceSchema.Get());
	}
#endif

	// Calculate total words
	OutNumWords = 0;
	for (const TMap<const void*, FSchemaStats>& Stats : {ClassStats, StructStats})
	{
		for (const TPair<const void*, FSchemaStats>& Pair : Stats)
		{
			OutNumWords += Pair.Value.NumWords();
		}
	}
	
	return ClassStats.Num() + StructStats.Num();
}

#endif // !UE_BUILD_SHIPPING
//////////////////////////////////////////////////////////////////////////
} // namespace UE::GC
