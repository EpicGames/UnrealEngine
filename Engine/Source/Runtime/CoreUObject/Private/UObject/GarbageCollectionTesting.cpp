// Copyright Epic Games, Inc. All Rights Reserved.

#include "GarbageCollectionTesting.h"
#include "UObject/GarbageCollectionSchema.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GarbageCollectionTesting)

/*
	TODO additional features for a future changeset:
	- gc.GenerateReachabilityStressData should have extra arg
	  to specify # of levels in UObject tree
	- gc.GenerateReachabilityStressData should have extra arg
	  to specify whether or not the tree is gc-rooted (to see
		how long it takes to collect)
	- UObjectReachabilityStressData can expose more members to
	  GC to make the token stream bigger
 */

static void GenerateReachabilityStressData(int Levels, UObjectReachabilityStressData* Data);

static int32 CanAllocateMoreUObjects()
{
	const int32 MinAvailableObjectCount = 64 * 1024;
	return GUObjectArray.GetObjectArrayEstimatedAvailable() >= MinAvailableObjectCount;
}

static UObjectReachabilityStressData* ConditionallyAllocateNewStressDataObject()
{
	if (CanAllocateMoreUObjects())
	{
		return NewObject<UObjectReachabilityStressData>();
	}
	return nullptr;
}

void GenerateReachabilityStressData(TArray<UObjectReachabilityStressData*>& Data)
{
	// Roughly NumRootObjects * 2^SubLevels of objects
	const int32 NumRootObjects = 50;
	const int32 SubLevels = 13;

	for (int32 Index = 0; Index < NumRootObjects; ++Index)
	{
		UObjectReachabilityStressData* RootData = ConditionallyAllocateNewStressDataObject();
		if (RootData)
		{
			Data.Add(RootData);
			RootData->AddToRoot();
			GenerateReachabilityStressData(SubLevels, RootData);
		}
		else
		{
			break;
		}
	}
}

static void GenerateReachabilityStressData(int Levels, UObjectReachabilityStressData* Data)
{
	if (Levels == 0)
	{
		return;
	}

	const int N = 2;
	for (int I = 0; I < N; ++I)
	{
		UObjectReachabilityStressData* Child = ConditionallyAllocateNewStressDataObject();
		if (Child)
		{
			GenerateReachabilityStressData(Levels - 1, Child);
			Data->Children.Add(Child);
		}
		else
		{
			break;
		}
	}
}

void UnlinkReachabilityStressData(TArray<UObjectReachabilityStressData*>& Data)
{
	for (UObjectReachabilityStressData* RootData : Data)
	{
		RootData->RemoveFromRoot();
	}
}
