// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "GarbageCollectionTesting.generated.h"

UCLASS(Transient)
class UObjectReachabilityStressData : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	TArray<TObjectPtr<UObjectReachabilityStressData>> Children;
};

void GenerateReachabilityStressData(TArray<UObjectReachabilityStressData*>& Data);
void UnlinkReachabilityStressData(TArray<UObjectReachabilityStressData*>& Data);
