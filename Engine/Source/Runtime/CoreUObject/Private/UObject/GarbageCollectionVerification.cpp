// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnObjGC.cpp: Unreal object garbage collection code.
=============================================================================*/

#include "UObject/GarbageCollectionVerification.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GarbageCollectionInternalFlags.h"
#include "HAL/ThreadSafeBool.h"
#include "Misc/TimeGuard.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectBase.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"
#include "UObject/GCObject.h"
#include "UObject/GCScopeLock.h"
#include "HAL/ExceptionHandling.h"
#include "UObject/UObjectClusters.h"
#include "Async/ParallelFor.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/FastReferenceCollector.h"
#include <atomic>

/*-----------------------------------------------------------------------------
   Garbage collection verification code.
-----------------------------------------------------------------------------*/

/**
* If set and VERIFY_DISREGARD_GC_ASSUMPTIONS is true, we verify GC assumptions about "Disregard For GC" objects and clusters.
*/
COREUOBJECT_API bool	GShouldVerifyGCAssumptions = !UE_BUILD_SHIPPING && !UE_BUILD_TEST && !WITH_EDITOR;
static FAutoConsoleVariableRef CVarShouldVerifyGCAssumptions(
	TEXT("gc.VerifyAssumptions"),
	GShouldVerifyGCAssumptions,
	TEXT("Whether to verify GC assumptions (disregard for GC, clustering) on each GC."),
	ECVF_Default
);

/** If set and VERIFY_DISREGARD_GC_ASSUMPTIONS is set, we verify GC assumptions when performing a full (blocking) purge */
COREUOBJECT_API bool	GShouldVerifyGCAssumptionsOnFullPurge = !UE_BUILD_SHIPPING && !WITH_EDITOR;
static FAutoConsoleVariableRef CVarShouldVerifyGCAssumptionsOnFullPurge(
	TEXT("gc.VerifyAssumptionsOnFullPurge"),
	GShouldVerifyGCAssumptionsOnFullPurge,
	TEXT("Whether to verify GC assumptions (disregard for GC, clustering) on full purge GCs."),
	ECVF_Default
);

/** If > 0 and VERIFY_DISREGARD_GC_ASSUMPTIONS is set, we verify GC assumptions on that fraction of GCs. */
COREUOBJECT_API float	GVerifyGCAssumptionsChance = 0.0f;
static FAutoConsoleVariableRef CVarVerifyGCAssumptionsChance (
	TEXT("gc.VerifyAssumptionsChance"),
	GVerifyGCAssumptionsChance,
	TEXT("Chance (0-1) to randomly verify GC assumptions on each GC."),
	ECVF_Default
);

#if VERIFY_DISREGARD_GC_ASSUMPTIONS

/**
 * Finds only direct references of objects passed to the TFastReferenceCollector and verifies if they meet Disregard for GC assumptions
 */
class FDisregardSetReferenceProcessor : public FSimpleReferenceProcessorBase
{
	FThreadSafeCounter NumErrors;

public:
	FDisregardSetReferenceProcessor()
		: NumErrors(0)
	{
	}
	int32 GetErrorCount() const
	{
		return NumErrors.GetValue();
	}
	FORCEINLINE_DEBUGGABLE void HandleTokenStreamObjectReference(FWorkerContext& ObjectsToSerializeStruct, UObject* ReferencingObject, UObject*& Object, FMemberId MemberId, EOrigin Origin, bool bAllowReferenceElimination)
	{
		if (Object)
		{
#if ENABLE_GC_OBJECT_CHECKS
			if (
#if DO_POINTER_CHECKS_ON_GC
				!IsPossiblyAllocatedUObjectPointer(Object) ||
#endif
				!Object->IsValidLowLevelFast())
			{
				FString DebugInfo;
				if (UClass *Class = (ReferencingObject ? ReferencingObject->GetClass() : nullptr))
				{
					DebugInfo = FString::Printf(TEXT("ReferencingObjectClass: %s, Property: %s"),
						*Class->GetFullName(), *ObjectsToSerializeStruct.SchemaStack->ToString());
				}
				else
				{
					// This means this objects is most likely being referenced by AddReferencedObjects
					DebugInfo = TEXT("Native Reference");
				}

				UE_LOGF(LogGarbage, Fatal, "Invalid object while verifying Disregard for GC assumptions: 0x%016llx, ReferencingObject: %ls, %ls, MemberId: %d",
					(int64)(PTRINT)Object,
					ReferencingObject ? *ReferencingObject->GetFullName() : TEXT("NULL"),
					*DebugInfo, MemberId.AsPrintableIndex());
			}
#endif // ENABLE_GC_OBJECT_CHECKS

			if (!(Object->IsRooted() ||
					  GUObjectArray.IsDisregardForGC(Object) ||
					  GUObjectArray.ObjectToObjectItem(Object)->GetOwnerIndex() > 0 ||
					  GUObjectArray.ObjectToObjectItem(Object)->HasAnyFlags(EInternalObjectFlags::ClusterRoot)))
			{
				UE_LOGF(LogGarbage, Warning, "Disregard for GC object %ls referencing %ls which is not part of root set. Property: %ls",
					*GetFullNameSafe(ReferencingObject),
					*Object->GetFullName(),
					ObjectsToSerializeStruct.SchemaStack ? *ObjectsToSerializeStruct.SchemaStack->ToString() : TEXT("unknown"));
				NumErrors.Increment();
			}
		}
	}
};

void VerifyGCAssumptions()
{	
	int32 MaxNumberOfObjects = GUObjectArray.GetObjectArrayNumPermanent();

	FDisregardSetReferenceProcessor Processor;

	int32 NumThreads = GetNumCollectReferenceWorkers();
	int32 NumberOfObjectsPerThread = (MaxNumberOfObjects / NumThreads) + 1;
	
	ParallelFor( TEXT("GC.VerifyAssumptions"),NumThreads,1, [&Processor, NumberOfObjectsPerThread, NumThreads, MaxNumberOfObjects](int32 ThreadIndex)
	{
		int32 FirstObjectIndex = ThreadIndex * NumberOfObjectsPerThread;
		int32 NumObjects = (ThreadIndex < (NumThreads - 1)) ? NumberOfObjectsPerThread : (MaxNumberOfObjects - (NumThreads - 1)*NumberOfObjectsPerThread);
		
		TArray<UObject*> ObjectsToSerialize;
		ObjectsToSerialize.Reserve(NumberOfObjectsPerThread + UE::GC::ObjectLookahead);

		for (int32 ObjectIndex = 0; ObjectIndex < NumObjects && (FirstObjectIndex + ObjectIndex) < MaxNumberOfObjects; ++ObjectIndex)
		{
			FUObjectItem& ObjectItem = GUObjectArray.GetObjectItemArrayUnsafe()[FirstObjectIndex + ObjectIndex];
			if (ObjectItem.GetObject() && ObjectItem.GetObject() != FGCObject::GGCObjectReferencer)
			{
				ObjectsToSerialize.Add(static_cast<UObject*>(ObjectItem.GetObject()));
			}
		}
		
		UE::GC::FWorkerContext Context;
		Context.SetInitialObjectsUnpadded(ObjectsToSerialize);
		CollectReferences(Processor, Context);
	});

	UE_CLOGF(Processor.GetErrorCount() > 0, LogGarbage, Fatal, "Encountered %d object(s) breaking Disregard for GC assumptions. Please check log for details.", Processor.GetErrorCount());
}

/**
* Finds only direct references of objects passed to the TFastReferenceCollector and verifies if they meet GC Cluster assumptions
*/
class FClusterVerifyReferenceProcessor : public FSimpleReferenceProcessorBase
{
	FThreadSafeCounter NumErrors;
	UObject* CurrentObject;
	FUObjectCluster* Cluster;
	UObject* ClusterRootObject;

public:
	FClusterVerifyReferenceProcessor()
		: NumErrors(0)
		, CurrentObject(nullptr)
		, Cluster(nullptr)
		, ClusterRootObject(nullptr)
	{
	}
	int32 GetErrorCount() const
	{
		return NumErrors.GetValue();
	}
	void SetCurrentObjectAndCluster(UObject* InRootOrClusterObject)
	{
		check(InRootOrClusterObject);
		CurrentObject = InRootOrClusterObject;
		Cluster = GUObjectClusters.GetObjectCluster(CurrentObject);
		check(Cluster);
		FUObjectItem* RootItem = GUObjectArray.IndexToObject(Cluster->RootIndex);
		check(RootItem && RootItem->GetObject());
		ClusterRootObject = static_cast<UObject*>(RootItem->GetObject());
	}
	/**
	* Handles UObject reference from the token stream. Performance is critical here so we're FORCEINLINING this function.
	*
	* @param bAllowReferenceElimination True if reference elimination is allowed (ignored when constructing clusters).
	*/
	FORCEINLINE_DEBUGGABLE void HandleTokenStreamObjectReference(FGCArrayStruct& ObjectsToSerializeStruct, UObject* ReferencingObject, UObject*& Object, FMemberId MemberId, EOrigin Origin, bool bAllowReferenceElimination)
	{
		if (Object)
		{
			if (ObjectsToSerializeStruct.GetReferencingObject() != CurrentObject)
			{
				SetCurrentObjectAndCluster(ObjectsToSerializeStruct.GetReferencingObject());
			}
			check(CurrentObject);

#if ENABLE_GC_OBJECT_CHECKS
			if (
#if DO_POINTER_CHECKS_ON_GC
				!IsPossiblyAllocatedUObjectPointer(Object) ||
#endif
				!Object->IsValidLowLevelFast())
			{
				FString DebugInfo;
				if (UClass *Class = (ReferencingObject ? ReferencingObject->GetClass() : nullptr))
				{
					DebugInfo = FString::Printf(TEXT("ReferencingObjectClass: %s, Property: %s"),
						*Class->GetFullName(), *ObjectsToSerializeStruct.SchemaStack->ToString());
				}
				else
				{
					// This means this objects is most likely being referenced by AddReferencedObjects
					DebugInfo = TEXT("Native Reference");
				}

#if UE_GCCLUSTER_VERBOSE_LOGGING
				DumpClusterToLog(*Cluster, true, true);
#endif

				UE_LOGF(LogGarbage, Fatal, "Invalid object while verifying cluster assumptions: 0x%016llx, ReferencingObject: %ls, %ls, MemberId: %d",
					(int64)(PTRINT)Object,
					ReferencingObject ? *ReferencingObject->GetFullName() : TEXT("NULL"),
					*DebugInfo, MemberId.AsPrintableIndex());
			}
#endif // ENABLE_GC_OBJECT_CHECKS

			FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
			if (ObjectItem->GetOwnerIndex() <= 0)
			{
				// Referenced object is a cluster root or not clustered
				if (ObjectItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot))
				{
					// Clusters need to be referenced by the current cluster otherwise they can also get GC'd too early.
					const FUObjectItem* ClusterRootObjectItem = GUObjectArray.ObjectToObjectItem(ClusterRootObject);
					const int32 OtherClusterRootIndex = GUObjectArray.ObjectToIndex(Object);
					const FUObjectItem* OtherClusterRootItem = GUObjectArray.IndexToObjectUnsafeForGC(OtherClusterRootIndex);
					check(OtherClusterRootItem && OtherClusterRootItem->GetObject());
					UObject* OtherClusterRootObject = static_cast<UObject*>(OtherClusterRootItem->GetObject());
					UE_CLOGF(
						OtherClusterRootIndex != Cluster->RootIndex  // Same cluster is legal 
					&&	!Cluster->ReferencedClusters.Contains(OtherClusterRootIndex)  // cluster-cluster reference is legal
					&&  !Cluster->MutableObjects.Contains(OtherClusterRootIndex),  // reference to an external object which later became a cluster root is legal 
						LogGarbage, Warning,
						"Object %ls from source cluster %ls (%d) is referencing cluster root object %ls (0x%016llx) (%d) which is not referenced by the source cluster.",
						*GetFullNameSafe(ReferencingObject),
						*ClusterRootObject->GetFullName(),
						ClusterRootObjectItem->GetClusterIndex(),
						*Object->GetFullName(),
						(int64)(PTRINT)Object,
						OtherClusterRootItem->GetClusterIndex());
				}
				else if (	!ObjectItem->HasAnyFlags(EInternalObjectFlags::RootSet) // Root set objects will stay alive that way 
						&&	!GUObjectArray.IsDisregardForGC(Object) // Disregard-for-GC objects are never freed
						&&  !Cluster->MutableObjects.Contains(GUObjectArray.ObjectToIndex(Object)) // Mutable object ref is traversed during GC regardless of if the object is cluster root or not 
				) 
				{
					// There is a danger this object could be freed leaving a dangling pointer in an object inside the cluster
					UE_LOGF(LogGarbage, Warning, "Object %ls (0x%016llx) from cluster %ls (0x%016llx / 0x%016llx) is referencing 0x%016llx %ls which is not part of root set or cluster.",
						*CurrentObject->GetFullName(),
						(int64)(PTRINT)CurrentObject,
						*ClusterRootObject->GetFullName(),
						(int64)(PTRINT)ClusterRootObject,
						(int64)(PTRINT)Cluster,
						(int64)(PTRINT)Object,
						*Object->GetFullName());
					NumErrors.Increment();
#if UE_BUILD_DEBUG
					FReferenceChainSearch RefChainSearch(Object, EReferenceChainSearchMode::Shortest | EReferenceChainSearchMode::PrintResults);
#endif
				}
			}
			else if (ObjectItem->GetOwnerIndex() != Cluster->RootIndex)
			{
				// If we're referencing an object from another cluster, make sure the other cluster is actually referenced by this cluster
				const FUObjectItem* ClusterRootObjectItem = GUObjectArray.ObjectToObjectItem(ClusterRootObject);
				const int32 OtherClusterRootIndex = ObjectItem->GetOwnerIndex();
				check(OtherClusterRootIndex > 0);
				const FUObjectItem* OtherClusterRootItem = GUObjectArray.IndexToObjectUnsafeForGC(OtherClusterRootIndex);
				check(OtherClusterRootItem && OtherClusterRootItem->GetObject());
				UObject* OtherClusterRootObject = static_cast<UObject*>(OtherClusterRootItem->GetObject());
				UE_CLOGF(
						OtherClusterRootIndex != Cluster->RootIndex  // Same cluster is legal 
					&&	!Cluster->ReferencedClusters.Contains(OtherClusterRootIndex)  // Cluster-cluster reference 
					&&	!Cluster->MutableObjects.Contains(GUObjectArray.ObjectToIndex(Object)), // Reference to an object which was later clustered
					LogGarbage, Warning,
					"Object %ls from source cluster %ls (%d) is referencing object %ls (0x%016llx) from cluster %ls (%d) which is not referenced by the source cluster.",
					*GetFullNameSafe(ReferencingObject),
					*ClusterRootObject->GetFullName(),
					ClusterRootObjectItem->GetClusterIndex(),
					*Object->GetFullName(),
					(int64)(PTRINT)Object,
					*OtherClusterRootObject->GetFullName(),
					OtherClusterRootItem->GetClusterIndex());
			}
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/**
	* Handles VCell reference from the token stream. Performance is critical here so we're FORCEINLINING this function.
	*
	* @param Context Context of the reference collection
	* @param ReferencingObject Object referencing the object to process.
	* @param Cell Cell being processed
	* @param MemberId Index to the token stream where the reference was found.
	* @param Origin Declares if a schema represents a blueprint generated type
	*/
	FORCEINLINE void HandleTokenStreamVerseCellReference(FWorkerContext& Context, UObject* ReferencingObject, Verse::VCell* Cell, FMemberId MemberId, EOrigin Origin)
	{
		if (Cell)
		{
			if (Context.GetReferencingObject() != CurrentObject)
			{
				SetCurrentObjectAndCluster(Context.GetReferencingObject());
			}
			check(CurrentObject);

			const FUObjectItem* ClusterRootObjectItem = GUObjectArray.ObjectToObjectItem(ClusterRootObject);
			UE_CLOGF(
				!Cluster->MutableCells.Contains(Cell),
				LogGarbage, Warning,
				"Object %ls from source cluster %ls (%d) is referencing cell (0x%016llx) which is not part of cluster.",
				*GetFullNameSafe(ReferencingObject),
				*ClusterRootObject->GetFullName(),
				ClusterRootObjectItem->GetClusterIndex(),
				(int64)(PTRINT)Cell);
		}
	}
#endif
};

void VerifyClustersAssumptions()
{
	int32 MaxNumberOfClusters = GUObjectClusters.GetClustersUnsafe().Num();
	int32 NumThreads = GetNumCollectReferenceWorkers();
	int32 NumberOfClustersPerThread = (MaxNumberOfClusters / NumThreads) + 1;
	
	FThreadSafeCounter NumErrors(0);

	ParallelFor( TEXT("GC.VerifyClusterAssumptions"),NumThreads,1, [&NumErrors, NumberOfClustersPerThread, NumThreads, MaxNumberOfClusters](int32 ThreadIndex)
	{
		int32 FirstClusterIndex = ThreadIndex * NumberOfClustersPerThread;
		int32 NumClusters = (ThreadIndex < (NumThreads - 1)) ? NumberOfClustersPerThread : (MaxNumberOfClusters - (NumThreads - 1) * NumberOfClustersPerThread);
				
		FClusterVerifyReferenceProcessor Processor;

		TArray<UObject*> ObjectsToSerialize;
		for (int32 ClusterIndex = 0; ClusterIndex < NumClusters && (FirstClusterIndex + ClusterIndex) < MaxNumberOfClusters; ++ClusterIndex)
		{
			FUObjectCluster& Cluster = GUObjectClusters.GetClustersUnsafe()[FirstClusterIndex + ClusterIndex];
			if (Cluster.RootIndex >= 0 && Cluster.Objects.Num())
			{
				ObjectsToSerialize.Reset(Cluster.Objects.Num() + 1 + UE::GC::ObjectLookahead);
				{
					FUObjectItem* RootItem = GUObjectArray.IndexToObject(Cluster.RootIndex);
					check(RootItem);
					check(RootItem->GetObject());
					ObjectsToSerialize.Add(static_cast<UObject*>(RootItem->GetObject()));
				}
				for (int32 ObjectIndex : Cluster.Objects)
				{
					FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(ObjectIndex);
					check(ObjectItem);
					check(ObjectItem->GetObject());
					ObjectsToSerialize.Add(static_cast<UObject*>(ObjectItem->GetObject()));
				}

				UE::GC::FWorkerContext Context;
				Context.SetInitialObjectsUnpadded(ObjectsToSerialize);
				CollectReferences(Processor, Context);
			}			
		}		
		NumErrors.Add(Processor.GetErrorCount());
	});


	UE_CLOGF(NumErrors.GetValue() > 0, LogGarbage, Fatal, "Encountered %d object(s) breaking GC Clusters assumptions. Please check log for details.", NumErrors.GetValue());
}

void VerifyObjectFlags()
{
	using namespace UE::GC::Private;

	int32 MaxNumberOfObjects = GUObjectArray.GetObjectArrayNum();
	int32 NumThreads = FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads());
	int32 NumberOfObjectsPerThread = (MaxNumberOfObjects / NumThreads) + 1;
	std::atomic<uint32> NumErrors(0);

	ParallelFor( TEXT("GC.VerifyObjectFlags"),NumThreads,1, [&NumErrors, NumberOfObjectsPerThread, NumThreads, MaxNumberOfObjects](int32 ThreadIndex)
	{
		int32 FirstObjectIndex = ThreadIndex * NumberOfObjectsPerThread;
		int32 NumObjects = (ThreadIndex < (NumThreads - 1)) ? NumberOfObjectsPerThread : (MaxNumberOfObjects - (NumThreads - 1) * NumberOfObjectsPerThread);

		for (int32 ObjectIndex = 0; ObjectIndex < NumObjects && (FirstObjectIndex + ObjectIndex) < MaxNumberOfObjects; ++ObjectIndex)
		{
			FUObjectItem& ObjectItem = GUObjectArray.GetObjectItemArrayUnsafe()[FirstObjectIndex + ObjectIndex];
			if (ObjectItem.GetObject())
			{
				UObject* Object = (UObject*)ObjectItem.GetObject();
				bool bHasObjectFlag = Object->HasAnyFlags(RF_MirroredGarbage);
				bool bHasInternalFlag = ObjectItem.HasAnyFlags(EInternalObjectFlags::Garbage);
				if (bHasObjectFlag != bHasInternalFlag)
				{
					UE_LOGF(LogGarbage, Warning, "RF_Garbage (%d) and EInternalObjectFlags::Garbage (%d) flag mismatch on %ls%ls",
						(int32)bHasObjectFlag,
						(int32)bHasInternalFlag,
						*FReferenceChainSearch::GetObjectFlags(FGCObjectInfo(Object)),
						*Object->GetFullName());

					++NumErrors;
				}

				if (!FGCFlags::IsReachable_ForGC(&ObjectItem) && !GUObjectArray.IsDisregardForGC(Object))
				{
					UE_LOGF(LogGarbage, Warning, "Object %ls%ls is NOT marked as Reachable at the beginning of GC",
						*FReferenceChainSearch::GetObjectFlags(FGCObjectInfo(Object)),
						*Object->GetFullName());

					++NumErrors;
				}

				if (FGCFlags::IsMaybeUnreachable_ForGC(&ObjectItem) || ObjectItem.IsUnreachable())
				{
					UE_LOGF(LogGarbage, Warning, "Object %ls%ls is marked with at least one of the unreachable flags at the beginning of GC",
						*FReferenceChainSearch::GetObjectFlags(FGCObjectInfo(Object)),
						*Object->GetFullName());

					++NumErrors;
				}
			}
		}
	});

	UE_CLOGF(NumErrors > 0, LogGarbage, Fatal, "Encountered %d object(s) breaking Object and Internal flag assumptions. Please check log for details.", (uint32)NumErrors);
}

/**
* Finds only direct references of objects passed to the TFastReferenceCollector and verifies if they ae reachable or not
*/
class FUnreachableReferenceProcessor : public FSimpleReferenceProcessorBase
{
	int32 NumErrors = 0;

public:
	FUnreachableReferenceProcessor() = default;

	int32 GetErrorCount() const
	{
		return NumErrors;
	}

	FORCEINLINE_DEBUGGABLE void HandleTokenStreamObjectReference(FGCArrayStruct& ObjectsToSerializeStruct, UObject* ReferencingObject, UObject*& Object, FMemberId MemberId, EOrigin Origin, bool bAllowReferenceElimination)
	{
		using namespace UE::GC::Private;

		if (Object)
		{
			FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
			if (ObjectItem->HasAnyFlags(FGCFlags::GetMaybeUnreachableFlagValue_ForGC() | EInternalObjectFlags::Unreachable))
			{
				if (ReferencingObject)
				{
					FUObjectItem* ReferencingObjectItem = GUObjectArray.ObjectToObjectItem(ReferencingObject);
					if (ReferencingObjectItem->GetOwnerIndex() > 0 && ReferencingObjectItem->GetOwnerIndex() == GUObjectArray.ObjectToIndex(Object))
					{
						// ReferencingObject is in Object's cluster so it's fine it it's still reachable
						return;
					}
				}

				FString DebugInfo;
				if (UClass* Class = (ReferencingObject ? ReferencingObject->GetClass() : nullptr))
				{
					DebugInfo = FString::Printf(TEXT("property: %s"),
						*ObjectsToSerializeStruct.SchemaStack->ToString());
				}
				else
				{
					// This means this objects is most likely being referenced by AddReferencedObjects
					DebugInfo = TEXT("Native Reference");
				}

				FString ReferencingObjectName;
				if (!ReferencingObject && FGCObject::GGCObjectReferencer)
				{
					if (FGCObject* CurrentlySerializingObject = FGCObject::GGCObjectReferencer->GetCurrentlySerializingObject())
					{
						ReferencingObjectName = CurrentlySerializingObject->GetReferencerName();
					}
				}
				if (ReferencingObjectName.IsEmpty())
				{
					ReferencingObjectName = GetFullNameSafe(ReferencingObject);
				}

				UE_LOGF(LogGarbage, Warning, "Object %ls%ls is being referenced by reachable object %ls through %ls",
					*FReferenceChainSearch::GetObjectFlags(FGCObjectInfo(Object)),
					*Object->GetFullName(),
					*ReferencingObjectName,
					*DebugInfo);

				NumErrors++;
			}
		}
	}
};

void VerifyNoUnreachableObjects(int32 NumUnreachable)
{
	using namespace UE::GC::Private;

	const double StartTime = FPlatformTime::Seconds();
	const int32 MaxNumberOfReachableObjects = GUObjectArray.GetObjectArrayNum();
	const int32 NumThreads = GetNumCollectReferenceWorkers();
	const int32 NumberOfObjectsPerThread = (MaxNumberOfReachableObjects / NumThreads) + 1;
	std::atomic<uint32> NumErrors(0);
	std::atomic<int32> VerifiedNumUnreachable(0);

	ParallelFor(TEXT("GC.VerifyNoUnreachableObjects"), NumThreads, 1, [&NumErrors, &VerifiedNumUnreachable, NumberOfObjectsPerThread, NumThreads, MaxNumberOfReachableObjects, NumUnreachable](int32 ThreadIndex)
	{
		int32 FirstObjectIndex = ThreadIndex * NumberOfObjectsPerThread;
		int32 NumObjects = (ThreadIndex < (NumThreads - 1)) ? NumberOfObjectsPerThread : (MaxNumberOfReachableObjects - (NumThreads - 1)*NumberOfObjectsPerThread);
		TArray<UObject*> ObjectsToSerialize;
		ObjectsToSerialize.Reserve(NumberOfObjectsPerThread);		

		int32 ThisThreadUnrachableObjectsNum = 0;

		for (int32 ObjectIndex = 0; ObjectIndex < NumObjects && (FirstObjectIndex + ObjectIndex) < GUObjectArray.GetObjectArrayNum(); ++ObjectIndex)
		{
			const int32 ObjectIndexInGlobalArray = FirstObjectIndex + ObjectIndex;
			FUObjectItem& ObjectItem = GUObjectArray.GetObjectItemArrayUnsafe()[ObjectIndexInGlobalArray];
			if (ObjectItem.GetObject())
			{
				UObject* Object = static_cast<UObject*>(ObjectItem.GetObject());
				if (!ObjectItem.HasAnyFlags(EInternalObjectFlags::Unreachable))
				{
					if (FGCFlags::IsMaybeUnreachable_ForGC(&ObjectItem))
					{
						UE_LOGF(LogGarbage, Warning, "Object [%d] %ls%ls is still marked as MaybeUnreachable after Reachability Analysis is complete.",
							ObjectIndexInGlobalArray,
							*FReferenceChainSearch::GetObjectFlags(FGCObjectInfo(Object)),
							*Object->GetFullName());
						NumErrors++;
					}
					if (!FGCFlags::IsReachable_ForGC(&ObjectItem) && !GUObjectArray.IsDisregardForGC(Object))
					{
						UE_LOGF(LogGarbage, Warning, "Object [%d] %ls%ls is NOT marked as Unreachable and NOT marked as Reachable.",
							ObjectIndexInGlobalArray,
							*FReferenceChainSearch::GetObjectFlags(FGCObjectInfo(Object)),
							*Object->GetFullName());
						NumErrors++;
					}					
					ObjectsToSerialize.Add(Object);
				}
				else
				{
					ThisThreadUnrachableObjectsNum++;

					if (FGCFlags::IsReachable_ForGC(&ObjectItem))
					{
						UE_LOGF(LogGarbage, Warning, "Object [%d] %ls%ls is still marked as Reachable.",
							ObjectIndexInGlobalArray,
							*FReferenceChainSearch::GetObjectFlags(FGCObjectInfo(Object)),
							*Object->GetFullName());
						NumErrors++;
					}
				}
			}
		}

		if (NumUnreachable > 0)
		{
			// No need to scan for unreachable obejcts if there's no unreachable objects. We only care about flag checks above in this case
			FUnreachableReferenceProcessor Processor;
			UE::GC::FWorkerContext Context;
			Context.SetInitialObjectsUnpadded(ObjectsToSerialize);
			CollectReferences(Processor, Context);
			NumErrors.fetch_add(Processor.GetErrorCount(), std::memory_order_acq_rel);
		}

		VerifiedNumUnreachable.fetch_add(ThisThreadUnrachableObjectsNum, std::memory_order_acq_rel);
	});

	if (VerifiedNumUnreachable.load() != NumUnreachable)
	{
		NumErrors++;
		UE_LOGF(LogGarbage, Warning, "The actual number of objects with the Unreachable flag (%d) does not match the number of gathered unreachable objects (%d).", VerifiedNumUnreachable.load(), NumUnreachable);
	}

	UE_CLOGF(NumErrors > 0, LogGarbage, Fatal, "Detected %d case(s) of breaking reachability assumptions.", NumErrors.load());

	UE_LOGF(LogGarbage, Log, "%f ms for VerifyNoUnreachableObjects", (FPlatformTime::Seconds() - StartTime) * 1000);
}

#endif // VERIFY_DISREGARD_GC_ASSUMPTIONS
#if PROFILE_GCConditionalBeginDestroy

TMap<FName, FCBDTime> CBDTimings;
TMap<UObject*, FName> CBDNameLookup;

void FScopedCBDProfile::DumpProfile()
{
	CBDTimings.ValueSort(TLess<FCBDTime>());
	int32 NumPrint = 0;
	for (auto& Item : CBDTimings)
	{
		UE_LOGF(LogGarbage, Log, "    %6d cnt %6.2fus per   %6.2fms total  %ls", Item.Value.Items, 1000.0f * 1000.0f * Item.Value.TotalTime / float(Item.Value.Items), 1000.0f * Item.Value.TotalTime, *Item.Key.ToString());
		if (NumPrint++ > 3000000000)
		{
			break;
		}
	}
	CBDTimings.Empty();
	CBDNameLookup.Empty();
}

#endif // PROFILE_GCConditionalBeginDestroy