// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GarbageCollectionVerification.h: Unreal realtime garbage collection helpers
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Class.h"

// UE_BUILD_SHIPPING has GShouldVerifyGCAssumptions=false by default
#define VERIFY_DISREGARD_GC_ASSUMPTIONS			!(UE_BUILD_SHIPPING)

#if VERIFY_DISREGARD_GC_ASSUMPTIONS

/** Verifies Disregard for GC assumptions */
void VerifyGCAssumptions();

/** Verifies GC Cluster assumptions */
void VerifyClustersAssumptions();

/** Verifies Object and Internal flag mirroring assumptions */
void VerifyObjectFlags();

/** Verifies Reachability assumptions */
void VerifyNoUnreachableObjects(int32 NumUnreachable);

#endif // VERIFY_DISREGARD_GC_ASSUMPTIONS


#define PROFILE_GCConditionalBeginDestroy 0
#define PROFILE_GCConditionalBeginDestroyByClass 0

#if PROFILE_GCConditionalBeginDestroy

struct FCBDTime
{
	double TotalTime;
	int32 Items;
	FCBDTime()
		: TotalTime(0.0)
		, Items(0)
	{
	}

	bool operator<(const FCBDTime& Other) const
	{
		return TotalTime > Other.TotalTime;
	}
};

extern TMap<FName, FCBDTime> CBDTimings;
extern TMap<UObject*, FName> CBDNameLookup;

struct FScopedCBDProfile
{
	FName Obj;
	double StartTime;

	FORCEINLINE FScopedCBDProfile(UObject* InObj)
		: StartTime(FPlatformTime::Seconds())
	{
		CBDNameLookup.Add(InObj, InObj->GetFName());
#if PROFILE_GCConditionalBeginDestroyByClass
		UObject* Outermost = ((UObject*)InObj->GetClass());
#else
		UObject* Outermost = ((UObject*)InObj->GetOutermost());
#endif
		Obj = Outermost->GetFName();
		if (Obj == NAME_None)
		{
			Obj = CBDNameLookup.FindRef(Outermost);
		}
	}
	FORCEINLINE ~FScopedCBDProfile()
	{
		double ThisTime = FPlatformTime::Seconds() - StartTime;
		FCBDTime& Rec = CBDTimings.FindOrAdd(Obj);
		Rec.Items++;
		Rec.TotalTime += ThisTime;
	}
	static void DumpProfile();
};


#else
struct FScopedCBDProfile
{
	FORCEINLINE FScopedCBDProfile(UObject*)
	{
	}
	FORCEINLINE static void DumpProfile()
	{
	}
};
#endif