// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/IClassReinstancer.h"

namespace UE::Private
{
	COREUOBJECT_API FReinstanceClassesPrepassFPtr GCreateClassReinstancerFPtr = nullptr;
}

TUniquePtr<UE::Private::IClassReinstancer> UE::Private::CreateClassReinstancer(TArray<UClass*>& InOutClasses)
{
	if (GCreateClassReinstancerFPtr)
	{
		return (*GCreateClassReinstancerFPtr)(InOutClasses);
	}
	return {};
}