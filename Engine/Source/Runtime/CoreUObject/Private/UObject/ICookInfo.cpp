// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ICookInfo.h"

#if WITH_EDITOR
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"
#include "Templates/UniquePtr.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"

#include <atomic>
#endif

#if WITH_EDITOR

namespace UE::Cook
{

/**
 * Tracks package loads under an FCookLoadScope that occur before the cooker has initialized;
 * after cooker initialization these loads are tracked by the cooker's FPackageTracker.
 */
class FCookLoadScopeStartupListener : public FUObjectArray::FUObjectCreateListener
{
public:
	FCookLoadScopeStartupListener();
	~FCookLoadScopeStartupListener();

	virtual void NotifyUObjectCreated(const UObjectBase* Object, int32 Index) override;
	virtual void OnUObjectArrayShutdown() override;

	void Unsubscribe();

public:
	TArray<TPair<FName, ECookLoadType>> StartupPackageLoadTypes;

private:
	bool bSubscribed = false;
};

static std::atomic<FCookLoadScopeStartupListener*> GCookLoadScopeStartupListener { nullptr };
static FCriticalSection GCookLoadScopeStartupListenerLock;
static thread_local ECookLoadType GCookLoadType = ECookLoadType::Unspecified;

const TCHAR* LexToString(EInstigator Value)
{
	switch (Value)
	{
#define EINSTIGATOR_VALUE_CALLBACK(Name, bAllowUnparameterized) case EInstigator::Name: return TEXT(#Name);
		EINSTIGATOR_VALUES(EINSTIGATOR_VALUE_CALLBACK)
#undef EINSTIGATOR_VALUE_CALLBACK
	default: return TEXT("OutOfRangeCategory");
	}
}

FString FInstigator::ToString() const
{
	TStringBuilder<256> Result;
	if (!Referencer.IsNone())
	{
		Result << Referencer << TEXT(" -> ") << LexToString(Category);
	}
	else
	{
		bool bCategoryAllowsUnparameterized = false;
		switch (Category)
		{
#define EINSTIGATOR_VALUE_CALLBACK(Name, bAllowUnparameterized) case EInstigator::Name: bCategoryAllowsUnparameterized = bAllowUnparameterized; break;
			EINSTIGATOR_VALUES(EINSTIGATOR_VALUE_CALLBACK)
#undef EINSTIGATOR_VALUE_CALLBACK
		default: break;
		}
		if (bCategoryAllowsUnparameterized)
		{
			Result << LexToString(Category);
		}
		else
		{
			Result << TEXT("<NoReferencer>") << TEXT(" -> ") << LexToString(Category);
		}
	}
	return FString(Result);
}

FCookInfoEvent FDelegates::ConfigureCookSession;
FCookInfoEvent FDelegates::CookStarted;
FCookInfoEvent FDelegates::CookFinished;
FValidateSourcePackage FDelegates::ValidateSourcePackage;
FPackageBlockedEvent FDelegates::PackageBlocked;
FCookInfoModifyCookDelegate FDelegates::ModifyCook;
FCookUpdateDisplayEvent FDelegates::CookUpdateDisplay;
FCookSaveIdleEvent FDelegates::CookSaveIdle;
FCookLoadIdleEvent FDelegates::CookLoadIdle;

const TCHAR* GetReferencedSetFilename()
{
	return TEXT("ReferencedSet.txt");
}

const TCHAR* GetReferencedSetOpName()
{
	return TEXT("ReferencedSet");
}

FCookLoadScopeStartupListener::FCookLoadScopeStartupListener()
{
	GUObjectArray.AddUObjectCreateListener(this);
	bSubscribed = true;
}

FCookLoadScopeStartupListener::~FCookLoadScopeStartupListener()
{
	Unsubscribe();
}

void FCookLoadScopeStartupListener::NotifyUObjectCreated(const class UObjectBase* Object, int32 Index)
{
	if (Object->GetClass() != UPackage::StaticClass())
	{
		return;
	}

	ECookLoadType CookLoadType = FCookLoadScope::GetCurrentValue();
	if (CookLoadType == ECookLoadType::Unspecified)
	{
		return;
	}

	FScopeLock ListenerScopeLock(&GCookLoadScopeStartupListenerLock);
	if (!GCookLoadScopeStartupListener)
	{
		// We have been called from the callback after we already unsubscribed
		// and deleted on another thread. Do not access anything on *this.
		return;
	}
	this->StartupPackageLoadTypes.Add(TPair<FName, ECookLoadType> { Object->GetFName(), CookLoadType});
}

void FCookLoadScopeStartupListener::OnUObjectArrayShutdown()
{
	Unsubscribe();
}

void FCookLoadScopeStartupListener::Unsubscribe()
{
	if (bSubscribed)
	{
		GUObjectArray.RemoveUObjectCreateListener(this);
		bSubscribed = false;
	}
}

void InitializeCookGlobals()
{
	check(!GCookLoadScopeStartupListener);
	// We need to construct outside of the GCookLoadScopeStartupListenerLock, because the constructor subscribes to
	// GUObjectArray which has its own lock; see the comment in SetCookerStartupComplete
	GCookLoadScopeStartupListener = new FCookLoadScopeStartupListener();
}

} // namespace UE::Cook

FCookLoadScope::FCookLoadScope(ECookLoadType ScopeType)
	: PreviousScope(UE::Cook::GCookLoadType)
{
	UE::Cook::GCookLoadType = ScopeType;
}

FCookLoadScope::~FCookLoadScope()
{
	UE::Cook::GCookLoadType = PreviousScope;
}

ECookLoadType FCookLoadScope::GetCurrentValue()
{
	return UE::Cook::GCookLoadType;
}

void FCookLoadScope::SetCookerStartupComplete(TArray<TPair<FName, ECookLoadType>>& OutStartupPackageLoadTypes)
{
	using namespace UE::Cook;

	if (!IsRunningCookCommandlet())
	{
		// InitializeCookGlobals is only called when IsRunningCookCommandlet;
		// We need that hook for this function to work, so we do not implement
		// StartupPackageLoadTypes when cooking through means other than running as CookCommandlet.
		return;
	}

	FCookLoadScopeStartupListener* Listener = GCookLoadScopeStartupListener;
	check(Listener); // Set by InitializeCookGlobals
	// We need to unsubscribe from GUObjectArray outside of GCookLoadScopeStartupListenerScopeLock, because
	// GUObjectArray might have its own lock that it holds when calling NotifyUObjectCreated, and we enter
	// GCookLoadScopeStartupListenerScopeLock inside of NotifyUObjectCreated. To avoid locking in a different
	// order and therefore deadlocking if another thread (e.g. AsyncLoadingThread) is adding a UPackage during
	// SetCookerStartupComplete, we therefore need to not hold our lock while unsubscribing.
	Listener->Unsubscribe();

	FScopeLock ListenerScopeLock(&GCookLoadScopeStartupListenerLock);
	GCookLoadScopeStartupListener = nullptr;
	OutStartupPackageLoadTypes = MoveTemp(Listener->StartupPackageLoadTypes);
	delete Listener;
}

void FCookLoadScope::GetCookerStartupPackages(TMap<FName, UE::Cook::FInstigator>& OutStartupPackages)
{
	OutStartupPackages.Empty();
	for (TObjectIterator<UPackage> It; It; ++It)
	{
		UPackage* Package = *It;
		if (Package->GetOuter() == nullptr && Package != GetTransientPackage())
		{
			OutStartupPackages.Add(Package->GetFName(), UE::Cook::FInstigator(UE::Cook::EInstigator::StartupPackage));
		}
	}

	TArray<TPair<FName, ECookLoadType>> StartupPackageLoadTypes;
	FCookLoadScope::SetCookerStartupComplete(StartupPackageLoadTypes);

	for (TPair<FName, ECookLoadType>& Pair : StartupPackageLoadTypes)
	{
		switch (Pair.Value)
		{
		case ECookLoadType::EditorOnly:
		{
			UE::Cook::FInstigator* Instigator = OutStartupPackages.Find(Pair.Key);
			if (Instigator)
			{
				*Instigator = UE::Cook::FInstigator(UE::Cook::EInstigator::EditorOnlyLoad);
			}
			break;
		}
		case ECookLoadType::UsedInGame:
		{
			UE::Cook::FInstigator* Instigator = OutStartupPackages.Find(Pair.Key);
			if (Instigator)
			{
				*Instigator = UE::Cook::FInstigator(UE::Cook::EInstigator::StartupPackageCookLoadScope);
			}
			break;
		}
		default:
			break;
		}
	}
}

#endif // WITH_EDITOR
