// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ImportExportCollector.h"

#if WITH_EDITORONLY_DATA

FImportExportCollector::FImportExportCollector(UPackage* InRootPackage)
	: RootPackage(InRootPackage)
	, RootPackageName(InRootPackage->GetFName())
{
	ArIsObjectReferenceCollector = true;
	ArIsModifyingWeakAndStrongReferences = true;
	SetIsSaving(true);
	SetIsPersistent(true);
}

void FImportExportCollector::Reset()
{
	Visited.Reset();
	Imports.Reset();
	EditorOnlyObjectCache.Reset();
}

void FImportExportCollector::AddExportToIgnore(UObject* Export)
{
	Visited.Add(Export, EVisitResult::Excluded);
}

void FImportExportCollector::SerializeObjectAndReferencedExports(UObject* RootObject)
{
	*this << RootObject;
	while (!ExportsExploreQueue.IsEmpty())
	{
		UObject* Export = ExportsExploreQueue.PopFrontValue();
		Export->Serialize(*this);
	}
}

FArchive& FImportExportCollector::operator<<(UObject*& Obj)
{
	if (!Obj)
	{
		return *this;
	}

	bool bFirstVisit = false;
	EVisitResult& VisitResult = Visited.FindOrAdd(Obj);
	switch (VisitResult)
	{
	case EVisitResult::Excluded: [[fallthrough]];
	case EVisitResult::Export:
		return *this;
	case EVisitResult::Import:
		// We have to call AddImport every time instead of early exiting because the SoftObjectPathCollectType might
		// need to be updated
		break;
	case EVisitResult::Uninitialized:
	{
		bFirstVisit = true;
		if (IsFilterEditorOnly() && CachedIsEditorOnlyObject(Obj))
		{
			bool bAllowedObject = ((bool)CallbackIsEditorOnlyObjectAllowed) &&
				CallbackIsEditorOnlyObjectAllowed(Obj);
			if (!bAllowedObject)
			{
				VisitResult = EVisitResult::Excluded;
				return *this;
			}
		}

		UPackage* Package = Obj->GetPackage();
		if (!Package)
		{
			VisitResult = EVisitResult::Excluded;
			return *this;
		}

		VisitResult = Package != RootPackage ? EVisitResult::Import : EVisitResult::Export;
		break;
	}
	default:
		checkNoEntry();
		return *this;
	}

	switch (VisitResult)
	{
	case EVisitResult::Import:
		AddImport(FSoftObjectPath(Obj), ESoftObjectPathCollectType::AlwaysCollect);
		break;
	case EVisitResult::Export:
		check(bFirstVisit); // We should have early exited above if revisiting an export
		ExportsExploreQueue.Add(Obj);
		break;
	default:
		checkNoEntry();
		return *this;
	}

	return *this;
}

FArchive& FImportExportCollector::operator<<(FSoftObjectPath& Value)
{
	FName CurrentPackage;
	FName PropertyName;
	ESoftObjectPathCollectType CollectType;
	ESoftObjectPathSerializeType SerializeType;
	FSoftObjectPathThreadContext& ThreadContext = FSoftObjectPathThreadContext::Get();
	ThreadContext.GetSerializationOptions(CurrentPackage, PropertyName, CollectType, SerializeType, this);

	if (UE::SoftObjectPath::IsCollectable(CollectType))
	{
		FName PackageName = Value.GetLongPackageFName();
		if (PackageName != RootPackageName && !PackageName.IsNone())
		{
			AddImport(Value, CollectType);
		}
	}
	return *this;
}

void FImportExportCollector::AddImport(const FSoftObjectPath& Path, ESoftObjectPathCollectType CollectType)
{
	ESoftObjectPathCollectType& ExistingImport = Imports.FindOrAdd(
		Path, ESoftObjectPathCollectType::EditorOnlyCollect);
	ExistingImport = Union(ExistingImport, CollectType);

	ESoftObjectPathCollectType& ExistingPackage = ImportedPackages.FindOrAdd(
		Path.GetLongPackageFName(), ESoftObjectPathCollectType::EditorOnlyCollect);
	ExistingPackage = Union(ExistingPackage, CollectType);
}

ESoftObjectPathCollectType FImportExportCollector::Union(ESoftObjectPathCollectType A, ESoftObjectPathCollectType B)
{
	return static_cast<ESoftObjectPathCollectType>(FMath::Max(static_cast<int>(A), static_cast<int>(B)));
}

bool FImportExportCollector::CachedIsEditorOnlyObject(const UObject* Object)
{
	using namespace UE::SavePackageUtilities;

#if WITH_EDITOR
	return IsEditorOnlyObject(Object, true /* bCheckRecursive */,
		[this](const UObject* InObject)
		{
			return EditorOnlyObjectCache.FindOrAdd(InObject, EEditorOnlyObjectResult::Uninitialized);
		},
		[this](const UObject* InObject, bool bEditorOnly)
		{
			EditorOnlyObjectCache.FindOrAdd(InObject) =
				bEditorOnly ? EEditorOnlyObjectResult::EditorOnly : EEditorOnlyObjectResult::NonEditorOnly;
		});
#else
	return false;
#endif
}

#endif // WITH_EDITORONLY_DATA