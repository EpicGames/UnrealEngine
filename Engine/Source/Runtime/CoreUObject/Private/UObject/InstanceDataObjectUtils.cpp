// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/InstanceDataObjectUtils.h"

#if WITH_EDITORONLY_DATA

#include "Async/ManualResetEvent.h"
#include "Async/SharedLock.h"
#include "Async/SharedMutex.h"
#include "Async/UniqueLock.h"
#include "HAL/IConsoleManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/ReverseIterate.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/ArchiveCountMem.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Serialization/ImportTypeHierarchy.h"
#include "UObject/Class.h"
#include "UObject/CommonFNames.h"
#include "UObject/EnumProperty.h"
#include "UObject/Field.h"
#include "UObject/IntrinsicClassFwd.h"
#include "UObject/PropertyStateTracking.h"
#include "UObject/Package.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/PropertyHelper.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPathNameTree.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectAnnotation.h"
#include "VerseVM/VVMVerseClass.h"

DEFINE_LOG_CATEGORY_STATIC(LogInstanceDataObject, Log, VeryVerbose);

namespace UE::IDOHelpers
{
	struct FSetupIsUnknownPropertyFnPtr
	{
		FSetupIsUnknownPropertyFnPtr()
		{
			GIsUnknownPropertyFunc = UE::IsPropertyLoose;
		}
		~FSetupIsUnknownPropertyFnPtr()
		{
			GIsUnknownPropertyFunc.Reset();
		}
	} SetupGetIsUnknownPropertyFnPtr;
}


// Implemented in Class.cpp
int32 CalculatePropertyIndex(const UStruct* Struct, const FProperty* Property, int32 ArrayIndex);

static const FName NAME_InitializedValues(ANSITEXTVIEW("_InitializedValues"));
static const FName NAME_SerializedValues(ANSITEXTVIEW("_SerializedValues"));

template <typename SuperType>
class TInstanceDataObjectPropertyValueFlags : public SuperType
{
public:
	using SuperType::SuperType;

	bool ActivateTrackingPropertyValueFlag(EPropertyValueFlags Flags, void* Data) const final
	{
		// Nothing to activate because tracking is either always on or always off.
		return IsTrackingPropertyValueFlag(Flags, Data);
	}

	bool IsTrackingPropertyValueFlag(EPropertyValueFlags Flags, const void* Data) const final
	{
		return !!GetPropertyValueFlagsProperty(Flags);
	}

	bool HasPropertyValueFlag(EPropertyValueFlags Flags, const void* Data, const FProperty* Property, int32 ArrayIndex) const final
	{
		if (Property == InitializedValuesProperty || Property == SerializedValuesProperty)
		{
			return true;
		}

		if (const FProperty* FlagsProperty = GetPropertyValueFlagsProperty(Flags))
		{
			const int32 PropertyIndex = CalculatePropertyIndex(this, Property, ArrayIndex);
			const int32 ByteIndex = PropertyIndex / 8;
			const int32 BitOffset = PropertyIndex % 8;
			if (ensureMsgf(ByteIndex < FlagsProperty->ArrayDim,
				TEXT("Property %s in %s has out of range index %d with capacity for %d."),
				*Property->GetAuthoredName(), *SuperType::GetPathName(), PropertyIndex, FlagsProperty->ArrayDim * 8))
			{
				const uint8* FlagsData = FlagsProperty->ContainerPtrToValuePtr<uint8>(Data, ByteIndex);
				return (*FlagsData & (1 << BitOffset)) != 0;
			}
		}
		// Default to initialized when tracking is inactive.
		return true;
	}

	void SetPropertyValueFlag(EPropertyValueFlags Flags, bool bValue, void* Data, const FProperty* Property, int32 ArrayIndex) const final
	{
		if (Property == InitializedValuesProperty || Property == SerializedValuesProperty)
		{
			return;
		}

		if (const FProperty* FlagsProperty = GetPropertyValueFlagsProperty(Flags))
		{
			const int32 PropertyIndex = CalculatePropertyIndex(this, Property, ArrayIndex);
			const int32 ByteIndex = PropertyIndex / 8;
			const int32 BitOffset = PropertyIndex % 8;
			if (ensureMsgf(ByteIndex < FlagsProperty->ArrayDim,
				TEXT("Property %s in %s has out of range index %d with capacity for %d."),
				*Property->GetAuthoredName(), *SuperType::GetPathName(), PropertyIndex, FlagsProperty->ArrayDim * 8))
			{
				uint8* FlagsData = FlagsProperty->ContainerPtrToValuePtr<uint8>(Data, ByteIndex);
				if (bValue)
				{
					*FlagsData |= (1 << BitOffset);
				}
				else
				{
					*FlagsData &= ~(1 << BitOffset);
				}
			}
		}
	}

	void ResetPropertyValueFlags(EPropertyValueFlags Flags, void* Data) const final
	{
		if (const FProperty* FlagsProperty = GetPropertyValueFlagsProperty(Flags))
		{
			uint8* FlagsData = FlagsProperty->ContainerPtrToValuePtr<uint8>(Data);
			FMemory::Memzero(FlagsData, FlagsProperty->ArrayDim);
		}
	}

	void SerializePropertyValueFlags(EPropertyValueFlags Flags, void* Data, FStructuredArchiveRecord Record, FArchiveFieldName Name) const final
	{
		const FProperty* FlagsProperty = GetPropertyValueFlagsProperty(Flags);
		if (TOptional<FStructuredArchiveSlot> Slot = Record.TryEnterField(Name, !!FlagsProperty))
		{
			checkf(FlagsProperty, TEXT("Type %s is missing a property that is needed to serialize property value flags."), *SuperType::GetPathName());
			uint8* FlagsData = FlagsProperty->ContainerPtrToValuePtr<uint8>(Data);
			Slot->Serialize(FlagsData, FlagsProperty->ArrayDim);
		}
	}

	const FProperty* GetPropertyValueFlagsProperty(EPropertyValueFlags Flags) const
	{
		switch (Flags)
		{
		case EPropertyValueFlags::Initialized:
			return InitializedValuesProperty;
		case EPropertyValueFlags::Serialized:
			return SerializedValuesProperty;
		default:
			checkNoEntry();
			return nullptr;
		}
	}

	FByteProperty* InitializedValuesProperty = nullptr;
	FByteProperty* SerializedValuesProperty = nullptr;
};

UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UInstanceDataObjectClass, NO_API)

/** Type used for InstanceDataObject classes. */
class UInstanceDataObjectClass final : public TInstanceDataObjectPropertyValueFlags<UClass>
{
	using SuperType = TInstanceDataObjectPropertyValueFlags<UClass>;

public:
	// Note that UClass is passed to the macro rather than SuperType, but SuperType is used directly for constructors. 
	DECLARE_CASTED_CLASS_INTRINSIC_NO_CTOR_NO_VTABLE_CTOR(UInstanceDataObjectClass, UClass, CLASS_Transient, TEXT("/Script/CoreUObject"), CASTCLASS_UClass, NO_API)
	RELAY_CONSTRUCTOR(UInstanceDataObjectClass, SuperType)
	UInstanceDataObjectClass(FVTableHelper& Helper) : SuperType(Helper) {};
};

IMPLEMENT_CORE_INTRINSIC_CLASS(UInstanceDataObjectClass, UClass,
{
});

UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UInstanceDataObjectStruct, NO_API)

/** Type used for InstanceDataObject structs to provide support for hashing and custom guids. */
class UInstanceDataObjectStruct final : public TInstanceDataObjectPropertyValueFlags<UScriptStruct>
{
	using SuperType = TInstanceDataObjectPropertyValueFlags<UScriptStruct>;

public:
	// Note that UScriptStruct is passed to the macro rather than SuperType, but SuperType is used directly for constructors. 
	DECLARE_CASTED_CLASS_INTRINSIC_NO_CTOR_NO_VTABLE_CTOR(UInstanceDataObjectStruct, UScriptStruct, CLASS_Transient, TEXT("/Script/CoreUObject"), CASTCLASS_UScriptStruct, NO_API)
	RELAY_CONSTRUCTOR(UInstanceDataObjectStruct, SuperType)
	UInstanceDataObjectStruct(FVTableHelper& Helper) : SuperType(Helper) {};

	uint32 GetStructTypeHash(const void* Src) const final;
	FGuid GetCustomGuid() const final { return Guid; }

	FGuid Guid;
};

IMPLEMENT_CORE_INTRINSIC_CLASS(UInstanceDataObjectStruct, UScriptStruct,
{
});

uint32 UInstanceDataObjectStruct::GetStructTypeHash(const void* Src) const
{
	class FBoolHash
	{
	public:
		inline void Hash(bool bValue)
		{
			BoolValues = (BoolValues << 1) | (bValue ? 1 : 0);
			if ((++BoolCount & 63) == 0)
			{
				Flush();
			}
		}

		inline uint32 CalculateHash()
		{
			if (BoolCount & 63)
			{
				Flush();
			}
			return BoolHash;
		}

	private:
		inline void Flush()
		{
			BoolHash = HashCombineFast(BoolHash, GetTypeHash(BoolValues));
			BoolValues = 0;
		}

		uint32 BoolHash = 0;
		uint32 BoolCount = 0;
		uint64 BoolValues = 0;
	};

	FBoolHash BoolHash;
	uint32 ValueHash = 0;
	for (TFieldIterator<const FProperty> It(this); It; ++It)
	{
		if (It->GetFName() == NAME_InitializedValues || It->GetFName() == NAME_SerializedValues)
		{
			continue;
		}
		if (const FBoolProperty* BoolProperty = CastField<const FBoolProperty>(*It))
		{
			for (int32 I = 0; I < It->ArrayDim; ++I)
			{
				BoolHash.Hash(BoolProperty->GetPropertyValue_InContainer(Src, I));
			}
		}
		else if (It->HasAllPropertyFlags(CPF_HasGetValueTypeHash))
		{
			for (int32 I = 0; I < It->ArrayDim; ++I)
			{
				uint32 Hash = It->GetValueTypeHash(It->ContainerPtrToValuePtr<void>(Src, I));
				ValueHash = HashCombineFast(ValueHash, Hash);
			}
		}
		else
		{
			UE_LOGFMT(LogInstanceDataObject, Warning,
				"Struct {StructType} contains property {PropertyName} of type {PropertyType} that is missing GetValueTypeHash.",
				UE::FAssetLog(this), It->GetFName(), WriteToString<128>(UE::FPropertyTypeName(*It)));
			ValueHash = HashCombineFast(ValueHash, It->ArrayDim);
		}
	}

	if (const uint32 Hash = BoolHash.CalculateHash())
	{
		ValueHash = HashCombineFast(ValueHash, Hash);
	}

	return ValueHash;
}

namespace UE
{
	static const FName NAME_DisplayName(ANSITEXTVIEW("DisplayName"));
	static const FName NAME_PresentAsTypeMetadata(ANSITEXTVIEW("PresentAsType"));
	static const FName NAME_IsInstanceDataObjectStruct(ANSITEXTVIEW("IsInstanceDataObjectClass"));
	static const FName NAME_ContainsLoosePropertiesMetadata(ANSITEXTVIEW("ContainsLooseProperties"));
	static const FName NAME_Class(ANSITEXTVIEW("Class"));
	static const FName NAME_VerseClass(ANSITEXTVIEW("VerseClass"));
	static const FName NAME_BlueprintGeneratedClass(ANSITEXTVIEW("BlueprintGeneratedClass"));
	static const FName NAME_VerseDevice(ANSITEXTVIEW("VerseDevice_C"));
	static const FName NAME_Entity(ANSITEXTVIEW("entity"));
	static const FName NAME_Component(ANSITEXTVIEW("component"));
	static const FName NAME_IDOMapKey(ANSITEXTVIEW("Key"));
	static const FName NAME_IDOMapValue(ANSITEXTVIEW("Value"));
	static const FName NAME_TemplateStructName(ANSITEXTVIEW("TemplateStructName"));

	template <typename T>
	struct TCacheItem
	{
		TWeakObjectPtr<T>     WeakPtr;
		UE::FManualResetEvent ReadyEvent;

		T* Extract()
		{
			if (!ReadyEvent.IsNotified())
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(IDO_CacheItem_Wait);
				ReadyEvent.Wait();
			}

			return WeakPtr.Get();
		}
	};

	static TMap<FBlake3Hash, TSharedPtr<TCacheItem<UInstanceDataObjectClass>>> IDOClassCache;
	static FSharedMutex IDOClassCacheMutex;

	static void OnInstanceDataObjectSupportChanged(IConsoleVariable*);

	bool bEverCreatedIDO = false;

	bool bEnableIDOSupport = false;
	bool bEverEnabledIDOSupport = true;
	FAutoConsoleVariableRef EnableIDOSupportCVar(
		TEXT("IDO.Enable"),
		bEnableIDOSupport,
		TEXT("Allows an IDO to be created for an object if its class has support."),
		FConsoleVariableDelegate::CreateStatic(OnInstanceDataObjectSupportChanged)
	);

	bool bEnableIDOSupportOnEveryObject = false;
	bool bEverEnabledIDOSupportOnEveryObject = false;
	FAutoConsoleVariableRef EnableIDOSupportOnEveryObjectCVar(
		TEXT("IDO.EnableOnEveryObject"),
		bEnableIDOSupportOnEveryObject,
		TEXT("Allows an IDO to be created for every object."),
		FConsoleVariableDelegate::CreateStatic(OnInstanceDataObjectSupportChanged)
	);

	bool bEnableIDOOnSpawnedObjects = true;
	FAutoConsoleVariableRef EnableIDOOnSpawnedObjects(
		TEXT("IDO.EnableOnSpawnedObjects"),
		bEnableIDOOnSpawnedObjects,
		TEXT("Allows an IDO to be created for every newly created (e.g., spawned) object.")
	);

	bool bEnableIDOUnknownProperties = true;
	FAutoConsoleVariableRef EnableIDOUnknownProperties(
		TEXT("IDO.Unknowns.EnableProperties"),
		bEnableIDOUnknownProperties,
		TEXT("When enabled, IDOs will include unknown properties.")
	);

	bool bEnableIDOUnknownEnums = true;
	FAutoConsoleVariableRef EnableIDOUnknownEnums(
		TEXT("IDO.Unknowns.EnableEnums"),
		bEnableIDOUnknownEnums,
		TEXT("When enabled, IDOs will include unknown enum names.")
	);

	bool bEnableIDOUnknownStructs = true;
	FAutoConsoleVariableRef EnableIDOUnknownStructs(
		TEXT("IDO.Unknowns.EnableStructs"),
		bEnableIDOUnknownStructs,
		TEXT("When enabled, IDOs will include unknown structs and the properties within them.")
	);

	FString ExcludedUnknownPropertyTypesVar = "VerseFunctionProperty";
	FAutoConsoleVariableRef ExcludedUnknownPropertyTypesCVar(
		TEXT("IDO.Unknowns.ExcludedTypes"),
		ExcludedUnknownPropertyTypesVar,
		TEXT("Comma separated list of property types that will be excluded from loose properties in IDOs.")
	);

	bool bEnableUninitializedUI = true;
	FAutoConsoleVariableRef EnableUninitializedUICVar(
		TEXT("IDO.EnableUninitializedAlertUI"),
		bEnableUninitializedUI,
		TEXT("Enables alert information for uninitialized properties. Requires IDO.Enable=true")
	);

	bool bEnableIDOImpersonationOnSave = true;
	FAutoConsoleVariableRef EnableIDOImpersonationOnSaveCVar(
		TEXT("IDO.Impersonation.EnableOnSave"),
		bEnableIDOImpersonationOnSave,
		TEXT("When enabled, IDOs will be saved instead of instances. Disabling this will stop data retention on save.")
	);

	bool bSkipImpersonationWithoutUnknowns = true;
	FAutoConsoleVariableRef SkipImpersonationWithoutUnknownsCVar(
		TEXT("IDO.Impersonation.SkipWithoutUnknownProperties"),
		bSkipImpersonationWithoutUnknowns,
		TEXT("When enabled, don't impersonate if the IDO has no unknown properties.")
	);

	bool bEnableIDOsForBlueprintArchetypes = true;
	FAutoConsoleVariableRef EnableIDOsForBlueprintArchetypesCVar(
		TEXT("IDO.EnableBlueprintArchetypes"),
		bEnableIDOsForBlueprintArchetypes,
		TEXT("When enabled, blueprint archetypes (and prefab archetypes) can have IDOs generated for them")
	);

	bool bEnableIDOsForBlueprintInstances = true;
	FAutoConsoleVariableRef EnableIDOsForBlueprintInstancesCVar(
		TEXT("IDO.EnableBlueprintInstances"),
		bEnableIDOsForBlueprintInstances,
		TEXT("When enabled, blueprint instances (and prefab instances) can have IDOs generated for them")
	);

	bool bEnableIDOArchetypeChain = true;
	FAutoConsoleVariableRef EnableIDOArchetypeChainCVar(
		TEXT("IDO.EnableArchetypeChain"),
		bEnableIDOArchetypeChain,
		TEXT("When enabled, IDOs will be constructed using an archetype chain")
	);

	bool bEnableIDOPlaceholderObjects = true;
	FAutoConsoleVariableRef EnableIDOPlaceholderObjectsCVar(
		TEXT("IDO.Placeholder.Enable"),
		bEnableIDOPlaceholderObjects,
		TEXT("If true, allows placeholder types to be created in place of missing types in order to allow instance data to be serialized."),
		ECVF_Default
	);

	bool bEnableFullIDOInherritance = true;
	FAutoConsoleVariableRef EnableFullIDOInherritanceCVar(
		TEXT("IDO.EnableFullInherritance"),
		bEnableFullIDOInherritance,
		TEXT("When enabled, IDO classes will inherrit from an IDO class of the template's super")
	);

	bool bEnableIDOOuterChain = true;
	FAutoConsoleVariableRef EnableIDOOuterChainCVar(
		TEXT("IDO.EnableOuterChain"),
		bEnableIDOOuterChain,
		TEXT("When enabled, IDOs of subobjects will be outered to the parent object's IDO")
	);

	FAutoConsoleCommand EnableIDOUnknownsCommand(
		TEXT("IDO.Unknowns.Enable"),
		TEXT("Use this command to toggle IDO.Unknowns.* on or off, or to report their current state."),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
		{
			if (Args.Num() == 1)
			{
				bool bEnabled = Args[0] == TEXT("True") || Args[0] == TEXT("1");
				EnableIDOUnknownProperties->Set(bEnabled);
				EnableIDOUnknownEnums->Set(bEnabled);
				EnableIDOUnknownStructs->Set(bEnabled);
			}

			bool bEnabled = bEnableIDOUnknownProperties && bEnableIDOUnknownEnums && bEnableIDOUnknownStructs;
			OutputDevice.Logf(TEXT("IDO.Unknowns.Enable = \"%s\""), bEnabled ? TEXT("True") : TEXT("False"));
		})
	);

	static void SetEnableAllIDOFeatures(bool bEnabled)
	{
		EnableIDOSupportCVar->Set(bEnabled);
		EnableIDOUnknownProperties->Set(bEnabled);
		EnableIDOUnknownEnums->Set(bEnabled);
		EnableIDOUnknownStructs->Set(bEnabled);
		EnableUninitializedUICVar->Set(bEnabled);
		EnableIDOImpersonationOnSaveCVar->Set(bEnabled);
		EnableIDOsForBlueprintArchetypesCVar->Set(bEnabled);
		EnableIDOsForBlueprintInstancesCVar->Set(bEnabled);
		EnableIDOArchetypeChainCVar->Set(bEnabled);
		EnableIDOPlaceholderObjectsCVar->Set(bEnabled);
		EnableFullIDOInherritanceCVar->Set(bEnabled);
		EnableIDOOuterChainCVar->Set(bEnabled);
	}

	static bool AreAllIDOFeaturesEnabled()
	{
		return
			bEnableIDOSupport &&
			bEnableIDOUnknownProperties &&
			bEnableIDOUnknownEnums &&
			bEnableIDOUnknownStructs &&
			bEnableUninitializedUI &&
			bEnableIDOImpersonationOnSave &&
			bEnableIDOsForBlueprintArchetypes &&
			bEnableIDOsForBlueprintInstances &&
			bEnableIDOArchetypeChain &&
			bEnableIDOPlaceholderObjects &&
			bEnableFullIDOInherritance &&
			bEnableIDOOuterChain;
	}

	FAutoConsoleCommand EnableAllIDOFeaturesCommand(
		TEXT("IDO.EnableAllFeatures"),
		TEXT("Call this method to toggle all IDO related features on"),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
		{
			if (Args.Num() == 1)
			{
				bool Val = Args[0] == TEXT("True") || Args[0] == TEXT("1");
				SetEnableAllIDOFeatures(Val);
			}

			OutputDevice.Logf(TEXT("IDO.EnableAllFeatures = \"%s\""), AreAllIDOFeaturesEnabled() ? TEXT("True") : TEXT("False"));
		})
	);

	static void OutputIDOStats(FOutputDevice& OutputDevice)
	{
		#if STATS
		struct MemoryMetric
		{
			const TCHAR* Unit;
			double Value;
		};

		const auto ConvertToMemoryMetric = [](size_t MemoryBytes) -> MemoryMetric
		{
			MemoryMetric Metric = {};

			size_t GB = 1024ULL * 1024ULL * 1024ULL;
			size_t MB = 1024ULL * 1024ULL;
			size_t KB = 1024ULL;
			Metric.Value = (double)MemoryBytes;

			if (MemoryBytes >= GB)
			{
				Metric.Value /= (double)GB;
				Metric.Unit = TEXT("GB");
			}
			else if (MemoryBytes >= MB)
			{
				Metric.Value /= (double)MB;
				Metric.Unit = TEXT("MB");
			}
			else if (MemoryBytes >= KB)
			{
				Metric.Value /= (double)KB;
				Metric.Unit = TEXT("KB");
			}
			else
			{
				Metric.Unit = TEXT("bytes");
			}

			return Metric;
		};

		FPropertyBagRepository::FPropertyBagRepositoryStats Stats;
		FPropertyBagRepository::Get().GatherStats(Stats);

		int32 NumIDOClasses = 0;
		size_t ClassMemoryBytes = 0;
		size_t CDOMemoryBytes = 0;
		{
			TUniqueLock Lock(IDOClassCacheMutex);

			for (const TPair<FBlake3Hash, TSharedPtr<TCacheItem<UInstanceDataObjectClass>>>& Pair : IDOClassCache)
			{
				UInstanceDataObjectClass* Class = Pair.Value->WeakPtr.Get();

				if (Class)
				{
					++NumIDOClasses;

					{
						FArchiveCountMem MemoryCount(Class);
						ClassMemoryBytes += MemoryCount.GetMax();
					}

					if (UObject* CDO = Class->GetDefaultObject(/*bCreateIfNeeded*/false))
					{
						FArchiveCountMem MemoryCount(CDO);
						CDOMemoryBytes += MemoryCount.GetMax();
					}
				}
				
			}
		}

		size_t TotalMemoryBytes = Stats.IDOMemoryBytes + ClassMemoryBytes + CDOMemoryBytes;

		MemoryMetric TotalMemory = ConvertToMemoryMetric(TotalMemoryBytes);
		MemoryMetric ObjectMemory = ConvertToMemoryMetric(Stats.IDOMemoryBytes);
		MemoryMetric ClassMemory = ConvertToMemoryMetric(ClassMemoryBytes);
		MemoryMetric CDOMemory = ConvertToMemoryMetric(CDOMemoryBytes);

		OutputDevice.Logf(TEXT("Number of IDOs = %d"), Stats.NumIDOs);
		OutputDevice.Logf(TEXT("Number of IDOs with loose properties = %d"), Stats.NumIDOsWithLooseProperties);
		OutputDevice.Logf(TEXT("Number of IDO classes = %u"), NumIDOClasses);
		OutputDevice.Logf(TEXT("Number of placeholder types = %u"), Stats.NumPlaceholderTypes);
		OutputDevice.Logf(TEXT("Total IDO memory = %.2f %s"), TotalMemory.Value, TotalMemory.Unit);
		OutputDevice.Logf(TEXT("    IDO object memory = %.2f %s"), ObjectMemory.Value, ObjectMemory.Unit);
		OutputDevice.Logf(TEXT("    IDO class memory = %.2f %s"), ClassMemory.Value, ClassMemory.Unit);
		OutputDevice.Logf(TEXT("    IDO CDO memory = %.2f %s"), CDOMemory.Value, CDOMemory.Unit);
		#else
		OutputDevice.Log(TEXT("Stats not enabled on current build"));
		#endif // STATS
	}

	static void DumpIDOs(FOutputDevice& OutputDevice)
	{
		FPropertyBagRepository::Get().DumpPropertyBagPlaceholderTypes(OutputDevice);

		OutputDevice.Log(TEXT("Instance Data Object classes:"));
		OutputDevice.Log(TEXT("------------------------------"));

		{
			TUniqueLock Lock(IDOClassCacheMutex);

			int32 NumIDOClasses = 0;

			for (const TPair<FBlake3Hash, TSharedPtr<TCacheItem<UInstanceDataObjectClass>>>& Pair : IDOClassCache)
			{
				UInstanceDataObjectClass* Class = Pair.Value->WeakPtr.Get();

				if (Class)
				{
					++NumIDOClasses;

					OutputDevice.Logf(TEXT("%s"), *Class->GetFullName());
				}

			}

			OutputDevice.Logf(TEXT("%d IDO class(es)"), NumIDOClasses);
		}

		FPropertyBagRepository::Get().DumpIDOs(OutputDevice);
	}

	FAutoConsoleCommand DumpIDOStatsCommand(
		TEXT("IDO.DumpStats"),
		TEXT("Prints statistics for all current Instance Data Objects."),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
		{
			OutputIDOStats(OutputDevice);
		})
	);

	FAutoConsoleCommand DumpIDOsCommand(
		TEXT("IDO.Dump"),
		TEXT("Prints all Instance Data Objects and their classes."),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& OutputDevice)
		{
			DumpIDOs(OutputDevice);
		})
	);

	static TSet<FString> GetExcludedUnknownPropertyTypes()
	{
		TArray<FString> Result;
		ExcludedUnknownPropertyTypesVar.ParseIntoArray(Result, TEXT(","));
		return TSet<FString>(Result);
	}

	bool IsInstanceDataObjectSupportEnabled()
	{
#if UE_EDITOR
		// Keep IDOs disabled during non-editor runs
		if (!GIsEditor)
		{
			return false;
		}

		// Keep IDOs disabled during non-cook commandlet runs
		if (IsRunningCommandlet() && !IsRunningCookCommandlet())
		{
			return false;
		}
#endif
		return bEnableIDOSupport;
	}

	bool IsUninitializedAlertUIEnabled()
	{
		return bEnableUninitializedUI;
	}

	bool IsInstanceDataObjectImpersonationEnabledOnSave(const UObject* Object)
	{
		if (bEnableIDOImpersonationOnSave)
		{
			if (bSkipImpersonationWithoutUnknowns)
			{
				UObject* Ido = FPropertyBagRepository::Get().FindInstanceDataObject(Object);
				// only impersonate if the object containse unknown (loose) properties
				return Ido && StructContainsLooseProperties(Ido->GetClass());
			}
			return true;
		}
		return false;
	}

	bool IsInstanceDataObjectArchetypeChainEnabled()
	{
		return bEnableIDOArchetypeChain;
	}

	bool IsInstanceDataObjectPlaceholderObjectSupportEnabled()
	{
		return bEnableIDOPlaceholderObjects && !IsRunningCookCommandlet(); // don't create placeholders when running a cook process
	}

	bool IsInstanceDataObjectOuterChainEnabled()
	{
		return bEnableIDOOuterChain;
	}

	static const UObject* GetBlueprintGeneratedObject(const UObject* InObject)
	{
		const UObject* Current = InObject;
		while (Current && !Current->IsA<UPackage>())
		{
			if (Current->GetClass()->GetClass()->GetFName() == NAME_BlueprintGeneratedClass)
			{
				return Current;
			}
			Current = Current->GetOuter();
		}
		return nullptr;
	}

	bool IsInstanceDataObjectSupportEnabledForClass(const UClass* Class)
	{
		return bEnableIDOSupport && (bEnableIDOSupportOnEveryObject || Class->CanCreateInstanceDataObject());
	}

	bool IsInstanceDataObjectSupportEnabledForGC(const UClass* Class)
	{
		// Garbage Collection must always consider IDOs once an IDO has been created in the relevant category.
		return bEverEnabledIDOSupport && (bEverEnabledIDOSupportOnEveryObject || Class->CanCreateInstanceDataObject());
	}

	bool IsInstanceDataObjectSupportEnabled(const UObject* InObject)
	{
		if (!InObject || !bEverEnabledIDOSupport)
		{
			return false;
		}

		if (IsInstanceDataObject(InObject))
		{
			return true;
		}

		// Property bag placeholder objects are always enabled for IDO support
		if (FPropertyBagRepository::IsPropertyBagPlaceholderObject(InObject))
		{
			return true;
		}

		// Assume that if this object has an IDO that it's enabled. This assumption is important for objects
		// that were reparented into the transient package but still need their loose properties CPFUOed to new instances
		if (FPropertyBagRepository::Get().HasInstanceDataObject(InObject))
		{
			return true;
		}

		if (!IsInstanceDataObjectSupportEnabled())
		{
			return false;
		}

		//@todo FH: change to check trait when available or use config object
		const UClass* ObjClass = InObject->GetClass();
		if (!IsInstanceDataObjectSupportEnabledForClass(ObjClass))
		{
			return false;
		}

		// respect flags for disabling the generation of blueprint or prefab archetypes and/or their instances
		if (!bEnableIDOsForBlueprintArchetypes || !bEnableIDOsForBlueprintInstances)
		{
			if (const UObject* BlueprintGeneratedObject = GetBlueprintGeneratedObject(InObject))
			{
				const bool bIsArchetype = BlueprintGeneratedObject->GetClass()->GetDefaultObject(false) == BlueprintGeneratedObject;
				if (!bEnableIDOsForBlueprintArchetypes && bIsArchetype)
				{
					return false;
				}
				if (!bEnableIDOsForBlueprintInstances && !bIsArchetype)
				{
					return false;
				}
			}
		}
		
		return true;
	}

	static void OnInstanceDataObjectSupportChanged(IConsoleVariable*)
	{
		bEverEnabledIDOSupport = bEnableIDOSupport || (bEverEnabledIDOSupport && bEverCreatedIDO);
		bEverEnabledIDOSupportOnEveryObject = bEnableIDOSupportOnEveryObject || (bEverEnabledIDOSupportOnEveryObject && bEverCreatedIDO);

		// The reference token stream is dependent on the return value of IsInstanceDataObjectSupportEnabledForClass.
		TArray<UClass*> AllClasses;
		AllClasses.Add(UObject::StaticClass());
		GetDerivedClasses(UObject::StaticClass(), AllClasses);
		for (UClass* Class : AllClasses)
		{
			// Only re-assemble if it has been assembled because this can run before intrinsic schemas are declared.
			if (Class->HasAnyClassFlags(CLASS_TokenStreamAssembled))
			{
				Class->AssembleReferenceTokenStream(/*bForce*/ true);
			}
		}
	}

	bool CanCreatePropertyBagPlaceholderTypeForImportClass(const UClass* ClassKind, const UE::Serialization::Private::FImportTypeHierarchy* ImportTypeHierarchy)
	{
		bool bResult = false;

		if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObjectFeatureEnabled(UE::FPropertyBagRepository::EPlaceholderObjectFeature::DoNotRestrictMissingTypeImports))
		{
			return true;
		}

		if (ClassKind)
		{
			// @todo - Expand to other import types (e.g. non-prefab BPs) later; for now restricted to Verse class objects and entity prefabs.

			if (ClassKind->GetFName() == NAME_VerseClass)
			{
				bResult = true;
			}
			else if (ImportTypeHierarchy && (ClassKind->GetFName() == NAME_BlueprintGeneratedClass))
			{
				const TArray<UE::Serialization::Private::FTypeResource>& SuperTypes = ImportTypeHierarchy->GetSuperTypes();

				for (const UE::Serialization::Private::FTypeResource& SuperType : SuperTypes)
				{
					if (SuperType.TypeName == NAME_Entity)
					{
						bResult = true;
						break;
					}
				}
			}
		}

		return bResult;
	}

	bool CanCreatePropertyBagPlaceholderForType(const UClass* ClassKind, const UStruct* Type)
	{
		bool bResult = false;

		if (CanCreatePropertyBagPlaceholderTypeForImportClass(ClassKind, /*ImportTypeHierarchy*/nullptr))
		{
			bResult = true;
		}
		else
		{
			// See comment in CanCreatePropertyBagPlaceholderTypeForImportClass: for now we only
			// support BlueprintGeneratedClasses that are entity prefabs.
			if (Type && ClassKind && (ClassKind->GetFName() == NAME_BlueprintGeneratedClass))
			{
				for (const UStruct* Struct : Type->GetSuperStructIterator())
				{
					if (Struct->GetFName() == NAME_Entity)
					{
						bResult = true;
						break;
					}
				}
			}
		}

		return bResult;
	}

	static bool CanCreatePropertyBagPlaceholderSuperType(FName TypeName, FName ClassName)
	{
		bool bResult = false;

		if ((TypeName == NAME_Component) && (ClassName == NAME_VerseClass))
		{
			bResult = true;
		}
		else if ((TypeName == NAME_Entity) && (ClassName == NAME_Class || ClassName == NAME_VerseClass))
		{
			bResult = true;
		}

		return bResult;
	}

	bool CanCreatePropertyBagPlaceholderSuperType(const UE::Serialization::Private::FTypeResource& SuperType)
	{
		return CanCreatePropertyBagPlaceholderSuperType(SuperType.TypeName, SuperType.ClassName);
	}

	bool CanCreatePropertyBagPlaceholderSuperType(const UStruct* SuperType)
	{
		return CanCreatePropertyBagPlaceholderSuperType(SuperType->GetFName(), SuperType->GetClass()->GetFName());
	}

	bool IsClassOfInstanceDataObjectClass(const UStruct* Class)
	{
		return Class->IsA(UInstanceDataObjectClass::StaticClass()) || Class->IsA(UInstanceDataObjectStruct::StaticClass());
	}

	bool StructContainsLooseProperties(const UStruct* Struct)
	{
		return Struct->GetBoolMetaData(NAME_ContainsLoosePropertiesMetadata);
	}

	bool StructIsInstanceDataObjectStruct(const UStruct* Struct)
	{
		return Struct->GetBoolMetaData(NAME_IsInstanceDataObjectStruct);
	}

	template <typename T>
	static void CleanUpInstanceDataObjectTypeCache(TMap<FBlake3Hash, TSharedPtr<TCacheItem<T>>>& Cache)
	{
		if (Cache.Num() % 64 == 0)
		{
			for (auto It = Cache.CreateIterator(); It; ++It)
			{
				// If it's not notified, its construction is in progress, do not remove it.
				if (!It->Value->WeakPtr.IsValid() && It->Value->ReadyEvent.IsNotified())
				{
					It.RemoveCurrent();
				}
			}
		}
	}

	static UScriptStruct* CreateInstanceDataObjectStruct(const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames, UScriptStruct* OwnerStruct, UObject* Outer, const FGuid& Guid, const TCHAR* OriginalName);

	static UStruct* CreateInstanceDataObjectStructRec(const UClass* StructClass, UStruct* TemplateStruct, UObject* Outer, const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames);

	template <typename StructType>
	StructType* CreateInstanceDataObjectStructRec(UStruct* TemplateStruct, UObject* Outer, const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames)
	{
		return CastChecked<StructType>(CreateInstanceDataObjectStructRec(StructType::StaticClass(), TemplateStruct, Outer, PropertyTree, EnumNames));
	}

	UEnum* FindOrCreateInstanceDataObjectEnum(UEnum* TemplateEnum, UObject* Outer, const FProperty* Property, const FUnknownEnumNames* EnumNames)
	{
		if (!bEnableIDOUnknownEnums || !TemplateEnum || !EnumNames)
		{
			return TemplateEnum;
		}

		TArray<FName> UnknownNames;
		bool bHasFlags = false;

		// Use the original type name because the template may be a fallback enum or an IDO.
		FPropertyTypeName EnumTypeName = FindOriginalType(Property);
		if (EnumTypeName.IsEmpty())
		{
			FPropertyTypeNameBuilder Builder;
			Builder.AddPath(TemplateEnum);
			EnumTypeName = Builder.Build();
		}

		EnumNames->Find(EnumTypeName, UnknownNames, bHasFlags);
		if (UnknownNames.IsEmpty())
		{
			return TemplateEnum;
		}

		int64 MaxEnumValue = -1;
		int64 CombinedEnumValues = 0;
		TArray<TPair<FName, int64>> EnumValueNames;
		TStringBuilder<128> EnumName(InPlace, EnumTypeName.GetName());

		const auto MakeFullEnumName = [&EnumName, Form = TemplateEnum->GetCppForm()](FName Name) -> FName
		{
			if (Form == UEnum::ECppForm::Regular)
			{
				return Name;
			}
			return FName(WriteToString<128>(EnumName, TEXTVIEW("::"), Name));
		};

		const auto MakeNextEnumValue = [&MaxEnumValue, &CombinedEnumValues, bHasFlags]() -> int64
		{
			if (!bHasFlags)
			{
				return ++MaxEnumValue;
			}
			const int64 NextEnumValue = ~CombinedEnumValues & (CombinedEnumValues + 1);
			CombinedEnumValues |= NextEnumValue;
			return NextEnumValue;
		};

		// Copy existing values except for MAX.
		const bool bContainsExistingMax = TemplateEnum->ContainsExistingMax();
		for (int32 Index = 0, Count = TemplateEnum->NumEnums() - (bContainsExistingMax ? 1 : 0); Index < Count; ++Index)
		{
			FName EnumValueName = TemplateEnum->GetNameByIndex(Index);
			int64 EnumValue = TemplateEnum->GetValueByIndex(Index);
			EnumValueNames.Emplace(EnumValueName, EnumValue);
			MaxEnumValue = FMath::Max(MaxEnumValue, EnumValue);
			CombinedEnumValues |= EnumValue;
		}

		// Copy unknown names and assign values sequentially.
		for (FName UnknownName : UnknownNames)
		{
			EnumValueNames.Emplace(MakeFullEnumName(UnknownName), MakeNextEnumValue());
		}

		// Copy or create MAX with a new value.
		const FName MaxEnumName = bContainsExistingMax ? TemplateEnum->GetNameByIndex(TemplateEnum->NumEnums() - 1) : MakeFullEnumName("MAX");
		EnumValueNames.Emplace(MaxEnumName, bHasFlags ? CombinedEnumValues : MaxEnumValue);

		// Construct a key for the enum cache.
		FBlake3Hash Key;
		{
			FBlake3 KeyBuilder;
			AppendHash(KeyBuilder, EnumTypeName);
			for (const TPair<FName, int64>& Name : EnumValueNames)
			{
				AppendHash(KeyBuilder, Name.Key);
				KeyBuilder.Update(&Name.Value, sizeof(Name.Value));
			}
			KeyBuilder.Update(&bHasFlags, sizeof(bHasFlags));
			Key = KeyBuilder.Finalize();
		}

		// Check if a cached enum exists for this key.
		static TMap<FBlake3Hash, TSharedPtr<TCacheItem<UEnum>>> EnumCache;
		static FSharedMutex EnumCacheMutex;

		TSharedPtr<TCacheItem<UEnum>> Item;
		{
			TSharedLock Lock(EnumCacheMutex);
			Item = EnumCache.FindRef(Key);
		}

		if (UEnum* Enum = Item ? Item->Extract() : nullptr)
		{
			return Enum;
		}
		
		{
			TUniqueLock Lock(EnumCacheMutex);
			CleanUpInstanceDataObjectTypeCache(EnumCache);
			Item = EnumCache.FindRef(Key);
			if (UEnum* Enum = Item ? Item->Extract() : nullptr)
			{
				return Enum;
			}

			Item = EnumCache.Add(Key, MakeShared<TCacheItem<UEnum>>());
		}

		// Construct a transient type that impersonates the original type.
		const FName InstanceDataObjectName(WriteToString<128>(EnumName, TEXTVIEW("_InstanceDataObject")));
		UEnum* NewEnum = NewObject<UEnum>(Outer, MakeUniqueObjectName(Outer, UEnum::StaticClass(), InstanceDataObjectName));

		const EEnumFlags EnumFlags = bHasFlags ? EEnumFlags::Flags : EEnumFlags::None;
		// TODO: Detect out-of-bounds values and increase the size of the underlying type accordingly.
		const UEnum::EUnderlyingType UnderlyingType = UEnum::EUnderlyingType::int64;
		constexpr UEnum::EAddMaxKeyIfMissing AddMaxKeyIfMissing = UEnum::EAddMaxKeyIfMissing::No;
		NewEnum->SetEnums(EnumValueNames, TemplateEnum->GetCppForm(), UnderlyingType, EnumFlags, AddMaxKeyIfMissing);
		NewEnum->SetMetaData(*WriteToString<32>(NAME_OriginalType), *WriteToString<128>(EnumTypeName));

		Item->WeakPtr = NewEnum;
		Item->ReadyEvent.Notify();

		return NewEnum;
	}

	static FString UnmanglePropertyName(const FName MaybeMangledName, bool& bOutNameWasMangled)
	{
		FString Result = MaybeMangledName.ToString();
		if (Result.StartsWith(TEXTVIEW("__verse_0x")))
		{
			// chop "__verse_0x" (10 char) + CRC (8 char) + "_" (1 char)
			Result = Result.RightChop(19);
			bOutNameWasMangled = true;
		}
		else
		{
			bOutNameWasMangled = false;
		}
		return Result;
	}

	// recursively re-instances all structs contained by this property to include loose properties
	static void ConvertToInstanceDataObjectProperty(FProperty* Property, FPropertyTypeName PropertyType, UObject* Outer, const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames)
	{
		if (!Property->HasMetaData(NAME_DisplayName))
		{
			bool bNeedsDisplayName = false;
			FString DisplayName = UnmanglePropertyName(Property->GetFName(), bNeedsDisplayName);
			if (bNeedsDisplayName)
			{
				Property->SetMetaData(NAME_DisplayName, MoveTemp(DisplayName));
			}
		}

		if (!PropertyTree)
		{
			// all sub-properties are known so we can use the regular type
			//return;
		}

		if (FStructProperty* AsStructProperty = CastField<FStructProperty>(Property))
		{
			// Structs that use native or binary serialization cannot safely generate an IDO.
			if (bEnableIDOUnknownStructs && !AsStructProperty->Struct->UseNativeSerialization() && !(AsStructProperty->Struct->StructFlags & STRUCT_Immutable))
			{
				//@note: Transfer existing metadata over as we build the InstanceDataObject from the struct or it owners, if any, this is useful for testing purposes
				TStringBuilder<256> OriginalName;
				if (const FString* OriginalType = FindOriginalTypeName(AsStructProperty))
				{
					OriginalName << *OriginalType;
				}

				if (OriginalName.Len() == 0)
				{
					UE::FPropertyTypeNameBuilder OriginalNameBuilder;
					OriginalNameBuilder.AddPath(AsStructProperty->Struct);
					OriginalName << OriginalNameBuilder.Build();
				}

				FGuid StructGuid;
				if (const FName StructGuidName = PropertyType.GetParameterName(1); !StructGuidName.IsNone())
				{
					FGuid::Parse(StructGuidName.ToString(), StructGuid);
				}

				AsStructProperty->Struct = CreateInstanceDataObjectStruct(PropertyTree, EnumNames, AsStructProperty->Struct, Outer, StructGuid, *OriginalName);
				AsStructProperty->SetMetaData(NAME_OriginalType, *OriginalName);
				AsStructProperty->SetMetaData(NAME_PresentAsTypeMetadata, *OriginalName);
			}
		}
		else if (FByteProperty* AsByteProperty = CastField<FByteProperty>(Property))
		{
			AsByteProperty->Enum = FindOrCreateInstanceDataObjectEnum(AsByteProperty->Enum, Outer, Property, EnumNames);
		}
		else if (FEnumProperty* AsEnumProperty = CastField<FEnumProperty>(Property))
		{
			AsEnumProperty->SetEnumForImpersonation(FindOrCreateInstanceDataObjectEnum(AsEnumProperty->GetEnum(), Outer, Property, EnumNames));
		}
		else if (FArrayProperty* AsArrayProperty = CastField<FArrayProperty>(Property))
		{
			ConvertToInstanceDataObjectProperty(AsArrayProperty->Inner, PropertyType.GetParameter(0), Outer, PropertyTree, EnumNames);
		}
		else if (FSetProperty* AsSetProperty = CastField<FSetProperty>(Property))
		{
			ConvertToInstanceDataObjectProperty(AsSetProperty->ElementProp, PropertyType.GetParameter(0), Outer, PropertyTree, EnumNames);
		}
		else if (FMapProperty* AsMapProperty = CastField<FMapProperty>(Property))
		{
			const FPropertyPathNameTree* KeyTree = nullptr;
			const FPropertyPathNameTree* ValueTree = nullptr;
			if (PropertyTree)
			{
				FPropertyPathName Path;
				Path.Push({NAME_IDOMapKey});
				KeyTree = PropertyTree->Find(Path).GetSubTree();
				Path.Pop();
				Path.Push({NAME_IDOMapValue});
				ValueTree = PropertyTree->Find(Path).GetSubTree();
				Path.Pop();
			}

			ConvertToInstanceDataObjectProperty(AsMapProperty->KeyProp, PropertyType.GetParameter(0), Outer, KeyTree, EnumNames);
			ConvertToInstanceDataObjectProperty(AsMapProperty->ValueProp, PropertyType.GetParameter(1), Outer, ValueTree, EnumNames);
		}
		else if (FOptionalProperty* AsOptionalProperty = CastField<FOptionalProperty>(Property))
		{
			ConvertToInstanceDataObjectProperty(AsOptionalProperty->GetValueProperty(), PropertyType.GetParameter(0), Outer, PropertyTree, EnumNames);
		}
	}

	// recursively sets NAME_ContainsLoosePropertiesMetadata on all properties that contain loose properties
	static void TrySetContainsLoosePropertyMetadata(FProperty* Property)
	{
		const auto Helper = [](FProperty* Property, const FFieldVariant& Inner)
		{
			if (Inner.HasMetaData(NAME_ContainsLoosePropertiesMetadata))
			{
				Property->SetMetaData(NAME_ContainsLoosePropertiesMetadata, TEXT("True"));
			}
		};

		if (FStructProperty* AsStructProperty = CastField<FStructProperty>(Property))
		{
			Helper(AsStructProperty, AsStructProperty->Struct);
		}
		else if (FArrayProperty* AsArrayProperty = CastField<FArrayProperty>(Property))
		{
			TrySetContainsLoosePropertyMetadata(AsArrayProperty->Inner);
			Helper(AsArrayProperty, AsArrayProperty->Inner);
		}
		else if (FSetProperty* AsSetProperty = CastField<FSetProperty>(Property))
		{
			TrySetContainsLoosePropertyMetadata(AsSetProperty->ElementProp);
			Helper(AsSetProperty, AsSetProperty->ElementProp);
		}
		else if (FMapProperty* AsMapProperty = CastField<FMapProperty>(Property))
		{
			TrySetContainsLoosePropertyMetadata(AsMapProperty->KeyProp);
			Helper(AsMapProperty, AsMapProperty->KeyProp);
			TrySetContainsLoosePropertyMetadata(AsMapProperty->ValueProp);
			Helper(AsMapProperty, AsMapProperty->ValueProp);
		}
		else if (FOptionalProperty* AsOptionalProperty = CastField<FOptionalProperty>(Property))
		{
			TrySetContainsLoosePropertyMetadata(AsOptionalProperty->GetValueProperty());
			Helper(AsOptionalProperty, AsOptionalProperty->GetValueProperty());
		}

		if (Property->GetBoolMetaData(NAME_IsLooseMetadata) || Property->GetBoolMetaData(NAME_ContainsLoosePropertiesMetadata))
		{
			Property->GetOwnerStruct()->SetMetaData(NAME_ContainsLoosePropertiesMetadata, TEXT("True"));
		}
	}

	// recursively gives a property the metadata and flags of a loose property
	static void MarkPropertyAsLoose(FProperty* Property, EPropertyFlags PropertyFlags = CPF_None)
	{
		Property->SetMetaData(NAME_IsLooseMetadata, TEXT("True"));
		Property->SetPropertyFlags(CPF_Edit | CPF_EditConst | PropertyFlags);

		if (const FArrayProperty* AsArrayProperty = CastField<FArrayProperty>(Property))
		{
			// experimental override serialization of arrays requires certain flags be set on the inner property (it will assert otherwise)
			if (PropertyFlags & CPF_ExperimentalOverridableLogic)
			{
				PropertyFlags &= ~CPF_ExperimentalOverridableLogic;
				if (AsArrayProperty->Inner->IsA<FObjectProperty>())
				{
					PropertyFlags |= CPF_InstancedReference | CPF_PersistentInstance;
				}
			}

			MarkPropertyAsLoose(AsArrayProperty->Inner, PropertyFlags);
		}
		else if (const FSetProperty* AsSetProperty = CastField<FSetProperty>(Property))
		{
			MarkPropertyAsLoose(AsSetProperty->ElementProp);
		}
		else if (const FMapProperty* AsMapProperty = CastField<FMapProperty>(Property))
		{
			// experimental override serialization of maps requires certain flags to be set on the key property (it will assert otherwise)
			if (PropertyFlags & CPF_ExperimentalOverridableLogic)
			{
				PropertyFlags &= ~CPF_ExperimentalOverridableLogic;
				if (AsMapProperty->KeyProp->IsA<FObjectProperty>())
				{
					PropertyFlags |= CPF_InstancedReference | CPF_PersistentInstance;
				}
			}

			MarkPropertyAsLoose(AsMapProperty->KeyProp, PropertyFlags);
			MarkPropertyAsLoose(AsMapProperty->ValueProp);	// override serialization doesn't require any flags on the value property
		}
		else if (const FOptionalProperty* AsOptionalProperty = CastField<FOptionalProperty>(Property))
		{
			MarkPropertyAsLoose(AsOptionalProperty->GetValueProperty());
		}
		else if (FObjectProperty* AsObjectProperty = CastField<FObjectProperty>(Property))
		{
			// TObjectPtr is required by UHT and thus for serializing its TPS data
			AsObjectProperty->SetPropertyFlags(CPF_TObjectPtr);

			// also assign the property class to UObject because loose properties can't infer their PropertyClass from TPS data so we'll assume it's as lenient as possible
			AsObjectProperty->PropertyClass = UObject::StaticClass();
		}

		// struct sub-properties don't need to be handled. If the sub-properties are loose, this method will get called on them regardless when they're created.
	}

	bool IsPropertyLoose(TNotNull<const FProperty*> Property)
	{
		return IsInstanceDataObjectSupportEnabled() && Property->GetBoolMetaData(NAME_IsLooseMetadata);
	}

	bool IsSerializingUnknownData(const FArchive& Archive, const FProperty* InProperty /* = nullptr*/)
	{
		if (!IsInstanceDataObjectSupportEnabled())
		{
			return false;
		}

		bool bHasUnknownProperty = false;
		if (const FArchiveSerializedPropertyChain * PropertyChain = Archive.GetSerializedPropertyChain())
		{
			bHasUnknownProperty = PropertyChain->HasUnknownProperty();
		}
		if (!bHasUnknownProperty && InProperty)
		{
			bHasUnknownProperty = InProperty->HasMetaData(NAME_IsLooseMetadata);
		}

		return bHasUnknownProperty;
	}


	// creates a struct that doesn't contain unknown (loose) properties except those in sub-structs
	static UStruct* CreateInstanceDataObjectSuperHierarchy(const UClass* StructClass, UStruct* TemplateStruct, UObject* Outer, const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames, TSet<FPropertyPathName>& OutSuperPropertyPathsFromTree)
	{
		UStruct* Super = nullptr;
		if (StructClass->IsChildOf<UClass>())
		{
			// classes should inherrit from an ido created out of the template super. This way they can be used as archetypes during serialization
			UClass* TemplateSuper = TemplateStruct ? Cast<UClass>(TemplateStruct->GetSuperStruct()) : nullptr;
			if (bEnableFullIDOInherritance && TemplateSuper && TemplateSuper != UObject::StaticClass())
			{
				Super = CreateInstanceDataObjectSuperHierarchy(StructClass, TemplateSuper, Outer, PropertyTree, EnumNames, OutSuperPropertyPathsFromTree);
			}
			else
			{
				Super = UObject::StaticClass();
			}
		}

		if (TemplateStruct)
		{
			const FName SuperName(WriteToString<128>(TemplateStruct->GetName(), TEXTVIEW("_Super")));
			const UClass* SuperStructClass = StructClass->GetSuperClass();
			UStruct* Result = NewObject<UStruct>(Outer, SuperStructClass, MakeUniqueObjectName(nullptr, SuperStructClass, SuperName));
			Result->SetSuperStruct(Super);
			Result->SetMetaData(NAME_IsInstanceDataObjectStruct, TEXT("True"));
			Result->SetMetaData(NAME_TemplateStructName, *TemplateStruct->GetName());

			// Gather properties
			TArray<FProperty*> SuperProperties;
			EFieldIterationFlags ExcludeSuperFlags = (EFieldIterationFlags)((uint8)EFieldIterationFlags::Default & ~(uint8)EFieldIterationFlags::IncludeSuper);
			for (const FProperty* TemplateProperty : TFieldRange<FProperty>(TemplateStruct, ExcludeSuperFlags))
			{
				FProperty* SuperProperty = CastFieldChecked<FProperty>(FField::Duplicate(TemplateProperty, Result));
				SuperProperties.Add(SuperProperty);

				FField::CopyMetaData(TemplateProperty, SuperProperty);

				FPropertyTypeName Type(TemplateProperty);

				// Find the sub-tree containing unknown properties for this template property.
				const FPropertyPathNameTree* SubTree = nullptr;
				if (PropertyTree)
				{
					FPropertyPathName Path;
					Path.Push({ TemplateProperty->GetFName(), Type });
					if (FPropertyPathNameTree::FConstNode Node = PropertyTree->Find(Path))
					{
						SubTree = Node.GetSubTree();
						OutSuperPropertyPathsFromTree.Add(MoveTemp(Path));
					}
				}

				ConvertToInstanceDataObjectProperty(SuperProperty, Type, Outer, SubTree, EnumNames);
				TrySetContainsLoosePropertyMetadata(SuperProperty);
			}

			// AddCppProperty expects reverse property order for StaticLink to work correctly
			for (FProperty* Property : ReverseIterate(SuperProperties))
			{
				Result->AddCppProperty(Property);
			}
			Result->Bind();
			Result->StaticLink(/*bRelinkExistingProperties*/true);
			return Result;
		}

		return Super;
	}

	// constructs an InstanceDataObject struct by merging the properties in 
	static UStruct* CreateInstanceDataObjectStructRec(const UClass* StructClass, UStruct* TemplateStruct, UObject* Outer, const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames)
	{

		// The super of IDOs map 1-1 with the template struct both in it's properties and it's inherritance chain. 
		// It can however have unknown properties in it's sub-structs. structs with those will appear in the SuperPropertyPathsFromTree set
		TSet<FPropertyPathName> SuperPropertyPathsFromTree;
		UStruct* Super = CreateInstanceDataObjectSuperHierarchy(StructClass, TemplateStruct, Outer, PropertyTree, EnumNames, SuperPropertyPathsFromTree);
		
		const FName InstanceDataObjectName = (TemplateStruct) ? FName(WriteToString<128>(TemplateStruct->GetName(), TEXTVIEW("_InstanceDataObject"))) : FName(TEXTVIEW("InstanceDataObject"));
		UStruct* Result = NewObject<UStruct>(Outer, StructClass, MakeUniqueObjectName(Outer, StructClass, InstanceDataObjectName));
		Result->SetSuperStruct(Super);
		Result->SetMetaData(NAME_IsInstanceDataObjectStruct, TEXT("True"));
		if (TemplateStruct)
		{
			Result->SetMetaData(NAME_TemplateStructName, *TemplateStruct->GetName());
		}

		// inherit ContainsLooseProperties metadata
		if (Super && Super->GetBoolMetaData(NAME_ContainsLoosePropertiesMetadata))
		{
			Result->SetMetaData(NAME_ContainsLoosePropertiesMetadata, TEXT("True"));
		}

		TSet<FString> ExcludedLoosePropertyTypes = GetExcludedUnknownPropertyTypes();

		// Gather "loose" properties for child Struct
		TArray<FProperty*> LooseInstanceDataObjectProperties;
		if (PropertyTree)
		{
			for (FPropertyPathNameTree::FConstIterator It = PropertyTree->CreateConstIterator(); It; ++It)
			{
				FName Name = It.GetName();
				if (Name == NAME_InitializedValues || Name == NAME_SerializedValues)
				{
					// In rare cases, these hidden properties will get serialized even though they are transient.
					// Ignore them here since they are generated below.
					continue;
				}
				FPropertyTypeName Type = It.GetType();
				FPropertyPathName Path;
				Path.Push({Name, Type});
				if (!SuperPropertyPathsFromTree.Contains(Path))
				{
					// Construct a property from the type and try to use it to serialize the value.
					FField* Field = FField::TryConstruct(Type.GetName(), Result, Name);
					if (FProperty* Property = CastField<FProperty>(Field); Property && Property->LoadTypeName(Type, It.GetNode().GetTag()))
					{
						if (ExcludedLoosePropertyTypes.Contains(Property->GetClass()->GetName()))
						{
							// skip loose types that have been explicitly excluded from IDOs
							continue;
						}
						EPropertyFlags PropertyFlags = CPF_None;
						if (const FPropertyTag* PropertyTag = It.GetNode().GetTag())
						{
							if (PropertyTag->bExperimentalOverridableLogic)
							{
								PropertyFlags |= CPF_ExperimentalOverridableLogic;
							}
						}
						ConvertToInstanceDataObjectProperty(Property, Type, Outer, It.GetNode().GetSubTree(), EnumNames);
						MarkPropertyAsLoose(Property, PropertyFlags);	// note: make sure not to mark until AFTER conversion, as this can mutate property flags on nested struct fields
						TrySetContainsLoosePropertyMetadata(Property);
						LooseInstanceDataObjectProperties.Add(Property);
						continue;
					}
					delete Field;
				}
			}
		}

		// Add hidden byte array properties to record whether its sibling properties were initialized or set by serialization.
		FByteProperty* InitializedValuesProperty = CastFieldChecked<FByteProperty>(FByteProperty::Construct(Result, NAME_InitializedValues));
		FByteProperty* SerializedValuesProperty = CastFieldChecked<FByteProperty>(FByteProperty::Construct(Result, NAME_SerializedValues));
		{
			InitializedValuesProperty->SetPropertyFlags(CPF_Transient | CPF_EditorOnly | CPF_SkipSerialization | CPF_NativeAccessSpecifierPrivate);
			SerializedValuesProperty->SetPropertyFlags(CPF_Transient | CPF_EditorOnly | CPF_SkipSerialization | CPF_NativeAccessSpecifierPrivate);
			Result->AddCppProperty(InitializedValuesProperty);
			Result->AddCppProperty(SerializedValuesProperty);
		}

		// Store generated properties to avoid scanning every property to find it when it is needed.
		if (UInstanceDataObjectClass* IdoClass = Cast<UInstanceDataObjectClass>(Result))
		{
			IdoClass->InitializedValuesProperty = InitializedValuesProperty;
			IdoClass->SerializedValuesProperty = SerializedValuesProperty;
		}
		else if (UInstanceDataObjectStruct* IdoStruct = Cast<UInstanceDataObjectStruct>(Result))
		{
			IdoStruct->InitializedValuesProperty = InitializedValuesProperty;
			IdoStruct->SerializedValuesProperty = SerializedValuesProperty;
		}

		// AddCppProperty expects reverse property order for StaticLink to work correctly
		for (FProperty* Property : ReverseIterate(LooseInstanceDataObjectProperties))
		{
			Result->AddCppProperty(Property);
		}

		// Count properties and set the size of the array of flags.
		int32 PropertyCount = -2; // Start at -2 to exclude the two hidden properties.
		for (TFieldIterator<FProperty> It(Result); It; ++It)
		{
			PropertyCount += It->ArrayDim;
		}
		const int32 PropertyCountBytes = FMath::Max(1, FMath::DivideAndRoundUp(PropertyCount, 8));
		InitializedValuesProperty->ArrayDim = PropertyCountBytes;
		SerializedValuesProperty->ArrayDim = PropertyCountBytes;

		Result->Bind();
		Result->StaticLink(/*bRelinkExistingProperties*/true);
		checkf(PropertyCount <= Result->TotalFieldCount,
			TEXT("Type %s had %d properties after linking when at least %d are expected."),
			*Result->GetPathName(), Result->TotalFieldCount, PropertyCount);
		return Result;
	}

	struct FSerializingDefaultsScope
	{
		UE_NONCOPYABLE(FSerializingDefaultsScope);

		inline FSerializingDefaultsScope(FArchive& Ar, const UObject* Object)
		{
			if (Object->HasAnyFlags(RF_ClassDefaultObject))
			{
				Archive = &Ar;
				Archive->StartSerializingDefaults();
			}
		}

		inline ~FSerializingDefaultsScope()
		{
			if (Archive)
			{
				Archive->StopSerializingDefaults();
			}
		}

		FArchive* Archive = nullptr;
	};

	TArray<uint8> SaveTaggedProperties(const UObject* Source)
	{
		FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();

		// Track only initialized properties when copying. This is required to skip uninitialized properties
		// during saving and to mark initialized properties during loading.
		const bool bIsCDO = Source->HasAnyFlags(RF_ClassDefaultObject);
		TGuardValue<bool> ScopedTrackInitializedProperties(SerializeContext->bTrackInitializedProperties, !bIsCDO);
		TGuardValue<bool> ScopedTrackSerializedProperties(SerializeContext->bTrackSerializedProperties, false);
		TGuardValue<bool> ScopedTrackUnknownProperties(SerializeContext->bTrackUnknownProperties, false);
		TGuardValue<bool> ScopedTrackUnknownEnumNames(SerializeContext->bTrackUnknownEnumNames, false);
		TGuardValue<bool> ImpersonatePropertiesScope(SerializeContext->bImpersonateProperties, IsInstanceDataObject(Source));

		TArray<uint8> Data;
		FObjectWriter Writer(Data);
		Writer.ArNoDelta = true;
		FSerializingDefaultsScope WriterDefaultsScope(Writer, Source);
		Source->GetClass()->SerializeTaggedProperties(Writer, (uint8*)Source, Source->GetClass(), nullptr);

		return Data;
	}

	void LoadTaggedProperties(const TArray<uint8>& Source, UObject* Dest)
	{
		FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();

		// Track only initialized properties when copying. This is required to skip uninitialized properties
		// during saving and to mark initialized properties during loading.
		const bool bIsCDO = Dest->HasAnyFlags(RF_ClassDefaultObject);
		TGuardValue<bool> ScopedTrackInitializedProperties(SerializeContext->bTrackInitializedProperties, !bIsCDO);
		TGuardValue<bool> ScopedTrackSerializedProperties(SerializeContext->bTrackSerializedProperties, false);
		TGuardValue<bool> ScopedTrackUnknownProperties(SerializeContext->bTrackUnknownProperties, false);
		TGuardValue<bool> ScopedTrackUnknownEnumNames(SerializeContext->bTrackUnknownEnumNames, false);
		TGuardValue<bool> ImpersonatePropertiesScope(SerializeContext->bImpersonateProperties, IsInstanceDataObject(Dest));

		FObjectReader Reader(Source);
		Reader.ArMergeOverrides = true;
		Reader.ArPreserveArrayElements = true;
		FSerializingDefaultsScope ReaderDefaultsScope(Reader, Dest);
		Dest->GetClass()->SerializeTaggedProperties(Reader, (uint8*)Dest, Dest->GetClass(), nullptr);
	}

	void CopyTaggedProperties(const UObject* Source, UObject* Dest)
	{
		LoadTaggedProperties(SaveTaggedProperties(Source), Dest);
	}

	UObject* GetTemplateForInstanceDataObject(const UObject* Instance, const UObject* InstanceArchetype, const UClass* InstanceDataObjectClass)
	{
		if (IsInstanceDataObjectArchetypeChainEnabled() && Instance && InstanceArchetype && InstanceDataObjectClass)
		{
			if (InstanceArchetype == Instance->GetClass()->GetDefaultObject(false))
			{
				// if the archetype is a CDO, we can simply use the IDO's CDO because it's data should match.
				return InstanceDataObjectClass->GetDefaultObject(false);
			}
			else
			{
				// attempt to find/create an IDO for the archetype. Unfortunately if there's unkown properties involved, the types may not match.
				UObject* ArchetypeIdo = UE::CreateInstanceDataObject(const_cast<UObject*>(InstanceArchetype));
				if (ArchetypeIdo && ArchetypeIdo->GetClass() && ArchetypeIdo->IsA(InstanceDataObjectClass))
				{
					return ArchetypeIdo;
				}
				else if (InstanceArchetype->GetArchetype())
				{
					// we need a archetype that is the same type as the IDO but has the same data as the owner's archetype
					FStaticConstructObjectParameters TemplateParams(InstanceDataObjectClass);
					TemplateParams.SetFlags |= EObjectFlags::RF_ArchetypeObject;
					TemplateParams.Outer = GetTransientPackage();
					UObject* Result = StaticConstructObject_Internal(TemplateParams);
					CopyTaggedProperties(InstanceArchetype, Result);
					return Result;
				}
			}
		}
		return nullptr;
	}

	static void SetClassFlags(UClass* IDOClass, const UClass* OwnerClass)
	{
		// always set
		IDOClass->AssembleReferenceTokenStream();
		IDOClass->ClassFlags |= CLASS_NotPlaceable | CLASS_Hidden | CLASS_HideDropDown;
		
		// copy flags from OwnerClass
		IDOClass->ClassFlags |= OwnerClass->ClassFlags & (
			CLASS_EditInlineNew | CLASS_CollapseCategories | CLASS_Const | CLASS_CompiledFromBlueprint | CLASS_HasInstancedReference);
	}

	static void TrackDefaultInitializedPropertiesInCDO(UObject* CDO, UClass* Class, const UVerseClass* OwnerClass)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TrackDefaultInitializedPropertiesInCDO)

		const UClass* OwnerSuperClass = OwnerClass->GetSuperClass();
		if (OwnerSuperClass)
		{
			if (const UVerseClass* VerseSuperClass = Cast<UVerseClass>(OwnerSuperClass))
			{
				TrackDefaultInitializedPropertiesInCDO(CDO, Class, VerseSuperClass);
			}
		}

		// If a property has been initialized by the Verse class' CDO then we consider everything
		// under it (i.e., any sub-objects) as initialized.
		for (const TFieldPath<FProperty>& OwnerFieldPath : OwnerClass->PropertiesWrittenByInitCDO)
		{
			FProperty* OwnerProperty = OwnerFieldPath.Get();

			if (FProperty* Property = Class->FindPropertyByName(OwnerProperty->GetFName()))
			{
				if (Property->HasAnyPropertyFlags(CPF_RequiredParm))
				{
					UE::FInitializedPropertyValueState(CDO).Set(Property);
				}

				// Keep track of visited property-parent pairs to avoid referencing cycles.
				TSet<TTuple<const FProperty*, void*>> VisitedPropParents;

				FPropertyVisitorPath Path;
				FPropertyVisitorData VisitorData(CDO, /*ParentStructData*/CDO);
				FPropertyVisitorContext VisitorContext(Path, VisitorData, FPropertyVisitorContext::EScope::All);
				PropertyVisitorHelpers::VisitProperty(Class, Property, VisitorContext, [CDO, &VisitedPropParents](const FPropertyVisitorContext& Context) -> EPropertyVisitorControlFlow
				{
					const FPropertyVisitorPath& PropertyPath = Context.Path;
					const FPropertyVisitorData& Data = Context.Data;
					const FProperty* Property = PropertyPath.Top().Property;
					void* Parent = Data.ParentStructData;
					TTuple<const FProperty*, void*> PropParent(Property, Parent);

					if (!Property || VisitedPropParents.Contains(PropParent))
					{
						return EPropertyVisitorControlFlow::StepOver;
					}

					bool bIsInCDO = true;

					const UStruct* ParentType = PropertyPath.Top().ParentStructType;

					if (ParentType && ParentType->IsChildOf<UObject>())
					{
						const UObject* ParentObject = (const UObject*)Parent;

						if (ParentObject && (ParentObject != CDO))
						{
							bIsInCDO = ParentObject->IsInOuter(CDO);
						}
					}

					// It is possible for the property and parent types to differ during re-instancing
					// when a new CDO is created. Skip tracking in this case.
					if (bIsInCDO && ParentType && Property->HasAnyPropertyFlags(CPF_RequiredParm) && ParentType->IsChildOf(Property->GetOwnerStruct()))
					{
						UE::FInitializedPropertyValueState(ParentType, Parent).Set(Property);
					}

					VisitedPropParents.Add(PropParent);

					if (bIsInCDO)
					{
						return EPropertyVisitorControlFlow::StepInto;
					}
					else
					{
						return EPropertyVisitorControlFlow::StepOver;
					}
				});
			}
		}
	}

	UClass* CreateInstanceDataObjectClass(const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames, UClass* OwnerClass, UObject* Outer)
	{
		PropertyTree = bEnableIDOUnknownProperties ? PropertyTree : nullptr;

		FBlake3Hash Key;
		{
			FBlake3 KeyBuilder;
			KeyBuilder.Update(MakeMemoryView(OwnerClass->GetSchemaHash(/*bSkipEditorOnly*/ false).GetBytes()));

			// Hash the index and serial number of the CDO because they will change if it is reinstanced.
			// The schema hash excludes modifications made by constructors, and those will of course only be run on construction.
			const UObject* DefaultObject = OwnerClass->GetDefaultObject();
			const int32 DefaultIndex = GUObjectArray.ObjectToIndex(DefaultObject);
			const int32 DefaultSerial = GUObjectArray.AllocateSerialNumber(DefaultIndex);
			KeyBuilder.Update(&DefaultIndex, sizeof(DefaultIndex));
			KeyBuilder.Update(&DefaultSerial, sizeof(DefaultSerial));

			if (PropertyTree)
			{
				AppendHash(KeyBuilder, *PropertyTree);
			}
			if (EnumNames)
			{
				AppendHash(KeyBuilder, *EnumNames);
			}
			Key = KeyBuilder.Finalize();
		}

		TSharedPtr<TCacheItem<UInstanceDataObjectClass>> Item;
		{
			TSharedLock Lock(IDOClassCacheMutex);
			Item = IDOClassCache.FindRef(Key);
		}

		if (UClass* Class = Item ? Item->Extract() : nullptr)
		{
			return Class;
		}

		{
			TUniqueLock Lock(IDOClassCacheMutex);

			CleanUpInstanceDataObjectTypeCache(IDOClassCache);

			Item = IDOClassCache.FindRef(Key);
			if (UClass* Class = Item ? Item->Extract() : nullptr)
			{
				return Class;
			}

			Item = IDOClassCache.Add(Key, MakeShared<TCacheItem<UInstanceDataObjectClass>>());
		}

		UInstanceDataObjectClass* NewClass = CreateInstanceDataObjectStructRec<UInstanceDataObjectClass>(OwnerClass, Outer, PropertyTree, EnumNames);
		if (const FString& DisplayName = OwnerClass->GetMetaData(NAME_DisplayName); !DisplayName.IsEmpty())
		{
			NewClass->SetMetaData(NAME_DisplayName, *DisplayName);
		}

		SetClassFlags(NewClass, OwnerClass);

		CopyTaggedProperties(OwnerClass->GetDefaultObject(), NewClass->GetDefaultObject());

		// We need to copy the initialization state of the owner's properties for properties initialized in Verse code:
		// e.g.: MyStruct<public>:test_struct = test_struct{ MyInt := 1 }
		// This is necessary for Verse struct properties, because struct types do not store their initialization state 
		// in any way outside of IDOs (in contrast to classes which use an annotation system).
		// NOTE: if IDOs become the only system for tracking initialization state then we should deprecate the use of:
		// TrackDefaultInitializedProperties in UVerseClass.
		if (const UVerseClass* OwnerVerseClass = Cast<UVerseClass>(OwnerClass))
		{
			TrackDefaultInitializedPropertiesInCDO(NewClass->GetDefaultObject(), NewClass, OwnerVerseClass);
		}

		Item->WeakPtr = NewClass;
		Item->ReadyEvent.Notify();

		return NewClass;
	}

	UScriptStruct* CreateInstanceDataObjectStruct(const FPropertyPathNameTree* PropertyTree, const FUnknownEnumNames* EnumNames, UScriptStruct* OwnerStruct, UObject* Outer, const FGuid& Guid, const TCHAR* OriginalName)
	{
		FBlake3Hash Key;
		{
			FBlake3 KeyBuilder;
			KeyBuilder.Update(MakeMemoryView(OwnerStruct->GetSchemaHash(/*bSkipEditorOnly*/ false).GetBytes()));
			KeyBuilder.Update(&Guid, sizeof(Guid));
			KeyBuilder.Update(MakeMemoryView(WriteToUtf8String<256>(OriginalName)));
			if (PropertyTree)
			{
				AppendHash(KeyBuilder, *PropertyTree);
			}
			if (EnumNames)
			{
				AppendHash(KeyBuilder, *EnumNames);
			}
			Key = KeyBuilder.Finalize();
		}

		static TMap<FBlake3Hash, TSharedPtr<TCacheItem<UInstanceDataObjectStruct>>> StructCache;
		static FSharedMutex StructCacheMutex;
		TSharedPtr<TCacheItem<UInstanceDataObjectStruct>> Item;
		{
			TSharedLock Lock(StructCacheMutex);
			Item = StructCache.FindRef(Key);
		}

		if (UScriptStruct* Struct = Item ? Item->Extract() : nullptr)
		{
			return Struct;
		}

		{
			TUniqueLock Lock(StructCacheMutex);

			CleanUpInstanceDataObjectTypeCache(StructCache);

			Item = StructCache.FindRef(Key);
			if (UScriptStruct* Struct = Item ? Item->Extract() : nullptr)
			{
				return Struct;
			}

			Item = StructCache.Add(Key, MakeShared<TCacheItem<UInstanceDataObjectStruct>>());
		}

		UInstanceDataObjectStruct* NewStruct = CreateInstanceDataObjectStructRec<UInstanceDataObjectStruct>(OwnerStruct, Outer, PropertyTree, EnumNames);
		NewStruct->Guid = Guid;
		NewStruct->SetMetaData(NAME_OriginalType, OriginalName);
		NewStruct->SetMetaData(NAME_PresentAsTypeMetadata, OriginalName);

		Item->WeakPtr = NewStruct;
		Item->ReadyEvent.Notify();

		return NewStruct;
	}

	static const FByteProperty* FindSerializedValuesProperty(const UStruct* Struct)
	{
		if (const UInstanceDataObjectClass* IdoClass = Cast<UInstanceDataObjectClass>(Struct))
		{
			return IdoClass->SerializedValuesProperty;
		}
		if (const UInstanceDataObjectStruct* IdoStruct = Cast<UInstanceDataObjectStruct>(Struct))
		{
			return IdoStruct->SerializedValuesProperty;
		}
		return CastField<FByteProperty>(Struct->FindPropertyByName(NAME_SerializedValues));
	}

bool IsInstanceDataObject(const UObject* Object)
{
	return Object && Object->GetClass()->UObject::IsA(UInstanceDataObjectClass::StaticClass());
}

UObject* CreateInstanceDataObject(UObject* Owner)
{
	// If an Ido already exists, skip the uneeded serialization and just return it
	if (UObject* Found = FPropertyBagRepository::Get().FindInstanceDataObject(Owner))
	{
		return Found;
	}

	TArray<uint8> OwnerData;

	FObjectWriter Writer(OwnerData);
	Owner->SerializeScriptProperties(Writer);

	FObjectReader Reader(OwnerData);
	Reader.ArMergeOverrides = true;
	Reader.ArPreserveArrayElements = true;
	return CreateInstanceDataObject(Owner, Reader, 0, Reader.TotalSize());
}

UObject* CreateInstanceDataObject(UObject* Owner, FArchive& Ar, int64 StartOffset, int64 EndOffset)
{
	bEverCreatedIDO = true;
	return FPropertyBagRepository::Get().CreateInstanceDataObject(Owner, Ar, StartOffset, EndOffset);
}

UObject* ResolveInstanceDataObject(UObject* Object)
{
	UObject* InstanceDataObject = FPropertyBagRepository::Get().FindInstanceDataObject(Object);
	return InstanceDataObject ? InstanceDataObject : Object;
}

} // UE

#endif // WITH_EDITORONLY_DATA
