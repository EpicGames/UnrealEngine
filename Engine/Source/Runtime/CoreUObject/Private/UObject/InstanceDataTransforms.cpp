// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/InstanceDataTransforms.h"
#if !UE_BUILD_SHIPPING_WITH_EDITOR && WITH_EDITORONLY_DATA 
#include "UObject/PropertyPathNameTree.h"
#include "UObject/PropertyStateTracking.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/PropertyPathFunctions.h"
#include "UObject/UObjectThreadContext.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "Misc/StringOutputDevice.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/OverrideEventHelper.h"
#include "UObject/OverridableManager.h"
#include "UObject/PropertyHelper.h"
#include "UObject/Package.h"
#include "UObject/UObjectSerializeContext.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "HAL/FileManager.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Misc/PackageName.h"
#include "VerseVM/VVMNames.h"
#include "HAL/IConsoleManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogInstanceDataTransforms, Display, Display);

// registers a method that can be looked up by a string name in FInstanceDataTransforms::RegisteredOperations
#define REGISTER_IDT_OPERATION(FunctionName) \
UE::Private::FRegesterInstanceDataTransformHelper _Registered##FunctionName{TEXT(#FunctionName), FunctionName}; \

static bool bEnableInstanceDataTransformSerialization = false;
static FAutoConsoleVariableRef CVarEnableInstanceDataTransformSerialization(
	TEXT("IDO.EnableInstanceDataTransformSerialization"),
	bEnableInstanceDataTransformSerialization,
	TEXT("Enable serialization of Instance Data Transforms"));

namespace UE::Private
{

	FString InstanceDataTransformDirectory(const FTopLevelAssetPath& ClassOrStructPath)
	{
		FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(ClassOrStructPath.GetPackageName());
		if (PackagePath.IsMountedPath())
		{
			PackagePath.SetHeaderExtension(EPackageExtension::Asset); // silence missing extension warnings
			FString Directory = FPaths::GetPath(PackagePath.GetLocalFullPath());
			if (int32 Found = Directory.Find(TEXT("Content")); Found != INDEX_NONE)
			{
				Directory.LeftInline(Found);
			}
			Directory /= TEXT("Config");

			return Directory / TEXT("InstanceDataTransforms.json");
		}
		return {};
	}

	struct FRegesterInstanceDataTransformHelper
	{
		FRegesterInstanceDataTransformHelper(FStringView Name, FInstanceDataTransforms::FTransformFunction Function)
		{
			FInstanceDataTransforms::Get().RegisterOperation(Name, Function);
		}
	};

	TArray<uint8> WriteToBuffer(UObject* Owner, const FPropertyPathNameResolver& Src)
	{
		const FProperty* SourceProperty = Src.Value.Property;
		const void* SourceContainer = Src.Value.Container;
		int32 SourceIndex = Src.Value.ArrayIndex;
		const void* SourceData = SourceProperty->ContainerPtrToValuePtr<void>(SourceContainer, SourceIndex == INDEX_NONE ? 0 : SourceIndex);

		TArray<uint8> Buffer;

		FObjectWriter ObjectWriter(Buffer); // using FObjectWriter so that object properties will serialize correctly
		FStructuredArchiveFromArchive StructuredWriter(ObjectWriter);

		// set up all the flags to act as if we're serializing all of 'Owner' and the TPS is about to write the Src data.
		FScopedObjectSerializeContext SerializeContext(Owner, ObjectWriter, Src.Path);

		// enable overridable serialization
		if (FOverriddenPropertySet* OverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(Src.Object))
		{
			FOverridableSerializationLogic::Enable(OverriddenProperties);
			for (const FEditPropertyChain::TDoubleLinkedListNode* Itr = Src.EventChain.GetHead(); Itr != Src.EventChain.GetTail(); Itr = Itr->GetNextNode())
			{
				ObjectWriter.PushSerializedProperty(Itr->GetValue(), Itr->GetValue()->IsEditorOnlyProperty());
			}
		}

		// todo: handle static arrays
		FPropertyTag WriteTag(const_cast<FProperty*>(SourceProperty), SourceIndex, (uint8*)SourceData);
		StructuredWriter.GetSlot() << WriteTag;
		WriteTag.SerializeTaggedProperty(StructuredWriter.GetSlot(), const_cast<FProperty*>(SourceProperty), (uint8*)SourceData, (uint8*)SourceData);
		if (FOverriddenPropertySet* OverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(Src.Object))
		{
			FOverridableSerializationLogic::Disable();
		}

		return Buffer;
	}

	bool ReadFromBuffer(UObject* Owner, const FPropertyPathNameResolver& Dst, TArray<uint8> Buffer, bool bAllowNoConvert)
	{
		bool bResult = false;

		FProperty* DestinationProperty = const_cast<FProperty*>(Dst.Value.Property);
		void* DestinationContainer = Dst.Value.Container;
		int32 DestinationIndex = Dst.Value.ArrayIndex;
		void* DestinationData = DestinationProperty->ContainerPtrToValuePtr<void>(DestinationContainer, DestinationIndex == INDEX_NONE ? 0 : DestinationIndex);
		
		FObjectReader ObjectReader(Buffer);
		FStructuredArchiveFromArchive StructuredReader(ObjectReader);

		// set up all the flags to act as if we're serializing all of 'Owner' and the TPS is about to read the Dst data.
		FScopedObjectSerializeContext ScopedSerializeContext(Owner, ObjectReader, Dst.Path);

		// enable overridable serialization
		if (FOverriddenPropertySet* OverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(Dst.Object))
		{
			FOverridableSerializationLogic::Enable(OverriddenProperties);
			for (const FEditPropertyChain::TDoubleLinkedListNode* Itr = Dst.EventChain.GetHead(); Itr != Dst.EventChain.GetTail(); Itr = Itr->GetNextNode())
			{
				ObjectReader.PushSerializedProperty(Itr->GetValue(), Itr->GetValue()->IsEditorOnlyProperty());
			}
		}

		FPropertyTag ReadTag;
		StructuredReader.GetSlot() << ReadTag;
		ReadTag.SetProperty(DestinationProperty);
		ReadTag.ArrayIndex = DestinationIndex;
		ReadTag.Name = DestinationProperty->GetFName();

		switch (DestinationProperty->ConvertFromType(ReadTag, StructuredReader.GetSlot(), (uint8*)DestinationContainer, Dst.Value.Struct, (uint8*)DestinationContainer))
		{
		case EConvertFromTypeResult::UseSerializeItem:
			if (bAllowNoConvert)
			{
				ReadTag.SerializeTaggedProperty(StructuredReader.GetSlot(), DestinationProperty, (uint8*)DestinationData, (uint8*)DestinationData);
				bResult = true;
			}
			break;
		case EConvertFromTypeResult::Serialized:
			bResult = true;
			break;
		case EConvertFromTypeResult::CannotConvert:
			break;
		case EConvertFromTypeResult::Converted:
			bResult = true;
			break;
		}

		if (FOverriddenPropertySet* OverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(Dst.Object))
		{
			FOverridableSerializationLogic::Disable();
		}

		return bResult;
	}

	// TODO: this method allows dangerous and narrowing conversions. That's fine for now because the user explicitly asked to do so but in the future it could be good
	// to include a flag in the RedirectProperty params that specifies weather the user is ok with narrowing conversions
	bool TryCopyOrConvert(const FPropertyPathNameResolver& Src, const FPropertyPathNameResolver& Dst)
	{
		const FProperty* SourceProperty = Src.Value.Property;
		const void* SourceContainer = Src.Value.Container;
		int32 SourceIndex = Src.Value.ArrayIndex;
		FProperty* DestinationProperty = const_cast<FProperty*>(Dst.Value.Property);
		void* DestinationContainer = Dst.Value.Container;
		int32 DestinationIndex = Dst.Value.ArrayIndex;

		const void* SourceData = SourceProperty->ContainerPtrToValuePtr<void>(SourceContainer, SourceIndex == INDEX_NONE ? 0 : SourceIndex);
		void* DestinationData = DestinationProperty->ContainerPtrToValuePtr<void>(DestinationContainer, DestinationIndex == INDEX_NONE ? 0 : DestinationIndex);

		UObject* Owner = Dst.Object;
		if (const UObject* Instance = UE::FPropertyBagRepository::Get().FindInstanceForDataObject(Dst.Object))
		{
			Owner = const_cast<UObject*>(Instance);
		}

		// attempt regular TPS conversion
		TArray<uint8> Buffer = WriteToBuffer(Owner, Src);
		bool bResult = ReadFromBuffer(Owner, Dst, Buffer, SourceProperty->GetID() == DestinationProperty->GetID());

		if (!bResult)
		{
			bool bTryTextSerialize = false;

			const auto IsStringType = [](const FProperty* Property)
				{
					static FName VerseStringName = "VerseStringProperty";
					return Property->IsA<FStrProperty>() || Property->IsA<FTextProperty>() || Property->IsA<FNameProperty>() || Property->GetID() == VerseStringName;
				};

			if (IsStringType(SourceProperty) || IsStringType(DestinationProperty))
			{
				// if either property is a string, text, or name, use text serialization
				bTryTextSerialize = true;
			}
			else if (const FStructProperty* SourceAsStructProperty = CastField<FStructProperty>(SourceProperty))
			{
				if (FStructProperty* DestinationAsStructProperty = CastField<FStructProperty>(DestinationProperty))
				{
					if (!SourceAsStructProperty->Struct->UseNativeSerialization() && !DestinationAsStructProperty->Struct->UseNativeSerialization())
					{
						// attempt to text serialize structs since ConvertFromType doesn't support them usually
						bTryTextSerialize = true;
					}
				}
			}

			// use ExportText_Direct and ImportText_Direct
			if (bTryTextSerialize)
			{
				FString StrBuffer;
				SourceProperty->ExportText_Direct(StrBuffer, SourceData, nullptr, nullptr, PPF_None);
				FStringOutputDevice ErrorOutput;
				DestinationProperty->ImportText_Direct(*StrBuffer, DestinationData, nullptr, PPF_None, &ErrorOutput);
				bResult = ErrorOutput.IsEmpty();
			}
		}

		if (bResult)
		{
			UE::FInitializedPropertyValueState(Dst.Value.Struct, DestinationContainer).Set(DestinationProperty, DestinationIndex);
			for (const FPropertyValueInContainer& Value : Dst.ContainerValues)
			{
				if (Value.IsDynamicContainerElement())
				{
					// elements of dynamic containers (like TArray) don't store Initialized property value state
					continue;
				}
				UE::FInitializedPropertyValueState(Value.Struct, Value.Container).Set(Value.Property, Value.ArrayIndex);
			}
		}

		return bResult;
	}

	bool InferMisingTypeInfo(FPropertyTypeName& Type, UStruct*& CurrentStruct)
	{
		// fix structs that are missing their guids
		if (Type.GetName() == NAME_StructProperty && Type.GetParameterCount() == 1)
		{
			const UE::FPropertyTypeName& StructType = Type.GetParameter(0);
			if (StructType.GetParameterCount() == 1)
			{
				FName StructName = StructType.GetName();
				FName PackageName = StructType.GetParameter(0).GetName();
				FTopLevelAssetPath StructAssetPath(PackageName, StructName);
				if (UScriptStruct* Struct = Cast<UScriptStruct>(StaticFindObject(UStruct::StaticClass(), StructAssetPath)))
				{
					// recompile the type with the guid included
					FPropertyTypeNameBuilder Builder;
					Builder.AddName(NAME_StructProperty);
					Builder.BeginParameters();
					Builder.AddPath(Struct);
					Builder.AddGuid(Struct->GetCustomGuid());
					Builder.EndParameters();
					Type = Builder.Build();
					CurrentStruct = Struct;
					return true;
				}
			}
		}
		return false;
	}

	void InferMissingPathInfo(FPropertyPathName& Path, UStruct* CurrentStruct)
	{
		for (int32 SegmentIndex = 0; SegmentIndex < Path.GetSegmentCount(); ++SegmentIndex)
		{
			FPropertyPathNameSegment Segment = Path.GetSegment(SegmentIndex);
			bool bUpdateSegment = false;
			if (FPackageName::IsVersePackage(CurrentStruct->GetPackage()->GetName()))
			{
				FString PropertyName = Segment.Name.ToString();
				if (!PropertyName.StartsWith(TEXT("__verse_0x")))
				{
					PropertyName = ::Verse::Names::Private::MangleCasedName(PropertyName, PropertyName);
					Segment.Name = FName(PropertyName);
					bUpdateSegment = true;
				}
			}
			bUpdateSegment |= InferMisingTypeInfo(Segment.Type, CurrentStruct);

			if (bUpdateSegment)
			{
				Path.SetSegment(SegmentIndex, Segment);
			}
		}
	}

	void FlattenPropertyTree(const FPropertyPathNameTree& UnknownPropertyTree, TArray<FPropertyPathName>& OutPaths, FPropertyPathName& Path)
	{
		for (FPropertyPathNameTree::FConstIterator Itr = UnknownPropertyTree.CreateConstIterator(); Itr; ++Itr)
		{
			Path.Push(FPropertyPathNameSegment{ Itr.GetName(), Itr.GetType() });
			OutPaths.Push(Path);
			if (const FPropertyPathNameTree* SubTree = Itr.GetNode().GetSubTree())
			{
				FlattenPropertyTree(*SubTree, OutPaths, Path);
			}
			Path.Pop();
		}
	}

	void GetTreePathsForStruct(const FPropertyPathNameTree& UnknownPropertyTree, TArray<FPropertyPathName>& OutPaths, FPropertyTypeName StructType, FPropertyPathName& CurPath)
	{
		for (FPropertyPathNameTree::FConstIterator Itr = UnknownPropertyTree.CreateConstIterator(); Itr; ++Itr)
		{
			CurPath.Push(FPropertyPathNameSegment{ Itr.GetName(), Itr.GetType() });

			FPropertyTypeName Type = Itr.GetType();
			if (Type.GetName() == NAME_StructProperty && Type.GetParameter(0) == StructType)
			{
				OutPaths.Push(CurPath);
			}
			else if (Type.GetName() == NAME_ArrayProperty || Type.GetName() == NAME_SetProperty)
			{
				FPropertyTypeName InnerType = Type.GetParameter(0);
				if (InnerType.GetName() == NAME_StructProperty && InnerType.GetParameter(0) == StructType)
				{
					OutPaths.Push(CurPath);
				}
			}
			else if (Type.GetName() == NAME_MapProperty)
			{
				FPropertyTypeName KeyType = Type.GetParameter(0);
				if (KeyType.GetName() == NAME_StructProperty && KeyType.GetParameter(0) == StructType)
				{
					OutPaths.Push(CurPath);
				}
				FPropertyTypeName ValType = Type.GetParameter(1);
				if (ValType.GetName() == NAME_StructProperty && ValType.GetParameter(0) == StructType)
				{
					OutPaths.Push(CurPath);
				}
			}

			if (const FPropertyPathNameTree* SubTree = Itr.GetNode().GetSubTree())
			{
				GetTreePathsForStruct(*SubTree, OutPaths, StructType, CurPath);
			}
			CurPath.Pop();
		}
	}

	void GetTreePathsForStruct(const FPropertyPathNameTree& UnknownPropertyTree, TArray<FPropertyPathName>& OutPaths, UScriptStruct* Struct)
	{
		FPropertyPathName BasePath;
		FPropertyTypeNameBuilder Builder;
		Builder.AddPath(Struct);
		GetTreePathsForStruct(UnknownPropertyTree, OutPaths, Builder.Build(), BasePath);
	}

	bool SubPathIsLeaf(const TSharedPtr<FPropertyPathNameTree>& UnknownPropertyTree, FPropertyPathName PathName)
	{
		PathName.Pop();
		while (PathName.GetSegmentCount() > 0)
		{
			if (UE::FPropertyPathNameTree::FNode Found = UnknownPropertyTree->Find(PathName))
			{
				return !Found.GetSubTree();
			}
			PathName.Pop();
		}
		return false;
	}

	// When performing a copy from a struct to another struct we may need to create unknown properties in the new location. This occurs when either the struct types
	// differ, or when the old struct has unknown properties that the new one doesn't. This method is called before ReinstanceWithCachedUnknownPropertyTree to grow
	// the property tree to fit the desired unknown properties.
	// 
	// To determine which unknown properties need to be added to the tree, we'll set up a serialization pass that copies the struct into a temp location but turn on
	// property tree collection flags in the serialization context. This will automatically grow the tree to fit the mismatched data.
	void AnticipateUnknownPropertiesForTypeConversion(const TSharedPtr<FPropertyPathNameTree>& UnknownPropertyTree, const FPropertyPathNameResolver& SrcValue, FPropertyPathName DstPath)
	{
		const FPropertyTypeName& SrcType = SrcValue.Path.GetSegment(SrcValue.Path.GetSegmentCount() - 1).Type;
		const FPropertyTypeName& DstType = DstPath.GetSegment(DstPath.GetSegmentCount() - 1).Type;

		// for now this method only handles converting structs
		if (SrcType.GetName() != NAME_StructProperty || DstType.GetName() != NAME_StructProperty)
		{
			return;
		}

		// if types don't match, we'll need to attempt serializing and see if any unknown properties were discovered
		FProperty* SrcProperty = const_cast<FProperty*>(SrcValue.Value.Property);
		const void* SourceData = SrcProperty->ContainerPtrToValuePtr<void>(SrcValue.Value.Container, SrcValue.Value.ArrayIndex == INDEX_NONE ? 0 : SrcValue.Value.ArrayIndex);
		
		UObject* Owner = SrcValue.Object;
		if (const UObject* Instance = UE::FPropertyBagRepository::Get().FindInstanceForDataObject(SrcValue.Object))
		{
			Owner = const_cast<UObject*>(Instance);
		}

		// WRITE TO BUFFER
		TArray<uint8> Buffer = WriteToBuffer(Owner, SrcValue);

		// READ FROM BUFFER into a temp location
		{
			FObjectReader ObjectReader(Buffer);
			FStructuredArchiveFromArchive StructuredReader(ObjectReader);
			FScopedObjectSerializeContext ScopedSerializeContext(Owner, ObjectReader, DstPath);

			FPropertyTag ReadTag;
			StructuredReader.GetSlot() << ReadTag;
			ReadTag.ArrayIndex = DstPath.GetSegment(DstPath.GetSegmentCount() - 1).Index;
			ReadTag.Name = DstType.GetName();

			TUniquePtr<FField> DstField(FField::TryConstruct(DstType.GetName(), nullptr, ReadTag.Name));
			FProperty* DstProperty = CastField<FProperty>(DstField.Get());
			if (!(DstProperty && DstProperty->LoadTypeName(DstType, &ReadTag)))
			{
				return;
			}
			DstProperty->Link(ObjectReader);
			ReadTag.SetProperty(DstProperty);
			void* DestinationData = DstProperty->AllocateAndInitializeValue();

			// because the ScopedSerializeContext has enabled unknown property tracking, the unknown property tree has now grown to include any needed unknown properties
			ReadTag.SerializeTaggedProperty(StructuredReader.GetSlot(), DstProperty, (uint8*)DestinationData, nullptr);
			DstProperty->DestroyAndFreeValue(DestinationData);
		}
	}

	static const FName NAME_IsLooseMetadata(ANSITEXTVIEW("IsLoose"));

	// Compare unknown property tree with the override set. If there's a branch that's not overridden but still in the tree, it should be pruned because
	// it's no longer needed given that the non-overridden properties won't get saved.
	void PruneUnknownPropertyTree(FPropertyPathNameTree* UnknownPropertyTree, const UStruct* Struct, const FOverriddenPropertyNode* Overrides)
	{
		TArray<FPropertyPathName> PathsToRemove;
		auto PushPathToRemove = [&PathsToRemove](const FPropertyPathNameSegment& Segment) {
			FPropertyPathName SubPath;
			SubPath.Push(Segment);
			PathsToRemove.Add(SubPath);
			};

		for (UE::FPropertyPathNameTree::FConstIterator Itr = UnknownPropertyTree->CreateConstIterator(); Itr; ++Itr)
		{
			const FProperty* Property = UE::FindPropertyByNameAndTypeName(Struct, Itr.GetName(), Itr.GetType());
			if (!Property)
			{
				PushPathToRemove({ Itr.GetName(), Itr.GetType() });
				continue;
			}

			// const cast needed because there's no non-const iterator. I'm being very careful to not iterate and modify at the same time so this is fine.
			FPropertyPathNameTree* SubTree = const_cast<FPropertyPathNameTree*>(Itr.GetNode().GetSubTree());
			const FStructProperty* AsStructProperty = CastField<FStructProperty>(Property);
			const FOverriddenPropertyNode* FoundOverride = Overrides->GetSubPropertyNodes().FindByKey(FOverriddenPropertyNodeID(Property));
			if (!FoundOverride)
			{
				PushPathToRemove({ Itr.GetName(), Itr.GetType() });
				continue;
			}

			// recurse into struct properties with children
			if (SubTree && !SubTree->IsEmpty() && AsStructProperty)
			{
				PruneUnknownPropertyTree(SubTree, AsStructProperty->Struct, FoundOverride);
			}

			// TODO: should we handle other containers here?

			// prune overridden leaves if they aren't marked as loose
			if (!SubTree || SubTree->IsEmpty())
			{
				if (!Property->GetBoolMetaData(NAME_IsLooseMetadata))
				{
					PushPathToRemove({ Itr.GetName(), Itr.GetType() });
				}
				continue;
			}
		}
		for (FPropertyPathName& Path : PathsToRemove)
		{
			UnknownPropertyTree->Remove(Path);
		}
	}
}

#if WITH_EDITOR
namespace UE::Private::IDT
{
	/**
	 * Remove a source FPropertyPathName from an IDO if it's an unknown property.
	 * @param IDO - The instance data object that contains the source unknown property
	 * @param Owner - the instance associated with the IDO
	 * @param Params - contains a map with the following parameters
	 *		- ["src"] the source FPropertyPathName as a json object. If this doesn't point to an unknown property, the operation will be skipped
	 */
	ETransformResult RemoveProperty(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner, const FPropertyPathName& BasePath, const FTransformParams& Params)
	{
		TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree = FUnknownPropertyTree(Owner).FindOrCreate();
		FUnknownEnumNames UnknownEnumNames = FUnknownEnumNames(Owner);

		FPropertyPathName SrcPath;
		if (!ensureAlwaysMsgf(Params && FPropertyPathName::FromJsonValue(Params->TryGetField(TEXT("src")), SrcPath),
			TEXT("Failed to parse src parameter for a RemoveProperty IDT operation")))
		{
			return ETransformResult::Break;
		}
		InferMissingPathInfo(SrcPath, Owner->GetClass());

		if (UnknownPropertyTree->Remove(SrcPath) || SubPathIsLeaf(UnknownPropertyTree, SrcPath))
		{
			// if a subpath was an overridden struct or container containing this property, we can't easily remove the property from the IDO
			// class so instead we'll just remove the override on that value so it doesn't get saved anymore
			FPropertyPathNameResolver SrcValue(SrcPath, IDO);
			while (SrcValue)
			{
				TSharedPtr<FPropertyChangedChainEvent> ClearOverrideChangeEvent;
				SrcValue.BuildChangeEvent(ClearOverrideChangeEvent, EPropertyChangeType::ResetToDefault);
				UE::SendClearOverriddenPropertyEvent(IDO, *ClearOverrideChangeEvent, ClearOverrideChangeEvent->PropertyChain);
				SrcValue.Next();
			}

			// create a new IDO with the modified property tree
			TObjectPtr<UObject> OldIDO = IDO;
			IDO = UE::FPropertyBagRepository::Get().ReinstanceWithCachedUnknownPropertyTree(Owner);

			// Cull nodes in the UnknownPropertyTree if they're not overridden
			if (const FOverriddenPropertySet* OverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(IDO))
			{
				if (const FOverriddenPropertyNode* OverrideTree = OverriddenProperties->GetOverriddenPropertyNode(nullptr))
				{
					PruneUnknownPropertyTree(UnknownPropertyTree.Get(), IDO->GetClass(), OverrideTree);
				}
			}

#if WITH_EDITOR
			FCoreUObjectDelegates::OnObjectsReinstanced.Broadcast({ {OldIDO, IDO} });
#endif
		}


		return ETransformResult::Continue;
	};
	REGISTER_IDT_OPERATION(RemoveProperty);

	/**
	 * Map a source FPropertyPathName into a destination FPropertyPathName relative to an object.
	 * @param IDO - The instance data object that contains the source unknown property
	 * @param Owner - the instance associated with the IDO
	 * @param Params - contains a map with the following parameters
	 *		- ["src"] the source FPropertyPathName as a json object. If this doesn't point to an unknown property, the operation will be skipped
	 *		- ["dst"] the destination FPropertyPathName as a json object. If this points to a non-existent property, the path will be added to the new IDO instance as an unknown property
	 */
	ETransformResult RedirectProperty(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner, const FPropertyPathName& BasePath, const FTransformParams& Params)
	{
		TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree = FUnknownPropertyTree(Owner).FindOrCreate();
		FUnknownEnumNames UnknownEnumNames = FUnknownEnumNames(Owner);

		FPropertyPathName SrcPath = BasePath;
		if (!ensureAlwaysMsgf(Params && FPropertyPathName::FromJsonValue(Params->TryGetField(TEXT("src")), SrcPath),
			TEXT("Failed to parse src parameter for a RedirectProperty IDT operation")))
		{
			return ETransformResult::Break;
		}
		InferMissingPathInfo(SrcPath, Owner->GetClass());

		FPropertyPathName DstPath = BasePath;
		if (!ensureAlwaysMsgf(Params && FPropertyPathName::FromJsonValue(Params->TryGetField(TEXT("dst")), DstPath),
			TEXT("Failed to parse dst parameter for a RedirectProperty IDT operation")))
		{
			return ETransformResult::Break;
		}
		InferMissingPathInfo(DstPath, Owner->GetClass());

		if (UnknownPropertyTree->Remove(SrcPath) || SubPathIsLeaf(UnknownPropertyTree, SrcPath))
		{
			FPropertyPathNameResolver DstInstanceValue(DstPath, Owner);
			if (!DstInstanceValue)
			{
				// DstPath is not in the Instance. This means we'll create a new unknown/loose property at DstPath
				UnknownPropertyTree->Add(DstPath);
			}

			// find value in old location (in the old IDO)
			FPropertyPathNameResolver SrcValue(SrcPath, IDO);
			if (!SrcValue)
			{
				return ETransformResult::Continue;
			}

			AnticipateUnknownPropertiesForTypeConversion(UnknownPropertyTree, SrcValue, DstPath);

			// create a new IDO with the modified property tree
			TObjectPtr<UObject> OldIDO = IDO;
			IDO = UE::FPropertyBagRepository::Get().ReinstanceWithCachedUnknownPropertyTree(Owner);

			// find destination location in the new IDO
			FPropertyPathNameResolver DstIDOValue(DstPath, IDO);
			check(DstIDOValue); // we should've just added this path to the IDO

			while (SrcValue && DstIDOValue) // because of wildcards in the path, it's possible that there's several values to copy!
			{
				TSharedPtr<FPropertyChangedChainEvent> DstIdoChangeEvent;
				DstIDOValue.BuildChangeEvent(DstIdoChangeEvent, EPropertyChangeType::ValueSet);

				// copy property from SrcValue to DstIDOValue (note that because we're using change events, the value will be changed in the instance as well)
				IDO->PreEditChange(DstIdoChangeEvent->PropertyChain);
				TryCopyOrConvert(SrcValue, DstIDOValue);
				IDO->PostEditChangeChainProperty(*DstIdoChangeEvent);

				TSharedPtr<FPropertyChangedChainEvent> ClearOverrideChangeEvent;
				SrcValue.BuildChangeEvent(ClearOverrideChangeEvent, EPropertyChangeType::ResetToDefault);
				UE::SendClearOverriddenPropertyEvent(IDO, *ClearOverrideChangeEvent, ClearOverrideChangeEvent->PropertyChain);

				/*
					because containers may contain more than one instance of a struct being redirected, we redirect them all by using wildcards in the path. 
					FPropertyPathName wildcards are resolved by treating INDEX_NONE as "all indices of this container". FPropertyPathNameResolver will resove 
					the 0th index first, then Next() will iterate to the next one
				*/
				SrcValue.Next();
				DstIDOValue.Next();
			}

			// this should never fail unless a user has used wildcards in the path they manually wrote into the IDT
			ensureMsgf(!SrcValue && !DstIDOValue, TEXT("An IDT ran that operates on every element of a container but the source and destination had a " 
				"different number of elements! This may have caused corruption in your data. Save at your own risk."));

			// Cull nodes in the UnknownPropertyTree if they're not overridden
			if (const FOverriddenPropertySet* OverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(IDO))
			{
				if (const FOverriddenPropertyNode* OverrideTree = OverriddenProperties->GetOverriddenPropertyNode(nullptr))
				{
					PruneUnknownPropertyTree(UnknownPropertyTree.Get(), IDO->GetClass(), OverrideTree);
				}
			}

#if WITH_EDITOR
			FCoreUObjectDelegates::OnObjectsReinstanced.Broadcast({ {OldIDO, IDO} });
#endif
		}

		return ETransformResult::Continue;
	};
	REGISTER_IDT_OPERATION(RedirectProperty);
}
#endif

namespace UE
{
	FInstanceDataTransformSet::ETransformResult UE::FInstanceDataTransformSet::FOperation::operator()(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner, const FPropertyPathName& BasePath) const
	{
		if (FInstanceDataTransforms::FTransformFunction* Function = FInstanceDataTransforms::Get().RegisteredOperations.Find(OpCode))
		{
			return (*Function)(IDO, Owner, BasePath, Params);
		}
		return FInstanceDataTransformSet::ETransformResult::Break;
	}

	FInstanceDataTransformSet::ETransformResult FInstanceDataTransformSet::FOperation::operator()(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner) const
	{
		return this->operator()(IDO, Owner, {});
	}

	FInstanceDataTransforms& FInstanceDataTransforms::Get()
	{
		static FInstanceDataTransforms Instance;
		return Instance;
	}

	TObjectPtr<UObject> FInstanceDataTransforms::PatchInstanceDataObject(TNotNull<UObject*> IDO, TNotNull<UObject*> Owner)
	{
		TObjectPtr<UObject> ResultIdo = IDO;
		TObjectPtr<UObject> ResultOwner = Owner;
		PatchClass(ResultIdo, ResultOwner);
		PatchStructs(ResultIdo, ResultOwner);

		return ResultIdo;
	}

	FInstanceDataTransformSet FInstanceDataTransforms::GetTransformSet(UStruct& ClassOrStruct)
	{
		const FTopLevelAssetPath StructPath = ClassOrStruct.GetStructPathName();
		if (bEnableInstanceDataTransformSerialization)
		{
			if (const FInstanceDataTransformSet* TransformSet = FindTransformsInternal(StructPath))
			{
				return *TransformSet;
			}
		}
		return FInstanceDataTransformSet{.StructPath= StructPath };
	}

	TObjectPtr<UObject> FInstanceDataTransforms::ApplyTransformSet(const FInstanceDataTransformSet& TransformSet, TNotNull<UObject*> IDO, TNotNull<UObject*> Owner)
	{
		TArray<FPropertyPathName> PathsToStruct;
		if (TransformSet.StructPath == Owner->GetClass()->GetClassPathName())
		{
			PathsToStruct.Emplace();
		}
		else if (UScriptStruct* FoundStruct = FindObject<UScriptStruct>(TransformSet.StructPath))
		{
			if (TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree = FUnknownPropertyTree(Owner).Find())
			{
				Private::GetTreePathsForStruct(*UnknownPropertyTree, PathsToStruct, FoundStruct);
			}
		}

		TObjectPtr<UObject> ResultIdo = IDO;
		TObjectPtr<UObject> ResultOwner = Owner;
		for (const FPropertyPathName& BasePath : PathsToStruct)
		{
			for (const FInstanceDataTransformSet::FOperation& Operation : TransformSet.Operations)
			{
				Operation(ResultIdo, ResultOwner, BasePath);
			}
		}
		return ResultIdo;
	}

	void FInstanceDataTransforms::SaveTransformSet(const FInstanceDataTransformSet& TransformSet)
	{
		if (!bEnableInstanceDataTransformSerialization)
		{
			UE_LOGF(LogInstanceDataTransforms, Error, "Current configuration does not allow you to save IDTs.");
			return;
		}

		FString ConfigFilepath = Private::InstanceDataTransformDirectory(TransformSet.StructPath);
		if (ConfigFilepath.IsEmpty())
		{
			UE_LOGF(LogInstanceDataTransforms, Error, "Couldn't find a valid IDT config path for '%ls'. Perhaps it's not mounted?", *TransformSet.StructPath.ToString());
			return;
		}

		FString JsonRaw;
		TSharedPtr<FJsonObject> ConfigJson;
		if (FFileHelper::LoadFileToString(JsonRaw, *ConfigFilepath))
		{
			TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonRaw);
			if (!FJsonSerializer::Deserialize(JsonReader, ConfigJson))
			{
				UE_LOGF(LogInstanceDataTransforms, Error, "'%ls' is malformed and could not deserialize.", *ConfigFilepath);
				return;
			}
		}
		else
		{
			ConfigJson = MakeShared<FJsonObject>();
			ConfigJson->SetStringField(TEXT("Package"), TransformSet.StructPath.GetPackageName().ToString());
			ConfigJson->SetObjectField(TEXT("Transforms"), MakeShared<FJsonObject>());
		}

		const TSharedPtr<FJsonObject>* TransformsMapPtr = nullptr;
		if (!ConfigJson->TryGetObjectField(TEXT("Transforms"), TransformsMapPtr) || !TransformsMapPtr)
		{
			UE_LOGF(LogInstanceDataTransforms, Error, "'Transforms' object was missing or malformed in '%ls'", *ConfigFilepath);
			return;
		}

		// create a json array for every transform in this struct
		TArray<TSharedPtr<FJsonValue>> StructTransformsArray;
		for (const FInstanceDataTransformSet::FOperation& Operation : TransformSet.Operations)
		{
			// create the new transform operation
			TSharedPtr<FJsonObject> NewTransform = MakeShared<FJsonObject>();
			NewTransform->SetStringField(TEXT("OpCode"), Operation.OpCode);
			if (Operation.Params)
			{
				NewTransform->SetObjectField(TEXT("Params"), Operation.Params);
			}

			// append the new operation to the Transforms list
			StructTransformsArray.Add(StaticCastSharedRef<FJsonValue>(MakeShared<FJsonValueObject>(NewTransform)));
		}

		// put StructTransformsArray in the map keyed to the struct name
		(*TransformsMapPtr)->SetArrayField(TransformSet.StructPath.GetAssetName().ToString(), StructTransformsArray);

		// update Json file on disk
		JsonRaw.Empty();
		TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<TCHAR>::Create(&JsonRaw);
		if (!FJsonSerializer::Serialize(ConfigJson.ToSharedRef(), JsonWriter))
		{
			UE_LOGF(LogInstanceDataTransforms, Error, "'%ls' failed to Serialize json for.", *ConfigFilepath);
			return;
		}

		if (!FFileHelper::SaveStringToFile(JsonRaw, *ConfigFilepath))
		{
			UE_LOGF(LogInstanceDataTransforms, Error, "'%ls' failed to save.", *ConfigFilepath);
			return;
		}
	}

	bool FInstanceDataTransforms::IsSerializationEnabled()
	{
		return bEnableInstanceDataTransformSerialization;
	}

	const FInstanceDataTransformSet* FInstanceDataTransforms::FindTransformsInternal(const FTopLevelAssetPath& ClassPath)
	{
		// the transform config for a class is found in <RootPackageDirectory>/Config/InstanceDataTransforms.json
		// TODO: I need to some research on wheather any work needs to be done in regards to making sure this file is treated properly by source control and the content worker.
		FString ConfigFilepath = Private::InstanceDataTransformDirectory(ClassPath);
		if (IFileManager::Get().FileExists(*ConfigFilepath))
		{
			FDateTime FileModificationTime = IFileManager::Get().GetTimeStamp(*ConfigFilepath);

			if (FDateTime* LastReadTime = ConfigFileLastReadTime.Find(ConfigFilepath))
			{
				// check if the config file has been updated since the last time we read it
				if (*LastReadTime < FileModificationTime)
				{
					ReadConfigFile(ConfigFilepath);
				}
			}
			else
			{
				ReadConfigFile(ConfigFilepath);
			}
		}

		return StructPathToIDTs.Find(ClassPath);
	}

	const FInstanceDataTransformSet* FInstanceDataTransforms::FindTransformsInternal(const TNotNull<UStruct*> Struct)
	{
		if(Struct->GetOuter() != Struct->GetPackage())
		{
			// can't support transforms for nested structs like 
			// anim blueprint's nested 'sparse class data':
			return nullptr;
		}
		return FindTransformsInternal(Struct->GetStructPathName());
	}

	void FInstanceDataTransforms::ReadConfigFile(const FString& ConfigFilepath)
	{
		TSharedPtr<FJsonObject> ConfigJson;
		FString JsonRaw;
		if (FFileHelper::LoadFileToString(JsonRaw, *ConfigFilepath))
		{
			TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonRaw);
			if (FJsonSerializer::Deserialize(JsonReader, ConfigJson))
			{
				FString PackagePathString;
				if (!ConfigJson->TryGetStringField(TEXT("Package"), PackagePathString))
				{
					UE_LOGF(LogInstanceDataTransforms, Warning, "'Package' feild was missing from '%ls'", *ConfigFilepath);
					return;
				}

				FPackagePath PackagePath;
				if (!FPackagePath::TryFromPackageName(PackagePathString, PackagePath))
				{
					UE_LOGF(LogInstanceDataTransforms, Warning, "'ClassPackage' feild was malformed in '%ls'", *ConfigFilepath);
					return;
				}

				const TSharedPtr<FJsonObject>* TransformsJson = nullptr;
				if (!ConfigJson->TryGetObjectField(TEXT("Transforms"), TransformsJson) || !TransformsJson)
				{
					UE_LOGF(LogInstanceDataTransforms, Warning, "'Transforms' object was missing or malformed in '%ls'", *ConfigFilepath);
					return;
				}

				for (const TPair<FJsonObject::FStringType, TSharedPtr<FJsonValue>>& TransformSetInfo : (*TransformsJson)->Values)
				{
					FTopLevelAssetPath StructPath(PackagePath.GetPackageFName(), FName(FStringView(TransformSetInfo.Key)));

					FInstanceDataTransformSet ReadTransformSet{ .StructPath = StructPath };

					const TArray<TSharedPtr<FJsonValue>>* TransformOperationsJson;
					if (!TransformSetInfo.Value->TryGetArray(TransformOperationsJson) || !TransformOperationsJson)
					{
						UE_LOGF(LogInstanceDataTransforms, Warning, "Transform array was malformed for '%ls' in '%ls'", *TransformSetInfo.Key, *ConfigFilepath);
						return;
					}

					for (const TSharedPtr<FJsonValue>& TransformOperationJson : *TransformOperationsJson)
					{
						TSharedPtr<FJsonObject>* TransformInfo;
						if (!TransformOperationJson->TryGetObject(TransformInfo) || !TransformInfo)
						{
							UE_LOGF(LogInstanceDataTransforms, Warning, "Found a malfomed object in the Transform array for '%ls' in '%ls'", *TransformSetInfo.Key, *ConfigFilepath);
							continue;
						}

						FInstanceDataTransformSet::FOperation Operation;

						if (!(*TransformInfo)->TryGetStringField(TEXT("OpCode"), Operation.OpCode))
						{
							UE_LOGF(LogInstanceDataTransforms, Warning, "An element of the Transform array for '%ls' in '%ls' was missing a 'OpCode' field", *TransformSetInfo.Key, *ConfigFilepath);
							continue;
						}
						const FTransformFunction* Func = RegisteredOperations.Find(Operation.OpCode);
						if (!Func)
						{
							UE_LOGF(LogInstanceDataTransforms, Warning, "The transform OpCode in '%ls' was malformed. '%ls' is not a valid operation.", *ConfigFilepath, *Operation.OpCode);
							continue;
						}

						if (const FTransformParams* ParamsPtr; (*TransformInfo)->TryGetObjectField(TEXT("Params"), ParamsPtr))
						{
							Operation.Params = *ParamsPtr;
						}
						ReadTransformSet.Operations.Emplace(MoveTemp(Operation));
					}
					StructPathToIDTs.Add(StructPath, ReadTransformSet);
				}

			}
		}
		ConfigFileLastReadTime.Add(ConfigFilepath, FDateTime::Now());
	}

	void FInstanceDataTransforms::RegisterOperation(FStringView Name, FTransformFunction Function)
	{
		RegisteredOperations.Add(FString(Name), Function);
	}

	void FInstanceDataTransforms::PatchClass(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner)
	{
		// find and run class IDTs
		if (const FInstanceDataTransformSet* TransformSet = FindTransformsInternal(Owner->GetClass()))
		{
			for (const FInstanceDataTransformSet::FOperation& Operation : TransformSet->Operations)
			{
				FPropertyPathName RootPath; // class IDTs operate at the root of the class
				ETransformResult Result = Operation(IDO, Owner);
				if (Result == ETransformResult::Break)
				{
					break;
				}
			}
		}
	}

	void FInstanceDataTransforms::PatchStructs(TObjectPtr<UObject>& IDO, TObjectPtr<UObject>& Owner)
	{
		// search the unknown property tree for structs. If there are any, look up IDTs associated with that struct type and run them
		TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree = FUnknownPropertyTree(Owner).FindOrCreate();

		// IDTs can modify the UnknownPropertyTree and iterating while modifying a tree is a no-no so pre-iterate and cache the paths that we want to visit ahead of time
		TArray<FPropertyPathName> Paths;
		FPropertyPathName BasePath;
		Private::FlattenPropertyTree(*UnknownPropertyTree, Paths, BasePath);

		// search the unknown property tree for structs. If there are any, look up IDTs associated with that struct type and run them
		for (int32 i = 0; i < Paths.Num(); ++i)
		{
			FPropertyPathName& Path = Paths[i];
			FPropertyPathNameSegment LeafSegment = Path.GetSegment(Path.GetSegmentCount() - 1);

			auto PatchStruct = [&Path, &IDO, &Owner, this](const FPropertyTypeName& StructPath)
				{
					if (UStruct* Struct = UE::FindObjectByTypePath<UScriptStruct>(StructPath))
					{
						if (const FInstanceDataTransformSet* TransformSet = FindTransformsInternal(Struct))
						{
							for (const FInstanceDataTransformSet::FOperation& Operation : TransformSet->Operations)
							{
								ETransformResult Result = Operation(IDO, Owner, Path);
								if (Result == ETransformResult::Break)
								{
									break;
								}
							}
						}
					}
				};

			if (LeafSegment.Type.GetName() == NAME_StructProperty)
			{
				PatchStruct(LeafSegment.Type.GetParameter(0));
			}
			else if (LeafSegment.Type.GetName() == NAME_ArrayProperty || LeafSegment.Type.GetName() == NAME_SetProperty)
			{
				FPropertyTypeName InnerType = LeafSegment.Type.GetParameter(0);
				if (InnerType.GetName() == NAME_StructProperty)
				{
					PatchStruct(InnerType.GetParameter(0));
				}
			}
			else if (LeafSegment.Type.GetName() == NAME_MapProperty)
			{
				FPropertyTypeName KeyType = LeafSegment.Type.GetParameter(0);
				if (KeyType.GetName() == NAME_StructProperty)
				{
					PatchStruct(KeyType.GetParameter(0));
				}
				FPropertyTypeName ValType = LeafSegment.Type.GetParameter(1);
				if (ValType.GetName() == NAME_StructProperty)
				{
					PatchStruct(ValType.GetParameter(0));
				}
			}
		}
	}


}

#endif // !UE_BUILD_SHIPPING_WITH_EDITOR && WITH_EDITORONLY_DATA 
