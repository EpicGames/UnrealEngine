// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LazyObjectPtr.cpp: Guid-based lazy pointer to UObject
=============================================================================*/

#include "UObject/LazyObjectPtr.h"
#include "Async/TransactionallySafeMutex.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "UObject/Package.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectAnnotation.h"

/** Annotation associating objects with their guids **/
static FUObjectAnnotationSparseSearchable<FUniqueObjectGuid,true> GuidAnnotation;

static TArray<TMap<FGuid, FGuid>> PIEGuidMap;
static FTransactionallySafeMutex PIEGuidMapLock;

static FGuid RemapGuid(const FUniqueObjectGuid& Guid, int32 PIEInstanceID)
{
	// An invalid Guid should remain an invalid Guid :
	if (Guid.IsDefault())
	{
		return FGuid();
	}

	check(PIEInstanceID != INDEX_NONE);

	UE::TScopeLock Lock(PIEGuidMapLock);

	if (PIEInstanceID >= PIEGuidMap.Num())
	{
		PIEGuidMap.AddDefaulted(PIEInstanceID + 1 - PIEGuidMap.Num());
	}

	FGuid& FoundGuid = PIEGuidMap[PIEInstanceID].FindOrAdd(Guid.GetGuid());

	if (!FoundGuid.IsValid())
	{
		FoundGuid = FGuid::NewGuid();
	}

	return FoundGuid;
};

/*-----------------------------------------------------------------------------
	FUniqueObjectGuid
-----------------------------------------------------------------------------*/

FUniqueObjectGuid::FUniqueObjectGuid(const class UObject* InObject)
	: Guid(GuidAnnotation.GetAnnotation(InObject).Guid)
{
}

FUniqueObjectGuid FUniqueObjectGuid::FixupForPIE(int32 PlayInEditorID) const
{
	return RemapGuid(GetGuid(), PlayInEditorID);
}

UObject* FUniqueObjectGuid::ResolveObject() const
{
	UObject* Result = GuidAnnotation.Find(*this);
	return Result;
}

FString FUniqueObjectGuid::ToString() const
{
	return Guid.ToString(EGuidFormats::UniqueObjectGuid);
}

void FUniqueObjectGuid::FromString(const FString& From)
{
	TArray<FString> Split;
	Split.Empty(4);
	if( From.ParseIntoArray( Split, TEXT("-"), false ) == 4 )
	{
		Guid.A=FParse::HexNumber(*Split[0]);
		Guid.B=FParse::HexNumber(*Split[1]);
		Guid.C=FParse::HexNumber(*Split[2]);
		Guid.D=FParse::HexNumber(*Split[3]);
	}
	else
	{
		Guid.Invalidate();
	}
}

FUniqueObjectGuid FUniqueObjectGuid::GetOrCreateIDForObject(FObjectPtr ObjectPtr)
{
	UObject* Object = ObjectPtr.Get();

	check(Object);
	checkSlow(IsInGameThread());
	FUniqueObjectGuid ObjectGuid(Object);
	if (!ObjectGuid.IsValid())
	{
#if WITH_EDITOR
		if (GIsCookerLoadingPackage)
		{
			UE_ASSET_LOG(LogUObjectGlobals, Warning, Object, TEXT("Creating a new object GUID for object '%s' during cooking - this asset should be resaved"), *Object->GetFullName());
		}
#endif
		ObjectGuid.Guid = FGuid::NewGuid();
		GuidAnnotation.AddAnnotation(Object, ObjectGuid);
		Object->MarkPackageDirty();
	}
	return ObjectGuid;
}

/*-----------------------------------------------------------------------------------------------------------
	FLazyObjectPtr
-------------------------------------------------------------------------------------------------------------*/

void FLazyObjectPtr::PossiblySerializeObjectGuid(UObject *Object, FStructuredArchive::FRecord Record)
{
	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	if (UnderlyingArchive.IsSaving() || UnderlyingArchive.IsCountingMemory())
	{
		FUniqueObjectGuid Guid = GuidAnnotation.GetAnnotation(Object);
		TOptional<FStructuredArchiveSlot> GuidSlot = Record.TryEnterField(TEXT("Guid"), Guid.IsValid());
		if (GuidSlot.IsSet())
		{
			if (UnderlyingArchive.GetPortFlags() & PPF_DuplicateForPIE)
			{
				Guid = RemapGuid(Guid, UE::GetPlayInEditorID());
			}

			GuidSlot.GetValue() << Guid;
		}
	}
	else if (UnderlyingArchive.IsLoading())
	{
		TOptional<FStructuredArchiveSlot> GuidSlot = Record.TryEnterField(TEXT("Guid"), false);
		if (GuidSlot.IsSet())
		{
			FUniqueObjectGuid Guid;
			GuidSlot.GetValue() << Guid;

			// Don't try and resolve GUIDs when loading a package for diffing
			const UPackage* Package = Object->GetOutermost();
			const bool bLoadedForDiff = Package->HasAnyPackageFlags(PKG_ForDiffing);
			if (!bLoadedForDiff && (!(UnderlyingArchive.GetPortFlags() & PPF_Duplicate) || (UnderlyingArchive.GetPortFlags() & PPF_DuplicateForPIE)))
			{
				check(!Guid.IsDefault());
				UObject* OtherObject = Guid.ResolveObject();
				if (OtherObject != Object) // on undo/redo, the object (potentially) already exists
				{
					const bool bDuplicate = OtherObject != nullptr;
					const bool bReassigning = FParse::Param(FCommandLine::Get(), TEXT("AssignNewMapGuids"));

					if (bDuplicate || bReassigning)
					{
						if (!bReassigning && OtherObject && OtherObject->HasAnyFlags(RF_NewerVersionExists))
						{
							GuidAnnotation.RemoveAnnotation(OtherObject);
							GuidAnnotation.AddAnnotation(Object, Guid);
						}
#if WITH_EDITOR
						else if (Object->GetOutermostObject()->GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor))
						{
							int32 PIEInstanceID = Object->GetOutermostObject()->GetPackage()->GetPIEInstanceID();

							if (PIEInstanceID != INDEX_NONE)
							{
								Guid = RemapGuid(Guid, PIEInstanceID);
								GuidAnnotation.AddAnnotation(Object, Guid);
							}
						}
#endif
						else
						{
							if (!bReassigning)
							{
								// Always warn for non-map packages, skip map packages in PIE, game, or when editor only. Editor-only maps are usually instances, and do not affect the game.
								const bool bInGame = FApp::IsGame() || Package->HasAnyPackageFlags(PKG_PlayInEditor);
								const bool bIsMapOrMapData = Package->ContainsMap() || Package->HasAnyPackageFlags(PKG_ContainsMapData);

								UE_CLOGF(!bIsMapOrMapData || (!bInGame && !Package->HasAnyPackageFlags(PKG_EditorOnly)), LogUObjectGlobals, Warning,
									"Guid referenced by %ls is already used by %ls, which should never happen in the editor but could happen at runtime with duplicate level loading or PIE",
									*Object->GetFullName(), *OtherObject->GetFullName());
							}
							else
							{
								UE_LOGF(LogUObjectGlobals, Warning, "Assigning new Guid to %ls", *Object->GetFullName());
							}
							// This guid is in use, which should never happen in the editor but could happen at runtime with duplicate level loading or PIE. If so give it an invalid GUID and don't add to the annotation map.
							Guid = FGuid();
						}
					}
					else
					{
						GuidAnnotation.AddAnnotation(Object, Guid);
					}
				}
			}
		}
	}
}

void FLazyObjectPtr::ResetPIEFixups()
{
	const int32 PlayInEditorID = UE::GetPlayInEditorID();
	check(PlayInEditorID != -1);
	UE::TScopeLock Lock(PIEGuidMapLock);
	if (PIEGuidMap.IsValidIndex(PlayInEditorID))
	{
		PIEGuidMap[PlayInEditorID].Reset();
	}
}
