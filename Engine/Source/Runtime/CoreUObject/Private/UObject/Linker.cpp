// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Linker.cpp: Unreal object linker.
=============================================================================*/

#include "UObject/Linker.h"
#include "Containers/StringView.h"
#include "Misc/PackageName.h"
#include "Misc/CommandLine.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "UObject/LinkerLoad.h"
#include "Misc/SecureHash.h"
#include "Internationalization/GatherableTextData.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "ProfilingDebugging/CookStats.h"
#include "UObject/CoreRedirects.h"
#include "UObject/LinkerManager.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/DebugSerializationFlags.h"
#include "UObject/ObjectResource.h"
#include "UObject/PackageResourceManager.h"
#include "Algo/Transform.h"

DEFINE_LOG_CATEGORY(LogLinker);

#define LOCTEXT_NAMESPACE "Linker"

/*-----------------------------------------------------------------------------
	Helper functions.
-----------------------------------------------------------------------------*/
namespace Linker
{
	FORCEINLINE bool IsCorePackage(const FName& PackageName)
	{
		return PackageName == NAME_Core || PackageName == GLongCorePackageName;
	}
}

/**
 * Type hash implementation. 
 *
 * @param	Ref		Reference to hash
 * @return	hash value
 */
uint32 GetTypeHash( const FDependencyRef& Ref  )
{
	return PointerHash(Ref.Linker) ^ Ref.ExportIndex;
}

/*----------------------------------------------------------------------------
	FCompressedChunk.
----------------------------------------------------------------------------*/

FCompressedChunk::FCompressedChunk()
:	UncompressedOffset(0)
,	UncompressedSize(0)
,	CompressedOffset(0)
,	CompressedSize(0)
{
}

/** I/O function */
FArchive& operator<<(FArchive& Ar,FCompressedChunk& Chunk)
{
	Ar << Chunk.UncompressedOffset;
	Ar << Chunk.UncompressedSize;
	Ar << Chunk.CompressedOffset;
	Ar << Chunk.CompressedSize;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FCompressedChunk& Chunk)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	Record << SA_VALUE(TEXT("UncompressedOffset"), Chunk.UncompressedOffset);
	Record << SA_VALUE(TEXT("UncompressedSize"), Chunk.UncompressedSize);
	Record << SA_VALUE(TEXT("CompressedOffset"), Chunk.CompressedOffset);
	Record << SA_VALUE(TEXT("CompressedSize"), Chunk.CompressedSize);
}

/*----------------------------------------------------------------------------
	Items stored in Unreal files.
----------------------------------------------------------------------------*/

FGenerationInfo::FGenerationInfo(int32 InExportCount, int32 InNameCount)
: ExportCount(InExportCount), NameCount(InNameCount)
{}

/** I/O functions
 * we use a function instead of operator<< so we can pass in the package file summary for version tests, since archive version hasn't been set yet
 */
void FGenerationInfo::Serialize(FArchive& Ar, const struct FPackageFileSummary& Summary)
{
	Ar << ExportCount << NameCount;
}

void FGenerationInfo::Serialize(FStructuredArchive::FSlot Slot, const struct FPackageFileSummary& Summary)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	Record << SA_VALUE(TEXT("ExportCount"), ExportCount) << SA_VALUE(TEXT("NameCount"), NameCount);
}

void FLinkerTables::SerializeSearchableNamesMap(FArchive& Ar)
{
	SerializeSearchableNamesMap(FStructuredArchiveFromArchive(Ar).GetSlot());
}

void FLinkerTables::SerializeSearchableNamesMap(FStructuredArchive::FSlot Slot)
{
	if (Slot.GetUnderlyingArchive().IsSaving())
	{
		// Sort before saving to keep order consistent
		SearchableNamesMap.KeySort(TLess<FPackageIndex>());

		for (TPair<FPackageIndex, TArray<FName> >& Pair : SearchableNamesMap)
		{
			Pair.Value.Sort(FNameLexicalLess());
		}
	}

	// Default Map serialize works fine
	Slot << SearchableNamesMap;
}

FName FLinkerTables::GetExportClassName(int32 ExportIndex) const
{
	if (ExportMap.IsValidIndex(ExportIndex))
	{
		const FObjectExport& Export = ExportMap[ExportIndex];
		if( !Export.ClassIndex.IsNull() )
		{
			return ImpExp(Export.ClassIndex).ObjectName;
		}
	}
	return NAME_Class;
}

SIZE_T FLinkerTables::GetAllocatedSize() const
{
	SIZE_T Result = 0;
	Result += ImportMap.GetAllocatedSize();
	Result += ExportMap.GetAllocatedSize();
	Result += DependsMap.GetAllocatedSize();
	for (const TArray<FPackageIndex>& DependencyList : DependsMap)
	{
		Result += DependencyList.GetAllocatedSize();
	}
	Result += SoftPackageReferenceList.GetAllocatedSize();
	Result += SearchableNamesMap.GetAllocatedSize();
	for (const TPair<FPackageIndex, TArray<FName>>& Pair : SearchableNamesMap)
	{
		Result += Pair.Value.GetAllocatedSize();
	}
	return Result;
}

/*----------------------------------------------------------------------------
	FLinker.
----------------------------------------------------------------------------*/
FLinker::FLinker(ELinkerType::Type InType, UPackage* InRoot)
: LinkerType(InType)
, LinkerRoot( InRoot )
, FilterClientButNotServer(false)
, FilterServerButNotClient(false)
{
	check(LinkerRoot);

	if( !GIsClient && GIsServer)
	{
		FilterClientButNotServer = true;
	}
	if( GIsClient && !GIsServer)
	{
		FilterServerButNotClient = true;
	}
}

/** Returns a descriptor of the PackagePath this Linker is reading from or writing to, usable for an identifier in warning and log messages */
FString FLinker::GetDebugName() const
{
	return FString("UnexpectedCallToBaseClassGetDebugName");
}


void FLinker::Serialize( FArchive& Ar )
{
	// This function is only used for counting memory, actual serialization uses a different path
	if( Ar.IsCountingMemory() )
	{
		// Can't use CountBytes as ExportMap is array of structs of arrays.
		Ar << ImportMap;
		Ar << ExportMap;
		Ar << DependsMap;
		Ar << SoftPackageReferenceList;
		Ar << GatherableTextDataMap;
		Ar << SearchableNamesMap;
		Ar << ImportTypeHierarchies;
	}
}

/**
 * Return the path name of the UObject represented by the specified import. 
 * (can be used with StaticFindObject)
 * 
 * @param	ImportIndex	index into the ImportMap for the resource to get the name for
 *
 * @return	the path name of the UObject represented by the resource at ImportIndex
 */
FString FLinkerTables::GetImportPathName(int32 ImportIndex)
{
	FString Result;
	for (FPackageIndex LinkerIndex = FPackageIndex::FromImport(ImportIndex); !LinkerIndex.IsNull();)
	{
		const FObjectResource& Resource = ImpExp(LinkerIndex);
		bool bSubobjectDelimiter=false;

		if (Result.Len() > 0 && GetClassName(LinkerIndex) != NAME_Package
			&& (Resource.OuterIndex.IsNull() || GetClassName(Resource.OuterIndex) == NAME_Package) )
		{
			bSubobjectDelimiter = true;
		}

		// don't append a dot in the first iteration
		if ( Result.Len() > 0 )
		{
			if ( bSubobjectDelimiter )
			{
				Result = FString(SUBOBJECT_DELIMITER) + Result;
			}
			else
			{
				Result = FString(".") + Result;
			}
		}

		Result = Resource.ObjectName.ToString() + Result;
		LinkerIndex = Resource.OuterIndex;
	}
	return Result;
}

/**
 * Return the path name of the UObject represented by the specified export.
 * (can be used with StaticFindObject)
 *
 * @param	RootPackagePath			Name of the root package for this export
 * @param	ExportIndex				index into the ExportMap for the resource to get the name for
 * @param	bResolveForcedExports	if true, the package name part of the return value will be the export's original package,
 *									not the name of the package it's currently contained within.
 *
 * @return	the path name of the UObject represented by the resource at ExportIndex
 */
FString FLinkerTables::GetExportPathName(const FString& RootPackagePath, int32 ExportIndex, bool bResolveForcedExports/*=false*/)
{
	TStringBuilder<64> Result;

	bool bForcedExport = false;
	bool bHasOuterImport = false;
	for ( FPackageIndex LinkerIndex = FPackageIndex::FromExport(ExportIndex); !LinkerIndex.IsNull(); LinkerIndex = ImpExp(LinkerIndex).OuterIndex )
	{ 
		bHasOuterImport |= LinkerIndex.IsImport();
		const FObjectResource& Resource = ImpExp(LinkerIndex);

		// don't append a dot in the first iteration
		if ( Result.Len() > 0 )
		{
			// if this export is not a UPackage but this export's Outer is a UPackage, we need to use subobject notation
			if ((Resource.OuterIndex.IsNull() || GetClassName(Resource.OuterIndex) == NAME_Package)
			  && GetClassName(LinkerIndex) != NAME_Package)
			{
				Result.Prepend(SUBOBJECT_DELIMITER);
			}
			else
			{
				Result.Prepend(TEXT("."));
			}
		}
		Result.Prepend(Resource.ObjectName.ToString());
		bForcedExport = bForcedExport || (LinkerIndex.IsExport() ? Exp(LinkerIndex).bForcedExport : false);
	}

	if ((bForcedExport && bResolveForcedExports) ||
		// if the export we are building the path of has an import in its outer chain, no need to append the LinkerRoot path
		bHasOuterImport )
	{
		// Result already contains the correct path name for this export
		return Result.ToString();
	}

	Result.Prepend(TEXT("."));
	Result.Prepend(RootPackagePath);
	return Result.ToString();
}

FString FLinkerTables::GetImportFullName(int32 ImportIndex)
{
	return ImportMap[ImportIndex].ClassName.ToString() + TEXT(" ") + GetImportPathName(ImportIndex);
}

FString FLinkerTables::GetExportFullName(const FString& RootPackagePath, int32 ExportIndex, bool bResolveForcedExports/*=false*/)
{
	FPackageIndex ClassIndex = ExportMap[ExportIndex].ClassIndex;
	FName ClassName = ClassIndex.IsNull() ? FName(NAME_Class) : ImpExp(ClassIndex).ObjectName;

	return ClassName.ToString() + TEXT(" ") + GetExportPathName(RootPackagePath, ExportIndex, bResolveForcedExports);
}

FPackageIndex FLinkerTables::ResourceGetOutermost(FPackageIndex LinkerIndex) const
{
	const FObjectResource* Res = &ImpExp(LinkerIndex);
	while (!Res->OuterIndex.IsNull())
	{
		LinkerIndex = Res->OuterIndex;
		Res = &ImpExp(LinkerIndex);
	}
	return LinkerIndex;
}

bool FLinkerTables::ResourceIsIn(FPackageIndex LinkerIndex, FPackageIndex OuterIndex) const
{
	LinkerIndex = ImpExp(LinkerIndex).OuterIndex;
	while (!LinkerIndex.IsNull())
	{
		LinkerIndex = ImpExp(LinkerIndex).OuterIndex;
		if (LinkerIndex == OuterIndex)
		{
			return true;
		}
	}
	return false;
}

bool FLinkerTables::DoResourcesShareOutermost(FPackageIndex LinkerIndexLHS, FPackageIndex LinkerIndexRHS) const
{
	return ResourceGetOutermost(LinkerIndexLHS) == ResourceGetOutermost(LinkerIndexRHS);
}

bool FLinkerTables::ImportIsInAnyExport(int32 ImportIndex) const
{
	FPackageIndex LinkerIndex = ImportMap[ImportIndex].OuterIndex;
	while (!LinkerIndex.IsNull())
	{
		LinkerIndex = ImpExp(LinkerIndex).OuterIndex;
		if (LinkerIndex.IsExport())
		{
			return true;
		}
	}
	return false;

}

bool FLinkerTables::AnyExportIsInImport(int32 ImportIndex) const
{
	FPackageIndex OuterIndex = FPackageIndex::FromImport(ImportIndex);
	for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
	{
		if (ResourceIsIn(FPackageIndex::FromExport(ExportIndex), OuterIndex))
		{
			return true;
		}
	}
	return false;
}

bool FLinkerTables::AnyExportShareOuterWithImport(int32 ImportIndex) const
{
	FPackageIndex Import = FPackageIndex::FromImport(ImportIndex);
	for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
	{
		if (ExportMap[ExportIndex].OuterIndex.IsImport()
			&& DoResourcesShareOutermost(FPackageIndex::FromExport(ExportIndex), Import))
		{
			return true;
		}
	}
	return false;
}

FString FLinker::GetExportPathName(int32 ExportIndex, const TCHAR* FakeRoot /*= nullptr*/, bool bResolveForcedExports/*=false*/)
{	
	if (FakeRoot)
	{
		// This is to mimic the original behavior where bResolveForcedExports was only respected if FakeRoot was not specified
		bResolveForcedExports = false;
	}
	return FLinkerTables::GetExportPathName(FakeRoot ? FakeRoot : LinkerRoot->GetPathName(), ExportIndex, bResolveForcedExports);
}

FString FLinker::GetExportFullName(int32 ExportIndex, const TCHAR* FakeRoot /*= nullptr*/, bool bResolveForcedExports/*=false*/)
{
	if (FakeRoot)
	{
		// This is to mimic the original behavior where bResolveForcedExports was only respected if FakeRoot was not specified
		bResolveForcedExports = false;
	}
	return FLinkerTables::GetExportFullName(FakeRoot ? FakeRoot : LinkerRoot->GetPathName(), ExportIndex, bResolveForcedExports);
}

void FLinker::StartScriptSHAGeneration()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ScriptSHA == NULL)
	{
		ScriptSHA = new FSHA1;
	}
	ScriptSHA->Reset();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FLinker::UpdateScriptSHAKey(const TArray<uint8>& ScriptCode)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ScriptSHA && ScriptCode.Num())
	{
		ScriptSHA->Update((uint8*)ScriptCode.GetData(), ScriptCode.Num());
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FLinker::GetScriptSHAKey(uint8* OutKey)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(ScriptSHA);
	ScriptSHA->Final();
	ScriptSHA->GetHash(OutKey);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FLinker::~FLinker()
{
	// free any SHA memory
	delete ScriptSHA;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS



/*-----------------------------------------------------------------------------
	Global functions
-----------------------------------------------------------------------------*/

static void LogGetPackageLinkerError(FUObjectSerializeContext* LoadContext, const FPackagePath& PackagePath, const FText& InErrorMessage, UObject* InOuter, uint32 LoadFlags)
{
	static FName NAME_LoadErrors("LoadErrors");
	struct Local
	{
		/** Helper function to output more detailed error info if available */
		static void OutputErrorDetail(FUObjectSerializeContext* InLoadContext, const FName& LogName)
		{
			FUObjectSerializeContext* LoadContextToReport = InLoadContext;
			if (LoadContextToReport && LoadContextToReport->SerializedObject && LoadContextToReport->SerializedImportLinker)
			{
				FMessageLog LoadErrors(LogName);

				TSharedRef<FTokenizedMessage> Message = LoadErrors.Info();
				Message->AddToken(FTextToken::Create(LOCTEXT("FailedLoad_Message", "Failed to load")));
				Message->AddToken(FAssetNameToken::Create(LoadContextToReport->SerializedImportLinker->GetImportPathName(LoadContextToReport->SerializedImportIndex)));
				Message->AddToken(FTextToken::Create(LOCTEXT("FailedLoad_Referenced", "Referenced by")));
				Message->AddToken(FUObjectToken::Create(LoadContextToReport->SerializedObject));
			}
		}
	};

	FLinkerLoad* SerializedPackageLinker = LoadContext ? LoadContext->SerializedPackageLinker : nullptr;
	UObject* SerializedObject = LoadContext ? LoadContext->SerializedObject : nullptr;
	FString LoadingFile = !PackagePath.IsEmpty() ? PackagePath.GetDebugName() : FString("NULL");
	
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("LoadingFile"), FText::FromString(LoadingFile));
	Arguments.Add(TEXT("ErrorMessage"), InErrorMessage);

	FText FullErrorMessage = FText::Format(LOCTEXT("FailedLoad", "Failed to load '{LoadingFile}': {ErrorMessage}"), Arguments);
	if (SerializedPackageLinker || SerializedObject)
	{
		FLinkerLoad* LinkerToUse = SerializedPackageLinker;
		if (!LinkerToUse)
		{
			LinkerToUse = SerializedObject->GetLinker();
		}
		FString LoadedByFile = LinkerToUse ? LinkerToUse->GetDebugName() : SerializedObject->GetOutermost()->GetName();
		FullErrorMessage = FText::FromString(FAssetMsg::GetAssetLogString(*LoadedByFile, FullErrorMessage.ToString()));
	}

	FMessageLog LoadErrors(NAME_LoadErrors);

	if( GIsEditor && !IsRunningCommandlet() )
	{
		// if we don't want to be warned, skip the load warning
		// Display log error regardless LoadFlag settings
		if (LoadFlags & (LOAD_NoWarn | LOAD_Quiet))
		{
			SET_WARN_COLOR(COLOR_RED);
			UE_LOGF(LogLinker, Log, "%ls", *FullErrorMessage.ToString());
			CLEAR_WARN_COLOR();
		}
		else
		{
			SET_WARN_COLOR(COLOR_RED);
			UE_LOGF(LogLinker, Warning, "%ls", *FullErrorMessage.ToString());
			CLEAR_WARN_COLOR();
			// we only want to output errors that content creators will be able to make sense of,
			// so any errors we cant get links out of we will just let be output to the output log (above)
			// rather than clog up the message log

			if (!PackagePath.IsEmpty() && InOuter != NULL)
			{
				FString PackageName = PackagePath.GetPackageNameOrFallback();
				FString OuterPackageName;
				if (!FPackageName::TryConvertFilenameToLongPackageName(InOuter->GetPathName(), OuterPackageName))
				{
					OuterPackageName = InOuter->GetPathName();
				}
				// Output the summary error & the filename link. This might be something like "..\Content\Foo.upk Out of Memory"
				TSharedRef<FTokenizedMessage> Message = LoadErrors.Error();
				Message->AddToken(FAssetNameToken::Create(PackageName));
				Message->AddToken(FTextToken::Create(FText::FromString(TEXT(":"))));
				Message->AddToken(FTextToken::Create(FullErrorMessage));
				Message->AddToken(FAssetNameToken::Create(OuterPackageName));
			}

			Local::OutputErrorDetail(LoadContext, NAME_LoadErrors);
		}
	}
	else
	{
		bool bLogMessageEmitted = false;
		// @see ResavePackagesCommandlet
		if( FParse::Param(FCommandLine::Get(),TEXT("SavePackagesThatHaveFailedLoads")) == true )
		{
			LoadErrors.Warning(FullErrorMessage);
		}
		else
		{
			// Gracefully handle missing packages
			bLogMessageEmitted = SafeLoadError(InOuter, LoadFlags, *FullErrorMessage.ToString());
		}

		// Only print out the message if it was not already handled by SafeLoadError
		if (!bLogMessageEmitted)
		{
			if (LoadFlags & (LOAD_NoWarn | LOAD_Quiet))
			{
				SET_WARN_COLOR(COLOR_RED);
				UE_LOGF(LogLinker, Log, "%ls", *FullErrorMessage.ToString());
				CLEAR_WARN_COLOR();
			}
			else
			{
				SET_WARN_COLOR(COLOR_RED);
				UE_LOGF(LogLinker, Warning, "%ls", *FullErrorMessage.ToString());
				CLEAR_WARN_COLOR();
				Local::OutputErrorDetail(LoadContext, NAME_LoadErrors);
			}
		}
	}
}

FString GetPrestreamPackageLinkerName(const TCHAR* InLongPackageName, bool bSkipIfExists)
{
	if (!InLongPackageName)
	{
		return FString();
	}

	FPackagePath PackagePath;
	if (!FPackagePath::TryFromMountedName(InLongPackageName, PackagePath))
	{
		return FString();
	}
	FName PackageFName(PackagePath.GetPackageFName());
	UPackage* ExistingPackage = bSkipIfExists ? FindObjectFast<UPackage>(nullptr, PackageFName) : nullptr;
	if (ExistingPackage)
	{
		return FString(); // we won't load this anyway, don't prestream
	}


	// Only look for packages on disk
	bool DoesPackageExist = FPackageName::DoesPackageExistEx(PackagePath, FPackageName::EPackageLocationFilter::FileSystem, false, &PackagePath) != FPackageName::EPackageLocationFilter::None;

	if (!DoesPackageExist)
	{
		return FString();
	}

	return PackagePath.GetLocalFullPath();
}

static FPackagePath GetPackagePath(UPackage* InOuter, const TCHAR* InPackageNameOrFileName)
{
	if (InPackageNameOrFileName)
	{
		FPackagePath PackagePath;
		if (FPackagePath::TryFromMountedName(InPackageNameOrFileName, PackagePath))
		{
			return PackagePath;
		}
		else
		{
			UE_LOGF(LogLinker, Warning, "GetPackagePath: Path \"%ls\" is not in a mounted path; returning empty PackagePath. LoadPackage or GetPackageLinker will fail.", InPackageNameOrFileName);
			return FPackagePath();
		}
	}
	else if (InOuter)
	{
		// Resolve filename from package name.
		return FPackagePath::FromPackageNameChecked(InOuter->GetName());
	}
	else
	{
		UE_LOGF(LogLinker, Warning, "GetPackagePath was passed an empty PackageName.");
		return FPackagePath();
	}
}

//
// Find or create the linker for a package.
//
FLinkerLoad* GetPackageLinker
(
	UPackage*		InOuter,
	const FPackagePath& InPackagePath,
	uint32			LoadFlags,
	UPackageMap*	Sandbox,
	FArchive*		InReaderOverride,
	FUObjectSerializeContext** InOutLoadContext,
	FLinkerLoad*	ImportLinker,
	const FLinkerInstancingContext* InstancingContext
)
{
	FUObjectSerializeContext* InExistingContext = InOutLoadContext ? *InOutLoadContext : nullptr;

	// See if the linker is already loaded.
	if (FLinkerLoad* Result = FLinkerLoad::FindExistingLinkerForPackage(InOuter))
	{
		return Result;
	}

	FPackagePath PackagePath(InPackagePath);
	FString PackageName = PackagePath.GetPackageName();
	FName PackageFName(*PackageName);
	if (PackageFName.IsNone())
	{
		// try to recover from this instead of throwing, it seems recoverable just by doing this
		LogGetPackageLinkerError(InExistingContext, InPackagePath, LOCTEXT("PackageResolveFailed", "Can't resolve asset name"), InOuter, LoadFlags);
		return nullptr;
	}

	UE_SCOPED_COOK_STAT(PackageFName, EPackageEventStatType::LoadPackage);
	// Process any package redirects
	{
		const FCoreRedirectObjectName NewPackageName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(NAME_None, NAME_None, PackageFName));
		if (NewPackageName.PackageName != PackageFName)
		{
			PackageFName = NewPackageName.PackageName;
			PackageName = PackageFName.ToString();
			if (!FPackagePath::TryFromMountedName(PackageName, PackagePath))
			{
				LogGetPackageLinkerError(InExistingContext, InPackagePath, LOCTEXT("InvalidPackageRedirect", "PackagePath has invalid FCoreRedirects redirector to unmounted contentroot"), InOuter, LoadFlags);
				return nullptr;
			}
		}
	}

	UPackage* TargetPackage = nullptr;
	if (InOuter)
	{
		TargetPackage = InOuter;
	}
	else
	{
		TargetPackage = FindObject<UPackage>(nullptr, *PackageName);
		if (TargetPackage && TargetPackage->GetOuter() != nullptr)
		{
			TargetPackage = nullptr;
		}
	}

	if (TargetPackage && TargetPackage->HasAnyPackageFlags(PKG_InMemoryOnly))
	{
		// This is a memory-only in package and so it has no linker and this is ok.
		return nullptr;
	}

	// The editor must not redirect packages for localization. We also shouldn't redirect script or in-memory packages (in-memory packages exited earlier so we don't need to check here).
	FString PackageNameToCreate = PackageName;
	if (!(GIsEditor || FPackageName::IsScriptPackage(PackageName)))
	{
		// Allow delegates to resolve the path
		FString PackageNameToLoad = FPackageName::GetDelegateResolvedPackagePath(PackageName);
		PackageNameToLoad = FPackageName::GetLocalizedPackagePath(PackageNameToLoad);
		if (PackageNameToLoad != PackageName)
		{
			PackageName = MoveTemp(PackageNameToLoad);
			PackageFName = FName(*PackageName);
			if (!FPackagePath::TryFromMountedName(PackageName, PackagePath))
			{
				LogGetPackageLinkerError(InExistingContext, InPackagePath, LOCTEXT("UnmountedContentRoot", "GetDelegateResolvedPackagePath or GetLocalizedPackagePath returned path to unmounted contentroot"), InOuter, LoadFlags);
				return nullptr;
			}
		}
	}

	// We need to call DoesPackageExist for a few reasons:
	// 1) Get the extension that the package is using
	// 2) Normalize the capitalization of the packagename to match the case on disk
	// 3) Early exit, avoiding the cost of creating and deleting a UPackage if the package does not exist on disk
#if WITH_EDITORONLY_DATA
	// If we're creating a package, we need to match the capitalization on disk so that the anyone doing source control operations based on the PackageName will send the proper capitalization to the possibly-casesensitive source control
	bool bMatchCaseOnDisk = TargetPackage == nullptr;
#else
	bool bMatchCaseOnDisk = false;
#endif
	bool bPackageExists = FPackageName::DoesPackageExist(PackagePath, bMatchCaseOnDisk, &PackagePath);
	PackageName = PackagePath.GetPackageName();
	if (!bPackageExists)
	{
		// Issue a warning if the caller didn't request nowarn/quiet, and the package isn't marked as known to be missing.
		bool bIssueWarning = (LoadFlags & (LOAD_NoWarn | LOAD_Quiet)) == 0 && !FLinkerLoad::IsKnownMissingPackage(InPackagePath.GetPackageFName());
		if (bIssueWarning)
		{
			// try to recover from this instead of throwing, it seems recoverable just by doing this
			LogGetPackageLinkerError(InExistingContext, InPackagePath, LOCTEXT("FileNotFoundShort", "Can't find file."), InOuter, LoadFlags);
		}
		return nullptr;
	}

	UPackage* CreatedPackage = nullptr;
	if (!TargetPackage)
	{
		if (PackageNameToCreate.Equals(PackageName, ESearchCase::IgnoreCase))
		{
			PackageNameToCreate = PackageName; // Take the capitalization from PackageName, which was normalized in the DoesPackageExist call
		}
		else
		{
#if WITH_EDITORONLY_DATA
			// Make sure the package name matches the capitalization on disk
			FPackagePath PackagePathToCreate = FPackagePath::FromPackageNameChecked(PackageNameToCreate);
			IPackageResourceManager::Get().TryMatchCaseOnDisk(PackagePathToCreate, &PackagePathToCreate);
			PackageNameToCreate = PackagePathToCreate.GetPackageName();
#endif
		}
		// Make sure the package name matches the name on disk
		// Create the package with the provided long package name.
		CreatedPackage = CreatePackage(*PackageNameToCreate);
		if (!CreatedPackage)
		{
			LogGetPackageLinkerError(InExistingContext, InPackagePath, LOCTEXT("FilenameToPackageShort", "Can't convert filename to asset name"), InOuter, LoadFlags);
			return nullptr;
		}
		if (LoadFlags & LOAD_PackageForPIE)
		{
			CreatedPackage->SetPackageFlags(PKG_PlayInEditor);
		}
		TargetPackage = CreatedPackage;
	}
	if (InOuter != TargetPackage)
	{
		// See if the Linker is already loaded for the TargetPackage we've found
		if (FLinkerLoad* Result = FLinkerLoad::FindExistingLinkerForPackage(TargetPackage))
		{
			if (InExistingContext)
			{
				TRefCountPtr<FUObjectSerializeContext> CurrentContext(FUObjectThreadContext::Get().GetSerializeContext());
				if ((CurrentContext->HasStartedLoading() && InExistingContext->GetBeginLoadCount() == 1) || IsInAsyncLoadingThread())
				{
					// Use the current context because it has already started loading objects (or we're in ALT where each package needs its own context)
					*InOutLoadContext = CurrentContext;
				}
				else if (CurrentContext != InExistingContext)
				{
					// Make sure the objects already loaded with the context associated with the existing linker
					// are copied to the context provided for this function call to make sure they all get loaded ASAP
					InExistingContext->AddUniqueLoadedObjects(CurrentContext->PRIVATE_GetObjectsLoadedInternalUseOnly());
				}
			}
			return Result;
		}
	}

	// Create new linker.
	TRefCountPtr<FUObjectSerializeContext> LoadContext(InExistingContext ? InExistingContext : FUObjectThreadContext::Get().GetSerializeContext());
	FLinkerLoad* Result = FLinkerLoad::CreateLinker(LoadContext, TargetPackage, PackagePath, LoadFlags, InReaderOverride, InstancingContext);

	if (!Result && CreatedPackage)
	{
		CreatedPackage->MarkAsGarbage();
	}

	return Result;
}

void ResetLinkerExports(UPackage* InPackage)
{
	FLinkerManager::Get().ResetLinkerExports(InPackage);
}

void ConditionalFlushAsyncLoadingForLinkers(TConstArrayView<FLinkerLoad*> InLinkers)
{
	// If there are currently pending async requests for any package, 
	// it's possible that some of those pending requests are for the given Linkers that our caller wants to flush.
	// Our caller wants to flush the Linkers because we must purge references to a linker from the async loader before the linker can be reset.
	// But we don't have a way to inspect the pending async requests and see which linker they are for. 
	// Flushing the new requests will also flush any other pending async requests for the same linkers.
	if (InLinkers.Num() > 0 && IsAsyncLoading())
	{
		TArray<int32, TInlineAllocator<4>> RequestIds;
		for (FLinkerLoad* Linker : InLinkers)
		{
			if (Linker->AsyncRoot != nullptr)
			{
				int32 Request = INDEX_NONE;
				if (!Linker->GetInstancingContext().IsInstanced())
				{
					Request = LoadPackageAsync(Linker->GetPackagePath(), FLoadPackageAsyncOptionalParams{ .PackagePriority = MAX_int32 });
				}
				else
				{
					FName RemappedPackageName = Linker->GetInstancingContext().RemapPackage(Linker->GetPackagePath().GetPackageFName());
					Request = LoadPackageAsync(Linker->GetPackagePath(), FLoadPackageAsyncOptionalParams{ .CustomPackageName = RemappedPackageName, .PackagePriority = MAX_int32, .InstancingContext = &Linker->GetInstancingContext() });
				}
				RequestIds.Add(Request);
			}
		}

		if (RequestIds.Num())
		{
			UE_LOGF(LogLinker, Log, "Conditionally flushing loading for linker(s) (%ls)", *GetPathNameSafe(InLinkers[0]->LinkerRoot));

			FlushAsyncLoading(RequestIds);
		}
	}
}

void ResetLoaders(UObject* InPkg)
{
	if (FLinkerLoad* Loader = FLinkerLoad::FindExistingLinkerForPackage(InPkg->GetPackage()))
	{
		// Make sure we're not in the middle of loading something in the background.
		ConditionalFlushAsyncLoadingForLinkers(MakeArrayView({ Loader }));
		// Detach all exports from the linker and dissociate the linker.
		FLinkerManager::Get().ResetLoaders(MakeArrayView({ Loader }));
	}
}

void ResetLoaders(TConstArrayView<UObject*> InOuters)
{
	TSet<FLinkerLoad*> LinkersToReset;
	for (UObject* Object : InOuters)
	{
		if (UPackage* TopLevelPackage = Object->GetPackage())
		{
			if (FLinkerLoad* LinkerToReset = FLinkerLoad::FindExistingLinkerForPackage(TopLevelPackage))
			{
				LinkersToReset.Add(LinkerToReset);
			}
		}
	}

	if (LinkersToReset.Num())
	{
		// Make sure we're not in the middle of loading something in the background.
		ConditionalFlushAsyncLoadingForLinkers(LinkersToReset.Array());
		FLinkerManager::Get().ResetLoaders(LinkersToReset);
	}
}

void ConditionalFlushAsyncLoadingForSave(UPackage* InPackage)
{
	if (FLinkerLoad* Loader = FLinkerLoad::FindExistingLinkerForPackage(InPackage->GetPackage()))
	{
		ConditionalFlushAsyncLoadingForLinkers(MakeArrayView({ Loader }));
	}
}

void ResetLoadersForSave(UPackage* Package, const TCHAR* Filename)
{
	FLinkerLoad* Loader = FLinkerLoad::FindExistingLinkerForPackage(Package);
	if (Loader)
	{
		if (FPackagePath::FromLocalPath(Filename) == Loader->GetPackagePath())
		{
			// Detach all exports from the linker and dissociate the linker.
			ConditionalFlushAsyncLoadingForLinkers(MakeArrayView({ Loader }));
			FLinkerManager::Get().ResetLoaders(MakeArrayView({ Loader }));
		}
	}
}

void ResetLoadersForSave(TConstArrayView<FPackageSaveInfo> InPackages)
{
	TSet<FLinkerLoad*> LinkersToReset;
	Algo::TransformIf(InPackages, LinkersToReset,
		[](const FPackageSaveInfo& InPackageSaveInfo)
		{
			FLinkerLoad* Loader = FLinkerLoad::FindExistingLinkerForPackage(InPackageSaveInfo.Package);
			return Loader && FPackagePath::FromLocalPath(InPackageSaveInfo.Filename) == Loader->GetPackagePath();
		},
		[](const FPackageSaveInfo& InPackageSaveInfo)
		{
			return FLinkerLoad::FindExistingLinkerForPackage(InPackageSaveInfo.Package);
		});

	if (LinkersToReset.Num())
	{
		ConditionalFlushAsyncLoadingForLinkers(LinkersToReset.Array());
		FLinkerManager::Get().ResetLoaders(LinkersToReset);
	}
}

void DeleteLoaders()
{
	FLinkerManager::Get().DeleteLinkers();
}

void DeleteLoader(FLinkerLoad* Loader)
{
	FLinkerManager::Get().RemoveLinker(Loader);
}

void EnsureLoadingComplete(UPackage* Package)
{
	FLinkerManager::Get().EnsureLoadingComplete(Package);
}

#undef LOCTEXT_NAMESPACE
