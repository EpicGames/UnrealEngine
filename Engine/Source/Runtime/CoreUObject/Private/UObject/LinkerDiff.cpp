// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerDiff.h"

#include "Algo/Transform.h"
#include "Containers/StringFwd.h"
#include "Misc/OutputDevice.h"
#include "Misc/StringBuilder.h"
#include "UObject/Linker.h"
#include "UObject/LinkerSave.h"
#include "UObject/Package.h"
#include "Internationalization/GatherableTextData.h"

PRAGMA_DISABLE_DEPRECATION_WARNINGS;
FLinkerDiff FLinkerDiff::CompareLinkers(FLinker* LHSLinker, FLinker* RHSLinker)
{
	FLinkerDiff Diff;
	Diff.Generate(LHSLinker, RHSLinker);
	return Diff;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS;

COREUOBJECT_API bool FLinkerDiff::HasDiffs() const
{
	return SummaryDiffs.Num() > 0
		|| NameMapDiffs.Num() > 0
		|| GatherableTextDataDiffs.Num() > 0
		|| ImportDiffs.Num() > 0
		|| ExportDiffs.Num() > 0
		|| SoftPackageReferenceDiffs.Num() > 0
		|| SearchableNameDiffs.Num() > 0;
}

void FLinkerDiff::PrintDiff(FOutputDevice& Ar)
{
	auto PrintDiffSection = [&Ar](const FString& HeaderName, const TArray<FString>& DiffSection)
	{
		if (DiffSection.Num() > 0)
		{
			Ar.Logf(TEXT("%s: %d"), *HeaderName, DiffSection.Num());
			for (const FString& Diff : DiffSection)
			{
				Ar.Logf(TEXT("\t%s"), *Diff);
			}
		}
	};

	if (HasDiffs())
	{
		Ar.Logf(TEXT("Save (Old vs New) Linker Comparison for: %s"), *PackageName);
		PrintDiffSection(TEXT("Summary Diff"), SummaryDiffs);
		PrintDiffSection(TEXT("NameMap Diff"), NameMapDiffs);
		PrintDiffSection(TEXT("GatherableTextData Diff"), GatherableTextDataDiffs);
		PrintDiffSection(TEXT("ImportMap Diff"), ImportDiffs);
		PrintDiffSection(TEXT("ExportMap and DependsMap Diffs Diff"), ExportDiffs);
		PrintDiffSection(TEXT("SoftPackageReference Diff"), SoftPackageReferenceDiffs);
		PrintDiffSection(TEXT("SearchableNames Diff"), SearchableNameDiffs);
	}
}

#define COMPARE_MEMBER(LHSRef, RHSRef, Member)		\
	if (LHSRef.Member != RHSRef.Member)				\
	{												\
		TStringBuilder<256> Builder;				\
		Builder.Append(TEXT(#Member));				\
		Builder.Append(TEXT(": "));					\
		Builder.Append(LexToString(LHSRef.Member));	\
		Builder.Append(TEXT(" vs "));				\
		Builder.Append(LexToString(RHSRef.Member));	\
		Diffs.Add(Builder.ToString());				\
	}
#define COMPARE_CALCULATED_VALUE(LHSValue, RHSValue, ValueNameText) \
	{												\
		auto LocalLHS = LHSValue;					\
		auto LocalRHS = RHSValue;					\
		if (LocalLHS != LocalRHS)					\
		{											\
			TStringBuilder<256> Builder;			\
			Builder.Append(ValueNameText);			\
			Builder.Append(TEXT(": "));				\
			Builder.Append(LexToString(LocalLHS));	\
			Builder.Append(TEXT(" vs "));			\
			Builder.Append(LexToString(LocalRHS));	\
			Diffs.Add(Builder.ToString());			\
		}											\
	}


void FLinkerDiff::Generate(FLinker* LHSLinker, FLinker* RHSLinker)
{
	check(LHSLinker->LinkerRoot == RHSLinker->LinkerRoot);
	PackageName = LHSLinker->LinkerRoot->GetName();
	GenerateSummaryDiff(LHSLinker, RHSLinker);
	GenerateNameMapDiff(LHSLinker, RHSLinker);
	GenerateGatherableTextDataMapDiff(LHSLinker, RHSLinker);
	GenerateImportMapDiff(LHSLinker, RHSLinker);
	GenerateExportAndDependsMapDiff(LHSLinker, RHSLinker);
	GenerateSofPackageReferenceDiff(LHSLinker, RHSLinker);
	GenerateSearchableNameMapDiff(LHSLinker, RHSLinker);
}

void FLinkerDiff::GenerateSummaryDiff(FLinker* LHSLinker, FLinker* RHSLinker)
{
	TArray<FString>& Diffs = SummaryDiffs;
	const FPackageFileSummary& LHSSummary = LHSLinker->Summary;
	const FPackageFileSummary& RHSSummary = RHSLinker->Summary;

	// Package Tag, file version
	COMPARE_MEMBER(LHSSummary, RHSSummary, Tag);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, FileVersionUE);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, FileVersionLicenseeUE);

	//CustomVersionContainer

	//COMPARE_MEMBER(LHSSummary, RHSSummary, TotalHeaderSize);
	COMPARE_MEMBER(LHSSummary, RHSSummary, GetPackageFlags());
	//FolderName
	COMPARE_MEMBER(LHSSummary, RHSSummary, NameCount);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, NameOffset);

	COMPARE_MEMBER(LHSSummary, RHSSummary, LocalizationId);

	COMPARE_MEMBER(LHSSummary, RHSSummary, GatherableTextDataCount);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, GatherableTextDataOffset);

	COMPARE_MEMBER(LHSSummary, RHSSummary, ExportCount);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, ExportOffset);

	COMPARE_MEMBER(LHSSummary, RHSSummary, ImportCount);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, ImportOffset);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, DependsOffset);

	COMPARE_MEMBER(LHSSummary, RHSSummary, SoftPackageReferencesCount);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, SoftPackageReferencesOffset);

	//COMPARE_MEMBER(LHSSummary, RHSSummary, SearchableNamesOffset);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, ThumbnailTableOffset);

#if WITH_EDITORONLY_DATA
	COMPARE_CALCULATED_VALUE(LHSSummary.GetSavedHash(), RHSSummary.GetSavedHash(), TEXT("SavedHash"));
	COMPARE_MEMBER(LHSSummary, RHSSummary, PersistentGuid);
#endif
	// Generations
	// EngineVersion
	// PackageSource

	COMPARE_MEMBER(LHSSummary, RHSSummary, CompressionFlags);
	COMPARE_MEMBER(LHSSummary, RHSSummary, bUnversioned);

	//COMPARE_MEMBER(LHSSummary, RHSSummary, AssetRegistryDataOffset);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, BulkDataStartOffset);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, WorldTileInfoDataOffset);

	// ChunkIDs

	COMPARE_MEMBER(LHSSummary, RHSSummary, PreloadDependencyCount);
	//COMPARE_MEMBER(LHSSummary, RHSSummary, PreloadDependencyOffset);
}

void FLinkerDiff::GenerateNameMapDiff(FLinker* LHSLinker, FLinker* RHSLinker)
{
	TArray<FString>& Diffs = NameMapDiffs;
	const TArray<FNameEntryId>& LHSNameMap = LHSLinker->NameMap;
	TSet<FNameEntryId> RHSNameSet(RHSLinker->NameMap);

	for (const FNameEntryId& NameEntryId : LHSNameMap)
	{
		FSetElementId Id = RHSNameSet.FindId(NameEntryId);
		if (Id.IsValidId())
		{
			RHSNameSet.Remove(Id);
		}
		// We haven't found the name in the RHS set, mark a missing name diff
		else
		{
			TStringBuilder<1024> Builder;
			Builder.Append(TEXT("Missing RHS Name: "));
			Builder.Append(FName::CreateFromDisplayId(NameEntryId, 0).ToString());
			Diffs.Add(Builder.ToString());
		}
	}
	// What's left in the set are RHS diff, mark a new name diff
	for (const FNameEntryId& NameEntryId : RHSNameSet)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(TEXT("New RHS Name: "));
		Builder.Append(FName::CreateFromDisplayId(NameEntryId, 0).ToString());
		Diffs.Add(Builder.ToString());
	}
}

inline uint32 GetTypeHash(const FGatherableTextData& TextData)
{
	return GetTypeHash(TextData.SourceData.SourceString);
}

bool operator==(const FGatherableTextData& LHS, const FGatherableTextData& RHS)
{
	return LHS.NamespaceName == RHS.NamespaceName &&
		LHS.SourceData.SourceString == RHS.SourceData.SourceString && 
		LHS.SourceData.SourceStringMetaData.IsExactMatch(RHS.SourceData.SourceStringMetaData) &&
		LHS.SourceSiteContexts.Num() == RHS.SourceSiteContexts.Num();
}

void FLinkerDiff::GenerateGatherableTextDataMapDiff(FLinker* LHSLinker, FLinker* RHSLinker)
{
	TArray<FString>& Diffs = GatherableTextDataDiffs;

	//@todo: Might need to check source site contexts
	const TArray<FGatherableTextData>& LHSGatherableTextDataMap = LHSLinker->GatherableTextDataMap;
	TSet<FGatherableTextData> RHSGatherableTextDataSet(RHSLinker->GatherableTextDataMap);

	for (const FGatherableTextData& TextData : LHSGatherableTextDataMap)
	{
		FSetElementId Id = RHSGatherableTextDataSet.FindId(TextData);
		if (Id.IsValidId())
		{
			RHSGatherableTextDataSet.Remove(Id);
		}
		// We haven't found the name in the RHS set, mark a missing text data diff
		else
		{
			TStringBuilder<1024> Builder;
			Builder.Append(TEXT("Missing RHS Gatherable Text: "));
			Builder.Append(TextData.SourceData.SourceString);
			Diffs.Add(Builder.ToString());
		}
	}
	for (const FGatherableTextData& TextData : RHSGatherableTextDataSet)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(TEXT("New RHS Text Data: "));
		Builder.Append(TextData.SourceData.SourceString);
		Diffs.Add(Builder.ToString());
	}
}

struct FLinkerImportObject
{
	FLinkerImportObject(FObjectImport& InImport, FLinker* AssociatedLinker)
		: Import(InImport)
		, Linker(AssociatedLinker)
	{}

	FObjectImport& Import;
	FLinker* Linker;
};

inline uint32 GetTypeHash(const FLinkerImportObject& Import)
{
	return HashCombine(GetTypeHash(Import.Import.ObjectName), GetTypeHash(Import.Linker->GetPathName(Import.Import.OuterIndex)));
}

bool operator==(const FLinkerImportObject& LHS, const FLinkerImportObject& RHS)
{
	return LHS.Import.ObjectName == RHS.Import.ObjectName &&
		LHS.Linker->GetPathName(LHS.Import.OuterIndex) == RHS.Linker->GetPathName(RHS.Import.OuterIndex) && 
		LHS.Import.ClassPackage == RHS.Import.ClassPackage &&
		LHS.Import.ClassName == RHS.Import.ClassName &&
		LHS.Import.GetPackageName() == RHS.Import.GetPackageName() &&
		LHS.Import.XObject == RHS.Import.XObject;
}

void FLinkerDiff::GenerateImportMapDiff(FLinker* LHSLinker, FLinker* RHSLinker)
{
	TArray<FString>& Diffs = ImportDiffs;
	TArray<FObjectImport>& LHSImportMap = LHSLinker->ImportMap;
	TSet<FLinkerImportObject> RHSImportSet;
	for (FObjectImport& Import : RHSLinker->ImportMap)
	{
		RHSImportSet.Add(FLinkerImportObject(Import, RHSLinker));
	}

	for (FObjectImport& Import : LHSImportMap)
	{
		FSetElementId Id = RHSImportSet.FindId(FLinkerImportObject(Import, LHSLinker));
		if (Id.IsValidId())
		{
			RHSImportSet.Remove(Id);
		}
		else
		{
			TStringBuilder<1024> Builder;
			Builder.Append(TEXT("Missing or different RHS Import: "));
			Builder.Append(Import.ObjectName.ToString());
			Builder.Append(TEXT(", Outer Path: "));
			Builder.Append(LHSLinker->GetPathName(Import.OuterIndex));
			Diffs.Add(Builder.ToString());
		}
	}

	for (const FLinkerImportObject& Import : RHSImportSet)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(TEXT("New or different RHS Import: "));
		Builder.Append(Import.Import.ObjectName.ToString());
		Builder.Append(TEXT(", Outer Path: "));
		Builder.Append(RHSLinker->GetPathName(Import.Import.OuterIndex));
		Diffs.Add(Builder.ToString());
	}
}

struct FLinkerExportObject
{
	FLinkerExportObject(FObjectExport& InExport, FLinker* AssociatedLinker, int32 InExportIndex)
		: Export(InExport)
		, Linker(AssociatedLinker)
		, ExportIndex(InExportIndex)
	{}

	FObjectExport& Export;
	FLinker* Linker;
	int32 ExportIndex;
};

inline uint32 GetTypeHash(const FLinkerExportObject& Export)
{
	return HashCombine(GetTypeHash(Export.Export.ObjectName), GetTypeHash(Export.Linker->GetPathName(Export.Export.OuterIndex)));
}

bool operator==(const FLinkerExportObject& LHS, const FLinkerExportObject& RHS)
{
	return LHS.Export.ObjectName == RHS.Export.ObjectName &&
		LHS.Linker->GetPathName(LHS.Export.OuterIndex) == RHS.Linker->GetPathName(RHS.Export.OuterIndex) &&
		LHS.Linker->GetClassName(LHS.Export.ClassIndex) == RHS.Linker->GetClassName(RHS.Export.ClassIndex) &&
		LHS.Linker->GetPathName(LHS.Export.SuperIndex) == RHS.Linker->GetPathName(RHS.Export.SuperIndex) &&
		LHS.Linker->GetPathName(LHS.Export.TemplateIndex) == RHS.Linker->GetPathName(RHS.Export.TemplateIndex);
}

void FLinkerDiff::GenerateExportAndDependsMapDiff(FLinker* LHSLinker, FLinker* RHSLinker)
{
	TArray<FString>& Diffs = ExportDiffs;
	TArray<FObjectExport>& LHSExportMap = LHSLinker->ExportMap;
	TSet<FLinkerExportObject> RHSExportSet;
	for (int32 RHSIndex = 0; RHSIndex < RHSLinker->ExportMap.Num(); ++RHSIndex)
	{
		RHSExportSet.Add(FLinkerExportObject(RHSLinker->ExportMap[RHSIndex], RHSLinker, RHSIndex));
	}

	for (int32 LHSIndex = 0; LHSIndex < LHSLinker->ExportMap.Num(); ++LHSIndex)
	{
		FObjectExport& Export = LHSLinker->ExportMap[LHSIndex];
		FSetElementId Id = RHSExportSet.FindId(FLinkerExportObject(Export, LHSLinker, LHSIndex));
		if (Id.IsValidId())
		{
			GenerateExportDiff(LHSLinker, FLinkerExportObject(Export, LHSLinker, LHSIndex), RHSExportSet[Id]);
			GenerateDependsArrayDiff(LHSLinker, LHSIndex, RHSLinker, RHSExportSet[Id].ExportIndex);
			RHSExportSet.Remove(Id);
		}
		else
		{
			TStringBuilder<1024> Builder;
			Builder.Append(TEXT("Missing or different RHS Export: "));
			Builder.Append(Export.ObjectName.ToString());
			Builder.Append(TEXT(", Outer Path: "));
			Builder.Append(LHSLinker->GetPathName(Export.OuterIndex));
			Diffs.Add(Builder.ToString());
		}
	}

	for (const FLinkerExportObject& Export : RHSExportSet)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(TEXT("New or different RHS Export: "));
		Builder.Append(Export.Export.ObjectName.ToString());
		Builder.Append(TEXT(", Outer Path: "));
		Builder.Append(RHSLinker->GetPathName(Export.Export.OuterIndex));
		Diffs.Add(Builder.ToString());
	}
}

void FLinkerDiff::GenerateSofPackageReferenceDiff(FLinker* LHSLinker, FLinker* RHSLinker)
{
	TArray<FString>& Diffs = SoftPackageReferenceDiffs;
	const TArray<FName>& LHSSoftPackageReferenceList = LHSLinker->SoftPackageReferenceList;
	TSet<FName> RHSSoftPackageReferenceSet(RHSLinker->SoftPackageReferenceList);

	for (const FName& SoftName : LHSSoftPackageReferenceList)
	{
		FSetElementId Id = RHSSoftPackageReferenceSet.FindId(SoftName);
		if (Id.IsValidId())
		{
			RHSSoftPackageReferenceSet.Remove(Id);
		}
		else
		{
			TStringBuilder<1024> Builder;
			Builder.Append(TEXT("Missing RHS Soft Package Reference: "));
			Builder.Append(SoftName.ToString());
			Diffs.Add(Builder.ToString());
		}
	}
	for (const FName& SoftName : RHSSoftPackageReferenceSet)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(TEXT("New RHS Soft Package Reference: "));
		Builder.Append(SoftName.ToString());
		Diffs.Add(Builder.ToString());
	}
}

FPackageIndex FindResourcePackageIndex(const FObjectResource& LHSObject, bool bIsImport, FLinker* LHSLinker, FLinker* RHSLinker)
{
	if (bIsImport)
	{
		for (int32 Index = 0; Index < RHSLinker->ImportMap.Num(); ++Index)
		{
			FObjectResource& Import = RHSLinker->ImportMap[Index];
			if (LHSObject.ObjectName == Import.ObjectName &&
				LHSLinker->GetPathName(LHSObject.OuterIndex) == RHSLinker->GetPathName(Import.OuterIndex))
			{
				return FPackageIndex::FromImport(Index);
			}
		}
	}
	else
	{
		for (int32 Index = 0; Index < RHSLinker->ExportMap.Num(); ++Index)
		{
			FObjectResource& Export = RHSLinker->ExportMap[Index];
			if (LHSObject.ObjectName == Export.ObjectName &&
				LHSLinker->GetPathName(LHSObject.OuterIndex) == RHSLinker->GetPathName(Export.OuterIndex))
			{
				return FPackageIndex::FromExport(Index);
			}
		}
	}
	return FPackageIndex();
}

void FLinkerDiff::GenerateSearchableNameMapDiff(FLinker* LHSLinker, FLinker* RHSLinker)
{
	TArray<FString>& Diffs = SearchableNameDiffs;
	const TMap<FPackageIndex, TArray<FName>>& LHSSearchableNamesMap = LHSLinker->SearchableNamesMap;
	const TMap<FPackageIndex, TArray<FName>>& RHSSearchableNamesMap = RHSLinker->SearchableNamesMap;

	for (const auto& SearchableNamePair : LHSSearchableNamesMap)
	{
		check(!SearchableNamePair.Key.IsNull());
		FPackageIndex RHSIndex = FindResourcePackageIndex(LHSLinker->ImpExp(SearchableNamePair.Key), SearchableNamePair.Key.IsImport(), LHSLinker, RHSLinker);
		if (!RHSIndex.IsNull() && RHSLinker->SearchableNamesMap.Contains(RHSIndex))
		{
			GenerateSearchableNameArrayDiff(SearchableNamePair.Value, RHSLinker->SearchableNamesMap[RHSIndex]);
		}
		else
		{
			TStringBuilder<1024> Builder;
			Builder.Append(TEXT("Missing RHS Searchable Name Map entry for: "));
			Builder.Append(LHSLinker->GetPathName(SearchableNamePair.Key));
			Diffs.Add(Builder.ToString());
		}
	}

	for (const auto& SearchableNamePair : RHSSearchableNamesMap)
	{
		FPackageIndex LHSIndex = FindResourcePackageIndex(RHSLinker->ImpExp(SearchableNamePair.Key), SearchableNamePair.Key.IsImport(), RHSLinker, LHSLinker);
		if (LHSIndex.IsNull())
		{
			TStringBuilder<1024> Builder;
			Builder.Append(TEXT("New RHS Searchable Name Map entry for: "));
			Builder.Append(RHSLinker->GetPathName(SearchableNamePair.Key));
			Diffs.Add(Builder.ToString());
		}
	}
}

void FLinkerDiff::GenerateExportDiff(FLinker* LHSLinker, const FLinkerExportObject& LHSExport, const FLinkerExportObject& RHSExport)
{
	TArray<FString> Diffs;

	COMPARE_MEMBER(LHSExport.Export, RHSExport.Export, ObjectFlags);
	COMPARE_MEMBER(LHSExport.Export, RHSExport.Export, SerialSize);
	COMPARE_MEMBER(LHSExport.Export, RHSExport.Export, PackageFlags);
	COMPARE_MEMBER(LHSExport.Export, RHSExport.Export, SerializationBeforeSerializationDependencies);
	COMPARE_MEMBER(LHSExport.Export, RHSExport.Export, CreateBeforeSerializationDependencies);
	COMPARE_MEMBER(LHSExport.Export, RHSExport.Export, SerializationBeforeCreateDependencies);
	COMPARE_MEMBER(LHSExport.Export, RHSExport.Export, CreateBeforeCreateDependencies);

	if (Diffs.Num() > 0)
	{
		ExportDiffs.Add(FString::Printf(TEXT("Export Diffs for resource: %s, Outer: %s"), *LHSExport.Export.ObjectName.ToString(), *LHSLinker->GetPathName(LHSExport.Export.OuterIndex)));
		ExportDiffs.Append(MoveTemp(Diffs));
	}
}

void FLinkerDiff::GenerateDependsArrayDiff(FLinker* LHSLinker, int32 LHSIndex, FLinker* RHSLinker, int32 RHSIndex)
{
	TArray<FString>& Diffs = ExportDiffs;

	TArray<FString> LHSExportDependencies;
	Algo::Transform(LHSLinker->DependsMap[LHSIndex], LHSExportDependencies, [LHSLinker](FPackageIndex Index)
		{
			return LHSLinker->GetPathName(Index);
		});
	TSet<FString> RHSExportDependencies;
	Algo::Transform(RHSLinker->DependsMap[RHSIndex], RHSExportDependencies, [RHSLinker](FPackageIndex Index)
		{
			return RHSLinker->GetPathName(Index);
		});


	TSet<FString> MissingDependencyDiffsToValidate;
	for (const FString& PathName : LHSExportDependencies)
	{
		FSetElementId Id = RHSExportDependencies.FindId(PathName);
		if (Id.IsValidId())
		{
			RHSExportDependencies.Remove(Id);
		}
		else
		{
			MissingDependencyDiffsToValidate.Add(PathName);
		}
	}

	// if we have missing dependencies, validate they are accurate, the old save mechanism was adding indirect references as dependencies, which we should be able to trim
	if (MissingDependencyDiffsToValidate.Num())
	{
		// @todo: no need for now
	}

	FString ResourcePathName = RHSLinker->GetPathName(FPackageIndex::FromExport(RHSIndex));
	for (const FString& PathName : MissingDependencyDiffsToValidate)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(TEXT("Missing RHS Dependency "));
		Builder.Append(PathName);
		Builder.Append(TEXT(" for resource "));
		Builder.Append(ResourcePathName);
		Diffs.Add(Builder.ToString());
	}

	for (const FString& PathName : RHSExportDependencies)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(TEXT("New RHS Dependency "));
		Builder.Append(PathName);
		Builder.Append(TEXT(" for resource "));
		Builder.Append(ResourcePathName);
		Diffs.Add(Builder.ToString());
	}

	// Validate preload dependencies
	FObjectExport& LHSExport = LHSLinker->ExportMap[LHSIndex];
	FObjectExport& RHSExport = RHSLinker->ExportMap[RHSIndex];

	if (LHSExport.FirstExportDependency != -1 && RHSExport.FirstExportDependency != -1)
	{
		TArray<FPackageIndex>& LHSDepList = static_cast<FLinkerSave*>(LHSLinker)->DepListForErrorChecking;
		TArray<FPackageIndex>& RHSDepList = static_cast<FLinkerSave*>(RHSLinker)->DepListForErrorChecking;

		TSet<FPackageIndex> RHSDepSet;
		int32 LHSStartIndex = 0, RHSStartIndex = 0;

		RHSStartIndex = RHSExport.FirstExportDependency;
		LHSStartIndex = LHSExport.FirstExportDependency;
		if (LHSExport.SerializationBeforeSerializationDependencies != RHSExport.SerializationBeforeSerializationDependencies)
		{
			for (int32 Index = 0; Index < RHSExport.SerializationBeforeSerializationDependencies; ++Index)
			{
				RHSDepSet.Add(RHSDepList[RHSStartIndex + Index]);
			}

			for (int32 Index = 0; Index < LHSExport.SerializationBeforeSerializationDependencies; ++Index)
			{
				FPackageIndex LHSDepIndex = LHSDepList[LHSStartIndex + Index];
				FPackageIndex RHSDepIndex = FindResourcePackageIndex(LHSLinker->ImpExp(LHSDepIndex), LHSDepIndex.IsImport(), LHSLinker, RHSLinker);

				FSetElementId Id = RHSDepSet.FindId(RHSDepIndex);
				if (Id.IsValidId())
				{
					RHSDepSet.Remove(Id);
				}
				else
				{
					TStringBuilder<1024> Builder;
					Builder.Append(TEXT("Missing RHS SerializationBeforeSerializationDependencies Preload Dependencies "));
					Builder.Append(LHSLinker->GetPathName(LHSDepIndex));
					Builder.Append(TEXT(" for resource "));
					Builder.Append(ResourcePathName);
					Diffs.Add(Builder.ToString());
				}

			}

			for (FPackageIndex RHSDepIndex : RHSDepSet)
			{
				TStringBuilder<1024> Builder;
				Builder.Append(TEXT("New RHS SerializationBeforeSerializationDependencies Preload Dependencies "));
				Builder.Append(RHSLinker->GetPathName(RHSDepIndex));
				Builder.Append(TEXT(" for resource "));
				Builder.Append(ResourcePathName);
				Diffs.Add(Builder.ToString());
			}
		}

		RHSStartIndex += RHSExport.SerializationBeforeSerializationDependencies;
		LHSStartIndex += LHSExport.SerializationBeforeSerializationDependencies;
		if (LHSExport.CreateBeforeSerializationDependencies != RHSExport.CreateBeforeSerializationDependencies)
		{
			RHSDepSet.Reset();
			for (int32 Index = 0; Index < RHSExport.CreateBeforeSerializationDependencies; ++Index)
			{
				RHSDepSet.Add(RHSDepList[RHSStartIndex + Index]);
			}

			for (int32 Index = 0; Index < LHSExport.CreateBeforeSerializationDependencies; ++Index)
			{
				FPackageIndex LHSDepIndex = LHSDepList[LHSStartIndex + Index];
				FPackageIndex RHSDepIndex = FindResourcePackageIndex(LHSLinker->ImpExp(LHSDepIndex), LHSDepIndex.IsImport(), LHSLinker, RHSLinker);

				FSetElementId Id = RHSDepSet.FindId(RHSDepIndex);
				if (Id.IsValidId())
				{
					RHSDepSet.Remove(Id);
				}
				else
				{
					TStringBuilder<1024> Builder;
					Builder.Append(TEXT("Missing RHS CreateBeforeSerializationDependencies Preload Dependencies "));
					Builder.Append(LHSLinker->GetPathName(LHSDepIndex));
					Builder.Append(TEXT(" for resource "));
					Builder.Append(ResourcePathName);
					Diffs.Add(Builder.ToString());
				}

			}

			for (FPackageIndex RHSDepIndex : RHSDepSet)
			{
				TStringBuilder<1024> Builder;
				Builder.Append(TEXT("New RHS CreateBeforeSerializationDependencies Preload Dependencies "));
				Builder.Append(RHSLinker->GetPathName(RHSDepIndex));
				Builder.Append(TEXT(" for resource "));
				Builder.Append(ResourcePathName);
				Diffs.Add(Builder.ToString());
			}
		}

		RHSStartIndex += RHSExport.CreateBeforeSerializationDependencies;
		LHSStartIndex += LHSExport.CreateBeforeSerializationDependencies;
		if (LHSExport.SerializationBeforeCreateDependencies != RHSExport.SerializationBeforeCreateDependencies)
		{
			RHSDepSet.Reset();
			for (int32 Index = 0; Index < RHSExport.SerializationBeforeCreateDependencies; ++Index)
			{
				RHSDepSet.Add(RHSDepList[RHSStartIndex + Index]);
			}

			for (int32 Index = 0; Index < LHSExport.SerializationBeforeCreateDependencies; ++Index)
			{
				FPackageIndex LHSDepIndex = LHSDepList[LHSStartIndex + Index];
				FPackageIndex RHSDepIndex = FindResourcePackageIndex(LHSLinker->ImpExp(LHSDepIndex), LHSDepIndex.IsImport(), LHSLinker, RHSLinker);

				FSetElementId Id = RHSDepSet.FindId(RHSDepIndex);
				if (Id.IsValidId())
				{
					RHSDepSet.Remove(Id);
				}
				else
				{
					TStringBuilder<1024> Builder;
					Builder.Append(TEXT("Missing RHS SerializationBeforeCreateDependencies Preload Dependencies "));
					Builder.Append(LHSLinker->GetPathName(LHSDepIndex));
					Builder.Append(TEXT(" for resource "));
					Builder.Append(ResourcePathName);
					Diffs.Add(Builder.ToString());
				}

			}

			for (FPackageIndex RHSDepIndex : RHSDepSet)
			{
				TStringBuilder<1024> Builder;
				Builder.Append(TEXT("New RHS SerializationBeforeCreateDependencies Preload Dependencies "));
				Builder.Append(RHSLinker->GetPathName(RHSDepIndex));
				Builder.Append(TEXT(" for resource "));
				Builder.Append(ResourcePathName);
				Diffs.Add(Builder.ToString());
			}
		}

		RHSStartIndex += RHSExport.SerializationBeforeCreateDependencies;
		LHSStartIndex += LHSExport.SerializationBeforeCreateDependencies;
		if (LHSExport.CreateBeforeCreateDependencies != RHSExport.CreateBeforeCreateDependencies)
		{
			RHSDepSet.Reset();
			for (int32 Index = 0; Index < RHSExport.CreateBeforeCreateDependencies; ++Index)
			{
				RHSDepSet.Add(RHSDepList[RHSStartIndex + Index]);
			}

			for (int32 Index = 0; Index < LHSExport.CreateBeforeCreateDependencies; ++Index)
			{
				FPackageIndex LHSDepIndex = LHSDepList[LHSStartIndex + Index];
				FPackageIndex RHSDepIndex = FindResourcePackageIndex(LHSLinker->ImpExp(LHSDepIndex), LHSDepIndex.IsImport(), LHSLinker, RHSLinker);

				FSetElementId Id = RHSDepSet.FindId(RHSDepIndex);
				if (Id.IsValidId())
				{
					RHSDepSet.Remove(Id);
				}
				else
				{
					TStringBuilder<1024> Builder;
					Builder.Append(TEXT("Missing RHS CreateBeforeCreateDependencies Preload Dependencies "));
					Builder.Append(LHSLinker->GetPathName(LHSDepIndex));
					Builder.Append(TEXT(" for resource "));
					Builder.Append(ResourcePathName);
					Diffs.Add(Builder.ToString());
				}

			}

			for (FPackageIndex RHSDepIndex : RHSDepSet)
			{
				TStringBuilder<1024> Builder;
				Builder.Append(TEXT("New RHS CreateBeforeCreateDependencies Preload Dependencies "));
				Builder.Append(RHSLinker->GetPathName(RHSDepIndex));
				Builder.Append(TEXT(" for resource "));
				Builder.Append(ResourcePathName);
				Diffs.Add(Builder.ToString());
			}
		}
	}
	else if (LHSExport.FirstExportDependency != RHSExport.FirstExportDependency)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(RHSExport.FirstExportDependency == -1 ? TEXT("Missing RHS Cooked Preload Dependencies for resource") : TEXT("New RHS Cooked Preload Dependencies for resource"));
		Builder.Append(ResourcePathName);
		Diffs.Add(Builder.ToString());
	}

}

void FLinkerDiff::GenerateSearchableNameArrayDiff(const TArray<FName>& LHSNameArray, const TArray<FName>& RHSNameArray)
{
	TArray<FString>& Diffs = SearchableNameDiffs;
	TSet<FName> RHSNameSet(RHSNameArray);

	for (const FName& Name : LHSNameArray)
	{
		FSetElementId Id = RHSNameSet.FindId(Name);
		if (Id.IsValidId())
		{
			RHSNameSet.Remove(Id);
		}
		else
		{
			TStringBuilder<1024> Builder;
			Builder.Append(TEXT("Missing RHS searchable name: "));
			Builder.Append(Name.ToString());
			Diffs.Add(Builder.ToString());
		}
	}

	for (const FName& Name : RHSNameSet)
	{
		TStringBuilder<1024> Builder;
		Builder.Append(TEXT("New RHS searchable name: "));
		Builder.Append(Name.ToString());
		Diffs.Add(Builder.ToString());
	}
}
