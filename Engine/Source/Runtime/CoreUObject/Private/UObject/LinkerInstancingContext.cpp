// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerInstancingContext.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/NameTypes.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Templates/Tuple.h"
#include "Misc/AutomationTest.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"

LLM_DEFINE_TAG(Loading_LinkerInstancingContext);

#if PLATFORM_FORCE_UE_LOCK_USAGE && !UE_AUTORTFM
	typedef UE::TUniqueLock<UE::FMutex> FLinkerInstancingContextReadLock;
	typedef UE::TUniqueLock<UE::FMutex> FLinkerInstancingContextWriteLock;
	typedef UE::FMutex FLinkerInstancingLock;
#else
	typedef UE::TReadScopeLock<FTransactionallySafeRWLock> FLinkerInstancingContextReadLock;
	typedef UE::TWriteScopeLock<FTransactionallySafeRWLock> FLinkerInstancingContextWriteLock;
	typedef FTransactionallySafeRWLock FLinkerInstancingLock;
#endif

class FLinkerInstancingContext::FSharedLinkerInstancingContextData
{
	/** The shared content needs to be protected as it could be modified by the caller as well as the loading thread */
	mutable FLinkerInstancingLock Lock;
	/** Map of original package name to their instance counterpart. */
	FLinkerInstancedPackageMap InstancedPackageMap;
	/** Optional function to map original package name to their instance counterpart. The result of this function should be immutable, as it will be cached. */
	TFunction<FName(FName)> InstancedPackageMapFunc;
	/** Map of original top level asset path to their instance counterpart. */
	TMap<FTopLevelAssetPath, FTopLevelAssetPath> PathMapping;
	/** Tags can be used to determine some loading behavior. */
	TSet<FName> Tags;
	/** Remap soft object paths */
	std::atomic<bool> bSoftObjectPathRemappingEnabled { true };

public:
	FSharedLinkerInstancingContextData() = default;
	explicit FSharedLinkerInstancingContextData(TSet<FName> InTags)
		: Tags(InTags)
	{
	}

	explicit FSharedLinkerInstancingContextData(bool bInSoftObjectPathRemappingEnabled)
		: bSoftObjectPathRemappingEnabled(bInSoftObjectPathRemappingEnabled)
	{
	}

	explicit FSharedLinkerInstancingContextData(const FSharedLinkerInstancingContextData& Other)
		: InstancedPackageMap(Other.InstancedPackageMap)
		, InstancedPackageMapFunc(Other.InstancedPackageMapFunc)
		, PathMapping(Other.PathMapping)
		, Tags(Other.Tags)
		, bSoftObjectPathRemappingEnabled(Other.GetSoftObjectPathRemappingEnabled())
	{
	}
		
	void AddPackageMapping(FName Original, FName Instanced)
	{
		FLinkerInstancingContextWriteLock ScopeLock(Lock);
		LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
		InstancedPackageMap.AddPackageMapping(Original, Instanced);
	}

	bool FindPackageMapping(FName Original, FName& Instanced) const
	{
		FLinkerInstancingContextReadLock ScopeLock(Lock);
		LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
		if (const FName* InstancedPtr = InstancedPackageMap.InstancedPackageMapping.Find(FPackageId::FromName(Original)))
		{
			Instanced = *InstancedPtr;
			return true;
		}
		if (InstancedPackageMapFunc)
		{
			const FName MappedResult = InstancedPackageMapFunc(Original);
			if (MappedResult != Original)
			{
				Instanced = MappedResult;
				return true;
			}
		}
		return false;
	}

	bool FixupSoftObjectPath(FSoftObjectPath& InOutSoftObjectPath) const
	{
		FLinkerInstancingContextReadLock ScopeLock(Lock);
		LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
		return InstancedPackageMap.FixupSoftObjectPath(InOutSoftObjectPath);
	}

	FSoftObjectPath RemapPath(const FSoftObjectPath& Path) const
	{
		FLinkerInstancingContextReadLock ScopeLock(Lock);
		if (const FTopLevelAssetPath* Remapped = PathMapping.Find(Path.GetAssetPath()))
		{
			return FSoftObjectPath(*Remapped, Path.GetSubPathString());
		}
		return Path;
	}

	bool RemapPath(const FSoftObjectPath& Path, FSoftObjectPath& OutRemappedPath) const
	{
		FLinkerInstancingContextReadLock ScopeLock(Lock);
		if (const FTopLevelAssetPath* Remapped = PathMapping.Find(Path.GetAssetPath()))
		{
			OutRemappedPath = FSoftObjectPath(*Remapped, Path.GetSubPathString());
			return true;
		}

		OutRemappedPath = Path;
		return false;
	}

	bool IsInstanced() const
	{
		FLinkerInstancingContextReadLock ScopeLock(Lock);
		return InstancedPackageMap.IsInstanced() || InstancedPackageMapFunc || PathMapping.Num() > 0;
	}

	void EnableAutomationTest()
	{
		FLinkerInstancingContextWriteLock ScopeLock(Lock);
		InstancedPackageMap.EnableAutomationTest();
	}

	void BuildPackageMapping(FName Original, FName Instanced, bool bInSoftObjectPathRemappingEnabled)
	{
		FLinkerInstancingContextWriteLock ScopeLock(Lock);
		LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
		InstancedPackageMap.BuildPackageMapping(Original, Instanced, bInSoftObjectPathRemappingEnabled);
	}

	bool ValidatePackageMappingAgainst(FSharedLinkerInstancingContextData& Other)
	{
		FLinkerInstancingContextReadLock ScopeLock(Lock);
		FLinkerInstancingContextReadLock OtherScopeLock(Other.Lock);
	
		return InstancedPackageMap.ValidateAgainst(Other.InstancedPackageMap);
	}

	void AddTag(FName NewTag)
	{
		FLinkerInstancingContextWriteLock ScopeLock(Lock);
		LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
		Tags.Add(NewTag);
	}

	void AppendTags(const TSet<FName>& NewTags)
	{
		FLinkerInstancingContextWriteLock ScopeLock(Lock);
		LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
		Tags.Append(NewTags);
	}

	bool HasTag(FName Tag) const
	{
		FLinkerInstancingContextReadLock ScopeLock(Lock);
		return Tags.Contains(Tag);
	}

	void SetSoftObjectPathRemappingEnabled(bool bInSoftObjectPathRemappingEnabled)
	{
		bSoftObjectPathRemappingEnabled = bInSoftObjectPathRemappingEnabled;
	}

	bool GetSoftObjectPathRemappingEnabled() const
	{
		return bSoftObjectPathRemappingEnabled;
	}

	void AddPathMapping(FSoftObjectPath Original, FSoftObjectPath Instanced)
	{
		FLinkerInstancingContextWriteLock ScopeLock(Lock);
		LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
		PathMapping.Emplace(Original.GetAssetPath(), Instanced.GetAssetPath());
	}

	void SetPackageMappingFunc(TFunction<FName(FName)> InInstancedPackageMapFunc)
	{
		FLinkerInstancingContextWriteLock ScopeLock(Lock);
		InstancedPackageMapFunc = MoveTemp(InInstancedPackageMapFunc);
	}

	FName RemapPackage(const FName& PackageName)
	{
		bool bAddPackageMapping = false;

		FName RemappedPackageName;
		{
			FLinkerInstancingContextReadLock ScopeLock(Lock);
			
			bool bWasFound;
			RemappedPackageName = InstancedPackageMap.RemapPackage(PackageName, &bWasFound);

			if (!bWasFound && InstancedPackageMapFunc)
			{
				RemappedPackageName = InstancedPackageMapFunc(PackageName);
				bAddPackageMapping = RemappedPackageName != PackageName; 
			}
		}

		if (bAddPackageMapping)
		{
			FLinkerInstancingContextWriteLock ScopeLock(Lock);
			LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
			InstancedPackageMap.AddPackageMapping(PackageName, RemappedPackageName);
		}

		return RemappedPackageName;
	}

	FPackageId RemapPackageId(const FPackageId& PackageId)
	{
		FPackageId RemappedPackageId;
		{
			FLinkerInstancingContextReadLock ScopeLock(Lock);
			RemappedPackageId = InstancedPackageMap.RemapPackageId(PackageId);
		}
		return RemappedPackageId;
	}
};

FLinkerInstancingContext::FLinkerInstancingContext()
{
	LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
	SharedData = MakeShared<FSharedLinkerInstancingContextData>();
}

FLinkerInstancingContext::FLinkerInstancingContext(TSet<FName> InTags)
{
	LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
	SharedData = MakeShared<FSharedLinkerInstancingContextData>(MoveTemp(InTags));
}

FLinkerInstancingContext::FLinkerInstancingContext(bool bInSoftObjectPathRemappingEnabled)
{
	LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);
	SharedData = MakeShared<FSharedLinkerInstancingContextData>(bInSoftObjectPathRemappingEnabled);
}

FLinkerInstancingContext::FLinkerInstancingContext(const FLinkerInstancingContext& InOther, EContextSharingMode Mode)
{
	LLM_SCOPE_BYTAG(Loading_LinkerInstancingContext);

	switch (Mode)
	{
		case EContextSharingMode::Share:
		{	
			// Share all the data
			ImmutableData = InOther.ImmutableData;
			SharedData = InOther.SharedData;
		}
		break;
		case EContextSharingMode::Link:
		{
			if (InOther.ImmutableData.IsValid())
			{
				check(InOther.SharedData.IsValid());
				
				// other context already has shared immutable data, so we share all the data
				ImmutableData = InOther.ImmutableData;
				SharedData = InOther.SharedData;
			}
			else
			{
				// promote source shared data to immutable data for this context
				ImmutableData = InOther.SharedData;
				// create a new shared data object to store mappings added in this context
				SharedData = MakeShared<FSharedLinkerInstancingContextData>();
				// copy relevant settings from LinkedImmutableData
				SharedData->SetSoftObjectPathRemappingEnabled(InOther.GetSoftObjectPathRemappingEnabled());
			}
		}
		break;
		case EContextSharingMode::Copy:
		{
			// copy the shared data
			SharedData = MakeShared<FSharedLinkerInstancingContextData>(*InOther.SharedData.Get());
			// keep sharing the immtuable data
			ImmutableData = InOther.ImmutableData;
		} 
		break;
	}	
}

FLinkerInstancingContext::FLinkerInstancingContext(const FLinkerInstancingContext& Other)	
	: FLinkerInstancingContext(Other, EContextSharingMode::Share)
{
	
}

FLinkerInstancingContext FLinkerInstancingContext::DuplicateContext(const FLinkerInstancingContext& InLinkerInstancingContext)
{
	return FLinkerInstancingContext(InLinkerInstancingContext, EContextSharingMode::Copy);
}

void FLinkerInstancingContext::EnableAutomationTest() 
{ 
	check(!ImmutableData.IsValid());
	SharedData->EnableAutomationTest();
}

void FLinkerInstancingContext::BuildPackageMapping(FName Original, FName Instanced)
{
	SharedData->BuildPackageMapping(Original, Instanced, GetSoftObjectPathRemappingEnabled());

#if DO_ENSURE
	if (ImmutableData.IsValid())
	{
		// check we didn't already built a package mapping on it 
		ensureAlwaysMsgf(SharedData->ValidatePackageMappingAgainst(*ImmutableData), TEXT("Validate against ImmutableData failed after building PackageMapping for Original: %s, Instanced: %s"), *Original.ToString(), *Instanced.ToString());
	}
#endif
}

bool FLinkerInstancingContext::FindPackageMapping(FName Original, FName& Instanced) const
{
	bool bFound = SharedData->FindPackageMapping(Original, Instanced);

	if (!bFound && ImmutableData.IsValid())
	{
		bFound = ImmutableData->FindPackageMapping(Original, Instanced);
	}
	return bFound;
}

bool FLinkerInstancingContext::IsInstanced() const
{
	return SharedData->IsInstanced() || (ImmutableData.IsValid() && ImmutableData->IsInstanced());
}

/** Remap the package name from the import table to its instanced counterpart, otherwise return the name unmodified. */
FName FLinkerInstancingContext::RemapPackage(const FName& PackageName) const
{
	FName RemappedName = SharedData->RemapPackage(PackageName);
		
	if (ImmutableData.IsValid() && RemappedName == PackageName)
	{
		RemappedName = ImmutableData->RemapPackage(PackageName);
	}
	
	return RemappedName;
}

FPackageId FLinkerInstancingContext::RemapPackageId(const FPackageId& PackageId) const
{
	FPackageId RemappedId = SharedData->RemapPackageId(PackageId);
		
	if (ImmutableData.IsValid() && RemappedId == PackageId)
	{
		RemappedId = ImmutableData->RemapPackageId(PackageId);
	}
	
	return RemappedId;
}

/**
 * Remap the top level asset part of the path name to its instanced counterpart, otherwise return the name unmodified.
 * i.e. remaps /Path/To/Package.AssetName:Inner to /NewPath/To/NewPackage.NewAssetName:Inner
 */
FSoftObjectPath FLinkerInstancingContext::RemapPath(const FSoftObjectPath& Path) const
{
	FSoftObjectPath RemappedPath;
	bool bRemapped = SharedData->RemapPath(Path, RemappedPath);
		
	if (ImmutableData.IsValid() && !bRemapped)
	{
		RemappedPath = ImmutableData->RemapPath(Path);
	}

	return RemappedPath;
}

/** Add a mapping from a package name to a new package name. There should be no separators (. or :) in these strings. */
void FLinkerInstancingContext::AddPackageMapping(FName Original, FName Instanced)
{
#if DO_ENSURE
	if (ImmutableData.IsValid())
	{
		// validate immutable package mapping doesn't contain this mapping or maps to the same package
		FName AlreadyMapped;
		if (ImmutableData->FindPackageMapping(Original, AlreadyMapped))
		{
			ensureAlwaysMsgf(Instanced == AlreadyMapped, TEXT("Trying to add duplicate package mapping, this add is masked by the previous one (present in the Immutable linked InstancingContext) Original: %s, Instanced: %s, InMapping: %s"), *Original.ToString(), *Instanced.ToString(), *AlreadyMapped.ToString());
		}
	}
#endif

	SharedData->AddPackageMapping(Original, Instanced);
}

/** Add a mapping function from a package name to a new package name. This function should be thread-safe, as it can be invoked from ALT. */
void FLinkerInstancingContext::AddPackageMappingFunc(TFunction<FName(FName)> InInstancedPackageMapFunc)
{
	check(!ImmutableData.IsValid());
	SharedData->SetPackageMappingFunc(InInstancedPackageMapFunc);
}

/** Add a mapping from a top level asset path (/Path/To/Package.AssetName) to another. */
void FLinkerInstancingContext::AddPathMapping(FSoftObjectPath Original, FSoftObjectPath Instanced)
{
#if DO_ENSURE
	ensureAlwaysMsgf(Original.GetSubPathString().IsEmpty(),
		TEXT("Linker instance remap paths should be top-level assets only: %s->"), *Original.ToString());
	ensureAlwaysMsgf(Instanced.GetSubPathString().IsEmpty(),
		TEXT("Linker instance remap paths should be top-level assets only: ->%s"), *Instanced.ToString());

	if (ImmutableData.IsValid())
	{
		// validate immutable path mapping doesn't contain this mapping or maps to the same package
		FSoftObjectPath AlreadyMapped;
		if (ImmutableData->RemapPath(Original, AlreadyMapped))
		{
			ensureAlwaysMsgf(Instanced == AlreadyMapped, TEXT("Trying to add duplicate path mapping, this add is masked by the previous one (present in the Immutable linked InstancingContext) Original: %s, Instanced: %s, InMapping: %s"), *Original.ToString(), *Instanced.ToString(), *AlreadyMapped.ToString());
		}	
	}
#endif

	SharedData->AddPathMapping(Original, Instanced);
}

void FLinkerInstancingContext::AddTag(FName NewTag)
{
	SharedData->AddTag(NewTag);
}

void FLinkerInstancingContext::AppendTags(const TSet<FName>& NewTags)
{
	SharedData->AppendTags(NewTags);
}

bool FLinkerInstancingContext::HasTag(FName Tag) const
{
	return SharedData->HasTag(Tag) || (ImmutableData.IsValid() && ImmutableData->HasTag(Tag));
}

void FLinkerInstancingContext::SetSoftObjectPathRemappingEnabled(bool bInSoftObjectPathRemappingEnabled)
{
	check(!ImmutableData.IsValid());
	SharedData->SetSoftObjectPathRemappingEnabled(bInSoftObjectPathRemappingEnabled);
}

bool FLinkerInstancingContext::GetSoftObjectPathRemappingEnabled() const
{
	return SharedData->GetSoftObjectPathRemappingEnabled() || (ImmutableData.IsValid() && ImmutableData->GetSoftObjectPathRemappingEnabled());
}

void FLinkerInstancingContext::FixupSoftObjectPath(FSoftObjectPath& InOutSoftObjectPath) const
{
	if (IsInstanced() && GetSoftObjectPathRemappingEnabled())
	{
		// Try remapping AssetPathName before remapping LongPackageName. RemapPath looks into both SharedData & ImmutableData
		if (FSoftObjectPath RemappedAssetPath = RemapPath(InOutSoftObjectPath); RemappedAssetPath != InOutSoftObjectPath)
		{
			InOutSoftObjectPath = RemappedAssetPath;
		}
		else
		{
			if (!SharedData->FixupSoftObjectPath(InOutSoftObjectPath) && ImmutableData)
			{
				ImmutableData->FixupSoftObjectPath(InOutSoftObjectPath);
			}
		}
	}
}

void FLinkerInstancedPackageMap::AddPackageMapping(FName Original, FName Instanced)
{
	FName Key = InstanceMappingDirection == EInstanceMappingDirection::OriginalToInstanced ? Original : Instanced;
	FName Value = InstanceMappingDirection == EInstanceMappingDirection::OriginalToInstanced ? Instanced : Original;

	const FName& StoredValue = InstancedPackageMapping.FindOrAdd(FPackageId::FromName(Key), Value);
	if (StoredValue == Value)
	{
		bIsInstanced |= !Value.IsNone();
	}
	else
	{
		ensureAlwaysMsgf(StoredValue == Value, TEXT("Trying to add duplicate package mapping, this add is masked by the previous one. Original: %s, Instanced: %s, InMapping: %s"), *Original.ToString(), *Instanced.ToString(), *StoredValue.ToString());
	}
}

void FLinkerInstancedPackageMap::BuildPackageMapping(FName Original, FName Instanced, const bool bBuildWorldPartitionCellMapping)
{
	check(GeneratedPackagesFolder.IsEmpty() && InstancedPackageSuffix.IsEmpty() && InstancedPackagePrefix.IsEmpty());

	AddPackageMapping(Original, Instanced);

	if (bBuildWorldPartitionCellMapping && bEnableNonEditorPath)
	{
#if WITH_EDITOR
		// This code path should only be enabled while testing
		check(GIsAutomationTesting);
#endif

		FNameBuilder TmpOriginal(Original);
		FNameBuilder TmpInstanced(Instanced);
		FStringView OriginalView = TmpOriginal.ToView();
		FStringView InstancedView = TmpInstanced.ToView();

		const int32 Index = InstancedView.Find(OriginalView, 0, ESearchCase::IgnoreCase);

		// Stash the suffix used for this instance so we can also apply it to generated packages
		if (Index != INDEX_NONE)
		{
			InstancedPackagePrefix = InstancedView.Mid(0, Index);
			InstancedPackageSuffix = InstancedView.Mid(Index + OriginalView.Len());
		}

		// Is this a generated partitioned map package? If so, we'll also need to handle re-mapping paths to our persistent map package
		if (!InstancedPackagePrefix.IsEmpty() || !InstancedPackageSuffix.IsEmpty())
		{
			const FStringView GeneratedFolderName = TEXTVIEW("/_Generated_/");
			
			// Does this package path include the generated folder?
			if (const int32 GeneratedFolderStartIndex = OriginalView.Find(GeneratedFolderName, 0, ESearchCase::IgnoreCase); GeneratedFolderStartIndex != INDEX_NONE)
			{
				// ... and is that generated folder immediately preceding the package name?
				const int32 GeneratedFolderEndIndex = GeneratedFolderStartIndex + GeneratedFolderName.Len();
				if (const int32 ExtraSlashIndex = OriginalView.Find(TEXTVIEW("/"), GeneratedFolderEndIndex, ESearchCase::IgnoreCase); ExtraSlashIndex == INDEX_NONE)
				{
					GeneratedPackagesFolder = OriginalView.Left(GeneratedFolderEndIndex);
					const FString PersistentSourcePackage(OriginalView.Left(GeneratedFolderStartIndex));
									
					FNameBuilder PersistentPackageInstanceNameBuilder;
					PersistentPackageInstanceNameBuilder.Append(InstancedPackagePrefix);
					PersistentPackageInstanceNameBuilder.Append(PersistentSourcePackage);
					PersistentPackageInstanceNameBuilder.Append(InstancedPackageSuffix);

					const FName PersistentPackageInstanceName(PersistentPackageInstanceNameBuilder);

					AddPackageMapping(*PersistentSourcePackage, PersistentPackageInstanceName);
				}
			}

			if (GeneratedPackagesFolder.IsEmpty())
			{
				GeneratedPackagesFolder = OriginalView;
				GeneratedPackagesFolder += GeneratedFolderName;
			}
		}
	}
}

bool FLinkerInstancedPackageMap::ValidateAgainst(FLinkerInstancedPackageMap& OtherPackageMap) const
{
	// This method ensures the object passed as a parameter doesn't contain any mapping that diverge from us
	bool bResult = true;

	for (TPair<FPackageId, FName> It : InstancedPackageMapping)
	{
		FName* OtherInstanced = OtherPackageMap.InstancedPackageMapping.Find(It.Key);

		bool bThisResult = (!OtherInstanced || (*OtherInstanced == It.Value));
		
		ensureAlwaysMsgf(bThisResult, TEXT("Discrepancy in instance package mapping for 0x%llx, maps to %s and %s"), It.Key.Value(), *It.Value.ToString(), *OtherInstanced->ToString());

		bResult &= bThisResult;
	}
	
	bResult &= (GeneratedPackagesFolder == OtherPackageMap.GeneratedPackagesFolder || OtherPackageMap.GeneratedPackagesFolder.IsEmpty());
	bResult &= (InstancedPackageSuffix == OtherPackageMap.InstancedPackageSuffix || OtherPackageMap.InstancedPackageSuffix.IsEmpty());
	bResult &= (InstancedPackagePrefix == OtherPackageMap.InstancedPackagePrefix || OtherPackageMap.InstancedPackagePrefix.IsEmpty());
	
	return bResult;
}

bool FLinkerInstancedPackageMap::FixupSoftObjectPath(FSoftObjectPath& InOutSoftObjectPath) const
{
	if (IsInstanced())
	{
		if (FName LongPackageName = InOutSoftObjectPath.GetLongPackageFName(), RemappedPackage = RemapPackage(LongPackageName); RemappedPackage != LongPackageName)
		{
			InOutSoftObjectPath = FSoftObjectPath(FTopLevelAssetPath(RemappedPackage, InOutSoftObjectPath.GetAssetFName()), InOutSoftObjectPath.GetSubPathString());
			return true;
		}
		else if (!GeneratedPackagesFolder.IsEmpty())
		{
#if WITH_EDITOR
			// This code path should only be enabled while testing
			check(GIsAutomationTesting);
#endif
			check(!InstancedPackagePrefix.IsEmpty() || !InstancedPackageSuffix.IsEmpty());

			FNameBuilder TmpSoftObjectPathBuilder;
			InOutSoftObjectPath.ToString(TmpSoftObjectPathBuilder);

			// Does this package path start with the generated folder path?
			FStringView TmpSoftObjectPathView = TmpSoftObjectPathBuilder.ToView();
			if (const int32 GeneratedFolderIndex = TmpSoftObjectPathView.Find(GeneratedPackagesFolder, 0, ESearchCase::IgnoreCase); GeneratedFolderIndex != INDEX_NONE)
			{
				check(GeneratedFolderIndex == 0 || (TmpSoftObjectPathView.StartsWith(InstancedPackagePrefix) && GeneratedFolderIndex == InstancedPackagePrefix.Len()));

				// ... and is that generated folder path immediately preceding the package name?
				if (const int32 ExtraSlashIndex = TmpSoftObjectPathView.Find(TEXTVIEW("/"), InstancedPackagePrefix.Len() + GeneratedPackagesFolder.Len(), ESearchCase::IgnoreCase); ExtraSlashIndex == INDEX_NONE)
				{
					FNameBuilder PackageNameBuilder;

					if (InstanceMappingDirection == EInstanceMappingDirection::OriginalToInstanced)
					{
						PackageNameBuilder.Append(InstancedPackagePrefix);
					}
					
					PackageNameBuilder.Append(InOutSoftObjectPath.GetLongPackageName().Mid(GeneratedFolderIndex));

					if (InstanceMappingDirection == EInstanceMappingDirection::OriginalToInstanced)
					{
						if (!PackageNameBuilder.ToView().EndsWith(InstancedPackageSuffix))
						{
							PackageNameBuilder.Append(InstancedPackageSuffix);
						}
					}
					else
					{
						check(InstanceMappingDirection == EInstanceMappingDirection::InstancedToOriginal);
						if (PackageNameBuilder.ToView().EndsWith(InstancedPackageSuffix))
						{
							PackageNameBuilder.RemoveSuffix(InstancedPackageSuffix.Len());
						}
					}
					FTopLevelAssetPath SuffixTopLevelAsset(FName(PackageNameBuilder), InOutSoftObjectPath.GetAssetFName());
					InOutSoftObjectPath = FSoftObjectPath(SuffixTopLevelAsset, InOutSoftObjectPath.GetSubPathString());
					return true;
				}
			}
		}
	}
	return false;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLinkerInstancingContextTests, "System.CoreUObject.LinkerInstancingContext", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter);
bool FLinkerInstancingContextTests::RunTest(const FString& Parameters)
{
	// Disabled SoftObjectPath Remapping
	{
		FName MappedSourcePackage("/Game/PathA/PathB/PackageName");
		FName UnmappedSourcePackage("/Game/PathA/PathB/PackageNameOther");
		FName InstancedPackage("/Game/PathA/PathB/PackageName_Instance");

		const FSoftObjectPath MappedSourcePackagePath(FTopLevelAssetPath(MappedSourcePackage, FName("PackageName")));
		const FSoftObjectPath MappedSourceObjectPath(FTopLevelAssetPath(MappedSourcePackage, FName("PackageName")), "PersistentLevel.Actor.Component");

		const bool bSoftObjectPathRemappingEnabled = false;
		FLinkerInstancingContext LinkerInstancingContext(bSoftObjectPathRemappingEnabled);
		LinkerInstancingContext.EnableAutomationTest();
		LinkerInstancingContext.BuildPackageMapping(MappedSourcePackage, InstancedPackage);

		TestEqual("RemapPackage with existing mapping - bSoftObjectPathRemappingEnabled(false)", LinkerInstancingContext.RemapPackage(MappedSourcePackage), InstancedPackage);
		TestEqual("RemapPackage without mapping - bSoftObjectPathRemappingEnabled(false)", LinkerInstancingContext.RemapPackage(UnmappedSourcePackage), UnmappedSourcePackage);

		FSoftObjectPath CopyMappedSourceObjectPackage = MappedSourcePackagePath;
		LinkerInstancingContext.FixupSoftObjectPath(CopyMappedSourceObjectPackage);
		TestEqual("FixupSoftObjectPath with no SubPathString - bSoftObjectPathRemappingEnabled(false)", CopyMappedSourceObjectPackage, MappedSourcePackagePath);

		FSoftObjectPath CopyMappedSourceObjectPath = MappedSourceObjectPath;
		LinkerInstancingContext.FixupSoftObjectPath(CopyMappedSourceObjectPath);
		TestEqual("FixupSoftObjectPath with SubPathString - bSoftObjectPathRemappingEnabled(false)", CopyMappedSourceObjectPath, MappedSourceObjectPath);
	}

	// Enabled SoftObjectPath Remapping
	{
		FName MappedSourcePackage("/Game/PathA/PathB/PackageName");
		FName UnmappedSourcePackage("/Game/PathA/PathB/PackageNameOther");
		FName InstancedPackage("/Game/PathA/PathB/PackageName_Instance");

		const FSoftObjectPath MappedSourcePackagePath(FTopLevelAssetPath(MappedSourcePackage, FName("PackageName")));
		const FSoftObjectPath RemappedPackagePath(FTopLevelAssetPath(InstancedPackage, MappedSourcePackagePath.GetAssetFName()));
		const FSoftObjectPath MappedSourceObjectPath(FTopLevelAssetPath(MappedSourcePackage, FName("PackageName")), "PersistentLevel.Actor.Component");
		const FSoftObjectPath RemappedObjectPath(FTopLevelAssetPath(InstancedPackage, MappedSourceObjectPath.GetAssetFName()), MappedSourceObjectPath.GetSubPathString());

		const bool bSoftObjectPathRemappingEnabled = true;
		FLinkerInstancingContext LinkerInstancingContext(bSoftObjectPathRemappingEnabled);
		LinkerInstancingContext.EnableAutomationTest();
		LinkerInstancingContext.BuildPackageMapping(MappedSourcePackage, InstancedPackage);

		TestEqual("RemapPackage with existing mapping - bSoftObjectPathRemappingEnabled(true)", LinkerInstancingContext.RemapPackage(MappedSourcePackage), InstancedPackage);
		TestEqual("RemapPackage without mapping - bSoftObjectPathRemappingEnabled(true)", LinkerInstancingContext.RemapPackage(UnmappedSourcePackage), UnmappedSourcePackage);

		FSoftObjectPath CopyMappedSourceObjectPackage = MappedSourcePackagePath;
		LinkerInstancingContext.FixupSoftObjectPath(CopyMappedSourceObjectPackage);
		TestEqual("FixupSoftObjectPath with no SubPathString - bSoftObjectPathRemappingEnabled(true)", CopyMappedSourceObjectPackage, RemappedPackagePath);

		FSoftObjectPath CopyMappedSourceObjectPath = MappedSourceObjectPath;
		LinkerInstancingContext.FixupSoftObjectPath(CopyMappedSourceObjectPath);
		TestEqual("FixupSoftObjectPath with SubPathString - bSoftObjectPathRemappingEnabled(true)", CopyMappedSourceObjectPath, RemappedObjectPath);
	}
	
	auto TestGeneratedPathFixup = [this](FName InSourceGeneratorPackage, FName InInstancedGeneratorPackage, FName InSourceGeneratedPackage, FName InInstancedGeneratedPackage, const FString& Description)
	{
		const bool bSoftObjectPathRemappingEnabled = true;
		FLinkerInstancingContext LinkerInstancingContext(bSoftObjectPathRemappingEnabled);
		LinkerInstancingContext.EnableAutomationTest();
		LinkerInstancingContext.BuildPackageMapping(InSourceGeneratedPackage, InInstancedGeneratedPackage);

		// Reverse mapping
		FLinkerInstancedPackageMap InstancingMap(FLinkerInstancedPackageMap::EInstanceMappingDirection::InstancedToOriginal);
		InstancingMap.EnableAutomationTest();
		InstancingMap.BuildPackageMapping(InSourceGeneratedPackage, InInstancedGeneratedPackage);

		TestEqual(FString::Format(TEXT("{0} - RemapPackage - _Generated_ Package with Suffix"), { Description }), LinkerInstancingContext.RemapPackage(InSourceGeneratedPackage), InInstancedGeneratedPackage);

		auto TestSoftObjectPathFixup = [this, &LinkerInstancingContext, &InstancingMap](const FSoftObjectPath& InSource, const FSoftObjectPath& InExpectedResult, const FString& Description)
		{
			FSoftObjectPath CopySource = InSource;
			LinkerInstancingContext.FixupSoftObjectPath(CopySource);
			TestEqual(FString::Format(TEXT("{0} - FixupSoftObjectPath"), { Description }), CopySource, InExpectedResult);

			LinkerInstancingContext.FixupSoftObjectPath(CopySource);
			TestEqual(FString::Format(TEXT("{0} - FixupSoftObjectPath already fixed up"), { Description }), CopySource, InExpectedResult);

			InstancingMap.FixupSoftObjectPath(CopySource);
			TestEqual(FString::Format(TEXT("{0} - Reverse FixupSoftObjectPath"), { Description }), CopySource, InSource);
		};

		const FSoftObjectPath SourcePackagePath(FTopLevelAssetPath(InSourceGeneratorPackage, FName("PackageName")));
		const FSoftObjectPath RemappedPackagePath(FTopLevelAssetPath(InInstancedGeneratorPackage, SourcePackagePath.GetAssetFName()));
		TestSoftObjectPathFixup(SourcePackagePath, RemappedPackagePath, Description);

		const FSoftObjectPath SourceObjectPath(FTopLevelAssetPath(InSourceGeneratorPackage, FName("PackageName")), "PersistentLevel.Actor.Component");
		const FSoftObjectPath RemappedSourceObjectPath(FTopLevelAssetPath(InInstancedGeneratorPackage, SourceObjectPath.GetAssetFName()), SourceObjectPath.GetSubPathString());
		TestSoftObjectPathFixup(SourceObjectPath, RemappedSourceObjectPath, Description);

		const FSoftObjectPath SourceGeneratedPackagePath(FTopLevelAssetPath(InSourceGeneratedPackage, FName("PackageName")));
		const FSoftObjectPath RemappedGeneratedPackagePath(FTopLevelAssetPath(InInstancedGeneratedPackage, SourceGeneratedPackagePath.GetAssetFName()));
		TestSoftObjectPathFixup(SourceGeneratedPackagePath, RemappedGeneratedPackagePath, Description);

		const FSoftObjectPath SourceGeneratedObjectPath(FTopLevelAssetPath(InSourceGeneratedPackage, FName("PackageName")), "PersistentLevel.Actor.Component");
		const FSoftObjectPath RemappedSourceGeneratedObjectPath(FTopLevelAssetPath(InInstancedGeneratedPackage, SourceGeneratedObjectPath.GetAssetFName()), SourceGeneratedObjectPath.GetSubPathString());
		TestSoftObjectPathFixup(SourceGeneratedObjectPath, RemappedSourceGeneratedObjectPath, Description);
	};

	{
		// Generated Package Mapping (Suffix)
		FName SourcePackage("/Game/PathA/PathB/PackageName");
		FName InstancedPackage("/Game/PathA/PathB/PackageName_LevelInstance1");
		FName SourceGeneratedPackage("/Game/PathA/PathB/PackageName/_Generated_/1AEHCD0PRR98XVM12PU4N1D8X");
		FName InstancedGeneratedPackage("/Game/PathA/PathB/PackageName/_Generated_/1AEHCD0PRR98XVM12PU4N1D8X_LevelInstance1");

		TestGeneratedPathFixup(SourcePackage, InstancedPackage, SourceGeneratedPackage, InstancedGeneratedPackage, TEXT("Suffix"));
	}

	{
		// Generated Package Mapping (Prefix)
		FName SourcePackage("/Game/PathA/PathB/PackageName");
		FName InstancedPackage("/Temp/Game/PathA/PathB/PackageName");
		FName SourceGeneratedPackage("/Game/PathA/PathB/PackageName/_Generated_/1AEHCD0PRR98XVM12PU4N1D8X");
		FName InstancedGeneratedPackage("/Temp/Game/PathA/PathB/PackageName/_Generated_/1AEHCD0PRR98XVM12PU4N1D8X");

		TestGeneratedPathFixup(SourcePackage, InstancedPackage, SourceGeneratedPackage, InstancedGeneratedPackage, TEXT("Prefix"));
	}

	{
		// Generated Package Mapping (Prefix + Suffix)
		FName SourcePackage("/Game/PathA/PathB/PackageName");
		FName InstancedPackage("/Temp/Game/PathA/PathB/PackageName_LevelInstance1");
		FName SourceGeneratedPackage("/Game/PathA/PathB/PackageName/_Generated_/1AEHCD0PRR98XVM12PU4N1D8X");
		FName InstancedGeneratedPackage("/Temp/Game/PathA/PathB/PackageName/_Generated_/1AEHCD0PRR98XVM12PU4N1D8X_LevelInstance1");

		TestGeneratedPathFixup(SourcePackage, InstancedPackage, SourceGeneratedPackage, InstancedGeneratedPackage, TEXT("Suffix+Prefix"));
	}

	return true;
}
