// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerLoad.h"
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
#include "Algo/ForEach.h"
#include "Algo/TopologicalSort.h"
#endif// UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
#include "AssetRegistry/AssetData.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Stats/StatsMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IConsoleManager.h"
#include "Misc/SlowTask.h"
#include "Async/Async.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ObjectThumbnail.h"
#include "Misc/App.h"
#include "UObject/CoreRedirects.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/MetaData.h"
#include "UObject/LinkerLoadImportBehavior.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/PackageResourceIoDispatcherBackend.h"
#include "UObject/PackageTrailer.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectSerializeContext.h"
#include "Misc/PackageName.h"
#include "Blueprint/BlueprintSupport.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PathViews.h"
#include "Misc/PreloadableFile.h"
#include "Misc/SecureHash.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/DebuggingDefines.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "ProfilingDebugging/CookStats.h"
#include "UObject/LinkerPlaceholderBase.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/LinkerPlaceholderFunction.h"
#include "UObject/LinkerManager.h"
#include "UObject/ObjectSerializeAccessScope.h"
#include "UObject/PropertyBagRepository.h"
#include "Serialization/DeferredMessageLog.h"
#include "UObject/UObjectThreadContext.h"
#include "Serialization/AsyncLoading.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "HAL/LowLevelMemStats.h"
#include "HAL/ThreadHeartBeat.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Serialization/BulkData.h"
#include "Serialization/AsyncLoadingPrivate.h"
#include "Serialization/Formatters/BinaryArchiveFormatter.h"
#include "Serialization/Formatters/JsonArchiveInputFormatter.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "Serialization/StructuredArchiveChildReader.h"
#include "Serialization/UnversionedPropertySerialization.h"
#include "Serialization/LoadTimeTracePrivate.h"
#include "Serialization/EditorBulkData.h"
#include "Serialization/ImportTypeHierarchy.h"
#include "HAL/FileManager.h"
#include "UObject/CoreRedirects.h"
#include "UObject/ICookInfo.h"
#include "UObject/PackageRelocation.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/StringBuilder.h"
#include "Misc/EngineBuildSettings.h"
#include "Internationalization/GatherableTextData.h"
#include "Async/MappedFileHandle.h"
#include "Async/UniqueLock.h"
#include "CoreGlobalsInternal.h"

class FTexture2DResourceMem;

#define LOCTEXT_NAMESPACE "LinkerLoad"

DECLARE_STATS_GROUP_VERBOSE(TEXT("Linker Load"), STATGROUP_LinkerLoad, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("Linker Preload"),STAT_LinkerPreload,STATGROUP_LinkerLoad);
DECLARE_CYCLE_STAT(TEXT("Linker Precache"),STAT_LinkerPrecache,STATGROUP_LinkerLoad);
DECLARE_CYCLE_STAT(TEXT("Linker Serialize"),STAT_LinkerSerialize,STATGROUP_LinkerLoad);
DECLARE_CYCLE_STAT(TEXT("Linker Load Deferred"), STAT_LinkerLoadDeferred, STATGROUP_LinkerLoad);

DECLARE_STATS_GROUP( TEXT( "Linker Count" ), STATGROUP_LinkerCount, STATCAT_Advanced );
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Linker Count"), STAT_LinkerCount, STATGROUP_LinkerCount);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Linker Count (Text Assets)"), STAT_TextAssetLinkerCount, STATGROUP_LinkerCount);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Live Linker Count"), STAT_LiveLinkerCount, STATGROUP_LinkerCount);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Fixup editor-only flags time"), STAT_EditorOnlyFixupTime, STATGROUP_LinkerCount);

FName FLinkerLoad::NAME_LoadErrors("LoadErrors");

LLM_DEFINE_TAG(UObject_Linker);
LLM_DEFINE_TAG(UObject_FLinkerLoad);

/**
* Helper function to determine and trace the most important asset class.
*/
void TrackPackageAssetClass(UPackage* Package, FLinkerLoad& LinkerLoad, const TArray<FObjectExport>& Exports)
{
#if ENABLE_COOK_STATS
	if (!ShouldTracePackageInfo() || Exports.Num() == 0 || !Package)
	{
		return;
	}

	FName PackageName = Package->GetFName();
	TStringBuilder<256> PackageNameStr(InPlace, PackageName);
	FStringView PackageLeafName = FPathViews::GetCleanFilename(PackageNameStr);
	const FObjectExport* MostImportant = nullptr;
	for (const FObjectExport& Export : Exports)
	{
		if (Export.bIsAsset && Export.ClassIndex.IsImport())
		{
			if (WriteToString<256>(Export.ObjectName) == PackageLeafName)
			{
				MostImportant = &Export;
				break;
			}
			if (!MostImportant)
			{
				MostImportant = &Export;
			}
		}
	}
	if (MostImportant)
	{
		TracePackageAssetClass(PackageName.ToUnstableInt(), LinkerLoad.Imp(MostImportant->ClassIndex).ObjectName.ToString());
	}
#endif
}

EPackageSegment GetBulkDataPackageSegmentFromFlags(const EBulkDataFlags BulkDataFlags, bool bLoadingFromCookedPackage)
{
	if (FBulkData::HasFlags(BulkDataFlags, BULKDATA_PayloadInSeparateFile) == false)
	{
		return bLoadingFromCookedPackage ? EPackageSegment::Exports : EPackageSegment::Header;
	}
	else if (BulkDataFlags & BULKDATA_OptionalPayload )
	{
		return EPackageSegment::BulkDataOptional;
	}
	else if (BulkDataFlags & BULKDATA_MemoryMappedPayload)
	{
		return EPackageSegment::BulkDataMemoryMapped;
	}
	else
	{
		return EPackageSegment::BulkDataDefault;
	}
}

#if WITH_EDITOR
bool FLinkerLoad::ShouldCreateThrottledSlowTask() const
{
	return ShouldReportProgress();
}

COREUOBJECT_API int32 GTreatVerifyImportErrorsAsWarnings = 0;
static FAutoConsoleVariableRef CVarTreatVerifyImportErrorsAsWarnings(
	TEXT("linker.TreatVerifyImportErrorsAsWarnings"),
	GTreatVerifyImportErrorsAsWarnings,
	TEXT("If true, the errors emitted due to verify import failures will be warnings instead."),
	ECVF_Default
);
#endif // WITH_EDITOR


int32 GAllowCookedDataInEditorBuilds = 0;
static FAutoConsoleVariableRef CVarAllowCookedDataInEditorBuilds(
	TEXT("cook.AllowCookedDataInEditorBuilds"),
		GAllowCookedDataInEditorBuilds,
	TEXT("If true, allows cooked assets to be loaded in the editor."),
	ECVF_Default
);

#if WITH_EDITOR
// force the cvar value in HybridCookerEditor mode
FDelayedAutoRegisterHelper GInitDevice(EDelayedRegisterRunPhase::IniSystemReady, []
	{
		if (IsRunningHybridCookedEditor())
		{
			GAllowCookedDataInEditorBuilds = 1;
		}
	});
#endif

int32 GSkipAsyncLoaderForCookedData = 0;
static FAutoConsoleVariableRef CVarSkipAsyncLoaderForCookedData(
	TEXT("cook.SkipAsyncLoaderForCookedData"),
	GSkipAsyncLoaderForCookedData,
	TEXT("If true, skip the async loader and load package header synchronously to reduce ping/pong between threads."),
	ECVF_Default
);

int32 GEnforcePackageCompatibleVersionCheck = 1;
static FAutoConsoleVariableRef CEnforcePackageCompatibleVersionCheck(
	TEXT("s.EnforcePackageCompatibleVersionCheck"),
	GEnforcePackageCompatibleVersionCheck,
	TEXT("If true, package loading will fail if the version stored in the package header is newer than the current engine version"),
	ECVF_Default
);

bool IsEnforcePackageCompatibleVersionCheck()
{
	return GEnforcePackageCompatibleVersionCheck != 0;
}

/**
 * Required to load packages saved from the editor domain between UE 5.0 and 5.2, the cvar is only provided in case the fix causes
 * unintended problems so that it can be disabled quickly.
 */
static TAutoConsoleVariable<bool> CVarApplyBulkDataFix(
	TEXT("Serialization.ApplyBulkDataOffsetFix"),
	true,
	TEXT("When true, we will try to fix potentially bad bulkdata offsets"));

bool GEnsureSerializePackageTrailer = true;
static FAutoConsoleVariableRef CVarEnsureSerializePackageTrailer(
	TEXT("Serialization.EnableSerializePackageTrailerEnsure"),
	GEnsureSerializePackageTrailer,
	TEXT("When true, an ensure will fire if we detect incorrect values when trying to serialize the package trailer"));

#if UE_INTERNAL_LINKER_BOUNDS
namespace UE::Private
{
	static bool bEnforceLinkerObjectBounds = true;
	static FAutoConsoleVariableRef CVarEnforceLinkerObjectBounds(
		TEXT("Serialization.EnforceLinkerObjectBounds"), bEnforceLinkerObjectBounds,
		TEXT("If true uasset loads will not read outside of the object bounds as defined in the uasset"),
		ECVF_Default);

	inline bool ShouldEnforceLinkerObjectBounds()
	{
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
		return true;
#else
		return bEnforceLinkerObjectBounds;
#endif
	}
}
#endif// UE_INTERNAL_LINKER_BOUNDS

/**
* Test whether the given package index is a valid import or export in this package
*/
bool FLinkerLoad::IsValidPackageIndex(FPackageIndex InIndex)
{
	return (InIndex.IsImport() && ImportMap.IsValidIndex(InIndex.ToImport()))
		|| (InIndex.IsExport() && ExportMap.IsValidIndex(InIndex.ToExport()));
}

bool FLinkerLoad::bActiveRedirectsMapInitialized = false;



/**
* DEPRECATED: Replace with FCoreRedirects format for newly added ini entries
*
* Here is the format for the ClassRedirection:
*
*  ; Basic redirects
*  ;ActiveClassRedirects=(OldClassName="MyClass",NewClassName="NewNativePackage.MyClass")
*	ActiveClassRedirects=(OldClassName="CylinderComponent",NewClassName="CapsuleComponent")
*  Note: For class name redirects, the OldClassName must be the plain OldClassName, it cannot be OldPackage.OldClassName
*
*	; Keep both classes around, but convert any existing instances of that object to a particular class (insert into the inheritance hierarchy
*	;ActiveClassRedirects=(OldClassName="MyClass",NewClassName="MyClassParent",InstanceOnly="true")
*
*/

void FLinkerLoad::CreateActiveRedirectsMap(const FString& GEngineIniName)
{
	// Soft deprecated, replaced by FCoreRedirects, but it will still read the old format for the foreseeable future

	// mark that this has been done at least once
	bActiveRedirectsMapInitialized = true;

	if (GConfig)
	{
		const FConfigSection* PackageRedirects = GConfig->GetSection( TEXT("/Script/Engine.Engine"), false, GEngineIniName );
		if (PackageRedirects)
		{
			TArray<FCoreRedirect> NewRedirects;
			FDeferredMessageLog RedirectErrors(NAME_LoadErrors);

			static FName ActiveClassRedirectsKey("ActiveClassRedirects");
			for( FConfigSection::TConstIterator It(*PackageRedirects); It; ++It )
			{
				if (It.Key() == ActiveClassRedirectsKey)
				{
					FName OldClassName = NAME_None;
					FName NewClassName = NAME_None;
					FName ObjectName = NAME_None;
					FName OldSubobjName = NAME_None;
					FName NewSubobjName = NAME_None;
					FName NewClassClass = NAME_None;
					FName NewClassPackage = NAME_None;

					bool bInstanceOnly = false;

					FParse::Bool( *It.Value().GetValue(), TEXT("InstanceOnly="), bInstanceOnly );
					FParse::Value( *It.Value().GetValue(), TEXT("ObjectName="), ObjectName );

					FParse::Value( *It.Value().GetValue(), TEXT("OldClassName="), OldClassName );
					FParse::Value( *It.Value().GetValue(), TEXT("NewClassName="), NewClassName );

					FParse::Value( *It.Value().GetValue(), TEXT("OldSubobjName="), OldSubobjName );
					FParse::Value( *It.Value().GetValue(), TEXT("NewSubobjName="), NewSubobjName );

					FParse::Value( *It.Value().GetValue(), TEXT("NewClassClass="), NewClassClass );
					FParse::Value( *It.Value().GetValue(), TEXT("NewClassPackage="), NewClassPackage );

					if (NewSubobjName != NAME_None || OldSubobjName != NAME_None)
					{
						check(OldSubobjName != NAME_None && OldClassName != NAME_None );
						FCoreRedirect& Redirect = NewRedirects.Emplace_GetRef(ECoreRedirectFlags::Type_Class, OldClassName.ToString(), OldClassName.ToString());
						Redirect.ValueChanges.Add(OldSubobjName.ToString(), NewSubobjName.ToString());
					}
					//instances only
					else if( bInstanceOnly )
					{
						// If NewClassName is none, register as removed instead
						if (NewClassName == NAME_None)
						{
							NewRedirects.Emplace(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly | ECoreRedirectFlags::Category_Removed, OldClassName.ToString(), NewClassName.ToString());
						}
						else
						{
							NewRedirects.Emplace(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, OldClassName.ToString(), NewClassName.ToString());
						}
					}
					//objects only on a per-object basis
					else if( ObjectName != NAME_None )
					{
						UE_LOGF(LogLinker, Warning, "Generic Object redirects are not supported with ActiveClassRedirects and never worked, move to new CoreRedirects system");
					}
					//full redirect
					else
					{
						if (NewClassName.ToString().Find(TEXT("."), ESearchCase::CaseSensitive) != NewClassName.ToString().Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromEnd))
						{
							RedirectErrors.Error(FText::Format(LOCTEXT("NestedRenameDisallowed", "{0} cannot contain a rename of nested objects for '{1}'; if you want to leave the outer alone, just specify the name with no path"), FText::FromName(ActiveClassRedirectsKey), FText::FromName(NewClassName)));
						}
						else
						{
							FCoreRedirect& Redirect = NewRedirects.Emplace_GetRef(ECoreRedirectFlags::Type_Class, OldClassName.ToString(), NewClassName.ToString());

							if (!NewClassClass.IsNone() || !NewClassPackage.IsNone())
							{
								Redirect.OverrideClassName = FCoreRedirectObjectName(NewClassClass, NAME_None, NewClassPackage);
							}
							else if (Redirect.NewName.ObjectName.ToString().StartsWith(TEXT("E"), ESearchCase::CaseSensitive))
							{
								// This might be an enum, so we have to register it
								NewRedirects.Emplace(ECoreRedirectFlags::Type_Enum, OldClassName.ToString(), NewClassName.ToString());
							}
							else
							{
								// This might be a struct redirect because many of them were registered incorrectly
								NewRedirects.Emplace(ECoreRedirectFlags::Type_Struct, OldClassName.ToString(), NewClassName.ToString());
							}
						}
					}
				}
				else if( It.Key() == TEXT("ActiveGameNameRedirects") )
				{
					FName OldGameName = NAME_None;
					FName NewGameName = NAME_None;

					FParse::Value( *It.Value().GetValue(), TEXT("OldGameName="), OldGameName );
					FParse::Value( *It.Value().GetValue(), TEXT("NewGameName="), NewGameName );

					NewRedirects.Emplace(ECoreRedirectFlags::Type_Package, OldGameName.ToString(), NewGameName.ToString());
				}
				else if ( It.Key() == TEXT("ActiveStructRedirects") )
				{
					FName OldStructName = NAME_None;
					FName NewStructName = NAME_None;

					FParse::Value( *It.Value().GetValue(), TEXT("OldStructName="), OldStructName );
					FParse::Value( *It.Value().GetValue(), TEXT("NewStructName="), NewStructName );

					NewRedirects.Emplace(ECoreRedirectFlags::Type_Struct, OldStructName.ToString(), NewStructName.ToString());
				}
				else if ( It.Key() == TEXT("ActivePluginRedirects") )
				{
					FString OldPluginName;
					FString NewPluginName;

					FParse::Value( *It.Value().GetValue(), TEXT("OldPluginName="), OldPluginName );
					FParse::Value( *It.Value().GetValue(), TEXT("NewPluginName="), NewPluginName );

					OldPluginName = FString("/") + OldPluginName + FString("/");
					NewPluginName = FString("/") + NewPluginName + FString("/");

					NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Option_MatchSubstring, OldPluginName, NewPluginName);
				}
				else if ( It.Key() == TEXT("KnownMissingPackages") )
				{
					FName KnownMissingPackage = NAME_None;

					FParse::Value( *It.Value().GetValue(), TEXT("PackageName="), KnownMissingPackage );

					NewRedirects.Emplace(ECoreRedirectFlags::Type_Package | ECoreRedirectFlags::Category_Removed, KnownMissingPackage.ToString(), FString());
				}
				else if (It.Key() == TEXT("TaggedPropertyRedirects"))
				{
					FName ClassName = NAME_None;
					FName OldPropertyName = NAME_None;
					FName NewPropertyName = NAME_None;

					FParse::Value(*It.Value().GetValue(), TEXT("ClassName="), ClassName);
					FParse::Value(*It.Value().GetValue(), TEXT("OldPropertyName="), OldPropertyName);
					FParse::Value(*It.Value().GetValue(), TEXT("NewPropertyName="), NewPropertyName);

					check(ClassName != NAME_None && OldPropertyName != NAME_None && NewPropertyName != NAME_None);

					NewRedirects.Emplace(ECoreRedirectFlags::Type_Property, FCoreRedirectObjectName(OldPropertyName, ClassName, NAME_None), FCoreRedirectObjectName(NewPropertyName, ClassName, NAME_None));
				}
				else if (It.Key() == TEXT("EnumRedirects"))
				{
					const FString& ConfigValue = It.Value().GetValue();
					FName EnumName = NAME_None;
					FName OldEnumEntry = NAME_None;
					FName NewEnumEntry = NAME_None;

					FString OldEnumSubstring;
					FString NewEnumSubstring;

					FParse::Value(*ConfigValue, TEXT("EnumName="), EnumName);
					if (FParse::Value(*ConfigValue, TEXT("OldEnumEntry="), OldEnumEntry))
					{
						FParse::Value(*ConfigValue, TEXT("NewEnumEntry="), NewEnumEntry);
						check(EnumName != NAME_None && OldEnumEntry != NAME_None && NewEnumEntry != NAME_None);
						FCoreRedirect& Redirect = NewRedirects.Emplace_GetRef(ECoreRedirectFlags::Type_Enum, EnumName.ToString(), EnumName.ToString());
						Redirect.ValueChanges.Add(OldEnumEntry.ToString(), NewEnumEntry.ToString());
					}
					else if (FParse::Value(*ConfigValue, TEXT("OldEnumSubstring="), OldEnumSubstring))
					{
						UE_LOGF(LogLinker, Warning, "OldEnumSubstring no longer supported! Replace with multiple entries or use the better syntax in the CoreRedirects section ");
					}
				}
			}

			FCoreRedirects::AddRedirectList(NewRedirects, FString::Printf(TEXT("ActiveClassRedirects %s"), *GEngineIniName));
		}
	}
	else
	{
		UE_LOGF(LogLinker, Warning, " **** ACTIVE CLASS REDIRECTS UNABLE TO INITIALIZE! (mActiveClassRedirects) **** ");
	}
}

FScopedCreateImportCounter::FScopedCreateImportCounter(FLinkerLoad* Linker, int32 Index)
{
	LoadContext = FUObjectThreadContext::Get().GetSerializeContext();

	// Remember the old linker and index
	PreviousLinker = LoadContext->SerializedImportLinker;
	PreviousIndex = LoadContext->SerializedImportIndex;
	// Remember the current linker and index.
	LoadContext->SerializedImportLinker = Linker;
	LoadContext->SerializedImportIndex = Index;
}

FScopedCreateImportCounter::~FScopedCreateImportCounter()
{
	// Restore old values
	LoadContext->SerializedImportLinker = PreviousLinker;
	LoadContext->SerializedImportIndex = PreviousIndex;
}


/** Helper struct to keep track of the CreateExport() entry/exit. */
struct FScopedCreateExportCounter
{
	/**
	 *	Constructor. Called upon CreateImport() entry.
	 *	@param Linker	- Current Linker
	 *	@param Index	- Index of the current Import
	 */
	FScopedCreateExportCounter(FLinkerLoad* Linker, int32 Index)
	{
		LoadContext = FUObjectThreadContext::Get().GetSerializeContext();

		// Remember the old linker and index
		PreviousLinker = LoadContext->SerializedExportLinker;
		PreviousIndex = LoadContext->SerializedExportIndex;
		// Remember the current linker and index.
		LoadContext->SerializedExportLinker = Linker;
		LoadContext->SerializedExportIndex = Index;
	}

	/** Destructor. Called upon CreateImport() exit. */
	~FScopedCreateExportCounter()
	{
		// Restore old values
		LoadContext->SerializedExportLinker = PreviousLinker;
		LoadContext->SerializedExportIndex = PreviousIndex;
	}

	/** Current load context object */
	FUObjectSerializeContext* LoadContext;
	/** Poreviously stored linker */
	FLinkerLoad* PreviousLinker;
	/** Previously stored index */
	int32 PreviousIndex;
};

namespace FLinkerDefs
{
	/** Number of progress steps for reporting status to a GUI while loading packages */
	const int32 TotalProgressSteps = 5;
}

/**
 * Creates a platform-specific ResourceMem. If an AsyncCounter is provided, it will allocate asynchronously.
 *
 * @param SizeX				Width of the stored largest mip-level
 * @param SizeY				Height of the stored largest mip-level
 * @param NumMips			Number of stored mips
 * @param TexCreateFlags	ETextureCreateFlags bit flags
 * @param AsyncCounter		If specified, starts an async allocation. If NULL, allocates memory immediately.
 * @return					Platform-specific ResourceMem.
 */
static FTexture2DResourceMem* CreateResourceMem(int32 SizeX, int32 SizeY, int32 NumMips, uint32 Format, uint32 TexCreateFlags, FThreadSafeCounter* AsyncCounter)
{
	FTexture2DResourceMem* ResourceMem = NULL;
	return ResourceMem;
}

static FORCEINLINE bool IsCoreUObjectPackage(const FName& PackageName)
{
	return PackageName == NAME_CoreUObject || PackageName == GLongCoreUObjectPackageName || PackageName == NAME_Core || PackageName == GLongCorePackageName;
}

/*----------------------------------------------------------------------------
	FLinkerLoad.
----------------------------------------------------------------------------*/

/**
 * Creates and returns a FLinkerLoad object.
 *
 * @param	Parent				Parent object to load into, can be NULL (most likely case)
 * @param	PackagePath			PackagePath to load from IPackageResourceManager
 * @param	LoadFlags			Load flags determining behavior
 *
 * @return	new FLinkerLoad object for Parent/ PackagePath
 */
FLinkerLoad* FLinkerLoad::CreateLinker(FUObjectSerializeContext* LoadContext, UPackage* Parent, const FPackagePath& PackagePath, uint32 LoadFlags, FArchive* InLoader /*= nullptr*/, const FLinkerInstancingContext* InstancingContext /*= nullptr*/)
{
	check(LoadContext);
	LLM_SCOPE_BYTAG(UObject_Linker);

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	// we don't want the linker permanently created with the
	// DeferDependencyLoads flag (we also want to be able to determine if the
	// linker aLready exists with that flag), so clear it before we attempt
	// CreateLinkerAsync()
	//
	// if this flag is present here, then we're most likely in a nested load and a
	// blueprint up the load chain needed an asset (most likely a user-defined
	// struct) loaded (we expect calls with LOAD_DeferDependencyLoads to be
	// coming from LoadPackageInternal)
	uint32 const DeferredLoadFlag = (LoadFlags & LOAD_DeferDependencyLoads);
	LoadFlags &= ~LOAD_DeferDependencyLoads;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	FLinkerLoad* Linker = CreateLinkerAsync(LoadContext, Parent, PackagePath, LoadFlags, InstancingContext,
		TFunction<void()>([](){})
		);
	{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
		// the linker could already have the DeferDependencyLoads flag present
		// (if this linker was already created further up the load chain, and
		// we're re-entering this to further finalize its creation)... we want
		// to make sure the DeferDependencyLoads flag is supplied (if it was
		// specified) fOr the duration of the Tick() below, because its call to
		// FinalizeCreation() could invoke further dependency loads
		TGuardValue<uint32> LinkerLoadFlagGuard(Linker->LoadFlags, Linker->LoadFlags | DeferredLoadFlag);
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

		if (InLoader)
		{
			// The linker can't have an associated loader here if we have a loader override
			check(!Linker->Loader);
			Linker->SetLoader(InLoader, true /* bInLoaderNeedsEngineVersionChecks */);
			// Set the basic archive flags on the linker
			Linker->ResetStatusInfo();
		}

		TGuardValue<FLinkerLoad*> SerializedPackageLinkerGuard(LoadContext->SerializedPackageLinker, Linker);
		if (Linker->Tick(0.f, false, false, nullptr) == LINKER_Failed)
		{
			return nullptr;
		}
	}
	FCoreUObjectDelegates::PackageCreatedForLoad.Broadcast(Parent);
	return Linker;
}

void FLinkerLoad::SetPackagePath(const FPackagePath& InPackagePath)
{
	PackagePath = InPackagePath;
}

void FLinkerLoad::SetLoader(FArchive* InLoader, bool bInLoaderNeedsEngineVersionChecks)
{
	Loader = InLoader;
	bLoaderNeedsEngineVersionChecks = bInLoaderNeedsEngineVersionChecks;

#if UE_INTERNAL_LINKER_BOUNDS
	SetBoundsImpl({ 0, InLoader->TotalSize()});
#endif // UE_INTERNAL_LINKER_BOUNDS

	check(StructuredArchive == nullptr);
	check(!StructuredArchiveRootRecord.IsSet());

	if (StructuredArchiveFormatter == nullptr)
	{
		// Create structured archive wrapper
		StructuredArchiveFormatter = new FBinaryArchiveFormatter(*this);
	}

	StructuredArchive = new FStructuredArchive(*StructuredArchiveFormatter);
	StructuredArchiveRootRecord.Emplace(StructuredArchive->Open().EnterRecord());
}

/**
 * Looks for an existing linker for the given package, without trying to make one if it doesn't exist
 */
FLinkerLoad* FLinkerLoad::FindExistingLinkerForPackage(const UPackage* Package)
{
	FLinkerLoad* Linker = nullptr;
	if (Package)
	{
		Linker = Package->GetLinker();
	}
	return Linker;
}

FLinkerLoad* FLinkerLoad::FindExistingLinkerForImport(int32 Index) const
{
	const FObjectImport& Import = ImportMap[Index];
	if (Import.SourceLinker != nullptr)
	{
		return Import.SourceLinker;
	}
	else if (Import.XObject != nullptr)
	{
		if (FLinkerLoad* ObjLinker = Import.XObject->GetLinker())
		{
			return ObjLinker;
		}
	}

	FLinkerLoad* FoundLinker = nullptr;
	if (Import.OuterIndex.IsNull() && (Import.ClassName == NAME_Package))
	{
		FString PackageName = Import.ObjectName.ToString();
		if (UPackage* FoundPackage = FindObject<UPackage>(/*Outer =*/nullptr, *PackageName))
		{
			FoundLinker = FLinkerLoad::FindExistingLinkerForPackage(FoundPackage);
		}
	}
	else if (Import.OuterIndex.IsImport())
	{
		FoundLinker = FindExistingLinkerForImport(Import.OuterIndex.ToImport());
	}
	return FoundLinker;
}


void FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(UObject* OldObject, UObject* NewObject,
	FUObjectSerializeContext* InLoadContext, bool bHideGarbageObjects)
{
	FLinkerLoad* OldObjectLinker = OldObject->GetLinker();
	// if this thing doesn't have a linker, then it wasn't loaded off disk and all of this is moot
	if (OldObjectLinker)
	{
		OldObjectLinker->PRIVATE_PatchNewObjectIntoExport(OldObject->GetLinkerIndex(), NewObject, InLoadContext,
			bHideGarbageObjects);
	}
}

/**
 * !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
 * CAUTION:  This function is potentially DANGEROUS.  Should only be used when you're really, really sure you know what you're doing.
 * !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
 *
 * Replaces OldObject's entry in its linker with NewObject, so that all subsequent loads of OldObject will return NewObject.
 * This is used to update instanced components that were serialized out, but regenerated during compile-on-load
 *
 * OldObject will be consigned to oblivion, and NewObject will take its place.
 *
 * WARNING!!!	This function is potentially very dangerous!  It should only be used at very specific times, and in very specific cases.
 *				If you're unsure, DON'T TRY TO USE IT!!!
 */
void FLinkerLoad::PRIVATE_PatchNewObjectIntoExport(int32 OldExportIndex, UObject* NewObject,
	FUObjectSerializeContext* InLoadContext, bool bHideGarbageObjects)
{
	if (!ExportMap.IsValidIndex(OldExportIndex))
	{
		return;
	}

	FObjectExport& ObjExport = ExportMap[OldExportIndex];
	UObject* OldObject = ObjExport.Object;

	// MarkAsGarbage on an object will make it return IsValidChecked == false, and it should not be used.
	// SavePackage will refuse to save it in this case, so we likewise should not give a pointer to it to
	// imports from other packages; it should be treated as if it was deleted and InvalidateExport called
	// on it instead. This is important during cook for imports of archetype objects that were deleted during
	// blueprint compile.
	bool bHideNewObject = bHideGarbageObjects && !IsValidChecked(NewObject);

	if (OldObject != NewObject)
	{
		if (OldObject)
		{
			// Since we don't copy the internal flags from Old to New, the mirrored flags for internal flags
			// need to not be copied as well.
			const EObjectFlags OldObjectFlags = OldObject->GetFlags() & ~(RF_MirroredGarbage);

			// Detach the old object to make room for the new
			OldObject->ClearFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects);
			OldObject->SetLinker(nullptr, INDEX_NONE, true /* bShouldDetachExisting */);

			// Copy flags from the old CDO.
			NewObject->SetFlags(OldObjectFlags);

			// If the object was in the ObjLoaded queue (exported, but not yet serialized), swap out for our new object
			FUObjectSerializeContext* LoadContext = InLoadContext ? InLoadContext
				: FUObjectThreadContext::Get().GetSerializeContext();
			if (!LoadContext->PRIVATE_PatchNewObjectIntoExport(OldObject, NewObject))
			{
				// Make sure the new object gets PostLoad called on it if it needs it:
				// it wasn't in the queue so add it ObjLoaded list
				if (OldObjectFlags & RF_NeedPostLoad)
				{
					LoadContext->AddLoadedObject(NewObject);
				}
			}
		}

		// bExportLoadFailed = true may have been previously set to true, and we now have a valid object, so should we
		// set it to false? Answer: No. Anything reading bExportLoadFailed will have first checked ObjExport.Object and
		// just returned that if non-null. So bExportLoadFailed is only used if garbage collection happens and the
		// NewObject is garbage collected. In that case, reloading the old object would be invalid; we need to reexecute
		// blueprint compile or whatever else figured out how to construct the NewObject, rather than using the data
		// that was serialized to disk for the old object.

		// Move the new object into the old object's slot, so any references to this object will now reference the new
		if (!bHideNewObject)
		{
			NewObject->SetLinker(this, OldExportIndex);
			ObjExport.Object = NewObject;
		}
	}
	if (bHideNewObject)
	{
		NewObject->ClearFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects);
		NewObject->SetLinker(nullptr, INDEX_NONE, true /* bShouldDetachExisting */);
		ObjExport.bExportLoadFailed = true;
	}

	// Recursively call PRIVATE_PatchNewObjectIntoExport for every old child object in the linker map that matches
	// a new child object, and recursively InvalidateExport for every old child object that doesn't have a match.
	// We need to do this because a change in identity for the outer object implies a change in identity for the
	// child objects.
	// We have to use children in the linker map rather than children in the in-memory OldObject because children
	// of the in-memory OldObject may have been renamed out of OldObject by e.g. blueprint reinstancing.
	TArray<int32> OldChildExports;
	FindDirectChildExportsFromExportTable(OldExportIndex, OldChildExports);
	TMap<FName, UObject*> NewChildMap;
	if (!bHideNewObject)
	{
		TArray<UObject*> NewChildren;
		GetObjectsWithOuter(NewObject, NewChildren, EGetObjectsFlags::None);
		NewChildMap.Reserve(NewChildren.Num());
		for (UObject* NewChild : NewChildren)
		{
			NewChildMap.Add(NewChild->GetFName(), NewChild);
		}
	}
	for (int32 OldChildIndex : OldChildExports)
	{
		FObjectExport& OldChildExport = ExportMap[OldChildIndex];
		UObject** NewChildPtr = NewChildMap.Find(OldChildExport.ObjectName);
		if (NewChildPtr)
		{
			PRIVATE_PatchNewObjectIntoExport(OldChildIndex, *NewChildPtr, InLoadContext, bHideGarbageObjects);
		}
		else
		{
			InvalidateExportIndex(OldChildIndex, bHideGarbageObjects);
		}
	}
}

void FLinkerLoad::InvalidateExport(UObject* OldObject, bool bHideGarbageObjects)
{
	FLinkerLoad* OldObjectLinker = OldObject->GetLinker();
	const int32 CachedLinkerIndex = OldObject->GetLinkerIndex();

	if (OldObjectLinker && OldObjectLinker->ExportMap.IsValidIndex(CachedLinkerIndex))
	{
		// Prevent any further loading as this export is now invalid
		OldObject->ClearFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects);
		bool bHideObject = bHideGarbageObjects && !IsValidChecked(OldObject);
		if (bHideObject)
		{
			OldObject->SetLinker(nullptr, INDEX_NONE, true /* bShouldDetachExisting */);
		}

		FObjectExport& ObjExport = OldObjectLinker->ExportMap[CachedLinkerIndex];
		ObjExport.bExportLoadFailed = true;

		// Recursively call InvalidateExport for every child object in the linker map.
		// We need to do this because an invalidation of an outer implies an invalidation of the child objects.
		// We have to use children in the linker map rather than children in the in-memory OldObject because children
		// of the in-memory object may have been renamed out of the object by e.g. blueprint reinstancing.
		TArray<int32> ChildExports;
		OldObjectLinker->FindDirectChildExportsFromExportTable(CachedLinkerIndex, ChildExports);
		for (int32 ChildExport : ChildExports)
		{
			OldObjectLinker->InvalidateExportIndex(ChildExport, bHideGarbageObjects);
		}
	}
}

void FLinkerLoad::FindDirectChildExportsFromExportTable(int32 ExportIndex, TArray<int32>& OutChildExports)
{
	int32 NumExports = ExportMap.Num();
	for (int32 ChildIndex = 0; ChildIndex < NumExports; ++ChildIndex)
	{
		const FObjectExport& ChildExport = ExportMap[ChildIndex];
		if (ChildExport.OuterIndex.IsExport() && ChildExport.OuterIndex.ToExport() == ExportIndex)
		{
			OutChildExports.Add(ChildIndex);
		}
	}
}

void FLinkerLoad::InvalidateExportIndex(int32 ExportIndex, bool bHideGarbageObjects)
{
	FObjectExport& ObjExport = ExportMap[ExportIndex];
	UObject* OldObject = ObjExport.Object;
	if (OldObject)
	{
		// Prevent any further loading as this export is now invalid
		OldObject->ClearFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects);
		bool bHideObject = bHideGarbageObjects && !IsValidChecked(OldObject);
		if (bHideObject)
		{
			OldObject->SetLinker(nullptr, INDEX_NONE, true /* bShouldDetachExisting */);
		}
	}
	ObjExport.bExportLoadFailed = true;

	// Recursively call InvalidateExport for every child object in the linker map; see comment in InvalidateExport.
	TArray<int32> ChildExports;
	FindDirectChildExportsFromExportTable(ExportIndex, ChildExports);
	for (int32 ChildExport : ChildExports)
	{
		InvalidateExportIndex(ChildExport, bHideGarbageObjects);
	}
}

/**
 * Creates a FLinkerLoad object for async creation. Tick has to be called manually till it returns
 * true in which case the returned linker object has finished the async creation process.
 *
 * @param	Parent				Parent object to load into, can be NULL (most likely case)
 * @param	PackagePath			PackagePath to load from IPackageResourceManager
 * @param	LoadFlags			Load flags determining behavior
 *
 * @return	new FLinkerLoad object for Parent/ PackagePath
 */
FLinkerLoad* FLinkerLoad::CreateLinkerAsync(FUObjectSerializeContext* LoadContext, UPackage* Parent, const FPackagePath& PackagePath, uint32 LoadFlags, const FLinkerInstancingContext* InstancingContext
	, TFunction<void()>&& InSummaryReadyCallback
	)
{
	check(Parent);

	// See whether there already is a linker for this parent/ linker root.
	FLinkerLoad* Linker = FindExistingLinkerForPackage(Parent);
	if (Linker)
	{
		UE_ASSET_LOG(LogStreaming, Log, Parent, TEXT("FLinkerLoad::CreateLinkerAsync: Found existing linker"));
	}

	// Create a new linker if there isn't an existing one.
	if( Linker == nullptr )
	{
		Linker = new FLinkerLoad(Parent, PackagePath, LoadFlags, InstancingContext ? FLinkerInstancingContext(*InstancingContext, FLinkerInstancingContext::EContextSharingMode::Link) : FLinkerInstancingContext());
		Parent->SetLinker(Linker);
	}

	check(Parent->GetLinker() == Linker);

	return Linker;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::ProcessPackageSummary(TMap<TPair<FName, FPackageIndex>, FPackageIndex>* ObjectNameWithOuterToExportMap)
{
	TRACE_LOADTIME_BEGIN_PROCESS_SUMMARY(this);
	LLM_SCOPE_BYTAG(UObject_Linker);

	ELinkerStatus Status = LINKER_Loaded;
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializePackageFileSummary);
		Status = SerializePackageFileSummary();
	}

	// Serialize the header for the package trailer
	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializePackageTrailer);
		Status = SerializePackageTrailer();
	}

	// Serialize the name map and register the names.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeNameMap);
		Status = SerializeNameMap();
	}

	// Serialize the soft object path list and register the paths.
	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeSoftObjectPathList);
		Status = SerializeSoftObjectPathList();
	}

	// Serialize the gatherable text data map.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeGatherableTextDataMap);
		Status = SerializeGatherableTextDataMap();
	}

	// Serialize the import map.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeImportMap);
		Status = SerializeImportMap();
	}

	// Serialize the export map.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeExportMap);
		Status = SerializeExportMap();
	}

#if WITH_TEXT_ARCHIVE_SUPPORT
	// Construct the exports readers
	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_ConstructExportsReaders);
		Status = ConstructExportsReaders();
	}
#endif

	// Fix up import map for backward compatible serialization.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_FixupImportMap);
		Status = FixupImportMap();
	}

	// Populate the linker instancing context for instance loading if needed.
	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_PopulateInstancingContext);
		Status = PopulateInstancingContext();
	}

	// Modify the ImportMap and SoftObjectPathList to account for the potential relocation of the packages
	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_ApplyRelocationToImportMapAndSoftObjectPathList);
		Status = RelocateReferences();
	}

	// Modify the SoftObjectPathList for the instancing context
	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_ApplyInstancingContextToSoftObjectPathList);
		Status = ApplyInstancingContext();
	}

	// Fix up export map for object class conversion
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_FixupExportMap);
		Status = FixupExportMap();
	}

#if WITH_METADATA
	// Serialize the meta data.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeMetaData);
		Status = SerializeMetaData();
	}
#endif

	// Serialize the dependency map.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeDependsMap);
		Status = SerializeDependsMap();
	}

	// Hash exports.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_CreateExportHash);
		Status = CreateExportHash();
	}

	// Find existing objects matching exports and associate them with this linker.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_FindExistingExports);
		Status = FindExistingExports();
	}

	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializePreloadDependencies);
		Status = SerializePreloadDependencies();
	}

	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeDataResources);
		Status = SerializeDataResourceMap();
	}

#if WITH_EDITORONLY_DATA
	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_SerializeImportTypeHierarchies);
		Status = SerializeImportTypeHierarchies();
	}

	// Add dynamic imports when needed.
	if (Status == LINKER_Loaded)
	{
		SCOPED_LOADTIMER(LinkerLoad_AddDynamicImports);
		Status = AddDynamicImports();
	}
#endif // WITH_EDITORONLY_DATA

	TRACE_LOADTIME_END_PROCESS_SUMMARY;

	// Finalize creation process.
	if( Status == LINKER_Loaded )
	{
		SCOPED_LOADTIMER(LinkerLoad_FinalizeCreation);
		Status = FinalizeCreation(ObjectNameWithOuterToExportMap);
	}

	return Status;
}

/**
 * Ticks an in-flight linker and spends InTimeLimit seconds on creation. This is a soft time limit used
 * if bInUseTimeLimit is true.
 *
 * @param	InTimeLimit		Soft time limit to use if bInUseTimeLimit is true
 * @param	bInUseTimeLimit	Whether to use a (soft) timelimit
 * @param	bInUseFullTimeLimit	Whether to use the entire time limit, even if blocked on I/O
 *
 * @return	true if linker has finished creation, false if it is still in flight
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::Tick( float InTimeLimit, bool bInUseTimeLimit, bool bInUseFullTimeLimit, TMap<TPair<FName, FPackageIndex>, FPackageIndex>* ObjectNameWithOuterToExportMap)
{
	ELinkerStatus Status = LINKER_Loaded;

	if (!bHasFinishedInitialization.load(std::memory_order_acquire))
	{
		UE::TUniqueLock UniqueLock(Mutex);
		// Now that we have the lock, double check that someone didn't finish init for us after checking above
		if (!bHasFinishedInitialization.load(std::memory_order_relaxed))
		{
			// Store variables used by functions below.
			TickStartTime = FPlatformTime::Seconds();
			bTimeLimitExceeded = false;
			bUseTimeLimit = bInUseTimeLimit;
			bUseFullTimeLimit = bInUseFullTimeLimit;
			TimeLimit = InTimeLimit;

			do
			{
				bool bCanSerializePackageFileSummary = false;
				{
					// Create loader, aka FArchive used for serialization and also precache the package file summary.
					// false is returned until any precaching is complete.
					SCOPED_LOADTIMER(LinkerLoad_CreateLoader);
					Status = CreateLoader(TFunction<void()>([]() {}));

					bCanSerializePackageFileSummary = (Status == LINKER_Loaded);
				}

				// Serialize the package file summary and presize the various arrays (name, import & export map)
				if (bCanSerializePackageFileSummary)
				{
					Status = ProcessPackageSummary(ObjectNameWithOuterToExportMap);
				}
			}
			// Loop till we are done if no time limit is specified, or loop until the real time limit is up if we want to use full time
			while (Status == LINKER_TimedOut &&
				(!bUseTimeLimit || (bUseFullTimeLimit && !IsTimeLimitExceeded(TEXT("Checking Full Timer"))))
				);
		}
	}

	if (Status == LINKER_Failed)
	{
		LinkerRoot->SetLinker(nullptr);
#if WITH_EDITOR

		if (LoadProgressScope)
		{
			delete LoadProgressScope;
			LoadProgressScope = nullptr;
		}
#endif
	}

	// Return whether we completed or not.
	return Status;
}

/**
 * Private constructor, passing arguments through from CreateLinker.
 *
 * @param	Parent				Parent object to load into, can be NULL (most likely case)
 * @param	PackagePath			PackagePath to load from IPackageResourceManager
 * @param	LoadFlags			Load flags determining behavior
 */
FLinkerLoad::FLinkerLoad(UPackage* InParent, const FPackagePath& InPackagePath, uint32 InLoadFlags, FLinkerInstancingContext InInstancingContext)
: FLinker(ELinkerType::Load, InParent)
, LoadFlags(InLoadFlags)
, bHaveImportsBeenVerified(false)
, TemplateForGetArchetypeFromLoader(nullptr)
, bForceSimpleIndexToObject(false)
, bLockoutLegacyOperations(false)
, bIsAsyncLoader(false)
, bIsDestroyingLoader(false)
#if WITH_EDITOR
, bDetachedLoader(false)
#endif // WITH_EDITOR
, StructuredArchive(nullptr)
, StructuredArchiveFormatter(nullptr)
, PackagePath(InPackagePath)
, Loader(nullptr)
, InstancingContext(MoveTemp(InInstancingContext))
, PackageTrailer(nullptr)
, AsyncRoot(nullptr)
, SoftObjectPathListIndex(0)
, GatherableTextDataMapIndex(0)
, ImportMapIndex(0)
, ExportMapIndex(0)
#if WITH_METADATA
, MetaDataMapIndex(0)
, NumObjectMetaDataMap(0)
, NumRootMetaDataMap(0)
#endif
, DependsMapIndex(0)
, ExportHashIndex(0)
, bHasFoundExistingExports(false)
, bHasFinishedInitialization(false)
, bHasSerializedPackageFileSummary(false)
, bHasSerializedPackageTrailer(false)
, bHasConstructedExportsReaders(false)
, bHasSerializedPreloadDependencies(false)
, bHasFixedUpImportMap(false)
, bHasPopulatedInstancingContext(false)
, bHasRelocatedReferences(false)
, bHasAppliedInstancingContext(false)
, bFixupExportMapDone(false)
, bIsGatheringDependencies(false)
, bTimeLimitExceeded(false)
, bUseTimeLimit(false)
, bUseFullTimeLimit(false)
, bLoaderNeedsEngineVersionChecks(true)
, bHasSerializedImportTypeHierarchies(false)
, bHasAddedDynamicImports(false)
#if WITH_EDITOR
, bExportsDuplicatesFixed(false)
, bIsPackageRelocated(false)
, bIsSerializingScriptProperties(false)
#endif // WITH_EDITOR
#if WITH_EDITORONLY_DATA
, DynamicImportsIndex(INDEX_NONE)
#endif // WITH_EDITORONLY_DATA
, IsTimeLimitExceededCallCount(0)
, TimeLimit(0.0f)
, TickStartTime(0.0)
#if WITH_EDITOR
, LoadProgressScope( nullptr )
#endif // WITH_EDITOR
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
, bForceBlueprintFinalization(false)
, DeferredCDOIndex(INDEX_NONE)
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
{
	static_assert((ExportHashCount & (ExportHashCount - 1)) == 0, "ExportHashCount must be power of two");
	LLM_SCOPE_BYTAG(UObject_Linker);

	if (PackagePath.GetHeaderExtension() == EPackageExtension::Unspecified)
	{
		UE_ASSET_LOG(LogPackageName, Error, PackagePath, TEXT("PackagePath is missing header extension when assigned to LinkerLoad"));
	}

	INC_DWORD_STAT(STAT_LinkerCount);
	INC_DWORD_STAT(STAT_LiveLinkerCount);
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	FLinkerManager::Get().AddLiveLinker(this);
#endif
	OwnerThread = FPlatformTLS::GetCurrentThreadId();

	// Check if the linker is instanced @todo: pass through a load flag?
	FName PackageNameToLoad = GetPackagePath().GetPackageFName();
	if (LinkerRoot->GetFName() != PackageNameToLoad)
	{
		InstancingContext.BuildPackageMapping(PackageNameToLoad, LinkerRoot->GetFName());
	}

	TRACE_LOADTIME_NEW_LINKER(this);
}

FLinkerLoad::~FLinkerLoad()
{
	TRACE_LOADTIME_DESTROY_LINKER(this);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	FLinkerManager::Get().RemoveLiveLinker(this);
#endif

	UE_CLOGF(!FUObjectThreadContext::Get().IsDeletingLinkers, LogLinker, Fatal, "Linkers can only be deleted by FLinkerManager.");

	// Detaches linker.
	Detach();

	DEC_DWORD_STAT(STAT_LiveLinkerCount);

#if WITH_EDITOR
	// Make sure this is deleted if it's still allocated
	delete LoadProgressScope;
#endif
	check(Loader == nullptr);
	check(StructuredArchive == nullptr);
	check(StructuredArchiveFormatter == nullptr);
}

/**
 * Returns whether the time limit allotted has been exceeded, if enabled.
 *
 * @param CurrentTask	description of current task performed for logging spilling over time limit
 * @param Granularity	Granularity on which to check timing, useful in cases where FPlatformTime::Seconds is slow (e.g. PC)
 *
 * @return true if time limit has been exceeded (and is enabled), false otherwise (including if time limit is disabled)
 */
bool FLinkerLoad::IsTimeLimitExceeded( const TCHAR* CurrentTask, int32 Granularity )
{
	IsTimeLimitExceededCallCount++;
	if( !IsTextFormat()
	&&  !bTimeLimitExceeded
	&&  bUseTimeLimit
	&&  (IsTimeLimitExceededCallCount % Granularity) == 0 )
	{
		double CurrentTime = FPlatformTime::Seconds();
		bTimeLimitExceeded = CurrentTime - TickStartTime > TimeLimit;
		if (!FPlatformProperties::HasEditorOnlyData())
		{
			// Log single operations that take longer than timelimit.
			if( (CurrentTime - TickStartTime) > (2.5 * TimeLimit) )
			{
				UE_ASSET_LOG(LogStreaming, Log, PackagePath, TEXT("FLinkerLoad: %s took (less than) %5.2f ms"),
					CurrentTask,
					(CurrentTime - TickStartTime) * 1000);
			}
		}
	}
	return bTimeLimitExceeded;
}

void FLinkerLoad::ResetStatusInfo()
{
	// Set status info.
	this->SetUEVer(GPackageFileUEVersion);
	this->SetLicenseeUEVer(GPackageFileLicenseeUEVersion);
	this->SetEngineVer(FEngineVersion::Current());
	this->SetIsLoading(true);
	this->SetIsPersistent(true);

	// Reset all custom versions
	ResetCustomVersions();
}

/**
 * Creates loader used to serialize content.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::CreateLoader(
	TFunction<void()>&& InSummaryReadyCallback
	)
{
	//DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::CreateLoader" ), STAT_LinkerLoad_CreateLoader, STATGROUP_LinkerLoad );

#if WITH_EDITOR

	if (!LoadProgressScope)
	{
		if (ShouldCreateThrottledSlowTask())
		{
			static const FText LoadingText = NSLOCTEXT("Core", "GenericLoading", "Loading...");
			LoadProgressScope = new FScopedSlowTask(FLinkerDefs::TotalProgressSteps, LoadingText);
		}
	}

#endif

	// This should have been initialized in InitUObject
	check(bActiveRedirectsMapInitialized);

	if( !Loader )
	{
#if WITH_EDITOR
		if (LoadProgressScope)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			static const FTextFormat LoadingFileTextFormat = NSLOCTEXT("Core", "LoadingFileWithFilename", "Loading file: {CleanFilename}...");
			FFormatNamedArguments FeedbackArgs;
			FeedbackArgs.Add( TEXT("CleanFilename"), FText::FromString( FPaths::GetCleanFilename( *GetDebugName() ) ) );
			LoadProgressScope->DefaultMessage = FText::Format(LoadingFileTextFormat, FeedbackArgs);
			LoadProgressScope->EnterProgressFrame();
		}
#endif

		// If want to be able to load cooked data in the editor we need to use FAsyncArchive which supports EDL cooked packages,
		// otherwise the generic file reader is faster in the editor so use that
		bool bCanUseAsyncLoader = (FPlatformProperties::RequiresCookedData() || GAllowCookedDataInEditorBuilds) && !GSkipAsyncLoaderForCookedData;

		if (bCanUseAsyncLoader)
		{
			FAsyncArchive* AsyncArchive = new FAsyncArchive(GetPackagePath(), this, TFunction<void()>([]() {}));
			Loader = AsyncArchive; // We're only allowed to delete any FAsyncArchive with this->DestroyLoader
			bLoaderNeedsEngineVersionChecks = !(LoadFlags & LOAD_DisableEngineVersionChecks) && AsyncArchive->NeedsEngineVersionChecks();
			if (AsyncArchive->IsError())
			{
				bool bRetryWithNormalArchive = AsyncArchive->GetLoadError() == FAsyncArchive::ELoadError::UnsupportedFormat;
				DestroyLoader();
				bCanUseAsyncLoader = false;
				if (!bRetryWithNormalArchive)
				{
					UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Error opening file."));
					return LINKER_Failed;
				}
			}
		}
		if (!Loader)
		{
			FOpenPackageResult OpenResult;
#if WITH_EDITOR
			if (FLinkerLoad::GetPreloadingEnabled() && FLinkerLoad::TryGetPreloadedLoader(GetPackagePath(), OpenResult))
			{
				// OpenResult set by TryGetPreloadedLoader
			}
			else
#endif
			{
				OpenResult = IPackageResourceManager::Get().OpenReadPackage(GetPackagePath());
			}
			Loader = OpenResult.Archive.Release();
			bLoaderNeedsEngineVersionChecks = !(LoadFlags & LOAD_DisableEngineVersionChecks) && OpenResult.bNeedsEngineVersionChecks;
			if (!Loader || Loader->IsError())
			{
				if (Loader)
				{
					DestroyLoader();
				}
				UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Error opening file."));
				return LINKER_Failed;
			}

#if WITH_TEXT_ARCHIVE_SUPPORT
			if (OpenResult.Format == EPackageFormat::Text)
			{
				INC_DWORD_STAT(STAT_TextAssetLinkerCount);
				DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::CreateTextArchiveFormatter"), STAT_LinkerLoad_CreateTextArchiveFormatter, STATGROUP_LinkerLoad);
				TRACE_CPUPROFILER_EVENT_SCOPE(FLinkerLoad::CreateTextArchiveFormatter);
				StructuredArchiveFormatter = new FJsonArchiveInputFormatter(*this, [this](const FPackageIndex Index)
				{
					if (Index.IsNull())
					{
						return (UObject*)nullptr;
					}
					else if (Index.IsImport())
					{
						return CreateImport(Index.ToImport());
					}
					else
					{
						check(Index.IsExport());
						return CreateExport(Index.ToExport());
					}
				});
			}
			else
#endif
			{
				check(OpenResult.Format == EPackageFormat::Binary);
			}
		}

#if DEVIRTUALIZE_FLinkerLoad_Serialize
		ActiveFPLB = Loader->ActiveFPLB; // make sure my fast past loading is using the FAA2 fast path buffer
#endif

		bool bHasHashEntry = FSHA1::GetFileSHAHash(*GetPackagePath().GetLocalFullPath(), nullptr);
		if ((LoadFlags & LOAD_MemoryReader) || bHasHashEntry)
		{
			// force preload into memory if file has an SHA entry
			// Serialize data from memory instead of from disk.
			const int64	BufferSize = Loader->TotalSize();
			void* Buffer = FMemory::Malloc(BufferSize);
			Loader->Serialize(Buffer, BufferSize);
			DestroyLoader();
			if (bHasHashEntry)
			{
				// create buffer reader and spawn SHA verify when it gets closed
				Loader = new FBufferReaderWithSHA(Buffer, BufferSize, true, *GetPackagePath().GetLocalFullPath(), true);
			}
			else
			{
				// create a buffer reader
				Loader = new FBufferReader(Buffer, BufferSize, true, true);
			}
			bIsAsyncLoader = false;
		}
		else
		{
			bIsAsyncLoader = bCanUseAsyncLoader;
		}

		SetLoader(Loader, bLoaderNeedsEngineVersionChecks);

		check(Loader);
		check(!Loader->IsError());

		//if( FLinkerLoad::FindExistingLinkerForPackage(LinkerRoot) )
		//{
		//	UE_LOGF(LogLinker, Warning, "Linker for '%ls' already exists", *LinkerRoot->GetName() );
		//	return LINKER_Failed;
		//}

		ResetStatusInfo();
	}
	
	{
		bool bExecuteNextStep = true;
		if( bHasSerializedPackageFileSummary == false )
		{
			if (bIsAsyncLoader)
			{
				bExecuteNextStep = GetAsyncLoader()->ReadyToStartReadingHeader(bUseTimeLimit, bUseFullTimeLimit, TickStartTime, TimeLimit);
			}
			else
			{
				int64 Size = Loader->TotalSize();
				if (Size <= 0)
				{
					DestroyLoader();
					UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Error opening file."));
					return LINKER_Failed;
				}
				// Precache up to one ECC block before serializing package file summary.
				// If the package is partially compressed, we'll know that quickly and
				// end up discarding some of the precached data so we can re-fetch
				// and decompress it.
				static int64 MinimumReadSize = 32 * 1024;
				checkSlow(MinimumReadSize >= 2048 && MinimumReadSize <= 1024 * 1024); // not a hard limit, but we should be loading at least a reasonable amount of data
				int64 PrecacheSize = FMath::Min(MinimumReadSize, Size);
				check( PrecacheSize > 0 );
				// Wait till we're finished precaching before executing the next step.
				bExecuteNextStep = Loader->Precache(0, PrecacheSize);
			}
		}

		return (bExecuteNextStep && !IsTimeLimitExceeded( TEXT("creating loader") )) ? LINKER_Loaded : LINKER_TimedOut;
	}
}

FLinkerLoad::ELinkerStatus FLinkerLoad::SerializePackageFileSummaryInternal()
{
#if WITH_EDITOR
	if (LoadProgressScope)
	{
		UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
		LoadProgressScope->EnterProgressFrame(1);
	}
#endif
	// Read summary from file.
	StructuredArchiveRootRecord.GetValue() << SA_VALUE(TEXT("Summary"), Summary);

	// Check tag.
	if (Summary.Tag != PACKAGE_FILE_TAG)
	{
		Async(EAsyncExecution::TaskGraphMainThread,
			[DebugName = GetDebugName()]()
			{
				FMessageLog("LoadErrors").Error(FText::Format(NSLOCTEXT("Core", "LinkerLoad_PkgSumCorrupted", "The summary for the package '{0}' is invalid. Check that the file is of the expected type and not corrupted."),
					FText::FromString(DebugName)));
			}
		);

		return LINKER_Failed;
	}

	// Validate the summary.
	if (Summary.IsFileVersionTooOld())
	{
		Async(EAsyncExecution::TaskGraphMainThread,
			[DebugName = GetDebugName(), FileVersion = Summary.GetFileVersionUE()]()
			{
				FMessageLog("LoadErrors").Warning(FText::Format(NSLOCTEXT("Core", "LinkerLoad_PkgVersionTooOld", "The package '{0}' was saved with an older version which is not backwards compatible with the current process. Min Required Version: {1}  Package Version: {2}"),
					FText::FromString(DebugName),
					(int32)VER_UE4_OLDEST_LOADABLE_PACKAGE,
					FileVersion.FileVersionUE4));
			}
		);

		return LINKER_Failed;
	}

	// Check that no content saved with a licensee version has snuck into the source tree. This can result in licensees builds being unable to open
	// the asset because their CL is very likely to be lower than ours.
	if (FEngineBuildSettings::IsInternalBuild())
	{
		// I think this is the better check without the outer IsInternalBuild, but that gives an extra degree of safety against this leading to false-positives
		// this late in 4.26's cycle
		if (FEngineVersion::Current().IsLicenseeVersion() == false && Summary.CompatibleWithEngineVersion.IsLicenseeVersion())
		{
			// Only warn about things under Engine and Engine/Plugins so licensee projects can be opened
			FString LocalFilename = FPaths::CreateStandardFilename(GetPackagePath().GetLocalFullPath());
			bool IsEngineContent = LocalFilename.StartsWith(FPaths::EngineContentDir()) || LocalFilename.StartsWith(FPaths::EnginePluginsDir());

			if (IsEngineContent)
			{
				UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("The file is Engine content that was saved with a licensee flag. This can result in the file failing to open on licensee builds"));
			}
		}
	}

	// Don't load packages that are only compatible with an engine version newer than the current one.
	if (bLoaderNeedsEngineVersionChecks && IsEnforcePackageCompatibleVersionCheck() &&
		(Summary.IsFileVersionTooNew() || !FEngineVersion::Current().IsCompatibleWith(Summary.CompatibleWithEngineVersion)))
	{
		// Send the warning to the game thread as slate is not thread-safe
		Async(EAsyncExecution::TaskGraphMainThread,
			[DebugName = GetDebugName(), CompatibleWith = Summary.CompatibleWithEngineVersion]()
			{
				FMessageLog("LoadErrors").Warning(FText::Format(NSLOCTEXT("Core", "LinkerLoad_EngineVersionIncompatible", "Package '{0}' has been saved with a newer engine version and can't be loaded. Current EngineVersion: {1} (Licensee={2}). Package EngineVersion: {3} (Licensee={4})"),
					FText::FromString(DebugName),
					FText::FromString(FEngineVersion::Current().ToString()),
					FEngineVersion::Current().IsLicenseeVersion(),
					FText::FromString(CompatibleWith.ToString()),
					CompatibleWith.IsLicenseeVersion()));
			}
		);

		return LINKER_Failed;
	}

	bool bIsCooked = (Summary.GetPackageFlags() & PKG_Cooked) != 0;
	SetIsLoadingFromCookedPackage(bIsCooked);
	Loader->SetIsLoadingFromCookedPackage(bIsCooked);

	// Set desired property tag format
	bool bUseUnversionedProperties = (Summary.GetPackageFlags() & PKG_UnversionedProperties) != 0;
	SetUseUnversionedPropertySerialization(bUseUnversionedProperties);
	Loader->SetUseUnversionedPropertySerialization(bUseUnversionedProperties);

	if (bLoaderNeedsEngineVersionChecks && !FPlatformProperties::RequiresCookedData()
		&& !Summary.SavedByEngineVersion.HasChangelist() && FEngineVersion::Current().HasChangelist())
	{
		// This warning can be disabled in ini with [Core.System] ZeroEngineVersionWarning=False
		static struct FInitZeroEngineVersionWarning
		{
			bool bDoWarn;
			FInitZeroEngineVersionWarning()
			{
				if (!GConfig->GetBool(TEXT("Core.System"), TEXT("ZeroEngineVersionWarning"), bDoWarn, GEngineIni))
				{
					bDoWarn = true;
				}
			}
			FORCEINLINE operator bool() const { return bDoWarn; }
		} ZeroEngineVersionWarningEnabled;
		if (ZeroEngineVersionWarningEnabled)
		{
			UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Asset has been saved with empty engine version. The asset will be loaded but may be incompatible."));
		}
	}

	// Don't load packages that were saved with package version newer than the current one.
	if (bLoaderNeedsEngineVersionChecks && ((Summary.IsFileVersionTooNew()) || (Summary.GetFileVersionLicenseeUE() > GPackageFileLicenseeUEVersion)))
	{
		// Send the warning to the game thread as slate is not thread-safe
		Async(EAsyncExecution::TaskGraphMainThread,
			[DebugName = GetDebugName(), FileVersion = Summary.GetFileVersionUE(), FileVersionLicensee = Summary.GetFileVersionLicenseeUE(), PackageFileUEVersion = GPackageFileUEVersion, PackageFileLicenseeUEVersion = GPackageFileLicenseeUEVersion]()
			{
				FMessageLog("LoadErrors").Warning(FText::Format(NSLOCTEXT("Core", "LinkerLoad_PkgVersionTooNew", "Package '{0}' contains a newer version than the current process supports. PackageVersion {1}, MaxExpected {2} : LicenseePackageVersion {3}, MaxExpected {4}."),
					FText::FromString(DebugName),
					FileVersion.ToValue(),
					PackageFileUEVersion.ToValue(),
					FileVersionLicensee,
					PackageFileLicenseeUEVersion));
			}
		);

		return LINKER_Failed;
	}

	// don't load packages that contain editor only data in builds that don't support that and vise versa
	if (!FPlatformProperties::HasEditorOnlyData() && !(Summary.GetPackageFlags() & PKG_FilterEditorOnly))
	{
		Async(EAsyncExecution::TaskGraphMainThread,
			[DebugName = GetDebugName()]()
			{
				FMessageLog("LoadErrors").Warning(FText::Format(NSLOCTEXT("Core", "LinkerLoad_InvalidEditorOnlyData", "Unable to load package '{0}'. Package contains EditorOnly data which is not supported by the current build."),
					FText::FromString(DebugName)));
			}
		);

		return LINKER_Failed;
	}

	// don't load packages that contain editor only data in builds that don't support that and vise versa
	if (FPlatformProperties::HasEditorOnlyData() && !!(Summary.GetPackageFlags() & PKG_FilterEditorOnly))
	{
		// This warning can be disabled in ini or project settings
		if (!GAllowCookedDataInEditorBuilds)
		{
			Async(EAsyncExecution::TaskGraphMainThread,
				[DebugName = GetDebugName()]()
				{
					FMessageLog("LoadErrors").Warning(FText::Format(NSLOCTEXT("Core", "LinkerLoad_InvalidCookedData", "Unable to load package '{0}'. Package contains cooked data which is not supported by the current build. Enable 'Allow Cooked Content In The Editor' in Project Settings under 'Engine - Cooker' section to load it."),
						FText::FromString(DebugName)));
				}
			);

			return LINKER_Failed;
		}
	}

	if (!FPlatformProperties::RequiresCookedData()
		// We can't check the post tag if the file is an EDL cooked package
		&& !bIsCooked
		&& !IsTextFormat() && bLoaderNeedsEngineVersionChecks)
	{
		// check if this package version stored the 4-byte magic post tag
		// get the offset of the post tag
		int64 MagicOffset = TotalSize() - sizeof(uint32);
		// store the current file offset
		int64 OriginalOffset = Tell();

		uint32 Tag = 0;

		// seek to the post tag and serialize it
		Seek(MagicOffset);
		*this << Tag;

		if (Tag != PACKAGE_FILE_TAG)
		{
			Async(EAsyncExecution::TaskGraphMainThread,
				[DebugName = GetDebugName()]()
				{
					FMessageLog("LoadErrors").Error(FText::Format(NSLOCTEXT("Core", "LinkerLoad_PkgTagCorrupted", "Unable to load package '{0}'. The end of package tag is not valid. Check that the file is of the expected type and not corrupted."),
						FText::FromString(DebugName)));
				}
			);

			return LINKER_Failed;
		}

		// seek back to the position after the package summary
		Seek(OriginalOffset);
	}

	return LINKER_Loaded;
}

/**
 * Serializes the package file summary.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializePackageFileSummary()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::SerializePackageFileSummary" ), STAT_LinkerLoad_SerializePackageFileSummary, STATGROUP_LinkerLoad );
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_FLinkerLoad);

	if (bHasSerializedPackageFileSummary == false)
	{
		if (Loader->IsError())
		{
			UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("The file contains unrecognizable data, check that it is of the expected type."));
			return LINKER_Failed;
		}
		if (bIsAsyncLoader)
		{
			GetAsyncLoader()->StartReadingHeader();
		}

		ELinkerStatus Status = SerializePackageFileSummaryInternal();

		if (Status == LINKER_Failed)
		{
			if (bIsAsyncLoader)
			{
				GetAsyncLoader()->EndReadingHeader();
			}

			return Status;
		}

		ELinkerStatus UpdateStatus = UpdateFromPackageFileSummary();
		if (UpdateStatus != LINKER_Loaded)
		{
			return UpdateStatus;
		}

		// Slack everything according to summary.
		ImportMap.Empty(Summary.ImportCount);
		ExportMap.Empty(Summary.ExportCount);
		GatherableTextDataMap.Empty(Summary.GatherableTextDataCount);
		NameMap.Empty(Summary.NameCount);
		// Depends map gets pre-sized in SerializeDependsMap if used.

		ImportTypeHierarchies.Empty(Summary.ImportTypeHierarchiesCount);

		// Avoid serializing it again.
		bHasSerializedPackageFileSummary = true;
	}

	return !IsTimeLimitExceeded( TEXT("serializing package file summary") ) ? LINKER_Loaded : LINKER_TimedOut;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::UpdateFromPackageFileSummary()
{
	// When unversioned, pretend we are the latest version
	bool bCustomVersionIsLatest = true;
	if (!Summary.bUnversioned)
	{
		TArray<FCustomVersionDifference> Diffs = FCurrentCustomVersions::Compare(Summary.GetCustomVersionContainer().GetAllVersions(), *GetDebugName());
		for (FCustomVersionDifference Diff : Diffs)
		{
			bCustomVersionIsLatest = false;
			if (Diff.Type == ECustomVersionDifference::Missing)
			{
				// Loading a package with custom version that we don't know about!
				// Temporarily just warn and continue. @todo: this needs to be fixed properly
				UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Package was saved with a custom version that is not present. Tag %s  Version %d"),
					*Diff.Version->Key.ToString(), Diff.Version->Version);
			}
			else if (Diff.Type == ECustomVersionDifference::Invalid)
			{
				UE_ASSET_LOG(LogLinker, Error, PackagePath, TEXT("Package was saved with an invalid custom version. Tag %s  Version %d"), *Diff.Version->Key.ToString(), Diff.Version->Version);

				Async(EAsyncExecution::TaskGraphMainThread,
					[DebugName = GetDebugName()]()
					{
						FMessageLog("LoadErrors")
							.SuppressLoggingToOutputLog(true)
							.Error(FText::Format(NSLOCTEXT("Core", "LinkerLoad_InvalidCustomVersion", "Package {0} was saved with an invalid custom version and cannot be loaded, see output log for details"),
								FText::FromString(DebugName)));
					}
				);

				return LINKER_Failed;
			}
			else if (Diff.Type == ECustomVersionDifference::Newer)
			{
				FCustomVersion LatestVersion = FCurrentCustomVersions::Get(Diff.Version->Key).GetValue();
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
				FString AppName = FApp::GetName();
				UE_ASSET_LOG(LogLinker, Error, PackagePath, TEXT("Package was saved with a newer custom version than %s. Rebuild %s to avoid this error - Tag %s Name '%s' PackageVersion %d  MaxExpected %d"),
					*AppName, *AppName,
					*Diff.Version->Key.ToString(), *LatestVersion.GetFriendlyName().ToString(), Diff.Version->Version, LatestVersion.Version);
#else
				// Loading a package with a newer custom version than the current one.
				UE_ASSET_LOG(LogLinker, Error, PackagePath, TEXT("Package was saved with a newer custom version than the current. Tag %s Name '%s' PackageVersion %d  MaxExpected %d"),
					*Diff.Version->Key.ToString(), *LatestVersion.GetFriendlyName().ToString(), Diff.Version->Version, LatestVersion.Version);

				Async(EAsyncExecution::TaskGraphMainThread,
					[DebugName = GetDebugName()]()
					{
						FMessageLog("LoadErrors")
							.SuppressLoggingToOutputLog(true)
							.Error(FText::Format(NSLOCTEXT("Core", "LinkerLoad_NewCustomVersion", "Package {0} was saved with a newer custom version than the current engine and cannot be loaded, see output log for details"),
								FText::FromString(DebugName)));
					}
				);

				return LINKER_Failed;
#endif // UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			}
		}
	}

	const FCustomVersionContainer& SummaryVersions = Summary.GetCustomVersionContainer();

	SetUEVer(Summary.GetFileVersionUE());
	SetLicenseeUEVer(Summary.GetFileVersionLicenseeUE());
	SetEngineVer(Summary.SavedByEngineVersion);
	SetCustomVersions(SummaryVersions);

	if (Summary.GetPackageFlags() & PKG_FilterEditorOnly)
	{
		SetFilterEditorOnly(true);
	}

	// Propagate fact that package cannot use lazy loading to archive (aka this).
	if (IsTextFormat())
	{
		ArAllowLazyLoading = false;
	}
	else
	{
		ArAllowLazyLoading = true;
	}

	// Loader needs to be the same version.
	if (Loader)
	{
		Loader->SetUEVer(Summary.GetFileVersionUE());
		Loader->SetLicenseeUEVer(Summary.GetFileVersionLicenseeUE());
		Loader->SetEngineVer(Summary.SavedByEngineVersion);
		Loader->SetCustomVersions(SummaryVersions);
	}

	if (UPackage* LinkerRootPackage = LinkerRoot)
	{
		const uint32 NewPackageFlags = Summary.GetPackageFlags() |
			// Preserve PKG_PlayInEditor and PKG_ForDiffing, they have been provided
			// by the caller, but are not identified as truly transient:
			(LinkerRootPackage->GetPackageFlags() & (PKG_PlayInEditor|PKG_ForDiffing));

		// Propagate package flags
		LinkerRootPackage->SetPackageFlagsTo(NewPackageFlags);

		// Propagate streaming install ChunkID
		LinkerRootPackage->SetChunkIDs(Summary.ChunkIDs);

		// Propagate package file size
		LinkerRootPackage->SetFileSize(Loader ? Loader->TotalSize() : 0);

		// Propagate package hashes
#if WITH_EDITORONLY_DATA
		LinkerRootPackage->SetSavedHash(Summary.GetSavedHash());
		LinkerRootPackage->SetPersistentGuid( Summary.PersistentGuid );
#endif

		// Remember the linker versions
		LinkerRootPackage->SetLinkerPackageVersion(Summary.GetFileVersionUE());
		LinkerRootPackage->SetLinkerLicenseeVersion(Summary.GetFileVersionLicenseeUE());

		// Only set the custom version if it is not already latest.
		// If it is latest, we will compare against latest in GetLinkerCustomVersion
		if (!bCustomVersionIsLatest)
		{
			LinkerRootPackage->SetLinkerCustomVersions(SummaryVersions);
		}

#if WITH_EDITORONLY_DATA
		LinkerRootPackage->bIsCookedForEditor = !!(Summary.GetPackageFlags() & PKG_FilterEditorOnly);
#endif
	}

	return LINKER_Loaded;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::SerializePackageTrailer()
{
	if (bHasSerializedPackageTrailer)
	{
		return LINKER_Loaded;
	}

	check(PackageTrailer == nullptr); // There should not be any pre-existing trailer if we are serializing one

	if (Summary.PayloadTocOffset <= 0)
	{
		bHasSerializedPackageTrailer = true;
		return LINKER_Loaded;
	}

	int64 CurPos = Tell();

	PackageTrailer = MakeUnique<UE::FPackageTrailer>();

	bool bResult = false;

	// We have seen a very low rate of package file where this value is corrupted and would cause us to seek outside of the
	// package file. Instead of crashing we can attempt to load the trailer from the end of the file rather than using the
	// offset. For now we will continue to track the problem with an ensure while we try to hunt down the source of the
	// corruption.
	if (Summary.PayloadTocOffset < TotalSize())
	{
		Seek(Summary.PayloadTocOffset);
		bResult = PackageTrailer->TryLoad(*this);

		if (!bResult && Summary.GetFileVersionUE().ToValue() == (int32)EUnrealEngineObjectUE5Version::DATA_RESOURCES)
		{
			// There was an issue that was causing incorrect values to be written to PayloadTocOffset for
			// a limited time. In these cases we can try the slower ::TryLoadBackwards method of loading
			// the trailer. Note that we only do this if the FileVersion is
			// EUnrealEngineObjectUE5Version::DATA_RESOURCES as the bug was introduced while this was the
			// current version, so any package with an older or newer version should be safe.

			Seek(TotalSize());
			bResult = PackageTrailer->TryLoadBackwards(*this);
		}
	}
	else
	{
		Seek(TotalSize());
		bResult = PackageTrailer->TryLoadBackwards(*this);

		// If we managed to load the trailer anyway then we should fix up the offset and warn the user that re-saving the package file will probably fix it.
		// Otherwise we can 
		if (bResult)
		{
			const int64 OldOffset = Summary.PayloadTocOffset;
			Summary.PayloadTocOffset = PackageTrailer->GetPackageTrailerOffset();

			// Although we can survive PayloadTocOffset being corrupted it still indicates a problem. The ensure will make sure that the
			// problem is recorded so it can be tracked if it is still occurring.
			ensureMsgf(!GEnsureSerializePackageTrailer, TEXT("FPackageFileSummary::PayloadTocOffset is incorrect for '%s'. Offset: %lld ArchiveSize: %lld CorrectedOffset: %lld"), *GetArchiveName(), OldOffset, TotalSize(), Summary.PayloadTocOffset);

			// Log a warning so that the user knows how to repair their package file (by re-saving it)
			Async(EAsyncExecution::TaskGraphMainThread,
				[DebugName = GetDebugName()]()
				{
					FMessageLog("LoadErrors") .Warning(
						FText::Format(NSLOCTEXT("Core", "LinkerLoad_IncorrectSummary", "Package {0} has incorrect values in it's FPackageFileSummary, please re-save to fix this!"),
						FText::FromString(DebugName)));
				}
			);
		}
	}

	Seek(CurPos); // Restore the original position

	if (!bResult)
	{
		// If the archive has an error then we found a package trailer but it failed to serialize
		// correctly and we most likely have a problem with the file.
		// If the load failed but the archive is fine then the package is just of an older format
		// and there never was a package trailer to load.
		if (IsError())
		{
			Async(EAsyncExecution::TaskGraphMainThread,
				[DebugName = GetDebugName()]()
				{
					FMessageLog("LoadErrors").Error(
						FText::Format(NSLOCTEXT("Core", "LinkerLoad_CorruptTrailer", "Package {0} has a corrupted package trailer"),
						FText::FromString(DebugName)));
				}
			);

			return LINKER_Failed;
		}

		PackageTrailer.Reset();
	}

	bHasSerializedPackageTrailer = true;

	return LINKER_Loaded;
}

/**
 * Serializes the name table.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeNameMap()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::SerializeNameMap" ), STAT_LinkerLoad_SerializeNameMap, STATGROUP_LinkerLoad );

	// Text archives don't have name tables
	if (IsTextFormat())
	{
		return LINKER_Loaded;
	}

	// The name map is the first item serialized. We wait till all the header information is read
	// before any serialization. @todo async, @todo seamless: this could be spread out across name,
	// import and export maps if the package file summary contained more detailed information on
	// serialized size of individual entries.
	const int32 NameCount = Summary.NameCount;
	if (NameMap.Num() == 0 && NameCount > 0)
	{
		Seek(Summary.NameOffset);

		// Make sure there is something to precache first.
		if (Summary.TotalHeaderSize > 0)
		{
			bool bFinishedPrecaching = true;

			// Precache name, import and export map.
			if (bIsAsyncLoader)
			{
				bFinishedPrecaching = GetAsyncLoader()->ReadyToStartReadingHeader(bUseTimeLimit, bUseFullTimeLimit, TickStartTime, TimeLimit);
			}
			else
			{
				bFinishedPrecaching = Loader->Precache(Summary.NameOffset, Summary.TotalHeaderSize - Summary.NameOffset);
			}

			if (!bFinishedPrecaching)
			{
				return LINKER_TimedOut;
			}
		}
	}

	SCOPED_LOADTIMER(LinkerLoad_SerializeNameMap_ProcessingEntries);

	NameMap.Reserve(NameCount);
	FNameEntrySerialized NameEntry(ENAME_LinkerConstructor);
	for (int32 Idx = NameMap.Num(); Idx < NameCount; ++Idx)
	{
		*this << NameEntry;
		NameMap.Emplace(FName(NameEntry).GetDisplayIndex());

		constexpr int32 TimeSliceGranularity = 128;
		if (Idx % TimeSliceGranularity == TimeSliceGranularity - 1 &&
			NameMap.Num() != NameCount && IsTimeLimitExceeded(TEXT("serializing name map")))
		{
			return LINKER_TimedOut;
		}
	}

	return LINKER_Loaded;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeSoftObjectPathList()
{
	// Text archives don't have soft object path tables at the moment
	if (IsTextFormat())
	{
		return LINKER_Loaded;
	}

	if (SoftObjectPathListIndex == 0 && Summary.SoftObjectPathsCount > 0)
	{
		Seek(Summary.SoftObjectPathsOffset);
	}

#if WITH_EDITOR
	FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None,
		ESoftObjectPathCollectType::PackageHeader, ESoftObjectPathSerializeType::AlwaysSerialize);
#endif // WITH_EDITOR

	FStructuredArchive::FStream Stream = StructuredArchiveRootRecord->EnterStream(TEXT("SoftObjectPathList"));
	while (SoftObjectPathListIndex < Summary.SoftObjectPathsCount && !IsTimeLimitExceeded(TEXT("serializing soft object path list"), 100))
	{
		FSoftObjectPath& SoftObjectPath = SoftObjectPathList.AddDefaulted_GetRef();
		FStructuredArchive::FSlot Slot = Stream.EnterElement();
		SoftObjectPath.SerializePath(Slot.GetUnderlyingArchive());
		++SoftObjectPathListIndex;
	}

	// Return whether we finished this step and it's safe to start with the next.
	return ((SoftObjectPathListIndex == Summary.SoftObjectPathsCount) && !IsTimeLimitExceeded(TEXT("serializing soft object path list"))) ? LINKER_Loaded : LINKER_TimedOut;
}

/**
 * Serializes the gatherable text data container.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeGatherableTextDataMap(bool bForceEnableForCommandlet)
{
#if WITH_EDITORONLY_DATA
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::SerializeGatherableTextDataMap" ), STAT_LinkerLoad_SerializeGatherableTextDataMap, STATGROUP_LinkerLoad );

	// Skip serializing gatherable text data if we are using seekfree loading
	if( !bForceEnableForCommandlet && !GIsEditor )
	{
		return LINKER_Loaded;
	}

	if( !IsTextFormat() && GatherableTextDataMapIndex == 0 && Summary.GatherableTextDataCount > 0 )
	{
		Seek( Summary.GatherableTextDataOffset );
	}

	FStructuredArchive::FStream Stream = StructuredArchiveRootRecord->EnterStream(TEXT("GatherableTextData"));
	while (GatherableTextDataMapIndex < Summary.GatherableTextDataCount && !IsTimeLimitExceeded(TEXT("serializing gatherable text data map"), 100))
	{
		FGatherableTextData& GatherableTextData = GatherableTextDataMap.AddDefaulted_GetRef();
		Stream.EnterElement() << GatherableTextData;
		GatherableTextDataMapIndex++;
	}

	return ((GatherableTextDataMapIndex == Summary.GatherableTextDataCount) && !IsTimeLimitExceeded( TEXT("serializing gatherable text data map") )) ? LINKER_Loaded : LINKER_TimedOut;
#else
	return LINKER_Loaded;
#endif
}

/**
 * Serializes the import map.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeImportMap()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::SerializeImportMap" ), STAT_LinkerLoad_SerializeImportMap, STATGROUP_LinkerLoad );

	if(!IsTextFormat() && ImportMapIndex == 0 && Summary.ImportCount > 0 )
	{
		Seek( Summary.ImportOffset );
	}

	FStructuredArchive::FStream Stream = StructuredArchiveRootRecord->EnterStream(TEXT("ImportTable"));

	while( ImportMapIndex < Summary.ImportCount && !IsTimeLimitExceeded(TEXT("serializing import map"),100) )
	{
		FObjectImport& Import = ImportMap.AddDefaulted_GetRef();
		Stream.EnterElement() << Import;
		ImportMapIndex++;
	}

	// Return whether we finished this step and it's safe to start with the next.
	return ((ImportMapIndex == Summary.ImportCount) && !IsTimeLimitExceeded( TEXT("serializing import map") )) ? LINKER_Loaded : LINKER_TimedOut;
}

/**
 * Fixes up the import map, performing remapping for backward compatibility and such.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::FixupImportMap()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::FixupImportMap" ), STAT_LinkerLoad_FixupImportMap, STATGROUP_LinkerLoad );

	if( bHasFixedUpImportMap == false )
	{
#if WITH_EDITOR
		if (LoadProgressScope)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			LoadProgressScope->EnterProgressFrame(1);
		}
#endif
		// Fix up imports, not required if everything is cooked.
		if (!FPlatformProperties::RequiresCookedData())
		{
			static const FName NAME_BlueprintGeneratedClass("BlueprintGeneratedClass");

			auto AddNewPackageImport = [this](FObjectImport*& CurrentImport, int32 CurrentIndex, FName NewPackageName)
				{
					int32 NewImportIndex = ImportMap.Num();
					FObjectImport& NewImport = ImportMap.AddDefaulted_GetRef();
					// Adding to ImportMap may have reallocated, so reassign CurrentImport
					CurrentImport = &ImportMap[CurrentIndex];
					NewImport.ClassName = NAME_Package;
					NewImport.ClassPackage = GLongCoreUObjectPackageName;
					NewImport.ObjectName = NewPackageName;
					NewImport.OuterIndex = FPackageIndex();
					NewImport.XObject = nullptr;
					NewImport.SourceLinker = nullptr;
					NewImport.SourceIndex = -1;
					return FPackageIndex::FromImport(NewImportIndex);
				};
			auto AddNewObjectImport = [this](FObjectImport*& CurrentImport, int32 CurrentIndex,
				FPackageIndex NewImportOuter, FName NewImportName)
				{
					int32 NewImportIndex = ImportMap.Num();
					FObjectImport& NewImport = ImportMap.AddDefaulted_GetRef();
					// Adding to ImportMap may have reallocated, so reassign CurrentImport
					CurrentImport = &ImportMap[CurrentIndex];
					NewImport.ClassName = NAME_Object; // Don't know the class, but we won't need it. Set it to UObject
					NewImport.ClassPackage = GLongCoreUObjectPackageName;
					NewImport.ObjectName = NewImportName;
					NewImport.OuterIndex = NewImportOuter;
					NewImport.XObject = nullptr;
					NewImport.SourceLinker = nullptr;
					NewImport.SourceIndex = -1;
					return FPackageIndex::FromImport(NewImportIndex);
				};

			TArray<int32> PackageIndexesToClear;
			for (int32 i=0; i<ImportMap.Num(); i++)
			{
				FObjectImport* Import = &ImportMap[i];

				// Compute class name first, as instance can override it
				const FCoreRedirect* ClassValueRedirect = nullptr;
				FCoreRedirectObjectName OldClassName(Import->ClassName, NAME_None, Import->ClassPackage), NewClassName;

				FCoreRedirects::RedirectNameAndValues(ECoreRedirectFlags::Type_Class, OldClassName, NewClassName, &ClassValueRedirect);

				if (ClassValueRedirect)
				{
					// Apply class value redirects before other redirects, to mirror old subobject order
					const FString* NewInstanceName = ClassValueRedirect->ValueChanges.Find(Import->ObjectName.ToString());
					if (NewInstanceName)
					{
						// Rename this import directly
						FString Was = GetImportFullName(i);
						Import->ObjectName = FName(**NewInstanceName);

						if (Import->ObjectName != NAME_None)
						{
							FString Now = GetImportFullName(i);
							UE_LOGF(LogLinker, Verbose, "FLinkerLoad::FixupImportMap() - Renamed object from %ls   to   %ls", *Was, *Now);
						}
						else
						{
							UE_LOGF(LogLinker, Verbose, "FLinkerLoad::FixupImportMap() - Removed object %ls", *Was);
						}
					}
				}

				FCoreRedirectObjectName OldObjectName(GetImportPathName(i)), NewObjectName;
				ECoreRedirectFlags ObjectRedirectFlags = FCoreRedirects::GetFlagsForTypeName(NewClassName.PackageName, NewClassName.ObjectName);
				const FCoreRedirect* ValueRedirect = nullptr;

				FCoreRedirects::RedirectNameAndValues(ObjectRedirectFlags, OldObjectName, NewObjectName, &ValueRedirect);

				if (ValueRedirect && ValueRedirect->OverrideClassName.IsValid())
				{
					// Override class name if found, even if the name didn't actually change
					NewClassName = ValueRedirect->OverrideClassName;
				}

				if (NewObjectName != OldObjectName)
				{
					if (Import->OuterIndex.IsNull())
					{
						// If this has no outer it's a package and we don't want to rename it, the subobject renames will handle creating the new package import
						// We do need to clear these at the end so it doesn't try to load nonexistent packages
						PackageIndexesToClear.Add(i);
					}
					else
					{
						FPackageIndex NewPackageIndex;
						if (!FindImportPackage(NewObjectName.PackageName, NewPackageIndex))
						{
							NewPackageIndex = AddNewPackageImport(Import, i, NewObjectName.PackageName);
						}

						FPackageIndex OuterIndex = NewPackageIndex;
						if (!NewObjectName.OuterName.IsNone())
						{
							TStringBuilder<256> OuterNameBuffer;
							OuterNameBuffer << NewObjectName.OuterName;
							FStringView OuterName(OuterNameBuffer);
							while (!OuterName.IsEmpty())
							{
								FStringView FirstOuter;
								FStringView Remainder;
								FPackageName::ObjectPathSplitFirstName(OuterName, FirstOuter, Remainder);
								FPackageIndex NewOuterIndex;
								FName FirstOuterName(FirstOuter);
								if (!FindImport(OuterIndex, FirstOuterName, NewOuterIndex))
								{
									NewOuterIndex = AddNewObjectImport(Import, i, OuterIndex, FirstOuterName);
								}
								OuterName = Remainder;
								OuterIndex = NewOuterIndex;
							}
						}

						Import->OuterIndex = OuterIndex;
#if WITH_EDITOR
						// If this is a class, set old name here
						if (ObjectRedirectFlags == ECoreRedirectFlags::Type_Class)
						{
							Import->OldClassName = Import->ObjectName;
						}
#endif
						// Change object name
						Import->ObjectName = NewObjectName.ObjectName;

						UE_LOGF(LogLinker, Verbose, "FLinkerLoad::FixupImportMap() - Pkg<%ls> - Renamed Object %ls -> %ls", *LinkerRoot->GetName(), *OldObjectName.ToString(), *NewObjectName.ToString());
					}
				}

				if (NewClassName != OldClassName)
				{
					// Swap class if needed
					if (Import->ClassPackage != NewClassName.PackageName && !IsCoreUObjectPackage(NewClassName.PackageName))
					{
						FPackageIndex NewPackageIndex;

						if (!FindImportPackage(NewClassName.PackageName, NewPackageIndex))
						{
							NewPackageIndex = AddNewPackageImport(Import, i, NewClassName.PackageName);
						}
					}
#if WITH_EDITOR
					Import->OldClassName = Import->ClassName;
#endif
					// Change class name/package
					Import->ClassPackage = NewClassName.PackageName;
					Import->ClassName = NewClassName.ObjectName;

					// Also change CDO name if needed
					FString NewDefaultObjectName = Import->ObjectName.ToString();

					if (NewDefaultObjectName.StartsWith(DEFAULT_OBJECT_PREFIX))
					{
						NewDefaultObjectName = FString(DEFAULT_OBJECT_PREFIX);
						NewDefaultObjectName += NewClassName.ObjectName.ToString();
						Import->ObjectName = FName(*NewDefaultObjectName);
					}

					UE_LOGF(LogLinker, Verbose, "FLinkerLoad::FixupImportMap() - Pkg<%ls> - Renamed Class %ls -> %ls", *LinkerRoot->GetName(), *OldClassName.ToString(), *NewClassName.ToString());
				}
			}

			// Clear any packages that got renamed, once all children have been fixed up
			for (int32 PackageIndex : PackageIndexesToClear)
			{
				FObjectImport& Import = ImportMap[PackageIndex];
				check(Import.OuterIndex.IsNull());
				Import.ObjectName = NAME_None;
			}
		}

		// Avoid duplicate work in async case.
		bHasFixedUpImportMap = true;
	}
	return IsTimeLimitExceeded( TEXT("fixing up import map") ) ? LINKER_TimedOut : LINKER_Loaded;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::PopulateInstancingContext()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLinkerLoad::PopulateInstancingContext);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::PopulateInstancingContext"), STAT_LinkerLoad_PopulateInstancingContext, STATGROUP_LinkerLoad);

	if (!bHasPopulatedInstancingContext)
	{
#if WITH_EDITOR
		// Generate Instance Remapping if needed
		if (IsContextInstanced())
		{
			auto AddInstancedMapping = [this](const FString& OuterPackageName, FName InstancingPackageName) -> bool
			{
				FName InstancedName;
				// if there's isn't already a remapping for that package, create one
				if (!InstancingContext.FindPackageMapping(InstancingPackageName, InstancedName))
				{
					InstancedName = *FLinkerInstancingContext::GetInstancedPackageName(OuterPackageName, InstancingPackageName.ToString());
					InstancingContext.AddPackageMapping(InstancingPackageName, InstancedName);
					return true;
				}
				return false;
			};

			FString LinkerPackageName = LinkerRoot->GetName();

			// Add import package we should instantiate since object in this instanced linker are outered to them
			for (const FObjectExport& Export : ExportMap)
			{
				if (Export.OuterIndex.IsImport())
				{
					FObjectImport* Import = &Imp(Export.OuterIndex);
					while (Import->OuterIndex.IsImport())
					{
						if (Import->HasPackageName())
						{
							AddInstancedMapping(LinkerPackageName, Import->PackageName);
						}
						Import = &Imp(Import->OuterIndex);
					}
					check(Import->OuterIndex.IsNull() && !Import->HasPackageName());
					AddInstancedMapping(LinkerPackageName, Import->ObjectName);
				}
			}

			// Also add import package, we should instantiate as their are outered to object in this package or one of their outer is already instanced
			auto HasInstancedOuterChain = [this](const FObjectImport* InImport) -> FName
			{
				while (InImport->OuterIndex.IsImport())
				{
					InImport = &Imp(InImport->OuterIndex);
					FName ImportPackageName = InImport->HasPackageName() ? InImport->GetPackageName() : (InImport->OuterIndex.IsNull() ? InImport->ObjectName : NAME_None);
					if (!ImportPackageName.IsNone())
					{
						FName InstancedRemap = InstancingContext.RemapPackage(ImportPackageName);
						if (InstancedRemap != ImportPackageName)
						{
							return InstancedRemap;
						}
					}
				}
				// return if the import outer is an export or not if we didn't find an instanced import
				return InImport->OuterIndex.IsExport() ? NAME_TRUE : NAME_FALSE;
			};

			for (int32 ImportIndex = 0; ImportIndex < ImportMap.Num(); ++ImportIndex)
			{
				const FObjectImport& Import = ImportMap[ImportIndex];
				if (Import.HasPackageName())
				{
					FName Result = HasInstancedOuterChain(&Import);
					// Outer chain has an export
					if (Result == NAME_TRUE)
					{
						AddInstancedMapping(LinkerPackageName, Import.PackageName);
					}
					// Outer chain has an instanced import
					else if (!Result.IsNone() && Result != NAME_FALSE)
					{
						const FString InstancedOuterNameStr = Result.ToString();
						const bool bAdded = AddInstancedMapping(InstancedOuterNameStr, Import.GetPackageName());
						if (bAdded)
						{
							UE_LOGF(LogLinker, Warning, "Mapping for '%ls' with external package '%ls' not provided while outer '%ls' is instanced."
								, *GetImportPathName(ImportIndex), *Import.GetPackageName().ToString(), *InstancedOuterNameStr);
						}
					}
				}
			}
		}
#endif
		// Avoid duplicate work in async case.
		bHasPopulatedInstancingContext = true;
	}
	return IsTimeLimitExceeded(TEXT("populating instancing context")) ? LINKER_TimedOut : LINKER_Loaded;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::RelocateReferences()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::RelocateReferences"), STAT_LinkerLoad_RelocateReferences, STATGROUP_LinkerLoad);

	if (!bHasRelocatedReferences)
	{
#if WITH_EDITOR
		// Validate if the package was moved and we want to generate fix up for references
		FString PackageNameToLoad = GetPackagePath().GetPackageName();

		UE::Package::Relocation::Private::FPackageRelocationContext RelocationArgs;

		bool bRelocated = UE::Package::Relocation::Private::ShouldApplyRelocation(Summary, PackageNameToLoad, RelocationArgs);
		// Do not consider a package relocated if it's being loaded for Diff
		if (bRelocated && (LoadFlags & LOAD_ForDiff) == 0)
		{
			UE_LOGF(LogPackageRelocation, Verbose, "Loading relocated package (%ls). The package was saved as (%ls).", *PackageNameToLoad, *Summary.PackageName);
			UE::Package::Relocation::Private::ApplyRelocationToObjectImportMap(RelocationArgs, ImportMap);
			UE::Package::Relocation::Private::ApplyRelocationToSoftObjectArray(RelocationArgs, SoftObjectPathList);

			bIsPackageRelocated = true;
		}
#endif

		// Avoid duplicate work in async case.
		bHasRelocatedReferences = true;
	}
	return IsTimeLimitExceeded(TEXT("relocating the ImportMap and SoftObjectPathList")) ? LINKER_TimedOut : LINKER_Loaded;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::ApplyInstancingContext()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::ApplyInstancingContext"), STAT_LinkerLoad_ApplyInstancingContext, STATGROUP_LinkerLoad);
	if (!bHasAppliedInstancingContext)
	{
		for (FSoftObjectPath& SoftObjectPath : SoftObjectPathList)
		{
			FixupSoftObjectPathForInstancedPackage(SoftObjectPath);
		}

		// Avoid duplicate work in async case.
		bHasAppliedInstancingContext = true;
	}

	return IsTimeLimitExceeded(TEXT("applying the instancing context to the SoftObjectPathList")) ? LINKER_TimedOut : LINKER_Loaded;;
}

/**
 * Serializes the export map.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeExportMap()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::SerializeExportMap" ), STAT_LinkerLoad_SerializeExportMap, STATGROUP_LinkerLoad );

	if(!IsTextFormat() && ExportMapIndex == 0 && Summary.ExportCount > 0)
	{
		Seek( Summary.ExportOffset );
	}

	FStructuredArchive::FStream Stream = StructuredArchiveRootRecord->EnterStream(TEXT("ExportTable"));

	while( ExportMapIndex < Summary.ExportCount && !IsTimeLimitExceeded(TEXT("serializing export map"),100) )
	{
		FObjectExport& Export = ExportMap.AddDefaulted_GetRef();
		Stream.EnterElement() << Export;
		Export.ThisIndex = FPackageIndex::FromExport(ExportMapIndex);
		Export.bWasFiltered = FilterExport(Export);
		ExportMapIndex++;
	}

	// Return whether we finished this step and it's safe to start with the next.
	if ((ExportMapIndex == Summary.ExportCount) && !IsTimeLimitExceeded(TEXT("serializing export map")))
	{
		TrackPackageAssetClass(LinkerRoot, *this, ExportMap);
		return LINKER_Loaded;
	}
	else
	{
		return LINKER_TimedOut;
	}
}

#if WITH_TEXT_ARCHIVE_SUPPORT

FStructuredArchiveSlot FLinkerLoad::GetExportSlot(FPackageIndex InExportIndex)
{
	check(InExportIndex.IsExport());
	int32 Index = InExportIndex.ToExport();
	return ExportReaders[Index]->GetRoot();
}

FString ExtractObjectName(const FString& InFullPath)
{
	FString ObjectName = InFullPath;
	int32 LastDot, LastSemi;
	InFullPath.FindLastChar(TEXT('.'), LastDot);
	InFullPath.FindLastChar(TEXT(':'), LastSemi);
	int32 StartOfObjectName = FMath::Max(LastDot, LastSemi);
	if (StartOfObjectName != INDEX_NONE)
	{
		return ObjectName.Right(ObjectName.Len() - StartOfObjectName - 1);
	}
	return ObjectName;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::ConstructExportsReaders()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::ConstructExportsReaders"), STAT_LinkerLoad_ConstructExportsReaders, STATGROUP_LinkerLoad);

	if (!bHasConstructedExportsReaders && IsTextFormat())
	{
		int32 NumExports = 0;
		FStructuredArchiveMap PackageExports = StructuredArchiveRootRecord->EnterMap(TEXT("Exports"), NumExports);

		ExportReaders.AddDefaulted(NumExports);
		for (int32 ExportIndex = 0; ExportIndex < NumExports; ++ExportIndex)
		{
			FString ExportName;
			ExportReaders[ExportIndex] = new FStructuredArchiveChildReader(PackageExports.EnterElement(ExportName));
		}

		bHasConstructedExportsReaders = true;
		return LINKER_Loaded;
	}
	else
	{
		return LINKER_Loaded;
	}
}

#endif // WITH_TEXT_ARCHIVE_SUPPORT

#if WITH_METADATA
/**
 * Serializes the meta data.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeMetaData()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::SerializeMetaData" ), STAT_LinkerLoad_SerializeMetaData, STATGROUP_LinkerLoad );

	if ( Summary.MetaDataOffset == 0 )
	{
		// This package was saved before meta data stored in the package summary
		return LINKER_Loaded;
	}

	FMetaData& PackageMetaData = LinkerRoot->GetMetaData();

	if (MetaDataMapIndex == 0)
	{
		if (!IsTextFormat())
		{
			Seek(Summary.MetaDataOffset);
		}

		*StructuredArchiveRootRecord << SA_VALUE(TEXT("NumObjectMetaDataMap"), NumObjectMetaDataMap);
		*StructuredArchiveRootRecord << SA_VALUE(TEXT("NumRootMetaDataMap"), NumRootMetaDataMap);
	}

	if (MetaDataMapIndex < NumObjectMetaDataMap)
	{
		FStructuredArchive::FStream Stream = StructuredArchiveRootRecord->EnterStream(TEXT("ObjectMetaDataMap"));
		while (MetaDataMapIndex < NumObjectMetaDataMap && !IsTimeLimitExceeded(TEXT("serializing meta data"), 100))
		{
			TPair<FSoftObjectPath, TMap<FName, FString>> ObjectMetaData;
			Stream.EnterElement() << ObjectMetaData;

			// Remap keys if needed
			TMap<FName, FString>& CurrentMap = ObjectMetaData.Value;
			for (TMap<FName, FString>::TIterator PairIt(CurrentMap); PairIt; ++PairIt)
			{
				const FName OldKey = PairIt.Key();
				const FName NewKey = FMetaData::GetRemappedKeyName(OldKey);
				if (NewKey != NAME_None)
				{
					const FString Value = PairIt.Value();
					PairIt.RemoveCurrent();
					CurrentMap.Add(NewKey, Value);
					UE_LOGF(LogLinker, Verbose, "Remapping old metadata key '%ls' to new key '%ls' on object '%ls'.", *OldKey.ToString(), *NewKey.ToString(), *ObjectMetaData.Key.ToString());
				}
			}

			PackageMetaData.ObjectMetaDataMap.Add(ObjectMetaData.Key, ObjectMetaData.Value);
			MetaDataMapIndex++;
		}
	}

	if ((MetaDataMapIndex >= NumObjectMetaDataMap) && (MetaDataMapIndex < (NumObjectMetaDataMap + NumRootMetaDataMap)))
	{
		FStructuredArchive::FStream Stream = StructuredArchiveRootRecord->EnterStream(TEXT("RootMetaDataMap"));
		while (MetaDataMapIndex < (NumObjectMetaDataMap + NumRootMetaDataMap) && !IsTimeLimitExceeded(TEXT("serializing meta data"), 100))
		{
			TPair<FName, FString> RootMetaData;
			Stream.EnterElement() << RootMetaData;

			// Remap keys if needed
			const FName OldKey = RootMetaData.Key;
			const FName NewKey = FMetaData::GetRemappedKeyName(OldKey);
			if (NewKey != NAME_None)
			{
				const FString Value = RootMetaData.Value;
				RootMetaData = TPair<FName, FString>(NewKey, Value);
				UE_LOGF(LogLinker, Verbose, "Remapping old metadata key '%ls' to new key '%ls' on root.", *OldKey.ToString(), *NewKey.ToString());
			}

			PackageMetaData.RootMetaDataMap.Add(RootMetaData.Key, RootMetaData.Value);
			MetaDataMapIndex++;
		}
	}

	// Return whether we finished this step and it's safe to start with the next.
	return ((MetaDataMapIndex == (NumObjectMetaDataMap + NumRootMetaDataMap)) && !IsTimeLimitExceeded( TEXT("serializing meta data") )) ? LINKER_Loaded : LINKER_TimedOut;
}
#endif

/**
 * Serializes the depends map.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeDependsMap()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::SerializeDependsMap" ), STAT_LinkerLoad_SerializeDependsMap, STATGROUP_LinkerLoad );

	// Skip serializing depends map if we are using seekfree loading
	if( FPlatformProperties::RequiresCookedData()
	// or we are neither Editor nor commandlet
	|| !(GIsEditor || IsRunningCommandlet()) )
	{
		return LINKER_Loaded;
	}

	if ( Summary.DependsOffset == 0 )
	{
		// This package was saved badly
		return LINKER_Loaded;
	}

	// depends map size is same as export map size
	if (DependsMapIndex == 0 && Summary.ExportCount > 0)
	{
		if (!IsTextFormat())
		{
			Seek(Summary.DependsOffset);
		}

		// Pre-size array to avoid re-allocation of array of arrays!
		DependsMap.AddZeroed(Summary.ExportCount);
	}

	FStructuredArchive::FStream Stream = StructuredArchiveRootRecord->EnterStream(TEXT("DependsMap"));
	while (DependsMapIndex < Summary.ExportCount && !IsTimeLimitExceeded(TEXT("serializing depends map"), 100))
	{
		TArray<FPackageIndex>& Depends = DependsMap[DependsMapIndex];
		Stream.EnterElement() << Depends;
		DependsMapIndex++;
	}

	// Return whether we finished this step and it's safe to start with the next.
	return ((DependsMapIndex == Summary.ExportCount) && !IsTimeLimitExceeded( TEXT("serializing depends map") )) ? LINKER_Loaded : LINKER_TimedOut;
}

/**
* Serializes the depends map.
*/
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializePreloadDependencies()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::SerializePreloadDependencies"), STAT_LinkerLoad_SerializePreloadDependencies, STATGROUP_LinkerLoad);

	// Skip serializing depends map if this is the editor or the data is missing
	if (bHasSerializedPreloadDependencies || Summary.PreloadDependencyCount < 1 || Summary.PreloadDependencyOffset <= 0)
	{
		return LINKER_Loaded;
	}

	if (!IsTextFormat())
	{
		Seek(Summary.PreloadDependencyOffset);
	}

	PreloadDependencies.AddUninitialized(Summary.PreloadDependencyCount);

	if ((IsSaving()            // if we are saving, we always do the ordinary serialize as a way to make sure it matches up with bulk serialization
		&& !IsCooking()            // but cooking and transacting is performance critical, so we skip that
		&& !IsTransacting())
		|| IsByteSwapping()        // if we are byteswapping, we need to do that per-element
		)
	{
		//@todoio check endiness and fastpath this as a single serialize
		FStructuredArchive::FStream Stream = StructuredArchiveRootRecord->EnterStream(TEXT("PreloadDependencies"));
		for (int32 Index = 0; Index < Summary.PreloadDependencyCount; Index++)
		{
			FPackageIndex Idx;
			Stream.EnterElement() << Idx;

			PreloadDependencies[Index] = Idx;
		}
	}
	else
	{
		check(!IsTextFormat());
		Serialize(PreloadDependencies.GetData(), Summary.PreloadDependencyCount * sizeof(FPackageIndex));
	}

	bHasSerializedPreloadDependencies = true;

	// Return whether we finished this step and it's safe to start with the next.
	return !IsTimeLimitExceeded(TEXT("serialize preload dependencies")) ? LINKER_Loaded : LINKER_TimedOut;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeDataResourceMap()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::SerializeDataResourceMap"), STAT_LinkerLoad_SerializeDataResourceMap, STATGROUP_LinkerLoad);

	TOptional<FStructuredArchive::FSlot> DataResourcesSlot;

	if (IsTextFormat())
	{
		DataResourcesSlot = StructuredArchiveRootRecord->TryEnterField(TEXT("DataResources"), false);
	}
	else if (Summary.DataResourceOffset > 0)
	{
		Seek(Summary.DataResourceOffset);
		DataResourcesSlot = StructuredArchiveRootRecord->EnterField(TEXT("DataResources"));
	}

	if (DataResourcesSlot.IsSet())
	{
		FObjectDataResource::Serialize(*DataResourcesSlot, DataResourceMap);
	}

	return LINKER_Loaded;
}

#if WITH_EDITORONLY_DATA

FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeImportTypeHierarchies()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::SerializeImportTypeHierarchies"), STAT_LinkerLoad_SerializeImportTypeHierarchies, STATGROUP_LinkerLoad);

	// Skip serializing import types if the package does not include any or when dealing with a text format.
	if (bHasSerializedImportTypeHierarchies || (Summary.ImportTypeHierarchiesCount == 0) || (Summary.ImportTypeHierarchiesOffset == 0) || IsTextFormat())
	{
		return LINKER_Loaded;
	}

	if (Summary.ImportTypeHierarchiesCount < 0)
	{
		return LINKER_Failed;
	}

	Seek(Summary.ImportTypeHierarchiesOffset);

	int32 NumTypeHierarchies = 0;
	FStructuredArchive::FArray Array = StructuredArchiveRootRecord->EnterArray(TEXT("ImportTypeHierarchies"), NumTypeHierarchies);

	if (NumTypeHierarchies == Summary.ImportTypeHierarchiesCount)
	{
		for (int32 Hierarchy = 0; Hierarchy < Summary.ImportTypeHierarchiesCount; ++Hierarchy)
		{
			FStructuredArchive::FSlot ElementSlot = Array.EnterElement();

			TPair<FPackageIndex, UE::Serialization::Private::FImportTypeHierarchy> Pair;
			ElementSlot << Pair;

			ImportTypeHierarchies.Add(Pair.Key, Pair.Value);
		}
	}
	else
	{
		return LINKER_Failed;
	}

	// Return whether we finished this step and it's safe to start with the next.
	if (StructuredArchiveRootRecord->GetUnderlyingArchive().IsCriticalError())
	{
		return LINKER_Failed;
	}
	if (IsTimeLimitExceeded(TEXT("serializing import type hierarchies")))
	{
		return LINKER_TimedOut;
	}

	bHasSerializedImportTypeHierarchies = true;

	return LINKER_Loaded;
}

FLinkerLoad::ELinkerStatus FLinkerLoad::AddDynamicImports()
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FLinkerLoad::AddDynamicImports"), STAT_LinkerLoad_AddDynamicImports, STATGROUP_LinkerLoad);

	if (bHasAddedDynamicImports)
	{
		return LINKER_Loaded;
	}

	/* Dynamic imports are imports that were not present in a package, and get added during package loading, before imports (dependencies) are loaded
	 * To use the feature, implement InjectDynamicImportsFor for the class of an object that must pull on additional imports, and set RF_HasDynamicImports on that object, then save the object's package.
	 * On its next load, InjectDynamicImportsFor will be invoked, letting the implementation add new entries into the linker tables.
	 * Be aware that the types of objects saved in a package might not represent the types active in the engine; classes can have been re-parented, deleted, etc. since the last time the package was saved.
	 * As such when changing the class hierarchy of objects that use Dynamic Imports, mitigation steps might be taken.
	 * 1 - A class that implemented InjectDynamicImportsFor was deleted; In this case the method will not be invoked anymore when loading a package that still has the type.
	 * 2 - A class that implements InjectDynamicImportsFor was renamed through class redirectors; the loop below checks into core redirects to try to remap classes, the method should still be invoked on the new type.
	 * 3 - A new class that implements InjectDynamicImportsFor was added in the class hierarchy of a type; in that case this new type is still not known to this package and the method will not be invoked until the package is re-saved.
	 * 4 - A new class that implements InjectDynamicImportsFor was added in the class hierarchy of a type, but there already was a more basic class implementing InjectDynamicImportsFor;
	 *     in that case, the old class will be invoked until the package is re-saved. If re-saving is not possible because of the invocation, you could mitigate with versioning by looking at the branch
	 *     version with CustomVer(xyz) with the Linker argument, like you would do in Serialize, or disabling it temporarily to re-save, or any other method.
	 * 5 - A class that implements InjectDynamicImportsFor was removed from the hierarchy, but the class still exists; in that case InjectDynamicImportsFor would still be invoked until the package is re-saved.
	 *     You would also need mitigation steps like in case number 4. */

	TArray<FDynamicImport> DynamicImports;

	// Check if we have exports that depend on external packages
	for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
	{
		const FObjectExport& Export = ExportMap[ExportIndex];
		if ((Export.ObjectFlags & RF_HasDynamicImports) == 0)
		{
			continue;
		}

		// Attempt to find a native class corresponding to this export
		UClass* FoundClass = nullptr;
		FString ClassName;
 
		if (Export.ClassIndex.IsImport())
		{
			ClassName = GetImportPathName(Export.ClassIndex);
		}
		else
		{
			// Since the class is in this package, we won't find it as it is not loaded yet and we will rely on the ImportTypeHierarchies table below.
			// But we might find a redirector for it, so we still get the class name.
			ClassName = GetExportPathName(Export.ClassIndex);
		}
 
		// Check for class redirectors
		FCoreRedirectObjectName RedirectedName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, ClassName);
		if (RedirectedName.IsValid())
		{
			if (UPackage* TypePackage = FindObjectFast<UPackage>(nullptr, RedirectedName.PackageName, EFindObjectFlags::ExactClass))
			{
				FoundClass = FindObjectFast<UClass>(TypePackage, RedirectedName.ObjectName, EFindObjectFlags::None);
			}
		}
		else
		{
			FoundClass = Cast<UClass>(StaticFindObject(UClass::StaticClass(), nullptr, ClassName));
		}

		// If we still cannot find it, go through its saved type hierarchy
		if (FoundClass == nullptr)
		{
			const UE::Serialization::Private::FImportTypeHierarchy* TypeHierarchy = ImportTypeHierarchies.Find(Export.ClassIndex);
			if (TypeHierarchy != nullptr && TypeHierarchy->GetSuperTypes().Num() > 0)
			{
				// Find the first native type, which should always be loaded
				for (const UE::Serialization::Private::FTypeResource& TypeRes : TypeHierarchy->GetSuperTypes())
				{
					// Only retain native classes
					if (TypeRes.ClassName == NAME_Class || TypeRes.ClassName == NAME_VerseClass)
					{
						// Check for class redirectors
						RedirectedName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, { TypeRes.TypeName, FName{}, TypeRes.PackageName } );
						if (RedirectedName.IsValid())
						{
							if (UPackage* TypePackage = FindObjectFast<UPackage>(nullptr, RedirectedName.PackageName, EFindObjectFlags::ExactClass))
							{
								FoundClass = FindObjectFast<UClass>(TypePackage, RedirectedName.ObjectName, EFindObjectFlags::None);
							}
						}
						else
						{
							if (UPackage* TypePackage = FindObjectFast<UPackage>(nullptr, TypeRes.PackageName, EFindObjectFlags::ExactClass))
							{
								FoundClass = FindObjectFast<UClass>(TypePackage, TypeRes.TypeName, EFindObjectFlags::None);
							}
						}

						if (FoundClass != nullptr)
						{
							break;
						}
					}
				}
			}
		}

		if (FoundClass != nullptr)
		{
			DynamicImports.SetNum(0);
			FoundClass->CallInjectDynamicImportsFor(*this, Export, DynamicImports);

			if (DynamicImports.Num() > 0)
			{
				ImportMap.Reserve(ImportMap.Num() + DynamicImports.Num());

				for (const FDynamicImport& DynamicImport : DynamicImports)
				{
					if (DynamicImport.PackageName == NAME_None)
					{
						continue;
					}

					// set the dynamic import index if necessary 
					if (DynamicImportsIndex == INDEX_NONE)
					{
						DynamicImportsIndex = ImportMap.Num();
					}

					// Create package import
					FObjectImport& PackageImport = ImportMap.AddDefaulted_GetRef();
					PackageImport.ObjectName = DynamicImport.PackageName;
					PackageImport.ClassPackage = GLongCoreUObjectPackageName;
					PackageImport.ClassName = NAME_Package;
					PackageImport.PackageName = NAME_None;
					PackageImport.SourceIndex = -1;
					PackageImport.bImportOptional = false;
					PackageImport.XObject = nullptr;
					PackageImport.SourceLinker = nullptr;

					if (!DynamicImport.ObjectName.IsNone())
					{
						// Create object import
						FObjectImport& ObjectImport = ImportMap.AddDefaulted_GetRef();
						ObjectImport.ObjectName = DynamicImport.ObjectName;
						ObjectImport.ClassName = DynamicImport.ClassName;
						ObjectImport.ClassPackage = DynamicImport.ClassPackage;
						ObjectImport.OuterIndex = FPackageIndex::FromExport(ExportIndex);
						ObjectImport.PackageName = DynamicImport.PackageName;
						ObjectImport.SourceIndex = -1;
						ObjectImport.bImportOptional = false;
						ObjectImport.XObject = nullptr;
						ObjectImport.SourceLinker = nullptr;
					}

					if (DynamicImport.InstanceName != NAME_None && DynamicImport.InstanceName != DynamicImport.PackageName)
					{
						InstancingContext.AddPackageMapping(DynamicImport.PackageName, DynamicImport.InstanceName);
					}
				}

				Summary.ImportCount = ImportMap.Num();
			}
		}
		else
		{
			UE_ASSET_LOG(LogLinker, Error, PackagePath, TEXT("AddDynamicImports failed to Find Class \"%s\" and its SuperClasses, some dynamic imports will not load correctly"),
				*ClassName);
		}
	}

	bHasAddedDynamicImports = true;

	return LINKER_Loaded;
}

#endif // WITH_EDITOR

/**
 * Serializes thumbnails
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::SerializeThumbnails( bool bForceEnableInGame/*=false*/ )
{
#if WITH_EDITORONLY_DATA
	// Skip serializing thumbnails if we are using seekfree loading
	if( !bForceEnableInGame && !GIsEditor )
	{
		return LINKER_Loaded;
	}

	TOptional<FStructuredArchive::FSlot> ThumbnailsSlot;

	if (IsTextFormat())
	{
		ThumbnailsSlot = StructuredArchiveRootRecord->TryEnterField(TEXT("Thumbnails"), false);
		if (!ThumbnailsSlot.IsSet())
		{
			return LINKER_Loaded;
		}
	}
	else
	{
		if(Summary.ThumbnailTableOffset > 0)
		{
			ThumbnailsSlot = StructuredArchiveRootRecord->EnterField(TEXT("Thumbnails"));
		}
	}

	if (ThumbnailsSlot.IsSet())
	{
		FStructuredArchive::FRecord Record = ThumbnailsSlot->EnterRecord();
		TOptional<FStructuredArchive::FSlot> IndexSlot;

		if (IsTextFormat())
		{
			IndexSlot = Record.TryEnterField(TEXT("Index"), false);
		}
		else
		{
			// Seek to the thumbnail table of contents
			Seek(Summary.ThumbnailTableOffset);
			IndexSlot.Emplace(Record.EnterField(TEXT("Index")));
		}

		if (IndexSlot.IsSet())
		{
			// Load number of thumbnails
			int32 ThumbnailCount = 0;

			FStructuredArchive::FArray IndexArray = IndexSlot->EnterArray(ThumbnailCount);

			// Allocate a new thumbnail map if we need one
			if (!LinkerRoot->HasThumbnailMap())
			{
				LinkerRoot->SetThumbnailMap(MakeUnique<FThumbnailMap>());
			}

			// Load thumbnail names and file offsets
			FThumbnailMap& ThumbnailMap = LinkerRoot->AccessThumbnailMap();
			TArray< FObjectFullNameAndThumbnail > ThumbnailInfoArray;
			for (int32 CurObjectIndex = 0; CurObjectIndex < ThumbnailCount; ++CurObjectIndex)
			{
				FStructuredArchive::FRecord IndexRecord = IndexArray.EnterElement().EnterRecord();
				FObjectFullNameAndThumbnail ThumbnailInfo;

				FString ObjectClassName;
				// Newer packages always store the class name for each asset
				IndexRecord << SA_VALUE(TEXT("ObjectClassName"), ObjectClassName);

				// Object path
				FString ObjectPathWithoutPackageName;
				IndexRecord << SA_VALUE(TEXT("ObjectPathWithoutPackageName"), ObjectPathWithoutPackageName);
				const FString ObjectPath(LinkerRoot->GetName() + TEXT(".") + ObjectPathWithoutPackageName);


				// Create a full name string with the object's class and fully qualified path
				const FString ObjectFullName(ObjectClassName + TEXT(" ") + ObjectPath);
				ThumbnailInfo.ObjectFullName = FName(*ObjectFullName);

				// File offset for the thumbnail (already saved out.)
				IndexRecord << SA_VALUE(TEXT("FileOffset"), ThumbnailInfo.FileOffset);

				// Only bother loading thumbnails that don't already exist in memory yet.  This is because when we
				// go to load thumbnails that aren't in memory yet when saving packages we don't want to clobber
				// thumbnails that were freshly-generated during that editor session
				if (!ThumbnailMap.Contains(ThumbnailInfo.ObjectFullName))
				{
					// Add to list of thumbnails to load
					ThumbnailInfoArray.Add(ThumbnailInfo);
				}
			}


			FStructuredArchive::FStream DataStream = Record.EnterStream(TEXT("Thumbnails"));

			// Now go and load and cache all of the thumbnails
			for (int32 CurObjectIndex = 0; CurObjectIndex < ThumbnailInfoArray.Num(); ++CurObjectIndex)
			{
				const FObjectFullNameAndThumbnail& CurThumbnailInfo = ThumbnailInfoArray[CurObjectIndex];

				// Seek to the location in the file with the image data
				if (!IsTextFormat())
				{
					Seek(CurThumbnailInfo.FileOffset);
				}

				// Load the image data
				FObjectThumbnail LoadedThumbnail;
				LoadedThumbnail.Serialize(DataStream.EnterElement());

				if (!LoadedThumbnail.HasValidImageData())
				{
					// If we failed to load the thumbnail, stop loading as it might be unsafe
					// to continue reading the stream.
					break;
				}

				// Store the data!
				ThumbnailMap.Add(CurThumbnailInfo.ObjectFullName, LoadedThumbnail);
			}
		}
	}
#endif // WITH_EDITORONLY_DATA

	// Finished!
	return LINKER_Loaded;
}



/**
 * Creates the export hash. This relies on the import and export maps having already been serialized.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::CreateExportHash()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::CreateExportHash" ), STAT_LinkerLoad_CreateExportHash, STATGROUP_LinkerLoad );

	// Initialize hash on first iteration.
	if( ExportHashIndex == 0 )
	{
		ExportHash.Reset(new int32[ExportHashCount]);
		for( int32 i=0; i<ExportHashCount; i++ )
		{
			ExportHash[i] = INDEX_NONE;
		}
	}

	// Set up export hash, potentially spread across several frames.
	while( ExportHashIndex < ExportMap.Num() && !IsTimeLimitExceeded(TEXT("creating export hash"),100) )
	{
		FObjectExport& Export = ExportMap[ExportHashIndex];

		const int32 iHash = GetHashBucket( Export.ObjectName );
		Export.HashNext = ExportHash[iHash];
		ExportHash[iHash] = ExportHashIndex;

		ExportHashIndex++;
	}

	// Return whether we finished this step and it's safe to start with the next.
	return ((ExportHashIndex == ExportMap.Num()) && !IsTimeLimitExceeded( TEXT("creating export hash") )) ? LINKER_Loaded : LINKER_TimedOut;
}

/**
 * Finds existing exports in memory and matches them up with this linker. This is required for PIE to work correctly
 * and also for script compilation as saving a package will reset its linker and loading will reload/ replace existing
 * objects without a linker.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::FindExistingExports()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::FindExistingExports" ), STAT_LinkerLoad_FindExistingExports, STATGROUP_LinkerLoad );

	if (!bHasFoundExistingExports.load(std::memory_order_acquire))
	{
		// only look for existing exports in the editor after it has started up
#if WITH_EDITOR
		if (LoadProgressScope)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			LoadProgressScope->EnterProgressFrame(1);
		}
		if (GIsEditor && GIsRunning)
		{
			// Hunt down any existing objects and hook them up to this linker unless the user is either currently opening this
			// package manually via the generic browser or the package is a map package. We want to overwrite (aka load on top)
			// the objects in those cases, so don't try to find existing exports.
			//
			bool bContainsMap			= LinkerRoot ? LinkerRoot->ContainsMap() : false;
			bool bRequestFindExisting = FCoreUObjectDelegates::ShouldLoadOnTop.IsBound() ? !FCoreUObjectDelegates::ShouldLoadOnTop.Execute(GetPackagePath().GetLocalFullPath()) : true;
			if( (!IsRunningCommandlet() && bRequestFindExisting && !bContainsMap) )
			{
				for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ExportIndex++)
				{
					FindExistingExport(ExportIndex);
				}
			}
		}
#endif // WITH_EDITOR

		// Avoid duplicate work in the case of async linker creation.
		bHasFoundExistingExports.store(true, std::memory_order_release);
	}
	return IsTimeLimitExceeded( TEXT("finding existing exports") ) ? LINKER_TimedOut : LINKER_Loaded;
}

/**
 * Finalizes linker creation, adding linker to loaders array and potentially verifying imports.
 */
FLinkerLoad::ELinkerStatus FLinkerLoad::FinalizeCreation(TMap<TPair<FName, FPackageIndex>, FPackageIndex>* ObjectNameWithOuterToExportMap)
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::FinalizeCreation" ), STAT_LinkerLoad_FinalizeCreation, STATGROUP_LinkerLoad );

	if (!bHasFinishedInitialization.load(std::memory_order_acquire))
	{
#if WITH_EDITOR
		if (LoadProgressScope)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			LoadProgressScope->EnterProgressFrame(1);
		}
#endif

		// Add this linker to the object manager's linker array.
		FLinkerManager::Get().AddLoader(this);

		if (bIsAsyncLoader)
		{
			GetAsyncLoader()->EndReadingHeader();
		}

		if ( !(LoadFlags & LOAD_NoVerify) )
		{
			Verify();
		}


		if (LinkerRoot)
		{
			TRACE_LOADTIME_PACKAGE_SUMMARY(this, LinkerRoot->GetFName(), Summary.TotalHeaderSize, Summary.ImportCount, Summary.ExportCount, 0);
		}

		// Avoid duplicate work in the case of async linker creation.
		bHasFinishedInitialization.store(true, std::memory_order_release);

#if WITH_EDITOR
		if (LoadProgressScope)
		{
		delete LoadProgressScope;
		LoadProgressScope = nullptr;
		}
#endif
	}

	return IsTimeLimitExceeded( TEXT("finalizing creation") ) ? LINKER_TimedOut : LINKER_Loaded;
}

/**
 * Before loading anything objects off disk, this function can be used to discover
 * the object in memory. This could happen in the editor when you save a package (which
 * destroys the linker) and then play PIE, which would cause the Linker to be
 * recreated. However, the objects are still in memory, so there is no need to reload
 * them.
 *
 * @param ExportIndex	The index of the export to hunt down
 * @return The object that was found, or NULL if it wasn't found
 */
UObject* FLinkerLoad::FindExistingExport(int32 ExportIndex)
{
	check(ExportMap.IsValidIndex(ExportIndex));
	FObjectExport& Export = ExportMap[ExportIndex];

	// if we were already found, leave early
	if (Export.Object)
	{
		return Export.Object;
	}

	// find the outer package for this object, if it's already loaded
	UObject* OuterObject = NULL;
	if (Export.OuterIndex.IsNull())
	{
		// this export's outer is the UPackage root of this loader
		OuterObject = LinkerRoot;
	}
	else if (Export.OuterIndex.IsExport())
	{
		// if we have a PackageIndex, then we are in a group or other object, and we should look for it
		OuterObject = FindExistingExport(Export.OuterIndex.ToExport());
	}
	else
	{
		// Our outer is actually an import
		OuterObject = FindExistingImport(Export.OuterIndex.ToImport());
	}

	// if we found one, keep going. if we didn't find one, then this package has never been loaded before
	if (OuterObject)
	{
		// find the class of this object
		UClass* TheClass = nullptr;
		if (Export.ClassIndex.IsNull())
		{
			TheClass = UClass::StaticClass();
		}
		else
		{
			// Check if this object export is a non-native class, non-native classes are always exports.
			// If so, then use the outer object as a package.
			UObject* ClassPackage = nullptr;
			if (Export.ClassIndex.IsExport())
			{
				ClassPackage = LinkerRoot;
			}
			else
			{
				FObjectImport& ClassImport = Imp(Export.ClassIndex);
				ClassPackage = StaticFindObjectFast(UPackage::StaticClass(), nullptr, ClassImport.ClassPackage, EFindObjectFlags::ExactClass);
			}
			if (ClassPackage)
			{
				TheClass = (UClass*)StaticFindObjectFast(UClass::StaticClass(), ClassPackage, ImpExp(Export.ClassIndex).ObjectName, EFindObjectFlags::None);
			}
			else
			{
				// RobM: No class package so try and find any class matching the name. Sounds sketchy and we should remove it
				TheClass = FindFirstObject<UClass>(*ImpExp(Export.ClassIndex).ObjectName.ToString(), EFindFirstObjectOptions::None, ELogVerbosity::Fatal, TEXT("finding existing export"));
			}
		}

		// if the class exists, try to find the object
		if (TheClass)
		{
			TheClass->GetDefaultObject(); // build the CDO if it isn't already built
			Export.Object = StaticFindObjectFast(TheClass, OuterObject, Export.ObjectName, EFindObjectFlags::ExactClass);

			// if we found an object, set it's linker to us
			if (Export.Object)
			{
				Export.Object->SetLinker(this, ExportIndex);
			}
		}
	}

	return Export.Object;
}

UObject* FLinkerLoad::FindExistingImport(int32 ImportIndex)
{
	check(ImportMap.IsValidIndex(ImportIndex));
	FObjectImport& Import = ImportMap[ImportIndex];

	// if the import object is already resolved just return it
	if (Import.XObject)
	{
		return Import.XObject;
	}

	// find the outer package for this object, if it's already loaded
	UObject* OuterObject = nullptr;

	if (Import.OuterIndex.IsNull())
	{
		// if the import outer is null then we have a package, resolve it, potentially remapping it
		FName ObjectName = InstancingContext.RemapPackage(Import.ObjectName);
		UPackage* Package = static_cast<UPackage*>(StaticFindObjectFast(UPackage::StaticClass(), nullptr, ObjectName, EFindObjectFlags::ExactClass));
		if (!IsPackageReferenceAllowed(Package))
		{
			return nullptr;
		}
		return Package;
	}
	// if our outer is an import, recurse to find it
	else if (Import.OuterIndex.IsImport())
	{
		OuterObject = FindExistingImport(Import.OuterIndex.ToImport());
	}
	// Otherwise our outer is actually an export from this package
	else
	{
		OuterObject = FindExistingExport(Import.OuterIndex.ToExport());
	}

	if (OuterObject)
	{
		// find the class of this object
		UClass* TheClass = nullptr;
		if ((Import.ClassName == NAME_Class && (Import.ClassPackage == GLongCoreUObjectPackageName || Import.ClassPackage == NAME_CoreUObject)) || Import.ClassName.IsNone())
		{
			TheClass = UClass::StaticClass();
		}
		else
		{
			//@todo: Could we have an import that has its class as an export?
			UPackage* ClassPackage = FindObject<UPackage>(nullptr, *Import.ClassPackage.ToString()); // FindObject because *theoretically* this could be an old package where ClassPackage was a short package name and FindObject handles that
			if (ClassPackage)
			{
				TheClass = FindObjectFast<UClass>(ClassPackage, Import.ClassName, EFindObjectFlags::None);
			}
		}

		// if the class exists, try to find the object
		if (TheClass)
		{
			return StaticFindObjectFast(UClass::StaticClass(), OuterObject, Import.ObjectName, EFindObjectFlags::ExactClass);
		}
	}
	return nullptr;
}

void FLinkerLoad::Verify()
{
	if (!bHaveImportsBeenVerified)
	{
		bool bShouldVerifyAllImports = IsRunningCommandlet();

#if WITH_EDITOR
		// In editor builds using OFPA, we need to resolve imports for BP classes referenced by the level script in order to be able
		// to properly reinstanced them. We could filter out imports to resolve here, but we resolve all of them instead.
		bShouldVerifyAllImports = true;
#endif

		if (!IsImportLazyLoadEnabled() && bShouldVerifyAllImports)
		{
#if WITH_EDITOR
			TOptional<FScopedSlowTask> SlowTask;
			if (ShouldCreateThrottledSlowTask())
			{
				static const FText LoadingImportsText = NSLOCTEXT("Core", "LinkerLoad_Imports", "Loading Imports");
				SlowTask.Emplace(static_cast<float>(Summary.ImportCount), LoadingImportsText);
			}
#endif
			UE_TRACK_REFERENCING_PACKAGE_SCOPED(LinkerRoot->GetFName(), PackageAccessTrackingOps::NAME_Load);

			// Validate all imports and map them to their remote linkers.
			for (int32 ImportIndex = 0; ImportIndex < Summary.ImportCount; ImportIndex++)
			{
				FObjectImport& Import = ImportMap[ImportIndex];

#if WITH_EDITOR
				if (SlowTask)
				{
					UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
					static const FText LoadingImportText = NSLOCTEXT("Core", "LinkerLoad_LoadingImportName", "Loading Import '{0}'");
					SlowTask->EnterProgressFrame(1, FText::Format(LoadingImportText, FText::FromString(Import.ObjectName.ToString())));
				}
#endif
				VerifyImport( ImportIndex );
			}
		}

		bHaveImportsBeenVerified = true;
	}
}

FName FLinkerLoad::GetExportClassPackage( int32 i )
{
	FObjectExport& Export = ExportMap[ i ];
	if( Export.ClassIndex.IsImport() )
	{
		FObjectImport& Import = Imp(Export.ClassIndex);
		if (!Import.OuterIndex.IsNull())
		{
			return ImpExp(Import.OuterIndex).ObjectName;
		}
		// Report an error if the report class index doesn't have an outer
		UE_LOGF(LogLinker, Error, "Export '%s' Class reported as a package '%s' which might indicate corruption", *Export.ObjectName.ToUtf8String(), *Import.ObjectName.ToUtf8String());
		return Import.ObjectName;
	}
	else if ( !Export.ClassIndex.IsNull() )
	{
		// the export's class is contained within the same package
		return LinkerRoot->GetFName();
	}
	else
	{
		return GLongCoreUObjectPackageName;
	}
}

FString FLinkerLoad::GetArchiveName() const
{
	return GetPackagePath().GetDebugName();
}


/**
 * Recursively gathers the dependencies of a given export (the recursive chain of imports
 * and their imports, and so on)

 * @param ExportIndex Index into the linker's ExportMap that we are checking dependencies
 * @param Dependencies Array of all dependencies needed
 * @param bSkipLoadedObjects Whether to skip already loaded objects when gathering dependencies
 */
#if WITH_EDITORONLY_DATA
void FLinkerLoad::GatherExportDependencies(int32 ExportIndex, TSet<FDependencyRef>& Dependencies, bool bSkipLoadedObjects)
{
	// make sure we have dependencies
	// @todo: remove this check after all packages have been saved up to VER_ADDED_LINKER_DEPENDENCIES
	if (DependsMap.Num() == 0)
	{
		return;
	}

	// validate data
	check(DependsMap.Num() == ExportMap.Num());

	// get the list of imports the export needs
	TArray<FPackageIndex>& ExportDependencies = DependsMap[ExportIndex];

//UE_LOGF(LogLinker, Warning, "Gathering dependencies for %ls", *GetExportFullName(ExportIndex));

	for (int32 DependIndex = 0; DependIndex < ExportDependencies.Num(); DependIndex++)
	{
		FPackageIndex ObjectIndex = ExportDependencies[DependIndex];

		// if it's an import, use the import version to recurse (which will add the export the import points to to the array)
		if (ObjectIndex.IsImport())
		{
			GatherImportDependencies(ObjectIndex.ToImport(), Dependencies, bSkipLoadedObjects);
		}
		else
		{
			int32 RefExportIndex = ObjectIndex.ToExport();
			FObjectExport& Export = ExportMap[RefExportIndex];

			if( (Export.Object) && ( bSkipLoadedObjects == true ) )
			{
				continue;
			}

			// fill out the ref
			FDependencyRef NewRef;
			NewRef.Linker = this;
			NewRef.ExportIndex = RefExportIndex;

			// Add to set and recurse if not already present.
			bool bIsAlreadyInSet = false;
			Dependencies.Add( NewRef, &bIsAlreadyInSet );
			if (!bIsAlreadyInSet && NewRef.Linker)
			{
				NewRef.Linker->GatherExportDependencies(RefExportIndex, Dependencies, bSkipLoadedObjects);
			}
		}
	}
}

/**
 * Recursively gathers the dependencies of a given import (the recursive chain of imports
 * and their imports, and so on). Will add itself to the list of dependencies

 * @param ImportIndex Index into the linker's ImportMap that we are checking dependencies
 * @param Dependencies Set of all dependencies needed
 * @param bSkipLoadedObjects Whether to skip already loaded objects when gathering dependencies
 */
void FLinkerLoad::GatherImportDependencies(int32 ImportIndex, TSet<FDependencyRef>& Dependencies, bool bSkipLoadedObjects)
{
	// get the import
	FObjectImport& Import = ImportMap[ImportIndex];

	// we don't need the top level package imports to be checked, since there is no real object associated with them
	if (Import.OuterIndex.IsNull())
	{
		return;
	}
	//	UE_LOGF(LogLinker, Warning, "  Dependency import %ls [%x, %d]", *GetImportFullName(ImportIndex), Import.SourceLinker, Import.SourceIndex);

	// if the object already exists, we don't need this import
	if (Import.XObject)
	{
		return;
	}

	FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();

	BeginLoad(LoadContext, TEXT("GatherImportDependencies"));

	// load the linker and find export in sourcelinker
	if (Import.SourceLinker == NULL || Import.SourceIndex == INDEX_NONE)
	{
#if DO_CHECK
		int32 NumObjectsBefore = GUObjectArray.GetObjectArrayNum();
#endif

		// temp storage we can ignore
		FString Unused;

		// remember that we are gathering imports so that VerifyImportInner will no verify all imports
		bIsGatheringDependencies = true;

		// if we failed to find the object, ignore this import
		// @todo: Tag the import to not be searched again
		VerifyImportInner(ImportIndex, Unused);

		// turn off the flag
		bIsGatheringDependencies = false;

		bool bIsValidImport = Import.SourceLinker != nullptr && Import.SourceIndex != INDEX_NONE;
		if (!bIsValidImport &&
			Import.XObject && // Found the XObject, so potentially report it as an import anyway
			!Import.XObject->IsNative() // Imports of native classes are not reported as dependencies
			)
		{
			// Imports that found their XObject are reported as dependencies, unless they are suppressed
			// XObject-found Imports are suppressed if they are native classes, native class CDOs, or subobjects of native class CDOs
			// XObject-found Imports are suppressed if they are transient non-native CDOs (or subobjects thereof)
			UObject* RootObject = Import.XObject;
			while (RootObject && RootObject->HasAllFlags(RF_DefaultSubObject))
			{
				RootObject = RootObject->GetOuter();
			}
			if (RootObject)
			{
				if (!RootObject->HasAnyFlags(RF_ClassDefaultObject))
				{
					// Not a CDO, so a valid import dependency
					bIsValidImport = true;
				}
				else if (!RootObject->GetClass()->IsNative() && !RootObject->HasAllFlags(EObjectFlags(RF_Transient)))
				{
					// A non-native, non-transient CDO is a valid import dependency
					bIsValidImport = true;
				}
			}
		}

		// make sure it succeeded
		if (!bIsValidImport)
		{
			// don't warn about the suppressed Import.XObject dependencies
			if (!Import.XObject)
			{
				UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("VerifyImportInner failed [(0x%" UPTRINT_X_FMT ", %d), (0x%" UPTRINT_X_FMT ", %d)] for %s"),
					(UPTRINT)Import.XObject,
					Import.XObject ? (Import.XObject->IsNative() ? 1 : 0) : 0,
					(UPTRINT)Import.SourceLinker,
					Import.SourceIndex,
					*GetImportFullName(ImportIndex));
			}
			EndLoad(LoadContext);
			return;
		}

#if DO_CHECK && !NO_LOGGING
		// only object we should create are one FLinkerLoad for source linker
		if (GUObjectArray.GetObjectArrayNum() - NumObjectsBefore > 2)
		{
			UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Created %d objects checking %s"), GUObjectArray.GetObjectArrayNum() - NumObjectsBefore, *GetImportFullName(ImportIndex));
		}
#endif
	}

	// save off information BEFORE calling EndLoad so that the Linkers are still associated
	FDependencyRef NewRef;
	if (Import.XObject)
	{
		UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Using non-native XObject %s!!!"), *Import.XObject->GetFullName());
		NewRef.Linker = Import.XObject->GetLinker();
		NewRef.ExportIndex = Import.XObject->GetLinkerIndex();
	}
	else
	{
		NewRef.Linker = Import.SourceLinker;
		NewRef.ExportIndex = Import.SourceIndex;
	}

	EndLoad(LoadContext);

	// Add to set and recurse if not already present.
	bool bIsAlreadyInSet = false;
	Dependencies.Add( NewRef, &bIsAlreadyInSet );
	if (!bIsAlreadyInSet && NewRef.Linker)
	{
		NewRef.Linker->GatherExportDependencies(NewRef.ExportIndex, Dependencies, bSkipLoadedObjects);
	}
}
#endif

FLinkerLoad::EVerifyResult FLinkerLoad::VerifyImport(int32 ImportIndex)
{
	FObjectImport& Import = ImportMap[ImportIndex];

	// keep a string of modifiers to add to the Editor Warning dialog
	FString WarningAppend;

	// try to load the object, but don't print any warnings on error (so we can try the redirector first)
	// note that a true return value here does not mean it failed or succeeded, just tells it how to respond to a further failure
	bool bCrashOnFail = VerifyImportInner(ImportIndex, WarningAppend);
	if (FPlatformProperties::HasEditorOnlyData() == false)
	{
		bCrashOnFail = false;
	}

	// by default, we haven't failed yet
	EVerifyResult Result = VERIFY_Success;

	// these checks find out if the VerifyImportInner was successful or not
	if (Import.SourceLinker && Import.SourceIndex == INDEX_NONE && Import.XObject == NULL && !Import.OuterIndex.IsNull() && Import.ObjectName != NAME_ObjectRedirector)
	{
		FUObjectSerializeContext* CurrentLoadContext = FUObjectThreadContext::Get().GetSerializeContext();

		// if we found the package, but not the object, look for a redirector
		FObjectImport OriginalImport = Import;
		Import.ClassName = NAME_ObjectRedirector;
		Import.ClassPackage = GLongCoreUObjectPackageName;

		// try again for the redirector
		VerifyImportInner(ImportIndex, WarningAppend);

		// if the redirector wasn't found, then it truly doesn't exist
		if (Import.SourceIndex == INDEX_NONE) //-V547
		{
			Result = VERIFY_Failed;
		}
		// otherwise, we found that the redirector exists
		else
		{
			// this notes that for any load errors we get that a ObjectRedirector was involved (which may help alleviate confusion
			// when people don't understand why it was trying to load an object that was redirected from or to)
			WarningAppend += LOCTEXT("LoadWarningSuffix_redirection", " [redirection]").ToString();

			// Create the redirector (no serialization yet)
			UObjectRedirector* Redir = dynamic_cast<UObjectRedirector*>(Import.SourceLinker->CreateExport(Import.SourceIndex));
			// this should probably never fail, but just in case
			if (!Redir)
			{
				Result = VERIFY_Failed;
			}
			else
			{
				// serialize in the properties of the redirector (to get the object the redirector point to)
				// Always load redirectors in case there was a circular dependency. This will allow inner redirector
				// references to always serialize fully here before accessing the DestinationObject
				// If the redirector is now in phase2, it already has been through serialization
				// and we don't want to risk racing with its PostLoad by forcing its reload needlessly.
				if (!Redir->HasAnyFlags(RF_LoadCompleted) && !Redir->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase2))
				{
					Redir->SetFlags(RF_NeedLoad);
					Preload(Redir);
				}

				UObject* DestObject = Redir->DestinationObject;

				// check to make sure the destination obj was loaded,
				if (DestObject == nullptr)
				{
					Result = VERIFY_Failed;
				}
				else
				{
					// Blueprint CDOs are always allowed to change class, otherwise we need to do a name check for all parent classes
					bool bIsValidClass = DestObject->HasAnyFlags(RF_ClassDefaultObject);
					UClass* CheckClass = DestObject->GetClass();

					while (!bIsValidClass && CheckClass)
					{
						if (CheckClass->GetFName() == OriginalImport.ClassName)
						{
							bIsValidClass = true;
							break;
						}
						CheckClass = CheckClass->GetSuperClass();
					}

					if (!bIsValidClass)
					{
						Result = VERIFY_Failed;
						// if the destination is a ObjectRedirector you've most likely made a nasty circular loop
						if (Redir->DestinationObject->GetClass() == UObjectRedirector::StaticClass())
						{
							WarningAppend += LOCTEXT("LoadWarningSuffix_circularredirection", " [circular redirection]").ToString();
						}
					}
					else
					{
						Result = VERIFY_Redirected;

						// now, fake our Import to be what the redirector pointed to
						Import.XObject = Redir->DestinationObject;
						CurrentLoadContext->IncrementImportCount();
						FLinkerManager::Get().AddLoaderWithNewImports(this);
					}
				}
			}
		}

		// fix up the import. We put the original data back for the ClassName and ClassPackage (which are read off disk, and
		// are expected not to change)
		Import.ClassName = OriginalImport.ClassName;
		Import.ClassPackage = OriginalImport.ClassPackage;

		// if nothing above failed, then we are good to go
		if (Result != VERIFY_Failed) //-V547
		{
			// we update the runtime information (SourceIndex, SourceLinker) to point to the object the redirector pointed to
			Import.SourceIndex = Import.XObject->GetLinkerIndex();
			Import.SourceLinker = Import.XObject->GetLinker();
		}
		else
		{
			// put us back the way we were and peace out
			Import = OriginalImport;

			// if the original VerifyImportInner told us that we need to throw an exception if we weren't redirected,
			// then do the throw here
			if (bCrashOnFail)
			{
				UE_ASSET_LOG(LogLinker, Fatal, PackagePath, TEXT("Failed import: %s %s (file %s)"), *Import.ClassName.ToString(), *GetImportFullName(ImportIndex), *Import.SourceLinker->GetDebugName());
				return Result;
			}
			// otherwise just printout warnings, and if in the editor, popup the EdLoadWarnings box
			else
			{
#if WITH_EDITOR
				// print warnings in editor, standalone game, or commandlet
				bool bSuppressLinkerError = IsSuppressableBlueprintImportError(ImportIndex);
				if (!bSuppressLinkerError)
				{
					FDeferredMessageLog LoadErrors(NAME_LoadErrors);
					// put something into the load warnings dialog, with any extra information from above (in WarningAppend)
					TSharedRef<FTokenizedMessage> TokenizedMessage = GTreatVerifyImportErrorsAsWarnings ? LoadErrors.Warning(FText()) : LoadErrors.Error(FText());
					TokenizedMessage->AddToken(FAssetNameToken::Create(LinkerRoot->GetName()));
					TokenizedMessage->AddToken(FTextToken::Create(FText::Format(LOCTEXT("ImportFailure", " : Failed import for {0}"), FText::FromName(GetImportClassName(ImportIndex)))));
					TokenizedMessage->AddToken(FAssetNameToken::Create(GetImportPathName(ImportIndex)));

					if (!WarningAppend.IsEmpty())
					{
						TokenizedMessage->AddToken(FTextToken::Create(FText::Format(LOCTEXT("ImportFailure_WarningIn", "{0} in {1}"),
							FText::FromString(WarningAppend),
							FText::FromString(LinkerRoot->GetName())))
							);
					}

					// Go through the depend map of the linker to find out what exports are referencing this import
					const FPackageIndex ImportPackageIndex = FPackageIndex::FromImport(ImportIndex);
					for (int32 CurrentExportIndex = 0; CurrentExportIndex < DependsMap.Num(); ++CurrentExportIndex)
					{
						const TArray<FPackageIndex>& DependsList = DependsMap[CurrentExportIndex];
						if (DependsList.Contains(ImportPackageIndex))
						{
							TokenizedMessage->AddToken(FTextToken::Create(
								FText::Format(LOCTEXT("ImportFailureExportReference", "Referenced by export {0}"),
									FText::FromName(GetExportClassName(CurrentExportIndex)))));
							TokenizedMessage->AddToken(FAssetNameToken::Create(GetExportPathName(CurrentExportIndex)));
						}
					}

					// try to get a pointer to the class of the original object so that we can display the class name of the missing resource
					UObject* ClassPackage = FindObject<UPackage>(nullptr, *Import.ClassPackage.ToString());
					UClass* FindClass = ClassPackage ? FindObject<UClass>(ClassPackage, *OriginalImport.ClassName.ToString()) : nullptr;

					// print warning about missing class
					if (!FindClass)
					{
						UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Missing Class %s for '%s'. Classes should not be removed if referenced by content; mark the class 'deprecated' instead."),
							*OriginalImport.ClassName.ToString(),
							*GetImportFullName(ImportIndex));
					}
				}
#endif // WITH_EDITOR
			}
		}
	}

	return Result;
}

// Internal Load package call so that we can pass the linker that requested this package as an import dependency
UPackage* LoadPackageInternal(UPackage* InOuter, const FPackagePath& PackagePath, uint32 LoadFlags, FLinkerLoad* ImportLinker, FArchive* InReaderOverride, const FLinkerInstancingContext* InstancingContext, const FPackagePath* DiffPackagePath);

/**
 * Finds and populates the import table for the specified package import.
 *
 * @param InOutImportMap	The import table
 * @param InPackageImport	The package import index
 */
void StaticFindAllImportObjects(TArray<FObjectImport>& InOutImportMap, FPackageIndex InPackageImport, const FPackagePath& PathOfPackageBeingLoaded)
{
	using FPackageIndexArray = TArray<FPackageIndex, TInlineAllocator<64>>;

	UPackage* Package = Cast<UPackage>(InOutImportMap[InPackageImport.ToImport()].XObject);
	check(Package && Package->HasAnyPackageFlags(PKG_Cooked));
	UE_LOGF(LogLinker, Verbose, "Finding all imports for cooked package import '%ls' ('%d')", *Package->GetFullName(), InPackageImport.ToImport());

	auto FindInners = [&InOutImportMap](FPackageIndex InOuter, FPackageIndexArray& OutInners) -> void
	{
		for (int32 ImportIndex = 0; ImportIndex < InOutImportMap.Num(); ++ImportIndex)
		{
			if (InOutImportMap[ImportIndex].OuterIndex == InOuter)
			{
				OutInners.Push(FPackageIndex::FromImport(ImportIndex));
			}
		}
	};

	TStringBuilder<256> ImportClassTemp;
	auto FindClass = [&ImportClassTemp](const FObjectImport& ObjectImport) -> UClass*
	{
		ObjectImport.ClassPackage.ToString(ImportClassTemp);
		if (UObject* ClassPackage = FindObject<UPackage>(nullptr, *ImportClassTemp))
		{
			ObjectImport.ClassName.ToString(ImportClassTemp);
			return FindObject<UClass>(ClassPackage, *ImportClassTemp);
		}
		return nullptr;
	};

	FPackageIndexArray Outers, Inners;
	Outers.Add(InPackageImport);

	while (Outers.Num() > 0)
	{
		FPackageIndex Outer = Outers.Pop();
		if (UObject* OuterObject = InOutImportMap[Outer.ToImport()].XObject)
		{
			Inners.Reset();
			FindInners(Outer, Inners);

			for (FPackageIndex Inner : Inners)
			{
				FObjectImport& ObjectImport = InOutImportMap[Inner.ToImport()];
				if (!ObjectImport.XObject)
				{
					UClass* Class = FindClass(ObjectImport);
					// Don't pass bExactClass=true when looking up redirected classes for cooked packages, as we don't know the exact class of what
					// we are looking for (could be UClass or UBlueprintGeneratedClass, see CreateImportClassAndPackage).
					if (UObject* InnerObject = StaticFindObjectFastInternal(Class, OuterObject, ObjectImport.ObjectName, EFindObjectFlags::None))
					{
						ObjectImport.XObject = InnerObject;
						Outers.Add(Inner);
					}
					else
					{
						// Suppress warning when the missing import's declared class, or the import path itself (if a class), is registered as known-missing via CoreRedirects.
						const FCoreRedirectObjectName ImportClassName(ObjectImport.ClassName, NAME_None, ObjectImport.ClassPackage);

						// Check if the object's class is known-missing
						bool bClassInstanceDeleted = FCoreRedirects::IsKnownMissing(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, ImportClassName);
						if (!bClassInstanceDeleted)
						{
							// Check if the object itself is a class that is known-missing
							ImportClassTemp.Reset();
							OuterObject->GetPathName(nullptr, ImportClassTemp);
							ImportClassTemp << TEXT(".");
							ImportClassTemp << ObjectImport.ObjectName.ToString();

							bClassInstanceDeleted = FCoreRedirects::IsKnownMissing(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, FCoreRedirectObjectName(ImportClassTemp.ToString()));
						}
															
						if (!bClassInstanceDeleted)
						{
							UE_ASSET_LOG(LogLinker, Warning, PathOfPackageBeingLoaded, TEXT("Failed to resolve import '%s' ('%d') in outer '%s' ('%d') within cooked package '%s'"),
								*ObjectImport.ObjectName.ToString(),
								Inner.ToImport(),
								*OuterObject->GetName(),
								Outer.ToImport(),
								*FAssetMsg::FormatPathForAssetLog(Package)
								);
						}
					}
				}
			}
		}
	}
}

/**
 * Safely verify that an import in the ImportMap points to a good object. This decides whether or not
 * a failure to load the object redirector in the wrapper is a fatal error or not (return value)
 *
 * @param	i	The index into this packages ImportMap to verify
 *
 * @return true if the wrapper should crash if it can't find a good object redirector to load
 */
bool FLinkerLoad::VerifyImportInner(const int32 ImportIndex, FString& WarningSuffix)
{
	SCOPED_LOADTIMER(LinkerLoad_VerifyImportInner);
	// Lambda used to load an import package
	auto LoadImportPackage = [this, ImportIndex](FObjectImport& Import, TOptional<FScopedSlowTask>& SlowTask) -> UPackage*
	{
		LLM_SCOPE(ELLMTag::UObject);
		LLM_SCOPE_BYTAG(UObject_FLinkerLoad);

		// Either this import is a UPackage or it has PackageName set.
		check(Import.ClassName == NAME_Package || Import.HasPackageName());

		UPackage* Package = nullptr;
		uint32 InternalLoadFlags = LoadFlags & (LOAD_NoVerify | LOAD_NoWarn | LOAD_Quiet);
		FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();

		// Resolve the package name for the import, potentially remapping it, if instancing
		FName PackageToLoad = !Import.HasPackageName() ? Import.ObjectName : Import.GetPackageName();
		FName PackageToLoadInto = InstancingContext.RemapPackage(PackageToLoad);
#if WITH_EDITOR
		if (SlowTask)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			SlowTask->EnterProgressFrame(30);
		}
#endif

		// helper to report missing native packages when an import is requested:
		const auto ReportMissingPackage = [&PackageToLoad, ImportIndex, this]() -> bool
			{
				TCHAR PackageToLoadBuffer[FName::StringBufferSize];
				PackageToLoad.ToString(PackageToLoadBuffer);
				if (FPackageName::IsScriptPackage(PackageToLoadBuffer))
				{
					if (!FLinkerLoad::IsKnownMissingPackage(PackageToLoad))
					{
						FLinkerLoad::AddKnownMissingPackage(PackageToLoad);
						UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("VerifyImport: Failed to find script package for import object '%s'"), *GetImportFullName(ImportIndex));
					}
					return true;
				}
				return false;
			};

		// Check if the package exist first, if it already exists, it is either already loaded or being loaded
		// In the fully loaded case we can entirely skip the loading
		// In the other case we do not want to trigger another load of the objects in that import, in case they contain dependencies to the package we are currently loading
		// and the current loader doesn't have the LOAD_DeferDependencyLoads flag
		Package = FindObjectFast<UPackage>(nullptr, PackageToLoadInto);
		if (LoadFlags & LOAD_SkipLoadImportedPackages)
		{
			if (!Package)
			{
				ReportMissingPackage();
				return nullptr;
			}
			Import.SourceLinker = FindExistingLinkerForPackage(Package);
			if (!Import.SourceLinker && Package->HasAnyPackageFlags(PKG_Cooked))
			{
				// Special case where we're verifying an import from a cooked package before we've performed the global import store lookup for this package in AsyncLoading2.cpp
				// Find the imports by name instead
				// Note: The cooked package might not be marked as fully loaded at this stage, but we will have created and serialized all its exports
				Import.XObject = Package;
				StaticFindAllImportObjects(ImportMap, FPackageIndex::FromImport(ImportIndex), PackagePath);
			}
			return Package;
		}
		if (Package == nullptr || !Package->IsFullyLoaded())
		{
			if (ReportMissingPackage())
			{
				return nullptr;
			}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			// when LOAD_DeferDependencyLoads is in play, we usually head off
			// dependency loads before we get to this point, but there are two
			// cases where we can reach here intentionally:
			//
			//   1) the package we're attempting to load is natiVe (and thusly,
			//      LoadPackageInternal() should fail, and retrun null)
			//
			//   2) the package we're attempting to load is a user defined
			//      struct asset, which we need to load because the blueprint
			//      class's layout depends on the struct's size... in this case,
			//      we choke off circular loads by propagating this flag along
			//      to the struct linker (so it doesn't load any blueprints)
			InternalLoadFlags |= (LoadFlags & LOAD_DeferDependencyLoads);
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

			// If the package name we need to load is different than the package we need to load into then we
			// are doing an instanced load (loading the data of package A on disk to package B in memory)
			// hence we create a package with a unique instance name provided by the instancing context
			// In the case of a non instanced load `PackageToLoad` and `PackageToLoadInto` will be the same and we won't be providing a package to load into since `Package` will be null.
			// If we are going through and instanced load we are also propagating the instancing context
			const FLinkerInstancingContext* LocalInstancingContext = nullptr;
			if (PackageToLoad != PackageToLoadInto)
			{
				Package = CreatePackage(*PackageToLoadInto.ToString());
				LocalInstancingContext = &GetInstancingContext();
			}
			FPackagePath PackagePathToLoad = FPackagePath::FromPackageNameChecked(PackageToLoad);
#if WITH_EDITOR
			FCookLoadScope ResetCookLoadScopeToUnspecified(ECookLoadType::Unspecified);
#endif

			Package = LoadPackageInternal(Package, PackagePathToLoad, InternalLoadFlags | LOAD_IsVerifying, this, nullptr /* InReaderOverride */,
				LocalInstancingContext, nullptr /* DiffPackagePath */);
		}
#if WITH_IOSTORE_IN_EDITOR
		if (Package && Package->HasAnyPackageFlags(PKG_Cooked) && Package->GetPackageId().IsValid())
		{
			// Cooked packages loaded from iostore are always fully loaded and has no attached
			// linkers. Static find all imported objects from this package.
			check(Package->IsFullyLoaded());
			Import.XObject = Package;
			StaticFindAllImportObjects(ImportMap, FPackageIndex::FromImport(ImportIndex), PackagePath);
		}
#endif

#if WITH_EDITOR
		if (SlowTask)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			SlowTask->EnterProgressFrame(30);
		}
#endif

		// if we couldn't create the package or it is
		// to be linked to any other package's ImportMaps
		if (!Package || Package->HasAnyPackageFlags(PKG_Compiling))
		{
			if (!FLinkerLoad::IsKnownMissingPackage(PackageToLoad))
			{
				FLinkerLoad::AddKnownMissingPackage(PackageToLoad);
				UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("VerifyImport: Failed to load package for import object '%s'"), *GetImportFullName(ImportIndex));
			}
			return nullptr;
		}

		if (!IsPackageReferenceAllowed(Package))
		{
			UE_LOGF(LogLinker, Warning, "VerifyImport: illegal reference to private package for import object '%ls'", *GetImportFullName(ImportIndex));
			return nullptr;
		}

		// while gathering dependencies, there is no need to verify all of the imports for the entire package
		if (bIsGatheringDependencies)
		{
			InternalLoadFlags |= LOAD_NoVerify;
		}

#if WITH_EDITOR
		if (SlowTask)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			SlowTask->EnterProgressFrame(40);
		}
#endif

		// Get the linker if the package hasn't been fully loaded already, this can happen in the case of LOAD_DeferDependencyLoads
		// or when circular dependency happen, get the linker so we are able to create the import properly at a later time.
		// When loading editor data never consider the package fully loaded and resolve the linker anyway, for cooked data, assign the linker if one is associated witht the package
#if WITH_IOSTORE_IN_EDITOR
		if (Package && !(Package->HasAnyPackageFlags(PKG_Cooked) && Package->GetPackageId().IsValid()))
#endif
		{
			const bool bWasFullyLoaded = Package && Package->IsFullyLoaded() && FPlatformProperties::RequiresCookedData();
			if (!bWasFullyLoaded)
			{
				Import.SourceLinker = GetPackageLinker(Package, FPackagePath::FromPackageNameChecked(PackageToLoad), InternalLoadFlags, nullptr, nullptr, &SerializeContext, nullptr, &InstancingContext);
			}
			else
			{
				Import.SourceLinker = FindExistingLinkerForPackage(Package);
			}
		}
#if WITH_METADATA
		if (Import.SourceLinker && !Package->HasAnyFlags(RF_LoadCompleted))
		{
			// If we didn't fully load, make sure our metadata is loaded before using this
			// We need this case for user defined structs due to the LOAD_DeferDependencyLoads code above
			Import.SourceLinker->LoadMetaDataFromExportMap(false);
		}
#endif // WITH_METADATA
		return Package;
	};

	check(IsLoading());
	FObjectImport& Import = ImportMap[ImportIndex];

	TOptional<FScopedSlowTask> SlowTask;
#if WITH_EDITOR
	if (ShouldCreateThrottledSlowTask())
	{
		static const FTextFormat VerifyingTextFormat = NSLOCTEXT("Core", "VerifyPackage_Scope", "Verifying '{0}'");
		SlowTask.Emplace(100.0f, FText::Format(VerifyingTextFormat, FText::FromName(Import.ObjectName)));
	}
#endif

	if
	(	(Import.SourceLinker && Import.SourceIndex != INDEX_NONE)
	||	Import.ClassPackage	== NAME_None
	||	Import.ClassName	== NAME_None
	||	Import.ObjectName	== NAME_None )
	{
		// Already verified, or not relevant in this context.
		return false;
	}

	if (Import.HasPackageName() || Import.OuterIndex.IsNull())
	{
		FName PackageToLoad = !Import.HasPackageName() ? Import.ObjectName : Import.GetPackageName();
		FName PackageToLoadInto = InstancingContext.RemapPackage(PackageToLoad);

		if (!PackageToLoad.IsNone() && PackageToLoadInto.IsNone())
		{
			// Import package was filtered out by instancing context
			return false;
		}
	}

	// Build the import object name on the stack and only once to avoid string temporaries
	TStringBuilder<256> ImportObjectName;
	Import.ObjectName.AppendString(ImportObjectName);

	bool SafeReplace = false;
	UObject* Pkg = nullptr;
	UPackage* TmpPkg = nullptr;

	// Find or load the linker load that contains the FObjectExport for this import
	if (Import.OuterIndex.IsNull() && Import.ClassName!=NAME_Package )
	{
		UE_ASSET_LOG(LogLinker, Error, PackagePath, TEXT("%s has an inappropriate outermost, it was probably saved with a deprecated outer"), *ImportObjectName);
		Import.SourceLinker = NULL;
		return false;
	}
	// This import is a UPackage, load it
	else if (Import.OuterIndex.IsNull())
	{
		TmpPkg = LoadImportPackage(Import, SlowTask);
	}
	else
	{
#if WITH_EDITOR
		if (SlowTask)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			SlowTask->EnterProgressFrame(50);
		}
#endif
		// if we have an assigned package, load it, this will also assign the import source linker (Import.SourceLinker)
		if (Import.HasPackageName())
		{
#if WITH_EDITOR
			if (SlowTask)
			{
				SlowTask->TotalAmountOfWork += 100;
			}
#endif
			Pkg = LoadImportPackage(Import, SlowTask);
		}

		// this import outer is also an import, so recurse verify into it.
		if (Import.OuterIndex.IsImport())
		{
			VerifyImport(Import.OuterIndex.ToImport());

			FObjectImport& OuterImport = Imp(Import.OuterIndex);
			if (!OuterImport.SourceLinker)
			{
				// if the import outer object has been resolved but no linker has been found, we import to a memory only package (i.e. compiled in)
				if (OuterImport.XObject)
				{
					FObjectImport* Top;
					for (Top = &OuterImport; Top->OuterIndex.IsImport(); Top = &Imp(Top->OuterIndex))
					{
						// for loop does what we need
					}

					UPackage* Package = Cast<UPackage>(Top->XObject);
					if (Package &&
						// Assign TmpPkg to resolve the object in memory when there is no source linker available only if the package is MemoryOnly
						// or we are loading an instanced package in which case the import package might be a duplicated pie package for example for which no linker exists
						(Package->HasAnyPackageFlags(PKG_InMemoryOnly) || IsContextInstanced()))
					{
						// This is an import to a memory-only package, just search for it in the package.
						TmpPkg = Package;
					}
				}
#if WITH_EDITOR
				else
				{
					// If we're serializing a redirector's destination object, validate/create the outer package object if it's a missing type. If this import
					// represents a non-native type object that's no longer valid, this will allow exports of that type to still be serialized to a property bag,
					// by creating a placeholder type object in its place. This way we won't lose any previously-serialized data for exports missing their type.
					const FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
					const UObjectRedirector* Redirector = Cast<UObjectRedirector>(SerializeContext->SerializedObject);
					if (Redirector && Redirector->IsSerializingDestinationObject() && TryCreatePlaceholderClassImport(ImportIndex))
					{
						// We don't need to do a package search for the import below since we've just created it. Return false to signal that there is no failure.
						check(Import.XObject);
						return false;
					}
				}
#endif
			}

#if WITH_EDITORONLY_DATA
			if (!Import.SourceLinker && OuterImport.SourceLinker && OuterImport.SourceLinker->DynamicImportsIndex != INDEX_NONE)
			{
				// We consider the exports of the dynamic imports of the outer as if they were exports of the outer's linker, so we
				// explore them to find a matching export before we default back to using the the outer's linker
				//
				// The reasoning behind that is that dynamic imports are used to move objects in external packages 
				// and we do not want all packages importing them to have to be resaved when this happens since we know 
				// how to resolve the situation efficiently
				const int32 ObjectHash = GetHashBucket( Import.ObjectName );

				// Prevents any other serialization function from modifying this linker while we're reading from it
				UE::TUniqueLock UniqueLock(OuterImport.SourceLinker->Mutex);

				for (int32 i = OuterImport.SourceLinker->DynamicImportsIndex; i < OuterImport.SourceLinker->ImportMap.Num(); i++)
				{
					FObjectImport& ObjImport = OuterImport.SourceLinker->Imp(FPackageIndex::FromImport(i));
					
					if (ObjImport.SourceLinker && ObjImport.SourceLinker->ExportHash)
					{
						FLinkerLoad* DynamicImportLinker = ObjImport.SourceLinker;
						
						// go through exports to find a matching object 
						for (int32 j= DynamicImportLinker->ExportHash[ObjectHash]; j!=INDEX_NONE; j=DynamicImportLinker->ExportMap[j].HashNext )
						{
							FObjectExport& ObjExport = DynamicImportLinker->ExportMap[j];

							if (ObjExport.ObjectName == Import.ObjectName)
							{
								// walk up the outer chains to compare & validate we've got right object
								FObjectResource* ImportOuterResource = ImpExpPtr(Import.OuterIndex);
								FObjectResource* ExportOuterResource = DynamicImportLinker->ImpExpPtr(ObjExport.OuterIndex);

								while( (ImportOuterResource && ExportOuterResource) 
									    && ImportOuterResource->ObjectName == ExportOuterResource->ObjectName)
								{
									ImportOuterResource = ImpExpPtr(ImportOuterResource->OuterIndex);
									ExportOuterResource = DynamicImportLinker->ImpExpPtr(ExportOuterResource->OuterIndex);
								} 

								// This means we managed to walk up the chain and all objects were a match
								if (!ImportOuterResource && !ExportOuterResource)
								{
									UE_LOGF(LogLinker, Verbose, "Assigning %ls as linker to object %ls in %ls", *DynamicImportLinker->GetLinker()->LinkerRoot->GetName(), *Import.ObjectName.ToString(), *LinkerRoot->GetName());
									Import.SourceLinker = DynamicImportLinker;
									break;
								}
							}
						}

						if (Import.SourceLinker)
						{
							break;
						}
					}
				}
			}
#endif // WITH_EDITORONLY_DATA

			// Copy the SourceLinker from the FObjectImport for our Outer if the SourceLinker hasn't been set yet,
			// Otherwise we may be overwriting a re-directed linker and SourceIndex is already from the redirected one
			// or we had an assigned package and our linker is already set.
			if (!Import.SourceLinker)
			{
				Import.SourceLinker = OuterImport.SourceLinker;
			}
		}
		else
		{
			check(Import.OuterIndex.IsExport());
			check(Import.HasPackageName()); // LoadImportPackage was responsible to set the SourceLinker
		}

#if WITH_EDITOR
		if (SlowTask)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			SlowTask->EnterProgressFrame(50);
		}
#endif

		// Now that we have a linker for the import, resolve the the export map index of our import in that linker
		// if we do not have a linker, then this import is native/in memory only
		if( Import.SourceLinker )
		{
			if (!Import.SourceLinker->bHasFoundExistingExports.load(std::memory_order_acquire))
			{
				// Prevents any other serialization function from modifying this linker while we're doing operations on it.
				UE::TUniqueLock UniqueLock(Import.SourceLinker->Mutex);
				UE_ASSET_LOG(LogLinker, Log, PackagePath, TEXT("Source linker '%s' has not processed all header information, ticking it now"), *GetNameSafe(Import.SourceLinker->LinkerRoot));
				// This means that the source linker timed out during it's async loading tick and that the header information hasn't been fully processed yet.
				// Make sure that the header information is available but don't process any imports or exports (LOAD_NoVerify).
				TGuardValue<uint32> LinkerLoadFlagGuard(Import.SourceLinker->LoadFlags, Import.SourceLinker->LoadFlags | LOAD_NoVerify);
				if (Import.SourceLinker->Tick(0.f, false, false, nullptr) == LINKER_Failed)
				{
					UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Failed ticking import source linker '%s'"), *GetNameSafe(Import.SourceLinker->LinkerRoot));
					return false;
				}
			}
			checkf(Import.SourceLinker->bHasFoundExistingExports.load(std::memory_order_acquire), TEXT("Imported linker (%s) should have had its exports discovered by the time we are verifying imports."), *GetNameSafe(Import.SourceLinker->LinkerRoot));

			// Assign the linker root of the source linker as the package we are looking for.
			Pkg = Import.SourceLinker->LinkerRoot;

			// Find this import within its existing linker.
			int32 iHash = GetHashBucket( Import.ObjectName );

			for( int32 j=Import.SourceLinker->ExportHash[iHash]; j!=INDEX_NONE; j=Import.SourceLinker->ExportMap[j].HashNext )
			{
				if (!ensureMsgf(Import.SourceLinker->ExportMap.IsValidIndex(j), TEXT("Invalid index [%d/%d] while attempting to import '%s' with LinkerRoot '%s'"),
					j, Import.SourceLinker->ExportMap.Num(), *ImportObjectName, *GetNameSafe(Import.SourceLinker->LinkerRoot)))
				{
					break;
				}
				else
				{
					const FObjectExport& SourceExport = Import.SourceLinker->ExportMap[ j ];
					if
					(
						SourceExport.ObjectName == Import.ObjectName
						// If we are not explicitly looking for a redirector, skip for now as it will be properly handled in VerifyImport
						&& ((Import.ClassName == NAME_ObjectRedirector) == (Import.SourceLinker->GetExportClassName(j) == NAME_ObjectRedirector))
					)
					{
						// at this point, SourceExport is an FObjectExport in another linker that looks like it
						// matches the FObjectImport we're trying to load - double check that we have the correct one
						if( Import.OuterIndex.IsImport() )
						{
							const FObjectImport& OuterImport = Imp(Import.OuterIndex);

							// OuterImport is the FObjectImport for this resource's Outer
							if( OuterImport.SourceLinker )
							{
								// if the import for our Outer doesn't have a SourceIndex, it means that
								// we haven't found a matching export for our Outer yet.  This should only
								// be the case if our Outer is a top-level UPackage
								if( OuterImport.SourceIndex==INDEX_NONE )
								{
									// At this point, we know our Outer is a top-level UPackage, so
									// if the FObjectExport that we found has an Outer that is
									// not a linker root, this isn't the correct resource
									if( !SourceExport.OuterIndex.IsNull() )
									{
										continue;
									}
								}
								// if our import and its outer share the same source linker, make sure the outer source index matches as expected, otherwise, skip resolving this import
								else if(Import.SourceLinker == OuterImport.SourceLinker)
								{
									if (FPackageIndex::FromExport(OuterImport.SourceIndex) != SourceExport.OuterIndex)
									{
										continue;
									}
								}
								else
								{
									// if the import and its outer do not share a source linker, validate the import entry of the outer in the source linker matches otherwise skip resolveing the outer
									check(SourceExport.OuterIndex.IsImport())
									const FObjectImport& SourceExportOuter = Import.SourceLinker->Imp(SourceExport.OuterIndex);
									if (SourceExportOuter.ObjectName != OuterImport.ObjectName)
									{
										continue;
									}
									else
									{
										if ((SourceExportOuter.ClassName != OuterImport.ClassName) || (SourceExportOuter.ClassPackage != OuterImport.ClassPackage))
										{
											// Since we don't have an exact match, do some additional verification when we create the outer import (where we have a valid class object).
											ImportsToVerifyOnCreate.Add(Import.OuterIndex.ToImport());
										}
									}
								}
							}
						}

						// Since import can have export outer and vice versa now, consider import and export sharing outers to be allowed
						auto IsPrivateImportAllowed = [this](int32 InImportIndex)
						{
							return ImportIsInAnyExport(InImportIndex) || AnyExportIsInImport(InImportIndex) || AnyExportShareOuterWithImport(InImportIndex);
						};

						const bool bIsImportPublic = !!(SourceExport.ObjectFlags & RF_Public);
						if( !bIsImportPublic && !IsPrivateImportAllowed(ImportIndex))
						{
							SafeReplace = SafeReplace || (GIsEditor && !IsRunningCommandlet());

							// determine if this find the thing that caused this import to be saved into the map
							FPackageIndex FoundIndex = FPackageIndex::FromImport(ImportIndex);
							for ( int32 i = 0; i < Summary.ExportCount; i++ )
							{
								const FObjectExport& Export = ExportMap[i];
								if ( Export.SuperIndex == FoundIndex )
								{
									UE_ASSET_LOG(LogLinker, Log, PackagePath, TEXT("Private import was referenced by export '%s' (parent)"), *Export.ObjectName.ToString());
									SafeReplace = false;
								}
								else if ( Export.ClassIndex == FoundIndex )
								{
									UE_ASSET_LOG(LogLinker, Log, PackagePath, TEXT("Private import was referenced by export '%s' (class)"), *Export.ObjectName.ToString());
									SafeReplace = false;
								}
								else if ( Export.OuterIndex == FoundIndex )
								{
									UE_ASSET_LOG(LogLinker, Log, PackagePath, TEXT("Private import was referenced by export '%s' (outer)"), *Export.ObjectName.ToString());
									SafeReplace = false;
								}
							}
							for ( int32 i = 0; i < Summary.ImportCount; i++ )
							{
								if ( i != ImportIndex )
								{
									const FObjectImport& TestImport = ImportMap[i];
									if ( TestImport.OuterIndex == FoundIndex )
									{
										UE_ASSET_LOG(LogLinker, Log, PackagePath, TEXT("Private import was referenced by import '%s' (outer)"), *ImportObjectName);
										SafeReplace = false;
									}
								}
							}

							if ( !SafeReplace )
							{
								UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Can't import private object %s %s"), *Import.ClassName.ToString(), *GetImportFullName(ImportIndex));
								return false;
							}
							else
							{
								FString Suffix = LOCTEXT("LoadWarningSuffix_privateobject", " [private]").ToString();
								if ( !WarningSuffix.Contains(Suffix) )
								{
									WarningSuffix += Suffix;
								}
								break;
							}
						}

						// Found the FObjectExport for this import
						if ((Import.ClassName != Import.SourceLinker->GetExportClassName(j)) || (Import.ClassPackage != Import.SourceLinker->GetExportClassPackage(j)))
						{
							// Since we don't have an exact match, do some additional verification when we create the import (where we have a valid class object).
							ImportsToVerifyOnCreate.Add(ImportIndex);
						}

						Import.SourceIndex = j;
						break;
					}
				}
			}
		}
	}

	bool bCameFromMemoryOnlyPackage = false;
	if (!Pkg && TmpPkg &&
		// Assign Pkg to resolve the object in memory when there is no source linker available only if the package is MemoryOnly
		// or we are loading an instanced package in which case the import package might be a duplicated pie package for example for which no linker exists
		(TmpPkg->HasAnyPackageFlags(PKG_InMemoryOnly) || IsContextInstanced()))
	{
		Pkg = TmpPkg; // this is a package that exists in memory only, so that is the package to search regardless of FindIfFail
		bCameFromMemoryOnlyPackage = true;

		if (IsCoreUObjectPackage(Import.ClassPackage) && Import.ClassName == NAME_Package && !TmpPkg->GetOuter())
		{
			if (InstancingContext.RemapPackage(Import.ObjectName) == TmpPkg->GetFName())
			{
				// except if we are looking for _the_ package...in which case we are looking for TmpPkg, so we are done
				Import.XObject = TmpPkg;
				FUObjectSerializeContext* CurrentLoadContext = FUObjectThreadContext::Get().GetSerializeContext();
				CurrentLoadContext->IncrementImportCount();
				FLinkerManager::Get().AddLoaderWithNewImports(this);
				return false;
			}
		}
	}

	// RobM: We should remove the bFindObjectByName path
	const bool bFindObjectByName = (Pkg == nullptr) && ((LoadFlags & LOAD_FindIfFail) != 0);

	// If not found in file, see if it's a public native transient class or field.
	if( Import.SourceIndex==INDEX_NONE && (Pkg!=nullptr || bFindObjectByName))
	{
		TStringBuilder<256> ImportClassTemp;
		Import.ClassPackage.ToString(ImportClassTemp);
		UObject* ClassPackage = FindObject<UPackage>( nullptr, *ImportClassTemp );
		if( ClassPackage )
		{
			Import.ClassName.ToString(ImportClassTemp);
			UClass* FindClass = FindObject<UClass>( ClassPackage, *ImportClassTemp);
			if( FindClass )
			{
				UObject* FindOuter			= Pkg;

				if ( Import.OuterIndex.IsImport() )
				{
					// if this import corresponds to an intrinsic class, OuterImport's XObject will be NULL if this import
					// belongs to the same package that the import's class is in; in this case, the package is the correct Outer to use
					// for finding this object
					// otherwise, this import represents a field of an intrinsic class, and OuterImport's XObject should be non-NULL (the object
					// that contains the field)
					FObjectImport& OuterImport	= Imp(Import.OuterIndex);
					if ( OuterImport.XObject != nullptr )
					{
						FindOuter = OuterImport.XObject;
					}
				}

				UObject* FindObject = FindImportFast(FindClass, bFindObjectByName ? nullptr : FindOuter, Import.ObjectName, bFindObjectByName);
				// Reference to in memory-only package's object, native transient class or CDO of such a class.
				bool bIsInMemoryOnlyOrNativeTransient = bCameFromMemoryOnlyPackage || (FindObject != nullptr && ((FindObject->IsNative() && FindObject->HasAllFlags(RF_Public | RF_Transient)) || (FindObject->HasAnyFlags(RF_ClassDefaultObject) && FindObject->GetClass()->IsNative() && FindObject->GetClass()->HasAllFlags(RF_Public | RF_Transient))));
				// Check for structs which have been moved to another header (within the same class package).
				if (!FindObject && bIsInMemoryOnlyOrNativeTransient && FindClass == UScriptStruct::StaticClass())
				{
					FindObject = StaticFindFirstObject(FindClass, *Import.ObjectName.ToString(),
						EFindFirstObjectOptions::ExactClass | EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous,
						ELogVerbosity::Warning, TEXT("Finding import by name"));

					if (FindObject && FindOuter->GetOutermost() != FindObject->GetOutermost())
					{
						// Limit the results to the same package.I
						FindObject = nullptr;
					}
				}
				if (FindObject != nullptr && ((LoadFlags & LOAD_FindIfFail) || bIsInMemoryOnlyOrNativeTransient))
				{
					Import.XObject = FindObject;
					FUObjectSerializeContext* CurrentLoadContext = FUObjectThreadContext::Get().GetSerializeContext();
					CurrentLoadContext->IncrementImportCount();
					FLinkerManager::Get().AddLoaderWithNewImports(this);
				}
				else
				{
					SafeReplace = true;
				}
			}
			else
			{
				SafeReplace = true;
			}
		}
		else
		{
			SafeReplace = true;
		}

		if (!Import.XObject && !SafeReplace)
		{
			return true;
		}
	}

	return false;
}

UObject* FLinkerLoad::CreateExportAndPreload(int32 ExportIndex, bool bForcePreload /* = false */)
{
	UObject* Object = CreateExport(ExportIndex);
	if (Object && (bForcePreload || dynamic_cast<UClass*>(Object) || Object->IsTemplate() || dynamic_cast<UObjectRedirector*>(Object)))
	{
		Preload(Object);
	}

	return Object;
}

UClass* FLinkerLoad::GetExportLoadClass(int32 Index)
{
	FObjectExport& Export = ExportMap[Index];

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	// VerifyImport() runs the risk of loading up another package, and we can't
	// have that when we're explicitly trying to block depEndency loads...
	// if this needs a class from another package, IndexToObject() should return
	// a ULinkerPlaceholderClass instead
	if (Export.ClassIndex.IsImport() && !(LoadFlags & LOAD_DeferDependencyLoads))
#else  // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (Export.ClassIndex.IsImport())
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	{
		// @TODO: I believe IndexToObject() -> CreateImport() will verify this
		//        for us, if it has to; so is this necessary?
		VerifyImport(Export.ClassIndex.ToImport());
	}

	return dynamic_cast<UClass*>(IndexToObject(Export.ClassIndex));
}

#if WITH_EDITOR
static UStruct* FindKnownSuperStruct(const UE::Serialization::Private::FImportTypeHierarchy* ImportTypeHierarchy)
{
	UStruct* SuperStruct = nullptr;

	if (ImportTypeHierarchy)
	{
		const TArray<UE::Serialization::Private::FTypeResource>& SuperTypes = ImportTypeHierarchy->GetSuperTypes();

		// Types are ordered bottom-up (child classes to parent classes).
		for (int32 TypeIndex = 0; TypeIndex < SuperTypes.Num(); ++TypeIndex)
		{
			const UE::Serialization::Private::FTypeResource& SuperTypeResource = SuperTypes[TypeIndex];

			if (UE::CanCreatePropertyBagPlaceholderSuperType(SuperTypeResource))
			{
				if (UPackage* SuperTypePackage = FindObjectFast<UPackage>(nullptr, SuperTypeResource.PackageName, EFindObjectFlags::ExactClass))
				{
					SuperStruct = FindObjectFast<UStruct>(SuperTypePackage, SuperTypeResource.TypeName, EFindObjectFlags::None);
				}
			}

			if (SuperStruct)
			{
				break;
			}
		}
	}

	if (!SuperStruct)
	{
		SuperStruct = UObject::StaticClass();
	}

	return SuperStruct;
}

UClass* FLinkerLoad::TryCreatePlaceholderClassImport(int32 ImportIndex)
{
	const bool bAllowPlaceholderImportTypes = UE::FPropertyBagRepository::IsPropertyBagPlaceholderObjectFeatureEnabled(UE::FPropertyBagRepository::EPlaceholderObjectFeature::ReplaceMissingTypeImportsOnLoad);
	if (!bAllowPlaceholderImportTypes)
	{
		return nullptr;
	}

	UClass* ClassObject = nullptr;
	FObjectImport& Import = ImportMap[ImportIndex];

	// If the import is already set, return NULL to indicate that we didn't create a placeholder type object.
	if (Import.XObject)
	{
		return nullptr;
	}

	if (UObject* ImportClassPackage = FindObjectFast<UPackage>(nullptr, Import.ClassPackage, EFindObjectFlags::ExactClass))
	{
		if (UClass* ImportClass = FindObjectFast<UClass>(ImportClassPackage, Import.ClassName, EFindObjectFlags::None))
		{
			FPackageIndex ImportPackageIndex = FPackageIndex::FromImport(ImportIndex);
			const UE::Serialization::Private::FImportTypeHierarchy* ImportTypeHierarchy = ImportTypeHierarchies.Find(ImportPackageIndex);

			if (UE::CanCreatePropertyBagPlaceholderTypeForImportClass(ImportClass, ImportTypeHierarchy))
			{
				// If the outer package import is also missing, create it now so that the full path remains the same.
				UPackage* ClassObjectPackage = Cast<UPackage>(IndexToObject(Import.OuterIndex));
				if (!ClassObjectPackage && Import.OuterIndex.IsImport())
				{
					FObjectImport& OuterImport = Imp(Import.OuterIndex);
					if (OuterImport.OuterIndex.IsNull())
					{
						ClassObjectPackage = CreatePackage(*OuterImport.ObjectName.ToString());

						// Flag that this package exists in memory only (i.e. it's not being loaded from disk).
						ClassObjectPackage->SetPackageFlags(PKG_InMemoryOnly);

						// Patch it into the import table so that we resolve to this package for future reference.
						OuterImport.XObject = ClassObjectPackage;
					}
				}

				if (ClassObjectPackage)
				{
					FName ClassObjectName = Import.ObjectName;
					EObjectFlags ClassObjectFlags = RF_Public;

					// If it's now a non-native struct type, use a placeholder class type with a unique name so exports can still be serialized.
					const UScriptStruct* ExistingType = FindObjectFast<UScriptStruct>(ClassObjectPackage, Import.ObjectName, EFindObjectFlags::None);
					if (ExistingType)
					{
						ClassObjectName = MakeUniqueObjectName(ClassObjectPackage, ImportClass, ClassObjectName);
					}

					UStruct* SuperStruct = FindKnownSuperStruct(ImportTypeHierarchy);

					// Create an opaque, non-native public type object that has no reflected properties.
					UStruct* StructObject = UE::FPropertyBagRepository::CreatePropertyBagPlaceholderType(ClassObjectPackage, ImportClass, ClassObjectName, ClassObjectFlags, SuperStruct);
					ClassObject = CastChecked<UClass>(StructObject);

					// Patch it into the import table so that we resolve to this class for any future exports of this type.
					Import.XObject = ClassObject;
				}
			}
		}
	}

	return ClassObject;
}

#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
void FLinkerLoad::CreateUniversalPlaceholders()
{
	// Pass 0: Create necessary packages:
	Algo::ForEach( ImportMap, [this](FObjectImport& Import)
	{
		TryCreateImportedPackage(Import);
	});
	// Pass 1: Create placeholder classes for classes that are declared in this 
	// package:
	Algo::ForEach( ExportMap, [this](const FObjectExport& Export)
	{
		TryCreatePlaceholderClassExport(Export);
	});
	// Pass 2: Create placeholder classes for exports that import their types.
	// If you do this in pass 1 you'll not give classes within the package a chance
	// to create themselves reasonably...
	Algo::ForEach( ExportMap, [this](const FObjectExport& Export)
	{
		TryCreateImportedClassForExport(Export);
	});
	// Pass 3: Create remaining placeholder classes for imports:
	TryCreateImportedClassesForImports();
	// The required classes are now mostly created, but more placeholders may
	// be created as we read properties from the objects (either from 
	// LoadAllObjects or EndLoad)
}

void FLinkerLoad::TryCreateImportedPackage(FObjectImport& Import)
{
	if( !Import.OuterIndex.IsNull() )
	{
		return;
	}
	
	FName ObjectName = InstancingContext.RemapPackage(Import.ObjectName);
	UPackage* Pkg = CreatePackage(*ObjectName.ToString());
	Pkg->SetPackageFlags(PKG_InMemoryOnly); // tag as in memory so we don't check for a corresponding FLinker
	Import.XObject = Pkg;
}

void FLinkerLoad::CreatePlaceholderImport(FObjectImport& Import)
{
	// Packages and obviously identifiable classes have already been created
	// by the prepass (CreateUniversalPlaceholders):
	UPackage* ClassPackage = FindObjectFast<UPackage>( nullptr, Import.ClassPackage, EFindObjectFlags::None );
	check(ClassPackage);
	UClass*	FindClass = FindObjectFast<UClass>( ClassPackage, Import.ClassName, EFindObjectFlags::None );
	check(FindClass);
	UObject* FindOuter = FindOuter = IndexToObject( Import.OuterIndex );;
	UObject* FindObject = FindImportFast(FindClass, FindOuter, Import.ObjectName); // import was already created, likely by native code
	if(FindObject)
	{
		return;
	}

	if(FindClass->IsChildOf(UClass::StaticClass()))
	{
		ForceCreatePlaceholderClassImport(Import);
	}
	else
	{
		FindObject = NewObject<UObject>(FindOuter, FindClass, Import.ObjectName, RF_Public);
		if(UScriptStruct* Struct = Cast<UScriptStruct>(FindObject))
		{
			// add a single property to avoid creating a 0 sized struct. Zero sized allocations
			// are supported by some systems, but containers are generally unhappy with zero stride
			// container elements. We can't just use the fallback struct as we want a proper
			// object name:
			FBoolProperty* Property = new FBoolProperty(Struct, "__PlaceholderSentinel");
			Struct->ChildProperties = Property;
			Struct->StaticLink(true); // make the struct roughly usable, it's just a dummy type
		}
		Import.XObject = FindObject;
	}
}

void FLinkerLoad::TryCreatePlaceholderClassExport(const FObjectExport& Export)
{
	if(!Export.ClassIndex.IsExport())
	{
		return;
	}
	
	const int32 ExportIndex = Export.ClassIndex.ToExport();
	FObjectExport& ExportClass = ExportMap[ExportIndex];
	if (ExportClass.Object)
	{
		check(ExportClass.Object->IsA<UClass>());
		return;
	}

	// We need two pieces of external information to create the class defined
	// in this package:
	//	1. The class of the class (e.g. UClass::StaticClass(), or 
	//		UBlueprintGeneratedClass::StaticClass())
	//	2. The class that the class inherits from (e.g. AActor::StaticClass())
	// Find or create both of them - at the moment these are always import so no
	// topo sort is needed for exports:
	check(ExportClass.ClassIndex.IsImport());
	UClass* const ClassOfClass = ForceCreatePlaceholderClassImport(
		ImportMap[ExportClass.ClassIndex.ToImport()], UClass::StaticClass());
		
	check(ExportClass.SuperIndex.IsImport());
	UClass* const SuperClass = ForceCreatePlaceholderClassImport(
		ImportMap[ExportClass.SuperIndex.ToImport()]);

	// With required imports resolved, we can now create the export object (which is
	// a class) and its cdo:
	UClass* const ClassObject = 
		Cast<UClass>(UE::FPropertyBagRepository::CreatePropertyBagPlaceholderType(
			LinkerRoot, ClassOfClass, ExportClass.ObjectName, RF_Public, SuperClass));
	ClassObject->SetLinker(this, ExportIndex);
	ClassObject->SetFlags(
		ExportClass.ObjectFlags|
		RF_NeedLoad|RF_NeedPostLoad|RF_NeedPostLoadSubobjects|RF_WasLoaded);
	FUObjectThreadContext::Get().GetSerializeContext()->AddLoadedObject(ClassObject);
	ExportClass.Object = ClassObject;

	UObject* const ClassCDO = ClassObject->GetDefaultObject(false);
	// find export index for the cdo, if any:
	const FName CDOName = ClassCDO->GetFName();
	const int32 CDOIndex = ExportMap.IndexOfByPredicate(
		[CDOName, ExportIndex](const FObjectExport& Export)
		{
			return Export.ObjectName == CDOName &&
				Export.ClassIndex.IsExport() &&
				Export.ClassIndex.ToExport() == ExportIndex;
		}
	);
	if(CDOIndex != INDEX_NONE)
	{
		ClassCDO->SetFlags(
			RF_NeedLoad|RF_NeedPostLoad|RF_NeedPostLoadSubobjects|RF_WasLoaded);
		FObjectExport& CDOExport = ExportMap[CDOIndex];
		FUObjectThreadContext::Get().GetSerializeContext()->AddLoadedObject(ClassCDO);
		CDOExport.Object = ClassCDO;
		ClassCDO->SetLinker(this, CDOIndex);
	}
}

void FLinkerLoad::TryCreateImportedClassForExport(const FObjectExport& Export)
{
	if(!Export.ClassIndex.IsImport())
	{
		return;
	}

	FObjectImport& Import = ImportMap[ Export.ClassIndex.ToImport() ];
	if(Import.XObject != nullptr) // already resolved, living with whatever we've found
	{
		return;
	}

	// Look for an object that already exists, even in our reduced condition - 
	// this allows coreuobject to function without any placeholders for coreuobject
	// level types:
	FObjectImport& OuterImport = ImportMap[ Import.OuterIndex.ToImport() ];
	if(!OuterImport.XObject)
	{
		OuterImport.XObject = FindObjectFast<UPackage>(
			nullptr, OuterImport.ObjectName, EFindObjectFlags::ExactClass);
	}

	if(OuterImport.XObject)
	{
		Import.XObject = StaticFindObjectFastInternal(
			nullptr, OuterImport.XObject, Import.ObjectName, EFindObjectFlags::ExactClass);
	}

	// failed to resolve, create the placeholder:
	if(!Import.XObject)
	{
		ForceCreatePlaceholderClassImport(ImportMap[Export.ClassIndex.ToImport()]);
	}
}

void FLinkerLoad::TryCreateImportedClassesForImports()
{
	// The purpose of this function is to build a complete and correct view of
	// *imported* classes, including consistent use of custom class types that
	// are not knowably derived from uclass because their associated native
	// code is not loaded.
	
	// Helper function to build a FSoftObjectPath from a not yet loaded FObjectImport:
	const auto ObjectImportToSoftObjectPath = [this](const FObjectImport& Import) -> FSoftObjectPath
	{
		if(Import.OuterIndex.IsNull())
		{
			return FSoftObjectPath(FTopLevelAssetPath(Import.ObjectName, FName()));
		}

		// common case is root level import, so preallocating for that:
		TArray<FName, TInlineAllocator<2>> Names;
		const FObjectResource* Current = &Import;

		while (Current)
		{
			Names.Add(Current->ObjectName);
			if (Current->OuterIndex.IsNull())
			{
				break;
			}
			Current = &ImpExp(Current->OuterIndex);
		}
		check(Names.Num() >= 2);
		FTopLevelAssetPath Path(Names[Names.Num() - 1], Names[Names.Num() - 2]);
		FUtf8String SubPath;
		if(Names.Num() > 2 )
		{
			for( int32 Iter = Names.Num() - 3; Iter >= 0; --Iter )
			{
				SubPath.Append(Names[Iter].ToString());
				if(Iter >= 1)
				{
					SubPath.Append(".");
				}
			}
		}
		
		return FSoftObjectPath(Path, SubPath);
	};

	// Gather all identities of all objects used as a class in
	// the import table:
	TSet<FSoftObjectPath> ImportedClasses;
	Algo::Transform( ImportMap, ImportedClasses, [&](const FObjectImport& Import)
	{
		return FSoftObjectPath(
			FTopLevelAssetPath(Import.ClassPackage, Import.ClassName)
		);
	});

	// root classes identified by usage as a class in an import
	// now look for classes that are instances of classes:
	TMap<FSoftObjectPath, int32> ClassToImportIndex;
	TMap<FSoftObjectPath, FSoftObjectPath> AllClassesWithClassTypes;
	TSet<FSoftObjectPath> DiscoveredClassTypes;
	const FSoftObjectPath ClassPath(UClass::StaticClass());
	const FSoftObjectPath ObjectPath(UObject::StaticClass());

	for( int32 ImportIndex = 0; ImportIndex < ImportMap.Num(); ++ImportIndex )
	{
		const FObjectImport& Import = ImportMap[ImportIndex];
		// This import may be a UClass, test it by seeing if it's used as
		// an imported class:
		FSoftObjectPath ImportPath = ObjectImportToSoftObjectPath(Import);
		if( ImportedClasses.Contains(ImportPath) )
		{
			ClassToImportIndex.Add(ImportPath, ImportIndex);
			check(!AllClassesWithClassTypes.Contains(ImportPath)); // duplicate import map?
			FSoftObjectPath ImportClassPath(
				FTopLevelAssetPath(Import.ClassPackage, Import.ClassName)
			);
			if(ImportClassPath == ClassPath)
			{
				// a normal class type:
				AllClassesWithClassTypes.Add( 
					ImportPath, 
					ObjectPath
				);
			}
			else
			{
				// a custom class type:
				AllClassesWithClassTypes.Add( 
					ImportPath, 
					ImportClassPath
				);
				DiscoveredClassTypes.Add(ImportClassPath);
			}
		}
	}

	// All imports categorized, finally categorize implicitly used
	// classes that are themselves not an import:
	// would prevent toposort from organizing our classes:
	AllClassesWithClassTypes.Add(ObjectPath, FSoftObjectPath());
	AllClassesWithClassTypes.Add(ClassPath, ObjectPath);
	for(const FSoftObjectPath& ImplicitClass : ImportedClasses)
	{
		if(!AllClassesWithClassTypes.Contains(ImplicitClass))
		{
			ClassToImportIndex.Add(ImplicitClass, INDEX_NONE);
			if(!DiscoveredClassTypes.Contains(ImplicitClass))
			{
				AllClassesWithClassTypes.Add(
					ImplicitClass, ObjectPath
				);
			}
			else
			{
				AllClassesWithClassTypes.Add(
					ImplicitClass, ClassPath
				);
			}
		}
	}

	// topological sort of the classes - check for cycles
	TArray<FSoftObjectPath> AllClassesOrdered;
	AllClassesWithClassTypes.GenerateKeyArray(AllClassesOrdered);

	const bool bOrderedClassNodes = Algo::TopologicalSort(AllClassesOrdered,
		[&AllClassesWithClassTypes](const FSoftObjectPath& Node)
		{
			TArray<FSoftObjectPath> Deps;
			// This lambda defines dependencies for each node
			// i.e. which nodes must come before this one.
			if (const FSoftObjectPath* Parent = AllClassesWithClassTypes.Find(Node))
			{
				Deps.Add(*Parent);
			}
			return Deps;
		}
	);
	check(bOrderedClassNodes); // cycles in class hierarchy, this should never have happened

	// Classes have been ordered and discovered, we can now create remaining placeholder
	// classes:
	for(const FSoftObjectPath& Node : AllClassesOrdered)
	{
		const UObject* Existing = Node.ResolveObject();
		if(Existing)
		{
			continue;
		}

		const int32 ImportIndex = ClassToImportIndex.FindChecked(Node);
		if(ImportIndex != INDEX_NONE)
		{
			ForceCreatePlaceholderClassImport(ImportMap[ImportIndex]);
		}
		else
		{
			// implicit import, just create the class (with a package if necessary):
			const FName PackageName = Node.GetLongPackageFName();
			UPackage* ClassPackage = FindObjectFast<UPackage>( 
				nullptr, PackageName, EFindObjectFlags::None );
			if(!ClassPackage)
			{
				ClassPackage = CreatePackage(*PackageName.ToString());
				// tag as in memory so no code checks for a corresponding FLinker:
				ClassPackage->SetPackageFlags(PKG_InMemoryOnly);
			}
			const FSoftObjectPath& CustomClass = AllClassesWithClassTypes.FindChecked(Node);
			UE::FPropertyBagRepository::CreatePropertyBagPlaceholderType(
				ClassPackage, 
				UClass::StaticClass(), 
				Node.GetAssetPath().GetAssetName(), 
				RF_Public, 
				CastChecked<UClass>(CustomClass.ResolveObject()));
		}
	}
}

UClass* FLinkerLoad::ForceCreatePlaceholderClassImport(FObjectImport& Import, UClass* SuperClassOverride)
{
	// Placeholder already created:
	if (Import.XObject)
	{
		return CastChecked<UClass>(Import.XObject);
	}

	if(!SuperClassOverride)
	{
		SuperClassOverride = UObject::StaticClass();
	}

	UObject* ImportClassPackage = FindObjectFast<UPackage>(nullptr, Import.ClassPackage, EFindObjectFlags::ExactClass);
	UClass* ImportClass = FindObjectFast<UClass>(ImportClassPackage, Import.ClassName, EFindObjectFlags::None);
	if(!ImportClass)
	{
		ImportClass = UClass::StaticClass();
	}

	// outer should have already been created by pass 0:
	UPackage* ClassObjectPackage = Cast<UPackage>(IndexToObject(Import.OuterIndex));
	check(ClassObjectPackage);
	
	UClass* PlaceholderClass = FindObjectFast<UClass>( ClassObjectPackage, Import.ObjectName, EFindObjectFlags::None );
	if(!PlaceholderClass)
	{
		PlaceholderClass = CastChecked<UClass>(UE::FPropertyBagRepository::CreatePropertyBagPlaceholderType(
			ClassObjectPackage, ImportClass, Import.ObjectName, RF_Public, SuperClassOverride));
	}

	Import.XObject = PlaceholderClass;
	return PlaceholderClass;
}
#endif // UE_INTERNAL_UNIVERSAL_PLACEHOLDERS

#if UE_INTERNAL_LINKER_BOUNDS
void FLinkerLoad::SetBoundsImpl(FArchiveBounds NewBounds)
{
	if(UE::Private::ShouldEnforceLinkerObjectBounds())
	{
		CurrentBounds = NewBounds;
	}
}

FArchiveBounds FLinkerLoad::GetBoundsImpl() const
{
	return CurrentBounds;
}
#endif // UE_INTERNAL_LINKER_BOUNDS

UClass* FLinkerLoad::TryCreatePlaceholderClassForExport(int32 ExportIndex)
{
	UClass* ClassObject = nullptr;
	FObjectExport& Export = ExportMap[ExportIndex];

	// If the class import is missing, create a placeholder for this export. This will allow us to instance and redirect its data into a property bag.
	if (Export.ClassIndex.IsImport())
	{
		ClassObject = TryCreatePlaceholderClassImport(Export.ClassIndex.ToImport());
	}

	return ClassObject;
}
#endif

#if WITH_METADATA
int32 FLinkerLoad::LoadMetaDataFromExportMap(bool bForcePreload)
{
	UDEPRECATED_MetaData* DeprecatedMetaData = nullptr;
	int32 DeprecatedMetaDataIndex = INDEX_NONE;

	// Try to find MetaData and load it first as other objects can depend on it.
	for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
	{
		if (ExportMap[ExportIndex].ObjectName == NAME_PackageMetaData && ExportMap[ExportIndex].OuterIndex.IsNull())
		{
			DeprecatedMetaData = Cast<UDEPRECATED_MetaData>(CreateExportAndPreload(ExportIndex, bForcePreload));
			DeprecatedMetaDataIndex = ExportIndex;
			break;
		}
	}

	// If not found then try to use old name and rename.
	if (DeprecatedMetaDataIndex == INDEX_NONE)
	{
		for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
		{
			if (ExportMap[ExportIndex].ObjectName == NAME_MetaData && ExportMap[ExportIndex].OuterIndex.IsNull())
			{
				UObject* Object = CreateExportAndPreload(ExportIndex, bForcePreload);
				Object->Rename(*FName(NAME_PackageMetaData).ToString(), NULL);

				DeprecatedMetaData = Cast<UDEPRECATED_MetaData>(Object);
				DeprecatedMetaDataIndex = ExportIndex;
				break;
			}
		}
	}

	if (LinkerRoot && IsValid(DeprecatedMetaData))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		LinkerRoot->DeprecatedMetaData = DeprecatedMetaData;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return DeprecatedMetaDataIndex;
}
#endif // WITH_METADATA

/**
 * Loads all objects in package.
 *
 * @param bForcePreload	Whether to explicitly call Preload (serialize) right away instead of being
 *						called from EndLoad()
 */
void FLinkerLoad::LoadAllObjects(bool bForcePreload)
{
	SCOPED_LOADTIMER(LinkerLoad_LoadAllObjects);
	UE_SCOPED_COOK_STAT(LinkerRoot->GetFName(), EPackageEventStatType::LoadPackage);
#if WITH_EDITOR
	TOptional<FScopedSlowTask> SlowTask;
	if (ShouldCreateThrottledSlowTask())
	{
		static const FText LoadingObjectText = NSLOCTEXT("Core", "LinkerLoad_LoadingObjects", "Loading Objects");
		SlowTask.Emplace(static_cast<float>(ExportMap.Num()), LoadingObjectText);
		SlowTask->Visibility = ESlowTaskVisibility::Invisible;
	}
#endif

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	// if we're re-entering a call to LoadAllObjects() while DeferDependencyLoads
	// is set, then we're not doing our job (we're risking an export needing
	// another external asset)... if this is hit, then we're most likely already
	// in this function (for this linker) further up the load chain; it should
	// finish the loads there
	check((LoadFlags & LOAD_DeferDependencyLoads) == 0);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	if ((LoadFlags & LOAD_Async) != 0)
	{
		bForcePreload = true;
	}

	double StartTime = FPlatformTime::Seconds();

#if WITH_METADATA
	// MetaData object index in this package.
	int32 DeprecatedMetaDataIndex = LoadMetaDataFromExportMap(bForcePreload);
#endif // WITH_METADATA

#if USE_STABLE_LOCALIZATION_KEYS
	if (GIsEditor && (LoadFlags & LOAD_ForDiff))
	{
		// If this package is being loaded for diffing, then we need to force it to have a unique package localization ID to avoid in-memory identity conflicts
		// Note: We set this on the archive first as finding/loading the meta-data (which ForcePackageNamespace does) may trigger the load of some objects within this package
		const FString PackageLocalizationId = FGuid::NewGuid().ToString();
		SetLocalizationNamespace(PackageLocalizationId);
		TextNamespaceUtil::ForcePackageNamespace(LinkerRoot, PackageLocalizationId);
	}
#endif // USE_STABLE_LOCALIZATION_KEYS

	// Tick the heartbeat if we're loading on the game thread
	const bool bShouldTickHeartBeat = IsInGameThread();
	
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	CreateUniversalPlaceholders();
#endif // UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	for(int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
	{
#if WITH_EDITOR
		if (SlowTask)
		{
			UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND();
			SlowTask->EnterProgressFrame(1);
		}
#endif

#if WITH_METADATA
		if (ExportIndex == DeprecatedMetaDataIndex)
		{
			continue;
		}
#endif

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
		// this is here to prevent infinite recursion; if IsExportBeingResolved()
		// returns true, then that means the export's class is currently being
		// force-generated... in that scenario, the export's Object member would
		// not have been set yet, and the call below to CreateExport() would put
		// us right back here in the same situation (CreateExport() needs the
		// export's Object set in order to return earlY... it's what makes this
		// function reentrant)
		//
		// since we don't actually use the export object here at this point,
		// then it is safe to skip over it (it's already being created further
		// up the callstack, so don't worry about it being missed)
		if (IsExportBeingResolved(ExportIndex))
		{
			continue;
		}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

		UObject* LoadedObject = CreateExportAndPreload(ExportIndex, bForcePreload);

		// If needed send a heartbeat, but no need to do it too often
		if (bShouldTickHeartBeat && (ExportIndex % 10) == 0)
		{
			FThreadHeartBeat::Get().HeartBeat();
		}
	}

	// Mark package as having been fully loaded.
	if( LinkerRoot )
	{
		LinkerRoot->MarkAsFullyLoaded();
	}
}

/**
 * Returns the ObjectName associated with the resource indicated.
 *
 * @param	ResourceIndex	location of the object resource
 *
 * @return	ObjectName for the FObjectResource at ResourceIndex, or NAME_None if not found
 */
FName FLinkerLoad::ResolveResourceName( FPackageIndex ResourceIndex )
{
	if (ResourceIndex.IsNull())
	{
		return NAME_None;
	}
	return ImpExp(ResourceIndex).ObjectName;
}

UObject* FLinkerLoad::ResolveResource(FPackageIndex Index)
{
	return IndexToObject(Index);
}

/**
 * Serialize the object data for the specified object from the unreal package file.  Loads any
 * additional resources required for the object to be in a valid state to receive the loaded
 * data, such as the object's Outer, Class, or ObjectArchetype.
 *
 * When this function exits, Object is guaranteed to contain the data stored that was stored on disk.
 *
 * @param	Object	The object to load data for.  If the data for this object isn't stored in this
 *					FLinkerLoad, routes the call to the appropriate linker.  Data serialization is
 *					skipped if the object has already been loaded (as indicated by the RF_NeedLoad flag
 *					not set for the object), so safe to call on objects that have already been loaded.
 *					Note that this function assumes that Object has already been initialized against
 *					its template object.
 *					If Object is a UClass and the class default object has already been created, calls
 *					Preload for the class default object as well.
 */

void FLinkerLoad::Preload( UObject* Object )
{
	LLM_SCOPE_BYTAG(UObject_Linker);
	//check(IsValidLowLevel());
	check(Object);

	// This is important to take the lock before checking the RF_NeedLoad flag in case
	// Preload is called on the same objects from 2 different threads, one might see the RF_NeedLoad and remove it
	// before the other, making one thread return from Preload before the object is fully serialized.
	UE::TUniqueLock UniqueLock(Mutex);

	// Preload the object if necessary.
	if (Object->HasAnyFlags(RF_NeedLoad))
	{
		if (Object->GetLinker() == this)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FLinkerLoad::Preload);
#if LOADTIMEPROFILERTRACE_ENABLED
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*WriteToString<256>(TEXT("FLinkerLoad::Preload "), Object->GetFName()), AssetLoadTimeChannel);
#endif // LOADTIMEPROFILERTRACE_ENABLED

			UClass* Cls = Cast<UClass>(Object);
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			bool const bIsNonNativeObject = !Object->GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn);
			// we can determine that this is a blueprint class/struct by checking if it
			// is a class/struct object AND if it is not native (blueprint
			// structs/classes are the Only asset package structs/classes we have)
			bool const bIsBlueprintClass = (Cls != nullptr) && bIsNonNativeObject && Cls->GetClass()->HasAnyClassFlags(CLASS_NeedsDeferredDependencyLoading);
			bool const bIsBlueprintStruct = (Cast<UScriptStruct>(Object) != nullptr) && bIsNonNativeObject;
			// to avoid cyclic dependency issues, we want to defer all external loads
			// that MAY rely on this class/struct (meaning all other blueprint packages)
			bool const bDeferDependencyLoads = (bIsBlueprintClass || bIsBlueprintStruct) && FBlueprintSupport::UseDeferredDependencyLoading();

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			// we should NEVER be pre-loading another blueprint class when the
			// DeferDependencyLoads flag is set (some other blueprint class/struct is
			// already being loaded further up the load chain, and this could introduce
			// a circular load)
			//
			// NOTE: we do allow Preload() calls for structs (because we need a struct
			//       loaded to determine its size), but structs will be prevented from
			//       further loading any of its BP class dependencies (we pass along the
			//       LOAD_DeferDependencyLoads flag)
			check(!bIsBlueprintClass || !Object->HasAnyFlags(RF_NeedLoad) || !(LoadFlags & LOAD_DeferDependencyLoads));
			// right now there are no known scenarios where someone requests a Preloa()
			// on a temporary ULinkerPlaceholderExportObject
			check(!Object->IsA<ULinkerPlaceholderExportObject>());
			ensure(Object->HasAnyFlags(RF_WasLoaded));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			// In certain situations, a constructed object has its initializer deferred (when its archetype hasn't been serialized).
			// In those cases, we shouldn't serialize the object yet (initialization needs to rUn first).
			// See the comment on DeferObjectPreload() for more info on the issue.
			if (FDeferredObjInitializationHelper::DeferObjectPreload(Object))
			{
				return;
			}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

			SCOPE_CYCLE_COUNTER(STAT_LinkerPreload);
			FScopeCycleCounterUObject PreloadScope(Object, GET_STATID(STAT_LinkerPreload));

			// If this is a struct, make sure that its parent struct is completely loaded
			if( UStruct* Struct = dynamic_cast<UStruct*>(Object) )
			{
				if( Struct->GetSuperStruct() )
				{
					Preload( Struct->GetSuperStruct() );
				}
			}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			TGuardValue<uint32> LoadFlagsGuard(LoadFlags, LoadFlags);
			if (bDeferDependencyLoads)
			{
				LoadFlags |= LOAD_DeferDependencyLoads;
			}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

			// make sure this object didn't get loaded in the above Preload call
			if (Object->HasAnyFlags(RF_NeedLoad))
			{
				// grab the resource for this Object
				int32 const ExportIndex = Object->GetLinkerIndex();
				FObjectExport& Export = ExportMap[ExportIndex];
				check(Export.Object==Object);

				const int64 SavedPos = Loader->Tell();
				int64 StartPos = Export.SerialOffset;
				int64 ExpectedSerialSize = Export.SerialSize;
#if WITH_EDITOR
				// for placeholder objects that have no explicit type, we only want to serialize the TPS stream
				bool bSerializeOnlyScriptProperties = false;
				const bool bDoesSavedClassMatchActualClass = DoesSavedClassMatchActualClass(ExportIndex);
				bool& bIsLoadingToPropertyBagObject = FUObjectThreadContext::Get().GetSerializeContext()->bIsLoadingToPropertyBagObject;
				TGuardValue<bool> ScopeGuard(bIsLoadingToPropertyBagObject, UE::FPropertyBagRepository::IsPropertyBagPlaceholderObject(Object));
				bool bCanRunNativeSerialize = !bIsLoadingToPropertyBagObject;
				if (UEVer() >= EUnrealEngineObjectUE5Version::SCRIPT_SERIALIZATION_OFFSET)
				{
					if (bIsLoadingToPropertyBagObject || !bDoesSavedClassMatchActualClass)
					{
						// note: script start/end offsets are relative to the export's offset in the file
						StartPos += Export.ScriptSerializationStartOffset;
						ExpectedSerialSize = Export.ScriptSerializationEndOffset - Export.ScriptSerializationStartOffset;
						// Some UObjects (e.g. URigHierarchy) don't invoke base serialize, and thus won't write out any tagged data to read, even the
						// none terminator. We can detect those by checking for 0 bytes expected and skip tagged serialization
						bSerializeOnlyScriptProperties = ExpectedSerialSize != 0;	// signals that we can safely narrow the load to SerializeScriptProperties()
						// even if we can't serialize the script properties, when
						// the above conditions are met we don't want to run the native serializer:
						bCanRunNativeSerialize = false;
					}
				}
#endif
				// move to the position in the file where this object's data
				// is stored
				Seek(StartPos);
				#if UE_INTERNAL_LINKER_BOUNDS
				// Update archive bounds with object known size:
				FArchiveBoundsScope ArchiveBounds(SetBounds(ExpectedSerialSize)); 
				#endif // UE_INTERNAL_LINKER_BOUNDS

				FAsyncArchive* AsyncLoader = GetAsyncLoader();

				{
					SCOPE_CYCLE_COUNTER(STAT_LinkerPrecache);
					// tell the file reader to read the raw data from disk
					if (AsyncLoader)
					{
						bool bReady = AsyncLoader->PrecacheWithTimeLimit(Export.SerialOffset, Export.SerialSize, bUseTimeLimit, bUseFullTimeLimit, TickStartTime, TimeLimit);
						UE_CLOGF(!(bReady || !bUseTimeLimit || !FPlatformProperties::RequiresCookedData()), LogLinker, Warning, "Hitch on async loading of %ls; this export was not properly precached.", *Object->GetFullName());
					}
					else
					{
						Loader->Precache(Export.SerialOffset, Export.SerialSize);
					}
				}

				// mark the object to indicate that it has been loaded
				Object->ClearFlags ( RF_NeedLoad );

				{
					SCOPE_CYCLE_COUNTER(STAT_LinkerSerialize);
					TRACE_LOADTIME_SERIALIZE_EXPORT_SCOPE(Object, Export.SerialSize);
					LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object->GetPackage(), ELLMTagSet::Assets);
					LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Object->GetClass(), ELLMTagSet::AssetClasses);
					UE_TRACE_METADATA_SCOPE_ASSET(Object, Object->GetClass());
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
					// communicate with FLinkerPlaceholderBase, what object is currently seriAlizing in
					FScopedPlaceholderContainerTracker SerializingObjTracker(Object);
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

#if WITH_EDITOR && WITH_TEXT_ARCHIVE_SUPPORT
					bool bClassSupportsTextFormat = UClass::IsSafeToSerializeToStructuredArchives(Object->GetClass());
#endif
#if WITH_EDITOR
					FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None,
						Object->IsEditorOnly() ? ESoftObjectPathCollectType::EditorOnlyCollect : ESoftObjectPathCollectType::AlwaysCollect,
						ESoftObjectPathSerializeType::AlwaysSerialize);
#endif

					UE::FScopedObjectSerializeContext ObjectSerializeScope(Object, *this);
#if WITH_EDITOR
					FGuardValue_Bitfield(bIsSerializingScriptProperties, false);
#endif // WITH_EDITOR

					if (Object->HasAnyFlags(RF_ClassDefaultObject))
					{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
						if ((LoadFlags & LOAD_DeferDependencyLoads) != 0)
						{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
							check((DeferredCDOIndex == INDEX_NONE) || (DeferredCDOIndex == ExportIndex));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

							// since serializing the CDO can introduce circular
							// dependencies, we want to stave that off until
							// we're ready to handle those
							DeferredCDOIndex = ExportIndex;
							// don't need to actually "consume" the data through
							// serialization though (since we seek back to
							// SavedPos later on)

							// reset the flag and return (don't worry, we make
							// sure to force load this later)
							Object->SetFlags(RF_NeedLoad);
							Seek(SavedPos);
							return;
						}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

#if WITH_EDITOR && WITH_TEXT_ARCHIVE_SUPPORT
						if (IsTextFormat())
						{
							FStructuredArchiveSlot ExportSlot = GetExportSlot(Export.ThisIndex);

							if (bClassSupportsTextFormat)
							{
								Object->GetClass()->SerializeDefaultObject(Object, ExportSlot);
							}
							else
							{
								FStructuredArchiveChildReader ChildReader(ExportSlot);
								FArchiveUObjectFromStructuredArchive Adapter(ChildReader.GetRoot());
								Object->GetClass()->SerializeDefaultObject(Object, Adapter.GetArchive());
							}
						}
						else
#endif
						{
							Object->GetClass()->SerializeDefaultObject(Object, *this);
						}

						Object->SetFlags(RF_LoadCompleted);
					}
					else
					{
#if WITH_EDITOR
						static const FName NAME_UObjectSerialize = FName("UObject::Serialize, Name, ClassName");
						FArchive::FScopeAddDebugData P(*this, NAME_UObjectSerialize);
						FArchive::FScopeAddDebugData N(*this, Object->GetFName());
						FArchive::FScopeAddDebugData C(*this, Object->GetClass()->GetFName());

						SCOPED_LOADTIMER_TEXT(*WriteToString<128>(GetClassTraceScope(Object), TEXTVIEW("_LoadSerialize")));
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(*Object->GetFullName(), AssetLoadTimeChannel);
#endif

#if WITH_EDITOR && WITH_TEXT_ARCHIVE_SUPPORT
						if (IsTextFormat())
						{
							FStructuredArchive::FSlot ExportSlot = GetExportSlot(Export.ThisIndex);

							if (bClassSupportsTextFormat)
							{
								Object->Serialize(ExportSlot.EnterRecord());
							}
							else
							{
								FStructuredArchiveChildReader ChildReader(ExportSlot);
								FArchiveUObjectFromStructuredArchive Adapter(ChildReader.GetRoot());

								if (bSerializeOnlyScriptProperties)
								{
									Object->SerializeScriptProperties(Adapter.GetArchive());
								}
								else
								{
									Object->Serialize(Adapter.GetArchive());
								}
							}
						}
						else
#endif
						{
							UE_SERIALIZE_ACCCESS_SCOPE(Object);
#if WITH_EDITOR
							if (bSerializeOnlyScriptProperties)
							{
								Object->SerializeScriptProperties(*this);
							}
							else if (bCanRunNativeSerialize)
#endif
							{
								Object->Serialize(*this);
							}
						}
#if WITH_EDITOR
						// Ensure begin/end marks were hit.
						check(!bIsSerializingScriptProperties);
#endif
						Object->SetFlags(RF_LoadCompleted);
					}
				}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
				{
					SCOPE_CYCLE_COUNTER(STAT_LinkerLoadDeferred);
					if ((LoadFlags & LOAD_DeferDependencyLoads) != (LoadFlagsGuard.GetOriginalValue() & LOAD_DeferDependencyLoads))
					{
						if (bIsBlueprintStruct)
						{
							ResolveDeferredDependencies((UScriptStruct*)Object);
							// user-defined-structs don't have classes/CDOs, so
							// we don't have to call FinalizeBlueprint() (to
							// serialize/regenerate them)
						}
						else
						{
							UClass* ObjectAsClass = (UClass*)Object;
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
							check(bIsBlueprintClass);
							// since class serialization reads in the class's CDO, then we can be certain that the CDO export object exists
							// (and DeferredExportIndex should reference it); FinalizeBlueprint() depends on DeferredExportIndex being set
							// (and since ResolveDeferredDependencies() can recurse into FinalizeBlueprint(), we check it here, before the
							// resolve is handled)
							//
							// however, sometimes DeferredExportIndex doesn't get set at all (we have to utilize FindCDOExportIndex() to set
							// it), and that happens when the class's ClassGeneratedBy is serialized in null... this will happen for cooked
							// builds (because Blueprints are editor-only objects)
							check((DeferredCDOIndex != INDEX_NONE) || FPlatformProperties::RequiresCookedData());

							if (DeferredCDOIndex == INDEX_NONE)
							{
								DeferredCDOIndex = FindCDOExportIndex(ObjectAsClass);
								check(DeferredCDOIndex != INDEX_NONE);
							}
#else  // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
							// just because DeferredCDOIndex wasn't set (in cooked/PIE scenarios) doesn't mean that we don't need it
							// (FinalizeBlueprint() relies on it being set), so here we make sure we flag the CDO so it gets resolved
							if (DeferredCDOIndex == INDEX_NONE)
							{
								DeferredCDOIndex = FindCDOExportIndex(ObjectAsClass);
							}
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

							ResolveDeferredDependencies(ObjectAsClass);
							FinalizeBlueprint(ObjectAsClass);
						}
					}
				}

				// Conceptually, we could run this here for CDOs and it shouldn't be a problem.
				//
				// We don't do it here for CDOs because we were already doing it for them in
				// ResolveDeferredExports(), and we don't want to destabalize the functional
				// load order of things (doing it here could cause subsequent loads which would
				// happen from a point in ResolveDeferredExports() where they didn't happen before - again, this
				// should be fine; we're just keeping the surface area of this to a minimum at this time)
				if (!Object->HasAnyFlags(RF_ClassDefaultObject))
				{
					// If this was an archetype object, there may be some initializers/preloads that
					// were waiting for it to be fully serialized
					FDeferredObjInitializationHelper::ResolveDeferredInitsFromArchetype(Object);
				}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING


				// Make sure we serialized the right amount of stuff.
				int64 Pos = Tell();
				int64 SizeSerialized = Pos - StartPos;
				if( SizeSerialized != ExpectedSerialSize )
				{
					if (Object->GetClass()->HasAnyClassFlags(CLASS_Deprecated))
					{
						UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("%s: Serial size mismatch: Got %d, Expected %" INT64_FMT), *Object->GetFullName(), (int32)SizeSerialized, ExpectedSerialSize);
					}
					else if (!UE::FPropertyBagRepository::IsPropertyBagPlaceholderObject(Object))
					{
						UE_ASSET_LOG(LogLinker, Fatal, PackagePath, TEXT("%s: Serial size mismatch: Got %d, Expected %" INT64_FMT), *Object->GetFullName(), (int32)SizeSerialized, ExpectedSerialSize);
					}
				}

				Seek(SavedPos);

				// if this is a UClass object and it already has a class default object
				if ( Cls != NULL && Cls->GetDefaultsCount() )
				{
					// make sure that the class default object is completely loaded as well
					Preload(Cls->GetDefaultObject());
				}

#if WITH_EDITOR
				// Check if this object's class has been changed by ActiveClassRedirects.
				FName OldClassName;
				if (Export.OldClassName != NAME_None && Object->GetClass()->GetFName() != Export.OldClassName)
				{
					// This happens when the class has changed only for object instance.
					OldClassName = Export.OldClassName;
				}
				else if (Export.ClassIndex.IsImport())
				{
					// Check if the class has been renamed / replaced in the import map.
					const FObjectImport& ClassImport = Imp(Export.ClassIndex);
					if (ClassImport.OldClassName != NAME_None && ClassImport.OldClassName != Object->GetClass()->GetFName())
					{
						OldClassName = ClassImport.OldClassName;
					}
				}
				else if (Export.ClassIndex.IsExport())
				{
					// Handle blueprints. This is slightly different from the other cases as we're looking for the first
					// native super of the blueprint class (first import).
					const FObjectExport* ClassExport = NULL;
					for (ClassExport = &Exp(Export.ClassIndex); ClassExport->SuperIndex.IsExport(); ClassExport = &Exp(ClassExport->SuperIndex));
					if (ClassExport->SuperIndex.IsImport())
					{
						const FObjectImport& ClassImport = Imp(ClassExport->SuperIndex);
						if (ClassImport.OldClassName != NAME_None)
						{
							OldClassName = ClassImport.OldClassName;
						}
					}
				}
				if (OldClassName != NAME_None)
				{
					// Notify if the object's class has changed as a result of active class redirects.
					Object->LoadedFromAnotherClass(OldClassName);
				}
#endif

				// It's ok now to call PostLoad on blueprint CDOs
				if (Object->HasAnyFlags(RF_ClassDefaultObject) && Object->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
				{
					Object->SetFlags(RF_NeedPostLoad|RF_WasLoaded);
					check(LinkerRoot && LinkerRoot == Object->GetOutermost());
					FUObjectThreadContext::Get().GetSerializeContext()->AddLoadedObject(Object);
				}
			}
		}
		else if (FLinkerLoad* Linker = Object->GetLinker())
		{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			// We need to take the lock before changing the flags
			UE::TUniqueLock UniqueLockInner(Linker->Mutex);
			uint32 const DeferredLoadFlag = (LoadFlags & LOAD_DeferDependencyLoads);
			TGuardValue<uint32> LoadFlagsGuard(Linker->LoadFlags, Linker->LoadFlags | DeferredLoadFlag);
#endif
			// Send to the object's linker.
			Linker->Preload(Object);
		}
	}
}

/**
 * Builds a string containing the full path for a resource in the export table.
 *
 * @param OutPathName		[out] Will contain the full path for the resource
 * @param ResourceIndex		Index of a resource in the export table
 */
void FLinkerLoad::BuildPathName( FString& OutPathName, FPackageIndex ResourceIndex ) const
{
	if ( ResourceIndex.IsNull() )
	{
		return;
	}
	const FObjectResource& Resource = ImpExp(ResourceIndex);
	BuildPathName( OutPathName, Resource.OuterIndex );
	if ( OutPathName.Len() > 0 )
	{
		OutPathName += TEXT('.');
	}
	OutPathName += Resource.ObjectName.ToString();
}

/**
 * Checks if the specified export should be loaded or not.
 * Performs similar checks as CreateExport().
 *
 * @param ExportIndex	Index of the export to check
 * @return				true of the export should be loaded
 */
bool FLinkerLoad::WillTextureBeLoaded( UClass* Class, int32 ExportIndex )
{
	const FObjectExport& Export = ExportMap[ ExportIndex ];

	// Already loaded?
	if ( Export.Object || FilterExport(Export))  // it was "not for" in all acceptable positions
	{
		return false;
	}

	// Build path name
	FString PathName;
	PathName.Reserve(256);
	BuildPathName( PathName, FPackageIndex::FromExport(ExportIndex) );

	UObject* ExistingTexture = StaticFindObjectFastExplicit( Class, Export.ObjectName, PathName, EFindObjectFlags::None, RF_NoFlags );
	if ( ExistingTexture )
	{
		return false;
	}
	else
	{
		return true;
	}
}

#if WITH_EDITORONLY_DATA
namespace UE::Private {
static FString GetPackageObjectFullName(FLinkerLoad* Linker, const FPackageIndex Index)
{
	if (Index.IsImport())
	{
		return Linker->GetImportFullName(Index);
	}
	else if (Index.IsExport())
	{
		return Linker->GetExportFullName(Index);
	}

	return TEXT("none");
}
}
#endif

bool FLinkerLoad::IsPackageReferenceAllowed(UPackage* InPackage)
{
	if (InPackage && (InPackage->GetAssetAccessSpecifier() == EAssetAccessSpecifier::Private))
	{
#if WITH_EDITOR
		// Package loaded for diff is not always in its original location (usually in /Temp/)
		// so we can't reliably compare mount points here
		if ((LoadFlags & LOAD_ForDiff) == 0)
#endif //if WITH_EDITOR
		{
			FName MountPointName = FPackageName::GetPackageMountPoint(FNameBuilder(LinkerRoot->GetFName()).ToView());
			FName ImportMountPointName = FPackageName::GetPackageMountPoint(FNameBuilder(InPackage->GetFName()).ToView());
			if (MountPointName != ImportMountPointName)
			{
				return false;
			}
		}
	}
	return true;
}

UObject* FLinkerLoad::CreateExport( int32 Index )
{
	FScopedCreateExportCounter ScopedCounter( this, Index );
	FDeferredMessageLog LoadErrors(NAME_LoadErrors);

	// Map the object into our table.
	FObjectExport& Export = ExportMap[ Index ];

	// Check whether we already loaded the object and if not whether the context flags allow loading it.
	if( !Export.Object && !FilterExport(Export) ) // for some acceptable position, it was not "not for"
	{
		TGuardValue<void*> GuardThreadContextAsyncPackage(FUObjectThreadContext::Get().AsyncPackage, AsyncRoot);
		UE_TRACK_REFERENCING_PACKAGE_SCOPED(LinkerRoot->GetFName(), PackageAccessTrackingOps::NAME_Load);
		FUObjectSerializeContext* CurrentLoadContext = FUObjectThreadContext::Get().GetSerializeContext();
		check(Export.ObjectName!=NAME_None || !(Export.ObjectFlags&RF_Public));
		check(IsLoading());

		UClass* LoadClass = GetExportLoadClass(Index);
		if( !LoadClass && !Export.ClassIndex.IsNull() ) // Hack to load packages with classes which do not exist.
		{
#if WITH_EDITOR
			// Try creating a placeholder type for it. This may allow us to instance and redirect its data into a property bag (to avoid data loss).
			LoadClass = TryCreatePlaceholderClassForExport(Index);
			if (!LoadClass)
#endif
			{
				Export.bExportLoadFailed = true;

				FString OuterName = Export.OuterIndex.IsNull() ? LinkerRoot->GetFullName() : GetFullImpExpName(Export.OuterIndex);
				FString ClassName = GetClassName(Export.ThisIndex).ToString();
				UE_CLOGF(Export.ObjectFlags & EObjectFlags::RF_Public, LogLinker, Warning, "Unable to load %ls with outer %ls because its class (%ls) does not exist", *Export.ObjectName.ToString(), *OuterName, *ClassName);
				return nullptr;
			}
		}

#if WITH_EDITOR
		// NULL (None) active class redirect.
		if( !LoadClass && Export.ObjectName.IsNone() && Export.ClassIndex.IsNull() && !Export.OldClassName.IsNone() )
		{
			return nullptr;
		}
#endif
		if( !LoadClass )
		{
			LoadClass = UClass::StaticClass();
		}

		check(LoadClass);

		// Check for a valid superstruct while there is still time to safely bail, if this export has one
		if( !Export.SuperIndex.IsNull() )
		{
			UStruct* SuperStruct = (UStruct*)IndexToObject( Export.SuperIndex );

#if WITH_EDITOR
			if (!SuperStruct)
			{
				// Try creating a placeholder type for the super struct.
				if (Export.SuperIndex.IsImport())
				{
					SuperStruct = TryCreatePlaceholderClassImport(Export.SuperIndex.ToImport());
				}
				else
				{
					SuperStruct = TryCreatePlaceholderClassForExport(Export.SuperIndex.ToExport());
				}
			}
#endif

			if( !SuperStruct )
			{
				if( LoadClass->IsChildOf(UFunction::StaticClass()) )
				{
#if WITH_EDITORONLY_DATA
					// In the case of a function object, the outer should be the function's class. For Blueprints, loading
					// the outer class may also invalidate this entry in the export map. In that case, we won't actually be
					// keeping the function object around, so there's no need to warn here about the missing parent object.
					UObject* ObjOuter = IndexToObject(Export.OuterIndex);
					if (ObjOuter && !Export.bExportLoadFailed)
					{
						UClass* FuncClass = Cast<UClass>(ObjOuter);
						if (FuncClass && FuncClass->ClassGeneratedBy && !FuncClass->ClassGeneratedBy->HasAnyFlags(RF_BeingRegenerated))
						{
							// If this is a function (NOT being regenerated) whose parent has been removed, give it a NULL parent, as we would have in the script compiler.
							UE_ASSET_LOG(LogLinker, Display, PackagePath, TEXT("CreateExport: Failed to load Parent for %s; removing parent information, but keeping function"), *GetExportFullName(Index));
						}
					}
#endif

					Export.SuperIndex = FPackageIndex();
				}
				else
				{
#if WITH_EDITORONLY_DATA
					bool bFailedToLoadGeneratedStruct = false;
					if( LoadClass->IsChildOf(UScriptStruct::StaticClass()) )
					{
						// Similar to functions, in the case of structures that are outered to a class (e.g. generated sparse
						// class data), it is also possible to legitimately fail to load the parent structure here as while
						// we will regenerate the structure on load, it wont appear in the export map until it is re-saved
						UObject* ObjOuter = IndexToObject(Export.OuterIndex);
						if (ObjOuter && !Export.bExportLoadFailed)
						{
							UClass* StructClass = Cast<UClass>(ObjOuter);
							if (StructClass && StructClass->ClassGeneratedBy && !StructClass->ClassGeneratedBy->HasAnyFlags(RF_BeingRegenerated))
							{
								UE_ASSET_LOG(LogLinker, Display, PackagePath, TEXT("CreateExport: Failed to load Parent for %s; resaving the parents of %s will remove this message"), *GetExportFullName(Index), *StructClass->ClassGeneratedBy->GetFullName());
								bFailedToLoadGeneratedStruct = true;
							}
						}
					}

					if (!bFailedToLoadGeneratedStruct && !FLinkerLoad::IsKnownMissingPackage(*GetExportFullName(Index)))
					{
						using namespace UE::Private;
						UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("CreateExport: Failed to load %s as Parent for %s - both will fail to load"), *GetPackageObjectFullName(this, Export.SuperIndex), *GetExportFullName(Index));
					}
#endif
					return nullptr;
				}
			}
			else
			{
				// SuperStruct needs to be fully linked so that UStruct::Link will have access to UObject::SuperStruct->PropertySize.
				// There are other attempts to force our super struct to load, and I have not verified that they can all be removed
				// in favor of this one:
				if (!SuperStruct->HasAnyFlags(RF_LoadCompleted)
					&& !SuperStruct->IsNative()
					&& SuperStruct->GetLinker()
					&& Export.SuperIndex.IsImport())
				{
					const UClass* AsClass = dynamic_cast<UClass*>(SuperStruct);
					if (AsClass && !AsClass->GetDefaultObject(false))
					{
						SuperStruct->SetFlags(RF_NeedLoad);
						Preload(SuperStruct);
					}
				}
			}
		}

		// Only UClass objects and FProperty objects of intrinsic classes can have Native flag set. Those property objects are never
		// serialized so we only have to worry about classes. If we encounter an object that is not a class and has Native flag set
		// we warn about it and remove the flag.
		if( (Export.ObjectFlags & RF_MarkAsNative) != 0 && !LoadClass->IsChildOf(UField::StaticClass()) )
		{
			UE_ASSET_LOG(LogLinker, Warning,PackagePath, TEXT("%s %s has RF_MarkAsNative set but is not a UField derived class"),*LoadClass->GetName(),*Export.ObjectName.ToString());
			// Remove RF_MarkAsNative;
			Export.ObjectFlags = EObjectFlags(Export.ObjectFlags & ~RF_MarkAsNative);
		}

		// Find or create the object's Outer.
		UObject* ThisParent = NULL;
		if( !Export.OuterIndex.IsNull() )
		{
			ThisParent = IndexToObject(Export.OuterIndex);
		}
		else if( Export.bForcedExport )
		{
			// Create the forced export in the TopLevel instead of LinkerRoot. Please note that CreatePackage
			// will find and return an existing object if one exists and only create a new one if there doesn't.
			Export.Object = CreatePackage( *Export.ObjectName.ToString() );
			check(Export.Object);
			CurrentLoadContext->IncrementForcedExportCount();
			FLinkerManager::Get().AddLoaderWithForcedExports(this);
		}
		else
		{
			ThisParent = LinkerRoot;
		}

		if (!LoadClass->HasAnyClassFlags(CLASS_Intrinsic) || Cast<ULinkerPlaceholderExportObject>(ThisParent))
		{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			if (LoadClass->HasAnyFlags(RF_NeedLoad))
			{
				Preload(LoadClass);
			}
			else if (Export.Object == nullptr)
			{
				bool const bExportWasDeferred = DeferExportCreation(Index, ThisParent);
				if (bExportWasDeferred)
				{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
					check(Export.Object != nullptr);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
					return Export.Object;
				}
			}
			else if (Cast<ULinkerPlaceholderExportObject>(Export.Object))
			{
				return Export.Object;
			}
#else  // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			Preload(LoadClass);
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

			// Check if the Preload() above caused the class to be regenerated (LoadClass will be out of date), and refresh the LoadClass pointer if that is the case
			if( LoadClass->HasAnyClassFlags(CLASS_NewerVersionExists) )
			{
				if( Export.ClassIndex.IsImport() )
				{
					FObjectImport& ClassImport = Imp(Export.ClassIndex);
					ClassImport.XObject = NULL;
				}

				LoadClass = (UClass*)IndexToObject( Export.ClassIndex );
			}

			if ( LoadClass->HasAnyClassFlags(CLASS_Deprecated) && GIsEditor && !IsRunningCommandlet() && !FApp::IsGame() )
			{
				if ( (Export.ObjectFlags&RF_ClassDefaultObject) == 0 )
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("ObjectName"), FText::FromString(GetExportFullName(Index)));
					Arguments.Add(TEXT("ClassName"), FText::FromString(LoadClass->GetPathName()));
					LoadErrors.Warning(FText::Format(LOCTEXT("LoadedDeprecatedClassInstance", "{ObjectName}: class {ClassName} has been deprecated."), Arguments));
				}
			}
		}

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		// we're going to have troubles if we're attempting to create an export
		// for a placeholder class past this point... placeholder-classes should
		// have generated an export-placeholder in the above
		// !LoadClass->HasAnyClassFlags(CLASS_Intrinsic) block (with the call to
		// DeferExportCreation)
		check(Cast<ULinkerPlaceholderClass>(LoadClass) == nullptr);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

		// detect cases where a class has been made transient when there are existing instances of this class in content packages,
		// and this isn't the class default object; when this happens, it can cause issues which are difficult to debug since they'll
		// only appear much later after this package has been loaded
		if ( LoadClass->HasAnyClassFlags(CLASS_Transient) && (Export.ObjectFlags&RF_ClassDefaultObject) == 0 && (Export.ObjectFlags&RF_ArchetypeObject) == 0 )
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("PackageName"), GetPackagePath().GetDebugNameText());
			Arguments.Add(TEXT("ObjectName"), FText::FromName(Export.ObjectName));
			Arguments.Add(TEXT("ClassName"), FText::FromString(LoadClass->GetPathName()));
			//@todo - should this actually be an assertion?
			LoadErrors.Warning(FText::Format(LOCTEXT("LoadingTransientInstance", "Attempting to load an instance of a transient class from disk - Package:'{PackageName}'  Object:'{ObjectName}'  Class:'{ClassName}'"), Arguments));
		}

		// If loading the object's Outer caused the object to be loaded or if it was a forced export package created
		// above, return it.
		if( Export.Object != nullptr )
		{
			return Export.Object;
		}
		else if (Export.bExportLoadFailed)
		{
			return nullptr;
		}

		// If we should have an outer but it doesn't exist because it was filtered out, we should silently be filtered out too
		if (Export.OuterIndex.IsExport() && ThisParent == nullptr && ExportMap[Export.OuterIndex.ToExport()].bWasFiltered)
		{
			Export.bWasFiltered = true;
			return nullptr;
		}

		// If outer was a redirector or an object that doesn't exist (but wasn't filtered) then log a warning
		UObjectRedirector* ParentRedirector = dynamic_cast<UObjectRedirector*>(ThisParent);
		if( ThisParent == NULL || ParentRedirector)
		{
			// mark this export as unloadable (so that other exports that
			// reference this one won't continue to execute the above logic), then return NULL
			Export.bExportLoadFailed = true;

			// otherwise, return NULL and let the calling code determine what to do
			const FString OuterName = Export.OuterIndex.IsNull() ? LinkerRoot->GetFullName() : GetFullImpExpName(Export.OuterIndex);

			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ObjectName"), FText::FromName(Export.ObjectName));
			Arguments.Add(TEXT("OuterName"), FText::FromString(OuterName));

			if (ParentRedirector)
			{
				LoadErrors.Warning(FText::Format(LOCTEXT("CreateExportFailedToLoadOuterIsRedirector", "CreateExport: Failed to load Outer for resource because it is a redirector '{ObjectName}': {OuterName}"), Arguments));
			}
			else
			{
				LoadErrors.Warning(FText::Format(LOCTEXT("CreateExportFailedToLoadOuter", "CreateExport: Failed to load Outer for resource '{ObjectName}': {OuterName}"), Arguments));
			}

			return nullptr;
		}

		// Find the Archetype object for the one we are loading and ensure it is preloaded since recursively creating exports
		// may find newly created archetype exports that have not been preloaded yet
		UObject* Template = UObject::GetArchetypeFromRequiredInfo(LoadClass, ThisParent, Export.ObjectName, Export.ObjectFlags);
		checkf(Template, TEXT("Failed to get template for class %s. ExportName=%s"), *LoadClass->GetPathName(), *Export.ObjectName.ToString());
		checkfSlow(((Export.ObjectFlags&RF_ClassDefaultObject)!=0 || Template->IsA(LoadClass)), TEXT("Mismatch between template %s and load class %s.  If this is a legacy blueprint or map, it may need to be resaved with bRecompileOnLoad turned off."), *Template->GetPathName(), *LoadClass->GetPathName());
		if (GetLoaderType() == ELoaderType::ZenLoader)
		{
			Preload(Template);
		}

		// we also need to ensure that the template has set up any instances
		Template->ConditionalPostLoadSubobjects();

		// Try to find existing object first in case we're a forced export to be able to reconcile. Also do it for the
		// case of async loading as we cannot in-place replace objects.

		UObject* ActualObjectWithTheName = StaticFindObjectFastInternal(nullptr, ThisParent, Export.ObjectName, EFindObjectFlags::ExactClass);

		// Find object after making sure it isn't already set. This would be bad as the code below NULLs it in a certain
		// case, which if it had been set would cause a linker detach mismatch.
		check(Export.Object == nullptr);
		if (ActualObjectWithTheName && (ActualObjectWithTheName->GetClass() == LoadClass))
		{
			Export.Object = ActualObjectWithTheName;
		}

		// Object is found in memory.
		if (Export.Object)
		{
			// Mark that we need to dissociate forced exports later on if we are a forced export.
			if (Export.bForcedExport)
			{
				CurrentLoadContext->IncrementForcedExportCount();
				FLinkerManager::Get().AddLoaderWithForcedExports(this);
			}
			// Associate linker with object to avoid detachment mismatches.
			else
			{
				Export.Object->SetLinker(this, Index);
				if (Export.OuterIndex.IsImport())
				{
					Export.Object->SetExternalPackage(LinkerRoot);
				}

				// If this object was allocated but never loaded (components created by a constructor) make sure it gets loaded
				// Don't do this for any packages that have previously fully loaded as they may have in memory changes
				CurrentLoadContext->AddLoadedObject(Export.Object);
				if (!Export.Object->HasAnyFlags(RF_LoadCompleted) && (!LinkerRoot->IsFullyLoaded() || IsBlueprintFinalizationPending()))
				{
					if (Export.Object->HasAnyFlags(RF_ClassDefaultObject))
					{
						// Class default objects cannot have PostLoadSubobjects called on them
						Export.Object->SetFlags(RF_NeedLoad | RF_NeedPostLoad | RF_WasLoaded);
					}
					else
					{
						Export.Object->SetFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_WasLoaded);
					}
				}
			}
			return Export.Object;
		}

		// In cases when an object has been consolidated but its package hasn't been saved, look for UObjectRedirector before
		// constructing the object and loading it again from disk (the redirector hasn't been saved yet so it's not part of the package)
#if WITH_EDITOR
		if ( GIsEditor && GIsRunning && !Export.Object )
		{
			UObjectRedirector* Redirector = (UObjectRedirector*)StaticFindObjectFast(UObjectRedirector::StaticClass(), ThisParent, Export.ObjectName, EFindObjectFlags::ExactClass);
			if (Redirector && Redirector->DestinationObject && Redirector->DestinationObject->IsA(LoadClass))
			{
				// A redirector has been found, replace this export with it.
				LoadClass = UObjectRedirector::StaticClass();
				// Create new import for UObjectRedirector class
				FObjectImport& RedirectorImport = ImportMap.Emplace_GetRef(UObjectRedirector::StaticClass());
				CurrentLoadContext->IncrementImportCount();
				FLinkerManager::Get().AddLoaderWithNewImports(this);
				Export.ClassIndex = FPackageIndex::FromImport(ImportMap.Num() - 1);
				Export.Object = Redirector;
				Export.Object->SetLinker( this, Index );
				// Return the redirector. It will be handled properly by the calling code
				return Export.Object;
			}
		}
#endif // WITH_EDITOR

		if (ActualObjectWithTheName && !ActualObjectWithTheName->GetClass()->IsChildOf(LoadClass))
		{
			if(UEVer() >= EUnrealEngineObjectUE5Version::SCRIPT_SERIALIZATION_OFFSET)
			{
				UE_ASSET_LOG(LogLinker, Log, PackagePath, TEXT("Object changed type on load: %s had class %s but is now %s"),
					*ActualObjectWithTheName->GetPathName(), *LoadClass->GetPathName(), *ActualObjectWithTheName->GetClass()->GetPathName());
				// native code created a new object with the same name but a different class, lets load on top of that object:
				Export.Object = ActualObjectWithTheName;
				Export.Object->SetLinker(this, Index);
				Export.Object->SetFlags(RF_NeedLoad|RF_NeedPostLoad|RF_NeedPostLoadSubobjects|RF_WasLoaded);
				CurrentLoadContext->AddLoadedObject(Export.Object);
				return ActualObjectWithTheName;
			}
			else
			{
				UE_ASSET_LOG(LogLinker, Error, PackagePath, TEXT("Failed import: class '%s' name '%s' outer '%s'. There is another object (of '%s' class) at the path."),
					*LoadClass->GetName(), *Export.ObjectName.ToString(), *ThisParent->GetName(), *ActualObjectWithTheName->GetClass()->GetName());
				return NULL;

			}
		}

		// Create the export object, marking it with the appropriate flags to
		// indicate that the object's data still needs to be loaded.
		EObjectFlags ObjectLoadFlags = Export.ObjectFlags;
		// if we are loading objects just to verify an object reference during script compilation,
		if (!GVerifyObjectReferencesOnly
		||	(ObjectLoadFlags&RF_ClassDefaultObject) != 0					// only load this object if it's a class default object
		||	LinkerRoot->HasAnyPackageFlags(PKG_ContainsScript)		// or we're loading an existing package and it's a script package
		||	ThisParent->IsTemplate(RF_ClassDefaultObject)			// or if its a subobject template in a CDO
		||	LoadClass->IsChildOf(UField::StaticClass())				// or if it is a UField
		||	LoadClass->IsChildOf(UObjectRedirector::StaticClass()))	// or if its a redirector to another object
		{
			ObjectLoadFlags = EObjectFlags(ObjectLoadFlags |RF_NeedLoad|RF_NeedPostLoad|RF_NeedPostLoadSubobjects|RF_WasLoaded);
		}

		FName NewName = Export.ObjectName;


		// If we are about to create a CDO, we need to ensure that all parent sub-objects are loaded
		// to get default value initialization to work. This matches code in ResolveDeferredExports
		if ((ObjectLoadFlags & RF_ClassDefaultObject) != 0)
		{
			TArray<UObject*> SubObjects;

			TFunction<void(UClass*)> PreloadSubobjects = [this, &SubObjects, &PreloadSubobjects](UClass* PreloadClass)
			{
				if (PreloadClass == nullptr || PreloadClass->IsNative())
				{
					return;
				}

				PreloadSubobjects(PreloadClass->GetSuperClass());
				SubObjects.Reset();

				GetObjectsWithOuter(PreloadClass->GetDefaultObject(), SubObjects, EGetObjectsFlags::None, /*ExclusionFlags=*/ RF_NoFlags, /*InternalExclusionFlags=*/ EInternalObjectFlags::Native);

				for (UObject* SubObject : SubObjects)
				{
					// Matching behavior in UBlueprint::ForceLoad to ensure that the subobject is actually loaded:
					if (SubObject->HasAnyFlags(RF_WasLoaded) &&
						(SubObject->HasAnyFlags(RF_NeedLoad) || !SubObject->HasAnyFlags(RF_LoadCompleted)))
					{
						SubObject->SetFlags(RF_NeedLoad);
						Preload(SubObject);
					}
				}
			};
			PreloadSubobjects(LoadClass->GetSuperClass());

			// Preload may have already created this object.
			if (Export.Object)
			{
				return Export.Object;
			}
		}

		// Initial saves of TRACK_OBJECT_EXPORT_IS_INHERITED incorrectly considered Blueprint-added component archetypes
		// and subobjects instanced from those archetypes as inherited instances. The flag itself is intended to denote an
		// instanced default subobject that's based on an archetype contained within the owner's archetype's set of instanced
		// default subobjects; that is, the subobject owner's archetype is expected to also contain a matching default subobject
		// instance with the same type/name. However, if the instanced subobject is based on an archetype that's owned by
		// something other than it's owner's archetype (e.g. Blueprint-added component archetypes, which are owned by the
		// Blueprint class object), such a match would not exist.
		if (Export.bIsInheritedInstance && Template->GetOuter()->IsA<UClass>())
		{
			Export.bIsInheritedInstance = false;
		}

		LoadClass->GetDefaultObject();

		FStaticConstructObjectParameters Params(LoadClass);
		Params.Outer = ThisParent;
		Params.Name = NewName;
		Params.SetFlags = ObjectLoadFlags;
		Params.Template = Template;
		// if our outer is actually an import, then the package we are an export of is not in our outer chain, set our package in that case
		Params.ExternalPackage = Export.OuterIndex.IsImport() ? LinkerRoot : nullptr;

		// Propagate relevant properties from the outer package to the external package
		if (Params.ExternalPackage)
		{
			Params.ExternalPackage->SetPackageFlags(ThisParent->GetPackage()->GetPackageFlags() & PKG_PlayInEditor);
			Params.ExternalPackage->SetPIEInstanceID(ThisParent->GetPackage()->GetPIEInstanceID());
		}

		{
			TRACE_LOADTIME_CREATE_EXPORT_SCOPE(this, &Export.Object);
			Export.Object = StaticConstructObject_Internal(Params);

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
			//if lazy load is enabled construct a packed ref if possible.
			//this is to have a reverse map of UObject to FPackedObjectRef
			if (UE::LinkerLoad::IsImportLazyLoadEnabled())
			{
				UE::CoreUObject::Private::MakePackedObjectRef(Export.Object);
			}
#endif
		}

		if (FPlatformProperties::RequiresCookedData())
		{
			if (GIsInitialLoad || GUObjectArray.IsOpenForDisregardForGC())
			{
				Export.Object->AddToRoot();
			}
		}

		LoadClass = Export.Object->GetClass(); // this may have changed if we are overwriting a CDO component

		if (NewName != Export.ObjectName)
		{
			// create a UObjectRedirector with the same name as the old object we are redirecting
			UObjectRedirector* Redir = NewObject<UObjectRedirector>(Export.Object->GetOuter(), Export.ObjectName, RF_Standalone | RF_Public);
			// point the redirector object to this object
			Redir->DestinationObject = Export.Object;
		}

		if( Export.Object )
		{
			const bool bIsBlueprintCDO = ((Export.ObjectFlags & RF_ClassDefaultObject) != 0) && LoadClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint) &&
				LoadClass->GetClass()->HasAnyClassFlags(CLASS_NeedsDeferredDependencyLoading);

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			const bool bDeferCDOSerialization = bIsBlueprintCDO && ((LoadFlags & LOAD_DeferDependencyLoads) != 0);
			if (bDeferCDOSerialization)
			{
				// if LOAD_DeferDependencyLoads is set, then we're already
				// serializing the blueprint's class somewhere up the chain...
				// we don't want the class regenerated while it in the middle of
				// serializing
				DeferredCDOIndex = Index;
				return Export.Object;
			}
			else if (bIsBlueprintCDO && IsBlueprintFinalizationPending())
			{
				// Class regeneration is deferred until Blueprint finalization, so just return the CDO.
				return Export.Object;
			}
			else
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			// Check to see if LoadClass is a blueprint, which potentialLy needs
			// to be refreshed and regenerated.  If so, regenerate and patch it
			// back into the export table
#if WITH_EDITOR
			// Allow cooked Blueprint classes to take the same regeneration code path in the editor context.
			if (bIsBlueprintCDO && (LoadClass->GetOutermost() != GetTransientPackage()))
#else
			if (!LoadClass->bCooked && bIsBlueprintCDO && (LoadClass->GetOutermost() != GetTransientPackage()))
#endif
			{
				{
					// For classes that are about to be regenerated, make sure we register them with the linker, so future references to this linker index will be valid
					const EObjectFlags OldFlags = Export.Object->GetFlags();
					Export.Object->ClearFlags(RF_NeedLoad|RF_NeedPostLoad|RF_NeedPostLoadSubobjects);
					Export.Object->SetLinker( this, Index, false );
					Export.Object->SetFlags(OldFlags);
				}

				if ( RegenerateBlueprintClass(LoadClass, Export.Object) )
				{
					return Export.Object;
				}
			}
			else
			{
				// we created the object, but the data stored on disk for this object has not yet been loaded,
				// so add the object to the list of objects that need to be loaded, which will be processed
				// in EndLoad()
				Export.Object->SetLinker( this, Index );
				CurrentLoadContext->AddLoadedObject(Export.Object);
			}
		}
		else
		{
			UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("FLinker::CreatedExport failed to construct object %s %s"), *LoadClass->GetName(), *Export.ObjectName.ToString() );
		}

		if ( Export.Object != NULL )
		{
			// If it's a struct or class, set its parent.
			if( UStruct* Struct = dynamic_cast<UStruct*>(Export.Object) )
			{
				if ( !Export.SuperIndex.IsNull() )
				{
					UStruct* SuperStruct = (UStruct*)IndexToObject(Export.SuperIndex);
					if (ULinkerPlaceholderFunction* Function = Cast<ULinkerPlaceholderFunction>(SuperStruct))
					{
						Function->AddDerivedFunction(Struct);
					}
					else
					{
						Struct->SetSuperStruct((UStruct*)IndexToObject(Export.SuperIndex));
					}
				}

				// If it's a class, bind it to C++.
				if (UClass* ClassObject = Cast<UClass>(Export.Object))
				{
					if (ClassObject->GetClass()->HasAnyClassFlags(CLASS_NeedsDeferredDependencyLoading))
					{
#if WITH_EDITOR
						// Before we serialize the class, begin a scoped class
						// dependency gather to create a list of other classes that
						// may need to be recompiled
						//
						// Even with "deferred dependency loading" turned on, we
						// still need this... one class/blueprint will always be
						// fully regenerated before another (there is no changing
						// that); so dependencies need to be recompiled later (with
						// all the regenerated classes in place)
						FScopedClassDependencyGather DependencyHelper(ClassObject, CurrentLoadContext);
#endif //WITH_EDITOR

						ClassObject->Bind();

						// Preload classes on first access.  Note that this may update the Export.Object, so ClassObject is not guaranteed to be valid after this point
						// If we're async loading on a cooked build we can skip this as there's no chance we will need to recompile the class.
						// Preload will be called during async package tick when the data has been precached
						if (!FPlatformProperties::RequiresCookedData())
						{
							Preload(Export.Object);
						}
					}
					else
					{
						ClassObject->Bind();
					}
				}
			}

			// Mark that we need to dissociate forced exports later on.
			if( Export.bForcedExport )
			{
				CurrentLoadContext->IncrementForcedExportCount();
				FLinkerManager::Get().AddLoaderWithForcedExports(this);
			}

#if WITH_EDITOR && !UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObject(Export.Object))
			{
				UE_ASSET_LOG(LogLinker, Log, PackagePath, TEXT("Export %s created with placeholder type: %s"), *Export.Object->GetName(), *LoadClass->GetFullName());
			}
#endif
		}
	}
	return Export.bExportLoadFailed ? nullptr : Export.Object;
}

bool FLinkerLoad::IsImportNative(const int32 Index) const
{
	const FObjectImport& Import = ImportMap[Index];

	bool bIsImportNative = false;
	// if this import has a linker, then it belongs to some (non-native) asset package
	if (Import.SourceLinker == nullptr)
	{
		if (!Import.OuterIndex.IsNull())
		{
			// need to check the package that this import belongs to, so recurse
			// up then import's outer chain
			bIsImportNative = IsImportNative(Import.OuterIndex.ToImport());
		}
		else if (UPackage* ExistingPackage = FindObject<UPackage>(/*Outer =*/nullptr, *Import.ObjectName.ToString()))
		{
			// @TODO: what if the package's outer isn't null... what does that mean?
			bIsImportNative = !ExistingPackage->GetOuter() && ExistingPackage->HasAnyPackageFlags(PKG_CompiledIn);
		}
	}

	return bIsImportNative;
}

// Return the loaded object corresponding to an import index; any errors are fatal.
UObject* FLinkerLoad::CreateImport( int32 Index )
{
	FScopedCreateImportCounter ScopedCounter( this, Index );
	FObjectImport& Import = ImportMap[ Index ];

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	// if this Import could possibly introduce a circular load (and we're
	// actively trying to avoid that at this point in the load process), then
	// this wiLl stub in the Import with a placeholder object, to be replace
	// later on (this will return true if the import was actually deferred)
	DeferPotentialCircularImport(Index);
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	if (Import.XObject != nullptr && Import.XObject->IsUnreachable())
	{
		// This is just a safeguard to catch potential bugs that should have been fixed by calling UnhashUnreachableObjects in Async Loading code
		UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Unreachable object found when creating import %s"), *Import.XObject->GetFullName());
		Import.XObject = nullptr;
	}

	// Imports can have no name if they were filtered out due to package redirects, skip in that case
	if (Import.XObject == nullptr && Import.ObjectName != NAME_None)
	{
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
		CreatePlaceholderImport(Import);
#endif // UE_INTERNAL_UNIVERSAL_PLACEHOLDERS

		if( Import.XObject == NULL )
		{
			EVerifyResult VerifyImportResult = VERIFY_Success;
			if( Import.SourceLinker == NULL )
			{
				VerifyImportResult = VerifyImport(Index);
			}
			if(Import.SourceIndex != INDEX_NONE)
			{
				check(Import.SourceLinker);
				// VerifyImport may have already created the import and SourceIndex has changed to point to the actual redirected object.
				// This can only happen in non-cooked builds since cooked builds don't have redirects and other cases are valid.
				// We also don't want to call CreateExport only when there was an actual redirector involved.
				if (FPlatformProperties::RequiresCookedData() || !Import.XObject || VerifyImportResult != VERIFY_Redirected)
				{
					Import.XObject = Import.SourceLinker->CreateExport(Import.SourceIndex);
				}
				// If an object has been replaced (consolidated) in the editor and its package hasn't been saved yet
				// it's possible to get UbjectRedirector here as the original export is dynamically replaced
				// with the redirector (the original object has been deleted but the data on disk hasn't been updated)
#if WITH_EDITOR
				if( GIsEditor )
				{
					UObjectRedirector* Redirector = dynamic_cast<UObjectRedirector*>(Import.XObject);
					if( Redirector )
					{
						// It may happen that the redirector is already being deserialized on the stack (i.e RF_LoadCompleted isn't set)
						// but RF_NeedLoad has been removed already. We need to reresolve in that case right away
						// otherwise the DestinationObject wouldn't be set until we unwind the stack and finish
						// the deserialization, which may be too late.
						// If the redirector is now in phase2, it already has been through serialization
						// and we don't want to risk racing with its PostLoad by forcing its reload needlessly.
						if (!Redirector->HasAnyFlags(RF_NeedLoad|RF_LoadCompleted) && !Redirector->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase2))
						{
							// Set the flag back if missing and preload hasn't completed yet so that
							// the preload we're going to run does something.
							Redirector->SetFlags(RF_NeedLoad);
						}
						Preload(Redirector);
						Import.XObject = Redirector->DestinationObject;
					}
				}
#endif
				FUObjectSerializeContext* CurrentLoadContext = FUObjectThreadContext::Get().GetSerializeContext();
				CurrentLoadContext->IncrementImportCount();
				FLinkerManager::Get().AddLoaderWithNewImports(this);
			}
		}

		if (Import.XObject == nullptr)
		{
			const FString OuterName = Import.OuterIndex.IsNull() ? LinkerRoot->GetFullName() : GetFullImpExpName(Import.OuterIndex);
			UE_LOGF(LogLinker, Verbose, "Failed to resolve import '%d' named '%ls' in '%ls'", Index, *Import.ObjectName.ToString(), *OuterName);
		}
		else if (ImportsToVerifyOnCreate.Contains(Index))
		{
			const UClass* ExpectedImportClass = nullptr;
			if (UPackage* ImportClassPackage = FindObjectFast<UPackage>(nullptr, InstancingContext.RemapPackage(Import.ClassPackage), EFindObjectFlags::None))
			{
				const UObject* FoundObject = FindObjectFast<UObject>(ImportClassPackage, Import.ClassName, EFindObjectFlags::None);
				ExpectedImportClass = Cast<UClass>(FoundObject);
				if (!ExpectedImportClass)
				{
					if (const UObjectRedirector* FoundRedirector = Cast<UObjectRedirector>(FoundObject))
					{
						ExpectedImportClass = Cast<UClass>(FoundRedirector->DestinationObject);
					}
				}
			}

			// Verify that the resolved import object's class is serialization-compatible with the expected result. Data loss will otherwise occur on load if this is not satisfied, so we warn about it. A re-save is required to fix up the import table and suppress this warning.
			if (!ExpectedImportClass || !Import.XObject->GetClass()->IsChildOf(ExpectedImportClass))
			{
				UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Resolved import with name '%s' from '%s' with a different class: import class '%s.%s', package class '%s.%s'. Resave to fix."),
								*Import.ObjectName.ToString(),
								*Import.SourceLinker->GetPackagePath().GetPackageName(),
								*Import.ClassPackage.ToString(), *Import.ClassName.ToString(),
								*Import.SourceLinker->GetExportClassPackage(Import.SourceIndex).ToString(), *Import.SourceLinker->GetExportClassName(Import.SourceIndex).ToString());
			}

			ImportsToVerifyOnCreate.Remove(Index);
		}
	}
	return Import.XObject;
}



// Map an import/export index to an object; all errors here are fatal.
UObject* FLinkerLoad::IndexToObject( FPackageIndex Index )
{
	if( Index.IsExport() )
	{
		#if PLATFORM_DESKTOP
			// Show a message box indicating, possible, corrupt data (desktop platforms only)
			if ( !ExportMap.IsValidIndex( Index.ToExport() ) && !FApp::IsUnattended() )
			{
				FText ErrorMessage, ErrorCaption;
				GConfig->GetText(TEXT("/Script/Engine.Engine"),
									TEXT("SerializationOutOfBoundsErrorMessage"),
									ErrorMessage,
									GEngineIni);
				GConfig->GetText(TEXT("/Script/Engine.Engine"),
					TEXT("SerializationOutOfBoundsErrorMessageCaption"),
					ErrorCaption,
					GEngineIni);

				UE_ASSET_LOG(LogLinker, Error, PackagePath, TEXT("Invalid export object index=%d. File is most likely corrupted. Please verify your installation."), Index.ToExport());

				if (GLog)
				{
					GLog->Flush();
				}

				FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ErrorMessage.ToString(), *ErrorCaption.ToString());

				check(false);
			}
		#else
			{
				UE_CLOGF( !ExportMap.IsValidIndex( Index.ToExport() ), LogLinker, Fatal, "Invalid export object index=%d while reading %ls. File is most likely corrupted. Please verify your installation.", Index.ToExport(), *GetDebugName() );
			}
		#endif

		return CreateExport( Index.ToExport() );
	}
	else if( Index.IsImport() )
	{
		#if PLATFORM_DESKTOP
			// Show a message box indicating, possible, corrupt data (desktop platforms only)
			if ( !ImportMap.IsValidIndex( Index.ToImport() ) && !FApp::IsUnattended() )
			{
				FText ErrorMessage, ErrorCaption;
				GConfig->GetText(TEXT("/Script/Engine.Engine"),
									TEXT("SerializationOutOfBoundsErrorMessage"),
									ErrorMessage,
									GEngineIni);
				GConfig->GetText(TEXT("/Script/Engine.Engine"),
					TEXT("SerializationOutOfBoundsErrorMessageCaption"),
					ErrorCaption,
					GEngineIni);

				UE_ASSET_LOG(LogLinker, Error, PackagePath, TEXT("Invalid import object index=%d. File is most likely corrupted. Please verify your installation."), Index.ToImport());

				if (GLog)
				{
					GLog->Flush();
				}

				FPlatformMisc::MessageBoxExt(EAppMsgType::Ok, *ErrorMessage.ToString(), *ErrorCaption.ToString());

				check(false);
			}
		#else
			{
				UE_CLOGF( !ImportMap.IsValidIndex( Index.ToImport() ), LogLinker, Fatal, "Invalid import object index=%d while reading %ls. File is most likely corrupted. Please verify your installation.", Index.ToImport(), *GetDebugName() );
			}
		#endif

		return CreateImport( Index.ToImport() );
	}
	else
	{
		return nullptr;
	}
}



// Detach an export from this linker.
void FLinkerLoad::DetachExport( int32 i )
{
	FObjectExport& Export = ExportMap[ i ];
	check(Export.Object);
	if( !Export.Object->IsValidLowLevel() )
	{
		UE_ASSET_LOG(LogLinker, Fatal, PackagePath, TEXT("Linker object %s %s is invalid"), *GetExportClassName(i).ToString(), *Export.ObjectName.ToString());
	}
	{
		const FLinkerLoad* ActualLinker = Export.Object->GetLinker();
		if (ActualLinker != this)
		{
			UObject* Object = Export.Object;
			UE_LOGF(LogLinker, Log, "Object            : %ls", *Object->GetFullName());
			//UE_LOGF(LogLinker, Log, "Object Linker     : %ls", *Object->GetLinker()->GetFullName() );
			UE_LOGF(LogLinker, Log, "Linker LinkerRoot : %ls", Object->GetLinker() ? *Object->GetLinker()->LinkerRoot->GetFullName() : TEXT("None"));
			//UE_LOGF(LogLinker, Log, "Detach Linker     : %ls", *GetFullName() );
			UE_LOGF(LogLinker, Log, "Detach LinkerRoot : %ls", *LinkerRoot->GetFullName());
			UE_ASSET_LOG(LogLinker, Fatal, PackagePath, TEXT("Linker object %s %s mislinked!"), *GetExportClassName(i).ToString(), *Export.ObjectName.ToString());
		}
	}

	if (Export.Object->GetLinkerIndex() == -1)
	{
		UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Linker object %s %s was already detached."), *GetExportClassName(i).ToString(), *Export.ObjectName.ToString());
	}
	else
	{
		checkf(Export.Object->GetLinkerIndex() == i, TEXT("Mismatched linker index in FLinkerLoad::DetachExport for %s in %s. Linker index was supposed to be %d, was %d"), *GetExportClassName(i).ToString(), *LinkerRoot->GetName(), i, Export.Object->GetLinkerIndex());
	}
	Export.Object->SetLinker(nullptr, INDEX_NONE);
}

void FLinkerLoad::LoadAndDetachAllBulkData()
{
#if WITH_EDITOR
	// Detach all lazy loaders.
	const bool bEnsureAllBulkDataIsLoaded = true;
	DetachAllBulkData(bEnsureAllBulkDataIsLoaded);
#endif
}

void FLinkerLoad::DestroyLoader()
{
	check(!bIsDestroyingLoader); // Destroying loader recursively is not safe
	bIsDestroyingLoader = true; // Some archives check for this to make sure they're not destroyed by random code
	FPlatformMisc::MemoryBarrier();
	if (Loader)
	{
		delete Loader;
		Loader = nullptr;
	}
	bIsDestroyingLoader = false;
}

void FLinkerLoad::DetachLoader()
{
#if WITH_EDITOR
	DetachAllBulkData(true);
#endif // WITH_EDITOR

	DestroyLoader();

#if WITH_EDITOR
	bDetachedLoader = true;
#endif // WITH_EDITOR
}

void FLinkerLoad::DetachExports()
{
	// Detach all objects linked with this linker.
	for (int32 ExportIndex = 0; ExportIndex < ExportMap.Num(); ++ExportIndex)
	{
		if (ExportMap[ExportIndex].Object)
		{
			DetachExport(ExportIndex);
		}
	}
}

void FLinkerLoad::Detach()
{
#if WITH_EDITOR
	// Detach all lazy loaders.
	const bool bEnsureAllBulkDataIsLoaded = false;
	DetachAllBulkData(bEnsureAllBulkDataIsLoaded);
#endif

	// Detach all objects linked with this linker.
	DetachExports();

	// Remove from object manager, if it has been added.
	FLinkerManager::Get().RemoveLoaderFromObjectLoadersAndLoadersWithNewImports(this);
	if (!FPlatformProperties::HasEditorOnlyData())
	{
		FUObjectSerializeContext* CurrentLoadContext = FUObjectThreadContext::Get().GetSerializeContext();
		CurrentLoadContext->RemoveDelayedLinkerClosePackage(this);
	}

	delete StructuredArchive;
	StructuredArchive = nullptr;
	for (FStructuredArchiveChildReader* Reader : ExportReaders)
	{
		delete Reader;
	}
	ExportReaders.Empty();
	delete StructuredArchiveFormatter;
	StructuredArchiveFormatter = nullptr;

	DestroyLoader();

	// Empty out no longer used arrays.
	NameMap.Empty();
	GatherableTextDataMap.Empty();
	ImportMap.Empty();
	ExportMap.Empty();

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	ResetDeferredLoadingState();
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	// Make sure we're never associated with LinkerRoot again.
	if (LinkerRoot)
	{
		LinkerRoot->SetLinker(nullptr);
		// When detaching the linker from its package, also empty its stored list of custom versions.
		// This is so that object post loaded in the editor in a package that has no associated linker consider that all package custom versions as latest
		// (i.e. when duplicating an object in the package)
		// The runtime *may* use the stored version in the package since there are never any linker associated with it when using iostore
		LinkerRoot->EmptyLinkerCustomVersion();
		LinkerRoot = nullptr;
	}

	UE_CLOGF(AsyncRoot != nullptr, LogStreaming, Error, "AsyncRoot still associated with Linker");
}

#if WITH_EDITOR

void FLinkerLoad::AttachBulkData( UObject* Owner, FBulkData* BulkData )
{
	UE::TUniqueLock _(BulkDataMutex);

	bool bAlreadyInSet = false;
	BulkDataLoaders.Add(BulkData, &bAlreadyInSet);
	check(!bAlreadyInSet);
}

void FLinkerLoad::AttachBulkData(UE::Serialization::FEditorBulkData* BulkData)
{
	UE::TUniqueLock _(BulkDataMutex);

	bool bAlreadyInSet = false;
	EditorBulkDataLoaders.Add(BulkData, &bAlreadyInSet);
	check(!bAlreadyInSet);
}

void FLinkerLoad::DetachBulkData( FBulkData* BulkData, bool bEnsureBulkDataIsLoaded )
{
	UE::TUniqueLock _(BulkDataMutex);

	const int32 RemovedCount = BulkDataLoaders.Remove( BulkData );
	if (RemovedCount!= 1)
	{
		UE_ASSET_LOG(LogLinker, Fatal, PackagePath, TEXT("Detachment inconsistency: %i"), RemovedCount);
	}

	BulkData->DetachFromArchive( this, bEnsureBulkDataIsLoaded );
}

void FLinkerLoad::DetachBulkData(UE::Serialization::FEditorBulkData* BulkData, bool bEnsureBulkDataIsLoaded)
{
	UE::TUniqueLock _(BulkDataMutex);

	const int32 RemovedCount = EditorBulkDataLoaders.Remove(BulkData);
	if (RemovedCount != 1)
	{
		UE_ASSET_LOG(LogLinker, Fatal, PackagePath, TEXT("Detachment inconsistency: %i"), RemovedCount);
	}

	BulkData->DetachFromDisk(this, bEnsureBulkDataIsLoaded);
}

void FLinkerLoad::DetachAllBulkData(bool bEnsureAllBulkDataIsLoaded)
{
	UE::TUniqueLock _(BulkDataMutex);

	for (FBulkData* BulkData : BulkDataLoaders)
	{
		BulkData->DetachFromArchive(this, bEnsureAllBulkDataIsLoaded);
	}

	BulkDataLoaders.Empty();

	for (UE::Serialization::FEditorBulkData* BulkData : EditorBulkDataLoaders)
	{
		BulkData->DetachFromDisk(this, bEnsureAllBulkDataIsLoaded);
	}

	EditorBulkDataLoaders.Empty();
}

#endif // WITH_EDITOR

FArchive& FLinkerLoad::operator<<( UObject*& Object )
{
	FPackageIndex Index;
	FArchive& Ar = *this;
	Ar << Index;

	Object = ResolveResource(Index);
	// UE_INTERNAL_UNIVERSAL_PLACEHOLDERS disables this because some native facilities like ObjectRedirectors
	// do not provide a ObjectProperty and we want to allow them to resolve objects:
#if WITH_EDITOR && !UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	if (Object && UE::FPropertyBagRepository::IsPropertyBagPlaceholderObject(Object))
	{
		if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObjectFeatureEnabled(UE::FPropertyBagRepository::EPlaceholderObjectFeature::SerializeExportReferencesOnLoad))
		{
			const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(GetSerializedProperty());
			if (!ObjectProperty || !Object->GetClass()->IsChildOf(ObjectProperty->PropertyClass))
			{
				// This is needed because the pointer's type is checked only at compile time, which may not match the property
				// bag placeholder object's type at runtime, and so we can't allow it to be dereferenced as the wrong base type.
				// Note: These currently won't be discovered for replacement at reinstancing time, so it will remain set to NULL.
				UE_LOGF(LogLinker, Warning, "Serializing reference to \"%ls\" as NULL to ensure type safety.", *Object->GetPathName());
				Object = nullptr;
			}
		}
		else
		{
			UE_LOGF(LogLinker, Log, "Serializing reference to \"%ls\" as NULL because it has a placeholder type, and reference loading is currently disabled.", *Object->GetPathName());
			Object = nullptr;
		}
	}
#endif
	return *this;
}

FArchive& FLinkerLoad::operator<<(FObjectPtr& ObjectPtr)
{
	FPackageIndex Index;
	FArchive& Ar = *this;
	Ar << Index;

	// Wrapper that only allows pointers to exports with placeholder types when type safety features are enabled.
	auto AsTypeSafeObjectPtr_Lambda = [this](UObject* ResolvedObject)
	{
#if WITH_EDITOR
		// Note: References to placeholder objects cannot resolve to it if the underlying pointer type is unsafe.
		if (ResolvedObject && UE::FPropertyBagRepository::IsPropertyBagPlaceholderObject(ResolvedObject))
		{
			if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObjectFeatureEnabled(UE::FPropertyBagRepository::EPlaceholderObjectFeature::SerializeExportReferencesOnLoad))
			{
				const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(GetSerializedProperty());
				if (!ObjectProperty || !ResolvedObject->GetClass()->IsChildOf(ObjectProperty->PropertyClass))
				{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE && UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
					// If type safety features are enabled, create a packed reference mapping for the placeholder-typed object.
					// This resolves to NULL on access since the reference can't be cast to a pointer bound to its original type.
					// However, the underlying value (when serialized) will always resolve to the placeholder object (e.g. for GC).
					return FObjectPtr(FObjectHandle{ UE::CoreUObject::Private::MakePackedObjectRef(ResolvedObject).EncodedRef });
#elif UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
					// Type hierarchies can be incomplete in UE_INTERNAL_UNIVERSAL_PLACEHOLDERS environments, 
					// but unless there's native code using the object we can allow the reference to exist:
					return FObjectPtr(ResolvedObject); 
#else
					// If type safety features are disabled, serialize it as an unsafe reference to a placeholder-typed object.
					// Note: Similar to hard references above, this means we won't find it for replacement at reinstancing time.
					UE_LOGF(LogLinker, Warning, "Serializing reference to \"%ls\" as NULL to ensure type safety. This will lead to data loss if the referencing object is saved.", *ResolvedObject->GetPathName());
					ResolvedObject = nullptr;
#endif
				}
			}
			else
			{
				UE_LOGF(LogLinker, Log, "Serializing reference to \"%ls\" as NULL because it has a placeholder type, and reference loading is currently disabled.", *ResolvedObject->GetPathName());
				ResolvedObject = nullptr;
			}
		}
#endif
		return FObjectPtr(ResolvedObject);
	};

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr();

	if (!Index.IsImport() || !AssetRegistry)
	{
		ObjectPtr = AsTypeSafeObjectPtr_Lambda(ResolveResource(Index));
	}
	else
	{
		using namespace UE::LinkerLoad;
		FObjectImport& Import = Imp(Index);
		if (!TryLazyImport(*AssetRegistry, Import, *this, ObjectPtr))
		{
			ObjectPtr = AsTypeSafeObjectPtr_Lambda(ResolveResource(Index));
		}
	}
#else
	ObjectPtr = AsTypeSafeObjectPtr_Lambda(ResolveResource(Index));
#endif

	return *this;
}

FArchive& FLinkerLoad::operator<<(FSoftObjectPath& Value)
{
	FArchive& Ar = *this;
	// if we have items in the soft object path list consider soft object path saved as index into it.
	// Otherwise deserialize them as usual, cooking for example might not want soft object path serialized as index
	if (SoftObjectPathList.Num() > 0)
	{
		int32 SoftObjectPathIndex = INDEX_NONE;
		Ar << SoftObjectPathIndex;
		if (SoftObjectPathList.IsValidIndex(SoftObjectPathIndex))
		{
			Value = SoftObjectPathList[SoftObjectPathIndex];

#if WITH_EDITOR
			Value.PostLoadPath(this);
#endif // WITH_EDITOR
		}
		else
		{
			Value = FSoftObjectPath();
			BadSoftObjectPathError(SoftObjectPathIndex);
			SetCriticalError();
		}
	}
	else
	{
		FArchiveUObject::operator<<(Value);
		FixupSoftObjectPathForInstancedPackage(Value);
	}
	return Ar;
}

void FLinkerLoad::BadSoftObjectPathError(int32 SoftObjIndex)
{
	UE_ASSET_LOG(LogLinker, Error, PackagePath,
		TEXT("Serialization error - FSoftObjectPath are serialized as an index recorded in the package header, the current deserialized index has value %i, outside of the valid range [0, %i)."),
		SoftObjIndex, SoftObjectPathList.Num());
}

void FLinkerLoad::BadNameIndexError(int32 NameIndex)
{
	UE_ASSET_LOG(LogLinker, Error, PackagePath,
		TEXT("Serialization error - FName are serialized as an index recorded in the package header, the current deserialized index has value %i, outside of the valid range [0, %i)."),
		NameIndex, NameMap.Num());
}

/**
 * Called when an object begins serializing property data using script serialization.
 */
void FLinkerLoad::MarkScriptSerializationStart( const UObject* Obj )
{
#if WITH_EDITOR
	bIsSerializingScriptProperties = true;
#endif
	if (Obj && Obj->GetLinker() == this)
	{
		int32 Index = Obj->GetLinkerIndex();
		if (ExportMap.IsValidIndex(Index))
		{
			FObjectExport& Export = ExportMap[Index];
			const int64 RelativeSerialOffset = Tell() - Export.SerialOffset;
			if (!UseUnversionedPropertySerialization() && UEVer() >= EUnrealEngineObjectUE5Version::SCRIPT_SERIALIZATION_OFFSET)
			{
				checkf(Export.ScriptSerializationStartOffset == RelativeSerialOffset,
					TEXT("Serialized script property start offset %" INT64_FMT " does not match offset during deserialization %" INT64_FMT " for object %s in %s."),
					Export.ScriptSerializationStartOffset, RelativeSerialOffset, *Export.ObjectName.ToString(), *LinkerRoot->GetName());
			}
			else
			{
				Export.ScriptSerializationStartOffset = RelativeSerialOffset;
			}
		}
	}
}

/**
 * Called when an object stops serializing property data using script serialization.
 */
void FLinkerLoad::MarkScriptSerializationEnd( const UObject* Obj )
{
#if WITH_EDITOR
	bIsSerializingScriptProperties = false;
#endif
	if (Obj && Obj->GetLinker() == this)
	{
		int32 Index = Obj->GetLinkerIndex();
		if (ExportMap.IsValidIndex(Index))
		{
			FObjectExport& Export = ExportMap[Index];
			const int64 RelativeSerialOffset = Tell() - Export.SerialOffset;
			if (!UseUnversionedPropertySerialization() && UEVer() >= EUnrealEngineObjectUE5Version::SCRIPT_SERIALIZATION_OFFSET)
			{
				checkf(Export.ScriptSerializationEndOffset == RelativeSerialOffset,
					TEXT("Serialized script property end offset %" INT64_FMT " does not match offset during deserialization %" INT64_FMT " for object %s in %s."),
					Export.ScriptSerializationEndOffset, RelativeSerialOffset, *Export.ObjectName.ToString(), *LinkerRoot->GetName());
			}
			else
			{
				Export.ScriptSerializationEndOffset = RelativeSerialOffset;
			}
		}
	}
}

bool FLinkerLoad::FindImportPackage(FName PackageName, FPackageIndex& PackageIdx)
{
	for (int32 ImportMapIdx = 0; ImportMapIdx < ImportMap.Num(); ImportMapIdx++)
	{
		if (ImportMap[ImportMapIdx].ObjectName == PackageName && ImportMap[ImportMapIdx].ClassName == NAME_Package)
		{
			PackageIdx = FPackageIndex::FromImport(ImportMapIdx);
			return true;
		}
	}

	return false;
}

bool FLinkerLoad::FindImport(FPackageIndex OuterIndex, FName ObjectName, FPackageIndex& OutObjectIndex)
{
	for (int32 ImportMapIdx = 0; ImportMapIdx < ImportMap.Num(); ImportMapIdx++)
	{
		if (ImportMap[ImportMapIdx].ObjectName == ObjectName && ImportMap[ImportMapIdx].OuterIndex == OuterIndex)
		{
			OutObjectIndex = FPackageIndex::FromImport(ImportMapIdx);
			return true;
		}
	}

	OutObjectIndex = FPackageIndex();
	return false;
}

bool FLinkerLoad::FindImport(FStringView FullObjectPath, FPackageIndex& OutObjectIndex)
{
	bool bIsValid = false;

	FStringView ClassName;
	FStringView PackageName;
	FStringView ObjectName;
	TArray<FStringView> SubobjectNames;

	FPackageName::SplitFullObjectPath(FullObjectPath, ClassName, PackageName, ObjectName, SubobjectNames);

	FName PackageFName(PackageName);
	FName ObjectFName(ObjectName);

	FPackageIndex PackageIndex;

	bIsValid = FindImportPackage(PackageFName, PackageIndex);

	if (bIsValid)
	{
		const bool bHasRootObject = !ObjectFName.IsNone();

		if (bHasRootObject)
		{
			FPackageIndex ObjectIndex;

			bIsValid = FindImport(PackageIndex, ObjectFName, ObjectIndex);

			if (bIsValid)
			{
				const bool bHasSubobjects = (SubobjectNames.Num() > 0);

				if (bHasSubobjects)
				{
					FPackageIndex CurrentOuterIndex = ObjectIndex;
					FPackageIndex SubobjectIndex;

					for (FStringView SubobjectName : SubobjectNames)
					{
						if (FindImport(CurrentOuterIndex, FName(SubobjectName), SubobjectIndex))
						{
							CurrentOuterIndex = SubobjectIndex;
						}
						else
						{
							bIsValid = false;
							break;
						}
					}

					if (bIsValid)
					{
						OutObjectIndex = SubobjectIndex;
					}
				}
				else
				{
					OutObjectIndex = ObjectIndex;
				}
			}
		}
		else
		{
			OutObjectIndex = PackageIndex;
		}
	}

	return bIsValid;
}

/**
 * Locates the class adjusted index and its package adjusted index for a given class name in the import map
 */
bool FLinkerLoad::FindImportClassAndPackage( FName ClassName, FPackageIndex &ClassIdx, FPackageIndex &PackageIdx )
{
	for ( int32 ImportMapIdx = 0; ImportMapIdx < ImportMap.Num(); ImportMapIdx++ )
	{
		if ( ImportMap[ImportMapIdx].ObjectName == ClassName && ImportMap[ImportMapIdx].ClassName == NAME_Class )
		{
			ClassIdx = FPackageIndex::FromImport(ImportMapIdx);
			PackageIdx = ImportMap[ImportMapIdx].OuterIndex;
			return true;
		}
	}

	return false;
}


UObject* FLinkerLoad::GetArchetypeFromLoader(const UObject* Obj)
{
	return FArchiveUObject::GetArchetypeFromLoader(Obj);
}


/**
* Attempts to find the index for the given class object in the import list and adds it + its package if it does not exist
*/
bool FLinkerLoad::CreateImportClassAndPackage( FName ClassName, FName PackageName, FPackageIndex &ClassIdx, FPackageIndex &PackageIdx )
{
	//look for an existing import first
	//might as well look for the package at the same time ...
	bool bPackageFound = false;
	for ( int32 ImportMapIdx = 0; ImportMapIdx < ImportMap.Num(); ImportMapIdx++ )
	{
		//save one iteration by checking for the package in this loop
		if( PackageName != NAME_None && ImportMap[ImportMapIdx].ClassName == NAME_Package && ImportMap[ImportMapIdx].ObjectName == PackageName )
		{
			bPackageFound = true;
			PackageIdx = FPackageIndex::FromImport(ImportMapIdx);
		}
		if ( ImportMap[ImportMapIdx].ObjectName == ClassName && ImportMap[ImportMapIdx].ClassName == NAME_Class )
		{
			ClassIdx = FPackageIndex::FromImport(ImportMapIdx);
			PackageIdx = ImportMap[ImportMapIdx].OuterIndex;
			return true;
		}
	}

	//an existing import couldn't be found, so add it
	//first add the needed package if it didn't already exist in the import map
	if( !bPackageFound )
	{
		FObjectImport& Import = ImportMap.AddDefaulted_GetRef();
		Import.ClassName = NAME_Package;
		Import.ClassPackage = GLongCoreUObjectPackageName;
		Import.ObjectName = PackageName;
		Import.OuterIndex = FPackageIndex();
		Import.XObject = nullptr;
		Import.SourceLinker = nullptr;
		Import.SourceIndex = -1;
		PackageIdx = FPackageIndex::FromImport(ImportMap.Num() - 1);
	}
	{
		//now add the class import
		FObjectImport& Import = ImportMap.AddDefaulted_GetRef();
		Import.ClassName = NAME_Class;
		Import.ClassPackage = GLongCoreUObjectPackageName;
		Import.ObjectName = ClassName;
		Import.OuterIndex = PackageIdx;
		Import.XObject = nullptr;
		Import.SourceLinker = nullptr;
		Import.SourceIndex = -1;
		ClassIdx = FPackageIndex::FromImport(ImportMap.Num() - 1);
	}

	return true;
}

TArray<FName> FLinkerLoad::FindPreviousNamesForClass(const FString& CurrentClassPath, bool bIsInstance)
{
	TArray<FName> OldNames;
	TArray<FCoreRedirectObjectName> OldObjectNames;

	if (FCoreRedirects::FindPreviousNames(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(CurrentClassPath), OldObjectNames))
	{
		for (FCoreRedirectObjectName& OldObjectName : OldObjectNames)
		{
			OldNames.AddUnique(OldObjectName.ObjectName);
		}
	}

	if (bIsInstance)
	{
		OldObjectNames.Empty();
		if (FCoreRedirects::FindPreviousNames(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, FCoreRedirectObjectName(CurrentClassPath), OldObjectNames))
		{
			for (FCoreRedirectObjectName& OldObjectName : OldObjectNames)
			{
				OldNames.AddUnique(OldObjectName.ObjectName);
			}
		}
	}

	return OldNames;
}

TArray<FString> FLinkerLoad::FindPreviousPathNamesForClass(const FString& CurrentClassPath, bool bIsInstance, bool bIncludeShortNames)
{
	TArray<FString> OldNames;
	TArray<FCoreRedirectObjectName> OldObjectNames;

	if (FCoreRedirects::FindPreviousNames(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(CurrentClassPath), OldObjectNames))
	{
		for (FCoreRedirectObjectName& OldObjectName : OldObjectNames)
		{
			if (bIncludeShortNames || !OldObjectName.PackageName.IsNone())
			{
				OldNames.AddUnique(OldObjectName.ToString());
			}
		}
	}

	if (bIsInstance)
	{
		OldObjectNames.Empty();
		if (FCoreRedirects::FindPreviousNames(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, FCoreRedirectObjectName(CurrentClassPath), OldObjectNames))
		{
			for (FCoreRedirectObjectName& OldObjectName : OldObjectNames)
			{
				if (bIncludeShortNames || !OldObjectName.PackageName.IsNone())
				{
					OldNames.AddUnique(OldObjectName.ToString());
				}
			}
		}
	}

	return OldNames;
}

FName FLinkerLoad::FindNewNameForEnum(const FName OldEnumName)
{
	FCoreRedirectObjectName OldName = FCoreRedirectObjectName(OldEnumName, NAME_None, NAME_None);
	FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(
		ECoreRedirectFlags::Type_Enum, OldName, ECoreRedirectMatchFlags::AllowPartialMatch);

	if (NewName != OldName)
	{
		return NewName.ObjectName;
	}
	return NAME_None;
}

FName FLinkerLoad::FindNewNameForStruct(const FName OldStructName)
{
	FCoreRedirectObjectName OldName = FCoreRedirectObjectName(OldStructName, NAME_None, NAME_None);
	FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(
		ECoreRedirectFlags::Type_Struct, OldName, ECoreRedirectMatchFlags::AllowPartialMatch);

	if (NewName != OldName)
	{
		return NewName.ObjectName;
	}
	return NAME_None;
}

FName FLinkerLoad::FindNewNameForClass(FName OldClassName, bool bIsInstance)
{
	FCoreRedirectObjectName OldName = FCoreRedirectObjectName(OldClassName, NAME_None, NAME_None);
	FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, OldName);

	if (NewName != OldName)
	{
		return NewName.ObjectName;
	}

	if (bIsInstance)
	{
		// Also check instance types
		NewName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, OldName);

		if (NewName != OldName)
		{
			return NewName.ObjectName;
		}
	}
	return NAME_None;
}

FString FLinkerLoad::FindNewPathNameForClass(const FString& OldClassNameOrPathName, bool bIsInstance)
{
	FCoreRedirectObjectName OldName = FCoreRedirectObjectName(OldClassNameOrPathName);
	FCoreRedirectObjectName NewName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, OldName);
	FString NewClassPathName;

	if (NewName != OldName)
	{
		NewClassPathName = NewName.ToString();
	}
	else if (bIsInstance)
	{
		// Also check instance types
		NewName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, OldName);

		if (NewName != OldName)
		{
			NewClassPathName = NewName.ToString();
		}
	}
	if (!NewClassPathName.IsEmpty() && FPackageName::IsShortPackageName(NewClassPathName))
	{
		UClass* ExistingClass = FindFirstObject<UClass>(*NewClassPathName, EFindFirstObjectOptions::None, ELogVerbosity::Fatal, TEXT("FindNewPathNameForClass"));
		if (ExistingClass)
		{
			NewClassPathName = ExistingClass->GetPathName();
		}
		else
		{
			UE_LOGF(LogLinker, Fatal, "No classes that match \"%ls\" class name found when looking for redirected class", *NewClassPathName);
		}
	}
	return NewClassPathName;
}

bool FLinkerLoad::IsKnownMissingPackage(FName PackageName)
{
	return FCoreRedirects::IsKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(NAME_None, NAME_None, PackageName));
}

void FLinkerLoad::AddKnownMissingPackage(FName PackageName)
{
	FCoreRedirects::AddKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(NAME_None, NAME_None, PackageName));
}

bool FLinkerLoad::RemoveKnownMissingPackage(FName PackageName)
{
	return FCoreRedirects::RemoveKnownMissing(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(NAME_None, NAME_None, PackageName));
}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
bool FLinkerLoad::IsImportLazyLoadEnabled()
{
	return UE::LinkerLoad::IsImportLazyLoadEnabled();
}
#endif

void FLinkerLoad::OnNewFileAdded(const FString& Filename)
{
	FString PackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName))
	{
		FName PackageFName(*PackageName);
		if (FLinkerLoad::IsKnownMissingPackage(PackageFName))
		{
			FLinkerLoad::RemoveKnownMissingPackage(PackageFName);
		}
	}
}

void FLinkerLoad::OnPakFileMounted(const IPakFile& NewlyLoadedContainer)
{
	// To be strictly correct we should check every known missing Package to see whether it exists in the PakFile and remove it only if so.
	// But the cost of that is be relatively high during loading, and the known missing system is for performance only.  So we instead clear the known missing
	// on every pak file.
	FCoreRedirects::ClearKnownMissing(ECoreRedirectFlags::Type_Package);
}



void FLinkerLoad::AddGameNameRedirect(const FName OldName, const FName NewName)
{
	TArray<FCoreRedirect> NewRedirects;
	NewRedirects.Emplace(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(NAME_None, NAME_None, OldName), FCoreRedirectObjectName(NAME_None, NAME_None, NewName));
	FCoreRedirects::AddRedirectList(NewRedirects, TEXT("AddGameNameRedirect"));
}

#if WITH_EDITOR

/**
* Checks if exports' indexes and names are equal.
*/
bool AreObjectExportsEqualForDuplicateChecks(const FObjectExport& Lhs, const FObjectExport& Rhs)
{
	return Lhs.ObjectName == Rhs.ObjectName
		&& Lhs.ClassIndex == Rhs.ClassIndex
		&& Lhs.OuterIndex == Rhs.OuterIndex;
}

/**
 * Helper function to sort ExportMap for duplicate checks.
 */
bool ExportMapSorter(const FObjectExport& Lhs, const FObjectExport& Rhs)
{
	// Check names first.
	if (Lhs.ObjectName != Rhs.ObjectName)
	{
		return Lhs.ObjectName.LexicalLess(Rhs.ObjectName);
	}

	// Names are equal, check classes.
	if (Lhs.ClassIndex < Rhs.ClassIndex)
	{
		return true;
	}

	if (Lhs.ClassIndex > Rhs.ClassIndex)
	{
		return false;
	}

	// Class names are equal as well, check outers.
	return Lhs.OuterIndex < Rhs.OuterIndex;
}

void FLinkerLoad::ReplaceExportIndexes(const FPackageIndex& OldIndex, const FPackageIndex& NewIndex)
{
	for (auto& Export : ExportMap)
	{
		if (Export.ClassIndex == OldIndex)
		{
			Export.ClassIndex = NewIndex;
		}

		if (Export.SuperIndex == OldIndex)
		{
			Export.SuperIndex = NewIndex;
		}

		if (Export.OuterIndex == OldIndex)
		{
			Export.OuterIndex = NewIndex;
		}
	}
}

bool FLinkerLoad::DoesSavedClassMatchActualClass(int32 ExportIndex) const
{
	const FObjectExport& Export = ExportMap[ExportIndex];
	check(Export.Object);
	FPackageIndex ClassIndex = Export.ClassIndex;
	const UClass* LoadClass = Cast<UClass>(GetCurrentObjectAtIndex(ClassIndex));

	if(!LoadClass)
	{
		return false;
	}

	if(Export.Object->GetClass() == LoadClass)
	{
		return true;
	}

	// class mismatch, if the native parent of LoadClass matches the 
	// native parent of the object, then use the native serialize:
	const auto GetNativeClass = [](const UClass* InClass)
	{
		while (InClass && !InClass->HasAnyClassFlags(CLASS_Native))
		{
			InClass = InClass->GetSuperClass();
		}
		return InClass;
	};
	return GetNativeClass(Export.Object->GetClass()) == GetNativeClass(LoadClass);
}

const UObject* FLinkerLoad::GetCurrentObjectAtIndex(FPackageIndex ObjectIndex) const
{
	if (ObjectIndex.IsNull())
	{
		return nullptr;
	}

	if (ObjectIndex.IsImport())
	{
		return ImportMap[ObjectIndex.ToImport()].XObject;
	}
	else
	{
		return ExportMap[ObjectIndex.ToExport()].Object;
	}
}

void FLinkerLoad::FixupDuplicateExports()
{
	// We need to operate on copy to avoid incorrect indexes after sorting
	auto ExportMapSorted = ExportMap;
	ExportMapSorted.Sort(ExportMapSorter);

	// ClassIndex, SuperIndex, OuterIndex
	int32 LastUniqueExportIndex = 0;
	for (int32 SortedIndex = 1; SortedIndex < ExportMapSorted.Num(); ++SortedIndex)
	{
		const FObjectExport& Original = ExportMapSorted[LastUniqueExportIndex];
		const FObjectExport& Duplicate = ExportMapSorted[SortedIndex];

		if (AreObjectExportsEqualForDuplicateChecks(Original, Duplicate))
		{
			// Duplicate entry found. Look through all Exports and update their ClassIndex, SuperIndex and OuterIndex
			// to point on original export instead of duplicate.
			const FPackageIndex& DuplicateIndex = Duplicate.ThisIndex;
			const FPackageIndex& OriginalIndex = Original.ThisIndex;
			ReplaceExportIndexes(DuplicateIndex, OriginalIndex);

			// Mark Duplicate as null, so we don't load it.
			Exp(Duplicate.ThisIndex).ThisIndex = FPackageIndex();
		}
		else
		{
			LastUniqueExportIndex = SortedIndex;
		}
	}
}
#endif // WITH_EDITOR

/**
* Allows object instances to be converted to other classes upon loading a package
*/
FLinkerLoad::ELinkerStatus FLinkerLoad::FixupExportMap()
{
	DECLARE_SCOPE_CYCLE_COUNTER( TEXT( "FLinkerLoad::FixupExportMap" ), STAT_LinkerLoad_FixupExportMap, STATGROUP_LinkerLoad );

#if WITH_EDITOR
	if (UEVer() < VER_UE4_SKIP_DUPLICATE_EXPORTS_ON_SAVE_PACKAGE && !bExportsDuplicatesFixed)
	{
		FixupDuplicateExports();
		bExportsDuplicatesFixed = true;
	}
#endif // WITH_EDITOR

	// No need to fixup exports if everything is cooked.
	if (!FPlatformProperties::RequiresCookedData())
	{
		if (bFixupExportMapDone)
		{
			return LINKER_Loaded;
		}

		for ( int32 ExportMapIdx = 0; ExportMapIdx < ExportMap.Num(); ExportMapIdx++ )
		{
			FObjectExport &Export = ExportMap[ExportMapIdx];
			if (!IsValidPackageIndex(Export.ClassIndex))
			{
				UE_ASSET_LOG(LogLinker, Warning, PackagePath, TEXT("Bad class index found on export %d"), ExportMapIdx);
				return LINKER_Failed;
			}
			FName NameClass = GetExportClassName(ExportMapIdx);
			FName NamePackage = GetExportClassPackage(ExportMapIdx);
			FString StrObjectName = Export.ObjectName.ToString();

			// ActorComponents outered to a BlueprintGeneratedClass (or even older ones that are outered to Blueprint) need to be marked RF_Public, but older content was
			// not created as such.  This updates the ExportTable such that they are correctly flagged when created and when other packages validate their imports.
#if !UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			if (UEVer() < VER_UE4_BLUEPRINT_GENERATED_CLASS_COMPONENT_TEMPLATES_PUBLIC)
			{
				if ((Export.ObjectFlags & RF_Public) == 0)
				{
					static const FName NAME_BlueprintGeneratedClass("BlueprintGeneratedClass");
					static const FName NAME_Blueprint("Blueprint");
					const FName OuterClassName = GetExportClassName(Export.OuterIndex);
					if (OuterClassName == NAME_BlueprintGeneratedClass || OuterClassName == NAME_Blueprint)
					{
						static const UClass* ActorComponentClass = FindObjectChecked<UClass>(nullptr, TEXT("/Script/Engine.ActorComponent"), EFindObjectFlags::ExactClass);
						static const FString BPGeneratedClassPostfix("_C");
						const FString NameClassString = NameClass.ToString();
						UPackage* ClassPackage = Cast<UPackage>(StaticFindObjectFast(UPackage::StaticClass(), nullptr, NamePackage));
						UClass* Class = Cast<UClass>(StaticFindObjectFast(UClass::StaticClass(), ClassPackage, NameClass));

						// It is (obviously) a component if the class is a child of actor component
						// and (almost certainly) a component if the class cannot be loaded but it ends in _C meaning it was generated from a blueprint
						// However, it (probably) isn't safe to load the blueprint class, so we just check the _C and it is (probably) good enough
						if (    ((Class != nullptr) && Class->IsChildOf(ActorComponentClass))
							 || ((Class == nullptr) && NameClassString.EndsWith(BPGeneratedClassPostfix)))
						{
							Export.ObjectFlags |= RF_Public;
						}
					}
				}
			}
#endif // !UE_INTERNAL_UNIVERSAL_PLACEHOLDERS

			// Look for subobject redirects and instance redirects
			FCoreRedirectObjectName OldClassName(NameClass, NAME_None, NamePackage);

			const TMap<FString, FString>* ValueChanges = FCoreRedirects::GetValueRedirects(ECoreRedirectFlags::Type_Class, OldClassName);

			if (ValueChanges)
			{
				// Apply class value redirects before other redirects, to mirror old subobject order
				const FString* NewInstanceName = ValueChanges->Find(Export.ObjectName.ToString());
				if (NewInstanceName)
				{
					// Rename this import directly
					FString Was = GetExportFullName(ExportMapIdx);
					Export.ObjectName = FName(**NewInstanceName);

					if (Export.ObjectName != NAME_None)
					{
						FString Now = GetExportFullName(ExportMapIdx);
						UE_LOGF(LogLinker, Verbose, "FLinkerLoad::FixupExportMap() - Renamed object from %ls   to   %ls", *Was, *Now);
					}
					else
					{
						Export.bExportLoadFailed = true;
						UE_LOGF(LogLinker, Verbose, "FLinkerLoad::FixupExportMap() - Removed object %ls", *Was);
					}
				}
			}

			// Never modify the default object instances
			if (!StrObjectName.StartsWith(DEFAULT_OBJECT_PREFIX))
			{
				FCoreRedirectObjectName NewClassInstanceName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, OldClassName);

				bool bClassInstanceDeleted = FCoreRedirects::IsKnownMissing(ECoreRedirectFlags::Type_Class | ECoreRedirectFlags::Category_InstanceOnly, OldClassName);
				if (bClassInstanceDeleted)
				{
					UE_LOGF(LogLinker, Log, "FLinkerLoad::FixupExportMap() - Pkg<%ls> [Obj<%ls> Cls<%ls> ClsPkg<%ls>] -> removed", *LinkerRoot->GetName(),
						*Export.ObjectName.ToString(), *NameClass.ToString(), *NamePackage.ToString());

					Export.ClassIndex = FPackageIndex();
					Export.OuterIndex = FPackageIndex();
					Export.ObjectName = NAME_None;
					Export.bWasFiltered = true;
#if WITH_EDITOR
					Export.OldClassName = NameClass;
#endif
				}
				else if (NewClassInstanceName != OldClassName)
				{
					FPackageIndex NewClassIndex;
					FPackageIndex NewPackageIndex;

					if (CreateImportClassAndPackage(NewClassInstanceName.ObjectName, NewClassInstanceName.PackageName, NewClassIndex, NewPackageIndex))
					{
						Export.ClassIndex = NewClassIndex;
#if WITH_EDITOR
						Export.OldClassName = NameClass;
#endif
						UE_LOGF(LogLinker, Log, "FLinkerLoad::FixupExportMap() - Pkg<%ls> [Obj<%ls> Cls<%ls> ClsPkg<%ls>] -> [Obj<%ls> Cls<%ls> ClsPkg<%ls>]", *LinkerRoot->GetName(),
							*Export.ObjectName.ToString(), *NameClass.ToString(), *NamePackage.ToString(),
							*Export.ObjectName.ToString(), *NewClassInstanceName.ObjectName.ToString(), *NewClassInstanceName.PackageName.ToString());
					}
					else
					{
						UE_LOGF(LogLinker, Log, "FLinkerLoad::FixupExportMap() - object redirection failed at %ls", *Export.ObjectName.ToString());
					}
				}
			}
		}
		bFixupExportMapDone = true;
		return !IsTimeLimitExceeded( TEXT("fixing up export map") ) ? LINKER_Loaded : LINKER_TimedOut;
	}
	else
	{
		return LINKER_Loaded;
	}
}

void FLinkerLoad::FlushCache()
{
	if (Loader)
	{
		Loader->FlushCache();
	}
}

bool FLinkerLoad::HasAnyObjectsPendingLoad() const
{
	for (const FObjectExport& Export : ExportMap)
	{
		if (Export.Object && Export.Object->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad))
		{
			return true;
		}
	}
	return false;
}

bool FLinkerLoad::AttachExternalReadDependency(FExternalReadCallback& ReadCallback)
{
	ExternalReadDependencies.Add(ReadCallback);
	return true;
}

bool FLinkerLoad::FinishExternalReadDependencies(double InTimeLimit)
{
	double LocalStartTime = FPlatformTime::Seconds();
	double RemainingTime = InTimeLimit;
	const int32 Granularity = 5;
	int32 Iteration = 0;

	while (ExternalReadDependencies.Num())
	{
		FExternalReadCallback& ReadCallback = ExternalReadDependencies.Last();

		bool bFinished = ReadCallback(RemainingTime);

		checkf(RemainingTime > 0.0 || bFinished, TEXT("FExternalReadCallback must be finished when RemainingTime is zero"));

		if (bFinished)
		{
			ExternalReadDependencies.RemoveAt(ExternalReadDependencies.Num() - 1);
		}

		// Update remaining time
		if (InTimeLimit > 0.0 && (++Iteration % Granularity) == 0)
		{
			RemainingTime = InTimeLimit - (FPlatformTime::Seconds() - LocalStartTime);
			if (RemainingTime <= 0.0)
			{
				return false;
			}
		}
	}

	return (ExternalReadDependencies.Num() == 0);
}

bool FLinkerLoad::IsContextInstanced() const
{
	return InstancingContext.IsInstanced();
}

bool FLinkerLoad::IsSoftObjectRemappingEnabled() const
{
	return IsContextInstanced() && InstancingContext.GetSoftObjectPathRemappingEnabled();
}

void FLinkerLoad::FixupSoftObjectPathForInstancedPackage(FSoftObjectPath& InOutSoftObjectPath)
{
	InstancingContext.FixupSoftObjectPath(InOutSoftObjectPath);
}

#if WITH_EDITOR
bool FLinkerLoad::bPreloadingEnabled = false;
bool FLinkerLoad::GetPreloadingEnabled()
{
	return bPreloadingEnabled;
}
void FLinkerLoad::SetPreloadingEnabled(bool bEnabled)
{
	bPreloadingEnabled = bEnabled;
}

bool FLinkerLoad::TryGetPreloadedLoader(const FPackagePath& InPackagePath, FOpenPackageResult& OutResult)
{
	return IPackageResourceManager::TryTakePreloadableArchive(InPackagePath, OutResult);
}

bool FLinkerLoad::IsLoadingToPropertyBagObject() const
{
	return FUObjectThreadContext::Get().GetSerializeContext()->bIsLoadingToPropertyBagObject;
}

#endif

bool FLinkerLoad::SerializeBulkData(FBulkData& BulkData, const FBulkDataSerializationParams& Params)
{
	using namespace UE::BulkData::Private;

	if (ShouldSkipBulkData() || IsTextFormat())
	{
		return false;
	}

	checkf(BulkData.IsUnlocked(), TEXT("Serialize bulk data FAILED, bulk data is locked"));

	FBulkMetaData& Meta = BulkData.BulkMeta;
	FBulkDataCookedIndex CookedIndex;
	int64 DuplicateSerialOffset = -1;
	SerializeBulkMeta(Meta, CookedIndex, DuplicateSerialOffset, Params.ElementSize);

	const bool bLazyLoadable = IsAllowingLazyLoading();
	if (bLazyLoadable)
	{
		Meta.AddFlags(BULKDATA_LazyLoadable);
#if WITH_EDITOR
		check(IsTextFormat() == false);
		BulkData.AttachedAr = this;
		AttachBulkData(Params.Owner, &BulkData);
#endif // WITH_EDITOR
	}

	const bool bExternalResource = Meta.HasAnyFlags(BULKDATA_WorkspaceDomainPayload);
	EPackageSegment Segment = GetBulkDataPackageSegmentFromFlags(Meta.GetFlags(), IsLoadingFromCookedPackage());
	BulkData.BulkChunkId = UE::CreatePackageResourceChunkId(PackagePath.GetPackageFName(), Segment, CookedIndex, bExternalResource);

	const bool bIsInline = Meta.HasAnyFlags(BULKDATA_PayloadAtEndOfFile) == false;
	if (bIsInline)
	{
		checkf(CookedIndex.IsDefault(), TEXT("Inline bulkdata cannot be assigned a chunk group!"));

		if (IsLoadingFromCookedPackage())
		{
			// Cooked packages are split into .uasset/.exp files and the offset needs to be adjusted accordingly.
			const int64 PkgHeaderSize = IPackageResourceManager::Get().FileSize(PackagePath, CookedIndex, EPackageSegment::Header);
			Meta.SetOffset(Tell() - PkgHeaderSize);
		}
		void* Payload = BulkData.ReallocateData(Meta.GetSize());
		BulkData.SerializeBulkData(*this, Payload, Meta.GetSize(), Meta.GetFlags());
	}
	else if (Meta.HasAnyFlags(BULKDATA_PayloadInSeparateFile))
	{
		// Streaming cooked bulk data / loading from Editor Domain and referencing Workspace domain bulk data
		if (Meta.HasAnyFlags(BULKDATA_DuplicateNonOptionalPayload))
		{
			checkf(CookedIndex.IsDefault(), TEXT("Bulkdata with duplicate non optional payloads cannot be assigned a chunk group!"));

			if (IPackageResourceManager::Get().DoesPackageExist(PackagePath, CookedIndex, EPackageSegment::BulkDataOptional))
			{
				BulkData.BulkChunkId = UE::CreatePackageResourceChunkId(PackagePath.GetPackageFName(), EPackageSegment::BulkDataOptional, FBulkDataCookedIndex::Default, bExternalResource);
				Meta.ClearFlags(BULKDATA_DuplicateNonOptionalPayload);
				Meta.AddFlags(BULKDATA_OptionalPayload);
				Meta.SetOffset(DuplicateSerialOffset);
			}
		}
		else if (Meta.HasAnyFlags(BULKDATA_MemoryMappedPayload))
		{
			checkf(CookedIndex.IsDefault(), TEXT("Bulkdata with memory mapped payloads cannot be assigned a chunk group!"));

			if (bLazyLoadable && Params.bAttemptMemoryMapping)
			{
				TUniquePtr<IMappedFileHandle> MappedFile;
				MappedFile.Reset(IPackageResourceManager::Get().OpenMappedHandleToPackage(PackagePath, EPackageSegment::BulkDataMemoryMapped));
				IMappedFileRegion* MappedRegion = MappedFile.IsValid() ? MappedFile->MapRegion(Meta.GetOffset(), Meta.GetSize(), true) : nullptr;
				if (MappedRegion)
				{
					BulkData.DataAllocation.SetMemoryMappedData(&BulkData, MappedFile.Release(), MappedRegion);
				}
				else
				{
					UE_LOGF(LogSerialization, Warning, "Memory map bulk data '%ls' FAILED", *PackagePath.GetDebugName());
					BulkData.ForceBulkDataResident();
				}
			}
		}
	}
	else
	{
		checkf(CookedIndex.IsDefault(), TEXT("Bulkdata stored within the same file cannot be assigned a chunk group!"));

		// Streaming uncooked bulk data (editor only)
		check(IsLoadingFromCookedPackage() == false);

		// Unless this package is loaded from the EditorDomain, the offset needs
		// to be adjusted to the start of non-inline bulk data in the .uasset file.
		if (Meta.HasAnyFlags(BULKDATA_WorkspaceDomainPayload) == false)
		{
			if (CVarApplyBulkDataFix.GetValueOnAnyThread())
			{
				// In theory we should never see the 'BULKDATA_NoOffsetFixUp' flag at this point, but for a time there was a bug that allowed
				// packages saved to the workspace domain to have the flag so we cannot assume that the offset is relative and need to check.
				// The outcome of this bug actually changed in the 'EUnrealEngineObjectUE5Version::DATA_RESOURCES' refactor which makes the
				// following checks more involved.

				// If 'BULKDATA_NoOffsetFixUp' is not set then we know that the offset is always relative and needs to be converted to absolute
				if (!Meta.HasAnyFlags(BULKDATA_NoOffsetFixUp))
				{
					Meta.SetOffset(Meta.GetOffset() + Summary.BulkDataStartOffset);
				}
				else
				{
					// If 'BULKDATA_NoOffsetFixUp' is set and the package was written after the 'EUnrealEngineObjectUE5Version::DATA_RESOURCES'
					// refactor then we know the offset is relative and needs to be converted. If the package was written before the refactor
					// and has the flag then we know the offset is already in absolute format and can be left unmodified.
					if (Summary.GetFileVersionUE() >= EUnrealEngineObjectUE5Version::DATA_RESOURCES)
					{
						Meta.SetOffset(Meta.GetOffset() + Summary.BulkDataStartOffset);
					}
				}
			}
			else
			{
				// Previous behavior before an attempted fix for bad data was added.
				Meta.SetOffset(Meta.GetOffset() + Summary.BulkDataStartOffset);
			}
		}

		if (bLazyLoadable == false)
		{
			FArchive& Ar = *this;
			FArchive::FScopeSeekTo _(Ar, Meta.GetOffset());
			
#if UE_INTERNAL_LINKER_BOUNDS
			FArchiveBoundsScope Bounds(Ar.SetBounds(Meta.GetSize()));
#endif//UE_INTERNAL_LINKER_BOUNDS

			void* Payload = BulkData.ReallocateData(Meta.GetSize());
			BulkData.SerializeBulkData(Ar, Payload, Meta.GetSize(), Meta.GetFlags());
		}
	}

	if (bLazyLoadable == false)
	{
		BulkData.ForceBulkDataResident();
		Meta.ClearFlags(BULKDATA_LazyLoadable);
		BulkData.BulkChunkId = FIoChunkId::InvalidChunkId;
	}

	return true;
}

void FLinkerLoad::SerializeBulkMeta(UE::BulkData::Private::FBulkMetaData& Meta, FBulkDataCookedIndex& CookedIndex, int64& DuplicateSerialOffset, int32 ElementSize)
{
	using namespace UE::BulkData::Private;
	FArchive& Ar = *this;

	if (DataResourceMap.IsEmpty())
	{
		FBulkMetaData::FromSerialized(Ar, ElementSize, Meta, DuplicateSerialOffset);
	}
	else
	{
		int32 DataResourceIndex = INDEX_NONE;
		Ar << DataResourceIndex;
		const FObjectDataResource& DataResource = DataResourceMap[DataResourceIndex];
		Meta.SetFlags(static_cast<EBulkDataFlags>(DataResource.LegacyBulkDataFlags));
		Meta.SetOffset(DataResource.SerialOffset);
		Meta.SetSize(DataResource.RawSize);
		Meta.SetSizeOnDisk(DataResource.SerialSize);
		DuplicateSerialOffset = DataResource.DuplicateSerialOffset;

		CookedIndex = DataResource.CookedIndex;
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		Meta.ClearFlags(BULKDATA_SingleUse);
	}
#endif // WITH_EDITOR
}

#undef LOCTEXT_NAMESPACE
