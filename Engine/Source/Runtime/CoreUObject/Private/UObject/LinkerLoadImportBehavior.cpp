// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerLoadImportBehavior.h"
#include "AssetRegistry/AssetData.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectResource.h"
#include "UObject/ObjectPathId.h"
#include "UObject/Package.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

namespace UE::LinkerLoad
{

/// @brief Finds LoadBehavior meta data recursively
/// @return Eager by default in not found
EImportBehavior FindLoadBehavior(const UClass& Class)
{
	//Package class can't have meta data because of UHT
	if (&Class == UPackage::StaticClass())
	{
		return EImportBehavior::LazyOnDemand;
	}

	static const FName Name_LoadBehavior("LoadBehavior");
	if (const FString* LoadBehaviorMeta = Class.FindMetaData(Name_LoadBehavior))
	{
		if (*LoadBehaviorMeta == TEXT("LazyOnDemand"))
		{
			return EImportBehavior::LazyOnDemand;
		}
		return EImportBehavior::Eager;
	}
	else
	{
		//look in super class to see if it has lazy load on
		const UClass* Super = Class.GetSuperClass();
		if (Super != nullptr)
		{
			return FindLoadBehavior(*Super);
		}
		return EImportBehavior::Eager;
	}
}

EImportBehavior GetPropertyImportLoadBehavior(const FObjectImport& Import, const FLinkerLoad& LinkerLoad)
{
	if (Import.XObject)
	{
		// If it was something that's been searched for, we've already attempted a resolve, might as well use it
		return EImportBehavior::Eager;
	}

	if (!LinkerLoad.IsImportLazyLoadEnabled() || !LinkerLoad.IsAllowingLazyLoading() || (LinkerLoad.LinkerRoot && LinkerLoad.LinkerRoot->HasAnyPackageFlags(PKG_PlayInEditor)))
	{
		return EImportBehavior::Eager;
	}

	// Attempt to get the meta from the referenced class.  This only looks in already loaded classes.  May need to resolve the class in the future.
	static const bool bDefaultLoadBehaviorTest = FParse::Param(FCommandLine::Get(), TEXT("DefaultLoadBehaviorTest"));
	if (bDefaultLoadBehaviorTest)
	{
		return EImportBehavior::LazyOnDemand;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(LinkerLoader::GetPropertyImportLoadBehavior);

	UObject* ClassPackage = FindObjectFast<UPackage>(nullptr, Import.ClassPackage);
	const UClass* FindClass = ClassPackage ? FindObjectFast<const UClass>(ClassPackage, Import.ClassName) : nullptr;
	if (FindClass != nullptr)
	{
		return FindLoadBehavior(*FindClass);
	}
	return EImportBehavior::Eager;
}

// recursively handles FAssetData redirectors
static bool HandleRedirector(const IAssetRegistryInterface& AssetRegistry, const FAssetData& InAssetData, TSet<FSoftObjectPath>& SeenPaths, FAssetData& AssetData)
{
	using namespace UE::CoreUObject::Private;

	FString RedirectedPath;
	if (!InAssetData.GetTagValue("DestinationObject", RedirectedPath))
	{
		return false;
	}

	ConstructorHelpers::StripObjectClass(RedirectedPath);
	FSoftObjectPath RedirectedPathName(RedirectedPath);

	bool bIsAlreadyInSet;
	SeenPaths.FindOrAdd(RedirectedPathName, &bIsAlreadyInSet);
	if(bIsAlreadyInSet)
	{
		//Recursive redirectors
		return false;
	}

	UE::AssetRegistry::EExists Exists = AssetRegistry.TryGetAssetByObjectPath(RedirectedPathName, AssetData);
	if (Exists == UE::AssetRegistry::EExists::Unknown)
	{
		return false;
	}
	if (Exists == UE::AssetRegistry::EExists::DoesNotExist)
	{
		//redirector pointed to something that doesn't exist
		//return true to resolve the ObjectPtr to null
		return true; 
	}
	if (AssetData.IsRedirector())
	{
		return HandleRedirector(AssetRegistry, AssetData, SeenPaths, AssetData);
	}
	return true;
}

static bool CanLazyImport(const IAssetRegistryInterface& AssetRegistry, const FSoftObjectPath& ObjectPath, FAssetData& AssetData)
{
	using namespace UE::CoreUObject::Private;
	UE::AssetRegistry::EExists Exists = AssetRegistry.TryGetAssetByObjectPath(ObjectPath, AssetData);
	if (Exists == UE::AssetRegistry::EExists::Unknown)
	{
		return false;
	}
	if (Exists == UE::AssetRegistry::EExists::DoesNotExist)
	{
		//object path doesn't exist. return true that it can be lazy loaded
		return true;
	}
	if (AssetData.IsRedirector())
	{
		TSet<FSoftObjectPath> SeenPaths;
		return HandleRedirector(AssetRegistry, AssetData, SeenPaths, AssetData);
	}
	return true;
}


bool CanLazyImport(const IAssetRegistryInterface& AssetRegistry, const FObjectImport& Import, const FLinkerLoad& LinkerLoad)
{
	//most of this is duplicated from TryLazyImport with the out parameter but it avoids a bunch of string allocations
	using namespace UE::CoreUObject::Private;
	EImportBehavior Behavior = GetPropertyImportLoadBehavior(Import, LinkerLoad);
	if (Behavior != EImportBehavior::LazyOnDemand)
	{
		return false;
	}

	static const FName Name_CoreUObjectPackage("/Script/CoreUObject");

	//packages need to handles differently since the can't be found by object path in the AssetRegistry
	if (Import.ClassPackage == Name_CoreUObjectPackage && Import.ClassName == NAME_Package)
	{
		FAssetPackageData PackageData;
		UE::AssetRegistry::EExists Exists = AssetRegistry.TryGetAssetPackageData(Import.ObjectName, PackageData);
		if (Exists == UE::AssetRegistry::EExists::Unknown)
		{
			return false;
		}
		if (Exists == UE::AssetRegistry::EExists::DoesNotExist)
		{
			//package doesn't exist. resolve to nullptr
			return true;
		}
		return true;
	}

	//build the a full objectpath for the AssetRegistry
	FObjectPathId ObjectPath;
	FName PackageName = FObjectPathId::MakeImportPathIdAndPackageName(Import, LinkerLoad, ObjectPath);
	FObjectRef ImportRef(PackageName, Import.ClassPackage, Import.ClassName, ObjectPath);

	TStringBuilder<FName::StringBufferSize> PathName;
	ImportRef.AppendPathName(PathName);

	FAssetData AssetData;
	return CanLazyImport(AssetRegistry, FSoftObjectPath(PathName), AssetData);
}


bool TryLazyImport(const IAssetRegistryInterface& AssetRegistry, const FObjectImport& Import, const FLinkerLoad& LinkerLoad, FObjectPtr& ObjectPtr)
{
	using namespace UE::CoreUObject::Private;
	if (GetPropertyImportLoadBehavior(Import, LinkerLoad) != EImportBehavior::LazyOnDemand )
	{
		return false;
	}

	static const FName Name_CoreUObjectPackage("/Script/CoreUObject");

	ObjectPtr = nullptr;

	//packages need to handles differently since the can't be found by object path in the AssetRegistry
	if (Import.ClassPackage == Name_CoreUObjectPackage && Import.ClassName == NAME_Package)
	{
		FAssetPackageData PackageData;
		UE::AssetRegistry::EExists Exists = AssetRegistry.TryGetAssetPackageData(Import.ObjectName, PackageData);
		if (Exists == UE::AssetRegistry::EExists::Unknown)
		{
			return false;
		}
		if (Exists == UE::AssetRegistry::EExists::DoesNotExist)
		{
			//package doesn't exist. resolve to nullptr
			return true;
		}

		UPackage* Package = FindObjectFast<UPackage>(nullptr, Import.ObjectName);
		if (Package)
		{
			//already loaded don't bother with setting up lazy load
			return false;
		}
		FObjectPathId ObjectPath;
		FObjectRef ImportRef(Import.ObjectName, Import.ClassPackage, Import.ClassName, ObjectPath);
		FPackedObjectRef PackedObjectRef = MakePackedObjectRef(ImportRef);
		ObjectPtr = FObjectPtr(FObjectHandle{ PackedObjectRef.EncodedRef });
		return true;
	}

	//build the a full objectpath for the AssetRegistry
	FObjectPathId ObjectPath;
	FName PackageName = FObjectPathId::MakeImportPathIdAndPackageName(Import, LinkerLoad, ObjectPath);
	FObjectRef ImportRef(PackageName, Import.ClassPackage, Import.ClassName, ObjectPath);

	TStringBuilder<FName::StringBufferSize> PathName;
	ImportRef.AppendPathName(PathName);

	FAssetData AssetData;
	if (!CanLazyImport(AssetRegistry, FSoftObjectPath(PathName), AssetData))
	{
		return false;
	}
	
	FNameBuilder NameBuilder;
	AssetData.AssetName.AppendString(NameBuilder);
	FObjectRef Ref = { AssetData.PackageName, AssetData.AssetClassPath.GetPackageName(), AssetData.AssetClassPath.GetAssetName(), FObjectPathId(NameBuilder) };
	FPackedObjectRef PackedObjectRef = MakePackedObjectRef(Ref);
	FObjectPtr Ptr(FObjectHandle{ PackedObjectRef.EncodedRef });
	ObjectPtr = Ptr;
	return true;
}


bool TryLazyLoad(const UClass& Class, const FSoftObjectPath& ObjectPath, TObjectPtr<UObject>& OutObjectPtr)
{
	using namespace UE::CoreUObject::Private;
	if (!FLinkerLoad::IsImportLazyLoadEnabled())
	{
		return false;
	}

	const EImportBehavior Behavior = FindLoadBehavior(Class);
	if (Behavior != EImportBehavior::LazyOnDemand)
	{
		return false;
	}
	const IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr();
	if (!AssetRegistry)
	{
		return false;
	}
	FObjectPtr& ObjectPtr = reinterpret_cast<FObjectPtr&>(OutObjectPtr);

	FAssetData AssetData;
	if (!CanLazyImport(*AssetRegistry, ObjectPath, AssetData))
	{
		return false;
	}
	FNameBuilder NameBuilder;
	AssetData.AssetName.AppendString(NameBuilder);
	FObjectRef ImportRef = { AssetData.PackageName, AssetData.AssetClassPath.GetPackageName(), AssetData.AssetClassPath.GetAssetName(), FObjectPathId(NameBuilder) };
	FPackedObjectRef PackedObjectRef = MakePackedObjectRef(ImportRef);
	FObjectPtr Ptr(FObjectHandle{ PackedObjectRef.EncodedRef });
	ObjectPtr = Ptr;
	return true;
}

bool IsImportLazyLoadEnabled()
{
#if WITH_LOW_LEVEL_TESTS //not ideal but need a way to force lazyload on for tests
	return true;
#else
	auto ImportLazyLoadEnabled = []()
	{
		if (FParse::Param(FCommandLine::Get(), TEXT("LazyLoadImports")))
		{
			return true;
		}
		else if (GConfig)
		{
			bool bLazyLoadImportsConfig = false;
			GConfig->GetBool(TEXT("Core.System.Experimental"), TEXT("LazyLoadImports"), bLazyLoadImportsConfig, GEngineIni);
			return bLazyLoadImportsConfig;
		}
		return false;
	};
	static const bool bImportLazyLoadEnabled = ImportLazyLoadEnabled();
	return bImportLazyLoadEnabled;
#endif
}

}

#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

