// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinkerManager.h: Unreal object linker manager
=============================================================================*/
#include "UObject/LinkerManager.h"
#include "Internationalization/GatherableTextData.h"
#include "UObject/Package.h"
#include "UObject/ObjectResource.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UObjectThreadContext.h"
#include "Async/ParallelFor.h"
#include "ProfilingDebugging/CsvProfiler.h"

FLinkerManager& FLinkerManager::Get()
{
	static TUniquePtr<FLinkerManager> Singleton = MakeUnique<FLinkerManager>();
	return *Singleton;
}

FLinkerManager::FLinkerManager() :
	bHasPendingCleanup(false)
{
}

FLinkerManager::~FLinkerManager()
{
}

bool FLinkerManager::Exec_Dev(class UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
#if !UE_BUILD_SHIPPING
	if (FParse::Command(&Cmd, TEXT("LinkerLoadList")))
	{		
		UE_LOGF(LogLinker, Display, "ObjectLoaders: %d", ObjectLoaders.Num());
		for (auto Linker : ObjectLoaders)
		{
			UE_LOGF(LogLinker, Display, "%ls", *Linker->GetDebugName());
		}

		UE_LOGF(LogLinker, Display, "LoadersWithNewImports: %d", LoadersWithNewImports.Num());
		for (auto Linker : LoadersWithNewImports)
		{
			UE_LOGF(LogLinker, Display, "%ls", *Linker->GetDebugName());
		}
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		UE_LOGF(LogLinker, Display, "LiveLinkers: %d", LiveLinkers.Num());
		for (auto Linker : LiveLinkers)
		{
			UE_LOGF(LogLinker, Display, "%ls", *Linker->GetDebugName());
		}
#endif
		return true;
	}
	else if (FParse::Command(&Cmd, TEXT("LINKERS")))
	{
		Ar.Logf(TEXT("Linkers:"));
		for (auto Linker : ObjectLoaders)
		{
			int32 NameSize = 0;
			for (int32 j = 0; j < Linker->NameMap.Num(); j++)
			{
				if (FNameEntryId Id = Linker->NameMap[j])
				{
					NameSize += FName::GetEntry(Id)->GetSizeInBytes();
				}
			}
			Ar.Logf
				(
				TEXT("%s (%s): Names=%i (%iK/%iK) Text=%i (%iK) Imports=%i (%iK) Exports=%i (%iK) Gen=%i Bulk=%i"),
				*Linker->GetDebugName(),
				*Linker->LinkerRoot->GetFullName(),
				Linker->NameMap.Num(),
				Linker->NameMap.Num() * sizeof(FName) / 1024,
				NameSize / 1024,
				Linker->GatherableTextDataMap.Num(),
				Linker->GatherableTextDataMap.Num() * sizeof(FGatherableTextData) / 1024,
				Linker->ImportMap.Num(),
				Linker->ImportMap.Num() * sizeof(FObjectImport) / 1024,
				Linker->ExportMap.Num(),
				Linker->ExportMap.Num() * sizeof(FObjectExport) / 1024,
				Linker->Summary.Generations.Num(),
#if WITH_EDITOR
				Linker->BulkDataLoaders.Num()
#else
				0
#endif // WITH_EDITOR
				);
		}

		return true;
	}
#endif // !UE_BUILD_SHIPPING

	return false;
}

extern void ConditionalFlushAsyncLoadingForLinkers(TConstArrayView<FLinkerLoad*> InLinkers);

void FLinkerManager::ResetLinkerExports(UPackage* InPackage)
{
	if (FLinkerLoad* LinkerToReset = FLinkerLoad::FindExistingLinkerForPackage(InPackage))
	{
		ConditionalFlushAsyncLoadingForLinkers({LinkerToReset});
		LinkerToReset->DetachExports();
	}
}

void FLinkerManager::ResetLoaders(UObject* InPkg)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLinkerManager::ResetLoaders);

	// Top level package to reset loaders for.
	UObject*		TopLevelPackage = InPkg ? InPkg->GetOutermost() : nullptr;

	// Find loader/ linker associated with toplevel package. We do this upfront as Detach resets LinkerRoot.
	if (TopLevelPackage)
	{
		// Linker to reset/ detach.
		FLinkerLoad* LinkerToReset = FLinkerLoad::FindExistingLinkerForPackage(CastChecked<UPackage>(TopLevelPackage));
		if (LinkerToReset)
		{
			{
#if THREADSAFE_UOBJECTS
				FScopeLock ObjectLoadersLock(&ObjectLoadersCritical);
#endif
				for (auto Linker : ObjectLoaders)
				{
					// Detach LinkerToReset from other linker's import table.
					if (Linker->LinkerRoot != TopLevelPackage)
					{
						for (auto& Import : Linker->ImportMap)
						{
							if (Import.SourceLinker)
							{
								// This code is a N^2 loop, searching the ImportMap's of all Loaders, looking for references
								// that matched the loader we are resetting, and nulling them.
								// But it looks as though all Loaders have their ImportrMaps null'd in DissociateImportsAndForcedExports.
								// So this ended up being a time consuming loop that did nothing.
								// To gain some confidence that this code is not needed, I've added these ensureMsgf's.
								// If these don't go off, then this code, and the back links may be removed.
								// 
								// Soaking to see if this code is actually needed.
								//ensureMsgf(false, TEXT("ResetLoaders has a non null SourceLinker! Linker %p, Import.SourceLinker %p, LinkerToReset = %p"), Linker, Import.SourceLinker, LinkerToReset);
								if (Import.SourceLinker == LinkerToReset)
								{
									Import.SourceLinker = nullptr;
									Import.SourceIndex = INDEX_NONE;
								}
							}
						}
					}
					else
					{
						check(Linker == LinkerToReset);
					}
				}
			}
			// Detach linker, also removes from array and sets LinkerRoot to NULL.
			LinkerToReset->LoadAndDetachAllBulkData();
			LinkerToReset->Detach();
			RemoveLinker(LinkerToReset);
			LinkerToReset = nullptr;
		}
	}
	else
	{
		// We just want a copy here
		TSet<FLinkerLoad*> LinkersToDetach;
		GetLoaders(LinkersToDetach);
		for (auto Linker : LinkersToDetach)
		{
			// Detach linker, also removes from array and sets LinkerRoot to NULL.
			Linker->LoadAndDetachAllBulkData();
			Linker->Detach();
			RemoveLinker(Linker);
		}
	}
}

void FLinkerManager::ResetLoaders(TConstArrayView<FLinkerLoad*> InLinkerLoad)
{
	TSet<FLinkerLoad*> LinkerLoads; 
	LinkerLoads.Append(InLinkerLoad);
	ResetLoaders(LinkerLoads);
}

void FLinkerManager::ResetLoaders(const TSet<FLinkerLoad*>& InLinkerLoads)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLinkerManager::ResetLoaders_Set);

	// Remove import references
	{
#if THREADSAFE_UOBJECTS
		FScopeLock ObjectLoadersLock(&ObjectLoadersCritical);
#endif
		for (FLinkerLoad* Linker : ObjectLoaders)
		{
			// Detach LinkerToReset from other linker's import table.
			if (!InLinkerLoads.Contains(Linker))
			{
				for (auto& Import : Linker->ImportMap)
				{			
					if (Import.SourceLinker)
					{
						// It looks as though all Loaders have their ImportrMaps null'd in DissociateImportsAndForcedExports.
						// So this ends up being a time consuming loop that does nothing.
						// To gain some confidence that this code is not needed, I've added these ensureMsgf's.
						// If these don't go off, then this code, and the back links may be removed.
						// 
						// Soaking to see if this code is actually needed.
						//ensureMsgf(false, TEXT("ResetLoaders has a non null SourceLinker! Linker %p, Import.SourceLinker %p"), Linker, Import.SourceLinker);

						if (InLinkerLoads.Contains(Import.SourceLinker))
						{
							Import.SourceLinker = NULL;
							Import.SourceIndex = INDEX_NONE;
						}
					}
				}
			}
		}
	}

	// Parallelize loading the bulk data since that part is thread-safe
	TArray<FLinkerLoad*> LinkerLoadsArray = InLinkerLoads.Array();
	ParallelFor(LinkerLoadsArray.Num(),
		[&LinkerLoadsArray](int32 Index)
		{
			LinkerLoadsArray[Index]->LoadAndDetachAllBulkData();
		},
		EParallelForFlags::Unbalanced
	);

	for (FLinkerLoad* LinkerToReset : InLinkerLoads)
	{
		// Detach linker, also removes from array and sets LinkerRoot to NULL.
		LinkerToReset->Detach();
	}
	// Remove all linkers in the specified set
	{
#if THREADSAFE_UOBJECTS
		FScopeLock PendingCleanupListLock(&PendingCleanupListCritical);
#endif
		PendingCleanupList.Append(LinkerLoadsArray);
		bHasPendingCleanup = true;
	}
}

void FLinkerManager::EnsureLoadingComplete(UPackage* Package)
{
	if (!Package)
	{
		return;
	}
	FLinkerLoad* Linker = FLinkerLoad::FindExistingLinkerForPackage(Package);
	if (!Linker)
	{
		return;
	}

	if (!Package->HasAnyPackageFlags(PKG_FilterEditorOnly))
	{
		Linker->SerializeThumbnails();
	}
}

void FLinkerManager::DissociateImportsAndForcedExports()
{
	{
		// In cooked builds linkers don't stick around long enough to make this worthwhile
		TSet<FLinkerLoad*> LocalLoadersWithNewImports;
		GetLoadersWithNewImportsAndEmpty(LocalLoadersWithNewImports);

		for (FLinkerLoad* Linker : LocalLoadersWithNewImports)
		{
			for (int32 ImportIndex = 0; ImportIndex < Linker->ImportMap.Num(); ImportIndex++)
			{
				FObjectImport& Import = Linker->ImportMap[ImportIndex];
				// The import object could be stale if it has been replaced by patching
				// logic or compile on load..
				bool bIsStale = false;
				if (Import.SourceLinker && Import.SourceIndex != INDEX_NONE)
				{
					bIsStale = Import.SourceLinker->ExportMap[Import.SourceIndex].Object != Import.XObject;
				}
				if (bIsStale || (Import.XObject && !Import.XObject->IsNative()))
				{
					Import.XObject = nullptr;
				}
				Import.SourceLinker = nullptr;
				// when the SourceLinker is reset, the SourceIndex must also be reset, or recreating
				// an import that points to a redirector will fail to find the redirector
				Import.SourceIndex = INDEX_NONE;
			}
			FUObjectThreadContext::Get().GetSerializeContext()->ResetImportCount();
		}
	}

	{		
		TSet<FLinkerLoad*> LocalLoadersWithForcedExports;
		GetLoadersWithForcedExportsAndEmpty(LocalLoadersWithForcedExports);
		for (FLinkerLoad* Linker : LocalLoadersWithForcedExports)
		{
			for (FObjectExport& Export : Linker->ExportMap)
			{
				if (Export.Object && Export.bForcedExport)
				{
					Export.Object->SetLinker(nullptr, INDEX_NONE);
					Export.ResetObject();
				}
			}
			FUObjectThreadContext::Get().GetSerializeContext()->ResetForcedExports();
		}
	}
}

void FLinkerManager::DeleteLinkers()
{
	check(IsInGameThread());

	if(bHasPendingCleanup)
	{
		bHasPendingCleanup = false;
	
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FLinkerManager_DeleteLinkers);
		CSV_SCOPED_TIMING_STAT_EXCLUSIVE(DeleteLinkers);

		TArray<FLinkerLoad*> CleanupArray;
		{
#if THREADSAFE_UOBJECTS
			FScopeLock PendingCleanupListLock(&PendingCleanupListCritical);
#endif
			CleanupArray = PendingCleanupList.Array();
			PendingCleanupList.Empty();
		}

		// Note that even though DeleteLinkers can only be called on the main thread,
		// we store the IsDeletingLinkers in TLS so that we're sure nothing on
		// another thread can delete linkers except FLinkerManager at the time
		// we enter this loop.
		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		ThreadContext.IsDeletingLinkers = true;
		for (FLinkerLoad* Linker : CleanupArray)
		{
			delete Linker;
		}
		ThreadContext.IsDeletingLinkers = false;
	}
}

void FLinkerManager::RemoveLinker(FLinkerLoad* Linker)
{
#if THREADSAFE_UOBJECTS
	FScopeLock PendingCleanupListLock(&PendingCleanupListCritical);
#endif
	if (Linker && !PendingCleanupList.Contains(Linker))
	{
		PendingCleanupList.Add(Linker);
		bHasPendingCleanup = true;
	}
}
