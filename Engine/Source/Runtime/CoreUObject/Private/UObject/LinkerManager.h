// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LinkerManager.h: Unreal object linker manager
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/CoreMisc.h"
#include "Misc/ScopeLock.h"
#include "HAL/ThreadSafeBool.h"

class FLinkerManager : private FSelfRegisteringExec
{
public:
	FLinkerManager();
	~FLinkerManager();

	static FLinkerManager& Get();

	FORCEINLINE void GetLoaders(TSet<FLinkerLoad*>& OutLoaders)
	{
#if THREADSAFE_UOBJECTS
		FScopeLock ObjectLoadersLock(&ObjectLoadersCritical);
#endif
		OutLoaders = ObjectLoaders;
	}

	FORCEINLINE void GetLoadersAndEmpty(TSet<FLinkerLoad*>& OutLoaders)
	{
#if THREADSAFE_UOBJECTS
		FScopeLock ObjectLoadersLock(&ObjectLoadersCritical);
#endif
		OutLoaders = MoveTemp(ObjectLoaders);
		check(ObjectLoaders.Num() == 0);
	}

	FORCEINLINE void AddLoader(FLinkerLoad* LinkerLoad)
	{
#if THREADSAFE_UOBJECTS
		FScopeLock ObjectLoadersLock(&ObjectLoadersCritical);
#endif
		ObjectLoaders.Add(LinkerLoad);
	}

	FORCEINLINE void RemoveLoaderFromObjectLoadersAndLoadersWithNewImports(FLinkerLoad* LinkerLoad)
	{
		{
#if THREADSAFE_UOBJECTS
			FScopeLock ObjectLoadersLock(&ObjectLoadersCritical);
#endif
			ObjectLoaders.Remove(LinkerLoad);
		}
		{
#if THREADSAFE_UOBJECTS
			FScopeLock ObjectLoadersLock(&LoadersWithNewImportsCritical);
#endif
			LoadersWithNewImports.Remove(LinkerLoad);
		}
	}

	FORCEINLINE void GetLoadersWithNewImportsAndEmpty(TSet<FLinkerLoad*>& OutLoaders)
	{
#if THREADSAFE_UOBJECTS
		FScopeLock ObjectLoadersLock(&LoadersWithNewImportsCritical);
#endif
		OutLoaders = MoveTemp(LoadersWithNewImports);
	}

	FORCEINLINE void AddLoaderWithNewImports(FLinkerLoad* LinkerLoad)
	{
#if THREADSAFE_UOBJECTS
		FScopeLock ObjectLoadersLock(&LoadersWithNewImportsCritical);
#endif
		LoadersWithNewImports.Add(LinkerLoad);
	}

	FORCEINLINE void GetLoadersWithForcedExportsAndEmpty(TSet<FLinkerLoad*>& OutLoaders)
	{
#if THREADSAFE_UOBJECTS
		FScopeLock ObjectLoadersLock(&LoadersWithForcedExportsCritical);
#endif
		OutLoaders = MoveTemp(LoadersWithForcedExports);
	}

	FORCEINLINE void AddLoaderWithForcedExports(FLinkerLoad* LinkerLoad)
	{
#if THREADSAFE_UOBJECTS
		FScopeLock ObjectLoadersLock(&LoadersWithForcedExportsCritical);
#endif
		LoadersWithForcedExports.Add(LinkerLoad);
	}

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	FORCEINLINE void AddLiveLinker(FLinkerLoad* Linker)
	{
		FScopeLock LiveLinkersLock(&LiveLinkersCritical);
		LiveLinkers.Add(Linker);
	}

	FORCEINLINE void RemoveLiveLinker(FLinkerLoad* Linker)
	{
		FScopeLock LiveLinkersLock(&LiveLinkersCritical);
		LiveLinkers.Remove(Linker);
	}
#endif

	/** Reset only the linker exports associated with this package. */
	void ResetLinkerExports(UPackage* InPackage);

	/** Empty the loaders */
	void ResetLoaders(UObject* InPkg);

	/** Empty the loaders */
	void ResetLoaders(TConstArrayView<FLinkerLoad*> InLinkerLoad);

	/** Empty the loaders from the specified set */
	void ResetLoaders(const TSet<FLinkerLoad*>& InLinkerLoadSet);

	/** Complete all loading (thumbnails/bulkdata) for the given Package */
	void EnsureLoadingComplete(UPackage* Package);

	/**
	* Dissociates all linker import and forced export object references. This currently needs to
	* happen as the referred objects might be destroyed at any time.
	*/
	void DissociateImportsAndForcedExports();

	/** Deletes all linkers that finished loading */
	void DeleteLinkers();

	/** Adds a linker to deferred cleanup list */
	void RemoveLinker(FLinkerLoad* Linker);

protected:
	// FSelfRegisteringExec interface
	virtual bool Exec_Dev(class UWorld* InWorld,const TCHAR* Cmd,FOutputDevice& Ar) override;

private:

	/** Map of packages to their open linkers **/
	TSet<FLinkerLoad*> ObjectLoaders;
#if THREADSAFE_UOBJECTS
	FCriticalSection ObjectLoadersCritical;
#endif

	/** List of loaders that have new imports **/
	TSet<FLinkerLoad*> LoadersWithNewImports;
#if THREADSAFE_UOBJECTS
	FCriticalSection LoadersWithNewImportsCritical;
#endif
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
	/** List of all the existing linker loaders **/
	FCriticalSection LiveLinkersCritical;
	TSet<FLinkerLoad*> LiveLinkers;
#endif
	
	/** List of loaders that have new imports **/
	TSet<FLinkerLoad*> LoadersWithForcedExports;
#if THREADSAFE_UOBJECTS
	FCriticalSection LoadersWithForcedExportsCritical;
#endif

	/** List of linkers to delete **/
	TSet<FLinkerLoad*>	PendingCleanupList;
#if THREADSAFE_UOBJECTS
	FCriticalSection PendingCleanupListCritical;
#endif
	FThreadSafeBool bHasPendingCleanup;
};
