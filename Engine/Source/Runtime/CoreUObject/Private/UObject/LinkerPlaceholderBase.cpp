// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerPlaceholderBase.h"

#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/PropertyOptional.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "Blueprint/BlueprintSupport.h"
#include "Async/UniqueLock.h"

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	#define DEFERRED_DEPENDENCY_ENSURE(EnsueExpr) ensure(EnsueExpr)
#else  // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	#define DEFERRED_DEPENDENCY_ENSURE(EnsueExpr)  (EnsueExpr)
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

/*******************************************************************************
 * LinkerPlaceholderObjectImpl
 ******************************************************************************/

/**  */
struct FPlaceholderContainerTracker : TThreadSingleton<FPlaceholderContainerTracker>
{	
	TArray<UObject*> PerspectiveReferencerStack;
	TArray<void*> PerspectiveRootDataStack;
	// as far as I can tell, structs are going to be the only bridging point 
	// between property ownership
	TArray<FFieldVariant> IntermediatePropertyStack;  // const FStructProperty*
};

/**  */
class FLinkerPlaceholderObjectImpl
{
public:
	/**
	 * A recursive method that replaces all leaf references to this object with 
	 * the supplied ReplacementValue.
	 *
	 * This function recurses the property chain (from class owner down) because 
	 * at the time of AddReferencingPropertyValue() we cannot know/record the 
	 * address/index of array properties (as they may change during array 
	 * re-allocation or compaction). So we must follow the property chain and 
	 * check every container (array, set, map) property member for references to 
	 * this (hence, the need for this recursive function).
	 * 
	 * @param  PropertyChain    An ascending outer chain, where the property at index zero is the leaf (referencer) property.
	 * @param  ChainIndex		An index into the PropertyChain that this call should start at and iterate DOWN to zero.
	 * @param  ValueAddress     The memory address of the value corresponding to the property at ChainIndex.
	 * @param  OldValue
	 * @param  ReplacementValue	The new object to replace all references to this with.
	 * @return The number of references that were replaced.
	 */
	static int32 ResolvePlaceholderValues(const TArray<FFieldVariant>& PropertyChain, int32 ChainIndex, uint8* ValueAddress, UObject* OldValue, UObject* ReplacementValue);
	
	/**
	 * Uses the current FPlaceholderContainerTracker::PerspectiveReferencerStack
	 * to search for a viable placeholder container (expected that it will be 
	 * the top of the stack).
	 * 
	 * @param  PropertyChainRef    Defines the nested property path through a UObject, where the end leaf property is one left referencing a placeholder.
	 * @return The UObject instance that is assumably referencing a placeholder (null if one couldn't be found)
	 */
	static UObject* FindPlaceholderContainer(const FLinkerPlaceholderBase::FPlaceholderValuePropertyPath& PropertyChainRef);
	static void* FindRawPlaceholderContainer(const FLinkerPlaceholderBase::FPlaceholderValuePropertyPath& PropertyChainRef);
};

//------------------------------------------------------------------------------
int32 FLinkerPlaceholderObjectImpl::ResolvePlaceholderValues(const TArray<FFieldVariant>& PropertyChain, int32 ChainIndex, uint8* ValueAddress, UObject* OldValue, UObject* ReplacementValue)
{
	int32 ReplacementCount = 0;

	for (int32 PropertyIndex = ChainIndex; PropertyIndex >= 0; --PropertyIndex)
	{
		FFieldVariant Property = PropertyChain[PropertyIndex]; // FProperty
		check(Property.IsA<FProperty>() || Property.IsA<UProperty>());
		if (PropertyIndex == 0)
		{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			check(Property.IsA<FObjectPropertyBase>() || Property.IsA<UObjectPropertyBase>());
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

			const FObjectPropertyBase* ReferencingProperty = Property.Get<FObjectPropertyBase>();
			check(ReferencingProperty);

			UObject* CurrentValue = ReferencingProperty->GetObjectPropertyValue(ValueAddress);
			if (CurrentValue == OldValue)
			{
				// @TODO: use an FArchiver with ReferencingProperty->SerializeItem() 
				//        so that we can utilize CheckValidObject()
				ReferencingProperty->SetObjectPropertyValue(ValueAddress, ReplacementValue);
				// @TODO: unfortunately, this is currently protected
				//ReferencingProperty->CheckValidObject(ValueAddress);

				++ReplacementCount;
			}
		}
		else if (const FArrayProperty* ArrayProperty = Property.Get<FArrayProperty>())
		{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			const FProperty* NextProperty = PropertyChain[PropertyIndex - 1].Get<FProperty>();
			check(NextProperty == ArrayProperty->Inner);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

			// because we can't know which array entry was set with a reference 
			// to this object, we have to comb through them all
			FScriptArrayHelper ArrayHelper(ArrayProperty, ValueAddress);
			for (int32 ArrayIndex = 0; ArrayIndex < ArrayHelper.Num(); ++ArrayIndex)
			{
				uint8* MemberAddress = ArrayHelper.GetRawPtr(ArrayIndex);
				ReplacementCount += ResolvePlaceholderValues(PropertyChain, PropertyIndex - 1, MemberAddress, OldValue, ReplacementValue);
			}

			// the above recursive call chewed through the rest of the
			// PropertyChain, no need to keep on here
			break;
		}
		else if (const UArrayProperty* ArrayUProperty = Property.Get<UArrayProperty>())
		{
			// With FProperties this should never happen
			check(false);
		}
		else if (const FSetProperty* SetProperty = Property.Get<FSetProperty>())
		{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			const FProperty* NextProperty = PropertyChain[PropertyIndex - 1].Get<FProperty>();
			check(NextProperty == SetProperty->ElementProp);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

			// because we can't know which set entry was set with a reference 
			// to this object, we have to comb through them all
			FScriptSetHelper SetHelper(SetProperty, ValueAddress);
			for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
			{
				uint8* ElementAddress = SetHelper.GetElementPtr(It);
				ReplacementCount += ResolvePlaceholderValues(PropertyChain, PropertyIndex - 1, ElementAddress, OldValue, ReplacementValue);
			}

			// the above recursive call chewed through the rest of the
			// PropertyChain, no need to keep on here
			break;
		}
		else if (const USetProperty* SetUProperty = Property.Get<USetProperty>())
		{
			// With FProperties this should never happen
			check(false);
		}
		else if (const FMapProperty* MapProperty = Property.Get<FMapProperty>())
		{
			const FProperty* NextProperty = PropertyChain[PropertyIndex - 1].Get<FProperty>();
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			check(NextProperty == MapProperty->KeyProp || NextProperty == MapProperty->ValueProp);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			// because we can't know which map entry was set with a reference 
			// to this object, we have to comb through them all
			FScriptMapHelper MapHelper(MapProperty, ValueAddress);
			for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
			{
				if (NextProperty == MapProperty->KeyProp)
				{
					uint8* KeyAddress = MapHelper.GetKeyPtr(It);
					ReplacementCount += ResolvePlaceholderValues(PropertyChain, PropertyIndex - 1, KeyAddress, OldValue, ReplacementValue);
				}
				else if (NextProperty == MapProperty->ValueProp)
				{
					uint8* MapValueAddress = MapHelper.GetValuePtr(It);
					ReplacementCount += ResolvePlaceholderValues(PropertyChain, PropertyIndex - 1, MapValueAddress, OldValue, ReplacementValue);
				}
			}

			// the above recursive call chewed through the rest of the
			// PropertyChain, no need to keep on here
			break;
		}
		else if (const UMapProperty* MapUProperty = Property.Get<UMapProperty>())
		{
			// With FProperties this should never happen
			check(false);
		}
		else if (const FOptionalProperty* OptionalProperty = Property.Get<FOptionalProperty>())
		{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			const FProperty* NextProperty = PropertyChain[PropertyIndex - 1].Get<FProperty>();
			check(NextProperty == OptionalProperty->GetValueProperty());
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

			if (void* SetValueAddress = OptionalProperty->GetValuePointerForReplaceIfSet(ValueAddress))
			{
				ReplacementCount += ResolvePlaceholderValues(PropertyChain, PropertyIndex - 1, static_cast<uint8*>(SetValueAddress), OldValue, ReplacementValue);
			}

			// the above recursive call chewed through the rest of the
			// PropertyChain, no need to keep on here
			break;
		}
		else if (const FProperty* NextProperty = PropertyChain[PropertyIndex - 1].Get<FProperty>())
		{
			ValueAddress = NextProperty->ContainerPtrToValuePtr<uint8>(ValueAddress, /*ArrayIndex =*/0);
		}
		else if (const UProperty* NextUProperty = PropertyChain[PropertyIndex - 1].Get<UProperty>())
		{
			ValueAddress = NextUProperty->ContainerPtrToValuePtr<uint8>(ValueAddress, /*ArrayIndex =*/0);
		}
	}

	return ReplacementCount;
}

//------------------------------------------------------------------------------
UObject* FLinkerPlaceholderObjectImpl::FindPlaceholderContainer(const FLinkerPlaceholderBase::FPlaceholderValuePropertyPath& PropertyChainRef)
{
	UObject* ContainerObj = nullptr;
	TArray<UObject*>& PossibleReferencers = FPlaceholderContainerTracker::Get().PerspectiveReferencerStack;

	UClass* OwnerClass = PropertyChainRef.GetOwnerClass();
	if ((OwnerClass != nullptr) && (PossibleReferencers.Num() > 0))
	{
		UObject* ReferencerCandidate = PossibleReferencers.Top();
		// we expect that the current object we're looking for (the one we're 
		// serializing) is at the top of the stack
		if (DEFERRED_DEPENDENCY_ENSURE(ReferencerCandidate->GetClass()->IsChildOf(OwnerClass)))
		{
			ContainerObj = ReferencerCandidate;
		}
		else
		{
			// if it's not the top element, then iterate backwards because this 
			// is meant to act as a stack, where the last entry is most likely 
			// the one we're looking for
			for (int32 CandidateIndex = PossibleReferencers.Num() - 2; CandidateIndex >= 0; --CandidateIndex)
			{
				ReferencerCandidate = PossibleReferencers[CandidateIndex];

				UClass* CandidateClass = ReferencerCandidate->GetClass();
				if (CandidateClass->IsChildOf(OwnerClass))
				{
					ContainerObj = ReferencerCandidate;
					break;
				}
			}
		}
	}
	return ContainerObj;
}

void* FLinkerPlaceholderObjectImpl::FindRawPlaceholderContainer(const FLinkerPlaceholderBase::FPlaceholderValuePropertyPath& PropertyChainRef)
{
	TArray<void*>& PossibleStructReferencers = FPlaceholderContainerTracker::Get().PerspectiveRootDataStack;
	return PossibleStructReferencers.Num() > 0 ? PossibleStructReferencers[0] : nullptr;
}

/*******************************************************************************
 * FPlaceholderContainerTracker / FScopedPlaceholderPropertyTracker
 ******************************************************************************/

//------------------------------------------------------------------------------
void FScopedPlaceholderContainerTracker::Push(UObject* InPlaceholderContainerCandidate)
{
	PlaceholderReferencerCandidate = InPlaceholderContainerCandidate;
	FPlaceholderContainerTracker::Get().PerspectiveReferencerStack.Add(InPlaceholderContainerCandidate);
}

//------------------------------------------------------------------------------
void FScopedPlaceholderContainerTracker::Pop()
{
	UObject* StackTop = FPlaceholderContainerTracker::Get().PerspectiveReferencerStack.Pop();
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	check(StackTop == PlaceholderReferencerCandidate);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
}

FScopedPlaceholderRawContainerTracker::FScopedPlaceholderRawContainerTracker(void* InData)
	:Data(InData)
{
	FPlaceholderContainerTracker::Get().PerspectiveRootDataStack.Add(InData);
}

FScopedPlaceholderRawContainerTracker::~FScopedPlaceholderRawContainerTracker()
{
	void* StackTop = FPlaceholderContainerTracker::Get().PerspectiveRootDataStack.Pop();
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	check(StackTop == Data);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
}

//------------------------------------------------------------------------------
void FScopedPlaceholderPropertyTracker::Push(FFieldVariant InIntermediateProperty)
{
	FPlaceholderContainerTracker& ContainerRepository = FPlaceholderContainerTracker::Get();
	if (ContainerRepository.PerspectiveReferencerStack.Num() > 0 || ContainerRepository.PerspectiveRootDataStack.Num() > 0)
	{
		check(InIntermediateProperty.IsA<UStructProperty>() || InIntermediateProperty.IsA<FStructProperty>());
		IntermediateProperty = InIntermediateProperty;
		ContainerRepository.IntermediatePropertyStack.Add(InIntermediateProperty);
	}
	// else, if there's nothing in the PerspectiveReferencerStack, then caching 
	// a property here would be pointless (the whole point of this is to be able 
	// to use this to look up the referencing object)
}

//------------------------------------------------------------------------------
void FScopedPlaceholderPropertyTracker::Pop()
{
	FPlaceholderContainerTracker& ContainerRepository = FPlaceholderContainerTracker::Get();
	if (IntermediateProperty.IsValid())
	{
		FFieldVariant StackTop = ContainerRepository.IntermediatePropertyStack.Pop();
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		check(StackTop == IntermediateProperty);
	}
	else
	{
		check(ContainerRepository.IntermediatePropertyStack.Num()  == 0);
		check(ContainerRepository.PerspectiveReferencerStack.Num() == 0);
		check(ContainerRepository.PerspectiveRootDataStack.Num() == 0);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	}
}

/*******************************************************************************
 * FLinkerPlaceholderBase::FPlaceholderValuePropertyPath
 ******************************************************************************/

//------------------------------------------------------------------------------
FLinkerPlaceholderBase::FPlaceholderValuePropertyPath::FPlaceholderValuePropertyPath(FFieldVariant ReferencingProperty)
{
	check(ReferencingProperty.IsA<FProperty>() || ReferencingProperty.IsA<UProperty>());

	PropertyChain.Add(ReferencingProperty);
	FFieldVariant PropertyOuter = ReferencingProperty.GetOwnerVariant();
	
	const TArray<FFieldVariant>& StructPropertyStack = FPlaceholderContainerTracker::Get().IntermediatePropertyStack;
	int32 StructStackIndex = StructPropertyStack.Num() - 1; // "top" of the array is the last element

	while (PropertyOuter.IsValid() && !(PropertyOuter.IsUObject() && PropertyOuter.ToUObject()->GetClass()->IsChildOf<UClass>()))
	{
		// handle nested properties (like array members)
		const FProperty* PropertyOwner = nullptr;
		if (!PropertyOuter.IsUObject())
		{
			PropertyOwner = CastField<const FProperty>(PropertyOuter.ToField());
			if (PropertyOwner)
			{
				PropertyChain.Add(PropertyOwner);
			}
		}
		// handle nested struct properties (use the FPlaceholderContainerTracker::IntermediatePropertyStack
		// to help trace the property path)
		else if (const UScriptStruct* StructOwner = Cast<const UScriptStruct>(PropertyOuter.ToUObject()))
		{
			if (StructStackIndex != INDEX_NONE)
			{
				// we expect the top struct property to be the one we're currently serializing
				//const FStructProperty* SerializingStructProp = StructPropertyStack[StructStackIndex];
				const FFieldVariant& SerializingStructProp = StructPropertyStack[StructStackIndex];
				UScriptStruct* InnerStruct = nullptr;
				if (FStructProperty* SerializingStructFProp = SerializingStructProp.Get<FStructProperty>())
				{
					InnerStruct = SerializingStructFProp->Struct;
				}
				else if (UStructProperty* SerializingStructUProp = SerializingStructProp.Get<UStructProperty>())
				{
					InnerStruct = SerializingStructUProp->Struct;
				}
				check(InnerStruct);
				if (DEFERRED_DEPENDENCY_ENSURE(InnerStruct->IsChildOf(StructOwner)))
				{
					PropertyOuter = SerializingStructProp;
					PropertyChain.Add(SerializingStructProp);
				}
				else
				{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
					PropertyChain.Empty(); // invalidate this FPlaceholderValuePropertyPath
					checkf(false, TEXT("Looks like we couldn't reliably determine the object that a placeholder value should belong to - are we missing a FScopedPlaceholderPropertyTracker?"));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TEST
					break;
				}
				--StructStackIndex;
			}
			else
			{
				// We're serializing a struct that isn't owned by a UObject (e.g. UUserDefinedStructEditorData::DefaultStructInstance)
				break;
			}
		}
		PropertyOuter = PropertyOuter.GetOwnerVariant();
	}

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	if (!DEFERRED_DEPENDENCY_ENSURE(PropertyOuter != nullptr))
	{
		PropertyChain.Empty(); // invalidate this FPlaceholderValuePropertyPath
	}
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TEST
}

//------------------------------------------------------------------------------
bool FLinkerPlaceholderBase::FPlaceholderValuePropertyPath::IsValid() const
{
	return (PropertyChain.Num() > 0) && 
		(PropertyChain[0].IsA<FObjectPropertyBase>() || PropertyChain[0].IsA<UObjectPropertyBase>()) &&
		PropertyChain.Last().GetOwnerClass();
}

//------------------------------------------------------------------------------
UClass* FLinkerPlaceholderBase::FPlaceholderValuePropertyPath::GetOwnerClass() const
{
	return (PropertyChain.Num() > 0) ? PropertyChain.Last().GetOwnerClass() : nullptr;
}

static inline uint8* ResolvePropertyAddress(FFieldVariant Field, void* Container)
{
	uint8* OutermostAddress = nullptr;
	if (FProperty* OutermostFProperty = Field.Get<FProperty>())
	{
		OutermostAddress = OutermostFProperty->ContainerPtrToValuePtr<uint8>((uint8*)Container, /*ArrayIndex =*/0);
	}
	else if (UProperty* OutermostUProperty = Field.Get<UProperty>())
	{
		OutermostAddress = OutermostUProperty->ContainerPtrToValuePtr<uint8>((uint8*)Container, /*ArrayIndex =*/0);
	}
	else
	{
		// We shouldn't get here
		check(false);
	}
	return OutermostAddress;
}

//------------------------------------------------------------------------------
int32 FLinkerPlaceholderBase::FPlaceholderValuePropertyPath::Resolve(FLinkerPlaceholderBase* Placeholder, UObject* Replacement, UObject* Container) const
{
	FFieldVariant OutermostProperty = PropertyChain.Last();

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	UClass* OwnerClass = OutermostProperty.GetOwnerClass();
	check(OwnerClass && Container->IsA(OwnerClass));
#endif 

	uint8* OutermostAddress = ResolvePropertyAddress(OutermostProperty, Container);
	return FLinkerPlaceholderObjectImpl::ResolvePlaceholderValues(PropertyChain, PropertyChain.Num() - 1, OutermostAddress, Placeholder->GetPlaceholderAsUObject(), Replacement);
}

int32 FLinkerPlaceholderBase::FPlaceholderValuePropertyPath::ResolveRaw(FLinkerPlaceholderBase* Placeholder, UObject* Replacement, void* Container) const
{
	FFieldVariant OutermostProperty = PropertyChain.Last();
	uint8* OutermostAddress = ResolvePropertyAddress(OutermostProperty, Container);
	return FLinkerPlaceholderObjectImpl::ResolvePlaceholderValues(PropertyChain, PropertyChain.Num() - 1, OutermostAddress, Placeholder->GetPlaceholderAsUObject(), Replacement);;
}
 
/*******************************************************************************
 * FLinkerPlaceholderBase
 ******************************************************************************/

//------------------------------------------------------------------------------
FLinkerPlaceholderBase::FLinkerPlaceholderBase()
	: bResolveWasInvoked(false)
{
}

//------------------------------------------------------------------------------
FLinkerPlaceholderBase::~FLinkerPlaceholderBase()
{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	check(!HasKnownReferences());
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
}

//------------------------------------------------------------------------------
bool FLinkerPlaceholderBase::AddReferencingPropertyValue(FFieldVariant ReferencingProperty, void* DataPtr)
{
	check(ReferencingProperty.IsA<FObjectPropertyBase>() || ReferencingProperty.IsA<UObjectPropertyBase>())
	FPlaceholderValuePropertyPath PropertyChain(ReferencingProperty);
	UObject* ReferencingContainer = FLinkerPlaceholderObjectImpl::FindPlaceholderContainer(PropertyChain);
	if (ReferencingContainer != nullptr)
	{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		check(ReferencingProperty.Get<FObjectPropertyBase>()->GetObjectPropertyValue(DataPtr) == GetPlaceholderAsUObject());
		check(PropertyChain.IsValid());
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

		ReferencingContainers.FindOrAdd(ReferencingContainer).Add(PropertyChain);
		return true;
	}
	else
	{
		void* ReferencingRootStruct = FLinkerPlaceholderObjectImpl::FindRawPlaceholderContainer(PropertyChain);
		if (ReferencingRootStruct)
		{
			ReferencingRawContainers.FindOrAdd(ReferencingRootStruct).Add(PropertyChain);
		}
		return ReferencingRootStruct != nullptr;
	}
}

//------------------------------------------------------------------------------
bool FLinkerPlaceholderBase::HasKnownReferences() const
{
	return (ReferencingContainers.Num() > 0) || ReferencingRawContainers.Num() > 0;
}

//------------------------------------------------------------------------------
int32 FLinkerPlaceholderBase::ResolveAllPlaceholderReferences(UObject* ReplacementObj)
{
	int32 ReplacementCount = ResolvePlaceholderPropertyValues(ReplacementObj);
	ReferencingContainers.Empty();
	ReferencingRawContainers.Empty();

	MarkAsResolved();
	return ReplacementCount;
}

//------------------------------------------------------------------------------
void FLinkerPlaceholderBase::SetupPlaceholderSubobject(ULinkerPlaceholderExportObject* PlaceholderSubobject)
{
	PlaceholderSubobjects.Add(PlaceholderSubobject);
	PlaceholderSubobject->OwningPlaceholder = CastChecked<ULinkerPlaceholderExportObject>( this->GetPlaceholderAsUObject() );
}

//------------------------------------------------------------------------------
bool FLinkerPlaceholderBase::HasBeenFullyResolved() const
{
	return IsMarkedResolved() && !HasKnownReferences();
}

//------------------------------------------------------------------------------
bool FLinkerPlaceholderBase::IsMarkedResolved() const
{
	return bResolveWasInvoked;
}

//------------------------------------------------------------------------------
void FLinkerPlaceholderBase::MarkAsResolved()
{
	bResolveWasInvoked = true;
}
 
//------------------------------------------------------------------------------
int32 FLinkerPlaceholderBase::ResolvePlaceholderPropertyValues(UObject* NewObjectValue)
{
	int32 ResolvedTotal = 0;

	for (auto& ReferencingPair : ReferencingContainers)
	{
		TWeakObjectPtr<UObject> ContainerPtr = ReferencingPair.Key;
		if (!ContainerPtr.IsValid())
		{
			continue;
		}
		UObject* Container = ContainerPtr.Get();

		for (const FPlaceholderValuePropertyPath& PropertyRef : ReferencingPair.Value)
		{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			check(Container->GetClass()->IsChildOf(PropertyRef.GetOwnerClass()));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

			int32 ResolvedCount = PropertyRef.Resolve(this, NewObjectValue, Container);
			ResolvedTotal += ResolvedCount;

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			// we expect that (because we have had ReferencingProperties added) 
			// there should be at least one reference that is resolved... if 
			// there were none, then a property could have changed its value 
			// after it was set to this
			// 
			// NOTE: this may seem it can be resolved by properties removing themselves
			//       from ReferencingProperties, but certain properties may be
			//       the inner of a container (array, set, map) property (meaning 
			//       there could be multiple references per property)... we'd 
			//       have to inc/decrement a property ref-count to resolve that 
			//       scenario
			check(ResolvedCount > 0);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		}
	}

	for (auto& ReferencingRawPair : ReferencingRawContainers)
	{
		void* RawValue = ReferencingRawPair.Key;
		check(RawValue);

		for (const FPlaceholderValuePropertyPath& PropertyRef : ReferencingRawPair.Value)
		{
			int32 ResolvedCount = PropertyRef.ResolveRaw(this, NewObjectValue, RawValue);
			ResolvedTotal += ResolvedCount;
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			check(ResolvedCount > 0);
#endif
		}
	}

	return ResolvedTotal;
}

/*******************************************************************************
 * TLinkerImportPlaceholder<UClass>
 ******************************************************************************/

//------------------------------------------------------------------------------
template<>
int32 TLinkerImportPlaceholder<UClass>::SpecializedResolvePropertyReferences(UClass* ReplacementClass)
{
	int32 ReplacementCount = 0;
	UClass* PlaceholderClass = CastChecked<UClass>(GetPlaceholderAsUObject());

	for (FFieldVariant& Property : ReferencingProperties)
	{
		if (FObjectPropertyBase* BaseObjProperty = Property.Get<FObjectPropertyBase>())
		{
			if (BaseObjProperty->PropertyClass == PlaceholderClass)
			{
				BaseObjProperty->PropertyClass = ReplacementClass;
				++ReplacementCount;
			}

			if (FClassProperty* ClassProperty = CastField<FClassProperty>(BaseObjProperty))
			{
				if (ClassProperty->MetaClass == PlaceholderClass)
				{
					ClassProperty->MetaClass = ReplacementClass;
					++ReplacementCount;
				}
			}
			else if (FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(BaseObjProperty))
			{
				if (SoftClassProperty->MetaClass == PlaceholderClass)
				{
					SoftClassProperty->MetaClass = ReplacementClass;
					++ReplacementCount;
				}	
			}
		}
#if WITH_EDITORONLY_DATA
		else if (UObjectPropertyBase* BaseUObjProperty = Property.Get<UObjectPropertyBase>())
		{
			if (BaseUObjProperty->PropertyClass == PlaceholderClass)
			{
				BaseUObjProperty->PropertyClass = ReplacementClass;
				if (FObjectPropertyBase* AssociatedFProperty = CastField<FObjectPropertyBase>(BaseUObjProperty->GetAssociatedFField()))
				{
					if (AssociatedFProperty->PropertyClass == PlaceholderClass)
					{
						AssociatedFProperty->PropertyClass = ReplacementClass;
					}
				}
				++ReplacementCount;
			}

			if (UClassProperty* ClassProperty = Cast<UClassProperty>(BaseUObjProperty))
			{
				if (ClassProperty->MetaClass == PlaceholderClass)
				{
					ClassProperty->MetaClass = ReplacementClass;
					if (FClassProperty* AssociatedFProperty = CastField<FClassProperty>(ClassProperty->GetAssociatedFField()))
					{
						if (AssociatedFProperty->MetaClass == PlaceholderClass)
						{
							AssociatedFProperty->MetaClass = ReplacementClass;
						}
					}
					++ReplacementCount;
				}
			}
			else if (USoftClassProperty* SoftClassProperty = Cast<USoftClassProperty>(BaseUObjProperty))
			{
				if (SoftClassProperty->MetaClass == PlaceholderClass)
				{
					SoftClassProperty->MetaClass = ReplacementClass;
					if (FSoftClassProperty* AssociatedFProperty = CastField<FSoftClassProperty>(SoftClassProperty->GetAssociatedFField()))
					{
						if (AssociatedFProperty->MetaClass == PlaceholderClass)
						{
							AssociatedFProperty->MetaClass = ReplacementClass;
						}
					}
					++ReplacementCount;
				}
			}
		}
#endif // WITH_EDITORONLY_DATA
		else if (FInterfaceProperty* InterfaceProp = Property.Get<FInterfaceProperty>())
		{
			if (InterfaceProp->InterfaceClass == PlaceholderClass)
			{
				InterfaceProp->InterfaceClass = ReplacementClass;
				++ReplacementCount;
			}
		}
#if WITH_EDITORONLY_DATA
		else if (UInterfaceProperty* UInterfaceProp = Property.Get<UInterfaceProperty>())
		{
			if (UInterfaceProp->InterfaceClass == PlaceholderClass)
			{
				UInterfaceProp->InterfaceClass = ReplacementClass;
				if (FInterfaceProperty* AssociatedFProperty = CastField<FInterfaceProperty>(UInterfaceProp->GetAssociatedFField()))
				{
					if (AssociatedFProperty->InterfaceClass == PlaceholderClass)
					{
						AssociatedFProperty->InterfaceClass = ReplacementClass;
					}
				}
				++ReplacementCount;
			}
		}
#endif // WITH_EDITORONLY_DATA
		else
		{
			checkf(false, TEXT("Unhandled property type: %s"), *Property.GetClassName());
		}
	}

	ReferencingProperties.Empty();
	return ReplacementCount;
}

/*******************************************************************************
 * TLinkerImportPlaceholder<UFunction>
 ******************************************************************************/

//------------------------------------------------------------------------------
template<>
int32 TLinkerImportPlaceholder<UFunction>::SpecializedResolvePropertyReferences(UFunction* ReplacementFunc)
{
	int32 ReplacementCount = 0;
	UFunction* PlaceholderFunc = CastChecked<UFunction>(GetPlaceholderAsUObject());

	for (FFieldVariant& Property : ReferencingProperties)
	{
		if (FDelegateProperty* DelegateProperty = Property.Get<FDelegateProperty>())
		{
			if (DelegateProperty->SignatureFunction == PlaceholderFunc)
			{
				DelegateProperty->SignatureFunction = ReplacementFunc;
				++ReplacementCount;
			}
		}
#if WITH_EDITORONLY_DATA
		else if (UDelegateProperty* DelegateUProperty = Property.Get<UDelegateProperty>())
		{
			if (DelegateUProperty->SignatureFunction == PlaceholderFunc)
			{
				DelegateUProperty->SignatureFunction = ReplacementFunc;
				if (FDelegateProperty* AssociatedFProperty = CastField<FDelegateProperty>(DelegateUProperty->GetAssociatedFField()))
				{
					if (AssociatedFProperty->SignatureFunction == PlaceholderFunc)
					{
						AssociatedFProperty->SignatureFunction = ReplacementFunc;
					}
				}
				++ReplacementCount;
			}
		}
#endif // WITH_EDITORONLY_DATA
		else if (FMulticastDelegateProperty* MulticastDelegateProperty = Property.Get<FMulticastDelegateProperty>())
		{
			if (MulticastDelegateProperty->SignatureFunction == PlaceholderFunc)
			{
				MulticastDelegateProperty->SignatureFunction = ReplacementFunc;
				++ReplacementCount;
			}
		}
#if WITH_EDITORONLY_DATA
		else if (UMulticastDelegateProperty* MulticastDelegateUProperty = Property.Get<UMulticastDelegateProperty>())
		{
			if (MulticastDelegateUProperty->SignatureFunction == PlaceholderFunc)
			{
				MulticastDelegateUProperty->SignatureFunction = ReplacementFunc;
				if (FDelegateProperty* AssociatedFProperty = CastField<FDelegateProperty>(MulticastDelegateUProperty->GetAssociatedFField()))
				{
					if (AssociatedFProperty->SignatureFunction == PlaceholderFunc)
					{
						AssociatedFProperty->SignatureFunction = ReplacementFunc;
					}
				}
				++ReplacementCount;
			}
		}
#endif // WITH_EDITORONLY_DATA
		else
		{
			checkf(false, TEXT("Unhandled property type: %s"), *Property.GetClassName());
		}
	}

	ReferencingProperties.Empty();
	return ReplacementCount;
}

#undef DEFERRED_DEPENDENCY_ENSURE
