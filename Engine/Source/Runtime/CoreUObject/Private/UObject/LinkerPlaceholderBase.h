// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ObjectResource.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Field.h"
#include "Async/UniqueLock.h"

FORCEINLINE bool UseCircularDependencyLoadDeferring()
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return true;
#else
	return false;
#endif
}

/*******************************************************************************
 * FPlaceholderContainerTracker / FScopedPlaceholderPropertyTracker
 ******************************************************************************/

/** 
 * To track placeholder property values, we need to know the root container 
 * instance that is set with the placeholder value (so we can reset it later). 
 * This here is designed to track objects that are actively being preloaded
 * (serialized in); so we have the container on hand, when a FObjectProperty 
 * value is set with a placeholder.
 */
struct FScopedPlaceholderContainerTracker
{
public:
	FScopedPlaceholderContainerTracker(UObject* InPlaceholderContainerCandidate)
	: bEnabled(UseCircularDependencyLoadDeferring())
	{
		if (bEnabled)
		{
			Push(InPlaceholderContainerCandidate);
		}
	}
	~FScopedPlaceholderContainerTracker()
	{
		if (bEnabled)
		{
			Pop();
		}
	}

private:
	const bool bEnabled;
	UObject* PlaceholderReferencerCandidate;

	void Push(UObject* InIntermediateProperty);
	void Pop();
};

/** 
 * Sometimes using FScopedPlaceholderContainerTracker above is not enough; we
 * could be working with a series of nested structs, where the owning object is 
 * somewhere up the chain, but the lower properties have no idea who that is. 
 * This provides us context, so we can navigate backwards and truly determine 
 * the object a property is writing to.
 */
struct FScopedPlaceholderPropertyTracker
{
public:
	 FScopedPlaceholderPropertyTracker(FFieldVariant InIntermediateProperty)
		 : bEnabled(UseCircularDependencyLoadDeferring())
	 {
		if (bEnabled)
		{
			Push(InIntermediateProperty);
		}
	 }
	
	~FScopedPlaceholderPropertyTracker()
	{
		if (bEnabled)
		{
			Pop();
		}
	}

private:
	const bool bEnabled;
	FFieldVariant IntermediateProperty;
	
	void Push(FFieldVariant InIntermediateProperty);
	void Pop();
};

/*******************************************************************************
 * TLinkerPlaceholderBase<>
 ******************************************************************************/
 
class FLinkerPlaceholderBase
{
public:
	FLinkerPlaceholderBase();
	virtual ~FLinkerPlaceholderBase();

	/** Set by the ULinkerLoad that created this instance, tracks what import/export this was used in place of. */
	FPackageIndex PackageIndex;

	/** Also set by the ULinkerLoad that created this instance; keeps track of the type of the original object whose load is being deferred. */
	TWeakObjectPtr<const UClass> DeferredObjectType;

	/**
	 * Attempts to find and store the referencing container object (along with  
	 * the specified property), so that we can replace the reference at a 
	 * later point. Can fail if the container could not be found.
	 * 
	 * @param  ReferencingProperty	The property whose object-value is referencing this.
	 * @param  DataPtr				Not saved off (as it can change), but used to verify that we pick the correct container.
	 * @return True if we successfully found a container object and are now tracking it, otherwise false.
	 */
	bool AddReferencingPropertyValue(FFieldVariant ReferencingProperty, void* DataPtr);

	/**
	 * A query method that let's us check to see if this class is currently 
	 * being referenced by anything (if this returns false, then a referencing 
	 * property could have forgotten to add itself... or, we've replaced all
	 * references).
	 * 
	 * @return True if this has anything stored in its ReferencingProperties container, otherwise false.
	 */
	virtual bool HasKnownReferences() const;

	/**
	 * Iterates over all known referencers and attempts to replace their 
	 * references to this with a new (hopefully proper) UObject.
	 * 
	 * @param  ReplacementObj	The object that you want all references to this replaced with.
	 * @return The number of references that were successfully replaced.
	 */
	virtual int32 ResolveAllPlaceholderReferences(UObject* ReplacementObj);

	/* 
	 * Sets up the provided placeholder as a subobject of this placeholder.
	 */
	void SetupPlaceholderSubobject(ULinkerPlaceholderExportObject* PlaceholderSubobject);

	/** Returns true if this is a deferred subobject (implying the owner will resolve it) */
	bool IsDeferredSubobject() const {return OwningPlaceholder != nullptr;}

	/** Returns the list of subobjects that are waiting for this placeholder */
	TArray<ULinkerPlaceholderExportObject*>& GetSubobjectPlaceholders() { return PlaceholderSubobjects; }

	/**
	 * Checks to see if 1) this placeholder has had 
	 * ResolveAllPlaceholderReferences() called on it, and 2) it doesn't have 
	 * any more references that have since been added.
	 * 
	 * @return True if ResolveAllPlaceholderReferences() has been ran, and no KNOWN references have been added since.
	 */
	bool HasBeenFullyResolved() const;
	
	/**
	 * Some of our internal validation checks rely on UObject compares (between 
	 * this placeholder and other values). Since it is expected that this is 
	 * sub-classed by a UObject, then we provide it this pure-virtual for the
	 * sub-class to implement (and return itself).
	 * 
	 * @return This as a UObject (null unless overridden by a sub-class).
	 */
	virtual UObject* GetPlaceholderAsUObject()
		PURE_VIRTUAL(FLinkerPlaceholderBase::GetPlaceholderAsUObject, return nullptr;)

	/**
	 * Checks to see if ResolveAllPlaceholderReferences() has been called on 
	 * this placeholder.
	 * 
	 * @return True if ResolveAllPlaceholderReferences() has been called, otherwise false.
	 */
	bool IsMarkedResolved() const;

protected:
	/**
	 * Flags this placeholder as resolved (so that IsMarkedResolved() and 
	 * HasBeenFullyResolved() can return true).
	 */
	void MarkAsResolved();

private:
	/**
	 * Iterates through ReferencingContainers and replaces any (KNOWN) 
	 * references to this placeholder (with 
	 * 
	 * @param  ReplacementObj    
	 * @return 
	 */
	int32 ResolvePlaceholderPropertyValues(UObject* ReplacementObj);
	
	/** Used to catch references that are added after we've already resolved all references */
	bool bResolveWasInvoked;

	/** 
	 * Handily tracks a series of nested properties through an object's class, 
	 * specifically for scenarios where the leaf property's value is referencing
	 * a linker-placeholder object. Used so we can later come back and easily  
	 * resolve (replace) the placeholder value with a legitamate object.
	 */
	struct FPlaceholderValuePropertyPath
	{
		FPlaceholderValuePropertyPath(FFieldVariant ReferencingProperty);

		/** 
		 * Validates that the internal property path points to a FObjectProperty, 
		 * and that the whole thing has a class owner. 
		 */
		bool IsValid() const;

		/**  
		 * Returns the outer class that seemingly owns the property path 
		 * represented by this struct. 
		 */
		UClass* GetOwnerClass() const;

		/**
		 * Replaces the (placeholder) object value at the end of this property
		 * chain with the specified Replacement object.
		 * 
		 * @param  Placeholder	The object value that you're seeking to replace.
		 * @param  Replacement	The new object value for the property represented by this struct.
		 * @param  Container    The object instance that you want the value changed for (within).
		 * @return The number of values successfully replaced (could be multiple for array properties).
		 */
		int32 Resolve(FLinkerPlaceholderBase* Placeholder, UObject* Replacement, UObject* Container) const;
		int32 ResolveRaw(FLinkerPlaceholderBase* Placeholder, UObject* Replacement, void* Container) const;

	private:
		/** Denotes the property hierarchy used to reach this leaf property that is referencing a placeholder*/
		TArray<FFieldVariant> PropertyChain;

	public:
		/** Support comparison functions that make this usable as a KeyValue for a TSet<> */
		friend uint32 GetTypeHash(const FPlaceholderValuePropertyPath& PlaceholderPropertyRef)
		{
			uint32 Hash = 0;
			for (const FFieldVariant& Property : PlaceholderPropertyRef.PropertyChain)
			{
				Hash = HashCombine(Hash, GetTypeHash(Property.GetRawPointer()));
			}
			return Hash;
		}

		/**  */
		friend bool operator==(const FPlaceholderValuePropertyPath& Lhs, const FPlaceholderValuePropertyPath& Rhs)
		{
			if (Lhs.PropertyChain.Num() == Rhs.PropertyChain.Num())
			{
				for (int32 PropIndex = 0; PropIndex < Lhs.PropertyChain.Num(); ++PropIndex)
				{
					if (Lhs.PropertyChain[PropIndex] != Rhs.PropertyChain[PropIndex])
					{
						return false;
					}
				}
				return true;
			}
			return false;
		};
	};
	friend class FLinkerPlaceholderObjectImpl; // for access to FPlaceholderValuePropertyPath
	typedef TSet<FPlaceholderValuePropertyPath> FReferencingPropertySet;

	/** Tracks container objects that have property values set to reference this placeholder (references that need to be replaced later) */
	TMap< TWeakObjectPtr<UObject>, FReferencingPropertySet > ReferencingContainers;
	TMap< void*, FReferencingPropertySet > ReferencingRawContainers;

	/** Data used to keep track of linker placeholders that are waiting for their outer placeholder to be resolved */
	TArray<ULinkerPlaceholderExportObject*> PlaceholderSubobjects;
	ULinkerPlaceholderExportObject* OwningPlaceholder;
};

/*******************************************************************************
 * TLinkerImportPlaceholderBase<>
 ******************************************************************************/
 
template<class PlaceholderType>
class TLinkerImportPlaceholder : public FLinkerPlaceholderBase
{
public:
	// FLinkerPlaceholderBase interface
	virtual bool  HasKnownReferences() const override;
	virtual int32 ResolveAllPlaceholderReferences(UObject* ReplacementObj) override;
	// End FLinkerPlaceholderBase interface

	/**
	 * Records the supplied property so that we can later replace it's 
	 * references to this placeholder with another (real) object.
	 * 
	 * @param  ReferencingProperty	A property that references this placeholder.
	 */
	void AddReferencingProperty(FFieldVariant ReferencingProperty);

	/**
	 * Removes the specified property from this class's internal tracking list 
	 * (which aims to keep track of properties utilizing this placeholder).
	 * 
	 * @param  ReferencingProperty	A property that used to use this placeholder and now no longer does.
	 */
	void RemoveReferencingProperty(FFieldVariant ReferencingProperty);

	/**
	 * Records a raw pointer, directly to the UObject* script expression (so 
	 * that we can switch-out its value in ResolveScriptReferences). 
	 *
	 * NOTE: We don't worry about creating some kind of weak ref to the script 
	 *       pointer (or facilitate a way for this tracked reference to be 
	 *       removed). We're not worried about the script ref being deleted 
	 *       before we call ResolveScriptReferences (because we expect that to
	 *       do this all within the same frame; before GC can be ran).
	 * 
	 * @param  ExpressionPtr    A direct pointer to the UObject* that is now referencing this placeholder.
	 */
	void AddReferencingScriptExpr(PlaceholderType** ExpressionPtr);

	/** 
	 * Records a child placeholder object. Not needed except for validiation purposes,
	 * since both the child object and this are placeholders:
	 */
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	void AddChildObject(UObject* Child);
#endif //USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	/**
	 * Records a derived function, which will have a reference back to its (placeholder) parent function.
	 * We need to update the derived function when the parent finishes loading.
	 */
	void AddDerivedFunction(UStruct* DerivedFunctionType);
private:
	/**
	 * Iterates through all known ReferencingProperties and replaces references
	 * to this placeholder with the supplied replacement object.
	 * 
	 * @param  ReplacementObj	The object that you want all references to this replaced with.
	 * @return The number of references that were successfully replaced.
	 */
	int32 ResolvePropertyReferences(PlaceholderType* ReplacementObj);

	/**
	 * Specialization stored in source file
	 */
	int32 SpecializedResolvePropertyReferences(PlaceholderType* ReplacementObj);

	/**
	 * Iterates through all known ReferencingScriptExpressions and replaces 
	 * references to this placeholder with the specified replacement object.
	 * 
	 * @param  ReplacementObj	The object that you want all references to this replaced with.
	 * @return The number of references that were successfully replaced.
	 */
	int32 ResolveScriptReferences(PlaceholderType* ReplacementObj);

	/** Links to UProperties that are currently directly using this placeholder */
	TSet<FFieldVariant> ReferencingProperties;

	/** Points directly at UObject* refs that were serialized in as part of script bytecode */
	TSet<PlaceholderType**> ReferencingScriptExpressions;

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	/** References to us that are equally transient, this is used in the case where we make a placeholder 
		that requires an outer that is also a placeholder. E.g. when we make a placeholder function it will 
		have a placeholder outer. */
	TArray<UObject*> ChildObjects;
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	TSet<UStruct*> DerivedFunctions;
};

// Templatized implementation
#include "UObject/LinkerPlaceholderBase.inl" // IWYU pragma: export
