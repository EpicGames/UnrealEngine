// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/UObjectHierarchyFwd.h"

class FLinkerPlaceholderBase;
struct FObjectImport;
template<class PlaceholderType> class TLinkerImportPlaceholder;

/*******************************************************************************
 * TLinkerImportPlaceholder<>
 ******************************************************************************/

//------------------------------------------------------------------------------
template<class PlaceholderType>
int32 TLinkerImportPlaceholder<PlaceholderType>::ResolveAllPlaceholderReferences(UObject* ReplacementObj)
{
	if (UObjectRedirector* ReplacementRedirector = Cast<UObjectRedirector>(ReplacementObj))
	{
		if (FLinkerLoad* ReplacementLinker = ReplacementRedirector->GetLinker())
		{
			if (!ReplacementRedirector->HasAnyFlags(RF_LoadCompleted))
			{
				// we're in the midst of serializing this redirector
				// somewhere up the stack, in some scenario like this:
				//
				//		- ClassA and ClassC both depend on ClassB
				//		- ClassB has a redirector to ClassB_2
				//		- ClassB_2 depends on ClassC
				//
				// if ClassA is loaded first, it then goes to load ClassB, which
				// seeks to serialize in its UObjectRedirector; before that's 
				// set it loads ClassB_2 and subsequently ClassC; ClassC ends up
				// here, needing to use the ClassB redirector, but we haven't 
				// returned up the stack for it to be set yet... here we force 
				// it to finish preloading (like we do in VerifyImport): 
				UE::TUniqueLock LinkerLock(ReplacementLinker->Mutex);
				ReplacementRedirector->SetFlags(RF_NeedLoad);
				ReplacementLinker->Preload(ReplacementRedirector);
			}
		}
		ReplacementObj = ReplacementRedirector->DestinationObject;
	}

	PlaceholderType* TypeCheckedReplacement = CastChecked<PlaceholderType>(ReplacementObj, ECastCheckedType::NullAllowed);

	int32 ReplacementCount = ResolvePropertyReferences(TypeCheckedReplacement);
	ReplacementCount += ResolveScriptReferences(TypeCheckedReplacement);

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	ReplacementCount += ChildObjects.Num();
	ChildObjects.Empty();
#endif//USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	ReplacementCount += DerivedFunctions.Num();
	for (auto Entry : DerivedFunctions)
	{
		Entry->SetSuperStruct(TypeCheckedReplacement);
	}
	DerivedFunctions.Empty();

	ReplacementCount += FLinkerPlaceholderBase::ResolveAllPlaceholderReferences(ReplacementObj);

	return ReplacementCount;
}

//------------------------------------------------------------------------------
template<class PlaceholderType>
bool TLinkerImportPlaceholder<PlaceholderType>::HasKnownReferences() const
{

	return FLinkerPlaceholderBase::HasKnownReferences() || 
		(ReferencingProperties.Num() > 0) || 
		(ReferencingScriptExpressions.Num() > 0) ||
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		ChildObjects.Num() > 0 || 
#endif //USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		DerivedFunctions.Num() > 0;
}

//------------------------------------------------------------------------------
template<class PlaceholderType>
void TLinkerImportPlaceholder<PlaceholderType>::AddReferencingProperty(FFieldVariant ReferencingProperty)
{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	UObject* ThisAsObject = GetPlaceholderAsUObject();
	check(ThisAsObject != nullptr);

	if (!ReferencingProperty.IsUObject())
	{
		if (FLinkerLoad* PropertyLinker = ReferencingProperty.Get<FProperty>()->GetLinker())
		{
			check(ThisAsObject->GetOutermost() == PropertyLinker->LinkerRoot);
			check(PropertyLinker->LoadFlags & LOAD_DeferDependencyLoads);
		}
	}
	// if this check hits, then we're adding dependencies after we've 
	// already resolved the placeholder (it won't be resolved again)
	check(!IsMarkedResolved());
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	ReferencingProperties.Add(ReferencingProperty);
}

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
//------------------------------------------------------------------------------
template<class PlaceholderType>
void TLinkerImportPlaceholder<PlaceholderType>::AddChildObject(UObject* Child)
{
	ChildObjects.Push(Child);
}
#endif //USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

//------------------------------------------------------------------------------
template<class PlaceholderType>
void TLinkerImportPlaceholder<PlaceholderType>::AddDerivedFunction(UStruct* DerivedFunctionType)
{
	DerivedFunctions.Add(DerivedFunctionType);
}

//------------------------------------------------------------------------------
template<class PlaceholderType>
void TLinkerImportPlaceholder<PlaceholderType>::RemoveReferencingProperty(FFieldVariant ReferencingProperty)
{
	ReferencingProperties.Remove(ReferencingProperty);
}

//------------------------------------------------------------------------------
template<class PlaceholderType>
void TLinkerImportPlaceholder<PlaceholderType>::AddReferencingScriptExpr(PlaceholderType** ExpressionPtr)
{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	check(*ExpressionPtr == GetPlaceholderAsUObject());
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	ReferencingScriptExpressions.Add(ExpressionPtr);
}

//------------------------------------------------------------------------------
template<class PlaceholderType>
int32 TLinkerImportPlaceholder<PlaceholderType>::ResolvePropertyReferences(PlaceholderType* ReplacementObj)
{
	if constexpr (std::is_same_v<PlaceholderType, UClass> || std::is_same_v<PlaceholderType, UFunction>)
	{
		return SpecializedResolvePropertyReferences(ReplacementObj);
	}
	else
	{
#if !CHECK_PUREVIRTUALS
		// requires template specialization (technically not "pure virtual"):
		PURE_VIRTUAL(TLinkerImportPlaceholder<PlaceholderType>::ResolvePropertyReferences, return 0;)
#endif
	}
}

//------------------------------------------------------------------------------
template<class PlaceholderType>
int32 TLinkerImportPlaceholder<PlaceholderType>::ResolveScriptReferences(PlaceholderType* ReplacementObj)
{
	int32 ReplacementCount = 0;
	PlaceholderType* PlaceholderObj = CastChecked<PlaceholderType>(GetPlaceholderAsUObject());

	for (PlaceholderType** ScriptRefPtr : ReferencingScriptExpressions)
	{
		PlaceholderType*& ScriptRef = *ScriptRefPtr;
		if (ScriptRef == PlaceholderObj)
		{
			ScriptRef = ReplacementObj;
			++ReplacementCount;
		}
	}

	ReferencingScriptExpressions.Empty();
	return ReplacementCount;
}
