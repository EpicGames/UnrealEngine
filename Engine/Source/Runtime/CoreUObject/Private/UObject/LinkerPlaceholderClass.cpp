// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/Package.h"

//------------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(LinkerPlaceholderClass)
ULinkerPlaceholderClass::ULinkerPlaceholderClass(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
void ULinkerPlaceholderClass::BeginDestroy()
{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	check(IsMarkedResolved() || HasAnyFlags(RF_ClassDefaultObject));
	check(!HasKnownReferences());
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	// by this point, we really shouldn't have any properties left (they should 
	// have all got replaced), but just in case (so things don't blow up with a
	// missing class)...
	ResolveAllPlaceholderReferences(UObject::StaticClass());

	Super::BeginDestroy();
}

//------------------------------------------------------------------------------
void ULinkerPlaceholderClass::Bind()
{
	ClassConstructor = InternalConstructor<ULinkerPlaceholderClass>;
	ClassVTableHelperCtorCaller = InternalVTableHelperCtorCaller<ULinkerPlaceholderClass>;
	Super::Bind();

	CppClassStaticFunctions = UOBJECT_CPPCLASS_STATICFUNCTIONS_FORCLASS(ULinkerPlaceholderClass);
}


