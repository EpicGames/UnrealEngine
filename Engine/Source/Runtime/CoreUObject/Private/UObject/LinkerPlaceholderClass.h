// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Class.h"
#include "UObject/LinkerPlaceholderBase.h"
#include "Async/UniqueLock.h"

#include "LinkerPlaceholderClass.generated.h"

/**  
 * A utility class for the deferred dependency loader, used to stub in temporary
 * class references so we don't have to load blueprint resources for their class.
 * Holds on to references where this is currently being utilized, so we can 
 * easily replace references to it later (once the real class is available).
 */ 
UCLASS(MinimalAPI)
class ULinkerPlaceholderClass : public UClass, public TLinkerImportPlaceholder<UClass>
{
public:
	GENERATED_BODY()

	ULinkerPlaceholderClass(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	// UObject interface.
	virtual void BeginDestroy() override;
	// End of UObject interface.

	// UField interface.
 	virtual void Bind() override;
	// End of UField interface.

	// FLinkerPlaceholderBase interface 
	virtual UObject* GetPlaceholderAsUObject() override { return (UObject*)(this); }
	// End of FLinkerPlaceholderBase interface
}; 

