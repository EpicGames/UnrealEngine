// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerPlaceholderExportObject.h"

//------------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(LinkerPlaceholderExportObject)
ULinkerPlaceholderExportObject::ULinkerPlaceholderExportObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

//------------------------------------------------------------------------------
void ULinkerPlaceholderExportObject::BeginDestroy()
{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	check(IsMarkedResolved() || HasAnyFlags(RF_ClassDefaultObject));
	check(!HasKnownReferences());
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	Super::BeginDestroy();
}
