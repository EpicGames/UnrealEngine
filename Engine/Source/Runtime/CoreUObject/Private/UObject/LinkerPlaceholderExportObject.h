// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Object.h"
#include "UObject/LinkerPlaceholderBase.h"

#include "LinkerPlaceholderExportObject.generated.h"

/**  
 * A utility class for the deferred dependency loader, used to stub in temporary
 * export objects so we don't spawn any Blueprint class instances before the 
 * class is fully regenerated.
 */ 
UCLASS(MinimalAPI)
class ULinkerPlaceholderExportObject : public UObject, public FLinkerPlaceholderBase
{
public:
	GENERATED_BODY()

	ULinkerPlaceholderExportObject(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	// UObject interface
	virtual void BeginDestroy() override;
	// End of UObject interface

	// FLinkerPlaceholderBase interface
	virtual UObject* GetPlaceholderAsUObject() override { return (UObject*)(this); }
	// End of FLinkerPlaceholderBase interface
}; 
