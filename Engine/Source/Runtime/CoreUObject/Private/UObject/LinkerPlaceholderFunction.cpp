// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerPlaceholderFunction.h"

//------------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(LinkerPlaceholderFunction)
ULinkerPlaceholderFunction::ULinkerPlaceholderFunction(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

