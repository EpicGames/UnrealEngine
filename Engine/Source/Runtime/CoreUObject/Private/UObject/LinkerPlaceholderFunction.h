// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "UObject/LinkerPlaceholderBase.h"
#include "Async/UniqueLock.h"

#include "LinkerPlaceholderFunction.generated.h"

/**  */
UCLASS(MinimalAPI)
class ULinkerPlaceholderFunction : public UFunction, public TLinkerImportPlaceholder<UFunction>
{
public:
	GENERATED_BODY()

	ULinkerPlaceholderFunction(const FObjectInitializer& ObjectInitializer);

	// FLinkerPlaceholderBase interface 
	virtual UObject* GetPlaceholderAsUObject() override { return (UObject*)(this); }
	// End of FLinkerPlaceholderBase interface
};
