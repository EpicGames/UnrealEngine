// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/LinkerSave.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "Misc/StringBuilder.h"
#include "Serialization/BulkData.h"
#include "Serialization/LargeMemoryWriter.h"
#include "UObject/Package.h"
#include "UObject/Class.h"
#include "Templates/Casts.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "UObject/SavePackage/SaveContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/UnrealType.h"
#include "HAL/PlatformStackWalk.h"
#if WITH_EDITORONLY_DATA
#include "IO/IoDispatcher.h"
#include "Serialization/DerivedData.h"
#endif

/*----------------------------------------------------------------------------
	FLinkerSave.
----------------------------------------------------------------------------*/

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TMap<FString, TArray<uint8> > FLinkerSave::PackagesToScriptSHAMap;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FLinkerSave::FLinkerSave(UPackage* InParent)
	: FLinker(ELinkerType::Save, InParent)
{
	check(InParent);
}

FLinkerSave::FLinkerSave(UPackage* InParent, const TCHAR* InFilename, bool bForceByteSwapping, bool bInSaveUnversioned)
	: FLinkerSave(InParent)
{
	(void) TryAssignFileSaver(FStringView(InFilename), bForceByteSwapping, bInSaveUnversioned);
}

FLinkerSave::FLinkerSave(UPackage* InParent, FArchive* InSaver, bool bForceByteSwapping, bool bInSaveUnversioned)
	: FLinkerSave(InParent)
{
	AssignSaver(InSaver, bForceByteSwapping, bInSaveUnversioned);
}

FLinkerSave::FLinkerSave(UPackage* InParent, bool bForceByteSwapping, bool bInSaveUnversioned)
	: FLinkerSave(InParent)
{
	AssignMemorySaver(bForceByteSwapping, bInSaveUnversioned);
}

void FLinkerSave::AssignSaver(FArchive* InSaver, bool bForceByteSwapping, bool bInSaveUnversioned)
{
	SetFilename(TEXT("$$Memory$$"));
	AssignSaverInternal(InSaver, bForceByteSwapping, bInSaveUnversioned);
}

void FLinkerSave::AssignMemorySaver(bool bForceByteSwapping, bool bInSaveUnversioned)
{
	check(LinkerRoot); // Must be non-null in constructor

	SetFilename(TEXT("$$Memory$$"));
	FArchive* LocalSaver = new FLargeMemoryWriter(0, false, *LinkerRoot->GetLoadedPath().GetDebugName());
	AssignSaverInternal(LocalSaver, bForceByteSwapping, bInSaveUnversioned);
}

bool FLinkerSave::TryAssignFileSaver(FStringView InFilename, bool bForceByteSwapping, bool bInSaveUnversioned)
{
	SetFilename(InFilename);
	// Create file saver.
	FArchive* LocalSaver = IFileManager::Get().CreateFileWriter(*WriteToString<256>(InFilename), 0);
	if (!LocalSaver)
	{
		TCHAR LastErrorText[1024];
		uint32 LastError = FPlatformMisc::GetLastError();
		if (LastError != 0)
		{
			FPlatformMisc::GetSystemErrorMessage(LastErrorText, UE_ARRAY_COUNT(LastErrorText), LastError);
		}
		else
		{
			FCString::Strcpy(LastErrorText, TEXT("Unknown failure reason."));
		}
		UE_LOGF(LogLinker, Error, "Error opening file '%.*ls': %ls",
			InFilename.Len(), InFilename.GetData(), LastErrorText);
		return false;
	}
	AssignSaverInternal(LocalSaver, bForceByteSwapping, bInSaveUnversioned);
	return true;
}

void FLinkerSave::AssignSaverInternal(FArchive* InSaver, bool bForceByteSwapping, bool bInSaveUnversioned)
{
	check(InSaver);
	check(LinkerRoot); // Must be non-null in constructor

	Saver = InSaver;
	UPackage* Package = LinkerRoot;

	// Set main summary info.
	Summary.Tag = PACKAGE_FILE_TAG;
	Summary.SetToLatestFileVersions(bInSaveUnversioned);
	Summary.SavedByEngineVersion = FEngineVersion::Current();
	Summary.CompatibleWithEngineVersion = FEngineVersion::CompatibleWith();
	Summary.SetPackageFlags(Package ? Package->GetPackageFlags() : 0);

#if USE_STABLE_LOCALIZATION_KEYS
	if (GIsEditor)
	{
		Summary.LocalizationId = TextNamespaceUtil::GetPackageNamespace(LinkerRoot);
		SetLocalizationNamespace(Summary.LocalizationId);
	}
#endif // USE_STABLE_LOCALIZATION_KEYS

#if WITH_EDITORONLY_DATA
	Summary.PackageName = Package->GetName();
#endif
	Summary.ChunkIDs = Package->GetChunkIDs();

	// Set FArchive flags.
	this->SetIsSaving(true);
	this->SetIsPersistent(true);
	ArForceByteSwapping = bForceByteSwapping;
#if WITH_EDITOR
	ArDebugSerializationFlags = Saver->ArDebugSerializationFlags;
#endif
}

bool FLinkerSave::CloseAndDestroySaver()
{
	delete Saver;
	Saver = nullptr;
	return true;
}

FLinkerSave::~FLinkerSave()
{
	CloseAndDestroySaver();
}

bool FLinkerSave::IsUnsaveable(TObjectPtr<UObject> Object) const
{
	if (SaveContext)
	{
		return SaveContext->IsUnsaveable(Object, SaveContext->GetCachedObjectStatus(Object), /*bEmitWarning*/false);
	}
	return false;
}

int32 FLinkerSave::MapName(FNameEntryId Id) const
{
	const int32* IndexPtr = NameIndices.Find(Id);

	if (IndexPtr)
	{
		return *IndexPtr;
	}

	return INDEX_NONE;
}

int32 FLinkerSave::MapSoftObjectPath(const FSoftObjectPath& SoftObjectPath) const
{
	const int32* IndexPtr = SoftObjectPathIndices.Find(SoftObjectPath);

	if (IndexPtr)
	{
		return *IndexPtr;
	}

	return INDEX_NONE;
}


FPackageIndex FLinkerSave::MapObject(TObjectPtr<const UObject> Object, bool bValidateExcluded) const
{
	if (Object)
	{
		const FPackageIndex *Found = ObjectIndicesMap.Find(Object);

		if (Found)
		{
			if (IsCooking() && CurrentlySavingExport.IsExport() &&
				Object.GetPackage().GetFName() != GLongCoreUObjectPackageName && // We assume nothing in coreuobject ever loads assets in a constructor
				*Found != CurrentlySavingExport) // would be weird, but I can't be a dependency on myself
			{
				const FObjectExport& SavingExport = Exp(CurrentlySavingExport);
				bool bFoundDep = false;
				if (SavingExport.FirstExportDependency >= 0)
				{
					int32 NumDeps = SavingExport.CreateBeforeCreateDependencies + SavingExport.CreateBeforeSerializationDependencies + SavingExport.SerializationBeforeCreateDependencies + SavingExport.SerializationBeforeSerializationDependencies;
					for (int32 DepIndex = SavingExport.FirstExportDependency; DepIndex < SavingExport.FirstExportDependency + NumDeps; DepIndex++)
					{
						if (DepListForErrorChecking[DepIndex] == *Found)
						{
							bFoundDep = true;
							break;
						}
					}
				}
				if (!bFoundDep)
				{
					if (SavingExport.Object && SavingExport.Object->IsA(UClass::StaticClass()))
					{
						UClass* Class = CastChecked<UClass>(SavingExport.Object);
						if (Class->GetDefaultObject() == Object
					#if WITH_EDITORONLY_DATA
							|| Class->ClassGeneratedBy == Object
					#endif
							)
						{
							bFoundDep = true; // the class is saving a ref to the CDO...which doesn't really work or do anything useful, but it isn't an error or it is saving a reference to the class that generated it 
						}
					}
				}
				if (!bFoundDep)
				{
					const FString ImpExpObjectNameString = ImpExp(*Found).ObjectName.ToString();
					const bool IsNativeDep = FPackageName::IsScriptPackage(ImpExpObjectNameString);
					if (!IsNativeDep)
					{
						UE_LOGF(LogLinker, Fatal, "Attempt to map an object during save that was not listed as a dependency. Saving Export %d %ls in %ls. Missing Dep on %ls %ls.",
							CurrentlySavingExport.ForDebugging(), *SavingExport.ObjectName.ToString(), *GetArchiveName(),
							Found->IsExport() ? TEXT("Export") : TEXT("Import"), *ImpExpObjectNameString
							);
					}
				}
			}

			return *Found;
		}
		else if (bValidateExcluded && SaveContext)
		{
			UE_CLOGF(SaveContext->FindCachedObjectStatus(Object) == nullptr, LogLinker, Error,
					"Invalid serialization during save of Export '%ls' in package '%ls'. Object '%ls' encountered during serialization without being previously harvested. This will cause the reference to be null at runtime. ",
					(CurrentlySavingExport.IsExport() ? *Exp(CurrentlySavingExport).ObjectName.ToString() : TEXT("Unknown")),
					*LinkerRoot->GetPathName(),
					*Object->GetPathName()
				);
		}
	}
	return FPackageIndex();
}

void FLinkerSave::MarkScriptSerializationStart(const UObject* Obj) 
{
	if (ensure(Obj == CurrentlySavingExportObject))
	{
		FObjectExport& Export = ExportMap[CurrentlySavingExport.ToExport()];
		Export.ScriptSerializationStartOffset = Tell();
	}
}

void FLinkerSave::MarkScriptSerializationEnd(const UObject* Obj) 
{
	if (ensure(Obj == CurrentlySavingExportObject))
	{
		FObjectExport& Export = ExportMap[CurrentlySavingExport.ToExport()];
		Export.ScriptSerializationEndOffset = Tell();
	}
}

void FLinkerSave::Seek( int64 InPos )
{
	Saver->Seek( InPos );
}

int64 FLinkerSave::Tell()
{
	return Saver->Tell();
}

void FLinkerSave::Serialize( void* V, int64 Length )
{
	Saver->Serialize( V, Length );
}

void FLinkerSave::OnPostSave(const FPackagePath& PackagePath, FObjectPostSaveContext ObjectSaveContext)
{
	for (TUniqueFunction<void(const FPackagePath&, FObjectPostSaveContext)>& Callback : PostSaveCallbacks)
	{
		Callback(PackagePath, ObjectSaveContext);
	}

	PostSaveCallbacks.Empty();
}
	
FString FLinkerSave::GetDebugName() const
{
	return GetFilename();
}

const FString& FLinkerSave::GetFilename() const
{
	return Filename;
}

void FLinkerSave::SetFilename(FStringView InFilename)
{
	Filename = FString(InFilename);
}

FString FLinkerSave::GetArchiveName() const
{
	return Saver->GetArchiveName();
}

FArchive& FLinkerSave::operator<<( FName& InName )
{
	int32 Save = MapName(InName.GetDisplayIndex());

	bool bNameMapped = Save != INDEX_NONE;
	if (!bNameMapped)
	{
		// Set an error on the archive and record the error on the log output if one is set.
		SetCriticalError();
		FString ErrorMessage = FString::Printf(TEXT("Name \"%s\" is not mapped when saving %s (object: %s, property: %s). This can mean that this object serialize function is not deterministic between reference harvesting and serialization."),
			*InName.ToString(),
			*GetArchiveName(),
			*FUObjectThreadContext::Get().GetSerializeContext()->SerializedObject->GetFullName(),
			*GetFullNameSafe(GetSerializedProperty()));
		ensureMsgf(false, TEXT("%s"), *ErrorMessage);
		if (LogOutput)
		{
			LogOutput->Logf(ELogVerbosity::Error, TEXT("%s"), *ErrorMessage);
		}
	}

	if (!CurrentlySavingExport.IsNull())
	{
		if (Save >= Summary.NamesReferencedFromExportDataCount)
		{
			SetCriticalError();
			FString ErrorMessage = FString::Printf(TEXT("Name \"%s\" is referenced from an export but not mapped in the export data names region when saving %s (object: %s, property: %s)."),
				*InName.ToString(),
				*GetArchiveName(),
				*FUObjectThreadContext::Get().GetSerializeContext()->SerializedObject->GetFullName(),
				*GetFullNameSafe(GetSerializedProperty()));
			ensureMsgf(false, TEXT("%s"), *ErrorMessage);
			if (LogOutput)
			{
				LogOutput->Logf(ELogVerbosity::Error, TEXT("%s"), *ErrorMessage);
			}
		}
	}

	int32 Number = InName.GetNumber();
	FArchive& Ar = *this;
	return Ar << Save << Number;
}

FArchive& FLinkerSave::operator<<(UObject*& Obj)
{
	SerializeObjectPointer(FObjectPtr(Obj));
	return *this;
}

FArchive& FLinkerSave::operator<<(FObjectPtr& Value)
{
	SerializeObjectPointer(Value);
	return *this;
}

void FLinkerSave::SerializeObjectPointer(const FObjectPtr& Obj)
{
	FPackageIndex Save;
	if (Obj)
	{
		Save = MapObject(TObjectPtr<const UObject>(Obj));
	}
	*this << Save;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FArchive& FLinkerSave::operator<<(Verse::VCell*& Cell)
{
	FPackageIndex Save;
	if (Cell)
	{
		if (const FPackageIndex* Found = CellIndicesMap.Find(Cell))
		{
			Save = *Found;
		}
	}
	*this << Save;
	return *this;
}
#endif

FArchive& FLinkerSave::operator<<(FSoftObjectPath& SoftObjectPath)
{
	// Map soft object path to indices if we aren't currently serializing the list itself
	// and we actually built one, cooking might want to serialize soft object path directly for example
	if (!bIsWritingHeaderSoftObjectPaths && SoftObjectPathList.Num() > 0)
	{
		int32 Save = MapSoftObjectPath(SoftObjectPath);
		bool bPathMapped = Save != INDEX_NONE;
		if (!bPathMapped)
		{
			// Set an error on the archive and record the error on the log output if one is set.
			SetCriticalError();
			FString ErrorMessage = FString::Printf(TEXT("SoftObjectPath \"%s\" is not mapped when saving %s (object: %s, property: %s). This can mean that this object serialize function is not deterministic between reference harvesting and serialization."),
				*SoftObjectPath.ToString(),
				*GetArchiveName(),
				*FUObjectThreadContext::Get().GetSerializeContext()->SerializedObject->GetFullName(),
				*GetFullNameSafe(GetSerializedProperty()));
			ensureMsgf(false, TEXT("%s"), *ErrorMessage);
			if (LogOutput)
			{
				LogOutput->Logf(ELogVerbosity::Error, TEXT("%s"), *ErrorMessage);
			}
		}
		return *this << Save;
	}
	else
	{
		return FArchiveUObject::operator<<(SoftObjectPath);
	}
}

FArchive& FLinkerSave::operator<<(FLazyObjectPtr& LazyObjectPtr)
{
	FUniqueObjectGuid ID;
	ID = LazyObjectPtr.GetUniqueID();
	return *this << ID;
}

bool FLinkerSave::ShouldSkipProperty(const FProperty* InProperty) const
{
	if (TransientPropertyOverrides && !TransientPropertyOverrides->IsEmpty())
	{
		const TSet<FProperty*>* Props = TransientPropertyOverrides->Find(CurrentlySavingExportObject);
		if (Props && Props->Contains(InProperty))
		{
			return true;
		}
	}
	return false;
}

void FLinkerSave::UsingCustomVersion(const struct FGuid& Guid)
{
	FArchiveUObject::UsingCustomVersion(Guid);

	// Here we're going to try and dump the callstack that added a new custom version after package summary has been serialized
	if (Summary.GetCustomVersionContainer().GetVersion(Guid) == nullptr)
	{
		FCustomVersion RegisteredVersion = FCurrentCustomVersions::Get(Guid).GetValue();

		FString CustomVersionWarning = FString::Printf(TEXT("Unexpected custom version \"%s\" used after package %s summary has been serialized. Callstack:\n"),
			*RegisteredVersion.GetFriendlyName().ToString(), *LinkerRoot->GetName());

		const int32 MaxStackFrames = 100;
		uint64 StackFrames[MaxStackFrames];
		int32 NumStackFrames = FPlatformStackWalk::CaptureStackBackTrace(StackFrames, MaxStackFrames);

		// Convert the stack trace to text, ignore the first functions (ProgramCounterToHumanReadableString)
		const int32 IgnoreStackLinesCount = 1;		
		ANSICHAR Buffer[1024];
		const ANSICHAR* CutoffFunction = "UPackage::Save";
		for (int32 Idx = IgnoreStackLinesCount; Idx < NumStackFrames; Idx++)
		{			
			Buffer[0] = '\0';
			FPlatformStackWalk::ProgramCounterToHumanReadableString(Idx, StackFrames[Idx], Buffer, sizeof(Buffer));
			CustomVersionWarning += TEXT("\t");
			CustomVersionWarning += Buffer;
			CustomVersionWarning += "\n";
			if (FCStringAnsi::Strstr(Buffer, CutoffFunction))
			{
				// Anything below UPackage::Save is not interesting from the point of view of what we're trying to find
				break;
			}
		}

		UE_LOGF(LogLinker, Warning, "%ls", *CustomVersionWarning);
	}
}

void FLinkerSave::SetUseUnversionedPropertySerialization(bool bInUseUnversioned)
{
	check(Saver); // Must be set before calling FArchive functions
	check(LinkerRoot); // Must be non-null in constructor

	FArchiveUObject::SetUseUnversionedPropertySerialization(bInUseUnversioned);
	Saver->SetUseUnversionedPropertySerialization(bInUseUnversioned);
	if (bInUseUnversioned)
	{
		Summary.SetPackageFlags(Summary.GetPackageFlags() | PKG_UnversionedProperties);
		LinkerRoot->SetPackageFlags(PKG_UnversionedProperties);
	}
	else
	{
		Summary.SetPackageFlags(Summary.GetPackageFlags() & ~PKG_UnversionedProperties);
		LinkerRoot->ClearPackageFlags(PKG_UnversionedProperties);
	}
}

void FLinkerSave::SetDebugSerializationFlags(uint32 InCustomFlags)
{
	check(Saver); // Must be set before calling FArchive functions

	FArchiveUObject::SetDebugSerializationFlags(InCustomFlags);
	Saver->SetDebugSerializationFlags(InCustomFlags);
}

void FLinkerSave::SetFilterEditorOnly(bool bInFilterEditorOnly)
{
	check(Saver); // Must be set before calling FArchive functions
	check(LinkerRoot); // Must be non-null in constructor

	FArchiveUObject::SetFilterEditorOnly(bInFilterEditorOnly);
	Saver->SetFilterEditorOnly(bInFilterEditorOnly);
	if (bInFilterEditorOnly)
	{
		Summary.SetPackageFlags(Summary.GetPackageFlags() | PKG_FilterEditorOnly);
		LinkerRoot->SetPackageFlags(PKG_FilterEditorOnly);
	}
	else
	{
		Summary.SetPackageFlags(Summary.GetPackageFlags() & ~PKG_FilterEditorOnly);
		LinkerRoot->ClearPackageFlags(PKG_FilterEditorOnly);
	}
}

#if WITH_EDITORONLY_DATA
UE::FDerivedData FLinkerSave::AddDerivedData(const UE::FDerivedData& Data)
{
	UE_LOGF(LogLinker, Warning, "Data will not be able to load because derived data is not saved yet.");

	UE::DerivedData::Private::FCookedData CookedData;

	const FPackageId PackageId = FPackageId::FromName(LinkerRoot->GetFName());
	const int32 ChunkIndex = ++LastDerivedDataIndex;
	checkf(ChunkIndex >= 0 && ChunkIndex < (1 << 24), TEXT("ChunkIndex %d is out of range."), ChunkIndex);

	// PackageId                 ChunkIndex Type
	// [00 01 02 03 04 05 06 07] [08 09 10] [11]
	*reinterpret_cast<uint8*>(&CookedData.ChunkId[11]) = static_cast<uint8>(EIoChunkType::DerivedData);
	*reinterpret_cast<uint32*>(&CookedData.ChunkId[7]) = NETWORK_ORDER32(ChunkIndex);
	*reinterpret_cast<uint64*>(&CookedData.ChunkId[0]) = PackageId.Value();

	CookedData.Flags = Data.GetFlags();
	return UE::FDerivedData(CookedData);
}
#endif // WITH_EDITORONLY_DATA

bool FLinkerSave::SerializeBulkData(FBulkData& BulkData, const FBulkDataSerializationParams& Params) 
{
	using namespace UE::BulkData::Private;

	auto CanSaveBulkDataByReference = [](FBulkData& BulkData) -> bool
	{
		return BulkData.GetBulkDataOffsetInFile() != INDEX_NONE &&
			// We don't support yet loading from a separate file
			!BulkData.IsInSeparateFile() &&
			// It is possible to have a BulkData marked as optional without putting it into a separate file, and we
			// assume that if BulkData is optional and in a separate file, then it is in the BulkDataOptional
			// segment. Rather than changing that assumption to support optional ExternalResource bulkdata, we
			// instead require that optional inlined/endofpackagedata BulkDatas can not be read from an
			// ExternalResource and must remain inline.
			!BulkData.IsOptional() &&
			// Inline or end-of-package-file data can only be loaded from the workspace domain package file if the
			// archive used by the bulk data was actually from the package file; BULKDATA_LazyLoadable is set by
			// Serialize iff that is the case										
			(BulkData.GetBulkDataFlags() & BULKDATA_LazyLoadable);
	};
	
	if (ShouldSkipBulkData())
	{
		return false;
	}

	const EBulkDataFlags BulkDataFlags	= static_cast<EBulkDataFlags>(BulkData.GetBulkDataFlags());
	int32 ResourceIndex					= DataResourceMap.Num();
	int64 PayloadSize					= BulkData.GetBulkDataSize();
	const bool bSupportsMemoryMapping	= IsCooking() && MemoryMappingAlignment >= 0;
	const bool bSaveAsResourceIndex		= IsCooking();

#if USE_RUNTIME_BULKDATA
	const bool bCustomElementSerialization = false;
#else
	const bool bCustomElementSerialization = BulkData.SerializeBulkDataElements != nullptr;
#endif
	
	TOptional<EFileRegionType> RegionToUse;
	if (bFileRegionsEnabled)
	{
		if (IsCooking())
		{
			RegionToUse = Params.RegionType;
		}
		else if (bDeclareRegionForEachAdditionalFile)
		{
			RegionToUse = EFileRegionType::None;
		}
	}
	FBulkMetaResource SerializedMeta;
	SerializedMeta.Flags = BulkDataFlags;
	SerializedMeta.ElementCount = PayloadSize / Params.ElementSize;
	SerializedMeta.SizeOnDisk = PayloadSize;

	if (bCustomElementSerialization)
	{
		// Force 64 bit precision when using custom element serialization
		FBulkData::SetBulkDataFlagsOn(SerializedMeta.Flags, static_cast<EBulkDataFlags>(BULKDATA_Size64Bit));
	}

	EBulkDataFlags FlagsToClear = static_cast<EBulkDataFlags>(BULKDATA_PayloadAtEndOfFile | BULKDATA_PayloadInSeparateFile | BULKDATA_WorkspaceDomainPayload | BULKDATA_ForceSingleElementSerialization | BULKDATA_NoOffsetFixUp);
	if (IsCooking())
	{
		FBulkData::SetBulkDataFlagsOn(FlagsToClear, static_cast<EBulkDataFlags>(BULKDATA_SerializeCompressed));
	}

	FBulkData::ClearBulkDataFlagsOn(SerializedMeta.Flags, FlagsToClear);

	const bool bSerializeInline =
		FBulkData::HasFlags(BulkDataFlags, BULKDATA_ForceInlinePayload) ||
		(IsCooking() && (FBulkData::HasFlags(BulkDataFlags, BULKDATA_Force_NOT_InlinePayload) == false)) ||
		IsTextFormat();

	if (bSerializeInline)
	{
		FArchive& Ar = *this;

		const int64 MetaOffset = Tell();
		if (bSaveAsResourceIndex)
		{
			Ar << ResourceIndex;
		}
		else
		{
			Ar << SerializedMeta;
		}

		SerializedMeta.Offset = Tell();
		SerializedMeta.SizeOnDisk = BulkData.SerializePayload(Ar, SerializedMeta.Flags, RegionToUse);
		if (bCustomElementSerialization)
		{
			PayloadSize = SerializedMeta.SizeOnDisk;
			SerializedMeta.ElementCount = PayloadSize / Params.ElementSize; 
		}

		if (bSaveAsResourceIndex == false)
		{
			FArchive::FScopeSeekTo _(Ar, MetaOffset);
			Ar << SerializedMeta;
		}
	}
	else
	{
		FBulkData::SetBulkDataFlagsOn(SerializedMeta.Flags, static_cast<EBulkDataFlags>(BULKDATA_PayloadAtEndOfFile));

		if (bSaveBulkDataToSeparateFiles)
		{
			check(bSaveBulkDataByReference == false);
			FBulkData::SetBulkDataFlagsOn(SerializedMeta.Flags, static_cast<EBulkDataFlags>(BULKDATA_PayloadInSeparateFile | BULKDATA_NoOffsetFixUp));
		}
		
		const bool bSaveByReference = bSaveBulkDataByReference && CanSaveBulkDataByReference(BulkData);
		if (bSaveByReference)
		{
			check(IsCooking() == false);
			FBulkData::SetBulkDataFlagsOn(SerializedMeta.Flags, static_cast<EBulkDataFlags>(BULKDATA_NoOffsetFixUp | BULKDATA_WorkspaceDomainPayload | BULKDATA_PayloadInSeparateFile));
		}

		if (bSaveBulkDataToSeparateFiles && FBulkData::HasFlags(SerializedMeta.Flags, BULKDATA_OptionalPayload))
		{
			FFileRegionMemoryWriter& Ar = GetOptionalBulkDataArchive(Params.CookedIndex);

			SerializedMeta.Offset = Ar.Tell();
			SerializedMeta.SizeOnDisk = BulkData.SerializePayload(Ar, SerializedMeta.Flags, RegionToUse);
		}
		else if (bSaveBulkDataToSeparateFiles && FBulkData::HasFlags(SerializedMeta.Flags, BULKDATA_MemoryMappedPayload) && bSupportsMemoryMapping)
		{
#if UE_DISABLE_COOKEDINDEX_FOR_MEMORYMAPPED
			UE_CLOGF(!Params.CookedIndex.IsDefault(), LogLinker, Warning, "%ls: Cooked Index is not supported for MemoryMappedPayloads, value will be ignored", *LinkerRoot->GetName());

			FFileRegionMemoryWriter& Ar = GetMemoryMappedBulkDataArchive(FBulkDataCookedIndex::Default);
#else
			FFileRegionMemoryWriter& Ar = GetMemoryMappedBulkDataArchive(Params.CookedIndex);
#endif // UE_DISABLE_COOKEDINDEX_FOR_MEMORYMAPPED

			if (int64 Padding = Align(Ar.Tell(), MemoryMappingAlignment) - Ar.Tell(); Padding > 0)
			{
				TArray<uint8> Zeros;
				Zeros.SetNumZeroed(int32(Padding));
				Ar.Serialize(Zeros.GetData(), Padding);
			}
			SerializedMeta.Offset = Ar.Tell();
			SerializedMeta.SizeOnDisk = BulkData.SerializePayload(Ar, SerializedMeta.Flags, RegionToUse);
		}
		else
		{
			if (bSaveBulkDataToSeparateFiles && FBulkData::HasFlags(SerializedMeta.Flags, BULKDATA_DuplicateNonOptionalPayload))
			{
#if UE_DISABLE_COOKEDINDEX_FOR_NONDUPLICATE
				UE_CLOGF(!Params.CookedIndex.IsDefault(), LogLinker, Warning, "%ls: Cooked Index is not supported for DuplicateNonOptionalPayloads, value will be ignored", *LinkerRoot->GetName());

				FFileRegionMemoryWriter& OptionalAr = GetOptionalBulkDataArchive(FBulkDataCookedIndex::Default);
#else
				FFileRegionMemoryWriter& OptionalAr = GetOptionalBulkDataArchive(Params.CookedIndex);
#endif // UE_DISABLE_COOKEDINDEX_FOR_NONDUPLICATE


				SerializedMeta.DuplicateFlags = SerializedMeta.Flags;
				SerializedMeta.DuplicateOffset = OptionalAr.Tell();
				SerializedMeta.DuplicateSizeOnDisk = BulkData.SerializePayload(OptionalAr, SerializedMeta.Flags, RegionToUse);

				FBulkData::ClearBulkDataFlagsOn(SerializedMeta.DuplicateFlags, BULKDATA_DuplicateNonOptionalPayload);
				FBulkData::SetBulkDataFlagsOn(SerializedMeta.DuplicateFlags, BULKDATA_OptionalPayload);
			}

			if (bSaveByReference)
			{
				SerializedMeta.Offset = BulkData.GetBulkDataOffsetInFile();
				SerializedMeta.SizeOnDisk = BulkData.GetBulkDataSizeOnDisk();
			}
			else
			{
				FFileRegionMemoryWriter& Ar = GetBulkDataArchive(Params.CookedIndex);
				
				SerializedMeta.Offset = Ar.Tell();
				SerializedMeta.SizeOnDisk = BulkData.SerializePayload(Ar, SerializedMeta.Flags, RegionToUse);
			}
		}

		if (bCustomElementSerialization)
		{
			PayloadSize = SerializedMeta.SizeOnDisk;
			SerializedMeta.ElementCount = PayloadSize / Params.ElementSize; 
		}

		FArchive& Ar = *this;
		if (bSaveAsResourceIndex)
		{
			Ar << ResourceIndex;
		}
		else
		{
			Ar << SerializedMeta;
		}
	}

	FObjectDataResource& DataResource = DataResourceMap.AddDefaulted_GetRef();
	DataResource.CookedIndex			= Params.CookedIndex;
	DataResource.RawSize				= PayloadSize;
	DataResource.SerialSize				= SerializedMeta.SizeOnDisk;
	DataResource.SerialOffset			= SerializedMeta.Offset;
	DataResource.DuplicateSerialOffset	= SerializedMeta.DuplicateOffset;
	DataResource.LegacyBulkDataFlags	= SerializedMeta.Flags;
	DataResource.OuterIndex				= ObjectIndicesMap.FindRef(Params.Owner);

#if WITH_EDITOR
	if (bUpdatingLoadedPath)
	{
		SerializedBulkData.Add(&BulkData, ResourceIndex);
	}
#endif //WITH_EDITOR

	return true;
}

void FLinkerSave::ForEachBulkDataCookedIndex(TUniqueFunction<void(FBulkDataCookedIndex, FFileRegionMemoryWriter&)>&& Func, EBulkDataPayloadType Type) const
{
	const TMap<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>>& Map = GetArchives(Type);
	for (const TPair<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>>& It : Map)
	{
		check(It.Value);
		Func(It.Key, *It.Value);
	}
}

FFileRegionMemoryWriter& FLinkerSave::GetBulkDataArchive(FBulkDataCookedIndex CookedIndex)
{
	TUniquePtr<FFileRegionMemoryWriter>& Ar = BulkDataAr.FindOrAdd(CookedIndex);
	if (!Ar.IsValid())
	{
		Ar = MakeUnique<FFileRegionMemoryWriter>();
	}
	return *Ar.Get();
}

FFileRegionMemoryWriter& FLinkerSave::GetOptionalBulkDataArchive(FBulkDataCookedIndex CookedIndex)
{
	TUniquePtr<FFileRegionMemoryWriter>& Ar = OptionalBulkDataAr.FindOrAdd(CookedIndex);
	if (!Ar.IsValid())
	{
		Ar = MakeUnique<FFileRegionMemoryWriter>();
	}
	return *Ar.Get();
}

FFileRegionMemoryWriter& FLinkerSave::GetMemoryMappedBulkDataArchive(FBulkDataCookedIndex CookedIndex)
{
	TUniquePtr<FFileRegionMemoryWriter>& Ar = MemoryMappedBulkDataAr.FindOrAdd(CookedIndex);
	if (!Ar.IsValid())
	{
		Ar = MakeUnique<FFileRegionMemoryWriter>();
	}
	return *Ar.Get();
}

bool FLinkerSave::HasCookedIndexBulkData() const
{
	for (const TPair<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>>& Iter : BulkDataAr)
	{
		if (!Iter.Key.IsDefault())
		{
			return true;
		}
	}

	return false;
}

const TMap<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>>& FLinkerSave::GetArchives(EBulkDataPayloadType Type) const
{
	switch (Type)
	{
		case EBulkDataPayloadType::Inline:
		case EBulkDataPayloadType::AppendToExports:
		case EBulkDataPayloadType::MemoryMapped:
			return MemoryMappedBulkDataAr;
		case EBulkDataPayloadType::BulkSegment:
			return BulkDataAr;
		case EBulkDataPayloadType::Optional:
			return OptionalBulkDataAr;
		default:
			checkNoEntry();
	}

	static TMap<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>> NoData;
	return NoData;
}

void FLinkerSave::OnPostSaveBulkData()
{
#if WITH_EDITOR
	ensure(SerializedBulkData.IsEmpty() || bUpdatingLoadedPath == true);
 
	for (TPair<FBulkData*, int32>& Kv : SerializedBulkData)
	{
		FBulkData& BulkData = *Kv.Key;
		const FObjectDataResource& DataResource = DataResourceMap[Kv.Value];
		BulkData.SetFlagsFromDiskWrittenValues(static_cast<EBulkDataFlags>(DataResource.LegacyBulkDataFlags), DataResource.SerialOffset, DataResource.SerialSize, Summary.BulkDataStartOffset);
	}
	
	SerializedBulkData.Empty();
#endif //WITH_EDITOR
}

void FLinkerSave::SetSerializedProperty(FProperty* InProperty)
{
	FArchiveUObject::SetSerializedProperty(InProperty);
	Saver->SetSerializedProperty(InProperty);
}

void FLinkerSave::SetSerializedPropertyChain(const FArchiveSerializedPropertyChain* InSerializedPropertyChain,
	class FProperty* InSerializedPropertyOverride)
{
	FArchiveUObject::SetSerializedPropertyChain(InSerializedPropertyChain, InSerializedPropertyOverride);
	Saver->SetSerializedPropertyChain(InSerializedPropertyChain, InSerializedPropertyOverride);
}

void FLinkerSave::PushSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty)
{
	FArchive::PushSerializedProperty(InProperty, bIsEditorOnlyProperty);
	Saver->PushSerializedProperty(InProperty, bIsEditorOnlyProperty);
}

void FLinkerSave::PopSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty)
{
	FArchive::PopSerializedProperty(InProperty, bIsEditorOnlyProperty);
	Saver->PopSerializedProperty(InProperty, bIsEditorOnlyProperty);
}

#if WITH_EDITORONLY_DATA
bool FLinkerSave::IsEditorOnlyPropertyOnTheStack() const
{
	return Saver->IsEditorOnlyPropertyOnTheStack();
}
#endif
