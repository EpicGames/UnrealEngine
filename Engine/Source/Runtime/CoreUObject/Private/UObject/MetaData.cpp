// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/MetaData.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeLock.h"
#include "HAL/IConsoleManager.h"
#include "Algo/Transform.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MetaData)

DEFINE_LOG_CATEGORY_STATIC(LogMetaData, Log, All);

//////////////////////////////////////////////////////////////////////////
// FMetaDataUtilities

#if WITH_METADATA

FAutoConsoleCommand FMetaDataUtilities::DumpAllConsoleCommand = FAutoConsoleCommand(
	TEXT( "Metadata.Dump" ),
	TEXT( "Dump all MetaData" ),
	FConsoleCommandDelegate::CreateStatic( &FMetaDataUtilities::DumpAllMetaData ) );

void FMetaDataUtilities::DumpMetaData(UPackage* Package)
{
	UE_LOGF(LogMetaData, Log, "METADATA %ls", *Package->GetName());
	FMetaData& PackageMetaData = Package->GetMetaData();

	for (TMap< FSoftObjectPath, TMap<FName, FString> >::TIterator It(PackageMetaData.ObjectMetaDataMap); It; ++It)
	{
		TMap<FName, FString>& MetaDataValues = It.Value();
		for (TMap<FName, FString>::TIterator MetaDataIt(MetaDataValues); MetaDataIt; ++MetaDataIt)
		{
			FName Key = MetaDataIt.Key();
			if (Key != FName("ToolTip"))
			{
				UE_LOGF(LogMetaData, Log, "%ls: %ls=%ls", *It.Key().ToString(), *MetaDataIt.Key().ToString(), *MetaDataIt.Value());
			}
		}
	}

	for (TMap<FName, FString>::TIterator MetaDataIt(PackageMetaData.RootMetaDataMap); MetaDataIt; ++MetaDataIt)
	{
		FName Key = MetaDataIt.Key();
		if (Key != FName("ToolTip"))
		{
			UE_LOGF(LogMetaData, Log, "Root: %ls=%ls", *MetaDataIt.Key().ToString(), *MetaDataIt.Value());
		}
	}
}

void FMetaDataUtilities::DumpAllMetaData()
{
	for (TObjectIterator<UPackage> It; It; ++It)
	{
		FMetaDataUtilities::DumpMetaData(*It);
	}
}

FMetaDataUtilities::FMoveMetadataHelperContext::FMoveMetadataHelperContext(UObject *SourceObject, bool bSearchChildren)
{
	// We only want to actually move things if we're in the editor
	if (GIsEditor)
	{
		check(SourceObject);
		UPackage* Package = SourceObject->GetPackage();
		check(Package);
		OldPackage = Package;
		OldObjectPath = FSoftObjectPath::ConstructFromObject(SourceObject);
		OldObject = SourceObject;
		bShouldSearchChildren = bSearchChildren;
	}
}

FMetaDataUtilities::FMoveMetadataHelperContext::~FMoveMetadataHelperContext()
{
	// We only want to actually move things if we're in the editor
	if (GIsEditor)
	{
		FSoftObjectPath NewObjectPath = FSoftObjectPath::ConstructFromObject(OldObject);
		if (NewObjectPath != OldObjectPath)
		{
			FMetaData& NewMetaData = OldObject->GetPackage()->GetMetaData();
			FMetaData& OldMetaData = OldPackage->GetMetaData();
			
			TMap<FName, FString> OldObjectMetaData;
			if (OldMetaData.ObjectMetaDataMap.RemoveAndCopyValue(OldObjectPath, OldObjectMetaData))
			{
				NewMetaData.SetObjectValues(OldObject, OldObjectMetaData);
			}

			if (bShouldSearchChildren)
			{
				TArray<UObject*> Children;
				GetObjectsWithOuter(OldObject, Children, EGetObjectsFlags::IncludeNestedObjects);

				for ( auto ChildIt = Children.CreateConstIterator(); ChildIt; ++ChildIt )
				{
					UObject* Child = *ChildIt;
					FSoftObjectPath ChildPath = FSoftObjectPath::ConstructFromObject(Child);
					ChildPath.SetPath(OldObjectPath.GetAssetPath(), ChildPath.GetSubPathUtf8String());

					TMap<FName, FString> ChildMetaData;
					if (OldMetaData.ObjectMetaDataMap.RemoveAndCopyValue(ChildPath, ChildMetaData))
					{
						NewMetaData.SetObjectValues(Child, ChildMetaData);
					}
				}
			}
		}
	}
}

#endif //WITH_METADATA

//////////////////////////////////////////////////////////////////////////
// UDEPRECATED_MetaData implementation.

TMap<FName, FName> UDEPRECATED_MetaData::KeyRedirectMap;
UE::FMutex UDEPRECATED_MetaData::KeyRedirectMapLock;

UDEPRECATED_MetaData::UDEPRECATED_MetaData(class FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer)
{}

void UDEPRECATED_MetaData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FEditorObjectVersion::GUID);

	if( Ar.IsSaving() )
	{
		// Remove entries belonging to destructed objects
		for (TMap< FWeakObjectPtr, TMap<FName, FString> >::TIterator It(ObjectMetaDataMap); It; ++It)
		{
			if (!It.Key().IsValid())
			{
				It.RemoveCurrent();
			}
		}
	}
	
	if (!Ar.IsLoading())
	{
		Ar << ObjectMetaDataMap;
		Ar << RootMetaDataMap;
	}
	else
	{
		{
			TMap< FWeakObjectPtr, TMap<FName, FString> > TempMap;
			Ar << TempMap;

			const bool bLoadFromLinker = (NULL != Ar.GetLinker());
			if (bLoadFromLinker && HasAnyFlags(RF_LoadCompleted))
			{
				UE_LOGF(LogMetaData, Verbose, "Metadata was already loaded by linker. %ls", *GetFullName());
			}
			else
			{
				if (bLoadFromLinker && ObjectMetaDataMap.Num())
				{
					UE_LOGF(LogMetaData, Verbose, "Metadata: Some values, filled while serialization, may be lost. %ls", *GetFullName());
				}
				Swap(ObjectMetaDataMap, TempMap);
			}
		}

		if (Ar.CustomVer(FEditorObjectVersion::GUID) >= FEditorObjectVersion::RootMetaDataSupport)
		{
			TMap<FName, FString> TempMap;
			Ar << TempMap;

			const bool bLoadFromLinker = (NULL != Ar.GetLinker());
			if (bLoadFromLinker && HasAnyFlags(RF_LoadCompleted))
			{
				UE_LOGF(LogMetaData, Verbose, "Root metadata was already loaded by linker. %ls", *GetFullName());
			}
			else
			{
				if (bLoadFromLinker && RootMetaDataMap.Num())
				{
					UE_LOGF(LogMetaData, Verbose, "Metadata: Some root values, filled while serialization, may be lost. %ls", *GetFullName());
				}
				Swap(RootMetaDataMap, TempMap);
			}
		}

		// Run redirects on loaded keys
		InitializeRedirectMap();

		for (TMap< FWeakObjectPtr, TMap<FName, FString> >::TIterator ObjectIt(ObjectMetaDataMap); ObjectIt; ++ObjectIt)
		{
			TMap<FName, FString>& CurrentMap = ObjectIt.Value();

			TArray<TPair<FName, FString>, TInlineAllocator<32>> RemappedKeys;

			for (TMap<FName, FString>::TIterator PairIt(CurrentMap); PairIt; ++PairIt)
			{
				const FName OldKey = PairIt->Key;
				const FName NewKey = KeyRedirectMap.FindRef(OldKey);
				if (NewKey != NAME_None)
				{
					RemappedKeys.Add( { NewKey, MoveTemp(PairIt->Value) });
					PairIt.RemoveCurrent();

					UE_LOGF(LogMetaData, Verbose, "Remapping old metadata key '%ls' to new key '%ls' on object '%ls'.", *OldKey.ToString(), *NewKey.ToString(), *ObjectIt.Key().Get()->GetPathName());
				}
			}

			for (const TPair<FName, FString>& RemappedKey : RemappedKeys)
			{
				CurrentMap.Emplace(RemappedKey.Key, RemappedKey.Value);
			}
		}

		for (TMap<FName, FString>::TIterator PairIt(RootMetaDataMap); PairIt; ++PairIt)
		{
			const FName OldKey = PairIt.Key();
			const FName NewKey = KeyRedirectMap.FindRef(OldKey);
			if (NewKey != NAME_None)
			{
				const FString Value = PairIt.Value();

				PairIt.RemoveCurrent();
				RootMetaDataMap.Add(NewKey, Value);

				UE_LOGF(LogMetaData, Verbose, "Remapping old metadata key '%ls' to new key '%ls' on root.", *OldKey.ToString(), *NewKey.ToString());
			}
		}

#if WITH_METADATA
		if (Ar.IsPersistent())
		{
			FMetaData& MetaData = GetPackage()->GetMetaData();

			auto IsValidEntry = [](const TPair<FWeakObjectPtr, TMap<FName, FString>>& Entry) { return Entry.Key.IsValid(); };
			auto ConvertEntry = [](const TPair<FWeakObjectPtr, TMap<FName, FString>>& Entry) { return TPair<FSoftObjectPath, TMap<FName, FString>>(Entry.Key.Get(), Entry.Value); };
			Algo::TransformIf(ObjectMetaDataMap, MetaData.ObjectMetaDataMap, IsValidEntry, ConvertEntry);

			MetaData.RootMetaDataMap.Append(RootMetaDataMap);

			ClearFlags(RF_Standalone);
			MarkAsGarbage();

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			GetPackage()->DeprecatedMetaData = nullptr;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// This deprecated object is no longer needed now that we have converted it to FMetaData. However,
			// if loading from disk, the linker might still be referring to the deprecated metadata export which
			// may be GC'd during load. Thus, we remove the export to avoid issues during loading.
			if (FLinkerLoad* Linker = GetLinker())
			{
				Linker->InvalidateExport(this, true /*bHideGarbageObjects*/);
			}
		}
#endif
	}
}

void UDEPRECATED_MetaData::InitializeRedirectMap()
{
	static std::atomic<bool> bKeyRedirectMapInitialized = false;

	if (!bKeyRedirectMapInitialized.load(std::memory_order_acquire))
	{
		if (GConfig)
		{
			UE::TScopeLock _(KeyRedirectMapLock);
			if (bKeyRedirectMapInitialized.load(std::memory_order_relaxed))
			{
				return;
			}

			const FName MetadataRedirectsName("MetadataRedirects");
			const FConfigSection* PackageRedirects = GConfig->GetSection(TEXT("CoreUObject.Metadata"), false, GEngineIni);
			if (PackageRedirects)
			{
				for (FConfigSection::TConstIterator It(*PackageRedirects); It; ++It)
				{
					if (It.Key() == MetadataRedirectsName)
					{
						FName OldKey = NAME_None;
						FName NewKey = NAME_None;

						FParse::Value(*It.Value().GetValue(), TEXT("OldKey="), OldKey);
						FParse::Value(*It.Value().GetValue(), TEXT("NewKey="), NewKey);

						check(OldKey != NewKey);
						check(OldKey != NAME_None);
						check(NewKey != NAME_None);
						check(!KeyRedirectMap.Contains(OldKey));
						check(!KeyRedirectMap.Contains(NewKey));

						KeyRedirectMap.Add(OldKey, NewKey);
					}			
				}
			}
			bKeyRedirectMapInitialized.store(true, std::memory_order_release);
		}
	}
}

#if WITH_METADATA
//////////////////////////////////////////////////////////////////////////
// FMetaData implementation.

TMap<FName, FName> FMetaData::KeyRedirectMap;
UE::FMutex FMetaData::KeyRedirectMapLock;

void FMetaData::RemapObjectKeys(FName OldPackageName, FName NewPackageName)
{
	for (TMap<FSoftObjectPath, TMap<FName, FString>>::TIterator It(ObjectMetaDataMap); It; ++It)
	{
		It.Key().RemapPackage(OldPackageName, NewPackageName);
	}
}

/**
 * Return the value for the given key in the given property
 * @param Object the object to lookup the metadata for
 * @param Key The key to lookup
 * @return The value if found, otherwise an empty string
 */
const FString& FMetaData::GetValue(const UObject* Object, FName Key)
{
	// if not found, return a static empty string
	static FString EmptyString;

	// every key needs to be valid
	if (Key == NAME_None)
	{
		return EmptyString;
	}

	// look up the existing map if we have it
	TMap<FName, FString>* ObjectValues = ObjectMetaDataMap.Find(Object);

	// if not, return empty
	if (ObjectValues == NULL)
	{
		return EmptyString;
	}

	// look for the property
	FString* ValuePtr = ObjectValues->Find(Key);
	
	// if we didn't find it, return NULL
	if (!ValuePtr)
	{
		return EmptyString;
	}

	// if we found it, return the pointer to the character data
	return *ValuePtr;

}

/**
 * Return the value for the given key in the given property
 * @param Object the object to lookup the metadata for
 * @param Key The key to lookup
 * @return The value if found, otherwise an empty string
 */
const FString& FMetaData::GetValue(const UObject* Object, const TCHAR* Key)
{
	// only find names, don't bother creating a name if it's not already there
	// (GetValue will return an empty string if Key is NAME_None)
	return GetValue(Object, FName(Key, FNAME_Find));
}

const FString* FMetaData::FindValue(const UObject* Object, FName Key)
{
	// every key needs to be valid
	if (Key == NAME_None)
	{
		return nullptr;
	}

	// look up the existing map if we have it
	TMap<FName, FString>* ObjectValues = ObjectMetaDataMap.Find(Object);

	// if not, return false
	if (ObjectValues == nullptr)
	{
		return nullptr;
	}

	// if we had the map, see if we had the key
	return ObjectValues->Find(Key);
}

const FString* FMetaData::FindValue(const UObject* Object, const TCHAR* Key)
{
	// only find names, don't bother creating a name if it's not already there
	// (HasValue will return false if Key is NAME_None)
	return FindValue(Object, FName(Key, FNAME_Find));
}

/**
 * Is there any metadata for this property?
 * @param Object the object to lookup the metadata for
 * @return TrUE if the property has any metadata at all
 */
bool FMetaData::HasObjectValues(const UObject* Object)
{
	return ObjectMetaDataMap.Contains(Object);
}

/**
 * Set the key/value pair in the Property's metadata
 * @param Object the object to set the metadata for
 * @Values The metadata key/value pairs
 */
void FMetaData::SetObjectValues(const UObject* Object, const TMap<FName, FString>& ObjectValues)
{
	ObjectMetaDataMap.Add(const_cast<UObject*>(Object), ObjectValues);
}

/**
 * Set the key/value pair in the Property's metadata
 * @param Object the object to set the metadata for
 * @Values The metadata key/value pairs
 */
void FMetaData::SetObjectValues(const UObject* Object, TMap<FName, FString>&& ObjectValues)
{
	ObjectMetaDataMap.Add(const_cast<UObject*>(Object), MoveTemp(ObjectValues));
}

/**
 * Set the key/value pair in the Property's metadata
 * @param Object the object to set the metadata for
 * @param Key A key to set the data for
 * @param Value The value to set for the key
 * @Values The metadata key/value pairs
 */
void FMetaData::SetValue(const UObject* Object, FName Key, const TCHAR* Value)
{
	check(Key != NAME_None);

	// look up the existing map if we have it
	TMap<FName, FString>* ObjectValues = ObjectMetaDataMap.Find(Object);

	// if not, create an empty map
	if (ObjectValues == NULL)
	{
		ObjectValues = &ObjectMetaDataMap.Add(const_cast<UObject*>(Object), TMap<FName, FString>());
	}

	// set the value for the key
	ObjectValues->Add(Key, Value);
}

// Set the Key/Value pair in the Object's metadata
void FMetaData::SetValue(const UObject* Object, const TCHAR* Key, const TCHAR* Value)
{
	SetValue(Object, FName(Key), Value);
}

void FMetaData::RemoveValue(const UObject* Object, const TCHAR* Key)
{
	RemoveValue(Object, FName(Key));
}

void FMetaData::RemoveValue(const UObject* Object, FName Key)
{
	check(Key != NAME_None);

	TMap<FName, FString>* ObjectValues = ObjectMetaDataMap.Find(Object);
	if (ObjectValues)
	{
		// set the value for the key
		ObjectValues->Remove(Key);
	}
}

TMap<FName, FString>* FMetaData::GetMapForObject(const UObject* Object)
{
	check(Object);
	UPackage* Package = Object->GetPackage();
	check(Package);
	FMetaData& Metadata = Package->GetMetaData();

	TMap<FName, FString>* Map = Metadata.ObjectMetaDataMap.Find(Object);
	return Map;
}

void FMetaData::CopyMetadata(UObject* SourceObject, UObject* DestObject)
{
	check(SourceObject);
	check(DestObject);

	// First get the source map
	TMap<FName, FString>* SourceMap = GetMapForObject(SourceObject);
	if (!SourceMap)
	{
		return;
	}

	// Then get the metadata for the destination
	UPackage* DestPackage = DestObject->GetPackage();
	check(DestPackage);
	FMetaData& DestMetadata = DestPackage->GetMetaData();

	// Iterate through source map, setting each key/value pair in the destination
	for (const auto& It : *SourceMap)
	{
		DestMetadata.SetValue(DestObject, *It.Key.ToString(), *It.Value);
	}
}

/**
 * Removes any metadata entries that are to objects not inside the same package as this FMetaData object.
 */
void FMetaData::RemoveMetaDataOutsidePackage(UPackage* MetaDataPackage)
{
	TArray<FSoftObjectPath> ObjectsToRemove;

	// Iterate over all entries..
	for (TMap< FSoftObjectPath, TMap<FName, FString> >::TIterator It(ObjectMetaDataMap); It; ++It)
	{
		FSoftObjectPath& ObjPath = It.Key();
		FWeakObjectPtr ObjPtr = ObjPath.ResolveObject();
		// See if its package is not the same as the MetaData's, or is invalid
		if( !ObjPtr.IsValid() || (ObjPtr.Get()->GetPackage() != MetaDataPackage))
		{
			// Add to list of things to remove
			ObjectsToRemove.Add(ObjPath);
		}
	}

	// Go through and remove any objects that need it
	for(int32 i=0; i<ObjectsToRemove.Num(); i++)
	{
		FSoftObjectPath& ObjPath = ObjectsToRemove[i];
		FWeakObjectPtr ObjPtr = ObjPath.ResolveObject();

		UObject* ObjectToRemove = ObjPtr.Get();
		if ((ObjectToRemove != NULL) && (ObjectToRemove->GetPackage() != GetTransientPackage()))
		{
			UE_LOGF(LogMetaData, Log, "Removing '%ls' ref from Metadata '%ls'", *ObjectToRemove->GetPathName(), *MetaDataPackage->GetPathName());
		}
		ObjectMetaDataMap.Remove( ObjPath );
	}
}

FName FMetaData::GetRemappedKeyName(FName OldKey)
{
	InitializeRedirectMap();
	return KeyRedirectMap.FindRef(OldKey);
}

void FMetaData::InitializeRedirectMap()
{
	static std::atomic<bool> bKeyRedirectMapInitialized = false;

	if (!bKeyRedirectMapInitialized.load(std::memory_order_acquire))
	{
		if (GConfig)
		{
			UE::TScopeLock _(KeyRedirectMapLock);
			if (bKeyRedirectMapInitialized.load(std::memory_order_relaxed))
			{
				return;
			}
			const FName MetadataRedirectsName("MetadataRedirects");
			const FConfigSection* PackageRedirects = GConfig->GetSection(TEXT("CoreUObject.Metadata"), false, GEngineIni);
			if (PackageRedirects)
			{
				for (FConfigSection::TConstIterator It(*PackageRedirects); It; ++It)
				{
					if (It.Key() == MetadataRedirectsName)
					{
						FName OldKey = NAME_None;
						FName NewKey = NAME_None;

						FParse::Value(*It.Value().GetValue(), TEXT("OldKey="), OldKey);
						FParse::Value(*It.Value().GetValue(), TEXT("NewKey="), NewKey);

						check(OldKey != NewKey);
						check(OldKey != NAME_None);
						check(NewKey != NAME_None);
						check(!KeyRedirectMap.Contains(OldKey));
						check(!KeyRedirectMap.Contains(NewKey));

						KeyRedirectMap.Add(OldKey, NewKey);
					}			
				}
			}
			bKeyRedirectMapInitialized.store(true, std::memory_order_release);
		}
	}
}
#endif