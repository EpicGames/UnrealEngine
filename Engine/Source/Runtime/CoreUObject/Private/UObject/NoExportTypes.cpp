// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/NoExportTypes.h"

#if WITH_APPLICATION_CORE
#include "GenericPlatform/ICursor.h"
#include "GenericPlatform/IInputInterface.h"
#include "GenericPlatform/InputDeviceMappingPolicy.h"
#else
#include "../../../ApplicationCore/Public/GenericPlatform/ICursor.h" // TODO, Bad! will not work with clang vfs overlay
#include "../../../ApplicationCore/Public/GenericPlatform/IInputInterface.h" // TODO, Bad! will not work with clang vfs overlay
#include "../../../ApplicationCore/Public/GenericPlatform/InputDeviceMappingPolicy.h" // TODO, Bad! will not work with clang vfs overlay
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(NoExportTypes)
