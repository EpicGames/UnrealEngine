// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/NonNullPropertyUtils.h"

#include "HAL/IConsoleManager.h"

namespace UE::CoreUObject
{
#if WITH_EDITORONLY_DATA
	TAutoConsoleVariable<bool> CVarEnableNullDeserializeToNonNullableOverride(
		TEXT("CoreUObject.EnableAllowSetNullToNonNullableOverride"),
		true,
		TEXT("Allows scoped overrides to enable null to be set on a NonNullable property")
	);
		
	thread_local FScopedSetNullOnNonNullable* TLSScopedSetNullOnNonNullableBehaviorStackHead = nullptr;
	
	FScopedSetNullOnNonNullable::FScopedSetNullOnNonNullable(TOptional<EAllowSetNullOnNonNullableBehavior> InBehavior)
		: Previous(nullptr)
	{
		if (InBehavior.IsSet() && CVarEnableNullDeserializeToNonNullableOverride.GetValueOnAnyThread())
		{
			bIsBehaviorSet = true;
			Behavior =  *InBehavior;

			// Push the stack object to the list
			Previous = TLSScopedSetNullOnNonNullableBehaviorStackHead;
			TLSScopedSetNullOnNonNullableBehaviorStackHead = this;
		}
		else
		{
			bIsBehaviorSet = false;
			Behavior = EAllowSetNullOnNonNullableBehavior::Disabled; // Set it to something
		}
	}

	FScopedSetNullOnNonNullable::~FScopedSetNullOnNonNullable()
	{
		if (bIsBehaviorSet)
		{	
			// Pop us from the stack and set the previous to head
			check(TLSScopedSetNullOnNonNullableBehaviorStackHead == this);
			TLSScopedSetNullOnNonNullableBehaviorStackHead = Previous;
		}
	}

	EAllowSetNullOnNonNullableBehavior FScopedSetNullOnNonNullable::GetBehavior() const
	{
		check(bIsBehaviorSet);
		return Behavior;
	}
	
	EAllowSetNullOnNonNullableBehavior AllowSetNullOnNonNullableBehavior()
	{
		if (TLSScopedSetNullOnNonNullableBehaviorStackHead)
		{
			return TLSScopedSetNullOnNonNullableBehaviorStackHead->GetBehavior();
		}
		return  EAllowSetNullOnNonNullableBehavior::Disabled;
	}
#endif
}
