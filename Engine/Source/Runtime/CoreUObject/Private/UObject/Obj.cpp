// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnObj.cpp: Unreal object manager.
=============================================================================*/

#include "CoreMinimal.h"
#include "UObject/NonNullPropertyUtils.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/CoreMisc.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Stats/Stats.h"
#include "Misc/ConfigAccessData.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"
#include "Misc/ITransactionObjectAnnotation.h"
#include "Misc/DataValidation.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectArchetypeHelper.h"
#include "UObject/Object.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/GarbageCollectionHistory.h"
#include "UObject/Class.h"
#include "UObject/UObjectPartials.h"
#include "UObject/EnumProperty.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "UObject/MetaData.h"
#include "UObject/TextProperty.h"
#include "Templates/Casts.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/PropertyStateTracking.h"
#include "UObject/UnrealType.h"
#include "UObject/StructPostLoadContext.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/OverridableManager.h"
#include "Serialization/ArchiveCountMem.h"
#include "Serialization/ArchiveShowReferences.h"
#include "Serialization/ArchiveFindCulprit.h"
#include "Misc/PackageName.h"
#include "Serialization/BulkData.h"
#include "UObject/LinkerLoad.h"
#include "Misc/RedirectCollector.h"
#include "Misc/PlayInEditorLoadingScope.h"
#include "UObject/GCScopeLock.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/CookStats.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "UObject/GCObject.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "Serialization/ArchiveDescribeReference.h"
#include "UObject/FindStronglyConnected.h"
#include "UObject/UObjectThreadContext.h"
#include "Misc/ExclusiveLoadPackageTimeTracker.h"
#include "Serialization/DeferredMessageLog.h"
#include "UObject/CoreRedirects.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Containers/VersePath.h"
#include "Serialization/LoadTimeTracePrivate.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/PropertyBagRepository.h"
#include "AutoRTFM.h"
#include "VerseVM/VVMVerseClass.h"
#include "Misc/AssetReadLoggerConfig.h"
#if UE_ENABLE_ASSET_READ_LOGGER
#include "Misc/AssetReadLogger.h"
#endif

DEFINE_LOG_CATEGORY(LogObj);

LLM_DEFINE_TAG(UObject_UObjectInternals);

namespace
{
	static int32 GPrematureConfigLoadErrorLevel = 0;
	static FAutoConsoleVariableRef CVarPrematureConfigLoadErrorLevel(
		TEXT("obj.PrematureConfigLoadErrorLevel"),
		GPrematureConfigLoadErrorLevel,
		TEXT("Determines the level of warning issued if LoadConfig is called on an object that is not yet sufficiently initialized. 0=Display, 1=Warning, 2=Ensure, 3=Check"));

	static int32 GUseNewPropertySaving = 0;
	static FAutoConsoleVariableRef CVarUseNewPropertySaving(
		TEXT("ini.UseNewPropertySaving"),
		GUseNewPropertySaving,
		TEXT("Determines if UObject property saving should use the Config APIs that can set a property (including arrays) directly in a static layer."));

	// Enable additional (slow) checks in CheckDefaultSubobjects.
	static constexpr bool bDoCheckSubobjects = false;
}

/*-----------------------------------------------------------------------------
	Globals.
-----------------------------------------------------------------------------*/

// Object manager internal variables.
/** Transient package.													*/
static UPackage*			GObjTransientPkg								= NULL;		

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Used to verify that the Super::BeginDestroyed chain is intact.			*/
	static TArray<UObject*,TInlineAllocator<16> >		DebugBeginDestroyed;
	/** Used to verify that the Super::FinishDestroyed chain is intact.			*/
	static TArray<UObject*,TInlineAllocator<16> >		DebugFinishDestroyed;
#endif

#if !UE_BUILD_SHIPPING
	/** Used for the "obj mark" and "obj markcheck" commands only			*/
	static FUObjectAnnotationSparseBool DebugMarkAnnotation;
	/** Used for the "obj invmark" and "obj invmarkcheck" commands only			*/
	static TArray<TWeakObjectPtr<UObject> >	DebugInvMarkWeakPtrs;
	static TArray<FString>			DebugInvMarkNames;
	/** Used for the "obj spikemark" and "obj spikemarkcheck" commands only			*/
	static FUObjectAnnotationSparseBool DebugSpikeMarkAnnotation;
	static TArray<FString>			DebugSpikeMarkNames;
#endif

#if WITH_EDITOR
	struct FPropagatedEditChangeAnnotation
	{
		/**
		 * If set, archetype edits will not mark an instance dirty unless it results
		 * in the instance realigning with the archetype after the change (that will
		 * then result in future archetype changes being propagated to the instance).
		 */
		uint8 bDeferredMarkAsDirty : 1;
		/**
		 * If set, this instance will be affected by an archetype change (i.e. it
		 * matched the archetype prior to propagating the change).
		 */
		uint8 bIdenticalToArchetype : 1;
		/**
		 * If set, the package containing this instance was already marked as dirty
		 * prior to propagating the change.
		 */
		uint8 bWasPackageDirtyOnEdit : 1;

		FPropagatedEditChangeAnnotation()
		: bDeferredMarkAsDirty(false)
		, bIdenticalToArchetype(false)
		, bWasPackageDirtyOnEdit(false)
		{}

		FORCEINLINE bool IsDefault() const
		{
			return !bDeferredMarkAsDirty;
		}
	};
	static FUObjectAnnotationSparse<FPropagatedEditChangeAnnotation, true> PropagatedEditChangeAnnotation;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	UObject::FAssetRegistryTag::FOnGetObjectAssetRegistryTags UObject::FAssetRegistryTag::OnGetExtraObjectTags;
	UObject::FAssetRegistryTag::FOnGetExtendedAssetRegistryTagsForSave UObject::FAssetRegistryTag::OnGetExtendedAssetRegistryTagsForSave;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	UObject::FAssetRegistryTag::FOnGetObjectAssetRegistryTagsWithContext UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext;
	UObject::FOnGetPreviewPlatform UObject::OnGetPreviewPlatform;
#endif // WITH_EDITOR

UE_AUTORTFM_ALWAYS_OPEN
UObject::UObject( EStaticConstructor, EObjectFlags InFlags )
: UObjectBaseUtility(InFlags | (RF_MarkAsNative | RF_MarkAsRootSet))
{
	EnsureNotRetrievingVTablePtr();
}

UE_AUTORTFM_ALWAYS_OPEN
UObject::UObject(FVTableHelper& Helper)
{
	EnsureRetrievingVTablePtrDuringCtor(TEXT("UObject(FVTableHelper& Helper)"));
}

void UObject::EnsureNotRetrievingVTablePtr() const
{
	UE_CLOGF(GIsRetrievingVTablePtr, LogCore, Fatal, "We are currently retrieving VTable ptr. Please use FVTableHelper constructor instead.");
}

UObject* UObject::CreateDefaultSubobject(FName SubobjectFName, UClass* ReturnType, UClass* ClassToCreateByDefault, bool bIsRequired, bool bIsTransient)
{
	FObjectInitializer* CurrentInitializer = FUObjectThreadContext::Get().TopInitializer();
	UE_CLOGF(!CurrentInitializer, LogObj, Fatal, "No object initializer found during construction.");
	UE_CLOGF(CurrentInitializer->Obj != this, LogObj, Fatal, "Using incorrect object initializer.");
	return CurrentInitializer->CreateDefaultSubobject(this, SubobjectFName, ReturnType, ClassToCreateByDefault, bIsRequired, bIsTransient);
}

UObject* UObject::CreateEditorOnlyDefaultSubobjectImpl(FName SubobjectName, UClass* ReturnType, bool bTransient)
{
	FObjectInitializer* CurrentInitializer = FUObjectThreadContext::Get().TopInitializer();
	return CurrentInitializer->CreateEditorOnlyDefaultSubobject(this, SubobjectName, ReturnType, bTransient);
}

void UObject::GetDefaultSubobjects(TArray<UObject*>& OutDefaultSubobjects)
{
	OutDefaultSubobjects.Reset();
	ForEachObjectWithOuter(this, [&OutDefaultSubobjects](UObject* Object)
	{
		if (Object->IsDefaultSubobject())
		{
			OutDefaultSubobjects.Add(Object);
		}
	}, EGetObjectsFlags::None);
}

UObject* UObject::GetDefaultSubobjectByName(FName ToFind)
{
	UObject* Object = nullptr;
	// If it is safe use the faster StaticFindObjectFast rather than searching all the subobjects
	if (!UE::IsSavingPackage() && !IsGarbageCollecting())
	{
		Object = StaticFindObjectFast(UObject::StaticClass(), this, ToFind);
		if (Object && !Object->IsDefaultSubobject())
		{
			Object = nullptr;
		}
	}
	else
	{
		TArray<UObject*> SubObjects;
		GetDefaultSubobjects(SubObjects);
		for (UObject* SubObject : SubObjects)
		{
			if (SubObject->GetFName() == ToFind)
			{
				Object = SubObject;
				break;
			}
		}
	}
	return Object;
}

bool UObject::Rename(const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UObject::Rename);

#if WITH_METADATA
	// This guarantees that if this UObject is actually renamed and changes packages
	// the metadata will be moved with it.
	FMetaDataUtilities::FMoveMetadataHelperContext MoveMetaData(this, true);
#endif // WITH_METADATA
	const bool bIsTest = Flags & REN_Test;
	if (NewOuter && NewOuter != GetOuter())
	{
		// Renaming the CDO of a Blueprint is a special case so we do not validate what would otherwise be incorrect use of Rename.
		// Moving objects to the transient package is commonly used halfway through destroying them so that is also fine, otherwise
		// proceed to check for incorrect use of Rename
		if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) && NewOuter->GetPackage() != GetTransientPackage())
		{
			// Check that we are not renaming a within object into an Outer of the wrong type
			if (!NewOuter->IsA(GetClass()->ClassWithin))
			{
				if (bIsTest)
				{
					return false;
				}

				UE_LOGF(LogObj, Fatal, "Cannot rename %ls into Outer %ls as it is not of type %ls",
					*GetFullName(),
					*NewOuter->GetFullName(),
					*GetClass()->ClassWithin->GetName());
			}
		}

		// If moving the object to a new package, remove its linker and detach the object
		const FLinkerLoad* Linker = GetLinker();
		const FLinkerLoad* NewOuterLinker = NewOuter->GetLinker();
		if (((Flags & REN_AllowPackageLinkerMismatch) == 0) && Linker != NewOuterLinker)
		{
			// It's possible we are moving to a new outer in the same package which doesn't
			// have a linker (in which case we don't want to clear the existing linker)
			if (NewOuterLinker || GetPackage() != NewOuter->GetPackage())
			{
				if (Linker && HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad))
				{
					if (bIsTest)
					{
						return false;
					}

					const bool bIsPreloading = HasAnyFlags(RF_NeedLoad);
					const TCHAR* PreloadingGuidance =
						TEXT("Ensure the target package has completed loading before renaming to correct this error. ") 
						TEXT("Renaming before an object has been serialized will leave the object uninitialized.");
					const TCHAR* PostloadingGuidance =
						TEXT("If this object is no longer valid / is being discarded, call FLinkerLoad::InvalidateExport(ObjectToRename) ")
						TEXT("before calling Rename, otherwise ensure this object has been postloaded by waiting for the target package's load to complete, or explicitly calling ConditionalPostload(ObjectToRename).");

					UE_LOGF(LogObj, Error,
						"Rename is moving object '%ls' to a new package (%ls) or a new linker while the object is still being loaded (it hasn't been %ls yet). " "%ls " "Renaming a partially loaded object without first calling FLinkerLoad::InvalidateExport(ObjectToRename) " "will result in a partially loaded package likely leading to crashes and/or corrupt data.",
						*GetFullName(),
						*NewOuter->GetPackage()->GetFullName(),
						(HasAnyFlags(RF_NeedLoad) ? TEXT("serialized") : TEXT("postloaded")),
						(bIsPreloading ? PreloadingGuidance : PostloadingGuidance)
					);
					FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
				}

				SetLinker(nullptr, INDEX_NONE, true /*bShouldDetachExisting*/, true /*bSilenceLogs*/);
			}
		}
	}
#if WITH_EDITOR // until we validate the runtime should also detach (renaming is uncommon)
	// If we are renaming a loaded object while keeping its outer and won't be creating a redirector, we need to detach from the linker if the user
	// hasn't specifically requested to allow for the linker mismatch. This way, we can reload the renamed object if the package is rerequested
	// Users who don't want such reload behaviour for the export should be calling InvalidateExport() on the object before renaming.
	else if (!bIsTest && ((Flags & (REN_AllowPackageLinkerMismatch | REN_DontCreateRedirectors)) == REN_DontCreateRedirectors) && HasAnyFlags(RF_WasLoaded))
	{
		if (UPackage* Package = Cast<UPackage>(this))
		{
			// When renaming a package, we must ensure that the LinkerAnnotations for each of its objects are detached to avoid "Detaching from existing linker" errors on 
			// objects loaded later on from the same package, because there is a small window of possible Object Index reuse until the next GC cycle.
			if (FLinkerLoad* Linker = Package->GetLinker()) // Note: invoking UPackage::GetLinker, not UObjectBaseUtility::GetLinker
			{
				Linker->DetachExports();
			}
		}
		else if (FLinkerLoad* Linker = GetLinker())
		{
			const int32 LinkerIndex = GetLinkerIndex();
			if (LinkerIndex != INDEX_NONE)
			{
				Linker->DetachExport(LinkerIndex);
			}
		}
	}
#endif

	// find an object with the same name and same class in the new outer
	bool bIsCaseOnlyChange = false;
	if (InName)
	{
		UObject* ExistingObject = nullptr;
		
		if (!(Flags & REN_ForceGlobalUnique))
		{
			ExistingObject = StaticFindObject(/*Class=*/ nullptr, NewOuter ? NewOuter : GetOuter(), InName, EFindObjectFlags::ExactClass);
		}
		else
		{
			ExistingObject = StaticFindFirstObject(/*Class=*/ nullptr, InName, EFindFirstObjectOptions::ExactClass);
		}
		if (ExistingObject == this)
		{
			if (ExistingObject->GetName().Equals(InName, ESearchCase::CaseSensitive))
			{
				// The name is exactly the same - there's nothing to change
				return true;
			}
			else
			{
				// This rename has only changed the case, so we need to allow it to continue, but won't create a redirector (since the internal FName comparison ignores case)
				bIsCaseOnlyChange = true;
			}
		}
		else if (ExistingObject)
		{
			if (bIsTest)
			{
				return false;
			}
			else
			{
				UE_LOGF(LogObj, Fatal,"Renaming an object (%ls) on top of an existing object (%ls) is not allowed", *GetFullName(), *ExistingObject->GetFullName());
			}
		}
	}

	// if we are just testing, and there was no conflict, then return a success
	if (bIsTest)
	{
		return true;
	}

	FName OldName = GetFName();
	FName NewName;
	bool bCreateRedirector = false;
	UObject* OldOuter = nullptr;

	if (!(Flags & REN_NonTransactional))
	{
		// Mark touched packages as dirty.
		if (Flags & REN_DoNotDirty)
		{
			// This will only mark dirty if in a transaction,
			// the object is transactional, and the object is
			// not in a PlayInEditor package.
			Modify(false);
		}
		else
		{
			// This will maintain previous behavior...
			// Which was to directly call MarkPackageDirty
			Modify(true);
		}
	}
	if (NewOuter)
	{
		if (!(Flags & REN_DoNotDirty))
		{
			NewOuter->MarkPackageDirty();
		}
	}

	{
		// Ensure that between StaticFindObjectFast and completion of LowLevelRename nothing else modifies
		// the UObject hash tables.
		FScopedUObjectHashTablesLock HashTablesLock;

		if (InName == nullptr)
		{
			// If null, null is passed in, then we are deliberately trying to get a new name
			// Otherwise if the outer is changing, try and maintain the name
			if (NewOuter && StaticFindObjectFastInternal(nullptr, NewOuter, OldName) == nullptr)
			{
				NewName = OldName;
			}
			else
			{
				NewName = MakeUniqueObjectName(NewOuter ? NewOuter : GetOuter(), GetClass(), FName(), !!(Flags & REN_ForceGlobalUnique) ? EUniqueObjectNameOptions::GloballyUnique : EUniqueObjectNameOptions::None);
			}
		}
		else
		{
			NewName = FName(InName);
		}

		//UE_LOGF(LogObj, Log,  "Renaming %ls to %ls", *OldName.ToString(), *NewName.ToString() );

		OldOuter = GetOuter();

		if (HasAnyFlags(RF_Public))
		{
			const bool bUniquePathChanged = ((NewOuter != NULL && OldOuter != NewOuter) || (OldName != NewName));
			const bool bRootPackage = GetClass() == UPackage::StaticClass() && OldOuter == NULL;
			const bool bRedirectionAllowed = ((Flags & REN_DontCreateRedirectors) == 0) && GIsEditor && !IsRunningGame();
			auto IsGameObject =
				[this]() -> bool
				{
					return GetPackage()->HasAnyPackageFlags(PKG_PlayInEditor);
				};

			// We need to create a redirector if we changed the Outer or Name of an object that can be referenced from other packages
			// [i.e. has the RF_Public flag] so that references to this object are not broken.
			bCreateRedirector = bRootPackage == false && bUniquePathChanged == true && bRedirectionAllowed == true && bIsCaseOnlyChange == false && IsGameObject() == false;
		}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		UE::CoreUObject::Private::UpdateRenamedObject(this, NewName, NewOuter);
#endif
		LowLevelRename(NewName, NewOuter);
	}

	// Create the redirector AFTER renaming the object. Two objects of different classes may not have the same fully qualified name.
	if (bCreateRedirector)
	{
		// Look for an existing redirector with the same name/class/outer in the old package.
		UObjectRedirector* Redirector = FindObject<UObjectRedirector>(OldOuter, *OldName.ToString(), EFindObjectFlags::ExactClass);

		// If it does not exist, create it.
		if ( Redirector == NULL )
		{
			// create a UObjectRedirector with the same name as the old object we are redirecting
			Redirector = NewObject<UObjectRedirector>(OldOuter, OldName, RF_Standalone | RF_Public, nullptr, false, nullptr, GetExternalPackage());
		}

		// point the redirector object to this object
		Redirector->DestinationObject = this;
	}

	PostRename(OldOuter, OldName);

	return true;
}


void UObject::PostLoad()
{
	// Note that it has propagated.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FUObjectThreadContext::Get().DebugPostLoad.RemoveSingle(this);
#endif

	/*
	By this point, all default properties have been loaded from disk
	for this object's class and all of its parent classes.  It is now
	safe to import config and localized data for "special" objects:
	- per-object config objects
	*/
	if( GetClass()->HasAnyClassFlags(CLASS_PerObjectConfig) )
	{
		LoadConfig();
	}
	CheckDefaultSubobjects();

#if WITH_EDITOR
	UE::FStructPostLoadContext::Get().OnPostLoad(this);
#endif
}

#if WITH_EDITOR
void UObject::PreEditChange(FProperty* PropertyAboutToChange)
{
	bool bShouldMarkAsDirty = true;

	if (GIsTransacting)
	{
		// Don't mark the outer package as dirty during an undo/redo operation.
		bShouldMarkAsDirty = false;
	}
	else if (PropertyAboutToChange && PropertyAboutToChange->HasAnyPropertyFlags(CPF_SkipSerialization))
	{
		// Don't mark the outer package as dirty if we're about to change a non-serializable property.
		bShouldMarkAsDirty = false;
	}
	else
	{
		FPropagatedEditChangeAnnotation Annotation = PropagatedEditChangeAnnotation.GetAnnotation(this);
		if (Annotation.bDeferredMarkAsDirty)
		{
			// Don't mark the outer package as dirty if annotated to be deferred (e.g. during propagation).
			bShouldMarkAsDirty = false;
		}
	}

	Modify(bShouldMarkAsDirty);
}


void UObject::PostEditChange(void)
{
	FPropertyChangedEvent EmptyPropertyUpdateStruct(NULL);
	this->PostEditChangeProperty(EmptyPropertyUpdateStruct);
}


void UObject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);

#if WITH_EDITORONLY_DATA
	if (const FProperty* Property = PropertyChangedEvent.MemberProperty)
	{
		const EPropertyChangeType::Type SetPropertyValueInitializedStates =
			EPropertyChangeType::ArrayAdd | EPropertyChangeType::ArrayRemove | EPropertyChangeType::ArrayClear | EPropertyChangeType::ArrayMove | EPropertyChangeType::ValueSet | EPropertyChangeType::Interactive;
		const bool bIsResetToDefaultOperation = (PropertyChangedEvent.ChangeType & EPropertyChangeType::ResetToDefault) == EPropertyChangeType::ResetToDefault;
		const bool bChangeTypeSetsInitState = (PropertyChangedEvent.ChangeType & SetPropertyValueInitializedStates) != 0;
		
		if (!bIsResetToDefaultOperation && bChangeTypeSetsInitState)
		{
			UE::FInitializedPropertyValueState(this).Set(Property);
		}
	}
#endif

	// Snapshot the transaction buffer for this object if this was from an interactive change
	// This allows listeners to be notified of intermediate changes of state
	if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive)
	{
		const FProperty* ChangedProperty = PropertyChangedEvent.MemberProperty;
		SnapshotTransactionBuffer(this, TArrayView<const FProperty*>(&ChangedProperty, 1));
	}
}


void UObject::PreEditChange( FEditPropertyChain& PropertyAboutToChange )
{
	const bool bIsEditingArchetypeProperty = HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) && !FApp::IsGame();

	if (bIsEditingArchetypeProperty)
	{
		// this object must now be included in the undo/redo buffer (needs to be 
		// done prior to the following PreEditChange() call, in case it attempts 
		// to store this object in the undo/redo transaction buffer)
		SetFlags(RF_Transactional);
	}

	// forward the notification to the FProperty* version of PreEditChange
	FProperty* PropertyBeingEdited = PropertyAboutToChange.GetActiveNode() ? PropertyAboutToChange.GetActiveNode()->GetValue() : nullptr;
	PreEditChange(PropertyBeingEdited);

	FCoreUObjectDelegates::OnPreObjectPropertyChanged.Broadcast(this, PropertyAboutToChange);

	if (bIsEditingArchetypeProperty)
	{
		// Get a list of all objects which will be affected by this change; 
		TArray<UObject*> Objects;
		GetArchetypeInstances(Objects);
		PropagatePreEditChange(Objects, PropertyAboutToChange);
	}
}


void UObject::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	FProperty* PropertyBeingEdited = PropertyChangedEvent.PropertyChain.GetActiveNode() ? PropertyChangedEvent.PropertyChain.GetActiveNode()->GetValue() : nullptr;
	FPropertyChangedEvent PropertyEvent(PropertyBeingEdited, PropertyChangedEvent.ChangeType);

	// Set up array index per object map so that GetArrayIndex returns a valid result
	TArray<TMap<FString, int32>> ArrayIndexForProperty;
	if (PropertyChangedEvent.Property)
	{
		const FString PropertyName = PropertyChangedEvent.Property->GetName();
		const int32 ArrayIndex = PropertyChangedEvent.GetArrayIndex(PropertyName);
		if (ArrayIndex != INDEX_NONE)
		{
			PropertyEvent.ObjectIteratorIndex = 0;
			ArrayIndexForProperty.AddDefaulted();
			ArrayIndexForProperty.Last().Add(PropertyName, ArrayIndex);
			PropertyEvent.SetArrayIndexPerObject(ArrayIndexForProperty);
		}
	}

	if( PropertyChangedEvent.PropertyChain.GetActiveMemberNode() )
	{
		PropertyEvent.SetActiveMemberProperty( PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue() );
	}

	// Propagate change to archetype instances first if necessary.
	if (!FApp::IsGame())
	{
		if (HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) && PropertyChangedEvent.PropertyChain.GetActiveMemberNode() == PropertyChangedEvent.PropertyChain.GetHead())
		{
			// Get a list of all archetype instances
			TArray<UObject*> ArchetypeInstances;
			GetArchetypeInstances(ArchetypeInstances);

			// Propagate the editchange call to archetype instances
			PropagatePostEditChange(ArchetypeInstances, PropertyChangedEvent);
		}
		else if (GetOuter()->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
		{
			// Get a list of all outer's archetype instances
			TArray<UObject*> ArchetypeInstances;
			GetOuter()->GetArchetypeInstances(ArchetypeInstances);

			// Find FProperty describing this in Outer.
			for (FProperty* Property = GetOuter()->GetClass()->RefLink; Property != nullptr; Property = Property->NextRef)
			{
				if (this != *Property->ContainerPtrToValuePtr<UObject*>(GetOuter()))
				{
					continue;
				}

				// Since we found property, propagate PostEditChange to all relevant components of archetype instances.
				TArray<UObject*> ArchetypeComponentInstances;
				for (UObject* ArchetypeInstance : ArchetypeInstances)
				{
					if (UObject* ComponentInstance = *Property->ContainerPtrToValuePtr<UObject*>(ArchetypeInstance))
					{
						ArchetypeComponentInstances.Add(ComponentInstance);
					}
				}

				PropagatePostEditChange(ArchetypeComponentInstances, PropertyChangedEvent);

				break;
			}
		}
	}

	PostEditChangeProperty(PropertyEvent);

	FCoreUObjectDelegates::OnPostObjectPropertyChanged.Broadcast(this, PropertyChangedEvent);
}

bool UObject::CanEditChange( const FProperty* InProperty ) const
{
	const bool bIsMutable = !InProperty->HasAnyPropertyFlags( CPF_EditConst );
	return bIsMutable;
}

bool UObject::CanEditChange(const FEditPropertyChain& PropertyChain) const
{
	return CanEditChange(PropertyChain.GetActiveNode()->GetValue());
}

void UObject::PropagatePreEditChange( TArray<UObject*>& AffectedObjects, FEditPropertyChain& PropertyAboutToChange )
{
	// This feature is not essential for overridable serialization
	// Disable it until we fix the fact that do it will create overrides on all the instances
	if (FOverridableManager::Get().IsEnabled(this))
	{
		return;
	}

	TArray<UObject*> Instances;

	for ( int32 i = 0; i < AffectedObjects.Num(); i++ )
	{
		UObject* Obj = AffectedObjects[i];

		// in order to ensure that all objects are saved properly, only process the objects which have this object as their
		// ObjectArchetype since we are going to call Pre/PostEditChange on each object (which could potentially affect which data is serialized
		if ( Obj->GetArchetype() == this || Obj->GetOuter()->GetArchetype() == this )
		{
			// add this object to the list that we're going to process
			Instances.Add(Obj);

			// remove this object from the input list so that when we pass the list to our instances they don't need to check those objects again.
			AffectedObjects.RemoveAt(i--);
		}
	}

	check(PropertyAboutToChange.GetActiveMemberNode() != nullptr);
	const FProperty* ChangedProperty = PropertyAboutToChange.GetActiveMemberNode()->GetValue();
	FPropagatedEditChangeAnnotation Annotation = PropagatedEditChangeAnnotation.GetAnnotation(this);

	// Determine if the changed property belongs to the archetype's class type (or a parent class).
	// Note: IsChildOf() returns false for a NULL owner class (i.e. non-class struct type changes).
	const bool bIsArchetypePropertyChange = GetClass()->IsChildOf(ChangedProperty->GetOwnerClass());

	for ( int32 i = 0; i < Instances.Num(); i++ )
	{
		UObject* Obj = Instances[i];

		// To defer marking instances as dirty, check to see if the instance
		// matches the value stored in its archetype, and flag it if so. We'll
		// use this later to determine if we need to mark the package as dirty,
		// rather than always marking all affected archetype instances as dirty.
		if (Annotation.bDeferredMarkAsDirty)
		{
			// Start with the assumption that the instance matches the archetype. In
			// that case, we won't need to dirty the package after applying the change.
			Annotation.bIdenticalToArchetype = true;

			// If the property that was changed is not a member of the archetype's class
			// type, then it means we're propagating a change event to instances without
			// having also propagated the value change. Thus, there's nothing to compare
			// in this case since we're not inferring dirty state from a property value.
			if (bIsArchetypePropertyChange)
			{
				// Note that some elements may match and thus will propagate, but we may
				// need to dirty the package later even if only one element differs here.
				for (int32 ArrayIdx = 0; ArrayIdx < ChangedProperty->ArrayDim; ++ArrayIdx)
				{
					if (!ChangedProperty->Identical_InContainer(this, Obj, ArrayIdx, PPF_DeepComparison))
					{
						Annotation.bIdenticalToArchetype = false;
						break;
					}
				}
			}

			// Determine if the package is already marked as dirty.
			Annotation.bWasPackageDirtyOnEdit = Obj->GetPackage()->IsDirty();

			// Temporarily annotate the instance for change propagation.
			PropagatedEditChangeAnnotation.AddAnnotation(Obj, Annotation);
		}

		// Note: This test is not the same as the flag above - change propagation can
		// be filtered via the event (e.g. container properties via the Property Editor).
		// For most cases (i.e. non-container), all archetype instances will pass here,
		// regardless of whether or not they differ from the default prior to the change.
		if ( PropertyAboutToChange.IsArchetypeInstanceAffected(Obj) )
		{
			// this object must now be included in any undo/redo operations
			Obj->SetFlags(RF_Transactional);

			// This will call ClearComponents in the Actor case, so that we do not serialize more stuff than we need to.
			Obj->PreEditChange(PropertyAboutToChange);

			// now recurse into this object, saving its instances
			Obj->PropagatePreEditChange(AffectedObjects, PropertyAboutToChange);
		}
	}
}

void UObject::PropagatePostEditChange( TArray<UObject*>& AffectedObjects, FPropertyChangedChainEvent& PropertyChangedEvent )
{
	// This feature is not essential for overridable serialization
	// Disable it until we fix the fact that do it will create overrides on all the instances
	if (FOverridableManager::Get().IsEnabled(this))
	{
		return;
	}

	TArray<UObject*> Instances;

	for ( int32 i = 0; i < AffectedObjects.Num(); i++ )
	{
		UObject* Obj = AffectedObjects[i];

		// in order to ensure that all objects are re-initialized properly, only process the objects which have this object as their
		// ObjectArchetype
		if ( Obj->GetArchetype() == this || Obj->GetOuter()->GetArchetype() == this )
		{
			// add this object to the list that we're going to process
			Instances.Add(Obj);

			// remove this object from the input list so that when we pass the list to our instances they don't need to check those objects again.
			AffectedObjects.RemoveAt(i--);
		}
	}

	check(PropertyChangedEvent.PropertyChain.GetActiveMemberNode() != nullptr);
	const FProperty* ChangedProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();

	TSet<UPackage*> PackagesMarkedAsDirty;
	for ( int32 i = 0; i < Instances.Num(); i++ )
	{
		UObject* Obj = Instances[i];

		UPackage* Package = Obj->GetPackage();
		check(Package);

		// Deferred marking instances as dirty - if our previous value did not
		// match the instance but now the current value does, we need to mark
		// the package dirty to indicate to the user that it needs to be saved.
		FPropagatedEditChangeAnnotation Annotation = PropagatedEditChangeAnnotation.GetAndRemoveAnnotation(Obj);
		if (Annotation.bDeferredMarkAsDirty && !Annotation.bWasPackageDirtyOnEdit && !PackagesMarkedAsDirty.Contains(Package))
		{
			// Clear the dirty flag if the previous value matched the archetype and if
			// the package was not already marked as dirty prior to change propagation.
			const bool bIsPackageDirty = Package->IsDirty();
			if (bIsPackageDirty && Annotation.bIdenticalToArchetype)
			{
				Package->SetDirtyFlag(false);
			}
			else if (!bIsPackageDirty && !Annotation.bIdenticalToArchetype)
			{
				// If any index matches, that element will no longer be delta-serialized,
				// so we need to dirty the package. If the property has multiple entries
				// and we arrived here, that means at least one element differed from its
				// previous value in the source, but that may not be the one that changed.
				for (int32 ArrayIdx = 0; ArrayIdx < ChangedProperty->ArrayDim; ++ArrayIdx)
				{
					if (ChangedProperty->Identical_InContainer(this, Obj, ArrayIdx, PPF_DeepComparison))
					{
						// Using this API so that we don't unnecessarily mark certain packages (e.g. transient).
						Obj->MarkPackageDirty();
						PackagesMarkedAsDirty.Add(Package);
						break;
					}
				}
			}
		}

		// Note: This is not the same as the flag above - change propagation can be
		// filtered via the event (e.g. container properties via the Property Editor),
		// but for most cases (i.e. non-container), all archetype instances pass here.
		if ( PropertyChangedEvent.HasArchetypeInstanceChanged(Obj) )
		{
			// notify the object that all changes are complete
			Obj->PostEditChangeChainProperty(PropertyChangedEvent);

			// now recurse into this object, loading its instances
			Obj->PropagatePostEditChange(AffectedObjects, PropertyChangedEvent);
		}
	}
}

void UObject::SetEditChangePropagationFlags(EEditChangePropagationFlags InFlags)
{
	FPropagatedEditChangeAnnotation Annotation = PropagatedEditChangeAnnotation.GetAnnotation(this);
	Annotation.bDeferredMarkAsDirty = !!(InFlags & EEditChangePropagationFlags::OnlyMarkRealignedInstancesAsDirty);
	PropagatedEditChangeAnnotation.AddAnnotation(this, MoveTemp(Annotation));
}

void UObject::PreEditUndo()
{
	PreEditChange(NULL);
}

void UObject::PostEditUndo()
{
	if( IsValidChecked(this) )
	{
		PostEditChange();
	}
}

void UObject::PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation)
{
	UObject::PostEditUndo();
}

void UObject::PostTransacted(const FTransactionObjectEvent& TransactionEvent)
{
	FCoreUObjectDelegates::OnObjectTransacted.Broadcast(this, TransactionEvent);
}

TSharedPtr<ITransactionObjectAnnotation> UObject::FindOrCreateTransactionAnnotation() const
{
	return FactoryTransactionAnnotation(ETransactionAnnotationCreationMode::FindOrCreate);
}

TSharedPtr<ITransactionObjectAnnotation> UObject::CreateAndRestoreTransactionAnnotation(FArchive& Ar) const
{
	TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation = FactoryTransactionAnnotation(ETransactionAnnotationCreationMode::DefaultInstance);
	if (TransactionAnnotation.IsValid())
	{
		TransactionAnnotation->Serialize(Ar);
		if (Ar.IsError())
		{
			TransactionAnnotation.Reset();
		}
	}
	return TransactionAnnotation;
}

bool UObject::IsSelectedInEditor() const
{
	return IsValidChecked(this) && GIsObjectSelectedInEditor && GIsObjectSelectedInEditor(this);
}

#endif // WITH_EDITOR

void UObject::PostRename(UObject* OldOuter, const FName OldName)
{
#if WITH_EDITOR
	if (HasAnyFlags(RF_WasLoaded))
	{
		FCoreUObjectDelegates::OnLoadedObjectRenamed.Broadcast(this, OldOuter, OldName);
	}
	FCoreUObjectDelegates::OnObjectRenamed.Broadcast(this, OldOuter, OldName);
#endif
}

/**
	Helper class for tracking the list of classes excluded on a certain target system (client/server)
*/
struct FClassExclusionData
{
	TSet<FName> ExcludedClassNames;
	TSet<FName> ExcludedPackageShortNames;
	TSet<FName> CachedExcludeList;
	TSet<FName> CachedIncludeList;

	bool IsExcluded(UClass* InClass)
	{
		FName OriginalClassName = InClass->GetFName();

		FScopeLock ScopeLock(&ExclusionListCrit);
		if (CachedExcludeList.Contains(OriginalClassName))
		{
			return true;
		}

		if (CachedIncludeList.Contains(OriginalClassName))
		{
			return false;
		}

		auto ModuleShortNameFromClass = [](const UClass* Class) -> FName
		{
			return FName(*FPackageName::GetShortName(Class->GetOutermost()));
		};

		while (InClass != nullptr)
		{
			if(ExcludedPackageShortNames.Num() && ExcludedPackageShortNames.Contains(ModuleShortNameFromClass(InClass)))
			{
				UE_LOGF(LogObj, Display, "Class %ls is excluded because its module is excluded in the current platform", *OriginalClassName.ToString());
				CachedExcludeList.Add(OriginalClassName);
				return true;
			}

			if (ExcludedClassNames.Contains(InClass->GetFName()))
			{
				CachedExcludeList.Add(OriginalClassName);
				return true;
			}

			InClass = InClass->GetSuperClass();
		}

		CachedIncludeList.Add(OriginalClassName);
		return false;
	}

	void UpdateExclusionList(const TArray<FString>& InClassNames, const TArray<FString>& InPackageShortNames)
	{
		FScopeLock ScopeLock(&ExclusionListCrit);

		ExcludedClassNames.Empty(InClassNames.Num());
		ExcludedPackageShortNames.Empty(InPackageShortNames.Num());
		CachedIncludeList.Empty();
		CachedExcludeList.Empty();

		for (const FString& ClassName : InClassNames)
		{
			ExcludedClassNames.Add(FName(*ClassName));
		}

		for (const FString& PkgName : InPackageShortNames)
		{
			ExcludedPackageShortNames.Add(FName(*PkgName));
		}
	}

private:
	FCriticalSection ExclusionListCrit;
};

FClassExclusionData GDedicatedServerExclusionList;
FClassExclusionData GDedicatedClientExclusionList;

bool UObject::NeedsLoadForServer() const
{
	bool Ret = false;
	UE_AUTORTFM_OPEN{
		Ret = !GDedicatedServerExclusionList.IsExcluded(GetClass());
	};
	return Ret;
}

void UObject::UpdateClassesExcludedFromDedicatedServer(const TArray<FString>& InClassNames, const TArray<FString>& InModulesNames)
{
	// Avoid adding an UE_AUTORTFM_OPEN as this clears a list which wont be rolled back correctly
	GDedicatedServerExclusionList.UpdateExclusionList(InClassNames, InModulesNames);
}

bool UObject::NeedsLoadForClient() const
{
	bool Ret = false;
	UE_AUTORTFM_OPEN{
		Ret = !GDedicatedClientExclusionList.IsExcluded(GetClass());
	};
	return Ret;
}

void UObject::UpdateClassesExcludedFromDedicatedClient(const TArray<FString>& InClassNames, const TArray<FString>& InModulesNames)
{
	// Avoid adding an UE_AUTORTFM_OPEN as this clears a list which wont be rolled back correctly
	GDedicatedClientExclusionList.UpdateExclusionList(InClassNames, InModulesNames);
}

bool UObject::NeedsLoadForTargetPlatform(const class ITargetPlatform* TargetPlatform) const
{
	return true;
}

bool UObject::CanCreateInCurrentContext(UObject* Template)
{
	check(Template);

	// Ded. server
	if (IsRunningDedicatedServer())
	{
		return Template->NeedsLoadForServer();
	}
	// Client only
	if (IsRunningClientOnly())
	{
		return Template->NeedsLoadForClient();
	}
	// Game, listen server etc.
	if (IsRunningGame())
	{
		return Template->NeedsLoadForClient() || Template->NeedsLoadForServer();
	}

	// other cases (e.g. editor)
	return true;
}


void UObject::GetArchetypeInstances( TArray<UObject*>& Instances )
{
	Instances.Reset();

	if ( HasAnyFlags(RF_ArchetypeObject|RF_ClassDefaultObject) )
	{

		// if this object is the class default object, any object of the same class (or derived classes) could potentially be affected
		if ( !HasAnyFlags(RF_ArchetypeObject) )
		{
			const bool bIncludeNestedObjects = true;
			ForEachObjectOfClass(GetClass(), [this, &Instances](UObject* Obj)
			{
				if (Obj != this)
				{
					Instances.Add(Obj);
				}
			}, bIncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage); // we need to evaluate CDOs as well, but nothing pending kill
		}
		else
		{
			const bool bIncludeNestedObjects = true;
			ForEachObjectOfClass(GetClass(), [this, &Instances](UObject* Obj)
			{
				if (Obj != this && Obj->IsBasedOnArchetype(this))
				{
					Instances.Add(Obj);
				}
			}, bIncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage); // we need to evaluate CDOs as well, but nothing pending kill

		}
	}
}

void UObject::BeginDestroy()
{
	// Sanity assertion to ensure ConditionalBeginDestroy is the only code calling us.
	if( !HasAnyFlags(RF_BeginDestroyed) )
	{
		UE_LOGF(LogObj, Fatal,
			"Trying to call UObject::BeginDestroy from outside of UObject::ConditionalBeginDestroy on object %ls. Please fix up the calling code.",
			*GetName()
			);
	}

	UE::CoreUObject::Private::BeginDestroyPartials(*GetClass(), this);

	// Remove from linker's export table.
	SetLinker( NULL, INDEX_NONE );

	// Remove from the internal hash tables
	// The object was hashed in the open (see UObjectBase constructor calling UObjectBase::AddObject) so we need to unhash it in the open too.
	// There's two places that can call this function:
	// - GC which always runs in the open
	// - Constructing an object on top of an existing object (may be run in the closed) where the object is first destroyed (here) and then re-constucted (this is where UObjectBase::AddObject is being called again)
	// Aborting the reconstuction will always leave the object in a hashed state so no need for an OnAbort handler
	UE_AUTORTFM_OPEN
	{
#if STATS || ENABLE_STATNAMEDEVENTS_UOBJECT
		((UObject*)this)->ResetStatID(); // reset the stat id since this thing is about to be destroyed
#endif
		UnhashObject(this);
	};

	// Remove any associated external package, at this point
	SetExternalPackage(nullptr);

#if WITH_EDITORONLY_DATA
	// Destroy any associated property bag.
	UE::FPropertyBagRepository::Get().DestroyOuterBag(this);
#endif
	
	// ensure BeginDestroy has been routed back to UObject::BeginDestroy.
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	DebugBeginDestroyed.RemoveSingle(this);
#endif
}


void UObject::FinishDestroy()
{
	if( !HasAnyFlags(RF_FinishDestroyed) )
	{
		UE_LOGF(LogObj, Fatal,
			"Trying to call UObject::FinishDestroy from outside of UObject::ConditionalFinishDestroy on object %ls. Please fix up the calling code.",
			*GetName()
			);
	}

	check( !GetLinker() );
	check( GetLinkerIndex()	== INDEX_NONE );

	DestroyNonNativeProperties();

	UClass* Class = GetClass();
	if (Class->GetPropertiesStartOffset())
	{
		UE::CoreUObject::Private::DestructPartials(*Class, this);
	}


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	DebugFinishDestroyed.RemoveSingle(this);
#endif
}


FString UObject::GetDetailedInfo() const
{
	FString Result;  
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IsThisNotNull(this, "UObject::GetDetailedInfo"))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		Result = GetDetailedInfoInternal();
	}
	else
	{
		Result = TEXT("None");
	}
	return Result;  
}

#if WITH_ENGINE

#if DO_CHECK || WITH_EDITOR
// Used to check to see if a derived class actually implemented GetWorld() or not
thread_local bool bGetWorldOverridden = false;
#endif // #if DO_CHECK || WITH_EDITOR

class UWorld* UObject::GetWorld() const
{
	if (UObject* Outer = GetOuter())
	{
		return Outer->GetWorld();
	}

#if DO_CHECK || WITH_EDITOR
	bGetWorldOverridden = false;
#endif
	return nullptr;
}

class UWorld* UObject::GetWorldChecked(bool& bSupported) const
{
#if DO_CHECK
	bGetWorldOverridden = true;
#endif

	UWorld* World = GetWorld();

#if DO_CHECK
	if (!bGetWorldOverridden)
	{
		static FRWLock       ReportedClassesLock;
		static TSet<UClass*> ReportedClasses;

		UClass* UnsupportedClass = GetClass();
		FWriteScopeLock ReportedClassesScopeLock(ReportedClassesLock);
		if (!ReportedClasses.Contains(UnsupportedClass))
		{
			UClass* SuperClass = UnsupportedClass->GetSuperClass();
			FString ParentHierarchy = (SuperClass ? SuperClass->GetName() : TEXT(""));
			while (SuperClass->GetSuperClass())
			{
				SuperClass = SuperClass->GetSuperClass();
				ParentHierarchy += FString::Printf(TEXT(", %s"), *SuperClass->GetName());
			}

			ensureAlwaysMsgf(false, TEXT("Unsupported context object of class %s (SuperClass(es) - %s). You must add a way to retrieve a UWorld context for this class."), *UnsupportedClass->GetName(), *ParentHierarchy);

			ReportedClasses.Add(UnsupportedClass);
		}
	}

	bSupported = bGetWorldOverridden;
	check(World && bSupported);
#else
	bSupported = World != nullptr;
#endif

	return World;
}

#if WITH_EDITOR

bool UObject::ImplementsGetWorld() const
{
	bGetWorldOverridden = true;
	GetWorld();
	return bGetWorldOverridden;
}

#endif // #if WITH_EDITOR
#endif // #if WITH_ENGINE

#define PROFILE_ConditionalBeginDestroy (0)

#if PROFILE_ConditionalBeginDestroy

struct FTimeCnt
{
	float TotalTime;
	int32 Count;

	FTimeCnt()
		: TotalTime(0.0f)
		, Count(0)
	{
	}

	bool operator<(const FTimeCnt& Other) const 
	{
		return TotalTime > Other.TotalTime;
	}
};

static TMap<FName, FTimeCnt> MyProfile;
#endif

bool UObject::ConditionalBeginDestroy()
{
#if !UE_BUILD_SHIPPING
	// if this object wasn't marked (but some were) then that means it was created and destroyed since the SpikeMark command was given
	// this object is contributing to the spike that is being investigated
	if (DebugSpikeMarkAnnotation.Num() > 0)
	{
		if(!DebugSpikeMarkAnnotation.Get(this))
		{
			DebugSpikeMarkNames.Add(GetFullName());
		}
	}
#endif
	
	check(IsValidLowLevel());
	if( !HasAnyFlags(RF_BeginDestroyed) )
	{
		SetFlags(RF_BeginDestroyed);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		checkSlow(!DebugBeginDestroyed.Contains(this));
		DebugBeginDestroyed.Add(this);
#endif

#if PROFILE_ConditionalBeginDestroy
		double StartTime = FPlatformTime::Seconds();
#endif

		BeginDestroy();

#if PROFILE_ConditionalBeginDestroy
		float ThisTime = float(FPlatformTime::Seconds() - StartTime);

		FTimeCnt& TimeCnt = MyProfile.FindOrAdd(GetClass()->GetFName());
		TimeCnt.Count++;
		TimeCnt.TotalTime += ThisTime;

		static float TotalTime = 0.0f;
		static int32 TotalCnt = 0;

		TotalTime += ThisTime;
		if ((++TotalCnt) % 1000 == 0)
		{
			UE_LOGF(LogObj, Log, "ConditionalBeginDestroy %d cnt %fus", TotalCnt, 1000.0f * 1000.0f * TotalTime / float(TotalCnt));

			MyProfile.ValueSort(TLess<FTimeCnt>());

			int32 NumPrint = 0;
			for (auto& Item : MyProfile)
			{
				UE_LOGF(LogObj, Log, "    %6d cnt %6.2fus per   %6.2fms total  %ls", Item.Value.Count, 1000.0f * 1000.0f * Item.Value.TotalTime / float(Item.Value.Count), 1000.0f * Item.Value.TotalTime, *Item.Key.ToString());
				if (NumPrint++ > 30)
				{
					break;
				}
			}
		}
#endif


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if( DebugBeginDestroyed.Contains(this) )
		{
			// class might override BeginDestroy without calling Super::BeginDestroy();
			UE_LOGF(LogObj, Fatal, "%ls failed to route BeginDestroy", *GetFullName() );
		}
#endif
		return true;
	}
	else 
	{
		return false;
	}
}

bool UObject::ConditionalFinishDestroy()
{
	check(IsValidLowLevel());
	if( !HasAnyFlags(RF_FinishDestroyed) )
	{
		SetFlags(RF_FinishDestroyed);
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		checkSlow(!DebugFinishDestroyed.Contains(this));
		DebugFinishDestroyed.Add(this);
#endif
		FinishDestroy();

		// Make sure this object can't be accessed via weak pointers after it's been FinishDestroyed
		GUObjectArray.ResetSerialNumber(this);

		// Make sure this object can't be found through any delete listeners (annotation maps etc) after it's been FinishDestroyed
		GUObjectArray.RemoveObjectFromDeleteListeners(this);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if( DebugFinishDestroyed.Contains(this) )
		{
			UE_LOGF(LogObj, Fatal, "%ls failed to route FinishDestroy", *GetFullName() );
		}
#endif
		return true;
	}
	else 
	{
		return false;
	}
}

void UObject::ConditionalPostLoad()
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectInternals);

	ensureAlwaysMsgf((GetLoaderType() != ELoaderType::ZenLoader) || !HasAnyFlags(RF_NeedLoad), TEXT("Object '%s' still has RF_NeedLoad and is not ready for postload!"), *GetFullName());

	if (HasAnyFlags(RF_NeedPostLoad))
	{
		if (!ensureAlwaysMsgf(IsInGameThread() || HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject) || IsPostLoadThreadSafe() || IsA(UClass::StaticClass()), 
			TEXT("Illegal call to ConditionalPostLoad() on '%s'"), *GetFullName()))
		{
			return;
		}
		UE_TRACK_REFERENCING_PACKAGE_SCOPED(this, PackageAccessTrackingOps::NAME_PostLoad);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		checkSlow(!ThreadContext.DebugPostLoad.Contains(this));
		ThreadContext.DebugPostLoad.Add(this);
#endif
		ClearFlags( RF_NeedPostLoad );

		UObject* ObjectArchetype = GetArchetype();
		if ( ObjectArchetype != NULL )
		{
			//make sure our archetype executes ConditionalPostLoad first.
			ObjectArchetype->ConditionalPostLoad();
		}

		ConditionalPostLoadSubobjects();

		{
			FExclusiveLoadPackageTimeTracker::FScopedPostLoadTracker Tracker(this);

			if (HasAnyFlags(RF_ClassDefaultObject))
			{
				GetClass()->PostLoadDefaultObject(this);
			}
			else
			{
#if WITH_EDITOR
				SCOPED_LOADTIMER_TEXT(*WriteToString<128>(GetClassTraceScope(this), TEXTVIEW("_PostLoad")));
#endif
				UPackage* Package = GetPackage();
				UE_SCOPED_COOK_STAT(Package->GetFName(), EPackageEventStatType::LoadPackage);
				LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Package, ELLMTagSet::Assets);
				LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetClass(), ELLMTagSet::AssetClasses);
				UE_TRACE_METADATA_SCOPE_ASSET_FNAME(GetFName(), GetClass()->GetFName(), Package->GetFName())
				TRACE_LOADTIME_POSTLOAD_OBJECT_SCOPE(this);

				PostLoad();

				LLM_PUSH_STATS_FOR_ASSET_TAGS();
			}
		}

#if WITH_EDITORONLY_DATA
		// Object has been deserialized, so update its associated IDO. Note that this should be done
		// *after* PostLoad() is called, since that could mutate already-serialized fields on this object,
		// and we need those changes to also be reflected on the IDO, which was created at serialization time.
		if (UE::IsInstanceDataObjectSupportEnabled(this) && !FUObjectThreadContext::Get().GetSerializeContext()->bImpersonateProperties)
		{
			UE::FPropertyBagRepository& PropertyBagRepository = UE::FPropertyBagRepository::Get();
			PropertyBagRepository.PostLoadInstanceDataObject(this);
		}
#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (ThreadContext.DebugPostLoad.Contains(this))
		{
			UE_LOGF(LogObj, Fatal, "%ls failed to route PostLoad.  Please call Super::PostLoad() in your <className>::PostLoad() function.", *GetFullName());
		}
#endif
	}
}

void UObject::ConditionalPreload()
{
	// The check for EInternalObjectFlags::AsyncLoadingPhase1 is important here to make sure we go
	// through the Preload function to get rid of any potential race condition that might happen
	// with the parallel loader threads when going wide.
	if (HasAnyFlags(RF_NeedLoad) || HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1))
	{
		if (FLinkerLoad* LinkerLoad = GetLinker())
		{
			LinkerLoad->Preload(this);
		}
	}
}

void UObject::PostLoadSubobjects( FObjectInstancingGraph* OuterInstanceGraph/*=NULL*/ )
{
	// if this class contains instanced object properties and a new object property has been added since this object was saved,
	// this object won't receive its own unique instance of the object assigned to the new property, since we don't instance object during loading
	// so go over all instanced object properties and look for cases where the value for that property still matches the default value.

	if( GetClass()->HasAnyClassFlags(CLASS_HasInstancedReference) )
	{
		UObject* ObjOuter = GetOuter();
		// make sure our Outer has already called ConditionalPostLoadSubobjects
		if (ObjOuter != NULL && ObjOuter->HasAnyFlags(RF_NeedPostLoadSubobjects) )
		{
			if (ObjOuter->HasAnyFlags(RF_NeedPostLoad) )
			{
				ObjOuter->ConditionalPostLoad();
			}
			else
			{
				ObjOuter->ConditionalPostLoadSubobjects();
			}
			if ( !HasAnyFlags(RF_NeedPostLoadSubobjects) )
			{
				// if calling ConditionalPostLoadSubobjects on our Outer resulted in ConditionalPostLoadSubobjects on this object, stop here
				return;
			}
		}

		// clear the flag so that we don't re-enter this method
		ClearFlags(RF_NeedPostLoadSubobjects);

		// Override serialization and class types with non-serialized references may require instancing in cooked build configurations.
		auto NeedsPostLoadSubobjectInstancing = [](TNotNull<const UObject*> InObj)
		{
			// The instance is modified, but the reference itself is not overridden/serialized
			if (FOverridableManager::Get().NeedSubObjectTemplateInstantiation(InObj))
			{
				return true;
			}

			// The class type has transient default subobjects that require instancing on load
			if (InObj->GetClass()->RequiresPostLoadSubobjectInstancing())
			{
				return true;
			}

			return false;
		};

		// Cooked data will already have its subobjects fully instanced as uninstanced subobjects are only due to newly introduced subobjects in
		// an archetype that an instance of that object hasn't been saved with. Platforms that include editor-only data still require this step
		// if the outer package is cooked in order to properly instance any editor-only subobjects that will not have otherwise been serialized.
		// 
		// @todo_LoadPerf - Consider modifying to bypass this for cooked packages if the class doesn't include any instanced editor-only fields.
		// Currently we do not have a flag that indicates this scenario, and it might be expensive to iterate over the linked property chain here.
		if (!FPlatformProperties::RequiresCookedData() || NeedsPostLoadSubobjectInstancing(this))
		{
			// In a cooked build, for references explicitly marked as always instanced, optimize to only those cases that did not get serialized.
			EObjectInstancingGraphOptions Options = EObjectInstancingGraphOptions::None;
			if (FPlatformProperties::RequiresCookedData())
			{
				Options = EObjectInstancingGraphOptions::InstanceTemplatesOnly;
			}

			FObjectInstancingGraph CurrentInstanceGraph(Options);

			FObjectInstancingGraph* InstanceGraph = OuterInstanceGraph;
			if (InstanceGraph == NULL)
			{
				CurrentInstanceGraph.SetDestinationRoot(this);
				CurrentInstanceGraph.SetLoadingObject(true);

				// if we weren't passed an instance graph to use, create a new one and use that
				InstanceGraph = &CurrentInstanceGraph;
			}

			TArray<UObject*> SerializedComponents;
			// fill the array with the top level default subobjects
			GetDefaultSubobjects(SerializedComponents);

			// now, add all of the instanced components to the instance graph that will be used for instancing any components that have been added
			// to this object's archetype since this object was last saved
			for (int32 ComponentIndex = 0; ComponentIndex < SerializedComponents.Num(); ComponentIndex++)
			{
				UObject* PreviouslyInstancedComponent = SerializedComponents[ComponentIndex];
				InstanceGraph->AddNewInstance(PreviouslyInstancedComponent);
			}

			InstanceSubobjectTemplates(InstanceGraph);
		}
	}
	else
	{
		// clear the flag so that we don't re-enter this method
		ClearFlags(RF_NeedPostLoadSubobjects);
	}
}

UScriptStruct* UObject::GetSparseClassDataStruct() const
{
	UClass* Class = GetClass();
	return Class ? Class->GetSparseClassDataStruct() : nullptr;
}

void UObject::ConditionalPostLoadSubobjects( FObjectInstancingGraph* OuterInstanceGraph/*=NULL*/ )
{
	if ( HasAnyFlags(RF_NeedPostLoadSubobjects) )
	{
		PostLoadSubobjects(OuterInstanceGraph);
	}
	CheckDefaultSubobjects();
}

void UObject::PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext)
{
}

void UObject::PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext)
{
}

void UObject::PreSave(FObjectPreSaveContext SaveContext)
{
	SaveContext.Data.bBaseClassCalled = true;
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPreSave.Broadcast(this, SaveContext);

	// Validate ClassWithin before save, matches code in StaticAllocateObjectErrorTests
	UClass* ObjClass = GetClass();
	if (ObjClass->ClassWithin != nullptr)
	{
		if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) && GetOuter() != nullptr && !GetOuter()->IsA(ObjClass->ClassWithin))
		{
			const FString ErrorMsg = FString::Printf(TEXT("Object %s with ClassWithin of %s is being saved in invalid Outer of class %s!"), *GetFullName(nullptr, EObjectFullNameFlags::IncludeClassPackage), *ObjClass->ClassWithin->GetPathName(), *GetOuter()->GetClass()->GetPathName());
			UE_LOGF(LogUObjectGlobals, Error, "%ls", *ErrorMsg);
			ensureMsgf(false, TEXT("%s"), *ErrorMsg);
		}
	}
#endif
	CollectSaveOverrides(FObjectCollectSaveOverridesContext(SaveContext.Data));
}

void UObject::CollectSaveOverrides(FObjectCollectSaveOverridesContext SaveContext)
{
}

#if WITH_EDITOR
bool UObject::CanModify() const
{
	return
		!HasAnyFlags(RF_NeedInitialization) && !IsGarbageCollecting() && !GExitPurge && !IsUnreachable() &&
		// Prevent modification while loading
		!HasAnyInternalFlags(EInternalObjectFlags::Async | EInternalObjectFlags_AsyncLoading) &&
		// Only the game-thread should be allowed to touch the transaction buffer at all
		IsInGameThread();
}

bool UObject::Modify( bool bAlwaysMarkDirty/*=true*/ )
{
	bool bSavedToTransactionBuffer = false;

	if (CanModify())
	{
		// Do not consider script packages, as they should never end up in the
		// transaction buffer and we don't want to mark them dirty here either.
		// We do want to consider PIE objects however
		if (GetOutermost()->HasAnyPackageFlags(PKG_ContainsScript | PKG_CompiledIn) == false || GetClass()->HasAnyClassFlags(CLASS_DefaultConfig | CLASS_Config))
		{
			// Attempt to mark the package dirty and save a copy of the object to the transaction
			// buffer. The save will fail if there isn't a valid transactor, the object isn't
			// transactional, etc.
			bSavedToTransactionBuffer = SaveToTransactionBuffer(this, bAlwaysMarkDirty);

			// If we failed to save to the transaction buffer, but the user requested the package
			// marked dirty anyway, do so
			if (!bSavedToTransactionBuffer && bAlwaysMarkDirty)
			{
				MarkPackageDirty();
			}
		}

		FCoreUObjectDelegates::BroadcastOnObjectModified(this);
	}

	return bSavedToTransactionBuffer;
}

bool UObject::IsCapturingAsRootObjectForTransaction() const
{
	return false;
}
#endif

bool UObject::IsSelected() const
{
#if WITH_EDITOR
	return IsSelectedInEditor();
#else
	return false;
#endif
}

void UObject::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	UClass *ObjClass = GetClass();
	if (!ObjClass->HasAnyClassFlags(CLASS_Intrinsic))
	{
		OutDeps.Add(ObjClass);

		if (HasAnyFlags(RF_ClassDefaultObject))
		{
			ObjClass->GetDefaultObjectPreloadDependencies(OutDeps);
		}
		else if (ObjClass->GetDefaultsCount() > 0)
		{
			OutDeps.Add(ObjClass->GetDefaultObject());
		}
	}

	// The iterator will recursively loop through all structs in structs/containers too.
	for (TPropertyValueIterator<FStructProperty> It(ObjClass, this); It; ++It)
	{
		const UScriptStruct* StructType = It.Key()->Struct;
		if (UScriptStruct::ICppStructOps* CppStructOps = StructType->GetCppStructOps())
		{
			void* StructDataPtr = const_cast<void*>(It.Value());
			CppStructOps->GetPreloadDependencies(StructDataPtr, OutDeps);
		}
	}
}

IMPLEMENT_FARCHIVE_SERIALIZER(UObject)

void UObject::Serialize(FStructuredArchive::FRecord Record)
{
	SCOPED_LOADTIMER(UObject_Serialize);

	{
		FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

		// This item is very special from a serialization standpoint. It is never actually serialized.
		UClass *ObjClass = GetClass();

		// Make sure this object's class's data is loaded.
		if (ObjClass->HasAnyFlags(RF_NeedLoad) )
		{
			UnderlyingArchive.Preload(ObjClass);

			// make sure this object's template data is loaded - the only objects
			// this should actually affect are those that don't have any defaults
			// to serialize.  for objects with defaults that actually require loading
			// the class default object should be serialized in FLinkerLoad::Preload, before
			// we've hit this code.
			if ( !HasAnyFlags(RF_ClassDefaultObject) && ObjClass->GetDefaultsCount() > 0 )
			{
				UnderlyingArchive.Preload(ObjClass->GetDefaultObject());
			}
		}

#if WITH_EDITORONLY_DATA
		// Allow Null to be deserialized into a NonNull property
		// Enabled for Undo/Redo purposes
		UE::CoreUObject::FScopedSetNullOnNonNullable ScopedAllowNullPtrDeserialization = [&UnderlyingArchive]()
		{
			TOptional<UE::CoreUObject::EAllowSetNullOnNonNullableBehavior> Behaviour;
			if (UnderlyingArchive.IsTransacting() && UnderlyingArchive.IsLoading())
			{
				Behaviour.Emplace(UE::CoreUObject::EAllowSetNullOnNonNullableBehavior::Enabled);
			}
			return UE::CoreUObject::FScopedSetNullOnNonNullable(Behaviour);
		}();
#endif
		
		// Special info.
		if ((!UnderlyingArchive.IsLoading() && !UnderlyingArchive.IsSaving() && !UnderlyingArchive.IsObjectReferenceCollector()))
		{
			// This item is very special from a serialization standpoint. It is never actually serialized.
			// Only query the outer and FName when necessary to avoid race conditions with UObject rename when serializing for reference collection
			FName LoadName = GetFName();
			Record << SA_VALUE(TEXT("LoadName"), LoadName);
			if (!UnderlyingArchive.IsIgnoringOuterRef())
			{
				// This item is very special from a serialization standpoint. It is never actually serialized.
				// Only query the outer and FName when necessary to avoid race conditions with UObject rename when serializing for reference collection
				UObject* LoadOuter = GetOuter();
				Record << SA_VALUE(TEXT("LoadOuter"), LoadOuter);
			}
			if (!UnderlyingArchive.IsIgnoringClassRef())
			{
				Record << SA_VALUE(TEXT("ObjClass"), ObjClass);
			}
		}
		// Special support for supporting undo/redo of renaming and changing Archetype.
		else if (UnderlyingArchive.IsTransacting())
		{
			if (!UnderlyingArchive.IsIgnoringOuterRef())
			{
				UPackage* LoadPackage = GetExternalPackage();

				// These items are very special from a serialization standpoint. They are never actually serialized.
				// Only query the outer and FName when necessary to avoid race conditions with UObject rename when serializing for reference collection
				FName LoadName = GetFName();
				UObject* LoadOuter = GetOuter();

				if (UnderlyingArchive.IsLoading())
				{
					Record << SA_VALUE(TEXT("LoadName"), LoadName) << SA_VALUE(TEXT("LoadOuter"), LoadOuter) << SA_VALUE(TEXT("LoadPackage"), LoadPackage);

					// If the name we loaded is different from the current one,
					// unhash the object, change the name and hash it again.
					bool bDifferentName = GetFName() != NAME_None && LoadName != GetFName();
					bool bDifferentOuter = LoadOuter != GetOuter();
					if ( bDifferentName == true || bDifferentOuter == true )
					{
						// Clear the name for use by this:
						UObject* Collision = StaticFindObjectFast(UObject::StaticClass(), LoadOuter, LoadName);
						if(Collision && Collision != this)
						{
							FName NewNameForCollision = MakeUniqueObjectName(LoadOuter, Collision->GetClass(), LoadName);
							checkf( StaticFindObjectFast(UObject::StaticClass(), LoadOuter, NewNameForCollision) == nullptr,
								TEXT("Failed to MakeUniqueObjectName for object colliding with transaction buffer state: %s %s"),
								*LoadName.ToString(),
								*NewNameForCollision.ToString()
							);
							Collision->LowLevelRename(NewNameForCollision,LoadOuter);
#if DO_CHECK
							UObject* SubsequentCollision = StaticFindObjectFast(UObject::StaticClass(), LoadOuter, LoadName);
							checkf( SubsequentCollision == nullptr,
								TEXT("Multiple name collisions detected in the transaction buffer: %" UPTRINT_x_FMT " %" UPTRINT_x_FMT " with name %s"),
								Collision,
								SubsequentCollision,
								*LoadName.ToString()
							);
#endif
						}
						
						LowLevelRename(LoadName,LoadOuter);
					}

					// Set the package override
					SetExternalPackage(LoadPackage);
				}
				else
				{
					Record << SA_VALUE(TEXT("LoadName"), LoadName) << SA_VALUE(TEXT("LoadOuter"), LoadOuter) << SA_VALUE(TEXT("LoadPackage"), LoadPackage);
				}
			}
		}

		// Serialize object properties which are defined in the class.
		// Handle derived UClass objects (exact UClass objects are native only and shouldn't be touched)
		if (ObjClass != UClass::StaticClass())
		{
			// Serializing the overridden properties if using UPS (like object duplication)
			// Also needed for transactions using TPS
			if (!UnderlyingArchive.IsCooking() && 
				!UnderlyingArchive.IsLoadingFromCookedPackage() &&
				(UnderlyingArchive.UseUnversionedPropertySerialization() || UnderlyingArchive.IsTransacting()))
			{
				FOverridableManager::Get().SerializeOverriddenProperties(this, Record);
#if WITH_EDITORONLY_DATA
				UE::FInitializedPropertyValueState(this).Serialize(Record);
#endif
			}

			SerializeScriptProperties(Record.EnterField(TEXT("Properties")));
		}

		// Keep track of transient
		if (UnderlyingArchive.IsTransacting())
		{
			bool WasTransient = HasAnyFlags(RF_Transient);
			if (UnderlyingArchive.IsLoading())
			{
				Record << SA_VALUE(TEXT("WasTransient"), WasTransient);
				if (WasTransient)
				{
					SetFlags(RF_Transient);
				}
				else
				{
					ClearFlags(RF_Transient);
				}
			}
			else if (UnderlyingArchive.IsSaving())
			{
				Record << SA_VALUE(TEXT("WasTransient"), WasTransient);
			}
		}

		// Serialize a GUID if this object has one mapped to it
		FLazyObjectPtr::PossiblySerializeObjectGuid(this, Record);

		// Keep track of sparse class data for undo/redo
		if (UnderlyingArchive.IsTransacting() && HasAnyFlags(RF_ClassDefaultObject))
		{
			UScriptStruct* SerializedSparseClassDataStruct = ObjClass->GetSparseClassDataStruct();
			if (UnderlyingArchive.IsSaving() && !ObjClass->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull))
			{
				SerializedSparseClassDataStruct = nullptr;
			}
			Record << SA_VALUE(TEXT("SparseClassDataStruct"), SerializedSparseClassDataStruct);

			if (SerializedSparseClassDataStruct)
			{
				ObjClass->SerializeSparseClassData(Record.EnterField(TEXT("SparseClassData")));
			}
		}

		// Memory counting (with proper alignment to match C++)
		SIZE_T Size = GetClass()->GetStructureSize();
		UnderlyingArchive.CountBytes(Size, Size);
	}
}

#if WITH_EDITORONLY_DATA
void UObject::DeclareCustomVersions(FArchive& Ar, const UClass* SpecificSubclass)
{
	// DeclareCustomVersions is called on the default object for each class
	// We first Serialize the object, which catches all the UsingCustomVersion statements
	// class authors have added unconditionally in their Serialize function
	UObject* CDO = SpecificSubclass->GetDefaultObject();
	// Some classes such as Default__BlueprintGeneratedClass do not have a CDO
	if (!CDO)
	{
		return;
	}
	CDO->Serialize(Ar);

	// To further catch CustomVersions used by non-native structs that are in an array or don't
	// otherwise exist on the default object, Construct an instance of the struct and serialize
	// it for every struct property in the Class.
	// Since structs can contain other structs, we do a tree search of the fields.
	struct FStackData
	{
		const UStruct* Struct;
		FProperty* NextProperty;
	};
	TArray<FStackData> StructStack;
	StructStack.Add(FStackData{ SpecificSubclass, SpecificSubclass->PropertyLink });
	TArray<uint8> AllocationBuffer;
	while (!StructStack.IsEmpty())
	{
		FStackData& StackData = StructStack.Last();
		FProperty*& Property = StackData.NextProperty;
		bool bPushedStack = false;
		while (Property)
		{
			FProperty* InnerProperty = Property;
			Property = Property->PropertyLinkNext;
			FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InnerProperty);
			if (ArrayProperty)
			{
				InnerProperty = ArrayProperty->Inner;
			}
			FStructProperty* StructProperty = CastField<FStructProperty>(InnerProperty);
			if (StructProperty)
			{
				UStruct* Struct = StructProperty->Struct;
				if (StructStack.ContainsByPredicate(
					[Struct](const FStackData& InStackData) { return InStackData.Struct == Struct; }))
				{
					// A cycle in the declarations (struct FA { FB B; }; struct FB { FA A; };)
					// This is invalid, but avoid an infinite loop by skipping the nested struct.
					continue;
				}
				// We handle structs that are direct members (not a pointer)
				// UObjects and structs cannot have a UObject as a direct member.
				// We rely on not having to handle it; we can construct Structs in our earliest calls,
				// but constructing a UObject during startup would cause problems.
				check(!Struct->IsA<UClass>());
				UScriptStruct* ScriptStruct = Cast<UScriptStruct>(Struct);
				if (ScriptStruct)
				{
					// Construct an instance and collect CustomProperties from it via Serialize
					int32 Size = ScriptStruct->GetPropertiesSize();
					int32 Alignment = ScriptStruct->GetMinAlignment();
					AllocationBuffer.SetNumUninitialized(Align(Size, Alignment) + Alignment);
					uint8* StructBytes = Align(AllocationBuffer.GetData(), Alignment);
					ScriptStruct->InitializeStruct(StructBytes);
					ScriptStruct->SerializeItem(Ar, StructBytes, nullptr);
					ScriptStruct->DestroyStruct(StructBytes);
				}
				StructStack.Add(FStackData{ Struct, Struct->PropertyLink });
				bPushedStack = true;
				break;
			}
		}
		if (!bPushedStack)
		{
			StructStack.Pop(EAllowShrinking::No);
		}
	}
}

void UObject::AppendToClassSchema(FAppendToClassSchemaContext& Context)
{
}

void UObject::DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass) 
{
}

void UObject::InjectDynamicImportsFor(const class FLinkerLoad& Linker, const struct FObjectExport& Export, TArray<struct FDynamicImport>& OutDynamicImports)
{
}
#endif

void UObject::SerializeScriptProperties(FArchive& Ar) const
{
	SerializeScriptProperties(FStructuredArchiveFromArchive(Ar).GetSlot());
}

#if WITH_EDITORONLY_DATA
namespace UE::Private
{
	/**
	 *  Query if an object has another object used in its stead for TPS serialization
	 *  This can be useful for backward compatibility testing or delaying version upgrading of data
	 */
	const UObject* GetDataImpersonator(const UObject* ThisObject)
	{
		if (ThisObject == nullptr)
		{
			return nullptr;
		}
		if (FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
			SerializeContext && !SerializeContext->bImpersonateProperties)
		{
			return nullptr;
		}
		return FPropertyBagRepository::Get().FindInstanceDataObject(ThisObject);
	}
}
#endif

void UObject::SerializeScriptProperties( FStructuredArchive::FSlot Slot ) const
{
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

#if WITH_EDITORONLY_DATA
	if (SerializeContext->SerializedObject == this)
	{
		SerializeContext->SerializedObjectScriptStartOffset = UnderlyingArchive.Tell();
	}
#endif
	UnderlyingArchive.MarkScriptSerializationStart(this);
	if( HasAnyFlags(RF_ClassDefaultObject) )
	{
		UnderlyingArchive.StartSerializingDefaults();
	}

	UClass *ObjClass = GetClass();

	if(UnderlyingArchive.IsTextFormat() || ((UnderlyingArchive.IsLoading() || UnderlyingArchive.IsSaving()) && !UnderlyingArchive.WantBinaryPropertySerialization()))
	{
		//@todoio GetArchetype is pathological for blueprint classes and the event driven loader; the EDL already knows what the archetype is; just calling this->GetArchetype() tries to load some other stuff.
		UObject* DiffObject = UnderlyingArchive.GetArchetypeFromLoader(this);
		if (!DiffObject)
		{
			DiffObject = GetArchetype();
		}

		// When migrating remote objects the only instance where we serialize a CDO is when resetting an object to its archetype state in which case
		// we want to serialize against the actual object class to get the right delta
		UClass* DiffClass = (HasAnyFlags(RF_ClassDefaultObject) && !UnderlyingArchive.HasAnyPortFlags(PPF_AvoidRemoteObjectMigration)) ? ObjClass->GetSuperClass() : ObjClass;

		// Query if this object data is being impersonated 
		const UObject* ThisObject = this;
#if WITH_EDITORONLY_DATA
		if (const UObject* Impersonator = UE::Private::GetDataImpersonator(ThisObject))
		{
			ThisObject = Impersonator;
			ObjClass = ThisObject->GetClass();

			// Find (or create) an impersonator for the archetype that matches the same class so it's serializable
			DiffObject = UE::GetTemplateForInstanceDataObject(this, DiffObject, Impersonator->GetClass());
			if (!DiffObject || !ThisObject->IsA(DiffObject->GetClass()))
			{
				DiffObject = ObjClass->GetDefaultObject();
			}
			DiffClass = DiffObject ? DiffObject->GetClass() : nullptr;

			ensureAlwaysMsgf(ThisObject->IsA(DiffClass), TEXT("Impersonation of '%s' using a different default class not appropriately supported at the moment. Class: '%s', DefaultClass: '%s'")
				, *ThisObject->GetPathName(), *ObjClass->GetPathName(), *DiffClass->GetPathName());
		}
#endif

#if WITH_EDITOR
		static const FBoolConfigValueHelper BreakSerializationRecursion(TEXT("StructSerialization"), TEXT("BreakSerializationRecursion"));
		const bool bBreakSerializationRecursion = BreakSerializationRecursion && UnderlyingArchive.IsLoading() && UnderlyingArchive.GetLinker();

		static const FName NAME_SerializeScriptProperties = FName("SerializeScriptProperties");
		FArchive::FScopeAddDebugData P(UnderlyingArchive, NAME_SerializeScriptProperties);
		FArchive::FScopeAddDebugData S(UnderlyingArchive, ObjClass->GetFName());
#else 
		const bool bBreakSerializationRecursion = false;
#endif
		ObjClass->SerializeTaggedProperties(Slot, (uint8*)ThisObject, DiffClass, (uint8*)DiffObject, bBreakSerializationRecursion ? ThisObject : nullptr);
	}
	else if (UnderlyingArchive.GetPortFlags() != 0 && !UnderlyingArchive.ArUseCustomPropertyList )
	{
		//@todoio GetArchetype is pathological for blueprint classes and the event driven loader; the EDL already knows what the archetype is; just calling this->GetArchetype() tries to load some other stuff.
		UObject* DiffObject = UnderlyingArchive.GetArchetypeFromLoader(this);
		if (!DiffObject)
		{
			DiffObject = GetArchetype();
		}
		ObjClass->SerializeBinEx(Slot, const_cast<UObject *>(this), DiffObject, DiffObject ? DiffObject->GetClass() : nullptr);
	}
	else
	{
		ObjClass->SerializeBin(Slot, const_cast<UObject *>(this));
	}

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		UnderlyingArchive.StopSerializingDefaults();
	}
	UnderlyingArchive.MarkScriptSerializationEnd(this);
#if WITH_EDITORONLY_DATA
	if (SerializeContext->SerializedObject == this)
	{
		SerializeContext->SerializedObjectScriptEndOffset = UnderlyingArchive.Tell();
	}
#endif
}


void UObject::BuildSubobjectMapping(UObject* OtherObject, TMap<UObject*, UObject*>& ObjectMapping) const
{
	UPackage* ThisPackage = GetOutermost();
	UPackage* OtherPackage = OtherObject->GetOutermost();

	// ForEachObjectWithOuter is taking a hash lock, and we can't load new objects inside a hash lock
	// so gather objects first, and process them after the hash lock is released.
	TArray<UObject*> ObjectsToProcess;
	ForEachObjectWithOuter(this, [&](UObject* InSubObject)
	{
		ObjectsToProcess.Add(InSubObject);
	}, EGetObjectsFlags::None, RF_NoFlags, EInternalObjectFlags::Garbage);

	for (UObject* SubObject : ObjectsToProcess)
	{
		if (ObjectMapping.Contains(SubObject))
		{
			return;
		}

		FString NewSubObjectName = SubObject->GetName();

		UClass* OtherSubObjectClass = SubObject->GetClass();
#if WITH_EDITORONLY_DATA
		if (OtherSubObjectClass->ClassGeneratedBy && OtherSubObjectClass->ClassGeneratedBy->GetOutermost() == ThisPackage)
		{
			// This is a generated class type, so we actually need to use the new generated class type from the new package otherwise our type check will fail
			FString NewClassName = OtherSubObjectClass->GetPathName(ThisPackage);
			NewClassName = FString::Printf(TEXT("%s.%s"), *OtherPackage->GetName(), *NewClassName);

			OtherSubObjectClass = LoadObject<UClass>(OtherPackage, *NewClassName);
		}
#endif

		UObject* OtherSubObject = StaticFindObjectFast(OtherSubObjectClass, OtherObject, *NewSubObjectName);
		ObjectMapping.Emplace(SubObject, OtherSubObject);

		if (OtherSubObject)
		{
			SubObject->BuildSubobjectMapping(OtherSubObject, ObjectMapping);
		}
	};
}

/**
 * FSubobjectReferenceFinder.
 * Helper class used to collect default subobjects of other objects than the referencing object.
 */
class FSubobjectReferenceFinder : public FReferenceCollector
{
public:

	/**
	 * Constructor
	 *
	 * @param InSubobjectArray	Array to add subobject references to
	 * @param	InObject	Referencing object.
	 */
	FSubobjectReferenceFinder(TArray<const UObject*>& InSubobjectArray, const UObject* InObject)
		:	ObjectArray(InSubobjectArray)
		, ReferencingObject(InObject)
	{
		check(ReferencingObject != NULL);
		FindSubobjectReferences();
	}

	/**
	 * Finds all default subobjects of other objects referenced by ReferencingObject.
	 */
	void FindSubobjectReferences()
	{
		if( !ReferencingObject->GetClass()->IsChildOf(UClass::StaticClass()) )
		{
			// Didn't dare switching from SerializeScriptProperties to new and faster AddPropertyReferencers.
			// This collector IsIgnoringTransient and SerializeScriptProperties will skip
			// transient default objects but AddPropertyReferencers / SerializeBin will skip all transient 
			// properties, including default ones. Not sure if this matters.
			// 
			// See FReferenceFinder::FindReferences whose collector doesn't ignore transient
			FVerySlowReferenceCollectorArchiveScope CollectorScope(GetVerySlowReferenceCollectorArchive(), ReferencingObject);
			ReferencingObject->SerializeScriptProperties(CollectorScope.GetArchive());
		}
		// CallAddReferencedObjects doesn't modify the object with FSubobjectReferenceFinder passed in as parameter but may modify when called by GC
		UObject* MutableReferencingObject = const_cast<UObject*>(ReferencingObject);
		MutableReferencingObject->CallAddReferencedObjects(*this);
	}

	// Begin FReferenceCollector interface.
	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		// Only care about unique default subobjects that are outside of the referencing object's outer chain.
		// Also ignore references to subobjects if they share the same Outer.
		// Ignore references from the subobject Outer's class (ComponentNameToDefaultObjectMap).
		if (InObject != NULL && InObject->HasAnyFlags(RF_DefaultSubObject) && ObjectArray.Contains(InObject) == false && InObject->IsIn(ReferencingObject) == false &&
			 (ReferencingObject->GetOuter() != InObject->GetOuter() && InObject != ReferencingObject->GetOuter()) &&
			 (InReferencingObject == NULL || (InReferencingObject != InObject->GetOuter()->GetClass() && ReferencingObject != InObject->GetOuter()->GetClass())))
		{
			check(InObject->IsValidLowLevel());
			ObjectArray.Add(InObject);
		}
	}
	virtual bool IsIgnoringArchetypeRef() const override { return true; }
	virtual bool IsIgnoringTransient() const override { return true; }
	// End FReferenceCollector interface.

protected:

	/** Stored reference to array of objects we add object references to. */
	TArray<const UObject*>&	ObjectArray;
	/** Object to check the references of. */
	const UObject* ReferencingObject;
};

#define CompCheck(Pred) \
	if (!(Pred)) \
	{ \
		Result = false; \
		UE_DEBUG_BREAK(); \
		UE_LOGF(LogObj, Error, "CompCheck %ls failed.", TEXT(#Pred)); \
	}

bool UObject::CanCheckDefaultSubObjects(bool bForceCheck, bool& bResult) const
{
	bool bCanCheck = true;
	bResult = true;
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsThisNotNull(this, "UObject::CanCheckDefaultSubObjects"))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		bResult = false; // these aren't in a suitable spot in their lifetime for testing
		bCanCheck = false;
	}
	if (bCanCheck && (HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects) || !IsValidChecked(this) || IsUnreachable() || GIsDuplicatingClassForReinstancing))
	{
		bResult = true; // these aren't in a suitable spot in their lifetime for testing
		bCanCheck = false;
	}
	// If errors are suppressed, we will not take the time to run this test unless forced to.
	bCanCheck = bCanCheck && (bForceCheck || bDoCheckSubobjects);
	return bCanCheck;
}

bool UObject::CheckDefaultSubobjects(bool bForceCheck /*= false*/) const
{
	bool Result = true;
	if (CanCheckDefaultSubObjects(bForceCheck, Result))
	{
		Result = CheckDefaultSubobjectsInternal();
	}
	return Result;
}

bool UObject::CheckDefaultSubobjectsInternal() const
{
	bool Result = true;

	UClass* ObjClass = GetClass();

	if (ObjClass != UFunction::StaticClass())
	{
		// Check for references to default subobjects of other objects.
		// There should never be a pointer to a subobject from outside of the outer (chain) it belongs to.
		TArray<const UObject*> OtherReferencedSubobjects;
		FSubobjectReferenceFinder DefaultSubobjectCollector(OtherReferencedSubobjects, this);
		for (int32 Index = 0; Index < OtherReferencedSubobjects.Num(); ++Index)
		{
			const UObject* TestObject = OtherReferencedSubobjects[Index];
			UE_LOGF(LogObj, Error, "%ls has a reference to default subobject (%ls) of %ls.", *GetFullName(), *TestObject->GetFullName(), *TestObject->GetOuter()->GetFullName());
		}
		CompCheck(OtherReferencedSubobjects.Num() == 0);
	}

#if 0 // usually overkill, but valid tests
	if (!HasAnyFlags(RF_ClassDefaultObject) && ObjClass->HasAnyClassFlags(CLASS_HasInstancedReference))
	{
		UObject *Archetype = GetArchetype();
		CompCheck(this != Archetype);
		Archetype->CheckDefaultSubobjects();
		if (Archetype != ObjClass->GetDefaultObject())
		{
			ObjClass->GetDefaultObject()->CheckDefaultSubobjects();
		}
	}
#endif

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		CompCheck(GetFName() == ObjClass->GetDefaultObjectName());
	}


	return Result;
}

/**
 * Determines whether the specified object should load values using PerObjectConfig rules
 */
bool UsesPerObjectConfig( UObject* SourceObject )
{
	checkSlow(SourceObject);
	return (SourceObject->GetClass()->HasAnyClassFlags(CLASS_PerObjectConfig) && !SourceObject->HasAnyFlags(RF_ClassDefaultObject));
}

/**
 * Returns the file to load ini values from for the specified object, taking into account PerObjectConfig-ness
 */
FString GetConfigFilename( UObject* SourceObject )
{
	checkSlow(SourceObject);

	// look at the class to get the config name
	return SourceObject->GetClass()->GetConfigName();
}

namespace UE::Object::Private
{

static void GetAssetRegistryTagFromProperty(const void* BaseMemoryLocation, const UObject* OwnerObject, FProperty* Prop, FAssetRegistryTagsContext Context)
{
	FStructProperty* StructProp = CastField<FStructProperty>(Prop);
	if (StructProp && StructProp->Struct && StructProp->Struct->GetFName() == GAssetBundleDataName)
	{
		const FAssetBundleData* Bundles = reinterpret_cast<const FAssetBundleData*>(Prop->ContainerPtrToValuePtr<uint8>(BaseMemoryLocation));

		if (Context.WantsBundleResult())
		{
			checkf(Context.GetBundleResult() == nullptr, TEXT("Object %s has more than one FAssetBundleData!"), *OwnerObject->GetPathName());
			Context.SetBundleResult(Bundles);
		}
		else
		{
			FString PropertyStr;
			PropertyStr.Reserve(8192);
			Prop->ExportTextItem_Direct(PropertyStr, Bundles, Bundles, nullptr, PPF_None);
			Context.AddTag(UObject::FAssetRegistryTag(GAssetBundleDataName, MoveTemp(PropertyStr), UObject::FAssetRegistryTag::ETagType::TT_Alphabetical));
		}
	}
	else if (Prop->HasAnyPropertyFlags(CPF_AssetRegistrySearchable))
	{
		UObject::FAssetRegistryTag::ETagType TagType = UObject::FAssetRegistryTag::ETagType::TT_Alphabetical;

		if (Prop->IsA(FIntProperty::StaticClass()) ||
			Prop->IsA(FFloatProperty::StaticClass()) ||
			Prop->IsA(FDoubleProperty::StaticClass()))
		{
			// ints and floats are always numerical
			TagType = UObject::FAssetRegistryTag::ETagType::TT_Numerical;
		}
		else if (Prop->IsA(FByteProperty::StaticClass()))
		{
			// bytes are numerical, enums are alphabetical
			FByteProperty* ByteProp = static_cast<FByteProperty*>(Prop);
			if (ByteProp->Enum)
			{
				TagType = UObject::FAssetRegistryTag::ETagType::TT_Alphabetical;
			}
			else
			{
				TagType = UObject::FAssetRegistryTag::ETagType::TT_Numerical;
			}
		}
		else if (Prop->IsA(FEnumProperty::StaticClass()))
		{
			// enums are alphabetical
			TagType = UObject::FAssetRegistryTag::ETagType::TT_Alphabetical;
		}
		else if (Prop->IsA(FArrayProperty::StaticClass()) || Prop->IsA(FMapProperty::StaticClass()) || Prop->IsA(FSetProperty::StaticClass())
			|| Prop->IsA(FStructProperty::StaticClass()))
		{
			// Arrays/maps/sets/structs are hidden, it is often too much information to display and sort
			TagType = UObject::FAssetRegistryTag::ETagType::TT_Hidden;
		}

		FString PropertyStr;
		if (const FTextProperty* TextProp = CastField<FTextProperty>(Prop))
		{
			const FText& TextValue = TextProp->GetPropertyValue_InContainer(BaseMemoryLocation);
			FTextStringHelper::WriteToBuffer(PropertyStr, TextValue, /*bRequireQuotes*/false, /*bStripPackageNamespace*/true);
		}
		else
		{
			PropertyStr.Reserve(8192);
			const uint8* PropertyAddr = Prop->ContainerPtrToValuePtr<uint8>(BaseMemoryLocation);
			Prop->ExportTextItem_Direct(PropertyStr, PropertyAddr, PropertyAddr, nullptr, PPF_None);
			PropertyStr.Shrink();
		}

		Context.AddTag(UObject::FAssetRegistryTag(Prop->GetFName(), MoveTemp(PropertyStr), TagType));
	}
}

static void GetAssetRegistryTagsFromSearchableProperties(const UObject* Object, FAssetRegistryTagsContext Context)
{
	check(nullptr != Object);

	for (TFieldIterator<FProperty> FieldIt( Object->GetClass() ); FieldIt; ++FieldIt)
	{
		GetAssetRegistryTagFromProperty(Object, Object, CastField<FProperty>(*FieldIt), Context);
	}

	UScriptStruct* SparseClassDataStruct = Object->GetClass()->GetSparseClassDataStruct();
	if (SparseClassDataStruct)
	{
		const void* SparseClassData = Object->GetClass()->GetSparseClassData(EGetSparseClassDataMethod::ArchetypeIfNull);
		for (TFieldIterator<FProperty> FieldIt(SparseClassDataStruct); FieldIt; ++FieldIt)
		{
			GetAssetRegistryTagFromProperty(SparseClassData, Object, CastField<FProperty>(*FieldIt), Context);
		}
	}
}

} // end namespace UE::Object::Private

const FName FPrimaryAssetId::PrimaryAssetTypeTag(TEXT("PrimaryAssetType"));
const FName FPrimaryAssetId::PrimaryAssetNameTag(TEXT("PrimaryAssetName"));
const FName FPrimaryAssetId::PrimaryAssetDisplayNameTag(TEXT("PrimaryAssetDisplayName"));

// This list of the objects that are forwarding from one GetAssetRegistry tags to the other has to be a list rather
// than merely a bool because some GetAssetRegistryTags calls call GetAssetRegistryTags on other objects, e.g. UWorld
// calls it on its LevelBlueprint.
// UE_DEPRECATED(5.4, "Used to provide backwards compatibility for the deprecated GetAssetRegistryTags function") // UE_DEPRECATED seems not to work with thread_local
thread_local TArray<const UObject*, TInlineAllocator<2>> GAssetRegistryTagsObjectsBeingForwarded;
// UE_DEPRECATED(5.4, "Used to provide backwards compatibility for the deprecated GetAssetRegistryTags function") // UE_DEPRECATED seems not to work with thread_local
thread_local FAssetBundleData const** TLegacyGetAssetRegistryTags_OutBundles = nullptr;


void UObject::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	if (!GAssetRegistryTagsObjectsBeingForwarded.Contains(this))
	{
		FAssetRegistryTagsContextData Context(this, EAssetRegistryTagsCaller::Uncategorized);
		Context.bWantsBundleResult = TLegacyGetAssetRegistryTags_OutBundles != nullptr;

		GAssetRegistryTagsObjectsBeingForwarded.Add(this);
		GetAssetRegistryTags(Context);
		check(!GAssetRegistryTagsObjectsBeingForwarded.IsEmpty() && GAssetRegistryTagsObjectsBeingForwarded.Last() == this);
		GAssetRegistryTagsObjectsBeingForwarded.Pop(EAllowShrinking::No);

		OutTags.Reserve(OutTags.Num() + Context.Tags.Num());
		for (TPair<FName, FAssetRegistryTag>& Pair : Context.Tags)
		{
			OutTags.Add(MoveTemp(Pair.Value));
		}
		if (TLegacyGetAssetRegistryTags_OutBundles)
		{
			checkf(*TLegacyGetAssetRegistryTags_OutBundles == nullptr, TEXT("Object %s has more than one FAssetBundleData!"), *GetPathName());
			*TLegacyGetAssetRegistryTags_OutBundles = Context.BundleResult;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UObject::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	using namespace UE::Object::Private;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	auto AddLegacyTags = [&Context](auto&& Callback)
	{
		TArray<FAssetRegistryTag> LegacyTags;
		Callback(LegacyTags);
		for (FAssetRegistryTag& Tag : LegacyTags)
		{
			Context.AddTag(MoveTemp(Tag));
		}
	};

	// Forward this call to the legacy version for classes that have not converted yet.
	if (!GAssetRegistryTagsObjectsBeingForwarded.Contains(this))
	{
		const FAssetBundleData* BundleResult = nullptr;
		TOptional<TGuardValue<const FAssetBundleData**>> WantsBundleScope;
		if (Context.WantsBundleResult())
		{
			WantsBundleScope.Emplace(TLegacyGetAssetRegistryTags_OutBundles, &BundleResult);
		}
		GAssetRegistryTagsObjectsBeingForwarded.Add(this);
		AddLegacyTags([this](TArray<FAssetRegistryTag>& Tags) { GetAssetRegistryTags(Tags); });
		check(!GAssetRegistryTagsObjectsBeingForwarded.IsEmpty() && GAssetRegistryTagsObjectsBeingForwarded.Last() == this);
		GAssetRegistryTagsObjectsBeingForwarded.Pop(EAllowShrinking::No);
		if (Context.WantsBundleResult() && BundleResult)
		{
			Context.SetBundleResult(BundleResult);
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;

	UE::Core::Private::FPlayInEditorLoadingScope Scope(INDEX_NONE);

	// Add primary asset info if valid
	FPrimaryAssetId PrimaryAssetId = GetPrimaryAssetId();
	if (PrimaryAssetId.IsValid())
	{
		Context.AddTag(FAssetRegistryTag(FPrimaryAssetId::PrimaryAssetTypeTag, PrimaryAssetId.PrimaryAssetType.ToString(),
			UObject::FAssetRegistryTag::TT_Alphabetical));
		Context.AddTag(FAssetRegistryTag(FPrimaryAssetId::PrimaryAssetNameTag, PrimaryAssetId.PrimaryAssetName.ToString(),
			UObject::FAssetRegistryTag::TT_Alphabetical));
	}

	GetAssetRegistryTagsFromSearchableProperties(this, Context);

#if WITH_EDITOR
	// Notify external sources that we need tags.
	FAssetRegistryTag::OnGetExtraObjectTagsWithContext.Broadcast(Context);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	AddLegacyTags([this](TArray<FAssetRegistryTag>& Tags)
		{ FAssetRegistryTag::OnGetExtraObjectTags.Broadcast(this, Tags); });
	if (Context.IsFullUpdate())
	{
		AddLegacyTags([this, &Context](TArray<FAssetRegistryTag>& Tags)
			{ GetExtendedAssetRegistryTagsForSave(Context.GetTargetPlatform(), Tags); });
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;

	// Check if there's a FMetaData for this object that has tags that are requested in the settings to be transferred to the Asset Registry
	const TSet<FName>& MetaDataTagsForAR = GetMetaDataTagsForAssetRegistry();
	if (MetaDataTagsForAR.Num() > 0)
	{
		TMap<FName, FString>* MetaDataMap = FMetaData::GetMapForObject(this);
		if (MetaDataMap)
		{
			for (TMap<FName, FString>::TConstIterator It(*MetaDataMap); It; ++It)
			{
				FName Tag = It->Key;
				if (!Tag.IsNone() && MetaDataTagsForAR.Contains(Tag))
				{
					Context.AddTag(FAssetRegistryTag(Tag, It->Value, UObject::FAssetRegistryTag::TT_Alphabetical));
				}
			}
		}
	}
#endif // WITH_EDITOR
}

#if WITH_EDITOR
void UObject::GetAdditionalAssetDataObjectsForCook(FArchiveCookContext& CookContext,
	TArray<UObject*>& OutObjects) const
{
}

void UObject::GetExtendedAssetRegistryTagsForSave(const ITargetPlatform* TargetPlatform, TArray<FAssetRegistryTag>& OutTags) const
{
	// DEPRECATION Note: This function will not return the data from classes that have been converted to use the new
	// FAssetRegistryTagsContext API. We could make it do so, but it would require extra effort because this function is
	// supposed to return only the expensive tags, and not the common tags that are also returned when called with
	// EAssetRegistryTagsCaller::AssetRegistryLoad. Because this function was designed only to be called from SavePackage,
	// and we have removed SavePackage's dependence on it, we decided not to make that extra effort. Any licensee calling
	// this function should instead call GetAssetRegistryTags with EAssetRegistryTagsCaller::SavePackage.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	FAssetRegistryTag::OnGetExtendedAssetRegistryTagsForSave.Broadcast(this, TargetPlatform, OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}
#endif // WITH_EDITOR

static FAssetDataTagMapSharedView MakeSharedTagMap(TMap<FName, UObject::FAssetRegistryTag>&& Tags)
{
	FAssetDataTagMap Out;
	Out.Reserve(Tags.Num());
	for (TPair<FName,UObject::FAssetRegistryTag>& Pair : Tags)
	{
		UObject::FAssetRegistryTag& Tag = Pair.Value;
		// Don't add empty tags
		if (!Tag.Name.IsNone() && !Tag.Value.IsEmpty())
		{
			Out.Add(Tag.Name, MoveTemp(Tag.Value));
		}
	}

	return FAssetDataTagMapSharedView(MoveTemp(Out));
}

static TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe> MakeSharedBundles(const FAssetBundleData* Bundles)
{
	if (Bundles && Bundles->Bundles.Num())
	{
		return MakeShared<FAssetBundleData, ESPMode::ThreadSafe>(*Bundles);
	}

	return TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe>();
}

void UObject::GetAssetRegistryTags(FAssetData& Out) const
{
	FAssetRegistryTagsContextData Context(this, EAssetRegistryTagsCaller::Uncategorized);
	GetAssetRegistryTags(Context, Out);
}

void UObject::GetAssetRegistryTags(FAssetRegistryTagsContext Context, FAssetData& Out) const
{
	using namespace UE::Object::Private;

	const FAssetBundleData* Bundles = nullptr;
	FAssetRegistryTagsContextData& ContextData = Context.Data;
	ContextData.bWantsBundleResult = true;
	GetAssetRegistryTags(Context);
	Out.TagsAndValues = MakeSharedTagMap(MoveTemp(ContextData.Tags));
	Out.TaggedAssetBundles = MakeSharedBundles(ContextData.BundleResult);
}

const FName& UObject::SourceFileTagName()
{
	static const FName SourceFilePathName("AssetImportData");
	return SourceFilePathName;
}

#if WITH_EDITOR

static void PostLoadAssetRegistryTagProperty(FProperty* Prop, UObject::FPostLoadAssetRegistryTagsContext& Context)
{
	// This TagType is ignored by the asset registry
	UObject::FAssetRegistryTag::ETagType TagType = UObject::FAssetRegistryTag::ETagType::TT_Alphabetical;

	if (Prop->HasAnyPropertyFlags(CPF_AssetRegistrySearchable))
	{
		if (FSoftObjectProperty* SoftObjectProp  = CastField<FSoftObjectProperty>(Prop))
		{
			// Old files may contain legacy format of FSofObjectPtr::ToString() which used to return 
			// an export path (ClassName'/Package/Name.ObjectName') however it now returns just a pathname (/Package/Name.ObjectName)
			FString ExportPath = Context.GetAssetData().GetTagValueRef<FString>(Prop->GetFName());
			int32 ClassSeparatorIndex = -1;
			if (!ExportPath.IsEmpty() && ExportPath[0] != '/' && ExportPath.FindChar('\'', ClassSeparatorIndex))
			{
				// Strip the class name and leave just the pathname of an object
				FString ObjectPath = FPackageName::ExportTextPathToObjectPath(ExportPath);
				Context.AddTagToUpdate(UObject::FAssetRegistryTag(Prop->GetFName(), ObjectPath, TagType));
			}
		}
		else if (Prop->IsA<FObjectPropertyBase>())
		{
			// Update the export path for short class names, but leave None alone to match save behavior
			FObjectPropertyBase* PropertyObject = CastFieldChecked<FObjectPropertyBase>(Prop);
			FString ExportPath = Context.GetAssetData().GetTagValueRef<FString>(Prop->GetFName());
			if (!ExportPath.IsEmpty() && ExportPath[0] != '/' && ExportPath != TEXT("None"))
			{
				FString ObjectPath = FPackageName::ExportTextPathToObjectPath(ExportPath);
				ExportPath = FObjectPropertyBase::GetExportPath(PropertyObject->PropertyClass->GetClassPathName(), ObjectPath);
				Context.AddTagToUpdate(UObject::FAssetRegistryTag(Prop->GetFName(), ExportPath, TagType));
			}
		}
	}
}

void UObject::ThreadedPostLoadAssetRegistryTags(FPostLoadAssetRegistryTagsContext& Context) const
{
	ensureMsgf(GetClass()->HasAnyClassFlags(CLASS_Native), TEXT("ThreadedPostLoadAssetRegistryTags should not be called on non-native types. Detected a call on type '%s'"),
		*GetClass()->GetName());

	// Before executing any postloads, check whether we will be able to complete.
	bool bHasAssetRegistrySearchableProperties = GetClass()->HasAssetRegistrySearchableProperties();
	UScriptStruct* SparseClassDataStruct = nullptr;
	const void* SparseClassData = nullptr;
	if (bHasAssetRegistrySearchableProperties)
	{
		SparseClassDataStruct = GetClass()->GetSparseClassDataStruct();
		if (SparseClassDataStruct)
		{
			SparseClassData = GetClass()->GetSparseClassData(
				Context.CanCreateObjects() ? EGetSparseClassDataMethod::ArchetypeIfNull
				: EGetSparseClassDataMethod::DeferIfNull);
			if (!SparseClassData)
			{
				Context.SetPostLoadCouldComplete(false);
				return;
			}
		}
	}

	ThreadedPostLoadAssetRegistryTagsOverride(Context);

	if (bHasAssetRegistrySearchableProperties)
	{
		for (TFieldIterator<FProperty> PropertyIt(GetClass()); PropertyIt; ++PropertyIt)
		{
			PostLoadAssetRegistryTagProperty(*PropertyIt, Context);
		}

		if (SparseClassDataStruct)
		{
			for (TFieldIterator<FProperty> PropertyIt(SparseClassDataStruct); PropertyIt; ++PropertyIt)
			{
				PostLoadAssetRegistryTagProperty(*PropertyIt, Context);
			}
		}
	}
}

static TSet<FName> MetaDataTagsForAssetRegistry;

TSet<FName>& UObject::GetMetaDataTagsForAssetRegistry()
{
	return MetaDataTagsForAssetRegistry;
}

void UObject::GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const
{
	OutMetadata.Add(FPrimaryAssetId::PrimaryAssetTypeTag,
		FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UObject", "PrimaryAssetType", "Primary Asset Type"))
		.SetTooltip(NSLOCTEXT("UObject", "PrimaryAssetTypeTooltip", "Type registered with the Asset Manager system"))
	);

	OutMetadata.Add(FPrimaryAssetId::PrimaryAssetNameTag,
		FAssetRegistryTagMetadata()
		.SetDisplayName(NSLOCTEXT("UObject", "PrimaryAssetName", "Primary Asset Name"))
		.SetTooltip(NSLOCTEXT("UObject", "PrimaryAssetNameTooltip", "Logical name registered with the Asset Manager system"))
	);
}
#endif // WITH_EDITOR

void UObject::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	if (CumulativeResourceSize.GetResourceSizeMode() == EResourceSizeMode::EstimatedTotal)
	{
		// Include this object's serialize size, and recursively call on direct subobjects
		FArchiveCountMem MemoryCount(this, true);
		CumulativeResourceSize.AddDedicatedSystemMemoryBytes(MemoryCount.GetMax());

		TArray<UObject*> SubObjects;
		GetObjectsWithOuter(this, SubObjects, EGetObjectsFlags::None);

		for (UObject* SubObject : SubObjects)
		{
#if WITH_EDITOR
			if (!SubObject->IsEditorOnly() && (SubObject->NeedsLoadForClient() || SubObject->NeedsLoadForServer()))
#endif // WITH_EDITOR
			{
				SubObject->GetResourceSizeEx(CumulativeResourceSize);
			}
		}
	}
}

bool UObject::IsAsset() const
{
	// Assets are not transient or CDOs. They must be public.
	const bool bHasValidObjectFlags = !HasAnyFlags(RF_Transient | RF_ClassDefaultObject) && HasAnyFlags(RF_Public) && IsValidChecked(this);

	if ( bHasValidObjectFlags && !GetClass()->HasAnyClassFlags(CLASS_Optional) )
	{
#if WITH_EDITOR
		// Don't count placeholder-typed objects (e.g. exports that were missing their type import on load).
		// This allows exports to be serialized to avoid data loss, but should not be registered as an asset.
		// Note: Currently, this requires that placeholders are always parented to UObject. If placeholders are
		// extended to other subtypes in the future, it may be necessary to also modify their IsAsset() overrides.
		if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObject(this))
		{
			return false;
		}
#endif
		// Don't count objects embedded in other objects (e.g. font textures, sequences, material expressions)
		UPackage* ObjectPackage = Cast<UPackage>(GetOuter());

		if ( !ObjectPackage )
		{
			// Check is the object is directly stored in an external package, in such case treat it as an asset also
			ObjectPackage = GetExternalPackage();
		}

		if ( ObjectPackage )
		{
			// Also exclude any objects found in the transient package, in a package that is transient or in a play in editor package.
			return ObjectPackage != GetTransientPackage() && !ObjectPackage->HasAnyFlags(RF_Transient) && !ObjectPackage->HasAnyPackageFlags(PKG_PlayInEditor);
		}
	}

	return false;
}

FPrimaryAssetId UObject::GetPrimaryAssetId() const
{
	// Check if we are an asset or a blueprint CDO
	if (FCoreUObjectDelegates::GetPrimaryAssetIdForObject.IsBound() &&
		(IsAsset() || (HasAnyFlags(RF_ClassDefaultObject) && !GetClass()->HasAnyClassFlags(CLASS_Native)))
		)
	{
		// Call global callback if bound
		return FCoreUObjectDelegates::GetPrimaryAssetIdForObject.Execute(this);
	}

	return FPrimaryAssetId();
}

#if !UE_STRIP_DEPRECATED_PROPERTIES 
bool UObject::IsLocalizedResource() const
{
	const UPackage* ObjPackage = GetOutermost();
	return ObjPackage && FPackageName::IsLocalizedPackage(ObjPackage->GetPathName());
}
#endif

bool UObject::IsSafeForRootSet() const
{
	if (IsInBlueprint())
	{
		return false;
	}

	// Exclude linkers from root set if we're using seekfree loading		
	if (IsValidChecked(this))
	{
		return true;
	}
	return false;
}

void UObject::TagSubobjects(EObjectFlags NewFlags) 
{
	// Collect a list of all things this element owns
	TArray<UObject*> MemberReferences;
	FReferenceFinder ComponentCollector(MemberReferences, this, false, true, true, true);
	ComponentCollector.FindReferences(this);

	for( TArray<UObject*>::TIterator it(MemberReferences); it; ++it )
	{
		UObject* CurrentObject = *it;
		if( CurrentObject && !CurrentObject->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS) && !CurrentObject->IsRooted())
		{
			CurrentObject->SetFlags(NewFlags);
			CurrentObject->TagSubobjects(NewFlags);
		}
	}
}

void UObject::ReloadConfig( UClass* ConfigClass/*=NULL*/, const TCHAR* InFilename/*=NULL*/, uint32 PropagationFlags/*=LCPF_None*/, FProperty* PropertyToLoad/*=NULL*/ )
{
	if (!GIsEditor)
	{
		LoadConfig(ConfigClass, InFilename, PropagationFlags | UE::LCPF_ReloadingConfigData | UE::LCPF_ReadParentSections, PropertyToLoad);
	}
#if WITH_EDITOR
	else
	{
		// When in the editor, raise change events so that the UI will update correctly when object configs are reloaded.
		PreEditChange(NULL);
		LoadConfig(ConfigClass, InFilename, PropagationFlags | UE::LCPF_ReloadingConfigData | UE::LCPF_ReadParentSections, PropertyToLoad);
		PostEditChange();
	}
#endif // WITH_EDITOR
}

/** Checks if a section specified as a long package name can be found as short name in ini. */
#if !UE_BUILD_SHIPPING
void CheckMissingSection(const FString& SectionName, const FString& IniFilename)
{
	// Apply lock striping to reduce contention.
	constexpr int32 MISSINGSECTIONS_BUCKETS = 31; /* prime number for best distribution using modulo */

	struct FMissingSections
	{
		FRWLock Lock;
		TSet<FString> Sections;
	};
	static FMissingSections MissingSections[MISSINGSECTIONS_BUCKETS];

	const FConfigSection* Sec = GConfig->GetSection(*SectionName, false, *IniFilename);

	if (Sec == nullptr)
	{
		const uint32 SectionNameHash = GetTypeHash(SectionName);
		FMissingSections& Bucket = MissingSections[SectionNameHash % MISSINGSECTIONS_BUCKETS];

		{
			FReadScopeLock ScopeLock(Bucket.Lock);
			if (Bucket.Sections.ContainsByHash(SectionNameHash, SectionName))
			{
				return;
			}
		}

		FWriteScopeLock ScopeLock(Bucket.Lock);

		if (Bucket.Sections.ContainsByHash(SectionNameHash, SectionName) == false)
		{
			FString ShortSectionName = FPackageName::GetShortName(SectionName);
			if (ShortSectionName != SectionName)
			{
				Sec = GConfig->GetSection(*ShortSectionName, false, *IniFilename);
				if (Sec != nullptr)
				{
					UE_LOGF(LogObj, Fatal, "Short class section names (%ls) are not supported, please use long name: %ls", *ShortSectionName, *SectionName);
				}
			}
			Bucket.Sections.AddByHash(SectionNameHash, SectionName);
		}
	}
}
#endif

#if WITH_EDITOR
static FName GConsoleVariableFName("ConsoleVariable");
#endif



void UObject::LoadConfig(UClass* ConfigClass/*=nullptr*/, const TCHAR* InFilename/*=nullptr*/,
	uint32 PropagationFlags/*=LCPF_None*/, FProperty* PropertyToLoad/*=nullptr*/,
	TArray<UE::ConfigAccessTracking::FConfigAccessData>* OutAccessedValues/*=nullptr*/)
{
	FLoadConfigParams Context;
	Context.ConfigClass = ConfigClass;
	Context.Filename = InFilename;
	Context.PropagationFlags = PropagationFlags;
	Context.PropertyToLoad = PropertyToLoad;
	Context.AccessedValues = OutAccessedValues;

	LoadConfig(Context);
}

class FPropertyLoadConfigContext
{
public:
	FPropertyLoadConfigContext(const UObject::FLoadConfigParams& InLoadParams, UObject* InObject)
		: LoadParams(InLoadParams)
		, ConfigSystem(LoadParams.ConfigSystem ? LoadParams.ConfigSystem : GConfig)
		, ActiveObject(InObject)
		, bTrackingAccessOnly(LoadParams.AccessedValues != nullptr) // if we are tracking, we don't acutally LoadConfig onto properties
	{
	}
	
	bool InitContext()
	{
		// all must pass for the Init to succeed, and for LoadConfig to continue
		if (PrepareObject() && Propagate() && PrepareConfig())
		{
			if (LoadParams.PropertyToLoad == nullptr)
			{
				UE_LOGF(LogConfig, VeryVerbose, "(%ls) '%ls' loading configuration from %ls",
					*ConfigClass->GetName(), *ActiveObject->GetName(), *Filename);
			}
			else
			{
				UE_LOGF(LogConfig, VeryVerbose, "(%ls) '%ls' loading configuration for property %ls from %ls",
					*ConfigClass->GetName(), *ActiveObject->GetName(), *LoadParams.PropertyToLoad->GetName(), *Filename);
			}

			return true;
		}
		
		return false;
	}
	
	bool InitForProperty(const FProperty* Property)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Config))
		{
			return false;
		}

		// if we're only supposed to load the value for a specific property, skip all others
		if (LoadParams.PropertyToLoad != nullptr && LoadParams.PropertyToLoad != Property)
		{
			return false;
		}
		
		const bool bGlobalConfig = EnumHasAnyFlags(Property->PropertyFlags, CPF_GlobalConfig);
		FString ModifiedConfigFilename;
		
		// if the property is GlobalConfig, then the class to load the property from is the class that owns the property
		// so we may need to look in a different section
		if (bGlobalConfig && Property->GetOwnerClass() != ConfigClass)
		{
			const UClass* OwnerClass = Property->GetOwnerClass();
			
			if (!bPerObject)
			{
				ClassSection = OwnerClass->GetPathName();
			}
			// if we forced a filename, then we will use the ConfigFile already found for the class
			if (LoadParams.Filename.IsEmpty())
			{
				// look up in the owning class' branch, which in rare cases is different than this class's branch
				// however, to see if it's different is a slowish function call and a string compare, so just always look up
				ModifiedConfigFilename = OwnerClass->GetConfigName();
				ConfigFile = ConfigSystem->Find(ModifiedConfigFilename);				
			}
			else
			{
				ConfigFile = ClassConfigFile;
			}
//			
		}
		else
		{
			// @todo precalc this in the PrepConfig above?
			if (!bPerObject)
			{
				ClassSection = ConfigClass->GetPathName();
			}
			ConfigFile = ClassConfigFile;
		}
		
		if (!bPerObject)
		{
			// allow the class to override the expected section name
			ActiveObject->OverrideConfigSection(ClassSection);
		}

		Key = Property->GetName();
		PortFlags = EPropertyPortFlags::PPF_SerializedAsImportText;

#if WITH_EDITOR
		const FString& CVarName = Property->GetMetaData(GConsoleVariableFName);
		if (!CVarName.IsEmpty())
		{
			Key = CVarName;
			PortFlags |= PPF_ConsoleVariable;
		}

		if (bTrackingAccessOnly)
		{
			const FString& PropFileName = ModifiedConfigFilename.IsEmpty() ? Filename : ModifiedConfigFilename;
			LoadParams.AccessedValues->Add(UE::ConfigAccessTracking::FConfigAccessData(
				UE::ConfigAccessTracking::ELoadType::ConfigSystem, ConfigPlatform, FName(*PropFileName),
				FName(*ClassSection), FName(*Key), nullptr /* InRequestingPlatform */));
			
			return false;
		}
#endif

		ConfigSection = ConfigFile->FindSection(ClassSection);
		// look in alternate section name for POC
		if (ConfigSection == nullptr && bPerObject && ClassPathSection.Len())
		{
			ConfigSection = ConfigFile->FindSection(ClassPathSection);
		}

#if !UE_BUILD_SHIPPING
		if (ConfigSection == nullptr && !FPlatformProperties::RequiresCookedData())
		{
			const FString& PropFileName = ModifiedConfigFilename.IsEmpty() ? Filename : ModifiedConfigFilename;
			CheckMissingSection(ClassSection, PropFileName);
		}
#endif

		UE_LOGF(LogConfig, VeryVerbose, "   Loading value for %ls from [%ls]", *Key, *ClassSection);

		// if we didn't find a section, no need to process this property - 
		// @todo: can we actually return false? some code may want to empty the property then skip if no Section was found
//		return ConfigSection == nullptr;
		return true;
	}

	bool ProcessArrayProperty(const FArrayProperty* ArrayProperty)
	{
		FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, ActiveObject);

#if WITH_EDITOR
		// Empty out any array properties if this is a PerPlatformConfig class
		// as we are replacing the values with the Platform's version when entering
		// a new preview platform.
		if (bIsPerPlatformConfig)
		{
			ArrayHelper.EmptyValues();
		}
#endif
		if (ConfigSection)
		{
			TArray<FConfigValue> List;
			const FName KeyName(*Key, FNAME_Find);
			ConfigSection->MultiFind(KeyName, List);

			const int32 Size = ArrayProperty->Inner->GetElementSize();

			// Only override default properties if there is something to override them with, or
			// if the config system had a "initialize to empty" entry
			if (!List.IsEmpty() || ConfigSection->EmptyInitializedKeys.Contains(KeyName))
			{
				ArrayHelper.EmptyAndAddValues(List.Num());
				for (int32 i = List.Num() - 1, c = 0; i >= 0; i--, c++)
				{
					ArrayProperty->Inner->ImportText_Direct(*List[i].GetValue(), ArrayHelper.GetRawPtr(c), ActiveObject, PortFlags);
				}
			}
			else
			{
				int32 Index = 0;
				const FConfigValue* ElementValue = nullptr;
				do
				{
					// Add array index number to end of key
					FString IndexedKey = FString::Printf(TEXT("%s[%i]"), *Key, Index);

					// Try to find value of key
					const FName IndexedName(*IndexedKey, FNAME_Find);
					if (IndexedName == NAME_None)
					{
						break;
					}
					ElementValue = ConfigSection->Find(IndexedName);

					// If found, import the element
					if (ElementValue != nullptr)
					{
						// expand the array if necessary so that Index is a valid element
						ArrayHelper.ExpandForIndex(Index);
						ArrayProperty->Inner->ImportText_Direct(*ElementValue->GetValue(), ArrayHelper.GetRawPtr(Index), ActiveObject, PortFlags);
					}

					Index++;
				} while (ElementValue || Index < ArrayHelper.Num());
			}
		}
		
		return true;
	}
	
	bool ProcessSetProperty(const FSetProperty* SetProperty)
	{
		bool bProcessedProperty = true;

		FScriptSetHelper_InContainer SetHelper(SetProperty, ActiveObject);

#if WITH_EDITOR
		// Empty out any set properties if this is a PerPlatformConfig class
		// as we are replacing the values with the Platform's version when entering
		// a new preview platform.
		if (bIsPerPlatformConfig)
		{
			SetHelper.EmptyElements();
		}
#endif

		if (ConfigSection)
		{
			TArray<FConfigValue> List;
			const FName KeyName(*Key, FNAME_Find);
			ConfigSection->MultiFind(KeyName, List);

			bool bSingleSetEntry = false;
			if (List.Num() == 1)
			{
				const FString& SingleListValue = List[0].GetValue();
				if (SingleListValue.Len() > 1
					&& SingleListValue[0] == TEXT('(')
					&& SingleListValue[SingleListValue.Len() - 1] == TEXT(')'))
				{
					// If we have a single entry in the set that is surrounded with parentheses, fall back to
					// the old processing method
					bSingleSetEntry = true;
				}
			}

			// Only override default properties if there is something to override them with.
			if (!bSingleSetEntry && !List.IsEmpty())
			{
				bProcessedProperty = true;
				SetHelper.EmptyElements(List.Num());

				// Each config value entry can possibly specify multiple set elements - create a temporary
				// set here that we can import to and add its elements to the object's property
				void* TempSet = FMemory::Malloc(SetProperty->GetSize(), SetProperty->GetMinAlignment());
				SetProperty->InitializeValue(TempSet);

				// Importing elements to the set inline does not check for duplicates - create a temporary
				// element that we can import to add uniquely to the set
				void* TempElement = FMemory::Malloc(SetProperty->ElementProp->GetSize(),
					SetProperty->ElementProp->GetMinAlignment());
				SetProperty->ElementProp->InitializeValue(TempElement);

				for (const FConfigValue& ListValue : List)
				{
					const FString& ListString = ListValue.GetValue();

					// Try to import the config value as an entire set first
					const TCHAR* SetImportResult =
						SetProperty->ImportText_Direct(*ListString, TempSet, ActiveObject, PortFlags);

					if (SetImportResult && SetImportResult != *ListString)
					{
						FScriptSetHelper TempSetHelper(SetProperty, TempSet);
						for (FScriptSetHelper::FIterator Itr = TempSetHelper.CreateIterator(); Itr; ++Itr)
						{
							SetHelper.AddElement(TempSetHelper.GetElementPtr(Itr));
						}
					}
					else
					{
						// If we failed to import the value as an entire set, try to import it as a single
						// element
						const TCHAR* ElementImportResult =
						SetProperty->ElementProp->ImportText_Direct(*ListString, TempElement, ActiveObject, PortFlags);

						if (ElementImportResult && ElementImportResult != *ListString)
						{
							SetHelper.AddElement(TempElement);
						}
					}

				}

				SetProperty->DestroyAndFreeValue(TempSet);
				SetProperty->ElementProp->DestroyAndFreeValue(TempElement);
				SetHelper.Rehash();
				
				return true;
			}
		}
		
		// if we get here, we didn't process the property as a Set, so return false, and we will try again 
		// with ProcessStandardProperty
		return false;
	}
	
	void ProcessStandardProperty(FProperty* Property)
	{
		for (int32 i = 0; i < Property->ArrayDim; i++)
		{
			// @todo can we optimize this somehow?
			if (Property->ArrayDim != 1)
			{
				Key = FString::Printf(TEXT("%s[%i]"), *Property->GetName(), i);
			}

			if (ConfigSection)
			{
				FString Value;
				if (ConfigSection->GetString(*Key, Value))
				{
					if (Property->ImportText_Direct(*Value, Property->ContainerPtrToValuePtr<uint8>(ActiveObject, i), ActiveObject, PortFlags) == nullptr)
					{
						// this should be an error as the properties from the .ini / .int file are not correctly being read in and probably are affecting things in subtle ways
						UE_LOGF(LogObj, Error, "LoadConfig (%ls): import failed for %ls in: %ls", *ActiveObject->GetPathName(), *Property->GetName(), *Value);
					}
				}
			}
		}		
	}
private:	
	
	bool PrepareObject()
	{
		ConfigClass = LoadParams.ConfigClass ? LoadParams.ConfigClass : ActiveObject->GetClass();
		if (!ConfigClass->HasAnyClassFlags(CLASS_Config))
		{
			return false;
		}

		bPerObject = UsesPerObjectConfig(ActiveObject);
		bIsPerPlatformConfig = ActiveObject->GetClass()->HasAnyClassFlags(CLASS_PerPlatformConfig);

		// handle some special cases and validations
		if (bPerObject)
		{
			HandlePerObject();
		}
		CheckEarlyAccess();
		
		// we only support tracking values in with the editor
#if !WITH_EDITOR
		if (bTrackingAccessOnly)
		{
			return false;
		}
#endif
		
		return true;
	}
	
	bool PrepareConfig()
	{
		if (LoadParams.OverrideFile != nullptr)
		{
			checkf(LoadParams.Filename.IsEmpty(), TEXT("Cannot have FLoadConfigParams::OverrideFile and FLoadConfigParams::Filename both set, it indicates a logic issue."));
			ClassConfigFile = LoadParams.OverrideFile;
			return true;
		}
		
		// if a filename was specified, load all properties from that file no matter which class they come from
		Filename = LoadParams.Filename.IsEmpty() ? GetConfigFilename(ActiveObject) : LoadParams.Filename;

		// does the class want to override the platform hierarchy (ignored if we passed in a specific ini file),
		// and if the name isn't the current running platform (no need to load extra files if already in GConfig)
		const TCHAR* ConfigOverridePlatform = ActiveObject->GetConfigOverridePlatform();

		if (LoadParams.Filename.IsEmpty() && ConfigOverridePlatform != nullptr &&
			FCString::Stricmp(ConfigOverridePlatform, ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName())) != 0)
		{
#if WITH_EDITOR
			ConfigPlatform = FName(ConfigOverridePlatform);
#endif
			if (!bTrackingAccessOnly)
			{
				ClassConfigFile = FConfigCacheIni::FindOrLoadPlatformConfig(PlatformOverrideConfig,
					*ActiveObject->GetClass()->ClassConfigName.ToString(), ConfigOverridePlatform);
//				bUseConfigOverride = true;
			}
		}
#if WITH_EDITOR
		else if (bIsPerPlatformConfig && UObject::OnGetPreviewPlatform.IsBound())
		{
			FName PreviewPlatform;
			if (UObject::OnGetPreviewPlatform.Execute(PreviewPlatform))
			{
				FString PreviewPlatformStr = PreviewPlatform.ToString();
				ConfigPlatform = FName(*PreviewPlatformStr);
				if (!bTrackingAccessOnly)
				{
					ClassConfigFile = FConfigCacheIni::FindOrLoadPlatformConfig(PlatformOverrideConfig,
						*ActiveObject->GetClass()->ClassConfigName.ToString(), *PreviewPlatformStr);
//					bUseConfigOverride = true;
				}
			}
		}
#endif
		
		// if we didn't have one specified already, we will use the ConfigSystem's branch for this class 
		if (ClassConfigFile == nullptr)
		{
			ClassConfigFile = ConfigSystem->Find(Filename);
		}
		
		// if we can't find the branch, nothing to do here (unless tracking, then let it continue tracking what would have been read)
		return ClassConfigFile != nullptr || bTrackingAccessOnly;
	}




	bool Propagate()
	{
		UClass* ParentClass = ConfigClass->GetSuperClass();
		if (ParentClass != nullptr)
		{
			if (ParentClass->HasAnyClassFlags(CLASS_Config))
			{
				if ((LoadParams.PropagationFlags & UE::LCPF_ReadParentSections) != 0)
				{
					// call LoadConfig on the parent class
					UObject::FLoadConfigParams ParentContext = LoadParams;
					ParentContext.ConfigClass = ParentClass;
					ParentContext.Filename = TEXT("");
					ActiveObject->LoadConfig(ParentContext);

					// if we are also notifying child classes or instances, stop here as this object's properties will be imported as a result of notifying the others
					if ((LoadParams.PropagationFlags & (UE::LCPF_PropagateToChildDefaultObjects | UE::LCPF_PropagateToInstances)) != 0)
					{
						return false;
					}
				}
				else if ((LoadParams.PropagationFlags & UE::LCPF_PropagateToChildDefaultObjects) != 0)
				{
					// not propagating the call upwards, but we are propagating the call to all child classes

					UObject::FLoadConfigParams ChildContext = LoadParams;
					// mask out the PropgateToParent and PropagateToChildren values
					ChildContext.PropagationFlags = (LoadParams.PropagationFlags & (UE::LCPF_PersistentFlags | UE::LCPF_PropagateToInstances));
					ChildContext.Filename = TEXT("");
					for (TObjectIterator<UClass> It; It; ++It)
					{
						if (It->IsChildOf(ConfigClass))
						{
							ChildContext.ConfigClass = *It;
							It->GetDefaultObject()->LoadConfig(ChildContext);
						}
					}

					// LoadConfig() was called on this object during iteration, so stop here 
					return false;
				}
				else if ((LoadParams.PropagationFlags & UE::LCPF_PropagateToInstances) != 0)
				{
					// call LoadConfig() on all instances of this class (except the CDO)
					// Do not propagate this call to parents, and do not propagate to children or instances (would be redundant) 

					UObject::FLoadConfigParams InstanceContext = LoadParams;
					// mask out the PropgateToParent and PropagateToChildren values
					InstanceContext.PropagationFlags = (LoadParams.PropagationFlags & (UE::LCPF_PersistentFlags));
					InstanceContext.Filename = TEXT("");
					for (TObjectIterator<UObject> It; It; ++It)
					{
						if (It->IsA(ConfigClass))
						{
							// null would also work
							InstanceContext.ConfigClass = It->GetClass();
							if (!GIsEditor || bTrackingAccessOnly)
							{
								It->LoadConfig(InstanceContext);
							}
	#if WITH_EDITOR
							else
							{
								It->PreEditChange(nullptr);
								It->LoadConfig(InstanceContext);
								It->PostEditChange();
							}
	#endif // WITH_EDITOR
						}
					}
				}
			}
			else if ((LoadParams.PropagationFlags & UE::LCPF_PropagateToChildDefaultObjects) != 0)
			{
				// we're at the base-most config class

				UObject::FLoadConfigParams ChildContext = LoadParams;
				// mask out the PropgateToParent and PropagateToChildren values
				ChildContext.PropagationFlags = (LoadParams.PropagationFlags & (UE::LCPF_PersistentFlags | UE::LCPF_PropagateToInstances));
				ChildContext.Filename = TEXT("");
				for (TObjectIterator<UClass> It; It; ++It)
				{
					if (It->IsChildOf(ConfigClass))
					{
						ChildContext.ConfigClass = *It;
						if (!GIsEditor || bTrackingAccessOnly)
						{
							It->GetDefaultObject()->LoadConfig(ChildContext);
						}
	#if WITH_EDITOR
						else
						{
							It->PreEditChange(nullptr);
							It->GetDefaultObject()->LoadConfig(ChildContext);
							It->PostEditChange();
						}
	#endif // WITH_EDITOR
					}
				}

				return false;
			}
			else if ((LoadParams.PropagationFlags & UE::LCPF_PropagateToInstances) != 0)
			{
				UObject::FLoadConfigParams InstanceContext = LoadParams;
				// mask out the PropgateToParent and PropagateToChildren values
				InstanceContext.PropagationFlags = (LoadParams.PropagationFlags & (UE::LCPF_PersistentFlags));
				InstanceContext.Filename = TEXT("");

				for (TObjectIterator<UObject> It; It; ++It)
				{
					if (It->GetClass() == ConfigClass)
					{
						InstanceContext.ConfigClass = It->GetClass();
						
						if (!GIsEditor || bTrackingAccessOnly)
						{
							It->LoadConfig(InstanceContext);
						}
	#if WITH_EDITOR
						else
						{
							It->PreEditChange(nullptr);
							It->LoadConfig(InstanceContext);
							It->PostEditChange();
						}
	#endif // WITH_EDITOR
					}
				}
			}
		}
		
		return true;
	}


	//------------------------------------------------------------------------------------
	
	void HandlePerObject()
	{
		FString PathNameString;
		UObject* Outermost = ActiveObject->GetOutermost();

		if (Outermost == GetTransientPackage())
		{
			PathNameString = ActiveObject->GetName();
		}
		else
		{
			ActiveObject->GetPathName(Outermost, PathNameString);
		}

		// this will be set per-property for non-per-object
		ClassSection = PathNameString + TEXT(" ") + ActiveObject->GetClass()->GetName();

		FString OverrideClassSection;
		ActiveObject->OverridePerObjectConfigSection(OverrideClassSection);
		if (OverrideClassSection.Len() && OverrideClassSection != ClassSection)
		{
			// If a we got a section name override no need to perform short class name checks
			ClassSection = MoveTemp(OverrideClassSection);
			// Keep ClassPathSection empty so that we don't check for it when the section name has been overridden
		}
		else
		{
			// Cache both version of per object config section name
			ClassPathSection = PathNameString + TEXT(" ") + ActiveObject->GetClass()->GetPathName();
		}
	}

	void CheckEarlyAccess()
	{
#if DO_CHECK
		// When a CDO is created before the Class is linked, it will have an empty set of properties. Among other issues, this can cause loading of config data to fail silently. 
		if (!ConfigClass->DebugIsPropertyChainReady())
		{
			FString ErrorMessage = FString::Printf(TEXT("Attempting to load config data for %s before the Class has been constructed/registered/linked (likely during module loading or early startup). This will result in the load silently failing and should be fixed."), *ActiveObject->GetName());
			if (GPrematureConfigLoadErrorLevel == 0)
			{
				UE_LOGF(LogObj, Display, "%ls", *ErrorMessage);
			}
			else if (GPrematureConfigLoadErrorLevel == 1)
			{
				UE_LOGF(LogObj, Warning, "%ls", *ErrorMessage);
			}
			else if (GPrematureConfigLoadErrorLevel == 2)
			{
				// Repeating the condition here just so it ends up in the output message
				ensureAlwaysMsgf(ConfigClass->DebugIsPropertyChainReady(), TEXT("%s"), *ErrorMessage);
			}
			else
			{
				checkf(ConfigClass->DebugIsPropertyChainReady(), TEXT("%s"), *ErrorMessage);
			}
		}
#endif

#if !IS_PROGRAM
		auto HaveSameProperties = [](const UStruct* Struct1, const UStruct* Struct2) -> bool
		{
			TFieldIterator<FProperty> It1(Struct1);
			TFieldIterator<FProperty> It2(Struct2);
			for (;; ++It1, ++It2)
			{
				bool bAtEnd1 = !It1;
				bool bAtEnd2 = !It2;
				// If one iterator is at the end and one isn't, the property lists are different
				if (bAtEnd1 != bAtEnd2)
				{
					return false;
				}
				// If both iterators have reached the end, the property lists are the same
				if (bAtEnd1)
				{
					return true;
				}
				// If the properties are different, the property lists are different
				if (*It1 != *It2)
				{
					return false;
				}
			}
		};
		// Do we have properties that don't exist yet?
		// If this happens then we're trying to load the config for an object that doesn't
		// know what its layout is. Usually a call to GetDefaultObject that occurs too early
		// because ProcessNewlyLoadedUObjects hasn't happened yet
		checkf(ConfigClass->PropertyLink != nullptr
			|| (ConfigClass->GetSuperStruct() && HaveSameProperties(ConfigClass, ConfigClass->GetSuperStruct()))
			|| (UE_WITH_CONSTINIT_UOBJECT ? !ConfigClass->HasAnyClassFlags(CLASS_Constructed) : ConfigClass->PropertiesSize == 0)
			|| IsEngineExitRequested(), // Ignore this check when exiting as we may have requested exit during init when not everything is initialized
			TEXT("class %s has uninitialized properties. Accessed too early?"), *ConfigClass->GetPathName());
#endif
	}

public:
	// the final class to walk over properties in LoadConfig
	const UClass* ConfigClass = nullptr;

private:

	///// settings for the entire class

	const UObject::FLoadConfigParams& LoadParams;
	FConfigCacheIni* ConfigSystem = nullptr;
	UObject* ActiveObject = nullptr;
	bool bPerObject = false;
	bool bIsPerPlatformConfig = false;
	// the default section for the class, but each property can potentially override it when setting ConfigSection
	FString ClassSection;
	FString ClassPathSection;
	// the filename to look up in ConfigSystem,
	FString Filename;
	// storage for a local file when loading another platform's config
	FConfigFile PlatformOverrideConfig;
	// the config file for the class, but it can change per property (rarely)
	const FConfigFile* ClassConfigFile = nullptr;

	///// per-property settings

	// flags to control the text import
	int32 PortFlags = 0;
	// the final file to load from
	const FConfigFile* ConfigFile = nullptr;
	// the final Section to read Key from
	const FConfigSection* ConfigSection = nullptr;
	// the key to use to look in ConfigSection
	FString Key;

	
	// the ConfigPlatform is only used for tracking
	bool bTrackingAccessOnly = false;
#if WITH_EDITOR
	FName ConfigPlatform;
#endif
};

void UObject::LoadConfig(const FLoadConfigParams& Params)
{
	SCOPE_CYCLE_COUNTER(STAT_LoadConfig);

	// make the local context that manages the complexity
	FPropertyLoadConfigContext PropertyContext(Params, this);
	
	// there are various reasons why we won't need to continue, so early out
	if (PropertyContext.InitContext() == false)
	{
		return;
	}
		
	for (FProperty* Property = PropertyContext.ConfigClass->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
#if WITH_EDITOR
		FSoftObjectPathSerializationScope SerializationScope(NAME_None, Property->GetFName(),
			Property->GetOwnerProperty()->HasMetaData(FSoftObjectPath::NAME_Untracked) ? ESoftObjectPathCollectType::NeverCollect :
			Property->IsEditorOnlyProperty() ? ESoftObjectPathCollectType::EditorOnlyCollect : ESoftObjectPathCollectType::AlwaysCollect,
			ESoftObjectPathSerializeType::AlwaysSerialize);
#endif

		if (!PropertyContext.InitForProperty(Property))
		{
			continue;
		}
		
		// Track if we loaded this config value using special handling (e.g. array or set)
		bool bProcessedProperty = false;

		FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
		FSetProperty* SetProperty = CastField<FSetProperty>(Property);

		if (ArrayProperty)
		{
			bProcessedProperty = PropertyContext.ProcessArrayProperty(ArrayProperty);
		}
		else if (SetProperty)
		{
			bProcessedProperty = PropertyContext.ProcessSetProperty(SetProperty);
		}
		
		if (!bProcessedProperty)
		{
			PropertyContext.ProcessStandardProperty(Property);			
		}
	}

	// if we are reloading config data after the initial class load, fire the callback now
	if (Params.AccessedValues == nullptr && (Params.PropagationFlags & UE::LCPF_ReloadingConfigData) != 0)
	{
		PostReloadConfig(Params.PropertyToLoad);
	}
}

void UObject::SaveConfig(uint64 RequiredPropertyFlags, const TCHAR* InFilename, FConfigCacheIni* Config/*=GConfig*/, bool bAllowCopyToDefaultObject/*=true*/)
{
	FSaveConfigContext Context;
	Context.RequiredPropertyFlags = (EPropertyFlags)RequiredPropertyFlags;
	Context.Filename = InFilename;
	Context.ConfigSystem = Config;
	Context.bAllowCopyToDefaultObject = bAllowCopyToDefaultObject;

	SaveConfig(Context);
}

void UObject::SaveConfig(const FSaveConfigContext& Context)
{
	if (!GetClass()->HasAnyClassFlags(CLASS_Config))
	{
		return;
	}

	// if a filename was specified, save all properties to that file no matter which class they come from
	const FString Filename = Context.Filename.Len() > 0 ? Context.Filename : GetConfigFilename(this);
	
	FConfigCacheIni* Config = Context.ConfigSystem;

	// If there is no existing branch for the filename and file operations are enabled, then add a new temporary branch and remove it after the flush.
	const bool bIsTemporaryBranch = Config->FindBranch(*Filename, Filename) == nullptr && !Config->AreFileOperationsDisabled();
	if (bIsTemporaryBranch)
	{
		Config->AddNewBranch(Filename);
	}

	// Determine whether the file we are writing to is a layer in the hierarchy, as opposed to the final Saved file
	const bool bIsLayerWrite = Filename == GetDefaultConfigFilename() || Filename == GetGlobalUserConfigFilename() || Context.bSaveToLayer;

	const bool bPerObject = UsesPerObjectConfig(this);
	FString Section = Context.OverrideConfigSectionName;

	if (bPerObject == true)
	{
		FString PathNameString;
		UObject* Outermost = GetOutermost();

		if (Outermost == GetTransientPackage())
		{
			PathNameString = GetName();
		}
		else
		{
			GetPathName(Outermost, PathNameString);
		}

		//RobM: we need to update this to use GetClass()->GetPathName() after we fix all places that format section names
		if (Context.OverrideConfigSectionName.IsEmpty())
		{
			Section = PathNameString + TEXT(" ") + GetClass()->GetName();

			OverridePerObjectConfigSection(Section);
		}
	}

	UObject* CDO = GetClass()->GetDefaultObject();

	// only copy the values to the CDO if this is GConfig and we're not saving the CDO
	const bool bCopyValues = (Context.bAllowCopyToDefaultObject && this != CDO && Context.ConfigSystem == GConfig);

	for (FProperty* Property = GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		// we just check names, in case this property is a duplicated but same property as this object's class
		if (!Context.PropertyNames.IsEmpty() && !Context.PropertyNames.Contains(Property->GetFName()))
		{
			continue;
		}

		if (!Property->HasAnyPropertyFlags(CPF_Config))
		{
			continue;
		}

		if ((Property->PropertyFlags & Context.RequiredPropertyFlags) == Context.RequiredPropertyFlags)
		{
			UClass* BaseClass = GetClass();

			if (EnumHasAnyFlags(Property->PropertyFlags, CPF_GlobalConfig))
			{
				// call LoadConfig() on child classes if any of the properties were global config
				BaseClass = Property->GetOwnerClass();
			}

			FString Key				= Property->GetName();
			int32 PortFlags			= EPropertyPortFlags::PPF_SerializedAsImportText;

#if WITH_EDITOR
			const FString& CVarName = Property->GetMetaData(GConsoleVariableFName);
			if (!CVarName.IsEmpty())
			{
				Key = CVarName;
				PortFlags |= PPF_ConsoleVariable;
			}
#endif // #if WITH_EDITOR

			if (!bPerObject && Context.OverrideConfigSectionName.IsEmpty())
			{
				Section = BaseClass->GetPathName();
				// allow the class to override the expected section name
				OverrideConfigSection(Section);
			}

			// globalconfig properties should always use the owning class's config file
			// specifying a value for InFilename will override this behavior (as it does with normal properties)
			const FString& PropFileName = ((Property->PropertyFlags & CPF_GlobalConfig) && Context.Filename.Len() == 0) ? Property->GetOwnerClass()->GetConfigName() : Filename;

			// Properties that are the same as the parent class' defaults should not be saved to ini
			// Before modifying any key in the section, first check to see if it is different from the parent.
			
#if WITH_METADATA
			static const FName NAME_DeprecatedProperty = "DeprecatedProperty";
			const bool bPropDeprecated = Property->HasAnyPropertyFlags(CPF_Deprecated) || Property->HasMetaData(NAME_DeprecatedProperty);
#else
			const bool bPropDeprecated = Property->HasAnyPropertyFlags(CPF_Deprecated);
#endif

			const bool bIsPropertyInherited = Property->GetOwnerClass() != GetClass();
			const bool bShouldCheckIfIdenticalBeforeAdding = !GetClass()->HasAnyClassFlags(CLASS_ConfigDoNotCheckDefaults) && !bPerObject && bIsPropertyInherited;
			UObject* SuperClassDefaultObject = GetClass()->GetSuperClass()->GetDefaultObject();

			bool bProcessedProperty = false;

			FArrayProperty* Array   = CastField<FArrayProperty>( Property );
			FSetProperty* SetProperty = CastField<FSetProperty>(Property);
			if (Array || SetProperty)
			{
				// write a property to a specific layer (with new path, for now the old way of writing to defaults is still below)
				if (Context.bSaveToLayer)
				{
					TArray<FString> ExportedValues;
					if (Array)
					{
						FScriptArrayHelper_InContainer ArrayHelper(Array, this);
						ExportedValues.AddDefaulted(ArrayHelper.Num());
						for (int32 i = 0; i < ArrayHelper.Num(); i++)
						{
							Array->Inner->ExportTextItem_Direct(ExportedValues[i], ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), this, PortFlags);
						}
						Config->SetArrayInSectionOfStaticLayer(*Section, *Key, ExportedValues, false, Property->GetOwnerClass()->GetConfigName(), Filename, true);
						bProcessedProperty = true;
					}
					else if (SetProperty)
					{
						FScriptSetHelper_InContainer SetHelper(SetProperty, this);
						// If we have one or fewer elements, fall back to the old export method so we match what we do
						// on import
						if (SetHelper.Num() > 1)
						{
							ExportedValues.AddDefaulted(SetHelper.Num());
							int Index = 0;
							for (FScriptSetHelper::FIterator It = SetHelper.CreateIterator(); It; ++It, Index++)
							{
								SetProperty->ElementProp->ExportTextItem_Direct(ExportedValues[Index], SetHelper.GetElementPtr(It), SetHelper.GetElementPtr(It), this, PortFlags);
							}
							Config->SetArrayInSectionOfStaticLayer(*Section, *Key, ExportedValues, false, Property->GetOwnerClass()->GetConfigName(), Filename, true);
							bProcessedProperty = true;
						}
					}
				}
				else
				{

					const FConfigSection* Sec = Config->GetSection(*Section, 1, PropFileName);
					// Layer ini's require the array syntax to be applied to the property name
					FString CompleteKey = FString::Printf(TEXT("%s%s"), bIsLayerWrite ? TEXT("+") : TEXT(""), *Key);
					if (Sec)
					{
						// Delete the old value for the property in the ConfigCache before (conditionally) adding in the new value
						Config->ResetKeyInSection(*Section, *CompleteKey, PropFileName);
					}

					if (!bPropDeprecated && (!bShouldCheckIfIdenticalBeforeAdding || !Property->Identical_InContainer(this, SuperClassDefaultObject)))
					{
						if (Array)
						{
							FScriptArrayHelper_InContainer ArrayHelper(Array, this);
							for (int32 i = 0; i < ArrayHelper.Num(); i++)
							{
								FString	Buffer;
								Array->Inner->ExportTextItem_Direct(Buffer, ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), this, PortFlags);
								Config->AddToSection(*Section, *CompleteKey, *Buffer, PropFileName);
							}
							if (ArrayHelper.Num() == 0 && bIsLayerWrite)
							{
								const FString EmptyKey = FString::Printf(TEXT("!%s"), *Key);
								Config->AddToSection(*Section, *EmptyKey, TEXT("__ClearArray__"), PropFileName);
							}
							bProcessedProperty = true;
						}
						else if (SetProperty)
						{
							FScriptSetHelper_InContainer SetHelper(SetProperty, this);

							// If we have one or fewer elements, fall back to the old export method so we match what we do
							// on import
							if (SetHelper.Num() > 1)
							{
								for (FScriptSetHelper::FIterator It = SetHelper.CreateIterator(); It; ++It)
								{
									FString	Buffer;
									SetProperty->ElementProp->ExportTextItem_Direct(Buffer, SetHelper.GetElementPtr(It), SetHelper.GetElementPtr(It), this, PortFlags);
									Config->AddToSection(*Section, *CompleteKey, *Buffer, PropFileName);
								}
								bProcessedProperty = true;
							}
						}
					}
				}
			}
			
			if (!bProcessedProperty)
			{
				TCHAR TempKey[MAX_SPRINTF] = {};
				for (int32 Index=0; Index<Property->ArrayDim; Index++)
				{
					if (Property->ArrayDim!=1)
					{
						FCString::Sprintf( TempKey, TEXT("%s[%i]"), *Property->GetName(), Index );
						Key = TempKey;
					}

					if (!bPropDeprecated && (!bShouldCheckIfIdenticalBeforeAdding || !Property->Identical_InContainer(this, SuperClassDefaultObject, Index)))
					{
						FString	Value;
						Property->ExportText_InContainer(Index, Value, this, this, this, PortFlags);
						if (Context.bSaveToLayer)
						{
							Config->SetInSectionOfStaticLayer(*Section, *Key, *Value, Property->GetOwnerClass()->GetConfigName(), Filename, true);
						}
						else
						{
							Config->SetString(*Section, *Key, *Value, PropFileName);
						}
					}
					else
					{
						if (Context.bSaveToLayer)
						{
							Config->ResetKeyInSectionOfStaticLayer(*Section, *Key, Property->GetOwnerClass()->GetConfigName(), Filename, true);
						}
						else
						{
							// If we are not writing it to config above, we should make sure that this property isn't stagnant in the cache.
							Config->ResetKeyInSection(*Section, *Key, PropFileName);
						}
					}
				}
			}

			if (bCopyValues)
			{
				void* ThisPropertyAddress = Property->ContainerPtrToValuePtr<void>( this );
				void* CDOPropertyAddr = Property->ContainerPtrToValuePtr<void>( CDO );

				Property->CopyCompleteValue(CDOPropertyAddr, ThisPropertyAddress);
			}
		}
	}

	// Only write out the config file if this is GConfig or if we are saving a temporary branch.
	if (Context.ConfigSystem == GConfig || bIsTemporaryBranch)
	{
		Config->Flush(bIsTemporaryBranch/*bRemoveFromCache*/, *Filename);
	}
}

static FString GetFinalOverridePlatform(const UObject* Obj)
{
	FString Platform;
	if (Obj->GetConfigOverridePlatform() != nullptr && FCString::Stricmp(Obj->GetConfigOverridePlatform(), ANSI_TO_TCHAR(FPlatformProperties::IniPlatformName())) != 0)
	{
		Platform = Obj->GetConfigOverridePlatform();
	}
	return Platform;
}

FString UObject::GetDefaultConfigFilename() const
{
	FString OverridePlatform = GetFinalOverridePlatform(this);
	if (OverridePlatform.Len())
	{
		bool bIsPlatformExtension = FPaths::DirectoryExists(FPaths::EnginePlatformExtensionDir(*OverridePlatform));
		FString RegularPath = FString::Printf(TEXT("%s%s"), *FPaths::SourceConfigDir(), *OverridePlatform);
		FString SelectedPath = RegularPath;

		bool bPlatformConfigExistsInRegular = FPaths::DirectoryExists(*RegularPath);

		// if the platform is an extension, create the new config in the extension path (Platforms/PlatformName/Config),
		// unless there exists a platform config in the regular path (Config/PlatformName)

		// PlatformExtension | ConfigExistsInRegularPath  |   Use path
		//   false                  false                      regular
		//   true                   false                      extension
		//   false                  true                       regular
		//   true                   true                       regular

		// if the project already uses platform configs in the regular directory, just use that, otherwise check if this is a platform extensions
		if (bIsPlatformExtension && !bPlatformConfigExistsInRegular)
		{
			SelectedPath = FPaths::Combine(FPaths::ProjectPlatformExtensionDir(*OverridePlatform), TEXT("Config"));
		}

		return FConfigCacheIni::NormalizeConfigIniPath(FString::Printf(TEXT("%s/%s%s.ini"), *SelectedPath, *OverridePlatform, *GetClass()->ClassConfigName.ToString()));
	}
	return FConfigCacheIni::NormalizeConfigIniPath(FString::Printf(TEXT("%sDefault%s.ini"), *FPaths::SourceConfigDir(), *GetClass()->ClassConfigName.ToString()));
}

FString UObject::GetGlobalUserConfigFilename() const
{
	return FString::Printf(TEXT("%sUnreal Engine/Engine/Config/User%s.ini"), FPlatformProcess::UserSettingsDir(), *GetClass()->ClassConfigName.ToString());
}

FString UObject::GetProjectUserConfigFilename() const
{
	return FString::Printf(TEXT("%sUser%s.ini"), *FPaths::ProjectConfigDir(), *GetClass()->ClassConfigName.ToString());
}

// @todo ini: Verify per object config objects
void UObject::UpdateSingleSectionOfConfigFile(const FString& ConfigIniName)
{
	// create a sandbox FConfigCache
	FConfigCacheIni Config(EConfigCacheType::Temporary);

	// add an empty file to the config so it doesn't read in the original file (see FConfigCacheIni.Find())
	FConfigFile& NewFile = Config.Add(ConfigIniName, FConfigFile());

	// save the object properties to this file
	SaveConfig(CPF_Config, *ConfigIniName, &Config);

	// do we need to use a special platform hierarchy?
	FString OverridePlatform = GetFinalOverridePlatform(this);
	bool bUpdateGConfig = OverridePlatform.Len() == 0;

	// if we are going to reload GConfig, we need to flush any pending writes to disk because we are going to
	// blow away the contents of GConfig's in memory version with what it on disk, but we don't want to lose the
	// local modifications
	if (bUpdateGConfig)
	{
		GConfig->Flush(false, *GetClass()->ClassConfigName.ToString());
	}

	// make sure SaveConfig wrote only to the file we expected
	NewFile.UpdateSections(*ConfigIniName, *GetClass()->ClassConfigName.ToString(), OverridePlatform.Len() ? *OverridePlatform : nullptr);

	// reload the file, so that it refresh the cache internally, unless a non-standard platform was used,
	// then we don't want to touch GConfig
	if (bUpdateGConfig)
	{
		FConfigContext Context = FConfigContext::ForceReloadIntoGConfig();
		// don't write the Saved out, as we just finished writing what we needed
		Context.bWriteDestIni = false;
		Context.Load(*GetClass()->ClassConfigName.ToString());
	}
}

bool UObject::TryUpdateDefaultConfigFile(const FString& SpecificFileLocation, bool bWarnIfFail)
{
	FString ConfigFile = SpecificFileLocation.IsEmpty() ? GetDefaultConfigFilename() : SpecificFileLocation;

	if (!FPaths::FileExists(ConfigFile) || !IFileManager::Get().IsReadOnly(*ConfigFile))
	{
		UpdateSingleSectionOfConfigFile(ConfigFile);

		return true;
	}

	if (bWarnIfFail)
	{
		UE_LOGF(LogObj, Warning, "Ini File '%ls' is read-only and cannot be written to", *ConfigFile);
	}

	return false;
}

void UObject::UpdateGlobalUserConfigFile()
{
	UpdateSingleSectionOfConfigFile(GetGlobalUserConfigFilename());
}

void UObject::UpdateProjectUserConfigFile()
{
	UpdateSingleSectionOfConfigFile(GetProjectUserConfigFilename());
}

void UObject::UpdateSinglePropertyInConfigFile(const FProperty* InProperty, const FString& InConfigIniName)
{
	if (GUseNewPropertySaving > 0)
	{
		// run SaveConfig, which has some logic to for diffing against parent class, and now has logic to also
		// directly update a single config property, including arrays, via FConfigCacheIni functions
		FSaveConfigContext Context;
		Context.Filename = InConfigIniName;
		Context.bSaveToLayer = true;
		Context.PropertyNames = { InProperty->GetFName() };
		SaveConfig(Context);
		return;
	}

	// Arrays and ini files are a mine field, for now we don't support this.
	if (!InProperty->IsA(FArrayProperty::StaticClass()))
	{
		// create a sandbox FConfigCache
		FConfigCacheIni Config(EConfigCacheType::Temporary);

		// add an empty file to the config so it doesn't read in the original file (see FConfigCacheIni.Find())
		FConfigFile& NewFile = Config.Add(InConfigIniName, FConfigFile());

		// save the object properties to this file
		SaveConfig(CPF_Config, *InConfigIniName, &Config);

		// Take the saved section for this object and have the config system process and write out the one property we care about.
		TArray<FString> Keys;
		NewFile.GetKeys(Keys);

		FString PropertyKey = InProperty->GetFName().ToString();
		if (!ensureMsgf(!Keys.IsEmpty(), TEXT("Unable to find section for property '%s' in '%s'. Make sure the property has the 'config' specifier.")
			, *PropertyKey
			, *GetNameSafe(GetClass())))
		{
			return;
		}

		const FString SectionName = Keys[0];

#if WITH_EDITOR
		const FString& CVarName = InProperty->GetMetaData(GConsoleVariableFName);
		if (!CVarName.IsEmpty())
		{
			PropertyKey = CVarName;
		}
#endif // #if WITH_EDITOR

		// do we need to use a special platform hierarchy?
		bool bUpdateGConfig = GetFinalOverridePlatform(this).Len() == 0;

		// if we are going to reload GConfig, we need to flush any pending writes to disk because we are going to
		// blow away the contents of GConfig's in memory version with what it on disk, but we don't want to lose the
		// local modifications
		if (bUpdateGConfig)
		{
			GConfig->Flush(false, *GetClass()->ClassConfigName.ToString());
		}

		NewFile.UpdateSinglePropertyInSection(*InConfigIniName, *PropertyKey, *SectionName);

		// reload the file, so that it refresh the cache internally, unless a non-standard platform was used,
		// then we don't want to touch GConfig
		if (bUpdateGConfig)
		{
			FConfigContext Context = FConfigContext::ForceReloadIntoGConfig();
			// don't write the Saved out, as we just finished writing what we needed
			Context.bWriteDestIni = false;
			Context.Load(*GetClass()->ClassConfigName.ToString());
		}
	}
	else
	{
		UE_LOGF(LogObj, Warning, "UObject::UpdateSinglePropertyInConfigFile does not support this property type.");
		return;
	}
}


void UObject::InstanceSubobjectTemplates( FObjectInstancingGraph* InstanceGraph )
{
	UClass *ObjClass = GetClass();
	if (ObjClass->HasAnyClassFlags(CLASS_HasInstancedReference) )
	{
		UObject *Archetype = GetArchetype();
		if (InstanceGraph)
		{
			ObjClass->InstanceSubobjectTemplates( this, Archetype, Archetype ? Archetype->GetClass() : NULL, this, InstanceGraph );
		}
		else
		{
			FObjectInstancingGraph TempInstanceGraph(this);
			ObjClass->InstanceSubobjectTemplates( this, Archetype, Archetype ? Archetype->GetClass() : NULL, this, &TempInstanceGraph );
		}
	}
	CheckDefaultSubobjects();
}



void UObject::ReinitializeProperties( UObject* SourceObject/*=NULL*/, FObjectInstancingGraph* InstanceGraph/*=NULL*/ )
{
	if ( SourceObject == NULL )
	{
		SourceObject = GetArchetype();
	}

	check(GetClass() == UObject::StaticClass() || (SourceObject && IsA(SourceObject->GetClass())));

	// Recreate this object based on the new archetype - using StaticConstructObject rather than manually tearing down and re-initializing
	// the properties for this object ensures that any cleanup required when an object is reinitialized from defaults occurs properly
	// for example, when re-initializing UPrimitiveComponents, the component must notify the rendering thread that its data structures are
	// going to be re-initialized
	FStaticConstructObjectParameters Params(GetClass());
	Params.Outer = GetOuter();
	Params.Name = GetFName();
	Params.SetFlags = GetFlags();
	Params.InternalSetFlags = GetInternalFlags();
	Params.Template = SourceObject;
	Params.bCopyTransientsFromClassDefaults = !HasAnyFlags(RF_ClassDefaultObject);
	Params.InstanceGraph = InstanceGraph;
	StaticConstructObject_Internal(Params);
}


/*-----------------------------------------------------------------------------
   Shutdown.
-----------------------------------------------------------------------------*/

/**
 * After a critical error, shutdown all objects which require
 * mission-critical cleanup, such as restoring the video mode,
 * releasing hardware resources.
 */
static void StaticShutdownAfterError()
{
	if( UObjectInitialized() )
	{
		static bool bShutdown = false;
		if( bShutdown )
		{
			return;
		}
		bShutdown = true;
		UE_LOGF(LogExit, Log, "Executing StaticShutdownAfterError" );

		for( FThreadSafeObjectIterator It; It; ++It )
		{
			UObject* Object = *It;
			if(Object)
			{
				Object->ShutdownAfterError();
			}
		}
	}
}

/*-----------------------------------------------------------------------------
   Command line.
-----------------------------------------------------------------------------*/
#include "UObject/ClassTree.h"

static void ShowIntrinsicClasses( FOutputDevice& Ar )
{
	FClassTree MarkedClasses(UObject::StaticClass());
	FClassTree UnmarkedClasses(UObject::StaticClass());

	for ( TObjectIterator<UClass> It; It; ++It )
	{
		if ( It->HasAnyClassFlags(CLASS_Native) )
		{
			if ( It->HasAllClassFlags(CLASS_Intrinsic) )
			{
				MarkedClasses.AddClass(*It);
			}
			else
			{
				UnmarkedClasses.AddClass(*It);
			}
		}
	}

	Ar.Logf(TEXT("INTRINSIC CLASSES WITH FLAG SET: %i classes"), MarkedClasses.Num());
	MarkedClasses.DumpClassTree(0, Ar);

	Ar.Logf(TEXT("INTRINSIC CLASSES WITHOUT FLAG SET: %i classes"), UnmarkedClasses.Num());
	UnmarkedClasses.DumpClassTree(0, Ar);
}

//
// Show the inheritance graph of all loaded classes.
//
static void ShowClasses( UClass* Class, FOutputDevice& Ar, int32 Indent )
{
	Ar.Logf( TEXT("%s%s (%d)"), FCString::Spc(Indent), *Class->GetName(), Class->GetPropertiesSize() );

	// Workaround for Visual Studio 2013 analyzer bug. Using a temporary directly in the range-for
	// errors if the analyzer is enabled.
	TObjectRange<UClass> Range;
	for( UClass* Obj : Range )
	{
		if( Obj->GetSuperClass() == Class )
		{
			ShowClasses( Obj, Ar, Indent+2 );
		}
	}
}

void UObject::OutputReferencers( FOutputDevice& Ar, FReferencerInformationList* Referencers/*=NULL*/ )
{
	bool bTempReferencers = false;
	if (!Referencers)
	{
		bTempReferencers = true;
		TArray<FReferencerInformation> InternalReferences;
		TArray<FReferencerInformation> ExternalReferences;

		RetrieveReferencers(&InternalReferences, &ExternalReferences);

		Referencers = new FReferencerInformationList(InternalReferences, ExternalReferences);
	}

	Ar.Log( TEXT("\r\n") );
	if ( Referencers->InternalReferences.Num() > 0 || Referencers->ExternalReferences.Num() > 0 )
	{
		if ( Referencers->ExternalReferences.Num() > 0 )
		{
			Ar.Logf( TEXT("External referencers of %s:\r\n"), *GetFullName() );

			for ( int32 RefIndex = 0; RefIndex < Referencers->ExternalReferences.Num(); RefIndex++ )
			{
				FReferencerInformation& RefInfo = Referencers->ExternalReferences[RefIndex];
				FString ObjectReachability = RefInfo.Referencer->GetFullName();

				if( RefInfo.Referencer->IsRooted() )
				{
					ObjectReachability += TEXT(" (root)");
				}
		
				if( RefInfo.Referencer->IsNative() )
				{
					ObjectReachability += TEXT(" (native)");
				}
		
				if( RefInfo.Referencer->HasAnyFlags(RF_Standalone) )
				{
					ObjectReachability += TEXT(" (standalone)");
				}

				UGCObjectReferencer* GCObjectReferencer = Cast<UGCObjectReferencer>(RefInfo.Referencer);
				Ar.Logf( TEXT("   %s (%i)\r\n"), *ObjectReachability, RefInfo.TotalReferences );
				for ( int32 i = 0; i < RefInfo.TotalReferences; i++ )
				{
					if ( i < RefInfo.ReferencingProperties.Num() )
					{
						const FProperty* Referencer = RefInfo.ReferencingProperties[i];
						Ar.Logf(TEXT("      %i) %s\r\n"), i, *Referencer->GetFullName());
					}
					else
					{
						FString ReferencerName;
						if (GCObjectReferencer && GCObjectReferencer->GetReferencerName(this, ReferencerName))
						{
							Ar.Logf(TEXT("      %i) [[FGCObject %s]]\r\n"), i, *ReferencerName);
						}
						else
						{
							Ar.Logf(TEXT("      %i) [[native reference]]\r\n"), i);
						}
					}
				}
			}
		}

		if ( Referencers->InternalReferences.Num() > 0 )
		{
			if ( Referencers->ExternalReferences.Num() > 0 )
			{
				Ar.Log(TEXT("\r\n"));
			}

			Ar.Logf( TEXT("Internal referencers of %s:\r\n"), *GetFullName() );
			for ( int32 RefIndex = 0; RefIndex < Referencers->InternalReferences.Num(); RefIndex++ )
			{
				FReferencerInformation& RefInfo = Referencers->InternalReferences[RefIndex];

				Ar.Logf( TEXT("   %s (%i)\r\n"), *RefInfo.Referencer->GetFullName(), RefInfo.TotalReferences );
				for ( int32 i = 0; i < RefInfo.TotalReferences; i++ )
				{
					if ( i < RefInfo.ReferencingProperties.Num() )
					{
						const FProperty* Referencer = RefInfo.ReferencingProperties[i];
						Ar.Logf(TEXT("      %i) %s\r\n"), i, *Referencer->GetFullName());
					}
					else
					{
						Ar.Logf(TEXT("      %i) [[native reference]]\r\n"), i);
					}
				}
			}
		}
	}
	else
	{
		Ar.Logf(TEXT("%s is not referenced"), *GetFullName());
	}

	Ar.Logf(TEXT("\r\n") );

	if (bTempReferencers)
	{
		delete Referencers;
	}
}

void UObject::RetrieveReferencers( TArray<FReferencerInformation>* OutInternalReferencers, TArray<FReferencerInformation>* OutExternalReferencers )
{
	for( FThreadSafeObjectIterator It; It; ++It )
	{
		UObject* Object = *It;

		if ( Object == this )
		{
			// this one is pretty easy  :)
			continue;
		}

		FArchiveFindCulprit ArFind(this,Object,false);
		TArray<const FProperty*> Referencers;

		int32 Count = ArFind.GetCount(Referencers);
		if ( Count > 0 )
		{
			CA_SUPPRESS(6011)
			if ( Object->IsIn(this) )
			{
				if (OutInternalReferencers != NULL)
				{
					// manually allocate just one element - much slower but avoids slack which improves success rate on consoles
					OutInternalReferencers->Reserve(OutInternalReferencers->Num() + 1);
					OutInternalReferencers->Emplace(Object, Count, Referencers);
				}
			}
			else
			{
				if (OutExternalReferencers != NULL)
				{
					// manually allocate just one element - much slower but avoids slack which improves success rate on consoles
					OutExternalReferencers->Reserve(OutExternalReferencers->Num() + 1);
					OutExternalReferencers->Emplace(Object, Count, Referencers);
				}
			}
		}
	}
}

void UObject::ParseParms( const TCHAR* Parms )
{
	if (!Parms)
	{
		return;
	}
	for( TFieldIterator<FProperty> It(GetClass()); It; ++It )
	{
		if (It->GetOwner<UObject>() != UObject::StaticClass())
		{
			FString Value;
			if( FParse::Value(Parms,*(FString(*It->GetName())+TEXT("=")),Value) )
			{
				It->ImportText_InContainer( *Value, this, this, 0 );
			}
		}
	}
}

/**
 * Maps object flag to human-readable string.
 */
class FObjectFlag
{
public:
	EObjectFlags	ObjectFlag;
	const TCHAR*	FlagName;
	FObjectFlag(EObjectFlags InObjectFlag, const TCHAR* InFlagName)
		:	ObjectFlag( InObjectFlag )
		,	FlagName( InFlagName )
	{}
};

/**
 * Initializes the singleton list of object flags.
 */
static TArray<FObjectFlag> PrivateInitObjectFlagList()
{
	TArray<FObjectFlag> ObjectFlagList;
#ifdef	DECLARE_OBJECT_FLAG
#error DECLARE_OBJECT_FLAG already defined
#else
#define DECLARE_OBJECT_FLAG( ObjectFlag ) ObjectFlagList.Add( FObjectFlag( RF_##ObjectFlag, TEXT(#ObjectFlag) ) );
	DECLARE_OBJECT_FLAG( ClassDefaultObject )
	DECLARE_OBJECT_FLAG( ArchetypeObject )
	DECLARE_OBJECT_FLAG( Transactional )
	DECLARE_OBJECT_FLAG( Public	)
	DECLARE_OBJECT_FLAG( TagGarbageTemp )
	DECLARE_OBJECT_FLAG( NeedLoad )
	DECLARE_OBJECT_FLAG( Transient )
	DECLARE_OBJECT_FLAG( Standalone )
	DECLARE_OBJECT_FLAG( BeginDestroyed )
	DECLARE_OBJECT_FLAG( FinishDestroyed )
	DECLARE_OBJECT_FLAG( NeedPostLoad )
#undef DECLARE_OBJECT_FLAG
#endif
	return ObjectFlagList;
}
/**
 * Dumps object flags from the selected objects to debugf.
 */
static void PrivateDumpObjectFlags(UObject* Object, FOutputDevice& Ar)
{
	static TArray<FObjectFlag> SObjectFlagList = PrivateInitObjectFlagList();

	if ( Object )
	{
		FString Buf( FString::Printf( TEXT("%s:\t"), *Object->GetFullName() ) );
		for ( int32 FlagIndex = 0 ; FlagIndex < SObjectFlagList.Num() ; ++FlagIndex )
		{
			const FObjectFlag& CurFlag = SObjectFlagList[ FlagIndex ];
			if ( Object->HasAnyFlags( CurFlag.ObjectFlag ) )
			{
				Buf += FString::Printf( TEXT("%s "), CurFlag.FlagName );
			}
		}
		Ar.Logf( TEXT("%s"), *Buf );
	}
}

/**
 * Recursively visits all object properties and dumps object flags.
 */
static void PrivateRecursiveDumpFlags(UStruct* Struct, void* Data, FOutputDevice& Ar)
{
	check(Data != NULL);
	for( TFieldIterator<FProperty> It(Struct); It; ++It )
	{
		if (It->GetOwnerClass() && It->GetOwnerClass()->GetPropertiesSize() != sizeof(UObject) )
		{
			for( int32 i=0; i<It->ArrayDim; i++ )
			{
				uint8* Value = It->ContainerPtrToValuePtr<uint8>(Data, i);
				FObjectPropertyBase* Prop = CastField<FObjectPropertyBase>(*It);
				if(Prop)
				{
					UObject* Obj = Prop->GetObjectPropertyValue(Value);
					PrivateDumpObjectFlags( Obj, Ar );
				}
				else if( FStructProperty* StructProperty = CastField<FStructProperty>(*It) )
				{
					PrivateRecursiveDumpFlags( StructProperty->Struct, Value, Ar );
				}
			}
		}
	}
}

/** 
 * Performs the work for "SET" and "SETNOPEC".
 *
 * @param	Str						reset of console command arguments
 * @param	Ar						output device to use for logging
 * @param	bNotifyObjectOfChange	whether to notify the object about to be changed via Pre/PostEditChange
 */
static void PerformSetCommand( const TCHAR* Str, FOutputDevice& Ar, bool bNotifyObjectOfChange )
{
	// Set a class default variable.
	TCHAR ObjectName[256], PropertyName[256];
	if (FParse::Token(Str, ObjectName, UE_ARRAY_COUNT(ObjectName), true) && FParse::Token(Str, PropertyName, UE_ARRAY_COUNT(PropertyName), true))
	{
		UClass* Class = FindFirstObject<UClass>(ObjectName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("PerformSetCommand"));
		if (Class != NULL)
		{
			FProperty* Property = FindFProperty<FProperty>(Class, PropertyName);
			if (Property != NULL)
			{
				while (*Str == ' ')
				{
					Str++;
				}
				GlobalSetProperty(Str, Class, Property, bNotifyObjectOfChange);
			}
			else
			{
				UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Unrecognized property %s on class %s"), PropertyName, ObjectName));
			}
		}
		else
		{
			UObject* Object = FindFirstObject<UObject>(ObjectName, EFindFirstObjectOptions::NativeFirst, ELogVerbosity::Warning, TEXT("PerformSetCommand"));
			if (Object != NULL)
			{
				FProperty* Property = FindFProperty<FProperty>(Object->GetClass(), PropertyName);
				if (Property != NULL)
				{
					while (*Str == ' ')
					{
						Str++;
					}

#if WITH_EDITOR
					if (!Object->HasAnyFlags(RF_ClassDefaultObject) && bNotifyObjectOfChange)
					{
						Object->PreEditChange(Property);
					}
#endif // WITH_EDITOR
					Property->ImportText_InContainer(Str, Object, Object, 0);
#if WITH_EDITOR
					if (!Object->HasAnyFlags(RF_ClassDefaultObject) && bNotifyObjectOfChange)
					{
						FPropertyChangedEvent PropertyEvent(Property);
						Object->PostEditChangeProperty(PropertyEvent);
					}
#endif // WITH_EDITOR
				}
			}
			else
			{
				UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Unrecognized class or object %s"), ObjectName));
			}
		}
	}
	else 
	{
		UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Unexpected input); format is 'set [class or object name] [property name] [value]")));
	}
}

/** Helper structure for property listing console command */
struct FListPropsWildcardPiece
{
	FString Str;
	bool bMultiChar;
	FListPropsWildcardPiece(const FString& InStr, bool bInMultiChar)
		: Str(InStr), bMultiChar(bInMultiChar)
	{}
};

void ParseFunctionFlags(uint32 Flags, TArray<const TCHAR*>& Results)
{
	const TCHAR* FunctionFlags[32] =
	{
		TEXT("Final"),					// FUNC_Final
		TEXT("0x00000002"),
		TEXT("BlueprintAuthorityOnly"),	// FUNC_BlueprintAuthorityOnly
		TEXT("BlueprintCosmetic"),		// FUNC_BlueprintCosmetic
		TEXT("0x00000010"),
		TEXT("0x00000020"),
		TEXT("Net"),					// FUNC_Net
		TEXT("NetReliable"),			// FUNC_NetReliable
		TEXT("NetRequest"),				// FUNC_NetRequest
		TEXT("Exec"),					// FUNC_Exec
		TEXT("Native"),					// FUNC_Native
		TEXT("Event"),					// FUNC_Event
		TEXT("NetResponse"),			// FUNC_NetResponse
		TEXT("Static"),					// FUNC_Static
		TEXT("NetMulticast"),			// FUNC_NetMulticast
		TEXT("0x00008000"),
		TEXT("MulticastDelegate"),		// FUNC_MulticastDelegate
		TEXT("Public"),					// FUNC_Public
		TEXT("Private"),				// FUNC_Private
		TEXT("Protected"),				// FUNC_Protected
		TEXT("Delegate"),				// FUNC_Delegate
		TEXT("NetServer"),				// FUNC_NetServer
		TEXT("HasOutParms"),			// FUNC_HasOutParms
		TEXT("HasDefaults"),			// FUNC_HasDefaults
		TEXT("NetClient"),				// FUNC_NetClient
		TEXT("DLLImport"),				// FUNC_DLLImport
		TEXT("BlueprintCallable"),		// FUNC_BlueprintCallable
		TEXT("BlueprintEvent"),			// FUNC_BlueprintEvent
		TEXT("BlueprintPure"),			// FUNC_BlueprintPure
		TEXT("0x20000000"),
		TEXT("Const"),					// FUNC_Const
		TEXT("0x80000000"),
	};

	for (int32 i = 0; i < 32; ++i)
	{
		const uint32 Mask = 1U << i;
		if ((Flags & Mask) != 0)
		{
			Results.Add(FunctionFlags[i]);
		}
	}
}


COREUOBJECT_API TArray<const TCHAR*> ParsePropertyFlags(EPropertyFlags InFlags)
{
	TArray<const TCHAR*> Results;

	static const TCHAR* PropertyFlags[] =
	{
		TEXT("CPF_Edit"),
		TEXT("CPF_ConstParm"),
		TEXT("CPF_BlueprintVisible"),
		TEXT("CPF_ExportObject"),
		TEXT("CPF_BlueprintReadOnly"),
		TEXT("CPF_Net"),
		TEXT("CPF_EditFixedSize"),
		TEXT("CPF_Parm"),
		TEXT("CPF_OutParm"),
		TEXT("CPF_ZeroConstructor"),
		TEXT("CPF_ReturnParm"),
		TEXT("CPF_DisableEditOnTemplate"),
		TEXT("CPF_NonNullable"),
		TEXT("CPF_Transient"),
		TEXT("CPF_Config"),
		TEXT("CPF_RequiredParm"),
		TEXT("CPF_DisableEditOnInstance"),
		TEXT("CPF_EditConst"),
		TEXT("CPF_GlobalConfig"),
		TEXT("CPF_InstancedReference"),
		TEXT("CPF_ExperimentalExternalObjects"),
		TEXT("CPF_DuplicateTransient"),
		TEXT("0x0000000000400000"),
		TEXT("0x0000000000800000"),
		TEXT("CPF_SaveGame"),	
		TEXT("CPF_NoClear"),
		TEXT("CPF_Virtual"),
		TEXT("CPF_ReferenceParm"),
		TEXT("CPF_BlueprintAssignable"),
		TEXT("CPF_Deprecated"),
		TEXT("CPF_IsPlainOldData"),
		TEXT("CPF_RepSkip"),
		TEXT("CPF_RepNotify"),
		TEXT("CPF_Interp"),
		TEXT("CPF_NonTransactional"),
		TEXT("CPF_EditorOnly"),
		TEXT("CPF_NoDestructor"),
		TEXT("0x0000002000000000"),
		TEXT("CPF_AutoWeak"),
		TEXT("CPF_ContainsInstancedReference"),
		TEXT("CPF_AssetRegistrySearchable"),
		TEXT("CPF_SimpleDisplay"),
		TEXT("CPF_AdvancedDisplay"),
		TEXT("CPF_Protected"),
		TEXT("CPF_BlueprintCallable"),
		TEXT("CPF_BlueprintAuthorityOnly"),
		TEXT("CPF_TextExportTransient"),
		TEXT("CPF_NonPIEDuplicateTransient"),
		TEXT("CPF_ExposeOnSpawn"),
		TEXT("CPF_PersistentInstance"),
		TEXT("CPF_UObjectWrapper"),
		TEXT("CPF_HasGetValueTypeHash"),
		TEXT("CPF_NativeAccessSpecifierPublic"),
		TEXT("CPF_NativeAccessSpecifierProtected"),
		TEXT("CPF_NativeAccessSpecifierPrivate"),
		TEXT("CPF_SkipSerialization"),
		TEXT("CPF_TObjectPtr"),
		TEXT("CPF_ExperimentalOverridableLogic"),
		TEXT("CPF_ExperimentalAlwaysOverriden"),
		TEXT("CPF_ExperimentalNeverOverriden"),
		TEXT("CPF_AllowSelfReference"),
	};

	uint64 Flags = InFlags;
	for (const TCHAR* FlagName : PropertyFlags)
	{
		if (Flags & 1)
		{
			Results.Add(FlagName);
		}

		Flags >>= 1;
	}

	return Results;
}

// #UObject: 2014-09-15 Move to ObjectCommads.cpp or ObjectExec.cpp
bool StaticExec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar )
{
	const TCHAR *Str = Cmd;

	if( FParse::Command(&Str,TEXT("GET")) )
	{
		// Get a class default variable.
		TCHAR ClassName[256], PropertyName[256];
		UClass* Class;
		FProperty* Property;
		if
		(	FParse::Token( Str, ClassName, UE_ARRAY_COUNT(ClassName), 1 )
		&&	(Class = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("StaticExec GET"))) != nullptr)
		{
			if
			(	FParse::Token( Str, PropertyName, UE_ARRAY_COUNT(PropertyName), 1 )
			&&	(Property=FindFProperty<FProperty>( Class, PropertyName)) != nullptr)
			{
				FString	Temp;
				if( Class->GetDefaultsCount() )
				{
					Property->ExportText_InContainer( 0, Temp, (uint8*)Class->GetDefaultObject(), (uint8*)Class->GetDefaultObject(), Class, PPF_IncludeTransient );
				}
				Ar.Log( *Temp );
			}
			else
			{
				UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Unrecognized property %s"), PropertyName ));
			}
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Unrecognized class %s"), ClassName ));
		}
		return true;
	}
	else if (FParse::Command(&Str, TEXT("LISTPROPS")))
	{
		// list all properties of the specified class that match the specified wildcard string
		TCHAR ClassName[256];
		UClass* Class;
		FString PropWildcard;

		if ( FParse::Token(Str, ClassName, UE_ARRAY_COUNT(ClassName), 1) &&
			(Class = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("StaticExec LISTPROPS"))) != nullptr &&
			FParse::Token(Str, PropWildcard, true) )
		{
			// split up the search string by wildcard symbols
			TArray<FListPropsWildcardPiece> WildcardPieces;
			bool bFound;
			do
			{
				bFound = false;
				int32 AsteriskPos = PropWildcard.Find(TEXT("*"));
				int32 QuestionPos = PropWildcard.Find(TEXT("?"));
				if (AsteriskPos != INDEX_NONE || QuestionPos != INDEX_NONE)
				{
					if (AsteriskPos != INDEX_NONE && (QuestionPos == INDEX_NONE || QuestionPos > AsteriskPos))
					{
						WildcardPieces.Emplace(PropWildcard.Left(AsteriskPos), true);
						PropWildcard.RightInline(PropWildcard.Len() - AsteriskPos - 1, EAllowShrinking::No);
						bFound = true;
					}
					else if (QuestionPos != INDEX_NONE)
					{
						WildcardPieces.Emplace(PropWildcard.Left(QuestionPos), false);
						PropWildcard.RightInline(PropWildcard.Len() - QuestionPos - 1, EAllowShrinking::No);
						bFound = true;
					}
				}
			} while (bFound);
			bool bEndedInConstant = (PropWildcard.Len() > 0);
			if (bEndedInConstant)
			{
				WildcardPieces.Emplace(PropWildcard, false);
			}

			// search for matches
			int32 Count = 0;
			for (TFieldIterator<FProperty> It(Class); It; ++It)
			{
				FProperty* Property = *It;

				Ar.Logf(TEXT("    Prop %s"), *FString::Printf(TEXT("%s at offset %d; %dx %d bytes of type %s"), *Property->GetName(), Property->GetOffset_ForDebug(), Property->ArrayDim, Property->GetElementSize(), *Property->GetClass()->GetName()));

				for (const TCHAR* Flag : ParsePropertyFlags(Property->PropertyFlags))
				{
					Ar.Logf(TEXT("      Flag %s"), Flag);
				}
			}
			for (TFieldIterator<FProperty> It(Class); It; ++It)
			{
				FString Match = It->GetName();
				bool bResult = true;
				for (int32 i = 0; i < WildcardPieces.Num(); i++)
				{
					if (WildcardPieces[i].Str.Len() > 0)
					{
						int32 Pos = Match.Find(WildcardPieces[i].Str, ESearchCase::IgnoreCase);
						if (Pos == INDEX_NONE || (i == 0 && Pos != 0))
						{
							bResult = false;
							break;
						}
						else if (i > 0 && !WildcardPieces[i - 1].bMultiChar && Pos != 1)
						{
							bResult = false;
							break;
						}

						Match.RightInline(Match.Len() - Pos - WildcardPieces[i].Str.Len(), EAllowShrinking::No);
					}
				}
				if (bResult)
				{
					// validate ending wildcard, if any
					if (bEndedInConstant)
					{
						bResult = (Match.Len() == 0);
					}
					else if (!WildcardPieces.Last().bMultiChar)
					{
						bResult = (Match.Len() == 1);
					}

					if (bResult)
					{
						FString ExtraInfo;
						if (FStructProperty* StructProperty = CastField<FStructProperty>(*It))
						{
							ExtraInfo = *StructProperty->Struct->GetName();
						}
						else if (FClassProperty* ClassProperty = CastField<FClassProperty>(*It))
						{
							ExtraInfo = FString::Printf(TEXT("SubclassOf<%s>"), *ClassProperty->MetaClass->GetName());
						}
						else if (FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(*It))
						{
							ExtraInfo = FString::Printf(TEXT("SoftClassPtr<%s>"), *SoftClassProperty->MetaClass->GetName());
						}
						else if (FObjectPropertyBase* ObjectPropertyBase = CastField<FObjectPropertyBase>(*It))
						{
							ExtraInfo = *ObjectPropertyBase->PropertyClass->GetName();
						}
						else
						{
							ExtraInfo = It->GetClass()->GetName();
						}
						Ar.Logf(TEXT("%i) %s (%s)"), Count, *It->GetName(), *ExtraInfo);
						Count++;
					}
				}
			}
			if (Count == 0)
			{
				Ar.Logf(TEXT("- No matches"));
			}
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("ListProps: expected format is 'ListProps [class] [wildcard]")));
		}

		return true;
	}
	else if (FParse::Command(&Str, TEXT("GETALL")))
	{
		// iterate through all objects of the specified type and return the value of the specified property for each object
		TCHAR ClassName[256], PropertyName[256];
		UClass* Class;
		FProperty* Property;

		if ( FParse::Token(Str,ClassName,UE_ARRAY_COUNT(ClassName), 1) &&
			(Class = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("StaticExec GETALL"))) != nullptr )
		{
			FParse::Token(Str,PropertyName,UE_ARRAY_COUNT(PropertyName),1);
			{
				Property=FindFProperty<FProperty>(Class,PropertyName);
				{
					int32 cnt = 0;
					UObject* LimitOuter = nullptr;

					const bool bHasOuter = FCString::Strifind(Str,TEXT("OUTER=")) ? true : false;
					ParseObject<UObject>(Str,TEXT("OUTER="),LimitOuter,nullptr);

					// Check for a specific object name
					TCHAR ObjNameStr[256];
					FName ObjName(NAME_None);
					if (FParse::Value(Str,TEXT("NAME="),ObjNameStr,UE_ARRAY_COUNT(ObjNameStr)))
					{
						ObjName = FName(ObjNameStr);
					}
					
					if( bHasOuter && !LimitOuter )
					{
						UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Failed to find outer %s"), FCString::Strifind(Str,TEXT("OUTER=")) ));
					}
					else
					{
						bool bShowDefaultObjects = FParse::Command(&Str,TEXT("SHOWDEFAULTS"));
						bool bShowPendingKills = FParse::Command(&Str, TEXT("SHOWPENDINGKILLS"));
						bool bShowDetailedInfo = FParse::Command(&Str, TEXT("DETAILED"));
						for ( FThreadSafeObjectIterator It; It; ++It )
						{
							UObject* CurrentObject = *It;

							if ( LimitOuter != NULL && !CurrentObject->IsIn(LimitOuter) )
							{
								continue;
							}

							if ( CurrentObject->IsTemplate(RF_ClassDefaultObject) && bShowDefaultObjects == false )
							{
								continue;
							}

							if (ObjName != NAME_None && CurrentObject->GetFName() != ObjName)
							{
								continue;
							}

							if ( (bShowPendingKills || IsValidChecked(CurrentObject)) && CurrentObject->IsA(Class) )
							{
								if (!Property)
								{
									if (bShowDetailedInfo)
									{
										Ar.Logf(TEXT("%i) %s %s"), cnt++, *CurrentObject->GetFullName(),*CurrentObject->GetDetailedInfo() );
									}
									else
									{
										Ar.Logf(TEXT("%i) %s"), cnt++, *CurrentObject->GetFullName());
									}
									continue;
								}
								if ( Property->ArrayDim > 1 || CastField<FArrayProperty>(Property) != NULL )
								{
									uint8* BaseData = Property->ContainerPtrToValuePtr<uint8>(CurrentObject);
									Ar.Logf(TEXT("%i) %s.%s ="), cnt++, *CurrentObject->GetFullName(), *Property->GetName());

									int32 ElementCount = Property->ArrayDim;

									FProperty* ExportProperty = Property;
									if ( Property->ArrayDim == 1 )
									{
										FArrayProperty* ArrayProp = CastField<FArrayProperty>(Property);
										FScriptArrayHelper ArrayHelper(ArrayProp, BaseData);

										BaseData = ArrayHelper.GetRawPtr();
										ElementCount = ArrayHelper.Num();
										ExportProperty = ArrayProp->Inner;
									}

									int32 ElementSize = ExportProperty->GetElementSize();
									for ( int32 ArrayIndex = 0; ArrayIndex < ElementCount; ArrayIndex++ )
									{
										FString ResultStr;
										uint8* ElementData = BaseData + ArrayIndex * ElementSize;
										ExportProperty->ExportTextItem_Direct(ResultStr, ElementData, NULL, CurrentObject, PPF_IncludeTransient);

										if (bShowDetailedInfo)
										{
											Ar.Logf(TEXT("\t%i: %s %s"), ArrayIndex, *ResultStr, *CurrentObject->GetDetailedInfo());
										}
										else
										{
											Ar.Logf(TEXT("\t%i: %s"), ArrayIndex, *ResultStr);
										}
									}
								}
								else
								{
									uint8* BaseData = (uint8*)CurrentObject;
									FString ResultStr;
									for (int32 i = 0; i < Property->ArrayDim; i++)
									{
										Property->ExportText_InContainer(i, ResultStr, BaseData, BaseData, CurrentObject, PPF_IncludeTransient);
									}

									if (bShowDetailedInfo)
									{
										Ar.Logf(TEXT("%i) %s.%s = %s %s"), cnt++, *CurrentObject->GetFullName(), *Property->GetName(), *ResultStr, *CurrentObject->GetDetailedInfo() );
									}
									else
									{
										Ar.Logf(TEXT("%i) %s.%s = %s"), cnt++, *CurrentObject->GetFullName(), *Property->GetName(), *ResultStr);
									}
								}
							}
						}
					}
				}
			}
		}
		else
		{
			UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("Unrecognized class %s"), ClassName ));
		}
		return true;
	}
	else if( FParse::Command(&Str,TEXT("SET")) )
	{
		PerformSetCommand( Str, Ar, true );
		return true;
	}
	else if( FParse::Command(&Str,TEXT("SETNOPEC")) )
	{
		PerformSetCommand( Str, Ar, false );
		return true;
	}
#if !UE_BUILD_SHIPPING
	else if( FParse::Command(&Str,TEXT("LISTFUNCS")) )
	{
		// LISTFUNCS <classname>
		TCHAR ClassName[256];

		if (FParse::Token(Str, ClassName, UE_ARRAY_COUNT(ClassName), true))
		{
			//if ( (Property=FindFProperty<FProperty>(Class,PropertyName)) != NULL )
			UClass* Class = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("StaticExec LISTFUNCS"));

			if (Class != NULL)
			{
				Ar.Logf(TEXT("Listing functions introduced in class %s (class flags = 0x%08X)"), ClassName, (uint32)Class->GetClassFlags());
				for (TFieldIterator<UFunction> It(Class); It; ++It)
				{
					UFunction* Function = *It;

					FString FunctionName = Function->GetName();
					Ar.Logf(TEXT("Function %s"), *FunctionName);
				}
			}
			else
			{
				Ar.Logf(TEXT("Could not find any classes named %s"), ClassName);
			}
		}
	}
	else if( FParse::Command(&Str,TEXT("LISTFUNC")) )
	{
		// LISTFUNC <classname> <functionname>
		TCHAR ClassName[256];
		TCHAR FunctionName[256];
		if (FParse::Token(Str, ClassName, UE_ARRAY_COUNT(ClassName), true) && FParse::Token(Str, FunctionName, UE_ARRAY_COUNT(FunctionName), true))
		{
			UClass* Class = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("StaticExec LISTFUNC"));

			if (Class != NULL)
			{
				UFunction* Function = FindUField<UFunction>(Class, FunctionName);

				if (Function != NULL)
				{
					Ar.Logf(TEXT("Processing function %s"), *Function->GetName());

					// Global properties
					if (Function->GetSuperFunction() != NULL)
					{
						Ar.Logf(TEXT("  Has super function (overrides a base class function)"));
					}

					// Flags
					TArray<const TCHAR*> Flags;
					ParseFunctionFlags(Function->FunctionFlags, Flags);
					for (int32 i = 0; i < Flags.Num(); ++i)
					{
						Ar.Logf(TEXT("  Flag %s"), Flags[i]);
					}

					
					// Parameters
					Ar.Logf(TEXT("  %d parameters taking up %d bytes, with return value at offset %d"), Function->NumParms, Function->ParmsSize, Function->ReturnValueOffset);
					for (TFieldIterator<FProperty> It(Function); It; ++It)
					{
						if (It->PropertyFlags & CPF_Parm)
						{
							FProperty* Property = *It;
							
							Ar.Logf(TEXT("    Parameter %s"), *FString::Printf(TEXT("%s at offset %d; %dx %d bytes of type %s"), *Property->GetName(), Property->GetOffset_ForDebug(), Property->ArrayDim, Property->GetElementSize(), *Property->GetClass()->GetName()));

							for (const TCHAR* Flag : ParsePropertyFlags(Property->PropertyFlags))
							{
								Ar.Logf(TEXT("      Flag %s"), Flag);
							}
						}
					}

					// Locals
					Ar.Logf(TEXT("  Total stack size %d bytes"), Function->PropertiesSize );

					for (TFieldIterator<FProperty> It(Function); It; ++It)
					{
						if ((It->PropertyFlags & CPF_Parm) == 0)
						{							
							FProperty* Property = *It;

							Ar.Logf(TEXT("    Local %s"), *FString::Printf(TEXT("%s at offset %d; %dx %d bytes of type %s"), *Property->GetName(), Property->GetOffset_ForDebug(), Property->ArrayDim, Property->GetElementSize(), *Property->GetClass()->GetName()));

							for (const TCHAR* Flag : ParsePropertyFlags(Property->PropertyFlags))
							{
								Ar.Logf(TEXT("      Flag %s"), Flag);
							}
						}
					}

					if (Function->Script.Num() > 0)
					{
						Ar.Logf(TEXT("  Has %d bytes of script bytecode"), Function->Script.Num());
					}
				}
			}
		}

		return true;
	}
	else if( FParse::Command(&Str,TEXT("OBJ")) )
	{
		if( FParse::Command(&Str,TEXT("CYCLES")) )
		{
			// find all cycles in the reference graph

			FFindStronglyConnected IndexSet;
			IndexSet.FindAllCycles();
			int32 MaxNum = 0;
			int32 TotalNum = 0;
			int32 TotalCnt = 0;
			for (int32 Index = 0; Index < IndexSet.Components.Num(); Index++)
			{
				TArray<UObject*>& StronglyConnected = IndexSet.Components[Index];
				MaxNum = FMath::Max<int32>(StronglyConnected.Num(), MaxNum);
				if (StronglyConnected.Num() > 1)
				{
					TotalNum += StronglyConnected.Num();
					TotalCnt++;
				}
			}
			// sort
			for (int32 CurrentNum = MaxNum; CurrentNum > 1; CurrentNum--)
			{
				for (int32 Index = 0; Index < IndexSet.Components.Num(); Index++)
				{
					TArray<UObject*>& StronglyConnected = IndexSet.Components[Index];
					if (StronglyConnected.Num() == CurrentNum)
					{
						Ar.Logf(TEXT("------------------------------------------------------------------------"));
						for (int32 IndexInner = 0; IndexInner < StronglyConnected.Num(); IndexInner++)
						{
							Ar.Logf(TEXT("%s"),*StronglyConnected[IndexInner]->GetFullName());
						}
						Ar.Logf(TEXT("    simple cycle ------------------"));
						TArray<UObject*>& SimpleCycle = IndexSet.SimpleCycles[Index];
						for (int32 IndexDescribe = 0; IndexDescribe < SimpleCycle.Num(); IndexDescribe++)
						{
							int32 Other = IndexDescribe + 1 < SimpleCycle.Num() ? IndexDescribe + 1 : 0;
							Ar.Logf(TEXT("    %s -> %s"), *SimpleCycle[Other]->GetFullName(), *SimpleCycle[IndexDescribe]->GetFullName());
							FArchiveDescribeReference(SimpleCycle[Other], SimpleCycle[IndexDescribe], Ar);
						}
					}
				}
			}

			Ar.Logf(TEXT("------------------------------------------------------------------------"));
			Ar.Logf(TEXT("%d total objects, %d total edges."), IndexSet.AllObjects.Num(), IndexSet.AllEdges.Num());
			Ar.Logf(TEXT("Non-permanent: %d objects, %d edges, %d strongly connected components, %d objects are included in cycles."), IndexSet.TempObjects.Num(), IndexSet.Edges.Num(), TotalCnt, TotalNum);
			return true;
		}
		else if (FParse::Command(&Str, TEXT("VERIFYCOMPONENTS")))
		{
			Ar.Logf(TEXT("------------------------------------------------------------------------------"));

			for (FThreadSafeObjectIterator It; It; ++It)
			{
				UObject* Target = *It;

				// Skip objects that are trashed
				if ((Target->GetOutermost() == GetTransientPackage())
					|| Target->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists)
					|| !IsValidChecked(Target))
				{
					continue;
				}

				TArray<UObject*> SubObjects;
				GetObjectsWithOuter(Target, SubObjects);

				TArray<FString> Errors;

				for (UObject* SubObjIt : SubObjects)
				{
					const UObject* SubObj = SubObjIt;
					const UClass* SubObjClass = SubObj->GetClass();
					const FString SubObjName = SubObj->GetName();

					if (!IsValid(SubObj))
					{
						continue;
					}

					if (SubObjClass->HasAnyClassFlags(CLASS_NewerVersionExists))
					{
						Errors.Add(FString::Printf(TEXT("  - %s has a stale class"), *SubObjName));
					}

					if (SubObjClass->GetOutermost() == GetTransientPackage())
					{
						Errors.Add(FString::Printf(TEXT("  - %s has a class in the transient package"), *SubObjName));
					}

					if (SubObj->GetOutermost() != Target->GetOutermost())
					{
						Errors.Add(FString::Printf(TEXT("  - %s has a different outer than its parent"), *SubObjName));
					}
					
					if (SubObj->GetName().Find(TEXT("TRASH_")) != INDEX_NONE)
					{
						Errors.Add(FString::Printf(TEXT("  - %s is TRASH'd"), *SubObjName));
					}

					if (SubObj->GetName().Find(TEXT("REINST_")) != INDEX_NONE)
					{
						Errors.Add(FString::Printf(TEXT("  - %s is a REINST"), *SubObjName));
					}
				}

				if (Errors.Num() > 0)
				{
					Ar.Logf(TEXT("Errors for %s"), *Target->GetName());

					for (const FString& ErrorStr : Errors)
					{
						Ar.Logf(TEXT("  - %s"), *ErrorStr);
					}
				}
			}

			Ar.Logf(TEXT("------------------------------------------------------------------------------"));
			return true;
		}
		else if( FParse::Command(&Str,TEXT("TRANSACTIONAL")) )
		{
			int32 Num=0;
			int32 NumTransactional=0;
			for( FThreadSafeObjectIterator It; It; ++It )
			{
				Num++;
				if (It->HasAnyFlags(RF_Transactional))
				{
					NumTransactional++;
				}
				UE_LOGF(LogObj, Log, "%1d %ls",(int32)It->HasAnyFlags(RF_Transactional),*It->GetFullName());
			}
			UE_LOGF(LogObj, Log, "%d/%d",NumTransactional,Num);
			return true;
		}
		else if( FParse::Command(&Str,TEXT("MARK")) )
		{
			UE_LOGF(LogObj, Log,  "Marking objects" );
			for( FThreadSafeObjectIterator It; It; ++It )
			{
				DebugMarkAnnotation.Set(*It);
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("MARKCHECK")) )
		{
			UE_LOGF(LogObj, Log,  "Unmarked (new) objects:" );
			for( FThreadSafeObjectIterator It; It; ++It )
			{
				if(!DebugMarkAnnotation.Get(*It))
				{
					UE_LOGF(LogObj, Log,  "%ls", *It->GetFullName() );
				}
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("INVMARK")) )
		{
			UE_LOGF(LogObj, Log,  "InvMarking existing objects" );
			DebugInvMarkWeakPtrs.Empty();
			DebugInvMarkNames.Empty();
			for( FThreadSafeObjectIterator It; It; ++It )
			{
				DebugInvMarkWeakPtrs.Add(TWeakObjectPtr<>(*It));
				DebugInvMarkNames.Add(It->GetFullName());
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("INVMARKCHECK")) )
		{
			UE_LOGF(LogObj, Log,  "Objects that were deleted:" );
			for (int32 Old = 0; Old < DebugInvMarkNames.Num(); Old++)
			{
				UObject *Object = DebugInvMarkWeakPtrs[Old].Get();
				if (Object)
				{
					check(TWeakObjectPtr<>(Object) == DebugInvMarkWeakPtrs[Old]);
					check(Object->GetFullName() == DebugInvMarkNames[Old]);
					check(!DebugInvMarkWeakPtrs[Old].IsStale());
					check(DebugInvMarkWeakPtrs[Old].IsValid());
				}
				else
				{
					check(DebugInvMarkWeakPtrs[Old].IsStale());
					check(!DebugInvMarkWeakPtrs[Old].IsValid());
					UE_LOGF(LogObj, Log,  "%ls", *DebugInvMarkNames[Old]);
				}
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("SPIKEMARK")) )
		{
			UE_LOGF(LogObj, Log,  "Spikemarking objects" );
			
			FlushAsyncLoading();

			DebugSpikeMarkAnnotation.ClearAll();
			for( FThreadSafeObjectIterator It; It; ++It )
			{
				DebugSpikeMarkAnnotation.Set(*It);
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("SPIKEMARKCHECK")) )
		{
			UE_LOGF(LogObj, Log,  "Spikemarked (created and then destroyed) objects:" );
			for( const FString& Name : DebugSpikeMarkNames )
			{
				UE_LOGF(LogObj, Log,  "  %ls", *Name );
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("REFS")) )
		{
			UObject* Object;
			if (ParseObject(Str,TEXT("NAME="),Object,nullptr))
			{
				
				EReferenceChainSearchMode SearchModeFlags = EReferenceChainSearchMode::PrintResults;
				int32 HistoryLevel = 0;

				FString Tok;
				while(FParse::Token(Str, Tok, false))
				{
					if (FCString::Stricmp(*Tok, TEXT("shortest")) == 0)
					{
						if ( !!(SearchModeFlags&EReferenceChainSearchMode::Longest) )
						{
							UE_LOGF(LogObj, Log, "Specifing 'shortest' AND 'longest' is invalid. Ignoring this occurence of 'shortest'.");
						}
						SearchModeFlags |= EReferenceChainSearchMode::Shortest;
					}
					else if (FCString::Stricmp(*Tok, TEXT("longest")) == 0)
					{
						if ( !!(SearchModeFlags&EReferenceChainSearchMode::Shortest) )
						{
							UE_LOGF(LogObj, Log, "Specifing 'shortest' AND 'longest' is invalid. Ignoring this occurence of 'longest'.");
						}
						SearchModeFlags |= EReferenceChainSearchMode::Longest;
					}
					else if (FCString::Stricmp(*Tok, TEXT("all")) == 0)
					{
						SearchModeFlags |= EReferenceChainSearchMode::PrintAllResults;
					}
					else if (FCString::Stricmp(*Tok, TEXT("external")) == 0)
					{
						SearchModeFlags |= EReferenceChainSearchMode::ExternalOnly;
					}
					else if (FCString::Stricmp(*Tok, TEXT("direct")) == 0)
					{
						SearchModeFlags |= EReferenceChainSearchMode::Direct;
					}
					else if (FCString::Stricmp(*Tok, TEXT("full")) == 0)
					{
						SearchModeFlags |= EReferenceChainSearchMode::FullChain;
					}
					else if (FCString::Stricmp(*Tok, TEXT("minimal")) == 0)
					{
						SearchModeFlags |= EReferenceChainSearchMode::Minimal;
					}
					else if (FCString::Stricmp(*Tok, TEXT("gconly")) == 0)
					{
						SearchModeFlags |= EReferenceChainSearchMode::GCOnly;
					}
#if ENABLE_GC_HISTORY
					else if (FParse::Value(Str, TEXT("history="), HistoryLevel))
					{
						if (FGCHistory::Get().IsActive())
						{
							int32 MaxHistoryLevel = FGCHistory::Get().GetHistorySize();
							if (FMath::Abs(HistoryLevel) > MaxHistoryLevel)
							{
								UE_LOGF(LogObj, Log, "GC History level %d will be clamped to the current max %d", HistoryLevel, MaxHistoryLevel);
								HistoryLevel = MaxHistoryLevel;
							}
						}
						else if (HistoryLevel != 0)
						{
							UE_LOGF(LogObj, Log, "GC History is not currently enabled. Use 'gc.HistorySize 1' console command to enable it.");
							HistoryLevel = 0;
						}
					}
#endif // ENABLE_GC_HISTORY
				}
				
				if (HistoryLevel == 0)
				{
					FReferenceChainSearch RefChainSearch(Object, SearchModeFlags);
				}
#if ENABLE_GC_HISTORY
				else
				{
					FReferenceChainSearch HistorySearch(SearchModeFlags);
					FGCSnapshot* GCSnapshot = FGCHistory::Get().GetSnapshot(FMath::Abs(HistoryLevel) - 1);
					if (GCSnapshot)
					{
						HistorySearch.PerformSearchFromGCSnapshot(Object, *GCSnapshot);
					}
					else
					{
						UE_LOGF(LogObj, Log, "There's not been that many (%d) GC runs to be able to search for object refs in GC history.", (HistoryLevel + 1));
					}
				}
#endif // ENABLE_GC_HISTORY
			}
			else
			{
				UE_LOGF(LogObj, Log, "Couldn't find object.");
			}
			return true;
		}
		else if (FParse::Command(&Str, TEXT("SINGLEREF")))
		{
			bool bListClass = false;
			UClass* Class;
			UClass* ReferencerClass = NULL;
			FString ReferencerName;
			if (ParseObject<UClass>(Str, TEXT("CLASS="), Class, nullptr) == false)
			{
				Class = UObject::StaticClass();
				bListClass = true;
			}
			if (ParseObject<UClass>(Str, TEXT("REFCLASS="), ReferencerClass, nullptr) == false)
			{
				ReferencerClass = NULL;
			}
			TCHAR TempStr[1024];
			if (FParse::Value(Str, TEXT("REFNAME="), TempStr, UE_ARRAY_COUNT(TempStr)))
			{
				ReferencerName = TempStr;
			}

			for (TObjectIterator<UObject> It; It; ++It)
			{
				UObject* Object = *It;
				if ((Object->IsA(Class)) && (Object->IsTemplate() == false) && (Object->HasAnyFlags(RF_ClassDefaultObject) == false))
				{
					TArray<FReferencerInformation> OutExternalReferencers;
					Object->RetrieveReferencers(NULL, &OutExternalReferencers);

					if (OutExternalReferencers.Num() == 1)
					{
						FReferencerInformation& Info = OutExternalReferencers[0];
						UObject* RefObj = Info.Referencer;
						if (RefObj)
						{
							bool bDumpIt = true;
							if (ReferencerName.Len() > 0)
							{
								if (RefObj->GetName() != ReferencerName)
								{
									bDumpIt = false;
								}
							}
							if (ReferencerClass)
							{
								if (RefObj->IsA(ReferencerClass) == false)
								{
									bDumpIt = false;
								}
							}

							if (bDumpIt)
							{
								FArchiveCountMem Count(Object);

								// Get the 'old-style' resource size and the truer resource size
								const SIZE_T ResourceSize = It->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
								const SIZE_T TrueResourceSize = It->GetResourceSizeBytes(EResourceSizeMode::Exclusive);
								
								if (bListClass)
								{
									Ar.Logf(TEXT("%64s: %64s, %8d,%8d,%8d,%8d"), *(Object->GetClass()->GetName()), *(Object->GetPathName()),
											(int32)Count.GetNum(), (int32)Count.GetMax(), (int32)ResourceSize, (int32)TrueResourceSize);
								}
								else
								{
									Ar.Logf(TEXT("%64s, %8d,%8d,%8d,%8d"), *(Object->GetPathName()),
										(int32)Count.GetNum(), (int32)Count.GetMax(), (int32)ResourceSize, (int32)TrueResourceSize);
								}
								Ar.Logf(TEXT("\t%s"), *(RefObj->GetPathName()));
							}
						}
					}
				}
			}
			return true;
		}
		else if( FParse::Command(&Str,TEXT("CLASSES")) )
		{
			ShowClasses( UObject::StaticClass(), Ar, 0 );
			return true;
		}
		else if( FParse::Command(&Str,TEXT("INTRINSICCLASSES")) )
		{
			ShowIntrinsicClasses(Ar);
			return true;
		}
		else if( FParse::Command(&Str,TEXT("DEPENDENCIES")) )
		{
			UPackage* Pkg;
			if( ParseObject<UPackage>(Str,TEXT("PACKAGE="),Pkg,NULL) )
			{
				TArray<UObject*> Exclude;

				// check if we want to ignore references from any packages
				for( int32 i=0; i<16; i++ )
				{
					TCHAR Temp[MAX_SPRINTF] = {};
					FCString::Sprintf( Temp, TEXT("EXCLUDE%i="), i );
					FName F;
					if (FParse::Value(Str, Temp, F))
					{
						Exclude.Add(CreatePackage(*F.ToString()));
					}
				}
				Ar.Logf( TEXT("Dependencies of %s:"), *Pkg->GetPathName() );

				bool Dummy=0;

				// Should we recurse into inner packages?
				bool bRecurse = FParse::Bool(Str, TEXT("RECURSE"), Dummy);

				// Iterate through the object list
				for( FThreadSafeObjectIterator It; It; ++It )
				{
					// if this object is within the package specified, serialize the object
					// into a specialized archive which logs object names encountered during
					// serialization -- rjp
					if ( It->IsIn(Pkg) )
					{
						if ( It->GetOuter() == Pkg )
						{
							FArchiveShowReferences ArShowReferences( Ar, Pkg, *It, Exclude );
						}
						else if ( bRecurse )
						{
							// Two options -
							// a) this object is a function or something (which we don't care about)
							// b) this object is inside a group inside the specified package (which we do care about)
							UObject* CurrentObject = *It;
							UObject* CurrentOuter = It->GetOuter();
							while ( CurrentObject && CurrentOuter )
							{
								// this object is a UPackage (a group inside a package)
								// abort
								if ( CurrentObject->GetClass() == UPackage::StaticClass() )
									break;

								// see if this object's outer is a UPackage
								if ( CurrentOuter->GetClass() == UPackage::StaticClass() )
								{
									// if this object's outer is our original package, the original object (It)
									// wasn't inside a group, it just wasn't at the base level of the package
									// (its Outer wasn't the Pkg, it was something else e.g. a function, state, etc.)
									/// ....just skip it
									if ( CurrentOuter == Pkg )
										break;

									// otherwise, we've successfully found an object that was in the package we
									// were searching, but would have been hidden within a group - let's log it
									FArchiveShowReferences ArShowReferences( Ar, CurrentOuter, CurrentObject, Exclude );
									break;
								}

								CurrentObject = CurrentOuter;
								CurrentOuter = CurrentObject->GetOuter();
							}
						}
					}
				}
			}
			else
				UE_LOGF(LogObj, Log, "Package wasn't found.");
			return true;
		}
		else if( FParse::Command(&Str,TEXT("BULK")) )
		{
			FBulkData::DumpBulkDataUsage( Ar );
			return true;
		}
		else if( FParse::Command(&Str,TEXT("LISTCONTENTREFS")) )
		{
			UClass*	Class		= NULL;
			UClass*	ListClass	= NULL;
			ParseObject<UClass>(Str, TEXT("CLASS="		), Class,		nullptr );
			ParseObject<UClass>(Str, TEXT("LISTCLASS="  ), ListClass,	nullptr );
		
			if( Class )
			{
				/** Helper class for only finding object references we "care" about. See operator << for details. */
				struct FArchiveListRefs : public FArchiveUObject
				{
					/** Set of objects ex and implicitly referenced by root based on criteria in << operator. */
					TSet<UObject*> ReferencedObjects;
					
					/** 
					 * Constructor, performing serialization of root object.
					 */
					FArchiveListRefs( UObject* InRootObject )
					{
						ArIsObjectReferenceCollector = true;
						RootObject = InRootObject;
						RootObject->Serialize( *this );
					}

				private:
					/** Src/ root object to serialize. */
					UObject* RootObject;

					// The serialize operator is private as we don't support changing RootObject. */
					FArchive& operator<<( UObject*& Object )
					{
						if ( Object != NULL )
						{
							// Avoid serializing twice.
							if( ReferencedObjects.Find( Object ) == NULL )
							{
								ReferencedObjects.Add( Object );

								// Recurse if we're in the same package.
								if( RootObject->GetOutermost() == Object->GetOutermost() 
								// Or if package doesn't contain script.
								||	!Object->GetOutermost()->HasAnyPackageFlags(PKG_ContainsScript) )
								{
									// Serialize object. We don't want to use the << operator here as it would call 
									// this function again instead of serializing members.
									Object->Serialize( *this );
								}
							}
						}							
						return *this;
					}
				};

				// Create list of object references.
				FArchiveListRefs ListRefsAr(Class);

				// Give a choice of whether we want sorted list in more human read-able format or whether we want to list in Excel.
				bool bShouldListAsCSV = FParse::Param( Str, TEXT("CSV") );

				// If specified only lists objects not residing in script packages.
				bool bShouldOnlyListContent = !FParse::Param( Str, TEXT("LISTSCRIPTREFS") );

				// Sort refs by class name (un-qualified name).
				struct FSortUObjectByClassName
				{
					FORCEINLINE bool operator()( const UObject& A, const UObject& B ) const
					{
						return A.GetClass()->GetName() <= B.GetClass()->GetName();
					}
				};
				ListRefsAr.ReferencedObjects.Sort( FSortUObjectByClassName() );
				
				if( bShouldListAsCSV )
				{
					UE_LOGF(LogObj, Log, ",Class,Object");
				}
				else
				{
					UE_LOGF(LogObj, Log, "Dumping references for %ls",*Class->GetFullName());
				}

				// Iterate over references and dump them to log. Either in CSV format or sorted by class.
				for( TSet<UObject*>::TConstIterator It(ListRefsAr.ReferencedObjects); It; ++It ) 
				{
					UObject* ObjectReference = *It;
					// Only list certain class if specified.
					if( (!ListClass || ObjectReference->GetClass() == ListClass)
					// Only list non-script objects if specified.
					&&	(!bShouldOnlyListContent || !ObjectReference->GetOutermost()->HasAnyPackageFlags(PKG_ContainsScript))
					// Exclude the transient package.
					&&	ObjectReference->GetOutermost() != GetTransientPackage() )
					{
						if( bShouldListAsCSV )
						{
							UE_LOGF(LogObj, Log, ",%ls,%ls",*ObjectReference->GetClass()->GetPathName(),*ObjectReference->GetPathName());
						}
						else
						{
							UE_LOGF(LogObj, Log, "   %ls",*ObjectReference->GetFullName());
						}
					}
				}
			}
		}
		else if ( FParse::Command(&Str,TEXT("FLAGS")) )
		{
			// Dump all object flags for objects rooted at the named object.
			TCHAR ObjectName[NAME_SIZE];
			UObject* Obj = NULL;
			if ( FParse::Token(Str,ObjectName,UE_ARRAY_COUNT(ObjectName), 1) )
			{
				Obj = FindFirstObject<UObject>(ObjectName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("FLAGS command"));
			}

			if ( Obj )
			{
				PrivateDumpObjectFlags( Obj, Ar );
				PrivateRecursiveDumpFlags( Obj->GetClass(), Obj, Ar );
			}

			return true;
		}
		else if (FParse::Command(&Str, TEXT("REP")))
		{
			// Lists all the properties of a class marked for replication
			// Usage:  OBJ REP CLASS=PlayerController
			UClass* Cls;

			if( ParseObject<UClass>( Str, TEXT("CLASS="), Cls, nullptr ) )
			{
				Ar.Logf(TEXT("=== Replicated properties for class: %s==="), *Cls->GetName());
				for ( TFieldIterator<FProperty> It(Cls); It; ++It )
				{
					if( (It->GetPropertyFlags() & CPF_Net) != 0 )
					{
						if( (It->GetPropertyFlags() & CPF_RepNotify) != 0 )
						{
							Ar.Logf(TEXT("   %s <%s>"), *It->GetName(), *It->RepNotifyFunc.ToString());
						}
						else
						{
							Ar.Logf(TEXT("   %s"), *It->GetName());
						}
					}
				}
			}
			else
			{
				UE_SUPPRESS(LogExec, Warning, Ar.Logf(TEXT("No class objects found using command '%s'"), Cmd));
			}

			return true;
		}
		else return false;
	}
	// For reloading config on a particular object
	else if( FParse::Command(&Str,TEXT("RELOADCONFIG")) ||
		FParse::Command(&Str,TEXT("RELOADCFG")))
	{
		TCHAR ClassName[256];
		// Determine the object/class name
		if (FParse::Token(Str,ClassName,UE_ARRAY_COUNT(ClassName),1))
		{
			UObject* ObjectToReload = nullptr;
			
			// Try to find a corresponding class
			UClass* ClassToReload = FindFirstObject<UClass>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("StaticExec RELOADCONFIG"));
			if (ClassToReload)
			{
				ObjectToReload = ClassToReload->GetDefaultObject();
			}
			else
			{
				// If the class is missing, search for an object with that name
				ObjectToReload = FindFirstObject<UObject>(ClassName, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("StaticExec RELOADCONFIG"));
			}
			if (ObjectToReload)
			{
				if (ObjectToReload->GetClass()->HasAnyClassFlags(EClassFlags::CLASS_Config))
				{
					// unload the branch so next access will load the static and dynamic layers
					GConfig->SafeUnloadBranch(*ObjectToReload->GetClass()->GetConfigName());
					
					// now updates all the class properties now that the config was reloaded from disk
					ObjectToReload->ReloadConfig();
				}
				else
				{
					Ar.Logf(TEXT("Class %s is not a config-enabled class."), *ObjectToReload->GetClass()->GetName());
				}
			}
		}
		return true;
	}
#endif // !UE_BUILD_SHIPPING
	// Route to self registering exec handlers.
	else if(FSelfRegisteringExec::StaticExec( InWorld, Cmd,Ar ))
	{
		return true;
	}
	
	return false; // Not executed
}

/*-----------------------------------------------------------------------------
	StaticInit & StaticExit.
-----------------------------------------------------------------------------*/

void StaticUObjectInit();
void InitUObject();
AUTORTFM_DISABLE void StaticExit();

void InitUObject()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(InitUObject);
	LLM_SCOPE(ELLMTag::InitUObject);

	FGCCSyncObject::Create();

	// Initialize redirects map
	FCoreRedirects::Initialize();
	for (const FString& Filename : GConfig->GetFilenames())
	{
		FCoreRedirects::ReadRedirectsFromIni(Filename);
		FLinkerLoad::CreateActiveRedirectsMap(Filename);
	}

	FCoreDelegates::OnShutdownAfterError.AddStatic(StaticShutdownAfterError);
	FCoreDelegates::OnExit.AddStatic(StaticExit);

	// Always register the UObjects callbacks from the module manager for dynamic loading/unloading.
	RegisterModularObjectsProcessing();

	struct Local
	{
		static bool IsPackageLoaded( FName PackageName )
		{
			return FindPackage( NULL, *PackageName.ToString() ) != NULL;
		}
	};
	FModuleManager::Get().IsPackageLoadedCallback().BindStatic(Local::IsPackageLoaded);
	
	FCoreDelegates::NewFileAddedDelegate.AddStatic(FLinkerLoad::OnNewFileAdded);
	FCoreDelegates::GetOnPakFileMounted2().AddStatic(FLinkerLoad::OnPakFileMounted);

	// Object initialization.
	StaticUObjectInit();

#if UE_ENABLE_ASSET_READ_LOGGER
	// Initialize asset read logger if enabled via command line
	FAssetReadLogger::Get().Initialize();
#endif

	FOverridableManager::Create();
#if WITH_EDITORONLY_DATA
	UE::FPropertyBagRepository::Get();
#endif
}

//
// Init the object manager and allocate tables.
//
void StaticUObjectInit()
{
	UObjectBaseInit();

	// Allocate special packages.
	GObjTransientPkg = NewObject<UPackage>(nullptr, TEXT("/Engine/Transient"), RF_Transient);
	GObjTransientPkg->AddToRoot();

	// Force GC verification on or off based on command line parameters.
	bool bCommandLineForcesVerifyGC = FParse::Param(FCommandLine::Get(), TEXT("VERIFYGC"));
	bool bCommandLineForcesNoVerifyGC = FParse::Param(FCommandLine::Get(), TEXT("NOVERIFYGC"));
	if (IConsoleVariable* CVarVerifyGCAssumptions = IConsoleManager::Get().FindConsoleVariable(TEXT("gc.VerifyAssumptions")))
	{
		if (bCommandLineForcesVerifyGC)
		{
			CVarVerifyGCAssumptions->Set(true, ECVF_SetByCommandline);
		}
		else if (bCommandLineForcesNoVerifyGC)
		{
			CVarVerifyGCAssumptions->Set(false, ECVF_SetByCommandline);
		}
	}
	if (IConsoleVariable* CVarVerifyGCAssumptionsOnFullPurge = IConsoleManager::Get().FindConsoleVariable(TEXT("gc.VerifyAssumptionsOnFullPurge")))
	{
		if (bCommandLineForcesVerifyGC)
		{
			CVarVerifyGCAssumptionsOnFullPurge->Set(true, ECVF_SetByCommandline);
		}
		else if (bCommandLineForcesNoVerifyGC)
		{
			CVarVerifyGCAssumptionsOnFullPurge->Set(false, ECVF_SetByCommandline);
		}
	}
	if (IConsoleVariable* CVarVerifyGCChance = IConsoleManager::Get().FindConsoleVariable(TEXT("gc.VerifyAssumptionsChance")))
	{
		if (bCommandLineForcesNoVerifyGC)
		{
			CVarVerifyGCChance->Set(0.0f, ECVF_SetByCommandline);
		}
	}

	UE::RemoteObject::Private::InitRemoteObjects();

	UE_LOGF(LogInit, Log, "Object subsystem initialized" );
}

// Internal cleanup functions
AUTORTFM_DISABLE void ShutdownGarbageCollection();
void CleanupLinkerAnnotations();
AUTORTFM_DISABLE void PurgeAllUObjectsOnExit();

//
// Shut down the object manager.
//
void StaticExit()
{
	if (UObjectInitialized() == false)
	{
		return;
	}

#if UE_ENABLE_ASSET_READ_LOGGER
	// Shutdown asset read logger and write output file
	FAssetReadLogger::Get().Shutdown();
#endif

	// Delete all linkers are pending destroy
	DeleteLoaders();

	// Cleanup root.
	if (GObjTransientPkg != NULL)
	{
		GObjTransientPkg->RemoveFromRoot();
		GObjTransientPkg = NULL;
	}
	
	PurgeAllUObjectsOnExit();

	ShutdownGarbageCollection();
	UObjectBaseShutdown();

	// Empty arrays to prevent falsely-reported memory leaks.
	FDeferredMessageLog::Cleanup();	
	CleanupLinkerAnnotations();
	FObjectArchetypeHelper::CleanupCachedArchetypes();
	UE::RemoteObject::Private::ShutdownRemoteObjects();

	UE_LOGF(LogExit, Log, "Object subsystem successfully closed." );
}


/*-----------------------------------------------------------------------------
	Misc.
-----------------------------------------------------------------------------*/

UPackage* GetTransientPackage()
{
	return GObjTransientPkg;
}

UObject* GetTransientPackageAsObject()
{
	return GObjTransientPkg;
}

UObject* GetTransientOuterForRename(UClass* ForClass)
{
	// if someone has tautologically placed themself within their own hierarchy then we'll
	// just assume they're ok with eventually being outered to a upackage, similar UPackage
	// is a UObject, so if someone demands that they be outered to 'a uobject' we'll 
	// just leave them directly parented to the transient package:
	if (ForClass->ClassWithin && ForClass->ClassWithin != ForClass && ForClass->ClassWithin != UObject::StaticClass())
	{
		FScopedAllowAbstractClassAllocation AllowAbstract;
		return NewObject<UObject>(GetTransientOuterForRename(ForClass->ClassWithin), ForClass->ClassWithin, NAME_None, RF_Transient);
	}
	return GetTransientPackage();
}

//keep this global to ensure that an actual write is prepared
volatile const UObject** GUObjectAbortNullPointer = nullptr;

/**
 * Abort with a member function call at the top of the callstack, helping to ensure that most platforms will stuff this object's memory into the resulting minidump.
 */
void UObject::AbortInsideMemberFunction() const
{
	//put a trace of this in the log to help diagnostics at a glance.
	UE_LOGF(LogObj, Warning, "UObject::AbortInsideMemberFunction called on object %ls.", *GetFullName());

	if (GLog)
	{
		GLog->Flush();
	}

	//a bit more ideally, we could set GIsCriticalError = true and call FPlatformMisc::RequestExit. however, not all platforms would generate a dump as a result of this.
	//as such, we commit an access violation right here. we explicitly want to avoid the standard platform error/AssertFailed paths as they are likely to pollute the
	//callstack. this in turn is more likely to prevent useful (e.g. this object) memory from making its way into a minidump.

	//this'll result in the address of this object being conveniently loaded into a register, so we don't have to dig a pointer out of the stack in the event of any
	//ambiguity/reg-stomping resulting from the log call above. in a test ps4 minidump, this also ensured that the debugger was able to automatically find the address of
	//"this" within the stack frame, which was otherwise made impossible due to register reuse in the log call above.
	*GUObjectAbortNullPointer = this;
}

/*-----------------------------------------------------------------------------
	Replication.
-----------------------------------------------------------------------------*/
		
/** Returns properties that are replicated for the lifetime of the actor channel */
void UObject::GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const
{

}

/** Called when this object begins replicating to initialize the state of custom property conditions */
void UObject::GetReplicatedCustomConditionState(FCustomPropertyConditionState& OutActiveState) const
{

}

void UObject::RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags)
{
}

/** Called right before receiving a bunch */
void UObject::PreNetReceive()
{

}

/** Called right after receiving a bunch */
void UObject::PostNetReceive()
{

}

/** Called right before being marked for destruction due to network replication */
void UObject::PreDestroyFromReplication()
{

}

#if WITH_EDITOR
/*-----------------------------------------------------------------------------
	Data Validation.
-----------------------------------------------------------------------------*/

EDataValidationResult UObject::IsDataValid(FDataValidationContext& Context) const
{
	return EDataValidationResult::NotValidated;
}

#endif // WITH_EDITOR
/** IsNameStableForNetworking means an object can be referred to its path name (relative to outer) over the network */
bool UObject::IsNameStableForNetworking() const
{
	//TODO: This should test for RF_ArchetypeObject too
	return HasAnyFlags(RF_WasLoaded | RF_DefaultSubObject | RF_ClassDefaultObject) || IsNative() || IsDefaultSubobject();
}

/** IsFullNameStableForNetworking means an object can be referred to its full path name over the network */
bool UObject::IsFullNameStableForNetworking() const
{
	if ( GetOuter() != NULL && !GetOuter()->IsFullNameStableForNetworking() )
	{
		return false; // If any outer isn't stable, we can't consider the full name stable
	}

	// Consider templates to always be stable
	return IsTemplate() || IsNameStableForNetworking();
}

/** IsSupportedForNetworking means an object can be referenced over the network */
bool UObject::IsSupportedForNetworking() const
{
	return IsFullNameStableForNetworking();
}

#if WITH_EDITOR
TFunction<bool(const UObject*)> GIsObjectSelectedInEditor;
#endif
