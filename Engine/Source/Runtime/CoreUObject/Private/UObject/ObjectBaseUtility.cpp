// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "Serialization/CustomVersion.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/Package.h"
#include "UObject/LinkerLoad.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"

/**
 * Returns the UE version of the linker for this object.
 *
 * @return	the UEversion of the engine's package file when this object
 *			was last saved, or GPackageFileUEVersion (current version) if
 *			this object does not have a linker, which indicates that
 *			a) this object is a native only class, or
 *			b) this object's linker has been detached, in which case it is already fully loaded
 */
FPackageFileVersion UObjectBaseUtility::GetLinkerUEVersion() const
{
	FLinkerLoad* Loader = GetLinker();

	// No linker.
	if (Loader == nullptr)
	{
		// the _Linker reference is never set for the top-most UPackage of a package (the linker root), so if this object
		// is the linker root, find our loader in the global list.
		if (GetOutermost() == this)
		{
			Loader = FLinkerLoad::FindExistingLinkerForPackage(CastChecked<UPackage>((const UObject*)this));
		}
	}

	if (Loader != nullptr)
	{
		// We have a linker so we can return its version.
		return Loader->UEVer();

	}
	else if (GetOutermost())
	{
		// Get the linker version associated with the package this object lives in
		return GetOutermost()->GetLinkerPackageVersion();
	}
	else
	{
		// We don't have a linker associated as we e.g. might have been saved or had loaders reset, ...
		return GPackageFileUEVersion;
	}
}

int32 UObjectBaseUtility::GetLinkerCustomVersion(FGuid CustomVersionKey) const
{
	FLinkerLoad* Loader = GetLinker();

	// No linker.
	if( Loader == nullptr )
	{
		// the _Linker reference is never set for the top-most UPackage of a package (the linker root), so if this object
		// is the linker root, find our loader in the global list.
		if( GetOutermost() == this )
		{
			Loader = FLinkerLoad::FindExistingLinkerForPackage(CastChecked<UPackage>((const UObject*)this));
		}
	}

	if ( Loader != nullptr)
	{
		// We have a linker so we can return its version.
		const FCustomVersion* CustomVersion = Loader->Summary.GetCustomVersionContainer().GetVersion(CustomVersionKey);
		return CustomVersion ? CustomVersion->Version : -1;
	}
	else if (GetOutermost() && GetOutermost()->GetLinkerCustomVersions().GetAllVersions().Num())
	{
		// Get the linker version associated with the package this object lives in
		const FCustomVersion* CustomVersion = GetOutermost()->GetLinkerCustomVersions().GetVersion(CustomVersionKey);
		return CustomVersion ? CustomVersion->Version : -1;
	}
	// We don't have a linker associated as we e.g. might have been saved or had loaders reset, ...
	// We must have a current version for this tag.
	return FCurrentCustomVersions::Get(CustomVersionKey).GetValue().Version;
}

/**
 * Returns the licensee version of the linker for this object.
 *
 * @return	the licensee version of the engine's package file when this object
 *			was last saved, or GPackageFileLicenseeVersion (current version) if
 *			this object does not have a linker, which indicates that
 *			a) this object is a native only class, or
 *			b) this object's linker has been detached, in which case it is already fully loaded
 */
int32 UObjectBaseUtility::GetLinkerLicenseeUEVersion() const
{
	FLinkerLoad* Loader = GetLinker();

	// No linker.
	if( Loader == nullptr)
	{
		// the _Linker reference is never set for the top-most UPackage of a package (the linker root), so if this object
		// is the linker root, find our loader in the global list.
		if( GetOutermost() == this )
		{
			Loader = FLinkerLoad::FindExistingLinkerForPackage(CastChecked<UPackage>((const UObject*)this));
		}
	}

	if ( Loader != nullptr)
	{
		// We have a linker so we can return its version.
		return Loader->LicenseeUEVer();
	}
	else if (GetOutermost())
	{
		// Get the linker version associated with the package this object lives in
		return GetOutermost()->GetLinkerLicenseeVersion();
	}
	else
	{
		// We don't have a linker associated as we e.g. might have been saved or had loaders reset, ...
		return GPackageFileLicenseeUEVersion;
	}
}

#if CPUPROFILERTRACE_ENABLED
COREUOBJECT_API FName GetClassTraceScope(const UObjectBaseUtility* Object)
{
	UClass* Class = Object->GetClass();
	if (Class->IsNative() || !IsRunningCommandlet())
	{
		return Class->GetFName();
	}
	else
	{
		static const FName NAME_Blueprint("Blueprint");
		return NAME_Blueprint;
	}
}
#endif

#if STATS && CPUPROFILERTRACE_ENABLED
void FScopeCycleCounterUObject::StartObjectTrace(const UObjectBaseUtility* Object)
{
	bool bTraceObjectFName = !IsRunningCommandlet();

#if LOADTIMEPROFILERTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(AssetLoadTimeChannel))
	{
		bTraceObjectFName = true;
	}
#endif // LOADTIMEPROFILERTRACE_ENABLED

	if (bTraceObjectFName)
	{
		StartTrace(Object->GetFName());
	}
	else
	{
		StartTrace(GetClassTraceScope(Object));
	}
}
#endif

#define UE_GARBAGE_ELIMINATION_ALLOWED !UE_USE_COMPACT_SET_AS_DEFAULT || UE_ENABLE_COMPACTHASHTABLE_KEYSTORAGE

// Console variable so that GarbageCollectorSettings work in the editor but we don't want to use it in runtime as we can't support changing its value from console
int32 GGarbageEliminationEnabled = UE_GARBAGE_ELIMINATION_ALLOWED;
static FAutoConsoleVariableRef CVarGarbageEliminationEnabled(
	TEXT("gc.GarbageEliminationEnabled"),
	GGarbageEliminationEnabled,
	TEXT("If true, objects marked as Garbage will be automatically nulled and destroyed by Garbage Collector."),
	ECVF_Default
);
bool UObjectBaseUtility::bGarbageEliminationEnabled = !!GGarbageEliminationEnabled;

void InitGarbageElimination()
{
	check(GConfig);
	bool bGarbageEliminationEnabled = UObjectBaseUtility::IsGarbageEliminationEnabled();
	if (GConfig->GetBool(TEXT("/Script/Engine.GarbageCollectionSettings"), TEXT("gc.PendingKillEnabled"), bGarbageEliminationEnabled, GEngineIni))
	{
		// The old gc.PendingKillEnabled will always take precedence over gc.GarbageEliminationEnabled because the new setting is always present in BaseEngine.ini
		// so until the project-level setting is renamed we assume the upgrade hasn't happened yet and the old setting should be used
		UE_LOGF(LogObj, Warning, "Deprecated ini setting [/Script/Engine.GarbageCollectionSettings] gc.PendingKillEnabled=%ls found. Please rename it to gc.GarbageEliminationEnabled",
			bGarbageEliminationEnabled ? TEXT("true") : TEXT("false"));
	}
	else
	{
		GConfig->GetBool(TEXT("/Script/Engine.GarbageCollectionSettings"), TEXT("gc.GarbageEliminationEnabled"), bGarbageEliminationEnabled, GEngineIni);
	}

	// Allow command-line overrides for easy testing without unique builds.
	if (FParse::Param(FCommandLine::Get(), TEXT("DisablePendingKill")) || FParse::Param(FCommandLine::Get(), TEXT("DisableGarbageElimination")))
	{
		bGarbageEliminationEnabled = false;
	}
	else if (FParse::Param(FCommandLine::Get(), TEXT("EnablePendingKill")) || FParse::Param(FCommandLine::Get(), TEXT("EnableGarbageElimination")))
	{
		bGarbageEliminationEnabled = true;
	}

	UObjectBaseUtility::SetGarbageEliminationEnabled(bGarbageEliminationEnabled);
}

void UObjectBaseUtility::SetGarbageEliminationEnabled(bool bEnabled)
{
#if UE_GARBAGE_ELIMINATION_ALLOWED
	// Try to sync even though we're not gonna use the console var
	UObjectBaseUtility::bGarbageEliminationEnabled = bEnabled;
	GGarbageEliminationEnabled = bEnabled;
#else
	if (bEnabled)
	{
		// Garbage elimination is not compatible when compact sets are default since it can modify key data which invalidates
		// the lookup table used to manage the set which can lead to crashes or infinite loops on subsequent operations.
		UE_LOGF(LogObj, Warning, "Request made to enable Garbage Elimination, however this feature is not currently compatible with compact sets. Leaving it disabled.");
	}

	UObjectBaseUtility::bGarbageEliminationEnabled = false;
	GGarbageEliminationEnabled = 0;
#endif
}