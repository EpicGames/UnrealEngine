// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectEditorOptionalSupport.h"
#include "CoreMinimal.h"
#include "UObject/UObjectThreadContext.h"
#include "Serialization/CustomVersion.h"
#include "UObject/UObjectSerializeContext.h"
#include "UObject/Class.h"


#if WITH_EDITORONLY_DATA

namespace UE::EditorOptional
{

void ConditionalUpgradeObject(FArchive& Ar, TNotNull<UObject*> SecondaryObject, const FGuid& VersionGuid, int Version)
{
	// attempt to fixup old content from before the class was split between runtime and EditorOptional classes
	// we use custom version to determine that the data is old, or always upgrade if a custom version is not
	// given to check against
	if (Ar.IsLoading() && (!VersionGuid.IsValid() || Ar.CustomVer(VersionGuid) < Version))
	{
		int64 ScriptStartOffset = -1;
		int64 ScriptEndOffset = -1;
		int64 CurrentOffset = Ar.Tell();
		UObject* MainObject = nullptr; 
		{
			// grab the threads serialization context, which we use to find where the current runtime object's
			// ScriptProperties begin, so we can load those same set of properties in the EO object
			// ScriptProperties are dynamically looked up, so the moved properties would have already been
			// skipped over before, and then we load the same set of properties again, this time loading the
			// ones that moved into the EO object
			// doing this in a scope since we are going to make a new context, just for extra safety
			FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
			bool bIsValid = SerializeContext != nullptr && SerializeContext->SerializedObjectScriptStartOffset != -1; 
			checkf(bIsValid, TEXT("Unable to retrieve a valid ScriptStartOffset from the active serialization context. Make sure to call UE::EditorOptional::ConditionalUpgradeObject() from an object's Serialize function _after_ calling Super::Serialize()."));
			
			ScriptStartOffset = SerializeContext->SerializedObjectScriptStartOffset;
			ScriptEndOffset = SerializeContext->SerializedObjectScriptEndOffset;
			MainObject = SerializeContext->SerializedObject;
		}
		
		// make a new context so when we leave the scope, state is cleaned up
		FScopedObjectSerializeContext RewindContext(SecondaryObject, Ar);
		
		// rewind and load
		Ar.Seek(ScriptStartOffset);
		SecondaryObject->SerializeScriptProperties(Ar);
		
		// verify it ended up where we expected
		checkf(Ar.Tell() == ScriptEndOffset, TEXT("Unexpected offset in file after loading ScriptProperties into the EditorOptional object"));
		
		Ar.Seek(CurrentOffset);
	}
}

void UpgradeObject(FArchive& Ar, TNotNull<UObject*> SecondaryObject)
{
	ConditionalUpgradeObject(Ar, SecondaryObject, FGuid(), -1);
}


UObject* CreateEditorOptionalObject(TNotNull<UObject*> MainObject, TNotNull<const UClass*> EditorOptionalClass, const TCHAR* OverrideName)
{
	checkf(EditorOptionalClass->HasAllClassFlags(CLASS_Optional), TEXT("The class (%s) used with UE::EditorOptional::CreateEditorOptionalObject() was not marked with UCLASS(Optional)"), *EditorOptionalClass->GetName());
	
	// get name and flags for the EO object
	const TCHAR* EditorOptionalName = OverrideName ? OverrideName : TEXT("EditorOptionalData");
	const EObjectFlags EditorOptionalFlags = MainObject->GetMaskedFlags(RF_PropagateToSubObjects);
	
	// create the EO object inside the main object
	return NewObject<UObject>(MainObject, EditorOptionalClass, EditorOptionalName, EditorOptionalFlags);
}

}
#endif

