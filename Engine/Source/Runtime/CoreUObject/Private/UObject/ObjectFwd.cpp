// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectFwd.h"
#include "UObject/Object.h"

namespace UE::CoreUObject::Private
{
UClass* GetClass(UObject* Obj)
{
	return Obj->GetClass();
}

FName GetFName(const UObject* Obj)
{
	return Obj->GetFName();
}

UObject* GetOuter(const UObject* Obj)
{
	return Obj->GetOuter();
}

bool IsIn(const UObject* Obj, const UObject* SomeOuter)
{
	return Obj->IsIn(SomeOuter);
}

FString GetPathName( const UObject* Obj, const UObject* StopOuter )
{
	return Obj->GetPathName(StopOuter);
}

UPackage* GetPackage(const UObject* Obj)
{
	return Obj->GetPackage();
}

FString GetFullName( const UObject* Obj, const UObject* StopOuter, EObjectFullNameFlags Flags)
{
	return Obj->GetFullName(StopOuter, Flags);
}

bool HasAnyFlags(const UObject* Obj, int32 FlagsToCheck)
{
	return Obj->HasAnyFlags(EObjectFlags(FlagsToCheck));
}
}
