// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectHandle.h"
#include "Async/TaskGraphInterfaces.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformAtomics.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeRWLock.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UObject/Class.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerLoadImportBehavior.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "UObject/ObjectPathId.h"
#include "UObject/PropertyBagRepository.h"

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

static inline FName GetNameOrNone(UObject* Object)
{
	return Object ? Object->GetFName() : NAME_None;
}

namespace UE::CoreUObject::Private
{

	bool operator==(const UE::CoreUObject::Private::FObjectHandleDataClassDescriptor& Lhs, const UE::CoreUObject::Private::FObjectHandleDataClassDescriptor& Rhs)
	{
		return (Lhs.PackageName == Rhs.PackageName) && (Lhs.ClassName == Rhs.ClassName);
	}

	class FPackageId
	{
		static constexpr uint32 InvalidId = ~uint32(0u);
		uint32 Id = InvalidId;

		inline explicit FPackageId(int32 InId) : Id(InId) {}

	public:
		FPackageId() = default;

		inline static FPackageId FromIndex(uint32 Index)
		{
			return FPackageId(Index);
		}

		inline bool IsValid() const
		{
			return Id != InvalidId;
		}

		inline uint32 ToIndex() const
		{
			check(Id != InvalidId);
			return Id;
		}

		inline bool operator==(FPackageId Other) const
		{
			return Id == Other.Id;
		}
	};
	
	struct FObjectId
	{
	public:
		FObjectId() = default;
		
		explicit FObjectId(uint32 Index)
			: PackedIndex(Index)
		{
		}

		static FObjectId FromIndex(uint32 Index)
		{
			return FObjectId(Index);
		}

		inline uint32 ToIndex() const
		{
			check(PackedIndex != 0);
			return PackedIndex - 1;
		}

		inline uint32 ToPackedIndex() const
		{
			return PackedIndex;
		}

		inline bool IsNone() const
		{
			return PackedIndex == 0;
		}

	private:
		uint32 PackedIndex = 0;
	};

	static_assert(sizeof(FObjectId) == sizeof(uint32), "FObjectId type must always compile to something equivalent to a uint32 size.");

	struct FObjectDescriptor
	{
		FObjectPathId ObjectPath;
		FObjectHandleDataClassDescriptor ClassDescriptor;
	};

	struct FObjectHandlePackageData
	{
		FMinimalName PackageName;
		TArray<FObjectDescriptor> ObjectDescriptors;
		FRWLock Lock;
	};

	static_assert(STRUCT_OFFSET(FObjectHandlePackageData, PackageName) == STRUCT_OFFSET(FObjectHandlePackageDebugData, PackageName), "FObjectHandlePackageData and FObjectHandlePackageDebugData must match in position of PackageNameField.");
	static_assert(STRUCT_OFFSET(FObjectHandlePackageData, ObjectDescriptors) == STRUCT_OFFSET(FObjectHandlePackageDebugData, ObjectDescriptors), "FObjectHandlePackageData and FObjectHandlePackageDebugData must match in position of ObjectPaths.");
	static_assert(sizeof(FObjectHandlePackageData) == sizeof(FObjectHandlePackageDebugData), "FObjectHandlePackageData and FObjectHandlePackageDebugData must match in size.");

	struct FObjectHandleIndex
	{
		FRWLock Lock; // @TODO: OBJPTR: Want to change this to a striped lock per object bucket to allow more concurrency when adding and looking up objects in a package
		TMap<FMinimalName, FPackageId> NameToPackageId;
		TArray<FObjectHandlePackageData> PackageData;

		TArray<FPackedObjectRef> ObjectIndexToPackedObjectRef;
	} GObjectHandleIndex;

	void InitObjectHandles(int32 MaxObjects)
	{
		GObjectHandleIndex.ObjectIndexToPackedObjectRef.SetNumZeroed(MaxObjects);
	}

	static FAutoConsoleCommand CmdPrintUnresolvedObjects(
		TEXT("LazyLoad.PrintUnresolvedObjects"),
		TEXT("Prints a list of all unresolved objects from the object handle index."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
			{
				FRWScopeLock GlobalLockScope(GObjectHandleIndex.Lock, SLT_ReadOnly);
				const int32 TotalObjectHandlePackages = GObjectHandleIndex.PackageData.Num();
				TArray<FName> LoadedPackages;
				TArray<FName> UnloadedPackages;
				for (int32 PackageIndex = 0; PackageIndex < TotalObjectHandlePackages; PackageIndex++)
				{
					FName PackageName;
					{
						FObjectHandlePackageData& PackageData = GObjectHandleIndex.PackageData[PackageIndex];
						FRWScopeLock LocalLockScope(PackageData.Lock, SLT_ReadOnly);
						PackageName = MinimalNameToName(PackageData.PackageName);
					}

					UPackage* Package = FindObjectFast<UPackage>(nullptr, PackageName);
					if (Package)
					{
						LoadedPackages.Add(PackageName);
					}
					else
					{
						UnloadedPackages.Add(PackageName);
					}
				}

				UnloadedPackages.Sort(FNameLexicalLess());
				for (FName PackageName : UnloadedPackages)
				{
					UE_LOGF(LogUObjectGlobals, Log, "Unloaded %ls", *PackageName.ToString());
				}

				LoadedPackages.Sort(FNameLexicalLess());
				for (FName PackageName : LoadedPackages)
				{
					UE_LOGF(LogUObjectGlobals, Log, "Loaded %ls", *PackageName.ToString());
				}

				UE_LOGF(LogUObjectGlobals, Log, "Unloaded Packages (%d out of %d)", UnloadedPackages.Num(), LoadedPackages.Num() + UnloadedPackages.Num());
			})
	);

	static inline FPackedObjectRef Pack(FPackageId PackageId, FObjectId ObjectId, bool bHasPlaceholderType = false)
	{
		checkf(PackageId.ToIndex() <= PackageIdMask, TEXT("Package count exceeded the space permitted within packed object references.  This implies over %d packages are in use."), PackageIdMask);
		return FPackedObjectRef{ static_cast<UPTRINT>(PackageId.ToIndex()) << PackageIdShift |
				static_cast<UPTRINT>(ObjectId.ToPackedIndex() << ObjectIdShift) |
				static_cast<UPTRINT>(bHasPlaceholderType << TypeIdShift) | 1 };
	}

	static inline void Unpack(FPackedObjectRef PackedObjectRef, FPackageId& OutPackageId, FObjectId& OutObjectId)
	{
		checkf((PackedObjectRef.EncodedRef & 1) == 1, TEXT("Packed object reference is malformed."));
		OutObjectId = FObjectId::FromIndex(static_cast<uint32>(PackedObjectRef.EncodedRef >> ObjectIdShift));
		OutPackageId = FPackageId::FromIndex(static_cast<uint32>((PackedObjectRef.EncodedRef >> PackageIdShift) & PackageIdMask));
	}

	static void MakeReferenceIds(FName PackageName, FName ClassPackageName, FName ClassName, FObjectPathId ObjectPath, FPackageId& OutPackageId, FObjectId& OutObjectId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::CoreUObject::Private::MakeReferenceIds);
		FMinimalName MinimalName = NameToMinimalName(PackageName);

		//Biases for read-only locking by default at the expense of having to do a second map search if a write is necessary
		//Doing one search with either a read or locked write seems impossible with the TMap interface at the moment.
		FRWScopeLock GlobalLockScope(GObjectHandleIndex.Lock, SLT_ReadOnly);
		FPackageId* FoundPackageId = GObjectHandleIndex.NameToPackageId.Find(MinimalName);
		FObjectHandlePackageData* PackageData = nullptr;
		bool PackageCreated = false;
		if (!FoundPackageId)
		{
			GlobalLockScope.ReleaseReadOnlyLockAndAcquireWriteLock_USE_WITH_CAUTION();
			//Has to be FindOrAdd, as the NameToPackageId may have changed between relinquishing the read lock
			//and acquiring the write lock.
			FPackageId NextId = FPackageId::FromIndex(GObjectHandleIndex.PackageData.Num());
			FoundPackageId = &GObjectHandleIndex.NameToPackageId.FindOrAdd(MinimalName, NextId);
			if (*FoundPackageId == NextId)
			{
				PackageData = &GObjectHandleIndex.PackageData.AddDefaulted_GetRef();
				PackageData->PackageName = NameToMinimalName(PackageName);
				GCoreObjectHandlePackageDebug = reinterpret_cast<UE::CoreUObject::Private::FObjectHandlePackageDebugData*>(GObjectHandleIndex.PackageData.GetData());
				PackageCreated = true;
			}
			else
			{
				PackageData = &GObjectHandleIndex.PackageData[FoundPackageId->ToIndex()];
			}
			//Can't reasonably switch back to a read lock here because the downgrade would have a window where 
			//the global map could be modified and invalidate the pointer we're holding to an element in it.
		}
		else
		{
			PackageData = &GObjectHandleIndex.PackageData[FoundPackageId->ToIndex()];
		}
		OutPackageId = *FoundPackageId;

		//optimization for creating a new package.
		//since the package is new, GlobalLockScope is write locked and nothing can be added or read
		if (PackageCreated)
		{
			if (!ObjectPath.IsNone())
			{
				FObjectHandleDataClassDescriptor DataClassDesc;
				if (!ClassName.IsNone() && !ClassPackageName.IsNone())
				{
					DataClassDesc.PackageName = NameToMinimalName(ClassPackageName);
					DataClassDesc.ClassName = NameToMinimalName(ClassName);
				}
			
				FObjectDescriptor ObjectDesc = { ObjectPath, DataClassDesc };
				OutObjectId = FObjectId::FromIndex(PackageData->ObjectDescriptors.Add(ObjectDesc) + 1);
			}
			return;
		}
		else if (ObjectPath.IsNone())
		{
			//don't store anything for packages
			return;
		}

		int32 PathIndex;
		{
			TReadScopeLock LocalLockScope(PackageData->Lock);
			PathIndex = PackageData->ObjectDescriptors.IndexOfByPredicate([&ObjectPath](const FObjectDescriptor& Desc)
			{
				return Desc.ObjectPath == ObjectPath;

			}); //linear search is fine as typically there is only one.
		}

		if (PathIndex == INDEX_NONE)
		{
			//ObjectPaths could have been modified when the read lock was released and the write
			//lock was acquired, so we must check and see if the ObjectPath was added in that window.
			TWriteScopeLock LocalLockScope(PackageData->Lock);
			PathIndex = PackageData->ObjectDescriptors.IndexOfByPredicate([&ObjectPath](const FObjectDescriptor& Desc)
			{
				return Desc.ObjectPath == ObjectPath;

			}); //linear search is fine as typically there is only one.

			if (PathIndex == INDEX_NONE)
			{
				FObjectHandleDataClassDescriptor DataClassDesc;
				if (!ClassName.IsNone() && !ClassPackageName.IsNone())
				{
					DataClassDesc.PackageName = NameToMinimalName(ClassPackageName);
					DataClassDesc.ClassName = NameToMinimalName(ClassName);
				}

				FObjectDescriptor ObjDesc = { ObjectPath, DataClassDesc };
				PathIndex = PackageData->ObjectDescriptors.Add(ObjDesc);
			}
		}

		OutObjectId = FObjectId::FromIndex(PathIndex + 1);
	}

	static inline FPackedObjectRef MakePackedObjectRef(FName PackageName, FName ClassPackageName, FName ClassName, FObjectPathId ObjectPath, bool bHasPlaceholderType = false)
	{
		FPackageId PackageId;
		FObjectId ObjectId;
		MakeReferenceIds(PackageName, ClassPackageName, ClassName, ObjectPath, PackageId, ObjectId);
		return Pack(PackageId, ObjectId, bHasPlaceholderType);
	}

	static void GetObjectDataFromId(FPackageId PackageId, FObjectId ObjectId, FMinimalName& OutPackageName, FObjectPathId& OutPathId, FMinimalName& OutClassPackageName, FMinimalName& OutClassName)
	{
		if (!PackageId.IsValid())
		{
			return;
		}

		FRWScopeLock GlobalLockScope(GObjectHandleIndex.Lock, SLT_ReadOnly);
		const uint32 PackageIndex = PackageId.ToIndex();

		if (PackageIndex >= static_cast<uint32>(GObjectHandleIndex.PackageData.Num()))
		{
			//checkf(false, TEXT("FObjectHandle: PackageIndex invalid.  This ObjectHandle is from malformed data."));
			return;
		}
		FObjectHandlePackageData& FoundPackageData = GObjectHandleIndex.PackageData[PackageIndex];
		FRWScopeLock LocalLockScope(FoundPackageData.Lock, SLT_ReadOnly);
		OutPackageName = FoundPackageData.PackageName;

		if (!ObjectId.IsNone())
		{
			int32 ObjectIndex = ObjectId.ToIndex();
			if (ObjectIndex >= FoundPackageData.ObjectDescriptors.Num())
			{
				//must reset to empty as the object id is invalid
				OutPackageName = FMinimalName();
				return;
			}
			const auto& ObjDesc = FoundPackageData.ObjectDescriptors[ObjectIndex];
			OutPathId = ObjDesc.ObjectPath;
			OutClassPackageName = ObjDesc.ClassDescriptor.PackageName;
			OutClassName = ObjDesc.ClassDescriptor.ClassName;
		}
		else
		{
			OutPathId = FObjectPathId();
			OutClassPackageName = FMinimalName(UPackage::StaticClass()->GetPackage()->GetFName());
			OutClassName = FMinimalName(UPackage::StaticClass()->GetFName());
		}
	}

	FPackedObjectRef GetOuter(FPackedObjectRef PackedRef)
	{
		if (PackedRef.IsNull())
		{
			return PackedRef;
		}
		FPackageId PackageId;
		FObjectId ObjectId;
		Unpack(PackedRef, PackageId, ObjectId);

		FMinimalName PackageName;
		FObjectPathId PathId(FObjectPathId::Invalid);
		FMinimalName ClassPackageName;
		FMinimalName ClassName;
		GetObjectDataFromId(PackageId, ObjectId, PackageName, PathId, ClassPackageName, ClassName);

		if (PathId.IsNone())
		{
			return FPackedObjectRef{ 0 }; //return null for packages as they have no outer
		}
		else if (PathId.IsSimple())
		{
			FObjectId PackageObjectId;
			return Pack(PackageId, PackageObjectId);
		}
		else if (PathId.IsWeakObj())
		{
			return MakePackedObjectRef(PathId.GetWeakObjPtr().Get()->GetOuter());
		}
		else
		{
			TStringBuilder<256> StringBuilder;
			FObjectPathId::ResolvedNameContainerType Names;
			PathId.Resolve(Names);
			for (int32 i = 0; i < Names.Num() - 1; ++i)
			{
				const FName& Name = Names[i];
				if (i != 0)
				{
					StringBuilder << '.'; //separate subobjects by periods a colon is unnecessary to produce a correct FObjectPathId
				}
				Name.AppendString(StringBuilder);
			}
			FObjectPathId OuterPathId(StringBuilder);
			FPackedObjectRef OuterPackedRef = MakePackedObjectRef(FName(PackageName), FName(), FName(), OuterPathId);

			return OuterPackedRef;
		}
	}
	
	FPackedObjectRef GetPackage(FPackedObjectRef PackedRef)
	{
		if (PackedRef.IsNull())
		{
			return PackedRef;
		}
		FPackageId PackageId;
		FObjectId ObjectId;
		Unpack(PackedRef, PackageId, ObjectId);

		//object id's for packages are always 0
		FPackedObjectRef PackagePackedRef = Pack(PackageId, FObjectId());
		return PackagePackedRef;
	}

	void FreeObjectHandle(const UObjectBase* Object)
	{
		int32 ObjectIndex = GUObjectArray.ObjectToIndex(Object);
		FPackedObjectRef& PackedObjectRef = GObjectHandleIndex.ObjectIndexToPackedObjectRef[ObjectIndex];
		PackedObjectRef.EncodedRef = 0;
	}

	void UpdateRenamedObject(const UObject* Object, FName NewName, UObject* NewOuter)
	{
		if (!UE::LinkerLoad::IsImportLazyLoadEnabled())
		{
			return;
		}
		int32 ObjectIndex = GUObjectArray.ObjectToIndex(Object);
		FPackedObjectRef PackedObjectRef = GObjectHandleIndex.ObjectIndexToPackedObjectRef[ObjectIndex];
		if (PackedObjectRef.EncodedRef == 0)
		{
			return;
		}

		check(NewName != NAME_None);

		FMinimalName MinimalName = NameToMinimalName(Object->GetFName());
		FObjectId ObjectId;
		FPackageId PackageId;
		Unpack(PackedObjectRef, PackageId, ObjectId);

		FRWScopeLock GlobalLockScope(GObjectHandleIndex.Lock, SLT_Write);
		FObjectHandlePackageData& PackageData = GObjectHandleIndex.PackageData[PackageId.ToIndex()];
		FRWScopeLock PackageLockScope(PackageData.Lock, SLT_ReadOnly);
		if (Object->GetClass() == UPackage::StaticClass())
		{
			//update the package name at existing index. existing object handles will be correct when unpacked
			//add in the new name pointing the existing index. new object handles will resolve to the existing index
			check(NewName != NAME_None);
			FMinimalName MinNewName = NameToMinimalName(NewName);
			PackageData.PackageName = MinNewName;
			GObjectHandleIndex.NameToPackageId.Add(MinNewName, PackageId);
			GObjectHandleIndex.NameToPackageId.Remove(MinimalName);
			return;
		}

		//must make a copy to avoid changing data while we are comparing
		FObjectPathId OldObjectPath = PackageData.ObjectDescriptors[ObjectId.ToIndex()].ObjectPath;
		if (OldObjectPath.IsWeakObj())
		{
			return;
		}

		const FMinimalName* OldObjectPathtData = nullptr;
		int32 OldObjectPathSize = 0;
		FMinimalName OldName;

		if (OldObjectPath.IsSimple())
		{
			OldName = OldObjectPath.GetSimpleName();
			OldObjectPathtData = &OldName;
			OldObjectPathSize = 1;
		}
		else
		{
			//this isn't thread safe. if the FStoredObjectPath moves this will now point to garbage
			const UE::CoreUObject::Private::FStoredObjectPath& StoredPath = OldObjectPath.GetStoredPath();
			OldObjectPathtData = StoredPath.GetData();
			OldObjectPathSize = StoredPath.NumElements;
		}

		//update all paths that start with OldObjectPath. including itself
		for (int32 PathIndex = 0; PathIndex < PackageData.ObjectDescriptors.Num(); ++PathIndex)
		{
			auto& CurrentObjectPath = PackageData.ObjectDescriptors[PathIndex].ObjectPath;
			if (CurrentObjectPath.IsWeakObj())
			{
				continue;
			}
			else if (CurrentObjectPath == OldObjectPath)
			{
				CurrentObjectPath.MakeWeakObjPtr(*Object);
				continue;
			}
			else if (CurrentObjectPath.IsSimple())
			{
				//nothing to do for simple paths. they won't be an inner of the object
				continue;
			}
			const UE::CoreUObject::Private::FStoredObjectPath& ThisPath = CurrentObjectPath.GetStoredPath();
			if (OldObjectPathSize < ThisPath.NumElements && CompareItems(ThisPath.GetData(), OldObjectPathtData, OldObjectPathSize))
			{
				FObjectPathId::ResolvedNameContainerType ResolvedNames;
				CurrentObjectPath.Resolve(ResolvedNames);

				//resolve all object along the path and convert the object paths to weak objects
				const UObject* CurrentObject = Object->GetPackage();
				for (int32 ObjectPathIndex = 0; ObjectPathIndex < ResolvedNames.Num() && CurrentObject; ++ObjectPathIndex)
				{
					CurrentObject = StaticFindObjectFastInternal(nullptr, CurrentObject, ResolvedNames[ObjectPathIndex]);
				}

				// not sure how this could happen, null out the object path
				if (!CurrentObject)
				{
					CurrentObjectPath.Reset();
				}
				else
				{
					CurrentObjectPath.MakeWeakObjPtr(*CurrentObject);
				}
			}
		}

	}

	FPackedObjectRef MakePackedObjectRef(const FObjectRef& ObjectRef)
	{
		if (ObjectRef.IsNull())
		{
			return FPackedObjectRef{ 0 };
		}

		return MakePackedObjectRef(ObjectRef.PackageName, ObjectRef.ClassPackageName, ObjectRef.ClassName, ObjectRef.GetObjectPath());
	}



	FObjectRef MakeObjectRef(FPackedObjectRef PackedObjectRef)
	{
		if (PackedObjectRef.IsNull())
		{
			return FObjectRef(NAME_None, NAME_None, NAME_None, UE::CoreUObject::Private::FObjectPathId());
		}

		TRACE_CPUPROFILER_EVENT_SCOPE(UE::CoreUObject::Private::MakeObjectRef);
		FObjectId ObjectId;
		FPackageId PackageId;
		Unpack(PackedObjectRef, PackageId, ObjectId);

		// Default reference must be invalid if GetObjectDataFromId doesn't populate the reference fields.
		FMinimalName PackageName;
		FObjectPathId PathId(FObjectPathId::Invalid);
		FMinimalName ClassPackageName;
		FMinimalName ClassName;
		GetObjectDataFromId(PackageId, ObjectId, PackageName, PathId, ClassPackageName, ClassName);
		if (PathId.IsWeakObj())
		{
			return FObjectRef(PathId.GetWeakObjPtr().Get());
		}
		return FObjectRef(MinimalNameToName(PackageName), MinimalNameToName(ClassPackageName), MinimalNameToName(ClassName), PathId);
	}


	FPackedObjectRef MakePackedObjectRef(const UObject* Object)
	{
		if (!Object)
		{
			return FPackedObjectRef{ 0 };
		}

		const bool bHasPlaceholderType = UE::FPropertyBagRepository::IsPropertyBagPlaceholderObject(Object);
		if (!bHasPlaceholderType && UE::LinkerLoad::FindLoadBehavior(*Object->GetClass()) == UE::LinkerLoad::EImportBehavior::Eager)
		{
			return FPackedObjectRef{ 0 };
		}

		int32 ObjectIndex = GUObjectArray.ObjectToIndex(Object);
		FPackedObjectRef& PackedObjectRef = UE::CoreUObject::Private::GObjectHandleIndex.ObjectIndexToPackedObjectRef[ObjectIndex];
		if (PackedObjectRef.EncodedRef != 0)
		{
			return PackedObjectRef;
		}

		FName PackageName = GetNameOrNone(Object->GetOutermost());

		UObject* Class = Object->GetClass();
		FName ClassPackageName = GetNameOrNone(Class->GetOutermost());
		PackedObjectRef = UE::CoreUObject::Private::MakePackedObjectRef(PackageName, ClassPackageName, GetNameOrNone(Class), FObjectPathId(Object), bHasPlaceholderType);
		return PackedObjectRef;
	}

	FPackedObjectRef FindExistingPackedObjectRef(const UObject* Object)
	{
		if (!GUObjectArray.IsValidIndex(Object))
		{
			return FPackedObjectRef {0};
		}

		const int32 ObjectIndex = GUObjectArray.ObjectToIndex(Object);
		return UE::CoreUObject::Private::GObjectHandleIndex.ObjectIndexToPackedObjectRef[ObjectIndex];
	}
}


#endif