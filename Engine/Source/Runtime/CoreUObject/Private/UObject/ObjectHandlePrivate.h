// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Object.h"
#include "UObject/UObjectArray.h"

namespace UE::CoreUObject::Private
{
	struct FObjectHandleUtils
	{
		FORCEINLINE static const ObjectPtr_Private::TNonAccessTrackedObjectPtr<UObject>& GetNonAccessTrackedOuter(UObjectBase* Object)
		{
			return Object->OuterPrivate;
		}

		FORCEINLINE static UObject* GetNonAccessTrackedOuterNoResolve(const UObjectBase* Object)
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			return Object->OuterPrivate.GetNoResolve();
#else
			return Object->OuterPrivate;
#endif 
		}

#if UE_WITH_REMOTE_OBJECT_HANDLE
		static void ChangeRemoteId(UObjectBase* Object, FRemoteObjectId Id);
#endif

		FORCEINLINE static FRemoteObjectId GetRemoteId(const UObjectBase* Object)
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			if (Object)
			{
				if (UNLIKELY(Object->InternalIndex < 0))
				{
					return Object->GetPendingRegistrantRemoteId();
				}
				else
				{
					return GUObjectArray.GetRemoteId(Object->InternalIndex);
				}
			}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
			return FRemoteObjectId();
		}
	};
}

