// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectHandleTracking.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeRWLock.h"
#include "AutoRTFM.h"

#if UE_WITH_OBJECT_HANDLE_TRACKING

// Simple lock class used to avoid needing to call into ntdll, which is very high overhead on VM cook machines.  Saves
// 22% of overall cook time for a large project with shader invalidation on the VM.  This class can support nesting read
// locks but not write locks, but there's no use case where write locks could nest for this code, so it shouldn't be a
// problem.  Eventually may be replaced with UE::FSharedMutex when that comes online.
struct FLightweightRWLock
{
	static const int32 Unlocked = 0;
	static const int32 LockedOffset = 0x7fffffff;		// Subtracted for write lock, producing a large negative number

	std::atomic_int32_t Mutex = 0;
};

class FLightweightReadScopeLock
{
public:
	FLightweightReadScopeLock(FLightweightRWLock& InLock)
		: Lock(InLock)
	{
		int32 LockValue = Lock.Mutex.fetch_add(1, std::memory_order_acquire);

		// If we attempt a read lock while there is a write lock, LockValue will be negative -- spin until it goes positive.
		// The write lock class sets the mutex value to a large negative value, and then adds that value back in for the
		// unlock, so any queued read lock increments will already be baked into the mutex value when it goes positive again.
		// Any threads with pending read locks will then be allowed to finish before another write lock can be obtained from
		// the FLightweightRWLock::Unlocked = 0 state.
		// 
		// This approach avoids the need for the initial modification of the Mutex to do a compare/exchange, making it
		// "wait free" for multiple readers, versus "lock free", where a thread may need to retry the compare/exchange if
		// another thread updated the value.
		while (LockValue < 0)
		{
			FPlatformProcess::Yield();
			LockValue = Lock.Mutex.load(std::memory_order_acquire);
		}
	}

	~FLightweightReadScopeLock()
	{
		Lock.Mutex.fetch_sub(1, std::memory_order_release);
	}

private:
	FLightweightRWLock& Lock;
};

class FLightweightWriteScopeLock
{
public:
	FLightweightWriteScopeLock(FLightweightRWLock& InLock)
		: Lock(InLock)
	{
		// Spin until we successfully transition from Unlocked to -LockedOffset state
		int32 ExpectedUnlocked = FLightweightRWLock::Unlocked;
		while (!InLock.Mutex.compare_exchange_weak(ExpectedUnlocked, -FLightweightRWLock::LockedOffset, std::memory_order_acquire))
		{
			FPlatformProcess::Yield();
			ExpectedUnlocked = FLightweightRWLock::Unlocked;
		}
	}

	~FLightweightWriteScopeLock()
	{
		// See comment above in FLightweightRWLockReadScope constructor -- we add LockedOffset rather than setting back to Unlocked,
		// as read lock attempts may have incremented the negative value mutex while the write lock was held.
		Lock.Mutex.fetch_add(FLightweightRWLock::LockedOffset, std::memory_order_release);
	}

private:
	FLightweightRWLock& Lock;
};

namespace UE::CoreUObject
{
	namespace Private
	{
		COREUOBJECT_API std::atomic<int32> HandleReadCallbackQuantity = 0;

		struct FObjectHandleCallbacks
		{
			static FObjectHandleCallbacks& Get()
			{
				static FObjectHandleCallbacks Callbacks;
				return Callbacks;
			}

			void OnHandleRead(TArrayView<const UObject* const> Objects)
			{
				UE_AUTORTFM_OPEN
				{
					FLightweightReadScopeLock _(HandleLock);
					for (auto&& Pair : ReadHandleCallbacks)
					{
						Pair.Value(Objects);
					}
				};
			}

			void OnHandleRead(const UObject* Object)
			{
				UE_AUTORTFM_OPEN
				{
					FLightweightReadScopeLock _(HandleLock);
					TArrayView<const UObject* const> Objects(&Object, 1);
					for (auto&& Pair : ReadHandleCallbacks)
					{
						Pair.Value(Objects);
					}
				};
			}

			void OnClassReferenceResolved(const FObjectRef& ObjectRef, UPackage* Package, UClass* Class)
			{
				UE_AUTORTFM_OPEN
				{
					FLightweightReadScopeLock _(HandleLock);
					for (auto&& Pair : ClassResolvedCallbacks)
					{
						Pair.Value(ObjectRef, Package, Class);
					}
				};
			}

			void OnReferenceResolved(const FObjectRef& ObjectRef, UPackage* Package, UObject* Object)
			{
				UE_AUTORTFM_OPEN
				{
					FLightweightReadScopeLock _(HandleLock);
					for (auto&& Pair : HandleResolvedCallbacks)
					{
						Pair.Value(ObjectRef, Package, Object);
					}
				};
			}

			void OnReferenceLoaded(const FObjectRef& ObjectRef, UPackage* Package, UObject* Object)
			{
				UE_AUTORTFM_OPEN
				{
					FLightweightReadScopeLock _(HandleLock);
					for (auto&& Pair : HandleLoadedCallbacks)
					{
						Pair.Value(ObjectRef, Package, Object);
					}
				};
			}

			FObjectHandleTrackingCallbackId AddObjectHandleReadCallback(FObjectHandleReadFunc Func)
			{
				FObjectHandleTrackingCallbackId Result;
				UE_AUTORTFM_OPEN
				{
					HandleReadCallbackQuantity.fetch_add(1, std::memory_order_relaxed);
					FLightweightWriteScopeLock _(HandleLock);
					NextHandleId++;
					ReadHandleCallbacks.Add({ NextHandleId, Func });
					Result = FObjectHandleTrackingCallbackId{ NextHandleId };
				};
				return Result;
			}

			void RemoveObjectHandleReadCallback(FObjectHandleTrackingCallbackId Handle)
			{
				UE_AUTORTFM_OPEN
				{
					HandleReadCallbackQuantity.fetch_sub(1, std::memory_order_relaxed);
					FLightweightWriteScopeLock _(HandleLock);
					for (int32 i = ReadHandleCallbacks.Num() - 1; i >= 0; --i)
					{
						auto& Pair = ReadHandleCallbacks[i];
						if (Pair.Key == Handle.Id)
						{
							ReadHandleCallbacks.RemoveAt(i);
						}
					}
				};
			}

			FObjectHandleTrackingCallbackId AddObjectHandleClassResolvedCallback(FObjectHandleClassResolvedFunc Func)
			{
				FObjectHandleTrackingCallbackId Result;
				UE_AUTORTFM_OPEN
				{
					FLightweightWriteScopeLock _(HandleLock);
					NextHandleId++;
					ClassResolvedCallbacks.Add({ NextHandleId, Func });
					Result = FObjectHandleTrackingCallbackId{ NextHandleId };
				};
				return Result;
			}

			void RemoveObjectHandleClassResolvedCallback(FObjectHandleTrackingCallbackId Handle)
			{
				UE_AUTORTFM_OPEN
				{
					FLightweightWriteScopeLock _(HandleLock);
					for (int32 i = ClassResolvedCallbacks.Num() - 1; i >= 0; --i)
					{
						auto& Pair = ClassResolvedCallbacks[i];
						if (Pair.Key == Handle.Id)
						{
							ClassResolvedCallbacks.RemoveAt(i);
						}
					}
				};
			}

			FObjectHandleTrackingCallbackId AddObjectHandleReferenceResolvedCallback(FObjectHandleReferenceResolvedFunc Func)
			{
				FObjectHandleTrackingCallbackId Result;
				UE_AUTORTFM_OPEN
				{
					FLightweightWriteScopeLock _(HandleLock);
					NextHandleId++;
					HandleResolvedCallbacks.Add({ NextHandleId, Func });
					Result = FObjectHandleTrackingCallbackId{ NextHandleId };
				};
				return Result;
			}

			void RemoveObjectHandleReferenceResolvedCallback(FObjectHandleTrackingCallbackId Handle)
			{
				UE_AUTORTFM_OPEN
				{
					FLightweightWriteScopeLock _(HandleLock);
					for (int32 i = HandleResolvedCallbacks.Num() - 1; i >= 0; --i)
					{
						auto& Pair = HandleResolvedCallbacks[i];
						if (Pair.Key == Handle.Id)
						{
							HandleResolvedCallbacks.RemoveAt(i);
						}
					}
				};
			}

			FObjectHandleTrackingCallbackId AddObjectHandleReferenceLoadedCallback(FObjectHandleReferenceLoadedFunc Func)
			{
				FObjectHandleTrackingCallbackId Result;
				UE_AUTORTFM_OPEN
				{
					FLightweightWriteScopeLock _(HandleLock);
					NextHandleId++;
					HandleLoadedCallbacks.Add({ NextHandleId, Func });
					Result = FObjectHandleTrackingCallbackId{ NextHandleId };
				};
				return Result;
			}

			void RemoveObjectHandleReferenceLoadedCallback(FObjectHandleTrackingCallbackId Handle)
			{
				UE_AUTORTFM_OPEN
				{
					FLightweightWriteScopeLock _(HandleLock);
					for (int32 i = HandleLoadedCallbacks.Num() - 1; i >= 0; --i)
					{
						auto& Pair = HandleLoadedCallbacks[i];
						if (Pair.Key == Handle.Id)
						{
							HandleLoadedCallbacks.RemoveAt(i);
						}
					}
				};
			}

		private:
			int32 NextHandleId = 0;
			TArray<TTuple<int32, FObjectHandleReadFunc>> ReadHandleCallbacks;
			TArray<TTuple<int32, FObjectHandleClassResolvedFunc>> ClassResolvedCallbacks;
			TArray<TTuple<int32, FObjectHandleReferenceResolvedFunc>> HandleResolvedCallbacks;
			TArray<TTuple<int32, FObjectHandleReferenceLoadedFunc>> HandleLoadedCallbacks;

			//using a single lock as currently there is not any contention. could add separate locks for each list. 
			FLightweightRWLock HandleLock;
		};

		void OnHandleReadInternal(TArrayView<const UObject* const> Objects)
		{
			FObjectHandleCallbacks::Get().OnHandleRead(Objects);
		}

		void OnHandleReadInternal(const UObject* Object)
		{
			FObjectHandleCallbacks::Get().OnHandleRead(Object);
		}

		void OnClassReferenceResolved(const FObjectRef& ObjectRef, UPackage* Package, UClass* Class)
		{
			FObjectHandleCallbacks::Get().OnClassReferenceResolved(ObjectRef, Package, Class);
		}

		void OnReferenceResolved(const FObjectRef& ObjectRef, UPackage* Package, UObject* Object)
		{
			FObjectHandleCallbacks::Get().OnReferenceResolved(ObjectRef, Package, Object);
		}

		void OnReferenceLoaded(const FObjectRef& ObjectRef, UPackage* Package, UObject* Object)
		{
			FObjectHandleCallbacks::Get().OnReferenceLoaded(ObjectRef, Package, Object);
		}

	}

	FObjectHandleTrackingCallbackId AddObjectHandleReadCallback(FObjectHandleReadFunc Func)
	{
		using namespace UE::CoreUObject::Private;
		return FObjectHandleCallbacks::Get().AddObjectHandleReadCallback(Func);
	}

	void RemoveObjectHandleReadCallback(FObjectHandleTrackingCallbackId Handle)
	{
		using namespace UE::CoreUObject::Private;
		FObjectHandleCallbacks::Get().RemoveObjectHandleReadCallback(Handle);
	}

	FObjectHandleTrackingCallbackId AddObjectHandleClassResolvedCallback(FObjectHandleClassResolvedFunc Func)
	{
		using namespace UE::CoreUObject::Private;
		return FObjectHandleCallbacks::Get().AddObjectHandleClassResolvedCallback(Func);
	}

	void RemoveObjectHandleClassResolvedCallback(FObjectHandleTrackingCallbackId Handle)
	{
		using namespace UE::CoreUObject::Private;
		FObjectHandleCallbacks::Get().RemoveObjectHandleClassResolvedCallback(Handle);
	}

	FObjectHandleTrackingCallbackId AddObjectHandleReferenceResolvedCallback(FObjectHandleReferenceResolvedFunc Func)
	{
		using namespace UE::CoreUObject::Private;
		return FObjectHandleCallbacks::Get().AddObjectHandleReferenceResolvedCallback(Func);
	}

	void RemoveObjectHandleReferenceResolvedCallback(FObjectHandleTrackingCallbackId Handle)
	{
		using namespace UE::CoreUObject::Private;
		FObjectHandleCallbacks::Get().RemoveObjectHandleReferenceResolvedCallback(Handle);
	}

	FObjectHandleTrackingCallbackId AddObjectHandleReferenceLoadedCallback(FObjectHandleReferenceLoadedFunc Func)
	{
		using namespace UE::CoreUObject::Private;
		return FObjectHandleCallbacks::Get().AddObjectHandleReferenceLoadedCallback(Func);
	}

	void RemoveObjectHandleReferenceLoadedCallback(FObjectHandleTrackingCallbackId Handle)
	{
		using namespace UE::CoreUObject::Private;
		FObjectHandleCallbacks::Get().RemoveObjectHandleReferenceLoadedCallback(Handle);
	}
}
#endif // UE_WITH_OBJECT_HANDLE_TRACKING
