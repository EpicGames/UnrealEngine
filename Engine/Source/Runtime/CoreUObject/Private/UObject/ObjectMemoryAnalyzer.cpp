// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectMemoryAnalyzer.h"
#include "UObject/Object.h"
#include "UObject/UObjectIterator.h"
#include "Serialization/ArchiveCountMem.h"

FObjectMemoryAnalyzer::FObjectMemoryAnalyzer(uint32 Flags)
	: BaseClass(NULL)
	, AnalyzeFlags(Flags)
{
}

FObjectMemoryAnalyzer::FObjectMemoryAnalyzer(class UClass* InBaseClass, uint32 Flags)
	: BaseClass(InBaseClass)
	, AnalyzeFlags(Flags)
{
	AnalyzeObjects(InBaseClass);
}

FObjectMemoryAnalyzer::FObjectMemoryAnalyzer(class UObject* InObject, uint32 Flags)
	: BaseClass(NULL)
	, AnalyzeFlags(Flags)
{
	AnalyzeObjects(ObjectList);
}

FObjectMemoryAnalyzer::FObjectMemoryAnalyzer(const TArray<class UObject*>& InObjectList, uint32 Flags)
	: BaseClass(NULL)
	, AnalyzeFlags(Flags)
{
	AnalyzeObjects(ObjectList);
}

void FObjectMemoryAnalyzer::AnalyzeObjects(const TArray<class UObject*>& InObjectList)
{
	for (int32 i=0; i < InObjectList.Num(); ++i)
	{
		AnalyzeObject(InObjectList[i]);
	}
}

void FObjectMemoryAnalyzer::AnalyzeObject(UObject* Object)
{
	if (ObjectList.Contains(Object) || 
		(!(AnalyzeFlags&EAnalyzeFlags::IncludeDefaultObjects) && Object->IsTemplate()))
	{
		return;
	}

	ObjectList.Add(Object);

	// Process root object
	{
		FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);

		if ( Object->HasAllFlags(RF_Standalone) )
		{
			Annotation.Flags |= FObjectMemoryUsage::EObjFlags::IsRoot;
		}
		ProcessSubObjRecursive(Object, Object);
	}

	{
		FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);

		SIZE_T InclusiveSize = CalculateSizeRecursive(Object);
		Annotation.InclusiveMemoryUsage = InclusiveSize;
	}
	
}

void FObjectMemoryAnalyzer::AnalyzeObjects( UClass* InBaseClass )
{
	if (InBaseClass == NULL)
	{
		InBaseClass = UObject::StaticClass();
	}

	uint32 ExclusionFlags = (AnalyzeFlags&EAnalyzeFlags::IncludeDefaultObjects)==0 ? (RF_ClassDefaultObject|RF_ArchetypeObject) : 0;
	// Determine root objects
	for( FThreadSafeObjectIterator It(InBaseClass, false, (EObjectFlags)ExclusionFlags); It; ++It )
	{
		UObject* Object	= *It;

		if (!(AnalyzeFlags&EAnalyzeFlags::IncludeDefaultObjects) && Object->IsTemplate()) { continue; };

		FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);

		if ( Object->HasAllFlags(RF_Standalone) )
		{
			Annotation.Flags |= FObjectMemoryUsage::EObjFlags::IsRoot;
		}
		ProcessSubObjRecursive(Object, Object);
	}

	// mark 'loose' objets as root objects as well
	for( FThreadSafeObjectIterator It(InBaseClass, false, (EObjectFlags)ExclusionFlags); It; ++It )
	{
		UObject* Object	= *It;

		if (!(AnalyzeFlags&EAnalyzeFlags::IncludeDefaultObjects) && Object->IsTemplate()) { continue; };
	
		FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);
			
		if (!Annotation.IsRoot() && !Annotation.IsReferencedByRoot() && !Annotation.IsReferencedByNonRoot())
		{
			Annotation.Flags |= FObjectMemoryUsage::EObjFlags::IsRoot;
		}
	}

	for( FThreadSafeObjectIterator It(InBaseClass, false, (EObjectFlags)ExclusionFlags); It; ++It )
	{
		UObject* Object	= *It;
		
		if (!(AnalyzeFlags&EAnalyzeFlags::IncludeDefaultObjects) && Object->IsTemplate()) { continue; };

		FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);
		
		SIZE_T InclusiveSize = CalculateSizeRecursive(Object);
		Annotation.InclusiveMemoryUsage = InclusiveSize;
	}
}

int32 FObjectMemoryAnalyzer::GetReferencedObjects(UObject* Obj, TArray<UObject*>& ReferencedObjects)
{
	FReferenceFinder ObjectReferenceCollector( ReferencedObjects, Obj, false, true, true, false);
	ObjectReferenceCollector.FindReferences( Obj );

	return ReferencedObjects.Num();
 }

void FObjectMemoryAnalyzer::ProcessSubObjRecursive(UObject* Root, UObject* Object)
{
	{
		FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);

		if ( Object->HasAllFlags(RF_Standalone) || Annotation.RootReferencer.Num() >= 1)
		{
			Annotation.Flags |= FObjectMemoryUsage::EObjFlags::IsRoot;
		}

		if (!Annotation.IsProcessed())
		{
			FArchiveCountMem MemCount(const_cast<UObject*>(Object));
			Annotation.ExclusiveMemoryUsage = MemCount.GetMax();

			Annotation.Flags |= FObjectMemoryUsage::EObjFlags::IsProcessed;
		}
	}
	
	TArray<UObject*> ReferencedObjects;
	GetReferencedObjects(Object, ReferencedObjects);
		
	for( int32 ObjIndex = 0; ObjIndex < ReferencedObjects.Num(); ObjIndex++ )
	{
		UObject* SubObj = ReferencedObjects[ObjIndex];

		ProcessSubObjRecursive(Root, SubObj);
	}
	
	if (Object != Root)
	{
		FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);
		FObjectMemoryUsage& RootAnnotation = MemUsageAnnotations.GetAnnotationRef(Root);

		if (RootAnnotation.IsRoot() || RootAnnotation.IsReferencedByRoot())
		{
			Annotation.Flags |= FObjectMemoryUsage::EObjFlags::IsReferencedByRoot;
			Annotation.RootReferencer.AddUnique(Root);
		}
		else if (!RootAnnotation.IsReferencedByRoot() )
		{
			Annotation.Flags |= FObjectMemoryUsage::EObjFlags::IsReferencedByNonRoot;
			Annotation.NonRootReferencer.AddUnique(Root);
		}
	}
}

SIZE_T FObjectMemoryAnalyzer::CalculateSizeRecursive(UObject* Object)
{
	FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);

	if (!Annotation.IsProcessed())
	{
		FArchiveCountMem MemCount(const_cast<UObject*>(Object));
		Annotation.ExclusiveMemoryUsage = MemCount.GetMax();

		Annotation.Flags |= FObjectMemoryUsage::EObjFlags::IsProcessed;
	}

	SIZE_T InclusiveSize = Annotation.ExclusiveMemoryUsage;
	Annotation.ExclusiveResourceSize = Object->GetResourceSizeBytes(EResourceSizeMode::Exclusive);
	Annotation.InclusiveResourceSize = Object->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);

	TArray<UObject*> ReferencedObjects;
	GetReferencedObjects(Object, ReferencedObjects);

	for( int32 ObjIndex = 0; ObjIndex < ReferencedObjects.Num(); ObjIndex++ )
	{
		UObject* SubObj = ReferencedObjects[ObjIndex];
		FObjectMemoryUsage& SubAnnotation = MemUsageAnnotations.GetAnnotationRef(SubObj);

		if (!SubAnnotation.IsRoot())
		{
			if (SubAnnotation.InclusiveMemoryUsage >= SubAnnotation.ExclusiveMemoryUsage)
			{
				InclusiveSize += SubAnnotation.InclusiveMemoryUsage;
			}
			else
			{
				SIZE_T SubObjInclusive = CalculateSizeRecursive(SubObj);
				SubAnnotation.InclusiveMemoryUsage = SubObjInclusive;

				InclusiveSize += SubObjInclusive;
			}
			Annotation.InclusiveResourceSize += SubAnnotation.InclusiveResourceSize;
		}
	}
	return InclusiveSize;
}

FString FObjectMemoryAnalyzer::GetFlagsString(const FObjectMemoryUsage& Annotation)
{
	FString FlagStr;
	int NumSetFlags = 0;

	if (Annotation.IsRoot())
	{
		FlagStr += TEXT("IsRoot");
		++NumSetFlags;
	}

	if (Annotation.IsReferencedByRoot())
	{
		if (NumSetFlags > 0)
		{
			FlagStr += TEXT(" | ");
		}

		FlagStr += TEXT("IsReferencedByRoot");
		++NumSetFlags;
	}
	if (Annotation.IsReferencedByNonRoot())
	{
		if (NumSetFlags > 0)
		{
			FlagStr += TEXT(" | ");
		}

		FlagStr += TEXT("IsReferencedByNonRoot");
		++NumSetFlags;
	}
	return FlagStr;
}

void FObjectMemoryAnalyzer::PrintSubObjects(FOutputDevice& Ar, const FString& Indent, UObject* Parent, uint32 PrintFlags)
{
	TArray<UObject*> ReferencedObjects;
	GetReferencedObjects(Parent, ReferencedObjects);

	for( int32 ObjIndex = 0; ObjIndex < ReferencedObjects.Num(); ObjIndex++ )
	{
		UObject* SubObj = ReferencedObjects[ObjIndex];
		const FObjectMemoryUsage& Annotation = GetObjectMemoryUsage(SubObj);

		if (!Annotation.IsRoot())
		{
			Ar.Logf( TEXT("%-100s %-10d %-10d %-10d %-10d"), *FString::Printf(TEXT("%s%s %s"), *Indent, *SubObj->GetClass()->GetName(), *SubObj->GetName()),
					 (int32)Annotation.InclusiveMemoryUsage, (int32)Annotation.ExclusiveMemoryUsage, 
					 (int32)(Annotation.InclusiveResourceSize/1024), (int32)(Annotation.ExclusiveResourceSize/1024) );

			if (!!(PrintFlags&EPrintFlags::PrintReferencer))
			{
				for (int32 i=0; i < Annotation.NonRootReferencer.Num(); ++i)
				{
					Ar.Logf(TEXT("%s  >> NonRootRef: %s"), *Indent, *Annotation.NonRootReferencer[i]->GetName());
				}

				for (int32 i=0; i < Annotation.RootReferencer.Num(); ++i)
				{
					Ar.Logf(TEXT("%s  >> RootRef: %s"), *Indent, *Annotation.RootReferencer[i]->GetName());
				}
			}
			
			if (!!(PrintFlags&EPrintFlags::PrintReferences))
			{
				PrintSubObjects(Ar, Indent + TEXT(" -> "), SubObj, PrintFlags);
			}
		}
	}
}

void FObjectMemoryAnalyzer::PrintResults(FOutputDevice& Ar, uint32 PrintFlags)
{
	TArray<FObjectMemoryUsage> Results;
	GetResults(Results);

	Results.Sort(FCompareFSortBySize(ESortKey::InclusiveTotal));

	Ar.Logf( TEXT("%-100s %-10s %-10s %-10s %-10s"), TEXT("Object"), TEXT("InclBytes"), TEXT("ExclBytes"), TEXT("InclResKBytes"), TEXT("ExclResKBytes") );
	
	for( int32 i=0; i < Results.Num(); ++i )
	{
		const FObjectMemoryUsage& Annotation = Results[i];

		if (Annotation.IsRoot() || (Annotation.RootReferencer.Num() == 0 && Annotation.NonRootReferencer.Num() == 0) )
		{
			Ar.Logf( TEXT("%-100s %-10d %-10d %-10d %-10d"), *FString::Printf(TEXT("%s %s"), *Annotation.Object->GetClass()->GetName(), *Annotation.Object->GetName()), 
					 (int32)Annotation.InclusiveMemoryUsage, (int32)Annotation.ExclusiveMemoryUsage, 
					 (int32)(Annotation.InclusiveResourceSize/1024), (int32)(Annotation.ExclusiveResourceSize/1024) );


			if (!!(PrintFlags&EPrintFlags::PrintReferences))
			{
				PrintSubObjects(Ar, TEXT(" -> "), Annotation.Object, PrintFlags);
			}
		}
	}
}

int32 FObjectMemoryAnalyzer::GetResults(TArray<FObjectMemoryUsage>& Results)
{
	if (BaseClass != NULL)
	{
		uint32 ExclusionFlags = (AnalyzeFlags&EAnalyzeFlags::IncludeDefaultObjects)==0 ? (RF_ClassDefaultObject|RF_ArchetypeObject) : 0;

		for( FThreadSafeObjectIterator It(BaseClass, false, (EObjectFlags)ExclusionFlags); It; ++It )
		{
			UObject* Object	= *It;
			if (!(AnalyzeFlags&EAnalyzeFlags::IncludeDefaultObjects) && Object->IsTemplate()) { continue; };
			FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Object);

			if (Annotation.IsRoot())
			{
				Annotation.Object = Object;
				Results.Add(Annotation);
			}
		}
	}

	if (ObjectList.Num() > 0)
	{
		for (int32 i=0; i < ObjectList.Num(); ++i)
		{
			FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(ObjectList[i]);
			check(Annotation.IsRoot());

			Annotation.Object = ObjectList[i];
			Results.Add(Annotation);
		}
	}
	return Results.Num();
}

const FObjectMemoryUsage& FObjectMemoryAnalyzer::GetObjectMemoryUsage( class UObject* Obj )
{
	const FObjectMemoryUsage& Annotation = MemUsageAnnotations.GetAnnotationRef(Obj);
	if (!Annotation.IsProcessed())
	{
		CalculateSizeRecursive(Obj);
		return MemUsageAnnotations.GetAnnotationRef(Obj);
	}

	return Annotation;
}
