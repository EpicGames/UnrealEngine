// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectPathId.h"
#include "Algo/FindLast.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "HAL/CriticalSection.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "UObject/Linker.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/Package.h"

FCriticalSection GComplexPathLock;  // Could be changed to an RWLock later if needed

// @TODO: OBJPTR: Consider if it is possible to have this be case-preserving while still having equality checks between two paths of differing case be equal.
// @TODO: OBJPTR: Evaluate if the inline array for the object paths needs to be changed to something more lightweight.
// Currently each unique object path takes up 48 bytes of memory:
//  *8 bytes per entry in the ComplexPathHashToId map
//  *40 byte per entry in the ComplexPaths array
//My expectation is that a 3 name object path is generous in almost every case.  48 bytes
//per complex path may be too expensive depending on how frequently we encounter complex paths.
//If so, we can consider a packed pool to store the array elements in or other options for
//representing shared path elements like just registering the paths as FNames and not having
//our own storage at all.
TMultiMap<uint32, uint32> GComplexPathHashToId;

TArray<UE::CoreUObject::Private::FStoredObjectPath> GComplexPaths;
/*
template <int a, int b>
struct assert_equality
{
	static_assert(a == b, "not equal");
};

//assert_equality<sizeof(TMultiMap<uint32, uint32>::ElementType), 0> A;
assert_equality<sizeof(TArray<FMinimalName, TInlineAllocator<3>>), 0> B;
*/

namespace UE::CoreUObject::Private
{
#if !UE_WITH_REMOTE_OBJECT_HANDLE
	FObjectKey MakeObjectKey(int32 ObjectIndex, int32 ObjectSerialNumber)
	{
		return FObjectKey(ObjectIndex, ObjectSerialNumber);
	}
#endif

	FStoredObjectPath::FStoredObjectPath(TConstArrayView<FMinimalName> InNames)
	{
		// Copy into storage in reverse
		NumElements = InNames.Num();

		FMinimalName* Dest = nullptr;

		if (NumElements > NumInlineElements)
		{
			Long = new FMinimalName[NumElements];
			Dest = Long;
		}
		else
		{
			Dest = Short;
		}

		for (int32 i = NumElements - 1; i >= 0; --i)
		{
			*Dest = InNames[i];
			++Dest;
		}
	}

	FStoredObjectPath::FStoredObjectPath(const FMinimalName* Parent, int32 NumParent, const FMinimalName* Child, int32 NumChild)
	{
		// Copy into storage in reverse
		NumElements = NumParent + NumChild;

		FMinimalName* Dest = nullptr;

		if (NumElements > NumInlineElements)
		{
			Long = new FMinimalName[NumElements];
			Dest = Long;
		}
		else
		{
			Dest = Short;
		}

		for (int32 i = 0; i < NumParent; ++i)
		{
			Dest[i] = Parent[i];
		}

		for (int32 i = 0; i < NumChild; ++i)
		{
			Dest[NumParent + i] = Child[i];
		}
	}

	FStoredObjectPath::~FStoredObjectPath()
	{
		if (NumElements > NumInlineElements)
		{
			delete[] Long;
		}
	}

	FStoredObjectPath::FStoredObjectPath(FStoredObjectPath&& Other)
		: NumElements(Other.NumElements)
	{
		Other.NumElements = 0;
		if (NumElements > NumInlineElements)
		{
			Long = Other.Long;
		}
		else
		{
			FMemory::Memcpy(Short, Other.Short, sizeof(Short));
		}
	}

	FStoredObjectPath& FStoredObjectPath::operator=(FStoredObjectPath&& Other)
	{
		this->~FStoredObjectPath();
		new(this) FStoredObjectPath(MoveTemp(Other));

		return *this;
	}

	TConstArrayView<FMinimalName> FStoredObjectPath::GetView() const
	{
		return TConstArrayView<FMinimalName>{ NumElements <= NumInlineElements ? Short : Long, NumElements };
	}

	static bool operator==(const FStoredObjectPath& A, const FStoredObjectPath& B)
	{
		return A.NumElements == B.NumElements && CompareItems(A.GetView().GetData(), B.GetView().GetData(), A.NumElements);
	}

	template <typename NameProducerType>
	void FObjectPathId::StoreObjectPathId(NameProducerType& NameProducer)
	{
		FName Name = NameProducer();

		if (Name == NAME_None)
		{
			return;
		}

		FName OuterName = NameProducer();
		if (OuterName == NAME_None)
		{
			int32 NameNumber = Name.GetNumber();

			//verify that the second most significant bits are not set
			if (!(NameNumber & SimpleNameMask) && !(NameNumber & WeakObjectMask))
			{
				Index = Name.GetComparisonIndex().ToUnstableInt();
				Number = NameNumber | SimpleNameMask;
				return;
			}
		}

		//Complex path scenario
		TArray<FMinimalName, TInlineAllocator<3>> MinimalNames;
		uint32 Key = GetTypeHash(Name.ToUnstableInt());
		MinimalNames.Emplace(NameToMinimalName(Name));
		while (OuterName != NAME_None)
		{
			Name = OuterName;
			OuterName = NameProducer();
			MinimalNames.Emplace(NameToMinimalName(Name));
			Key = HashCombine(Key, GetTypeHash(Name.ToUnstableInt()));
		}

		// Reverse path so we can compare it to stored paths
		FStoredObjectPath PathToStore(MinimalNames);

		FScopeLock ComplexPathScopeLock(&GComplexPathLock);
		for (typename TMultiMap<uint32, uint32>::TConstKeyIterator It(GComplexPathHashToId, Key); It; ++It)
		{
			uint32 PotentialPathId = It.Value();
			if (GComplexPaths[PotentialPathId - 1] == PathToStore)
			{
				Index = PotentialPathId;
				Number = 0;
				return;
			}
		}

		//add one because a path id of 0 indicates null
		uint32 NewId = GComplexPaths.Emplace(MoveTemp(PathToStore)) + 1;
		GComplexPathHashToId.Add(Key, NewId);
		Index = NewId;
		Number = 0;
		GCoreComplexObjectPathDebug = (FStoredObjectPathDebug*)GComplexPaths.GetData();
	}

	FObjectPathId::FObjectPathId(const UObject* Object)
	{
		struct FLoadedObjectNamePathProducer
		{
			FLoadedObjectNamePathProducer(const UObject* InObject)
				: CurrentObject(InObject)
			{}

			FName operator()(void)
			{
				if ((CurrentObject == nullptr) || (CurrentObject->GetClass() == UPackage::StaticClass()))
				{
					return NAME_None;
				}
				FName RetVal = CurrentObject->GetFName();
				CurrentObject = CurrentObject->GetOuter();
				return RetVal;
			}
		private:
			const UObject* CurrentObject;
		} NamePathProducer(Object);

		StoreObjectPathId(NamePathProducer);
	}

	FObjectPathId::FObjectPathId(const FObjectImport& Import, const FLinkerTables& LinkerTables)
	{
		MakeImportPathIdAndPackageName(Import, LinkerTables, *this);
	}

	FName FObjectPathId::MakeImportPathIdAndPackageName(const FObjectImport& Import, const FLinkerTables& LinkerTables, FObjectPathId& OutPathId)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FObjectPathId::MakeImportPathIdAndPackageName);

		struct FImportObjectNamePathProducer
		{
			FImportObjectNamePathProducer(const FObjectImport& InImport, const FLinkerTables& InLinkerTables)
				: CurrentImport(&InImport)
				, LinkerTables(InLinkerTables)
			{}

			FName operator()(void)
			{
				if ((CurrentImport == nullptr) || CurrentImport->OuterIndex.IsNull())
				{
					return NAME_None;
				}
				FName RetVal = CurrentImport->ObjectName;
				CurrentImport = &LinkerTables.Imp(CurrentImport->OuterIndex);
				return RetVal;
			}

			FName GetPackageName()
			{
				if (CurrentImport && CurrentImport->OuterIndex.IsNull())
				{
					return CurrentImport->ObjectName;
				}
				return NAME_None;
			}
		private:
			const FObjectImport* CurrentImport;
			const FLinkerTables& LinkerTables;
		} NamePathProducer(Import, LinkerTables);

		OutPathId.StoreObjectPathId(NamePathProducer);
		return NamePathProducer.GetPackageName();
	}

	template <typename CharType>
	struct TStringViewNamePathProducer
	{
		TStringViewNamePathProducer(TStringView<CharType> StringPath)
			: CurrentStringView(StringPath)
		{}

		FName operator()(void)
		{
			if (CurrentStringView.IsEmpty())
			{
				return NAME_None;
			}

			int32 FoundIndex = INDEX_NONE;
			if (!FindLastSeparator(FoundIndex))
			{
				FName RetVal(CurrentStringView);
				CurrentStringView.Reset();
				return RetVal;
			}

			FName RetVal(CurrentStringView.RightChop(FoundIndex + 1));
			CurrentStringView.RemoveSuffix(CurrentStringView.Len() - FoundIndex);
			return RetVal;
		}
	private:
		static inline bool IsPathIdSeparator(CharType Char) { return Char == '.' || Char == ':'; }

		bool FindLastSeparator(int32& OutIndex)
		{
			if (const CharType* Separator = Algo::FindLastByPredicate(CurrentStringView, IsPathIdSeparator))
			{
				OutIndex = UE_PTRDIFF_TO_INT32(Separator - CurrentStringView.GetData());
				return true;
			}
			OutIndex = INDEX_NONE;
			return false;
		}

		TStringView<CharType> CurrentStringView;
	};

	FObjectPathId::FObjectPathId(FWideStringView StringPath)
	{
		TStringViewNamePathProducer<WIDECHAR> NamePathProducer(StringPath);
		StoreObjectPathId(NamePathProducer);
	}

	FObjectPathId::FObjectPathId(FAnsiStringView StringPath)
	{
		TStringViewNamePathProducer<ANSICHAR> NamePathProducer(StringPath);
		StoreObjectPathId(NamePathProducer);
	}

	void FObjectPathId::Resolve(ResolvedNameContainerType& OutContainer) const
	{
		check(IsValid());

		if (IsNone())
		{
			return;
		}

		if (Number & SimpleNameMask)
		{
			//truncate Path to int and shift back to get the number
			uint32 NameNumber = Number & ~SimpleNameMask;
			uint32 NameIndex = Index;
			FNameEntryId EntryId = FNameEntryId::FromUnstableInt(NameIndex);
			OutContainer.Emplace(EntryId, EntryId, NameNumber);
			return;
		}

		FScopeLock ComplexPathScopeLock(&GComplexPathLock);
		const FStoredObjectPath& FoundContainer = GComplexPaths[Index - 1];

		for (FMinimalName Name : FoundContainer.GetView())
		{
			OutContainer.Emplace(MinimalNameToName(Name));
		}
	}

	FMinimalName FObjectPathId::GetSimpleName() const
	{
		check(IsSimple());

		uint32 NameNumber = Number & ~SimpleNameMask;
		uint32 NameIndex = Index;
		FNameEntryId EntryId = FNameEntryId::FromUnstableInt(NameIndex);
		FMinimalName Name = NameToMinimalName(FName(EntryId, EntryId, NameNumber));
		return Name;
	}

	void FObjectPathId::MakeWeakObjPtr(const UObject& Object)
	{
		int32 ObjectIndex = GUObjectArray.ObjectToIndex(&Object);
		int32 ObjectSerialNumber = GUObjectArray.AllocateSerialNumber(ObjectIndex);

		//check that ObjectSerialNumber does not have WeakObjMask bit set
		check(!(ObjectSerialNumber & WeakObjectMask));
		Index = ObjectIndex;
		Number = ObjectSerialNumber | WeakObjectMask;
	}

	FWeakObjectPtr FObjectPathId::GetWeakObjPtr() const
	{
		check(IsWeakObj());

		int32 ObjectIndex = Index;
		int32 ObjectSerial = Number & ~WeakObjectMask;
#if !UE_WITH_REMOTE_OBJECT_HANDLE
		FObjectKey Key = UE::CoreUObject::Private::MakeObjectKey(ObjectIndex, ObjectSerial);

		return Key.ResolveObjectPtr();
#else
		checkf(false, TEXT("FObjectPathId::GetWeakObjPtr() can't be used with remote object handles"));
		return FWeakObjectPtr();
#endif
	}

	const FStoredObjectPath& FObjectPathId::GetStoredPath() const
	{
		check(IsValid() && !IsNone());
		const FStoredObjectPath& FoundContainer = GComplexPaths[Index - 1];
		return FoundContainer;
	}
}