// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "HAL/Platform.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"

class FLinkerTables;
class UObject;
struct FObjectImport;
struct FObjectRef;

// Declared in the header so the type exists in debug info for debugger visualization, not a public part of the API.
namespace UE::CoreUObject::Private
{
	struct FPackedObjectRef;

	struct FStoredObjectPath
	{
		static constexpr const int32 NumInlineElements = 3;
		int32 NumElements;

		union
		{
			FMinimalName Short[NumInlineElements];
			FMinimalName* Long;
		};

		FStoredObjectPath(TConstArrayView<FMinimalName> InNames);
		FStoredObjectPath(const FMinimalName* Parent, int32 NumParent, const FMinimalName* Child, int32 NumChild);
		~FStoredObjectPath();
		FStoredObjectPath(const FStoredObjectPath&) = delete;
		FStoredObjectPath(FStoredObjectPath&&);
		FStoredObjectPath& operator=(const FStoredObjectPath&) = delete;
		FStoredObjectPath& operator=(FStoredObjectPath&&);

		TConstArrayView<FMinimalName> GetView() const;
		const FMinimalName* GetData() const { return NumElements > NumInlineElements ? Long : &Short[0]; }
	};

	// @TODO: OBJPTR: Should this be changed to FObjectImportPathId? It is already written to target the specific patterns in import paths,
	//		 if we go further in the future and do things like store classname/classpackage info, to the path segments, it would be
	//		 even more specific to imports.

	/**
	 * FObjectPathId is a type tailored to represent object paths, with a focus on avoiding overhead for common import path patterns
	 * It is primarily for use by FObjectRef/FPackedObjectRef/FObjectHandle. Most object paths used by FObjectRef will be imports,
	 * and are frequently a single FName without any outer components. FObjectPathId can transform to/from most single FNames without
	 * any need for a table lookup. For the less common case of an imported object path composed of more than one FName, or if the imported
	 * object path has an exceedingly high number component, FObjectPathId transformations will have to do a table lookup.
	 *
	 * NOTE: FObjectPathId is NOT case sensitive.  Two FObjectPathIds that were constructed with the same path but of differing case
	 *		 will compare as equal.  This also means that constructing an FObjectPathId from a path then calling Resolve on it may
	 *		 produce the same set of names as in the original path, but of different case.  THIS WILL CAUSE NON-DETERMINISM.
	 */
	class FObjectPathId
	{
	public:
		FObjectPathId() = default;
		FObjectPathId(const FObjectPathId& Other) = default;
		explicit FObjectPathId(const UObject* Object);
		FObjectPathId(const FObjectImport& Import, const FLinkerTables& LinkerTables);

		enum EInvalid { Invalid = 0 };

		explicit FObjectPathId(EInvalid)
			: Index(0)
			, Number(SimpleNameMask | WeakObjectMask)
		{
		}

		static FName MakeImportPathIdAndPackageName(const FObjectImport& Import, const FLinkerTables& LinkerTables, FObjectPathId& OutPathId);

		explicit FObjectPathId(FWideStringView StringPath);
		explicit FObjectPathId(FAnsiStringView StringPath);

		FObjectPathId& operator=(const FObjectPathId& Other) = default;
		inline bool operator==(const FObjectPathId& Other) const { return Index == Other.Index && Number == Other.Number; }
		inline bool IsNone() const { return Index == 0 && Number == 0; }
		inline bool IsValid() const { return Number != (SimpleNameMask | WeakObjectMask); }
		inline bool IsSimple() const { return IsValid() && Number & SimpleNameMask; }
		inline bool IsWeakObj() const { return IsValid() && Number & WeakObjectMask; }

		// @TODO: OBJPTR: Is there a better interface to output the resolved name without:
		// 		 1) Requiring multiple lookups in the table in case of complex paths
		//		 2) Requiring string allocation
		//		 3) Constraining how we might change the internal storage of object paths in the future
		using ResolvedNameContainerType = TArray<FName, TInlineAllocator<3>>;
		void Resolve(ResolvedNameContainerType& OutContainer) const;
		FWeakObjectPtr GetWeakObjPtr() const;
		FMinimalName GetSimpleName() const;
		void MakeWeakObjPtr(const UObject& Object);
		const FStoredObjectPath& GetStoredPath() const;

		void Reset()
		{
			Index = 0;
			Number = 0;
		}
	private:
		
		template <typename NameProducerType>
		void StoreObjectPathId(NameProducerType& NameProducer);

		static constexpr uint32 WeakObjectMask = ~((~0u) >> 1);       //most significant bit
		static constexpr uint32 SimpleNameMask = WeakObjectMask >> 1; //second most significant bits

		uint32 Index = 0;
		uint32 Number = 0;

		friend FORCEINLINE uint32 GetTypeHash(FObjectPathId Value)
		{
			return HashCombineFast(GetTypeHash(Value.Index), GetTypeHash(Value.Number));
		}
	};
}
