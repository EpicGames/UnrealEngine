// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectPtr.h"
#include "UObject/Class.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/RemoteObjectPathName.h"
#include "UObject/UObjectHash.h"

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

namespace  UE::CoreUObject::Private
{
	FPackedObjectRef GetOuter(FPackedObjectRef);
	FPackedObjectRef GetPackage(FPackedObjectRef);
}

FString FObjectPtr::GetPathName() const
{
	FObjectHandle LocalHandle = Handle;
	if (IsObjectHandleResolved(LocalHandle) && !IsObjectHandleNull(LocalHandle))
	{
		return ResolveObjectHandleNoRead(LocalHandle)->GetPathName();
	}
	else
	{
		FObjectRef ObjectRef = UE::CoreUObject::Private::MakeObjectRef(UE::CoreUObject::Private::ReadObjectHandlePackedObjectRefNoCheck(LocalHandle));
		return ObjectRef.GetPathName();
	}
}

FName FObjectPtr::GetFName() const
{
	FObjectHandle LocalHandle = Handle;
	if (IsObjectHandleResolved(LocalHandle) && !IsObjectHandleNull(LocalHandle))
	{
		return ResolveObjectHandleNoRead(LocalHandle)->GetFName();
	}
	else
	{
		FObjectRef ObjectRef = UE::CoreUObject::Private::MakeObjectRef(UE::CoreUObject::Private::ReadObjectHandlePackedObjectRefNoCheck(LocalHandle));
		return ObjectRef.GetFName();
	}
}

FString FObjectPtr::GetFullName(EObjectFullNameFlags Flags) const
{
	FObjectHandle LocalHandle = Handle;
	if (IsObjectHandleResolved(LocalHandle) && !IsObjectHandleNull(LocalHandle))
	{
		return ResolveObjectHandleNoRead(LocalHandle)->GetFullName(nullptr, Flags);
	}
	else
	{
		FObjectRef ObjectRef = UE::CoreUObject::Private::MakeObjectRef(UE::CoreUObject::Private::ReadObjectHandlePackedObjectRefNoCheck(LocalHandle));
		return ObjectRef.GetFullName(Flags);
	}
}

FObjectPtr FObjectPtr::GetOuter() const
{
	FObjectHandle LocalHandle = Handle;
	if (IsObjectHandleResolved(LocalHandle) && !IsObjectHandleNull(LocalHandle))
	{
		return FObjectPtr(ResolveObjectHandleNoRead(LocalHandle)->GetOuter());
	}
	UE::CoreUObject::Private::FPackedObjectRef PackedRef = UE::CoreUObject::Private::GetOuter(ReadObjectHandlePackedObjectRefNoCheck(LocalHandle));
	FObjectPtr Ptr(FObjectHandle{ PackedRef.EncodedRef });
	return Ptr;
}

FObjectPtr FObjectPtr::GetPackage() const
{
	FObjectHandle LocalHandle = Handle;
	if (IsObjectHandleResolved(LocalHandle) && !IsObjectHandleNull(LocalHandle))
	{
		return FObjectPtr(ResolveObjectHandleNoRead(LocalHandle)->GetPackage());
	}
	UE::CoreUObject::Private::FPackedObjectRef PackedRef = UE::CoreUObject::Private::GetPackage(ReadObjectHandlePackedObjectRefNoCheck(LocalHandle));
	FObjectPtr Ptr(FObjectHandle{ PackedRef.EncodedRef });
	return Ptr;
}

bool FObjectPtr::IsIn(FObjectPtr SomeOuter) const
{
	FObjectHandle SomeOuterHandle = SomeOuter.Handle;
	FObjectHandle LocalHandle = Handle;
	if (IsObjectHandleNull(LocalHandle) || IsObjectHandleNull(SomeOuterHandle))
	{
		return false;
	}
	// TODO: Handle without resolving. Need to decide how ObjectPtrs handle objects in external packages.
	return ResolveObjectHandleNoRead(LocalHandle)->IsIn(ResolveObjectHandleNoRead(SomeOuterHandle));
}

#elif UE_WITH_REMOTE_OBJECT_HANDLE

FString FObjectPtr::GetPathName() const
{
	FObjectHandle LocalHandle = Handle;
	if (!IsObjectHandleNull(LocalHandle))
	{
		if (IsObjectHandleResolved_ForGC(LocalHandle))
		{
			return ReadObjectHandlePointerNoCheck(LocalHandle)->GetPathName();
		}
		else
		{
			FRemoteObjectPathName RemotePathName(LocalHandle.ToStub()->Id);
			if (RemotePathName.Num())
			{
				return RemotePathName.ToString();
			}
			else if (const UObject* ResolvedObject = ResolveObjectHandleNoRead(LocalHandle))
			{
				return ResolvedObject->GetPathName();
			}
		}
	}
	return FString("None");
}

FName FObjectPtr::GetFName() const
{
	using namespace UE::RemoteObject::Private;
	using namespace UE::RemoteObject::Handle;

	FObjectHandle LocalHandle = Handle;
	if (!IsObjectHandleNull(LocalHandle))
	{
		if (IsObjectHandleResolved_ForGC(LocalHandle))
		{
			return ReadObjectHandlePointerNoCheck(LocalHandle)->GetFName();
		}
		else if (const FRemoteObjectStub* Stub = LocalHandle.ToStub())
		{
			const bool bCannotMigrate = !AutoRTFM::IsClosed();
			if (bCannotMigrate || !Stub->Name.IsNone())
			{
				return Stub->Name;
			}
		}
		// If the above two branches failed to return the name, we need to resolve the object
		if (const UObject* ResolvedObject = ResolveObjectHandleNoRead(LocalHandle))
		{
			return ResolvedObject->GetFName();
		}
	}
	return NAME_None;
}

FString FObjectPtr::GetFullName(EObjectFullNameFlags Flags) const
{
	FObjectHandle LocalHandle = Handle;
	if (!IsObjectHandleNull(LocalHandle) && IsObjectHandleResolved_ForGC(LocalHandle))
	{
		return ReadObjectHandlePointerNoCheck(LocalHandle)->GetFullName(nullptr, Flags);
	}
	return UE::CoreUObject::Private::GetFullName(ResolveObjectHandleNoRead(LocalHandle), nullptr, Flags);
}

FObjectPtr FObjectPtr::GetOuter() const
{
	using namespace UE::RemoteObject::Private;
	using namespace UE::RemoteObject::Handle;

	FObjectHandle LocalHandle = Handle;
	if (!IsObjectHandleNull(LocalHandle))
	{
		if (IsObjectHandleResolved_ForGC(LocalHandle))
		{
			return FObjectPtr(ReadObjectHandlePointerNoCheck(LocalHandle)->GetOuter());
		}
		else if (const FRemoteObjectStub* Stub = LocalHandle.ToStub())
		{
			// Only rely on the OuterId if it's actually valid. Otherwise we have no way of distinguishing between a stub that represnts a package or an incomplete stub
			if (Stub->OuterId.IsValid())
			{
				return FObjectPtr(Stub->OuterId);
			}
		}
		if (const UObject* ResolvedObject = ResolveObjectHandleNoRead(LocalHandle))
		{
			return FObjectPtr(ResolvedObject->GetOuter());
		}
	}
	return FObjectPtr();
}

FObjectPtr FObjectPtr::GetPackage() const
{
	using namespace UE::RemoteObject::Private;
	using namespace UE::RemoteObject::Handle;

	FObjectHandle LocalHandle = Handle;
	if (!IsObjectHandleNull(LocalHandle))
	{
		if (IsObjectHandleResolved_ForGC(LocalHandle))
		{
			return FObjectPtr(ReadObjectHandlePointerNoCheck(LocalHandle)->GetPackage());
		}
		else if (const FRemoteObjectStub* Stub = LocalHandle.ToStub())
		{
			for (const FRemoteObjectStub* StubIt = Stub; StubIt;)
			{
				if (StubIt->OuterId.IsValid())
				{
					// Object represented by the stub has an outer so try and find it
					if (UObject* Outer = StaticFindObjectFastInternal(StubIt->OuterId))
					{
						return FObjectPtr(Outer->GetPackage());
					}
					else
					{
						// Or try and find the outer's stub
						StubIt = FindRemoteObjectStub(StubIt->OuterId);
					}
				}
				else
				{
					// Currently we can't assume that all stubs have the OuterId info valid so we're going to have to resolve the object
					break;
				}
			}
		}

		// If we got here then we were not able to resolve the package from a remote object or its stub so we need to resolve the object
		if (const UObject* ResolvedObject = ResolveObjectHandleNoRead(LocalHandle))
		{
			return FObjectPtr(ResolvedObject->GetPackage());
		}
	}
	return FObjectPtr();
}

bool FObjectPtr::IsIn(FObjectPtr SomeOuter) const
{
	FObjectHandle SomeOuterHandle = SomeOuter.Handle;
	FObjectHandle LocalHandle = Handle;
	if (IsObjectHandleNull(LocalHandle) || IsObjectHandleNull(SomeOuterHandle))
	{
		return false;
	}
	if (IsObjectHandleResolved_ForGC(LocalHandle) && IsObjectHandleResolved_ForGC(SomeOuterHandle))
	{
		UObject* Object = ReadObjectHandlePointerNoCheck(LocalHandle);
		UObject* Outer = ReadObjectHandlePointerNoCheck(SomeOuterHandle);
		return Object->IsIn(Outer);
	}
	return ResolveObjectHandleNoRead(LocalHandle)->IsIn(ResolveObjectHandleNoRead(SomeOuterHandle));
}

#endif

bool FObjectPtr::IsA(const UClass* SomeBase) const
{
	checkfSlow(SomeBase, TEXT("IsA(NULL) cannot yield meaningful results"));

	if (const UClass* ThisClass = GetClass())
	{
		return ThisClass->IsChildOf(SomeBase);
	}

	return false;
}