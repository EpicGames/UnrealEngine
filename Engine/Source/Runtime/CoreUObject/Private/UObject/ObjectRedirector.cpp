// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnRedirector.cpp: Object redirector implementation.
=============================================================================*/

#include "UObject/ObjectRedirector.h"

#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "Templates/Casts.h"
#include "UObject/GarbageCollectionSchema.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/RegisterCompiledInObjects.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	UObjectRedirector
-----------------------------------------------------------------------------*/


// If this object redirector is pointing to an object that won't be serialized anyway, set the RF_Transient flag
// so that this redirector is also removed from the package.
void UObjectRedirector::PreSave(FObjectPreSaveContext ObjectSaveContext)
{
	if (DestinationObject == NULL
	||	DestinationObject->HasAnyFlags(RF_Transient)
	||	DestinationObject->IsInPackage(GetTransientPackage()) )
	{
		Modify();
		SetFlags(RF_Transient);

		if ( DestinationObject != NULL )
		{
			DestinationObject->Modify();
			DestinationObject->SetFlags(RF_Transient);
		}
	}

	Super::PreSave(ObjectSaveContext);
}

IMPLEMENT_FARCHIVE_SERIALIZER(UObjectRedirector);

void UObjectRedirector::Serialize( FStructuredArchive::FRecord Record )
{
	Super::Serialize(Record);

	{
#if WITH_EDITOR
		TGuardValue<bool> IsSerializingDestinationObjectScope(bIsSerializingDestinationObject, true);
#endif
		Record << SA_VALUE(TEXT("DestinationObject"), DestinationObject);
	}
}

bool UObjectRedirector::NeedsLoadForEditorGame() const
{
	return true;
}

void UObjectRedirector::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	FString DestVal;
	if ( DestinationObject != nullptr )
	{
		DestVal = FObjectPropertyBase::GetExportPath(DestinationObject);
	}
	else
	{
		DestVal = TEXT("None");
	}

	Context.AddTag(FAssetRegistryTag("DestinationObject", DestVal, UObject::FAssetRegistryTag::TT_Alphabetical));
}

/**
 * Callback for retrieving a textual representation of natively serialized properties.  Child classes should implement this method if they wish
 * to have natively serialized property values included in things like diffcommandlet output.
 *
 * @param	out_PropertyValues	receives the property names and values which should be reported for this object.  The map's key should be the name of
 *								the property and the map's value should be the textual representation of the property's value.  The property value should
 *								be formatted the same way that FProperty::ExportText formats property values (i.e. for arrays, wrap in quotes and use a comma
 *								as the delimiter between elements, etc.)
 * @param	ExportFlags			bitmask of EPropertyPortFlags used for modifying the format of the property values
 *
 * @return	return true if property values were added to the map.
 */
bool UObjectRedirector::GetNativePropertyValues( TMap<FString,FString>& out_PropertyValues, uint32 ExportFlags/*=0*/ ) const
{
	UObject* StopOuter = NULL;

	// determine how the caller wants object values to be formatted
	if ( (ExportFlags&PPF_SimpleObjectText) != 0 )
	{
		StopOuter = GetOutermost();
	}

	out_PropertyValues.Add(TEXT("DestinationObject"), DestinationObject->GetFullName(StopOuter));
	return true;
}

IMPLEMENT_CORE_INTRINSIC_CLASS(UObjectRedirector, UObject,
	{
		UE::GC::DeclareIntrinsicMembers(Class, { UE_GC_MEMBER(UObjectRedirector, DestinationObject) });
	}
);
