// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectRef.h"

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING

#include "Misc/PackageName.h"
#include "UObject/Package.h"
#include "UObject/UObjectHash.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/LinkerLoad.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/ObjectPathId.h"

DEFINE_LOG_CATEGORY(LogObjectRef);

FObjectRef::FObjectRef(FName PackageName, FName ClassPackageName, FName ClassName, UE::CoreUObject::Private::FObjectPathId ObjectPath)
	: PackageName(PackageName)
	, ClassPackageName(ClassPackageName)
	, ClassName(ClassName)
	, ObjectPathId(*reinterpret_cast<uint64*>(&ObjectPath))
{
}

FObjectRef::FObjectRef(const UObject* Object)
	: PackageName(NAME_None)
	, ClassPackageName(NAME_None)
	, ClassName(NAME_None)
	, ObjectPathId(0)
{
	if (Object)
	{
		ClassName = Object->GetClass()->GetFName();

		UObject* ClassOuter = Object->GetClass()->GetOuter();
		ClassPackageName = ClassOuter ? ClassOuter->GetFName() : NAME_None;

		UObject* ObjectOuter = Object->GetOutermost();
		PackageName = ObjectOuter ? ObjectOuter->GetFName() : NAME_None;
		UE::CoreUObject::Private::FObjectPathId ObjectPath(Object);

		ObjectPathId = *reinterpret_cast<uint64*>(&ObjectPath);
	}
}

UE::CoreUObject::Private::FObjectPathId FObjectRef::GetObjectPath() const
{
	static_assert(sizeof(uint64) == sizeof(UE::CoreUObject::Private::FObjectPathId));
	return *reinterpret_cast<const UE::CoreUObject::Private::FObjectPathId*>(&ObjectPathId);
}

FName FObjectRef::GetFName() const
{
	if (PackageName == NAME_None)
	{
		return NAME_None;
	}
	else if (ObjectPathId == 0)
	{
		return PackageName;
	}

	UE::CoreUObject::Private::FObjectPathId ObjectPath = GetObjectPath();
	UE::CoreUObject::Private::FObjectPathId::ResolvedNameContainerType ResolvedNames;
	ObjectPath.Resolve(ResolvedNames);

	if (ResolvedNames.Num() > 0)
	{
		const FName& Name = ResolvedNames[ResolvedNames.Num() - 1];
		return Name;
	}

	return NAME_None;
}

void FObjectRef::AppendClassPathName(FStringBuilderBase& OutClassPathNameBuilder, EObjectFullNameFlags Flags) const
{
	if (EnumHasAllFlags(Flags, EObjectFullNameFlags::IncludeClassPackage))
	{
		ClassPackageName.AppendString(OutClassPathNameBuilder);
		OutClassPathNameBuilder << TEXT(".");
	}

	ClassName.AppendString(OutClassPathNameBuilder);
}

void FObjectRef::AppendPathName(FStringBuilderBase& OutPathNameBuilder) const
{
	if (PackageName == NAME_None)
	{
		OutPathNameBuilder.Append(TEXT("None"));
		return;
	}

	PackageName.AppendString(OutPathNameBuilder);

	UE::CoreUObject::Private::FObjectPathId ObjectPath = GetObjectPath();
	UE::CoreUObject::Private::FObjectPathId::ResolvedNameContainerType ResolvedNames;
	ObjectPath.Resolve(ResolvedNames);

	for (int Index = 0; Index < ResolvedNames.Num(); ++Index)
	{
		if (Index == 1)
		{
			OutPathNameBuilder << SUBOBJECT_DELIMITER_CHAR;
		}
		else
		{
			OutPathNameBuilder << TEXT(".");
		}

		const FName& Name = ResolvedNames[Index];
		Name.AppendString(OutPathNameBuilder);
	}
}


static inline UPackage* FindOrLoadPackage(FName PackageName, int32 LoadFlags, OUT bool& bWasPackageLoaded)
{
	bWasPackageLoaded = false;

	// @TODO: OBJPTR: Want to replicate the functional path of an import here.  See things like FindImportFast in BlueprintSupport.cpp
	// 		 for additional behavior that we're not handling here yet.
	FName* ScriptPackageName = FPackageName::FindScriptPackageName(PackageName);
	UPackage* TargetPackage = (UPackage*)StaticFindObjectFastInternal(UPackage::StaticClass(), nullptr, PackageName);
	if (!ScriptPackageName && !TargetPackage)
	{
		// @TODO: OBJPTR: When using the "external package" feature, we will have objects that have a differing package path vs "outer hierarchy" path
		//				  The package path should be used when loading.  The "outer hierarchy" path may need to be used when finding existing objects in memory.
		//				  This will need further evaluation and testing before lazy load can be enabled.
		// @TODO: OBJPTR: Instancing context may be important to consider when loading the package.
		if (FLinkerLoad::IsKnownMissingPackage(PackageName))
		{
			return nullptr;
		}
		LoadFlags |= LOAD_NoWarn | LOAD_NoVerify; //This does nothing? | LOAD_DisableDependencyPreloading;
		TargetPackage = LoadPackage(nullptr, *PackageName.ToString(), LoadFlags);
		bWasPackageLoaded = true;
	}
	return TargetPackage;
}

UClass* FObjectRef::ResolveObjectRefClass(uint32 LoadFlags /*= LOAD_None*/) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ResolveObjectRef);
	UClass* ClassObject = nullptr;
	UPackage* ClassPackage = nullptr;
	if (!ClassPackageName.IsNone())
	{
		bool bWasPackageLoaded = false;
		ClassPackage = FindOrLoadPackage(ClassPackageName, LoadFlags, bWasPackageLoaded);

		if (!ClassName.IsNone())
		{
			ClassObject = (UClass*)StaticFindObjectFastInternal(UClass::StaticClass(), ClassPackage, ClassName);

			if (ClassObject)
			{
				ClassObject->ConditionalPreload();
				ClassObject->GetDefaultObject(); // build the CDO if it isn't already built
			}
		}
	}

	UE::CoreUObject::Private::OnClassReferenceResolved(*this, ClassPackage, ClassObject);
	return ClassObject;
}

class FFullyLoadPackageOnHandleResolveTask
{
public:
	FFullyLoadPackageOnHandleResolveTask(UPackage* InPackage) : Package(InPackage)
	{
	}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FFullyLoadPackageOnHandleResolveTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::GameThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::FireAndForget; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		Package->GetLinker()->LoadAllObjects(true);
	}

private:
	UPackage* Package = nullptr;
};

UObject* FObjectRef::Resolve(uint32 LoadFlags /*= LOAD_None*/) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ResolveObjectRef);

	UE::CoreUObject::Private::FObjectPathId ObjectPath = GetObjectPath();
	if (IsNull() || !ObjectPath.IsValid())
	{
		UE::CoreUObject::Private::OnReferenceResolved(*this, nullptr, nullptr);
		return nullptr;
	}

	bool bWasObjectOrPackageLoaded = false;

	UPackage* TargetPackage = FindOrLoadPackage(PackageName, LoadFlags, bWasObjectOrPackageLoaded);

	if (!TargetPackage)
	{
		UE::CoreUObject::Private::OnReferenceResolved(*this, nullptr, nullptr);
		return nullptr;
	}

	UE::CoreUObject::Private::FObjectPathId::ResolvedNameContainerType ResolvedNames;
	ObjectPath.Resolve(ResolvedNames);

	UObject* CurrentObject = TargetPackage;
	for (int32 ObjectPathIndex = 0; ObjectPathIndex < ResolvedNames.Num(); ++ObjectPathIndex)
	{
		UObject* PreviousOuter = CurrentObject;
		CurrentObject = StaticFindObjectFastInternal(nullptr, CurrentObject, ResolvedNames[ObjectPathIndex]);
		if (UObjectRedirector* Redirector = dynamic_cast<UObjectRedirector*>(CurrentObject))
		{
			CurrentObject = Redirector->DestinationObject;
			if (CurrentObject != nullptr)
			{
				TargetPackage = CurrentObject->GetPackage();
			}
		}

		if (!CurrentObject && !TargetPackage->IsFullyLoaded() && TargetPackage->GetLinker() && TargetPackage->GetLinker()->IsLoading())
		{
			if (IsInAsyncLoadingThread() || IsInGameThread())
			{
				TargetPackage->GetLinker()->LoadAllObjects(true);
				bWasObjectOrPackageLoaded = true;
			}
			else
			{
				// Shunt the load request to happen on the game thread and block on its completion.  This is a deadlock risk!  The game thread may be blocked waiting on this thread.
				UE_LOGF(LogObjectRef, Warning, "Resolve of object in package '%ls' from a non-game thread was shunted to the game thread.", *PackageName.ToString());
				TGraphTask<FFullyLoadPackageOnHandleResolveTask>::CreateTask().ConstructAndDispatchWhenReady(TargetPackage)->Wait();
				bWasObjectOrPackageLoaded = true;
			}

			CurrentObject = StaticFindObjectFastInternal(nullptr, PreviousOuter, ResolvedNames[ObjectPathIndex]);
			if (UObjectRedirector* Redirector = dynamic_cast<UObjectRedirector*>(CurrentObject))
			{
				CurrentObject = Redirector->DestinationObject;
				if (CurrentObject != nullptr)
				{
					TargetPackage = CurrentObject->GetPackage();
				}
			}
		}

		if (!CurrentObject)
		{
			UE::CoreUObject::Private::OnReferenceResolved(*this, TargetPackage, nullptr);
			return nullptr;
		}
	}

	if (CurrentObject->HasAnyFlags(RF_NeedLoad) && TargetPackage->GetLinker())
	{
		if (IsInAsyncLoadingThread() || IsInGameThread())
		{
			TargetPackage->GetLinker()->LoadAllObjects(true);
			bWasObjectOrPackageLoaded = true;
		}
		else
		{
			// Shunt the load request to happen on the game thread and block on its completion.  This is a deadlock risk!  The game thread may be blocked waiting on this thread.
			UE_LOGF(LogObjectRef, Warning, "Resolve of object in package '%ls' from a non-game thread was shunted to the game thread.", *PackageName.ToString());
			TGraphTask<FFullyLoadPackageOnHandleResolveTask>::CreateTask().ConstructAndDispatchWhenReady(TargetPackage)->Wait();
			bWasObjectOrPackageLoaded = true;
		}
	}
	UE::CoreUObject::Private::OnReferenceResolved(*this, TargetPackage, CurrentObject);

	if (bWasObjectOrPackageLoaded)
	{
		UE::CoreUObject::Private::OnReferenceLoaded(*this, TargetPackage, CurrentObject);
	}

	return CurrentObject;
}



#endif