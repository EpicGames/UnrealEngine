// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectRename.h"

#include "Misc/StringBuilder.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobalsInternal.h"

namespace UE::Object
{
	void RenameLeakedPackage(UPackage* Package)
	{
		FCoreUObjectInternalDelegates::GetOnLeakedPackageRenameDelegate().Broadcast(Package);
		FName NewName = MakeUniqueObjectName(nullptr, UPackage::StaticClass(), Package->GetFName());
		TStringBuilder<FName::StringBufferSize> NewNameString;
		NewNameString << NewName;
		UE_LOGF(LogObj, Log, "Renaming leaked package %ls to %ls", *Package->GetName(), *NewNameString);
		Package->Rename(*NewNameString, nullptr, REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
	}
}
