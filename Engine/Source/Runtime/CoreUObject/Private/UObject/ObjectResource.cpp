// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectResource.h"

#include "AutoRTFM.h"
#include "UObject/Class.h"
#include "Templates/Casts.h"
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/Inline/VVMCellInline.h"
#include "VerseVM/VVMUniqueString.h"
#endif

/*-----------------------------------------------------------------------------
	Helper functions.
-----------------------------------------------------------------------------*/
namespace
{
	FORCEINLINE bool IsCorePackage(const FName& PackageName)
	{
		return PackageName == NAME_Core || PackageName == GLongCorePackageName;
	}
}

/*-----------------------------------------------------------------------------
	FObjectResource
-----------------------------------------------------------------------------*/

FObjectResource::FObjectResource()
{}

FObjectResource::FObjectResource( UObject* InObject )
:	ObjectName		( InObject ? InObject->GetFName() : FName(NAME_None)		)
{
}

/*-----------------------------------------------------------------------------
	FObjectExport.
-----------------------------------------------------------------------------*/

FObjectExport::FObjectExport()
: FObjectResource()
, ObjectFlags(RF_NoFlags)
, SerialSize(0)
, SerialOffset(0)
, ScriptSerializationStartOffset(0)
, ScriptSerializationEndOffset(0)
, Object(NULL)
, HashNext(INDEX_NONE)
, bForcedExport(false)
, bNotForClient(false)
, bNotForServer(false)
, bNotAlwaysLoadedForEditorGame(true)
, bIsAsset(false)
, bIsInheritedInstance(false)
, bGeneratePublicHash(false)
, bExportLoadFailed(false)
, bWasFiltered(false)
, PackageFlags(0)
, FirstExportDependency(-1)
, SerializationBeforeSerializationDependencies(0)
, CreateBeforeSerializationDependencies(0)
, SerializationBeforeCreateDependencies(0)
, CreateBeforeCreateDependencies(0)

{}

FObjectExport::FObjectExport( UObject* InObject, bool bInNotAlwaysLoadedForEditorGame /*= true*/)
: FObjectResource(InObject)
, ObjectFlags(InObject ? InObject->GetMaskedFlags(RF_Load) : RF_NoFlags)
, SerialSize(0)
, SerialOffset(0)
, ScriptSerializationStartOffset(0)
, ScriptSerializationEndOffset(0)
, Object(InObject)
, HashNext(INDEX_NONE)
, bForcedExport(false)
, bNotForClient(false)
, bNotForServer(false)
, bNotAlwaysLoadedForEditorGame(bInNotAlwaysLoadedForEditorGame)
, bIsAsset(false)
, bIsInheritedInstance(false)
, bGeneratePublicHash(false)
, bExportLoadFailed(false)
, bWasFiltered(false)
, PackageFlags(0)
, FirstExportDependency(-1)
, SerializationBeforeSerializationDependencies(0)
, CreateBeforeSerializationDependencies(0)
, SerializationBeforeCreateDependencies(0)
, CreateBeforeCreateDependencies(0)
{
	if(Object)		
	{
		bNotForClient = !Object->NeedsLoadForClient();
		bNotForServer = !Object->NeedsLoadForServer();
		bIsAsset = Object->IsAsset();

		// Flag this export as an inherited instance if the object's archetype exists within the set
		// of default subobjects owned by the object owner's archetype. This is used by the linker to
		// determine whether or not the subobject should be instanced as an export on load. Note that
		// if the archetype is owned by a different object, we treat it as a non-default subobject and
		// thus exclude it from consideration. This is because an instanced subobject with a non-
		// standard archetype won't find a matching instance in its owner's archetype subobject set,
		// and thus wouldn't pass the instancing check on load. One example of a non-default instanced
		// subobject is a Blueprint-added component, whose archetype is owned by the class object.
		const UObject* Archetype = Object->GetArchetype();
		if (Archetype->IsDefaultSubobject() && (Archetype->GetOuter() == Object->GetOuter()->GetArchetype()))
		{
			bIsInheritedInstance = true;
		}
	}
}

void FObjectExport::ResetObject()
{
	Object = nullptr;
	bExportLoadFailed = false;
	bWasFiltered = false;
}

FArchive& operator<<(FArchive& Ar, FObjectExport& E)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << E;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FObjectExport& E)
{
	FArchive& BaseArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	Record << SA_VALUE(TEXT("ClassIndex"), E.ClassIndex);
	Record << SA_VALUE(TEXT("SuperIndex"), E.SuperIndex);

	if (BaseArchive.UEVer() >= VER_UE4_TemplateIndex_IN_COOKED_EXPORTS)
	{
		Record << SA_VALUE(TEXT("TemplateIndex"), E.TemplateIndex);
	}

	Record << SA_VALUE(TEXT("OuterIndex"), E.OuterIndex);
	Record << SA_VALUE(TEXT("ObjectName"), E.ObjectName);

	uint32 Save = E.ObjectFlags & RF_Load;
	Record << SA_VALUE(TEXT("ObjectFlags"), Save);

	if (BaseArchive.IsLoading())
	{
		E.ObjectFlags = EObjectFlags(Save & RF_Load);
	}

	if (BaseArchive.UEVer() < VER_UE4_64BIT_EXPORTMAP_SERIALSIZES)
	{
		int32 SerialSize = (int32)E.SerialSize;
		Record << SA_VALUE(TEXT("SerialSize"), SerialSize);
		E.SerialSize = (int64)SerialSize;

		int32 SerialOffset = (int32)E.SerialOffset;
		Record << SA_VALUE(TEXT("SerialOffset"), SerialOffset);
		E.SerialOffset = (int64)SerialOffset;
	}
	else
	{
		Record << SA_VALUE(TEXT("SerialSize"), E.SerialSize);
		Record << SA_VALUE(TEXT("SerialOffset"), E.SerialOffset);
	}

	#define SERIALIZE_BIT_TO_RECORD(bValue) { \
		bool b = E.bValue; \
		Record << SA_VALUE(TEXT(#bValue), b); \
		E.bValue = b; \
	}

	SERIALIZE_BIT_TO_RECORD(bForcedExport);
	SERIALIZE_BIT_TO_RECORD(bNotForClient);
	SERIALIZE_BIT_TO_RECORD(bNotForServer);

	if (BaseArchive.UEVer() < EUnrealEngineObjectUE5Version::REMOVE_OBJECT_EXPORT_PACKAGE_GUID)
	{
		FGuid DummyPackageGuid;
		Record << SA_VALUE(TEXT("PackageGuid"), DummyPackageGuid);
	}

	if (BaseArchive.UEVer() >= EUnrealEngineObjectUE5Version::TRACK_OBJECT_EXPORT_IS_INHERITED)
	{
		SERIALIZE_BIT_TO_RECORD(bIsInheritedInstance);
	}

	Record << SA_VALUE(TEXT("PackageFlags"), E.PackageFlags);

	if (BaseArchive.UEVer() >= VER_UE4_LOAD_FOR_EDITOR_GAME)
	{
		SERIALIZE_BIT_TO_RECORD(bNotAlwaysLoadedForEditorGame);
	}

	if (BaseArchive.UEVer() >= VER_UE4_COOKED_ASSETS_IN_EDITOR_SUPPORT)
	{
		SERIALIZE_BIT_TO_RECORD(bIsAsset);
	}

	if (BaseArchive.UEVer() >= EUnrealEngineObjectUE5Version::OPTIONAL_RESOURCES)
	{
		SERIALIZE_BIT_TO_RECORD(bGeneratePublicHash);
	}

	if (BaseArchive.UEVer() >= VER_UE4_PRELOAD_DEPENDENCIES_IN_COOKED_EXPORTS)
	{
		Record << SA_VALUE(TEXT("FirstExportDependency"), E.FirstExportDependency);
		Record << SA_VALUE(TEXT("SerializationBeforeSerializationDependencies"), E.SerializationBeforeSerializationDependencies);
		Record << SA_VALUE(TEXT("CreateBeforeSerializationDependencies"), E.CreateBeforeSerializationDependencies);
		Record << SA_VALUE(TEXT("SerializationBeforeCreateDependencies"), E.SerializationBeforeCreateDependencies);
		Record << SA_VALUE(TEXT("CreateBeforeCreateDependencies"), E.CreateBeforeCreateDependencies);
	}	
	
	if (!BaseArchive.UseUnversionedPropertySerialization() && BaseArchive.UEVer() >= EUnrealEngineObjectUE5Version::SCRIPT_SERIALIZATION_OFFSET)
	{
		// Note: this path may be taken when saving as well (for fast package duplication)
		Record << SA_VALUE(TEXT("ScriptSerializationStartOffset"), E.ScriptSerializationStartOffset);
		Record << SA_VALUE(TEXT("ScriptSerializationEndOffset"), E.ScriptSerializationEndOffset);
	}
	else if (BaseArchive.IsLoading())
	{
		E.ScriptSerializationStartOffset = 0;
		E.ScriptSerializationEndOffset = 0;
	}

	#undef SERIALIZE_BIT_TO_RECORD
}

/*-----------------------------------------------------------------------------
	FObjectTextExport.
-----------------------------------------------------------------------------*/

void operator<<(FStructuredArchive::FSlot Slot, FObjectTextExport& E)
{
	FArchive& BaseArchive = Slot.GetUnderlyingArchive();
	FString ClassName, OuterName, SuperStructName;
	check(BaseArchive.IsTextFormat());

	if (BaseArchive.IsSaving())
	{
		check(E.Export.Object);

		UClass* ObjClass = E.Export.Object->GetClass();
		if (ObjClass != UClass::StaticClass())
		{
			ClassName = ObjClass->GetFullName();
		}

		if (E.Export.Object->GetOuter() != E.Outer)
		{
			OuterName = E.Export.Object->GetOuter() ? E.Export.Object->GetOuter()->GetFullName() : FString();
		}

		if (UStruct* Struct = Cast<UStruct>(E.Export.Object))
		{
			if (Struct->GetSuperStruct() != nullptr)
			{
				SuperStructName = Struct->GetSuperStruct()->GetFullName();
			}
		}
	}

	Slot << SA_ATTRIBUTE(TEXT("Class"), ClassName);
	Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("Outer"), OuterName, FString());
	Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("SuperStruct"), SuperStructName, FString());

	if (BaseArchive.IsLoading())
	{
		E.ClassName = ClassName;
		E.OuterName = OuterName;
		E.SuperStructName = SuperStructName;
	}

	uint32 Save = E.Export.ObjectFlags & RF_Load;
	Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("ObjectFlags"), Save, 0);
	if (BaseArchive.IsLoading())
	{
		E.Export.ObjectFlags = EObjectFlags(Save & RF_Load);
	}

	#define SERIALIZE_BIT_TO_SLOT(bValue) { \
		bool b = E.Export.bValue; \
		Slot << SA_OPTIONAL_ATTRIBUTE(TEXT(#bValue), b, false); \
		E.Export.bValue = b; \
	}

	SERIALIZE_BIT_TO_SLOT(bForcedExport);
	SERIALIZE_BIT_TO_SLOT(bNotForClient);
	SERIALIZE_BIT_TO_SLOT(bNotForServer);

	if (BaseArchive.UEVer() < EUnrealEngineObjectUE5Version::REMOVE_OBJECT_EXPORT_PACKAGE_GUID)
	{
		FGuid DummyPackageGuid;
		Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("PackageGuid"), DummyPackageGuid, FGuid());
	}

	Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("PackageFlags"), E.Export.PackageFlags, 0);

	SERIALIZE_BIT_TO_SLOT(bNotAlwaysLoadedForEditorGame);
	SERIALIZE_BIT_TO_SLOT(bIsAsset);

	#undef SERIALIZE_BIT_TO_SLOT
}

/*-----------------------------------------------------------------------------
	FObjectImport.
-----------------------------------------------------------------------------*/

FObjectImport::FObjectImport()
	: FObjectResource()
	, SourceIndex(INDEX_NONE)
	, bImportOptional(false)
	, XObject(nullptr)
	, SourceLinker(nullptr)
{
}

FObjectImport::FObjectImport(UObject* InObject)
	: FObjectResource(InObject)
	, ClassPackage(InObject ? InObject->GetClass()->GetOuter()->GetFName() : NAME_None)
	, ClassName(InObject ? InObject->GetClass()->GetFName() : NAME_None)
	, SourceIndex(INDEX_NONE)
	, bImportOptional(false)
	, XObject(InObject)
	, SourceLinker(nullptr)
{
}

FObjectImport::FObjectImport(UObject* InObject, UClass* InClass)
	: FObjectResource(InObject)
	, ClassPackage((InObject && InClass) ? InClass->GetOuter()->GetFName() : NAME_None)
	, ClassName((InObject && InClass) ? InClass->GetFName() : NAME_None)
	, SourceIndex(INDEX_NONE)
	, bImportOptional(false)
	, XObject(InObject)
	, SourceLinker(nullptr)
{
}

FArchive& operator<<( FArchive& Ar, FObjectImport& I )
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << I;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FObjectImport& I)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	Record << SA_VALUE(TEXT("ClassPackage"), I.ClassPackage);
	Record << SA_VALUE(TEXT("ClassName"), I.ClassName);
	Record << SA_VALUE(TEXT("OuterIndex"), I.OuterIndex);
	Record << SA_VALUE(TEXT("ObjectName"), I.ObjectName);

	if (Slot.GetUnderlyingArchive().UEVer() >= VER_UE4_NON_OUTER_PACKAGE_IMPORT)
	{
		if (Slot.GetUnderlyingArchive().IsSaving())
		{
			FName SavePackageName = I.PackageName;
			if (Slot.GetUnderlyingArchive().IsFilterEditorOnly() && SavePackageName.IsNone())
			{
				// Hack to avoid collecting the "None" name in cooked packages that does not use the external package feature.
				// This minimizes the patch size impact, else the PackageName addition would impact the majority of cooked packages.
				// Serialize an existing name table entry instead, and reset it on load for the legacy cooked loading path.
				// Note that the PackageStoreOptimizer strips this data anyway, so it will never be loaded in the iostore loading path.
				ensureMsgf(I.ObjectName.IsNone() || I.ObjectName != I.PackageName,
					TEXT("Unexpected import names in archive %s: ObjectName=%s, PackageName=%s"),
					*Slot.GetUnderlyingArchive().GetArchiveName(), *I.ObjectName.ToString(), *I.PackageName.ToString());
				SavePackageName = I.ObjectName;
			}
			Record << SA_VALUE(TEXT("PackageName"), SavePackageName);
		}
		else
		{
			Record << SA_VALUE(TEXT("PackageName"), I.PackageName);
		}
	}

	if (Slot.GetUnderlyingArchive().UEVer() >= EUnrealEngineObjectUE5Version::OPTIONAL_RESOURCES)
	{
		Record << SA_VALUE(TEXT("bImportOptional"), I.bImportOptional);
	}

	if (Slot.GetUnderlyingArchive().IsLoading())
	{
		I.SourceLinker = NULL;
		I.SourceIndex = INDEX_NONE;
		I.XObject = NULL;
		if (Slot.GetUnderlyingArchive().IsFilterEditorOnly() && I.PackageName == I.ObjectName)
		{
			// Reset the data from the above save hack to avoid collecting the "None" name in cooked packages that
			// does not use the external package feature.
			I.PackageName = FName{};
		}
	}
}

FArchive& FObjectDataResource::Serialize(FArchive& Ar, TArray<FObjectDataResource>& DataResources)
{
	Serialize(FStructuredArchiveFromArchive(Ar).GetSlot(), DataResources);
	return Ar;
}

void FObjectDataResource::Serialize(FStructuredArchive::FSlot Slot, TArray<FObjectDataResource>& DataResources)
{
	auto SerializeDataResource = [](FStructuredArchive::FSlot Slot, uint32 Version, FObjectDataResource& D)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		
		Record << SA_VALUE(TEXT("Flags"), D.Flags);
		if (Version >= (uint32)FObjectDataResource::EVersion::AddedCookedIndex)
		{
			Record << SA_VALUE(TEXT("CookedIndex"), D.CookedIndex);
		}
		Record << SA_VALUE(TEXT("SerialOffset"), D.SerialOffset);
		Record << SA_VALUE(TEXT("DuplicateSerialOffset"), D.DuplicateSerialOffset);
		Record << SA_VALUE(TEXT("SerialSize"), D.SerialSize);
		Record << SA_VALUE(TEXT("RawSize"), D.RawSize);
		Record << SA_VALUE(TEXT("OuterIndex"), D.OuterIndex);
		Record << SA_VALUE(TEXT("LegacyBulkDataFlags"), D.LegacyBulkDataFlags);
	};

	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	uint32 Version = static_cast<uint32>(FObjectDataResource::EVersion::Latest);
	Record << SA_VALUE(TEXT("Version"), Version);
	int32 Count = DataResources.Num();
	Record << SA_VALUE(TEXT("Count"), Count);

	if (Count == 0)
	{
		return;
	}

	if (Slot.GetUnderlyingArchive().IsLoading())
	{
		DataResources.SetNum(Count);
	}

	FStructuredArchive::FStream Stream = Record.EnterStream(TEXT("Resources"));
	for (FObjectDataResource& DataResource : DataResources)
	{
		SerializeDataResource(Stream.EnterElement(), Version, DataResource);
	}
}

FCellExport::FCellExport()
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	: CppClassInfo(nullptr)
#else
	: CppClassInfo(NAME_None)
#endif
	, SerialOffset(0)
	, SerialLayoutSize(0)
	, SerialSize(0)
	, Cell(nullptr)
	, FirstExportDependency(-1)
	, SerializationBeforeSerializationDependencies(0)
	, CreateBeforeSerializationDependencies(0)
{
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FCellExport::FCellExport(Verse::VCell* InCell)
	: CppClassInfo(InCell->GetCppClassInfo())
	, SerialOffset(0)
	, SerialLayoutSize(0)
	, SerialSize(0)
	, Cell(InCell)
	, FirstExportDependency(-1)
	, SerializationBeforeSerializationDependencies(0)
	, CreateBeforeSerializationDependencies(0)
{
}
#endif

FArchive& operator<<(FArchive& Ar, FCellExport& E)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << E;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FCellExport& E)
{
	FArchive& BaseArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	FName CppClassInfo;
	FUtf8String VersePath;
	if (BaseArchive.IsSaving())
	{
		CppClassInfo = E.CppClassInfo->Name;
		VersePath = E.VersePath->AsStringView();
	}
	Record << SA_VALUE(TEXT("CppClassInfo"), CppClassInfo);
	Record << SA_VALUE(TEXT("VersePath"), VersePath);
	if (BaseArchive.IsLoading())
	{
		AutoRTFM::UnreachableIfClosed("VerseVM does not support serialization in the closed");

		E.CppClassInfo = Verse::VCppClassInfoRegistry::GetCppClassInfo(*CppClassInfo.ToString());

		Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
		E.VersePath = &Verse::VUniqueString::New(Context, VersePath);
	}
#else
	Record << SA_VALUE(TEXT("CppClassInfo"), E.CppClassInfo);
	Record << SA_VALUE(TEXT("VersePath"), E.VersePath);
#endif

	Record << SA_VALUE(TEXT("SerialOffset"), E.SerialOffset);
	Record << SA_VALUE(TEXT("SerialLayoutSize"), E.SerialLayoutSize);
	Record << SA_VALUE(TEXT("SerialSize"), E.SerialSize);

	Record << SA_VALUE(TEXT("FirstExportDependency"), E.FirstExportDependency);
	Record << SA_VALUE(TEXT("SerializationBeforeSerializationDependencies"), E.SerializationBeforeSerializationDependencies);
	Record << SA_VALUE(TEXT("CreateBeforeSerializationDependencies"), E.CreateBeforeSerializationDependencies);
}

FCellImport::FCellImport()
	: Cell(nullptr)
{
}

FCellImport::FCellImport(Verse::VCell* InCell)
	: Cell(InCell)
{
}

FArchive& operator<<(FArchive& Ar, FCellImport& I)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << I;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FCellImport& I)
{
	FArchive& BaseArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	Record << SA_VALUE(TEXT("PackageIndex"), I.PackageIndex);
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	FUtf8String VersePath;
	if (BaseArchive.IsSaving())
	{
		VersePath = I.VersePath->AsStringView();
	}
	Record << SA_VALUE(TEXT("VersePath"), VersePath);
	if (BaseArchive.IsLoading())
	{
		AutoRTFM::UnreachableIfClosed("VerseVM does not support serialization in the closed");

		Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
		I.VersePath = &Verse::VUniqueString::New(Context, VersePath);
	}
#else
	Record << SA_VALUE(TEXT("VersePath"), I.VersePath);
#endif

	if (Slot.GetUnderlyingArchive().IsLoading())
	{
		I.Cell = nullptr;
	}
}
