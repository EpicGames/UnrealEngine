// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectSerializeAccessScope.h"
#include "UObject/Object.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/Linker.h"
#include "HAL/PlatformStackWalk.h"

namespace UE::CoreUObject::Private
{
#if UE_WITH_OBJECT_HANDLE_TRACKING
	bool IsEnableSerializeAccessWarnings()
	{
		auto IsEnabled = []()
		{
			if (FParse::Param(FCommandLine::Get(), TEXT("EnableSerializeAccessWarnings")))
			{
				return true;
			}
			else if (GConfig)
			{
				bool bConfig = false;
				GConfig->GetBool(TEXT("Core.System.Experimental"), TEXT("EnableSerializeAccessWarnings"), bConfig, GEngineIni);
				return bConfig;
			}
			return false;
		};
		static const bool bIsEnabled = IsEnabled();
		return bIsEnabled;
	}
	static void ScopeCallback(const UObject& Object, const UObject* ReadObject)
	{
		
	}

	struct FObjectSerializeAccessScopeImpl
	{
		FObjectSerializeAccessScopeImpl(bool bEnabled)
			: bEnabled(bEnabled)
		{
			if (bEnabled)
			{
				FObjectHandleReadFunc Func = [this](const TArrayView<const UObject* const>& ReadObject)
				{
					FLocalData& Data = GetThreadLocalData();
					if (!IsEnableSerializeAccessWarnings() || (Data.CurrentScope == nullptr || Data.CurrentScope->bSuspended))
						return;

					const UObject& CurrentObject = Data.CurrentScope->Object;

					bool bIsAlreadyInSet;
					Data.Types.FindOrAdd(CurrentObject.GetClass(), &bIsAlreadyInSet);
					if (bIsAlreadyInSet)
					{
						return;
					}

					const SIZE_T StackTraceSize = 65535;
					ANSICHAR StackTrace[StackTraceSize];
					StackTrace[0] = 0;
					FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, 3);
					if (ReadObject.Num() == 1)
					{
						UE_LOGF(LogLinker, Warning, "%ls trying to access referenced object %ls during serialize\n%ls", *CurrentObject.GetClass()->GetFullName(), (ReadObject[0] ? *ReadObject[0]->GetFullName() : TEXT("None")), ANSI_TO_TCHAR(StackTrace));
					}
					else
					{
						UE_LOGF(LogLinker, Warning, "%ls trying to access an array of referenced objects during serialize\n%ls", *CurrentObject.GetClass()->GetFullName(), ANSI_TO_TCHAR(StackTrace));
					}
				};

				CallbackHandle = AddObjectHandleReadCallback(Func);
			}
		}

		~FObjectSerializeAccessScopeImpl()
		{
			if (CallbackHandle.IsValid())
			{
				RemoveObjectHandleReadCallback(CallbackHandle);
			}
		}

		static FObjectSerializeAccessScopeImpl& Get()
		{
			static FObjectSerializeAccessScopeImpl Impl(IsEnableSerializeAccessWarnings());
			return Impl;
		}

		//thread local data
		struct FLocalData
		{
			FObjectSerializeAccessScope* CurrentScope;

			//types that have been seen
			TSet<const UClass*> Types;
		};

		static FLocalData& GetThreadLocalData()
		{
			static thread_local FLocalData Data;
			return Data;
		}

		void Push(FObjectSerializeAccessScope& Scope)
		{
			if (!bEnabled)
				return;
			FLocalData& LocalData = GetThreadLocalData();
			FObjectSerializeAccessScope* Current = LocalData.CurrentScope;
			Scope.Parent = Current;
			LocalData.CurrentScope = &Scope;
		}

		void Pop(FObjectSerializeAccessScope& Scope)
		{
			if (!bEnabled)
				return;
			
			FLocalData& LocalData = GetThreadLocalData();
			FObjectSerializeAccessScope* Current = LocalData.CurrentScope;
			check(&Scope == Current);
			LocalData.CurrentScope = Scope.Parent;
		}

		void Suspend()
		{
			if (!bEnabled)
				return;
			
			FLocalData LocalData = GetThreadLocalData();
			FObjectSerializeAccessScope* Current = LocalData.CurrentScope;
			if (Current)
			{
				Current->bSuspended = true;
			}
		}
		void Resume()
		{
			if (!bEnabled)
				return;
			
			FLocalData LocalData = GetThreadLocalData();
			FObjectSerializeAccessScope* Current = LocalData.CurrentScope;
			if (Current)
			{
				Current->bSuspended = false;
			}
		}

		FObjectHandleTrackingCallbackId CallbackHandle;
		bool bEnabled;

	};

	FObjectSerializeAccessScope::FObjectSerializeAccessScope(const UObject& Object)
		: Object(Object)
		, Parent(nullptr)
		, bSuspended(false)
	{
		FObjectSerializeAccessScopeImpl::Get().Push(*this);
	}

	FObjectSerializeAccessScope::~FObjectSerializeAccessScope()
	{
		FObjectSerializeAccessScopeImpl::Get().Pop(*this);
	}

	FObjectSerializeAccessScope::FSuspendScope::FSuspendScope()
	{
		FObjectSerializeAccessScopeImpl::Get().Suspend();
	}

	FObjectSerializeAccessScope::FSuspendScope::~FSuspendScope()
	{
		FObjectSerializeAccessScopeImpl::Get().Resume();
	}
#endif
}