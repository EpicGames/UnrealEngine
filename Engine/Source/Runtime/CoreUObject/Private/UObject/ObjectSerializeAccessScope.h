// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/ObjectHandleTracking.h"

#if UE_WITH_OBJECT_HANDLE_TRACKING

namespace UE::CoreUObject::Private
{
	/// @brief scope to log messages when a TObjectPtr is accessed
	struct FObjectSerializeAccessScope
	{
		/// @param object to being tracking for
		FObjectSerializeAccessScope(const UObject& Object);
		~FObjectSerializeAccessScope();

		FObjectSerializeAccessScope(const FObjectSerializeAccessScope&) = delete;
		FObjectSerializeAccessScope& operator=(const FObjectSerializeAccessScope&) = delete;

		///@brief suspends the threads current FObjectSerializeAccessScope if any
		struct FSuspendScope
		{
			FSuspendScope();
			~FSuspendScope();

			FSuspendScope(const FSuspendScope&) = delete;
			FSuspendScope& operator=(const FSuspendScope&) = delete;
		};

	private:
		friend struct FObjectSerializeAccessScopeImpl;

		//object being serialized
		const UObject& Object;

		//parent scope for the serialization stack
		FObjectSerializeAccessScope* Parent;
		
		//suspend to pause logging.
		//progress bars for instance can "pause" serialize
		//and do other work that kick object handle reads
		bool bSuspended;
	};
}
#define UE_SERIALIZE_ACCCESS_SCOPE(Object) UE::CoreUObject::Private::FObjectSerializeAccessScope UE_JOIN(_Scope, __LINE__)(*(Object))
#define UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND() UE::CoreUObject::Private::FObjectSerializeAccessScope::FSuspendScope UE_JOIN(_Suspend, __LINE__)

#else

#define UE_SERIALIZE_ACCCESS_SCOPE(Object)
#define UE_SERIALIZE_ACCCESS_SCOPE_SUSPEND()

#endif