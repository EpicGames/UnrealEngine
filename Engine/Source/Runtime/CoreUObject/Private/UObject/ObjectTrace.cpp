// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectTrace.h"

#if UE_UOBJECT_TRACE_ENABLED

#include "Containers/VersePath.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "Logging/LogCategory.h"
#include "Misc/AssertionMacros.h"
#include "ProfilingDebugging/StringsTrace.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Trace/Config.h"
#include "Trace/Trace.inl"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectBaseUtility.h"

DEFINE_LOG_CATEGORY_STATIC(LogUObjectTrace, Log, All);

UE_TRACE_MINIMAL_CHANNEL_DEFINE(CoreUObjectChannel,
	"Information about the currently loaded UObjects. Used by Object Profiler and Memory Snapshot to explore memory usage.",
	false)

namespace UE::Trace
{

////////////////////////////////////////////////////////////////////////////////////////////////////

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, BeginSnapshot, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SizeOfUObject)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SizeOfUField)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SizeOfUStruct)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SizeOfUClass)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SizeOfUFunction)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SizeOfUPackage)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ObjectArrayNum)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ObjectArrayNumMinusAvailable)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ObjectArrayNumPermanent)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, EndSnapshot, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, ObjectSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ClassId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, OuterId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Flags)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::AnsiString, NameAnsi)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, NameWide)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, VerseExtraSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, Path)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, ResourceSizeExtraSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, SystemMemoryBytes) // exclusive estimated system memory
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, VideoMemoryBytes) // exclusive estimated video memory
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, TotalResourceSizeExtraSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, SystemMemoryBytes) // total estimated system memory
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, VideoMemoryBytes) // total estimated video memory
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, FieldExtraSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, StructExtraSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, SuperId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, InheritanceSuperId)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, StructureSize)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, ClassExtraSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, FunctionExtraSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Flags)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8, NumParms)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint16, ParmsSize)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, PackageExtraSpec, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, Id)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, PackageId)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, LocalFullPath)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, SourcePackageName)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN(CoreUObject, ObjectRef, NoSync)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ReferencerId)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, ObjectId)
UE_TRACE_MINIMAL_EVENT_END()

////////////////////////////////////////////////////////////////////////////////////////////////////

static FAutoConsoleCommand CVarStartCapturingUObjectTraceSnapshot(
	TEXT("obj.TraceSnapshot"),
	TEXT("Capture a UObject Trace Snapshot (minimal tracing)."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			EUObjectTraceFlags Options = EUObjectTraceFlags::None;
			for (const FString& Arg : Args)
			{
				if (Arg == TEXT("totals"))
				{
					Options = EUObjectTraceFlags(uint32(Options) | uint32(EUObjectTraceFlags::IncludeTotalResourceSizes));
				}
				else if (Arg == TEXT("packages"))
				{
					Options = EUObjectTraceFlags(uint32(Options) | uint32(EUObjectTraceFlags::IncludePackageTotalResourceSizes));
				}
			}
			FUObjectTrace::StartCapturingSnapshot(Options);
		})
);

////////////////////////////////////////////////////////////////////////////////////////////////////
// FUObjectTrace
////////////////////////////////////////////////////////////////////////////////////////////////////

std::atomic<bool> FUObjectTrace::bIsCapturingSnapshot = false;

void FUObjectTrace::StartCapturingSnapshot(EUObjectTraceFlags Options)
{
	check(IsInGameThread());

	if (bIsCapturingSnapshot)
	{
		UE_LOGF(LogUObjectTrace, Warning, "Already capturing a UObject Trace Snapshot.");
		return;
	}

	const bool bShouldToggleChannel = !CoreUObjectChannel.IsEnabled();
	if (bShouldToggleChannel)
	{
		const TCHAR* FailReason = nullptr;
		if (!CoreUObjectChannel.Toggle(true, &FailReason))
		{
			UE_LOGF(LogUObjectTrace, Error, "Failed to enable the CoreUObject trace channel.");
			return;
		}
	}

	const uint64 StartTime = FPlatformTime::Cycles64();
	bIsCapturingSnapshot = true;
	OutputBeginSnapshot(Options);

	uint32 ObjectsTraced = 0;
	int32 ObjectArrayNum = GUObjectArray.GetObjectArrayNum();
	for (int32 ObjectIndex = 0; ObjectIndex < ObjectArrayNum; ++ObjectIndex)
	{
		UObjectBase* Object = FIndexToObject::IndexToObject(ObjectIndex, false);
		if (OutputObjectSpec(Object, Options))
		{
			++ObjectsTraced;
		}
	}

	OutputEndSnapshot(Options);
	bIsCapturingSnapshot = false;
	const uint64 EndTime = FPlatformTime::Cycles64();

	UE_LOGF(LogUObjectTrace, Log, "UObject Trace Snapshot captured in %.3f seconds (%u objects).",
		(double)(EndTime - StartTime) * FPlatformTime::GetSecondsPerCycle64(), ObjectsTraced);

	if (bShouldToggleChannel)
	{
		const TCHAR* FailReason = nullptr;
		if (CoreUObjectChannel.Toggle(false, &FailReason))
		{
			UE_LOGF(LogUObjectTrace, Error, "Failed to disable the CoreUObject trace channel.");
		}
	}
}

void FUObjectTrace::OutputBeginSnapshot(EUObjectTraceFlags Options)
{
	int32 ObjectArrayNum = GUObjectArray.GetObjectArrayNum();
	int32 ObjectArrayNumMinusAvailable = GUObjectArray.GetObjectArrayNumMinusAvailable();
	int32 ObjectArrayNumPermanent = GUObjectArray.GetObjectArrayNumPermanent();

	UE_TRACE_MINIMAL_LOG(CoreUObject, BeginSnapshot, CoreUObjectChannel)
		<< BeginSnapshot.Cycle(FPlatformTime::Cycles64())
		<< BeginSnapshot.SizeOfUObject(static_cast<uint32>(sizeof(UObject)))
		<< BeginSnapshot.SizeOfUField(static_cast<uint32>(sizeof(UField)))
		<< BeginSnapshot.SizeOfUStruct(static_cast<uint32>(sizeof(UStruct)))
		<< BeginSnapshot.SizeOfUClass(static_cast<uint32>(sizeof(UClass)))
		<< BeginSnapshot.SizeOfUFunction(static_cast<uint32>(sizeof(UFunction)))
		<< BeginSnapshot.SizeOfUPackage(static_cast<uint32>(sizeof(UPackage)))
		<< BeginSnapshot.ObjectArrayNum(ObjectArrayNum)
		<< BeginSnapshot.ObjectArrayNumMinusAvailable(ObjectArrayNumMinusAvailable)
		<< BeginSnapshot.ObjectArrayNumPermanent(ObjectArrayNumPermanent)
	;
}

void FUObjectTrace::OutputEndSnapshot(EUObjectTraceFlags Options)
{
	UE_TRACE_MINIMAL_LOG(CoreUObject, EndSnapshot, CoreUObjectChannel)
		<< EndSnapshot.Cycle(FPlatformTime::Cycles64());
}

bool FUObjectTrace::OutputObjectSpec(UObjectBase* Object, EUObjectTraceFlags Options)
{
	if (!bool(CoreUObjectChannel))
	{
		return false;
	}

	if (!Object)
	{
		return false;
	}

	constexpr uint32 InvalidObjectId = uint32(-1);

	const uint32 Id = Object->GetUniqueID();
	if (Id == InvalidObjectId)
	{
		return false;
	}

	UObject* Obj = Cast<UObject>(Object);
	check(Obj);

	UClass* ObjectClass = Object->GetClass();
	UObject* OuterObject = Object->GetOuter();
	const EObjectFlags Flags = Object->GetFlags();

	FString ObjectName;
	FWideStringView ObjectNameWide;
	TAnsiStringBuilder<1024> ObjectNameAnsiBuilder;
	if (!Object->GetFName().TryAppendAnsiString(ObjectNameAnsiBuilder))
	{
		Object->GetFName().ToString(ObjectName);
		ObjectNameWide = ObjectName;
	}
	FAnsiStringView ObjectNameAnsi = ObjectNameAnsiBuilder.ToView();

	UE_TRACE_MINIMAL_LOG(CoreUObject, ObjectSpec, CoreUObjectChannel)
		<< ObjectSpec.Id(Id)
		<< ObjectSpec.ClassId(ObjectClass ? ObjectClass->GetUniqueID() : InvalidObjectId)
		<< ObjectSpec.OuterId(OuterObject ? OuterObject->GetUniqueID() : InvalidObjectId)
		<< ObjectSpec.Flags(uint32(Flags))
		<< ObjectSpec.NameAnsi(ObjectNameAnsi.GetData(), ObjectNameAnsi.Len())
		<< ObjectSpec.NameWide(ObjectNameWide.GetData(), ObjectNameWide.Len());

	constexpr bool bTraceVerseExtraSpec = true;
	if (bTraceVerseExtraSpec)
	{
		UE::Core::FVersePath VersePath = Obj->GetVersePath();
		if (VersePath.IsValid())
		{
			FStringView VersePathView = VersePath.AsStringView();
			UE_TRACE_MINIMAL_LOG(CoreUObject, VerseExtraSpec, CoreUObjectChannel)
				<< VerseExtraSpec.Id(Id)
				<< VerseExtraSpec.Path(VersePathView.GetData(), VersePathView.Len());
		}
	}

	constexpr bool bTraceResourceSize = true;
	if (bTraceResourceSize)
	{
		// Exclusive Estimated (System/Video) Memory Size
		FResourceSizeEx ResourceSize(EResourceSizeMode::Exclusive);
		Obj->GetResourceSizeEx(ResourceSize);
		SIZE_T SystemMemoryBytes = ResourceSize.GetDedicatedSystemMemoryBytes() + ResourceSize.GetUnknownMemoryBytes();
		SIZE_T VideoMemoryBytes = ResourceSize.GetDedicatedVideoMemoryBytes();

		if (SystemMemoryBytes != 0 || VideoMemoryBytes != 0)
		{
			UE_TRACE_MINIMAL_LOG(CoreUObject, ResourceSizeExtraSpec, CoreUObjectChannel)
				<< ResourceSizeExtraSpec.Id(Id)
				<< ResourceSizeExtraSpec.SystemMemoryBytes(SystemMemoryBytes)
				<< ResourceSizeExtraSpec.VideoMemoryBytes(VideoMemoryBytes);
		}
	}

	bool bTraceTotalResourceSize = ((uint32(Options) & uint32(EUObjectTraceFlags::IncludeTotalResourceSizes)) != 0);
	if (bTraceTotalResourceSize)
	{
		const bool bIncludePackageTotalResourceSizes = ((uint32(Options) & uint32(EUObjectTraceFlags::IncludePackageTotalResourceSizes)) != 0);
		if (!bIncludePackageTotalResourceSizes && (Cast<UPackage>(Object) != nullptr))
		{
			bTraceTotalResourceSize = false;
		}
	}

	if (bTraceTotalResourceSize)
	{
		// Total Estimated (System/Video) Memory Size
		FResourceSizeEx TotalResourceSize(EResourceSizeMode::EstimatedTotal);
		Obj->GetResourceSizeEx(TotalResourceSize);
		SIZE_T SystemMemoryBytes = TotalResourceSize.GetDedicatedSystemMemoryBytes() + TotalResourceSize.GetUnknownMemoryBytes();
		SIZE_T VideoMemoryBytes = TotalResourceSize.GetDedicatedVideoMemoryBytes();

		if (SystemMemoryBytes != 0 || VideoMemoryBytes != 0)
		{
			UE_TRACE_MINIMAL_LOG(CoreUObject, TotalResourceSizeExtraSpec, CoreUObjectChannel)
				<< TotalResourceSizeExtraSpec.Id(Id)
				<< TotalResourceSizeExtraSpec.SystemMemoryBytes(SystemMemoryBytes)
				<< TotalResourceSizeExtraSpec.VideoMemoryBytes(VideoMemoryBytes);
		}
	}

	constexpr bool bTraceExtraSpecs = true;
	if (bTraceExtraSpecs)
	{
		// UField -> UObject
		if (UField* ObjAsField = Cast<UField>(Object))
		{
			UE_TRACE_MINIMAL_LOG(CoreUObject, FieldExtraSpec, CoreUObjectChannel)
				<< FieldExtraSpec.Id(Id);
		}

		// UStruct -> UField -> UObject
		if (UStruct* ObjAsStruct = Cast<UStruct>(Object))
		{
			UStruct* ObjSuper = ObjAsStruct->GetSuperStruct();
			UStruct* ObjInheritanceSuper = ObjAsStruct->GetInheritanceSuper();
			const int32 StructureSize = ObjAsStruct->GetStructureSize();
			UE_TRACE_MINIMAL_LOG(CoreUObject, StructExtraSpec, CoreUObjectChannel)
				<< StructExtraSpec.Id(Id)
				<< StructExtraSpec.SuperId(ObjSuper ? ObjSuper->GetUniqueID() : InvalidObjectId)
				<< StructExtraSpec.InheritanceSuperId(ObjInheritanceSuper ? ObjInheritanceSuper->GetUniqueID() : InvalidObjectId)
				<< StructExtraSpec.StructureSize(StructureSize);
		}

		// UClass -> UStruct -> UField -> UObject
		if (UClass* ObjAsClass = Cast<UClass>(Object))
		{
			UE_TRACE_MINIMAL_LOG(CoreUObject, ClassExtraSpec, CoreUObjectChannel)
				<< ClassExtraSpec.Id(Id);
		}

		// UFunction -> UStruct -> UField -> UObject
		if (UFunction* ObjAsFunction = Cast<UFunction>(Object))
		{
			UE_TRACE_MINIMAL_LOG(CoreUObject, FunctionExtraSpec, CoreUObjectChannel)
				<< FunctionExtraSpec.Id(Id)
				<< FunctionExtraSpec.Flags(uint32(ObjAsFunction->FunctionFlags))
				<< FunctionExtraSpec.NumParms(ObjAsFunction->NumParms)
				<< FunctionExtraSpec.ParmsSize(ObjAsFunction->ParmsSize);
		}

		// UPackage -> UObject
		if (UPackage* ObjAsPackage = Cast<UPackage>(Object))
		{
			const uint64 PackageId = ObjAsPackage->GetPackageId().Value();
			const FPackagePath& PackagePath = ObjAsPackage->GetLoadedPath();
			FString LocalFullPath = PackagePath.GetLocalFullPath();
			FWideStringView LocalFullPathView = LocalFullPath;

			// Emit the editor source package name when it differs from the runtime name (e.g. for streamed/instanced level packages).
			FString SourcePkgNameStr;
			const FName SourcePkgFName = PackagePath.GetPackageFName();
			if (!SourcePkgFName.IsNone() && SourcePkgFName != ObjAsPackage->GetFName())
			{
				SourcePkgFName.ToString(SourcePkgNameStr);
			}
			FWideStringView SourcePkgNameView = SourcePkgNameStr;

			UE_TRACE_MINIMAL_LOG(CoreUObject, PackageExtraSpec, CoreUObjectChannel)
				<< PackageExtraSpec.Id(Id)
				<< PackageExtraSpec.PackageId(PackageId)
				<< PackageExtraSpec.LocalFullPath(LocalFullPathView.GetData(), LocalFullPathView.Len())
				<< PackageExtraSpec.SourcePackageName(SourcePkgNameView.GetData(), SourcePkgNameView.Len());
		}

		// UWorld -> UObject
		// ULevel -> UObject
		// UTexture -> UStreamableRenderAsset -> UObject
		// UMaterial -> UObject
		// UStaticMesh -> UStreamableRenderAsset -> UObject
		// USkeletalMesh -> USkinnedAsset -> UStreamableRenderAsset -> UObject
		// AActor -> UObject
		//TODO: Object->TraceExtraSpec();
	}

	return true;
}

void FUObjectTrace::OutputObjectRef(UObjectBase* Referencer, UObjectBase* Object)
{
	if (!bool(CoreUObjectChannel))
	{
		return;
	}

	check(Referencer);
	check(Object);

	constexpr uint32 InvalidObjectId = uint32(-1);

	uint32 ReferencerId = Referencer->GetUniqueID();
	if (ReferencerId == InvalidObjectId)
	{
		return;
	}

	uint32 ObjectId = Object->GetUniqueID();
	if (ObjectId == InvalidObjectId)
	{
		return;
	}

	UE_TRACE_MINIMAL_LOG(CoreUObject, ObjectRef, CoreUObjectChannel)
		<< ObjectRef.ReferencerId(ReferencerId)
		<< ObjectRef.ObjectId(ObjectId);
}

} // namespace UE::Trace

#endif // UE_UOBJECT_TRACE_ENABLED
