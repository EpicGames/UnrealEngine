// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ObjectVisibility.h"
#include "UObject/UObjectThreadContext.h"
#include "HAL/IConsoleManager.h"

static bool GUseObjectVisibilityFilterForAsyncLoading = true;
static FAutoConsoleVariableRef CVarUseObjectVisibilityFilterForAsyncLoading(
	TEXT("s.UseObjectVisibilityFilterForAsyncLoading"),
	GUseObjectVisibilityFilterForAsyncLoading,
	TEXT("When active and supported by the current loader, will prevent objects still in the early loading phase from being discovered during postload\n")
	TEXT("to avoid race conditions and manipulation of objects that still haven't been deserialized.")
);

namespace UE
{

	EInternalObjectFlags GetAsyncLoadingInternalFlagsExclusion()
	{
		if (IsInAsyncLoadingThread() || IsInParallelLoadingThread())
		{
			return GUseObjectVisibilityFilterForAsyncLoading ? FUObjectThreadContext::Get().AsyncVisibilityFilter : EInternalObjectFlags::None;
		}

		return EInternalObjectFlags_AsyncLoading;
	}

} // namespace UE