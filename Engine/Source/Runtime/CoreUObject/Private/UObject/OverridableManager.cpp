// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/OverridableManager.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/PropertyNames.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OverridableManager)

/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 */
 FOverridableManager* FOverridableManager::OverridableManager = nullptr;

void FOverridableManager::Create()
{
	if (!OverridableManager)
	{
		OverridableManager = new FOverridableManager();
	}
}

bool FOverridableManager::IsEnabled(TNotNull<const UObject*> Object)
{
#if WITH_EDITORONLY_DATA
	return !GetAnnotation(Object).IsDefault();
#else
	return false;
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::Enable(TNotNull<UObject*> Object)
{
#if WITH_EDITORONLY_DATA
	FindOrAdd(Object);
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::Disable(TNotNull<UObject*> Object, bool bPropagateToSubObject/*= false*/)
{
#if WITH_EDITORONLY_DATA
	RemoveAnnotation(Object);

	if (bPropagateToSubObject)
	{
		TArray<UObject*> InstancedSubObjects;
		GetObjectsWithOuter(Object, InstancedSubObjects, EGetObjectsFlags::None);
		for (UObject* InstancedSubObject : InstancedSubObjects)
		{
			checkf(InstancedSubObject, TEXT("Expecting non null SubObjects"));
			DisableInstancedSubObject(Object, InstancedSubObject);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::DisableInstancedSubObject(TNotNull<const UObject*> Object, TNotNull<UObject*> InstancedSubObject)
{
#if WITH_EDITORONLY_DATA
	if (InstancedSubObject->IsIn(Object))
	{
		Disable(InstancedSubObject, /*bPropagateToSubObject*/ true);
	}
#endif // WITH_EDITORONLY_DATA
}


void FOverridableManager::InheritEnabledFrom(TNotNull<UObject*> Object, const UObject* DefaultData)
{
#if WITH_EDITORONLY_DATA
	if (!IsEnabled(Object))
	{
		const UObject* Outer = Object->GetOuter();
		if ((Outer && IsEnabled(Outer)) || (DefaultData && IsEnabled(DefaultData)))
		{
			Enable(Object);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

bool FOverridableManager::NeedSubObjectTemplateInstantiation(TNotNull<const UObject*> Object)
{
#if WITH_EDITORONLY_DATA
	if (const FOverriddenPropertySet* OverriddenProperties = Find(Object))
	{
		return OverriddenProperties->bNeedsSubobjectTemplateInstantiation;
	}
	return false;
#else
	return NeedsSubobjectTemplateInstantiation.Get(Object);
#endif // WITH_EDITORONLY_DATA
}

FOverriddenPropertySet* FOverridableManager::GetOverriddenPropertiesInternal(TNotNull<UObject*> Object)
{
#if WITH_EDITORONLY_DATA
	return Find(Object);
#else
	return nullptr;
#endif // WITH_EDITORONLY_DATA
}

const FOverriddenPropertySet* FOverridableManager::GetOverriddenPropertiesInternal(TNotNull<const UObject*> Object)
{
#if WITH_EDITORONLY_DATA
	return Find(Object);
#else
	return nullptr;
#endif // WITH_EDITORONLY_DATA
}


FOverriddenPropertySet* FOverridableManager::RestoreOverrideOperation(TNotNull<UObject*> Object, EOverriddenPropertyOperation Operation, const bool bNeedsSubobjectTemplateInstantiation, const bool bDontClearOverrides)
{
	// When sub property overrides are going to be restored, it will turn this object back to modified, so let's not restore that operation.
	// This allows cleanup of the modified state if we end up not overriding any sub properties
	if (Operation == EOverriddenPropertyOperation::Modified)
	{
		Operation = EOverriddenPropertyOperation::None;
	}

#if WITH_EDITORONLY_DATA
	FOverriddenPropertySet& ObjectOverriddenProperties = FindOrAdd(Object);

	// IDOs need to deserialize a couple times in a row and they use ArMergeOverrides to help. This sets the bDontClearOverrides parameter for this function,
	// which tells us not to stomp the existing overrides. if bDontClearOverrides is true, it's important that we don't reset the tree or change non-none 
	// property operations to None.
	if (!bDontClearOverrides)
	{
		ObjectOverriddenProperties.Reset(/*bShouldDirtyObject*/false);
	}

	if (!bDontClearOverrides || ObjectOverriddenProperties.GetOverriddenPropertyOperation(nullptr, nullptr) == EOverriddenPropertyOperation::None)
	{
		ObjectOverriddenProperties.RestoreOverriddenPropertyOperation(Operation, /*CurrentPropertyChain*/nullptr, /*Property*/nullptr);
		ObjectOverriddenProperties.bNeedsSubobjectTemplateInstantiation = bNeedsSubobjectTemplateInstantiation;
	}
	return &ObjectOverriddenProperties;
#else
	if (bNeedsSubobjectTemplateInstantiation)
	{
		NeedsSubobjectTemplateInstantiation.Set(Object);
	}
	else
	{
		NeedsSubobjectTemplateInstantiation.Clear(Object);
	}
	return nullptr;
#endif // WITH_EDITORONLY_DATA

}

void FOverridableManager::RestoreOverrideState(TNotNull<const UObject*> OldObject, TNotNull<UObject*> NewObject)
{
	if (const FOverriddenPropertySet* OldOverriddenProperties = GetOverriddenProperties(OldObject))
	{
		if (FOverriddenPropertySet* NewOverriddenProperties = GetOverriddenProperties(NewObject))
		{
			NewOverriddenProperties->RestoreOverriddenState(*OldOverriddenProperties);
		}
	}
}

EOverriddenState FOverridableManager::GetOverriddenState(TNotNull<UObject*> Object, 
	const TFunction<bool(TNotNull<const FProperty*>)>& ConsiderPropertyForOverriddenState /*= TFunction<bool(TNotNull<const FProperty*>)>*/, 
	const FPropertyVisitorPath& Path /*= FPropertyVisitorPath()*/, bool* bOutInheritedState /*= nullptr*/, bool bSkipAddedState /*= false*/,
	const TFunction<bool(TNotNull<const UObject*>)>& ConsiderSubObjectForOverriddenState /*= {}*/)
{
#if WITH_EDITORONLY_DATA
	if (const FOverriddenPropertySet* OverriddenProperties = GetOverriddenProperties(Object))
	{
		if (Path.Num() == 0)
		{
			// Transient should not affect the overridden state
			if (Object->HasAnyFlags(RF_Transient))
			{
				return EOverriddenState::NoOverrides;
			}

			// Looking if the Archetype of the object was a CDO is not working in the case of a delete and readd.
			// So now we explicitly use the bWasAdded to remember if an object was added.
			if (!bSkipAddedState && OverriddenProperties->WasAdded())
			{
				return EOverriddenState::Added;
			}
		}

		bool bAllOverridden = true;
		bool bHasOverrides = false;
		Object->GetClass()->Visit(Object, [this, Object, OverriddenProperties, Path, &bAllOverridden, &bHasOverrides, &ConsiderPropertyForOverriddenState, &bOutInheritedState, &ConsiderSubObjectForOverriddenState](const FPropertyVisitorContext& Context)
		{
			const EOverriddenPropertyOperation Operation = OverriddenProperties->GetOverriddenPropertyOperation(Context.Path.GetRootIterator());
			bool bIsEqual = false;

			const FProperty* Property = Context.Path.Top().Property;
			const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property);

			if (Context.Path.Num() <= Path.Num())
			{
				if (!Context.Path.Contained(Path, &bIsEqual))
				{
					return EPropertyVisitorControlFlow::StepOver;
				}

				// Recurse if we can tell that from the operation that it contains overrides,
				// if the operation is none, still recurse if it is an object property
				if (Operation == EOverriddenPropertyOperation::Modified ||
					(ObjectProperty && Operation == EOverriddenPropertyOperation::None))
				{
					return EPropertyVisitorControlFlow::StepInto;
				}

				if (Operation == EOverriddenPropertyOperation::Replace)
				{
					bHasOverrides = true;
					if (bOutInheritedState)
					{
						*bOutInheritedState = !bIsEqual;
					}
				}
				else
				{
					bAllOverridden = false;
				}

				return EPropertyVisitorControlFlow::Stop;
			}

			// Skip always overridden properties
			if (Property->HasAnyPropertyFlags(CPF_ExperimentalAlwaysOverriden))
			{
				return EPropertyVisitorControlFlow::StepOver;
			}

			// Check if the property should be considered for the overridden state evaluation 
			if (ConsiderPropertyForOverriddenState.IsSet() && !ConsiderPropertyForOverriddenState(Property))
			{
				return EPropertyVisitorControlFlow::StepOver;
			}

			// Check if the SubObject should be considered for the overridden state evaluation 
			UObject* SubObject = nullptr;
			if (ObjectProperty)
			{
				SubObject = ObjectProperty->GetObjectPropertyValue(Context.Data.PropertyData);
				if (SubObject)
				{
					if (!SubObject->IsIn(Object))
					{
						SubObject = nullptr;
					}					
					else if (ConsiderSubObjectForOverriddenState && !ConsiderSubObjectForOverriddenState(SubObject))
					{
						return EPropertyVisitorControlFlow::StepOver;
					}
				}
			}
				
			// No need to dig deeper if this is a replace or an add
			if (Operation == EOverriddenPropertyOperation::Replace || 
				Operation == EOverriddenPropertyOperation::Add)
			{
				bHasOverrides = true;
				return !bAllOverridden ? EPropertyVisitorControlFlow::Stop : EPropertyVisitorControlFlow::StepOver;
			}

			// Recurse if the property contains inner ones
			if (Context.Path.Top().bContainsInnerProperties)
			{
				if (Operation == EOverriddenPropertyOperation::Modified)
				{
					// We will walk through all sub-properties below, except for removed and added elements, so we test for those here.
					FArchiveSerializedPropertyChain CurrentPropertyChain = Context.Path.ToSerializedPropertyChain();
					if (const FOverriddenPropertyNode* Node = OverriddenProperties->GetOverriddenPropertyNode(&CurrentPropertyChain))
					{
						bHasOverrides |= Node->GetSubPropertyNodes().ContainsByPredicate([](const FOverriddenPropertyNode& SubNode)
							{
								return SubNode.GetOperation() == EOverriddenPropertyOperation::Remove || 
									SubNode.GetOperation() == EOverriddenPropertyOperation::Add;
							});
					}
				}

				if (ObjectProperty)
				{
					if (SubObject) // Only recurse on object properties if they point to an instanced subobject.
					{
						switch(GetOverriddenState(SubObject, ConsiderPropertyForOverriddenState, {}, nullptr, false, ConsiderSubObjectForOverriddenState))
						{
						case EOverriddenState::NoOverrides:
							bAllOverridden = false;
							break;
						case EOverriddenState::HasOverrides:
							bAllOverridden = false;
							bHasOverrides = true;
							break;
						case EOverriddenState::AllOverridden:
							bHasOverrides = true;
							break;
						default:
							break;
						}
						return EPropertyVisitorControlFlow::StepOver;
					}
				}
				else
				{
					return EPropertyVisitorControlFlow::StepInto;
				}
			}

			
			if (Operation == EOverriddenPropertyOperation::None)
			{
				bAllOverridden = false;
				return bHasOverrides ? EPropertyVisitorControlFlow::Stop : EPropertyVisitorControlFlow::StepOver;
			}

			return EPropertyVisitorControlFlow::Stop;
		});

		if (bHasOverrides)
		{
			return bAllOverridden ? EOverriddenState::AllOverridden : EOverriddenState::HasOverrides;
		}
	}
#endif // WITH_EDITORONLY_DATA
	return EOverriddenState::NoOverrides;
}

void FOverridableManager::OverrideAllObjectProperties(TNotNull<UObject*> Object)
{
#if WITH_EDITORONLY_DATA
	if (FOverriddenPropertySet* ThisObjectOverriddenProperties = Find(Object))
	{
		// Passing no property node means we are overriding the object itself
		ThisObjectOverriddenProperties->OverrideProperty(FPropertyVisitorPath::InvalidIterator(), Object);
	}
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::OverrideProperty(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath)
{
#if WITH_EDITORONLY_DATA
	if (FOverriddenPropertySet* ThisObjectOverriddenProperties = Find(Object))
	{
		ThisObjectOverriddenProperties->OverrideProperty(PropertyPath.GetRootIterator(), Object);
	}
#endif // WITH_EDITORONLY_DATA
}

bool FOverridableManager::ClearOverriddenProperty(TNotNull<UObject*> Object, FPropertyVisitorPath::Iterator PropertyIterator)
{
#if WITH_EDITORONLY_DATA
	if (FOverriddenPropertySet* ThisObjectOverriddenProperties = Find(Object))
	{
		return ThisObjectOverriddenProperties->ClearOverriddenProperty(PropertyIterator);
	}
#endif // WITH_EDITORONLY_DATA
	return false;
}

void FOverridableManager::PreOverrideProperty(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, const EPropertyChangeType::Type ChangeType)
{
#if WITH_EDITORONLY_DATA
	NotifyPropertyChange(EPropertyNotificationType::PreEdit, Object, PropertyPath.GetRootIterator(), ChangeType);
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::PostOverrideProperty(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, const EPropertyChangeType::Type ChangeType)
{
#if WITH_EDITORONLY_DATA
	NotifyPropertyChange(EPropertyNotificationType::PostEdit, Object, PropertyPath.GetRootIterator(), ChangeType);
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::NotifyPropertyChange(const EPropertyNotificationType Notification, TNotNull<UObject*> Object, FPropertyVisitorPath::Iterator PropertyIterator, const EPropertyChangeType::Type ChangeType)
{
#if WITH_EDITORONLY_DATA
	if (FOverriddenPropertySet* ThisObjectOverriddenProperties = Find(Object))
	{
		ThisObjectOverriddenProperties->NotifyPropertyChange(Notification, PropertyIterator, ChangeType, Object);
	}
#endif // WITH_EDITORONLY_DATA
}

EOverriddenPropertyOperation FOverridableManager::GetOverriddenPropertyOperation(TNotNull<UObject*> Object, FPropertyVisitorPath::Iterator PropertyIterator, bool* bOutInheritedOperation)
{
#if WITH_EDITORONLY_DATA
	if (const FOverriddenPropertySet* ThisObjectOverriddenProperties = Find(Object))
	{
		return ThisObjectOverriddenProperties->GetOverriddenPropertyOperation(PropertyIterator, bOutInheritedOperation);
	}
#endif // WITH_EDITORONLY_DATA
	return EOverriddenPropertyOperation::None;
}

void FOverridableManager::ClearOverrides(TNotNull<UObject*> Object)
{
#if WITH_EDITORONLY_DATA
	if (FOverriddenPropertySet* ThisObjectOverriddenProperties = Find(Object))
	{
		ThisObjectOverriddenProperties->Reset(/*bShouldDirtyObject*/true);
	}
	PropagateClearOverridesToInstancedSubObjects(Object);
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::ClearInstancedSubObjectOverrides(TNotNull<const UObject*> Object, TNotNull<UObject*> InstancedSubObject)
{
#if WITH_EDITORONLY_DATA
	if (InstancedSubObject->IsIn(Object))
	{
		ClearOverrides(InstancedSubObject);
	}
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::PropagateClearOverridesToInstancedSubObjects(TNotNull<UObject*> Object)
{
#if WITH_EDITORONLY_DATA
	// Cannot use TPropertyValueIterator<FObjectProperty> here because the object might not have 
	// the object class setup correctly when this is called from PostInit.
	// @todo: need to figure out why this is a problem. Maybe we should not even this method during PostInit.
	TArray<UObject*> InstancedSubObjects;
	GetObjectsWithOuter(Object, InstancedSubObjects, EGetObjectsFlags::None);
	for (UObject* InstancedSubObject : InstancedSubObjects)
	{
		checkf(InstancedSubObject, TEXT("Expecting non null SubObjects"));

		// There are some cases where the property has information about that should be an instanced subobject, but it is not owned by us.
		ClearInstancedSubObjectOverrides(Object, InstancedSubObject);
	}
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::SerializeOverriddenProperties(TNotNull<UObject*> Object, FStructuredArchive::FRecord ObjectRecord)
{
#if WITH_EDITORONLY_DATA
	const FArchiveState& ArchiveState = ObjectRecord.GetArchiveState();
	FOverriddenPropertySet* OverriddenProperties = ArchiveState.IsSaving() ? GetOverriddenProperties(Object) : nullptr;
	TOptional<FStructuredArchiveSlot> OverriddenPropertiesSlot = ObjectRecord.TryEnterField(TEXT("OverridenProperties"), OverriddenProperties != nullptr);
	if (OverriddenPropertiesSlot.IsSet())
	{
		EOverriddenPropertyOperation Operation = OverriddenProperties ? OverriddenProperties->GetOverriddenPropertyOperation((FArchiveSerializedPropertyChain*)nullptr, (FProperty*)nullptr) : EOverriddenPropertyOperation::None;
		*OverriddenPropertiesSlot << SA_ATTRIBUTE( TEXT("OverriddenOperation"), Operation);

		if (ArchiveState.IsLoading())
		{
			OverriddenProperties = RestoreOverrideOperation(Object, Operation, /*bNeedsSubobjectTemplateInstantiation*/false);
			checkf(OverriddenProperties, TEXT("Expecting a overridden property set returned"));
		}

		if (Operation != EOverriddenPropertyOperation::None)
		{
			FOverriddenPropertySet::StaticStruct()->SerializeItem(*OverriddenPropertiesSlot, OverriddenProperties, /* Defaults */ nullptr);
		}
	}
	else if (ArchiveState.IsLoading())
	{
		Disable(Object);
	}
#endif // WITH_EDITORONLY_DATA
}

FOverridableManager::FOverridableManager()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReinstanced.AddRaw(this, &FOverridableManager::HandleObjectsReInstantiated);
	FCoreUObjectDelegates::OnVerseDeadObjectReferences.AddRaw(this, &FOverridableManager::HandleDeadObjectReferences);
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddRaw(this, &FOverridableManager::PreOverridePropertyChain);
	FCoreUObjectDelegates::OnPostObjectPropertyChanged.AddRaw(this, &FOverridableManager::PostOverrideProperty);
#endif
}

void FOverridableManager::HandleObjectsReInstantiated(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
{
#if WITH_EDITORONLY_DATA
	UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_ReadOnly);
	for (const auto& Pair : GetAnnotationMap())
	{
		if (FOverriddenPropertySet* OverriddenProperties = Pair.Value.OverriddenProperties.Get())
		{
			OverriddenProperties->HandleObjectsReInstantiated(OldToNewInstanceMap);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void FOverridableManager::HandleDeadObjectReferences(const TSet<UClass*>& DeadClasses, const TSet<UObject*>& ActiveInstances, const TSet<UObject*>& TemplateInstances)
{
#if WITH_EDITORONLY_DATA
	UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_ReadOnly);
	for (const auto& Pair : GetAnnotationMap())
	{
		if (FOverriddenPropertySet* OverriddenProperties = Pair.Value.OverriddenProperties.Get())
		{
			OverriddenProperties->HandleDeadObjectReferences(ActiveInstances, TemplateInstances);
		}
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
FOverriddenPropertySet* FOverridableManager::Find(TNotNull<const UObject*> Object)
{
	FOverriddenPropertyAnnotation Annotation = GetAnnotation(Object);
	return Annotation.OverriddenProperties.Get();
}

FOverriddenPropertySet& FOverridableManager::FindChecked(TNotNull<const UObject*> Object)
{
	FOverriddenPropertyAnnotation Annotation = GetAnnotation(Object);
	checkf(!Annotation.IsDefault(), TEXT("Caller is expecting the object to have overridable serialization enabled"));
	return *Annotation.OverriddenProperties.Get();
}

FOverriddenPropertySet& FOverridableManager::FindOrAdd(TNotNull<UObject*> Object)
{
	FOverriddenPropertyAnnotation Annotation = GetAnnotation(Object);
	if (Annotation.IsDefault())
	{
		Annotation.OverriddenProperties = MakeShared<FOverriddenPropertySet>(Object);
		AddAnnotation(Object, Annotation);
	}

	return *Annotation.OverriddenProperties;
}
#endif // WITH_EDITORONLY_DATA
