// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/OverriddenObjectsExternalHandler.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/PropertyBagRepository.h"

#if WITH_EDITOR

uint32 GetTypeHash(const FPropertyArrayExternalObjectHandler::FPropertyKey& Key)
{
	uint32 Hash = GetTypeHash(Key.ClassName);
	Hash = HashCombine(Hash, GetTypeHash(Key.PropertyName));
	return Hash;
}

FPropertyArrayExternalObjectHandler::FPropertyKey::FPropertyKey(FName InClassName, FName InPropertyName) :
	ClassName(InClassName), PropertyName(InPropertyName)
{
}

FPropertyArrayExternalObjectHandler::FPropertyKey::FPropertyKey(UObject* Object, const FProperty* Property)
{
	PropertyName = Property->GetFName();
	UStruct* Struct = Cast<UStruct>(Property->Owner.ToUObject());

	if (UE::IsClassOfInstanceDataObjectClass(Object->GetClass()))
	{
		// mirrored from InstanceDataObjectUtils.cpp
		static const FName NAME_TemplateStructName(ANSITEXTVIEW("TemplateStructName"));

		// use the ClassName the IDO class was created from 
		ClassName = FName(Struct->GetMetaData(NAME_TemplateStructName));
	}
	else
	{
		ClassName = Struct->GetFName();
	}
}


bool FPropertyArrayExternalObjectHandler::GetExternalOverrides(UObject* Object, const FProperty* Property, FOverridableObjectArrayOverrides* Overrides)
{
	FPropertyKey Key(Object, Property);
	
	if (IClassPropertyArrayExternalOverridesHandler** Handler = PerClassPropertyHandlers.Find(Key))
	{
		return (*Handler)->GetExternalOverrides(Object, Overrides);
	}

	return false;
}


bool FPropertyArrayExternalObjectHandler::IsSavingExternal(UObject* Object, const  FProperty* Property)
{
	FPropertyKey Key(Object, Property);

	if (IClassPropertyArrayExternalOverridesHandler** Handler = PerClassPropertyHandlers.Find(Key))
	{
		return (*Handler)->IsSavingExternal(Object);
	}

	return false;
}

bool FPropertyArrayExternalObjectHandler::NotifyRemove(UObject* Object, const class FProperty* Property, UObject* RemovedObject, UObject* RemovedArchetype)
{
	FPropertyKey Key(Object, Property);
	if (IClassPropertyArrayExternalOverridesHandler** Handler = PerClassPropertyHandlers.Find(Key))
	{
		(*Handler)->NotifyRemove(Object, RemovedObject, RemovedArchetype);
		return true;
	}

	return false;
}

bool FPropertyArrayExternalObjectHandler::NotifyAdd(UObject* Object, const class FProperty* Property, UObject* AddedObject)
{
	FPropertyKey Key(Object, Property);
	if (IClassPropertyArrayExternalOverridesHandler** Handler = PerClassPropertyHandlers.Find(Key))
	{
		(*Handler)->NotifyAdd(Object, AddedObject);
		return true;
	}

	return false;
}

void FPropertyArrayExternalObjectHandler::RegisterHandler(UClass* Class, FName PropertyName, IClassPropertyArrayExternalOverridesHandler* Handler)
{
	FPropertyKey Key(Class->GetFName(), PropertyName);
	
	checkf(!PerClassPropertyHandlers.Find(Key), TEXT("Attempting double registration of an IClassPropertyArrayExternalOverridesHandler, Class: %s, Property: %s"), *Class->GetName(), *PropertyName.ToString());
	PerClassPropertyHandlers.Add(Key, Handler);
}

void FPropertyArrayExternalObjectHandler::UnregisterHandler(IClassPropertyArrayExternalOverridesHandler* Handler)
{	
	for (TPair<FPropertyKey, IClassPropertyArrayExternalOverridesHandler*> Pair : PerClassPropertyHandlers)
	{
		if(Pair.Value == Handler)
		{
			PerClassPropertyHandlers.Remove(Pair.Key);
			break;
		}
	}
}

FPropertyArrayExternalObjectHandler FPropertyArrayExternalObjectHandler::HandlerObj;

FPropertyArrayExternalObjectHandler& FPropertyArrayExternalObjectHandler::Get()
{
	return HandlerObj;
}

#endif //#if WITH_EDITOR