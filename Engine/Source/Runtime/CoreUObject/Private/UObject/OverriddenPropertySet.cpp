// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/OverriddenPropertySet.h"

#include "HAL/IConsoleManager.h"
#include "UObject/OverridableManager.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/PropertyOptional.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/UObjectArchetypeHelper.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/OverriddenObjectsExternalHandler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OverriddenPropertySet)

/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 */

DEFINE_LOG_CATEGORY(LogOverridableObject);

//----------------------------------------------------------------------//
// FOverridableSerializationLogicInternalAdapter
//----------------------------------------------------------------------//

struct FOverridableSerializationLogicInternalAdapter
{
	static void SetCapability(FOverridableSerializationLogic::ECapabilities InCapability, bool bEnable)
	{
		if (bEnable)
		{
			FOverridableSerializationLogic::Capabilities = FOverridableSerializationLogic::Capabilities | InCapability;
		}
		else
		{
			FOverridableSerializationLogic::Capabilities = FOverridableSerializationLogic::Capabilities & ~InCapability;
		}
	}
};

namespace Private
{
	bool bEnableT3D = true;
	FAutoConsoleVariableRef CVar_bT3DOverrideSerializationEnabled(
		TEXT("OverridableSerializationLogic.Capabilities.T3D"),
		bEnableT3D,
		TEXT("Enables serialization of override state into/from T3D"),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
		{
			FOverridableSerializationLogicInternalAdapter::SetCapability(FOverridableSerializationLogic::ECapabilities::T3DSerialization, bEnableT3D);
		}));

	bool bEnableSubObjectsShadowSerialization = true;
	FAutoConsoleVariableRef CVar_bEnableSubObjectsShadowSerialization(
		TEXT("OverridableSerializationLogic.Capabilities.SubObjectsShadowSerialization"),
		bEnableSubObjectsShadowSerialization,
		TEXT("Enables shadow serialization of subobject"),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
		{
			FOverridableSerializationLogicInternalAdapter::SetCapability(FOverridableSerializationLogic::ECapabilities::SubObjectsShadowSerialization, bEnableSubObjectsShadowSerialization);
		}));

	struct FCapabilitiesAutoInitializer
	{
		FCapabilitiesAutoInitializer()
		{
			FOverridableSerializationLogicInternalAdapter::SetCapability(FOverridableSerializationLogic::ECapabilities::T3DSerialization, bEnableT3D);
			FOverridableSerializationLogicInternalAdapter::SetCapability(FOverridableSerializationLogic::ECapabilities::SubObjectsShadowSerialization, bEnableSubObjectsShadowSerialization);
		}
	};

	static FCapabilitiesAutoInitializer CapabilitiesAutoInitializer;
	
	/** Custom test if the given object is a default subobject. Overrides support non-derived subobject instances, which is not covered by UObject::IsDefaultSubObject. */
	static bool IsDefaultSubObject(TNotNull<const UObject*> SubObject)
	{
		if (SubObject->HasAnyFlags(RF_DefaultSubObject))
		{
			return true;
		}

		const UObject* Archetype = SubObject->GetArchetype();
		if (Archetype && !Archetype->HasAnyFlags(RF_ClassDefaultObject))
		{
			return true;
		}
	
		return false;
	}

	/** Custom test if the object created by the given params will be a default subobject. */
	static bool IsNewObjectDefaultSubObject(TNotNull<const UClass*> Class, EObjectFlags Flags, const UObject* Template)
	{
		if (EnumHasAnyFlags(Flags, RF_DefaultSubObject))
		{
			return true;
		}
	
		if (!Template)
		{
			Template = Class->GetDefaultObject();
		}
	
		if (Template && !Template->HasAnyFlags(RF_ClassDefaultObject))
		{
			return true;
		}
	
		return false;
	}
}

//----------------------------------------------------------------------//
// FOverridableSerializationLogic
//----------------------------------------------------------------------//

ENUM_CLASS_FLAGS(FOverridableSerializationLogic::ECapabilities);

FOverridableSerializationLogic::ECapabilities FOverridableSerializationLogic::Capabilities = FOverridableSerializationLogic::ECapabilities::None;
thread_local bool FOverridableSerializationLogic::bUseOverridableSerialization = false;
thread_local FOverriddenPropertySet* FOverridableSerializationLogic::OverriddenProperties = nullptr;
thread_local FPropertyVisitorPath* FOverridableSerializationLogic::OverriddenPortTextPropertyPath = nullptr;

EOverriddenPropertyOperation FOverridableSerializationLogic::GetOverriddenPropertyOperation(const int32 PortFlags, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property, const void* DataPtr, const void* DefaultValue)
{
	checkf(bUseOverridableSerialization, TEXT("Nobody should use this method if it is not setup to use overridable serialization"));
	if (!OverriddenProperties)
	{
		return EOverriddenPropertyOperation::None;
	}

	const FProperty* CurrentProperty = Property ? Property : (CurrentPropertyChain ? CurrentPropertyChain->GetPropertyFromStack(0) : nullptr);
	checkf( CurrentProperty, TEXT("Expecting a property to get OS operation on") );

	if (CurrentProperty->HasAnyPropertyFlags(CPF_ExperimentalNeverOverriden))
	{
		return EOverriddenPropertyOperation::None;
	}

	const EOverriddenPropertyOperation OverriddenOperation = OverriddenProperties->GetOverriddenPropertyOperation(CurrentPropertyChain, Property);
	if (OverriddenOperation != EOverriddenPropertyOperation::None)
	{
		return OverriddenOperation;
	}

	// In the case of a CDO owning default value, we might need to serialize it to keep its value.
	if (DataPtr && DefaultValue && OverriddenProperties->IsCDOOwningProperty(*CurrentProperty))
	{
		// Only need serialize this value if it is different from the default property value
		if (!CurrentProperty->Identical(DataPtr, DefaultValue, PortFlags))
		{
			return 	EOverriddenPropertyOperation::Replace;
		}
	}

	if (ShouldPropertyShadowSerializeSubObject(CurrentProperty))
	{
		return EOverriddenPropertyOperation::SubObjectsShadowing;
	}

	return EOverriddenPropertyOperation::None;
}

bool FOverridableSerializationLogic::ShouldPropertyShadowSerializeSubObject(TNotNull<const FProperty*> Property)
{
	// Check if the shadow serialization of subobject is enabled
	if (!HasCapabilities(ECapabilities::SubObjectsShadowSerialization))
	{
		return false;
	}

	// We shadow serialize every object property
	if (CastField<FObjectPropertyBase>(Property))
	{
		return true;
	}

	// Otherwise check if the property is in the reference linked list
	// @Todo optimized by caching the call to FProperty::ContainsObjectReference() maybe as a CPF_ContainsReferences?
	checkf(Property->GetOwnerStruct(), TEXT("Expecting an owner struct for this type of property"));
	FProperty* CurrentRefLink = Property->GetOwnerStruct()->RefLink;
	while (CurrentRefLink != nullptr)
	{
		if (CurrentRefLink == Property)
		{
			return true;
		}
		CurrentRefLink = CurrentRefLink->NextRef;
	}

	return false;
}

bool FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraph(TNotNull<const UObject*> Object, TOptional<bool> UsesOverrides)
{
	if (!UsesOverrides.IsSet())
	{
		UsesOverrides = HasCapabilities(ECapabilities::T3DSerialization) && FOverridableManager::Get().IsEnabled(Object);
	}
	
	// Ideally this rule should be applied more widely but for now we limit it to objects that use overridable serialization.
	return UsesOverrides.GetValue() && !Private::IsDefaultSubObject(Object);
}

bool FOverridableSerializationLogic::ShouldUseDedicatedInstancingGraphForNewObject(TNotNull<const UClass*> Class, EObjectFlags Flags,
	const UObject* Template, bool bUsesOverrides)
{
	return bUsesOverrides && !Private::IsNewObjectDefaultSubObject(Class, Flags, Template);
}

bool FOverridableSerializationLogic::HasCapabilities(ECapabilities InCapabilities)
{
	return (Capabilities & InCapabilities) == InCapabilities;
}

FOverriddenPropertySet* FOverridableSerializationLogic::GetOverriddenPropertiesSlow()
{
	return OverriddenProperties;
}

EOverriddenPropertyOperation FOverridableSerializationLogic::GetOverriddenPropertyOperation(const FArchive& Ar, FProperty* Property /*= nullptr*/, uint8* DataPtr /*= nullptr*/, const uint8* DefaultValue /*= nullptr*/)
{
	const FArchiveSerializedPropertyChain* CurrentPropertyChain = Ar.GetSerializedPropertyChain();
	const EOverriddenPropertyOperation Operation = GetOverriddenPropertyOperation(Ar.GetPortFlags(), CurrentPropertyChain, Property, DataPtr, DefaultValue);

	// During transactions, we do not want any subobject shadow serialization
	return Operation == EOverriddenPropertyOperation::SubObjectsShadowing && Ar.IsTransacting() ? EOverriddenPropertyOperation::None : Operation;
}

EOverriddenPropertyOperation FOverridableSerializationLogic::GetOverriddenPropertyOperationForPortText(const void* DataPtr, const void* DefaultValue, int32 PortFlags)
{
	checkf(OverriddenPortTextPropertyPath, TEXT("Expecting an overridden port text path"));

	const FArchiveSerializedPropertyChain CurrentPropertyChain = OverriddenPortTextPropertyPath->ToSerializedPropertyChain();
	const EOverriddenPropertyOperation Operation = GetOverriddenPropertyOperation(PortFlags, &CurrentPropertyChain, nullptr, DataPtr, DefaultValue);
	return Operation;
}

FPropertyVisitorPath* FOverridableSerializationLogic::GetOverriddenPortTextPropertyPath()
{
	return OverriddenPortTextPropertyPath;
}

void FOverridableSerializationLogic::SetOverriddenPortTextPropertyPath(FPropertyVisitorPath& Path)
{
	checkf(OverriddenPortTextPropertyPath == nullptr, TEXT("Should not set a path on top of an existing one"));
	OverriddenPortTextPropertyPath = &Path;
}

void FOverridableSerializationLogic::ResetOverriddenPortTextPropertyPath()
{
	OverriddenPortTextPropertyPath = nullptr;
}

//----------------------------------------------------------------------//
// FOverridableSerializationScope
//----------------------------------------------------------------------//
FEnableOverridableSerializationScope::FEnableOverridableSerializationScope(bool bEnableOverridableSerialization, FOverriddenPropertySet* OverriddenProperties)
{
	if (bEnableOverridableSerialization)
	{
		if (FOverridableSerializationLogic::IsEnabled())
		{
			bWasOverridableSerializationEnabled = true;
			SavedOverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties();
			FOverridableSerializationLogic::Disable();
		}
		FOverridableSerializationLogic::Enable(OverriddenProperties);
		bOverridableSerializationEnabled = true;
	}
}

FEnableOverridableSerializationScope::~FEnableOverridableSerializationScope()
{
	if (bOverridableSerializationEnabled)
	{
		FOverridableSerializationLogic::Disable();
		if (bWasOverridableSerializationEnabled)
		{
			FOverridableSerializationLogic::Enable(SavedOverriddenProperties);
		}
	}
}

//----------------------------------------------------------------------//
// FOverridableTextImportPropertyPathScope
//----------------------------------------------------------------------//
FOverridableTextPortPropertyPathScope::FOverridableTextPortPropertyPathScope(const FProperty* InProperty, int32 InIndex/* = INDEX_NONE*/, EPropertyVisitorInfoType InPropertyInfo /*= EPropertyVisitorInfoType::None*/)
{
	if (!FOverridableSerializationLogic::IsEnabled())
	{
		return;
	}

	checkf(InProperty, TEXT("Expecting a valid property ptr"));

	// Save property for comparison in the destructor
	Property = InProperty;

	FPropertyVisitorPath* Path = FOverridableSerializationLogic::GetOverriddenPortTextPropertyPath();
	if (!Path)
	{
		FOverridableSerializationLogic::SetOverriddenPortTextPropertyPath(DefaultPath);
		Path = &DefaultPath;
	}

	Path->Push(FPropertyVisitorInfo(InProperty, InIndex, InPropertyInfo));
}

FOverridableTextPortPropertyPathScope::~FOverridableTextPortPropertyPathScope()
{
	if (Property)
	{
		FPropertyVisitorPath* Path = FOverridableSerializationLogic::GetOverriddenPortTextPropertyPath();
		checkf(Path, TEXT("Expecting a valid path "));
		checkf(Path->Num(), TEXT("Expecting at least one property in the path"));
		verifyf(Path->Pop().Property == Property, TEXT("Expecting at the top property to match the one we pushed in the constructor"));
		if (!Path->Num())
		{
			FOverridableSerializationLogic::ResetOverriddenPortTextPropertyPath();
		}
	}
}

//----------------------------------------------------------------------//
// FOverriddenPropertyPath
//----------------------------------------------------------------------//
FOverriddenPropertyPath::FOverriddenPropertyPath(const FName Name)
: Path{Name}
, CachedHash(GetTypeHash(Name))
{}

FOverriddenPropertyPath& FOverriddenPropertyPath::operator=(const FName Name)
{
	Path = {Name};
	CachedHash = GetTypeHash(Name);
	return *this;
}

FOverriddenPropertyPath FOverriddenPropertyPath::operator+(const FName Name) const
{
	FOverriddenPropertyPath NewPath(*this);
	NewPath += Name;
	return NewPath;
}

FOverriddenPropertyPath FOverriddenPropertyPath::operator+(const FOverriddenPropertyPath& Other) const
{
	FOverriddenPropertyPath NewPath(*this);
	NewPath += Other;
	return NewPath;
}

void FOverriddenPropertyPath::operator+=(const FName Name)
{
	Path.Emplace(Name);
	CachedHash = HashCombine(CachedHash, GetTypeHash(Name));
}

void FOverriddenPropertyPath::operator+=(const TArray<FName>& Names)
{
	Path.Reserve(Path.Num() + Names.Num());
	for (const FName& Name : Names)
	{
		this->operator+=(Name);
	}
}

FString FOverriddenPropertyPath::ToString() const
{
	return FString::JoinBy(Path, TEXT("."), [](FName Name) { return Name.ToString(); });
}

//----------------------------------------------------------------------//
// FOverriddenPropertyNodeID
//----------------------------------------------------------------------//
FOverriddenPropertyNodeID::FOverriddenPropertyNodeID(TNotNull<const FProperty*> Property)
{
	Path = Property->GetFName();

	// append typename to the end of the property ID
	UE::FPropertyTypeNameBuilder TypeNameBuilder;
	Property->SaveTypeName(TypeNameBuilder);
	for (const UE::FPropertyTypeNameNode& Node : TypeNameBuilder)
	{
		Path += Node.Name;
	}
}

FOverriddenPropertyNodeID::FOverriddenPropertyNodeID(const FOverriddenPropertyNodeID& ParentNodeID, const FOverriddenPropertyNodeID& NodeID)
	: Path(ParentNodeID.Path + NodeID.Path) // Combine the 2 node ids
	, Object(NodeID.Object)
{
}

FOverriddenPropertyNodeID::FOverriddenPropertyNodeID(TNotNull<const UObject*> InObject)
: Object(NotNullGet(InObject))
{
	const UObject* KeyObject = InObject;
#if WITH_EDITORONLY_DATA
	// subobject pointers inside IDOs never point to another IDO so to keep that consistent, we'll always redirect IDO pointers here to the instance counterpart
	if (const UObject* Instance = UE::FPropertyBagRepository::Get().FindInstanceForDataObject(InObject))
	{
		KeyObject = Instance;
	}
#endif

	// Note: Using ObjectIndex by itself is not sufficient for an enduring unique identifier
	// as re-instantiation can cause a reuse of the index for another object. Appending the serial solves this issue
	const int32 ObjectIndex = GUObjectArray.ObjectToIndex(KeyObject);
	Path = FName(FString::Printf(TEXT("ID_%d"), ObjectIndex), GUObjectArray.AllocateSerialNumber(ObjectIndex));
}

FOverriddenPropertyNodeID FOverriddenPropertyNodeID::RootNodeId()
{
	FOverriddenPropertyNodeID Result;
	Result.Path = FName("root");
	return Result;
}

FOverriddenPropertyNodeID FOverriddenPropertyNodeID::FromMapKey(const FProperty* KeyProperty, const void* KeyData)
{
	if (const FObjectPropertyBase* KeyObjectProperty = CastField<FObjectPropertyBase>(KeyProperty))
	{
		if (const UObject* Object = KeyObjectProperty->GetObjectPropertyValue(KeyData))
		{
			return FOverriddenPropertyNodeID(Object);
		}
	}
	else
	{
		FString KeyString;
		KeyProperty->ExportTextItem_Direct(KeyString, KeyData, /*DefaultValue*/nullptr, /*Parent*/nullptr, PPF_None);
		FOverriddenPropertyNodeID Result;
		Result.Path = FName(KeyString);
		return Result;
	}
		
	checkf(false, TEXT("This case is not handled"))
	return FOverriddenPropertyNodeID();
}

int32 FOverriddenPropertyNodeID::ToMapInternalIndex(FScriptMapHelper& MapHelper) const
{
	// Special case for object we didn't use the pointer to create the key
	if (const FObjectPropertyBase* KeyObjectProperty = CastField<FObjectPropertyBase>(MapHelper.KeyProp))
	{
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It) 
		{
			if (UObject* CurrentObject = KeyObjectProperty->GetObjectPropertyValue(MapHelper.GetKeyPtr(It)))
			{
				if ((*this) == FOverriddenPropertyNodeID(CurrentObject))
				{
					return It.GetInternalIndex();
				}
			}
		}
	}
	else
	{
		// Default case, just import the text as key value for comparison
		void* TempKeyValueStorage = FMemory_Alloca(MapHelper.MapLayout.SetLayout.Size);
		MapHelper.KeyProp->InitializeValue(TempKeyValueStorage);

		FString KeyToFind(ToString());
		MapHelper.KeyProp->ImportText_Direct(*KeyToFind, TempKeyValueStorage, nullptr, PPF_None);

		const int32 InternalIndex = MapHelper.FindMapPairIndexFromHash(TempKeyValueStorage);

		MapHelper.KeyProp->DestroyValue(TempKeyValueStorage);

		return InternalIndex;
	}
	return INDEX_NONE;
}

FScopedPropertyValue FOverriddenPropertyNodeID::ToMapKey(FScriptMapHelper& MapHelper) const
{
	// Special case for object we didn't use the pointer to create the key
	if (const FObjectPropertyBase* KeyObjectProperty = CastField<FObjectPropertyBase>(MapHelper.KeyProp))
	{
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			void* KeyPtr = MapHelper.GetKeyPtr(It);
			if (UObject* CurrentObject = KeyObjectProperty->GetObjectPropertyValue(KeyPtr))
			{
				if ((*this) == FOverriddenPropertyNodeID(CurrentObject))
				{
					FScopedPropertyValue Result(MapHelper.KeyProp);
					MapHelper.KeyProp->CopyCompleteValue(Result.GetData(), KeyPtr);
					return MoveTemp(Result);
				}
			}
		}
	}
	else
	{
		// Default case, just import the text as key value for comparison
		FString KeyToFind(ToString());
		FScopedPropertyValue Result(MapHelper.KeyProp);
		MapHelper.KeyProp->ImportText_Direct(*KeyToFind, Result.GetData(), nullptr, PPF_None);
		return MoveTemp(Result);
	}
	return FScopedPropertyValue();
}

bool FOverriddenPropertyNodeID::operator==(const FOverriddenPropertyNodeID& Other) const
{
	if (Path == Other.Path)
	{
		return true;
	}

	// After reinstantiation, we do not change the path we only patch the ptr.
	// Unfortunately we do not have a stable id that is stable through reinstantiation,
	// so the only way is to compare ptr.
	constexpr bool bEvenIfPendingKill = true;
	if (Object.IsValid(bEvenIfPendingKill) && 
		Other.Object.IsValid(bEvenIfPendingKill) && 
		Object.Get(bEvenIfPendingKill) == Other.Object.Get(bEvenIfPendingKill))
	{
		return true;
	}

	return false;
}

void FOverriddenPropertyNodeID::HandleObjectsReInstantiated(const TMap<UObject*, UObject*>& Map)
{
	if (const UObject* ObjectPtr = Object.Get(/*bEvenIfGarbage*/true))
	{
		if (UObject*const* ReplacedObject = Map.Find(ObjectPtr))
		{
			Object = *ReplacedObject;
		}
	}
}

void FOverriddenPropertyNodeID::HandleDeadObjectReferences(const TSet<UObject*>& ActiveInstances, const TSet<UObject*>& TemplateInstances)
{
	if (const UObject* ObjectPtr = Object.Get(/*bEvenIfGarbage*/true))
	{
		if (ActiveInstances.Find(ObjectPtr) || TemplateInstances.Find(ObjectPtr) )
		{
			Object.Reset();
		}
	}
}

//----------------------------------------------------------------------//
// FOverriddenPropertyNode
//----------------------------------------------------------------------//
bool FOverriddenPropertyNode::Serialize(FArchive& Ar)
{
	FOverriddenPropertyNodeID::StaticStruct()->SerializeItem(Ar, &NodeID, nullptr);
	Ar << Operation;
	Ar << SubPropertyNodes;
	return true;
}

void FOverriddenPropertyNode::SetOperation(EOverriddenPropertyOperation InOperation)
{
	if (Operation != InOperation)
	{
		Operation = InOperation;
	}
}

FOverriddenPropertyNode& FOverriddenPropertyNode::FindOrAddNode(const FOverriddenPropertyNodeID& InNodeID)
{
	if (FOverriddenPropertyNode* SubNode = SubPropertyNodes.FindByKey(InNodeID))
	{
		return *SubNode;
	}

	// We can safely assume that this node is at least modified from now on
	if (Operation == EOverriddenPropertyOperation::None)
	{
		Operation = EOverriddenPropertyOperation::Modified;
	}

	// Not found add the node
	return SubPropertyNodes.Emplace_GetRef(InNodeID);
}

bool FOverriddenPropertyNode::RemoveSubPropertyNode(const FOverriddenPropertyNodeID& InNodeID)
{
	if (SubPropertyNodes.Remove(InNodeID))
	{
		return true;
	}
	return false;
}

void FOverriddenPropertyNode::Reset()
{
	SetOperation(EOverriddenPropertyOperation::None);

	if (!SubPropertyNodes.IsEmpty())
	{
		SubPropertyNodes.Empty();
	}
}

void FOverriddenPropertyNode::HandleObjectsReInstantiated(const TMap<UObject*, UObject*>& Map)
{
	NodeID.HandleObjectsReInstantiated(Map);
	for (FOverriddenPropertyNode& SubNode : SubPropertyNodes)
	{
		SubNode.HandleObjectsReInstantiated(Map);
	}
}

void FOverriddenPropertyNode::HandleDeadObjectReferences(const TSet<UObject*>& ActiveInstances, const TSet<UObject*>& TemplateInstances)
{
	NodeID.HandleDeadObjectReferences(ActiveInstances, TemplateInstances);
	for (FOverriddenPropertyNode& SubNode : SubPropertyNodes)
	{
		SubNode.HandleDeadObjectReferences(ActiveInstances, TemplateInstances);
	}
}

//----------------------------------------------------------------------//
// FOverriddenPropertySet
//----------------------------------------------------------------------//

void FOverriddenPropertySet::RestoreOverriddenState(const FOverriddenPropertySet& FromOverriddenProperties)
{
	bWasAdded = FromOverriddenProperties.bWasAdded;
}

EOverriddenPropertyOperation FOverriddenPropertySet::GetOverriddenPropertyOperation(const FOverriddenPropertyNode* ParentPropertyNode, FPropertyVisitorPath::Iterator PropertyIterator, bool* bOutInheritedOperation, const void* Data) const
{
	FOverridableManager& OverridableManager = FOverridableManager::Get();

	const void* SubValuePtr = Data;
	const FOverriddenPropertyNode* OverriddenPropertyNode = ParentPropertyNode;
	int32 ArrayIndex = INDEX_NONE;
	while (PropertyIterator && (!OverriddenPropertyNode || OverriddenPropertyNode->GetOperation() != EOverriddenPropertyOperation::Replace))
	{
		ArrayIndex = INDEX_NONE;

		const FProperty* CurrentProperty = PropertyIterator->Property;
		SubValuePtr = CurrentProperty->ContainerPtrToValuePtr<void>(SubValuePtr, 0); //@todo support static arrays

		const FOverriddenPropertyNode* CurrentOverriddenPropertyNode = nullptr;
		if (OverriddenPropertyNode)
		{
			FOverriddenPropertyNodeID NodeID(CurrentProperty);
			if (const FOverriddenPropertyNode* FoundNode = OverriddenPropertyNode->GetSubPropertyNodes().FindByKey(NodeID))
			{
				CurrentOverriddenPropertyNode = FoundNode;
			}
		}

		FPropertyVisitorPath::Iterator NextPropertyIterator = PropertyIterator+1;
		// Special handling for instanced subobjects 
		if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(CurrentProperty))
		{
			if (NextPropertyIterator)
			{
				// Forward any sub queries to the subobject
				UObject* SubObject = TryGetInstancedSubObjectValue(ObjectProperty, SubValuePtr);
				// This should not be needed in the property grid, as it should already been called on the subobject.
				return SubObject ? OverridableManager.GetOverriddenPropertyOperation(SubObject, NextPropertyIterator, bOutInheritedOperation) : EOverriddenPropertyOperation::None;
			}
		}
		// Special handling for array of instanced subobjects 
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentProperty))
		{
			// This code handles inner properties at the same time as the container property,
			// skip the container one if the next one is equal to its inner property
			if (NextPropertyIterator && NextPropertyIterator->Property == ArrayProperty->Inner)
			{
				++PropertyIterator;
				++NextPropertyIterator;
			}
			ArrayIndex = PropertyIterator->Index;
			checkf(ArrayIndex == INDEX_NONE || PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::ContainerIndex, TEXT("Expecting a container index"));

			// Only special case is instanced subobjects, otherwise we fallback to full array override
			checkf(ArrayProperty->Inner, TEXT("Expecting an inner type for Arrays"));
			if (const FObjectPropertyBase* InnerObjectProperty = CastField<FObjectPropertyBase>(ArrayProperty->Inner))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, SubValuePtr);
				if(ArrayHelper.IsValidIndex(ArrayIndex))
				{
					UObject* SubObject = TryGetInstancedSubObjectValue(InnerObjectProperty, ArrayHelper.GetElementPtr(ArrayIndex));
					if (NextPropertyIterator)
					{
						// Forward any sub queries to the subobject
						return SubObject ? OverridableManager.GetOverriddenPropertyOperation(SubObject, NextPropertyIterator, bOutInheritedOperation) : EOverriddenPropertyOperation::None;
					}

					if(CurrentOverriddenPropertyNode && SubObject)
					{
						// Caller wants to know about any override state on the reference of the subobject itself
						const FOverriddenPropertyNodeID  SubObjectID(SubObject);
						if (const FOverriddenPropertyNode* SubObjectOverriddenPropertyNode = CurrentOverriddenPropertyNode->GetSubPropertyNodes().FindByKey(SubObjectID))
						{
							if (bOutInheritedOperation)
							{
								*bOutInheritedOperation = false;
							}
							return SubObjectOverriddenPropertyNode->GetOperation();
						}
					}
				}
			}
		}
		// Special handling for maps and values of instance subobjects
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(CurrentProperty))
		{
			// This code handles inner properties at the same time as the container property,
			// skip the container one if the next one is equal to one of its inner properties
			if (NextPropertyIterator && 
			    (NextPropertyIterator->Property == MapProperty->KeyProp ||
				 NextPropertyIterator->Property == MapProperty->ValueProp) )
			{
				++PropertyIterator;
				++NextPropertyIterator;
			}

			ArrayIndex = PropertyIterator->Index;
			checkf(ArrayIndex == INDEX_NONE || 
				PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::ContainerIndex || 
				PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::MapKey || 
				PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::MapValue, TEXT("Expecting a container type"));

			checkf(MapProperty->ValueProp, TEXT("Expecting a value type for Maps"));
			FScriptMapHelper MapHelper(MapProperty, SubValuePtr);

			const int32 InternalMapIndex = ArrayIndex != INDEX_NONE ? MapHelper.FindInternalIndex(ArrayIndex) : INDEX_NONE;
			if(MapHelper.IsValidIndex(InternalMapIndex))
			{
				if (NextPropertyIterator)
				{
					// Forward any sub queries to the subobject
					if (const FObjectPropertyBase* ValueObjectProperty = CastField<FObjectPropertyBase>(MapProperty->ValueProp))
					{
						if (UObject* ValueSubObject = TryGetInstancedSubObjectValue(ValueObjectProperty, MapHelper.GetValuePtr(InternalMapIndex)))
						{
							return OverridableManager.GetOverriddenPropertyOperation(ValueSubObject, NextPropertyIterator, bOutInheritedOperation);
						}
					}
					// This is not a subobject property or an null subobject, return none at this point
					return EOverriddenPropertyOperation::None;
				}

				if(CurrentOverriddenPropertyNode)
				{
					// Caller wants to know about any override state on the reference of the map pair itself
					checkf(MapProperty->KeyProp, TEXT("Expecting a key type for Maps"));
					FOverriddenPropertyNodeID OverriddenKeyID = FOverriddenPropertyNodeID::FromMapKey(MapProperty->KeyProp, MapHelper.GetKeyPtr(InternalMapIndex));

					if (const FOverriddenPropertyNode* SubObjectOverriddenPropertyNode = CurrentOverriddenPropertyNode->GetSubPropertyNodes().FindByKey(OverriddenKeyID))
					{
						if (bOutInheritedOperation)
						{
							*bOutInheritedOperation = false;
						}
						return SubObjectOverriddenPropertyNode->GetOperation();
					}
				}
			}
		}

		OverriddenPropertyNode = CurrentOverriddenPropertyNode;
		// While digging down the path, if there is one property that is always overridden
		// stop there and return replace
		if (CurrentProperty->HasAnyPropertyFlags(CPF_ExperimentalAlwaysOverriden))
		{
			if (bOutInheritedOperation)
			{
				*bOutInheritedOperation = NextPropertyIterator ? true : false;
			}
			return EOverriddenPropertyOperation::Replace;
		}

		++PropertyIterator;
	}

	if (bOutInheritedOperation)
	{
		*bOutInheritedOperation = PropertyIterator || ArrayIndex != INDEX_NONE;
	}
	return OverriddenPropertyNode ? OverriddenPropertyNode->GetOperation() : EOverriddenPropertyOperation::None;
}

bool FOverriddenPropertySet::ClearOverriddenProperty(FOverriddenPropertyNode& ParentPropertyNode, FPropertyVisitorPath::Iterator PropertyIterator, const void* Data)
{
	FOverridableManager& OverridableManager = FOverridableManager::Get();
	if (!PropertyIterator)
	{
		// if no property iterator is provided, clear all overrides
		OverridableManager.ClearOverrides(Owner);
		return true;
	}

	bool bClearedOverrides = false;
	const void* SubValuePtr = Data;
	FOverriddenPropertyNode* OverriddenPropertyNode = &ParentPropertyNode;
	int32 ArrayIndex = INDEX_NONE;
	TArray<FOverriddenPropertyNode*> TraversedNodes;
	TraversedNodes.Push(OverriddenPropertyNode);
	while (PropertyIterator && (!OverriddenPropertyNode || OverriddenPropertyNode->GetOperation() != EOverriddenPropertyOperation::Replace))
	{
		ArrayIndex = INDEX_NONE;

		const FProperty* CurrentProperty = PropertyIterator->Property;
		SubValuePtr = CurrentProperty->ContainerPtrToValuePtr<void>(SubValuePtr, 0); //@todo support static arrays

		FOverriddenPropertyNode* CurrentOverriddenPropertyNode = nullptr;
		if (OverriddenPropertyNode)
		{
			FOverriddenPropertyNodeID NodeID(CurrentProperty);
			if (FOverriddenPropertyNode* FoundPropertyNode = OverriddenPropertyNode->GetSubPropertyNodes().FindByKey(NodeID))
			{
				CurrentOverriddenPropertyNode = FoundPropertyNode;
				TraversedNodes.Push(CurrentOverriddenPropertyNode);
			}
		}

		// Special handling for instanced subobjects 
		FPropertyVisitorPath::Iterator NextPropertyIterator = PropertyIterator+1;
		if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(CurrentProperty))
		{
			UObject* SubObject = TryGetInstancedSubObjectValue(ObjectProperty, SubValuePtr);
			if (NextPropertyIterator)
			{
				return SubObject ? OverridableManager.ClearOverriddenProperty(SubObject, NextPropertyIterator) : false;
			}
			if (SubObject)
			{
				OverridableManager.ClearOverrides(SubObject);
			}
			bClearedOverrides = CurrentOverriddenPropertyNode != nullptr;
		}
		// Special handling for array of instanced subobjects 
		else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(CurrentProperty))
		{
			// This code handles inner properties at the same time as the container property,
			// skip the container one if the next one is equal to its inner property
			if (NextPropertyIterator && NextPropertyIterator->Property == ArrayProperty->Inner)
			{
				++PropertyIterator;
				++NextPropertyIterator;
			}
			ArrayIndex = PropertyIterator->Index;
			checkf(ArrayIndex == INDEX_NONE || PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::ContainerIndex, TEXT("Expecting a container index"));

			// Only special case is instanced subobjects, otherwise we fallback to full array override
			if (FObjectPropertyBase* InnerObjectProperty = CastField<FObjectPropertyBase>(ArrayProperty->Inner))
			{
				FScriptArrayHelper ArrayHelper(ArrayProperty, SubValuePtr);

				if(ArrayIndex == INDEX_NONE)
				{
					// This is a case of the entire array needs to be cleared
					// Need to loop through every sub object and clear them
					for (int i = 0; i < ArrayHelper.Num(); ++i)
					{
						if (UObject* SubObject = TryGetInstancedSubObjectValue(InnerObjectProperty, ArrayHelper.GetElementPtr(i)))
						{
							OverridableManager.ClearInstancedSubObjectOverrides(Owner, SubObject);
						}
					}
					bClearedOverrides = true;
				}
				else if(ArrayHelper.IsValidIndex(ArrayIndex))
				{
					UObject* SubObject = TryGetInstancedSubObjectValue(InnerObjectProperty, ArrayHelper.GetElementPtr(ArrayIndex));
					if (NextPropertyIterator)
					{
						return SubObject ? OverridableManager.ClearOverriddenProperty(SubObject, NextPropertyIterator) : false;
					}
					if (CurrentOverriddenPropertyNode && SubObject)
					{
						const FOverriddenPropertyNodeID  SubObjectID(SubObject);
						if (CurrentOverriddenPropertyNode->RemoveSubPropertyNode(SubObjectID))
						{
							OverridableManager.ClearInstancedSubObjectOverrides(Owner, SubObject);
							return true;
						}
					}
				}
			}
		}
		// Special handling for maps and values of instance subobjects 
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(CurrentProperty))
		{
			// This code handles inner properties at the same time as the container property,
			// skip the container one if the next one is equal to one of its inner properties
			if (NextPropertyIterator && 
			    (NextPropertyIterator->Property == MapProperty->KeyProp ||
				 NextPropertyIterator->Property == MapProperty->ValueProp) )
			{
				++PropertyIterator;
				++NextPropertyIterator;
			}

			ArrayIndex = PropertyIterator->Index;
			checkf(ArrayIndex == INDEX_NONE || 
				PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::ContainerIndex || 
				PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::MapKey || 
				PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::MapValue, TEXT("Expecting a container type"));

			FScriptMapHelper MapHelper(MapProperty, SubValuePtr);

			const int32 InternalMapIndex = ArrayIndex != INDEX_NONE ? MapHelper.FindInternalIndex(ArrayIndex) : INDEX_NONE;
			const FObjectPropertyBase* ValueObjectProperty = CastField<FObjectPropertyBase>(MapProperty->ValueProp);

			// If there is a next node, it is probably because the map value is holding an instanced subobject and the user is changing value on it.
			// So forward the call to the instanced subobject
			if (NextPropertyIterator)
			{
				if (MapHelper.IsValidIndex(InternalMapIndex))
				{
					checkf(MapProperty->ValueProp, TEXT("Expecting a value type for Maps"));
					UObject* ValueSubObject = TryGetInstancedSubObjectValue(ValueObjectProperty, MapHelper.GetValuePtr(InternalMapIndex));
					return ValueSubObject ? OverridableManager.ClearOverriddenProperty(ValueSubObject, NextPropertyIterator) : false;
				}

				// Unable the reconcile what we need to clear.
				return false;
			}

			if(InternalMapIndex == INDEX_NONE)
			{
				// Users want to clear all of the overrides on the array, but in the case of instanced subobject, we need to clear the overrides on them as well.
				if (ValueObjectProperty)
				{
					// This is a case of the entire array needs to be cleared
					// Need to loop through every sub object and clear them
					for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
					{
						if (UObject* ValueSubObject = TryGetInstancedSubObjectValue(ValueObjectProperty, MapHelper.GetValuePtr(It.GetInternalIndex())))
						{
							OverridableManager.ClearInstancedSubObjectOverrides(Owner, ValueSubObject);
						}
					}
				}
				bClearedOverrides = true;
			}
			else if (MapHelper.IsValidIndex(InternalMapIndex) && CurrentOverriddenPropertyNode)
			{
				checkf(MapProperty->KeyProp, TEXT("Expecting a key type for Maps"));
				FOverriddenPropertyNodeID OverriddenKeyID = FOverriddenPropertyNodeID::FromMapKey(MapProperty->KeyProp, MapHelper.GetKeyPtr(InternalMapIndex));

				FOverriddenPropertyNodeID CurrentPropKey;
				if (CurrentOverriddenPropertyNode->RemoveSubPropertyNode(OverriddenKeyID))
				{
					if (UObject* ValueSubObject = TryGetInstancedSubObjectValue(ValueObjectProperty, MapHelper.GetValuePtr(InternalMapIndex)))
					{
						// In the case of a instanced subobject, clear all the overrides on the subobject as well
						OverridableManager.ClearInstancedSubObjectOverrides(Owner, ValueSubObject);
					}

					return true;
				}
			}
		}

		OverriddenPropertyNode = CurrentOverriddenPropertyNode;
		++PropertyIterator;
	}

	auto CleanupClearedNodes = [this, &TraversedNodes]()
	{
		// Go through each traversed property in reversed order to do cleanup
		// We need to continue the cleanup until there is more overrides than just the one we are removing
		FOverriddenPropertyNode* LastCleanedNode = nullptr;
		while (FOverriddenPropertyNode* CurrentNode = !TraversedNodes.IsEmpty() ? TraversedNodes.Pop() : nullptr)
		{
			if (LastCleanedNode)
			{
				// In the case there are other overrides, just cleanup that node and stop.
				if (CurrentNode->GetSubPropertyNodes().Num() > 1)
				{
					verifyf(CurrentNode->RemoveSubPropertyNode(LastCleanedNode->GetNodeID()), TEXT("Expecting the node to always be removed"));
					LastCleanedNode = nullptr;
					break;
				}
			}

			CurrentNode->Reset();
			LastCleanedNode = CurrentNode;
		}
	};

	if (PropertyIterator || OverriddenPropertyNode == nullptr)
	{
		if (bClearedOverrides)
		{
			CleanupClearedNodes();
		}

		return bClearedOverrides;
	}

	if (ArrayIndex != INDEX_NONE)
	{
		return false;
	}

	CleanupClearedNodes();
	return true;
}

void FOverriddenPropertySet::NotifyPropertyChange(FOverriddenPropertyNode* ParentPropertyNode, const EPropertyNotificationType Notification, FPropertyVisitorPath::Iterator PropertyIterator, const EPropertyChangeType::Type ChangeType, const void* Data, bool& bNeedsCleanup)
{
	// Because the PreEdit API doesn't provide Index info, we need to snapshot changed containers during pre-edit
	// so we can intuit which element was removed during PostEdit. This is a map-like structure that stores the latest snapshot for each contianer property
	struct FContainerInfo
	{
		const UObject* Owner = nullptr;
		const FProperty* Property = nullptr;
	};
	static struct
	{
		// There's not many elements so we're using an array of TPairs for cache friendliness. A TMap would work fine here as well.
		uint8* Find(TNotNull<const UObject*> ContainerOwner, TNotNull<const FProperty*> ContainerProperty) const
		{
			for (const TPair<FContainerInfo, uint8*>& Element : Data)
			{
				if (Element.Key.Owner == ContainerOwner &&
					Element.Key.Property == ContainerProperty)
				{
					return Element.Value;
				}
			}
			return nullptr;
		}

		void Free(TNotNull<const UObject*> ContainerOwner, TNotNull<const FProperty*> ContainerProperty)
		{
			for (int32 I = 0; I < Data.Num(); ++I)
			{
				if (Data[I].Key.Owner == ContainerOwner &&
					Data[I].Key.Property == ContainerProperty)
				{
					ContainerProperty->DestroyValue(Data[I].Value);
					FMemory::Free(Data[I].Value);
					Data.RemoveAtSwap(I);
					return;
				}
			}
		}

		uint8* FindOrAdd(TNotNull<const UObject*> ContainerOwner, TNotNull<const FProperty*> ContainerProperty, bool& bAlreadySaved)
		{
			if (uint8* Found = Find(ContainerOwner, ContainerProperty))
			{
				bAlreadySaved = true;
				return Found;
			}
			bAlreadySaved = false;
			FContainerInfo Key = {ContainerOwner, ContainerProperty};
			int32 I = Data.Add({ Key, (uint8*)FMemory::Malloc(ContainerProperty->GetSize(), ContainerProperty->GetMinAlignment()) });
			ContainerProperty->InitializeValue(Data[I].Value);
			return Data[I].Value;
		}

		TArray<TPair<FContainerInfo, uint8*>> Data;
	} SavedPreEditContainers;

	checkf(IsValid(Owner), TEXT("Expecting a valid overridable owner"));

	if (ChangeType & EPropertyChangeType::ResetToDefault)
	{
		if (ParentPropertyNode && Notification == EPropertyNotificationType::PostEdit)
		{
			ClearOverriddenProperty(*ParentPropertyNode, PropertyIterator, Data);
			const FProperty* Property = PropertyIterator ? PropertyIterator->Property : nullptr;
			if (Property)
			{
				// Need to clear out the pre edit list if it was sent with a property and owner
				SavedPreEditContainers.Free(Owner, Property);
			}
		}
		return;
	}

	FOverridableManager& OverridableManager = FOverridableManager::Get();
	if (!PropertyIterator)
	{
		checkf(ParentPropertyNode == &RootNode, TEXT("Only expecting the root node in this code path"));

		NotifyAllPropertyChange(ParentPropertyNode, Notification, ChangeType, Owner->GetClass(), Owner);
		return;
	}

	const FProperty* Property = PropertyIterator->Property;
	checkf(Property, TEXT("Expecting a valid property"));

	const void* SubValuePtr = Property->ContainerPtrToValuePtr<void>(Data, 0); //@todo support static arrays

	FOverriddenPropertyNode* SubPropertyNode = nullptr;
	if (ParentPropertyNode)
	{
		FOverriddenPropertyNode& SubPropertyNodeRef = ParentPropertyNode->FindOrAddNode(FOverriddenPropertyNodeID(Property));
		SubPropertyNode = SubPropertyNodeRef.GetOperation() != EOverriddenPropertyOperation::Replace ? &SubPropertyNodeRef : nullptr;
	}

	ON_SCOPE_EXIT
	{
		if (!ParentPropertyNode || Notification != EPropertyNotificationType::PostEdit)
		{
			return;
		}

		if (SubPropertyNode && SubPropertyNode->GetSubPropertyNodes().IsEmpty() &&
			(bNeedsCleanup || 
			 SubPropertyNode->GetOperation() == EOverriddenPropertyOperation::None || 
			 SubPropertyNode->GetOperation() == EOverriddenPropertyOperation::Modified))
		{
			FOverriddenPropertyNodeID RemoveNodeID(Property);
			ParentPropertyNode->RemoveSubPropertyNode(RemoveNodeID);
			if (ParentPropertyNode->GetOperation() == EOverriddenPropertyOperation::Modified && ParentPropertyNode->GetSubPropertyNodes().IsEmpty())
			{
				ParentPropertyNode->SetOperation(EOverriddenPropertyOperation::None);
			}
		}
	};

	FPropertyVisitorPath::Iterator NextPropertyIterator = PropertyIterator+1;
	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		// Only special case is instanced subobjects, otherwise we fallback to full array override
		if (FObjectPropertyBase* InnerObjectProperty = CastField<FObjectPropertyBase>(ArrayProperty->Inner))
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, SubValuePtr);

			// This code handles inner properties at the same time as the container property,
			// skip the container one if the next one is equal to its inner property
			if (NextPropertyIterator && NextPropertyIterator->Property == ArrayProperty->Inner)
			{
				++PropertyIterator;
				++NextPropertyIterator;
			}
			int ArrayIndex = PropertyIterator->Index;
			checkf(ArrayIndex == INDEX_NONE || PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::ContainerIndex, TEXT("Expecting a container index"));

			if (!NextPropertyIterator)
			{
				checkf(ArrayProperty->Inner, TEXT("Expecting an inner type for Arrays"));

				auto ArrayReplace = [&]
				{
					if (SubPropertyNode)
					{
						// Overriding all entry in the array
						SubPropertyNode->SetOperation(EOverriddenPropertyOperation::Replace);
					}

					// This is a case of the entire array is overridden
					// Need to loop through every sub object and override all properties
					for (int i = 0; i < ArrayHelper.Num(); ++i)
					{
						if (UObject* SubObject = TryGetInstancedSubObjectValue(InnerObjectProperty, ArrayHelper.GetElementPtr(i)))
						{
							OverridableManager.OverrideAllObjectProperties(SubObject);
						}
					}
				};

				// Only arrays flagged overridable logic can record deltas, for now just override entire array
				if (!ArrayProperty->HasAnyPropertyFlags(CPF_ExperimentalOverridableLogic))
				{
					if (Notification == EPropertyNotificationType::PostEdit)
					{
						if (ChangeType == EPropertyChangeType::Unspecified && ArrayIndex == INDEX_NONE)
						{
							// Overriding all entry in the array + override instanced sub objects
							ArrayReplace();
						}
						else if (SubPropertyNode)
						{
							// Overriding all entry in the array
							SubPropertyNode->SetOperation(EOverriddenPropertyOperation::Replace);
						}
					}
					return;
				}

				if (Notification == EPropertyNotificationType::PreEdit)
				{
					bool bAlreadySaved = false;
					uint8* PreEditArray = SavedPreEditContainers.FindOrAdd(Owner, ArrayProperty, bAlreadySaved);
					if (bAlreadySaved)
					{
						// The saved values needs to be identical, otherwise warns about missing post edit and continue to override with correct values
						// This happens when we receive multiple time the same event from subobject to owner and so on
						if (ArrayProperty->Identical(SubValuePtr, PreEditArray, 0))
						{
							UE_LOGF(LogOverridableObject, Verbose, "This owner:%ls(0x%p) has already handled this property:%ls", 
								*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
							return;
						}
						UE_LOGF(LogOverridableObject, Warning, "This owner:%ls(0x%p) has already a saved value for property(%ls) and it is expected be identical, this is an indication of a missing post edit",
							*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
					}

					FScriptArrayHelper PreEditArrayHelper(ArrayProperty, PreEditArray);
					PreEditArrayHelper.EmptyAndAddValues(ArrayHelper.Num());
					for (int32 i = 0; i < ArrayHelper.Num(); i++)
					{
						InnerObjectProperty->SetObjectPropertyValue(PreEditArrayHelper.GetElementPtr(i), TryGetInstancedSubObjectValue(InnerObjectProperty, ArrayHelper.GetElementPtr(i)));
					}
					return;
				}
				
				ON_SCOPE_EXIT
				{
					if (Notification == EPropertyNotificationType::PostEdit)
					{
						SavedPreEditContainers.Free(Owner, ArrayProperty);
					}
				};

				uint8* SavedPreEditArray = SavedPreEditContainers.Find(Owner, ArrayProperty);
				if (!SavedPreEditArray)
				{
					UE_LOGF(LogOverridableObject, Verbose, "Owner:%ls(0x%p) The property(%ls) seem to already be handled, skipping this post edit notification", 
						*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
					return;
				}
				FScriptArrayHelper PreEditArrayHelper(ArrayProperty, SavedPreEditArray);

				auto ArrayAddImpl = [&]()
				{
					if (!ensureMsgf(ArrayHelper.IsValidIndex(ArrayIndex), TEXT("ArrayAdd(%s) operation is expecting a valid index(%i), data might be lost"), *ArrayProperty->GetName(), ArrayIndex))
					{
						UE_LOGF(LogOverridableObject, Error, "ArrayAdd(%ls) operation is expecting a valid index(%i), data might be lost", *ArrayProperty->GetName(), ArrayIndex);
						return;
					}

					if (UObject* AddedSubObject = TryGetInstancedSubObjectValue(InnerObjectProperty, ArrayHelper.GetElementPtr(ArrayIndex)))
					{
						if(SubPropertyNode)
						{
							const FOverriddenPropertyNodeID  AddedSubObjectID(AddedSubObject);
							FOverriddenPropertyNode& AddedSubObjectNode = SubPropertyNode->FindOrAddNode(AddedSubObjectID);
							AddedSubObjectNode.SetOperation(EOverriddenPropertyOperation::Add);

							// Notify the subobject that it was added
							if (FOverriddenPropertySet* AddedSubObjectOverriddenProperties = OverridableManager.GetOverriddenProperties(AddedSubObject))
							{
								AddedSubObjectOverriddenProperties->bWasAdded = true;
							}
#if WITH_EDITOR
							FPropertyArrayExternalObjectHandler::Get().NotifyAdd(Owner, Property, AddedSubObject);
#endif
						}
					}
				};

				auto ArrayRemoveImpl = [&]()
				{
					if (!ensureMsgf(PreEditArrayHelper.IsValidIndex(ArrayIndex), TEXT("ArrayRemove(%s) operation is expecting a valid index(%i), data might be lost"), *ArrayProperty->GetName(), ArrayIndex))
					{
						UE_LOGF(LogOverridableObject, Error, "ArrayRemove(%ls) operation is expecting a valid index(%i), data might be lost", *ArrayProperty->GetName(), ArrayIndex);
						return;
					}

					if (UObject* RemovedSubObject = TryGetInstancedSubObjectValue(InnerObjectProperty, PreEditArrayHelper.GetElementPtr(ArrayIndex)))
					{
						if(SubPropertyNode)
						{
							// Check if there is a matching archetype for this object
							UObject* RemovedSubObjectArchetype = RemovedSubObject->GetArchetype();
							if (RemovedSubObjectArchetype && !RemovedSubObjectArchetype->HasAnyFlags(RF_ClassDefaultObject))
							{
								const FOverriddenPropertyNodeID RemovedSubObjectID (RemovedSubObjectArchetype);
								FOverriddenPropertyNode& RemovedSubObjectNode = SubPropertyNode->FindOrAddNode(RemovedSubObjectID);
								if (RemovedSubObjectNode.GetOperation() == EOverriddenPropertyOperation::Add)
								{
									// An add then a remove becomes no opt
									if (SubPropertyNode->RemoveSubPropertyNode(RemovedSubObjectID))
									{
										bNeedsCleanup = true;
									}
								}
								else
								{
									RemovedSubObjectNode.SetOperation(EOverriddenPropertyOperation::Remove);
								}

#if WITH_EDITOR
								FPropertyArrayExternalObjectHandler::Get().NotifyRemove(Owner, Property, RemovedSubObject, RemovedSubObjectArchetype);
#endif
							}
							else
							{
								// Figure out if it is a remove of a previously added element
								const FOverriddenPropertyNodeID RemovedSubObjectID (RemovedSubObject);
								if (const FOverriddenPropertyNode* AddedSubObjectNode = SubPropertyNode->GetSubPropertyNodes().FindByKey(RemovedSubObjectID))
								{
									if (AddedSubObjectNode->GetOperation() != EOverriddenPropertyOperation::Add)
									{
										UE_LOGF(LogOverridableObject, Warning, "This removed object:%ls(0x%p) was not tracked as an add in the overridden properties", *GetNameSafe(RemovedSubObject), RemovedSubObject);
									}

									// An add then a remove becomes no opt
									if (SubPropertyNode->RemoveSubPropertyNode(RemovedSubObjectID))
									{
										bNeedsCleanup = true;
									}

#if WITH_EDITOR
									FPropertyArrayExternalObjectHandler::Get().NotifyRemove(Owner, Property, RemovedSubObject, nullptr);
#endif
								}
								else
								{
									UE_LOGF(LogOverridableObject, Log, "This removed object:%ls(0x%p) was not tracked in the overridden properties", *GetNameSafe(RemovedSubObject), RemovedSubObject);
								}
							}
						}
					}
				};

				// Note: Currently, if CPF_ExperimentalOverridableLogic is set, we also require the property to be explicitly marked as an instanced subobject.
				checkf(InnerObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance), TEXT("Only instanced array properties support experimental overridable logic"));

				if (ChangeType & EPropertyChangeType::ValueSet)
				{
					checkf(ArrayIndex != INDEX_NONE, TEXT("ValueSet change type should have associated indexes"));
				}

				if (ChangeType & (EPropertyChangeType::ValueSet | EPropertyChangeType::Unspecified))
				{
					if (ArrayIndex != INDEX_NONE)
					{
						// Overriding a single entry in the array
						ArrayRemoveImpl();
						ArrayAddImpl();
					}
					else
					{
						ArrayReplace();
					}
					return;
				}

				if (ChangeType & EPropertyChangeType::ArrayAdd)
				{
					ArrayAddImpl();
					return;
				}

				if (ChangeType & EPropertyChangeType::ArrayRemove)
				{
					ArrayRemoveImpl();
					return;
				}
			
				if (ChangeType & EPropertyChangeType::ArrayClear)
				{
					checkf(ArrayIndex == INDEX_NONE, TEXT("ArrayClear change type should not have associated indexes"));

					for (int i = 0; i < PreEditArrayHelper.Num(); ++i)
					{
						ArrayIndex = i;
						ArrayRemoveImpl();
					}
					return;
				}
			
				if (ChangeType & EPropertyChangeType::ArrayMove)
				{
					UE_LOGF(LogOverridableObject, Log, "ArrayMove change type is not going to change anything as ordering of object isn't supported yet");
					return;
				}

				UE_LOGF(LogOverridableObject, Verbose, "Property change type is not supported will default to full array override");
			}
			// Can only forward to subobject if we have a valid index
			else if (ArrayHelper.IsValidIndex(ArrayIndex))
			{
				if (UObject* SubObject = TryGetInstancedSubObjectValue(InnerObjectProperty, ArrayHelper.GetElementPtr(ArrayIndex)))
				{
					// This should not be needed in the property grid, as it should already been called on the subobject itself.
					OverridableManager.NotifyPropertyChange(Notification, SubObject, NextPropertyIterator, ChangeType);
					return;
				}
			}
			else
			{
				UE_LOGF(LogOverridableObject, Warning, "Owner:%ls(0x%p) Information is suggesting a subobject path but we are unable to forward the notification because we do not know the array index of %ls, we might end up with missed match pre/post edit",
					*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
			}
		}
	}
	// @todo support set in the overridable serialization
	//else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
	//{
	//	
	//}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
	{
		// Special handling of instanced subobjects
		checkf(MapProperty->KeyProp, TEXT("Expecting a key type for Maps"));
		FObjectPropertyBase* KeyObjectProperty = CastField<FObjectPropertyBase>(MapProperty->KeyProp);

		// SubObjects
		checkf(MapProperty->ValueProp, TEXT("Expecting a value type for Maps"));
		FObjectPropertyBase* ValueObjectProperty = CastField<FObjectPropertyBase>(MapProperty->ValueProp);

		FScriptMapHelper MapHelper(MapProperty, SubValuePtr);

		// This code handles inner properties at the same time as the container property,
		// skip the container one if the next one is equal to one of its inner properties
		if (NextPropertyIterator && 
		    (NextPropertyIterator->Property == MapProperty->KeyProp ||
			 NextPropertyIterator->Property == MapProperty->ValueProp) )
		{
			++PropertyIterator;
			++NextPropertyIterator;
		}

		int32 LogicalMapIndex = PropertyIterator->Index;
		checkf(LogicalMapIndex == INDEX_NONE || 
			PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::ContainerIndex || 
			PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::MapKey || 
			PropertyIterator->PropertyInfo == EPropertyVisitorInfoType::MapValue, TEXT("Expecting a container type"));

		int32 InternalMapIndex = LogicalMapIndex != INDEX_NONE ? MapHelper.FindInternalIndex(LogicalMapIndex) : INDEX_NONE;
		if (!NextPropertyIterator)
		{
			if (Notification == EPropertyNotificationType::PreEdit)
			{
				bool bAlreadySaved = false;
				uint8* PreEditMap = SavedPreEditContainers.FindOrAdd(Owner, MapProperty, bAlreadySaved);
				if (bAlreadySaved)
				{
					// The saved values needs to be identical, otherwise warns about missing post edit and continue to override with correct values
					// This happens when we receive multiple time the same event from subobject to owner and so on
					if (MapProperty->Identical(SubValuePtr, PreEditMap, 0))
					{
						UE_LOGF(LogOverridableObject, Verbose, "This owner:%ls(0x%p) has already handled this property:%ls",
							*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
						return;
					}

					UE_LOGF(LogOverridableObject, Warning, "This owner:%ls(0x%p) has already a saved value for property(%ls) and it is expected be identical, this is an indication of a missing post edit",
						*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
				}
				FScriptMapHelper PreEditMapHelper(MapProperty, PreEditMap);
				PreEditMapHelper.EmptyValues();
				for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
				{
					PreEditMapHelper.AddPair(MapHelper.GetKeyPtr(It.GetInternalIndex()), MapHelper.GetValuePtr(It.GetInternalIndex()));
				}
				return;
			}

			uint8* SavedPreEditMap = SavedPreEditContainers.Find(Owner, MapProperty);
			if (!SavedPreEditMap)
			{
				UE_LOGF(LogOverridableObject, Verbose, "Owner:%ls(0x%p) The property(%ls) seem to already be handled, skipping this post edit notification",
					*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
				return;
			}
			checkf(SavedPreEditMap, TEXT("Expecting the same property as the pre edit flow"));
			FScriptMapHelper PreEditMapHelper(MapProperty, SavedPreEditMap);
			// The logical should map directly to the pre edit map internal index as we skipped all the invalid entries
			int32 InternalPreEditMapIndex = LogicalMapIndex;

			ON_SCOPE_EXIT
			{
				if (Notification == EPropertyNotificationType::PostEdit)
				{
					SavedPreEditContainers.Free(Owner, MapProperty);
				}
			};

			auto MapReplace = [&]()
			{
				// Overriding a all entries in the map
				if (SubPropertyNode)
				{
					SubPropertyNode->SetOperation(EOverriddenPropertyOperation::Replace);
				}

				// This is a case of the entire array is overridden
				// Need to loop through every sub object and setup them up as overridden
				for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
				{
					if(SubPropertyNode)
					{
						FOverriddenPropertyNodeID OverriddenKeyID = FOverriddenPropertyNodeID::FromMapKey(MapProperty->KeyProp, MapHelper.GetKeyPtr(It.GetInternalIndex()));
						FOverriddenPropertyNode& OverriddenKeyNode = SubPropertyNode->FindOrAddNode(OverriddenKeyID);
						OverriddenKeyNode.SetOperation(EOverriddenPropertyOperation::Replace);
					}

					// @todo support instanced object as a key in maps
					//if (UObject* KeySubObject = TryGetInstancedSubObjectValue(KeyObjectProperty, MapHelper.GetKeyPtr(It.GetInternalIndex())))
					//{
					//	checkf(false, TEXT("Keys as an instanced subobject is not supported yet"));
					//	OverridableManager.OverrideAllObjectProperties(KeySubObject);
					//}
					if (UObject* ValueSubObject = TryGetInstancedSubObjectValue(ValueObjectProperty, MapHelper.GetValuePtr(It.GetInternalIndex())))
					{
						OverridableManager.OverrideAllObjectProperties(ValueSubObject);
					}
				}
			};

			auto MapAddImpl = [&]()
			{
				if (!ensureMsgf(MapHelper.IsValidIndex(InternalMapIndex), TEXT("MapAdd(%s) operation is expecting a valid index(%i), data might be lost"), *MapProperty->GetName(), InternalMapIndex))
				{
					UE_LOGF(LogOverridableObject, Error, "MapAdd(%ls) operation is expecting a valid index(%i), data might be lost", *MapProperty->GetName(), InternalMapIndex);
					return;
				}

				if(SubPropertyNode)
				{
					FOverriddenPropertyNodeID AddedKeyID = FOverriddenPropertyNodeID::FromMapKey(MapProperty->KeyProp, MapHelper.GetKeyPtr(InternalMapIndex));
					FOverriddenPropertyNode& AddedKeyNode = SubPropertyNode->FindOrAddNode(AddedKeyID);
					AddedKeyNode.SetOperation(EOverriddenPropertyOperation::Add);
				}
			};

			auto MapRemoveImpl = [&]()
			{
				if (!ensureMsgf(PreEditMapHelper.IsValidIndex(InternalPreEditMapIndex), TEXT("MapRemove(%s) operation is expecting a valid index(%i), data might be lost"), *MapProperty->GetName(), InternalPreEditMapIndex))
				{
					UE_LOGF(LogOverridableObject, Error, "MapRemove(%ls) operation is expecting a valid index(%i), data might be lost", *MapProperty->GetName(), InternalPreEditMapIndex);
					return;
				}

				if(SubPropertyNode)
				{
					FOverriddenPropertyNodeID RemovedKeyID = FOverriddenPropertyNodeID::FromMapKey(MapProperty->KeyProp, PreEditMapHelper.GetKeyPtr(InternalPreEditMapIndex));
					FOverriddenPropertyNode& RemovedKeyNode = SubPropertyNode->FindOrAddNode(RemovedKeyID);
					if (RemovedKeyNode.GetOperation() == EOverriddenPropertyOperation::Add)
					{
						// @Todo support remove/add/remove
						if (SubPropertyNode->RemoveSubPropertyNode(RemovedKeyID))
						{
							bNeedsCleanup = true;
						}
					}
					else
					{
						RemovedKeyNode.SetOperation(EOverriddenPropertyOperation::Remove);
					}
				}
			};

			// Only maps flagged overridable logic can be handled here
			if (!MapProperty->HasAnyPropertyFlags(CPF_ExperimentalOverridableLogic))
			{
				if (ChangeType == EPropertyChangeType::Unspecified && InternalMapIndex == INDEX_NONE)
				{
					// Overriding all entry in the array + override instanced sub obejects
					MapReplace();
				}
				else if(SubPropertyNode)
				{
					// Overriding all entry in the array
					SubPropertyNode->SetOperation(EOverriddenPropertyOperation::Replace);
				}
				return;
			}

			// Ensure that an object key type is not explicitly marked as an instanced subobject. This is not supported yet.
			checkf(!KeyObjectProperty || !KeyObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance), TEXT("Keys as an instanced subobject is not supported yet"));
			// Note: Currently, if CPF_ExperimentalOverridableLogic is set on the map, we require its value type to be explicitly marked as an instanced subobject.
			checkf(!ValueObjectProperty || ValueObjectProperty->HasAllPropertyFlags(CPF_PersistentInstance), TEXT("Values must be instanced to support map overrides"));

			if (ChangeType & EPropertyChangeType::ValueSet)
			{
				checkf(LogicalMapIndex != INDEX_NONE, TEXT("ValueSet change type should have associated indexes"));
			}

			if (ChangeType & (EPropertyChangeType::ValueSet | EPropertyChangeType::Unspecified))
			{
				if(LogicalMapIndex != INDEX_NONE)
				{
					// Overriding a single entry in the map
					MapRemoveImpl();
					MapAddImpl();
				}
				else
				{
					MapReplace();
				}
				return;
			}

			if (ChangeType & EPropertyChangeType::ArrayAdd)
			{
				MapAddImpl();
				return;
			}

			if (ChangeType & EPropertyChangeType::ArrayRemove)
			{
				MapRemoveImpl();
				return;
			}
			
			if (ChangeType & EPropertyChangeType::ArrayClear)
			{
				checkf(InternalPreEditMapIndex == INDEX_NONE, TEXT("ArrayClear change type should not have associated indexes"));

				for (FScriptMapHelper::FIterator It(PreEditMapHelper); It; ++It)
				{
					InternalPreEditMapIndex = It.GetInternalIndex();
					MapRemoveImpl();
				}
				return;
			}
			
			if (ChangeType & EPropertyChangeType::ArrayMove)
			{
				UE_LOGF(LogOverridableObject, Log, "ArrayMove change type is not going to change anything as ordering of object isn't supported yet");
				return;
			}

			if (ChangeType & EPropertyChangeType::ArrayAdd)
			{
				MapAddImpl();
				return;
			}

			UE_LOGF(LogOverridableObject, Verbose, "Property change type is not supported will default to full array override");;
		}
		// Can only forward to subobject if we have a valid index
		else if (MapHelper.IsValidIndex(InternalMapIndex))
		{
			// @todo support instanced object as a key in maps
			//if (UObject* SubObject = TryGetInstancedSubObjectValue(KeyObjectProperty, MapHelper.GetValuePtr(InternalMapIndex)))
			//{
			//	checkf(false, TEXT("Keys as an instanced subobject is not supported yet"));
			//	// This should not be needed in the property grid, as it should already been called on the subobject.
			//	OverridableManager.NotifyPropertyChange(Notification, *SubObject, NextPropertyIterator, ChangeType);
			//	return;
			//}

			if (UObject* SubObject = TryGetInstancedSubObjectValue(ValueObjectProperty, MapHelper.GetValuePtr(InternalMapIndex)))
			{
				// This should not be needed in the property grid, as it should already been called on the subobject.
				OverridableManager.NotifyPropertyChange(Notification, SubObject, NextPropertyIterator, ChangeType);
				return;
			}
		}
		else
		{
			UE_LOGF(LogOverridableObject, Warning, "Owner:%ls(0x%p) Information is suggesting a subobject path but we are unable to forward the notification because we do not know the map index of %ls, we might end up with missed match pre/post edit",
				*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
		}
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		// No individual property serialization from this point on
		if (StructProperty->Struct->UseNativeSerialization())
		{
			if (Notification == EPropertyNotificationType::PostEdit && SubPropertyNode)
			{
				SubPropertyNode->SetOperation(EOverriddenPropertyOperation::Replace);
			}
		}
		else if (!NextPropertyIterator)
		{
			NotifyAllPropertyChange(SubPropertyNode, Notification, ChangeType, StructProperty->Struct, SubValuePtr);
		}
		else
		{
			NotifyPropertyChange(SubPropertyNode, Notification, NextPropertyIterator, ChangeType, SubValuePtr, bNeedsCleanup);
		}
		return;
	}
	else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
	{
		UObject* SubObject = TryGetInstancedSubObjectValue(ObjectProperty, SubValuePtr);
		if (!NextPropertyIterator)
		{
			if (Notification == EPropertyNotificationType::PreEdit)
			{
				bool bAlreadySaved = false;
				uint8* PreEditValuePtr = SavedPreEditContainers.FindOrAdd(Owner, ObjectProperty, bAlreadySaved);
				if (bAlreadySaved)
				{
					// The saved values needs to be identical, otherwise warns about missing post edit and continue to override with correct values
					// This happens when we receive multiple time the same event from subobject to owner and so on
					if (ObjectProperty->Identical(SubValuePtr, PreEditValuePtr, 0))
					{
						UE_LOGF(LogOverridableObject, Verbose, "This owner:%ls(0x%p) has already handled this property:%ls",
							*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
						return;
					}

					UE_LOGF(LogOverridableObject, Warning, "This owner:%ls(0x%p) has already a saved value for property(%ls) and it is expected be identical, this is an indication of a missing post edit",
						*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
				}

				ObjectProperty->SetObjectPtrPropertyValue(PreEditValuePtr, SubObject);
			}

			ON_SCOPE_EXIT
			{
				if (Notification == EPropertyNotificationType::PostEdit)
				{
					SavedPreEditContainers.Free(Owner, ObjectProperty);
				}
			};

			if (Notification == EPropertyNotificationType::PostEdit)
			{
				if (SubPropertyNode)
				{
					SubPropertyNode->SetOperation(EOverriddenPropertyOperation::Replace);
				}

				uint8* PreEditValuePtr = SavedPreEditContainers.Find(Owner, ObjectProperty);
				if (!PreEditValuePtr)
				{
					// PreEditValuePtr might be null on an unset TOptional<UObject*> as we never did a pre-edit on the inner object property as it was unset.
					UE_LOGF(LogOverridableObject, Verbose, "Owner:%ls(0x%p) The property(%ls) didn't have saved memory, might have inconsistent behavior",
						*GetNameSafe(Owner), Owner.Get(), *Property->GetName());
				}
				UObject* OldSubObject = PreEditValuePtr ? TryGetInstancedSubObjectValue(ObjectProperty, PreEditValuePtr) : nullptr;

				// Handle the case where subobject was nulled out by clearing any previous edit as the reinstancer can still find this object as it might not be GC'd yet
				if (OldSubObject && OldSubObject != SubObject)
				{
					OverridableManager.ClearOverrides(OldSubObject);
				}
				// If the object stayed the same, that means the users want to override all object properties
				else if (SubObject && SubObject == OldSubObject)
				{
					OverridableManager.OverrideAllObjectProperties(SubObject);
				}
			}
		}
		else if (SubObject)
		{
			// This should not be needed in the property grid, as it should already been called on the subobject.
			OverridableManager.NotifyPropertyChange(Notification, SubObject, NextPropertyIterator, ChangeType);
		}
		return;
	}
	else if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
	{
		if (!NextPropertyIterator)
		{
			if (Notification == EPropertyNotificationType::PostEdit && SubPropertyNode)
			{
				SubPropertyNode->SetOperation(EOverriddenPropertyOperation::Replace);
			}
		}
		else if (OptionalProperty->IsSet(Data))
		{
			NotifyPropertyChange(SubPropertyNode, Notification, NextPropertyIterator, ChangeType, OptionalProperty->GetValuePointerForRead(SubValuePtr), bNeedsCleanup);
		}
		return;
	}

	UE_CLOGF(NextPropertyIterator, LogOverridableObject, Verbose, "Unsupported property type(%ls), fallback to overriding entire property", *Property->GetName());
	if (Notification == EPropertyNotificationType::PostEdit)
	{
		if (SubPropertyNode)
		{
			// Replacing this entire property
			SubPropertyNode->SetOperation(EOverriddenPropertyOperation::Replace);
		}
	}
}

void FOverriddenPropertySet::NotifyAllPropertyChange(FOverriddenPropertyNode* ParentPropertyNode, const EPropertyNotificationType Notification, const EPropertyChangeType::Type ChangeType, TNotNull<const UStruct*> Struct, const void* Data)
{
	for (TFieldIterator<FProperty> It(Struct, EFieldIteratorFlags::IncludeSuper); It; ++It)
	{
		// Skip always overridden properties
		if (It->HasAnyPropertyFlags(CPF_ExperimentalAlwaysOverriden))
		{
			continue;
		}

		bool bNeedsCleanup = false;
		FPropertyVisitorPath Path(FPropertyVisitorInfo(It.operator*()));
		NotifyPropertyChange(ParentPropertyNode, Notification, Path.GetRootIterator(), ChangeType, Data, bNeedsCleanup);
	}
}

UObject* FOverriddenPropertySet::TryGetInstancedSubObjectValue(const FObjectPropertyBase* FromProperty, const void* ValuePtr) const
{
	// Property can be NULL - in that case there is no value.
	if (!FromProperty)
	{
		return nullptr;
	}

	// subobject pointers in IDOs point to the instance subobjects. For this purpose we need to redirect them to IDO subobjects
	UObject* SubObject = FromProperty->GetObjectPropertyValue(ValuePtr);
	const UObject* ExpectedOuter = Owner;
	TFunction<UObject* (UObject*)> RedirectMethod = [](UObject* Obj) {return Obj; };
#if WITH_EDITORONLY_DATA
	if (const UObject* Instance = UE::FPropertyBagRepository::Get().FindInstanceForDataObject(Owner))
	{
		RedirectMethod = UE::ResolveInstanceDataObject;
		ExpectedOuter = Instance;
	}
#endif // WITH_EDITORONLY_DATA

	if (FromProperty->HasAnyPropertyFlags(CPF_PersistentInstance)
		|| (FromProperty->IsA<FObjectProperty>() && SubObject && SubObject->IsIn(ExpectedOuter)))
	{
		return RedirectMethod(SubObject);
	}

	return nullptr;
}

EOverriddenPropertyOperation FOverriddenPropertySet::GetOverriddenPropertyOperation(FPropertyVisitorPath::Iterator PropertyIterator, bool* bOutInheritedOperation /*= nullptr*/) const
{
	return GetOverriddenPropertyOperation(&RootNode, PropertyIterator, bOutInheritedOperation, Owner);
}

bool FOverriddenPropertySet::ClearOverriddenProperty(FPropertyVisitorPath::Iterator PropertyIterator)
{
	return ClearOverriddenProperty(RootNode, PropertyIterator, Owner);
}

void FOverriddenPropertySet::OverrideProperty(FPropertyVisitorPath::Iterator PropertyIterator, const void* Data)
{
	bool bNeedsCleanup = false;
	NotifyPropertyChange(&RootNode, EPropertyNotificationType::PreEdit, PropertyIterator, EPropertyChangeType::Unspecified, Data, bNeedsCleanup);
	NotifyPropertyChange(&RootNode, EPropertyNotificationType::PostEdit, PropertyIterator, EPropertyChangeType::Unspecified, Data, bNeedsCleanup);
}

void FOverriddenPropertySet::SetExternalizedOverriddenProperty(const FOverriddenPropertyNodeID& NodeID, bool bExternal)
{	
	FOverriddenPropertyNode& Node = RootNode.FindOrAddNode(NodeID);
	FOverriddenPropertyNodeID ExternalizedID;

	if (bExternal)
	{
		FOverriddenPropertyNode& ExternalizeNode = Node.FindOrAddNode(ExternalizedID);
		ExternalizeNode.SetOperation(EOverriddenPropertyOperation::Externalized);
	}
	else
	{
		Node.RemoveSubPropertyNode(ExternalizedID);
		if (Node.GetSubPropertyNodes().Num() == 0)
		{
			Node.SetOperation(EOverriddenPropertyOperation::None);
		}
	}
}

void FOverriddenPropertySet::NotifyPropertyChange(const EPropertyNotificationType Notification, FPropertyVisitorPath::Iterator PropertyIterator, const EPropertyChangeType::Type ChangeType, const void* Data)
{
	bool bNeedsCleanup = false;
	NotifyPropertyChange(&RootNode, Notification, PropertyIterator, ChangeType, Data, bNeedsCleanup);
}

EOverriddenPropertyOperation FOverriddenPropertySet::GetOverriddenPropertyOperation(const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property) const
{
	return GetOverriddenPropertyOperation(&RootNode, CurrentPropertyChain, Property);
}

FOverriddenPropertyNode* FOverriddenPropertySet::RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation Operation, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property)
{
	return RestoreOverriddenPropertyOperation(Operation, RootNode, CurrentPropertyChain, Property);
}

FOverriddenPropertyNode* FOverriddenPropertySet::ConditionallyRestoreOverriddenPropertyOperation(EOverriddenPropertyOperation Operation, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property)
{
	// 'None', 'Modified' and 'SubObjectShadowing' operations are not needed to be restored on a property,
	// because 'None' is equal to the node not existing and
	// 'Modified' will be restored when the sub property overrides will be restored successfully
	if (Operation != EOverriddenPropertyOperation::None && 
		Operation != EOverriddenPropertyOperation::Modified &&
		Operation != EOverriddenPropertyOperation::SubObjectsShadowing)
	{
		// Prevent marking as replaced the properties that are always overridden
		if (!Property || Operation != EOverriddenPropertyOperation::Replace || !Property->HasAnyPropertyFlags(CPF_ExperimentalAlwaysOverriden))
		{
			return RestoreOverriddenPropertyOperation(Operation, CurrentPropertyChain, Property);
		}
	}

	return nullptr;
}

const FOverriddenPropertyNode* FOverriddenPropertySet::GetOverriddenPropertyNode(const FArchiveSerializedPropertyChain* CurrentPropertyChain) const
{
	return GetOverriddenPropertyNode(RootNode, CurrentPropertyChain);
}

const FOverriddenPropertyNode* FOverriddenPropertySet::GetOverriddenPropertyNode(const FOverriddenPropertyNodeID& NodeID) const
{
	if (const FOverriddenPropertyNode* FoundNode = RootNode.GetSubPropertyNodes().FindByKey(NodeID))
	{
		return FoundNode;
	}
	return nullptr;
}

EOverriddenPropertyOperation FOverriddenPropertySet::GetOverriddenPropertyOperation(const FOverriddenPropertyNode* ParentPropertyNode, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property) const
{
	// No need to look further
	// if it is the entire property is replaced or
	// if it is the FOverriddenPropertySet struct which is always Overridden
	if (ParentPropertyNode && ParentPropertyNode->GetOperation() == EOverriddenPropertyOperation::Replace)
	{
		return EOverriddenPropertyOperation::Replace;
	}

	// @Todo optimize find a way to not have to copy the property chain here.
	FArchiveSerializedPropertyChain PropertyChain(CurrentPropertyChain ? *CurrentPropertyChain : FArchiveSerializedPropertyChain());
	if(Property)
	{
		PropertyChain.PushProperty(Property, Property->IsEditorOnlyProperty());
	}

	TArray<class FProperty*, TInlineAllocator<8>>::TConstIterator PropertyIterator = PropertyChain.GetRootIterator();
	const FOverriddenPropertyNode* OverriddenPropertyNode = ParentPropertyNode;
	while (PropertyIterator && (!OverriddenPropertyNode || (OverriddenPropertyNode->GetOperation() != EOverriddenPropertyOperation::Replace)))
	{
		const FProperty* CurrentProperty = (*PropertyIterator);
		if (OverriddenPropertyNode)
		{
			FOverriddenPropertyNodeID NodeID(CurrentProperty);
			if (const FOverriddenPropertyNode* FoundNode = OverriddenPropertyNode->GetSubPropertyNodes().FindByKey(NodeID))
			{
				OverriddenPropertyNode = FoundNode;
				checkf(OverriddenPropertyNode, TEXT("Expecting a node"));
			}
			else
			{
				OverriddenPropertyNode = nullptr;
			}
		}
		// While digging down the path, if there is one property that is always overridden
		// stop there and return replace
		if (CurrentProperty->HasAnyPropertyFlags(CPF_ExperimentalAlwaysOverriden))
		{
			return EOverriddenPropertyOperation::Replace;
		}
		++PropertyIterator;
	}

	return OverriddenPropertyNode ? OverriddenPropertyNode->GetOperation() : EOverriddenPropertyOperation::None;
}

FOverriddenPropertyNode* FOverriddenPropertySet::RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation Operation, FOverriddenPropertyNode& ParentPropertyNode, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property)
{
	// No need to look further
	// if it is the entire property is replaced or
	// if it is the FOverriddenPropertySet struct which is always Overridden
	if (ParentPropertyNode.GetOperation() == EOverriddenPropertyOperation::Replace && Property)
	{
		return nullptr;
	}

	// @Todo optimize find a way to not have to copy the property chain here.
	FArchiveSerializedPropertyChain PropertyChain(CurrentPropertyChain ? *CurrentPropertyChain : FArchiveSerializedPropertyChain());
	if (Property)
	{
		PropertyChain.PushProperty(Property, Property->IsEditorOnlyProperty());
	}

	TArray<class FProperty*, TInlineAllocator<8>>::TConstIterator PropertyIterator = PropertyChain.GetRootIterator();
	FOverriddenPropertyNode* OverriddenPropertyNode = &ParentPropertyNode;
	while (PropertyIterator && OverriddenPropertyNode->GetOperation() != EOverriddenPropertyOperation::Replace)
	{
		const FProperty* CurrentProperty = (*PropertyIterator);
		// While digging down the path, if the property is always overridden then there should not be anything to do more
		if (CurrentProperty->HasAnyPropertyFlags(CPF_ExperimentalAlwaysOverriden))
		{
			return nullptr;
		}
		OverriddenPropertyNode = &OverriddenPropertyNode->FindOrAddNode(FOverriddenPropertyNodeID(CurrentProperty));
		++PropertyIterator;
	}

	// Might have stop before as one of the parent property was completely replaced.
	if (!PropertyIterator)
	{
		// If everything is replaced from here, remove all subnode operations
		if (Operation == EOverriddenPropertyOperation::Replace)
		{
			OverriddenPropertyNode->Reset();
		}
		OverriddenPropertyNode->SetOperation(Operation);
		return OverriddenPropertyNode;
	}

	return nullptr;
}

EOverriddenPropertyOperation FOverriddenPropertySet::GetSubPropertyOperation(const FOverriddenPropertyNodeID& NodeID) const
{
	if (const FOverriddenPropertyNode* SubPropertyNode = GetOverriddenPropertyNode(NodeID))
	{
		return SubPropertyNode->GetOperation();
	}
	return EOverriddenPropertyOperation::None;
}

FOverriddenPropertyNode* FOverriddenPropertySet::RestoreSubPropertyOperation(EOverriddenPropertyOperation Operation, FOverriddenPropertyNode& Node, const FOverriddenPropertyNodeID& NodeID)
{
	FOverriddenPropertyNode& OverriddenPropertyNode = Node.FindOrAddNode(NodeID);
	OverriddenPropertyNode.SetOperation(Operation);
	return &OverriddenPropertyNode;
}

FOverriddenPropertyNode* FOverriddenPropertySet::RestoreSubObjectOperation(EOverriddenPropertyOperation Operation, FOverriddenPropertyNode& Node, TNotNull<UObject*> SubObject)
{
	const FOverriddenPropertyNodeID SubObjectID(SubObject);
	FOverriddenPropertyNode* SubObjectNode = RestoreSubPropertyOperation(Operation, Node, SubObjectID);

	if (Operation == EOverriddenPropertyOperation::Add)
	{
		// Notify the subobject that it was added
		if (FOverriddenPropertySet* AddedSubObjectOverriddenProperties = FOverridableManager::Get().GetOverriddenProperties(SubObject))
		{
			AddedSubObjectOverriddenProperties->bWasAdded = true;
		}
	}

	return SubObjectNode;
}

bool FOverriddenPropertySet::IsCDOOwningProperty(const FProperty& Property) const
{
	checkf(Owner, TEXT("Expecting a valid overridable owner"));
	if (!Owner->HasAnyFlags(RF_ClassDefaultObject))
	{
		return false;
	}

	// We need to serialize only if the property owner is the current CDO class
	// Otherwise on derived class, this is done in parent CDO or it should be explicitly overridden if it is different than the parent value
	// This is sort of like saying it overrides the default property initialization value.
	return Property.GetOwnerClass() == Owner->GetClass();
}

void FOverriddenPropertySet::Reset(bool bShouldDirtyObject)
{
	RootNode.Reset();
}

void FOverriddenPropertySet::HandleObjectsReInstantiated(const TMap<UObject*, UObject*>& Map)
{
#if WITH_EDITOR
	// When there is a cached archetype, it is an indicator this object is about to be replaced
	// So no need to replace any ptr, otherwise we might not be able to reconstitute the right information
	if (FEditorCacheArchetypeManager::Get().GetCachedArchetype(Owner))
	{
		return;
	}
#endif // WITH_EDITOR

	RootNode.HandleObjectsReInstantiated(Map);
}

void FOverriddenPropertySet::HandleDeadObjectReferences(const TSet<UObject*>& ActiveInstances, const TSet<UObject*>& TemplateInstances)
{
	RootNode.HandleDeadObjectReferences(ActiveInstances, TemplateInstances);
}

const FOverriddenPropertyNode* FOverriddenPropertySet::GetOverriddenPropertyNode(const FOverriddenPropertyNode& ParentPropertyNode, const FArchiveSerializedPropertyChain* CurrentPropertyChain) const
{
	if (!CurrentPropertyChain)
	{
		return &ParentPropertyNode;
	}

	TArray<class FProperty*, TInlineAllocator<8>>::TConstIterator PropertyIterator = CurrentPropertyChain->GetRootIterator();
	const FOverriddenPropertyNode* OverriddenPropertyNode = &ParentPropertyNode;
	while (PropertyIterator && OverriddenPropertyNode)
	{
		const FProperty* CurrentProperty = (*PropertyIterator);
		FOverriddenPropertyNodeID NodeID(CurrentProperty);
		if (const FOverriddenPropertyNode* FoundNode = OverriddenPropertyNode->GetSubPropertyNodes().FindByKey(NodeID))
		{
			OverriddenPropertyNode = FoundNode;
		}
		else
		{
			OverriddenPropertyNode = nullptr;
			break;
		}
		++PropertyIterator;
	}

	return OverriddenPropertyNode;
}

FOverridableObjectArrayOverrides::FOverridableObjectArrayOverrides() :
	ObjectsOperation(EOverriddenPropertyOperation::None)
{
}

FScopedPropertyValue::FScopedPropertyValue(FProperty* InProperty)
	: Property(InProperty)
	, Data(FMemory::Malloc(InProperty->GetSize(), InProperty->GetMinAlignment()))
{
	Property->InitializeValue(Data);
}

FScopedPropertyValue::~FScopedPropertyValue()
{
	if (Data)
	{
		Property->DestroyAndFreeValue(Data);
	}
}

FScopedPropertyValue::FScopedPropertyValue(FScopedPropertyValue&& Other)
	: Property(Other.Property)
	, Data(Other.Data)
{
	if (this != &Other)
	{
		Other.Data = nullptr;
		Other.Property = nullptr;
	}
}

FScopedPropertyValue& FScopedPropertyValue::operator=(FScopedPropertyValue&& Other)
{
	if (this != &Other)
	{
		if (Data)
		{
			Property->DestroyAndFreeValue(Data);
		}

		Data = Other.Data;
		Property = Other.Property;

		Other.Data = nullptr;
		Other.Property = nullptr;
	}
	return *this;
}

FScopedPropertyValue::operator bool() const
{
	return !!Data;
}

void* FScopedPropertyValue::GetData() const
{
	return Data;
}

static void ToDebugString_Internal(const FOverriddenPropertyNode& Node, FStringBuilderBase& StrBuilder, int StackDepth = 0)
{
	for (int D = 0; D < StackDepth; ++D)
	{
		StrBuilder.Append(TEXT("  "));
	}

	StrBuilder.Append(GetOverriddenOperationString(Node.GetOperation()));
	StrBuilder.Append(TEXT(" "));
	StrBuilder.Append(Node.GetNodeID().ToDebugString());
	StrBuilder.Append(TEXT("\n"));

	for (const FOverriddenPropertyNode& SubNode : Node.GetSubPropertyNodes())
	{
		ToDebugString_Internal(SubNode, StrBuilder, StackDepth + 1);
	}
}

FString FOverriddenPropertySet::ToDebugString() const
{
	TStringBuilder<32> Builder;
	ToDebugString_Internal(RootNode, Builder);
	return Builder.ToString();
}
