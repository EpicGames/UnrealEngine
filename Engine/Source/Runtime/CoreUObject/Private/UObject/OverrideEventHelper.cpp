// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/OverrideEventHelper.h"
#include "UObject/OverridableManager.h"


namespace UE
{
	// helper class that constructs a change event using FPropertyVisitorPaths
	class ProxyChangeEvent : public FEditPropertyChain, public FPropertyChangedChainEvent
	{
	public:
		ProxyChangeEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, EPropertyChangeType::Type ChangeType)
			: FPropertyChangedChainEvent(*this, CreateChangeEvent(Object, PropertyPath, ChangeType))
		{
			ArrayIndicesPerObject.AddDefaulted(ObjectIteratorIndex + 1);
			for (const FPropertyVisitorInfo& Info : PropertyPath.GetPath())
			{
				FEditPropertyChain::AddTail(const_cast<FProperty*>(Info.Property));
				ArrayIndicesPerObject[ObjectIteratorIndex].Add(Info.Property->GetName(), Info.Index);
			}
			FPropertyChangedChainEvent::SetArrayIndexPerObject(ProxyChangeEvent::ArrayIndicesPerObject);
		}

		ProxyChangeEvent(TNotNull<UObject*> Object, EPropertyChangeType::Type ChangeType)
			: FPropertyChangedChainEvent(*this, CreateChangeEvent(Object, {}, ChangeType))
		{}

		ProxyChangeEvent(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain, EPropertyChangeType::Type ChangeType)
			: FPropertyChangedChainEvent(*this, const_cast<FPropertyChangedEvent&>(PropertyEvent))
		{
			FPropertyChangedChainEvent::ChangeType = ChangeType;
			// The copy constructor of PropertyChain was deleted so copy it manually here.
			for (FEditPropertyChain::TDoubleLinkedListNode* Itr = PropertyChain.GetHead(); Itr && Itr->GetValue(); Itr = Itr->GetNextNode())
			{
				FEditPropertyChain::AddTail(Itr->GetValue());
				if (PropertyChain.GetActiveMemberNode() == Itr)
				{
					FEditPropertyChain::ActiveMemberPropertyNode = FEditPropertyChain::GetTail();
				}
				if (PropertyChain.GetActiveNode() == Itr)
				{
					FEditPropertyChain::ActivePropertyNode = FEditPropertyChain::GetTail();
				}
			}
		}

	private:
		static FPropertyChangedEvent CreateChangeEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, EPropertyChangeType::Type ChangeType)
		{
			FProperty* ChangedProperty = PropertyPath.Num() > 0 ? const_cast<FProperty*>(PropertyPath.Top().Property) : nullptr;
			FPropertyChangedEvent Result(ChangedProperty, ChangeType, { Object });
			Result.ObjectIteratorIndex = 0;
			return Result;
		}

		TArray<TMap<FString, int32>> ArrayIndicesPerObject;
	};

	void SendOverrideAllObjectPropertiesEvent(TNotNull<UObject*> Object)
	{
#if UE_EDITOR
		// an event with no provided property path will override the entire object
		ProxyChangeEvent ChangeEvent(Object, EPropertyChangeType::Unspecified);
		FOverridableManager::Get().PreOverrideProperty(Object, ChangeEvent);
		FOverridableManager::Get().PostOverrideProperty(Object, ChangeEvent);
		FPropertyBagRepository::Get().PostEditChangeChainProperty(Object, ChangeEvent);
#endif
	}

	void SendClearOverridesEvent(TNotNull<UObject*> Object)
	{
#if UE_EDITOR
		// a ResetToDefault event with no provided property path will clear all of the object's overrides
		ProxyChangeEvent ChangeEvent(Object, EPropertyChangeType::ResetToDefault);
		FOverridableManager::Get().PreOverrideProperty(Object, ChangeEvent);
		FOverridableManager::Get().PostOverrideProperty(Object, ChangeEvent);
		FPropertyBagRepository::Get().PostEditChangeChainProperty(Object, ChangeEvent);
#endif
	}

	void SendOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, EPropertyChangeType::Type ChangeType)
	{
#if UE_EDITOR
		ProxyChangeEvent ChangeEvent(Object, PropertyPath, ChangeType);
		FOverridableManager::Get().PreOverrideProperty(Object, ChangeEvent);
		FOverridableManager::Get().PostOverrideProperty(Object, ChangeEvent);
		FPropertyBagRepository::Get().PostEditChangeChainProperty(Object, ChangeEvent);
#endif
	}

	void SendOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain)
	{
#if UE_EDITOR
		ProxyChangeEvent ChangeEvent(Object, PropertyEvent, PropertyChain, PropertyEvent.ChangeType);
		FOverridableManager::Get().PreOverrideProperty(Object, ChangeEvent);
		FOverridableManager::Get().PostOverrideProperty(Object, ChangeEvent);
		FPropertyBagRepository::Get().PostEditChangeChainProperty(Object, ChangeEvent);
#endif
	}

	void SendClearOverriddenPropertyEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath)
	{
#if UE_EDITOR
		ProxyChangeEvent ChangeEvent(Object, PropertyPath, EPropertyChangeType::ResetToDefault);
		FOverridableManager::Get().PreOverrideProperty(Object, ChangeEvent);
		FOverridableManager::Get().PostOverrideProperty(Object, ChangeEvent);
		FPropertyBagRepository::Get().PostEditChangeChainProperty(Object, ChangeEvent);
#endif
	}

	void SendClearOverriddenPropertyEvent(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain)
	{
#if UE_EDITOR
		ProxyChangeEvent ChangeEvent(Object, PropertyEvent, PropertyChain, EPropertyChangeType::ResetToDefault);
		FOverridableManager::Get().PreOverrideProperty(Object, ChangeEvent);
		FOverridableManager::Get().PostOverrideProperty(Object, ChangeEvent);
		FPropertyBagRepository::Get().PostEditChangeChainProperty(Object, ChangeEvent);
#endif
	}

	void SendPreOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath)
	{
#if UE_EDITOR
		ProxyChangeEvent ChangeEvent(Object, PropertyPath, EPropertyChangeType::Unspecified);
		FOverridableManager::Get().PreOverrideProperty(Object, ChangeEvent);
#endif
	}

	void SendPreOverridePropertyEvent(TNotNull<UObject*> Object, const FEditPropertyChain& PropertyChain)
	{
#if UE_EDITOR
		FOverridableManager::Get().PreOverridePropertyChain(Object, PropertyChain);
#endif
	}

	void SendPostOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, const EPropertyChangeType::Type ChangeType)
	{
#if UE_EDITOR
		ProxyChangeEvent ChangeEvent(Object, PropertyPath, ChangeType);
		FOverridableManager::Get().PostOverrideProperty(Object, ChangeEvent);
		FPropertyBagRepository::Get().PostEditChangeChainProperty(Object, ChangeEvent);
#endif
	}

	void SendPostOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain)
	{
#if UE_EDITOR
		ProxyChangeEvent ChangeEvent(Object, PropertyEvent, PropertyChain, PropertyEvent.ChangeType);
		FOverridableManager::Get().PostOverrideProperty(Object, ChangeEvent);
		FPropertyBagRepository::Get().PostEditChangeChainProperty(Object, ChangeEvent);
#endif
	}
}
