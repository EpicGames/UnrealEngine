// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/Package.h"

#include "AssetRegistry/AssetData.h"
#include "Cooker/CookDependency.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformMath.h"
#include "HAL/IConsoleManager.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/ITransaction.h"
#include "Misc/PackageName.h"
#include "UObject/GarbageCollectionSchema.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerSave.h"
#include "UObject/LinkerManager.h"
#include "UObject/MetaData.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectThreadContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Package)

FText AssetAccessSpecifierAsText(EAssetAccessSpecifier InAccess)
{
#define LOCTEXT_NAMESPACE "AssetAccessSpecifier"
	switch (InAccess)
	{
	case EAssetAccessSpecifier::Public:
	{
		return LOCTEXT("Public", "Public");
	}

	case EAssetAccessSpecifier::EpicInternal:
	{
		return LOCTEXT("EpicInternal", "Epic Internal");
	}

	case EAssetAccessSpecifier::Private:
	{
		return LOCTEXT("Private", "Private");
	}
	default:
		ensureMsgf(false, TEXT("Unknwon EAssetAccessSpecifier value %d"), EnumToUnderlyingType(InAccess));
		return FText::GetEmpty();
	}
#undef LOCTEXT_NAMESPACE
}

/*-----------------------------------------------------------------------------
	UPackage.
-----------------------------------------------------------------------------*/

#if WITH_EDITOR && !UE_BUILD_SHIPPING
static FAutoConsoleVariable DumpStackTraceOnPackageDirty(
	TEXT("Package.DumpStackTraceOnDirty"),
	0,
	TEXT("Dumps a stack trace every time a package is dirtied. Executes the number of times specified, -1 = infinite."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		if (!IsInGameThread())
		{
			return;
		}
		
		static FDelegateHandle PackageDirtyStateChangedEventHandle;
		int32 MaxRunCount = Variable->GetInt();
		static int32 RunsSoFar = 0;

		if (PackageDirtyStateChangedEventHandle.IsValid())
		{
			UPackage::PackageDirtyStateChangedEvent.Remove(PackageDirtyStateChangedEventHandle);
			PackageDirtyStateChangedEventHandle.Reset();
			RunsSoFar = 0;
		}

		if (MaxRunCount != 0)
		{
			PackageDirtyStateChangedEventHandle = UPackage::PackageDirtyStateChangedEvent.AddLambda([MaxRunCount](const UPackage* InPackage)
			{
				if(MaxRunCount != -1 && RunsSoFar >= MaxRunCount)
				{
					return;
				}
				if (InPackage && InPackage->IsDirty())
				{
					const SIZE_T StackTraceSize = 65536;
					ANSICHAR StackTrace[StackTraceSize] = { 0 };
					FPlatformStackWalk::StackWalkAndDump(StackTrace, StackTraceSize, 1);
					UE_LOGF(LogObj, Warning, "***** Package %ls Marked Dirty ******\n%ls", *InPackage->GetPathName(), ANSI_TO_TCHAR(StackTrace));
					RunsSoFar++;
				}
			});
		}
	}));
#endif // WITH_EDITOR && !UE_BUILD_SHIPPING

UPackage::FPreSavePackageWithContext UPackage::PreSavePackageWithContextEvent;
UPackage::FOnPackageSavedWithContext UPackage::PackageSavedWithContextEvent;
/** Delegate to notify subscribers when the dirty state of a package is changed.
 *  Allows the editor to register the modified package as one that should be prompted for source control checkout. 
 *  Use Package->IsDirty() to get the updated dirty state of the package */
UPackage::FOnPackageDirtyStateChanged UPackage::PackageDirtyStateChangedEvent;
/** 
 * Delegate to notify subscribers when a package is marked as dirty via UObjectBaseUtility::MarkPackageDirty 
 * Note: Unlike FOnPackageDirtyStateChanged, this is always called, even when the package is already dirty
 * Use bWasDirty to check the previous dirty state of the package
 * Use Package->IsDirty() to get the updated dirty state of the package
 */
UPackage::FOnPackageMarkedDirty UPackage::PackageMarkedDirtyEvent;

#if WITH_METADATA
UPackage::FOnMetaDataTransferRequestedDelegate UPackage::OnMetaDataTransferRequestedDelegate;
#endif

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Silence deprecation warnings for deprecated CookedHash and LinkerSave members
FSavePackageResultStruct::FSavePackageResultStruct()
	: Result(ESavePackageResult::Error), TotalFileSize(0), SerializedPackageFlags(0)
{
}
FSavePackageResultStruct::FSavePackageResultStruct(ESavePackageResult InResult)
	: Result(InResult), TotalFileSize(0), SerializedPackageFlags(0)
{
}
FSavePackageResultStruct::FSavePackageResultStruct(ESavePackageResult InResult, int64 InTotalFileSize)
	: Result(InResult), TotalFileSize(InTotalFileSize), SerializedPackageFlags(0)
{
}
FSavePackageResultStruct::FSavePackageResultStruct(ESavePackageResult InResult, int64 InTotalFileSize,
	uint32 InSerializedPackageFlags)
	: Result(InResult), TotalFileSize(InTotalFileSize), SerializedPackageFlags(InSerializedPackageFlags)
{
}
FSavePackageResultStruct::FSavePackageResultStruct(ESavePackageResult InResult, int64 InTotalFileSize,
	uint32 InSerializedPackageFlags, TPimplPtr<FLinkerSave> Linker)
	: Result(InResult), TotalFileSize(InTotalFileSize), SerializedPackageFlags(InSerializedPackageFlags)
{
	Linker.Reset();
}
FSavePackageResultStruct::FSavePackageResultStruct(FSavePackageResultStruct&& Other) = default;
FSavePackageResultStruct& FSavePackageResultStruct::operator=(FSavePackageResultStruct&& Other) = default;
FSavePackageResultStruct::~FSavePackageResultStruct() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS;

void UPackage::PostInitProperties()
{
	Super::PostInitProperties();
	if ( !HasAnyFlags(RF_ClassDefaultObject) )
	{
		bDirty = false;
	}

	SetLinkerPackageVersion(GPackageFileUEVersion);
	SetLinkerLicenseeVersion(GPackageFileLicenseeUEVersion);

#if WITH_EDITORONLY_DATA
	// Always generate a new unique PersistentGuid, required for new disk packages.
	// For existing disk packages it will be replaced with the existing PersistentGuid when loading the package summary.
	// For existing script packages it will be replaced in ConstructUPackage with the CRC of the generated code files.
	PersistentGuid = FGuid::NewGuid();

	SetPIEInstanceID(INDEX_NONE);
	bIsCookedForEditor = false;
	bIsDynamicPIEPackagePending = false;
#endif // WITH_EDITORONLY_DATA
}


/**
 * Marks/Unmarks the package's bDirty flag
 */
void UPackage::SetDirtyFlag( bool bInIsDirty )
{
	// Early out if there is no change to the flag
	if (bDirty != bInIsDirty)
	{
		if ( GetOutermost() != GetTransientPackage() )
		{
			if ( GUndo != nullptr
				// PIE and script/class packages should never end up in the transaction buffer as we cannot undo during gameplay.
				&& !GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor|PKG_ContainsScript|PKG_CompiledIn) )
			{
				// make sure we're marked as transactional
				SetFlags(RF_Transactional);

				// don't call Modify() since it calls SetDirtyFlag()
				GUndo->SaveObject( this );
			}

			// Update dirty bit after we saved the object in the transaction buffer.
			bDirty = bInIsDirty;

			if (GIsEditor									// Only fire the callback in editor mode
				&& !HasAnyPackageFlags(PKG_ContainsScript)	// Skip script packages
				&& !HasAnyPackageFlags(PKG_PlayInEditor)	// Skip packages for PIE
				&& GetTransientPackage() != this )			// Skip the transient package
			{
				// Package is changing dirty state, let the editor know so we may prompt for source control checkout
				PackageDirtyStateChangedEvent.Broadcast(this);
			}
		}
	}
}

/**
 * Serializer
 * Save the value of bDirty into the transaction buffer, so that undo/redo will also mark/unmark the package as dirty, accordingly
 */
void UPackage::Serialize( FArchive& Ar )
{
	Super::Serialize(Ar);

	if (Ar.IsCountingMemory())
	{		
		if (FLinker* Loader = GetLinker())
		{
			Loader->Serialize(Ar);
		}
	}
}

#if WITH_EDITOR
bool UPackage::bSupportCookerSoftGC = false;
TMap<UPackage*, TArrayView<TObjectPtr<UObject>>> UPackage::SoftGCPackageToObjectList;
void UPackage::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	if (bSupportCookerSoftGC)
	{
		UPackage* ThisPackage = static_cast<UPackage*>(InThis);
		auto* ObjectList = SoftGCPackageToObjectList.Find(ThisPackage);
		if (ObjectList)
		{
			for (auto& Object : *ObjectList)
			{
				Collector.AddReferencedObject(Object);
			}
		}
	}
	Super::AddReferencedObjects(InThis, Collector);
}
#endif

UObject* UPackage::FindAssetInPackage(EObjectFlags RequiredTopLevelFlags) const
{
	UObject* Asset = nullptr;
	bool bAssetValid = false;

	ForEachObjectWithPackage(this, [&Asset, &bAssetValid, RequiredTopLevelFlags](UObject* Object)
		{
			if (Object->IsAsset() && !UE::AssetRegistry::FFiltering::ShouldSkipAsset(Object) &&
				(RequiredTopLevelFlags == RF_NoFlags || Object->HasAnyFlags(RequiredTopLevelFlags)))
			{
				const bool bIsValid = IsValid(Object);
				const bool bIsUAsset = FAssetData::IsUAsset(Object);

				if (!Asset)
				{
					Asset = Object;
					bAssetValid = bIsValid;
					// stop iterating if Asset is valid and also a UAsset
					return !(bIsValid && bIsUAsset);
				}
				else if(bIsValid)
				{
					// Overwrite found asset if previous was invalid or new one is a UAsset
					if (!bAssetValid || bIsUAsset)
					{
						Asset = Object;
						bAssetValid = true;
					}
					// stop iterating if found asset is a UAsset
					return !bIsUAsset;
				}
			}
			return true;
		}, EGetObjectsFlags::None);
	return Asset;
}

TArray<UPackage*> UPackage::GetExternalPackages() const
{
	TSet<UPackage*> Result;
	ForEachObjectWithPackage(const_cast<UPackage*>(this), 
		[this, &Result](UObject* Object)
		{
			ForEachObjectWithOuter(Object, 
				[this, &Result](UObject* InObject)
				{
					UPackage* ObjectPackage = InObject->GetExternalPackage();
					if (ObjectPackage && ObjectPackage != this)
					{
						Result.Add(ObjectPackage);
					}
				});
			return true;
		}, EGetObjectsFlags::None);
	return Result.Array();
}

#if WITH_METADATA
/**
 * Gets (after possibly creating) a metadata object for this package
 *
 * @return A valid FMetaData pointer for all objects in this package
 */
FMetaData& UPackage::GetMetaData()
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (IsValid(DeprecatedMetaData))
	{
		DeprecatedMetaData->ConditionalPreload();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return MetaData;
}
#endif

/**
 * Fully loads this package. Safe to call multiple times and won't clobber already loaded assets.
 */
void UPackage::FullyLoad()
{
	// Make sure we're a topmost package.
	checkf(GetOuter()==nullptr, TEXT("Package is not topmost. Name:%s Path: %s"), *GetName(), *GetPathName());

	// Only perform work if we're not already fully loaded.
	if(!IsFullyLoaded())
	{
		// Re-load this package.
		LoadPackage(nullptr, *GetName(), LOAD_None);
	}
}

const FPackagePath& UPackage::GetLoadedPath() const
{
	return LoadedPath;
}

void UPackage::SetLoadedPath(const FPackagePath& InPackagePath)
{
	LoadedPath = InPackagePath;
}

/**
 * Returns whether the package is fully loaded.
 *
 * @return true if fully loaded or no file associated on disk, false otherwise
 */
bool UPackage::IsFullyLoaded() const
{
	if (bHasBeenFullyLoaded)
	{
		return true;
	}

	// We set bHasBeenFullyLoaded to true when it is read for some special cases

	if (GetFileSize() != 0)
	{
		// If it has a filesize, it is a normal on-disk package, therefore is not a special case, and we respect the current 'false' value of bHasBeenFullyLoaded
		return false;
	}

	if (HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
	{
		// If it's in the middle of an async load, don't make any changes and respect the current 'false' value of bHasBeenFullyLoaded
		return false;
	}

	if (HasAnyPackageFlags(PKG_CompiledIn))
	{
		// Native packages don't have a file size but are always considered fully loaded.
		bHasBeenFullyLoaded = true;
		return true;
	}

	// Newly created packages aren't loaded and therefore haven't been marked as being fully loaded. They are treated as fully
	// loaded packages though in this case, which is why we are looking to see whether the package exists on disk and assume it
	// has been fully loaded if it doesn't.
	// Try to find matching package in package file cache. We use the LoadedPath here as it may be loaded into a temporary package
	FString DummyFilename;
	FPackagePath SourcePackagePath = !LoadedPath.IsEmpty() ? LoadedPath : FPackagePath::FromPackageNameChecked(GetName());
	if (!FPackageName::DoesPackageExist(SourcePackagePath, &SourcePackagePath) ||
		(GIsEditor && IPackageResourceManager::Get().FileSize(SourcePackagePath) < 0))
	{
		// Package has NOT been found, so we assume it's a newly created one and therefore fully loaded.
		bHasBeenFullyLoaded = true;
		return true;
	}

	// Not a special case; respect the current 'false' value of bHasBeenFullyLoaded
	return false;
}

void UPackage::FinishDestroy()
{
	// Detach linker if still attached, we do this in ::FinishDestroy rather than ::BeginDestroy so that the linker remains attached
	// and valid for all UObjects in the package until they have all returned ::IsReadyForFinishDestroy as true. This means that 
	// UObjects with ongoing asynchronous compilation work can safely cancel that work in ::BeginDestroy and wait for it to finish
	// in ::IsReadyForFinishDestroy without worrying that the package file will be yanked out from under it.
	if (FLinkerLoad* Linker = GetLinker())
	{
		// Detach() below will most likely null the LinkerLoad so keep a temp copy so that we can still call RemoveLinker on it
		Linker->Detach();
		FLinkerManager::Get().RemoveLinker(Linker);
		SetLinker(nullptr);
	}

	Super::FinishDestroy();
}

bool UPackage::IsPostLoadThreadSafe() const
{
	return true;
}

bool UPackage::Rename(const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags)
{
#if WITH_METADATA
	const FName OldPackageName = GetFName();
#endif

	if (!Super::Rename(InName, NewOuter, Flags))
	{
		return false;
	}

	if (Flags & REN_Test)
	{
		return true;
	}

#if WITH_METADATA
	const FName NewPackageName = GetFName();
	if (OldPackageName != NewPackageName)
	{
		MetaData.RemapObjectKeys(OldPackageName, NewPackageName);
	}
#endif

	return true;
}

#if WITH_EDITORONLY_DATA

FIoHash UPackage::GetSavedHash() const
{
	return SavedHash;
}

void UPackage::SetSavedHash(const FIoHash& InSavedHash)
{
	SavedHash = InSavedHash;
}

#endif
