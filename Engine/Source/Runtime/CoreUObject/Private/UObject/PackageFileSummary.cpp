// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PackageFileSummary.h"

#include "HAL/PlatformMath.h"
#include "Misc/Compression.h"
#include "Runtime/Launch/Resources/Version.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UObjectGlobals.h"

FPackageFileSummary::FPackageFileSummary()
{
	FMemory::Memzero( this, sizeof(*this) );
}

/** Converts file version to custom version system version */
static ECustomVersionSerializationFormat GetCustomVersionFormatForArchive(int32 LegacyFileVersion)
{
	ECustomVersionSerializationFormat CustomVersionFormat = ECustomVersionSerializationFormat::Unknown;
	if (LegacyFileVersion == -2)
	{
		CustomVersionFormat = ECustomVersionSerializationFormat::Enums;
	}
	else if (LegacyFileVersion < -2 && LegacyFileVersion >= -5)
	{
		CustomVersionFormat = ECustomVersionSerializationFormat::Guids;
	}
	else if (LegacyFileVersion < -5)
	{
		CustomVersionFormat = ECustomVersionSerializationFormat::Optimized;
	}
	check(CustomVersionFormat != ECustomVersionSerializationFormat::Unknown);
	return CustomVersionFormat;
}

static void FixCorruptEngineVersion(const FPackageFileVersion& ObjectVersion, FEngineVersion& Version)
{
	// The move of EpicInternal.txt in CL 12740027 broke checks for non-licensee builds in UGS. resulted in checks for Epic internal builds in UGS breaking, and assets being saved out with the licensee flag set.
	// Detect such assets and clear the licensee bit.
	if (ObjectVersion < VER_UE4_CORRECT_LICENSEE_FLAG
		&& Version.GetMajor() == 4
		&& Version.GetMinor() == 26
		&& Version.GetPatch() == 0
		&& Version.GetChangelist() >= 12740027
		&& Version.IsLicenseeVersion())
	{
		Version.Set(4, 26, 0, Version.GetChangelist(), Version.GetBranch());
	}
}

void operator<<(FStructuredArchive::FSlot Slot, FPackageFileSummary& Sum)
{
	FArchive& BaseArchive = Slot.GetUnderlyingArchive();
	bool bCanStartSerializing = true;
	int64 ArchiveSize = 0;
	if (BaseArchive.IsLoading())
	{
		// Sanity checks before we even start serializing the archive
		ArchiveSize = BaseArchive.TotalSize();
		const int64 MinimumPackageSize = 32; // That should get us safely to Sum.TotalHeaderSize
		bCanStartSerializing = ArchiveSize >= MinimumPackageSize;
		UE_CLOGF(!bCanStartSerializing, LogLinker, Warning,
			"Failed to read package file summary, the file \"%ls\" is too small (%lld bytes, expected at least %lld bytes)",
			*BaseArchive.GetArchiveName(), ArchiveSize, MinimumPackageSize);
	}

	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	int64 StartOffset = BaseArchive.Tell();

	if (bCanStartSerializing)
	{
		Record << SA_VALUE(TEXT("Tag"), Sum.Tag);
	}
	// only keep loading if we match the magic
	if (Sum.Tag == PACKAGE_FILE_TAG || Sum.Tag == PACKAGE_FILE_TAG_SWAPPED)
	{
		// The package has been stored in a separate endianness than the linker expected so we need to force
		// endian conversion. Latent handling allows the PC version to retrieve information about cooked packages.
		if (Sum.Tag == PACKAGE_FILE_TAG_SWAPPED)
		{
			// Set proper tag.
			Sum.Tag = PACKAGE_FILE_TAG;
			// Toggle forced byte swapping.
			if (BaseArchive.ForceByteSwapping())
			{
				BaseArchive.SetByteSwapping(false);
			}
			else
			{
				BaseArchive.SetByteSwapping(true);
			}
		}
		/**
		* The package file version number when this package was saved.
		*
		* Lower 16 bits stores the UE3 engine version
		* Upper 16 bits stores the UE licensee version
		* For newer packages this is -7
		*		-2 indicates presence of enum-based custom versions
		*		-3 indicates guid-based custom versions
		*		-4 indicates removal of the UE3 version. Packages saved with this ID cannot be loaded in older engine versions
		*		-5 indicates the replacement of writing out the "UE3 version" so older versions of engine can gracefully fail to open newer packages
		*		-6 indicates optimizations to how custom versions are being serialized
		*		-7 indicates the texture allocation info has been removed from the summary
		*		-8 indicates that the UE5 version has been added to the summary
		*		-9 indicates a contractual change in when early exits are required based on FileVersionTooNew. At or
		*		   after this LegacyFileVersion, we support changing the PackageFileSummary serialization format for
		*		   all bytes serialized after FileVersionLicensee, and that format change can be conditional on any
		*		   of the versions parsed before that point. All packageloaders that understand the -9
		*		   legacyfileformat are required to early exit without further serialization at that point if
		*		   FileVersionTooNew is true.
		*/
		const int32 CurrentLegacyFileVersion = -9;
		int32 LegacyFileVersion = CurrentLegacyFileVersion;
		Record << SA_VALUE(TEXT("LegacyFileVersion"), LegacyFileVersion);

		if (BaseArchive.IsLoading())
		{
			if (LegacyFileVersion < 0) // means we have modern version numbers
			{
				if (LegacyFileVersion < CurrentLegacyFileVersion)
				{
					// we can't safely load more than this because the legacy version code differs in ways we can not predict.
					// Make sure that the linker will fail to load with it.
					Sum.FileVersionUE.Reset();
					Sum.FileVersionLicenseeUE = 0;
					return;
				}

				if (LegacyFileVersion != -4)
				{
					int32 LegacyUE3Version = 0;
					Record << SA_VALUE(TEXT("LegacyUE3Version"), LegacyUE3Version);
				}

				Record << SA_VALUE(TEXT("FileVersionUE4"), Sum.FileVersionUE.FileVersionUE4);
				
				if (LegacyFileVersion <= -8)
				{
					Record << SA_VALUE(TEXT("FileVersionUE5"), Sum.FileVersionUE.FileVersionUE5);
				}
			
				Record << SA_VALUE(TEXT("FileVersionLicenseeUE4"), Sum.FileVersionLicenseeUE);

				// Record whether the summary was unversioned when it was loaded off disk
				Sum.bUnversioned = !Sum.FileVersionUE.FileVersionUE4 && !Sum.FileVersionUE.FileVersionUE5 && !Sum.FileVersionLicenseeUE;
				if (Sum.bUnversioned)
				{
#if WITH_EDITOR
					if (!GAllowUnversionedContentInEditor)
					{
						// The editor cannot safely load unversioned content so exit before we apply the current version
						// to the summary. This will cause calls to ::IsFileVersionTooOld to return false
						UE_LOGF(LogLinker, Warning, "Failed to read package file summary, the file \"%ls\" is unversioned and we cannot safely load unversioned files in the editor.", *BaseArchive.GetArchiveName());
						return;
					}
#endif

					// Use the latest supported versions
					Sum.FileVersionUE = GPackageFileUEVersion;
					Sum.FileVersionLicenseeUE = GPackageFileLicenseeUEVersion;
				}

				if (Sum.FileVersionUE < VER_UE4_OLDEST_LOADABLE_PACKAGE 
					|| (IsEnforcePackageCompatibleVersionCheck() && Sum.IsFileVersionTooNew()))
				{
					// We early exit as soon as possible if the fileversion is too old or too new so that changes can
					// be made to the expected format after this point, and still have older versions of the editor
					// gracefully abort the load of the newer package rather than crashing. They might crash on format
					// differences if they try to serialize arrays from the wrong offset.
					return;
				}

				if (Sum.GetFileVersionUE() >= EUnrealEngineObjectUE5Version::PACKAGE_SAVED_HASH)
				{
					Record << SA_VALUE(TEXT("SavedHash"), Sum.SavedHash);
					Record << SA_VALUE(TEXT("TotalHeaderSize"), Sum.TotalHeaderSize);
				}

				if (LegacyFileVersion <= -2)
				{
					Sum.CustomVersionContainer.Serialize(Record.EnterField(TEXT("CustomVersions")), GetCustomVersionFormatForArchive(LegacyFileVersion));
				}

				if (Sum.bUnversioned)
				{
					// Overwrite the CustomVersionContainer that we deserialized; it was an empty container written
					// during unversioned savepackage. Overwrite with the latest version of all custom versions.
					Sum.CustomVersionContainer = FCurrentCustomVersions::GetAll();
				}
			}
			else
			{
				// This is probably an old UE3 file, make sure that the linker will fail to load with it.
				Sum.FileVersionUE.Reset();
				Sum.FileVersionLicenseeUE = 0;
				return;
			}
		}
		else
		{
			if (Sum.bUnversioned)
			{
				int32 Zero = 0;
				Record << SA_VALUE(TEXT("LegacyUE3version"), Zero); // LegacyUE3version
				Record << SA_VALUE(TEXT("FileVersionUE4"), Zero); // VersionUE4
				Record << SA_VALUE(TEXT("FileVersionUE5"), Zero); // VersionUE5
				Record << SA_VALUE(TEXT("FileVersionLicenseeUE4"), Zero); // VersionLicenseeUE4
			}
			else
			{
				// Must write out the last UE3 engine version, so that older versions identify it as new
				int32 LegacyUE3Version = 864;
				Record << SA_VALUE(TEXT("LegacyUE3Version"), LegacyUE3Version);
				Record << SA_VALUE(TEXT("FileVersionUE4"), Sum.FileVersionUE.FileVersionUE4);
				Record << SA_VALUE(TEXT("FileVersionUE5"), Sum.FileVersionUE.FileVersionUE5);
				Record << SA_VALUE(TEXT("FileVersionLicenseeUE4"), Sum.FileVersionLicenseeUE);
			}

#if WITH_EDITORONLY_DATA
			if (BaseArchive.IsSaving() && !BaseArchive.IsTextFormat())
			{
				// GetSavedHashRelativeOffset must be kept in sync with the bytes written during serialize;
				// we use it during saving to seek back to the offset of the SavedHash and rewrite it after
				// calculating it.
				int64 CurrentRelativeOffset = BaseArchive.Tell() - StartOffset;
				int64 ExpectedRelativeOffset = Sum.GetSavedHashRelativeOffset();
				checkf(CurrentRelativeOffset == ExpectedRelativeOffset,
					TEXT("Expected (CurrentRelativeOffset) %" INT64_FMT " == %" INT64_FMT " (ExpectedRelativeOffset)"),
					CurrentRelativeOffset, ExpectedRelativeOffset);
			}
#endif
			// Saved FileVersion might not be current FileVersion, when called from AssetHeaderPatcher
			if (Sum.GetFileVersionUE() >= EUnrealEngineObjectUE5Version::PACKAGE_SAVED_HASH)
			{
				Record << SA_VALUE(TEXT("SavedHash"), Sum.SavedHash);
				Record << SA_VALUE(TEXT("TotalHeaderSize"), Sum.TotalHeaderSize);
			}

			// Serialise custom version map.
			if (Sum.bUnversioned)
			{
				FCustomVersionContainer NoCustomVersions;
				NoCustomVersions.Serialize(Record.EnterField(TEXT("CustomVersions")));
			}
			else
			{
				Sum.CustomVersionContainer.Serialize(Record.EnterField(TEXT("CustomVersions")));
			}
		}

		if (Sum.GetFileVersionUE() < EUnrealEngineObjectUE5Version::PACKAGE_SAVED_HASH)
		{
			Record << SA_VALUE(TEXT("TotalHeaderSize"), Sum.TotalHeaderSize);
		}
		Record << SA_VALUE(TEXT("PackageName"), Sum.PackageName); //@note used to be FolderName, now unused and deprecated

		if (BaseArchive.IsCooking())
		{
			Sum.PackageFlags |= PKG_Cooked;
		}

		Record << SA_VALUE(TEXT("PackageFlags"), Sum.PackageFlags);

		if (BaseArchive.IsLoading())
		{
			// Transient flags should be set to false when saving or loading
			Sum.PackageFlags &= ~PKG_TransientFlags;
		}

		if (Sum.PackageFlags & PKG_FilterEditorOnly)
		{
			BaseArchive.SetFilterEditorOnly(true);
		}

		Record << SA_VALUE(TEXT("NameCount"), Sum.NameCount) << SA_VALUE(TEXT("NameOffset"), Sum.NameOffset);

		// Sometimes its useful to be able to save out files with older versions.
		// So this code needs to be Symmetrical for saving and loading unless an error is encountered.
		if (Sum.FileVersionUE >= EUnrealEngineObjectUE5Version::ADD_SOFTOBJECTPATH_LIST)
		{
			Record << SA_VALUE(TEXT("SoftObjectPathsCount"), Sum.SoftObjectPathsCount) << SA_VALUE(TEXT("SoftObjectPathsOffset"), Sum.SoftObjectPathsOffset);
		}

		if (!BaseArchive.IsFilterEditorOnly())
		{
			if (Sum.FileVersionUE >= VER_UE4_ADDED_PACKAGE_SUMMARY_LOCALIZATION_ID)
			{
				Record << SA_VALUE(TEXT("LocalizationId"), Sum.LocalizationId);
			}
		}

		if (Sum.FileVersionUE >= VER_UE4_SERIALIZE_TEXT_IN_PACKAGES)
		{
			Record << SA_VALUE(TEXT("GatherableTextDataCount"), Sum.GatherableTextDataCount) << SA_VALUE(TEXT("GatherableTextDataOffset"), Sum.GatherableTextDataOffset);
		}
		Record << SA_VALUE(TEXT("ExportCount"), Sum.ExportCount) << SA_VALUE(TEXT("ExportOffset"), Sum.ExportOffset);
		Record << SA_VALUE(TEXT("ImportCount"), Sum.ImportCount) << SA_VALUE(TEXT("ImportOffset"), Sum.ImportOffset);

		if (Sum.FileVersionUE >= EUnrealEngineObjectUE5Version::VERSE_CELLS)
		{
			Record << SA_VALUE(TEXT("CellExportCount"), Sum.CellExportCount) << SA_VALUE(TEXT("CellExportOffset"), Sum.CellExportOffset);
			Record << SA_VALUE(TEXT("CellImportCount"), Sum.CellImportCount) << SA_VALUE(TEXT("CellImportOffset"), Sum.CellImportOffset);
		}

		if (Sum.FileVersionUE >= EUnrealEngineObjectUE5Version::METADATA_SERIALIZATION_OFFSET)
		{
			Record << SA_VALUE(TEXT("MetaDataOffset"), Sum.MetaDataOffset);
		}		

		Record << SA_VALUE(TEXT("DependsOffset"), Sum.DependsOffset);

		if (Sum.FileVersionUE >= VER_UE4_ADD_STRING_ASSET_REFERENCES_MAP)
		{
			Record << SA_VALUE(TEXT("SoftPackageReferencesCount"), Sum.SoftPackageReferencesCount) << SA_VALUE(TEXT("SoftPackageReferencesOffset"), Sum.SoftPackageReferencesOffset);
		}

		if (Sum.FileVersionUE >= VER_UE4_ADDED_SEARCHABLE_NAMES)
		{
			Record << SA_VALUE(TEXT("SearchableNamesOffset"), Sum.SearchableNamesOffset);
		}

		Record << SA_VALUE(TEXT("ThumbnailTableOffset"), Sum.ThumbnailTableOffset);

		if (Sum.FileVersionUE >= EUnrealEngineObjectUE5Version::IMPORT_TYPE_HIERARCHIES)
		{
			Record << SA_VALUE(TEXT("ImportTypeHierarchiesCount"), Sum.ImportTypeHierarchiesCount) << SA_VALUE(TEXT("ImportTypeHierarchiesOffset"), Sum.ImportTypeHierarchiesOffset);
		}
		else
		{
			Sum.ImportTypeHierarchiesCount = 0;
			Sum.ImportTypeHierarchiesOffset = 0;
		}

		if (Sum.GetFileVersionUE() < EUnrealEngineObjectUE5Version::PACKAGE_SAVED_HASH)
		{
			FGuid LegacyGuid;
			if (!BaseArchive.IsLoading())
			{
				FMemory::Memcpy(&LegacyGuid, &Sum.SavedHash.GetBytes(),
					FMath::Min(sizeof(decltype(Sum.SavedHash.GetBytes())), sizeof(LegacyGuid)));
			}
			Record << SA_VALUE(TEXT("Guid"), LegacyGuid);
			if (BaseArchive.IsLoading())
			{
				Sum.SavedHash = FIoHash::Zero;
				FMemory::Memcpy(&Sum.SavedHash.GetBytes(), &LegacyGuid,
					FMath::Min(sizeof(decltype(Sum.SavedHash.GetBytes())), sizeof(LegacyGuid)));
			}
		}

#if WITH_EDITORONLY_DATA
		if (!BaseArchive.IsFilterEditorOnly())
		{
			if (Sum.FileVersionUE >= VER_UE4_ADDED_PACKAGE_OWNER)
			{
				Record << SA_VALUE(TEXT("PersistentGuid"), Sum.PersistentGuid);
			}
			else
			{
				// By assigning the current package guid, we maintain a stable persistent guid, so we can reference this package even if it wasn't resaved.
				FIoHash SavedHash = Sum.GetSavedHash();
				Sum.PersistentGuid = FGuid();
				FMemory::Memcpy(&Sum.PersistentGuid, &SavedHash.GetBytes(),
					FMath::Min(sizeof(Sum.PersistentGuid), sizeof(decltype(SavedHash.GetBytes()))));
			}

			// The owner persistent guid was added in VER_UE4_ADDED_PACKAGE_OWNER but removed in the next version VER_UE4_NON_OUTER_PACKAGE_IMPORT
			if (Sum.FileVersionUE >= VER_UE4_ADDED_PACKAGE_OWNER && Sum.FileVersionUE < VER_UE4_NON_OUTER_PACKAGE_IMPORT)
			{
				FGuid OwnerPersistentGuid;
				Record << SA_VALUE(TEXT("OwnerPersistentGuid"), OwnerPersistentGuid);
			}
		}
#endif

		int32 GenerationCount = Sum.Generations.Num();
		Record << SA_VALUE(TEXT("GenerationCount"), GenerationCount);
		if (BaseArchive.IsLoading() && GenerationCount > 0)
		{
			Sum.Generations.Reset(GenerationCount);
			Sum.Generations.AddZeroed(GenerationCount);
		}

		FStructuredArchive::FStream GenerationsStream = Record.EnterStream(TEXT("Generations"));
		for (int32 i = 0; i<GenerationCount; i++)
		{
			Sum.Generations[i].Serialize(GenerationsStream.EnterElement(), Sum);
			if (BaseArchive.IsLoading() && BaseArchive.IsError())
			{
				return;
			}
		}

		if (Sum.GetFileVersionUE() >= VER_UE4_ENGINE_VERSION_OBJECT)
		{
			if (BaseArchive.IsCooking() || (BaseArchive.IsSaving() && !FEngineVersion::Current().HasChangelist()))
			{
				FEngineVersion EmptyEngineVersion;
				Record << SA_VALUE(TEXT("SavedByEngineVersion"), EmptyEngineVersion);
			}
			else
			{
				Record << SA_VALUE(TEXT("SavedByEngineVersion"), Sum.SavedByEngineVersion);
				FixCorruptEngineVersion(Sum.GetFileVersionUE(), Sum.SavedByEngineVersion);
			}
		}
		else
		{
			int32 EngineChangelist = 0;
			Record << SA_VALUE(TEXT("EngineChangelist"), EngineChangelist);

			if (BaseArchive.IsLoading() && EngineChangelist != 0)
			{
				Sum.SavedByEngineVersion.Set(4, 0, 0, EngineChangelist, TEXT(""));
			}
		}

		if (Sum.GetFileVersionUE() >= VER_UE4_PACKAGE_SUMMARY_HAS_COMPATIBLE_ENGINE_VERSION)
		{
			if (BaseArchive.IsCooking() || (BaseArchive.IsSaving() && !FEngineVersion::Current().HasChangelist()))
			{
				FEngineVersion EmptyEngineVersion;
				Record << SA_VALUE(TEXT("CompatibleWithEngineVersion"), EmptyEngineVersion);
			}
			else
			{
				Record << SA_VALUE(TEXT("CompatibleWithEngineVersion"), Sum.CompatibleWithEngineVersion);
				FixCorruptEngineVersion(Sum.GetFileVersionUE(), Sum.CompatibleWithEngineVersion);
			}
		}
		else
		{
			if (BaseArchive.IsLoading())
			{
				Sum.CompatibleWithEngineVersion = Sum.SavedByEngineVersion;
			}
		}

		Record << SA_VALUE(TEXT("CompressionFlags"), Sum.CompressionFlags);
		if (!FCompression::VerifyCompressionFlagsValid(Sum.CompressionFlags))
		{
			UE_LOGF(LogLinker, Warning, "Failed to read package file summary, the file \"%ls\" has invalid compression flags (%d).", *BaseArchive.GetArchiveName(), Sum.CompressionFlags);
			Sum.InvalidateFileVersion();
			return;
		}

		TArray<FCompressedChunk> CompressedChunks;
		Record << SA_VALUE(TEXT("CompressedChunks"), CompressedChunks);

		if (CompressedChunks.Num())
		{
			// this file has package level compression, we won't load it.
			UE_LOGF(LogLinker, Warning, "Failed to read package file summary, the file \"%ls\" is has package level compression (and is probably cooked). These old files cannot be loaded in the editor.", *BaseArchive.GetArchiveName());
			Sum.InvalidateFileVersion();
			return; // we can't safely load more than this because we just changed the version to something it is not.
		}

		Record << SA_VALUE(TEXT("PackageSource"), Sum.PackageSource);

		// No longer used: List of additional packages that are needed to be cooked for this package (ie streaming levels)
		// Keeping the serialization code for backwards compatibility without bumping the package version
		TArray<FString>	AdditionalPackagesToCook;
		Record << SA_VALUE(TEXT("AdditionalPackagesToCook"), AdditionalPackagesToCook);

		if (LegacyFileVersion > -7)
		{
			int32 NumTextureAllocations = 0;
			Record << SA_VALUE(TEXT("NumTextureAllocations"), NumTextureAllocations);
			// We haven't used texture allocation info for ages and it's no longer supported anyway
			check(NumTextureAllocations == 0);
		}

		Record << SA_VALUE(TEXT("AssetRegistryDataOffset"), Sum.AssetRegistryDataOffset);
		Record << SA_VALUE(TEXT("BulkDataStartOffset"), Sum.BulkDataStartOffset);

		if (Sum.GetFileVersionUE() >= VER_UE4_WORLD_LEVEL_INFO)
		{
			Record << SA_VALUE(TEXT("WorldTileInfoDataOffset"), Sum.WorldTileInfoDataOffset);
		}

		if (Sum.GetFileVersionUE() >= VER_UE4_CHANGED_CHUNKID_TO_BE_AN_ARRAY_OF_CHUNKIDS)
		{
			Record << SA_VALUE(TEXT("ChunkIDs"), Sum.ChunkIDs);
		}
		else if (Sum.GetFileVersionUE() >= VER_UE4_ADDED_CHUNKID_TO_ASSETDATA_AND_UPACKAGE)
		{
			// handle conversion of single ChunkID to an array of ChunkIDs
			if (BaseArchive.IsLoading())
			{
				int ChunkID = -1;
				Record << SA_VALUE(TEXT("ChunkID"), ChunkID);

				// don't load <0 entries since an empty array represents the same thing now
				if (ChunkID >= 0)
				{
					Sum.ChunkIDs.Add(ChunkID);
				}
			}
		}
		if (Sum.FileVersionUE >= VER_UE4_PRELOAD_DEPENDENCIES_IN_COOKED_EXPORTS)
		{
			Record << SA_VALUE(TEXT("PreloadDependencyCount"), Sum.PreloadDependencyCount) << SA_VALUE(TEXT("PreloadDependencyOffset"), Sum.PreloadDependencyOffset);
		}
		else
		{
			Sum.PreloadDependencyCount = -1;
			Sum.PreloadDependencyOffset = 0;
		}

		if (Sum.FileVersionUE >= EUnrealEngineObjectUE5Version::NAMES_REFERENCED_FROM_EXPORT_DATA)
		{
			Record << SA_VALUE(TEXT("NamesReferencedFromExportDataCount"), Sum.NamesReferencedFromExportDataCount);
		}
		else
		{
			Sum.NamesReferencedFromExportDataCount = Sum.NameCount;
		}

		if (Sum.FileVersionUE >= EUnrealEngineObjectUE5Version::PAYLOAD_TOC)
		{
			Record << SA_VALUE(TEXT("PayloadTocOffset"), Sum.PayloadTocOffset);
		}
		else
		{
			Sum.PayloadTocOffset = INDEX_NONE;
		}
		
		if (Sum.GetFileVersionUE() >= EUnrealEngineObjectUE5Version::DATA_RESOURCES)
		{
			Record << SA_VALUE(TEXT("DataResourceOffset"), Sum.DataResourceOffset);
		}
		else
		{
			Sum.DataResourceOffset = -1;
		}
	}
}

#if WITH_EDITORONLY_DATA
FIoHash FPackageFileSummary::GetSavedHash() const
{
	return SavedHash;
}

void FPackageFileSummary::SetSavedHash(const FIoHash& InSavedHash)
{
	SavedHash = InSavedHash;
}

int64 FPackageFileSummary::GetSavedHashRelativeOffset() const
{
	// This offset is constructed by copying the code from void operator<<(FStructuredArchive::FSlot Slot, FPackageFileSummary& Sum)
	// with LegacyFileVersion == CurrentLegacyFileVersion
	return
		sizeof(Tag)
		+ sizeof(int32) // LegacyFileVersion
		+ sizeof(int32) // LegacyUE3Version
		+ sizeof(FileVersionUE.FileVersionUE4)
		+ sizeof(FileVersionUE.FileVersionUE5)
		+ sizeof(FileVersionLicenseeUE);
}
#endif

FArchive& operator<<( FArchive& Ar, FPackageFileSummary& Sum )
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << Sum;
	return Ar;
}

void FPackageFileSummary::SetCustomVersionContainer(const FCustomVersionContainer& InContainer)
{
	CustomVersionContainer = InContainer;
	CustomVersionContainer.SortByKey();
}

void FPackageFileSummary::SetFileVersions(const int32 EpicUE4, const int32 EpicUE5, const int32 LicenseeUE, const bool bInSaveUnversioned)
{
	// We could also make sure that EpicUE4 is >= VER_UE4_OLDEST_LOADABLE_PACKAGE but there might be a use case for setting
	// an out of date version.
	check(EpicUE4 <= EUnrealEngineObjectUE4Version::VER_UE4_AUTOMATIC_VERSION);
	check(EpicUE5 <= (int32)EUnrealEngineObjectUE5Version::AUTOMATIC_VERSION);

	FileVersionUE.FileVersionUE4 = EpicUE4;
	FileVersionUE.FileVersionUE5 = EpicUE5;
	FileVersionLicenseeUE = LicenseeUE;

	bUnversioned = bInSaveUnversioned;
}

bool FPackageFileSummary::IsFileVersionValid() const
{
#if WITH_EDITOR
	if (!GAllowUnversionedContentInEditor)
	{
		return !bUnversioned;
	}
#endif

	return true;
}

void FPackageFileSummary::SetPackageFlags(uint32 InPackageFlags)
{
	// Package summary flags should be set to false for all transient and memory-only flags
	PackageFlags = InPackageFlags & ~(PKG_TransientFlags | PKG_InMemoryOnly);
}
