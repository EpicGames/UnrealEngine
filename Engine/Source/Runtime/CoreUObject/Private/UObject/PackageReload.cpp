// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PackageReload.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/UObjectHash.h"
#include "UObject/GCObject.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/ScopedSlowTask.h"
#include "UObject/Package.h"
#include "UObject/Linker.h"
#include "Templates/Casts.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "UObject/ReferencerFinder.h"
#endif

#if WITH_EDITOR
TAutoConsoleVariable<bool> CVarPackageReloadEnableFastPath(
	TEXT("PackageReload.EnableFastPath"),
	false,
	TEXT("When 'true', an optimized codepath is used to speed up reloading packages (experimental)."),
	ECVF_Default);
#endif

class FPackageReferencersHelper
{
#if WITH_EDITOR
	/**
	 * Set of all objects that could potentially refer to any of the packages that are about to be reloaded.
	 */
	TSet<TWeakObjectPtr<UObject>> PotentialReferencerObjects;

	/**
	 * Callback handles.
	 */
	FDelegateHandle OnObjectsReplacedHandle;
	FDelegateHandle OnObjectConstructedHandle;

	/**
	 * Callback to be called when one or more UObjects are replaced with others.
	 */
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& OldToNewInstanceMap)
	{
		for (const TPair<UObject*, UObject*>& Pair : OldToNewInstanceMap)
		{
			if (Pair.Key && Pair.Value)
			{
				TWeakObjectPtr<UObject> OldObject = MakeWeakObjectPtr<UObject>(Pair.Key);
				TWeakObjectPtr<UObject> NewObject = MakeWeakObjectPtr<UObject>(Pair.Value);
				if (PotentialReferencerObjects.Contains(OldObject))
				{
					PotentialReferencerObjects.Add(NewObject);
					PotentialReferencerObjects.Remove(OldObject);
				}
			}
		}
	}

	/**
	 * Callback to be called when a new UObject is constructed.
	 */
	void OnObjectConstructed(UObject* InObject)
	{
		TWeakObjectPtr<UObject> Object = MakeWeakObjectPtr<UObject>(InObject);
		PotentialReferencerObjects.Add(Object);
	}

	/**
	 * Recursively retrieve all objects that reference any object in the InReferencedObjects.
	 */
	static void RecursiveRetrieveReferencers(const TArray<UObject*>& InReferencedObjects, TSet<TWeakObjectPtr<UObject>>& OutReferencingObjects)
	{
		TArray<UObject*> FoundReferencerObjects = FReferencerFinder::GetAllReferencers(InReferencedObjects, nullptr, EReferencerFinderFlags::None);

		// Convert them into weak pointers to deal with objects that get GC'd during the reload operation.
		OutReferencingObjects.Reserve(OutReferencingObjects.Num() + FoundReferencerObjects.Num());
		for (UObject* ReferencerObject : FoundReferencerObjects)
		{
			OutReferencingObjects.Emplace(MakeWeakObjectPtr(ReferencerObject));
		}
	}
#endif

public:
	FPackageReferencersHelper(const TArrayView<FReloadPackageData>& InPackagesToReload)
	{
#if WITH_EDITOR
		if (CVarPackageReloadEnableFastPath.GetValueOnAnyThread())
		{
			OnObjectsReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FPackageReferencersHelper::OnObjectsReplaced);
			OnObjectConstructedHandle = FCoreUObjectDelegates::OnObjectConstructed.AddRaw(this, &FPackageReferencersHelper::OnObjectConstructed);

			// Build list of UObjects that represent the packages about to be reloaded.
			TArray<UObject*> ObjectsToReload;
			ObjectsToReload.Reserve(InPackagesToReload.Num());

			for (const FReloadPackageData& PackageToReloadData : InPackagesToReload)
			{
				ObjectsToReload.Add(PackageToReloadData.PackageToReload);
				TArray<UObject*> Inners;
				GetObjectsWithOuter(PackageToReloadData.PackageToReload, Inners);
				ObjectsToReload.Append(Inners);
			}

			// Collect all other objects that reference any of them.
			RecursiveRetrieveReferencers(ObjectsToReload, PotentialReferencerObjects);
		}
#endif
	}

	~FPackageReferencersHelper()
	{
#if WITH_EDITOR
		if (OnObjectConstructedHandle.IsValid())
		{
			FCoreUObjectDelegates::OnObjectConstructed.Remove(OnObjectConstructedHandle);
			OnObjectConstructedHandle.Reset();
		}
		if (OnObjectsReplacedHandle.IsValid())
		{
			FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectsReplacedHandle);
			OnObjectsReplacedHandle.Reset();
		}
#endif
	}

public:
	void ForEachObject(TFunctionRef<void(UObject*)> Operation)
	{
#if WITH_EDITOR
		if (CVarPackageReloadEnableFastPath.GetValueOnAnyThread())
		{
			for (TWeakObjectPtr<UObject> Object : PotentialReferencerObjects)
			{
				UObject* PotentialReferencer = Object.Get();
				if (PotentialReferencer != nullptr)
				{
					Operation(PotentialReferencer);
				}
			}
		}
		else
#endif
		{
			for (FThreadSafeObjectIterator ObjIter(UObject::StaticClass(), false, RF_NoFlags, EInternalObjectFlags::Garbage); ObjIter; ++ObjIter)
			{
				UObject* PotentialReferencer = *ObjIter;
				Operation(PotentialReferencer);
			}
		}
	}
};

namespace PackageReloadInternal
{

/**
 * Reference to an existing package that prevents it being GC'd while we're still using it (via FExistingPackageReferences).
 * Once we're done with it, we clear out the strong reference and use the weak reference to verify that it was purged correctly via GC.
 */
struct FExistingPackageReference
{
	FExistingPackageReference(UPackage* InPackage)
		: RawRef(InPackage)
		, StrongRef(InPackage)
		, WeakRef(InPackage)
	{
	}

	UPackage* RawRef;
	TObjectPtr<UPackage> StrongRef;
	TWeakObjectPtr<UPackage> WeakRef;
};

/**
 * Array wrapper that prevents the packages inside the FExistingPackageReference instances being GC'd while we're still using them.
 */
struct FExistingPackageReferences : public FGCObject
{
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		for (FExistingPackageReference& Ref : Refs)
		{
			// Note: We deliberate don't ARO RawRef here
			Collector.AddReferencedObject(Ref.StrongRef);
		}
	}

	virtual FString GetReferencerName() const
	{
		return TEXT("FExistingPackageReferences");
	}

	TArray<FExistingPackageReference> Refs;
};

/**
 * Reference to an replacement package that prevents it being GC'd while we're still using it (via FNewPackageReferences).
 * This also includes the event data used when broadcasting package reload events for this package.
 */
struct FNewPackageReference
{
	FNewPackageReference(UPackage* InPackage)
		: Package(InPackage)
		, EventData()
	{
	}

	TObjectPtr<UPackage> Package;
	TSharedPtr<FPackageReloadedEvent> EventData;
};

/**
 * Array wrapper that prevents the packages inside the FNewPackageReferences instances being GC'd while we're still using them.
 */
struct FNewPackageReferences : public FGCObject
{
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		for (FNewPackageReference& Ref : Refs)
		{
			Collector.AddReferencedObject(Ref.Package);
			if(Ref.EventData)
			{
				Ref.EventData->AddReferencedObjects(Collector);
			}
		}
	}

	virtual FString GetReferencerName() const
	{
		return TEXT("FNewPackageReferences");
	}

	TArray<FNewPackageReference> Refs;
};

/**
 * Used to map objects from the old package, to objects in the new package, including the index of the package being reloaded.
 */
struct FObjectAndPackageIndex
{
	FObjectAndPackageIndex(UObject* InObject, const int32 InPackageIndex)
		: Object(InObject)
		, PackageIndex(InPackageIndex)
	{
	}

	UObject* Object;
	int32 PackageIndex;
};

/**
 * Custom archive type used to re-point any in-memory references to objects in the old package to objects in the new package, or null if there is no replacement object.
 */
class FReplaceObjectReferencesArchive : public FArchiveUObject, public FReferenceCollector
{
public:
	FReplaceObjectReferencesArchive(UObject* InPotentialReferencer, const TMap<UObject*, FObjectAndPackageIndex>& InOldObjectToNewData, const TArray<FExistingPackageReference>& InExistingPackages, const TArray<FNewPackageReference>& InNewPackages)
		: PotentialReferencer(InPotentialReferencer)
		, OldObjectToNewData(InOldObjectToNewData)
		, ExistingPackages(InExistingPackages)
		, NewPackages(InNewPackages)
	{
		ArIsObjectReferenceCollector = true;
		ArIsModifyingWeakAndStrongReferences = true;
		ArIgnoreOuterRef = true;
		ArNoDelta = true;
	}

	virtual ~FReplaceObjectReferencesArchive()
	{
	}

	virtual FString GetArchiveName() const override
	{
		return TEXT("FReplaceObjectReferencesArchive");
	}

	virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
	{
		(*this) << Object;
	}

	virtual void HandleObjectReferences(UObject** InObjects, const int32 ObjectNum, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		for (int32 ObjectIndex = 0; ObjectIndex < ObjectNum; ++ObjectIndex)
		{
			UObject*& Object = InObjects[ObjectIndex];
			(*this) << Object;
		}
	}

	virtual bool IsIgnoringArchetypeRef() const override
	{
		return false;
	}

	virtual bool IsIgnoringTransient() const override
	{
		return false;
	}

	FArchive& operator<<(UObject*& ObjRef)
	{
		if (ObjRef && ObjRef != PotentialReferencer)
		{
			UObject* NewObject = nullptr;
			TSharedPtr<FPackageReloadedEvent> PackageEventData;
			if (GetNewObjectAndEventData(ObjRef, NewObject, PackageEventData))
			{
				ObjRef = NewObject;
				PackageEventData->AddObjectReferencer(PotentialReferencer);
			}
		}

		return *this;
	}

	bool GetNewObjectAndEventData(UObject* InOldObject, UObject*& OutNewObject, TSharedPtr<FPackageReloadedEvent>& OutEventData) const
	{
		const FObjectAndPackageIndex* ObjectAndPackageIndexPtr = OldObjectToNewData.Find(InOldObject);

		// Only fix-up references to objects outside of the potential referencer package, as internal object references will be orphaned automatically
		if (ObjectAndPackageIndexPtr && PotentialReferencer->GetOutermost() != ExistingPackages[ObjectAndPackageIndexPtr->PackageIndex].RawRef)
		{
			OutNewObject = ObjectAndPackageIndexPtr->Object;
			OutEventData = NewPackages[ObjectAndPackageIndexPtr->PackageIndex].EventData;
			return true;
		}

		return false;
	}

	UObject* PotentialReferencer;
	const TMap<UObject*, FObjectAndPackageIndex>& OldObjectToNewData;
	const TArray<FExistingPackageReference>& ExistingPackages;
	const TArray<FNewPackageReference>& NewPackages;
};

/**
 * Given a package, mark it and all its sub-objects with the RF_NewerVersionExists flag so that other systems can detect that they're being replaced.
 */
void MarkPackageReplaced(UPackage* InPackage)
{
	InPackage->SetFlags(RF_NewerVersionExists);
	ForEachObjectWithPackage(InPackage, [](UObject* InSubObject)
	{
		InSubObject->SetFlags(RF_NewerVersionExists);
		return true; // continue
	});
}

/**
 * Given a package, remove the RF_NewerVersionExists flag from it and all its sub-objects.
 */
void ClearPackageReplaced(UPackage* InPackage)
{
	InPackage->ClearFlags(RF_NewerVersionExists);
	ForEachObjectWithPackage(InPackage, [](UObject* InSubObject)
	{
		InSubObject->ClearFlags(RF_NewerVersionExists);
		return true; // continue
	});
}

/**
 * Given an object, put it into a state where a GC may purge it (assuming there are no external references).
 */
void MakeObjectPurgeable(UObject* InObject)
{
	if (InObject->IsRooted())
	{
		InObject->RemoveFromRoot();
	}
	InObject->ClearFlags(RF_Public | RF_Standalone);
}

/**
 * Given a package, put it into a state where a GC may purge it (assuming there are no external references).
 */
void MakePackagePurgeable(UPackage* InPackage)
{
	MakeObjectPurgeable(InPackage);
	ForEachObjectWithPackage(InPackage, [](UObject* InObject)
	{
		MakeObjectPurgeable(InObject);
		return true; // continue
	});
}

/**
 * Given an object, dump anything that is still externally referencing it to the log.
 */
void DumpExternalReferences(UObject* InObject, UPackage* InPackage)
{
	TArray<FString> ExternalRefDumps;

	{
		FReferenceChainSearch ObjectRefChains(InObject, EReferenceChainSearchMode::Default);
		for (const FReferenceChainSearch::FReferenceChain* ObjectRefChain : ObjectRefChains.GetReferenceChains())
		{
			for (int32 NodeIndex = 0; NodeIndex < ObjectRefChain->Num(); ++NodeIndex)
			{
				const FReferenceChainSearch::FGraphNode* ObjectRefChainLink = ObjectRefChain->GetNode(NodeIndex);
				UObject* LinkObject = ObjectRefChainLink->ObjectInfo->TryResolveObject();
				checkf(LinkObject, TEXT("Unable to resolve object: %s when dumping external references"), *ObjectRefChainLink->ObjectInfo->GetPathName());
				const bool bIsExternalRef = LinkObject->GetOutermost() != InPackage;
				if (bIsExternalRef)
				{
					ExternalRefDumps.Emplace(ObjectRefChainLink->ObjectInfo->GetFullName());
				}
			}
		}
	}

	if (ExternalRefDumps.Num() > 0)
	{
		UE_LOGF(LogUObjectGlobals, Display, "ReloadPackage external references for '%ls'.", *InObject->GetPathName());
		for (const FString& ExternalRefDump : ExternalRefDumps)
		{
			UE_LOGF(LogUObjectGlobals, Display, "    %ls", *ExternalRefDump);
		}
	}
}

/**
 * Given a package, validate and prepare it for reload.
 * @return The package to be reloaded, or null if the given package isn't valid to be reloaded.
 */
UPackage* ValidateAndPreparePackageForReload(UPackage* InExistingPackage)
{
	// We can't reload memory-only packages
	if (InExistingPackage->HasAnyPackageFlags(PKG_InMemoryOnly))
	{
		UE_LOGF(LogUObjectGlobals, Warning, "ReloadPackage cannot reload '%ls' as it is marked PKG_InMemoryOnly.", *InExistingPackage->GetName());
		return nullptr;
	}

	// Make sure the package has finished loading before we try and unload it again
	if (!InExistingPackage->IsFullyLoaded())
	{
		FlushAsyncLoading();
		InExistingPackage->FullyLoad();
	}
	ResetLoaders(InExistingPackage);

	return InExistingPackage;
}

/**
 * Given a package, reload it from disk.
 * @return The package that was reloaded, or null if the given package couldn't be reloaded.
 */
UPackage* LoadReplacementPackage(UPackage* InExistingPackage, const uint32 InLoadFlags)
{
	if (!InExistingPackage)
	{
		return nullptr;
	}

	const FString ExistingPackageName = InExistingPackage->GetName();

	// Rename the old package, and then load the new one in its place
	const ERenameFlags PkgRenameFlags = REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional | REN_SkipGeneratedClasses | REN_AllowPackageLinkerMismatch;
	InExistingPackage->Rename(*MakeUniqueObjectName(Cast<UPackage>(InExistingPackage->GetOuter()), UPackage::StaticClass(), *FString::Printf(TEXT("%s_DEADPACKAGE"), *InExistingPackage->GetName())).ToString(), nullptr, PkgRenameFlags);
	MarkPackageReplaced(InExistingPackage);
	UPackage* NewPackage = LoadPackage(Cast<UPackage>(InExistingPackage->GetOuter()), *ExistingPackageName, InLoadFlags);
	if (!NewPackage)
	{
		UE_LOGF(LogUObjectGlobals, Warning, "ReloadPackage cannot reload '%ls' as the new package failed to load. The old package will be restored.", *ExistingPackageName);

		// Make sure that the failed load attempt hasn't left any objects behind that would prevent the rename
		if (UPackage* FailedPackage = FindObject<UPackage>(Cast<UPackage>(InExistingPackage->GetOuter()), *ExistingPackageName))
		{
			FailedPackage->Rename(*MakeUniqueObjectName(Cast<UPackage>(FailedPackage->GetOuter()), UPackage::StaticClass(), *FString::Printf(TEXT("%s_DEADPACKAGE"), *FailedPackage->GetName())).ToString(), nullptr, PkgRenameFlags);
			MakePackagePurgeable(FailedPackage);
		}

		// Failed to load the new package, give the old package its original name and bail!
		InExistingPackage->Rename(*ExistingPackageName, nullptr, PkgRenameFlags);
		ClearPackageReplaced(InExistingPackage);
		return nullptr;
	}

	// Make sure the package has finished loading before we try and find things from inside it
	if (!NewPackage->IsFullyLoaded())
	{
		FlushAsyncLoading();
		NewPackage->FullyLoad();
	}

	return NewPackage;
}

/**
 * Given an old and new package, generate the event payload data needed to fix-up references to objects from the old package to the corresponding objects in the new package.
 * @return The event payload data, or null if either given package is invalid.
 */
TSharedPtr<FPackageReloadedEvent> GeneratePackageReloadEvent(UPackage* InExistingPackage, UPackage* InNewPackage)
{
	TSharedPtr<FPackageReloadedEvent> PackageReloadedEvent;

	if (InExistingPackage && InNewPackage)
	{
		TMap<UObject*, UObject*> RedirectedObjectsMap;
		RedirectedObjectsMap.Emplace(InExistingPackage, InNewPackage);
		InExistingPackage->BuildSubobjectMapping(InNewPackage, RedirectedObjectsMap);

		for (const auto& ObjectMappingPair : RedirectedObjectsMap)
		{
			UObject* ExistingObject = ObjectMappingPair.Key;
			UObject* NewObject = ObjectMappingPair.Value;

			if (NewObject)
			{
				// Pass on the root-set state from the old object to the new one
				if (ExistingObject->IsRooted())
				{
					NewObject->AddToRoot();
				}

				// Pass on some important flags to the new object
				{
					const EObjectFlags FlagsToPassToNewObject = ExistingObject->GetMaskedFlags(RF_Public | RF_Standalone | RF_Transactional);
					NewObject->SetFlags(FlagsToPassToNewObject);
				}
			}
			else if (ExistingObject->HasAnyFlags(RF_Transient))
			{
				UE_LOGF(LogUObjectGlobals, Display, "ReloadPackage failed to find a replacement object for '%ls' (transient) in the new package '%ls'. Any existing references to this object will be nulled out.", *ExistingObject->GetPathName(InExistingPackage), *InNewPackage->GetName());
			}
			else
			{
				UE_LOGF(LogUObjectGlobals, Warning, "ReloadPackage failed to find a replacement object for '%ls' in the new package '%ls'. Any existing references to this object will be nulled out.", *ExistingObject->GetPathName(InExistingPackage), *InNewPackage->GetName());
			}
		}

		PackageReloadedEvent = MakeShared<FPackageReloadedEvent>(InExistingPackage, InNewPackage, MoveTemp(RedirectedObjectsMap));
	}

	return PackageReloadedEvent;
}

void SortPackagesForReload(const FName PackageName, TSet<FName>& ProcessedPackages, TArray<UPackage*>& SortedPackagesToReload, const TMap<FName, UPackage*>& AllPackagesToReload, IAssetRegistryInterface& InAssetRegistry)
{
	ProcessedPackages.Add(PackageName);

	TArray<FName> PackageDependencies;
	InAssetRegistry.GetDependencies(PackageName, PackageDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);

	// Recursively go through processing each new dependency until we run out
	for (const FName& Dependency : PackageDependencies)
	{
		if (!ProcessedPackages.Contains(Dependency))
		{
			SortPackagesForReload(Dependency, ProcessedPackages, SortedPackagesToReload, AllPackagesToReload, InAssetRegistry);
		}
	}

	// Add this package to the sorted array now that its dependencies have been processed
	if (AllPackagesToReload.Contains(PackageName))
	{
		SortedPackagesToReload.Emplace(AllPackagesToReload[PackageName]);
	}
}

} // namespace PackageReloadInternal

void SortPackagesForReload(TArray<UPackage*>& PackagesToReload)
{
	// We need to sort the packages to reload so that dependencies are reloaded before the assets that depend on them
	if (PackagesToReload.Num() > 1)
	{
		IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr();
		checkf(AssetRegistry, TEXT("SortPackagesForReload requires the asset registry to perform dependency analysis, but no asset registry is available."));

		TSet<FName> ProcessedPackages;
		ProcessedPackages.Reserve(PackagesToReload.Num());

		TArray<UPackage*> SortedPackagesToReload;
		SortedPackagesToReload.Reserve(PackagesToReload.Num());

		TMap<FName, UPackage*> AllPackagesToReload;
		AllPackagesToReload.Reserve(PackagesToReload.Num());
		for (UPackage* PackageToReload : PackagesToReload)
		{
			AllPackagesToReload.Emplace(PackageToReload->GetFName(), PackageToReload);
		}

		for (UPackage* PackageToReload : PackagesToReload)
		{
			if (!ProcessedPackages.Contains(PackageToReload->GetFName()))
			{
				PackageReloadInternal::SortPackagesForReload(PackageToReload->GetFName(), ProcessedPackages, SortedPackagesToReload, AllPackagesToReload, *AssetRegistry);
			}
		}

		PackagesToReload = MoveTemp(SortedPackagesToReload);
	}
}

UPackage* ReloadPackage(UPackage* InPackageToReload, const uint32 InLoadFlags)
{
	TArray<UPackage*> ReloadedPackages;
	
	FReloadPackageData ReloadPackageData(InPackageToReload, InLoadFlags);
	ReloadPackages(TArrayView<FReloadPackageData>(&ReloadPackageData, 1), ReloadedPackages, 1);

	return ReloadedPackages[0];
}

void ReloadPackages(const TArrayView<FReloadPackageData>& InPackagesToReload, TArray<UPackage*>& OutReloadedPackages, int32 InNumPackagesPerBatch)
{
	FPackageReferencersHelper PackageReferencersHelper(InPackagesToReload);

	FString Msg;
	Msg.Append(FString::Printf(TEXT("Reloading %d Package(s):"), InPackagesToReload.Num()));
	const int32 MAX_PACKAGES_TO_LOG = 10;
	for(int32 I = 0; I < InPackagesToReload.Num() && I < MAX_PACKAGES_TO_LOG; ++I)
	{
		const FReloadPackageData& ReloadPackageData = InPackagesToReload[I];
		Msg.Append(TEXT("\n"));
		Msg.Append(FString::Printf(TEXT("\tAsset Name: %s"), *ReloadPackageData.PackageToReload->GetName()));
	}
	UE_LOGF(LogUObjectGlobals, Log, "%ls", *Msg);

	FScopedSlowTask ReloadingPackagesSlowTask((float)InPackagesToReload.Num(), NSLOCTEXT("CoreUObject", "ReloadingPackages", "Reloading Packages"));
	ReloadingPackagesSlowTask.MakeDialogDelayed(3.0f);

	// Cache the current dirty state of all packages so we can restore it after the reload
	TSet<FName> DirtyPackages;
	ForEachObjectOfClass(UPackage::StaticClass(), [&DirtyPackages](UObject* InPackageObj)
	{
		UPackage* Package = CastChecked<UPackage>(InPackageObj);
		if (Package->IsDirty())
		{
			DirtyPackages.Add(Package->GetFName());
		}
	}, false);

	// Gather up the list of all packages to reload (note: this array may include null packages!)
	PackageReloadInternal::FExistingPackageReferences ExistingPackages;
	ExistingPackages.Refs.Reserve(InPackagesToReload.Num());
	{
		FScopedSlowTask PreparingPackagesForReloadSlowTask((float)InPackagesToReload.Num(), NSLOCTEXT("CoreUObject", "PreparingPackagesForReload", "Preparing Packages for Reload"));

		for (const FReloadPackageData& PackageToReloadData : InPackagesToReload)
		{
			PreparingPackagesForReloadSlowTask.EnterProgressFrame(1.0f);
			ExistingPackages.Refs.Emplace(PackageReloadInternal::ValidateAndPreparePackageForReload(PackageToReloadData.PackageToReload));
		}

		if (ExistingPackages.Refs.Num() > 0)
		{
			// Run a GC before we start to clean-up any lingering objects that may reference things we're about to reload
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}

	// Rename the existing packages, load the new packages, then fix-up any references
	PackageReloadInternal::FNewPackageReferences NewPackages;
	NewPackages.Refs.Reserve(ExistingPackages.Refs.Num());
	{
		// Process the packages in batches to avoid consuming too much memory due to a lack of GC
		int32 PackageIndex = 0;
		while (PackageIndex < ExistingPackages.Refs.Num())
		{
			FCoreUObjectDelegates::OnPackageReloaded.Broadcast(EPackageReloadPhase::PreBatch, nullptr);

			const int32 BatchStartIndex = PackageIndex;
			for (; PackageIndex < ExistingPackages.Refs.Num(); ++PackageIndex)
			{
				UPackage* ExistingPackage = ExistingPackages.Refs[PackageIndex].RawRef;

				const FText ProgressText = ExistingPackage 
					? FText::Format(NSLOCTEXT("CoreUObject", "ReloadingPackagef", "Reloading {0}..."), FText::FromName(ExistingPackage->GetFName()))
					: NSLOCTEXT("CoreUObject", "ReloadingPackages", "Reloading Packages");
				ReloadingPackagesSlowTask.EnterProgressFrame(1, ProgressText);

				{
					FPackageReloadedEvent TempReloadEvent(ExistingPackage, nullptr, TMap<UObject*, UObject*>());
					FCoreUObjectDelegates::OnPackageReloaded.Broadcast(EPackageReloadPhase::PrePackageLoad, &TempReloadEvent);
				}

				check(NewPackages.Refs.Num() == PackageIndex);
				NewPackages.Refs.Emplace(PackageReloadInternal::LoadReplacementPackage(ExistingPackage, InPackagesToReload[PackageIndex].LoadFlags));

				UPackage* NewPackage = NewPackages.Refs[PackageIndex].Package;
				NewPackages.Refs[PackageIndex].EventData = PackageReloadInternal::GeneratePackageReloadEvent(ExistingPackage, NewPackage);

				// Interdependencies between packages (in particular Blueprints) make it unsafe to run this logic in batches. 
				// There are a number of edge cases that would have to be addressed if the batching logic were to be re-enabled, but 
				// most likely the blueprint reparenting step would have to take in a TMap<UObject*, UObject*> that it could update
				// when it decided that it needed to replace an instance due to hierarchy changes (e.g. class layout changing
				// due to SuperStruct changes). For now, just process assets in a batch size of 1:
				const bool bEndBatch = true; //(PackageIndex == (BatchStartIndex + InNumPackagesPerBatch - 1)) || (ExistingPackage && ExistingPackage->ContainsMap());
				if (bEndBatch)
				{
					++PackageIndex; // We still need to move on-to the next package for the next batch
					break;
				}
			}

			const int32 NumPackagesInBatch = PackageIndex - BatchStartIndex;

			FScopedSlowTask FixingUpReferencesSlowTask((float)((NumPackagesInBatch * 4) + GUObjectArray.GetObjectArrayNum()), NSLOCTEXT("CoreUObject", "FixingUpReferences", "Fixing-Up References"));

			// Pre-pass to notify things that the package old package is about to be fixed-up
			TArray<UObject*> PotentialReferencers;
			TMap<UObject*, PackageReloadInternal::FObjectAndPackageIndex> OldObjectToNewData;
			for (int32 BatchPackageIndex = BatchStartIndex; BatchPackageIndex < PackageIndex; ++BatchPackageIndex)
			{
				FixingUpReferencesSlowTask.EnterProgressFrame(1.0f);

				PackageReloadInternal::FNewPackageReference& NewPackageData = NewPackages.Refs[BatchPackageIndex];
				if (NewPackageData.EventData.IsValid())
				{
					FCoreUObjectDelegates::OnPackageReloaded.Broadcast(EPackageReloadPhase::PrePackageFixup, NewPackageData.EventData.Get());
					FCoreUObjectDelegates::OnPackageReloaded.Broadcast(EPackageReloadPhase::OnPackageFixup, NewPackageData.EventData.Get());

					// Build up the mapping of old objects to the package index that contains them; this is needed to track per-package references correctly
					OldObjectToNewData.Reserve(OldObjectToNewData.Num() + NewPackageData.EventData->GetRepointedObjects().Num());
					for (const auto& ObjectMappingPair : NewPackageData.EventData->GetRepointedObjects())
					{
						OldObjectToNewData.Add(ObjectMappingPair.Key, PackageReloadInternal::FObjectAndPackageIndex(ObjectMappingPair.Value, BatchPackageIndex));
						
						// Also do reference replacement on the loaded objects, in case they have buggy
						// Lifecycle functions that have created references to the DEADPACKAGE:
						if (ObjectMappingPair.Value)
						{
							PotentialReferencers.Add(ObjectMappingPair.Value);
						}
					}
				}
			}

			// Main pass to go through and fix-up any references pointing to data from the old package to point to data from the new package
			// todo: multi-thread this like FHotReloadModule::ReplaceReferencesToReconstructedCDOs?
			//The FThreadSafeObjectIterator will lock the global UObject array, to avoid potential deadlock we simply build the list
			//of the potential referencers and do the reference fix serialize outside of the FThreadSafeObjectIterator.

			PackageReferencersHelper.ForEachObject(
				[&PotentialReferencers] (UObject* PotentialReferencer)
				{
					// Mutating the old versions of classes can result in us replacing the SuperStruct pointer, which results
					// in class layout change and subsequently crashes because instances will not match this new class layout:
					UClass* AsClass = Cast<UClass>(PotentialReferencer);
					if (!AsClass)
					{
						AsClass = PotentialReferencer->GetTypedOuter<UClass>();
					}

					if (AsClass)
					{
						if (AsClass->HasAnyClassFlags(CLASS_NewerVersionExists) ||
							AsClass->HasAnyFlags(RF_NewerVersionExists))
						{
							return;
						}
					}
					PotentialReferencers.Add(PotentialReferencer);
				}
			);

			// Main reference replacement pass - this should fix as many references as possible
			// allowing users to run code that has no visibility into the old data and
			// therefore is not at risk of out of date reads or writes:
			for (UObject* PotentialReferencer : PotentialReferencers)
			{
				FixingUpReferencesSlowTask.EnterProgressFrame(1.0f);

				PackageReloadInternal::FReplaceObjectReferencesArchive ReplaceRefsArchive(PotentialReferencer, OldObjectToNewData, ExistingPackages.Refs, NewPackages.Refs);
				PotentialReferencer->Serialize(ReplaceRefsArchive); // Deal with direct references during Serialization
				PotentialReferencer->GetClass()->CallAddReferencedObjects(PotentialReferencer, ReplaceRefsArchive); // Deal with indirect references via AddReferencedObjects
			}

			// The above fix-up also repoints the StrongRef in FExistingPackageReference, so we'll fix that up again now to prevent the old package from being GC'd
			for (int32 BatchPackageIndex = BatchStartIndex; BatchPackageIndex < PackageIndex; ++BatchPackageIndex)
			{
				FixingUpReferencesSlowTask.EnterProgressFrame(1.0f);

				ExistingPackages.Refs[BatchPackageIndex].StrongRef = ExistingPackages.Refs[BatchPackageIndex].RawRef;
			}

			// Final pass to clean-up any remaining references prior to GC
			// Note: We do this as a separate pass to preparing the objects for GC as this callback may prematurely invoke a GC that invalidates some data we're working with
			for (int32 BatchPackageIndex = BatchStartIndex; BatchPackageIndex < PackageIndex; ++BatchPackageIndex)
			{
				FixingUpReferencesSlowTask.EnterProgressFrame(1.0f);

				PackageReloadInternal::FNewPackageReference& NewPackageData = NewPackages.Refs[BatchPackageIndex];
				if (NewPackageData.EventData.IsValid())
				{
					FCoreUObjectDelegates::OnPackageReloaded.Broadcast(EPackageReloadPhase::PostPackageFixup, NewPackageData.EventData.Get());
				}
			}

			FCoreUObjectDelegates::OnPackageReloaded.Broadcast(EPackageReloadPhase::PostBatchPreGC, nullptr);

			// Purge old packages that have had a replacement package loaded
			for (int32 BatchPackageIndex = BatchStartIndex; BatchPackageIndex < PackageIndex; ++BatchPackageIndex)
			{
				FixingUpReferencesSlowTask.EnterProgressFrame(1.0f);

				UPackage* ExistingPackage = ExistingPackages.Refs[BatchPackageIndex].RawRef;
				UPackage* NewPackage = NewPackages.Refs[BatchPackageIndex].Package;

				if (ExistingPackage && NewPackage)
				{
					// Allow the old package to be GC'd
					PackageReloadInternal::MakePackagePurgeable(ExistingPackage);
					ExistingPackages.Refs[BatchPackageIndex].StrongRef = nullptr;
					NewPackages.Refs[BatchPackageIndex].EventData.Reset();
				}
			}

			if (PackageIndex > 0 && (PackageIndex % InNumPackagesPerBatch) == 0)
			{
				CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
			}

			FCoreUObjectDelegates::OnPackageReloaded.Broadcast(EPackageReloadPhase::PostBatchPostGC, nullptr);
		}
	}

	// Clean any packages that we dirtied as part of the replacement process
	ForEachObjectOfClass(UPackage::StaticClass(), [&DirtyPackages](UObject* InPackageObj)
	{
		UPackage* Package = CastChecked<UPackage>(InPackageObj);
		if (Package->IsDirty() && !DirtyPackages.Contains(Package->GetFName()))
		{
			Package->SetDirtyFlag(false);
		}
	}, false);
	
	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	// Finalization and error reporting
	OutReloadedPackages.Reserve(ExistingPackages.Refs.Num());
	for (int32 PackageIndex = 0; PackageIndex < ExistingPackages.Refs.Num(); ++PackageIndex)
	{
		UPackage* ExistingPackage = ExistingPackages.Refs[PackageIndex].WeakRef.Get();
		UPackage* NewPackage = NewPackages.Refs[PackageIndex].Package;

		check(OutReloadedPackages.Num() == PackageIndex);
		OutReloadedPackages.Emplace(NewPackage);

		// Report any old packages that failed to purge
		if (ExistingPackage && NewPackage)
		{
			UE_LOGF(LogUObjectGlobals, Warning, "ReloadPackage failed to purge the old package '%ls'. This is unexpected, and likely means that it was still externally referenced.", *ExistingPackage->GetName());

			const bool bDumpExternalReferences = DO_GUARD_SLOW || (WITH_EDITOR && GIsEditor);
			if (bDumpExternalReferences)
			{
				PackageReloadInternal::DumpExternalReferences(ExistingPackage, ExistingPackage);
				//ForEachObjectWithPackage(ExistingPackage, [](UObject* InExistingObject)
				//{
				//	PackageReloadInternal::DumpExternalReferences(InExistingObject, ExistingPackage);
				//	return true; // continue
				//}, EGetObjectsFlags::IncludeNestedObjects, RF_NoFlags, EInternalObjectFlags::Garbage);
			}
		}
	}
}
