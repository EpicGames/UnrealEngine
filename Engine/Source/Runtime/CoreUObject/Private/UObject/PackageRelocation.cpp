// Copyright Epic Games, Inc.All Rights Reserved.

#include "UObject/PackageRelocation.h"

#include "HAL/IConsoleManager.h"
#include "Misc/PathViews.h"
#include "String/ParseTokens.h"
#include "UObject/ObjectResource.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"

DEFINE_LOG_CATEGORY(LogPackageRelocation);

namespace UE::Package::Relocation::Private
{
	static TAutoConsoleVariable<int32> CVarRelocationMode(
			TEXT("Package.Relocation"),
			0,
			TEXT("Define when we should run the relocation logic for the dependencies of a package. Note changing this value at runtime won't update the cached depencencies in the asset registry.\n")
			TEXT("  0: Off (never apply relocation. References to other relocated packages will give errors and fail to resolve.)\n")
			TEXT("  1: Relocate any asset saved after EUnrealEngineObjectUE5Version::ADD_SOFTOBJECTPATH_LIST (5.1). Default value for the new projects.\n")
		);

	bool ShouldApplyRelocation(const FPackageFileSummary& PackageSummary, FStringView LoadedPackageName, FPackageRelocationContext& OutPackageRelocationContext)
	{
		OutPackageRelocationContext.CurrentPackagePath = FPathViews::GetPath(LoadedPackageName);
		OutPackageRelocationContext.OriginalPackagePath = FPathViews::GetPath(PackageSummary.PackageName);

		if (OutPackageRelocationContext.CurrentPackagePath == OutPackageRelocationContext.OriginalPackagePath)
		{
			// Don't apply relocation to packages that have renamed their leaf filename but remain in the same parent directory
			return false;
		}

		switch (CVarRelocationMode.GetValueOnAnyThread())
		{
		case 1:
			if (PackageSummary.GetFileVersionUE() < EUnrealEngineObjectUE5Version::ADD_SOFTOBJECTPATH_LIST)
			{
				return false;
			}
			break;

		default:
			return false;
		}

		bool bHasClassesPrefix = false;
		FStringView MountPointName = FPathViews::GetMountPointNameFromPath(OutPackageRelocationContext.OriginalPackagePath, &bHasClassesPrefix);
		if (MountPointName.IsEmpty() || bHasClassesPrefix)
		{
			return false;
		}

		/*
		 * The function GetMountPointNameFromPath return the mount point name without the '/' at the front and at the end.
		 * So we use the original package path and the MountPointName length + 2 to reintroduce the '/' to the cached mount point.
		 */
		OutPackageRelocationContext.OriginalPackageMount = FStringView(OutPackageRelocationContext.OriginalPackagePath.GetData(), MountPointName.Len() + 2);

		return true;
	}

	bool TryRelocateReference(const FPackageRelocationContext& InPackageRelocationContext, FStringView InPackageNameToRelocate, FStringBuilderBase& OutNewLocation)
	{
		// Validate that the package to relocate is the same mount point as the original package name
		if (!InPackageNameToRelocate.StartsWith(InPackageRelocationContext.OriginalPackageMount))
		{
			return false;
		}

		// Compute the info to generate a relative path of the InPackageName against the OriginalPackagePath. (this is faster and work better for this scenario that the one from FPaths)
		const UE::String::EParseTokensOptions ParseOptions = UE::String::EParseTokensOptions::SkipEmpty;
		 
		int32 FolderUpCount = 0;
		TArray<FStringView, TInlineAllocator<16>> ToAppend;
		{
			const FStringView PackagePathToRelocateWithoutMountPoint = InPackageNameToRelocate.RightChop(InPackageRelocationContext.OriginalPackageMount.Len());
			TArray<FStringView, TInlineAllocator<16>> PackageNameArrayBuffer;
			UE::String::ParseTokens(PackagePathToRelocateWithoutMountPoint, TEXTVIEW("/"), PackageNameArrayBuffer, ParseOptions);

			const FStringView OriginalPackagePathWithoutMountPoint = InPackageRelocationContext.OriginalPackagePath.RightChop(InPackageRelocationContext.OriginalPackageMount.Len());
			TArray<FStringView, TInlineAllocator<16>> OriginalPackagePathArrayBuffer;
			UE::String::ParseTokens(OriginalPackagePathWithoutMountPoint, TEXTVIEW("/"), OriginalPackagePathArrayBuffer, ParseOptions);


			int32 IdenticalDepth = 0;
			while (IdenticalDepth < PackageNameArrayBuffer.Num() && IdenticalDepth < OriginalPackagePathArrayBuffer.Num() && PackageNameArrayBuffer[IdenticalDepth] == OriginalPackagePathArrayBuffer[IdenticalDepth])
			{
				++IdenticalDepth;
			}

			FolderUpCount = OriginalPackagePathArrayBuffer.Num() - IdenticalDepth;

			ToAppend.Reserve(PackageNameArrayBuffer.Num() - IdenticalDepth);
			for (int32 Index = IdenticalDepth; Index < PackageNameArrayBuffer.Num(); ++Index)
			{
				ToAppend.Add(MoveTemp(PackageNameArrayBuffer[Index]));
			}
		}

		TArray<FStringView, TInlineAllocator<16>> CurrentPackageNameArrayBuffer;
		UE::String::ParseTokens(InPackageRelocationContext.CurrentPackagePath, TEXTVIEW("/"), CurrentPackageNameArrayBuffer, ParseOptions);

		OutNewLocation.Reset();

		if (CurrentPackageNameArrayBuffer.Num() <= FolderUpCount)
		{
			// Bail out. This path doesn't exist.
			return true;
		}


		for (int32 Index = 0; Index < CurrentPackageNameArrayBuffer.Num() - FolderUpCount; ++Index)
		{
			OutNewLocation.AppendChar(TEXT('/'));
			OutNewLocation.Append(CurrentPackageNameArrayBuffer[Index]);
		}

		for (int32 Index = 0; Index < ToAppend.Num(); ++Index)
		{
			OutNewLocation.AppendChar(TEXT('/'));
			OutNewLocation.Append(ToAppend[Index]);
		}

		return true;
	}

	void ApplyRelocationToObjectImportMap(const FPackageRelocationContext& InPackageRelocationContext, TArrayView<FObjectImport> ImportMapView)
	{
		for (FObjectImport& Import : ImportMapView)
		{
			if (!Import.OuterIndex.IsNull())
			{
				continue;
			}

			// Generate relative fix-up for package path from the same original package mount point of the package we are loading
			FNameBuilder ImportPackageName(Import.ObjectName);
			FNameBuilder RelocatedPackageName;
			if (TryRelocateReference(InPackageRelocationContext, ImportPackageName.ToView(), RelocatedPackageName))
			{
				Import.ObjectName = *RelocatedPackageName;
			}
		}
	}

	void ApplyRelocationToSoftObjectArray(const FPackageRelocationContext& InPackageRelocationContext, TArrayView<FSoftObjectPath> SoftObjectPaths)
	{
		for (FSoftObjectPath& SoftObjectPath : SoftObjectPaths)
		{
			FNameBuilder PackageName(SoftObjectPath.GetAssetPath().GetPackageName());
			FNameBuilder RelocatedPackageName;
			if (TryRelocateReference(InPackageRelocationContext, PackageName.ToView(), RelocatedPackageName))
			{
				if (RelocatedPackageName.Len() != 0)
				{
					FTopLevelAssetPath AssetPath = SoftObjectPath.GetAssetPath();
					SoftObjectPath.SetPath(FTopLevelAssetPath(FName(*RelocatedPackageName), AssetPath.GetAssetName()));
				}
				else
				{
					SoftObjectPath.Reset();
				}
			}
		}
	}

	void ApplyRelocationToNameArray(const FPackageRelocationContext& InPackageRelocationContext, TArrayView<FName> PackageNames)
	{
		for (FName& PackageName : PackageNames)
		{
			FNameBuilder Package(PackageName);
			FNameBuilder RelocatedPackageName;
			if (TryRelocateReference(InPackageRelocationContext, Package.ToView(), RelocatedPackageName))
			{
				PackageName = *RelocatedPackageName;
			}
		}
	}
}