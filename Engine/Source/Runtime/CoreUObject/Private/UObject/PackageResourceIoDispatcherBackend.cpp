// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PackageResourceIoDispatcherBackend.h"

#include "Async/AsyncFileHandle.h"
#include "Async/MappedFileHandle.h"
#include "Misc/PackageSegment.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/IoStoreTrace.h"
#include "Serialization/BulkDataCookedIndex.h"
#include "UObject/PackageResourceManager.h"

namespace UE
{

static_assert(sizeof(FBulkDataCookedIndex::ValueType) == sizeof(uint8)); // If FBulkDataCookedIndex changes type then we need update CreatePackageResourceChunkId
constexpr uint8 CookedIndexByteIdx = 8; // Position in the byte array that we are storing the cooked index in

FIoChunkId CreatePackageResourceChunkId(const FName& PackageName, EPackageSegment Segment, const FBulkDataCookedIndex& CookedIndex, bool bExternalResource)
{
	const int32 Index = PackageName.GetComparisonIndex().ToUnstableInt();
	const int32 Number = PackageName.GetNumber();
	
	uint8 Id[12] = {0};
	FMemory::Memcpy(Id, &Index, sizeof(int32)); // Bytes 0-3
	FMemory::Memcpy(Id + sizeof(int32), &Number, sizeof(int32)); // Bytes 4-7
	Id[CookedIndexByteIdx] = CookedIndex.GetValue();
	Id[9] = uint8(Segment);
	Id[10] = uint8(bExternalResource);
	Id[11] = uint8(EIoChunkType::PackageResource);

	FIoChunkId ChunkId;
	ChunkId.Set(Id, sizeof(Id));

	return ChunkId;
}

bool TryGetPackageNameFromChunkId(const FIoChunkId& ChunkId, FName& OutPackageName, EPackageSegment& OutSegment, bool& bOutExternal)
{
	if (ChunkId.GetChunkType() != EIoChunkType::PackageResource)
	{
		return false;
	}

	const uint8* Id = ChunkId.GetData();
	const int32 NameIndex = *reinterpret_cast<const int32*>(Id);
	const int32 NameNumber = *reinterpret_cast<const int32*>(Id + sizeof(int32));

	OutPackageName = FName::CreateFromDisplayId(FNameEntryId::FromUnstableInt(NameIndex), NameNumber);
	OutSegment = static_cast<EPackageSegment>(Id[9]);
	bOutExternal = static_cast<bool>(Id[10]);

	return true;
}

bool TryGetPackagePathFromChunkId(const FIoChunkId& ChunkId, FPackagePath& OutPath, EPackageSegment& OutSegment, bool& bExternal)
{
	FName PackageName;
	if (TryGetPackageNameFromChunkId(ChunkId, PackageName, OutSegment, bExternal))
	{
		return FPackagePath::TryFromPackageName(PackageName, OutPath);
	}

	return false;
}

bool TryGetPackagePathFromChunkId(const FIoChunkId& ChunkId, FPackagePath& OutPath, EPackageSegment& OutSegment, bool& bExternal, FBulkDataCookedIndex& OutCookedIndex)
{
	FName PackageName;
	if (TryGetPackageNameFromChunkId(ChunkId, PackageName, OutSegment, bExternal))
	{
		OutCookedIndex = FBulkDataCookedIndex(ChunkId.GetData()[CookedIndexByteIdx]);
		return FPackagePath::TryFromPackageName(PackageName, OutPath);
	}

	return false;
}

inline EAsyncIOPriorityAndFlags ConvertToAsyncIoPriority(const int32 IoDispatcherProperty)
{
	if (IoDispatcherProperty < IoDispatcherPriority_Low)
	{
		return AIOP_MIN;
	}

	if (IoDispatcherProperty < (IoDispatcherPriority_Medium - 1))
	{
		return AIOP_Low;
	}

	if (IoDispatcherProperty < IoDispatcherPriority_Medium)
	{
		return AIOP_BelowNormal;
	}

	if (IoDispatcherProperty < IoDispatcherPriority_High)
	{
		return AIOP_Normal;
	}

	if (IoDispatcherProperty < IoDispatcherPriority_Max)
	{
		return AIOP_High;
	}
	
	return AIOP_CriticalPath;
}

class FPackageResourceIoBackend final
	: public IIoDispatcherBackend
{
	class FPendingRequests
	{
		struct FHandles
		{
			TUniquePtr<IAsyncReadFileHandle> FileHandle;
			TUniquePtr<IAsyncReadRequest> RequestHandle;
		};

	public:
		void Add(FIoRequestImpl* Request, TUniquePtr<IAsyncReadFileHandle>&& FileHandle, TFunction<IAsyncReadRequest*(IAsyncReadFileHandle&)>&& MakeReadRequest)
		{
			FScopeLock _(&CriticalSection);
			FHandles& Handles = Lookup.Add(Request);
			Handles.FileHandle = MoveTemp(FileHandle);
			Handles.RequestHandle.Reset(MakeReadRequest(*Handles.FileHandle));
		}

		void Remove(FIoRequestImpl* Request)
		{
			FScopeLock _(&CriticalSection);
			
			if (FHandles* Handles = Lookup.Find(Request))
			{
				Handles->RequestHandle->WaitCompletion();
				Handles->RequestHandle.Reset();

				Lookup.Remove(Request);
			}
		}

		void Cancel(FIoRequestImpl* Request)
		{
			FScopeLock _(&CriticalSection);
			if (FHandles* Handles = Lookup.Find(Request))
			{
				Handles->RequestHandle->Cancel();
			}
		}

	private:
		FCriticalSection CriticalSection;
		TMap<FIoRequestImpl*, FHandles> Lookup;
	};

	class FCompletedRequests
	{
	public:
		void Enqueue(FIoRequestImpl* Request)
		{
			FScopeLock _(&CriticalSection);

			if (Tail)
			{
				Tail->NextRequest = Request;
			}
			else
			{
				Head = Request;
			}
			Tail = Request;
		}

		FIoRequestImpl* Dequeue()
		{
			FScopeLock _(&CriticalSection);

			FIoRequestImpl* Completed = Head;
			Head = Tail = nullptr;

			return Completed;
		}

	private:
		FCriticalSection CriticalSection;
		FIoRequestImpl* Head = nullptr;
		FIoRequestImpl* Tail = nullptr;
	};

public:
	FPackageResourceIoBackend(IPackageResourceManager& Mgr);
	~FPackageResourceIoBackend();

	virtual void Initialize(TSharedRef<const FIoDispatcherBackendContext> Context) override;
	virtual void ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved) override;
	virtual void CancelIoRequest(FIoRequestImpl* Request) override;
	virtual void UpdatePriorityForIoRequest(FIoRequestImpl* Request) override { }
	virtual bool DoesChunkExist(const FIoChunkId& ChunkId) const override;
	virtual TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId) const override;
	virtual FIoRequestImpl* GetCompletedIoRequests() override;
	virtual TIoStatusOr<FIoMappedRegion> OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options) override;
	virtual const TCHAR* GetName() const override;
private:
	bool Resolve(FIoRequestImpl* Request);

	IPackageResourceManager& ResourceMgr;
	TSharedPtr<const FIoDispatcherBackendContext> BackendContext;
	FPendingRequests PendingRequests;
	FCompletedRequests CompletedRequests;
};

FPackageResourceIoBackend::FPackageResourceIoBackend(IPackageResourceManager& Mgr)
	: ResourceMgr(Mgr)
{
}

FPackageResourceIoBackend::~FPackageResourceIoBackend()
{
}

void FPackageResourceIoBackend::Initialize(TSharedRef<const FIoDispatcherBackendContext> Context)
{
	BackendContext = Context;
}

bool FPackageResourceIoBackend::Resolve(FIoRequestImpl* Request)
{
	const FIoChunkId& ChunkId = Request->ChunkId;

	FPackagePath Path;
	EPackageSegment Segment;
	bool bExternalResource = false;
	FBulkDataCookedIndex CookedIndex;

	if (TryGetPackagePathFromChunkId(ChunkId, Path, Segment, bExternalResource, CookedIndex) == false)
	{
		return false;
	}

	checkf(!bExternalResource || CookedIndex.IsDefault(), TEXT("Cannot use 'CookedIndices' with packages in the workspace domain"));

	TUniquePtr<IAsyncReadFileHandle> FileHandle = bExternalResource
		? ResourceMgr.OpenAsyncReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, Path.GetPackageName()).Handle
		: ResourceMgr.OpenAsyncReadPackage(Path, CookedIndex, Segment).Handle;

	if (FileHandle.IsValid() == false)
	{
		return false;
	}
	
	if (Request->Options.GetSize() == 0)
	{
		void* UserSuppliedMemory = Request->Options.GetTargetVa();

		FIoBuffer Buffer = UserSuppliedMemory	? FIoBuffer(FIoBuffer::Wrap, UserSuppliedMemory, 0)
												: FIoBuffer(0);

		Request->SetResult(Buffer);

		CompletedRequests.Enqueue(Request);
		BackendContext->WakeUpDispatcherThreadDelegate.Execute();

		return true;
	}

	PendingRequests.Add(Request, MoveTemp(FileHandle), [this, Request](IAsyncReadFileHandle& FileHandle)
	{
		FAsyncFileCallBack Callback = [this, Request](bool bWasCancelled, IAsyncReadRequest* FileReadRequest)
		{
			if (uint8* Data = FileReadRequest->GetReadResults())
			{
				const bool bUserSuppliedMemory = Request->Options.GetTargetVa() != nullptr;

				FIoBuffer Buffer = bUserSuppliedMemory
					? FIoBuffer(FIoBuffer::Wrap, Data, Request->Options.GetSize())
					: FIoBuffer(FIoBuffer::AssumeOwnership, Data, Request->Options.GetSize());

				Request->SetResult(Buffer);
				TRACE_IOSTORE_BACKEND_REQUEST_COMPLETED(Request, Request->Options.GetSize());
			}
			else
			{
				Request->SetFailed();
				TRACE_IOSTORE_BACKEND_REQUEST_FAILED(Request);
			}
			
			CompletedRequests.Enqueue(Request);
			BackendContext->WakeUpDispatcherThreadDelegate.Execute();
		};

		const FIoReadOptions& Options = Request->Options;
		const EAsyncIOPriorityAndFlags AsyncIoPriority = ConvertToAsyncIoPriority(Request->Priority);

		TRACE_IOSTORE_BACKEND_REQUEST_STARTED(Request, this);
		return FileHandle.ReadRequest(
			Options.GetOffset(),
			Options.GetSize(),
			AsyncIoPriority,
			&Callback,
			(uint8*)Options.GetTargetVa());
	});

	return true;
}

void FPackageResourceIoBackend::ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved)
{
	while (FIoRequestImpl* Request = Requests.PopHead())
	{
		if (Resolve(Request) == false)
		{
			OutUnresolved.AddTail(Request);
		}
	}
}

void FPackageResourceIoBackend::CancelIoRequest(FIoRequestImpl* Request)
{
	PendingRequests.Cancel(Request);
}

bool FPackageResourceIoBackend::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	FPackagePath Path;
	EPackageSegment Segment;
	bool bExternalResource = false;
	FBulkDataCookedIndex CookedIndex;

	if (TryGetPackagePathFromChunkId(ChunkId, Path, Segment, bExternalResource, CookedIndex) == false)
	{
		return false;
	}

	
	return ResourceMgr.DoesPackageExist(Path, CookedIndex, Segment);
}

TIoStatusOr<uint64> FPackageResourceIoBackend::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	FPackagePath Path;
	EPackageSegment Segment;
	bool bExternalResource = false;
	FBulkDataCookedIndex CookedIndex;

	if (TryGetPackagePathFromChunkId(ChunkId, Path, Segment, bExternalResource, CookedIndex) == false)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	
	if (int64 FileSize = ResourceMgr.FileSize(Path, CookedIndex, Segment); FileSize > 0)
	{
		return static_cast<uint64>(FileSize);
	}

	return FIoStatus(EIoErrorCode::NotFound);
}

FIoRequestImpl* FPackageResourceIoBackend::GetCompletedIoRequests()
{
	FIoRequestImpl* Requests = CompletedRequests.Dequeue();

	for (FIoRequestImpl* It = Requests; It != nullptr; It = It->NextRequest)
	{
		PendingRequests.Remove(It);
	}

	return Requests;
}

TIoStatusOr<FIoMappedRegion> FPackageResourceIoBackend::OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options)
{
	FPackagePath Path;
	EPackageSegment Segment;
	bool bExternalResource = false;
	FBulkDataCookedIndex ChunkGroup;

	if (TryGetPackagePathFromChunkId(ChunkId, Path, Segment, bExternalResource, ChunkGroup) == false)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	TUniquePtr<IMappedFileHandle> FileHandle(ResourceMgr.OpenMappedHandleToPackage(Path));
	
	if (FileHandle.IsValid())
	{
		TUniquePtr<IMappedFileRegion> MappedRegion(
			FileHandle->MapRegion(static_cast<int64>(Options.GetOffset()), static_cast<int64>(Options.GetSize())));

		if (MappedRegion.IsValid())
		{
			return FIoMappedRegion { FileHandle.Release(), MappedRegion.Release() };
		}
	}

	return FIoStatus(EIoErrorCode::NotFound);
}

const TCHAR* FPackageResourceIoBackend::GetName() const
{
	return TEXT("PackageResource");
}

TSharedRef<IIoDispatcherBackend> MakePackageResourceIoDispatcherBackend(IPackageResourceManager& Mgr)
{
	return MakeShared<FPackageResourceIoBackend>(Mgr);
}

} // namespace UE
