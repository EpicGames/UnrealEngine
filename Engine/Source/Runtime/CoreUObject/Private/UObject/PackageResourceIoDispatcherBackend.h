// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoDispatcherBackend.h"
#include "Misc/PackageSegment.h"
#include "Templates/SharedPointer.h"

class FBulkDataCookedIndex;
class FIoChunkId;
class IPackageResourceManager;

namespace UE
{

COREUOBJECT_API TSharedRef<IIoDispatcherBackend> MakePackageResourceIoDispatcherBackend(IPackageResourceManager& Mgr);

COREUOBJECT_API FIoChunkId CreatePackageResourceChunkId(const FName& PackageName, EPackageSegment Segment, const FBulkDataCookedIndex& CookedIndex, bool bExternalResource);

COREUOBJECT_API bool TryGetPackageNameFromChunkId(const FIoChunkId& ChunkId, FName& OutPackageName, EPackageSegment& OutSegment, bool& bOutExternal); 

} // namespace UE
