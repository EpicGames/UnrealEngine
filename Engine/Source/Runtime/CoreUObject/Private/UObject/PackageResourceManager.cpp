// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PackageResourceManager.h"

#include "Async/AsyncFileHandle.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageSegment.h"
#include "Misc/PreloadableFile.h"
#include "Misc/ScopeLock.h"
#include "Serialization/Archive.h"
#include "UObject/PackageResourceManagerFile.h"
#include "UObject/PackageResourceIoDispatcherBackend.h"

DEFINE_LOG_CATEGORY(LogPackageResourceManager);

namespace
{
	IPackageResourceManager* GPackageResourceManager = nullptr;
	FSetPackageResourceManager GSetPackageResourceManagerDelegate;
	FOnClearPackageResourceManager GOnClearPackageResourceManagerDelegate;
}

IPackageResourceManager& IPackageResourceManager::Get()
{
	check(GPackageResourceManager);
	return *GPackageResourceManager;
}

FSetPackageResourceManager& IPackageResourceManager::GetSetPackageResourceManagerDelegate()
{
	return GSetPackageResourceManagerDelegate;
}

FOnClearPackageResourceManager& IPackageResourceManager::GetOnClearPackageResourceManagerDelegate()
{
	return GOnClearPackageResourceManagerDelegate;
}

void IPackageResourceManager::Initialize()
{
	FSetPackageResourceManager& SetManagerDelegate = GetSetPackageResourceManagerDelegate(); 
	if (SetManagerDelegate.IsBound())
	{
		// Allow the editor or licensee project to define the PackageResourceManager
		GPackageResourceManager = SetManagerDelegate.Execute();
	}

	// Assign the default PackageResourceManager if it was not set by a higher level source
	if (!GPackageResourceManager)
	{
		GPackageResourceManager = MakePackageResourceManagerFile();
	}

	if (FIoDispatcher::IsInitialized())
	{
		TSharedRef<IIoDispatcherBackend> Backend = UE::MakePackageResourceIoDispatcherBackend(*GPackageResourceManager);
		FIoDispatcher::Get().Mount(Backend);
	}
}

void IPackageResourceManager::Shutdown()
{
	delete GPackageResourceManager;
	GPackageResourceManager = nullptr;
	FOnClearPackageResourceManager& OnClearManagerDelegate = GetOnClearPackageResourceManagerDelegate();
	OnClearManagerDelegate.Broadcast();
}

bool IPackageResourceManager::DoesPackageExist(const FPackagePath& PackagePath, FPackagePath* OutUpdatedPath)
{
	return DoesPackageExist(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header, OutUpdatedPath);
}

int64 IPackageResourceManager::FileSize(const FPackagePath& PackagePath, FPackagePath* OutUpdatedPath)
{
	return FileSize(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header, OutUpdatedPath);
}

FOpenPackageResult IPackageResourceManager::OpenReadPackage(const FPackagePath& PackagePath, FPackagePath* OutUpdatedPath)
{
	return OpenReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header, OutUpdatedPath);
}

FOpenAsyncPackageResult IPackageResourceManager::OpenAsyncReadPackage(const FPackagePath& PackagePath)
{
	return OpenAsyncReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header);
}

IMappedFileHandle* IPackageResourceManager::OpenMappedHandleToPackage(const FPackagePath& PackagePath, FPackagePath* OutUpdatedPath)
{
	return OpenMappedHandleToPackage(PackagePath, EPackageSegment::Header, OutUpdatedPath);
}

void IPackageResourceManager::FindPackagesRecursive(TArray<FPackagePath>& OutPackages, FStringView PackageMount,
	FStringView FileMount, FStringView RootRelPath, FStringView BasenameWildcard)
{
	TArray<TPair<FPackagePath, EPackageSegment>> PackageSegments;
	FindPackagesRecursive(PackageSegments, PackageMount, FileMount, RootRelPath, BasenameWildcard);
	OutPackages.Reserve(PackageSegments.Num());
	for (const TPair<FPackagePath, EPackageSegment>& PackageSegment : PackageSegments)
	{
		if (PackageSegment.Get<1>() == EPackageSegment::Header)
		{
			OutPackages.Add(PackageSegment.Get<0>());
		}
	}
}

void IPackageResourceManager::IteratePackagesInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
	FPackagePathVisitor Callback)
{
	IteratePackagesInPath(PackageMount, FileMount, RootRelPath,
		[&Callback](const FPackagePath& PackagePath, EPackageSegment Segment) -> bool
		{
			if (Segment != EPackageSegment::Header)
			{
				return true;
			}
			return Callback(PackagePath);
		});
}

void IPackageResourceManager::IteratePackagesInLocalOnlyDirectory(FStringView RootDir, FPackagePathVisitor Callback)
{
	IteratePackagesInLocalOnlyDirectory(RootDir,
		[&Callback](const FPackagePath& PackagePath, EPackageSegment Segment) -> bool
		{
			if (Segment != EPackageSegment::Header)
			{
				return true;
			}
			return Callback(PackagePath);
		});
}

void IPackageResourceManager::IteratePackagesStatInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
	FPackagePathStatVisitor Callback)
{
	IteratePackagesStatInPath(PackageMount, FileMount, RootRelPath,
		[&Callback](const FPackagePath& PackagePath, EPackageSegment Segment, const FFileStatData& StatData) -> bool
		{
			if (Segment != EPackageSegment::Header)
			{
				return true;
			}
			return Callback(PackagePath, StatData);
		});
}

/**
 * Call the callback - with stat data - on all packages in the given local path
 *
 * PackageResourceManagers that do not support LocalOnly paths will return without calling the Callback
 *
 * @param RootDir The local path on disk to search
 * @param Callback The callback called on each package
 * @param bOutSupported If nonnull, receives a true or false value for whether this PackageResourceManager supports
 *        searching LocalOnlyDirectories
 */
void IPackageResourceManager::IteratePackagesStatInLocalOnlyDirectory(FStringView RootDir, FPackagePathStatVisitor Callback)
{
	IteratePackagesStatInLocalOnlyDirectory(RootDir,
		[&Callback](const FPackagePath& PackagePath, EPackageSegment Segment, const FFileStatData& StatData) -> bool
		{
			if (Segment != EPackageSegment::Header)
			{
				return true;
			}
			return Callback(PackagePath, StatData);
		});
}

#if WITH_EDITOR
TMap<FName, TPair<TSharedPtr<FPreloadableArchive>, FOpenPackageResult>> IPackageResourceManager::PreloadedPaths;
FCriticalSection IPackageResourceManager::PreloadedPathsLock;

bool IPackageResourceManager::TryRegisterPreloadableArchive(const FPackagePath& PackagePath,
	const TSharedPtr<FPreloadableArchive>& PreloadableArchive, const FOpenPackageResult& OpenResult)
{
	const FName PackageName = PackagePath.GetPackageFName();
	if (PackageName.IsNone())
	{
		return false;
	}
	check(PreloadableArchive.IsValid());
	FScopeLock ScopeLock(&PreloadedPathsLock);
	TPair<TSharedPtr<FPreloadableArchive>, FOpenPackageResult>& ExistingData = PreloadedPaths.FindOrAdd(PackageName);
	TSharedPtr<FPreloadableArchive>& ExistingArchive = ExistingData.Get<0>();
	FOpenPackageResult& ExistingResult = ExistingData.Get<1>();
	if (ExistingArchive)
	{
		if (ExistingArchive.Get() == PreloadableArchive.Get())
		{
			check(ExistingResult.Format == OpenResult.Format && ExistingResult.bNeedsEngineVersionChecks == OpenResult.bNeedsEngineVersionChecks);
			return true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		ExistingArchive = PreloadableArchive;
		ExistingResult.CopyMetaData(OpenResult);
		return true;
	}
}

bool IPackageResourceManager::TryTakePreloadableArchive(const FPackagePath& PackagePath, FOpenPackageResult& OutResult)
{
	OutResult.Format = EPackageFormat::Binary;
	OutResult.bNeedsEngineVersionChecks = true;
	OutResult.Archive = nullptr;
	const FName PackageName = PackagePath.GetPackageFName();
	if (PackageName.IsNone())
	{
		return false;
	}
	FScopeLock ScopeLock(&PreloadedPathsLock);
	if (PreloadedPaths.Num() == 0)
	{
		return false;
	}

	TPair<TSharedPtr<FPreloadableArchive>, FOpenPackageResult> ExistingData;
	if (!PreloadedPaths.RemoveAndCopyValue(PackageName, ExistingData))
	{
		return false;
	}
	TSharedPtr<FPreloadableArchive>& PreloadableArchive = ExistingData.Get<0>();
	FOpenPackageResult& OpenResult = ExistingData.Get<1>();
	if (!PreloadableArchive || !PreloadableArchive->IsInitialized())
	{
		// Someone has called Close on the Archive already.
		return false;
	}

	OutResult.CopyMetaData(OpenResult);
	OutResult.Archive = TUniquePtr<FArchive>(PreloadableArchive->DetachLowerLevel());
	// If DetachedLowerLevel returns non-null, the PreloadableArchive is in PreloadHandle mode;
	// it is not preloading bytes, but instead is only providing a pre-opened (and possibly primed) sync handle
	if (!OutResult.Archive)
	{
		if (!PreloadableArchive->HasValidData())
		{
			// It should be impossible for the PreloadableArchive to not have valid data, because the code that
			// registers the Archives should not register it in those cases. But we have seen this case come up
			// once. So check for invalid data and log a warning, and avoid a followup serialization error by returning 
			// false.
			UE_LOG(LogPackageResourceManager, Warning,
				TEXT("Registered PreloadableArchive for package %s does not have valid data; this should be impossible. We will ignore the registered archive and load without preloading.")
				TEXT("\n\tIsInitialized = %s, Size = %" INT64_FMT "."),
				*PackageName.ToString(), *LexToString(PreloadableArchive->IsInitialized()), PreloadableArchive->TotalSize());
			OutResult = FOpenPackageResult();
			return false;
		}

		// Otherwise the archive is in PreloadBytes mode, and we need to return a proxy to it.
		OutResult.Archive = TUniquePtr<FArchive>(new FPreloadableArchiveProxy(PreloadableArchive));
	}
	return true;
}

bool IPackageResourceManager::UnRegisterPreloadableArchive(const FPackagePath& PackagePath)
{
	FName PackageName = PackagePath.GetPackageFName();
	if (PackageName.IsNone())
	{
		return false;
	}
	FScopeLock ScopeLock(&PreloadedPathsLock);
	int32 NumRemoved = PreloadedPaths.Remove(PackageName);
	return NumRemoved > 0;
}
#endif

EPackageFormat ExtensionToPackageFormat(EPackageExtension Extension)
{
	switch (Extension)
	{
	case EPackageExtension::TextAsset: return EPackageFormat::Text;
	case EPackageExtension::TextMap: return EPackageFormat::Text;
	default: return EPackageFormat::Binary;
	}
}

FOpenAsyncPackageResult::FOpenAsyncPackageResult(TUniquePtr<IAsyncReadFileHandle>&& InHandle, EPackageFormat InFormat, bool bInNeedsEngineVersionChecks)
	: Handle(MoveTemp(InHandle))
	, Format(InFormat)
	, bNeedsEngineVersionChecks(bInNeedsEngineVersionChecks)
{
}

FOpenAsyncPackageResult::~FOpenAsyncPackageResult()
{
	// Defined in CPP so we can forward declare IAsyncReadFileHandle
}

#if WITH_EDITOR
EEditorDomainEnabled IsEditorDomainEnabled()
{
	static EEditorDomainEnabled EnabledValue = []()
	{
		if (!GIsEditor || (IsRunningCommandlet() && !IsRunningCookCommandlet()))
		{
			return EEditorDomainEnabled::Disabled;
		}

		if (FParse::Param(FCommandLine::Get(), TEXT("NoEditorDomain")))
		{
			return EEditorDomainEnabled::Utilities;
		}

		if (FParse::Param(FCommandLine::Get(), TEXT("EditorDomain")))
		{
			return EEditorDomainEnabled::PackageResourceManager;
		}

		bool bEnabledByConfig = true;
		GConfig->GetBool(TEXT("EditorDomain"), TEXT("EditorDomainEnabled"), bEnabledByConfig, GEditorIni);
		if (GConfig->GetBool(TEXT("CookSettings"), TEXT("EditorDomainEnabled"), bEnabledByConfig, GEditorIni))
		{
			UE_LOGF(LogPackageResourceManager, Error,
				"Editor.ini:[CookSettings]:EditorDomainEnabled is deprecated, use Editor.ini:[EditorDomain]:EditorDomainEnabled instead.");
		}
		return bEnabledByConfig ? EEditorDomainEnabled::PackageResourceManager : EEditorDomainEnabled::Utilities;
	}();
	return EnabledValue;
}
#endif
