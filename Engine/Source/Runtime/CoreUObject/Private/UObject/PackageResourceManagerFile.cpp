// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PackageResourceManagerFile.h"

#include "CoreMinimal.h"

#include "Async/AsyncFileHandleNull.h"
#include "Async/MappedFileHandle.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/PackageSegment.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "UObject/PackageResourceManager.h"

/**
 * A PackageResourceManager that reads package payloads from the content directories on disk
 */
class FPackageResourceManagerFile final
	: public IPackageResourceManager
{
public:
	FPackageResourceManagerFile();
	virtual ~FPackageResourceManagerFile() = default;

	virtual bool SupportsLocalOnlyPaths() override
	{
		return true;
	}

	virtual bool SupportsPackageOnlyPaths() override
	{
		return false;
	}

	virtual bool DoesPackageExist(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) override;
	virtual int64 FileSize(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) override;
	virtual FOpenPackageResult OpenReadPackage(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) override;
	virtual FOpenAsyncPackageResult OpenAsyncReadPackage(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment) override;
	virtual IMappedFileHandle* OpenMappedHandleToPackage(const FPackagePath& PackagePath, EPackageSegment PackageSegment,
		FPackagePath* OutUpdatedPath = nullptr) override;
	virtual bool TryMatchCaseOnDisk(const FPackagePath& PackagePath, FPackagePath* OutPackagePath = nullptr) override;

	virtual TUniquePtr<FArchive> OpenReadExternalResource(EPackageExternalResource ResourceType, FStringView Identifier) override;
	virtual bool DoesExternalResourceExist(EPackageExternalResource ResourceType, FStringView Identifier) override;
	virtual FOpenAsyncPackageResult OpenAsyncReadExternalResource(
		EPackageExternalResource ResourceType, FStringView Identifier) override;

	virtual void FindPackagesRecursive(TArray<TPair<FPackagePath, EPackageSegment>>& OutPackages,
		FStringView PackageMount, FStringView FileMount, FStringView RootRelPath, FStringView BasenameWildcard) override;
	virtual void IteratePackagesInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
		FPackageSegmentVisitor Callback) override;
	virtual void IteratePackagesInLocalOnlyDirectory(FStringView RootDir, FPackageSegmentVisitor Callback) override;
	virtual void IteratePackagesStatInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
		FPackageSegmentStatVisitor Callback) override;
	virtual void IteratePackagesStatInLocalOnlyDirectory(FStringView RootDir,
		FPackageSegmentStatVisitor Callback) override;

protected:
	/**
	 * Enumerate the possible extensions for the given PackagePath and segment and call Callback on the full LocalPath.
	 * Callback returns true if the given fullpath should be used, in which case iteration stops and OutUpdatedPath is
	 * assigned the chosen extension.
	 * PackagePaths without a LocalPath (PackageNameOnly PackagePaths) will result in no calls made to the Callback.
	 * @param PackagePath The PackagePath used to construct the LocalPath.
	 * @param CookedIndex The cooked index used by the bulkdata, other segment types can just pass in default
	 * @param PackageSegment The PackageSegment used to construct the LocalPath.
	 * @param OutUpdatedPath If non-null and a LocalPath is found for which Callback returns true, will be set equal
	 *        to a copy of PackagePath, and if a header extension was found, will have the header extension set
	 * @param Callback bool (*Callback)(const TCHAR* FullLocalPath, EPackageExtension Extension), the callback used to
	 *        check whether an extension is the correct one (because the FullLocalPath exists)
	 */
	template <typename CallbackType>
	void IteratePossibleFiles(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath, const CallbackType& Callback);

	/**
	 * Base class of classes used in IteratePackagesInPath functions,
	 * with some functionality to convert filename to FPackagePath
	 */
	class FDirectoryVisitorBaseMounted
	{
	public:
		FStringView PackageMount;
		FString FileMount;
		FString RootDir;

		explicit FDirectoryVisitorBaseMounted(FStringView InPackageMount, FStringView InFileMount,
			FStringView InRootRelPath);
		bool TryConvertToPackageVisit(const TCHAR* FilenameOrDirectory, bool bIsDirectory,
			FPackagePath& OutPackagePath, EPackageSegment& OutPackageSegment);
	};

	/**
	 * Base class of classes used in IteratePackagesInLocalOnlyDirectory functions,
	 * with some functionality to convert filename to FPackagePath
	 */
	class FDirectoryVisitorBaseLocalOnly
	{
	public:
		FString RootDir;

		explicit FDirectoryVisitorBaseLocalOnly(FStringView InRootPath);
		bool TryConvertToPackageVisit(const TCHAR* FilenameOrDirectory, bool bIsDirectory,
			FPackagePath& OutPackagePath, EPackageSegment& OutPackageSegment);
	};
};

IPackageResourceManager* MakePackageResourceManagerFile()
{
	return new FPackageResourceManagerFile();
}


FPackageResourceManagerFile::FPackageResourceManagerFile()
{
}

template <typename CallbackType>
void FPackageResourceManagerFile::IteratePossibleFiles(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
	EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath, const CallbackType& Callback)
{
	TStringBuilder<256> FilePath;
	FilePath << PackagePath.GetLocalBaseFilenameWithPath();
	int32 BaseNameLen = FilePath.Len();
	if (BaseNameLen == 0)
	{
		return;
	}

	if (!CookedIndex.IsDefault())
	{
		FilePath << CookedIndex.GetAsExtension();
		BaseNameLen = FilePath.Len();
	}

	for (EPackageExtension Extension : PackagePath.GetPossibleExtensions(PackageSegment))
	{
		FilePath.RemoveSuffix(FilePath.Len() - BaseNameLen);
		FilePath << ((Extension != EPackageExtension::Custom) ?
			FStringView(LexToString(Extension)) : PackagePath.GetExtensionString(EPackageSegment::Header));
		if (Callback(FilePath.ToString(), Extension))
		{
			if (OutUpdatedPath)
			{
				*OutUpdatedPath = PackagePath;
				if (PackageSegment == EPackageSegment::Header &&
					PackagePath.GetHeaderExtension() == EPackageExtension::Unspecified)
				{
					check(Extension != EPackageExtension::Custom);
					OutUpdatedPath->SetHeaderExtension(Extension, FStringView());
				}
			}
			break;
		}
	}
}

bool FPackageResourceManagerFile::DoesPackageExist(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
	EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath)
{
	bool bResult = false;
	IFileManager* FileManager = &IFileManager::Get();
	IteratePossibleFiles(PackagePath, CookedIndex, PackageSegment, OutUpdatedPath,
		[FileManager, &bResult](const TCHAR* Filename, EPackageExtension Extension)
		{
			if (FileManager->FileExists(Filename))
			{
				bResult = true;
				return true;
			}
			return false;
		});
	return bResult;
}

int64 FPackageResourceManagerFile::FileSize(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
	EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath)
{
	int64 Result = INDEX_NONE;

	IFileManager* FileManager = &IFileManager::Get();
	IteratePossibleFiles(PackagePath, CookedIndex, PackageSegment, OutUpdatedPath,
		[FileManager, &Result](const TCHAR* Filename, EPackageExtension Extension)
		{
			Result = FileManager->FileSize(Filename);
			if (Result != INDEX_NONE)
			{
				return true;
			}
			return false;
		});
	return Result;
}


FOpenPackageResult FPackageResourceManagerFile::OpenReadPackage(const FPackagePath& PackagePath,
	FBulkDataCookedIndex CookedIndex, EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath)
{
	FOpenPackageResult Result{ nullptr, EPackageFormat::Binary, true /* bNeedsEngineVersionChecks */};

	IFileManager* FileManager = &IFileManager::Get();
	IteratePossibleFiles(PackagePath, CookedIndex, PackageSegment, OutUpdatedPath,
		[FileManager, &Result](const TCHAR* Filename, EPackageExtension Extension)
		{
#if !WITH_TEXT_ARCHIVE_SUPPORT
			if (Extension == EPackageExtension::TextAsset || Extension == EPackageExtension::TextMap)
			{
				return false;
			}
#endif
			Result.Archive = TUniquePtr<FArchive>(FileManager->CreateFileReader(Filename));
			if (Result.Archive)
			{
#if WITH_TEXT_ARCHIVE_SUPPORT
				if (Extension == EPackageExtension::TextAsset || Extension == EPackageExtension::TextMap)
				{
					Result.Format = EPackageFormat::Text;
				}
				else
#endif
				{
					Result.Format = EPackageFormat::Binary;
				}
				return true;
			}
			return false;
		});
	return Result;
}

FOpenAsyncPackageResult FPackageResourceManagerFile::OpenAsyncReadPackage(const FPackagePath& PackagePath,
	FBulkDataCookedIndex CookedIndex, EPackageSegment PackageSegment)
{
	FOpenAsyncPackageResult Result { nullptr, EPackageFormat::Binary, true /* bNeedsEngineVersionChecks */ };

	EPackageExtension Extension = EPackageExtension::Unspecified;
	TConstArrayView<EPackageExtension> Extensions = PackagePath.GetPossibleExtensions(PackageSegment);
	if (Extensions.Num() == 1)
	{
		Extension = Extensions[0];
	}
	else if (Extensions.Num() > 1)
	{
		FPackagePath UpdatedPackagePath;
		if (DoesPackageExist(PackagePath, CookedIndex, PackageSegment, &UpdatedPackagePath))
		{
			FStringView CustomExtension;
			Extension = UpdatedPackagePath.GetExtension(PackageSegment, CustomExtension);
			check(Extension != EPackageExtension::Unspecified);
		}
	}
	if (Extension != EPackageExtension::Unspecified)
	{
		TStringBuilder<256> FilePath;
		FilePath << PackagePath.GetLocalBaseFilenameWithPath();
		if (FilePath.Len() > 0)
		{
			if (!CookedIndex.IsDefault())
			{
				FilePath << CookedIndex.GetAsExtension();
			}

			FilePath << ((Extension != EPackageExtension::Custom) ?
				FStringView(LexToString(Extension)) : PackagePath.GetExtensionString(EPackageSegment::Header));
			Result.Handle.Reset(FPlatformFileManager::Get().GetPlatformFile().OpenAsyncRead(FilePath.ToString()));
			check(Result.Handle); // OpenAsyncRead guarantees a non-null return value
			Result.Format = ExtensionToPackageFormat(Extension);
		}
	}
	if (!Result.Handle)
	{
		Result.Handle.Reset(new FAsyncReadFileHandleNull());
	}
	return Result;
}

IMappedFileHandle* FPackageResourceManagerFile::OpenMappedHandleToPackage(const FPackagePath& PackagePath,
	EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath)
{
	IMappedFileHandle* Result = nullptr;

	IPlatformFile* PlatformFile = &FPlatformFileManager::Get().GetPlatformFile();
	IteratePossibleFiles(PackagePath, FBulkDataCookedIndex::Default, PackageSegment, OutUpdatedPath,
		[PlatformFile, &Result](const TCHAR* Filename, EPackageExtension Extension)
		{
			FOpenMappedResult Value = PlatformFile->OpenMappedEx(Filename);
			if (Value.HasError())
			{
				return false;
			}
			Result = Value.StealValue().Release();
			return true;
		});
	return Result;
}

bool FPackageResourceManagerFile::TryMatchCaseOnDisk(const FPackagePath& PackagePath, FPackagePath* OutPackagePath)
{
	IPlatformFile* PlatformFile = &FPlatformFileManager::Get().GetPlatformFile();
	FString FilenameOnDisk;
	EPackageExtension ExtensionOnDisk;
	IteratePossibleFiles(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header, nullptr,
		[PlatformFile, &FilenameOnDisk, &ExtensionOnDisk](const TCHAR* Filename, EPackageExtension Extension)
		{
			// TODO: Optimize this function to only hit the disk once by creating a
			// IPlatformFile::GetFilenameOnDisk function that also returns whether the file exists
			if (PlatformFile->FileExists(Filename))
			{
				FilenameOnDisk = PlatformFile->GetFilenameOnDisk(Filename);
				ExtensionOnDisk = Extension;
				return true;
			}
			return false;
		});

	if (!FilenameOnDisk.IsEmpty())
	{
		if (OutPackagePath)
		{
			if (!FPackagePath::TryMatchCase(PackagePath, FilenameOnDisk, *OutPackagePath))
			{
				UE_LOGF(LogPackageName, Verbose,
					"TryMatchCaseOnDisk: Unexpected non-matching LocalPath \"%ls\" found when searching for PackagePath \"%ls\". Case will not be normalized.",
					*FilenameOnDisk, *PackagePath.GetLocalFullPath());
			}
			OutPackagePath->SetHeaderExtension(ExtensionOnDisk, PackagePath.GetCustomExtension());
		}
		return true;
	}
	else
	{
		return false;
	}
}

TUniquePtr<FArchive> FPackageResourceManagerFile::OpenReadExternalResource(EPackageExternalResource ResourceType, FStringView Identifier)
{
	switch (ResourceType)
	{
	case EPackageExternalResource::WorkspaceDomainFile:
	{
		FPackagePath PackagePath;
		if (!FPackagePath::TryFromPackageName(Identifier, PackagePath))
		{
			return TUniquePtr<FArchive>();
		}
		return OpenReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header).Archive;
	}
	default:
		checkNoEntry();
		return TUniquePtr<FArchive>();
	}
}

bool FPackageResourceManagerFile::DoesExternalResourceExist(EPackageExternalResource ResourceType, FStringView Identifier)
{
	switch (ResourceType)
	{
	case EPackageExternalResource::WorkspaceDomainFile:
	{
		FPackagePath PackagePath;
		if (!FPackagePath::TryFromPackageName(Identifier, PackagePath))
		{
			return false;
		}
		return DoesPackageExist(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header);
	}
	default:
		checkNoEntry();
		return false;
	}
}

FOpenAsyncPackageResult FPackageResourceManagerFile::OpenAsyncReadExternalResource(EPackageExternalResource ResourceType,
	FStringView Identifier)
{
	switch (ResourceType)
	{
	case EPackageExternalResource::WorkspaceDomainFile:
	{
		FPackagePath PackagePath;
		if (!FPackagePath::TryFromPackageName(Identifier, PackagePath))
		{
			return FOpenAsyncPackageResult{ TUniquePtr<IAsyncReadFileHandle>(new FAsyncReadFileHandleNull()), EPackageFormat::Binary };
		}
		return OpenAsyncReadPackage(PackagePath, FBulkDataCookedIndex::Default, EPackageSegment::Header);
	}
	default:
		checkNoEntry();
		return FOpenAsyncPackageResult{ TUniquePtr<IAsyncReadFileHandle>(new FAsyncReadFileHandleNull()), EPackageFormat::Binary };
	}
}


void FPackageResourceManagerFile::FindPackagesRecursive(TArray<TPair<FPackagePath, EPackageSegment>>& OutPackages,
	FStringView PackageMount, FStringView FileMount, FStringView RootRelPath, FStringView BasenameWildcard)
{
	check(PackageMount.EndsWith(TEXT("/")));
	check(FileMount.EndsWith(TEXT("/")));
	FString FileMountAbsPath = FPaths::ConvertRelativePathToFull(FString(FileMount));
	if (!FileMountAbsPath.EndsWith(TEXT("/")))
	{
		FileMountAbsPath = FileMountAbsPath + TEXT("/");
	}
	TStringBuilder<256> RootFileAbsPath;
	RootFileAbsPath << FileMountAbsPath << RootRelPath;

	TStringBuilder<256> BasenameWildcardNullTerminated;
	BasenameWildcardNullTerminated << BasenameWildcard;

	TArray<FString> FoundFilenames;
	IFileManager::Get().FindFilesRecursive(FoundFilenames, RootFileAbsPath.ToString(),
		BasenameWildcardNullTerminated.ToString(), true /* Files */, false /* Directories */);
	OutPackages.Reserve(OutPackages.Num() + FoundFilenames.Num());
	TStringBuilder<256> ResultFile;
	FString BufferPath;
	for (const FString& Filename : FoundFilenames)
	{
		FStringView RelPath;
		if (Filename.StartsWith(FileMountAbsPath, ESearchCase::IgnoreCase))
		{
			RelPath = FStringView(Filename);
		}
		else
		{
			// Check whether the reason it doesn't start with FileMountAbsPath is because it is not normalized or is e.g. a relative path
			// ConvertRelativePathToFull will normalize it in addition to converting it to an absolute path
			BufferPath = FPaths::ConvertRelativePathToFull(Filename);
			if (BufferPath.StartsWith(FileMountAbsPath, ESearchCase::IgnoreCase))
			{
				RelPath = FStringView(BufferPath);
			}
			else
			{
				UE_LOGF(LogPackageResourceManager, Warning,
					"FindPackagesRecursive: Filename \"%ls\" returned from FindFilesRecursive does not start with RootPath \"%ls\". Ignoring it.",
					*Filename, *FileMountAbsPath);
				continue;
			}
		}
		RelPath.RightChopInline(FileMountAbsPath.Len());
		int32 ExtensionStart;
		EPackageExtension Extension = FPackagePath::ParseExtension(RelPath, &ExtensionStart);
		if (Extension == EPackageExtension::Custom || // Files with unrecognized extensions on disk are not returned for IteratePackages
			Extension == EPackageExtension::Unspecified // An empty extension is not a valid package extension
			)
		{
			continue;
		}
		RelPath = RelPath.Left(ExtensionStart);
		EPackageSegment PackageSegment = ExtensionToSegment(Extension);
		FPackagePath OutPackagePath = FPackagePath::FromMountedComponents(PackageMount, FileMount, RelPath,
			PackageSegment == EPackageSegment::Header ? Extension : EPackageExtension::Unspecified);
		OutPackages.Add(TPair<FPackagePath, EPackageSegment>(MoveTemp(OutPackagePath), PackageSegment));
	}
}

FPackageResourceManagerFile::FDirectoryVisitorBaseMounted::FDirectoryVisitorBaseMounted(FStringView InPackageMount,
	FStringView InFileMount, FStringView InRootRelPath)
{
	PackageMount = InPackageMount;
	FileMount = FPaths::ConvertRelativePathToFull(FString(InFileMount));
	RootDir = FileMount + FString(InRootRelPath);
}

bool FPackageResourceManagerFile::FDirectoryVisitorBaseMounted::TryConvertToPackageVisit(const TCHAR* FilenameOrDirectory,
	bool bIsDirectory, FPackagePath& OutPackagePath, EPackageSegment& OutPackageSegment)
{
	if (bIsDirectory)
	{
		return false;
	}

	FStringView Filename(FilenameOrDirectory);
	FString BufferPath;
	if (!Filename.StartsWith(FileMount, ESearchCase::IgnoreCase))
	{
		// Check whether the reason it doesn't start with the text of the FileMount is because it is not normalized and is e.g. a relative path
		// ConvertRelativePathToFull will normalize it in addition to converting it to an absolute path
		BufferPath = Filename;
		BufferPath = FPaths::ConvertRelativePathToFull(MoveTemp(BufferPath));
		Filename = BufferPath;
		if (!Filename.StartsWith(FileMount, ESearchCase::IgnoreCase))
		{
			UE_LOGF(LogPackageResourceManager, Warning,
				"FDirectoryVisitorBaseMounted: FileManager IterateDirectoryRecursively(\"%ls\") returned file \"%ls\" that is not a subpath of the root \"%ls\".",
				*RootDir, FilenameOrDirectory, *FileMount);
			return false;
		}
	}
	FStringView RelPath(Filename.RightChop(FileMount.Len()));
	int32 ExtensionStart;
	EPackageExtension Extension = FPackagePath::ParseExtension(RelPath, &ExtensionStart);
	if (Extension == EPackageExtension::Custom || // Files with unrecognized extensions on disk are not returned for IteratePackages
		Extension == EPackageExtension::Unspecified // Empty extension is not a valid package extension
		)
	{
		return false;
	}
	RelPath = RelPath.Left(ExtensionStart);
	OutPackageSegment = ExtensionToSegment(Extension);
	OutPackagePath = FPackagePath::FromMountedComponents(PackageMount, FileMount, RelPath,
		OutPackageSegment == EPackageSegment::Header ? Extension : EPackageExtension::Unspecified);
	return true;
}

FPackageResourceManagerFile::FDirectoryVisitorBaseLocalOnly::FDirectoryVisitorBaseLocalOnly(FStringView InRootPath)
{
	RootDir = FPaths::ConvertRelativePathToFull(FString(InRootPath));
}

bool FPackageResourceManagerFile::FDirectoryVisitorBaseLocalOnly::TryConvertToPackageVisit(const TCHAR* FilenameOrDirectory,
	bool bIsDirectory, FPackagePath& OutPackagePath, EPackageSegment& OutPackageSegment)
{
	if (bIsDirectory)
	{
		return false;
	}
	FStringView Filename(FilenameOrDirectory);
	int32 ExtensionStart;
	EPackageExtension Extension = FPackagePath::ParseExtension(Filename, &ExtensionStart);
	if (Extension == EPackageExtension::Custom || // Files with unrecognized extensions on disk are not returned for IteratePackages
		Extension == EPackageExtension::Unspecified // Empty extension is not a valid package extension
		)
	{
		return false;
	}
	OutPackageSegment = ExtensionToSegment(Extension);
	OutPackagePath = FPackagePath::FromLocalPath(Filename);
	return true;
}

void FPackageResourceManagerFile::IteratePackagesInPath(FStringView PackageMount, FStringView FileMount,
	FStringView RootRelPath, FPackageSegmentVisitor Callback)
{
	class FPackageVisitor : public IPlatformFile::FDirectoryVisitor, public FDirectoryVisitorBaseMounted
	{
	public:
		FPackageSegmentVisitor Callback;
		explicit FPackageVisitor(FStringView InPackageMount, FStringView InFileMount, FStringView InRootRelPath,
			FPackageSegmentVisitor InCallback)
			: FDirectoryVisitorBaseMounted(InPackageMount, InFileMount, InRootRelPath)
			, Callback(InCallback)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			FPackagePath PackagePath;
			EPackageSegment PackageSegment;
			if (!TryConvertToPackageVisit(FilenameOrDirectory, bIsDirectory, PackagePath, PackageSegment))
			{
				return true;
			}
			return Callback(PackagePath, PackageSegment);
		}
	};

	FPackageVisitor PackageVisitor(PackageMount, FileMount, RootRelPath, Callback);
	IFileManager::Get().IterateDirectoryRecursively(*PackageVisitor.RootDir, PackageVisitor);
}

void FPackageResourceManagerFile::IteratePackagesInLocalOnlyDirectory(FStringView RootDir,
	FPackageSegmentVisitor Callback)
{
	class FPackageVisitor : public IPlatformFile::FDirectoryVisitor, public FDirectoryVisitorBaseLocalOnly
	{
	public:
		FPackageSegmentVisitor Callback;
		explicit FPackageVisitor(FStringView InRootDir, FPackageSegmentVisitor InCallback)
			: FDirectoryVisitorBaseLocalOnly(InRootDir)
			, Callback(InCallback)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			FPackagePath PackagePath;
			EPackageSegment PackageSegment;
			if (!TryConvertToPackageVisit(FilenameOrDirectory, bIsDirectory, PackagePath, PackageSegment))
			{
				return true;
			}
			return Callback(PackagePath, PackageSegment);
		}
	};

	FPackageVisitor PackageVisitor(RootDir, Callback);
	IFileManager::Get().IterateDirectoryRecursively(*PackageVisitor.RootDir, PackageVisitor);
}

void FPackageResourceManagerFile::IteratePackagesStatInPath(FStringView PackageMount, FStringView FileMount,
	FStringView RootRelPath, FPackageSegmentStatVisitor Callback)
{
	class FPackageVisitor : public IPlatformFile::FDirectoryStatVisitor, public FDirectoryVisitorBaseMounted
	{
	public:
		FPackageSegmentStatVisitor Callback;
		explicit FPackageVisitor(FStringView InPackageMount, FStringView InFileMount, FStringView InRootRelPath,
			FPackageSegmentStatVisitor InCallback)
			: FDirectoryVisitorBaseMounted(InPackageMount, InFileMount, InRootRelPath)
			, Callback(InCallback)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData) override
		{
			FPackagePath PackagePath;
			EPackageSegment PackageSegment;
			if (!TryConvertToPackageVisit(FilenameOrDirectory, StatData.bIsDirectory, PackagePath, PackageSegment))
			{
				return true;
			}
			return Callback(PackagePath, PackageSegment, StatData);
		}
	};

	FPackageVisitor PackageVisitor(PackageMount, FileMount, RootRelPath, Callback);
	IFileManager::Get().IterateDirectoryStatRecursively(*PackageVisitor.RootDir, PackageVisitor);
}

void FPackageResourceManagerFile::IteratePackagesStatInLocalOnlyDirectory(FStringView RootDir,
	FPackageSegmentStatVisitor Callback)
{
	class FPackageVisitor : public IPlatformFile::FDirectoryStatVisitor, public FDirectoryVisitorBaseLocalOnly
	{
	public:
		FPackageSegmentStatVisitor Callback;
		explicit FPackageVisitor(FStringView InRootDir, const FPackageSegmentStatVisitor& InCallback)
			: FDirectoryVisitorBaseLocalOnly(InRootDir)
			, Callback(InCallback)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData) override
		{
			FPackagePath PackagePath;
			EPackageSegment PackageSegment;
			if (!TryConvertToPackageVisit(FilenameOrDirectory, StatData.bIsDirectory, PackagePath, PackageSegment))
			{
				return true;
			}
			return Callback(PackagePath, PackageSegment, StatData);
		}
	};

	FPackageVisitor PackageVisitor(RootDir, Callback);
	IFileManager::Get().IterateDirectoryStatRecursively(*PackageVisitor.RootDir, PackageVisitor);
}
