// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PackageTrailer.h"

#include "Algo/Count.h"
#include "Algo/RemoveIf.h"
#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "Math/GuardedInt.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackagePath.h"
#include "Misc/PackageSegment.h"
#include "UObject/Package.h"
#include "UObject/PackageResourceManager.h"

namespace UE
{

/** The version number for the FPackageTrailer format */
enum class EPackageTrailerVersion : uint32
{
	// The original trailer format when it was first added
	INITIAL = 0,
	// Access mode is now per payload and found in FLookupTableEntry 
	ACCESS_PER_PAYLOAD = 1,
	// Added EPayloadAccessMode to FLookupTableEntry
	PAYLOAD_FLAGS = 2,

	// -----<new versions can be added before this line>-------------------------------------------------
	// - this needs to be the last line (see note below)
	AUTOMATIC_VERSION_PLUS_ONE,
	AUTOMATIC_VERSION = AUTOMATIC_VERSION_PLUS_ONE - 1
};

// These asserts are here to make sure that any changes to the size of disk constants are intentional.
// If the change was intentional then just update the assert.
static_assert(FPackageTrailer::FHeader::StaticHeaderSizeOnDisk == 28, "FPackageTrailer::FHeader size has been changed, if this was intentional then update this assert");
static_assert(Private::FLookupTableEntry::SizeOnDisk == 49, "FLookupTableEntry size has been changed, if this was intentional then update this assert");
static_assert(FPackageTrailer::FFooter::SizeOnDisk == 20, "FPackageTrailer::FFooter size has been changed, if this was intentional then update this assert");

namespace
{

/** Utility for recording failed package open reasons */
void LogPackageOpenFailureMessage(const FString& DebugName)
{
	// TODO: Check the various error paths here again!
	const uint32 SystemError = FPlatformMisc::GetLastError();
	// If we have a system error we can give a more informative error message but don't output it if the error is zero as 
	// this can lead to very confusing error messages.
	if (SystemError != 0)
	{
		TCHAR SystemErrorMsg[2048] = { 0 };
		FPlatformMisc::GetSystemErrorMessage(SystemErrorMsg, UE_ARRAY_COUNT(SystemErrorMsg), SystemError);
		UE_LOGF(LogSerialization, Error, "Could not open the file '%ls' for reading due to system error: '%ls' (%d))", *DebugName, SystemErrorMsg, SystemError);
	}
	else
	{
		UE_LOGF(LogSerialization, Error, "Could not open (%ls) to read FPackageTrailer with an unknown error", *DebugName);
	}
}

/* Utility for finding a lookup table entry for reading from inside of a FHeader*/
const Private::FLookupTableEntry* FindEntryInHeader(const FPackageTrailer::FHeader& Header, const FIoHash& Id)
{
	const Private::FLookupTableEntry* Entry = Header.PayloadLookupTable.FindByPredicate([&Id](const Private::FLookupTableEntry& Entry)->bool
		{
			return Entry.Identifier == Id;
		});

	return Entry;
}

/* Utility for finding a lookup table entry for writing from inside of a FHeader */
Private::FLookupTableEntry* FindEntryInHeader(FPackageTrailer::FHeader& Header, const FIoHash& Id)
{
	Private::FLookupTableEntry* Entry = Header.PayloadLookupTable.FindByPredicate([&Id](const Private::FLookupTableEntry& Entry)->bool
		{
			return Entry.Identifier == Id;
		});

	return Entry;
}

/** 
 * Utility to check if a FLookupTableEntry seems valid for the trailer.
 * @param	Entry					The entry we are trying to validate
 * @param	PayloadDataEndPoint		The end point of the payloads stored locally. Anything
 *									exceeding this point is not valud.
 * 
 * @return							True if it looks like the entry could exist in the 
  *									current trailer, false if 
 *									there are obvious problems.
 */
bool IsEntryValid(const Private::FLookupTableEntry& Entry, uint64 PayloadDataEndPoint)
{
	// If the entry is virtualized then we should assume that it is valid.
	if (Entry.IsVirtualized())
	{
		return true;
	}

	// If not virtualized then the offset should not be negative
	if (Entry.OffsetInFile < 0)
	{
		return false;
	}

	// PayloadDataEndPoint only applies if the entry is for a locally stored payload
	if (Entry.IsLocal())
	{
		// A locally stored payload must end before PayloadDataEndPoint
		if ((uint64)Entry.OffsetInFile + Entry.CompressedSize > PayloadDataEndPoint)
		{
			return false;
		}
	}

	return true;
}

} //namespace

namespace Private
{

FLookupTableEntry::FLookupTableEntry(const FIoHash& InIdentifier, uint64 InRawSize)
	: Identifier(InIdentifier)
	, RawSize(InRawSize)
{

}

void FLookupTableEntry::Serialize(FArchive& Ar, EPackageTrailerVersion PackageTrailerVersion)
{
	Ar << Identifier;
	Ar << OffsetInFile;
	Ar << CompressedSize;
	Ar << RawSize;

	if (Ar.IsSaving() || PackageTrailerVersion >= EPackageTrailerVersion::PAYLOAD_FLAGS)
	{
		Ar << Flags;
		Ar << FilterFlags;
	}

	if (Ar.IsSaving() || PackageTrailerVersion >= EPackageTrailerVersion::ACCESS_PER_PAYLOAD)
	{
		Ar << AccessMode;
	}
}

} // namespace Private

FPackageTrailerBuilder FPackageTrailerBuilder::CreateFromTrailer(const FPackageTrailer& Trailer, FArchive& Ar, FString DebugContext)
{
	FPackageTrailerBuilder Builder(MoveTemp(DebugContext));

	for (const Private::FLookupTableEntry& Entry : Trailer.Header.PayloadLookupTable)
	{
		checkf(!Entry.Identifier.IsZero(), TEXT("PackageTrailer for package should not contain invalid FIoHash entry. Package '%s'"), *Builder.GetDebugContext());

		switch (Entry.AccessMode)
		{
			case EPayloadAccessMode::Local:
			{
				FCompressedBuffer Payload = Trailer.LoadLocalPayload(Entry.Identifier, Ar);
				Builder.LocalEntries.Add(Entry.Identifier, LocalEntry(MoveTemp(Payload), Entry.FilterFlags));
			}
			break;

			case EPayloadAccessMode::Referenced:
			{
				Builder.ReferencedEntries.Add(Entry.Identifier, ReferencedEntry(Entry.OffsetInFile, Entry.CompressedSize, Entry.RawSize));
			}
			break;

			case EPayloadAccessMode::Virtualized:
			{
				Builder.VirtualizedEntries.Add(Entry.Identifier, VirtualizedEntry(Entry.RawSize));
			}
			break;

			default:
			{
				checkNoEntry();
			}
		}
	}

	return Builder;
}

TUniquePtr<UE::FPackageTrailerBuilder> FPackageTrailerBuilder::CreateReferenceToTrailer(const class FPackageTrailer& Trailer, FString DebugContext)
{
	TUniquePtr<UE::FPackageTrailerBuilder> Builder = MakeUnique<UE::FPackageTrailerBuilder>(MoveTemp(DebugContext));

	for (const Private::FLookupTableEntry& Entry : Trailer.Header.PayloadLookupTable)
	{
		checkf(!Entry.Identifier.IsZero(), TEXT("PackageTrailer for package should not contain invalid FIoHash entry. Package '%s'"), *Builder->GetDebugContext());

		switch (Entry.AccessMode)
		{
			case EPayloadAccessMode::Local:
			{
				const int64 AbsoluteOffset = Trailer.FindPayloadOffsetInFile(Entry.Identifier);
				checkf(AbsoluteOffset != INDEX_NONE, TEXT("PackageTrailer for package should not contain invalid payload offsets. Package '%s'"), *Builder->GetDebugContext());

				Builder->ReferencedEntries.Add(Entry.Identifier, ReferencedEntry(AbsoluteOffset, Entry.CompressedSize, Entry.RawSize));
			}
			break;
			
			case EPayloadAccessMode::Referenced:
			{
				checkf(false, TEXT("Attempting to create a reference to a trailer that already contains reference payload entries. Package '%s'"), *Builder->GetDebugContext());
			}
			break;
			
			case EPayloadAccessMode::Virtualized:
			{
				Builder->VirtualizedEntries.Add(Entry.Identifier, VirtualizedEntry(Entry.RawSize));
			}
			break;

			default:
			{
				checkNoEntry();
			}

		}
	}

	return Builder;
}

FPackageTrailerBuilder::FPackageTrailerBuilder(FString&& InDebugContext)
	: DebugContext(MoveTemp(InDebugContext))
{
}

void FPackageTrailerBuilder::AddPayload(const FIoHash& Identifier, FCompressedBuffer Payload, UE::Virtualization::EPayloadFilterReason FilterFlags, AdditionalDataCallback&& Callback)
{
	Callbacks.Emplace(MoveTemp(Callback));

	AddPayload(Identifier, Payload, FilterFlags);

}

void FPackageTrailerBuilder::AddPayload(const FIoHash& Identifier, FCompressedBuffer Payload, UE::Virtualization::EPayloadFilterReason FilterFlags)
{
	if (!Identifier.IsZero())
	{
		// If the payload already exists and the DisableVirtualization flag has been passed in
		// then we need to make sure that it is applied.
		// TODO: This will have to be done here for every new flag added as we don't know what
		// future flags will want to do in the case of a duplicate entry but we probably need
		// a nicer way to handle this longer term.
		if (LocalEntry* Entry = LocalEntries.Find(Identifier))
		{
			Entry->FilterFlags |= FilterFlags;
		}
		else
		{
			LocalEntries.Add(Identifier, LocalEntry(MoveTemp(Payload), FilterFlags));
		}
	}
}

void FPackageTrailerBuilder::AddVirtualizedPayload(const FIoHash& Identifier, int64 RawSize)
{
	if (!Identifier.IsZero())
	{
		VirtualizedEntries.FindOrAdd(Identifier, VirtualizedEntry(RawSize));
	}
}

bool FPackageTrailerBuilder::UpdatePayloadAsLocal(const FIoHash& Identifier, FCompressedBuffer Payload)
{
	check(ReferencedEntries.IsEmpty());

	if (!Identifier.IsZero() && VirtualizedEntries.Remove(Identifier) > 0)
	{
		check(LocalEntries.Find(Identifier) == nullptr);
		LocalEntries.Add(Identifier, LocalEntry(MoveTemp(Payload), UE::Virtualization::EPayloadFilterReason::None));

		return true;
	}

	return false;
}

bool FPackageTrailerBuilder::BuildAndAppendTrailer(FLinkerSave* Linker, FArchive& DataArchive)
{
	int64 CurrentOffset = DataArchive.Tell();
	return BuildAndAppendTrailer(Linker, DataArchive, CurrentOffset);
}

bool FPackageTrailerBuilder::BuildAndAppendTrailer(FLinkerSave* Linker, FArchive& DataArchive, int64& InOutArchiveOffset)
{
	// Note that we do not serialize containers directly as we want a file format that is 
	// 100% under our control. This will allow people to create external scripts that can
	// parse and manipulate the trailer without needing to worry that we might change how
	// our containers serialize. 
	
	RemoveDuplicateEntries();

	FPackageTrailer Trailer;
	
	Trailer.Header.Tag = FPackageTrailer::FHeader::HeaderTag;
	Trailer.Header.Version = (int32)EPackageTrailerVersion::AUTOMATIC_VERSION;

	Trailer.Header.HeaderLength = CalculatePotentialHeaderSize();
	
	Trailer.Header.PayloadsDataLength = 0;
	Trailer.Header.PayloadLookupTable.Reserve(LocalEntries.Num() + ReferencedEntries.Num() + VirtualizedEntries.Num());

	for (const TPair<FIoHash, LocalEntry>& It : LocalEntries)
	{
		checkf(!It.Key.IsZero(), TEXT("PackageTrailer should not contain invalid FIoHash values. Package '%s'"), *DebugContext);

		Private::FLookupTableEntry& Entry = Trailer.Header.PayloadLookupTable.AddDefaulted_GetRef();
		Entry.Identifier = It.Key;
		Entry.OffsetInFile = Trailer.Header.PayloadsDataLength;
		Entry.CompressedSize = It.Value.Payload.GetCompressedSize();
		Entry.RawSize = It.Value.Payload.GetRawSize();
		Entry.AccessMode = EPayloadAccessMode::Local;
		Entry.Flags = EPayloadFlags::None;
		Entry.FilterFlags = It.Value.FilterFlags;

		Trailer.Header.PayloadsDataLength += It.Value.Payload.GetCompressedSize();
	}

	for (const TPair<FIoHash, ReferencedEntry>& It : ReferencedEntries)
	{
		checkf(!It.Key.IsZero(), TEXT("PackageTrailer should not contain invalid FIoHash values. Package '%s'"), *DebugContext);

		Private::FLookupTableEntry& Entry = Trailer.Header.PayloadLookupTable.AddDefaulted_GetRef();
		Entry.Identifier = It.Key;
		Entry.OffsetInFile = It.Value.Offset;
		Entry.CompressedSize = It.Value.CompressedSize;
		Entry.RawSize = It.Value.RawSize;
		Entry.AccessMode = EPayloadAccessMode::Referenced;
	}

	for (const TPair<FIoHash, VirtualizedEntry>& It : VirtualizedEntries)
	{
		checkf(!It.Key.IsZero(), TEXT("PackageTrailer should not contain invalid FIoHash values. Package '%s'"), *DebugContext);

		Private::FLookupTableEntry& Entry = Trailer.Header.PayloadLookupTable.AddDefaulted_GetRef();
		Entry.Identifier = It.Key;
		Entry.OffsetInFile = INDEX_NONE;
		Entry.CompressedSize = INDEX_NONE;
		Entry.RawSize = It.Value.RawSize;
		Entry.AccessMode = EPayloadAccessMode::Virtualized;
	}

	// Now that we have the complete trailer we can serialize it to the archive

	Trailer.PackageTrailerOffset = InOutArchiveOffset;

	const int64 TrailerPositionInDataArchive = DataArchive.Tell();
	DataArchive << Trailer.Header;

	checkf((TrailerPositionInDataArchive + Trailer.Header.HeaderLength) == DataArchive.Tell(),
		TEXT("Header length was calculated as %d bytes but we wrote %" INT64_FMT " bytes!"), 
		Trailer.Header.HeaderLength, 
		DataArchive.Tell() - TrailerPositionInDataArchive);

	const int64 PayloadPosInDataArchive = DataArchive.Tell();

	for (TPair<FIoHash, LocalEntry>& It : LocalEntries)
	{
		DataArchive << It.Value.Payload;
	}

	checkf((PayloadPosInDataArchive + Trailer.Header.PayloadsDataLength) == DataArchive.Tell(),
		TEXT("Total payload length was calculated as %" INT64_FMT " bytes but we wrote %" INT64_FMT " bytes!"), 
		Trailer.Header.PayloadsDataLength, 
		DataArchive.Tell() - PayloadPosInDataArchive);

	FPackageTrailer::FFooter Footer = Trailer.CreateFooter();
	DataArchive << Footer;

	checkf((TrailerPositionInDataArchive + Footer.TrailerLength) == DataArchive.Tell(),
		TEXT("Trailer length was calculated as %" INT64_FMT " bytes but we wrote %" INT64_FMT " bytes!"), 
		Footer.TrailerLength, 
		DataArchive.Tell() - TrailerPositionInDataArchive);

	// Invoke any registered callbacks and pass in the trailer, this allows the callbacks to poll where 
	// in the output archive the payload has been stored.
	if (Linker != nullptr)
	{
		for (const AdditionalDataCallback& Callback : Callbacks)
		{
			Callback(*Linker, Trailer);
		}
	}

	// Minor sanity check that ::GetTrailerLength works
	check(CalculateTrailerLength() == (DataArchive.Tell() - TrailerPositionInDataArchive) || DataArchive.IsError());
	InOutArchiveOffset += DataArchive.Tell() - TrailerPositionInDataArchive;

	return !DataArchive.IsError();
}

bool FPackageTrailerBuilder::IsEmpty() const
{
	return LocalEntries.IsEmpty() && ReferencedEntries.IsEmpty() && VirtualizedEntries.IsEmpty();
}

bool FPackageTrailerBuilder::IsLocalPayloadEntry(const FIoHash& Identifier) const
{
	return LocalEntries.Find(Identifier) != nullptr;
}

bool FPackageTrailerBuilder::IsReferencedPayloadEntry(const FIoHash& Identifier) const
{
	return ReferencedEntries.Find(Identifier) != nullptr;
}

bool FPackageTrailerBuilder::IsVirtualizedPayloadEntry(const FIoHash& Identifier) const
{
	return VirtualizedEntries.Find(Identifier) != nullptr;
}

uint64 FPackageTrailerBuilder::CalculateTrailerLength()
{
	// For now we need to call ::RemoveDuplicateEntries to make sure that we are calculating
	// the correct length, hence the method not being const
	RemoveDuplicateEntries();

	const uint64 HeaderLength = (uint64)CalculatePotentialHeaderSize();
	const uint64 PayloadsLength = CalculatePotentialPayloadSize();
	const uint64 FooterLength = FPackageTrailer::FFooter::SizeOnDisk;

	return HeaderLength + PayloadsLength + FooterLength;
}

int32 FPackageTrailerBuilder::GetNumPayloads() const
{
	return GetNumLocalPayloads() + GetNumReferencedPayloads() + GetNumVirtualizedPayloads();
}

int32 FPackageTrailerBuilder::GetNumLocalPayloads() const
{
	return LocalEntries.Num();
}

int32 FPackageTrailerBuilder::GetNumReferencedPayloads() const
{
	return ReferencedEntries.Num();
}

int32 FPackageTrailerBuilder::GetNumVirtualizedPayloads() const
{
	return VirtualizedEntries.Num();
}

uint32 FPackageTrailerBuilder::CalculatePotentialHeaderSize() const
{
	const uint32 DynamicHeaderSizeOnDisk = (GetNumPayloads() * Private::FLookupTableEntry::SizeOnDisk); // Add the length of the lookup table

	return FPackageTrailer::FHeader::StaticHeaderSizeOnDisk + DynamicHeaderSizeOnDisk;
}

uint64 FPackageTrailerBuilder::CalculatePotentialPayloadSize() const
{
	uint64 PayloadsLength = 0;
	for (const TPair<FIoHash, LocalEntry>& KV : LocalEntries)
	{
		const FCompressedBuffer& Payload = KV.Value.Payload;

		PayloadsLength += Payload.GetCompressedSize();
	}

	return PayloadsLength;
}

void FPackageTrailerBuilder::RemoveDuplicateEntries()
{
	for (auto Iter = LocalEntries.CreateIterator(); Iter; ++Iter)
	{
		if (VirtualizedEntries.Contains(Iter.Key()))
		{
			UE_LOGF(LogSerialization, Verbose,
				"Replacing localized payload '%ls' with the virtualized version when building the package trailer for '%ls'",
				*LexToString(Iter.Key()),
				*DebugContext);

			Iter.RemoveCurrent();
		}
	}

	for (auto Iter = ReferencedEntries.CreateIterator(); Iter; ++Iter)
	{
		if (VirtualizedEntries.Contains(Iter.Key()))
		{
			UE_LOGF(LogSerialization, Verbose,
				"Replacing localized payload '%ls' with the virtualized version when building the package trailer for '%ls'",
				*LexToString(Iter.Key()),
				*DebugContext);

			Iter.RemoveCurrent();
		}
	}
}

bool FPackageTrailer::TryLoadFromPackage(const FPackagePath& PackagePath, FPackageTrailer& OutTrailer)
{
	//TODO: Need to do this in a way that supports text based assets
	TUniquePtr<FArchive> PackageAr = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, PackagePath.GetPackageName());

	if (PackageAr.IsValid())
	{
		PackageAr->Seek(PackageAr->TotalSize());
		return OutTrailer.TryLoadBackwards(*PackageAr);
	}
	else
	{
		LogPackageOpenFailureMessage(PackagePath.GetDebugName());
		return false;
	}
}

bool FPackageTrailer::TryLoadFromFile(const FString& Path, FPackageTrailer& OutTrailer)
{
	TUniquePtr<FArchive> PackageAr(IFileManager::Get().CreateFileReader(*Path));

	if (PackageAr.IsValid())
	{
		PackageAr->Seek(PackageAr->TotalSize());
		return OutTrailer.TryLoadBackwards(*PackageAr);
	}
	else
	{
		LogPackageOpenFailureMessage(Path);
		return false;
	}
}

bool FPackageTrailer::TryLoadFromArchive(FArchive& Ar, FPackageTrailer& OutTrailer)
{
	Ar.Seek(Ar.TotalSize());
	return OutTrailer.TryLoadBackwards(Ar);
}

bool FPackageTrailer::TryLoad(FArchive& Ar)
{
	check(Ar.IsLoading());

	PackageTrailerOffset = Ar.Tell();

	Header.Tag = 0; // Make sure we ignore anything previously loaded
	Ar << Header.Tag;

	// Make sure that we are parsing a valid FPackageTrailer
	if (Header.Tag != FPackageTrailer::FHeader::HeaderTag)
	{
		return false;
	}

	Ar << Header.Version;
	Ar << Header.HeaderLength;
	Ar << Header.PayloadsDataLength;

	// The header and the payloads should not exceed the remaining data in the archive
	const FGuardedInt64 FinalTrailerBytePos = FGuardedInt64(PackageTrailerOffset) + Header.HeaderLength + FGuardedInt64(Header.PayloadsDataLength);
	if (FinalTrailerBytePos.InvalidOrGreaterThan(Ar.TotalSize()))
	{
		Ar.SetError();
		return false;
	}

	EPayloadAccessMode LegacyAccessMode = EPayloadAccessMode::Local;
	if (Header.Version < (uint32)EPackageTrailerVersion::ACCESS_PER_PAYLOAD)
	{
		Ar << LegacyAccessMode;
	}

	int32 NumPayloads = 0;
	Ar << NumPayloads;
	
	// Make sure that we serialized a valid value for 'NumPayloads' before we try to allocate memory based off it
	if (Ar.IsError())
	{
		return false; 
	}

	// Make sure that there is enough data remaining in the archive to load the look up table
	if (NumPayloads * Private::FLookupTableEntry::SizeOnDisk > Header.HeaderLength)
	{
		Ar.SetError();
		return false;
	}

	Header.PayloadLookupTable.Reserve(NumPayloads);

	for (int32 Index = 0; Index < NumPayloads; ++Index)
	{
		Private::FLookupTableEntry& Entry = Header.PayloadLookupTable.AddDefaulted_GetRef();
		Entry.Serialize(Ar, (EPackageTrailerVersion)Header.Version);

		if (!IsEntryValid(Entry, Header.PayloadsDataLength))
		{
			// If an entry is invalid then something is likely up with the data so signal an error
			Ar.SetError();
			return false;
		}

		if (Header.Version < (uint32)EPackageTrailerVersion::ACCESS_PER_PAYLOAD)
		{
			Entry.AccessMode = Entry.OffsetInFile != INDEX_NONE ? LegacyAccessMode : EPayloadAccessMode::Virtualized;
		}
	}

	return !Ar.IsError();
}

bool FPackageTrailer::TryLoadBackwards(FArchive& Ar)
{
	check(Ar.IsLoading());

	const int64 FooterPos = Ar.Tell() - (int64)FFooter::SizeOnDisk;
	if (FooterPos <= 0)
	{
		return false;
	}

	Ar.Seek(FooterPos);

	FFooter Footer;

	Ar << Footer.Tag;
	Ar << Footer.TrailerLength;
	Ar << Footer.PackageTag;

	// First make sure that we were able to serialize everything
	if (Ar.IsError())
	{
		return false;
	}

	// Then check the package tag as this indicates if the data/file is corrupted or not
	if (Footer.PackageTag != PACKAGE_FILE_TAG)
	{
		Ar.SetError();
		return false;
	}

	// Now check the footer tag which will indicate if this is actually a FPackageTrailer that we are parsing
	if (Footer.Tag != FFooter::FooterTag)
	{
		return false;
	}

	Ar.Seek(Ar.Tell() - Footer.TrailerLength);

	// Lastly make sure we were able to see to where we believe the header is before we try to load it
	if (Ar.IsError())
	{
		return false;
	}

	return TryLoad(Ar);
}

FCompressedBuffer FPackageTrailer::LoadLocalPayload(const FIoHash& Id, FArchive& Ar) const
{
	// TODO: This method should be able to load the payload in all cases, but we need a good way of passing the Archive/PackagePath 
	// to the trailer etc. Would work if we stored the package path in the trailer.
	const Private::FLookupTableEntry* Entry = FindEntryInHeader(Header, Id);

	if (Entry == nullptr || Entry->AccessMode != EPayloadAccessMode::Local)
	{
		return FCompressedBuffer();
	}

	const FGuardedInt64 PayloadOffsetInFile = FGuardedInt64(PackageTrailerOffset) + Header.HeaderLength + Entry->OffsetInFile;
	if (PayloadOffsetInFile.InvalidOrLessThan(0) || PayloadOffsetInFile.InvalidOrGreaterOrEqual(Ar.TotalSize()))
	{
		UE_LOGF(LogSerialization, Warning, "Unable to load payload '%ls' from disk, it's offset is outside of the valid range (0-%lld) from archive '%ls'",
			*LexToString(Id),
			Ar.Tell(),
			*Ar.GetArchiveName());

		return FCompressedBuffer();
	}

	Ar.Seek(PayloadOffsetInFile.Get(0));

	FCompressedBuffer Payload;
	Ar << Payload;

	return Payload;
}

bool FPackageTrailer::UpdatePayloadAsVirtualized(const FIoHash& Identifier)
{
	Private::FLookupTableEntry* Entry = FindEntryInHeader(Header, Identifier);

	if (Entry != nullptr)
	{
		Entry->AccessMode = EPayloadAccessMode::Virtualized;
		Entry->OffsetInFile = INDEX_NONE;
		Entry->CompressedSize = INDEX_NONE; // Once the payload is virtualized we cannot be sure on the compression
											// being used and so cannot know the compressed size
		return true;
	}
	else
	{
		return false;
	}
}

void FPackageTrailer::ForEachPayload(TFunctionRef<void(const FIoHash&, uint64, uint64, EPayloadAccessMode, UE::Virtualization::EPayloadFilterReason)> Callback) const
{
	for (const Private::FLookupTableEntry& Entry : Header.PayloadLookupTable)
	{
		Callback(Entry.Identifier, Entry.CompressedSize, Entry.RawSize, Entry.AccessMode, Entry.FilterFlags);
	}
}

EPayloadStatus FPackageTrailer::FindPayloadStatus(const FIoHash& Id) const
{
	const Private::FLookupTableEntry* Entry = FindEntryInHeader(Header, Id);

	if (Entry == nullptr)
	{
		return EPayloadStatus::NotFound;
	}

	switch (Entry->AccessMode)
	{
		case EPayloadAccessMode::Local:
			return EPayloadStatus::StoredLocally;

		case EPayloadAccessMode::Referenced:
			return EPayloadStatus::StoredAsReference;

		case EPayloadAccessMode::Virtualized:
			return EPayloadStatus::StoredVirtualized;

		default:
			checkNoEntry();
			return EPayloadStatus::NotFound;
	}
}

int64 FPackageTrailer::FindPayloadOffsetInFile(const FIoHash& Id) const
{
	if (!Id.IsZero())
	{
		const Private::FLookupTableEntry* Entry = FindEntryInHeader(Header, Id);

		//TODO Better way to return an error?
		check(PackageTrailerOffset != INDEX_NONE);
		check(Header.PayloadsDataLength != INDEX_NONE);
		check(Entry != nullptr);

		switch (Entry->AccessMode)
		{
			case EPayloadAccessMode::Local:
				return PackageTrailerOffset + Header.HeaderLength + Entry->OffsetInFile;

			case EPayloadAccessMode::Referenced:
				return Entry->OffsetInFile;

			case EPayloadAccessMode::Virtualized:
				return INDEX_NONE;

			default:
				checkNoEntry();
				return INDEX_NONE;
		}
	}
	else
	{
		return INDEX_NONE;
	}
}

int64 FPackageTrailer::FindPayloadSizeOnDisk(const FIoHash& Id) const
{
	if (!Id.IsZero())
	{
		const Private::FLookupTableEntry* Entry = FindEntryInHeader(Header, Id);

		if (Entry != nullptr)
		{
			return Entry->CompressedSize;
		}
	}

	return INDEX_NONE;
}

int64 FPackageTrailer::GetTrailerLength() const
{
	return Header.HeaderLength + Header.PayloadsDataLength + FFooter::SizeOnDisk;
}

FPayloadInfo FPackageTrailer::GetPayloadInfo(const FIoHash& Id) const
{
	const Private::FLookupTableEntry* Entry = FindEntryInHeader(Header, Id);

	if (Entry != nullptr)
	{
		FPayloadInfo Info;

		Info.OffsetInFile = Entry->OffsetInFile;
		Info.CompressedSize = Entry->CompressedSize;
		Info.RawSize = Entry->RawSize;
		Info.AccessMode = Entry->AccessMode;
		Info.Flags = Entry->Flags;
		Info.FilterFlags = Entry->FilterFlags;

		return Info;
	}
	else
	{
		return FPayloadInfo();
	}
}

TArray<FIoHash> FPackageTrailer::GetPayloads(EPayloadStorageType StorageType) const
{
	TArray<FIoHash> Identifiers;
	Identifiers.Reserve(Header.PayloadLookupTable.Num());

	for (const Private::FLookupTableEntry& Entry : Header.PayloadLookupTable)
	{
		switch (StorageType)
		{
			case EPayloadStorageType::Any:
				Identifiers.Add(Entry.Identifier);
				break;

			case EPayloadStorageType::Local:
				if (Entry.IsLocal())
				{
					Identifiers.Add(Entry.Identifier);
				}
				break;

			case EPayloadStorageType::Virtualized:
				if (Entry.IsVirtualized())
				{
					Identifiers.Add(Entry.Identifier);
				}
				break;

			default:
				checkNoEntry();
		}	
	}

	return Identifiers;
}

int32 FPackageTrailer::GetNumPayloads(EPayloadStorageType Type) const
{
	int32 Count = 0;

	switch (Type)
	{
		case EPayloadStorageType::Any:
			Count = Header.PayloadLookupTable.Num();
			break;

		case EPayloadStorageType::Local:
			Count = (int32)Algo::CountIf(Header.PayloadLookupTable, [](const Private::FLookupTableEntry& Entry) { return Entry.AccessMode == EPayloadAccessMode::Local; });
			break;

		case EPayloadStorageType::Referenced:
			Count = (int32)Algo::CountIf(Header.PayloadLookupTable, [](const Private::FLookupTableEntry& Entry) { return Entry.AccessMode == EPayloadAccessMode::Referenced; });
			break;

		case EPayloadStorageType::Virtualized:
			Count = (int32)Algo::CountIf(Header.PayloadLookupTable, [](const Private::FLookupTableEntry& Entry) { return Entry.AccessMode == EPayloadAccessMode::Virtualized; });
			break;

		default:
			checkNoEntry();
	}
	
	return Count;
}

TArray<FIoHash> FPackageTrailer::GetPayloads(EPayloadFilter Filter) const
{
	TArray<FIoHash> Identifiers;
	Identifiers.Reserve(Header.PayloadLookupTable.Num());

	for (const Private::FLookupTableEntry& Entry : Header.PayloadLookupTable)
	{
		switch (Filter)
		{
			case EPayloadFilter::CanVirtualize:
				if (Entry.IsLocal() && Entry.FilterFlags == UE::Virtualization::EPayloadFilterReason::None)
				{
					Identifiers.Add(Entry.Identifier);
				}
				break;

			default:
				checkNoEntry();
		}
	}

	return Identifiers;
}

int32 FPackageTrailer::GetNumPayloads(EPayloadFilter Filter) const
{
	int32 Count = 0;

	switch (Filter)
	{
		case EPayloadFilter::CanVirtualize:
			Count = (int32)Algo::CountIf(Header.PayloadLookupTable, [](const Private::FLookupTableEntry& Entry)
				{
					return Entry.AccessMode == EPayloadAccessMode::Local && Entry.FilterFlags == UE::Virtualization::EPayloadFilterReason::None;
				});
			break;

		default:
			checkNoEntry();
	}

	return Count;
}
FPackageTrailer::FFooter FPackageTrailer::CreateFooter() const
{
	FFooter Footer;

	Footer.Tag = FPackageTrailer::FFooter::FooterTag;
	Footer.TrailerLength = Header.HeaderLength + Header.PayloadsDataLength + FPackageTrailer::FFooter::SizeOnDisk;
	Footer.PackageTag = PACKAGE_FILE_TAG;

	return Footer;
}

FArchive& operator<<(FArchive& Ar, FPackageTrailer::FHeader& Header)
{
	// Make sure that we save the most up to date version	
	Header.Version = (int32)EPackageTrailerVersion::AUTOMATIC_VERSION;

	Ar << Header.Tag;
	Ar << Header.Version;
	Ar << Header.HeaderLength;
	Ar << Header.PayloadsDataLength;

	int32 NumPayloads = Header.PayloadLookupTable.Num();
	Ar << NumPayloads;

	for (Private::FLookupTableEntry& Entry : Header.PayloadLookupTable)
	{
		Entry.Serialize(Ar, (EPackageTrailerVersion)Header.Version);
	}

	return Ar;
}

FArchive& operator<<(FArchive& Ar, FPackageTrailer::FFooter& Footer)
{
	Ar << Footer.Tag;
	Ar << Footer.TrailerLength;
	Ar << Footer.PackageTag;

	return Ar;
}

bool FindPayloadsInPackageFile(const FPackagePath& PackagePath, EPayloadStorageType Filter, TArray<FIoHash>& OutPayloadIds)
{
	if (FPackageName::IsTextPackageExtension(PackagePath.GetHeaderExtension()))
	{
		UE_LOGF(LogSerialization, Warning, "Attempting to call 'FindPayloadsInPackageFile' on a text based asset '%ls' this is not currently supported", *PackagePath.GetDebugName());
		return false;
	}

	TUniquePtr<FArchive> Ar = IPackageResourceManager::Get().OpenReadExternalResource(EPackageExternalResource::WorkspaceDomainFile, PackagePath.GetPackageName());

	if (Ar.IsValid())
	{
		Ar->Seek(Ar->TotalSize());

		FPackageTrailer Trailer;
		
		if (Trailer.TryLoadBackwards(*Ar))
		{
			OutPayloadIds = Trailer.GetPayloads(Filter);
			return true;
		}
		else
		{
			UE_LOGF(LogSerialization, Warning, "Failed to parse the FPackageTrailer for '%ls'", *PackagePath.GetDebugName());
			return false;
		}	
	}
	else
	{
		UE_LOGF(LogSerialization, Warning, "Unable to open '%ls' for reading", *PackagePath.GetDebugName());
		return false;
	}
}

} //namespace UE

FString LexToString(UE::Virtualization::EPayloadFilterReason FilterFlags)
{
	using namespace UE::Virtualization;

	if (FilterFlags == EPayloadFilterReason::None)
	{
		return TEXT("None");
	}
	else
	{
		TStringBuilder<512> Builder;
		auto AddSeparatorIfNeeded = [&Builder]()
		{
			if (Builder.Len() != 0)
			{
				Builder << TEXT("|");
			}
		};

		if (EnumHasAllFlags(FilterFlags, EPayloadFilterReason::Asset))
		{
			Builder << TEXT("Asset");
		}

		if (EnumHasAllFlags(FilterFlags, EPayloadFilterReason::Path))
		{
			AddSeparatorIfNeeded();
			Builder << TEXT("Path");
		}

		if (EnumHasAllFlags(FilterFlags, EPayloadFilterReason::MinSize))
		{
			AddSeparatorIfNeeded();
			Builder << TEXT("MinSize");
		}

		if (EnumHasAllFlags(FilterFlags, EPayloadFilterReason::EditorBulkDataCode))
		{
			AddSeparatorIfNeeded();
			Builder << TEXT("EditorBulkDataCode");
		}

		if (EnumHasAllFlags(FilterFlags, EPayloadFilterReason::MapContent))
		{
			AddSeparatorIfNeeded();
			Builder << TEXT("MapContent");
		}

		// In case a new entry was added to EPayloadFilterReason without this function being
		// updated we need to check if any of the other bits are set and if they are print 
		// and unknown entry.
		// We don't want a Max or Count entry being added to EPayloadFilterReason as we'd then
		// need to validate all places taking the enum to make sure that nobody is using it.
		const uint16 Mask = ~uint16(	EPayloadFilterReason::Asset |
										EPayloadFilterReason::Path |
										EPayloadFilterReason::MinSize |
										EPayloadFilterReason::EditorBulkDataCode |
										EPayloadFilterReason::MapContent);

		const bool bHasUnknownBits = ((uint16)FilterFlags & Mask) != 0;
		if (bHasUnknownBits)
		{
			AddSeparatorIfNeeded();
			Builder << TEXT("UnknownBits");
		}

		return Builder.ToString();
	}
}

