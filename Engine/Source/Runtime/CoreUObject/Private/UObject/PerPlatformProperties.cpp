// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PerPlatformProperties.h"
#include "Concepts/StaticStructProvider.h"
#include "Misc/DelayedAutoRegister.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PerPlatformProperties)

#include "UObject/PerPlatformPropertiesImpl.inl"

#if WITH_PREVIEW_PPX_DATA

FName PPXPreviewOverrides::PerPlatformOverride = NAME_None;
FName PPXPreviewOverrides::PerPlatformGroupOverride = NAME_None;

void PPXPreviewOverrides::SetPPXPreviewOverrides(FName& InPerPlatformOverride, FName& InPerPlatformGroupOverride)
{
	PerPlatformOverride = InPerPlatformOverride;
	PerPlatformGroupOverride = InPerPlatformGroupOverride;
}

#endif

template COREUOBJECT_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformInt, int32, NAME_IntProperty>&);
template COREUOBJECT_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformFloat, float, NAME_FloatProperty>&);
template COREUOBJECT_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformBool, bool, NAME_BoolProperty>&);
template COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, TPerPlatformProperty<FPerPlatformInt, int32, NAME_IntProperty>&);
template COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, TPerPlatformProperty<FPerPlatformFloat, float, NAME_FloatProperty>&);
template COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, TPerPlatformProperty<FPerPlatformBool, bool, NAME_BoolProperty>&);

template COREUOBJECT_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformFrameRate, FFrameRate, NAME_FrameRate>&);
template COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, TPerPlatformProperty<FPerPlatformFrameRate, FFrameRate, NAME_FrameRate>&);

FString FPerPlatformInt::ToString() const
{
	FString Result = FString::FromInt(Default);

#if WITH_EDITORONLY_DATA
	TArray<FName> SortedPlatforms;
	PerPlatform.GetKeys(/*out*/ SortedPlatforms);
	SortedPlatforms.Sort(FNameLexicalLess());

	for (FName Platform : SortedPlatforms)
	{
		Result = FString::Printf(TEXT("%s, %s=%d"), *Result, *Platform.ToString(), PerPlatform.FindChecked(Platform));
	}
#endif

	return Result;
}
