// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PrimaryAssetId.h"
#include "Misc/StringBuilder.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PrimaryAssetId)

bool FPrimaryAssetType::ExportTextItem(FString& ValueStr, FPrimaryAssetType const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (!(PortFlags & PPF_Delimited))
	{
		ValueStr += ToString();
	}
	else
	{
		ValueStr += FString::Printf(TEXT("\"%s\""), *ToString().ReplaceCharWithEscapedChar());
	}

	return true;
}

bool FPrimaryAssetType::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	// This handles both quoted and unquoted
	FString ImportedString;
	const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, ImportedString, 1);

	if (!NewBuffer)
	{
		return false;
	}

	*this = FPrimaryAssetType(*ImportedString);
	Buffer = NewBuffer;

	return true;
}

bool FPrimaryAssetType::SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		FName InName;
		Slot << InName;
		*this = FPrimaryAssetType(InName);
		return true;
	}
	else if (Tag.Type == NAME_StrProperty)
	{
		FString InString;
		Slot << InString;
		*this = FPrimaryAssetType(*InString);
		return true;
	}

	return false;
}

FPrimaryAssetId FPrimaryAssetId::ParseTypeAndName(const TCHAR* TypeAndName, uint32 Len)
{
	for (uint32 Idx = 0; Idx < Len; ++Idx)
	{
		if (TypeAndName[Idx] == ':')
		{
			FName Type(Idx, TypeAndName);
			FName Name(Len - Idx - 1, TypeAndName + Idx + 1);
			return FPrimaryAssetId(Type, Name);
		}
	}

	return FPrimaryAssetId();
}
	
FPrimaryAssetId FPrimaryAssetId::ParseTypeAndName(FName TypeAndName)
{
	TCHAR Str[FName::StringBufferSize];
	uint32 Len = TypeAndName.ToString(Str);
	return ParseTypeAndName(Str, Len);
}

bool FPrimaryAssetId::ExportTextItem(FString& ValueStr, FPrimaryAssetId const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (!(PortFlags & PPF_Delimited))
	{
		ValueStr += ToString();
	}
	else
	{
		ValueStr += FString::Printf(TEXT("\"%s\""), *ToString().ReplaceCharWithEscapedChar());
	}

	return true;
}

bool FPrimaryAssetId::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	// This handles both quoted and unquoted
	FString ImportedString;
	const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, ImportedString, 1);

	if (!NewBuffer)
	{
		return false;
	}

	*this = FPrimaryAssetId(ImportedString);
	Buffer = NewBuffer;

	return true;
}

bool FPrimaryAssetId::SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		FName InName;
		Slot << InName;
		*this = FPrimaryAssetId(InName.ToString());
		return true;
	}
	else if (Tag.Type == NAME_StrProperty)
	{
		FString InString;
		Slot << InString;
		*this = FPrimaryAssetId(InString);
		return true;
	}

	return false;
}

FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FPrimaryAssetId& Id)
{
	return Builder << Id.PrimaryAssetType.GetName() << ":" << Id.PrimaryAssetName;
}

void SerializeForLog(FCbWriter& Writer, const FPrimaryAssetId& Value)
{
	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("PrimaryAssetId"));
	Writer.AddString(ANSITEXTVIEW("$text"), WriteToUtf8String<256>(Value.PrimaryAssetType.GetName(), ':', Value.PrimaryAssetName));
	Writer.AddString(ANSITEXTVIEW("PrimaryAssetType"), WriteToUtf8String<256>(Value.PrimaryAssetType));
	Writer.AddString(ANSITEXTVIEW("PrimaryAssetName"), WriteToUtf8String<256>(Value.PrimaryAssetName));
	Writer.EndObject();
}
