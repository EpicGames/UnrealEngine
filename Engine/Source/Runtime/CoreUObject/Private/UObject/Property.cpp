// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "HAL/IConsoleManager.h"
#include "Hash/Blake3.h"
#include "Math/Box2D.h"
#include "Math/InterpCurvePoint.h"
#include "Math/RandomStream.h"
#include "Math/Ray.h"
#include "Math/Sphere.h"
#include "Misc/AsciiSet.h"
#include "Misc/FallbackStruct.h"
#include "Misc/Guid.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Misc/StringOutputDevice.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/FrameNumber.h"
#include "Misc/Timecode.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Serialization/TestUndeclaredScriptStructObjectReferences.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/CoreRedirects.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/OverridableManager.h"
#include "UObject/Package.h"
#include "UObject/PropertyHelper.h"
#include "UObject/PropertyTypeName.h"
#include "UObject/PropertyVisitor.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITORONLY_DATA
#include "UObject/PropertyStateTracking.h"
#endif

DEFINE_LOG_CATEGORY(LogProperty);

namespace UE::CoreUObject::Private
{
	static int32 EnsureAgainstLargeProperties = 1;
	static FAutoConsoleVariableRef CVarEnsureAgainstLargeProperties(TEXT("CoreUObject.EnsureAgainstLargeProperties"), EnsureAgainstLargeProperties, TEXT("Ensure (warn) against properties that could possibly break in future versions of the Engine due a reduction in the max size of FProperty::ElementSize."));
}

// List the core ones here as they have already been included (and can be used without CoreUObject!)
template<typename T>
struct TVector3StructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithStructuredSerializer = true,
		WithStructuredSerializeFromMismatchedTag = true,
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
template<> struct TStructOpsTypeTraits<FVector3f> : public TVector3StructOpsTypeTraits<FVector3f> {};
template<> struct TStructOpsTypeTraits<FVector3d> : public TVector3StructOpsTypeTraits<FVector3d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector3f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector3d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector);	// Aliased

template<typename T>
struct TIntPointStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
template<> struct TStructOpsTypeTraits<FInt32Point> : public TIntPointStructOpsTypeTraits<FInt32Point> {};
template<> struct TStructOpsTypeTraits<FInt64Point> : public TIntPointStructOpsTypeTraits<FInt64Point> {};
template<> struct TStructOpsTypeTraits<FUint32Point> : public TIntPointStructOpsTypeTraits<FUint32Point> {};
template<> struct TStructOpsTypeTraits<FUint64Point> : public TIntPointStructOpsTypeTraits<FUint64Point> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Int32Point);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Int64Point);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Uint32Point);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Uint64Point);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", IntPoint);		// Aliased
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", UintPoint);		// Aliased

template<typename T>
struct TIntVectorStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
template<> struct TStructOpsTypeTraits<FInt32Vector2> : public TIntVectorStructOpsTypeTraits<FInt32Vector2> {};
template<> struct TStructOpsTypeTraits<FInt64Vector2> : public TIntVectorStructOpsTypeTraits<FInt64Vector2> {};
template<> struct TStructOpsTypeTraits<FUint32Vector2> : public TIntVectorStructOpsTypeTraits<FUint32Vector2> {};
template<> struct TStructOpsTypeTraits<FUint64Vector2> : public TIntVectorStructOpsTypeTraits<FUint64Vector2> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Int32Vector2);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Int64Vector2);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Uint32Vector2);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Uint64Vector2);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", IntVector2);		// Aliased
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", UintVector2);	// Aliased

template<> struct TStructOpsTypeTraits<FInt32Vector3> : public TIntVectorStructOpsTypeTraits<FInt32Vector3> {};
template<> struct TStructOpsTypeTraits<FInt64Vector3> : public TIntVectorStructOpsTypeTraits<FInt64Vector3> {};
template<> struct TStructOpsTypeTraits<FUint32Vector3> : public TIntVectorStructOpsTypeTraits<FUint32Vector3> {};
template<> struct TStructOpsTypeTraits<FUint64Vector3> : public TIntVectorStructOpsTypeTraits<FUint64Vector3> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Int32Vector);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Int64Vector);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Uint32Vector);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Uint64Vector);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", IntVector);		// Aliased
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", UintVector);		// Aliased

template<> struct TStructOpsTypeTraits<FInt32Vector4> : public TIntVectorStructOpsTypeTraits<FInt32Vector4> {};
template<> struct TStructOpsTypeTraits<FInt64Vector4> : public TIntVectorStructOpsTypeTraits<FInt64Vector4> {};
template<> struct TStructOpsTypeTraits<FUint32Vector4> : public TIntVectorStructOpsTypeTraits<FUint32Vector4> {};
template<> struct TStructOpsTypeTraits<FUint64Vector4> : public TIntVectorStructOpsTypeTraits<FUint64Vector4> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Int32Vector4);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Int64Vector4);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Uint32Vector4);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Uint64Vector4); 
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", IntVector4);		// Aliased
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", UintVector4);	// Aliased

template<typename T>
struct TVector2StructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
template<> struct TStructOpsTypeTraits<FVector2f> : public TVector2StructOpsTypeTraits<FVector2f> {};
template<> struct TStructOpsTypeTraits<FVector2d> : public TVector2StructOpsTypeTraits<FVector2d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector2f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector2d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector2D);

template<typename T>
struct TVector4StructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
template<> struct TStructOpsTypeTraits<FVector4f> : public TVector4StructOpsTypeTraits<FVector4f> {};
template<> struct TStructOpsTypeTraits<FVector4d> : public TVector4StructOpsTypeTraits<FVector4d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector4f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector4d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Vector4);	// Aliased

template<typename T>
struct TPlaneStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
template<> struct TStructOpsTypeTraits<FPlane4f> : public TPlaneStructOpsTypeTraits<FPlane4f> {};
template<> struct TStructOpsTypeTraits<FPlane4d> : public TPlaneStructOpsTypeTraits<FPlane4d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Plane4f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Plane4d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Plane);	// Aliased

template<typename T>
struct TRotatorStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

template<> struct TStructOpsTypeTraits<FRotator3f> : public TRotatorStructOpsTypeTraits<FRotator3f> {};
template<> struct TStructOpsTypeTraits<FRotator3d> : public TRotatorStructOpsTypeTraits<FRotator3d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Rotator3f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Rotator3d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Rotator);	// Aliased

template<typename T>
struct TBox3StructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
template<> struct TStructOpsTypeTraits<FBox3f> : public TBox3StructOpsTypeTraits<FBox3f> {};
template<> struct TStructOpsTypeTraits<FBox3d> : public TBox3StructOpsTypeTraits<FBox3d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Box3f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Box3d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Box);		// Aliased

template<typename T>
struct TBox2StructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializeFromMismatchedTag = true,		
	};
};
template<> struct TStructOpsTypeTraits<FBox2f> : public TBox2StructOpsTypeTraits<FBox2f> {};
template<> struct TStructOpsTypeTraits<FBox2d> : public TBox2StructOpsTypeTraits<FBox2d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Box2f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Box2d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Box2D);

template<typename T>
struct TMatrixStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

template<> struct TStructOpsTypeTraits<FMatrix44f> : public TMatrixStructOpsTypeTraits<FMatrix44f> {};
template<> struct TStructOpsTypeTraits<FMatrix44d> : public TMatrixStructOpsTypeTraits<FMatrix44d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Matrix44f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Matrix44d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Matrix);	// Aliased

template<typename T>
struct TBoxSphereBoundsStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializeFromMismatchedTag = true,
	};
};
template<> struct TStructOpsTypeTraits<FBoxSphereBounds3f> : public TBoxSphereBoundsStructOpsTypeTraits<FBoxSphereBounds3f> {};
template<> struct TStructOpsTypeTraits<FBoxSphereBounds3d> : public TBoxSphereBoundsStructOpsTypeTraits<FBoxSphereBounds3d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", BoxSphereBounds3f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", BoxSphereBounds3d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", BoxSphereBounds);	// Aliased

template<>
struct TStructOpsTypeTraits<FOrientedBox> : public TStructOpsTypeTraitsBase2<FOrientedBox>
{
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", OrientedBox);

template<>
struct TStructOpsTypeTraits<FLinearColor> : public TStructOpsTypeTraitsBase2<FLinearColor>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithStructuredSerializer = true,
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", LinearColor);

template<>
struct TStructOpsTypeTraits<FColor> : public TStructOpsTypeTraitsBase2<FColor>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Color);


template<typename T>
struct TQuatStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum 
	{
		//quat is somewhat special in that it initialized w to one
		WithNoInitConstructor = true,
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithIdentical = true,
		WithSerializer = true,
		WithSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
template<> struct TStructOpsTypeTraits<FQuat4f> : public TQuatStructOpsTypeTraits<FQuat4f> {};
template<> struct TStructOpsTypeTraits<FQuat4d> : public TQuatStructOpsTypeTraits<FQuat4d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Quat4f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Quat4d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Quat);		// Aliased to one of FQuat4f/FQuat4d

template<>
struct TStructOpsTypeTraits<FTwoVectors> : public TStructOpsTypeTraitsBase2<FTwoVectors>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithZeroConstructor = true,
		WithSerializer = true,
		WithNoDestructor = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", TwoVectors);


template<typename T>
struct TRay3StructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializeFromMismatchedTag = true,		
	};
};
template<> struct TStructOpsTypeTraits<FRay3f> : public TRay3StructOpsTypeTraits<FRay3f> {};
template<> struct TStructOpsTypeTraits<FRay3d> : public TRay3StructOpsTypeTraits<FRay3d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Ray3f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Ray3d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Ray);


template<typename T>
struct TSphere3StructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		WithIdenticalViaEquality = true,
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
		WithSerializeFromMismatchedTag = true,
	};
};
template<> struct TStructOpsTypeTraits<FSphere3f> : public TSphere3StructOpsTypeTraits<FSphere3f> {};
template<> struct TStructOpsTypeTraits<FSphere3d> : public TSphere3StructOpsTypeTraits<FSphere3d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Sphere3f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Sphere3d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Sphere); 

template<>
struct TStructOpsTypeTraits<FInterpCurvePointFloat> : public TStructOpsTypeTraitsBase2<FInterpCurvePointFloat>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", InterpCurvePointFloat);

template<>
struct TStructOpsTypeTraits<FInterpCurvePointVector2D> : public TStructOpsTypeTraitsBase2<FInterpCurvePointVector2D>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", InterpCurvePointVector2D);

template<>
struct TStructOpsTypeTraits<FInterpCurvePointVector> : public TStructOpsTypeTraitsBase2<FInterpCurvePointVector>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", InterpCurvePointVector);

template<>
struct TStructOpsTypeTraits<FInterpCurvePointQuat> : public TStructOpsTypeTraitsBase2<FInterpCurvePointQuat>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", InterpCurvePointQuat);

template<>
struct TStructOpsTypeTraits<FInterpCurvePointTwoVectors> : public TStructOpsTypeTraitsBase2<FInterpCurvePointTwoVectors>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", InterpCurvePointTwoVectors);

template<>
struct TStructOpsTypeTraits<FInterpCurvePointLinearColor> : public TStructOpsTypeTraitsBase2<FInterpCurvePointLinearColor>
{
	enum
	{
		WithNoInitConstructor = true,
	};
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", InterpCurvePointLinearColor);

template<>
struct TStructOpsTypeTraits<FGuid> : public TStructOpsTypeTraitsBase2<FGuid>
{
	enum 
	{
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithZeroConstructor = true,
		WithSerializer = true,
		WithStructuredSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Guid);

template<typename T>
struct TTransformStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<T>
{
	enum
	{
		//WithSerializer = true,
		WithIdentical = true,
		WithSerializeFromMismatchedTag = true,
	};
};
template<> struct TStructOpsTypeTraits<FTransform3f> : public TTransformStructOpsTypeTraits<FTransform3f> {};
template<> struct TStructOpsTypeTraits<FTransform3d> : public TTransformStructOpsTypeTraits<FTransform3d> {};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Transform3f);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Transform3d);
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Transform); // Aliased

template<>
struct TStructOpsTypeTraits<FRandomStream> : public TStructOpsTypeTraitsBase2<FRandomStream>
{
	enum 
	{
		WithNoInitConstructor = true,
		WithZeroConstructor = true,
	};
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", RandomStream);

template<>
struct TStructOpsTypeTraits<FDateTime> : public TStructOpsTypeTraitsBase2<FDateTime>
{
	enum 
	{
		WithCopy = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithSerializer = true,
		WithNetSerializer = true,
		WithZeroConstructor = true,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", DateTime);

template<>
struct TStructOpsTypeTraits<FTimespan> : public TStructOpsTypeTraitsBase2<FTimespan>
{
	enum 
	{
		WithCopy = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithSerializer = true,
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithZeroConstructor = true,
		WithIdenticalViaEquality = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Timespan);

template<>
struct TStructOpsTypeTraits<FFrameNumber> : public TStructOpsTypeTraitsBase2<FFrameNumber>
{
	enum
	{
		WithZeroConstructor = true,
		WithSerializer = true,
		WithIdenticalViaEquality = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", FrameNumber);

template<>
struct TStructOpsTypeTraits<FFrameRate> : public TStructOpsTypeTraitsBase2<FFrameRate>
{
	enum
	{
		// The native function has a custom serializer but assets have already been created with the generic UPROPERTY serializer,
		// so we can't switch them to use a custom serializer without breaking assets (creates mismatched sizes in data).
		// WithSerializer = true, 
		WithIdenticalViaEquality = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", FrameRate);

template<>
struct TStructOpsTypeTraits<FFrameTime> : public TStructOpsTypeTraitsBase2<FFrameTime>
{
	enum
	{
		WithZeroConstructor = true,
		// The native function has a custom serializer but assets have already been created with the generic UPROPERTY serializer,
		// so we can't switch them to use a custom serializer without breaking assets (creates mismatched sizes in data).
		// WithSerializer = true, 
		WithIdenticalViaEquality = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", FrameTime);

// Qualified Frame Times can't be zero-initialized because they contain FrameRates,
// and have no equality operator.
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", QualifiedFrameTime);

template<>
struct TStructOpsTypeTraits<FTimecode> : public TStructOpsTypeTraitsBase2<FTimecode>
{
	enum
	{
		WithZeroConstructor = true,
		WithIdenticalViaEquality = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", Timecode);



template<>
struct TStructOpsTypeTraits<FFallbackStruct> : public TStructOpsTypeTraitsBase2<FFallbackStruct>
{
};
UE_IMPLEMENT_STRUCT("/Script/CoreUObject", FallbackStruct);

/*-----------------------------------------------------------------------------
	Helpers.
-----------------------------------------------------------------------------*/

const TCHAR* LexToString(EPropertyObjectReferenceType Type)
{
	switch(Type)
	{
	case EPropertyObjectReferenceType::None:
		return TEXT("None");
	case EPropertyObjectReferenceType::Strong:
		return TEXT("Strong");
	case EPropertyObjectReferenceType::Weak:
		return TEXT("Weak");
	case EPropertyObjectReferenceType::Soft:
		return TEXT("Soft");
	case EPropertyObjectReferenceType::Conservative:
		return TEXT("Conservative");
	}	
	return TEXT("Unknown");
}

constexpr FAsciiSet AlphaNumericChars = "abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ0123456789";

FORCEINLINE constexpr bool IsValidTokenStart(TCHAR FirstChar, bool bDottedNames)
{
	return AlphaNumericChars.Test(FirstChar) || (bDottedNames && FirstChar == '/') || FirstChar > 255;
}

FORCEINLINE constexpr FStringView ParsePropertyToken(const TCHAR* Str, bool DottedNames)
{
	constexpr FAsciiSet RegularTokenChars = AlphaNumericChars  + '_' + '-' + '+';
	constexpr FAsciiSet RegularNonTokenChars = ~RegularTokenChars;
	constexpr FAsciiSet DottedNonTokenChars = ~(RegularTokenChars + '.' + '/' + (char)SUBOBJECT_DELIMITER_CHAR);
	FAsciiSet CurrentNonTokenChars = DottedNames ? DottedNonTokenChars : RegularNonTokenChars;

	const TCHAR* TokenEnd = FAsciiSet::FindFirstOrEnd(Str, CurrentNonTokenChars);
	return FStringView(Str, UE_PTRDIFF_TO_INT32(TokenEnd - Str));
}

//
// Parse a token.
//
template <typename TStringType>
const TCHAR* ReadPropertyToken(const TCHAR* Buffer, TStringType& Out, bool bDottedNames)
{
	if (*Buffer == TEXT('"'))
	{
		int32 NumCharsRead = 0;
		if (!FParse::QuotedString(Buffer, Out, &NumCharsRead))
		{
			UE_LOGF(LogProperty, Warning, "ReadToken: Bad quoted string: %ls", Buffer);
			return nullptr;
		}
		Buffer += NumCharsRead;

		// TODO special handling of null-terminator here?
	}
	else if (IsValidTokenStart(*Buffer, bDottedNames))
	{
		FStringView Token = ParsePropertyToken(Buffer, bDottedNames);
		Out += Token;
		Buffer += Token.Len();
	}
	else
	{
		// Get just one.
		if (*Buffer)
		{
			Out += *Buffer;
		}
	}
	return Buffer;
}

const TCHAR* FPropertyHelpers::ReadToken( const TCHAR* Buffer, FString& Out, bool bDottedNames)
{
	return ReadPropertyToken(Buffer, Out, bDottedNames);
}

const TCHAR* FPropertyHelpers::ReadToken( const TCHAR* Buffer, FStringBuilderBase& Out, bool bDottedNames)
{
	return ReadPropertyToken(Buffer, Out, bDottedNames);
}

//
// Parse a terminated token.
//
template <typename TStringType>
const TCHAR* ReadTerminatedPropertyToken(const TCHAR* Buffer, TStringType& Out, const TCHAR Terminator, const int32* MaxTokenLen)
{
	TStringBuilder<256> Temp;
	while (*Buffer && *Buffer != Terminator && (!MaxTokenLen || Temp.Len() < *MaxTokenLen)) // Walk to the Terminator, with an optional limit to guard against bad data
	{
		if (*Buffer == TEXT('"'))
		{
			int32 NumCharsRead = 0;
			if (!FParse::QuotedString(Buffer, Temp, &NumCharsRead))
			{
				UE_LOGF(LogProperty, Warning, "ReadTerminatedToken: Bad quoted string: %ls", Buffer);
				return nullptr;
			}
			Buffer += NumCharsRead;
		}
		else
		{
			Temp += *Buffer++;
		}
	}
	if (*Buffer != Terminator) // Verify we found the Terminator
	{
		return nullptr;
	}
	Out += Temp.ToView();
	return Buffer;
}

const TCHAR* FPropertyHelpers::ReadTerminatedToken(const TCHAR* Buffer, FString& Out, const TCHAR Terminator, const int32* MaxTokenLen)
{
	return ReadTerminatedPropertyToken(Buffer, Out, Terminator, MaxTokenLen);
}

const TCHAR* FPropertyHelpers::ReadTerminatedToken(const TCHAR* Buffer, FStringBuilderBase& Out, const TCHAR Terminator, const int32* MaxTokenLen)
{
	return ReadTerminatedPropertyToken(Buffer, Out, Terminator, MaxTokenLen);
}

/*-----------------------------------------------------------------------------
	FProperty implementation.
-----------------------------------------------------------------------------*/

#if UE_GAME && UE_FNAME_OUTLINE_NUMBER && !WITH_METADATA
	static_assert(sizeof(FProperty) <= 104, "FProperty was optimized to reduce its size so most of the classes that inherent from it will fall within 112 bytes bin of MallocBinned3");
#endif

IMPLEMENT_FIELD(FProperty)

void FProperty::SetElementSize(int32 NewSize)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ElementSize = NewSize;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

//
// Constructors.
//
FProperty::FProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, ArrayDim(1)
	, ElementSize(0)
	, PropertyFlags(CPF_None)
	, RepIndex(0)
	, BlueprintReplicationCondition(COND_None)
	, Offset_Internal(0)
	, PropertyLinkNext(nullptr)
	, NextRef(nullptr)
	, DestructorLinkNext(nullptr)
	, PostConstructLinkNext(nullptr)
{
}

FProperty::FProperty(FFieldVariant InOwner, const UECodeGen_Private::FPropertyParamsBaseWithOffset& Prop, EPropertyFlags AdditionalPropertyFlags /*= CPF_None*/)
	: Super(InOwner, UTF8_TO_TCHAR(Prop.NameUTF8))
	, ArrayDim(1)
	, ElementSize(0)
	, PropertyFlags(Prop.PropertyFlags | AdditionalPropertyFlags)
	, RepIndex(0)
	, BlueprintReplicationCondition(COND_None)
	, Offset_Internal(0)
	, PropertyLinkNext(nullptr)
	, NextRef(nullptr)
	, DestructorLinkNext(nullptr)
	, PostConstructLinkNext(nullptr)
{
	this->Offset_Internal = Prop.Offset;

	Init();
}

FProperty::FProperty(FFieldVariant InOwner, const UECodeGen_Private::FPropertyParamsBaseWithoutOffset& Prop, EPropertyFlags AdditionalPropertyFlags /*= CPF_None*/)
	: Super(InOwner, UTF8_TO_TCHAR(Prop.NameUTF8))
	, ArrayDim(1)
	, ElementSize(0)
	, PropertyFlags(Prop.PropertyFlags | AdditionalPropertyFlags)
	, RepIndex(0)
	, BlueprintReplicationCondition(COND_None)
	, Offset_Internal(0)
	, PropertyLinkNext(nullptr)
	, NextRef(nullptr)
	, DestructorLinkNext(nullptr)
	, PostConstructLinkNext(nullptr)
{
	Init();
}

#if WITH_EDITORONLY_DATA
FProperty::FProperty(UField* InField)
	: Super(InField)
	, PropertyLinkNext(nullptr)
	, NextRef(nullptr)
	, DestructorLinkNext(nullptr)
	, PostConstructLinkNext(nullptr)
{
	UProperty* SourceProperty = CastChecked<UProperty>(InField);
	ArrayDim = SourceProperty->ArrayDim;
	SetElementSize(SourceProperty->ElementSize);
	PropertyFlags = SourceProperty->PropertyFlags;
	RepIndex = SourceProperty->RepIndex;
	Offset_Internal = SourceProperty->Offset_Internal;
	BlueprintReplicationCondition = SourceProperty->BlueprintReplicationCondition;
}
#endif // WITH_EDITORONLY_DATA

void FProperty::Init()
{
#if !WITH_EDITORONLY_DATA
	//@todo.COOKER/PACKAGER: Until we have a cooker/packager step, this can fire when WITH_EDITORONLY_DATA is not defined!
	//	checkSlow(!HasAnyPropertyFlags(CPF_EditorOnly));
#endif // WITH_EDITORONLY_DATA
	checkSlow(GetOwnerUField()->HasAllFlags(RF_Transient));

	if (GetOwner<UObject>())
	{
		UField* OwnerField = GetOwnerChecked<UField>();
		OwnerField->AddCppProperty(this);
	}
	else
	{
		FField* OwnerField = GetOwnerChecked<FField>();
		OwnerField->AddCppProperty(this);
	}
}

//
// Serializer.
//
void FProperty::Serialize( FArchive& Ar )
{
	// Make sure that we aren't saving a property to a package that shouldn't be serialised.
#if WITH_EDITORONLY_DATA
	check(!Ar.IsFilterEditorOnly() || !IsEditorOnlyProperty());
#endif // WITH_EDITORONLY_DATA

	Super::Serialize(Ar);

	Ar << ArrayDim;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Ar << ElementSize;

	const bool EnsureAgainstLargeProperties = UE::CoreUObject::Private::EnsureAgainstLargeProperties > 0;
	constexpr int32 ExpectedMaxSize = (1 << 24);
	ensureMsgf(!EnsureAgainstLargeProperties || GetElementSize() < ExpectedMaxSize, TEXT("%s has ElementSize %d which will violate an upcoming change to lower the max ElementSize.  Consider breaking up the property. Disable this warning with CoreUObject.EnsureAgainstLargeProperties 0"), *GetName(), GetElementSize());
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	EPropertyFlags SaveFlags = PropertyFlags & ~CPF_ComputedFlags;
	// Archive the basic info.
	Ar << (uint64&)SaveFlags;
	if (Ar.IsLoading())
	{
		PropertyFlags = (SaveFlags & ~CPF_ComputedFlags) | (PropertyFlags & CPF_ComputedFlags);

	}

	if (FPlatformProperties::HasEditorOnlyData() == false)
	{
		// Make sure that we aren't saving a property to a package that shouldn't be serialised.
		check( !IsEditorOnlyProperty() );
	}

	decltype(RepIndex) DefaultRepIndex = 0;
	Ar << DefaultRepIndex; // RepIndex is built lazily on demand. Serialize default value for backwards compat, but discard it.
	Ar << RepNotifyFunc;

	if (Ar.IsLoading())
	{
		Offset_Internal = 0;
		DestructorLinkNext = nullptr;
	}

	Ar << BlueprintReplicationCondition;
}

void FProperty::PostDuplicate(const FField& InField)
{
	const FProperty& Source = static_cast<const FProperty&>(InField);
	ArrayDim = Source.ArrayDim;
	SetElementSize(Source.GetElementSize());
	PropertyFlags = Source.PropertyFlags;
	RepIndex = Source.RepIndex;
	Offset_Internal = Source.Offset_Internal;
	RepNotifyFunc = Source.RepNotifyFunc;
	BlueprintReplicationCondition = Source.BlueprintReplicationCondition;

	Super::PostDuplicate(InField);
}

void FProperty::CopySingleValueToScriptVM( void* Dest, void const* Src ) const
{
	CopySingleValue(Dest, Src);
}

void FProperty::CopyCompleteValueToScriptVM( void* Dest, void const* Src ) const
{
	CopyCompleteValue(Dest, Src);
}

void FProperty::CopySingleValueFromScriptVM( void* Dest, void const* Src ) const
{
	CopySingleValue(Dest, Src);
}

void FProperty::CopyCompleteValueFromScriptVM( void* Dest, void const* Src ) const
{
	CopyCompleteValue(Dest, Src);
}

void FProperty::CopyCompleteValueToScriptVM_InContainer( void* OutValue, void const* InContainer ) const
{
	if (HasGetter())
	{
		CallGetter(InContainer, OutValue);
	}
	else
	{
		const void* InObj = ContainerPtrToValuePtr<uint8>(InContainer);
		CopyCompleteValue(OutValue, InObj);
	}
}

void FProperty::CopyCompleteValueFromScriptVM_InContainer( void* OutContainer, void const* InValue ) const
{
	if (HasSetter())
	{
		CallSetter(OutContainer, InValue);
	}
	else
	{
		void* OutObj = ContainerPtrToValuePtr<uint8>(OutContainer);
		CopyCompleteValue(OutObj, InValue);
	}
}

void FProperty::ClearValueInternal( void* Data ) const
{
	checkf(0, TEXT("%s failed to handle ClearValueInternal, but it was not CPF_NoDestructor | CPF_ZeroConstructor"), *GetFullName());
}
void FProperty::DestroyValueInternal( void* Dest ) const
{
	checkf(0, TEXT("%s failed to handle DestroyValueInternal, but it was not CPF_NoDestructor"), *GetFullName());
}
void FProperty::InitializeValueInternal( void* Dest ) const
{
	checkf(0, TEXT("%s failed to handle InitializeValueInternal, but it was not CPF_ZeroConstructor"), *GetFullName());
}

bool FProperty::ContainsClearOnFinishDestroyInternal( TArray<const FStructProperty*>& EncounteredStructProps ) const
{
	return false;
}

void FProperty::FinishDestroyInternal( void* Data ) const
{
	// Empty
}

/**
 * Verify that modifying this property's value via ImportText is allowed.
 * 
 * @param	PortFlags	the flags specified in the call to ImportText
 *
 * @return	true if ImportText should be allowed
 */
bool FProperty::ValidateImportFlags( uint32 PortFlags, FOutputDevice* ErrorHandler ) const
{
	// PPF_RestrictImportTypes is set when importing defaultproperties; it indicates that
	// we should not allow config/localized properties to be imported here
	if ((PortFlags & PPF_RestrictImportTypes) && (PropertyFlags & CPF_Config))
	{
		FString ErrorMsg = FString::Printf(TEXT("Import failed for '%s': property is config (Check to see if the property is listed in the DefaultProperties.  It should only be listed in the specific .ini file)"), *GetName());

		if (ErrorHandler)
		{
			ErrorHandler->Logf(TEXT("%s"), *ErrorMsg);
		}
		else
		{
			UE_LOGF(LogProperty, Warning, "%ls", *ErrorMsg);
		}

		return false;
	}

	return true;
}

FString FProperty::GetNameCPP() const
{
	return HasAnyPropertyFlags(CPF_Deprecated) ? GetName() + TEXT("_DEPRECATED") : GetName();
}

FString FProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	ExtendedTypeText = TEXT("F");
	ExtendedTypeText += GetClass()->GetName();
	return TEXT("PROPERTY");
}

bool FProperty::ExportText_Direct(FString& ValueStr, TNotNull<const void*> Data, const void* Delta, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	const bool bExportOverride = FOverridableSerializationLogic::IsEnabled() && FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization);
	if( Data==Delta || 
		(bExportOverride && FOverridableSerializationLogic::GetOverriddenPropertyOperationForPortText(Data, Delta, PortFlags) != EOverriddenPropertyOperation::None)  ||
		(!bExportOverride && !Identical(Data,Delta,PortFlags)) )
	{
		ExportText_Internal(
			ValueStr,
			(uint8*)NotNullGet(Data),
			EPropertyPointerType::Direct,
			(uint8*)Delta,
			Parent,
			PortFlags,
			ExportRootScope
		);
		return true;
	}

	return false;
}

bool FProperty::ShouldSerializeValue(FArchive& Ar) const
{
	// Skip the property if the archive says we should
	if (Ar.ShouldSkipProperty(this))
	{
		return false;
	}

	// Skip non-SaveGame properties if we're saving game state
	if (!(PropertyFlags & CPF_SaveGame) && Ar.IsSaveGame())
	{
		return false;
	}

	const uint64 SkipFlags = CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient | CPF_NonTransactional | CPF_Deprecated | CPF_DevelopmentAssets | CPF_SkipSerialization;
	if (!(PropertyFlags & SkipFlags))
	{
		return true;
	}

	// Skip properties marked Transient when persisting an object, unless we're saving an archetype
	if ((PropertyFlags & CPF_Transient) && Ar.IsPersistent() && !Ar.IsSerializingDefaults())
	{
		return false;
	}

	// Skip properties marked DuplicateTransient when duplicating
	if ((PropertyFlags & CPF_DuplicateTransient) && (Ar.GetPortFlags() & PPF_Duplicate))
	{
		return false;
	}

	// Skip properties marked NonPIEDuplicateTransient when duplicating, but not when we're duplicating for PIE
	if ((PropertyFlags & CPF_NonPIEDuplicateTransient) && !(Ar.GetPortFlags() & PPF_DuplicateForPIE) && (Ar.GetPortFlags() & PPF_Duplicate))
	{
		return false;
	}

	// Skip properties marked NonTransactional when transacting
	if ((PropertyFlags & CPF_NonTransactional) && Ar.IsTransacting())
	{
		return false;
	}

	// Skip deprecated properties when saving or transacting, unless the archive has explicitly requested them
	if ((PropertyFlags & CPF_Deprecated) && !Ar.HasAllPortFlags(PPF_UseDeprecatedProperties) && (Ar.IsSaving() || Ar.IsTransacting() || Ar.WantBinaryPropertySerialization()))
	{
		return false;
	}

	// Skip properties marked SkipSerialization, unless the archive is forcing them
	if ((PropertyFlags & CPF_SkipSerialization) && (Ar.WantBinaryPropertySerialization() || !Ar.HasAllPortFlags(PPF_ForceTaggedSerialization)))
	{
		return false;
	}

	// Skip editor-only properties when the archive is rejecting them
	if (IsEditorOnlyProperty() && Ar.IsFilterEditorOnly())
	{
#if WITH_METADATA
		// we can't tell if we are loading an optional object at this point, so we allow any AllowedInOptional properties while loading
		// and then the later code can 
		bool bLoadingWithEditorData = FPlatformProperties::HasEditorOnlyData() && Ar.IsLoading();
		bool bAllowed = (bLoadingWithEditorData || Ar.IsSavingOptionalObject()) && HasMetaData("AllowedInOptional");
		if (bAllowed)
		{
			UE_LOGF(LogInit, Display, "allowing serialize of EO property, even when filtering EO data in an Optional object - %ls", *GetFullName());
		}
#else
		bool bAllowed = false;
#endif
		return bAllowed;
	}

	// Otherwise serialize!
	return true;
}


//
// Net serialization.
//
bool FProperty::NetSerializeItem( FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData ) const
{
	SerializeItem( FStructuredArchiveFromArchive(Ar).GetSlot(), Data, NULL );
	return 1;
}

bool FProperty::SupportsNetSharedSerialization() const
{
	return true;
}

#if WITH_EDITORONLY_DATA
void FProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	AppendHash(Builder, NamePrivate);
	// Hash the EditorOnlyProperty so if a property changes from UsedInGame to EditorOnly it will still cause
	// a change in the KeepEditorOnlyProperties hash.
	uint8 FlagsRelevantToSchema = (IsEditorOnlyProperty() ? 1 : 0);
	Builder.Update(&FlagsRelevantToSchema, sizeof(FlagsRelevantToSchema));
	Builder.Update(&ArrayDim, sizeof(ArrayDim));
	AppendHash(Builder, GetID());
}
#endif


//
// Return whether the property should be exported.
//
bool FProperty::ShouldPort( uint32 PortFlags/*=0*/ ) const
{
	// if no size, don't export
	if (GetSize() <= 0)
	{
		return false;
	}

	if (HasAnyPropertyFlags(CPF_Deprecated) && !(PortFlags & (PPF_ParsingDefaultProperties | PPF_UseDeprecatedProperties)))
	{
		return false;
	}

	// if we're parsing default properties or the user indicated that transient properties should be included
	if (HasAnyPropertyFlags(CPF_Transient) && !(PortFlags & (PPF_ParsingDefaultProperties | PPF_IncludeTransient)))
	{
		return false;
	}

	// if we're copying, treat DuplicateTransient as transient
	if ((PortFlags & PPF_Copy) && HasAnyPropertyFlags(CPF_DuplicateTransient | CPF_TextExportTransient) && !(PortFlags & (PPF_ParsingDefaultProperties | PPF_IncludeTransient)))
	{
		return false;
	}

	// if we're not copying for PIE and NonPIETransient is set, don't export
	if (!(PortFlags & PPF_DuplicateForPIE) && HasAnyPropertyFlags(CPF_NonPIEDuplicateTransient))
	{
		return false;
	}

	// if we're only supposed to export components and this isn't a component property, don't export
	if ((PortFlags & PPF_SubobjectsOnly) && !ContainsInstancedObjectProperty())
	{
		return false;
	}

	// hide non-Edit properties when we're exporting for the property window
	if ((PortFlags & PPF_PropertyWindow) && !(PropertyFlags & CPF_Edit))
	{
		return false;
	}

	return true;
}

//
// Return type id for encoding properties in .u files.
//
FName FProperty::GetID() const
{
	return GetClass()->GetFName();
}

void FProperty::InstanceSubobjects( void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, struct FObjectInstancingGraph* InstanceGraph )
{
}

int32 FProperty::GetMinAlignment() const
{
	return 1;
}


//
// Link property loaded from file.
//
void FProperty::LinkInternal(FArchive& Ar)
{
	check(0); // Link shouldn't call super...and we should never link an abstract property, like this base class
}

EConvertFromTypeResult FProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	return EConvertFromTypeResult::UseSerializeItem;
}

namespace UE::CoreUObject::Private
{
	[[noreturn]] void OnInvalidPropertySize(uint32 InvalidPropertySize, const FProperty* Prop)
	{
		UE_LOGF(LogProperty, Fatal, "Invalid property size %u when linking property %ls of size %d", InvalidPropertySize, *Prop->GetFullName(), Prop->GetSize());
		for (;;);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EPropertyVisitorControlFlow FProperty::Visit(const FPropertyVisitorData& Data, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorPath&, const FPropertyVisitorData&)> InFunc) const
{
	return Visit(Data, [&InFunc](const FPropertyVisitorContext& Context) -> EPropertyVisitorControlFlow
	{
		return InFunc(Context.Path, Context.Data);
	});
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

EPropertyVisitorControlFlow FProperty::Visit(const FPropertyVisitorData& Data, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const
{
	FPropertyVisitorPath Path(FPropertyVisitorInfo(this));
	FPropertyVisitorContext Context(Path, Data);
	return Visit(Context, InFunc);
}

EPropertyVisitorControlFlow FProperty::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const
{
	checkf(Context.Path.Top().Property == this, TEXT("The property set in the info has to match to this property"));

	return InFunc(Context);
}

void* FProperty::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	FPropertyVisitorPath Path(FPropertyVisitorInfo(this));
	return PropertyVisitorHelpers::ResolveVisitedPathInfo_Generic(this, Path, Data, Info);
}

int32 FProperty::SetupOffset()
{
	UObject* OwnerUObject = GetOwner<UObject>();
	if (OwnerUObject && (OwnerUObject->GetClass()->ClassCastFlags & CASTCLASS_UStruct))
	{
		UStruct* OwnerStruct = (UStruct*)OwnerUObject;
		Offset_Internal = Align(OwnerStruct->GetPropertiesSize(), GetMinAlignment());
	}
	else
	{
		Offset_Internal = Align(0, GetMinAlignment());
	}

	uint32 UnsignedTotal = (uint32)Offset_Internal + (uint32)GetSize();
	if (UnsignedTotal >= (uint32)MAX_int32)
	{
		UE::CoreUObject::Private::OnInvalidPropertySize(UnsignedTotal, this);
	}
	return (int32)UnsignedTotal;
}

void FProperty::SetOffset_Internal(int32 NewOffset)
{
	Offset_Internal = NewOffset;
}

bool FProperty::SameType(const FProperty* Other) const
{
	return Other && (GetClass() == Other->GetClass());
}

void* FProperty::AllocateAndInitializeValue() const
{
	void* Memory = (uint8*)FMemory::MallocZeroed(GetSize(), GetMinAlignment());
	if (!HasAnyPropertyFlags(CPF_ZeroConstructor)) // this stuff is already zero
	{
		InitializeValue(Memory);
	}
	return Memory;
}

void FProperty::DestroyAndFreeValue(void* InMemory) const
{
	if (InMemory)
	{
		DestroyValue(InMemory);
		FMemory::Free(InMemory);
	}
}

void* FProperty::GetValueAddressAtIndex_Direct(const FProperty* Inner, void* InValueAddress, int32 Index) const
{
	checkf(Inner == nullptr, TEXT("%s should not have an inner property or it's missing specialized GetValueAddressAtIndex_Direct override"), *GetFullName());
	checkf(Index < ArrayDim && Index >= 0, TEXT("Array index (%d) out of range"), Index);
	return (uint8*)InValueAddress + GetElementSize() * Index;
}

void FProperty::SetSingleValue_InContainer(void* OutContainer, const void* InValue, int32 ArrayIndex) const
{
	checkf(ArrayIndex <= ArrayDim, TEXT("ArrayIndex (%d) must be less than the property %s array size (%d)"), ArrayIndex, *GetFullName(), ArrayDim);
	if (!HasSetter())
	{
		// Fast path - direct memory access
		CopySingleValue(ContainerVoidPtrToValuePtrInternal((void*)OutContainer, ArrayIndex), InValue);
	}
	else
	{
		if (ArrayDim == 1)
		{
			// Slower but no mallocs. We can copy the value directly to the resulting param
			CallSetter(OutContainer, InValue);
		}
		else
		{
			// Malloc a temp value that is the size of the array. We will then copy the entire array to the temp value
			uint8* ValueArray = (uint8*)AllocateAndInitializeValue();
			GetValue_InContainer(OutContainer, ValueArray);
			// Replace the value at the specified index in the temp array with the InValue
			CopySingleValue(ValueArray + ArrayIndex * GetElementSize(), InValue);
			// Now call a setter to replace the entire array and then destroy the temp value
			CallSetter(OutContainer, ValueArray);
			DestroyAndFreeValue(ValueArray);
		}
	}
}

void FProperty::GetSingleValue_InContainer(const void* InContainer, void* OutValue, int32 ArrayIndex) const
{
	checkf(ArrayIndex <= ArrayDim, TEXT("ArrayIndex (%d) must be less than the property %s array size (%d)"), ArrayIndex, *GetFullName(), ArrayDim);
	if (!HasGetter())
	{
		// Fast path - direct memory access
		CopySingleValue(OutValue, ContainerVoidPtrToValuePtrInternal((void*)InContainer, ArrayIndex));
	}
	else
	{
		if (ArrayDim == 1)
		{
			// Slower but no mallocs. We can copy the value directly to the resulting param
			CallGetter(InContainer, OutValue);
		}
		else
		{
			// Malloc a temp value that is the size of the array. Getter will then copy the entire array to the temp value
			uint8* ValueArray = (uint8*)AllocateAndInitializeValue();
			GetValue_InContainer(InContainer, ValueArray);
			// Copy the item we care about and free the temp array
			CopySingleValue(OutValue, ValueArray + ArrayIndex * GetElementSize());
			DestroyAndFreeValue(ValueArray);
		}
	}
}

void FProperty::PerformOperationWithSetter(void* OutContainer, void* DirectPropertyAddress, TFunctionRef<void(void*)> DirectValueAccessFunc) const
{
	if (OutContainer && HasSetterOrGetter()) // If there's a getter we need to allocate a temp value even if there's no setter
	{
		// When modifying container or struct properties that have a setter or getter function we first allocate a temp value
		// that we can operate on directly (add new elements or modify existing ones)
		void* LocalValuePtr = AllocateAndInitializeValue();
		// Copy the value to the allocated local (using a getter if present)
		GetValue_InContainer(OutContainer, LocalValuePtr);

		// Perform operation on the temp value
		DirectValueAccessFunc(LocalValuePtr);

		// Assign the temp value back to the property using a setter function
		SetValue_InContainer(OutContainer, LocalValuePtr);
		// Destroy and free the temp value
		DestroyAndFreeValue(LocalValuePtr);
	}
	else
	{
		// When there's no setter or getter present it's ok to perform the operation directly on the container / struct memory
		if (!DirectPropertyAddress)
		{
			checkf(OutContainer, TEXT("Container pointr must be valid if DirectPropertyAddress is not valid"));
			DirectPropertyAddress = PointerToValuePtr(OutContainer, EPropertyPointerType::Container);
		}
		DirectValueAccessFunc(DirectPropertyAddress);
	}
}

void FProperty::PerformOperationWithGetter(void* OutContainer, const void* DirectPropertyAddress, TFunctionRef<void(const void*)> DirectValueAccessFunc) const
{
	if (OutContainer && HasGetter())
	{
		// When modifying container or struct properties that have a getter function we first allocate a temp value
		// that we can operate on directly (add new elements or modify existing ones)
		void* LocalValuePtr = AllocateAndInitializeValue();
		// Copy the value to the allocated local using a getter
		GetValue_InContainer(OutContainer, LocalValuePtr);

		// Perform read-only operation on the temp value
		DirectValueAccessFunc(LocalValuePtr);

		// Destroy and free the temp value
		DestroyAndFreeValue(LocalValuePtr);
	}
	else
	{
		if (!DirectPropertyAddress)
		{
			checkf(OutContainer, TEXT("Container pointr must be valid if DirectPropertyAddress is not valid"));
			DirectPropertyAddress = PointerToValuePtr(OutContainer, EPropertyPointerType::Container);
		}
		DirectValueAccessFunc(DirectPropertyAddress);
	}
}

/**
 * Attempts to read an array index (xxx) sequence.  Handles const/enum replacements, etc.
 * @param	ObjectStruct	the scope of the object/struct containing the property we're currently importing
 * @param	Str				[out] pointer to the the buffer containing the property value to import
 * @param	Warn			the output device to send errors/warnings to
 * @return	the array index for this defaultproperties line.  INDEX_NONE if this line doesn't contains an array specifier, or 0 if there was an error parsing the specifier.
 */
static const int32 ReadArrayIndex(const UStruct* ObjectStruct, const TCHAR*& Str, FOutputDevice* Warn)
{
	const TCHAR* Start = Str;
	int32 Index = INDEX_NONE;
	SkipWhitespace(Str);

	if (*Str == '(' || *Str == '[')
	{
		Str++;
		FString IndexText;
		while ( *Str && *Str != ')' && *Str != ']' )
		{
			// Stop at either we reach the = or the beginning of the overridable operation ex: <replace>
			if ( *Str == TCHAR('=') || *Str == TCHAR('<'))
			{
				// we've encountered an equals sign before the closing bracket
				Warn->Logf(ELogVerbosity::Warning, TEXT("Missing ')' in default properties subscript: %s"), Start);
				return 0;
			}

			IndexText += *Str++;
		}

		if ( *Str++ )
		{
			if (IndexText.Len() > 0 )
			{
				if (FChar::IsAlpha(IndexText[0]))
				{
					FName IndexTokenName = FName(*IndexText, FNAME_Find);
					if (IndexTokenName != NAME_None)
					{
						// Search for the enum in question.
						Index = IntCastChecked<int32>(UEnum::LookupEnumName(FName(), IndexTokenName, EFindFirstObjectOptions::NativeFirst /* Only native enums can be used as array indices */));
						if (Index == INDEX_NONE)
						{
							Index = 0;
							Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid subscript in default properties: %s"), Start);
						}
					}
					else
					{
						Index = 0;

						// unknown or invalid identifier specified for array subscript
						Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid subscript in default properties: %s"), Start);
					}
				}
				else if (FChar::IsDigit(IndexText[0]))
				{
					Index = FCString::Atoi(*IndexText);
				}
				else
				{
					// unknown or invalid identifier specified for array subscript
					Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid subscript in default properties: %s"), Start);
				}
			}
			else
			{
				Index = 0;

				// nothing was specified between the opening and closing parenthesis
				Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid subscript in default properties: %s"), Start);
			}
		}
		else
		{
			Index = 0;
			Warn->Logf(ELogVerbosity::Warning, TEXT("Missing ')' in default properties subscript: %s"), Start );
		}
	}
	return Index;
}

/**
 * Attempts to read an array index (xxx) sequence.  Handles const/enum replacements, etc.
 * @param	ObjectStruct	the scope of the object/struct containing the property we're currently importing
 * @param	Str				[out] pointer to the the buffer containing the property value to import
 * @param	Warn			the output device to send errors/warnings to
 * @return	the array index for this defaultproperties line.  INDEX_NONE if this line doesn't contains an array specifier, or 0 if there was an error parsing the specifier.
 */
static TOptional<EOverriddenPropertyOperation> ReadOverriddenOperation(const TCHAR*& Str, FOutputDevice* Warn)
{
	const TCHAR* Start = Str;
	TOptional<EOverriddenPropertyOperation> Operation;
	SkipWhitespace(Str);

	if (*Str == '<')
	{
		Str++;
		FString IndexText;
		while (*Str && *Str != '>')
		{
			if (*Str == TCHAR('='))
			{
				// we've encountered an equals sign before the closing bracket
				Warn->Logf(ELogVerbosity::Warning, TEXT("Missing '>' in default properties subscript: %s"), Start);
				return Operation;
			}

			IndexText += *Str++;
		}

		if (*Str++)
		{
			if (IndexText.Len() > 0)
			{
				Operation = GetOverriddenOperationFromString(IndexText);
				if (!Operation.IsSet())
				{
					// unknown or invalid identifier specified for array subscript
					Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid overridable in default properties: %s"), Start);
				}
			}
			else
			{
				// nothing was specified between the opening and closing parenthesis
				Warn->Logf(ELogVerbosity::Warning, TEXT("Empty overridable operation: %s"), Start);
			}
		}
		else
		{
			Warn->Logf(ELogVerbosity::Warning, TEXT("Missing '>' in default properties subscript: %s"), Start );
		}
	}
	if (FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::T3DSerialization))
	{
		return Operation;
	}
	return TOptional<EOverriddenPropertyOperation>();
}


/** 
 * Do not attempt to import this property if there is no value for it - i.e. (Prop1=,Prop2=)
 * This normally only happens for empty strings or empty dynamic arrays, and the alternative
 * is for strings and dynamic arrays to always export blank delimiters, such as Array=() or String="", 
 * but this tends to cause problems with inherited property values being overwritten, especially in the localization 
 * import/export code

 * The safest way is to interpret blank delimiters as an indication that the current value should be overwritten with an empty
 * value, while the lack of any value or delimiter as an indication to not import this property, thereby preventing any current
 * values from being overwritten if this is not the intent.

 * Thus, arrays and strings will only export empty delimiters when overriding an inherited property's value with an 
 * empty value.
 */
static bool IsPropertyValueSpecified( const TCHAR* Buffer )
{
	return Buffer && *Buffer && *Buffer != TCHAR(',') && *Buffer != TCHAR(')');
}

const TCHAR* FProperty::ImportSingleProperty( const TCHAR* Str, void* DestData, const UStruct* ObjectStruct, UObject* SubobjectOuter, int32 PortFlags,
											FOutputDevice* Warn, TArray<FDefinedProperty>& DefinedProperties )
{
	check(ObjectStruct);

	constexpr FAsciiSet Whitespaces(" \t");
	constexpr FAsciiSet Delimiters("=([.<");

	// strip leading whitespace
	const TCHAR* Start = FAsciiSet::Skip(Str, Whitespaces);
	FName PropertyName;
	
	if (*Start == '"')
	{
		int32 OutQuotedLen;
		FString OutUnquotedString;
		FParse::QuotedString(Start, OutUnquotedString, &OutQuotedLen);
		PropertyName = FName(OutUnquotedString);
		
		// advance iterator to next delimiter
		Str = FAsciiSet::FindFirstOrEnd(Start + OutQuotedLen, Delimiters);
	}
	else // legacy format requires that we support un-quoted and un-escaped property names
	{
		// find first delimiter
		Str = FAsciiSet::FindFirstOrEnd(Start, Delimiters);
		// check if delimiter was found...
		if (*Str)
		{
			// strip trailing whitespace
			int32 Len = UE_PTRDIFF_TO_INT32(Str - Start);
			while (Len > 0 && Whitespaces.Contains(Start[Len - 1]))
			{
				--Len;
			}
			PropertyName = FName(Len, Start);
		}
	}
	
	if (*Str && !PropertyName.IsNone())
	{
		FProperty* Property = FindFProperty<FProperty>(ObjectStruct, PropertyName);

		if (Property == nullptr)
		{
			// Check for redirects
			FName NewPropertyName = FindRedirectedPropertyName(ObjectStruct, PropertyName);

			if (NewPropertyName != NAME_None)
			{
				Property = FindFProperty<FProperty>(ObjectStruct, NewPropertyName);
			}

			if (!Property)
			{
				Property = ObjectStruct->CustomFindProperty(PropertyName);
			}
		}		

		if (Property == NULL)
		{
			UE_SUPPRESS(LogExec, Verbose, Warn->Logf(TEXT("Unknown property in %s: %s "), *ObjectStruct->GetName(), Start));
			return Str;
		}

		if (!Property->ShouldPort(PortFlags))
		{
			UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Cannot perform text import on property '%s' here: %s"), *Property->GetName(), Start));
			return Str;
		}

		FOverridableTextPortPropertyPathScope Scope(Property);

		// Parse an array operation, if present.
		enum EArrayOp
		{
			ADO_None,
			ADO_Add,
			ADO_Remove,
			ADO_RemoveIndex,
			ADO_Empty,
		};

		EArrayOp ArrayOp = ADO_None;
		if(*Str == '.')
		{
			Str++;
			if(FParse::Command(&Str,TEXT("Empty")))
			{
				ArrayOp = ADO_Empty;
			}
			else if(FParse::Command(&Str,TEXT("Add")))
			{
				ArrayOp = ADO_Add;
			}
			else if(FParse::Command(&Str,TEXT("Remove")))
			{
				ArrayOp = ADO_Remove;
			}
			else if (FParse::Command(&Str,TEXT("RemoveIndex")))
			{
				ArrayOp = ADO_RemoveIndex;
			}
			// Special case for struct properties
			else if (FStructProperty* const StructProperty = ExactCastField<FStructProperty>(Property))
			{
				TArray<FDefinedProperty> StructDefinedProperties;
				return ImportSingleProperty(Str, Property->ContainerPtrToValuePtr<void>(DestData), StructProperty->Struct, SubobjectOuter, PortFlags, Warn, StructDefinedProperties);
			}
		}

		FArrayProperty* const ArrayProperty = ExactCastField<FArrayProperty>(Property);

		// Parse overridable info
		TOptional<EOverriddenPropertyOperation> Operation = ReadOverriddenOperation(Str, Warn);
		FOverriddenPropertySet* OverriddenProperties = nullptr;
		FPropertyVisitorPath* Path = nullptr;
		// Skip add and remove operations as they should be handled independently
		if (Operation.IsSet())
		{
			OverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties();
			Path = FOverridableSerializationLogic::GetOverriddenPortTextPropertyPath();
			if (OverriddenProperties &&
				!Property->HasAllPropertyFlags(CPF_ExperimentalAlwaysOverriden) &&
				Operation.GetValue() != EOverriddenPropertyOperation::Add && 
				Operation.GetValue() != EOverriddenPropertyOperation::Remove)
			{
				checkf(Path, TEXT("Expecting a path"));
				FArchiveSerializedPropertyChain Chain = Path->ToSerializedPropertyChain();
				OverriddenProperties->ConditionallyRestoreOverriddenPropertyOperation(Operation.GetValue(), &Chain, /*Property*/nullptr);
			}
		}

		FMulticastDelegateProperty* const MulticastDelegateProperty = CastField<FMulticastDelegateProperty>(Property);
		if( MulticastDelegateProperty != NULL && ArrayOp != ADO_None )
		{
			// Allow Add(), Remove() and Empty() on multi-cast delegates
			if( ArrayOp == ADO_Add || ArrayOp == ADO_Remove || ArrayOp == ADO_Empty )
			{
				SkipWhitespace(Str);
				if(*Str++ != '(')
				{
					UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Missing '(' in default properties multi-cast delegate operation: %s"), Start));
					return Str;
				}
				SkipWhitespace(Str);

				if( ArrayOp == ADO_Empty )
				{
					// Clear out the delegate
					MulticastDelegateProperty->ClearDelegate(SubobjectOuter, Property->ContainerPtrToValuePtr<void>(DestData));
				}
				else
				{
					FStringOutputDevice ImportError;

					const TCHAR* Result = NULL;
					if( ArrayOp == ADO_Add )
					{
						// Add a function to a multi-cast delegate
						Result = MulticastDelegateProperty->ImportText_Add(Str, Property->ContainerPtrToValuePtr<void>(DestData), PortFlags, SubobjectOuter, &ImportError);
					}
					else if( ArrayOp == ADO_Remove )
					{
						// Remove a function from a multi-cast delegate
						Result = MulticastDelegateProperty->ImportText_Remove(Str, Property->ContainerPtrToValuePtr<void>(DestData), PortFlags, SubobjectOuter, &ImportError);
					}
				
					// Spit any error we had while importing property
					if (ImportError.Len() > 0)
					{
						TArray<FString> ImportErrors;
						ImportError.ParseIntoArray(ImportErrors, LINE_TERMINATOR, true);

						for ( int32 ErrorIndex = 0; ErrorIndex < ImportErrors.Num(); ErrorIndex++ )
						{
							Warn->Logf(ELogVerbosity::Warning, TEXT("%s"), *ImportErrors[ErrorIndex]);
						}
					}
					else if (Result == NULL || Result == Str)
					{
						Warn->Logf(ELogVerbosity::Warning, TEXT("Unable to parse parameter value '%s' in defaultproperties multi-cast delegate operation: %s"), Str, Start);
					}
					// in the failure case, don't return NULL so the caller can potentially skip less and get values further in the string
					if (Result != NULL)
					{
						Str = Result;
					}

				}
			}
			else
			{
				UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Unsupported operation on multi-cast delegate variable: %s"), Start));
				return Str;
			}
			SkipWhitespace(Str);
			if (*Str != ')')
			{
				UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Missing ')' in default properties multi-cast delegate operation: %s"), Start));
				return Str;
			}
			Str++;
		}
		else if (ArrayOp != ADO_None)
		{
			if (ArrayProperty == NULL)
			{
				UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Array operation performed on non-array variable: %s"), Start));
				return Str;
			}

			FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, DestData);
			if (ArrayOp == ADO_Empty)
			{
				ArrayHelper.EmptyValues();
				SkipWhitespace(Str);
				if (*Str++ != '(')
				{
					UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Missing '(' in default properties array operation: %s"), Start));
					return Str;
				}
			}
			else if (ArrayOp == ADO_Add || ArrayOp == ADO_Remove)
			{
				SkipWhitespace(Str);
				if(*Str++ != '(')
				{
					UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Missing '(' in default properties array operation: %s"), Start));
					return Str;
				}
				SkipWhitespace(Str);

				if(ArrayOp == ADO_Add)
				{
					int32	Index = ArrayHelper.AddValue();

					const TCHAR* Result = ArrayProperty->Inner->ImportText_Direct(Str, ArrayHelper.GetRawPtr(Index), SubobjectOuter, PortFlags, Warn);
					if ( Result == NULL || Result == Str )
					{
						Warn->Logf(ELogVerbosity::Warning, TEXT("Unable to parse parameter value '%s' in defaultproperties array operation: %s"), Str, Start);
						return Str;
					}
					else
					{
						Str = Result;
					}
				}
				else
				{
					int32 Size = ArrayProperty->Inner->GetElementSize();

					uint8* Temp = (uint8*)FMemory_Alloca(Size);
					ArrayProperty->Inner->InitializeValue(Temp);
							
					// export the value specified to a temporary buffer
					const TCHAR* Result = ArrayProperty->Inner->ImportText_Direct(Str, Temp, SubobjectOuter, PortFlags, Warn);
					if ( Result == NULL || Result == Str )
					{
						Warn->Logf(ELogVerbosity::Error, TEXT("Unable to parse parameter value '%s' in defaultproperties array operation: %s"), Str, Start);
						ArrayProperty->Inner->DestroyValue(Temp);
						return Str;
					}
					else
					{
						// find the array member corresponding to this value
						bool bFound = false;
						for(uint32 Index = 0;Index < (uint32)ArrayHelper.Num();Index++)
						{
							const void* ElementDestData = ArrayHelper.GetRawPtr(Index);
							if(ArrayProperty->Inner->Identical(Temp,ElementDestData))
							{
								ArrayHelper.RemoveValues(Index--);
								bFound = true;
							}
						}
						if (!bFound)
						{
							Warn->Logf(ELogVerbosity::Warning, TEXT("%s.Remove(): Value not found in array"), *ArrayProperty->GetName());
						}
						ArrayProperty->Inner->DestroyValue(Temp);
						Str = Result;
					}
				}
			}
			else if (ArrayOp == ADO_RemoveIndex) //-V547
			{
				SkipWhitespace(Str);
				if(*Str++ != '(')
				{
					UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Missing '(' in default properties array operation:: %s"), Start ));
					return Str;
				}
				SkipWhitespace(Str);

				FString strIdx;
				while (*Str != ')')
				{		
					if (*Str == 0)
					{
						UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Missing ')' in default properties array operation: %s"), Start));
						return Str;
					}
					strIdx += *Str;
					Str++;
				}
				int32 removeIdx = FCString::Atoi(*strIdx);
				if (ArrayHelper.IsValidIndex(removeIdx))
				{
					ArrayHelper.RemoveValues(removeIdx);
				}
				else
				{
					Warn->Logf(ELogVerbosity::Warning, TEXT("%s.RemoveIndex(%d): Index not found in array"), *ArrayProperty->GetName(), removeIdx);
				}
			}
			SkipWhitespace(Str);
			if (*Str != ')')
			{
				UE_SUPPRESS(LogExec, Warning, Warn->Logf(TEXT("Missing ')' in default properties array operation: %s"), Start));
				return Str;
			}
			Str++;
		}
		else
		{
			// try to read an array index
			int32 Index = ReadArrayIndex(ObjectStruct, Str, Warn);

			// strip whitespace before =
			SkipWhitespace(Str);
			if (*Str++ != '=')
			{
				Warn->Logf(ELogVerbosity::Warning, TEXT("Missing '=' in default properties assignment: %s"), Start );
				return Str;
			}
			// strip whitespace after =
			SkipWhitespace(Str);

			auto ImportText = [SubobjectOuter, PortFlags, Warn, Start, ObjectStruct](const TCHAR* Buffer, const FProperty* Property, void* PropertyPtr)
			{
				FStringOutputDevice ImportError;
				const TCHAR* Result = Property->ImportText_Direct(Buffer, PropertyPtr, SubobjectOuter, PortFlags, &ImportError);

				// Spit any error we had while importing property
				if (ImportError.Len() > 0)
				{
					TArray<FString> ImportErrors;
					ImportError.ParseIntoArray(ImportErrors,LINE_TERMINATOR,true);

					Warn->Logf(ELogVerbosity::Warning, TEXT("While importing text for property '%s' in '%s':"), *Property->GetName(), *ObjectStruct->GetName());
					for (int32 ErrorIndex = 0; ErrorIndex < ImportErrors.Num(); ErrorIndex++)
					{
						Warn->Logf(ELogVerbosity::Warning, TEXT("%s"), *ImportErrors[ErrorIndex]);
					}
				}
				else if (Result == Buffer && *Buffer == TCHAR('\0'))
				{
					Warn->Logf(ELogVerbosity::Warning, TEXT("Invalid property value in defaults: %s"), Start);
				}
				return Result;
			};

			// Avoid shadowed reference values (not directly imported) - callers should advance
			// @todo - IDOs will need to capture the referenced instance to help avoid data loss
			if (Operation.IsSet() && Operation.GetValue() == EOverriddenPropertyOperation::SubObjectsShadowing)
			{
				return Str;
			}

			// Handle modifying overridable operations on arrays
			if (Operation.IsSet() && Operation.GetValue() != EOverriddenPropertyOperation::Replace)
			{
				if (ArrayProperty)
				{
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, DestData);
					switch(Operation.GetValue())
					{
						case EOverriddenPropertyOperation::Remove:
						{
							checkf(ArrayProperty->Inner->HasAnyPropertyFlags(CPF_PersistentInstance), TEXT("Only instanced sub object is supporting remove operation"));
							const FObjectPropertyBase* InnerObjectProperty = CastFieldChecked<FObjectPropertyBase>(ArrayProperty->Inner);

							uint8* TempValueStorage = (uint8*)FMemory_Alloca(InnerObjectProperty->GetElementSize());
							InnerObjectProperty->InitializeValue(TempValueStorage);

							const TCHAR* Result = ImportText(Str, InnerObjectProperty, TempValueStorage);
							if (UObject* RemovedSubObject = InnerObjectProperty->GetObjectPropertyValue(TempValueStorage))
							{
								const int32 ArrayNum = ArrayHelper.Num();
								for (int i = 0; i < ArrayNum; ++i)
								{
									UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetElementPtr(i));
									if (CurrentObject->GetArchetype() == RemovedSubObject)
									{
										ArrayHelper.RemoveValues(i);
										break;
									}
								}

								if (Path)
								{
									// Need to fetch the ArrayOverriddenPropertyNode every loop as the previous iteration might have reallocated the node.
									FArchiveSerializedPropertyChain Chain = Path->ToSerializedPropertyChain();
									if (FOverriddenPropertyNode* ArrayOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, &Chain, /*Property*/nullptr) : nullptr)
									{
										// Rebuild the overridden info
										OverriddenProperties->RestoreSubObjectOperation(EOverriddenPropertyOperation::Remove, *ArrayOverriddenPropertyNode, RemovedSubObject);
									}
								}
							}

							InnerObjectProperty->DestroyValue(TempValueStorage);
							return Result;
						}
						case EOverriddenPropertyOperation::Add:
						{
							// Special case for instanced sub objects
							if (const FObjectPropertyBase* InnerObjectProperty =  ArrayProperty->Inner->HasAnyPropertyFlags(CPF_PersistentInstance) ? CastField<FObjectPropertyBase>(ArrayProperty->Inner) : nullptr)
							{
								uint8* TempValueStorage = (uint8*)FMemory_Alloca(InnerObjectProperty->GetElementSize());
								InnerObjectProperty->InitializeValue(TempValueStorage);

								const TCHAR* Result = ImportText(Str, InnerObjectProperty, TempValueStorage);
								if (UObject* AddedSubObject = InnerObjectProperty->GetObjectPropertyValue(TempValueStorage))
								{
									UObject* AddedSubObjectArchetype = AddedSubObject->GetArchetype();
									const int32 ArrayNum = ArrayHelper.Num();
									for (int i = 0; i < ArrayNum; ++i)
									{
										UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetElementPtr(i));
										if (CurrentObject == AddedSubObject || (CurrentObject == AddedSubObjectArchetype))
										{
											Index = i;
											break;
										}
									}

									if (Index == INDEX_NONE)
									{
										Index = ArrayHelper.Num();
										ArrayHelper.ExpandForIndex(Index);
									}

									InnerObjectProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(Index), AddedSubObject);

									if (Path)
									{
										// Need to fetch the ArrayOverriddenPropertyNode every loop as the previous iteration might have reallocated the node.
										FArchiveSerializedPropertyChain Chain = Path->ToSerializedPropertyChain();
										if (FOverriddenPropertyNode* ArrayOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, &Chain, /*Property*/nullptr) : nullptr)
										{
											// Rebuild the overridden info
											OverriddenProperties->RestoreSubObjectOperation(EOverriddenPropertyOperation::Add, *ArrayOverriddenPropertyNode, AddedSubObject);
										}
									}
								}

								InnerObjectProperty->DestroyValue(TempValueStorage);
								return Result;
							}

							Index = ArrayHelper.Num();
							ArrayHelper.ExpandForIndex(Index);
							break;
						}
						case EOverriddenPropertyOperation::Externalized:
						{
							checkf(ArrayProperty->Inner->HasAnyPropertyFlags(CPF_PersistentInstance), TEXT("Only instanced sub object is supporting externalized operation"));
							const FObjectPropertyBase* InnerObjectProperty = CastFieldChecked<FObjectPropertyBase>(ArrayProperty->Inner);

							if (Path)
							{
								FArchiveSerializedPropertyChain Chain = Path->ToSerializedPropertyChain();
								if (FOverriddenPropertyNode* ArrayOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, &Chain, /*Property*/nullptr) : nullptr)
								{
									OverriddenProperties->RestoreSubPropertyOperation(EOverriddenPropertyOperation::Externalized, *ArrayOverriddenPropertyNode, FOverriddenPropertyNodeID());
								}
							}

							break;
						}
						default:
							checkf(false, TEXT("Unsupported array operation while importing text"));
							return nullptr;
					}
				}
				else if (FMapProperty* const MapProperty = ExactCastField<FMapProperty>(Property))
				{
					FScriptMapHelper_InContainer MapHelper(MapProperty, DestData);

					uint8* TempKeyValueStorage = (uint8*)FMemory_Alloca(MapProperty->MapLayout.SetLayout.Size);
					MapProperty->KeyProp->InitializeValue(TempKeyValueStorage);
					MapProperty->ValueProp->InitializeValue(TempKeyValueStorage + MapProperty->MapLayout.ValueOffset);
					ON_SCOPE_EXIT
					{
						MapProperty->KeyProp->DestroyValue(TempKeyValueStorage);
						MapProperty->ValueProp->DestroyValue(TempKeyValueStorage + MapProperty->MapLayout.ValueOffset);
					};

					switch(Operation.GetValue())
					{
						case EOverriddenPropertyOperation::Remove:
						{
							const TCHAR* Result = ImportText(Str, MapHelper.KeyProp, TempKeyValueStorage);
							MapHelper.RemovePair(TempKeyValueStorage);

							if (Path)
							{
								// Need to fetch the MapOverriddenPropertyNode every loop as the previous might have reallocated the node.
								FArchiveSerializedPropertyChain Chain = Path->ToSerializedPropertyChain();
								if (FOverriddenPropertyNode* MapOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, &Chain, /*Property*/nullptr) : nullptr)
								{
									// Rebuild the overridden info
									FOverriddenPropertyNodeID RemovedKeyID = FOverriddenPropertyNodeID::FromMapKey(MapProperty->KeyProp, TempKeyValueStorage);
									OverriddenProperties->RestoreSubPropertyOperation(EOverriddenPropertyOperation::Remove, *MapOverriddenPropertyNode, RemovedKeyID);
								}
							}

							return Result;
						}
						case EOverriddenPropertyOperation::Modified:
						{
							const TCHAR* Result = Str;

							SkipWhitespace(Result);
							if (*Result++ != TCHAR('('))
							{
								return nullptr;
							}
							SkipWhitespace(Result);

							Result = ImportText(Result, MapHelper.KeyProp, TempKeyValueStorage);
							const int32 InternalIndex = MapHelper.FindMapPairIndexFromHash(TempKeyValueStorage);

							SkipWhitespace(Result);
							if (*Result++ != TCHAR(','))
							{
								return nullptr;
							}

							uint8* ValuePtr = InternalIndex != INDEX_NONE ? MapHelper.GetValuePtr(InternalIndex) : TempKeyValueStorage + MapHelper.MapLayout.ValueOffset;
							Result = ImportText(Result, MapHelper.ValueProp, ValuePtr);

							SkipWhitespace(Result);
							if (*Result++ != TCHAR(')'))
							{
								return nullptr;
							}

							if (Path)
							{
								// Need to fetch the MapOverriddenPropertyNode every loop as the previous might have reallocated the node.
								FArchiveSerializedPropertyChain Chain = Path->ToSerializedPropertyChain();
								if (FOverriddenPropertyNode* MapOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, &Chain, /*Property*/nullptr) : nullptr)
								{
									// Rebuild the overridden info
									FOverriddenPropertyNodeID ModifiedKeyID = FOverriddenPropertyNodeID::FromMapKey(MapProperty->KeyProp, TempKeyValueStorage);
									OverriddenProperties->RestoreSubPropertyOperation(EOverriddenPropertyOperation::Modified, *MapOverriddenPropertyNode, ModifiedKeyID);
								}
							}
							return Result;
						}
						case EOverriddenPropertyOperation::Add:
						{
							const TCHAR* Result = Str;
							SkipWhitespace(Result);
							if (*Result++ != TCHAR('('))
							{
								return nullptr;
							}
							SkipWhitespace(Result);

							Result = ImportText(Result, MapHelper.KeyProp, TempKeyValueStorage);

							SkipWhitespace(Result);
							if (*Result++ != TCHAR(','))
							{
								return nullptr;
							}
							SkipWhitespace(Result);

							void* ValuePtr =  MapHelper.FindOrAdd(TempKeyValueStorage);;
							Result = ImportText(Result, MapHelper.ValueProp, ValuePtr);

							SkipWhitespace(Result);
							if (*Result++ != TCHAR(')'))
							{
								return nullptr;
							}

							if (Path)
							{
								// Need to fetch the MapOverriddenPropertyNode every loop as the previous might have reallocated the node.
								FArchiveSerializedPropertyChain Chain = Path->ToSerializedPropertyChain();
								if (FOverriddenPropertyNode* MapOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, &Chain, /*Property*/nullptr) : nullptr)
								{
									// Rebuild the overridden info
									FOverriddenPropertyNodeID AddedKeyID = FOverriddenPropertyNodeID::FromMapKey(MapProperty->KeyProp, TempKeyValueStorage);
									OverriddenProperties->RestoreSubPropertyOperation(EOverriddenPropertyOperation::Add, *MapOverriddenPropertyNode, AddedKeyID);
								}
							}
							return Result;
						}
						default:
							checkf(false, TEXT("Unsupported map operation while importing text"));
							return nullptr;
					}
				}
			}

			// check for out of bounds on static arrays
			if (ArrayProperty == nullptr && Index >= Property->ArrayDim)
			{
				Warn->Logf(ELogVerbosity::Warning, TEXT("Out of bound array default property (%i/%i): %s"), Index, Property->ArrayDim, Start);
				return Str;
			}

			// check to see if this property has already imported data
			FDefinedProperty D;
			D.Property = Property;
			D.Index = Index;
			if (DefinedProperties.Find(D) != INDEX_NONE)
			{
				Warn->Logf(ELogVerbosity::Warning, TEXT("redundant data: %s"), Start);
				return Str;
			}
			DefinedProperties.Add(D);

			if (!IsPropertyValueSpecified(Str) && ArrayProperty == nullptr)
			{
				// if we're not importing default properties for classes (i.e. we're pasting something in the editor or something)
				// and there is no property value for this element, skip it, as that means that the value of this element matches
				// the intrinsic null value of the property type and we want to skip importing it
				return Str;
			}

			// disallow importing of an object's name from here
			// not done above with ShouldPort() check because this is intentionally exported so we don't want it to cause errors on import
			if (Property->GetFName() != NAME_Name || !Property->GetOwnerVariant().IsUObject() || Property->GetOwner<UObject>()->GetFName() != NAME_Object)
			{
				if (Index > -1 && ArrayProperty != nullptr) //set single dynamic array element
				{
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, DestData);

					ArrayHelper.ExpandForIndex(Index);

					const TCHAR* Result = ImportText(Str, ArrayProperty->Inner, ArrayHelper.GetRawPtr(Index));
					// in the failure case, don't return NULL so the caller can potentially skip less and get values further in the string
					if (Result != nullptr)
					{
						Str = Result;
					}
				}
				else if (FEnumProperty* const EnumProperty = ExactCastField<FEnumProperty>(Property))
				{
					//We need to support the case where a bitmask enum is surrounded by Quotes. Non-struct properties
					//remove quotes before passing to ImportText, so we do the same here by checking for a leading
					//quote, and then consuming the closing quote after importing the enum property string

					if (Index == INDEX_NONE)
					{
						Index = 0;
					}

					SkipWhitespace(Str);
					
					bool bQuoteToBeConsumed = false;
					if (*Str == TCHAR('"'))
					{
						Str++;
						bQuoteToBeConsumed = true;
					}

					const TCHAR* Result = ImportText(Str, Property, Property->ContainerPtrToValuePtr<void>(DestData, Index));

#if WITH_EDITORONLY_DATA
					UE::FInitializedPropertyValueState(ObjectStruct, DestData).Set(Property, Index);
#endif

					// in the failure case, don't return NULL so the caller can potentially skip less and get values further in the string
					if (Result != nullptr)
					{
						if (bQuoteToBeConsumed)
						{
							SkipWhitespace(Result);
							if (*Result == TCHAR('"'))
							{
								Str = Result + 1;
							}
							else
							{
								Warn->Logf(ELogVerbosity::Warning, TEXT("Property delineated by quotes is missing ending quote: %s"), Start);
								Str = Result;
							}
						}
						else
						{
							Str = Result;
						}
					}
				}
				else
				{
					if (Index == INDEX_NONE)
					{
						Index = 0;
					}


					const TCHAR* Result = ImportText(Str, Property, Property->ContainerPtrToValuePtr<void>(DestData, Index));

#if WITH_EDITORONLY_DATA
					UE::FInitializedPropertyValueState(ObjectStruct, DestData).Set(Property, Index);
#endif
					
					// in the failure case, don't return NULL so the caller can potentially skip less and get values further in the string
					if (Result != nullptr)
					{
						Str = Result;
					}
				}
			}
		}
	}
	return Str;
}

#if WITH_EDITORONLY_DATA
static FCoreRedirectObjectName BuildCoreRedirectPropertyName(FName PropertyName, UE::FPropertyTypeName TypeName)
{
	FName OuterName = TypeName.GetName();
	if (const int32 OuterCount = TypeName.GetParameterCount() - 1; OuterCount > 0)
	{
		TStringBuilder<256> OuterChain;
		for (int32 OuterIndex = 0; OuterIndex < OuterCount; ++OuterIndex)
		{
			OuterChain << TypeName.GetParameterName(OuterIndex + 1) << (OuterIndex == 0 ? TEXT(':') : TEXT('.'));
		}
		OuterChain << OuterName;
		OuterName = FName(OuterChain);
	}

	const FName PackageName = TypeName.GetParameterName(0);
	return FCoreRedirectObjectName(PropertyName, OuterName, PackageName);
}
#endif // WITH_EDITORONLY_DATA

FName FProperty::FindRedirectedPropertyName(const UStruct* ObjectStruct, FName OldName)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FProperty::FindRedirectedPropertyName"), STAT_LinkerLoad_FindRedirectedPropertyName, STATGROUP_LoadTimeVerbose);

	// ObjectStruct may be a nested struct, so extract path
	UPackage* StructPackage = ObjectStruct->GetOutermost();
	FName PackageName = StructPackage->GetFName();
	// Avoid GetPathName string allocation and FName initialization when there is only one outer
	FName OuterName = (StructPackage == ObjectStruct->GetOuter()) ? ObjectStruct->GetFName() : FName(ObjectStruct->GetPathName(StructPackage));

	FCoreRedirectObjectName OldRedirectName(OldName, OuterName, PackageName);
	FCoreRedirectObjectName NewRedirectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Property, OldRedirectName);
	if (NewRedirectName != OldRedirectName)
	{
		return NewRedirectName.ObjectName;
	}

#if WITH_EDITORONLY_DATA
	// Compare against the original type of an impersonated type.
	if (const UE::FPropertyTypeName OriginalType = UE::FindOriginalType(ObjectStruct); !OriginalType.IsEmpty())
	{
		OldRedirectName = BuildCoreRedirectPropertyName(OldName, OriginalType);
		NewRedirectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Property, OldRedirectName);
		if (NewRedirectName != OldRedirectName)
		{
			return NewRedirectName.ObjectName;
		}
	}
#endif // WITH_EDITORONLY_DATA


	return NAME_None;
}

/**
 * Returns the hash value for an element of this property.
 */
uint32 FProperty::GetValueTypeHash(const void* Src) const
{
	check(PropertyFlags & CPF_HasGetValueTypeHash); // make sure the type is hashable
	check(Src);
	return GetValueTypeHashInternal(Src);
}

void FProperty::CopyValuesInternal( void* Dest, void const* Src, int32 Count  ) const
{
	check(0); // if you are not memcpyable, then you need to deal with the virtual call
}

uint32 FProperty::GetValueTypeHashInternal(const void* Src) const
{
	checkf(false, TEXT("Missing implementation for GetValueTypeHashInternal with ID %s and Name %s"), *GetID().ToString(), *GetName()); // you need to deal with the virtual call
	return 0;
}

void FProperty::InitializeIntrusiveUnsetOptionalValue(void* Data) const 
{
	checkf(false, TEXT("Missing implementation for InitializeIntrusiveUnsetOptionalValue for property type returning true from HasIntrusiveUnsetOptionalState"));
}

bool FProperty::IsIntrusiveOptionalValueSet(const void* Data) const 
{
	checkf(false, TEXT("Missing implementation for IsIntrusiveOptionalValueSet for property type returning true from HasIntrusiveUnsetOptionalState"));
	return false;
}

void FProperty::ClearIntrusiveOptionalValue(void* Data) const 
{
	checkf(false, TEXT("Missing implementation for ClearIntrusiveOptionalValue for property type returning true from HasIntrusiveUnsetOptionalState"));
}

void FProperty::EmitIntrusiveOptionalReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath)
{
	checkf(false, TEXT("Missing implementation for EmitIntrusiveOptionalReferenceInfo for property type returning true from HasIntrusiveUnsetOptionalState"));
}

#if WITH_EDITORONLY_DATA
UPropertyWrapper* FProperty::GetUPropertyWrapper()
{
	UStruct* OwnerStruct = GetOwnerStruct();
	UPropertyWrapper* Wrapper = nullptr;
	if (OwnerStruct)
	{
		// Find an existing wrapper object
		for (UPropertyWrapper* ExistingWrapper : OwnerStruct->PropertyWrappers)
		{
			if (ExistingWrapper->GetProperty() == this)
			{
				Wrapper = ExistingWrapper;
				break;
			}
		}
		if (!Wrapper)
		{
			// Try to find the class of a new wrapper object mathich this property's class
			FString WrapperClassName = GetClass()->GetName();
			WrapperClassName += TEXT("Wrapper");
			UClass* WrapperClass = Cast<UClass>(StaticFindObjectFast(UClass::StaticClass(), UPackage::StaticClass()->GetOutermost(), *WrapperClassName));
			if (!WrapperClass)
			{
				// Default to generic wrapper class
				WrapperClass = UPropertyWrapper::StaticClass();
			}
			Wrapper = NewObject<UPropertyWrapper>(OwnerStruct, WrapperClass, *FString::Printf(TEXT("%sWrapper"), *GetName()));
			check(Wrapper);
			Wrapper->SetProperty(this);
			OwnerStruct->PropertyWrappers.Add(Wrapper);
		}
	}
	return Wrapper;
}
#endif //  WITH_EDITORONLY_DATA

bool FProperty::UseBinaryOrNativeSerialization(const FArchive& Ar) const
{
	return Ar.WantBinaryPropertySerialization();
}

bool FProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	return ensureMsgf(GetID() == Type.GetName(),
		TEXT("Failed to load property '%s' of type '%s' from type name '%s'"),
		*WriteToString<64>(GetFName()), *WriteToString<64>(GetID()), *WriteToString<64>(Type.GetName()));
}

void FProperty::SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const
{
	Type.AddName(GetID());
}

bool FProperty::CanSerializeFromTypeName(UE::FPropertyTypeName Type) const
{
	return Type.GetName() == GetID();
}

FProperty* UStruct::FindPropertyByName(FName InName) const
{
	for (FProperty* Property = PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
	{
		if (Property->GetFName() == InName)
		{
			return Property;
		}
	}

	return nullptr;
}

FProperty* UStruct::FindPropertyByOffset(int32 Offset) const
{
	for (FProperty* Property = PropertyLink; Property != nullptr; Property = Property->PropertyLinkNext)
	{
		if (Property->GetOffset_ForInternal() == Offset)
		{
			return Property;
		}
	}

	return nullptr;
}
