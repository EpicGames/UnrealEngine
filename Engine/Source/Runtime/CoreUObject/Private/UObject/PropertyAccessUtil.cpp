// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyAccessUtil.h"
#include "UObject/EnumProperty.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Misc/DefaultValueHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyAccessUtil)

namespace UE::PropertyAccessUtil::Private
{

bool IsRealNumberConversion(const FProperty* InSrcProp, const FProperty* InDestProp)
{
	check(InSrcProp);
	check(InDestProp);

	const bool bIsRealNumberConversion =
		(InSrcProp->IsA<FDoubleProperty>() && InDestProp->IsA<FFloatProperty>()) ||
		(InSrcProp->IsA<FFloatProperty>() && InDestProp->IsA<FDoubleProperty>());

	return bIsRealNumberConversion;
}

void ConvertRealNumber(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue, int InCount)
{
	check(InSrcProp);
	check(InSrcValue);
	check(InDestProp);
	check(InDestValue);
	check(InCount > 0);
	check(InCount <= InSrcProp->ArrayDim);

	if (const FDoubleProperty* SrcDoubleProp = CastField<FDoubleProperty>(InSrcProp))
	{
		const FFloatProperty* DestFloatProp = CastFieldChecked<FFloatProperty>(InDestProp);
		for (int32 Idx = 0; Idx < InCount; ++Idx)
		{
			const void* SrcElemValue = static_cast<const uint8*>(InSrcValue) + (InSrcProp->GetElementSize() * Idx);
			void* DestElemValue = static_cast<uint8*>(InDestValue) + (InDestProp->GetElementSize() * Idx);

			const double Value = SrcDoubleProp->GetFloatingPointPropertyValue(SrcElemValue);
			DestFloatProp->SetFloatingPointPropertyValue(DestElemValue, Value);
		}
	}
	else if (const FFloatProperty* SrcFloatProp = CastField<FFloatProperty>(InSrcProp))
	{
		const FDoubleProperty* DestDoubleProp = CastFieldChecked<FDoubleProperty>(InDestProp);
		for (int32 Idx = 0; Idx < InCount; ++Idx)
		{
			const void* SrcElemValue = static_cast<const uint8*>(InSrcValue) + (InSrcProp->GetElementSize() * Idx);
			void* DestElemValue = static_cast<uint8*>(InDestValue) + (InDestProp->GetElementSize() * Idx);

			const double Value = SrcFloatProp->GetFloatingPointPropertyValue(SrcElemValue);
			DestDoubleProp->SetFloatingPointPropertyValue(DestElemValue, Value);
		}
	}
	else
	{
		checkf(false, TEXT("Invalid property type used with ConvertRealNumber!"));
	}
}

bool AreRealNumbersIdentical(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, const void* InDestValue)
{
	check(InSrcProp);
	check(InSrcValue);
	check(InDestProp);
	check(InDestValue);

	bool bIdentical = false;

	if (const FDoubleProperty* SrcDoubleProp = CastField<FDoubleProperty>(InSrcProp))
	{
		const FFloatProperty* DestFloatProp = CastFieldChecked<FFloatProperty>(InDestProp);

		const double SrcValue = SrcDoubleProp->GetFloatingPointPropertyValue(InSrcValue);
		const double DestValue = DestFloatProp->GetFloatingPointPropertyValue(InDestValue);

		bIdentical = (SrcValue == DestValue);
		
	}
	else if (const FFloatProperty* SrcFloatProp = CastField<FFloatProperty>(InSrcProp))
	{
		const FDoubleProperty* DestDoubleProp = CastFieldChecked<FDoubleProperty>(InDestProp);
		
		const double SrcValue = SrcFloatProp->GetFloatingPointPropertyValue(InSrcValue);
		const double DestValue = DestDoubleProp->GetFloatingPointPropertyValue(InDestValue);

		bIdentical = (SrcValue == DestValue);
	}
	else
	{
		checkf(false, TEXT("Invalid property type used with AreRealNumbersIdentical!"));
	}

	return bIdentical;
}

}

namespace PropertyAccessUtil
{

const UEnum* GetPropertyEnumType(const FProperty* InProp)
{
	if (const FByteProperty* ByteProp = CastField<FByteProperty>(InProp))
	{
		return ByteProp->Enum;
	}
	if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(InProp))
	{
		return EnumProp->GetEnum();
	}
	return nullptr;
}

int64 GetPropertyEnumValue(const FProperty* InProp, const void* InPropValue)
{
	if (const FByteProperty* ByteProp = CastField<FByteProperty>(InProp))
	{
		return ByteProp->GetSignedIntPropertyValue(InPropValue);
	}
	if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(InProp))
	{
		return EnumProp->GetUnderlyingProperty()->GetSignedIntPropertyValue(InPropValue);
	}
	return INDEX_NONE;
}

bool SetPropertyEnumValue(const FProperty* InProp, void* InPropValue, const int64 InEnumValue)
{
	if (const FByteProperty* ByteProp = CastField<FByteProperty>(InProp))
	{
		ByteProp->SetPropertyValue(InPropValue, (uint8)InEnumValue);
		return true;
	}
	if (const FEnumProperty* EnumProp = CastField<FEnumProperty>(InProp))
	{
		EnumProp->GetUnderlyingProperty()->SetIntPropertyValue(InPropValue, InEnumValue);
		return true;
	}
	return false;
}

bool ArePropertiesCompatible(const FProperty* InSrcProp, const FProperty* InDestProp)
{
	using namespace UE::PropertyAccessUtil::Private;

	// Enum properties can either be a ByteProperty with an enum set, or an EnumProperty
	// We allow coercion between these two types if they're using the same enum type
	if (const UEnum* DestEnumType = GetPropertyEnumType(InDestProp))
	{
		const UEnum* SrcEnumType = GetPropertyEnumType(InSrcProp);
		if (SrcEnumType == DestEnumType)
		{
			return true;
		}

		// Blueprints don't always set the Enum field on the ByteProperty when setting properties, so we also 
		// allow assigning from a raw ByteProperty (for type safety there we rely on the compiler frontend)
		const FByteProperty* SrcByteProp = CastField<FByteProperty>(InSrcProp);
		if (SrcByteProp && !SrcByteProp->Enum && InDestProp->IsA<FEnumProperty>())
		{
			return true;
		}
	}

	if (IsRealNumberConversion(InSrcProp, InDestProp))
	{
		return true;
	}

	// Compare the classes as these must be an *exact* match as the access is low-level and without property coercion
	if (InSrcProp->GetClass() != InDestProp->GetClass())
	{
		return false;
	}

	// Containers also need to check their inner types
	if (const FArrayProperty* SrcArrayProp = CastField<FArrayProperty>(InSrcProp))
	{
		const FArrayProperty* DestArrayProp = CastFieldChecked<FArrayProperty>(InDestProp);
		return ArePropertiesCompatible(SrcArrayProp->Inner, DestArrayProp->Inner);
	}
	if (const FSetProperty* SrcSetProp = CastField<FSetProperty>(InSrcProp))
	{
		const FSetProperty* DestSetProp = CastFieldChecked<FSetProperty>(InDestProp);
		return ArePropertiesCompatible(SrcSetProp->ElementProp, DestSetProp->ElementProp);
	}
	if (const FMapProperty* SrcMapProp = CastField<FMapProperty>(InSrcProp))
	{
		const FMapProperty* DestMapProp = CastFieldChecked<FMapProperty>(InDestProp);
		return ArePropertiesCompatible(SrcMapProp->KeyProp, DestMapProp->KeyProp)
			&& ArePropertiesCompatible(SrcMapProp->ValueProp, DestMapProp->ValueProp);
	}

	return true;
}

bool IsSinglePropertyIdentical(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, const void* InDestValue)
{
	using namespace UE::PropertyAccessUtil::Private;

	if (!ArePropertiesCompatible(InSrcProp, InDestProp))
	{
		return false;
	}

	if (const FBoolProperty* SrcBoolProp = CastField<FBoolProperty>(InSrcProp))
	{
		const FBoolProperty* DestBoolProp = CastFieldChecked<FBoolProperty>(InDestProp);

		// Bools can be represented as bitfields, we we have to handle the compare a little differently to only check the bool we want
		const bool bSrcBoolValue = SrcBoolProp->GetPropertyValue(InSrcValue);
		const bool bDestBoolValue = DestBoolProp->GetPropertyValue(InDestValue);
		return bSrcBoolValue == bDestBoolValue;
	}

	if (IsRealNumberConversion(InSrcProp, InDestProp))
	{
		return AreRealNumbersIdentical(InSrcProp, InSrcValue, InDestProp, InDestValue);
	}

	return InSrcProp->Identical(InSrcValue, InDestValue);
}

bool IsCompletePropertyIdentical(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, const void* InDestValue)
{
	bool bIsIdentical = InSrcProp->ArrayDim == InDestProp->ArrayDim;
	for (int32 Idx = 0; Idx < InSrcProp->ArrayDim && bIsIdentical; ++Idx)
	{
		const void* SrcElemValue = static_cast<const uint8*>(InSrcValue) + (InSrcProp->GetElementSize() * Idx);
		const void* DestElemValue = static_cast<const uint8*>(InDestValue) + (InDestProp->GetElementSize() * Idx);
		bIsIdentical &= IsSinglePropertyIdentical(InSrcProp, SrcElemValue, InDestProp, DestElemValue);
	}
	return bIsIdentical;
}

bool CopySinglePropertyValue(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue)
{
	using namespace UE::PropertyAccessUtil::Private;

	if (!ArePropertiesCompatible(InSrcProp, InDestProp))
	{
		return false;
	}

	// Enum properties can either be a ByteProperty with an enum set, or an EnumProperty
	// We allow coercion between these two types as long as they're using the same enum type (as validated by ArePropertiesCompatible)
	if (const UEnum* EnumType = GetPropertyEnumType(InDestProp))
	{
		const int64 SrcEnumValue = GetPropertyEnumValue(InSrcProp, InSrcValue);
		return SetPropertyEnumValue(InDestProp, InDestValue, SrcEnumValue);
	}

	if (const FBoolProperty* SrcBoolProp = CastField<FBoolProperty>(InSrcProp))
	{
		const FBoolProperty* DestBoolProp = CastFieldChecked<FBoolProperty>(InDestProp);

		// Bools can be represented as bitfields, we we have to handle the copy a little differently to only extract the bool we want
		const bool bBoolValue = SrcBoolProp->GetPropertyValue(InSrcValue);
		DestBoolProp->SetPropertyValue(InDestValue, bBoolValue);
		return true;
	}

	if (IsRealNumberConversion(InSrcProp, InDestProp))
	{
		ConvertRealNumber(InSrcProp, InSrcValue, InDestProp, InDestValue, 1);
		return true;
	}

	InSrcProp->CopySingleValue(InDestValue, InSrcValue);
	return true;
}

bool CopyCompletePropertyValue(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue)
{
	using namespace UE::PropertyAccessUtil::Private;

	if (!ArePropertiesCompatible(InSrcProp, InDestProp) || InSrcProp->ArrayDim != InDestProp->ArrayDim)
	{
		if (InDestProp->ArrayDim > 1)
		{
			// handle assignment of a dynamic array to a fixed array:
			if (const FArrayProperty* SrcArray = CastField<FArrayProperty>(InSrcProp))
			{
				if (ArePropertiesCompatible(SrcArray->Inner, InDestProp))
				{
					FScriptArrayHelper SrcArrayHelper(SrcArray, InSrcValue);
					if (SrcArrayHelper.Num() == InDestProp->ArrayDim)
					{
						for (int32 I = 0; I < InDestProp->ArrayDim; ++I)
						{
							void* DestValue = static_cast<uint8*>(InDestValue) + InDestProp->GetElementSize() * I;
							CopySinglePropertyValue(SrcArray->Inner, SrcArrayHelper.GetElementPtr(I), InDestProp, DestValue);
						}
						return true;
					}
				}
			}
		}
		else if (InSrcProp->ArrayDim > 1)
		{
			// handle assignment of a fixed array to a dynamic array:
			if (const FArrayProperty* DstArray = CastField<FArrayProperty>(InDestProp))
			{
				if (ArePropertiesCompatible(DstArray->Inner, InSrcProp))
				{
					FScriptArrayHelper DstArrayHelper(DstArray, InDestValue);
					DstArrayHelper.Resize(InSrcProp->ArrayDim);
					for (int32 I = 0; I < InSrcProp->ArrayDim; ++I)
					{
						const void* SrcValue = static_cast<const uint8*>(InSrcValue) + InSrcProp->GetElementSize() * I;
						CopySinglePropertyValue(InSrcProp, SrcValue, DstArray->Inner, DstArrayHelper.GetElementPtr(I));
					}
					return true;
				}
			}
		}
		return false;
	}

	// Enum properties can either be a ByteProperty with an enum set, or an EnumProperty
	// We allow coercion between these two types as long as they're using the same enum type (as validated by ArePropertiesCompatible)
	if (const UEnum* EnumType = GetPropertyEnumType(InDestProp))
	{
		bool bSuccess = true;
		for (int32 Idx = 0; Idx < InSrcProp->ArrayDim; ++Idx)
		{
			const void* SrcElemValue = static_cast<const uint8*>(InSrcValue) + (InSrcProp->GetElementSize() * Idx);
			void* DestElemValue = static_cast<uint8*>(InDestValue) + (InDestProp->GetElementSize() * Idx);

			const int64 SrcEnumValue = GetPropertyEnumValue(InSrcProp, SrcElemValue);
			bSuccess &= SetPropertyEnumValue(InDestProp, DestElemValue, SrcEnumValue);
		}
		return bSuccess;
	}

	if (const FBoolProperty* SrcBoolProp = CastField<FBoolProperty>(InSrcProp))
	{
		const FBoolProperty* DestBoolProp = CastFieldChecked<FBoolProperty>(InDestProp);
		for (int32 Idx = 0; Idx < InSrcProp->ArrayDim; ++Idx)
		{
			const void* SrcElemValue = static_cast<const uint8*>(InSrcValue) + (InSrcProp->GetElementSize() * Idx);
			void* DestElemValue = static_cast<uint8*>(InDestValue) + (InDestProp->GetElementSize() * Idx);

			// Bools can be represented as bitfields, we we have to handle the copy a little differently to only extract the bool we want
			const bool bBoolValue = SrcBoolProp->GetPropertyValue(SrcElemValue);
			DestBoolProp->SetPropertyValue(DestElemValue, bBoolValue);
		}
		return true;
	}
	
	if (IsRealNumberConversion(InSrcProp, InDestProp))
	{
		ConvertRealNumber(InSrcProp, InSrcValue, InDestProp, InDestValue, InSrcProp->ArrayDim);
		return true;
	}

	InSrcProp->CopyCompleteValue(InDestValue, InSrcValue);
	return true;
}

EPropertyAccessResultFlags GetPropertyValue_Object(const FProperty* InObjectProp, const UObject* InObject, const FProperty* InDestProp, void* InDestValue, const int32 InArrayIndex)
{
	check(InObject->IsA(InObjectProp->GetOwnerClass()));
	return GetPropertyValue_InContainer(InObjectProp, InObject, InDestProp, InDestValue, InArrayIndex);
}

EPropertyAccessResultFlags GetPropertyValue_InContainer(const FProperty* InContainerProp, const void* InContainerData, const FProperty* InDestProp, void* InDestValue, const int32 InArrayIndex)
{
	if (InArrayIndex == INDEX_NONE || InContainerProp->ArrayDim == 1)
	{
		const void* SrcValue = InContainerProp->ContainerPtrToValuePtr<void>(InContainerData);
		return GetPropertyValue_DirectComplete(InContainerProp, SrcValue, InDestProp, InDestValue);
	}
	else
	{
		check(InArrayIndex < InContainerProp->ArrayDim);
		const void* SrcValue = InContainerProp->ContainerPtrToValuePtr<void>(InContainerData, InArrayIndex);
		return GetPropertyValue_DirectSingle(InContainerProp, SrcValue, InDestProp, InDestValue);
	}
}

EPropertyAccessResultFlags GetPropertyValue_DirectSingle(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue)
{
	EPropertyAccessResultFlags Result = CanGetPropertyValue(InSrcProp);
	if (Result != EPropertyAccessResultFlags::Success)
	{
		return Result;
	}

	return GetPropertyValue([InSrcProp, InSrcValue, InDestProp, InDestValue]()
	{
		return CopySinglePropertyValue(InSrcProp, InSrcValue, InDestProp, InDestValue);
	});
}

EPropertyAccessResultFlags GetPropertyValue_DirectComplete(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue)
{
	EPropertyAccessResultFlags Result = CanGetPropertyValue(InSrcProp);
	if (Result != EPropertyAccessResultFlags::Success)
	{
		return Result;
	}

	return GetPropertyValue([InSrcProp, InSrcValue, InDestProp, InDestValue]()
	{
		return CopyCompletePropertyValue(InSrcProp, InSrcValue, InDestProp, InDestValue);
	});
}

EPropertyAccessResultFlags GetPropertyValue(const FPropertyAccessGetFunc& InGetFunc)
{
	const bool bGetResult = InGetFunc();
	return bGetResult
		? EPropertyAccessResultFlags::Success
		: EPropertyAccessResultFlags::ConversionFailed;
}

EPropertyAccessResultFlags CanGetPropertyValue(const FProperty* InProp)
{
	if (!InProp->HasAnyPropertyFlags(CPF_Edit | CPF_BlueprintVisible | CPF_BlueprintAssignable))
	{
		return EPropertyAccessResultFlags::PermissionDenied | EPropertyAccessResultFlags::AccessProtected;
	}

	return EPropertyAccessResultFlags::Success;
}

bool GetArchetypeInstancesInheritingPropertyValue(const FProperty* InObjectProp, UObject* InObject, TArray<UObject*>& OutArchetypeInstances)
{
	// Clear result container
	OutArchetypeInstances.Empty();

	if (!InObjectProp || !InObject)
	{
		// Invalid parameters
		return false;
	}

	if (!InObject->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		// InObject isn't an archetype
		return false;
	}

	// Get all archetype instances, which includes instances through multiple levels of inheritance
	TArray<UObject*> Instances;
	InObject->GetArchetypeInstances(Instances);

	//
	const bool bIsSparseProp = !InObject->GetClass()->HasProperty(InObjectProp);

	// Transform object address to the property's reference value address
	const void* ObjectContainerData = bIsSparseProp ? InObject->GetClass()->GetOrCreateSparseClassData() : InObject;
	const void* ObjectReferenceValue = InObjectProp->ContainerPtrToValuePtr<void>(ObjectContainerData);

	// Find archetype instances that inherited the property value, which is true when:
	// - It has the same value as the archetype
	// - And this is true for every archetype between InObject (base archetype) and Inst
	// Since archetype instances may be visited multiple times - directly or as archetype 
	// of another instance - we'll cache their property comparison result.
	TSet<UObject*> SameValueInstances;
	TSet<UObject*> DifferentValueInstances;
	for (UObject* CandidateInstance : Instances)
	{
		// If we've already determined for this instance that it has a different value while processing it 
		// as the archetype for an earlier iterated instance, ignore it
		if (DifferentValueInstances.Contains(CandidateInstance))
		{
			continue;
		}

		// Visit archetype upwards until InObject is reached or abort if any archetype with a wrong value is encountered
		UObject* InstIter = CandidateInstance;
		bool bInheritsArchetypeValue = true;
		do
		{
			if (SameValueInstances.Contains(InstIter))
			{
				// Instance has been evaluated during another processing of another instance,
				// and property value was the same. Continue traveling up.
			}
			else if (DifferentValueInstances.Contains(InstIter))
			{
				// Instance has been evaluated during another processing of another instance,
				// and property value was different. This candidate is rejected.
				bInheritsArchetypeValue = false;
				continue;
			}
			else
			{
				// Instance hasn't been evaluated, compare properties now.

				// Get the container of the property: either the object itself or if the property is stored externally
				// in the class's sparse data, get the sparse data container.
				void* ContainerData = bIsSparseProp ? InstIter->GetClass()->GetOrCreateSparseClassData() : InstIter;
				if (ContainerData != nullptr)
				{
					// Get the value pointer inside the container
					void* InstanceValue = InObjectProp->ContainerPtrToValuePtr<void>(ContainerData);

					if (PropertyAccessUtil::IsCompletePropertyIdentical(InObjectProp, ObjectReferenceValue, InObjectProp, InstanceValue))
					{
						SameValueInstances.Add(InstIter);
					}
					else
					{
						DifferentValueInstances.Add(InstIter);
						bInheritsArchetypeValue = false;
						continue;
					}
				}
				else
				{
					// Could not access sparse class data. At moment of writing this, a child blueprint class with sparse data
					// can have a null SparseDataClass in editor until it's compiled manually at least once. Since the data is
					// yet to be initialized, ignore the candidate now. Serialization will take care of inheritance later.
					bInheritsArchetypeValue = false;
					continue;
				}
			}

			// Process archetype of this instance, it may be the base archetype but there may be
			// multiple levels of archetypes in-between.
			InstIter = InstIter->GetArchetype();
			ensureMsgf(InstIter, TEXT("Unexpected: while traveling up archetypes did not encounter the base archetype"));
		} while (InstIter && InstIter != InObject && bInheritsArchetypeValue);

		if (bInheritsArchetypeValue && InstIter == InObject)
		{
			OutArchetypeInstances.Add(CandidateInstance);
		}
	}

	// Successfully searched for archetype instances
	return true;
}

bool GetArchetypeInstancesInheritingPropertyValue_AsContainerData(const FProperty* InObjectProp, UObject* InObject, TArray<void*>& OutArchetypeInstContainers)
{
	OutArchetypeInstContainers.Empty();

	// First get all the archetype UObjects that inherit InObject's value
	TArray<UObject*> CandidateInstances;
	GetArchetypeInstancesInheritingPropertyValue(InObjectProp, InObject, CandidateInstances);

	// If the property is part of the UClass's SparseClassDataStruct, the value is not stored in the UObject's allocated
	// memory but externally in the sparse data struct. Check which is the case and then output the mem address of the
	// relevant container (UObject, or the sparse data struct).
	const bool bIsSparseProp = !InObject->GetClass()->HasProperty(InObjectProp);
	const void* ObjectContainerData = bIsSparseProp ? InObject->GetClass()->GetOrCreateSparseClassData() : InObject;
	for (UObject* CandidateInstance : CandidateInstances)
	{
		if (bIsSparseProp)
		{
			// If the property is stored in sparse data struct, resolve address to that data.
			void* CandidateContainerData = CandidateInstance->GetClass()->GetOrCreateSparseClassData();

			// Objects of the same class share the sparse data struct. If this candidate shares the data struct 
			// with the reference object, ignore it since the reference object will already be updated.
			if (CandidateContainerData != ObjectContainerData)
			{
				// Instances of the same class share the sparse data struct, so make sure to add
				// mem addresses uniquely.
				OutArchetypeInstContainers.AddUnique(CandidateContainerData);
			}
		}
		else
		{
			// If not a sparse property, the data is stored in the instance itself so add the instance
			// pointer as container.
			OutArchetypeInstContainers.Add(CandidateInstance);
		}
	}

	return false;
}

EPropertyAccessResultFlags SetPropertyValue_Object(const FProperty* InObjectProp, UObject* InObject, const FProperty* InSrcProp, const void* InSrcValue, const int32 InArrayIndex, const uint64 InReadOnlyFlags, const EPropertyAccessChangeNotifyMode InNotifyMode)
{
	check(InObject->IsA(InObjectProp->GetOwnerClass()));

	// If the object is an archetype, we'll propagate the value change to instances that currently inheriting the object's value.
	// Collect all archetype instances that satisfy that condition.
	TArray<void*> PropInheritingInstContainers;
	if (InObject->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		GetArchetypeInstancesInheritingPropertyValue_AsContainerData(InObjectProp, InObject, PropInheritingInstContainers);
	}

	return SetPropertyValue_InContainer(InObjectProp, InObject, PropInheritingInstContainers, InSrcProp, InSrcValue, InArrayIndex, InReadOnlyFlags, IsObjectTemplate(InObject), [InObjectProp, InObject, InNotifyMode]()
	{
		return BuildBasicChangeNotify(InObjectProp, InObject, InNotifyMode, /*ChangeType=*/EPropertyChangeType::ValueSet);
	});
}

EPropertyAccessResultFlags SetPropertyValue_InContainer(const FProperty* InContainerProp, void* InContainerData, const TArray<void*>& InArchetypeInstContainerData, const FProperty* InSrcProp, const void* InSrcValue, const int32 InArrayIndex, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const FPropertyAccessBuildChangeNotifyFunc& InBuildChangeNotifyFunc)
{
	// If the object we're setting the property value of is an archetype, InArchetypeInstContainerData will have been
	// populated with pointers to archetype instances that also need their property value set. Transform the container 
	// pointers of archetype instances to their value pointers.
	TArray<void*> ArchetypeInstDestValues;
	if (InOwnerIsTemplate)
	{
		ArchetypeInstDestValues.Reserve(InArchetypeInstContainerData.Num());
		for (void* InstContainerData : InArchetypeInstContainerData)
		{
			void* InstValuePtr = InContainerProp->ContainerPtrToValuePtr<void>(InstContainerData);
			ArchetypeInstDestValues.Add(InstValuePtr);
		}
	}

	if (InArrayIndex == INDEX_NONE || InContainerProp->ArrayDim == 1)
	{
		void* DestValue = InContainerProp->ContainerPtrToValuePtr<void>(InContainerData);
		return SetPropertyValue_DirectComplete(InSrcProp, InSrcValue, InContainerProp, DestValue, ArchetypeInstDestValues, InReadOnlyFlags, InOwnerIsTemplate, InBuildChangeNotifyFunc);
	}
	else
	{
		check(InArrayIndex < InContainerProp->ArrayDim);
		void* DestValue = InContainerProp->ContainerPtrToValuePtr<void>(InContainerData, InArrayIndex);
		return SetPropertyValue_DirectSingle(InSrcProp, InSrcValue, InContainerProp, DestValue, ArchetypeInstDestValues, InReadOnlyFlags, InOwnerIsTemplate, InBuildChangeNotifyFunc);
	}
}

EPropertyAccessResultFlags SetPropertyValue_DirectSingle(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue, const TArray<void*>& InArchetypeInstDestValues, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const FPropertyAccessBuildChangeNotifyFunc& InBuildChangeNotifyFunc)
{
	EPropertyAccessResultFlags Result = CanSetPropertyValue(InDestProp, InReadOnlyFlags, InOwnerIsTemplate);
	if (Result != EPropertyAccessResultFlags::Success)
	{
		return Result;
	}

	return SetPropertyValue([InSrcProp, InSrcValue, InDestProp, InDestValue, InArchetypeInstDestValues, InOwnerIsTemplate](const FPropertyAccessChangeNotify* InChangeNotify)
	{
		bool bResult = true;
		const bool bIdenticalValue = IsSinglePropertyIdentical(InSrcProp, InSrcValue, InDestProp, InDestValue);
		EmitPreChangeNotify(InChangeNotify, bIdenticalValue);
		if (!bIdenticalValue)
		{
			// Special case: modifying an archetype, process instances
			if (InOwnerIsTemplate)
			{
				// Fill archetype instances who currently have the same value as the archetype
				for (void* ArchetypeInstDestValue : InArchetypeInstDestValues)
				{
					// Only expecting to get archetype instances passed in whose current value is the same as the pre-change archetype value
					ensureMsgf(IsSinglePropertyIdentical(InDestProp, InDestValue, InDestProp, ArchetypeInstDestValue), TEXT("SetPropertyValue modifying archetype instance which had non-inherited value"));

					// Write the new property to the archetype instance
					CopySinglePropertyValue(InSrcProp, InSrcValue, InDestProp, ArchetypeInstDestValue);
				}
			}
			
			bResult = CopySinglePropertyValue(InSrcProp, InSrcValue, InDestProp, InDestValue);
		}
		EmitPostChangeNotify(InChangeNotify, bIdenticalValue);

		return bResult;
	}, InBuildChangeNotifyFunc);
}

EPropertyAccessResultFlags SetPropertyValue_DirectComplete(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue, const TArray<void*>& InArchetypeInstDestValues, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const FPropertyAccessBuildChangeNotifyFunc& InBuildChangeNotifyFunc)
{
	EPropertyAccessResultFlags Result = CanSetPropertyValue(InDestProp, InReadOnlyFlags, InOwnerIsTemplate);
	if (Result != EPropertyAccessResultFlags::Success)
	{
		return Result;
	}

	return SetPropertyValue([InSrcProp, InSrcValue, InDestProp, InDestValue, InArchetypeInstDestValues, InOwnerIsTemplate](const FPropertyAccessChangeNotify* InChangeNotify)
	{
		bool bResult = true;
		const bool bIdenticalValue = IsCompletePropertyIdentical(InSrcProp, InSrcValue, InDestProp, InDestValue);
		EmitPreChangeNotify(InChangeNotify, bIdenticalValue);
		if (!bIdenticalValue)
		{
			// Special case: modifying an archetype, process instances
			if (InOwnerIsTemplate)
			{
				// Fill archetype instances who currently have the same value as the archetype
				for (void* ArchetypeInstDestValue : InArchetypeInstDestValues)
				{
#if DO_ENSURE
					// Only expecting to get archetype instances passed in whose current value is the same as the pre-change archetype value
					if (!IsCompletePropertyIdentical(InDestProp, InDestValue, InDestProp, ArchetypeInstDestValue))
					{
						FString A, B;
						InDestProp->ExportTextItem_Direct(A, InDestValue, nullptr, nullptr, 0);
						InDestProp->ExportTextItem_Direct(B, ArchetypeInstDestValue, nullptr, nullptr, 0);
						ensureMsgf(IsCompletePropertyIdentical(InDestProp, InDestValue, InDestProp, ArchetypeInstDestValue), TEXT("SetPropertyValue modifying archetype instance which had non-inherited value: %s (template) vs. %s (instance)"), *A, *B);
					}
#endif

					// Write the new property to the archetype instance
					CopyCompletePropertyValue(InSrcProp, InSrcValue, InDestProp, ArchetypeInstDestValue);
				}
			}

			bResult = CopyCompletePropertyValue(InSrcProp, InSrcValue, InDestProp, InDestValue);
		}
		EmitPostChangeNotify(InChangeNotify, bIdenticalValue);

		return bResult;
	}, InBuildChangeNotifyFunc);
}

EPropertyAccessResultFlags SetPropertyValue(const FPropertyAccessSetFunc& InSetFunc, const FPropertyAccessBuildChangeNotifyFunc& InBuildChangeNotifyFunc)
{
	TUniquePtr<FPropertyAccessChangeNotify> ChangeNotify = InBuildChangeNotifyFunc();
	const bool bSetResult = InSetFunc(ChangeNotify.Get());
	return bSetResult
		? EPropertyAccessResultFlags::Success
		: EPropertyAccessResultFlags::ConversionFailed;
}

EPropertyAccessResultFlags CanSetPropertyValue(const FProperty* InProp, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate)
{
	if (!InProp->HasAnyPropertyFlags(CPF_Edit | CPF_BlueprintVisible | CPF_BlueprintAssignable))
	{
		return EPropertyAccessResultFlags::PermissionDenied | EPropertyAccessResultFlags::AccessProtected;
	}

	if (InOwnerIsTemplate)
	{
		if (InProp->HasAnyPropertyFlags(CPF_DisableEditOnTemplate))
		{
			return EPropertyAccessResultFlags::PermissionDenied | EPropertyAccessResultFlags::CannotEditTemplate;
		}
	}
	else
	{
		if (InProp->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
		{
			return EPropertyAccessResultFlags::PermissionDenied | EPropertyAccessResultFlags::CannotEditInstance;
		}
	}

	if (InProp->HasAnyPropertyFlags(InReadOnlyFlags))
	{
		return EPropertyAccessResultFlags::PermissionDenied | EPropertyAccessResultFlags::ReadOnly;
	}

	return EPropertyAccessResultFlags::Success;
}

void EmitPreChangeNotify(const FPropertyAccessChangeNotify* InChangeNotify, const bool InIdenticalValue)
{
#if WITH_EDITOR
	if (InChangeNotify && InChangeNotify->NotifyMode != EPropertyAccessChangeNotifyMode::Never)
	{
		check(InChangeNotify->ChangedObject);

		if (!InIdenticalValue || InChangeNotify->NotifyMode == EPropertyAccessChangeNotifyMode::Always)
		{
			// Notify that a change is about to occur
			InChangeNotify->ChangedObject->PreEditChange(const_cast<FEditPropertyChain&>(InChangeNotify->ChangedPropertyChain));
		}
	}
#endif
}

void EmitPostChangeNotify(const FPropertyAccessChangeNotify* InChangeNotify, const bool InIdenticalValue)
{
#if WITH_EDITOR
	if (InChangeNotify && InChangeNotify->NotifyMode != EPropertyAccessChangeNotifyMode::Never)
	{
		check(InChangeNotify->ChangedObject);

		if (!InIdenticalValue || InChangeNotify->NotifyMode == EPropertyAccessChangeNotifyMode::Always)
		{
			// ArrayIndicesPerObject storage must outlive PropertyEvent and PropertyChainEvent below.
			TArray<TMap<FString, int32>> ArrayIndicesPerObject;

			// Notify that the change has occurred
			FPropertyChangedEvent PropertyEvent(InChangeNotify->ChangedPropertyChain.GetActiveNode()->GetValue(), InChangeNotify->ChangeType, MakeArrayView(&InChangeNotify->ChangedObject, 1));
			PropertyEvent.SetActiveMemberProperty(InChangeNotify->ChangedPropertyChain.GetActiveMemberNode()->GetValue());

			if (!InChangeNotify->ElementIndicesMap.IsEmpty())
			{
				ArrayIndicesPerObject.Add(InChangeNotify->ElementIndicesMap);
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				PropertyEvent.SetArrayIndexPerObject(MakeArrayView(ArrayIndicesPerObject));
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				PropertyEvent.ObjectIteratorIndex = 0;
			}

			FPropertyChangedChainEvent PropertyChainEvent(const_cast<FEditPropertyChain&>(InChangeNotify->ChangedPropertyChain), PropertyEvent);
			InChangeNotify->ChangedObject->PostEditChangeChainProperty(PropertyChainEvent);
		}
	}
#endif
}

TUniquePtr<FPropertyAccessChangeNotify> BuildBasicChangeNotify(const FProperty* InProp, const UObject* InObject, const EPropertyAccessChangeNotifyMode InNotifyMode, const EPropertyChangeType::Type ChangeType)
{
	const UScriptStruct* SparseStruct = InObject->GetClass()->GetSparseClassDataStruct();
	const bool bIsValidSparseProp = SparseStruct &&
		SparseStruct->IsChildOf(InProp->GetOwnerStruct());
	check(InObject->IsA(InProp->GetOwnerClass()) || bIsValidSparseProp);
#if WITH_EDITOR
	if (InNotifyMode != EPropertyAccessChangeNotifyMode::Never)
	{
		TUniquePtr<FPropertyAccessChangeNotify> ChangeNotify = MakeUnique<FPropertyAccessChangeNotify>();
		ChangeNotify->ChangeType = ChangeType;
		ChangeNotify->ChangedObject = const_cast<UObject*>(InObject);
		ChangeNotify->ChangedPropertyChain.AddHead(const_cast<FProperty*>(InProp));
		ChangeNotify->ChangedPropertyChain.SetActivePropertyNode(const_cast<FProperty*>(InProp));
		ChangeNotify->ChangedPropertyChain.SetActiveMemberPropertyNode(const_cast<FProperty*>(InProp));
		ChangeNotify->NotifyMode = InNotifyMode;
		return ChangeNotify;
	}
#endif
	return nullptr;
}

bool IsObjectTemplate(const UObject* InObject)
{
	// Templates can edit default properties
	if (InObject->IsTemplate())
	{
		return true;
	}
	
	// Assets can edit default properties, filter out objects with external packages as they are treated as assets but aren't
	if (!InObject->IsPackageExternal() && InObject->IsAsset())
	{
		return true;
	}

	// Objects within an asset that are edit-inline can edit default properties, as this mimics the inlining that the details panel shows
	if (InObject->GetClass()->HasAnyClassFlags(CLASS_EditInlineNew))
	{
		for (const UObject* Outer = InObject->GetOuter(); Outer; Outer = Outer->GetOuter())
		{
			if (Outer->IsAsset())
			{
				return true;
			}
		}
	}

	return false;
}

FProperty* FindPropertyByName(const FName InPropName, const UStruct* InStruct)
{
	FProperty* Prop = InStruct->FindPropertyByName(InPropName);

	if (!Prop)
	{
		const FName NewPropName = FProperty::FindRedirectedPropertyName(const_cast<UStruct*>(InStruct), InPropName);
		if (!NewPropName.IsNone())
		{
			Prop = InStruct->FindPropertyByName(NewPropName);
		}
	}

	if (!Prop)
	{
		Prop = InStruct->CustomFindProperty(InPropName);
	}

	return Prop;
}

bool ImportDefaultPropertyValue(const FProperty* InProp, void* InPropValue, const FString& InDefaultValue, FOutputDevice* ErrorText)
{
	if (InDefaultValue.IsEmpty() && !(InProp->IsA<FStrProperty>() || InProp->IsA<FTextProperty>()))
	{
		return false;
	}

	bool bImportedText = false;

	// Certain struct types export using a non-standard default value, so we have to import them manually rather than use ImportText
	if (const FStructProperty* StructProp = CastField<FStructProperty>(InProp))
	{
		if (StructProp->Struct == TBaseStructure<FVector>::Get())
		{
			FVector* Vector = (FVector*)InPropValue;
			bImportedText = FDefaultValueHelper::ParseVector(InDefaultValue, *Vector);
		}
		else if (StructProp->Struct == TBaseStructure<FVector2D>::Get())
		{
			FVector2D* Vector2D = (FVector2D*)InPropValue;
			bImportedText = FDefaultValueHelper::ParseVector2D(InDefaultValue, *Vector2D);
		}
		else if (StructProp->Struct == TBaseStructure<FRotator>::Get())
		{
			FRotator* Rotator = (FRotator*)InPropValue;
			bImportedText = FDefaultValueHelper::ParseRotator(InDefaultValue, *Rotator);
		}
		else if (StructProp->Struct == TBaseStructure<FColor>::Get())
		{
			FColor* Color = (FColor*)InPropValue;
			bImportedText = FDefaultValueHelper::ParseColor(InDefaultValue, *Color);
		}
		else if (StructProp->Struct == TBaseStructure<FLinearColor>::Get())
		{
			FLinearColor* LinearColor = (FLinearColor*)InPropValue;
			bImportedText = FDefaultValueHelper::ParseLinearColor(InDefaultValue, *LinearColor);
		}
	}

	if (!bImportedText)
	{
		bImportedText = InProp->ImportText_Direct(*InDefaultValue, InPropValue, nullptr, PPF_None, ErrorText) != nullptr;
	}

	return bImportedText;
}

}
