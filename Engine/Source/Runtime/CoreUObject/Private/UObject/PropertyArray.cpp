// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "Misc/ScopeExit.h"
#include "Templates/Greater.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/OverriddenObjectsExternalHandler.h"
#include "UObject/PropertyHelper.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectThreadContext.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteExecutor.h"
#endif

/*-----------------------------------------------------------------------------
	FArrayProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FArrayProperty)

FArrayProperty::FArrayProperty(FFieldVariant InOwner, const FName& InName, EArrayPropertyFlags InArrayPropertyFlags)
	: Super(InOwner, InName)
	, Inner(nullptr)
{
	ArrayFlags = InArrayPropertyFlags;
	SetElementSize(TTypeFundamentals::CPPSize);
}

FArrayProperty::FArrayProperty(FFieldVariant InOwner, const UECodeGen_Private::FArrayPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
	, Inner(nullptr)
{
	ArrayFlags = Prop.ArrayFlags;
	SetElementSize(TTypeFundamentals::CPPSize);
}

#if WITH_EDITORONLY_DATA
FArrayProperty::FArrayProperty(UField* InField)
	: Super(InField)
	, ArrayFlags(EArrayPropertyFlags::None)
{
	UArrayProperty* SourceProperty = CastChecked<UArrayProperty>(InField);
	Inner = CastField<FProperty>(SourceProperty->Inner->GetAssociatedFField());
	if (!Inner)
	{
		Inner = CastField<FProperty>(CreateFromUField(SourceProperty->Inner));
		SourceProperty->Inner->SetAssociatedFField(Inner);
	}
}
#endif // WITH_EDITORONLY_DATA

FArrayProperty::~FArrayProperty()
{
	delete Inner;
	Inner = nullptr;
}

void FArrayProperty::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	if (Inner)
	{
		Inner->GetPreloadDependencies(OutDeps);
	}
}

void FArrayProperty::PostDuplicate(const FField& InField)
{
	const FArrayProperty& Source = static_cast<const FArrayProperty&>(InField);
	Inner = CastFieldChecked<FProperty>(FField::Duplicate(Source.Inner, this));
	Super::PostDuplicate(InField);
}

void FArrayProperty::LinkInternal(FArchive& Ar)
{
	//FLinkerLoad* MyLinker = GetLinker();
	//if (MyLinker)
	//{
	//	MyLinker->Preload(this);
	//}
	//Ar.Preload(Inner);
	Inner->Link(Ar);

	SetElementSize(TTypeFundamentals::CPPSize);
}
bool FArrayProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	checkSlow(Inner);

	FScriptArrayHelper ArrayHelperA(this, A);

	const int32 ArrayNum = ArrayHelperA.Num();
	if (B == nullptr)
	{
		return ArrayNum == 0;
	}

	FScriptArrayHelper ArrayHelperB(this, B);
	if (ArrayNum != ArrayHelperB.Num())
	{
		return false;
	}

	for (int32 ArrayIndex = 0; ArrayIndex < ArrayNum; ArrayIndex++)
	{
		if (!Inner->Identical(ArrayHelperA.GetRawPtr(ArrayIndex), ArrayHelperB.GetRawPtr(ArrayIndex), PortFlags))
		{
			return false;
		}
	}

	return true;
}

static bool CanBulkSerialize(FProperty* Property)
{
#if PLATFORM_LITTLE_ENDIAN
	// All numeric properties except TEnumAsByte
	uint64 CastFlags = Property->GetClass()->GetCastFlags();
	if (!!(CastFlags & CASTCLASS_FNumericProperty))
	{
		bool bEnumAsByte = (CastFlags & CASTCLASS_FByteProperty) != 0 && static_cast<FByteProperty*>(Property)->Enum;
		return !bEnumAsByte;
	}
#endif

	return false;
}

void FArrayProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	check(Inner);
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	const bool bIsTextFormat = UnderlyingArchive.IsTextFormat();
	const bool bUPS = UnderlyingArchive.UseUnversionedPropertySerialization();
	bool bExperimentalOverridableLogic = HasAnyPropertyFlags(CPF_ExperimentalOverridableLogic);
	FUObjectSerializeContext* Context = FUObjectThreadContext::Get().GetSerializeContext();
	TOptional<FPropertyTag> MaybeInnerTag;

	// Ensure that the Inner itself has been loaded before calling SerializeItem() on it
	//UnderlyingArchive.Preload(Inner);

	FScriptArrayHelper ArrayHelper(this, Value);
	int32 ElementCount = ArrayHelper.Num();

	// Custom branch for UPS to try and take advantage of bulk serialization
	if (bUPS)
	{
		checkf(!UnderlyingArchive.ArUseCustomPropertyList, TEXT("Custom property lists are not supported with UPS"));
		checkf(!bIsTextFormat, TEXT("Text-based archives are not supported with UPS"));

		if (CanBulkSerialize(Inner))
		{
			// We need to enter the slot as *something* to keep the structured archive system happy,
			// but which maps down to straight writes to the underlying archive.
			FStructuredArchiveStream Stream = Slot.EnterStream();

			Stream.EnterElement() << ElementCount;

			if (UnderlyingArchive.IsLoading())
			{
				ArrayHelper.EmptyAndAddUninitializedValues(ElementCount);
			}

			Stream.EnterElement().Serialize(ArrayHelper.GetRawPtr(), ElementCount * Inner->GetElementSize());
		}
		else
		{
			FStructuredArchiveArray Array = Slot.EnterArray(ElementCount);

			if (UnderlyingArchive.IsLoading())
			{
				ArrayHelper.EmptyAndAddValues(ElementCount);
			}

			FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Inner, this);
			for (int32 i = 0; i < ElementCount; ++i)
			{
#if WITH_EDITOR
				static const FName NAME_UArraySerialize = FName("FArrayProperty::Serialize");
				FName NAME_UArraySerializeCount = FName(NAME_UArraySerialize);
				NAME_UArraySerializeCount.SetNumber(i);
				FArchive::FScopeAddDebugData P(UnderlyingArchive, NAME_UArraySerializeCount);
#endif
				Inner->SerializeItem(Array.EnterElement(), ArrayHelper.GetRawPtr(i));
			}
		}

		return;
	}

	if (bIsTextFormat && Inner->IsA<FStructProperty>() && UnderlyingArchive.UEVer() < EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
	{
		MaybeInnerTag.Emplace(Inner, /*Index*/ 0, (uint8*)NotNullGet(Value));

		FName StructName;
		FGuid StructGuid;
		Slot << SA_ATTRIBUTE(TEXT("InnerStructName"), StructName);
		Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("InnerStructGuid"), StructGuid, FGuid());

		UE::FPropertyTypeNameBuilder Builder;
		Builder.AddName(NAME_StructProperty);
		Builder.BeginParameters();
		Builder.AddName(StructName);
		if (StructGuid.IsValid())
		{
			Builder.AddGuid(StructGuid);
		}
		Builder.EndParameters();
		MaybeInnerTag->SetType(Builder.Build());
	}

	TOptional<FPropertyTag> SerializeFromMismatchedTag;

	auto SerializeContainerItem = [this, &SerializeFromMismatchedTag](FStructuredArchiveSlot Slot, uint8* Item)
	{
		if (const FPropertyTag* Tag = SerializeFromMismatchedTag.GetPtrOrNull())
		{
			const int64 StartOfProperty = Slot.GetUnderlyingArchive().Tell();
			FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Inner);
			switch (StructProperty->ConvertFromType(*Tag, Slot, Item, nullptr, nullptr))
			{
				case EConvertFromTypeResult::Converted:
				case EConvertFromTypeResult::Serialized:
					return;
				case EConvertFromTypeResult::CannotConvert:
					// FStructProperty::ConvertFromType doesn't handle setting the default, so do it here.
					StructProperty->Struct->InitializeDefaultValue(Item);
					Slot.GetUnderlyingArchive().Seek(StartOfProperty + Tag->Size);	// Skip this item
					return;
				case EConvertFromTypeResult::UseSerializeItem:
					// Fall through to default SerializeItem
					break;
			}
		}

		Inner->SerializeItem(Slot, Item);
	};

	const FPropertyTag* PropertyTag = FPropertyTagScope::GetCurrentPropertyTag();

	// Make sure the container is reloading accordingly to the value set in the property tag if any
	if (UnderlyingArchive.IsLoading() && PropertyTag)
	{
		bExperimentalOverridableLogic = PropertyTag->bExperimentalOverridableLogic;
	}

	// *** Experimental *** Special serialization path for array with overridable serialization
	if (bExperimentalOverridableLogic)
	{
		checkf(!UnderlyingArchive.ArUseCustomPropertyList, TEXT("Using custom property list is not supported by overridable serialization"));

		// Container for temporarily tracking some indices & communicating with external objects handler
		FOverridableObjectArrayOverrides	ContainerOverrides;

		const FObjectPropertyBase* InnerObjectProperty = CastField<FObjectPropertyBase>(Inner);
		
		// this indicates whether or not we can use external objects overrides in this particular serializing situation
		const bool bCouldUseExternalObjects = InnerObjectProperty && HasAnyPropertyFlags(CPF_ExperimentalExternalObjects) && UnderlyingArchive.IsPersistent() && !UnderlyingArchive.HasAnyPortFlags(PPF_DuplicateForPIE);

		// this indicates whether or not this object is to use external objects for overrides (can be overriden in certain deprecation cases)
		// this flag will either loaded from the serialized tag, or calculated during save
		const bool bTaggedUsingExternalsObjects = PropertyTag ? PropertyTag->bExperimentalExternalObjects.Get(false) : false;
		
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		if (UnderlyingArchive.IsLoading())
		{
#if WITH_EDITOR
			bool bUsingExternalObjects = bTaggedUsingExternalsObjects && bCouldUseExternalObjects;

			if (bCouldUseExternalObjects)
			{
				// Final determination for bUsingExternalObjects is delegated to handler if the property
				// was saved before started to include this state in the tag
				bool bUseHandlerForUsingExternalObjects = false;
				if (PropertyTag && !PropertyTag->bExperimentalExternalObjects.IsSet())
				{
					bUseHandlerForUsingExternalObjects = true;
				}

				// Invoke the external object handler to obtain the overrides for this property and the 'usingExternalObjects' state if we get it from there
				check(Context->SerializedObject);
				if ((bTaggedUsingExternalsObjects || bUseHandlerForUsingExternalObjects))
				{					
					bool bHandlerUsingExternalObjects = FPropertyArrayExternalObjectHandler::Get().GetExternalOverrides(Context->SerializedObject, this, &ContainerOverrides);
					if (bUseHandlerForUsingExternalObjects)
					{
						bUsingExternalObjects = bHandlerUsingExternalObjects;
					}
					else
					{
						// in case we're keeping this value from the tag validate all everything is as expected
						check(bHandlerUsingExternalObjects == bUsingExternalObjects);
					}
				}
			}
#else			 
			check(!bTaggedUsingExternalsObjects);
			const bool bUsingExternalObjects = false;
#endif // WITH_EDITOR

			int32 NumReplaced = 0;
			FStructuredArchive::FArray ReplacedArray = Record.EnterArray(TEXT("Replaced"), NumReplaced);
			if (NumReplaced != INDEX_NONE)
			{
				if (bUsingExternalObjects)
				{
					check(ContainerOverrides.ObjectsOperation == EOverriddenPropertyOperation::Replace);
					check(NumReplaced == 0);
				}
				else
				{
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Inner, this);

					ArrayHelper.EmptyAndAddValues(NumReplaced);
				
					for (int32 i = 0; i < NumReplaced; i++)
					{
						SerializeContainerItem(ReplacedArray.EnterElement(), ArrayHelper.GetRawPtr(i));
					}
				}
			}
			else
			{
				// Only Array of Instanced subobject are handled here as sort of a set where the matching key is done using the archetype
				checkf(InnerObjectProperty->HasAnyPropertyFlags(CPF_PersistentInstance), TEXT("Only supported code path here is the instanced subobjects"));

				FOverriddenPropertySet* OverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties();
				
#if WITH_EDITORONLY_DATA
				FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
				const bool bIsLoadingToPropertyBagObject = SerializeContext->bIsLoadingToPropertyBagObject;
#else
				const bool bIsLoadingToPropertyBagObject = false;
#endif
				checkf(Defaults || bIsLoadingToPropertyBagObject, TEXT("Expecting overridable serialization to have defaults to compare to"));

				FScriptArrayHelper DefaultsArrayHelper(this, Defaults);

				// If the arrays are not the same size, the code will not be able to remove items correctly
				// as it searches for the items to remove in the default array but removes them from this array.
				checkf (bUsingExternalObjects || bIsLoadingToPropertyBagObject || (DefaultsArrayHelper.Num() == ArrayHelper.Num()), TEXT("The array \'%s\' is expected to be the same size as it default for removal logic."), *GetName());

				auto FindObject = [InnerObjectProperty](UObject* Object, UObject* Object2, FScriptArrayHelper& ArrayHelper) -> int32
				{
					if (Object)
					{
						const int32 ArrayNum = ArrayHelper.Num();
						for (int i = 0; i < ArrayNum; ++i)
						{
							UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetElementPtr(i));
							if (CurrentObject == Object || (Object2 && CurrentObject == Object2))
							{
								return i;
							}
						}
					}
					return INDEX_NONE;
				};

				uint8* TempValueStorage = nullptr;
				ON_SCOPE_EXIT
				{
					if (TempValueStorage)
					{
						InnerObjectProperty->DestroyValue(TempValueStorage);
						FMemory::Free(TempValueStorage);
					}
				};

				int32 NumRemoved = 0;
				FStructuredArchive::FArray RemovedArray = Record.EnterArray(TEXT("Removed"), NumRemoved);
				if (NumRemoved != 0)
				{
					TArray<int32> IndicesToRemove;
					TempValueStorage = (uint8*)FMemory::Malloc(InnerObjectProperty->GetElementSize());
					InnerObjectProperty->InitializeValue(TempValueStorage);

					for (int32 i = 0; i < NumRemoved; ++i)
					{
						{
							FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Inner, this);
							SerializeContainerItem(RemovedArray.EnterElement(), TempValueStorage);
						}

						if (UObject* RemovedSubObject = InnerObjectProperty->GetObjectPropertyValue(TempValueStorage))
						{
							// when loading into property bags we don't have a default. luckily we don't need accurate overrides for that so we can skip this step.
							// we'll reseirialize this into the IDO later and that will get the correct override state
							if (!bIsLoadingToPropertyBagObject)
							{
								int32 Index = FindObject(RemovedSubObject, nullptr, DefaultsArrayHelper);
								if (Index != INDEX_NONE)
								{
									IndicesToRemove.Add(Index);
								}
								else
								{
									UE_LOGF(LogOverridableObject, VeryVerbose, "Unable to load removed item %ls(0x%p)", *GetNameSafe(RemovedSubObject), RemovedSubObject);
								}
							}

							ContainerOverrides.RemovedArchetypes.Add(RemovedSubObject);
						}
					}

					IndicesToRemove.Sort(TGreater<>());
					for (int32 IndexToRemove : IndicesToRemove)
					{
						ArrayHelper.RemoveValues(IndexToRemove);
					}
				}

				// We should not enter this code anymore but keeping for eventual support of modified values like struct
				int32 NumModified = 0;
				FStructuredArchive::FArray ModifiedArray = Record.EnterArray(TEXT("Modified"), NumModified);
				if (NumModified != 0)
				{
					if (!TempValueStorage)
					{
						TempValueStorage = (uint8*)FMemory::Malloc(InnerObjectProperty->GetElementSize());
						InnerObjectProperty->InitializeValue(TempValueStorage);
					}

					for (int32 i = 0; i < NumModified; ++i)
					{
						{
							FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Inner, this);
							SerializeContainerItem(ModifiedArray.EnterElement(), TempValueStorage);
						}

						if (UObject* ModifiedObject = InnerObjectProperty->GetObjectPropertyValue(TempValueStorage))
						{
							int32 Index = FindObject(ModifiedObject->GetArchetype(), ModifiedObject, ArrayHelper);
							if (Index != INDEX_NONE)
							{
								InnerObjectProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(Index), ModifiedObject);
							}
						}
					}
				}

				// Support of subobject shadowed serialization
				if (UnderlyingArchive.UEVer() >= EUnrealEngineObjectUE5Version::OS_SUB_OBJECT_SHADOW_SERIALIZATION)
				{
					int32 NumShadowed = 0;
					FStructuredArchive::FArray ShadowedArray = Record.EnterArray(TEXT("Shadowed"), NumShadowed);
					if (NumShadowed != 0)
					{
						if (!TempValueStorage)
						{
							TempValueStorage = (uint8*)FMemory::Malloc(InnerObjectProperty->GetElementSize());
							InnerObjectProperty->InitializeValue(TempValueStorage);
						}

						for (int32 i = 0; i < NumShadowed; ++i)
						{
							{
								FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Inner, this);
								SerializeContainerItem(ShadowedArray.EnterElement(), TempValueStorage);
							}

							// Only modifying property when loading loose properties or placeholders
#if WITH_EDITORONLY_DATA
							if (Context->bImpersonateProperties)
							{
								int32 AddIndex = ArrayHelper.Num();
								if (UObject* ShadowedObject = InnerObjectProperty->GetObjectPropertyValue(TempValueStorage))
								{
									int32 Index = FindObject(ShadowedObject->GetArchetype(), ShadowedObject, ArrayHelper);
									if (Index == INDEX_NONE)
									{
										ArrayHelper.AddValue();
										Index = AddIndex++;
									}
									InnerObjectProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(Index), ShadowedObject);
								}
							}
#endif // WITH_EDITORONLY_DATA

						}
					}
				}


				int32 NumAdded = 0;
				FStructuredArchive::FArray AddedArray = Record.EnterArray(TEXT("Added"), NumAdded);
				if (NumAdded != 0)
				{
					if (!TempValueStorage)
					{
						TempValueStorage = (uint8*)FMemory::Malloc(InnerObjectProperty->GetElementSize());
						InnerObjectProperty->InitializeValue(TempValueStorage);
					}

					int32 AddIndex = ArrayHelper.Num();
					for (int32 i = 0; i < NumAdded; ++i)
					{
						{
							FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Inner, this);
							SerializeContainerItem(AddedArray.EnterElement(), TempValueStorage);
						}

						if (UObject* AddedSubObject = InnerObjectProperty->GetObjectPropertyValue(TempValueStorage))
						{
							int32 Index = FindObject(AddedSubObject->GetArchetype(), AddedSubObject, ArrayHelper);
							if (Index == INDEX_NONE)
							{
								ArrayHelper.AddValue();
								Index = AddIndex++;
							}
							InnerObjectProperty->SetObjectPropertyValue(ArrayHelper.GetRawPtr(Index), AddedSubObject);

							ContainerOverrides.Added.Add(AddedSubObject);
						}
					}
				}

				// If we were tagged as using external objects restore the externalize override state even if we didn't
				// use it for serialization. 
				// 
				// For example, objects being reinstanced will pass through a non persistent archive during CopyPropertiesForUnrelatedObjects
				// and this will lose the override state if we don't restore it here. 
				if (bTaggedUsingExternalsObjects)
				{
					if (OverriddenProperties)
					{
						if (FOverriddenPropertyNode* ArrayOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, UnderlyingArchive.GetSerializedPropertyChain(), /*Property*/nullptr) : nullptr)
						{
							OverriddenProperties->RestoreSubPropertyOperation(EOverriddenPropertyOperation::Externalized, *ArrayOverriddenPropertyNode, FOverriddenPropertyNodeID());
						}
					}
				}
			
				// Restore overrides
				for (int32 i = 0; i < ContainerOverrides.RemovedArchetypes.Num(); i++)
				{
					// Need to fetch the ArrayOverriddenPropertyNode every loop as the previous iteration might have reallocated the node.
					if (FOverriddenPropertyNode* ArrayOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, UnderlyingArchive.GetSerializedPropertyChain(), /*Property*/nullptr) : nullptr)
					{
						// Rebuild the overridden info
						OverriddenProperties->RestoreSubObjectOperation(EOverriddenPropertyOperation::Remove, *ArrayOverriddenPropertyNode, ContainerOverrides.RemovedArchetypes[i]);
					}
				}
				for (int32 i = 0; i < ContainerOverrides.Added.Num(); i++)
				{
					// Need to fetch the ArrayOverriddenPropertyNode every loop as the previous iteration might have reallocated the node.
					if (FOverriddenPropertyNode* ArrayOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, UnderlyingArchive.GetSerializedPropertyChain(), /*Property*/nullptr) : nullptr)
					{
						// Rebuild the overridden info
						OverriddenProperties->RestoreSubObjectOperation(EOverriddenPropertyOperation::Add, *ArrayOverriddenPropertyNode, ContainerOverrides.Added[i]);
					}
				}

			}
		}
		else
		{
			bool bUsingExternalObjects = bTaggedUsingExternalsObjects && bCouldUseExternalObjects;

			// Container for temporarily tracking some indices
			TArray<int32> RemovedIndices;
			TArray<int32> AddedIndices;
			TArray<int32> ShadowIndices;

			bool bReplaceArray = !Defaults || !UnderlyingArchive.DoDelta() || UnderlyingArchive.IsTransacting();

			if (!bReplaceArray)
			{
				EOverriddenPropertyOperation ArrayOverrideOp = EOverriddenPropertyOperation::None;
				FOverriddenPropertySet* OverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties();
				if (OverriddenProperties)
				{
					ArrayOverrideOp = OverriddenProperties->GetOverriddenPropertyOperation(UnderlyingArchive.GetSerializedPropertyChain(), /*Property*/nullptr);
					bReplaceArray = ArrayOverrideOp == EOverriddenPropertyOperation::Replace;
				}
				else
				{
					// In the case we do not have overridable serialization enable, let write the entire content of the array
					bReplaceArray = true;
				}

				if (!bReplaceArray)
				{
					FScriptArrayHelper DefaultsArrayHelper(this, Defaults);

					if (FOverridableSerializationLogic::ShouldPropertyShadowSerializeSubObject(this))
					{
						const int32 ArrayNum = ArrayHelper.Num();
						for( int i = 0; i < ArrayNum; i++)
						{
							ShadowIndices.Add(i);
						}
					}

					// Only array of instanced subobjects are handled here as sort of a set where the matching key is done using the archetype
					checkf(InnerObjectProperty&& InnerObjectProperty->HasAnyPropertyFlags(CPF_PersistentInstance), TEXT("Expecting only arrays of instanced subobjects"));

					if (OverriddenProperties && ArrayOverrideOp != EOverriddenPropertyOperation::None)
					{
						auto FindObject = [InnerObjectProperty](const FOverriddenPropertyNodeID& ObjectToFind, FScriptArrayHelper& ArrayHelper) -> int32
						{
							const int32 ArrayNum = ArrayHelper.Num();
							for (int i = 0; i < ArrayNum; ++i)
							{
								if (UObject* CurrentObject = InnerObjectProperty->GetObjectPropertyValue(ArrayHelper.GetElementPtr(i)))
								{
									if (ObjectToFind == FOverriddenPropertyNodeID(CurrentObject))
									{
										return i;
									}
								}
							}

							return INDEX_NONE;
						};

						if (const FOverriddenPropertyNode* ArrayOverriddenPropertyNode = OverriddenProperties->GetOverriddenPropertyNode(UnderlyingArchive.GetSerializedPropertyChain()))
						{
							for (const FOverriddenPropertyNode& SubNode : ArrayOverriddenPropertyNode->GetSubPropertyNodes())
							{
								switch (SubNode.GetOperation())
								{
								case EOverriddenPropertyOperation::Remove:
									{
										const int32 DefaultIndex = FindObject(SubNode.GetNodeID(), DefaultsArrayHelper);
										if (DefaultIndex != INDEX_NONE)
										{
											RemovedIndices.Add(DefaultIndex);
										}
										else
										{
											UE_LOGF(LogOverridableObject, VeryVerbose, "Unable to save deleted item %ls", *SubNode.GetNodeID().ToDebugString());
										}
										break;
									}
								case EOverriddenPropertyOperation::Add:
									{
										const int32 Index = FindObject(SubNode.GetNodeID(), ArrayHelper);
 										if (Index != INDEX_NONE)
										{
											ShadowIndices.Remove(Index);
											AddedIndices.Add(Index);
										}
										else
										{
											UE_LOGF(LogOverridableObject, VeryVerbose, "Unable to save added item %ls", *SubNode.GetNodeID().ToDebugString());
										}
										break;
									}
								case EOverriddenPropertyOperation::Externalized:
									{
										// nothing to do here
										break;
									}
								default:
									checkf(false, TEXT("Unsupported operation type"));
									break;
								}
							}
						}
					}
				}

				ContainerOverrides.ObjectsOperation = ArrayOverrideOp;
			}

			if (bReplaceArray)
			{
				if (bUsingExternalObjects)
				{
					// External objects always acquire their size on reload
					int32 NumReplaced = 0;
					FStructuredArchive::FArray ReplacedArray = Record.EnterArray(TEXT("Replaced"), NumReplaced);
				}
				else
				{
					int32 NumReplaced = ArrayHelper.Num();
					FStructuredArchive::FArray ReplacedArray = Record.EnterArray(TEXT("Replaced"), NumReplaced);

					for (int32 i = 0; i < NumReplaced; ++i)
					{
						SerializeContainerItem(ReplacedArray.EnterElement(), ArrayHelper.GetRawPtr(i));
					}
				}
			}
			else
			{
				FScriptArrayHelper DefaultsArrayHelper(this, Defaults);

#if WITH_EDITOR		
				if (bUsingExternalObjects)
				{
					// These are stored as external objects so we let the serialized object handle the storage 
					// of the override info.
					int32 NoOpReplaced = INDEX_NONE;
					int32 NoOp = 0;
					Record.EnterArray(TEXT("Replaced"), NoOpReplaced);
					Record.EnterArray(TEXT("Removed"), NoOp);
					Record.EnterArray(TEXT("Modified"), NoOp);
					Record.EnterArray(TEXT("Shadowed"), NoOp);
					Record.EnterArray(TEXT("Added"), NoOp);
				}
				else
#endif // WITH_EDITOR
				{					
					int32 NumReplaced = INDEX_NONE;
					Record.EnterArray(TEXT("Replaced"), NumReplaced);

					int32 NumRemoved = RemovedIndices.Num();
					FStructuredArchive::FArray RemovedArray = Record.EnterArray(TEXT("Removed"), NumRemoved);
					for (int32 i : RemovedIndices)
					{
						SerializeContainerItem(RemovedArray.EnterElement(), DefaultsArrayHelper.GetRawPtr(i));
					}

					// We should not have modified elements anymore but keeping for eventual support of modified values like struct
					int32 NumModified = 0;
					FStructuredArchive::FArray ModifiedArray = Record.EnterArray(TEXT("Modified"), NumModified);

					// Support of subobject shadowed serialization
					// Introduced from EUnrealEngineObjectUE5Version::OS_SUB_OBJECT_SHADOW_SERIALIZATION
					int32 NumShadowed = ShadowIndices.Num();
					FStructuredArchive::FArray ShadowedArray = Record.EnterArray(TEXT("Shadowed"), NumShadowed);
					for (int32 i : ShadowIndices)
					{
						SerializeContainerItem(ShadowedArray.EnterElement(), ArrayHelper.GetRawPtr(i));
					}

					int32 NumAdded = AddedIndices.Num();
					FStructuredArchive::FArray AddedArray = Record.EnterArray(TEXT("Added"), NumAdded);
					for (int32 i : AddedIndices)
					{
						SerializeContainerItem(AddedArray.EnterElement(), ArrayHelper.GetRawPtr(i));
					}
				}
			}
		}
		return;
	}

	FStructuredArchiveArray Array = Slot.EnterArray(ElementCount);

	if (UnderlyingArchive.IsLoading())
	{
		// If using a custom property list, don't empty the array on load. Not all indices may have been serialized, so we need to preserve existing values at those slots.
		if (UnderlyingArchive.ArUseCustomPropertyList || UnderlyingArchive.ArPreserveArrayElements)
		{
			const int32 OldNum = ArrayHelper.Num();
			if (ElementCount > OldNum)
			{
				ArrayHelper.AddValues(ElementCount - OldNum);
			}
			else if (ElementCount < OldNum)
			{
				ArrayHelper.RemoveValues(ElementCount, OldNum - ElementCount);
			}
		}
		else
		{
			ArrayHelper.EmptyAndAddValues(ElementCount);
		}
	}
	ArrayHelper.CountBytes(UnderlyingArchive);

	// Serialize a PropertyTag for the inner property of this array, allows us to validate the inner struct to see if it has changed
	if (UnderlyingArchive.UEVer() < EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME &&
		UnderlyingArchive.UEVer() >= VER_UE4_INNER_ARRAY_TAG_INFO &&
		Inner->IsA<FStructProperty>())
	{
		if (!MaybeInnerTag)
		{
			MaybeInnerTag.Emplace(Inner, 0, (uint8*)NotNullGet(Value));
			UnderlyingArchive << MaybeInnerTag.GetValue();
		}

		FPropertyTag& InnerTag = MaybeInnerTag.GetValue();

		if (UnderlyingArchive.IsLoading())
		{
			if (UE::FPropertyTypeName NewTypeName = ApplyRedirectsToPropertyType(InnerTag.GetType(), Inner); !NewTypeName.IsEmpty())
			{
				InnerTag.SetType(NewTypeName);
			}

			// Check if the Inner property can successfully serialize, the type may have changed
			if (!Inner->CanSerializeFromTypeName(InnerTag.GetType()))
			{
				FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Inner);

				// Attempt mismatched tag serialization if available
				const FName StructName = InnerTag.GetType().GetParameterName(0);
				if ((StructProperty->Struct->StructFlags & STRUCT_SerializeFromMismatchedTag) && (InnerTag.Type != NAME_StructProperty || StructName != StructProperty->Struct->GetFName()))
				{
					SerializeFromMismatchedTag = InnerTag;
				}
				else
				{
					UE_LOGF(LogClass, Warning, "Array Property %ls has a struct type mismatch (tag %ls != prop %ls) in package: %ls. If that struct got renamed, add an entry to ActiveStructRedirects.",
						*WriteToString<32>(InnerTag.Name), *WriteToString<64>(InnerTag.GetType().GetParameter(0)), *WriteToString<64>(UE::FPropertyTypeName(StructProperty).GetParameter(0)), *UnderlyingArchive.GetArchiveName());

#if WITH_EDITOR
					// Ensure the structure is initialized
					for (int32 i = 0; i < ElementCount; i++)
					{
						StructProperty->Struct->InitializeDefaultValue(ArrayHelper.GetRawPtr(i));
					}
#endif // WITH_EDITOR

					if (!bIsTextFormat)
					{
						// Skip the property
						const int64 StartOfProperty = UnderlyingArchive.Tell();
						const int64 RemainingSize = InnerTag.Size - (UnderlyingArchive.Tell() - StartOfProperty);
						uint8 B;
						for (int64 i = 0; i < RemainingSize; i++)
						{
							UnderlyingArchive << B;
						}
					}
					return;
				}
			}
		}

	#if WITH_EDITORONLY_DATA
		if (Context->bTrackSerializedPropertyPath)
		{
			// Update the path with types from the inner tag if the outer tag is incomplete.
			if (const int32 SegmentIndex = Context->SerializedPropertyPath.GetSegmentCount() - 1; SegmentIndex >= 0)
			{
				UE::FPropertyPathNameSegment Segment = Context->SerializedPropertyPath.GetSegment(SegmentIndex);
				if (Segment.Type.GetName() == NAME_ArrayProperty)
				{
					const UE::FPropertyTypeName InnerTypeName = Segment.Type.GetParameter(0);
					if (InnerTypeName.GetName() == NAME_StructProperty && InnerTypeName.GetParameterCount() == 0)
					{
						UE::FPropertyTypeNameBuilder NewTypeBuilder;
						NewTypeBuilder.AddName(NAME_ArrayProperty);
						NewTypeBuilder.BeginParameters();
						NewTypeBuilder.AddType(InnerTag.GetType());
						NewTypeBuilder.EndParameters();
						Segment.Type = NewTypeBuilder.Build();
						Context->SerializedPropertyPath.SetSegment(SegmentIndex, Segment);
					}
				}
			}
		}
	#endif
	}

	// need to know how much data this call to SerializeItem consumes, so mark where we are
	int64 DataOffset = UnderlyingArchive.Tell();

	// If we're using a custom property list, first serialize any explicit indices
	int32 i = 0;
	bool bSerializeRemainingItems = true;
	bool bUsingCustomPropertyList = UnderlyingArchive.ArUseCustomPropertyList;
	if (bUsingCustomPropertyList && UnderlyingArchive.ArCustomPropertyList != nullptr)
	{
		// Initially we only serialize indices that are explicitly specified (in order)
		bSerializeRemainingItems = false;

		const FCustomPropertyListNode* CustomPropertyList = UnderlyingArchive.ArCustomPropertyList;
		const FCustomPropertyListNode* PropertyNode = CustomPropertyList;
		FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Inner, this);
		while (PropertyNode && i < ElementCount && !bSerializeRemainingItems)
		{
			if (PropertyNode->Property != Inner)
			{
				// A null property value signals that we should serialize the remaining array values in full starting at this index
				if (PropertyNode->Property == nullptr)
				{
					i = PropertyNode->ArrayIndex;
				}

				bSerializeRemainingItems = true;
			}
			else
			{
				// Set a temporary node to represent the item
				FCustomPropertyListNode ItemNode = *PropertyNode;
				ItemNode.ArrayIndex = 0;
				ItemNode.PropertyListNext = nullptr;
				UnderlyingArchive.ArCustomPropertyList = &ItemNode;

				// Serialize the item at this array index
				i = PropertyNode->ArrayIndex;
				SerializeContainerItem(Array.EnterElement(), ArrayHelper.GetRawPtr(i));
				PropertyNode = PropertyNode->PropertyListNext;

				// Restore the current property list
				UnderlyingArchive.ArCustomPropertyList = CustomPropertyList;
			}
		}
	}

	if (bSerializeRemainingItems)
	{
		// Temporarily suspend the custom property list (as we need these items to be serialized in full)
		UnderlyingArchive.ArUseCustomPropertyList = false;

		// Serialize each item until we get to the end of the array
		FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Inner, this);
		while (i < ElementCount)
		{
#if WITH_EDITOR
			static const FName NAME_UArraySerialize = FName("FArrayProperty::Serialize");
			FName NAME_UArraySerializeCount = FName(NAME_UArraySerialize);
			NAME_UArraySerializeCount.SetNumber(i);
			FArchive::FScopeAddDebugData P(UnderlyingArchive, NAME_UArraySerializeCount);
#endif
			SerializeContainerItem(Array.EnterElement(), ArrayHelper.GetRawPtr(i++));
		}

		// Restore use of the custom property list (if it was previously enabled)
		UnderlyingArchive.ArUseCustomPropertyList = bUsingCustomPropertyList;
	}

	if (MaybeInnerTag.IsSet() && UnderlyingArchive.IsSaving() && !bIsTextFormat)
	{
		FPropertyTag& InnerTag = MaybeInnerTag.GetValue();

		// set the tag's size
		InnerTag.Size = IntCastChecked<int32>(UnderlyingArchive.Tell() - DataOffset);

		if (InnerTag.Size > 0)
		{
			// mark our current location
			DataOffset = UnderlyingArchive.Tell();

			// go back and re-serialize the size now that we know it
			UnderlyingArchive.Seek(InnerTag.SizeOffset);
			UnderlyingArchive << InnerTag.Size;

			// return to the current location
			UnderlyingArchive.Seek(DataOffset);
		}
	}
}

bool FArrayProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData) const
{
	UE_LOGF(LogProperty, Fatal, "Deprecated code path");
	return 1;
}

void FArrayProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	SerializeSingleField(Ar, Inner, this);
	checkSlow(Inner);
}
void FArrayProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
	if (Inner)
	{
		Inner->AddReferencedObjects(Collector);
	}
}

FString FArrayProperty::GetCPPTypeCustom(FString* ExtendedTypeText, uint32 CPPExportFlags, const FString& InnerTypeText, const FString& InInnerExtendedTypeText) const
{
	if (ExtendedTypeText != nullptr)
	{
		FString InnerExtendedTypeText = InInnerExtendedTypeText;
		if (InnerExtendedTypeText.Len() && InnerExtendedTypeText.Right(1) == TEXT(">"))
		{
			// if our internal property type is a template class, add a space between the closing brackets b/c VS.NET cannot parse this correctly
			InnerExtendedTypeText += TEXT(" ");
		}
		else if (!InnerExtendedTypeText.Len() && InnerTypeText.Len() && InnerTypeText.Right(1) == TEXT(">"))
		{
			// if our internal property type is a template class, add a space between the closing brackets b/c VS.NET cannot parse this correctly
			InnerExtendedTypeText += TEXT(" ");
		}
		*ExtendedTypeText = FString::Printf(TEXT("<%s%s>"), *InnerTypeText, *InnerExtendedTypeText);
	}
	return TEXT("TArray");
}

FString FArrayProperty::GetCPPType(FString* ExtendedTypeText/*=nullptr*/, uint32 CPPExportFlags/*=0*/) const
{
	checkSlow(Inner);
	FString InnerExtendedTypeText;
	FString InnerTypeText;
	if (ExtendedTypeText != nullptr)
	{
		InnerTypeText = Inner->GetCPPType(&InnerExtendedTypeText, CPPExportFlags & ~CPPF_ArgumentOrReturnValue); // we won't consider array inners to be "arguments or return values"
	}
	return GetCPPTypeCustom(ExtendedTypeText, CPPExportFlags, InnerTypeText, InnerExtendedTypeText);
}

FString FArrayProperty::GetCPPMacroType(FString& ExtendedTypeText) const
{
	checkSlow(Inner);
	ExtendedTypeText = Inner->GetCPPType();
	return TEXT("TARRAY");
}
void FArrayProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	checkSlow(Inner);

	uint8* TempArrayStorage = nullptr;
	void* PropertyValuePtr = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		// Allocate temporary map as we first need to initialize it with the value provided by the getter function and then export it
		TempArrayStorage = (uint8*)AllocateAndInitializeValue();
		PropertyValuePtr = TempArrayStorage;
		FProperty::GetValue_InContainer(ContainerOrPropertyPtr, PropertyValuePtr);
	}
	else
	{
		PropertyValuePtr = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
	}

	ON_SCOPE_EXIT
	{
		DestroyAndFreeValue(TempArrayStorage);
	};

	FScriptArrayHelper ArrayHelper(this, PropertyValuePtr);

	int32 DefaultSize = 0;
	if (DefaultValue)
	{
		FScriptArrayHelper DefaultArrayHelper(this, DefaultValue);
		DefaultSize = DefaultArrayHelper.Num();
		DefaultValue = DefaultArrayHelper.GetRawPtr(0);
	}

	ExportTextInnerItem(ValueStr, Inner, ArrayHelper.GetRawPtr(0), ArrayHelper.Num(), DefaultValue, DefaultSize, Parent, PortFlags, ExportRootScope);
}

void FArrayProperty::ExportTextInnerItem(FString& ValueStr, const FProperty* Inner, const void* PropertyValue, int32 NumElements, const void* DefaultValue, int32 DefaultSize, UObject* Parent, int32 PortFlags, UObject* ExportRootScope)
{
	checkSlow(Inner);
	checkSlow(PropertyValue || NumElements == 0);

	uint8* StructDefaults = nullptr;
	const FStructProperty* StructProperty = CastField<FStructProperty>(Inner);

	const bool bReadableForm = (0 != (PPF_BlueprintDebugView & PortFlags));
	const bool bExternalEditor = (0 != (PPF_ExternalEditor & PortFlags));

	// ArrayProperties only export a diff because array entries are cleared and recreated upon import. Static arrays are overwritten when importing,
	// so we export the entire struct to ensure all data is copied over correctly. Behavior is currently inconsistent when copy/pasting between the two types.
	// In the future, static arrays could export diffs if the property being imported to is reset to default before the import.
	// When exporting to an external editor, we want to save defaults so all information is available for editing
	if (StructProperty != nullptr && Inner->ArrayDim == 1 && !bExternalEditor)
	{
		checkSlow(StructProperty->Struct);
		StructDefaults = (uint8*)FMemory::Malloc(StructProperty->Struct->GetStructureSize() * Inner->ArrayDim);
		StructProperty->InitializeValue(StructDefaults);
	}

	int32 Count = 0;
	for (int32 i = 0; i < NumElements; i++)
	{
		++Count;
		if (!bReadableForm)
		{
			if (Count == 1)
			{
				ValueStr += TCHAR('(');
			}
			else
			{
				ValueStr += TCHAR(',');
			}
		}
		else
		{
			if (Count > 1)
			{
				ValueStr += TCHAR('\n');
			}
			ValueStr += FString::Printf(TEXT("[%i] "), i);
		}

		uint8* PropData = (uint8*)PropertyValue + i * Inner->GetElementSize();

		// Always use struct defaults if the inner is a struct, for symmetry with the import of array inner struct defaults
		uint8* PropDefault = nullptr;
		if (bExternalEditor)
		{
			PropDefault = PropData;
		}
		else if (StructProperty)
		{
			PropDefault = StructDefaults;
		}
		else
		{
			if (DefaultValue && DefaultSize > i)
			{
				PropDefault = (uint8*)DefaultValue + i * Inner->GetElementSize();
			}
		}

		Inner->ExportTextItem_Direct(ValueStr, PropData, PropDefault, Parent, PortFlags|PPF_Delimited, ExportRootScope);
	}

	if ((Count > 0) && !bReadableForm)
	{
		ValueStr += TEXT(")");
	}
	if (StructDefaults)
	{
		StructProperty->DestroyValue(StructDefaults);
		FMemory::Free(StructDefaults);
	}
}

const TCHAR* FArrayProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const
{
	uint8* TempArrayStorage = nullptr;
	ON_SCOPE_EXIT
	{
		if (TempArrayStorage)
		{
			// TempArrayStorage is used by property setter so if it was allocated call the setter now
			FProperty::SetValue_InContainer(ContainerOrPropertyPtr, TempArrayStorage);

			// Destroy and free the temp array used by property setter
			DestroyAndFreeValue(TempArrayStorage);
		}
	};

	void* ArrayPtr = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		// Allocate temporary map as we first need to initialize it with the parsed items and then use the setter to update the property
		TempArrayStorage = (uint8*)AllocateAndInitializeValue();
		ArrayPtr = TempArrayStorage;
	}
	else
	{
		ArrayPtr = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
	}

	FScriptArrayHelper ArrayHelper(this, ArrayPtr);
	return ImportTextInnerItem(Buffer, Inner, ArrayPtr, PortFlags, OwnerObject, &ArrayHelper, ErrorText);
}

const TCHAR* FArrayProperty::ImportTextInnerItem(const TCHAR* Buffer, const FProperty* Inner, void* Data, int32 PortFlags, UObject* Parent, FScriptArrayHelper* ArrayHelper, FOutputDevice* ErrorText)
{
	checkSlow(Inner);

	// If we export an empty array we export an empty string, so ensure that if we're passed an empty string
	// we interpret it as an empty array.
	if (*Buffer == TCHAR('\0') || *Buffer == TCHAR(')') || *Buffer == TCHAR(','))
	{
		if (ArrayHelper)
		{
			ArrayHelper->EmptyValues();
		}
		return Buffer;
	}

	if (*Buffer++ != TCHAR('('))
	{
		return nullptr;
	}

	if (ArrayHelper)
	{
		ArrayHelper->EmptyValues();
		ArrayHelper->ExpandForIndex(0);
	}

	SkipWhitespace(Buffer);

	if (*Buffer == TCHAR(')'))
	{
		// We didn't find any items. Clear the array again
		if (ArrayHelper)
		{
			ArrayHelper->EmptyValues();
		}
		++Buffer; // consume closing parenthesis
		return Buffer;
	}

	int32 Index = 0;
	while (*Buffer != TCHAR(')'))
	{
		SkipWhitespace(Buffer);

		if (*Buffer != TCHAR(','))
		{
			uint8* Address = ArrayHelper ? ArrayHelper->GetRawPtr(Index) : ((uint8*)Data + Inner->GetElementSize() * Index);
			// Parse the item
			checkf(ArrayHelper == nullptr || Inner->GetOffset_ForInternal() == 0, TEXT("Expected the Inner property of the FArrayProperty."));
			Buffer = Inner->ImportText_Direct(Buffer, Address, Parent, PortFlags | PPF_Delimited, ErrorText);

			if (!Buffer)
			{
				return nullptr;
			}

			SkipWhitespace(Buffer);
		}


		if (*Buffer == TCHAR(','))
		{
			Buffer++;
			Index++;
			if (ArrayHelper)
			{
				ArrayHelper->ExpandForIndex(Index);
			}
			else if (Index >= Inner->ArrayDim)
			{
				ErrorText->Logf(ELogVerbosity::Warning, TEXT("%s is a fixed-sized array of %i values. Additional data after %i has been ignored during import."), *Inner->GetName(), Inner->ArrayDim, Inner->ArrayDim);
				break;
			}
		}
		else
		{
			break;
		}
	}

	// Make sure we ended on a )
	if (*Buffer++ != TCHAR(')'))
	{
		return nullptr;
	}

	return Buffer;
}

#if WITH_EDITORONLY_DATA
void FArrayProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (Inner)
	{
		Inner->AppendSchemaHash(Builder, bSkipEditorOnly);
	}
}
#endif

void FArrayProperty::AddCppProperty(FProperty* Property)
{
	check(!Inner);
	check(Property);

	Inner = Property;
}

void FArrayProperty::CopyValuesInternal(void* Dest, void const* Src, int32 Count) const
{
	check(Count == 1); // this was never supported, apparently
	FScriptArrayHelper SrcArrayHelper(this, Src);
	FScriptArrayHelper DestArrayHelper(this, Dest);

	int32 Num = SrcArrayHelper.Num();
	if (!(Inner->PropertyFlags & CPF_IsPlainOldData))
	{
		DestArrayHelper.EmptyAndAddValues(Num);
	}
	else
	{
		DestArrayHelper.EmptyAndAddUninitializedValues(Num);
	}

	if (Num)
	{
		size_t Size = Inner->GetElementSize();
		uint8* SrcData = (uint8*)SrcArrayHelper.GetRawPtr();
		uint8* DestData = (uint8*)DestArrayHelper.GetRawPtr();
		if (!(Inner->PropertyFlags & CPF_IsPlainOldData))
		{
			// If we are in the closed body of a transaction, we have a special case for `FObjectProperty` that vastly
			// improves performance when copying arrays around (in the order of 4x faster). We **do not want** to do
			// this optimization with anything that derives from `FObjectProperty` though, so we **do not use** `IsA`
			// as that would all catch classes that inherited from `FObjectProperty`!
			if (AutoRTFM::IsClosed() && (Inner->GetClass() == FObjectProperty::StaticClass()))
			{
				#if UE_WITH_REMOTE_OBJECT_HANDLE
				// CopyCompleteValue will resolve handles in the RawPtr <- TObjectPtr case.
				// Since we can't abort while in an open scope, use a speculation scope to defer any
				// migrations until this scope ends.
				UE::RemoteExecutor::FSpeculationExecutionScope AvoidMigrationsInTheOpen;
				#endif

				// Since we've already written to each location that we are going to be copying over below, we know that
				// all of the record-writes that the transactional system would have inserted are no-ops. Also, each copy
				// of a `TObjectPtr` will go into the open to do mark the object as reachable, so by doing a single open
				// here we can avoid doing N calls to record-write and N calls to open for each object in the array. Big
				// saving on large arrays!
				UE_AUTORTFM_OPEN_NO_VALIDATION
				{
					// This optimization assumes that `FObjectProperty` is just backed by a `TObjectPtr` which when copying
					// does not do any heap allocations, just pulls over the pointer to the underlying `UObject` and then
					// marks it as reachable for incremental reachability analysis. Lets check that this is still the case
					// here so that if anyone ever changes that in future this line will shout at them!
					static_assert(std::is_same_v<decltype(FObjectProperty::GetDefaultPropertyValue()), TObjectPtr<UObject>>);

					for (int32 i = 0; i < Num; i++)
					{
						Inner->CopyCompleteValue(DestData + i * Size, SrcData + i * Size);
					}
				};
			}
			else
			{
				for (int32 i = 0; i < Num; i++)
				{
					Inner->CopyCompleteValue(DestData + i * Size, SrcData + i * Size);
				}
			}
		}
		else
		{
			FMemory::Memcpy(DestData, SrcData, Num * Size);
		}
	}
}
void FArrayProperty::ClearValueInternal(void* Data) const
{
	FScriptArrayHelper ArrayHelper(this, Data);
	ArrayHelper.EmptyValues();
}
void FArrayProperty::DestroyValueInternal(void* Dest) const
{
	FScriptArrayHelper ArrayHelper(this, Dest);
	ArrayHelper.EmptyValues();

	//@todo UE potential double destroy later from this...would be ok for a script array, but still
	ArrayHelper.DestroyContainer_Unsafe();
}

bool FArrayProperty::ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const
{
	check(Inner);
	return Inner->ContainsFinishDestroy(EncounteredStructProps);
}

void FArrayProperty::FinishDestroyInternal( void* Data ) const
{
	if (!Data)
	{
		return;
	}
	check(Inner);
	if ((Inner->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)) == 0)
	{
		FScriptArrayHelper ArrayHelper(this, Data);
		for (int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ElementIndex++)
		{
			Inner->FinishDestroy(ArrayHelper.GetRawPtr(ElementIndex));
		}
	}	
}

/**
 * Creates new copies of components
 *
 * @param	Data				pointer to the address of the instanced object referenced by this UComponentProperty
 * @param	DefaultData			pointer to the address of the default value of the instanced object referenced by this UComponentProperty
 * @param	Owner				the object that contains this property's data
 * @param	InstanceGraph		contains the mappings of instanced objects and components to their templates
 */
void FArrayProperty::InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, FObjectInstancingGraph* InstanceGraph)
{
	if (Data
		&& (Inner->ContainsInstancedObjectProperty() || InOwner->GetClass()->ShouldUseDynamicSubobjectInstancing()))
	{
		FScriptArrayHelper ArrayHelper(this, Data);
		FScriptArrayHelper DefaultArrayHelper(this, DefaultData);

		int32 InnerElementSize = Inner->GetElementSize();
		void* TempElement = FMemory_Alloca(InnerElementSize);

		for (int32 ElementIndex = 0; ElementIndex < ArrayHelper.Num(); ElementIndex++)
		{
			uint8* DefaultValue = (DefaultData && ElementIndex < DefaultArrayHelper.Num()) ? DefaultArrayHelper.GetRawPtr(ElementIndex) : nullptr;
			FMemory::Memmove(TempElement, ArrayHelper.GetRawPtr(ElementIndex), InnerElementSize);
			Inner->InstanceSubobjects(TempElement, DefaultValue, InOwner, InstanceGraph);
			if (ElementIndex < ArrayHelper.Num())
			{
				FMemory::Memmove(ArrayHelper.GetRawPtr(ElementIndex), TempElement, InnerElementSize);
			}
			else
			{
				Inner->DestroyValue(TempElement);
			}
		}
	}
}

bool FArrayProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && Inner && Inner->SameType(((FArrayProperty*)Other)->Inner);
}

EConvertFromTypeResult FArrayProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	if (Tag.Type != NAME_ArrayProperty)
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}

	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	const FPackageFileVersion Version = UnderlyingArchive.UEVer();
	if (Version >= EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
	{
		if (CanSerializeFromTypeName(Tag.GetType()))
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}
	}
	else
	{
		const FName InnerTypeName = Tag.GetType().GetParameterName(0);
		if (InnerTypeName.IsNone() || InnerTypeName == Inner->GetID())
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}
	}

	if (Tag.bExperimentalOverridableLogic)
	{
		return EConvertFromTypeResult::CannotConvert;
	}

	int32 ElementCount = 0;
	if (UnderlyingArchive.IsTextFormat())
	{
		Slot.EnterArray(ElementCount);
	}
	else
	{
		UnderlyingArchive << ElementCount;
	}

	FScriptArrayHelper ScriptArrayHelper(this, ContainerPtrToValuePtr<void>(Data));
	ScriptArrayHelper.EmptyAndAddValues(ElementCount);

	FPropertyTag InnerPropertyTag;
	InnerPropertyTag.SetProperty(Inner);
	InnerPropertyTag.SetType(Tag.GetType().GetParameter(0));
	InnerPropertyTag.Name = Tag.Name;
	InnerPropertyTag.ArrayIndex = 0;

	if (Version < EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME && Version >= VER_UE4_INNER_ARRAY_TAG_INFO && InnerPropertyTag.Type == NAME_StructProperty)
	{
		UnderlyingArchive << InnerPropertyTag;
	}

	if (ElementCount == 0)
	{
		return EConvertFromTypeResult::Converted;
	}

	FUObjectSerializeContext* Context = FUObjectThreadContext::Get().GetSerializeContext();
	FStructuredArchive::FStream ValueStream = Slot.EnterStream();

	TOptional<FBoolProperty> InnerBool;
	if (InnerPropertyTag.Type == NAME_BoolProperty)
	{
		// ConvertFromType expects a bool value to be stored on the property tag.
		// Call SerializeItem to populate BoolVal before calling ConvertFromType.
		InnerBool.Emplace(this, GetFName());
	}

	FStructuredArchive::FSlot FirstElementSlot = ValueStream.EnterElement();
	if (InnerBool)
	{
		InnerBool->SerializeItem(FirstElementSlot, &InnerPropertyTag.BoolVal, nullptr);
	}
	EConvertFromTypeResult ConvertResult = Inner->ConvertFromType(InnerPropertyTag, FirstElementSlot, ScriptArrayHelper.GetRawPtr(0), DefaultsStruct, nullptr);
	if (ConvertResult == EConvertFromTypeResult::Converted || ConvertResult == EConvertFromTypeResult::Serialized)
	{
		for (int32 ElementIndex = 1; ElementIndex < ElementCount; ++ElementIndex)
		{
			FStructuredArchive::FSlot ElementSlot = ValueStream.EnterElement();
			if (InnerBool)
			{
				InnerBool->SerializeItem(ElementSlot, &InnerPropertyTag.BoolVal, nullptr);
			}
			ConvertResult = Inner->ConvertFromType(InnerPropertyTag, ElementSlot, ScriptArrayHelper.GetRawPtr(ElementIndex), DefaultsStruct, nullptr);
			check(ConvertResult == EConvertFromTypeResult::Converted || ConvertResult == EConvertFromTypeResult::Serialized);
		}
		return ConvertResult;
	}
	else
	{
		UE_LOGF(LogClass, Warning, "Array Inner Type mismatch in %ls - Previous (%ls) Current(%ls) in package: %ls",
			*WriteToString<32>(Tag.Name), *WriteToString<32>(InnerPropertyTag.GetType()), *WriteToString<32>(UE::FPropertyTypeName(Inner)), *UnderlyingArchive.GetArchiveName());
		return EConvertFromTypeResult::CannotConvert;
	}
}

FField* FArrayProperty::GetInnerFieldByName(const FName& InName)
{
	if (Inner && Inner->GetFName() == InName)
	{
		return Inner;
	}
	return nullptr;
}

void FArrayProperty::GetInnerFields(TArray<FField*>& OutFields)
{
	if (Inner)
	{
		OutFields.Add(Inner);
		Inner->GetInnerFields(OutFields);
	}
}

void* FArrayProperty::GetValueAddressAtIndex_Direct(const FProperty* InInner, void* InValueAddress, int32 Index) const
{
	FScriptArrayHelper ArrayHelper(this, InValueAddress);
	checkf(Inner == InInner, TEXT("Passed in inner must be identical to the array property inner property"));
	if (Index < ArrayHelper.Num() && Index >= 0)
	{
		return ArrayHelper.GetRawPtr(Index);
	}
	else
	{
		return nullptr;
	}
}

bool FArrayProperty::UseBinaryOrNativeSerialization(const FArchive& Ar) const
{
	if (Super::UseBinaryOrNativeSerialization(Ar))
	{
		return true;
	}

	const FProperty* LocalInner = Inner;
	check(LocalInner);
	return LocalInner->UseBinaryOrNativeSerialization(Ar);
}

bool FArrayProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	if (!Super::LoadTypeName(Type, Tag))
	{
		return false;
	}

	const UE::FPropertyTypeName InnerType = Type.GetParameter(0);
	FField* Field = FField::TryConstruct(InnerType.GetName(), this, GetFName());
	if (FProperty* Property = CastField<FProperty>(Field); Property && Property->LoadTypeName(InnerType, Tag))
	{
		Inner = Property;
		return true;
	}
	delete Field;
	return false;
}

void FArrayProperty::SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const
{
	Super::SaveTypeName(Type);

	const FProperty* LocalInner = Inner;
	check(LocalInner);
	Type.BeginParameters();
	LocalInner->SaveTypeName(Type);
	Type.EndParameters();
}

bool FArrayProperty::CanSerializeFromTypeName(UE::FPropertyTypeName Type) const
{
	if (!Super::CanSerializeFromTypeName(Type))
	{
		return false;
	}

	const FProperty* LocalInner = Inner;
	check(LocalInner);
	return LocalInner->CanSerializeFromTypeName(Type.GetParameter(0));
}

EPropertyVisitorControlFlow FArrayProperty::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Conmtext*/)> InFunc) const
{
	FScriptArrayHelper ArrayHelper(this, Context.Data.PropertyData);
	const int32 ArrayNum = ArrayHelper.Num();

	// Indicate in the path that this property contains inner properties
	Context.Path.Top().bContainsInnerProperties = ArrayNum > 0;

	EPropertyVisitorControlFlow RetVal = Super::Visit(Context, InFunc);

	if (RetVal == EPropertyVisitorControlFlow::StepInto)
	{
		checkf(Inner, TEXT("Expecting a valid inner property type"));

		for (int32 ContainerIndex = 0; ContainerIndex < ArrayNum; ContainerIndex++)
		{
			FPropertyVisitorScope Scope(Context.Path, FPropertyVisitorInfo(Inner, ContainerIndex, EPropertyVisitorInfoType::ContainerIndex));
			FPropertyVisitorContext SubContext = Context.VisitPropertyData(ArrayHelper.GetRawPtr(ContainerIndex));

			RetVal = Inner->Visit(SubContext, InFunc);
			if (RetVal == EPropertyVisitorControlFlow::Stop)
			{
				return EPropertyVisitorControlFlow::Stop;
			}
			if (RetVal == EPropertyVisitorControlFlow::StepOut)
			{
				return EPropertyVisitorControlFlow::StepOver;
			}
		}
	}
	return RetVal;
}

void* FArrayProperty::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	if (Info.PropertyInfo == EPropertyVisitorInfoType::ContainerIndex && Info.Property == Inner)
	{
		return GetValueAddressAtIndex_Direct(Info.Property, Data, Info.Index);
	}

	return nullptr;
}

bool FArrayProperty::HasIntrusiveUnsetOptionalState() const
{
	return true;
}

void FArrayProperty::InitializeIntrusiveUnsetOptionalValue(void* Data) const
{
	// FScriptArray's unset state constructor is good enough
	Super::InitializeIntrusiveUnsetOptionalValue(Data);
}

bool FArrayProperty::IsIntrusiveOptionalValueSet(const void* Data) const
{
	// FScriptArray's unset state comparison is good enough
	return Super::IsIntrusiveOptionalValueSet(Data);
}

void FArrayProperty::ClearIntrusiveOptionalValue(void* Data) const
{
	// Destroy any inner elements first, because FScriptArray's destructor will only free memory
	if (IsIntrusiveOptionalValueSet(Data))
	{
		FScriptArrayHelper ArrayHelper(this, Data);
		ArrayHelper.EmptyValues();

		// Call Super to actually reset the optional to the unset state, now that any elements have been destroyed
		Super::ClearIntrusiveOptionalValue(Data);
	}
}
