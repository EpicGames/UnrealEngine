// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyBagRepository.h"

#include "UObject/PropertyPathFunctions.h"

#if WITH_EDITORONLY_DATA

#include "Containers/Queue.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"
#include "Serialization/ArchiveProxy.h"
#include "Serialization/ArchiveCountMem.h"
#include "UObject/CommonFNames.h"
#include "UObject/GarbageCollection.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/InstanceDataTransforms.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyStateTracking.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectSerializeContext.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/OverridableManager.h"
#include "UObject/PropertyPathNameTree.h"
#include "Templates/UnrealTemplate.h"
#include "VerseVM/VVMVerseClass.h"

#if WITH_EDITOR
#include "HAL/IConsoleManager.h"
#include "Misc/CommandLine.h"
#endif
#include <UObject/InstanceDataTransforms.h>

DEFINE_LOG_CATEGORY_STATIC(LogPropertyBagRepository, Log, All);

namespace UE
{

/** Defined in InstanceDataObjectUtils.cpp */
void CopyTaggedProperties(const UObject* Source, UObject* Dest);

/** Internal registry that tracks the current set of types for property bag container objects instanced as placeholders for package exports that have invalid or missing class imports on load. */
class FPropertyBagPlaceholderTypeRegistry
{
public:
	static FPropertyBagPlaceholderTypeRegistry& Get()
	{
		static FPropertyBagPlaceholderTypeRegistry Instance;
		return Instance;
	}

	void Add(const UStruct* Type)
	{
		PendingPlaceholderTypes.Enqueue(Type);
	}

	void Remove(const UStruct* Type)
	{
		PlaceholderTypes.Remove(Type);
	}

	bool Contains(const UStruct* Type)
	{
		ConsumePendingPlaceholderTypes();
		return PlaceholderTypes.Contains(Type);
	}

	void GetTypes(TSet<TObjectPtr<const UStruct>>& OutTypes)
	{
		ConsumePendingPlaceholderTypes();
		
		FScopeLock ScopeLock(&CriticalSection);
		OutTypes.Append(PlaceholderTypes);
	}

	int32 Num() const
	{
		return PlaceholderTypes.Num();
	}

protected:
	FPropertyBagPlaceholderTypeRegistry() = default;
	FPropertyBagPlaceholderTypeRegistry(const FPropertyBagPlaceholderTypeRegistry&) = delete;
	FPropertyBagPlaceholderTypeRegistry& operator=(const FPropertyBagPlaceholderTypeRegistry&) = delete;

	void ConsumePendingPlaceholderTypes()
	{
		if (!PendingPlaceholderTypes.IsEmpty())
		{
			FScopeLock ScopeLock(&CriticalSection);

			TObjectPtr<const UStruct> PendingType;
			while(PendingPlaceholderTypes.Dequeue(PendingType))
			{
				PlaceholderTypes.Add(PendingType);
			}
		}
	}

private:
	FCriticalSection CriticalSection;

	// List of types that have been registered.
	TSet<TObjectPtr<const UStruct>> PlaceholderTypes;

	// Types that have been added but not yet registered. Utilizes a thread-safe queue so we can avoid race conditions during an async load.
	TQueue<TObjectPtr<const UStruct>> PendingPlaceholderTypes;
};

class FPropertyBagRepositoryLock
{
#if THREADSAFE_UOBJECTS
	const FPropertyBagRepository* Repo;	// Technically a singleton, but just in case...
#endif
public:
	FORCEINLINE FPropertyBagRepositoryLock(const FPropertyBagRepository* InRepo)
	{
#if THREADSAFE_UOBJECTS
		if (!(IsGarbageCollectingAndLockingUObjectHashTables() && IsInGameThread()))	// Mirror object hash tables behaviour exactly for now
		{
			Repo = InRepo;
			InRepo->Lock();
		}
		else
		{
			Repo = nullptr;
		}
#else
		check(IsInGameThread());
#endif
	}
	FORCEINLINE ~FPropertyBagRepositoryLock()
	{
#if THREADSAFE_UOBJECTS
		if (Repo)
		{
			Repo->Unlock();
		}
#endif
	}
};

class FArchetypeMatchingArchive : public FArchiveProxy
{
public:

	FArchetypeMatchingArchive(FArchive& InInnerArchive, UObject* InObject, UObject* InArchetype)
		: FArchiveProxy(InInnerArchive)
		, Object(InObject)
		, Archetype(InArchetype)
	{		
	}

	UObject* GetArchetypeFromLoader(const UObject* Obj) override
	{
		if (Archetype && (Obj == Object))
		{
			return Archetype;
		}
		else
		{
			return InnerArchive.GetArchetypeFromLoader(Obj);
		}
	}

private:

	UObject* Object = nullptr;
	UObject* Archetype = nullptr;
};

void FPropertyBagRepository::FPropertyBagAssociationData::Destroy()
{
	if (InstanceDataObject && InstanceDataObject->IsValidLowLevel())
	{
		InstanceDataObject = nullptr;
	}
}

FPropertyBagRepository::FPropertyBagRepository()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnPostObjectPropertyChanged.AddRaw(this, &FPropertyBagRepository::PostEditChangeChainProperty);
#endif // WITH_EDITOR
}

FPropertyBagRepository& FPropertyBagRepository::Get()
{
	static FPropertyBagRepository Repo;
	return Repo;
}

void FPropertyBagRepository::ReassociateObjects(const TMap<UObject*, UObject*>& ReplacedObjects)
{
	if (!IsInstanceDataObjectSupportEnabled())
	{
		return;
	}

	FPropertyBagRepositoryLock LockRepo(this);
	FPropertyBagAssociationData OldBagData;
	for (const TPair<UObject*, UObject*>& Pair : ReplacedObjects)
	{
		if (AssociatedData.RemoveAndCopyValue(Pair.Key, OldBagData))
		{
			InstanceDataObjectToOwner.Remove(OldBagData.InstanceDataObject);

			if (FPropertyBagAssociationData* NewBagData = AssociatedData.Find(Pair.Value))
			{
				// copy data from owner to IDO. This can be needed if for example a type mismatch was detected but the serialization code still
				// inferred some info. Like for example if an array type mismatch occured and the new array was auto-grown in the owner to match the
				// old one. This array growth needs to be copied over to the IDO to maintain it
				CopyTaggedProperties(Pair.Value, NewBagData->InstanceDataObject);

#if !UE_BUILD_SHIPPING_WITH_EDITOR
				// Run IDT operations on this instance to recover data
				TObjectPtr<UObject> InOutInstance = const_cast<UObject*>(Pair.Value);
				UE::FInstanceDataTransforms::Get().PatchInstanceDataObject(NewBagData->InstanceDataObject, InOutInstance);
#endif
			}

			OldBagData.Destroy();
		}
		else if (UStruct* TypeObject = Cast<UStruct>(Pair.Key))
		{
			if (IsPropertyBagPlaceholderType(TypeObject))
			{
				FPropertyBagPlaceholderTypeRegistry::Get().Remove(TypeObject);
			}
		}
	}
}

static FProperty* FindPropertyByNameAndType(const UStruct* Struct, const FProperty* SourceProperty)
{
	return FindPropertyByNameAndTypeName(Struct, SourceProperty->GetFName(), FPropertyTypeName(SourceProperty));
}

static void CopyProperty(const FProperty* SourceProperty, const void* SourceValue, const FProperty* DestProperty, void* DestValue)
{
	check(SourceProperty->GetID() == DestProperty->GetID());
	if (SourceProperty->SameType(DestProperty))
	{
		SourceProperty->CopySingleValue(DestValue, SourceValue);
	}
	else if (const FStructProperty* SourcePropertyAsStruct = CastField<FStructProperty>(SourceProperty))
	{
		const UStruct* SourceStruct = SourcePropertyAsStruct->Struct;
		UStruct* DestStruct = CastFieldChecked<FStructProperty>(DestProperty)->Struct;
		for (FProperty* SourceChild : TFieldRange<FProperty>(SourceStruct))
		{
			if (FProperty* DestChild = FindPropertyByNameAndType(DestStruct, SourceChild))
			{
				CopyProperty(SourceChild, SourceChild->ContainerPtrToValuePtr<void>(SourceValue),
					DestChild, DestChild->ContainerPtrToValuePtr<void>(DestValue));
			}
		}
	}
	else if (const FOptionalProperty* SourcePropertyAsOptional = CastField<FOptionalProperty>(SourceProperty))
	{
		const FOptionalProperty* DestPropertyAsOptional = CastFieldChecked<FOptionalProperty>(DestProperty);
		FOptionalPropertyLayout SourceOptionalLayout(SourcePropertyAsOptional->GetValueProperty());
		FOptionalPropertyLayout DestOptionalLayout(DestPropertyAsOptional->GetValueProperty());
		if (!SourceOptionalLayout.IsSet(SourceValue))
		{
			DestOptionalLayout.MarkUnset(DestValue);
		}
		else
		{
			const void* SourceChildValue = SourceOptionalLayout.GetValuePointerForRead(SourceValue);
			void* DestChildValue = DestOptionalLayout.MarkSetAndGetInitializedValuePointerToReplace(DestValue);
			
			CopyProperty(SourceOptionalLayout.GetValueProperty(), SourceChildValue,
				DestOptionalLayout.GetValueProperty(), DestChildValue);
		}
	}
	else if (const FArrayProperty* SourcePropertyAsArray = CastField<FArrayProperty>(SourceProperty))
	{
		const FArrayProperty* DestPropertyAsArray = CastFieldChecked<FArrayProperty>(DestProperty);
		FScriptArrayHelper SourceArray(SourcePropertyAsArray, SourceValue);
		FScriptArrayHelper DestArray(DestPropertyAsArray, DestValue);
		DestArray.Resize(SourceArray.Num());
		for (int32 I = 0; I < SourceArray.Num(); ++I)
		{
			CopyProperty(SourcePropertyAsArray->Inner, SourceArray.GetElementPtr(I),
					DestPropertyAsArray->Inner, DestArray.GetElementPtr(I));
		}
	}
	else if (const FSetProperty* SourcePropertyAsSet = CastField<FSetProperty>(SourceProperty))
	{
		const FSetProperty* DestPropertyAsSet = CastFieldChecked<FSetProperty>(DestProperty);
		FScriptSetHelper SourceSet(SourcePropertyAsSet, SourceValue);
		FScriptSetHelper DestSet(DestPropertyAsSet, DestValue);
		DestSet.Set->Empty(0, DestSet.SetLayout);
		for (FScriptSetHelper::FIterator Itr = SourceSet.CreateIterator(); Itr; ++Itr)
		{
			void* DestChild = DestSet.GetElementPtr(DestSet.AddUninitializedValue());
			DestSet.ElementProp->InitializeValue(DestChild);
			
			CopyProperty(SourceSet.ElementProp, SourceSet.GetElementPtr(Itr.GetInternalIndex()),
					DestSet.ElementProp, DestChild);
		}
		DestSet.Rehash();
	}
	else if (const FMapProperty* SourcePropertyAsMap = CastField<FMapProperty>(SourceProperty))
	{
		const FMapProperty* DestPropertyAsMap = CastFieldChecked<FMapProperty>(DestProperty);
		FScriptMapHelper SourceMap(SourcePropertyAsMap, SourceValue);
		FScriptMapHelper DestMap(DestPropertyAsMap, DestValue);
		DestMap.EmptyValues();
		for (FScriptMapHelper::FIterator Itr = SourceMap.CreateIterator(); Itr; ++Itr)
		{
			void* DestChildKey = DestMap.GetKeyPtr(DestMap.AddUninitializedValue());
			DestMap.KeyProp->InitializeValue(DestChildKey);
			
			CopyProperty(SourceMap.KeyProp, SourceMap.GetKeyPtr(Itr.GetInternalIndex()),
					DestMap.KeyProp, DestChildKey);
			
			void* DestChildValue = DestMap.GetValuePtr(DestMap.AddUninitializedValue());
			DestMap.ValueProp->InitializeValue(DestChildValue);
			
			CopyProperty(SourceMap.ValueProp, SourceMap.GetValuePtr(Itr.GetInternalIndex()),
					DestMap.ValueProp, DestChildValue);
		}
		DestMap.Rehash();
	}
}


// This is a utility struct that's used to take an existing FPropertyChangedChainEvent, and remap its property chain to instead have the properties
// of a different class. In the process of remapping, it also resolves the data pointer of both the source property, and the resolved property.
struct FRemappedChainEvent : public FPropertyChangedChainEvent
{
public:
	FRemappedChainEvent(const FPropertyChangedChainEvent& InEvent, const UObject* InSourceObject, const UObject* InObjectToResolve)
		: FPropertyChangedChainEvent(this->FRemappedChainEvent::PropertyChain, InEvent)
		, SourceEvent(InEvent)
		, SourceChainNode(nullptr)
		, ResolvedChainNode(nullptr)
		, SourceField(InSourceObject->GetClass())
		, ResolvedField(InObjectToResolve->GetClass())
		, SourceMemoryPtr(const_cast<UObject*>(InSourceObject))
		, ResolvedMemoryPtr(const_cast<UObject*>(InObjectToResolve))
		, SourceObject(InSourceObject)
		, ObjectToResolve(InObjectToResolve)
	{
		ResolvedArrayIndices.AddDefaulted(InEvent.ObjectIteratorIndex + 1);
		SetArrayIndexPerObject(ResolvedArrayIndices);

		// Iterate property chain, Remap fields, and resolve data pointers
		while(!ConstructionComplete())
		{
			if (UStruct* ResolvedStruct = ResolvedField.Get<UStruct>())
			{
				SourceChainNode = (SourceChainNode) ? SourceChainNode->GetNextNode() : SourceEvent.PropertyChain.GetActiveMemberNode();
				check(SourceChainNode);
				
				SourceField = SourceChainNode->GetValue();
				SourceMemoryPtr = SourceMemoryPtr ? SourceField.Get<FProperty>()->ContainerPtrToValuePtr<void>(SourceMemoryPtr) : nullptr;
				
				ResolvedChainNode = ResolveCurChainNode(ResolvedStruct, SourceField.Get<FProperty>());
				ResolvedField = ResolvedChainNode ? ResolvedChainNode->GetValue() : nullptr;
				ResolvedMemoryPtr = (ResolvedChainNode && ResolvedMemoryPtr) ? ResolvedField.Get<FProperty>()->ContainerPtrToValuePtr<void>(ResolvedMemoryPtr) : nullptr;
			}
			else if (UObject* ResolvedObject = ResolvedField.Get<UObject>())
			{
				check(false);
			}
			else if (FArrayProperty* ResolvedArrayProperty = ResolvedField.Get<FArrayProperty>())
			{
				FArrayProperty* SourceArrayProperty = SourceField.Get<FArrayProperty>();
				int32 ArrayIndex = SourceEvent.GetArrayIndex(SourceArrayProperty->GetName());
				if (ArrayIndex == INDEX_NONE)
				{
					checkf(SourceChainNode->GetNextNode() == nullptr, TEXT("Expected this to be the last property because there's no index"));
					SourceField = nullptr;
					ResolvedChainNode = nullptr;
					ResolvedField = nullptr;
				}
				else
				{
					FScriptArrayHelper SourceArrayHelper(SourceArrayProperty, SourceMemoryPtr);
	                FScriptArrayHelper ResolvedArrayHelper(ResolvedArrayProperty, ResolvedMemoryPtr);
	                if(ResolvedMemoryPtr && !ResolvedArrayHelper.IsValidIndex(ArrayIndex))
	                {
						if (ChangeType & EPropertyChangeType::ArrayAdd)
						{
							check(ResolvedArrayHelper.Num() == ArrayIndex);
							ResolvedArrayHelper.Resize(ArrayIndex + 1);
						}
						else
						{
							ResolvedMemoryPtr = nullptr;
						}
	                }
	                ResolvedArrayIndices[SourceEvent.ObjectIteratorIndex].Add(ResolvedArrayProperty->GetName(), ArrayIndex);
	                SourceMemoryPtr = SourceMemoryPtr ? SourceArrayHelper.GetRawPtr(ArrayIndex) : nullptr;
	                ResolvedMemoryPtr = ResolvedMemoryPtr ? ResolvedArrayHelper.GetRawPtr(ArrayIndex) : nullptr;
	                SourceField = SourceArrayProperty->Inner;
	                ResolvedField = ResolvedArrayProperty->Inner;
				}
			}
			else if (FSetProperty* ResolvedSetProperty = ResolvedField.Get<FSetProperty>())
			{
				FSetProperty* SourceSetProperty = SourceField.Get<FSetProperty>();
                int32 SetIndex = SourceEvent.GetArrayIndex(SourceSetProperty->GetName());
                if (SetIndex == INDEX_NONE)
                {
                	checkf(SourceChainNode->GetNextNode() == nullptr, TEXT("Expected this to be the last property because there's no index"));
                	SourceField = nullptr;
                	ResolvedChainNode = nullptr;
                	ResolvedField = nullptr;
                }
                else
                {
                	FScriptSetHelper SourceSetHelper(SourceSetProperty, SourceMemoryPtr);
                	FScriptSetHelper ResolvedSetHelper(ResolvedSetProperty, ResolvedMemoryPtr);
                	if(ResolvedMemoryPtr && !ResolvedSetHelper.IsValidIndex(SetIndex))
                	{
						if (ChangeType & EPropertyChangeType::ArrayAdd)
						{
							check(ResolvedSetHelper.Num() == SetIndex);
							int32 AddedIndex = ResolvedSetHelper.AddUninitializedValue();
							check(AddedIndex == SetIndex);
						}
						else
						{
							ResolvedMemoryPtr = nullptr;
						}
                	}
                    ResolvedArrayIndices[SourceEvent.ObjectIteratorIndex].Add(ResolvedSetProperty->GetName(), SetIndex);
                	SourceMemoryPtr = SourceMemoryPtr ? SourceSetHelper.GetElementPtr(SetIndex) : nullptr;
                	ResolvedMemoryPtr = ResolvedMemoryPtr ? ResolvedSetHelper.GetElementPtr(SetIndex) : nullptr;
                	SourceField = SourceSetProperty->ElementProp;
                	ResolvedField = ResolvedSetProperty->ElementProp;
                }
			}
			else if (FMapProperty* ResolvedMapProperty = ResolvedField.Get<FMapProperty>())
			{
				FMapProperty* SourceMapProperty = SourceField.Get<FMapProperty>();
				int32 MapIndex = SourceEvent.GetArrayIndex(SourceMapProperty->GetName());
				if (MapIndex == INDEX_NONE)
				{
					checkf(SourceChainNode->GetNextNode() == nullptr, TEXT("Expected this to be the last property because there's no index"));
					SourceField = nullptr;
					ResolvedChainNode = nullptr;
					ResolvedField = nullptr;
				}
				else
				{
					FScriptMapHelper SourceMapHelper(SourceMapProperty, SourceMemoryPtr);
					FScriptMapHelper ResolvedMapHelper(ResolvedMapProperty, ResolvedMemoryPtr);
					if(ResolvedMemoryPtr && !ResolvedMapHelper.IsValidIndex(MapIndex))
					{
						if (ChangeType & EPropertyChangeType::ArrayAdd)
						{
							check(ResolvedMapHelper.Num() == MapIndex);
							int32 AddedIndex = ResolvedMapHelper.AddUninitializedValue();
							check(AddedIndex == MapIndex);
						}
						else
						{
							ResolvedMemoryPtr = nullptr;
						}
					}
	                ResolvedArrayIndices[SourceEvent.ObjectIteratorIndex].Add(ResolvedMapProperty->GetName(), MapIndex);
					SourceMemoryPtr = SourceMemoryPtr ? SourceMapHelper.GetValuePtr(MapIndex) : nullptr;
					ResolvedMemoryPtr = ResolvedMemoryPtr ? ResolvedMapHelper.GetValuePtr(MapIndex) : nullptr;
					SourceField = SourceMapProperty->ValueProp;
					ResolvedField = ResolvedMapProperty->ValueProp;
				}
			}
			else if (FStructProperty* ResolvedStructProperty = ResolvedField.Get<FStructProperty>())
			{
				FStructProperty* SourceStructProperty = SourceField.Get<FStructProperty>();
				SourceField = SourceStructProperty->Struct;
				ResolvedField = ResolvedStructProperty->Struct;
			}
			else if (FObjectProperty* ResolvedObjectProperty = ResolvedField.Get<FObjectProperty>())
			{
				UObject* ResolvedSubObject = GetObjectRefFromProperty(ResolvedObjectProperty, ResolvedMemoryPtr, ObjectToResolve);
				ResolvedField = ResolvedSubObject ? ResolvedSubObject->GetClass() : ResolvedObjectProperty->PropertyClass.Get();
				ResolvedMemoryPtr = ResolvedSubObject;
				
				FObjectProperty* SourceObjectProperty = SourceField.Get<FObjectProperty>();
				UObject* SourceSubObject = GetObjectRefFromProperty(SourceObjectProperty, SourceMemoryPtr, SourceObject);
				SourceField = SourceSubObject ? SourceSubObject->GetClass() : SourceObjectProperty->PropertyClass.Get();
				SourceMemoryPtr = SourceSubObject;
			}
			else if (FOptionalProperty* ResolvedOptionalProperty = ResolvedField.Get<FOptionalProperty>())
			{
				FOptionalProperty* SourceOptionalProperty = SourceField.Get<FOptionalProperty>();
				
				SourceField = SourceOptionalProperty->GetValueProperty();
            	ResolvedField = ResolvedOptionalProperty->GetValueProperty();
				if (SourceMemoryPtr)
				{
					if (SourceOptionalProperty->IsSet(SourceMemoryPtr))
					{
						SourceMemoryPtr = SourceOptionalProperty->GetValuePointerForReadOrReplace(SourceMemoryPtr);
					}
				
					if (ResolvedMemoryPtr)
					{
						if (!ResolvedOptionalProperty->IsSet(ResolvedMemoryPtr))
						{
							ResolvedMemoryPtr = ResolvedOptionalProperty->MarkSetAndGetInitializedValuePointerToReplace(ResolvedMemoryPtr);
						}
						else
						{
							ResolvedMemoryPtr = ResolvedOptionalProperty->GetValuePointerForReadOrReplace(ResolvedMemoryPtr);
						}
					}
				}
			}
			else
			{
				checkf(SourceChainNode->GetNextNode() == nullptr, TEXT("Expected this to be the last property in the chain because it's not a type known to contain properties"));
				SourceChainNode = nullptr;
				SourceField = nullptr;
				
				ResolvedChainNode = nullptr;
				ResolvedField = nullptr;
			}
			
			if (SourceEvent.Property == SourceField.Get<FProperty>())
			{
				Property = ResolvedField.Get<FProperty>();
			}
			if (SourceEvent.MemberProperty == SourceField.Get<FProperty>())
			{
				MemberProperty = ResolvedField.Get<FProperty>();
			}
			if (SourceEvent.PropertyChain.GetActiveNode() == SourceChainNode)
			{
				PropertyChain.SetActivePropertyNode(ResolvedChainNode ? ResolvedChainNode->GetValue() : nullptr);
			}
			if (SourceEvent.PropertyChain.GetActiveMemberNode() == SourceChainNode)
			{
				PropertyChain.SetActiveMemberPropertyNode(ResolvedChainNode ? ResolvedChainNode->GetValue() : nullptr);
			}
		}

		// For add changes, SourceField probably has one more array index that needs to be added. Add that here
		if (int32 AddChangeIndex = SourceEvent.GetArrayIndex(SourceField.GetName()); AddChangeIndex != INDEX_NONE)
		{
			ResolvedArrayIndices[SourceEvent.ObjectIteratorIndex].Add(ResolvedField.GetName(), AddChangeIndex);
		}
	}

	bool ConstructionComplete() const
	{
		if (!ResolvedField)
		{
			// a field in the chain wasn't in the destination class. This is an unrecoverable failure to resolve.
			return true;
		}
		if (FProperty* SourceProperty = SourceField.Get<FProperty>())
		{
			if (SourceEvent.GetArrayIndex(SourceProperty->GetName()) != INDEX_NONE && SourceProperty->Owner.Get<FProperty>() == SourceEvent.Property)
			{
				return false;
			}
			return SourceProperty == SourceEvent.Property;
		}
		check(SourceField.IsValid());
		return SourceEvent.PropertyChain.GetActiveMemberNode() == nullptr;
	}

	FEditPropertyChain::TDoubleLinkedListNode* ResolveCurChainNode(const UStruct* ResolvedStruct, const FProperty* SourceProperty)
	{
		if (FProperty* FoundProperty = FindPropertyByNameAndType(ResolvedStruct, SourceProperty))
		{
			PropertyChain.AddTail(FoundProperty);
			return PropertyChain.GetTail();
		}
		return nullptr;
	}

	static UObject* GetObjectRefFromProperty(const FObjectProperty* Property, const void* Memory, const UObject* OwningObject)
	{
		if (!Memory)
		{
			return nullptr;
		}
		UObject* SubObject = Property->GetObjectPropertyValue(Memory);
		if (SubObject && UE::IsClassOfInstanceDataObjectClass(OwningObject->GetClass()))
		{
			if (UObject* FoundIdo = FPropertyBagRepository::Get().FindInstanceDataObject(SubObject))
			{
				SubObject = FoundIdo;
			}
		}
		return SubObject;
	}

	FEditPropertyChain PropertyChain;
	const FPropertyChangedChainEvent& SourceEvent;
	FEditPropertyChain::TDoubleLinkedListNode* SourceChainNode;
	
	FEditPropertyChain::TDoubleLinkedListNode* ResolvedChainNode;
	TArray<TMap<FString, int32>> ResolvedArrayIndices;
	
	FFieldVariant SourceField;
	FFieldVariant ResolvedField;
	
	void* SourceMemoryPtr;
	void* ResolvedMemoryPtr;
	const UObject* SourceObject;
	const UObject* ObjectToResolve;
};

void FPropertyBagRepository::PostEditChangeChainProperty(UObject* Object, const FPropertyChangedChainEvent& PropertyChangedEvent)
{
#if WITH_EDITOR
	static TSet<TSoftObjectPtr<UObject>> ChangeCallbacksToSkip;
	if (ChangeCallbacksToSkip.Remove(Object))
	{
		// avoids infinite recursion
		return;
	}
	
	auto CopyChanges = [&PropertyChangedEvent](const UObject* Source, UObject* Dest)
	{
		FRemappedChainEvent RemappedChainEvent(PropertyChangedEvent, Source, Dest);
		
		
		FProperty* SourceProperty = RemappedChainEvent.SourceField.Get<FProperty>();
		FProperty* DestProperty = RemappedChainEvent.ResolvedField.Get<FProperty>();
		if (SourceProperty && DestProperty && RemappedChainEvent.SourceMemoryPtr && RemappedChainEvent.ResolvedMemoryPtr)
		{
			Dest->PreEditChange(RemappedChainEvent.PropertyChain);
			CopyProperty(
				SourceProperty,
				RemappedChainEvent.SourceMemoryPtr,
				DestProperty,
				RemappedChainEvent.ResolvedMemoryPtr);
			Dest->PostEditChangeChainProperty(RemappedChainEvent);
		} 
	};
	
	if (UObject* Ido = Get().FindInstanceDataObject(Object))
	{
		// if this object is an instance, modify it's IDO as well
		ChangeCallbacksToSkip.Add(Ido); // avoid infinite recursion
		CopyChanges(Object, Ido);
	}
	else if (UObject* Instance = const_cast<UObject*>(Get().FindInstanceForDataObject(Object)))
	{
		// if this object is an InstanceDataObject, modify it's owner as well
		ChangeCallbacksToSkip.Add(Instance); // avoid infinite recursion
		CopyChanges(Object, Instance);
	}
#endif
}

void FPropertyBagRepository::PostLoadInstanceDataObject(const UObject* Owner)
{
	// fixups may have been applied to the instance during PostLoad and they need to be copied to its IDO
	FPropertyBagAssociationData* BagData = nullptr;
	{
		FPropertyBagRepositoryLock LockRepo(this);
		BagData = AssociatedData.Find(Owner);
	}
	if (BagData)
	{
		if (BagData->InstanceDataObject)
		{
			// copy data from owner to IDO
			CopyTaggedProperties(Owner, BagData->InstanceDataObject);

			// the owner's PostLoad() may have mutated its instanced subobjects as well (e.g. pointer fixup). to handle
			// that case, we look for any instanced subobjects that have already had their PostLoad() called, as those will
			// not have a chance to get their IDO data fixed up to match changes potentially made by its owner's PostLoad().
			TArray<UObject*> InstancedSubObjects;
			GetObjectsWithOuter(Owner, InstancedSubObjects, EGetObjectsFlags::None);
			for (UObject* InstancedSubObject : InstancedSubObjects)
			{
				if (!InstancedSubObject->HasAnyFlags(RF_NeedPostLoad))
				{
					PostLoadInstanceDataObject(InstancedSubObject);
				}
			}

#if !UE_BUILD_SHIPPING_WITH_EDITOR && !UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			// Run IDT operations on this instance to recover data
			TObjectPtr<UObject> InOutInstance = const_cast<UObject*>(Owner);
			UE::FInstanceDataTransforms::Get().PatchInstanceDataObject(BagData->InstanceDataObject, InOutInstance);
#endif
		}
	}
}

// TODO: Remove this? Bag destruction to be handled entirely via UObject::BeginDestroy() (+ FPropertyBagProperty destructor)?
void FPropertyBagRepository::DestroyOuterBag(const UObject* Owner)
{
	FPropertyBagRepositoryLock LockRepo(this);
	RemoveAssociationUnsafe(Owner);
}

bool FPropertyBagRepository::RequiresFixup(const UObject* Object, bool bIncludeOuter) const
{
	TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree = FUnknownPropertyTree(Object).Find();
	bool bResult = UnknownPropertyTree ? !UnknownPropertyTree->IsEmpty() : false;
	if (!bResult && bIncludeOuter)
	{
		ForEachObjectWithOuterBreakable(Object,
			[&bResult, this](UObject* Object) 
			{
				TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree = FUnknownPropertyTree(Object).Find();
				if (UnknownPropertyTree && !UnknownPropertyTree->IsEmpty())
				{
					bResult = true;
					return false;
				}
				return true;
			}, EGetObjectsFlags::IncludeNestedObjects);
	}
	return bResult;
}

bool FPropertyBagRepository::RequiresFixup(const UObject* Object, const FPropertyPathName& Path) const
{
	TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree = FUnknownPropertyTree(Object).Find();
	if (!UnknownPropertyTree || UnknownPropertyTree->IsEmpty())
	{
		return false;
	}

	// look for either the full path in the tree, or a leaf node in the tree that matches the beginning of the path.
	// the latter scenario can occur because if a struct is a leaf, all it's child nodes are implicitly also unknown properties so they're not included in the tree
	UE::FPropertyPathName SubPath = Path;
	while (SubPath.GetSegmentCount() > 0)
	{
		if (UE::FPropertyPathNameTree::FNode Found = UnknownPropertyTree->Find(SubPath))
		{
			return (SubPath.GetSegmentCount() == Path.GetSegmentCount()) || (!Found.GetSubTree());
		}
		SubPath.Pop();
	}

	return false;
}

bool FPropertyBagRepository::RemoveAssociationUnsafe(const UObject* Owner)
{
	const UStruct* OwnerAsTypeObject = Cast<const UStruct>(Owner);
	if (OwnerAsTypeObject && IsPropertyBagPlaceholderType(OwnerAsTypeObject))
	{
		FPropertyBagPlaceholderTypeRegistry::Get().Remove(OwnerAsTypeObject);
		return true;
	}

	FPropertyBagAssociationData OldData;
	if (AssociatedData.RemoveAndCopyValue(Owner, OldData))
	{
		InstanceDataObjectToOwner.Remove(OldData.InstanceDataObject);
		OldData.Destroy();
		return true;
	}
	return false;
}

bool FPropertyBagRepository::HasInstanceDataObject(const UObject* Object) const
{
	FPropertyBagRepositoryLock LockRepo(this);
	// May be lazily instantiated, but implied from existence of object data.
	return AssociatedData.Contains(Object);
}

UObject* FPropertyBagRepository::FindInstanceDataObject(const UObject* Object)
{
	FPropertyBagRepositoryLock LockRepo(this);
	const FPropertyBagAssociationData* BagData = AssociatedData.Find(Object);
	return BagData ? BagData->InstanceDataObject : nullptr;
}

const UObject* FPropertyBagRepository::FindInstanceDataObject(const UObject* Object) const
{
	return const_cast<FPropertyBagRepository*>(this)->FindInstanceDataObject(Object);
}

void FPropertyBagRepository::FindNestedInstanceDataObject(const UObject* Owner, bool bRequiresFixupOnly, TFunctionRef<void(UObject*)> Callback)
{
	FPropertyBagRepositoryLock LockRepo(this);

	if (const FPropertyBagAssociationData* BagData = AssociatedData.Find(Owner); 
		BagData && BagData->InstanceDataObject && (!bRequiresFixupOnly || RequiresFixup(Owner, false)))
	{
		Callback(BagData->InstanceDataObject);
	}

	ForEachObjectWithOuter(Owner,
		[this, bRequiresFixupOnly, Callback](UObject* Object)
		{
			if (const FPropertyBagAssociationData* BagData = AssociatedData.Find(Object);
				BagData && BagData->InstanceDataObject && (!bRequiresFixupOnly || RequiresFixup(Object, false)))
			{
				Callback(BagData->InstanceDataObject);
			}
		}, EGetObjectsFlags::IncludeNestedObjects);
}

void FPropertyBagRepository::AddReferencedInstanceDataObject(const UObject* Object, FReferenceCollector& Collector)
{
	TObjectPtr<UObject> InstanceDataObject;
	{
		FPropertyBagRepositoryLock LockRepo(this);
		FPropertyBagAssociationData* BagData = AssociatedData.Find(Object);
		if (!BagData || !BagData->InstanceDataObject)
		{
			return;
		}
		InstanceDataObject = BagData->InstanceDataObject;
	}
	Collector.AddReferencedObject(InstanceDataObject, Object);
}

const UObject* FPropertyBagRepository::FindInstanceForDataObject(const UObject* InstanceDataObject) const
{
	FPropertyBagRepositoryLock LockRepo(this);
	const UObject* const* Owner = InstanceDataObjectToOwner.Find(InstanceDataObject);
	return Owner ? *Owner : nullptr;
}

UObject* FPropertyBagRepository::ReinstanceWithCachedUnknownPropertyTree(UObject* Owner)
{
	if (FPropertyBagRepositoryLock LockRepo(this); FPropertyBagAssociationData* BagData = AssociatedData.Find(Owner))
	{
		TSharedPtr<FPropertyPathNameTree> UnknownPropertyTree = FUnknownPropertyTree(Owner).FindOrCreate();
		FUnknownEnumNames UnknownEnumNames = FUnknownEnumNames(Owner);

		UObject* IDO = BagData->InstanceDataObject;
		UClass* NewIdoClass = CreateInstanceDataObjectClass(UnknownPropertyTree.Get(), &UnknownEnumNames, Owner->GetClass(), IDO->GetClass()->GetOuter());

		UObject* OwnerArchetype = nullptr;
		if (FArchive* Archive = Owner->GetLinker())
		{
			OwnerArchetype = Archive->GetArchetypeFromLoader(Owner);
		}
		if (!OwnerArchetype)
		{
			OwnerArchetype = Owner->GetArchetype();
		}
		UObject* Template = GetTemplateForInstanceDataObject(Owner, OwnerArchetype, NewIdoClass);

		// Construct the new IDO.
		FStaticConstructObjectParameters ConstructParams(NewIdoClass);
		ConstructParams.SetFlags |= EObjectFlags::RF_Transactional;
		// Set the RF_ArchetypeObject flag on all IDO archetypes so that they can be identified as such.
		if (IDO->HasAllFlags(EObjectFlags::RF_ArchetypeObject))
		{
			ConstructParams.SetFlags |= EObjectFlags::RF_ArchetypeObject;
		}

		ConstructParams.Name = MakeUniqueObjectName(IDO->GetOuter(), ConstructParams.Class, Owner->GetFName(), EUniqueObjectNameOptions::UniversallyUnique);
		ConstructParams.Outer = IDO->GetOuter();
		ConstructParams.Template = Template;

		UObject* NewIDO = StaticConstructObject_Internal(ConstructParams);

		CopyTaggedProperties(IDO, NewIDO);

		// replace the old IDO with the new one in the repository
		InstanceDataObjectToOwner.Remove(BagData->InstanceDataObject);
		BagData->InstanceDataObject = NewIDO;
		InstanceDataObjectToOwner.Add(NewIDO, Owner);

		return NewIDO;
	}
	return nullptr;
}

UObject* FPropertyBagRepository::CreateInstanceDataObject(UObject* Owner, FArchive& Archive, int64 StartOffset, int64 EndOffset, bool bIsArchetype)
{
	// Limit the scope of the lock during to the find because this calls itself recursively to handle archetypes.
	if (FPropertyBagRepositoryLock LockRepo(this); FPropertyBagAssociationData* BagData = AssociatedData.Find(Owner))
	{
		check(BagData->InstanceDataObject);
		return BagData->InstanceDataObject;
	}

	// use the IDO of the outer as the new outer if there is one, otherwise use the transient package
	UObject* Outer = nullptr;
	if (IsInstanceDataObjectOuterChainEnabled())
	{
		FPropertyBagRepositoryLock LockRepo(this);
		if (const FPropertyBagAssociationData* OuterData = AssociatedData.Find(Owner->GetOuter()))
		{
			Outer = OuterData->InstanceDataObject;
		}
	}
	if (!Outer)
	{
		Outer = GetTransientPackage();
	}

	// Construct the class for the IDO.
	TSharedPtr<FPropertyPathNameTree> PropertyTree = FUnknownPropertyTree(Owner).Find();
	const FUnknownEnumNames EnumNames(Owner);
	const FUnknownEnumNames* EnumNamesPtr = !EnumNames.IsEmpty() ? &EnumNames : nullptr;
	// Don't create Instance Data Objects if they're not needed to store unknown properties
	if (!PropertyTree || PropertyTree->IsEmpty())
	{
		return nullptr;
	}
	const UClass* InstanceDataObjectClass = CreateInstanceDataObjectClass(PropertyTree.Get(), EnumNamesPtr, Owner->GetClass(), GetTransientPackage());

	// Find the template for the IDO.

	//@todoio GetArchetype is pathological for blueprint classes and the event driven loader; the EDL already knows what the archetype is; just calling this->GetArchetype() tries to load some other stuff.
	UObject* OwnerArchetype = Archive.GetArchetypeFromLoader(Owner);
	if (!OwnerArchetype)
	{
		OwnerArchetype = Owner->GetArchetype();
	}
	UObject* Template = GetTemplateForInstanceDataObject(Owner, OwnerArchetype, InstanceDataObjectClass);

	// Construct the IDO.
	FStaticConstructObjectParameters Params(InstanceDataObjectClass);
	Params.SetFlags |= EObjectFlags::RF_Transactional;
	// Set the RF_ArchetypeObject flag on all IDO archetypes so that they can be identified as such.
	if (bIsArchetype || Owner->HasAllFlags(EObjectFlags::RF_ArchetypeObject))
	{
		Params.SetFlags |= EObjectFlags::RF_ArchetypeObject;
	}

	Params.Name = MakeUniqueObjectName(Outer, Params.Class, Owner->GetFName(), EUniqueObjectNameOptions::UniversallyUnique);
	Params.Outer = Outer;
	Params.Template = Template;

	UObject* InstanceDataObject = StaticConstructObject_Internal(Params);

	FPropertyBagRepositoryLock LockRepo(this);
	InstanceDataObjectToOwner.Add(InstanceDataObject, Owner);

	FPropertyBagAssociationData& BagData = AssociatedData.Add(Owner);
	BagData.InstanceDataObject = InstanceDataObject;

	// Load the IDO.
	if (StartOffset != EndOffset)
	{
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
		// we don't need archetype matching and the FArchetypeMatchingArchive doesn't bounds
		// check so is unsafe in universal placeholder contexts
		FArchive& ArchiveWrapper = Archive;
#else // !UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
		// We want to force the template we used as the archetype during SerializeScriptProperties. For this reason 
		// we wrap the archive in a proxy that will return the correct archetype.
		FArchetypeMatchingArchive ArchiveWrapper(Archive, InstanceDataObject, Template);
#endif

		const int64 OffsetToRestore = ArchiveWrapper.Tell();
		FScopedObjectSerializeContext ObjectSerializeContext(InstanceDataObject, ArchiveWrapper);
		ArchiveWrapper.Seek(StartOffset);
		{
			FGuardValue_Bitfield(ArchiveWrapper.ArMergeOverrides, true);
			InstanceDataObject->SerializeScriptProperties(ArchiveWrapper);
		}
		ensureMsgf(ArchiveWrapper.Tell() == EndOffset,
			TEXT("Serializing %s into its IDO consumed %" INT64_FMT " bytes when %" INT64_FMT " bytes were expected."),
			*Owner->GetPathName(), ArchiveWrapper.Tell() - StartOffset, EndOffset - StartOffset);
		ArchiveWrapper.Seek(OffsetToRestore);
	}
	return BagData.InstanceDataObject;
}

// Not sure this is necessary.
void FPropertyBagRepository::ShrinkMaps()
{
	FPropertyBagRepositoryLock LockRepo(this);
	AssociatedData.Compact();
	InstanceDataObjectToOwner.Compact();
}

bool FPropertyBagRepository::IsPropertyBagPlaceholderType(const UStruct* Type)
{
	if (!Type)
	{
		return false;
	}

	return FPropertyBagPlaceholderTypeRegistry::Get().Contains(Type);
}

bool FPropertyBagRepository::IsPropertyBagPlaceholderObject(const UObject* Object)
{
	if (!Object)
	{
		return false;
	}

	return IsPropertyBagPlaceholderType(Object->GetClass());
}

namespace Private
{
#if WITH_EDITOR
	using EPlaceholderObjectFeature = UE::FPropertyBagRepository::EPlaceholderObjectFeature;
	constexpr uint32 PropertyBagPlaceholderObjectFeatureFlag(EPlaceholderObjectFeature Feature)
	{
		return 1 << static_cast<std::underlying_type_t<EPlaceholderObjectFeature>>(Feature);
	}

	static uint32 PropertyBagPlaceholderObjectEnabledFeaturesMask =
		PropertyBagPlaceholderObjectFeatureFlag(EPlaceholderObjectFeature::ReplaceMissingTypeImportsOnLoad);

	#define DEFINE_IDO_PLACEHOLDER_FEATURE_FLAG_CVAR(FEATURE) \
		static bool bEnablePropertyBagPlaceholderObjectFeature_##FEATURE = !!(PropertyBagPlaceholderObjectEnabledFeaturesMask & PropertyBagPlaceholderObjectFeatureFlag(EPlaceholderObjectFeature::FEATURE)); \
		static FAutoConsoleVariableRef CVarEnablePropertyBagPlaceholderObjectFeature_##FEATURE( \
			TEXT("IDO.Placeholder.Feature."#FEATURE), \
			bEnablePropertyBagPlaceholderObjectFeature_##FEATURE, \
			TEXT("Enable/disable IDO placeholder feature: "#FEATURE), \
			FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*) \
			{ \
				if (bEnablePropertyBagPlaceholderObjectFeature_##FEATURE) \
				{ \
					PropertyBagPlaceholderObjectEnabledFeaturesMask |= PropertyBagPlaceholderObjectFeatureFlag(EPlaceholderObjectFeature::FEATURE); \
				} \
				else \
				{ \
					PropertyBagPlaceholderObjectEnabledFeaturesMask &= ~PropertyBagPlaceholderObjectFeatureFlag(EPlaceholderObjectFeature::FEATURE); \
				} \
			}), \
			ECVF_Default \
		)

	DEFINE_IDO_PLACEHOLDER_FEATURE_FLAG_CVAR(ReplaceMissingTypeImportsOnLoad);
	DEFINE_IDO_PLACEHOLDER_FEATURE_FLAG_CVAR(SerializeExportReferencesOnLoad);
	DEFINE_IDO_PLACEHOLDER_FEATURE_FLAG_CVAR(ReplaceMissingReinstancedTypes);
	DEFINE_IDO_PLACEHOLDER_FEATURE_FLAG_CVAR(ReplaceDeadClassInstanceTypes);
	DEFINE_IDO_PLACEHOLDER_FEATURE_FLAG_CVAR(DoNotRestrictMissingTypeImports);
	DEFINE_IDO_PLACEHOLDER_FEATURE_FLAG_CVAR(AllowSavingPlaceholderExports);
#endif
}

bool FPropertyBagRepository::IsPropertyBagPlaceholderObjectSupportEnabled()
{
#if WITH_EDITOR && UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
	return IsInstanceDataObjectSupportEnabled() && IsInstanceDataObjectPlaceholderObjectSupportEnabled();
#else
	return false;
#endif
}

bool FPropertyBagRepository::IsPropertyBagPlaceholderObjectFeatureEnabled(EPlaceholderObjectFeature Feature)
{
#if WITH_EDITOR
	if (!IsPropertyBagPlaceholderObjectSupportEnabled())
	{
		return false;
	}

	return !!(Private::PropertyBagPlaceholderObjectEnabledFeaturesMask & Private::PropertyBagPlaceholderObjectFeatureFlag(Feature));
#else
	return false;
#endif
}

UStruct* FPropertyBagRepository::CreatePropertyBagPlaceholderType(UObject* Outer, UClass* Class, FName Name, EObjectFlags Flags, UStruct* SuperStruct)
{
	// Generate and link a new type object using the given SuperStruct as its base.
	UStruct* PlaceholderType = NewObject<UStruct>(Outer, Class, Name, Flags);
	PlaceholderType->SetSuperStruct(SuperStruct);
	PlaceholderType->Bind();
	PlaceholderType->StaticLink(/*bRelinkExistingProperties =*/ true);

	// Extra configuration needed for class types.
	if (UClass* PlaceholderTypeAsClass = Cast<UClass>(PlaceholderType))
	{
		// Create and configure its CDO as if it were loaded - for non-native class types, this is required.
		UObject* PlaceholderClassDefaults = PlaceholderTypeAsClass->GetDefaultObject();
		// CDOs for UClasses themselves cannot typically be post loaded
		// because they will attempt to bind to their native constructor
		// and there may be none for placeholder types..
		if(!PlaceholderClassDefaults->IsA<UClass>())
		{
			PlaceholderTypeAsClass->PostLoadDefaultObject(PlaceholderClassDefaults);
		}

		// This class is for internal use and should not be exposed for selection or instancing in the editor.
		PlaceholderTypeAsClass->ClassFlags |= CLASS_Hidden | CLASS_HideDropDown;

		// Required by garbage collection for class types.
		PlaceholderTypeAsClass->AssembleReferenceTokenStream();
	}
	// Extra configuration needed for verse class types.
	if (UVerseClass* PlaceholderTypeAsVerseClass = Cast<UVerseClass>(PlaceholderType))
	{
		PlaceholderTypeAsVerseClass->SolClassFlags |= VCLASS_FinalSuper | VCLASS_Castable | VCLASS_Concrete;
	}
	
	// Use the property bag repository for now to register property bag placeholder types for query purposes.
	// Note: Object lifetimes of this type and its instances depend on existing references that are serialized.
	FPropertyBagPlaceholderTypeRegistry::Get().Add(PlaceholderType);

	return PlaceholderType;
}

UObject* FPropertyBagRepository::FindFirstInstanceThatNeedsFixup(const UClass* Class)
{
	UObject* FoundInstance = nullptr;

	ForEachObjectOfClass(Class, [this, &FoundInstance](UObject* Instance)
		{
			// Quickly return if already found (No breakable version of ForEachObjectOfClass).
			if (FoundInstance != nullptr)
			{
				return;
			}

			if (RequiresFixup(Instance))
			{
				FoundInstance = Instance;
			}
		}, /*bIncludeDerivedClasses =*/ false);

	return FoundInstance;
}

#if STATS
void FPropertyBagRepository::GatherStats(FPropertyBagRepositoryStats& Stats)
{
	FMemory::Memset(Stats, 0);

	Stats.NumPlaceholderTypes = FPropertyBagPlaceholderTypeRegistry::Get().Num();

	FPropertyBagRepositoryLock LockRepo(this);

	for (const TPair<const UObject*, FPropertyBagAssociationData>& Pair : AssociatedData)
	{ 
		const FPropertyBagAssociationData& BagData = Pair.Value;

		if (BagData.InstanceDataObject != nullptr)
		{
			++Stats.NumIDOs;

			FArchiveCountMem MemoryCount(BagData.InstanceDataObject);
			Stats.IDOMemoryBytes += MemoryCount.GetMax();

			if (RequiresFixup(Pair.Key))
			{
				++Stats.NumIDOsWithLooseProperties;
			}
		}
	}
}
#endif // STATS

void FPropertyBagRepository::DumpIDOs(FOutputDevice& OutputDevice)
{
	OutputDevice.Log(TEXT("Instance Data Objects:"));
	OutputDevice.Log(TEXT("------------------------------"));

	int32 NumIDOs = 0;

	for (const TPair<const UObject*, FPropertyBagAssociationData>& Pair : AssociatedData)
	{
		const FPropertyBagAssociationData& BagData = Pair.Value;

		if (BagData.InstanceDataObject != nullptr)
		{
			++NumIDOs;

			OutputDevice.Logf(TEXT("%s"), *BagData.InstanceDataObject->GetFullName());

			const UObject* Owner = FindInstanceForDataObject(BagData.InstanceDataObject);
			OutputDevice.Logf(TEXT("  Owner: %s"), (Owner ? *Owner->GetFullName() : TEXT("NULL")));
		}
	}

	OutputDevice.Logf(TEXT("%d IDO(s)"), NumIDOs);
}

void FPropertyBagRepository::DumpPropertyBagPlaceholderTypes(FOutputDevice& OutputDevice)
{
	OutputDevice.Log(TEXT("Placeholder Types:"));
	OutputDevice.Log(TEXT("------------------------------"));

	TSet<TObjectPtr<const UStruct>> Types;
	FPropertyBagPlaceholderTypeRegistry::Get().GetTypes(Types);

	for (TObjectPtr<const UStruct> Type : Types)
	{
		OutputDevice.Logf(TEXT("%s"), *Type->GetFullName());
	}

	OutputDevice.Logf(TEXT("%d Placeholder type(s)"), Types.Num());
}

} // UE

#endif // WITH_EDITORONLY_DATA