// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "UObject/NonNullPropertyUtils.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/ObjectMacros.h"
#include "Templates/Casts.h"
#include "UObject/Interface.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectHash.h"
#include "Blueprint/BlueprintSupport.h"
#include "UObject/CoreNet.h"
#include "UObject/PropertyHelper.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/LinkerLoadImportBehavior.h"
#include "UObject/UObjectThreadContext.h"
#include "Misc/StringBuilder.h"

namespace UE::CoreUObject::Private
{
	TAutoConsoleVariable<int32> CVarNonNullableBehavior(
		TEXT("CoreUObject.NonNullableBehavior"),
		(int32)ENonNullableBehavior::CreateDefaultObjectIfPossible,
		TEXT("Sets the behavior when a non-null property cannot be resolved into an object reference - 0=Leave property null and log a warning, 1=Leave property null and log an error, 2=Create a default object and log a warning if successful, or leave it null and log an error if unsuccessful")
	);

	ENonNullableBehavior GetNonNullableBehavior()
	{
		int32 Value = CVarNonNullableBehavior.GetValueOnAnyThread();
		if (Value < 0 || Value > 2)
		{
			Value = 2;
		}
		return (ENonNullableBehavior)Value;
	}
}

/*-----------------------------------------------------------------------------
	FObjectPropertyBase.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FObjectPropertyBase)

FObjectPropertyBase::FObjectPropertyBase(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, PropertyClass(nullptr)
{
}

FObjectPropertyBase::FObjectPropertyBase(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParams& Prop, EPropertyFlags AdditionalPropertyFlags /*= CPF_None*/)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop, AdditionalPropertyFlags)
{
	PropertyClass = Prop.ClassFunc ? Prop.ClassFunc(ETypeConstructPhase::Inner) : nullptr;
}
FObjectPropertyBase::FObjectPropertyBase(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParamsWithoutClass& Prop, EPropertyFlags AdditionalPropertyFlags /*= CPF_None*/)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop, AdditionalPropertyFlags)
	, PropertyClass(nullptr)
{
}

#if WITH_EDITORONLY_DATA
FObjectPropertyBase::FObjectPropertyBase(UField* InField)
	: Super(InField)
{
	UObjectPropertyBase* SourceProperty = CastChecked<UObjectPropertyBase>(InField);
	PropertyClass = SourceProperty->PropertyClass;
}
#endif // WITH_EDITORONLY_DATA

void FObjectPropertyBase::InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, FObjectInstancingGraph* InstanceGraph )
{
	for ( int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++ )
	{
		TObjectPtr<UObject> CurrentObjectPtr = GetObjectPtrPropertyValue((uint8*)Data + ArrayIndex * GetElementSize());
		UObject* CurrentValue = CurrentObjectPtr.Get();
		if (CurrentObjectPtr.IsResolved() && CurrentValue)
		{
			TObjectPtr<UObject> SubobjectTemplate = DefaultData ? GetObjectPtrPropertyValue((uint8*)DefaultData + ArrayIndex * GetElementSize()): nullptr;
			EInstancePropertyValueFlags Flags = EInstancePropertyValueFlags::None;
			if (HasAnyPropertyFlags(CPF_InstancedReference))
			{
				Flags |= EInstancePropertyValueFlags::CausesInstancing;
			}
			if (HasAnyPropertyFlags(CPF_AllowSelfReference))
			{
				Flags |= EInstancePropertyValueFlags::AllowSelfReference;
			}
			UObject* NewValue = InstanceGraph->InstancePropertyValue(SubobjectTemplate, CurrentValue, InOwner, Flags);
			SetObjectPropertyValue((uint8*)Data + ArrayIndex * GetElementSize(), NewValue);
		}
	}
}

bool FObjectPropertyBase::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	// We never return Identical when duplicating for PIE because we want to be sure to serialize everything. An example is the LevelScriptActor being serialized against its CDO,
	// which contains actor references. We want to serialize those references so they are fixed up.
	if ((PortFlags & PPF_DuplicateForPIE) != 0)
	{
		return false;
	}

	UObject* ObjectA = GetObjectPropertyValue(A);
	UObject* ObjectB = B ? GetObjectPropertyValue(B) : nullptr;

	return StaticIdentical(ObjectA, ObjectB, PortFlags);
}

bool FObjectPropertyBase::StaticIdentical(UObject* ObjectA, UObject* ObjectB, uint32 PortFlags)
{
	if (ObjectA == ObjectB)
	{
		return true;
	}
	if (!ObjectA || !ObjectB)
	{
		return false;
	}

	bool bResult = false;

	// In order for a deep comparison of instanced objects to match both objects must have the same class and name
	if (ObjectA->GetClass() == ObjectB->GetClass() && ObjectA->GetFName() == ObjectB->GetFName())
	{
		bool bPerformDeepComparison = (PortFlags&PPF_DeepComparison) != 0;
		if (((PortFlags&PPF_DeepCompareInstances) != 0) && !bPerformDeepComparison)
		{
			bPerformDeepComparison = !(ObjectA->IsTemplate() && ObjectB->IsTemplate());
		}

		if (bPerformDeepComparison)
		{
			if ((PortFlags&PPF_DeepCompareDSOsOnly) != 0)
			{
				if (UObject* DSO = ObjectA->GetClass()->GetDefaultSubobjectByName(ObjectA->GetFName()))
				{
					checkSlow(ObjectA->IsDefaultSubobject() && ObjectB->IsDefaultSubobject() && DSO == ObjectB->GetClass()->GetDefaultSubobjectByName(ObjectB->GetFName()));
				}
				else
				{
					bPerformDeepComparison = false;
				}
			}

			if (bPerformDeepComparison)
			{
				bResult = AreInstancedObjectsIdentical(ObjectA, ObjectB, PortFlags);
			}
		}
	}
	return bResult;
}

bool FObjectPropertyBase::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData) const
{
	UObject* Object = GetObjectPropertyValue(Data);
	bool Result = Map->SerializeObject( Ar, PropertyClass, Object );
	// Prevent serializing invalid objects through network
	if (!Object || IsValidChecked(Object))
	{
		SetObjectPropertyValue(Data, Object);
	}
	return Result;
}
void FObjectPropertyBase::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar << PropertyClass;

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(PropertyClass))
		{
			PlaceholderClass->AddReferencingProperty(this);
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

void FObjectPropertyBase::PostDuplicate(const FField& InField)
{
	const FObjectPropertyBase& Source = static_cast<const FObjectPropertyBase&>(InField);
	PropertyClass = Source.PropertyClass;
	Super::PostDuplicate(InField);
}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
void FObjectPropertyBase::SetPropertyClass(UClass* NewPropertyClass)
{
	if (ULinkerPlaceholderClass* NewPlaceholderClass = Cast<ULinkerPlaceholderClass>(NewPropertyClass))
	{
		NewPlaceholderClass->AddReferencingProperty(this);
	}
	
	if (ULinkerPlaceholderClass* OldPlaceholderClass = Cast<ULinkerPlaceholderClass>(PropertyClass))
	{
		OldPlaceholderClass->RemoveReferencingProperty(this);
	}
	PropertyClass = NewPropertyClass;
}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

void FObjectPropertyBase::AddReferencedObjects(FReferenceCollector& Collector)
{	
	Collector.AddReferencedObject( PropertyClass );
	Super::AddReferencedObjects( Collector );
}

FString FObjectPropertyBase::GetExportPath(FTopLevelAssetPath ClassPathName, const FString& ObjectPathName)
{
	TStringBuilder<256> StringBuilder;
	StringBuilder << ClassPathName.GetPackageName() << "." << ClassPathName.GetAssetName() << "'" << ObjectPathName << "'";
	return FString(StringBuilder);
}

FString FObjectPropertyBase::GetExportPath(const TObjectPtr<const UObject>& Object, const UObject* Parent /*= nullptr*/, const UObject* ExportRootScope /*= nullptr*/, const uint32 PortFlags /*= PPF_None*/)
{
	bool bExportFullyQualified = true;

	// When exporting from one package or graph to another package or graph, we don't want to fully qualify the name, as it may refer
	// to a level or graph that doesn't exist or cause a linkage to a node in a different graph
	const UObject* StopOuter = nullptr;
	if (PortFlags & PPF_ExportsNotFullyQualified)
	{
		StopOuter = (ExportRootScope || (Parent == nullptr)) ? ExportRootScope : Parent->GetOutermost();
		bExportFullyQualified = StopOuter && !Object->IsIn(StopOuter);

		// Also don't fully qualify the name if it's a sibling of the root scope, since it may be included in the exported set of objects
		if (bExportFullyQualified)
		{
			StopOuter = StopOuter->GetOuter();
			bExportFullyQualified = (StopOuter == nullptr) || (!Object->IsIn(StopOuter));
		}
	}

	// if we want a full qualified object reference, use the pathname, otherwise, use just the object name
	if (bExportFullyQualified)
	{
		StopOuter = nullptr;
		if ( (PortFlags&PPF_SimpleObjectText) != 0 && Parent != nullptr )
		{
			StopOuter = Parent->GetOutermost();
		}
	}
	else if (Parent != nullptr && Object->IsIn(Parent))
	{
		StopOuter = Parent;
	}

	// Take the path name relative to the stopping point outermost ptr.
	// This is so that cases like a component referencing a component in another actor work correctly when pasted
	FString PathName = StopOuter ? Object->GetPathName(StopOuter) : Object.GetPathName();
	const FTopLevelAssetPath ClassPathName = Object.GetClass()->GetClassPathName();
	FString ExportPath = GetExportPath(ClassPathName, *PathName);
	// Object names that contain invalid characters and paths that contain spaces must be put into quotes to be handled correctly
	if (PortFlags & PPF_Delimited)
	{
		ExportPath = FString::Printf(TEXT("\"%s\""), *ExportPath.ReplaceQuotesWithEscapedQuotes());
	}
	return ExportPath;
}

void FObjectPropertyBase::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	TObjectPtr<UObject> Temp;

	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		GetValue_InContainer(PropertyValueOrContainer, &Temp);
	}
	else
	{
		Temp = GetObjectPtrPropertyValue(PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType));
	}

	if (!Temp)
	{
		ValueStr += TEXT("None");
	}
	else
	{
		if (PortFlags & PPF_DebugDump)
		{
			ValueStr += Temp ? Temp->GetFullName() : TEXT("None");
		}
		else if (
		         Parent && !Parent->HasAnyFlags(RF_ClassDefaultObject)
				 // @NOTE: OBJPTR: In the event that we're trying to handle a default subobject, the requirement would 
				 // be that it's inside the package we are currently in, which means the Temp pointer should be resolved 
				 // already.  So don't move forward with the check unless it's resolved, we don't want to force a deferred
				 // loaded asset if we don't have to with this check.
				 && Temp.IsResolved() && Temp && Temp->IsDefaultSubobject() 
				 // Use short name only if import will find it by searching Parent's outer chain.
				 // Import searches upward for objects with matching name,
				 // so Temp->GetOuter() must be Parent or an ancestor of Parent to avoid ambiguity.
				 && (Temp->GetOuter() && (Temp->GetOuter() == Parent || Parent->IsInOuter(Temp->GetOuter())))
				)
		{
			if (PortFlags & PPF_Delimited)
			{
				ValueStr += Temp ? FString::Printf(TEXT("\"%s\""), *Temp->GetName().ReplaceQuotesWithEscapedQuotes()) : TEXT("None");
			}
			else
			{
				ValueStr += Temp.GetName();
			}
		}
		else
		{
			ValueStr += GetExportPath(Temp, Parent, ExportRootScope, PortFlags);
		}
	}
}

/**
 * Parses a text buffer into an object reference.
 *
 * @param	Property			the property that the value is being importing to
 * @param	OwnerObject			the object that is importing the value; used for determining search scope.
 * @param	RequiredMetaClass	the meta-class for the object to find; if the object that is resolved is not of this class type, the result is nullptr.
 * @param	PortFlags			bitmask of EPropertyPortFlags that can modify the behavior of the search
 * @param	Buffer				the text to parse; should point to a textual representation of an object reference.  Can be just the object name (either fully 
 *								fully qualified or not), or can be formatted as a const object reference (i.e. SomeClass'SomePackage.TheObject')
 *								When the function returns, Buffer will be pointing to the first character after the object value text in the input stream.
 * @param	ResolvedValue		receives the object that is resolved from the input text.
 *
 * @return	true if the text is successfully resolved into a valid object reference of the correct type, false otherwise.
 */
bool FObjectPropertyBase::ParseObjectPropertyValue(const FProperty* Property, UObject* OwnerObject, UClass* RequiredMetaClass, uint32 PortFlags, const TCHAR*& Buffer, TObjectPtr<UObject>& out_ResolvedValue, FUObjectSerializeContext* InSerializeContext /*= nullptr*/, bool bAllowAnyPackage /*= true*/)
{
	check(Property);
	if (!RequiredMetaClass)
	{
		UE_LOGF(LogProperty, Error, "ParseObjectPropertyValue Error: RequiredMetaClass is null, for property: %ls ", *Property->GetFullName());
		out_ResolvedValue = nullptr;
		return false;
	}

 	const TCHAR* InBuffer = Buffer;

	TStringBuilder<256> Temp;
	Buffer = FPropertyHelpers::ReadToken(Buffer, /* out */ Temp, true);
	if ( Buffer == nullptr )
	{
		return false;
	}

	if ( Temp == TEXTVIEW("None") )
	{
		out_ResolvedValue = nullptr;
	}
	else
	{
		UClass*	ObjectClass = RequiredMetaClass;

		SkipWhitespace(Buffer);

		bool bWarnOnnullptr = (PortFlags&PPF_CheckReferences)!=0;

		if (*Buffer == TEXT('\''))
		{
			// This is likely the remnants of stripping off the class part of a fully qualified object path
			// eg) /Script/Module.Class'/Game/Foo.Foo:Name With A Space.Bar' -> '/Game/Foo.Foo:Name With A Space.Bar'
			// We walk to the next ' so that we correctly handle object paths with spaces in them, which is technically 
			// not allowed by INVALID_OBJECTNAME_CHARACTERS, but happens all the time in BPs
			Temp.Reset();
			{
				const int32 MaxTokenLength = FName::StringBufferSize - 1;
				Buffer = FPropertyHelpers::ReadTerminatedToken(++Buffer, /*Out*/Temp, TEXT('\''), &MaxTokenLength);
			}
			if (!Buffer || *Buffer++ != TEXT('\''))
			{
				return false;
			}

			// ignore the object class, it isn't fully qualified, and searching globally might get the wrong one!
			// Try the find the object.
			out_ResolvedValue = FObjectPropertyBase::FindImportedObject(Property, OwnerObject, ObjectClass, RequiredMetaClass, *Temp, PortFlags, InSerializeContext, bAllowAnyPackage);
		}
		else
		{
			// Try the find the object.
			out_ResolvedValue = FObjectPropertyBase::FindImportedObject(Property, OwnerObject, ObjectClass, RequiredMetaClass, *Temp, PortFlags, InSerializeContext, bAllowAnyPackage);
		}

		if ( out_ResolvedValue != nullptr && !out_ResolvedValue.GetClass()->IsChildOf(RequiredMetaClass) )
		{
			if (bWarnOnnullptr )
			{
				UE_LOGF(LogProperty, Error, "%ls: bad cast in '%ls'", *Property->GetFullName(), InBuffer );
			}

			out_ResolvedValue = nullptr;
			return false;
		}

		// If we couldn't find it or load it, we'll have to do without it.
		if ( out_ResolvedValue == nullptr )
		{
			if( bWarnOnnullptr )
			{
				UE_LOGF(LogProperty, Warning, "%ls: unresolved reference to '%ls'", *Property->GetFullName(), InBuffer );
			}
			return false;
		}
	}

	return true;
}

const TCHAR* FObjectPropertyBase::ImportText_Internal(const TCHAR* InBuffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	const TCHAR* Buffer = InBuffer;
	TObjectPtr<UObject> Result = nullptr;

	bool bOk = ParseObjectPropertyValue(this, Parent, PropertyClass, PortFlags, Buffer, Result, FUObjectThreadContext::Get().GetSerializeContext());

	bool bNeedsInstancing = false;
	if (Result && ((PortFlags & PPF_InstanceSubobjects) != 0))
	{
		if (HasAnyPropertyFlags(CPF_InstancedReference))
		{
			// Always duplicate the imported reference for explicitly-instanced properties.
			bNeedsInstancing = true;
		}
		else if (Parent && Parent->GetClass()->ShouldUseDynamicSubobjectInstancing())
		{
			// Only duplicate the imported reference when scoped to the parent or its archetype.
			if (Result->IsInOuter(Parent) || Result->IsInOuter(Parent->GetArchetype()))
			{
				bNeedsInstancing = true;
			}
		}
	}

	if (bNeedsInstancing)
	{
		FName DesiredName = Result->GetFName();

		// If an object currently exists with the same name as the imported object that is to be instanced
		// 
		if (UObject* ExistingObject = static_cast<UObject*>(FindObjectWithOuter(Parent, nullptr, DesiredName)))
		{
			ExistingObject->Rename(nullptr, nullptr, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		}
		
		const EObjectFlags MaskedOuterFlags = Parent ? Parent->GetMaskedFlags(RF_PropagateToSubObjects) : RF_AllFlags;
		FObjectDuplicationParameters ObjectDuplicationParams = InitStaticDuplicateObjectParams(Result, Parent, DesiredName, MaskedOuterFlags);
		EnumRemoveFlags(ObjectDuplicationParams.FlagMask, RF_ArchetypeObject);
		if (Parent && Parent->IsTemplate())
		{
			EnumAddFlags(ObjectDuplicationParams.ApplyFlags, RF_ArchetypeObject);
		}
		else
		{
			EnumRemoveFlags(ObjectDuplicationParams.ApplyFlags, RF_ArchetypeObject);
		}
		Result = StaticDuplicateObjectEx(ObjectDuplicationParams);
	}

	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		SetObjectPtrPropertyValue_InContainer(ContainerOrPropertyPtr, Result);
	}
	else
	{
		SetObjectPtrPropertyValue(PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType), Result);
	}
	return Buffer;
}

TObjectPtr<UObject> FObjectPropertyBase::FindImportedObject( const FProperty* Property, UObject* OwnerObject, UClass* ObjectClass, UClass* RequiredMetaClass, const TCHAR* Text, uint32 PortFlags/*=0*/, FUObjectSerializeContext* InSerializeContext /*= nullptr*/, bool bAllowAnyPackage /*= true*/)
{
	TObjectPtr<UObject>	Result = nullptr;
	check( ObjectClass->IsChildOf(RequiredMetaClass) );

	bool AttemptNonQualifiedSearch = (PortFlags & PPF_AttemptNonQualifiedSearch) != 0; 

	// if we are importing default properties, first look for a matching subobject by
	// looking through the archetype chain at each outer and stop once the outer chain reaches the owning class's default object
	if (PortFlags & PPF_ParsingDefaultProperties)
	{
		for (UObject* SearchStart = OwnerObject; Result == nullptr && SearchStart != nullptr; SearchStart = SearchStart->GetOuter())
		{
			UObject* ScopedSearchRoot = SearchStart;
			while (Result == nullptr && ScopedSearchRoot != nullptr)
			{
				Result = StaticFindObjectSafe(ObjectClass, ScopedSearchRoot, Text);
				// don't think it's possible to get a non-subobject here, but it doesn't hurt to check
				if (Result != nullptr && !Result->IsTemplate(RF_ClassDefaultObject))
				{
					Result = nullptr;
				}

				ScopedSearchRoot = ScopedSearchRoot->GetArchetype();
			}
			if (SearchStart->HasAnyFlags(RF_ClassDefaultObject))
			{
				break;
			}
		}
	}
	
	// if we have a parent, look in the parent, then it's outer, then it's outer, ... 
	// this is because exported object properties that point to objects in the level aren't
	// fully qualified, and this will step up the nested object chain to solve any name
	// collisions within a nested object tree
	UObject* ScopedSearchRoot = OwnerObject;
	while (Result == nullptr && ScopedSearchRoot != nullptr)
	{
		Result = StaticFindObjectSafe(ObjectClass, ScopedSearchRoot, Text);
		// disallow class default subobjects here while importing defaults
		// this prevents the use of a subobject name that doesn't exist in the scope of the default object being imported
		// from grabbing some other subobject with the same name and class in some other arbitrary default object
		if (Result != nullptr && (PortFlags & PPF_ParsingDefaultProperties) && Result->IsTemplate(RF_ClassDefaultObject))
		{
			Result = nullptr;
		}

		ScopedSearchRoot = ScopedSearchRoot->GetOuter();
	}

	if (Result == nullptr)
	{
		// attempt to find a fully qualified object
		Result = StaticFindObjectSafe(ObjectClass, nullptr, Text);

		if (Result == nullptr && (PortFlags & PPF_SerializedAsImportText))
		{
			// Check string asset redirectors
			FSoftObjectPath Path(Text);
			if (Path.PreSavePath())
			{
				Result = StaticFindObjectSafe(ObjectClass, nullptr, *Path.ToString());
			}
		}

		if (Result == nullptr && bAllowAnyPackage)
		{
			// RobM: We should delete this path
			// match any object of the correct class who shares the same name regardless of package path
			Result = StaticFindFirstObject(ObjectClass, Text, EFindFirstObjectOptions::None, ELogVerbosity::Warning, TEXT("FindImportedObject"));
			// disallow class default subobjects here while importing defaults
			if (Result != nullptr && (PortFlags & PPF_ParsingDefaultProperties) && Result->IsTemplate(RF_ClassDefaultObject))
			{
				Result = nullptr;
			}
		}
	}

	// if we haven;t found it yet, then try to find it without a qualified name
	if (!Result)
	{
		const TCHAR* Dot = FCString::Strrchr(Text, '.');
		if (Dot && AttemptNonQualifiedSearch)
		{
			// search with just the object name
			Result = FindImportedObject(Property, OwnerObject, ObjectClass, RequiredMetaClass, Dot + 1, 0);
		}
		FString NewText(Text);
		// if it didn't have a dot, then maybe they just gave a uasset package name
		if (!Dot && !Result)
		{
			int32 LastSlash = NewText.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
			if (LastSlash >= 0)
			{
				NewText += TEXT(".");
				NewText += (Text + LastSlash + 1);
				Dot = FCString::Strrchr(*NewText, '.');
			}
		}
		// If we still can't find it, try to load it. (Only try to load fully qualified names)
		if(!Result && Dot && !UE::IsSavingPackage())
		{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE 
			FSoftObjectPath Path(Text);
			if (UE::LinkerLoad::TryLazyLoad(*RequiredMetaClass, Path, Result))
			{
				return Result;
			}
#endif

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			FLinkerLoad* Linker = (OwnerObject != nullptr) ? OwnerObject->GetClass()->GetLinker() : nullptr;
			if (Linker == nullptr)
			{
				// Fall back on the Properties owner. That is probably the thing that has triggered this load:
				Linker = Property->GetLinker();
			}
			const bool bDeferAssetImports = (Linker != nullptr) && (Linker->LoadFlags & LOAD_DeferDependencyLoads);

			if (bDeferAssetImports)
			{
				Result = Linker->RequestPlaceholderValue(Property, ObjectClass, Text);
			}
			
			if (Result == nullptr)
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			{
				const uint32 LoadFlags = LOAD_NoWarn | LOAD_FindIfFail;		

				UE_LOGF(LogProperty, Verbose, "FindImportedObject is attempting to import [%ls] (class = %ls) with StaticLoadObject", Text, *GetFullNameSafe(ObjectClass));
				Result = StaticLoadObject(ObjectClass, nullptr, Text, nullptr, LoadFlags, nullptr, true);

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
				check(!bDeferAssetImports || !Result || !FBlueprintSupport::IsInBlueprintPackage(Result));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			}
		}
	}

	// if we found an object, and we have a parent, make sure we are in the same package or share an outer if the found object is private, unless it's a cross level property
	if (Result && !Result->HasAnyFlags(RF_Public) && OwnerObject && !OwnerObject->HasAnyFlags(RF_Transient)
		&& Result->GetOutermostObject() != OwnerObject->GetOutermostObject()
		&& Result->GetPackage() != OwnerObject->GetPackage())
	{
		const FObjectPropertyBase* ObjectProperty = CastField<const FObjectPropertyBase>(Property);
		if (!ObjectProperty || !ObjectProperty->AllowCrossLevel())
		{
			UE_LOGF(LogProperty, Warning, "Illegal TEXT reference to a private object in external package (%ls) from referencer (%ls).  Import failed...", *Result->GetFullName(), *OwnerObject->GetFullName());
			Result = nullptr;
		}
	}

	check(!Result || Result->IsA(RequiredMetaClass));
	return Result;
}

FName FObjectPropertyBase::GetID() const
{
	return NAME_ObjectProperty;
}

UObject* FObjectPropertyBase::GetObjectPropertyValue(const void* PropertyValueAddress) const
{
	unimplemented(); // needs to be implemented by the derived class
	return nullptr;
}

TObjectPtr<UObject> FObjectPropertyBase::GetObjectPtrPropertyValue(const void* PropertyValueAddress) const
{
	unimplemented(); // needs to be implemented by the derived class
	return TObjectPtr<UObject>();
}

UObject* FObjectPropertyBase::GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	unimplemented(); // needs to be implemented by the derived class
	return nullptr;
}

TObjectPtr<UObject> FObjectPropertyBase::GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	unimplemented(); // needs to be implemented by the derived class
	return TObjectPtr<UObject>();
}

void FObjectPropertyBase::SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const
{
	unimplemented(); // needs to be implemented by the derived class
}

void FObjectPropertyBase::SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const
{
	unimplemented(); // needs to be implemented by the derived class
}

void FObjectPropertyBase::SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex) const
{
	unimplemented(); // needs to be implemented by the derived class
}

void FObjectPropertyBase::SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex) const
{
	unimplemented(); // needs to be implemented by the derived class
}

void FObjectPropertyBase::SetObjectPropertyValue(void* PropertyValueAddress, UObject* Value) const
{
	if (Value || !HasAnyPropertyFlags(CPF_NonNullable) || (UE::CoreUObject::AllowSetNullOnNonNullableBehavior() == UE::CoreUObject::EAllowSetNullOnNonNullableBehavior::Enabled))
	{
		SetObjectPropertyValueUnchecked(PropertyValueAddress, Value);
	}
	else
	{
		UE_LOGF(LogProperty, Verbose /*Warning*/, "Trying to assign null object value to non-nullable \"%ls\"", *GetFullName());
	}
}

void FObjectPropertyBase::SetObjectPtrPropertyValue(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const
{
	if (Ptr || !HasAnyPropertyFlags(CPF_NonNullable) || (UE::CoreUObject::AllowSetNullOnNonNullableBehavior() == UE::CoreUObject::EAllowSetNullOnNonNullableBehavior::Enabled))
	{
		SetObjectPtrPropertyValueUnchecked(PropertyValueAddress, Ptr);
	}
	else
	{
		UE_LOGF(LogProperty, Verbose /*Warning*/, "Trying to assign null object value to non-nullable \"%ls\"", *GetFullName());
	}
}

void FObjectPropertyBase::SetObjectPropertyValue_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex) const
{
	if (Value || !HasAnyPropertyFlags(CPF_NonNullable) || (UE::CoreUObject::AllowSetNullOnNonNullableBehavior() == UE::CoreUObject::EAllowSetNullOnNonNullableBehavior::Enabled))
	{
		SetObjectPropertyValueUnchecked_InContainer(ContainerAddress, Value, ArrayIndex);
	}
	else
	{
		UE_LOGF(LogProperty, Verbose /*Warning*/, "Trying to assign null object value to non-nullable \"%ls\"", *GetFullName());
	}
}

void FObjectPropertyBase::SetObjectPtrPropertyValue_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex) const
{
	if (Ptr || !HasAnyPropertyFlags(CPF_NonNullable) || (UE::CoreUObject::AllowSetNullOnNonNullableBehavior() == UE::CoreUObject::EAllowSetNullOnNonNullableBehavior::Enabled))
	{
		SetObjectPtrPropertyValueUnchecked_InContainer(ContainerAddress, Ptr, ArrayIndex);
	}
	else
	{
		UE_LOGF(LogProperty, Verbose /*Warning*/, "Trying to assign null object value to non-nullable \"%ls\"", *GetFullName());
	}
}

bool FObjectPropertyBase::AllowCrossLevel() const
{
	return false;
}

bool FObjectPropertyBase::AllowObjectTypeReinterpretationTo(const FObjectPropertyBase* Other) const
{
	return false;
}

static bool CanPropertyHoldObject(const FObjectPropertyBase* Prop, TObjectPtr<const UObject> Obj, FString* FailReason = nullptr)
{
	UClass* PropertyClass = Prop->PropertyClass;
	checkSlow(PropertyClass);

	UClass* ObjectClass = Obj.GetClass();
	checkSlow(ObjectClass);

	UClass* InterfaceClassToCheck = nullptr;

#if WITH_METADATA
	// Check if the object has metadata that states that the object must implement a particular interface
	UClass* MetaDataClass = Prop->GetOwnerProperty()->GetClassMetaData(TEXT("ObjectMustImplement"));
	if (MetaDataClass)
	{
		InterfaceClassToCheck = MetaDataClass;
	}
	else
#endif // WITH_METADATA
	{
		// Check if the object is or is derived from the property class
		if (!ObjectClass->IsChildOf(PropertyClass) && !ObjectClass->GetAuthoritativeClass()->IsChildOf(PropertyClass))
		{
			if (!PropertyClass->IsChildOf<UInterface>())
			{
				if (FailReason)
				{
					*FailReason = FString::Printf(TEXT("object '%s' is incompatible with object property of type '%s'"), *ObjectClass->GetName(), *PropertyClass->GetName());
				}
				return false;
			}

			// Is it even possible to have an object property with an interface class set?  Check it anyway.
			InterfaceClassToCheck = PropertyClass;
		}
	}

	// Check if the object implements the interface
	if (InterfaceClassToCheck && !ObjectClass->ImplementsInterface(InterfaceClassToCheck))
	{
		if (FailReason)
		{
			*FailReason = FString::Printf(TEXT("class '%s' does not implement interface '%s'"), *ObjectClass->GetName(), *InterfaceClassToCheck->GetName());
		}
		return false;
	}

	return true;
}

UObject* FObjectPropertyBase::ConstructDefaultObjectValueIfNecessary(UObject* ExistingValue, FString* OutFailureReason, const void* Defaults) const
{
	UObject* Template = nullptr;
	UClass* NewClass = PropertyClass;
	UObject* NewDefaultObjectValue = nullptr;
	bool bCanPropertyHoldExistingValue = false;

	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	UObject* Outer = SerializeContext ? SerializeContext->SerializedObject : nullptr;
	if (!Outer)
	{
		Outer = GetTransientPackage();
	}

	if (ExistingValue)
	{
		UClass* ExistingValueClass = ExistingValue->GetClass();
		// Sanity check to make sure the existing value class matches the property class
		if (ExistingValueClass && CanPropertyHoldObject(this, ExistingValue))
		{
			if (ExistingValue->IsTemplate() && 	// Existing value is a template so we can construct a new value with it as the archetype
				ExistingValue->GetOuter() != Outer) // Unless the template's Outer is the same as the new Outer in which case the template (ExistingValue) IS the object we can reuse
			{
				// We probably got here because an object value failed to load (missing import class) and the property is left with a template of default subobject
				Template = ExistingValue;
				NewClass = ExistingValueClass;
			}
			else
			{
				// Existing value is not a template or a template is what this property was pointing to so we can use it directly
				// Similar to the above condition but the property was not referencing an instanced value in which case it's ok to leave the CDO default here
				NewDefaultObjectValue = ExistingValue;
			}

			bCanPropertyHoldExistingValue = true;
		}
	}

	// We cannot know which object to create if the class is abstract or if there is a required interface
	if (!NewDefaultObjectValue)
	{
#if WITH_METADATA
		UClass* MetaDataClass = this->GetOwnerProperty()->GetClassMetaData(TEXT("ObjectMustImplement"));
#endif

		if (!ensureMsgf(PropertyClass, TEXT("Malformed %s: PropertyClass=null."), *GetFullName()))
		{
			if (OutFailureReason)
			{
				*OutFailureReason = TEXT("malformed property: PropertyClass=null");
			}
		}
		else if (NewClass->HasAnyClassFlags(CLASS_Abstract))
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FString::Printf(TEXT("class '%s' is abstract"), *NewClass->GetName());
			}
		}
#if WITH_METADATA
		// CanPropertyHoldObject has already determined that we can instantiate an object with NewClass so when bCanPropertyHoldExistingValue
		// is true we bypass the check for 'ObjectMustImplement'. 
		else if (!bCanPropertyHoldExistingValue && MetaDataClass)
		{
			if (OutFailureReason)
			{
				*OutFailureReason = FString::Printf(TEXT("interface '%s' cannot be instantiated"), *MetaDataClass->GetName());
			}
		}
#endif // WITH_METADATA
		else
		{
			FName ExistingName = ExistingValue ? ExistingValue->GetFName() : NAME_None;
			UObject* ObjectWithExistingName = (ExistingName != NAME_None) ? StaticFindObjectFastInternal(/*Class*/nullptr, Outer, ExistingName, EFindObjectFlags::ExactClass) : nullptr;
			// We can use the existing name if: 
			// a) there is no existing object with that name in the outer, or
			// b) we are replacing an object of the same type.
			bool bCanReplaceObjectWithExistingName = !ObjectWithExistingName || ObjectWithExistingName->GetClass()->IsChildOf(NewClass);

			FName NewName = (bCanReplaceObjectWithExistingName) ? ExistingName : NAME_None;

			// Filter out any flags that do not belong to the RF_Load or RF_PropagateToSubObjects groups. We mostly want to exclude
			// UObject life-time and internal GC flags.
			EObjectFlags NewFlags = Outer->GetMaskedFlags(RF_PropagateToSubObjects);
			if (ExistingValue && ExistingValue != Template)
			{
				NewFlags |= ExistingValue->GetMaskedFlags(RF_Load | RF_Transient);
			}
			
			// We might be in the middle of serializing the containing object, so we have to treat the default value as if it were
			// also being loaded in order to defer any subobject instancing and bytecode execution until after the load is complete.
			if (SerializeContext && Outer->HasAnyFlags(RF_WasLoaded) && !Outer->HasAnyFlags(RF_LoadCompleted))
			{
				NewFlags |= RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects;
			}
			
			if (!Template && Defaults)
			{
				Template = GetObjectPropertyValue(Defaults);
			}
			
			if (Template && !Template->IsA(NewClass))
			{
				Template = nullptr;
			}

			// Existing value did not exist, was not type-compatible or is being used as a template
			// Existing value may be null in case we were serializing an array of UObjects that failed to load (missing import class). Since the array is first pre-allocated with null values
			// it will not have any existing objects to instantiate
			NewDefaultObjectValue = NewObject<UObject>(Outer, NewClass, NewName, NewFlags, Template);

			// Register the new value as a loaded object if necessary to make sure it gets included in the PostLoad phase.
			if (SerializeContext && NewDefaultObjectValue->HasAnyFlags(RF_NeedLoad))
			{
				NewDefaultObjectValue->ClearFlags(RF_NeedLoad);
				SerializeContext->AddLoadedObject(NewDefaultObjectValue);
			}
		}
	}

	return NewDefaultObjectValue;
}

void FObjectPropertyBase::CheckValidObject(void* ValueAddress, TObjectPtr<UObject> OldValue, const void* Defaults) const
{
	const TObjectPtr<UObject> Object = GetObjectPtrPropertyValue(ValueAddress);
	if (!Object)
	{
		return;
	}
#if UE_WITH_REMOTE_OBJECT_HANDLE
	// Verifying the object class will result in resolving remote objects and we don't want that.
	// However, since remote objects are guarnteed to have compatible binaries it's not required to validate objects.
	if (Object.GetResidence() != EResidence::Local)
	{
		return;
	}
#endif
	//
	// here we want to make sure the the object value still matches the 
	// object type expected by the property...

	UClass* ObjectClass = Object.GetClass();
	UE_CLOGF(!ObjectClass, LogProperty, Fatal, "Object without class referenced by %ls, object: 0x%016llx %ls", *GetPathName(), (int64)(PTRINT)ValueAddress, *Object.GetPathName());

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	auto IsDeferringValueLoad = [&]()
	{
		FLinkerLoad* PropertyLinker = GetLinker();
		return ((PropertyLinker == nullptr) || (PropertyLinker->LoadFlags & LOAD_DeferDependencyLoads)) &&
			(ObjectClass->IsChildOf<ULinkerPlaceholderExportObject>() || ObjectClass->IsChildOf<ULinkerPlaceholderClass>());
	};

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	check( IsDeferringValueLoad() || (!Object->IsA<ULinkerPlaceholderExportObject>() && !Object->IsA<ULinkerPlaceholderClass>()) );
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

#else  // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING 
	auto IsDeferringValueLoad = [&]() { return false; };
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	// Is a null class even possible here?
	FString HoldFailReasonString;
	if ((PropertyClass != nullptr) && !CanPropertyHoldObject(this, Object, &HoldFailReasonString))
	{
		// we could be in the middle of replacing references to the 
		// PropertyClass itself (in the middle of an FArchiveReplaceObjectRef 
		// pass)... if this is the case, then we might have already replaced 
		// the object's class, but not the PropertyClass yet (or vise-versa)... 
		// so we use this to ensure, in that situation, that we don't clear the 
		// object value (if CLASS_NewerVersionExists is set, then we are likely 
		// in the middle of an FArchiveReplaceObjectRef pass)
		bool bIsReplacingClassRefs = PropertyClass && PropertyClass->HasAnyClassFlags(CLASS_NewerVersionExists) != ObjectClass->HasAnyClassFlags(CLASS_NewerVersionExists);
		if (!bIsReplacingClassRefs && !IsDeferringValueLoad())
		{
			UObject* DefaultValue = nullptr;

			FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
			UObject* Outer = SerializeContext ? SerializeContext->SerializedObject : nullptr;
			if (!Outer)
			{
				Outer = GetTransientPackage();
			}
			if (!HasAnyPropertyFlags(CPF_NonNullable))
			{
				UE_LOGF(LogProperty, Warning,
					"Serialized %ls for a property of %ls. Reference will be nulled.\n    ReferencingObject = %ls\n    Property = %ls\n    Item = %ls",
					*ObjectClass->GetFullName(),
					*PropertyClass->GetFullName(),
					*GetFullNameSafe(Outer),
					*GetFullName(),
					*Object.GetFullName()
				);
			}
			else
			{
				using UE::CoreUObject::Private::ENonNullableBehavior;
				using UE::CoreUObject::Private::GetNonNullableBehavior;

				ENonNullableBehavior NonNullableBehavior = GetNonNullableBehavior();
				FString DefaultValueFailureReason;
				if (NonNullableBehavior == ENonNullableBehavior::CreateDefaultObjectIfPossible)
				{
					DefaultValue = ConstructDefaultObjectValueIfNecessary(OldValue, &DefaultValueFailureReason, Defaults);
					if (!DefaultValueFailureReason.IsEmpty())
					{
						DefaultValueFailureReason.InsertAt(0, TEXT(" as "));
					}
				}

				if (DefaultValue)
				{
					UE_LOGF(LogProperty, Warning,
						"Serialized %ls for a non-nullable property of %ls but %ls. Reference will be defaulted to %ls (previously: %ls).\n    ReferencingObject = %ls\n    Property = %ls\n    Item = %ls",
						*ObjectClass->GetFullName(),
						*PropertyClass->GetFullName(),
						*HoldFailReasonString,
						*GetFullNameSafe(DefaultValue),
						*GetFullNameSafe(OldValue),
						*GetFullNameSafe(Outer),
						*GetFullName(),
						*Object.GetFullName()
					);
				}
				else if (NonNullableBehavior == ENonNullableBehavior::LogWarning)
				{
					UE_LOGF(LogProperty, Warning,
						"Serialized %ls for a non-nullable property of %ls but %ls. Reference will be nulled%ls (previously: %ls) - will cause a runtime error if accessed.\n    ReferencingObject = %ls\n    Property = %ls\n    Item = %ls",
						*ObjectClass->GetFullName(),
						*PropertyClass->GetFullName(),
						*HoldFailReasonString,
						*DefaultValueFailureReason,
						*GetFullNameSafe(OldValue),
						*GetFullNameSafe(Outer),
						*GetFullName(),
						*Object.GetFullName()
					);
				}
				else
				{
					UE_LOGF(LogProperty, Error,
						"Serialized %ls for a non-nullable property of %ls but %ls. Reference will be nulled%ls (previously: %ls) - will cause a runtime error if accessed.\n    ReferencingObject = %ls\n    Property = %ls\n    Item = %ls",
						*ObjectClass->GetFullName(),
						*PropertyClass->GetFullName(),
						*HoldFailReasonString,
						*DefaultValueFailureReason,
						*GetFullNameSafe(OldValue),
						*GetFullNameSafe(Outer),
						*GetFullName(),
						*Object.GetFullName()
					);
				}
			}

			SetObjectPropertyValueUnchecked(ValueAddress, DefaultValue);
		}
	}
}

bool FObjectPropertyBase::SameType(const FProperty* Other) const
{
	return (Super::SameType(Other) || (Other && Other->IsA<FObjectPropertyBase>() && ((FObjectPropertyBase*)Other)->AllowObjectTypeReinterpretationTo(this))) && 
			 (PropertyClass == ((FObjectPropertyBase*)Other)->PropertyClass);
}

EPropertyVisitorControlFlow FObjectPropertyBase::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Conmtext*/)> InFunc) const
{
	// Indicate in the path that this property contains inner properties
	Context.Path.Top().bContainsInnerProperties = true;

	EPropertyVisitorControlFlow RetVal = Super::Visit(Context, InFunc);

	if (RetVal == EPropertyVisitorControlFlow::StepInto)
	{
		if (const TObjectPtr<UObject> Object = GetObjectPropertyValue(Context.Data.PropertyData))
		{
			FPropertyVisitorContext SubContext = Context.VisitPropertyData(Object.Get());

			RetVal = Object.GetClass()->Visit(SubContext, InFunc);
		}
	}
	return RetVal;
}

void* FObjectPropertyBase::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	if (const TObjectPtr<UObject> Object = GetObjectPropertyValue(Data))
	{
		return Object.GetClass()->ResolveVisitedPathInfo(Object.Get(), Info);
	}

	return nullptr;
}
