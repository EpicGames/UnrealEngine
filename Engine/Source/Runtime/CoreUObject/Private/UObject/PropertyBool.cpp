// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "Hash/Blake3.h"
#include "UObject/PropertyHelper.h"
#include "UObject/UnrealTypePrivate.h"

/*-----------------------------------------------------------------------------
	FBoolProperty.
-----------------------------------------------------------------------------*/

IMPLEMENT_FIELD(FBoolProperty)

FBoolProperty::FBoolProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, FieldSize(0)
	, ByteOffset(0)
	, ByteMask(1)
	, FieldMask(1)
{
	SetBoolSize(1, false, 1);
}

FBoolProperty::FBoolProperty(FFieldVariant InOwner, const UECodeGen_Private::FBoolPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithoutOffset&)Prop, CPF_HasGetValueTypeHash)
	, FieldSize(0)
	, ByteOffset(0)
	, ByteMask(1)
	, FieldMask(1)
{
	uint32 Offset = 0;
	uint32 BitMask = 0;
	if (Prop.SetBitFunc)
	{
		DetermineBitfieldOffsetAndMask(Offset, BitMask, Prop.SetBitFunc, Prop.SizeOfOuter);
		check(BitMask);
	}

	SetOffset_Internal(Offset);
	SetBoolSize(Prop.ElementSize, !!(Prop.Flags & UECodeGen_Private::EPropertyGenFlags::NativeBool), BitMask);
}

#if WITH_EDITORONLY_DATA
FBoolProperty::FBoolProperty(UField* InField)
	: Super(InField)
{
	UBoolProperty* SourceProperty = CastChecked<UBoolProperty>(InField);
	FieldSize = SourceProperty->FieldSize;
	ByteOffset = SourceProperty->ByteOffset;
	ByteMask = SourceProperty->ByteMask;
	FieldMask = SourceProperty->FieldMask;
}
#endif // WITH_EDITORONLY_DATA

void FBoolProperty::PostDuplicate(const FField& InField)
{
	const FBoolProperty& Source = static_cast<const FBoolProperty&>(InField);
	FieldSize = Source.FieldSize;
	ByteOffset = Source.ByteOffset;
	ByteMask = Source.ByteMask;
	FieldMask = Source.FieldMask;
	Super::PostDuplicate(InField);
}

void FBoolProperty::SetBoolSize( const uint32 InSize, const bool bIsNativeBool, const uint32 InBitMask /*= 0*/ )
{
	if (bIsNativeBool)
	{
		PropertyFlags |= (CPF_IsPlainOldData | CPF_NoDestructor | CPF_ZeroConstructor);
	}
	else
	{
		PropertyFlags &= ~(CPF_IsPlainOldData | CPF_ZeroConstructor);
		PropertyFlags |= CPF_NoDestructor;
	}
	uint32 TestBitmask = InBitMask ? InBitMask : 1;
	SetElementSize(InSize);
	FieldSize = (uint8)GetElementSize();
	ByteOffset = 0;
	if (bIsNativeBool)
	{		
		ByteMask = true;
		FieldMask = 255;
	}
	else
	{
		// Calculate ByteOffset and get ByteMask.
		for (ByteOffset = 0; ByteOffset < InSize && ((ByteMask = *((uint8*)&TestBitmask + ByteOffset)) == 0); ByteOffset++);
		FieldMask = ByteMask;
	}
	check((int32)FieldSize == GetElementSize());
	check(GetElementSize() != 0);
	check(FieldMask != 0);
	check(ByteMask != 0);
}

void FBoolProperty::DetermineBitfieldOffsetAndMask(uint32& Offset, uint32& BitMask, void (*SetBit)(void* Obj), SIZE_T SizeOf)
{
	TUniquePtr<uint8[]> Buffer = MakeUnique<uint8[]>(SizeOf);

	SetBit(Buffer.Get());

	// Here we are making the assumption that bitfields are aligned in the struct. Probably true.
	// If not, it may be ok unless we are on a page boundary or something, but the check will fire in that case.
	// Have faith.
	for (uint32 TestOffset = 0; TestOffset < SizeOf; TestOffset++)
	{
		if (uint8 Mask = Buffer[TestOffset])
		{
			Offset = TestOffset;
			BitMask = (uint32)Mask;
			check(FMath::RoundUpToPowerOfTwo(BitMask) == BitMask); // better be only one bit on
			break;
		}
	}
}

int32 FBoolProperty::GetMinAlignment() const
{
	int32 Alignment = 0;
	switch(GetElementSize())
	{
	case sizeof(uint8):
		Alignment = alignof(uint8); break;
	case sizeof(uint16):
		Alignment = alignof(uint16); break;
	case sizeof(uint32):
		Alignment = alignof(uint32); break;
	case sizeof(uint64):
		Alignment = alignof(uint64); break;
	default:
		UE_LOGF(LogProperty, Fatal, "Unsupported FBoolProperty %ls size %d.", *GetName(), (int32)GetElementSize());
	}
	return Alignment;
}
void FBoolProperty::LinkInternal(FArchive& Ar)
{
#if UE_WITH_CONSTINIT_UOBJECT
	// Can't call FField::IsNative here because sometimes bool props don't have an owner when linked in the python script plugin
	if (UObject* Obj = GetOwnerUObject(); Obj && Obj->IsNative())
	{
		// Calculate bit mask settings since we couldn't do it at compile time
		bool bIsNativeBool = IsNativeBool();
		uint32 Offset = 0;
		uint32 BitMask = 0;
		if (SetBitFunc)
		{
			// Outer's PropertiesSize was set to sizeof() in constinit constructor
			DetermineBitfieldOffsetAndMask(Offset, BitMask, SetBitFunc, GetOwnerChecked<UStruct>()->PropertiesSize);
			check(BitMask);
		}

		SetOffset_Internal(Offset);
		SetBoolSize(GetElementSize(), bIsNativeBool, BitMask);
	}
#endif

	check(FieldSize != 0);
	SetElementSize(FieldSize);
	if (IsNativeBool())
	{
		PropertyFlags |= (CPF_IsPlainOldData | CPF_NoDestructor | CPF_ZeroConstructor);
	}
	else
	{
		PropertyFlags &= ~(CPF_IsPlainOldData | CPF_ZeroConstructor);
		PropertyFlags |= CPF_NoDestructor;
	}
	PropertyFlags |= CPF_HasGetValueTypeHash;
}
void FBoolProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	Ar << FieldSize;
	Ar << ByteOffset;
	Ar << ByteMask;
	Ar << FieldMask;

	// Serialize additional flags which will help to identify this FBoolProperty type and size.
	uint8 BoolSize = (uint8)GetElementSize();
	Ar << BoolSize;
	uint8 NativeBool = false;
	if( Ar.IsLoading())
	{
		Ar << NativeBool;
		SetBoolSize( BoolSize, !!NativeBool );
	}
	else
	{
		NativeBool = Ar.IsSaving() ? (IsNativeBool() ? 1 : 0) : 0;
		Ar << NativeBool;
	}
}
FString FBoolProperty::GetCPPType( FString* ExtendedTypeText/*=NULL*/, uint32 CPPExportFlags/*=0*/ ) const
{
	check(FieldSize != 0);

	if (IsNativeBool() 
		|| ((CPPExportFlags & (CPPF_Implementation|CPPF_ArgumentOrReturnValue)) == (CPPF_Implementation|CPPF_ArgumentOrReturnValue))
		|| ((CPPExportFlags & CPPF_BlueprintCppBackend) != 0))
	{
		// Export as bool if this is actually a bool or it's being exported as a return value of C++ function definition.
		return TEXT("bool");
	}
	else
	{
		// Bitfields
		switch(GetElementSize())
		{
		case sizeof(uint64):
			return TEXT("uint64");
		case sizeof(uint32):
			return TEXT("uint32");
		case sizeof(uint16):
			return TEXT("uint16");
		case sizeof(uint8):
			return TEXT("uint8");
		default:
			UE_LOGF(LogProperty, Fatal, "Unsupported FBoolProperty %ls size %d.", *GetName(), GetElementSize());
			break;
		}
	}
	return TEXT("uint32");
}

FString FBoolProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	check(FieldSize != 0);
	if (IsNativeBool())
	{
		return TEXT("UBOOL");
	}
	else
	{
		switch(GetElementSize())
		{
		case sizeof(uint64):
			return TEXT("UBOOL64");
		case sizeof(uint32):
			return TEXT("UBOOL32");
		case sizeof(uint16):
			return TEXT("UBOOL16");
		case sizeof(uint8):
			return TEXT("UBOOL8");
		default:
			UE_LOGF(LogProperty, Fatal, "Unsupported FBoolProperty %ls size %d.", *GetName(), GetElementSize());
			break;
		}
	}
	return TEXT("UBOOL32");
}

template<typename T>
void LoadFromType(FBoolProperty* Property, const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data)
{
	T IntValue;
	Slot << IntValue;

	if (IntValue != 0)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (IntValue != 1)
		{
			if constexpr (std::is_unsigned_v<T>)
			{
				UE_LOG(LogClass, Log, TEXT("Loading %s property (%s) that is now a bool - value '%" UINT64_FMT "', expecting 0 or 1. Value set to true."), *Tag.Type.ToString(), *Property->GetPathName(), (uint64)IntValue);
			}
			else
			{
				UE_LOG(LogClass, Log, TEXT("Loading %s property (%s) that is now a bool - value '%" INT64_FMT "', expecting 0 or 1. Value set to true."), *Tag.Type.ToString(), *Property->GetPathName(), (int64)IntValue);
			}
		}
#endif
		Property->SetPropertyValue_InContainer(Data, true, Tag.ArrayIndex);
	}
	else
	{
		Property->SetPropertyValue_InContainer(Data, false, Tag.ArrayIndex);
	}
}

EConvertFromTypeResult FBoolProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	if (Tag.Type == NAME_IntProperty)
	{
		LoadFromType<int32>(this, Tag, Slot, Data);
	}
	else if (Tag.Type == NAME_Int8Property)
	{
		LoadFromType<int8>(this, Tag, Slot, Data);
	}
	else if (Tag.Type == NAME_Int16Property)
	{
		LoadFromType<int16>(this, Tag, Slot, Data);
	}
	else if (Tag.Type == NAME_Int64Property)
	{
		LoadFromType<int64>(this, Tag, Slot, Data);
	}
	else if (Tag.Type == NAME_ByteProperty)
	{
		// Disallow conversion of enum to bool.
		if (Tag.GetType().GetParameterCount() > 0)
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}

		// Disallow a nested byte property prior to complete type names because it was impossible to distinguish from a nested enum.
		if (GetOwner<FProperty>() && Slot.GetArchiveState().UEVer() < EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}

		LoadFromType<uint8>(this, Tag, Slot, Data);
	}
	else if (Tag.Type == NAME_UInt16Property)
	{
		LoadFromType<uint16>(this, Tag, Slot, Data);
	}
	else if (Tag.Type == NAME_UInt32Property)
	{
		LoadFromType<uint32>(this, Tag, Slot, Data);
	}
	else if (Tag.Type == NAME_UInt64Property)
	{
		LoadFromType<uint64>(this, Tag, Slot, Data);
	}
	else
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}

	return EConvertFromTypeResult::Converted;
}

#if WITH_EDITORONLY_DATA
void FBoolProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	Builder.Update(&ByteOffset, sizeof(ByteOffset));
	Builder.Update(&ByteMask, sizeof(ByteMask));
	Builder.Update(&FieldMask, sizeof(FieldMask));
}
#endif


void FBoolProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	check(FieldSize != 0);
	uint8 LocalByteValue = 0;
	bool bValue = false;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		GetValue_InContainer(ContainerOrPropertyPtr, &bValue);
	}
	else
	{
		LocalByteValue = *((uint8*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType) + ByteOffset);
		bValue = 0 != (LocalByteValue & FieldMask);
	}
	const TCHAR* Temp = (bValue ? TEXT("True") : TEXT("False"));
	ValueStr += FString::Printf( TEXT("%s"), Temp );
}
const TCHAR* FBoolProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	FString Temp; 
	Buffer = FPropertyHelpers::ReadToken( Buffer, Temp );
	if( !Buffer )
	{
		return NULL;
	}

	check(FieldSize != 0);
	uint8 LocalByteValue = 0;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		bool bValue = false;
		GetValue_InContainer(ContainerOrPropertyPtr, &bValue);
		LocalByteValue = bValue ? FieldMask : 0;
	}
	else
	{
		LocalByteValue = *((uint8*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType) + ByteOffset);
	}

	const FCoreTexts& CoreTexts = FCoreTexts::Get();
	if( Temp==TEXT("1") || Temp==TEXT("True") || Temp==*CoreTexts.True.ToString() || Temp == TEXT("Yes") || Temp == *CoreTexts.Yes.ToString() )
	{
		LocalByteValue |= ByteMask;
	}
	else 
	if( Temp==TEXT("0") || Temp==TEXT("False") || Temp==*CoreTexts.False.ToString() || Temp == TEXT("No") || Temp == *CoreTexts.No.ToString() )
	{
		LocalByteValue &= ~FieldMask;
	}
	else
	{
		//UE_LOGF(LogProperty, Log,  "Import: Failed to get bool" );
		return NULL;
	}

	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		bool bValue = 0 != (LocalByteValue & FieldMask);
		SetValue_InContainer(ContainerOrPropertyPtr, &bValue);
	}
	else
	{
		*((uint8*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType) + ByteOffset) = LocalByteValue;
	}

	return Buffer;
}
bool FBoolProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	check(FieldSize != 0);
	const uint8* ByteValueA = (const uint8*)NotNullGet(A) + ByteOffset;
	const uint8* ByteValueB = (const uint8*)B + ByteOffset;
	return ((*ByteValueA ^ (B ? *ByteValueB : 0)) & FieldMask) == 0;
}

void FBoolProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	check(FieldSize != 0);
	uint8* ByteValue = (uint8*)NotNullGet(Value) + ByteOffset;
	uint8 B = (*ByteValue & FieldMask) ? 1 : 0;
	Slot << B;
	*ByteValue = ((*ByteValue) & ~FieldMask) | (B ? ByteMask : 0);
}

bool FBoolProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData) const
{
	check(FieldSize != 0);
	uint8* ByteValue = (uint8*)NotNullGet(Data) + ByteOffset;
	uint8 Value = ((*ByteValue & FieldMask)!=0);
	Ar.SerializeBits( &Value, 1 );
	*ByteValue = ((*ByteValue) & ~FieldMask) | (Value ? ByteMask : 0);
	return true;
}
void FBoolProperty::CopyValuesInternal( void* Dest, void const* Src, int32 Count  ) const
{
	check(FieldSize != 0 && !IsNativeBool());
	for (int32 Index = 0; Index < Count; Index++)
	{
		uint8* DestByteValue = (uint8*)Dest + Index * GetElementSize() + ByteOffset;
		uint8* SrcByteValue = (uint8*)Src + Index * GetElementSize() + ByteOffset;
		*DestByteValue = (*DestByteValue & ~FieldMask) | (*SrcByteValue & FieldMask);
	}
}
void FBoolProperty::ClearValueInternal( void* Data ) const
{
	check(FieldSize != 0);
	uint8* ByteValue = (uint8*)Data + ByteOffset;
	*ByteValue &= ~FieldMask;
}

void FBoolProperty::InitializeValueInternal( void* Data ) const
{
	check(FieldSize != 0);
	uint8* ByteValue = (uint8*)Data + ByteOffset;
	*ByteValue &= ~FieldMask;
}

uint32 FBoolProperty::GetValueTypeHashInternal(const void* Src) const
{
	uint8* SrcByteValue = (uint8*)Src + ByteOffset;
	return GetTypeHash(*SrcByteValue & FieldMask);
}
