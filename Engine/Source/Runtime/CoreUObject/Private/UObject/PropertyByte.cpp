// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "Algo/Find.h"
#include "Hash/Blake3.h"
#include "UObject/PropertyHelper.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectThreadContext.h"

#if WITH_EDITORONLY_DATA
#include "UObject/PropertyStateTracking.h"
#endif

// Implemented in EnumProperty.cpp
bool TryLoadEnumValueByName(FStructuredArchive::FSlot Slot, FArchive& UnderlyingArchive, UEnum* Enum, FName& OutEnumValueName, int64& OutEnumValue);

/*-----------------------------------------------------------------------------
	FByteProperty.
-----------------------------------------------------------------------------*/

IMPLEMENT_FIELD(FByteProperty)

FByteProperty::FByteProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, Enum(nullptr)
{
}

FByteProperty::FByteProperty(FFieldVariant InOwner, const UECodeGen_Private::FBytePropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
	this->Enum = Prop.EnumFunc ? Prop.EnumFunc(ETypeConstructPhase::Outer) : nullptr;
}

#if WITH_EDITORONLY_DATA
FByteProperty::FByteProperty(UField* InField)
	: Super(InField)
{
	UByteProperty* SourceProperty = CastChecked<UByteProperty>(InField);
	Enum = SourceProperty->Enum;
}
#endif // WITH_EDITORONLY_DATA

void FByteProperty::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	OutDeps.Add(Enum);
}

void FByteProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	uint8* ByteValuePtr = (uint8*)NotNullGet(Value);

	if(Enum && UnderlyingArchive.UseToResolveEnumerators())
	{
		Slot.EnterStream();
		const int64 ResolvedIndex = Enum->ResolveEnumerator(UnderlyingArchive, *ByteValuePtr);
		*ByteValuePtr = static_cast<uint8>(ResolvedIndex);
		return;
	}

	// Serialize enum values by name unless we're not saving or loading OR for backwards compatibility
	const bool bUseBinarySerialization = (Enum == NULL) || (!UnderlyingArchive.IsLoading() && !UnderlyingArchive.IsSaving());
	if( bUseBinarySerialization )
	{
		Super::SerializeItem(Slot, ByteValuePtr, Defaults);
	}
	// Loading
	else if (UnderlyingArchive.IsLoading())
	{
		FName EnumValueName;
		int64 EnumValue = 0;
		if (!TryLoadEnumValueByName(Slot, UnderlyingArchive, Enum, EnumValueName, EnumValue))
		{
		#if WITH_EDITORONLY_DATA
			FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
			if (UNLIKELY(SerializeContext->bTrackUnknownEnumNames))
			{
				UE::FUnknownEnumNames(SerializeContext->SerializedObject).Add(Enum, UE::FindOriginalType(this), EnumValueName);
			}
		#endif
		}
		*ByteValuePtr = IntCastChecked<uint8>(EnumValue);
	}
	// Saving
	else
	{
		FName EnumValueName;
		uint8 ByteValue = *ByteValuePtr;

		// subtract 1 because the last entry in the enum's Names array
		// is the _MAX entry
		if ( Enum->IsValidEnumValue(ByteValue) )
		{
			EnumValueName = Enum->GetNameByValue(ByteValue);

		#if WITH_EDITORONLY_DATA
			// Fix up the type name when this property is impersonating another enum type.
			if (UE::FPropertyTypeName OriginalType = UE::FindOriginalType(this); !OriginalType.IsEmpty())
			{
				EnumValueName = FName(EnumValueName.ToString().Replace(*Enum->GetName(), *OriginalType.GetName().ToString()));
			}
		#endif
		}
		else
		{
			EnumValueName = NAME_None;
		}
		Slot << EnumValueName;
	}
}
bool FByteProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	if (Ar.EngineNetVer() < FEngineNetworkCustomVersion::EnumSerializationCompat)
	{
		Ar.SerializeBits(Data, Enum ? FMath::CeilLogTwo64(Enum->GetMaxEnumValue()) : 8);
	}
	else
	{
		Ar.SerializeBits(Data, GetMaxNetSerializeBits());
	}

	return true;
}
void FByteProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar << Enum;
	if (Enum != nullptr)
	{
		Ar.Preload(Enum);
	}
}

void FByteProperty::PostDuplicate(const FField& InField)
{
	const FByteProperty& Source = static_cast<const FByteProperty&>(InField);
	Enum = Source.Enum;
	Super::PostDuplicate(InField);
}

void FByteProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Enum, nullptr);
	Super::AddReferencedObjects(Collector);
}
FString FByteProperty::GetCPPType( FString* ExtendedTypeText/*=NULL*/, uint32 CPPExportFlags/*=0*/ ) const
{
	if (Enum)
	{
		const bool bEnumClassForm = Enum->GetCppForm() == UEnum::ECppForm::EnumClass;
		const bool bNonNativeEnum = Enum->GetClass() != UEnum::StaticClass(); // cannot use RF_Native flag, because in UHT the flag is not set
		const bool bRawParam = (CPPExportFlags & CPPF_ArgumentOrReturnValue)
			&& (((PropertyFlags & CPF_ReturnParm) || !(PropertyFlags & CPF_OutParm))
				|| bNonNativeEnum);
		const bool bConvertedCode = (CPPExportFlags & CPPF_BlueprintCppBackend) && bNonNativeEnum;

		FString FullyQualifiedEnumName;
		if (!Enum->CppType.IsEmpty())
		{
			FullyQualifiedEnumName = Enum->CppType;
		}
		else
		{
			// This would give the wrong result if it's a namespaced type and the CppType hasn't
			// been set, but we do this here in case existing code relies on it... somehow.
			if ((CPPExportFlags & CPPF_BlueprintCppBackend) && bNonNativeEnum)
			{
				ensure(Enum->CppType.IsEmpty());
				FullyQualifiedEnumName = ::UnicodeToCPPIdentifier(Enum->GetName(), false, TEXT("E__"));
			}
			else
			{
				FullyQualifiedEnumName = Enum->GetName();
			}
		}
		 
		if (bEnumClassForm || bRawParam || bConvertedCode)
		{
			return FullyQualifiedEnumName;
		}
		else
		{
			return FString::Printf(TEXT("TEnumAsByte<%s>"), *FullyQualifiedEnumName);
		}
	}
	return Super::GetCPPType(ExtendedTypeText, CPPExportFlags);
}

template <typename OldIntType>
struct TConvertIntToEnumProperty
{
	static void Convert(FStructuredArchive::FSlot Slot, FByteProperty* Property, UEnum* Enum, void* Obj, const FPropertyTag& Tag)
	{
		OldIntType OldValue;
		Slot << OldValue;

		ConvertValue(OldValue, Property, Enum, Obj, Tag);
	}

	static void ConvertValue(OldIntType OldValue, FByteProperty* Property, UEnum* Enum, void* Obj, const FPropertyTag& Tag)
	{
		uint8 NewValue = (uint8)OldValue;
		if (OldValue > (OldIntType)TNumericLimits<uint8>::Max() || !Enum->IsValidEnumValue(NewValue))
		{
			if constexpr (std::is_unsigned_v<OldIntType>)
			{
				UE_LOG(
					LogClass,
					Warning,
					TEXT("Failed to find valid enum value '%" UINT64_FMT "' for enum type '%s' when converting property '%s' during property loading - setting to '%s'"),
					(uint64)OldValue,
					*Enum->GetName(),
					*Property->GetName(),
					*Enum->GetNameByValue(Enum->GetMaxEnumValue()).ToString()
				);
			}
			else
			{
				UE_LOG(
					LogClass,
					Warning,
					TEXT("Failed to find valid enum value '%" INT64_FMT "' for enum type '%s' when converting property '%s' during property loading - setting to '%s'"),
					(int64)OldValue,
					*Enum->GetName(),
					*Property->GetName(),
					*Enum->GetNameByValue(Enum->GetMaxEnumValue()).ToString()
				);
			}

			NewValue = IntCastChecked<uint8>(Enum->GetMaxEnumValue());
		}

		Property->SetPropertyValue_InContainer(Obj, NewValue, Tag.ArrayIndex);
	}
};

EConvertFromTypeResult FByteProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	const EName* TagType= Tag.Type.ToEName();
	if (UNLIKELY(Tag.Type.GetNumber() || !TagType))
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}

	switch (*TagType)
	{
	default:
		return EConvertFromTypeResult::UseSerializeItem;
	case NAME_ByteProperty:
		if ((Tag.GetType().GetParameterCount() == 0) != (Enum == nullptr))
		{
			// A byte property gained or lost an enum.
			uint8 PreviousValue = 0;
			if (Enum)
			{
				// A nested property would lose its enum name on previous versions. Handle this case for backward compatibility reasons.
				if (GetOwner<FProperty>() && Slot.GetArchiveState().UEVer() < EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
				{
					return EConvertFromTypeResult::UseSerializeItem;
				}

				// Read the byte and assume its value corresponds to a valid enumerator.
				Slot << PreviousValue;
			}
			else
			{
				// Attempt to find the enum from the tag and find the byte value from the enum.
				PreviousValue = (uint8)ReadEnumAsInt64(Slot, DefaultsStruct, Tag);
			}

			SetPropertyValue_InContainer(Data, PreviousValue, Tag.ArrayIndex);
			return EConvertFromTypeResult::Converted;
		}
	#if WITH_EDITORONLY_DATA
		if (UNLIKELY(Enum && FUObjectThreadContext::Get().GetSerializeContext()->bTrackUnknownProperties && !CanSerializeFromTypeName(Tag.GetType())))
		{
			FName EnumValueName;
			int64 EnumValue = 0;
			TryLoadEnumValueByName(Slot, Slot.GetUnderlyingArchive(), Enum, EnumValueName, EnumValue);
			SetPropertyValue_InContainer(Data, IntCastChecked<uint8>(EnumValue), Tag.ArrayIndex);
			return EConvertFromTypeResult::Converted;
		}
	#endif
		return EConvertFromTypeResult::UseSerializeItem;
	case NAME_EnumProperty:
		if (Enum)
		{
			FName EnumValueName;
			int64 EnumValue = 0;
			TryLoadEnumValueByName(Slot, Slot.GetUnderlyingArchive(), Enum, EnumValueName, EnumValue);
			SetPropertyValue_InContainer(Data, IntCastChecked<uint8>(EnumValue), Tag.ArrayIndex);
		}
		else
		{
			// Attempt to find the enum from the tag and find the byte value from the enum.
			uint8 PreviousValue = (uint8)ReadEnumAsInt64(Slot, DefaultsStruct, Tag);
			SetPropertyValue_InContainer(Data, PreviousValue, Tag.ArrayIndex);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_Int8Property:
		if (Enum)
		{
			TConvertIntToEnumProperty<int8>::Convert(Slot, this, Enum, Data, Tag);
		}
		else
		{
			ConvertFromArithmeticValue<int8>(Slot, Data, Tag);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_Int16Property:
		if (Enum)
		{
			TConvertIntToEnumProperty<int16>::Convert(Slot, this, Enum, Data, Tag);
		}
		else
		{
			ConvertFromArithmeticValue<int16>(Slot, Data, Tag);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_IntProperty:
		if (Enum)
		{
			TConvertIntToEnumProperty<int32>::Convert(Slot, this, Enum, Data, Tag);
		}
		else
		{
			ConvertFromArithmeticValue<int32>(Slot, Data, Tag);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_Int64Property:
		if (Enum)
		{
			TConvertIntToEnumProperty<int64>::Convert(Slot, this, Enum, Data, Tag);
		}
		else
		{
			ConvertFromArithmeticValue<int64>(Slot, Data, Tag);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_UInt16Property:
		if (Enum)
		{
			TConvertIntToEnumProperty<uint16>::Convert(Slot, this, Enum, Data, Tag);
		}
		else
		{
			ConvertFromArithmeticValue<uint16>(Slot, Data, Tag);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_UInt32Property:
		if (Enum)
		{
			TConvertIntToEnumProperty<uint32>::Convert(Slot, this, Enum, Data, Tag);
		}
		else
		{
			ConvertFromArithmeticValue<uint32>(Slot, Data, Tag);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_UInt64Property:
		if (Enum)
		{
			TConvertIntToEnumProperty<uint64>::Convert(Slot, this, Enum, Data, Tag);
		}
		else
		{
			ConvertFromArithmeticValue<uint64>(Slot, Data, Tag);
		}
		return EConvertFromTypeResult::Converted;
	case NAME_BoolProperty:
		if (Enum)
		{
			TConvertIntToEnumProperty<uint64>::ConvertValue(Tag.BoolVal, this, Enum, Data, Tag);
		}
		else
		{
			SetPropertyValue_InContainer(Data, Tag.BoolVal, Tag.ArrayIndex);
		}
		return EConvertFromTypeResult::Converted;
	}
}

#if WITH_EDITORONLY_DATA
void FByteProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (Enum)
	{
		FNameBuilder NameBuilder;
		Enum->GetPathName(nullptr, NameBuilder);
		Builder.Update(NameBuilder.GetData(), NameBuilder.Len() * sizeof(NameBuilder.GetData()[0]));
		int32 Num = Enum->NumEnums();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			AppendHash(Builder, Enum->GetNameByIndex(Index));
		}
	}
}
#endif

void FByteProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (!Enum || (PortFlags & PPF_ConsoleVariable))
	{
		Super::ExportText_Internal(ValueStr, PropertyValueOrContainer, PropertyPointerType, DefaultValue, Parent, PortFlags, ExportRootScope);
		return;
	}

	UE::CoreUObject::Private::ExportEnumToBuffer(Enum, this, this, ValueStr, PropertyValueOrContainer, PropertyPointerType, DefaultValue, Parent, PortFlags, ExportRootScope);
}
const TCHAR* FByteProperty::ImportText_Internal(const TCHAR* InBuffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	if( Enum && (PortFlags & PPF_ConsoleVariable) == 0 )
	{
		return UE::CoreUObject::Private::ImportEnumFromBuffer(Enum, this, this, TEXT("FByteProperty"), InBuffer, ContainerOrPropertyPtr, PropertyPointerType, ErrorText);
	}
	
	// Interpret "True" and "False" as 1 and 0. This is mostly for importing a property that was exported as a bool and is imported as a non-enum byte.
	// Also allow for ConsoleVariable-backed enums to attempt to convert True/False to 1/0 in case a bool cvar has been converted to an enum. 
	// Enum properties backed by an integer CVar are stored as number values, so this code will only do anything when reading an old .ini file with True/False values
	// We log a warning so users can fix up their .ini files to use integer values that map to the enum
	FString Temp;
	const TCHAR* Buffer = FPropertyHelpers::ReadToken(InBuffer, Temp);
	if (!Buffer)
	{
		return nullptr;
	}

	const FCoreTexts& CoreTexts = FCoreTexts::Get();

	if (Temp == TEXT("True") || Temp == *(CoreTexts.True.ToString()))
	{
		uint64 TrueValue = 1ull;
		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			SetValue_InContainer(ContainerOrPropertyPtr, static_cast<uint8>(TrueValue));
		}
		else
		{
			SetIntPropertyValue(PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType), TrueValue);
		}

		if (Enum)
		{
			UE_LOGF(LogClass, Warning, "ConsoleVariable-Backed Enum Property of type '%ls' was set from a string. Please update the cvar in your ini files.", *Enum->GetPathName());
		}

		return Buffer;
	}
	else if (Temp == TEXT("False") || Temp == *(CoreTexts.False.ToString()))
	{
		uint64 FalseValue = 0ull;
		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			SetValue_InContainer(ContainerOrPropertyPtr, static_cast<uint8>(FalseValue));
		}
		else
		{
			SetIntPropertyValue(PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType), FalseValue);
		}

		if (Enum)
		{
			UE_LOGF(LogClass, Warning, "ConsoleVariable-Backed Enum Property of type '%ls' was set from a string. Please update the cvar in your ini files.", *Enum->GetPathName());
		}

		return Buffer;
	}

	return Super::ImportText_Internal( InBuffer, ContainerOrPropertyPtr, PropertyPointerType, Parent, PortFlags, ErrorText );
}

UEnum* FByteProperty::GetIntPropertyEnum() const
{
	return Enum;
}

uint64 FByteProperty::GetMaxNetSerializeBits() const
{
	const uint64 MaxBits = 8;
	const uint64 DesiredBits = Enum ? FMath::CeilLogTwo64(Enum->GetMaxEnumValue() + 1) : MaxBits;

	return FMath::Min(DesiredBits, MaxBits);
}

bool FByteProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	if (!Super::LoadTypeName(Type, Tag))
	{
		return false;
	}

	const UE::FPropertyTypeName TypePath = Type.GetParameter(0);
	if (TypePath.IsEmpty())
	{
		return true;
	}

	if ((Enum = UE::FindObjectByTypePath<UEnum>(TypePath)))
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	Enum = StaticEnum<EFallbackEnum>();
	SetMetaData(UE::NAME_OriginalType, *WriteToString<256>(TypePath));
	return true;
#else
	return false;
#endif
}

void FByteProperty::SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const
{
	Super::SaveTypeName(Type);

	if (const UEnum* LocalEnum = Enum)
	{
		Type.BeginParameters();
	#if WITH_EDITORONLY_DATA
		if (const UE::FPropertyTypeName OriginalType = UE::FindOriginalType(this); !OriginalType.IsEmpty())
		{
			Type.AddType(OriginalType);
		}
		else
	#endif // WITH_EDITORONLY_DATA
		{
			Type.AddPath(LocalEnum);
		}
		Type.EndParameters();
	}
}

bool FByteProperty::CanSerializeFromTypeName(UE::FPropertyTypeName Type) const
{
	if (!Super::CanSerializeFromTypeName(Type))
	{
		return false;
	}

	const FName EnumName = Type.GetParameterName(0);
	if (const UEnum* LocalEnum = Enum)
	{
		if (EnumName == LocalEnum->GetFName())
		{
			return true;
		}

	#if WITH_EDITORONLY_DATA
		if (const UE::FPropertyTypeName OriginalType = UE::FindOriginalType(this); !OriginalType.IsEmpty())
		{
			return EnumName == OriginalType.GetName();
		}
	#endif // WITH_EDITORONLY_DATA

		return false;
	}
	return EnumName.IsNone();
}
