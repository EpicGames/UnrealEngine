// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/PropertyHelper.h"
#include "Hash/Blake3.h"

/*-----------------------------------------------------------------------------
	FClassProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FClassProperty)

FClassProperty::FClassProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, MetaClass(nullptr)
{
}

FClassProperty::FClassProperty(FFieldVariant InOwner, const UECodeGen_Private::FClassPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FObjectPropertyParams&)Prop)
{
	if (!PropertyClass)
	{
		PropertyClass = UClass::StaticClass();
	}
	MetaClass = Prop.MetaClassFunc ? Prop.MetaClassFunc(ETypeConstructPhase::Inner) : nullptr;
}

#if WITH_EDITORONLY_DATA
FClassProperty::FClassProperty(UField* InField)
	: Super(InField)
{
	UClassProperty* SourceProperty = CastChecked<UClassProperty>(InField);
	MetaClass = SourceProperty->MetaClass;
}
#endif // WITH_EDITORONLY_DATA

void FClassProperty::PostDuplicate(const FField& InField)
{
	const FClassProperty& Source = static_cast<const FClassProperty&>(InField);
	MetaClass = Source.MetaClass;
	Super::PostDuplicate(InField);
}

void FClassProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar << MetaClass;

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
		{
			PlaceholderClass->AddReferencingProperty(this);
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	if( !MetaClass )
	{
		// If we failed to load the MetaClass and we're not a CDO, that means we relied on a class that has been removed or doesn't exist.
		// The most likely cause for this is either an incomplete recompile, or if content was migrated between games that had native class dependencies
		// that do not exist in this game.  We allow blueprint classes to continue, because compile on load will error out, and stub the class that was using it
		UClass* TestClass = dynamic_cast<UClass*>(GetOwnerStruct());
		if( TestClass && TestClass->HasAllClassFlags(CLASS_Native) && !TestClass->HasAllClassFlags(CLASS_NewerVersionExists) && (TestClass->GetOutermost() != GetTransientPackage()) )
		{
			checkf(false, TEXT("Class property tried to serialize a missing class.  Did you remove a native class and not fully recompile?"));
		}
	}
}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
void FClassProperty::SetMetaClass(UClass* NewMetaClass)
{
	if (ULinkerPlaceholderClass* NewPlaceholderClass = Cast<ULinkerPlaceholderClass>(NewMetaClass))
	{
		NewPlaceholderClass->AddReferencingProperty(this);
	}

	if (ULinkerPlaceholderClass* OldPlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
	{
		OldPlaceholderClass->RemoveReferencingProperty(this);
	}
	MetaClass = NewMetaClass;
}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

void FClassProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( MetaClass );
	Super::AddReferencedObjects( Collector );
}

const TCHAR* FClassProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	const TCHAR* Result = FObjectProperty::ImportText_Internal( Buffer, ContainerOrPropertyPtr, PropertyPointerType, Parent, PortFlags, ErrorText );
	if( Result )
	{
		void* Data = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
		if (UClass* AssignedPropertyClass = dynamic_cast<UClass*>(GetObjectPropertyValue(Data)))
		{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			FLinkerLoad* ObjectLinker = (Parent != nullptr) ? Parent->GetClass()->GetLinker() : GetLinker();
			auto IsDeferringValueLoad = [](const UClass* Class)->bool
			{
				const ULinkerPlaceholderClass* Placeholder = Cast<ULinkerPlaceholderClass>(Class);
				return Placeholder && !Placeholder->IsMarkedResolved();
			};
			const bool bIsDeferringValueLoad = IsDeferringValueLoad(MetaClass) || ((!ObjectLinker || (ObjectLinker->LoadFlags & LOAD_DeferDependencyLoads) != 0) && IsDeferringValueLoad(AssignedPropertyClass));

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
			check( bIsDeferringValueLoad || !(Cast<ULinkerPlaceholderClass>(MetaClass) || Cast<ULinkerPlaceholderClass>(AssignedPropertyClass)) );
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

#else  // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING 
			bool const bIsDeferringValueLoad = false;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

			// Validate metaclass.
			if ((!AssignedPropertyClass->IsChildOf(MetaClass)) && !bIsDeferringValueLoad)
			{
				// the object we imported doesn't implement our interface class
				ErrorText->Logf(TEXT("Invalid object '%s' specified for property '%s'"), *AssignedPropertyClass->GetFullName(), *GetName());
				UObject* NullObj = nullptr;
				if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
				{
					SetValue_InContainer(ContainerOrPropertyPtr, NullObj);
				}
				else
				{
					SetObjectPropertyValue(PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType), NullObj);
				}
				Result = NULL;
			}
		}
	}
	return Result;
}

FString FClassProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const
{
	check(MetaClass);
	if (EnumHasAnyFlags(PropertyFlags, CPF_TObjectPtr))
	{
		if (!EnumHasAnyFlags((EPropertyExportCPPFlags)CPPExportFlags, CPPF_NoTObjectPtr))
		{
			return FString::Printf(TEXT("TObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
		}
	}
	else if (EnumHasAnyFlags(PropertyFlags, CPF_UObjectWrapper))
	{
		return FString::Printf(TEXT("TSubclassOf<%s%s>"), MetaClass->GetPrefixCPP(), *MetaClass->GetName());
	}
	return FString::Printf(TEXT("%s%s*"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
}

FString FClassProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	if (PropertyFlags & CPF_TObjectPtr)
	{
		ExtendedTypeText = FString::Printf(TEXT("TObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
		return TEXT("OBJECTPTR");
	}
	ExtendedTypeText = TEXT("UClass");
	return TEXT("OBJECT");
}

bool FClassProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && (MetaClass == ((FClassProperty*)Other)->MetaClass);
}

bool FClassProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	TObjectPtr<UObject> ObjectA = GetObjectPtrPropertyValue(A);
	TObjectPtr<UObject> ObjectB = B ? GetObjectPtrPropertyValue(B) : TObjectPtr<UObject>();

#if DO_CHECK && UE_WITH_REMOTE_OBJECT_HANDLE
	// Object.IsA<UClass>() will try to resolve a remote object so skip these checks if we're running with remote object support and serializing remote objects
	if (!(PortFlags & PPF_AvoidRemoteObjectMigration))
#endif
	{
		// We need to see A and B in the same check.
		checkf(
			(ObjectA == nullptr || ObjectA.IsA<UClass>()) && (ObjectB == nullptr || ObjectB.IsA<UClass>()),
			TEXT("FClassProperty::Identical - ObjectA: (Name: %s Path: %s Class: %s) ObjectB: (Name: %s Path: %s Class: %s)"),
			ObjectA ? *ObjectA->GetName() : TEXT("nullptr"),
			ObjectA ? *ObjectA->GetPathName() : TEXT("nullptr"),
			ObjectA ? *ObjectA->GetClass()->GetName() : TEXT("nullptr"),
			ObjectB ? *ObjectB->GetName() : TEXT("nullptr"),
			ObjectB ? *ObjectB->GetPathName() : TEXT("nullptr"),
			ObjectB ? *ObjectB->GetClass()->GetName() : TEXT("nullptr")
		);
	}

	return (ObjectA == ObjectB);
}

#if WITH_EDITORONLY_DATA
void FClassProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (MetaClass)
	{
		// Hash the class's name instead of recursively hashing the class; the class's schema does not impact how we serialize our pointer to it
		FNameBuilder ObjectPath;
		MetaClass->GetPathName(nullptr, ObjectPath);
		Builder.Update(ObjectPath.GetData(), ObjectPath.Len() * sizeof(ObjectPath.GetData()[0]));
	}
}
#endif
