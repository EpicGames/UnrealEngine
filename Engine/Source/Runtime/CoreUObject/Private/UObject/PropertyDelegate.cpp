// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/PropertyHelper.h"
#include "UObject/LinkerPlaceholderFunction.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "Hash/Blake3.h"

/*-----------------------------------------------------------------------------
	FDelegateProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FDelegateProperty)

FDelegateProperty::FDelegateProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, SignatureFunction(nullptr)
{
}

FDelegateProperty::FDelegateProperty(FFieldVariant InOwner, const UECodeGen_Private::FDelegatePropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
	SignatureFunction = Prop.SignatureFunctionFunc ? Prop.SignatureFunctionFunc(ETypeConstructPhase::Outer) : nullptr;
}

#if WITH_EDITORONLY_DATA
FDelegateProperty::FDelegateProperty(UField* InField)
	: Super(InField)
{
	UDelegateProperty* SourceProperty = CastChecked<UDelegateProperty>(InField);
	SignatureFunction = SourceProperty->SignatureFunction;
}
#endif // WITH_EDITORONLY_DATA

void FDelegateProperty::PostDuplicate(const FField& InField)
{
	const FDelegateProperty& Source = static_cast<const FDelegateProperty&>(InField);
	SignatureFunction = Source.SignatureFunction;
	Super::PostDuplicate(InField);
}

void FDelegateProperty::InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, FObjectInstancingGraph* InstanceGraph )
{
	for( int32 i=0; i<ArrayDim; i++ )
	{
		FScriptDelegate& DestDelegate = ((FScriptDelegate*)Data)[i];
		UObject* CurrentUObject = DestDelegate.GetUObject();

		if (CurrentUObject)
		{
			UObject *Template = NULL;

			if (DefaultData)
			{
				FScriptDelegate& DefaultDelegate = ((FScriptDelegate*)DefaultData)[i];
				Template = DefaultDelegate.GetUObject();
			}

			UObject* NewUObject = InstanceGraph->InstancePropertyValue(Template, CurrentUObject, InOwner, EInstancePropertyValueFlags::AllowSelfReference | EInstancePropertyValueFlags::DoNotCreateNewInstance);
			DestDelegate.BindUFunction(NewUObject, DestDelegate.GetFunctionName());
		}
	}
}

bool FDelegateProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags ) const
{
	const FScriptDelegate* DA = (const FScriptDelegate*)NotNullGet(A);
	const FScriptDelegate* DB = (const FScriptDelegate*)B;

	if (!DB)
	{
		return !DA->IsBound();
	}
	
	return *DA == *DB;
}


void FDelegateProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	Slot << *GetPropertyValuePtr(Value);
}


bool FDelegateProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	// Do not allow replication of delegates, as there is no way to make this secure (it allows the execution of any function in any object, on the remote client/server)
	return 1;
}


FString FDelegateProperty::GetCPPType( FString* ExtendedTypeText/*=NULL*/, uint32 CPPExportFlags/*=0*/ ) const
{
	check(SignatureFunction);

	FString UnmangledFunctionName = SignatureFunction->GetName().LeftChop( FString( HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX ).Len() );
	const bool bBlueprintCppBackend = (0 != (CPPExportFlags & EPropertyExportCPPFlags::CPPF_BlueprintCppBackend));
	const bool bNative = SignatureFunction->IsNative();
	if (bBlueprintCppBackend && bNative)
	{
		UStruct* StructOwner = Cast<UStruct>(SignatureFunction->GetOuter());
		if (StructOwner)
		{
			return FString::Printf(TEXT("%s%s::F%s"), StructOwner->GetPrefixCPP(), *StructOwner->GetName(), *UnmangledFunctionName);
		}
	}
	else
	{
		const bool NonNativeClassOwner = (SignatureFunction->GetOwnerClass() && !SignatureFunction->GetOwnerClass()->HasAnyClassFlags(CLASS_Native));
		if (bBlueprintCppBackend && NonNativeClassOwner)
		{
			// The name must be valid, this removes spaces, ?, etc from the user's function name. It could
			// be slightly shorter because the postfix ("__pf") is not needed here because we further post-
			// pend to the string. Normally the postfix is needed to make sure we don't mangle to a valid
			// identifier and collide:
			UnmangledFunctionName = UnicodeToCPPIdentifier(UnmangledFunctionName, false, TEXT(""));
			// the name must be unique
			const FString OwnerName = UnicodeToCPPIdentifier(SignatureFunction->GetOwnerClass()->GetName(), false, TEXT(""));
			const FString NewUnmangledFunctionName = FString::Printf(TEXT("%s__%s"), *UnmangledFunctionName, *OwnerName);
			UnmangledFunctionName = NewUnmangledFunctionName;
		}
		if (0 != (CPPExportFlags & EPropertyExportCPPFlags::CPPF_CustomTypeName))
		{
			UnmangledFunctionName += TEXT("__SinglecastDelegate");
		}
	}
	return FString("F") + UnmangledFunctionName;
}

void FDelegateProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	auto ExportDelegateAsText = [&ValueStr](FScriptDelegate* ScriptDelegate)
	{
		check(ScriptDelegate != NULL);
		bool bDelegateHasValue = ScriptDelegate->GetFunctionName() != NAME_None;
		ValueStr += FString::Printf(TEXT("%s.%s"),
			ScriptDelegate->GetUObject() != NULL ? *ScriptDelegate->GetUObject()->GetPathName() : TEXT("(null)"),
			*ScriptDelegate->GetFunctionName().ToString());
	};

	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		FScriptDelegate LocalScriptDelegate;
		GetValue_InContainer(PropertyValueOrContainer, &LocalScriptDelegate);
		ExportDelegateAsText(&LocalScriptDelegate);
	}
	else
	{
		FScriptDelegate* ScriptDelegate = (FScriptDelegate*)PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType);
		ExportDelegateAsText(ScriptDelegate);
	}
}

const TCHAR* FDelegateProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	const TCHAR* Result = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		FScriptDelegate LocalScriptDelegate;
		Result = DelegatePropertyTools::ImportDelegateFromText(*(FScriptDelegate*)&LocalScriptDelegate, SignatureFunction, Buffer, Parent, ErrorText);
		SetValue_InContainer(ContainerOrPropertyPtr, LocalScriptDelegate);
	}
	else
	{
		void* PropertyValue = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
		Result = DelegatePropertyTools::ImportDelegateFromText(*(FScriptDelegate*)PropertyValue, SignatureFunction, Buffer, Parent, ErrorText);
	}

	return Result;
}

void FDelegateProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar << SignatureFunction;

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (auto PlaceholderFunc = Cast<ULinkerPlaceholderFunction>(SignatureFunction))
		{
			PlaceholderFunc->AddReferencingProperty(this);
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

void FDelegateProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (SignatureFunction && !SignatureFunction->IsA<ULinkerPlaceholderFunction>())
#endif//USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	{
		Collector.AddReferencedObject(SignatureFunction);
	}
	Super::AddReferencedObjects(Collector);
}

bool FDelegateProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && (SignatureFunction == ((FDelegateProperty*)Other)->SignatureFunction);
}

#if WITH_EDITORONLY_DATA
void FDelegateProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (SignatureFunction)
	{
		// Recursively hash the SignatureFunction. The function's schema does not impact how we serialize our pointer
		// to it, but it is used by the blueprint compiler to validate that a subscriber to the delegate in a blueprint
		// matches the function prototype of the SignatureFunction.
		const FIoHash& FunctionSchemaHash = SignatureFunction->GetSchemaHash(bSkipEditorOnly);
		Builder.Update(&FunctionSchemaHash, sizeof(FunctionSchemaHash));
	}
}
#endif

