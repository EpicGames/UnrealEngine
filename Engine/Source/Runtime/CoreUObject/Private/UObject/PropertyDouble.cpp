// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Math/PreciseFP.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	FDoubleProperty.
-----------------------------------------------------------------------------*/

FDoubleProperty::FDoubleProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
	PropertyFlags |= CPF_HasGetValueTypeHash;
}

FDoubleProperty::FDoubleProperty(FFieldVariant InOwner, const UECodeGen_Private::FDoublePropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
	PropertyFlags |= CPF_HasGetValueTypeHash;
}

#if WITH_EDITORONLY_DATA
FDoubleProperty::FDoubleProperty(UField* InField)
	: Super(InField)
{
	PropertyFlags |= CPF_HasGetValueTypeHash;
}
#endif // WITH_EDITORONLY_DATA

bool FDoubleProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	return UE::PreciseFPEqual(*static_cast<const double*>(NotNullGet(A)), B ? *static_cast<const double*>(B) : 0.0);
}

uint32 FDoubleProperty::GetValueTypeHashInternal(const void* Src) const
{
	return UE::PreciseFPHash(*static_cast<const double*>(Src));
}
