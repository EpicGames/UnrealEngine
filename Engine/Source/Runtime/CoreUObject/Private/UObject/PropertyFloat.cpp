// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Math/PreciseFP.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	FFloatProperty.
-----------------------------------------------------------------------------*/

FFloatProperty::FFloatProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
	PropertyFlags |= CPF_HasGetValueTypeHash;
}

FFloatProperty::FFloatProperty(FFieldVariant InOwner, const UECodeGen_Private::FFloatPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
	PropertyFlags |= CPF_HasGetValueTypeHash;
}

#if WITH_EDITORONLY_DATA
FFloatProperty::FFloatProperty(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA

bool FFloatProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	return UE::PreciseFPEqual(*static_cast<const float*>(NotNullGet(A)), B ? *static_cast<const float*>(B) : 0.0f);
}

uint32 FFloatProperty::GetValueTypeHashInternal(const void* Src) const
{
	return UE::PreciseFPHash(*static_cast<const float*>(Src));
}
