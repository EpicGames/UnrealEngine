// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyHelper.h"

#include "Logging/StructuredLog.h"
#include "Containers/StringView.h"
#include "Misc/AsciiSet.h"
#include "String/Find.h"
#include "String/ParseTokens.h"
#include "UObject/Class.h"
#include "UObject/CoreNet.h"
#include "UObject/CoreRedirects.h"
#include "UObject/EnumProperty.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/PropertyTypeName.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(PropertyHelper)

void SkipWhitespace(const TCHAR*& Str)
{
	while(FChar::IsWhitespace(*Str))
	{
		Str++;
	}
}

bool AreInstancedObjectsIdentical( UObject* ObjectA, UObject* ObjectB, uint32 PortFlags )
{
	check(ObjectA && ObjectB);
	bool Result = true;

	// we check for recursive comparison here...if we have encountered this pair before on the call stack, the graphs are isomorphic
	struct FRecursionCheck
	{
		UObject* ObjectA;
		UObject* ObjectB;
		uint32 PortFlags;

		bool operator==(const FRecursionCheck& Other) const
		{
			return ObjectA == Other.ObjectA && ObjectB == Other.ObjectB && PortFlags == Other.PortFlags;
		}
	};
	static thread_local TArray<FRecursionCheck> RecursionCheck;
	FRecursionCheck Test;
	Test.ObjectA = ObjectA;
	Test.ObjectB = ObjectB;
	Test.PortFlags = PortFlags;
	if (!RecursionCheck.Contains(Test))
	{
		RecursionCheck.Push(Test);
		for ( FProperty* Prop = ObjectA->GetClass()->PropertyLink; Prop && Result; Prop = Prop->PropertyLinkNext )
		{
			// only the properties that could have been modified in the editor should be compared
			// (skipping the name and archetype properties, since name will almost always be different)
			bool bConsiderProperty;
			if ( (PortFlags&PPF_Copy) == 0 )
			{
				bConsiderProperty = Prop->ShouldDuplicateValue();
			}
			else
			{
				bConsiderProperty = (Prop->PropertyFlags&CPF_Edit) != 0;
			}

			if ( bConsiderProperty )
			{
				for ( int32 i = 0; i < Prop->ArrayDim && Result; i++ )
				{
					if ( !Prop->Identical_InContainer(ObjectA, ObjectB, i, PortFlags) )
					{
						Result = false;
					}
				}
			}
		}
		if (Result)
		{
			// Allow the component to compare its native/ intrinsic properties.
			Result = ObjectA->AreNativePropertiesIdenticalTo( ObjectB );
		}
		RecursionCheck.Pop();
	}
	return Result;
}

bool FDeltaIndexHelper::SerializeNext(FArchive &Ar, int32& Index)
{
	NET_CHECKSUM(Ar);
	if (Ar.IsSaving())
	{
		uint32 Delta = Index - LastIndex;
		Ar.SerializeIntPacked(Delta);
		LastIndex = Index;
		LastIndexFull = Index;
	}
	else
	{
		uint32 Delta = 0;
		Ar.SerializeIntPacked(Delta);
		Index = (Delta == 0 ? INDEX_NONE : LastIndex + Delta);
		LastIndex = Index;
		LastIndexFull = Index;
	}

	return (Index != INDEX_NONE && !Ar.IsError());
}

void FDeltaIndexHelper::SerializeNext(FArchive &OutBunch, FArchive &OutFull, int32 Index)
{
	NET_CHECKSUM(OutBunch);
	NET_CHECKSUM(OutFull);
		
	// Full state
	uint32 DeltaFull = Index - LastIndexFull;
	OutFull.SerializeIntPacked(DeltaFull);
	LastIndexFull = Index;

	// Delta State
	uint32 Delta = Index - LastIndex;
	OutBunch.SerializeIntPacked(Delta);
}

void FDeltaIndexHelper::SerializeEarlyEnd(FArchive &Ar)
{
	NET_CHECKSUM(Ar);

	uint32 End = 0;
	Ar.SerializeIntPacked(End);
}

const TCHAR* DelegatePropertyTools::ImportDelegateFromText( FScriptDelegate& Delegate, const UFunction* SignatureFunction, const TCHAR* Buffer, UObject* Parent, FOutputDevice* ErrorText )
{
	SkipWhitespace(Buffer);

	Delegate.Unbind();

	// Get object name
	while (*Buffer == TEXT('('))
	{
		++Buffer;
	}

	// Find the function name separator before finding the rest of the object name.
	const TCHAR* DelegatePathEnd = FAsciiSet::FindFirstOrEnd(Buffer, ",)");
	const TCHAR* NameEnd = FAsciiSet::FindLastOrEnd(Buffer, ".");

	if (NameEnd > DelegatePathEnd || *NameEnd != TEXT('.'))
	{
		// Didn't find the separator before the end - probably malformed text
		return nullptr;
	}

	FStringView ObjNameView(Buffer, UE_PTRDIFF_TO_INT32(NameEnd - Buffer));
	Buffer += ObjNameView.Len();

	// Handle (null).None syntax used for empty delegates in ExportText
	bool bIsNullClass = *Buffer == TCHAR(')') && ObjNameView == TEXT("null");
	if (bIsNullClass)
	{
		++Buffer;
	}

	// Get function name
	FStringView FuncNameView;
	UClass* Cls = nullptr;
	UObject* Object = nullptr;
	if (*Buffer == TCHAR('.'))
	{
		++Buffer;
		const TCHAR* FuncNameEnd = FAsciiSet::FindFirstOrEnd(Buffer, "),");
		FuncNameView = FStringView(Buffer, UE_PTRDIFF_TO_INT32(FuncNameEnd - Buffer));
		Buffer += FuncNameView.Len();
	}
	else
	{
		// if there's no dot, then a function name was specified without any object qualifier
		// if we're importing for a subobject, assume the parent object as the source
		// otherwise, assume Parent for the source
		// if neither are valid, then the importing fails
		if (Parent == nullptr)
		{
			FString DelegateString = Delegate.GetFunctionName().ToString();
			FString SignatureFunctionString = GetNameSafe(SignatureFunction);
			ErrorText->Logf(ELogVerbosity::Warning, TEXT("Cannot import unqualified delegate name; no object to search. Delegate=%s SignatureFunction=%s"),
				*DelegateString, *SignatureFunctionString);
			return nullptr;
		}

		// since we don't support nested components, we only need to check one level deep
		if (!Parent->HasAnyFlags(RF_ClassDefaultObject) && Parent->GetOuter() && Parent->GetOuter()->HasAnyFlags(RF_ClassDefaultObject))
		{
			Object = Parent->GetOuter();
		}
		else
		{
			Object = Parent;
		}

		Cls = Object->GetClass();
		FuncNameView = ObjNameView;
	}

	if (bIsNullClass)
	{
		FName ParsedName(FuncNameView);
		if (ParsedName == NAME_None)
		{
			// Deliberately null
			return Buffer;
		}

		FString DelegateString = Delegate.GetFunctionName().ToString();
		FString SignatureFunctionString = GetNameSafe(SignatureFunction);
		ErrorText->Logf(ELogVerbosity::Warning, TEXT("Cannot import delegate with function name %.*s but null class. Delegate=%s SignatureFunction=%s"),
			FuncNameView.Len(), FuncNameView.GetData(), *DelegateString, *SignatureFunctionString);
		return nullptr;
	}

	if (!Cls)
	{
		Cls = UClass::TryFindTypeSlow<UClass>(ObjNameView);
		if (Cls)
		{
			Object = Cls->GetDefaultObject();
		}
		else
		{
			// Check outer chain before checking all packages
			UObject* OuterToCheck = Parent;
			while (Object == nullptr && OuterToCheck)
			{
				Object = StaticFindObject(UObject::StaticClass(), OuterToCheck, ObjNameView);
				OuterToCheck = OuterToCheck->GetOuter();
			}

			if (!Object)
			{
				Object = StaticFindFirstObject(UObject::StaticClass(), ObjNameView, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
				if (!Object)
				{
					ErrorText->Logf(ELogVerbosity::Warning,TEXT("Unable to find object %.*s for delegate"), ObjNameView.Len(), ObjNameView.GetData());
					return nullptr;
				}
			}
			Cls = Object->GetClass();
		}
	}

	// Check function params.
	UFunction* Func = FindUField<UFunction>( Cls, *FString(FuncNameView) );
	if (!Func)
	{
		ErrorText->Logf(ELogVerbosity::Warning,TEXT("Unable to find function %.*s in object %.*s for delegate (found class: %s)"),FuncNameView.Len(),FuncNameView.GetData(),ObjNameView.Len(),ObjNameView.GetData(),*GetNameSafe(Cls));
		return nullptr;
	}

	// Find the delegate UFunction to check params
	check(SignatureFunction != nullptr && "Invalid delegate property");

	// check return type and params
	if (Func->NumParms != SignatureFunction->NumParms)
	{
		ErrorText->Logf(ELogVerbosity::Warning,TEXT("Function %s does not match number of params with delegate signature %s"),*Func->GetName(), *SignatureFunction->GetName());
		return nullptr;
	}

	int32 Count=0;
	for( TFieldIterator<FProperty> It1(Func),It2(SignatureFunction); Count<SignatureFunction->NumParms; ++It1,++It2,++Count )
	{
		if( It1->GetClass()!=It2->GetClass() || (It1->PropertyFlags&CPF_OutParm)!=(It2->PropertyFlags&CPF_OutParm) )
		{
			ErrorText->Logf(ELogVerbosity::Warning,TEXT("Function %s does not match param types with delegate signature %s"), *Func->GetName(), *SignatureFunction->GetName());
			return nullptr;
		}
	}

	//UE_LOGF(LogProperty, Log, "... importing delegate FunctionName:'%ls'(%.*ls)   Object:'%ls'(%.*ls)",Func != nullptr ? *Func->GetName() : TEXT("nullptr"), FuncNameView.Len(), FuncNameView.GetData(), Object != nullptr ? *Object->GetFullName() : TEXT("nullptr"), ObjNameView.Len(), ObjNameView.GetData());

	Delegate.BindUFunction(Object, Func->GetFName());

	return Buffer;
}

namespace UE
{

UObject* FindObjectByTypePath(UClass* Class, FPropertyTypeName TypePath)
{
	if (const FName Name = TypePath.GetName(); !Name.IsNone())
	{
		TStringBuilder<256> Path;
		if (int32 ParamCount = TypePath.GetParameterCount())
		{
			Path << TypePath.GetParameterName(0) << TEXT('.');
			for (int32 OuterIndex = 1; OuterIndex < ParamCount; ++OuterIndex)
			{
				Path << TypePath.GetParameterName(OuterIndex) << (OuterIndex == 1 ? TEXT(':') : TEXT('.'));
			}
		}

		const int32 NameIndex = Path.Len();
		Path << Name;

		if (NameIndex)
		{
			if (UObject* Object = StaticFindObject(Class, (UObject*)nullptr, *Path))
			{
				return Object;
			}
		}

		if (UObject* Object = StaticFindFirstObject(Class, *Path + NameIndex, EFindFirstObjectOptions::NativeFirst))
		{
			UE_CLOGFMT(NameIndex, LogProperty, Warning, "{Class} '{Path}' was found by its short name but not its path.", Class->GetFName(), Path);
			return Object;
		}
	}
	return nullptr;
}

#if WITH_EDITORONLY_DATA
const FString* FindOriginalTypeName(const UField* Field)
{
	return Field ? Field->FindMetaData(NAME_OriginalType) : nullptr;
}

const FString* FindOriginalTypeName(const FProperty* Property)
{
	// Prioritize metadata on the property over metadata on the type.
	if (const FString* OriginalType = Property->FindMetaData(NAME_OriginalType))
	{
		return OriginalType;
	}

	// Search the owner chain to support metadata defined in UPROPERTY on a container for testing purposes.
	for (FField* OwnerField = Property->Owner.ToField(); OwnerField; OwnerField = OwnerField->Owner.ToField())
	{
		if (const FString* OriginalType = OwnerField->FindMetaData(NAME_OriginalType))
		{
			return OriginalType;
		}
	}

	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		return FindOriginalTypeName(StructProperty->Struct);
	}
	if (const FClassProperty* ClassProperty = CastField<FClassProperty>(Property))
	{
		return FindOriginalTypeName(ClassProperty->PropertyClass);
	}
	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		return FindOriginalTypeName(EnumProperty->GetEnum());
	}
	if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		return FindOriginalTypeName(ByteProperty->Enum);
	}
	return nullptr;
}

FPropertyTypeName FindOriginalType(const UField* Field)
{
	if (const FString* OriginalType = FindOriginalTypeName(Field))
	{
		if (UE::FPropertyTypeNameBuilder Type; Type.TryParse(*OriginalType))
		{
			return Type.Build();
		}
	}
	return {};
}

FPropertyTypeName FindOriginalType(const FProperty* Property)
{
	if (const FString* OriginalType = FindOriginalTypeName(Property))
	{
		if (UE::FPropertyTypeNameBuilder Type; Type.TryParse(*OriginalType))
		{
			return Type.Build();
		}
	}
	return {};
}
#endif // WITH_EDITORONLY_DATA

static FCoreRedirectObjectName BuildCoreRedirectObjectName(FPropertyTypeName TypeName)
{
	FName OuterName;
	if (const int32 OuterCount = TypeName.GetParameterCount() - 1; OuterCount <= 1)
	{
		OuterName = TypeName.GetParameterName(1);
	}
	else
	{
		TStringBuilder<256> OuterChain;
		for (int32 OuterIndex = 0; OuterIndex < OuterCount; ++OuterIndex)
		{
			OuterChain << TypeName.GetParameterName(OuterIndex + 1) << (OuterIndex == 0 ? TEXT(':') : TEXT('.'));
		}
		OuterChain.RemoveSuffix(1);
		OuterName = FName(OuterChain);
	}

	const FName ObjectName = TypeName.GetName();
	const FName PackageName = TypeName.GetParameterName(0);
	checkfSlow(PackageName.IsNone() || String::FindFirstChar(WriteToString<256>(PackageName), TEXT('/')) != INDEX_NONE,
		TEXT("PropertyTypeName with an outer name but not a package name is not supported yet."));
	return FCoreRedirectObjectName(ObjectName, OuterName, PackageName);
}

static void BuildPropertyName(const FCoreRedirectObjectName& Redirect, FPropertyTypeNameBuilder& Builder)
{
	Builder.AddName(Redirect.ObjectName);
	if (!Redirect.PackageName.IsNone() || !Redirect.OuterName.IsNone())
	{
		Builder.BeginParameters();
		if (!Redirect.PackageName.IsNone())
		{
			Builder.AddName(Redirect.PackageName);
		}
		if (!Redirect.OuterName.IsNone())
		{
			TStringBuilder<256> OuterChain(InPlace, Redirect.OuterName);
			String::ParseTokensMultiple(OuterChain, {TEXT('.'), TEXT(':')}, [&Builder](FStringView Token)
			{
				Builder.AddName(FName(Token));
			});
		}
		Builder.EndParameters();
	}
}

template <typename PropertyType>
struct TFindRedirectForPropertyTraits;

template <>
struct TFindRedirectForPropertyTraits<FStructProperty>
{
	static const UField* GetField(const FStructProperty* Property) { return Property->Struct; }
	static ECoreRedirectFlags GetFlags() { return ECoreRedirectFlags::Type_Struct; }
};

template <>
struct TFindRedirectForPropertyTraits<FByteProperty>
{
	static const UField* GetField(const FByteProperty* Property) { return Property->Enum; }
	static ECoreRedirectFlags GetFlags() { return ECoreRedirectFlags::Type_Enum; }
};

template <>
struct TFindRedirectForPropertyTraits<FEnumProperty>
{
	static const UField* GetField(const FEnumProperty* Property) { return Property->GetEnum(); }
	static ECoreRedirectFlags GetFlags() { return ECoreRedirectFlags::Type_Enum; }
};

template <typename PropertyType>
static bool FindRedirectForProperty(FPropertyTypeName OldType, FPropertyTypeNameBuilder& NewType, const PropertyType* Property)
{
	using FTraits = TFindRedirectForPropertyTraits<PropertyType>;
	const UField* Field = Property ? FTraits::GetField(Property) : nullptr;

	// If the type in the tag matches the field then skip looking for redirects.
	// This is necessary to handle redirects where the old name continues to be used.
	const FCoreRedirectObjectName OldNameRedirect = BuildCoreRedirectObjectName(OldType);
	if (Field && Field->GetFName() == OldNameRedirect.ObjectName)
	{
		return false;
	}

	// Look for a partial match if there is a field to compare against, otherwise require a complete match.
	FCoreRedirectObjectName NewNameRedirect = FCoreRedirects::GetRedirectedName(FTraits::GetFlags(), OldNameRedirect,
		Field ? ECoreRedirectMatchFlags::AllowPartialMatch : ECoreRedirectMatchFlags::None);
	if (NewNameRedirect == OldNameRedirect)
	{
		return false;
	}

	// If a partial match does not match the field then repeat the lookup without allowing partial matches.
	// This is necessary to handle redirects of the original type of a property that changed type.
	if (Field)
	{
		FName FieldName = Field->GetFName();

	#if WITH_EDITORONLY_DATA
		// Compare against the original type of an impersonated type.
		if (const FPropertyTypeName OriginalType = FindOriginalType(Property); !OriginalType.IsEmpty())
		{
			FieldName = OriginalType.GetName();
			if (FieldName == OldNameRedirect.ObjectName)
			{
				return false;
			}
		}
	#endif // WITH_EDITORONLY_DATA

		if (FieldName != NewNameRedirect.ObjectName)
		{
			NewNameRedirect = FCoreRedirects::GetRedirectedName(FTraits::GetFlags(), OldNameRedirect, ECoreRedirectMatchFlags::None);
			if (OldNameRedirect == NewNameRedirect)
			{
				return false;
			}
		}
	}

	BuildPropertyName(NewNameRedirect, NewType);
	return true;
}

static void BuildNewNameForPropertyType(FPropertyTypeName OldTypeName, FPropertyTypeNameBuilder& NewTypeName, const FProperty* Property, bool& bHasRedirect)
{
	const FName OldName = OldTypeName.GetName();
	NewTypeName.AddName(OldName);

	const int32 OldParamCount = OldTypeName.GetParameterCount();
	if (OldParamCount == 0)
	{
		return;
	}

	int32 OldParamIndex = 0;
	NewTypeName.BeginParameters();

	if (const EName* OldEName = OldName.ToEName(); OldEName && OldName.GetNumber() == NAME_NO_NUMBER_INTERNAL)
	{
		switch (*OldEName)
		{
		case NAME_StructProperty:
		{
			if (FindRedirectForProperty(OldTypeName.GetParameter(OldParamIndex), NewTypeName, CastField<FStructProperty>(Property)))
			{
				++OldParamIndex;
				bHasRedirect = true;
			}
			break;
		}
		case NAME_ByteProperty:
		{
			if (FindRedirectForProperty(OldTypeName.GetParameter(OldParamIndex), NewTypeName, CastField<FByteProperty>(Property)))
			{
				++OldParamIndex;
				bHasRedirect = true;
			}
			break;
		}
		case NAME_EnumProperty:
		{
			if (FindRedirectForProperty(OldTypeName.GetParameter(OldParamIndex), NewTypeName, CastField<FEnumProperty>(Property)))
			{
				++OldParamIndex;
				bHasRedirect = true;
			}
			break;
		}
		case NAME_ArrayProperty:
		{
			const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property);
			const FProperty* Inner = ArrayProperty ? ArrayProperty->Inner : nullptr;
			BuildNewNameForPropertyType(OldTypeName.GetParameter(OldParamIndex++), NewTypeName, Inner, bHasRedirect);
			break;
		}
		case NAME_OptionalProperty:
		{
			const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property);
			const FProperty* Value = OptionalProperty ? OptionalProperty->GetValueProperty() : nullptr;
			BuildNewNameForPropertyType(OldTypeName.GetParameter(OldParamIndex++), NewTypeName, Value, bHasRedirect);
			break;
		}
		case NAME_SetProperty:
		{
			const FSetProperty* SetProperty = CastField<FSetProperty>(Property);
			const FProperty* Element = SetProperty ? SetProperty->ElementProp : nullptr;
			BuildNewNameForPropertyType(OldTypeName.GetParameter(OldParamIndex++), NewTypeName, Element, bHasRedirect);
			break;
		}
		case NAME_MapProperty:
		{
			const FMapProperty* MapProperty = CastField<FMapProperty>(Property);
			const FProperty* Key = MapProperty ? MapProperty->KeyProp : nullptr;
			const FProperty* Value = MapProperty ? MapProperty->ValueProp : nullptr;
			BuildNewNameForPropertyType(OldTypeName.GetParameter(OldParamIndex++), NewTypeName, Key, bHasRedirect);
			BuildNewNameForPropertyType(OldTypeName.GetParameter(OldParamIndex++), NewTypeName, Value, bHasRedirect);
			break;
		}
		default:
			break;
		}
	}

	for (; OldParamIndex < OldParamCount; ++OldParamIndex)
	{
		BuildNewNameForPropertyType(OldTypeName.GetParameter(OldParamIndex), NewTypeName, nullptr, bHasRedirect);
	}
	NewTypeName.EndParameters();
}

FPropertyTypeName ApplyRedirectsToPropertyType(FPropertyTypeName OldTypeName, const FProperty* Property)
{
	bool bHasRedirect = false;
	FPropertyTypeNameBuilder NewTypeName;
	BuildNewNameForPropertyType(OldTypeName, NewTypeName, Property, bHasRedirect);
	return bHasRedirect ? NewTypeName.Build() : FPropertyTypeName();
}

} // UE
