// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/PropertyPathName.h"
#include "UObject/PropertyTypeName.h"
#include "UObject/ScriptDelegateFwd.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/WeakObjectPtrFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "PropertyHelper.generated.h"

class FProperty;
class UField;
class UFunction;
class UObject;

DECLARE_LOG_CATEGORY_EXTERN(LogEnum, Log, All);
DECLARE_LOG_CATEGORY_EXTERN(LogProperty, Log, All);

UENUM()
enum class EFallbackEnum : uint8
{
};

/**
 * Advances the character pointer past any spaces or tabs.
 * 
 * @param	Str		the buffer to remove whitespace from
 */
COREUOBJECT_API void SkipWhitespace(const TCHAR*& Str);

/**
 * Determine whether the editable properties of CompA and CompB are identical. Used
 * to determine whether the instanced object has been modified in the editor.
 * 
 * @param	ObjectA		the first Instanced Object to compare
 * @param	ObjectB		the second Instanced Object to compare
 * @param	PortFlags	flags for modifying the criteria used for determining whether the components are identical
 *
 * @return	true if the values of all of the editable properties of CompA match the values in CompB
 */
COREUOBJECT_API bool AreInstancedObjectsIdentical( UObject* ObjectA, UObject* ObjectB, uint32 PortFlags );

/** Helper struct for serializing index deltas
 *		-Serialize delta index as a packed int (Hope to get 1 byte per index)
 *		-Serialize 0 delta to signify 'no more' (INDEX_NONE would take 5 bytes in packed format)
 */
struct FDeltaIndexHelper
{
	FDeltaIndexHelper() = default;

	/** Serialize Index as delta from previous index. Return false if we should stop */
	COREUOBJECT_API bool SerializeNext(FArchive &Ar, int32& Index);

	/** Helper for NetSerializeItemDelta which has full/partial/old archives. Wont auto-advance LastIndex, must call Increment after this */
	COREUOBJECT_API void SerializeNext(FArchive &OutBunch, FArchive &OutFull, int32 Index);

	/** Sets LastIndex for delta state. Must be called if using SerializeNet(OutBunch, Outfull, Old, Index) */
	void Increment(int32 NewIndex)
	{
		LastIndex = NewIndex;
	}

	/** Serialize early end (0) */
	COREUOBJECT_API void SerializeEarlyEnd(FArchive &Ar);

	int32 LastIndex     = -1; // Start at -1 so index 0 can be serialized as delta=1 (so that 0 can be reserved for 'no more')
	int32 LastIndexFull = -1; // Separate index for full state since it will never be rolled back
};

namespace DelegatePropertyTools
{
	/**
	 * Imports a single-cast delegate as "object.function", or "function" (self is object) from a text buffer
	 *
	 * @param	Delegate			The delegate to import data into
	 * @param	SignatureFunction	The delegate property's signature function, used to validate parameters and such
	 * @param	Buffer				The text buffer to import from
	 * @param	Parent				The property object's parent object
	 * @param	ErrorText			Output device for emitting errors and warnings
	 *
	 * @return	The adjusted text buffer pointer on success, or NULL on failure
	 */
	COREUOBJECT_API const TCHAR* ImportDelegateFromText( FScriptDelegate& Delegate, const UFunction* SignatureFunction, const TCHAR* Buffer, UObject* Parent, FOutputDevice* ErrorText );
}

namespace UE
{

UObject* FindObjectByTypePath(UClass* Class, FPropertyTypeName TypePath);

template <typename T>
T* FindObjectByTypePath(FPropertyTypeName TypePath)
{
	return (T*)FindObjectByTypePath(T::StaticClass(), TypePath);
}

#if WITH_EDITORONLY_DATA
inline const FName NAME_OriginalType(ANSITEXTVIEW("OriginalType"));
const FString* FindOriginalTypeName(const UField* Field);
const FString* FindOriginalTypeName(const FProperty* Property);
FPropertyTypeName FindOriginalType(const UField* Field);
FPropertyTypeName FindOriginalType(const FProperty* Property);
#endif // WITH_EDITORONLY_DATA

/**
 * Applies core redirects to type names and paths within the property type.
 *
 * Allows partial matches only when a property is given and matches the layout of the type name.
 * Example: StructProperty(OldName) will not have an OldName redirect applied if Property is an FIntProperty.
 *
 * @param OldTypeName   The type name to apply core redirects to.
 * @param Property      Optional. The property that the type name might correspond to.
 * @return The new type name, if redirects were applied, otherwise an empty type name.
 */
FPropertyTypeName ApplyRedirectsToPropertyType(FPropertyTypeName OldTypeName, const FProperty* Property = nullptr);

/**
 * Pushes a segment to SerializedPropertyPath for the lifetime of this object if path tracking is active.
 */
class FSerializedPropertyPathScope
{
public:
	[[nodiscard]] FSerializedPropertyPathScope(FUObjectSerializeContext* InContext, const FPropertyPathNameSegment& InSegment)
	{
	#if WITH_EDITORONLY_DATA
		if (InContext->bTrackSerializedPropertyPath)
		{
			Context = InContext;
			Context->SerializedPropertyPath.Push(InSegment);
		}
	#endif
	}

	~FSerializedPropertyPathScope()
	{
	#if WITH_EDITORONLY_DATA
		if (Context)
		{
			Context->SerializedPropertyPath.Pop();
		}
	#endif
	}

	FSerializedPropertyPathScope(const FSerializedPropertyPathScope&) = delete;
	FSerializedPropertyPathScope& operator=(const FSerializedPropertyPathScope&) = delete;

private:
	FUObjectSerializeContext* Context = nullptr;
};

} // UE
