// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	FIntProperty.
-----------------------------------------------------------------------------*/

FIntProperty::FIntProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FIntProperty::FIntProperty(FFieldVariant InOwner, const UECodeGen_Private::FIntPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
FIntProperty::FIntProperty(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA
