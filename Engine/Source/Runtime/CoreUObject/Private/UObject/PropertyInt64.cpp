// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	FInt64Property.
-----------------------------------------------------------------------------*/

FInt64Property::FInt64Property(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FInt64Property::FInt64Property(FFieldVariant InOwner, const UECodeGen_Private::FInt64PropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
FInt64Property::FInt64Property(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA
