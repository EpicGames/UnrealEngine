// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	FInt8Property.
-----------------------------------------------------------------------------*/

FInt8Property::FInt8Property(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FInt8Property::FInt8Property(FFieldVariant InOwner, const UECodeGen_Private::FInt8PropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
FInt8Property::FInt8Property(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA
