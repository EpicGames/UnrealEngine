// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "Templates/Casts.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "Hash/Blake3.h"
#include "UObject/CoreNet.h"
#include "Misc/EngineNetworkCustomVersion.h"

/*-----------------------------------------------------------------------------
	FInterfaceProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FInterfaceProperty)

FInterfaceProperty::FInterfaceProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, InterfaceClass(nullptr)
{
}

FInterfaceProperty::FInterfaceProperty(FFieldVariant InOwner, const UECodeGen_Private::FInterfacePropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
	this->PropertyFlags &= (~CPF_InterfaceClearMask);
	this->InterfaceClass = Prop.InterfaceClassFunc ? Prop.InterfaceClassFunc(ETypeConstructPhase::Inner) : nullptr;
}

#if WITH_EDITORONLY_DATA
FInterfaceProperty::FInterfaceProperty(UField* InField)
	: Super(InField)
{
	UInterfaceProperty* SourceProperty = CastChecked<UInterfaceProperty>(InField);
	InterfaceClass = SourceProperty->InterfaceClass;
}
#endif // WITH_EDITORONLY_DATA

void FInterfaceProperty::PostDuplicate(const FField& InField)
{
	const FInterfaceProperty& Source = static_cast<const FInterfaceProperty&>(InField);
	InterfaceClass = Source.InterfaceClass;
	Super::PostDuplicate(InField);
}

/**
 * Returns the text to use for exporting this property to header file.
 *
 * @param	ExtendedTypeText	for property types which use templates, will be filled in with the type
 * @param	CPPExportFlags		flags for modifying the behavior of the export
 */
FString FInterfaceProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	checkSlow(InterfaceClass);

	UClass* ExportClass = InterfaceClass;
	while ( ExportClass && !ExportClass->HasAnyClassFlags(CLASS_Native) )
	{
		ExportClass = ExportClass->GetSuperClass();
	}
	check(ExportClass);
	check(ExportClass->HasAnyClassFlags(CLASS_Interface));

	ExtendedTypeText = FString::Printf(TEXT("I%s"), *ExportClass->GetName());
	return TEXT("TINTERFACE");
}

/**
 * Returns the text to use for exporting this property to header file.
 *
 * @param	ExtendedTypeText	for property types which use templates, will be filled in with the type
 * @param	CPPExportFlags		flags for modifying the behavior of the export
 */
FString FInterfaceProperty::GetCPPType( FString* ExtendedTypeText/*=NULL*/, uint32 CPPExportFlags/*=0*/ ) const
{
	checkSlow(InterfaceClass);

	if ( ExtendedTypeText != NULL )
	{
		UClass* ExportClass = InterfaceClass;
		if (0 == (CPPF_BlueprintCppBackend & CPPExportFlags))
		{
			while (ExportClass && !ExportClass->HasAnyClassFlags(CLASS_Native))
			{
				ExportClass = ExportClass->GetSuperClass();
			}
		}
		check(ExportClass);
		check(ExportClass->HasAnyClassFlags(CLASS_Interface) || 0 != (CPPF_BlueprintCppBackend & CPPExportFlags));

		*ExtendedTypeText = FString::Printf(TEXT("<I%s>"), *ExportClass->GetName());
	}

	return TEXT("TScriptInterface");
}

void FInterfaceProperty::LinkInternal(FArchive& Ar)
{
	// for now, we won't support instancing of interface properties...it might be possible, but for the first pass we'll keep it simple
	PropertyFlags &= ~CPF_InterfaceClearMask;
	Super::LinkInternal(Ar);
}

bool FInterfaceProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags/* = 0*/) const
{
	FScriptInterface* InterfaceA = (FScriptInterface*)NotNullGet(A);
	FScriptInterface* InterfaceB = (FScriptInterface*)B;

	if (!InterfaceB)
	{
		return InterfaceA->GetObject() == nullptr;
	}

	return (InterfaceA->GetObject() == InterfaceB->GetObject() && InterfaceA->GetInterface() == InterfaceB->GetInterface());
}

void FInterfaceProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FScriptInterface* InterfaceValue = (FScriptInterface*)NotNullGet(Value);

	Slot << InterfaceValue->GetObjectRef();
	if (UnderlyingArchive.IsLoading() || UnderlyingArchive.IsTransacting() || UnderlyingArchive.IsObjectReferenceCollector())
	{
		if ( InterfaceValue->GetObject() != NULL )
		{
			InterfaceValue->SetInterface(InterfaceValue->GetObject()->GetInterfaceAddress(InterfaceClass));
		}
		else
		{
			InterfaceValue->SetInterface(NULL);
		}
	}
}

bool FInterfaceProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData) const
{
	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);

	if (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::InterfacePropertySerialization)
	{ 
		FScriptInterface* InterfaceValue = (FScriptInterface*)NotNullGet(Data);
		bool Result = Map->SerializeObject(Ar, InterfaceClass, MutableView(InterfaceValue->GetObjectRef()));
		if (Ar.IsLoading())
		{
			if (InterfaceValue->GetObject() != nullptr)
			{
				InterfaceValue->SetInterface(InterfaceValue->GetObject()->GetInterfaceAddress(InterfaceClass));
			}
			else
			{
				InterfaceValue->SetInterface(nullptr);
			}
		}
		return Result;
	}
	return false;
}

void FInterfaceProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	UObject* Temp = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		FScriptInterface InterfaceValue;
		GetValue_InContainer(PropertyValueOrContainer, &InterfaceValue);
		Temp = InterfaceValue.GetObject();
	}
	else
	{
		FScriptInterface* InterfaceValue = (FScriptInterface*)PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType);
		Temp = InterfaceValue->GetObject();
	}

	if( Temp != NULL )
	{
		bool bExportFullyQualified = true;

		// When exporting from one package or graph to another package or graph, we don't want to fully qualify the name, as it may refer
		// to a level or graph that doesn't exist or cause a linkage to a node in a different graph
		UObject* StopOuter = NULL;
		if (PortFlags & PPF_ExportsNotFullyQualified)
		{
			StopOuter = (ExportRootScope || (Parent == NULL)) ? ExportRootScope : Parent->GetOutermost();
			bExportFullyQualified = !Temp->IsIn(StopOuter);
		}

		// if we want a full qualified object reference, use the pathname, otherwise, use just the object name
		if (bExportFullyQualified)
		{
			StopOuter = NULL;
			if ( (PortFlags&PPF_SimpleObjectText) != 0 && Parent != NULL )
			{
				StopOuter = Parent->GetOutermost();
			}
		}
		
		ValueStr += FString::Printf( TEXT("%s'%s'"), *Temp->GetClass()->GetName(), *Temp->GetPathName(StopOuter) );
	}
	else
	{
		ValueStr += TEXT("None");
	}
}

const TCHAR* FInterfaceProperty::ImportText_Internal(const TCHAR* InBuffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	FScriptInterface* InterfaceValue = (FScriptInterface*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
	TObjectPtr<UObject> ResolvedObject = InterfaceValue->GetObject();

	auto SetInterfaceValue = [InterfaceValue, ContainerOrPropertyPtr, PropertyPointerType, this](UObject* NewObject, void* NewInterfaceAddress)
	{
		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			FScriptInterface LocalInterfaceValue;
			LocalInterfaceValue.SetObject(NewObject);
			LocalInterfaceValue.SetInterface(NewInterfaceAddress);
			SetValue_InContainer(ContainerOrPropertyPtr, LocalInterfaceValue);
		}
		else
		{
			InterfaceValue->SetObject(NewObject);
			if (NewObject)
			{
				// if NewObject was set to nullptr, SetObject will take care of clearing the interface address too
				InterfaceValue->SetInterface(NewInterfaceAddress);
			}
		}
	};

	const TCHAR* Buffer = InBuffer;
	if (!FObjectPropertyBase::ParseObjectPropertyValue(this, Parent, UObject::StaticClass(), PortFlags, Buffer, ResolvedObject))
	{
		SetInterfaceValue(ResolvedObject, InterfaceValue->GetInterface());
		return nullptr;
	}

	// so we should now have a valid object
	if (ResolvedObject == nullptr)
	{
		// if ParseObjectPropertyValue returned true but ResolvedObject is nullptr, the imported text was "None".  Make sure the interface pointer
		// is cleared, then stop
		SetInterfaceValue(nullptr, nullptr);
		return Buffer;
	}

	void* NewInterfaceAddress = ResolvedObject->GetInterfaceAddress(InterfaceClass);
	if (NewInterfaceAddress == nullptr)
	{
		// If this is a bp implementation of a native interface, set object but clear the interface
		if (ResolvedObject->GetClass()->ImplementsInterface(InterfaceClass))
		{
			SetInterfaceValue(ResolvedObject, nullptr);
			return Buffer;
		}

		// the object we imported doesn't implement our interface class
		ErrorText->Logf( TEXT("%s: specified object doesn't implement the required interface class '%s': %s"),
						*GetFullName(), *InterfaceClass->GetName(), InBuffer );

		return nullptr;
	}

	SetInterfaceValue(ResolvedObject, NewInterfaceAddress);
	return Buffer;
}

bool FInterfaceProperty::ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType /*= EPropertyObjectReferenceType::Strong*/) const
{
	return !!(InReferenceType & EPropertyObjectReferenceType::Strong);
}

/** Manipulates the data referenced by this FProperty */
void FInterfaceProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	Ar << InterfaceClass;

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(InterfaceClass))
		{
			PlaceholderClass->AddReferencingProperty(this);
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	if ( !InterfaceClass )
 	{
		// If we failed to load the InterfaceClass and we're not a CDO, that means we relied on a class that has been removed or doesn't exist.
		// The most likely cause for this is either an incomplete recompile, or if content was migrated between games that had native class dependencies
		// that do not exist in this game.  We allow blueprint classes to continue, because compile-on-load will error out, and stub the class that was using it
		UClass* TestClass = dynamic_cast<UClass*>(GetOwnerStruct());
		if( TestClass && TestClass->HasAllClassFlags(CLASS_Native) && !TestClass->HasAllClassFlags(CLASS_NewerVersionExists) && (TestClass->GetOutermost() != GetTransientPackage()) )
		{
			checkf(false, TEXT("Interface property tried to serialize a missing interface.  Did you remove a native class and not fully recompile?"));
		}
 	}
}


#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
void FInterfaceProperty::SetInterfaceClass(UClass* NewInterfaceClass)
{
	if (ULinkerPlaceholderClass* NewPlaceholderClass = Cast<ULinkerPlaceholderClass>(NewInterfaceClass))
	{
		NewPlaceholderClass->AddReferencingProperty(this);
	}

	if (ULinkerPlaceholderClass* OldPlaceholderClass = Cast<ULinkerPlaceholderClass>(InterfaceClass))
	{
		OldPlaceholderClass->RemoveReferencingProperty(this);
	}
	InterfaceClass = NewInterfaceClass;
}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

bool FInterfaceProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && (InterfaceClass == ((FInterfaceProperty*)Other)->InterfaceClass);
}

#if WITH_EDITORONLY_DATA
void FInterfaceProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (InterfaceClass)
	{
		// Hash the class's name instead of recursively hashing the class; the class's schema does not impact how we serialize our pointer to it
		FNameBuilder ObjectPath;
		InterfaceClass->GetPathName(nullptr, ObjectPath);
		Builder.Update(ObjectPath.GetData(), ObjectPath.Len() * sizeof(ObjectPath.GetData()[0]));
	}
}
#endif

void FInterfaceProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(InterfaceClass);
	Super::AddReferencedObjects(Collector);
}
