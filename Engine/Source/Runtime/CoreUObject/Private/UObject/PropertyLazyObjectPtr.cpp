// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyHelper.h"

/*-----------------------------------------------------------------------------
	FLazyObjectProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FLazyObjectProperty)

FLazyObjectProperty::FLazyObjectProperty(FFieldVariant InOwner, const FName& InName)
	: TFObjectPropertyBase(InOwner, InName)
{
}

FLazyObjectProperty::FLazyObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FLazyObjectPropertyParams& Prop)
	: Super(InOwner, Prop)
{
}

#if WITH_EDITORONLY_DATA
FLazyObjectProperty::FLazyObjectProperty(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA

FString FLazyObjectProperty::GetCPPType(FString* ExtendedTypeText/*=NULL*/, uint32 CPPExportFlags/*=0*/) const
{
	check(PropertyClass);
	return FString::Printf(TEXT("TLazyObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
}

FString FLazyObjectProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	ExtendedTypeText = FString::Printf(TEXT("TLazyObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
	return TEXT("LAZYOBJECT");
}

FName FLazyObjectProperty::GetID() const
{
	return NAME_LazyObjectProperty;
}

void FLazyObjectProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	// We never serialize our reference while the garbage collector is harvesting references
	// to objects, because we don't want lazy pointers to keep objects from being garbage collected

	if( !UnderlyingArchive.IsObjectReferenceCollector() || UnderlyingArchive.IsModifyingWeakAndStrongReferences() )
	{
		UObject* ObjectValue = GetObjectPropertyValue(Value);

		Slot << *(FLazyObjectPtr*)NotNullGet(Value);

		if ((UnderlyingArchive.IsLoading() || UnderlyingArchive.IsModifyingWeakAndStrongReferences()) && ObjectValue != GetObjectPropertyValue(Value))
		{
			CheckValidObject(Value, ObjectValue);
		}
	}
	else
	{
		// TODO: This isn't correct, but it keeps binary serialization happy. We should ALWAYS be serializing the pointer
		// to the archive in this function, and allowing the underlying archive to ignore it if necessary
		Slot.EnterStream();
	}
}

bool FLazyObjectProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	FLazyObjectPtr ObjectA = *(FLazyObjectPtr*)NotNullGet(A);
	FLazyObjectPtr ObjectB = B ? *((FLazyObjectPtr*)B) : FLazyObjectPtr();

	// Compare actual pointers. We don't do this during PIE because we want to be sure to serialize everything. An example is the LevelScriptActor being serialized against its CDO,
	// which contains actor references. We want to serialize those references so they are fixed up.
	const bool bDuplicatingForPIE = (PortFlags&PPF_DuplicateForPIE) != 0;
	bool bResult = !bDuplicatingForPIE ? (ObjectA == ObjectB) : false;
	// always serialize the cross level references, because they could be NULL
	// @todo: okay, this is pretty hacky overall - we should have a PortFlag or something
	// that is set during SavePackage. Other times, we don't want to immediately return false
	// (instead of just this ExportDefProps case)
	// instance testing
	if (!bResult && ObjectA.IsValid() && ObjectB.IsValid() && ObjectA->GetClass() == ObjectB->GetClass())
	{
		bool bPerformDeepComparison = (PortFlags&PPF_DeepComparison) != 0;
		if ((PortFlags&PPF_DeepCompareInstances) && !bPerformDeepComparison)
		{
			bPerformDeepComparison = ObjectA->IsTemplate() != ObjectB->IsTemplate();
		}

		if (!bResult && bPerformDeepComparison)
		{
			// In order for deep comparison to be match they both need to have the same name and that name needs to be included in the instancing table for the class
			if (ObjectA->GetFName() == ObjectB->GetFName() && ObjectA->GetClass()->GetDefaultSubobjectByName(ObjectA->GetFName()))
			{
				checkSlow(ObjectA->IsDefaultSubobject() && ObjectB->IsDefaultSubobject() && ObjectA->GetClass()->GetDefaultSubobjectByName(ObjectA->GetFName()) == ObjectB->GetClass()->GetDefaultSubobjectByName(ObjectB->GetFName())); // equivalent
				bResult = AreInstancedObjectsIdentical(ObjectA.Get(), ObjectB.Get(), PortFlags);
			}
		}
	}
	return bResult;
}

UObject* FLazyObjectProperty::GetObjectPropertyValue(const void* PropertyValueAddress) const
{
	return GetPropertyValue(PropertyValueAddress).Get();
}

TObjectPtr<UObject> FLazyObjectProperty::GetObjectPtrPropertyValue(const void* PropertyValueAddress) const
{
	return TObjectPtr<UObject>(GetPropertyValue(PropertyValueAddress).Get());
}

UObject* FLazyObjectProperty::GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	UObject* Result = nullptr;
	GetWrappedUObjectPtrValues<FLazyObjectPtr>(&Result, ContainerAddress, EPropertyMemoryAccess::InContainer, ArrayIndex, 1);
	return Result;
}

TObjectPtr<UObject> FLazyObjectProperty::GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	TObjectPtr<UObject> Result = nullptr;
	GetWrappedUObjectPtrValues<FLazyObjectPtr>(&Result, ContainerAddress, EPropertyMemoryAccess::InContainer, ArrayIndex, 1);
	return Result;
}

void FLazyObjectProperty::SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const
{
	SetPropertyValue(PropertyValueAddress, TCppType(Value));
}

void FLazyObjectProperty::SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const
{
	SetPropertyValue(PropertyValueAddress, TCppType(Ptr));
}

void FLazyObjectProperty::SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex) const
{
	SetWrappedUObjectPtrValues<FLazyObjectPtr>(ContainerAddress, EPropertyMemoryAccess::InContainer, &Value, ArrayIndex, 1);
}

void FLazyObjectProperty::SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex) const
{
	SetWrappedUObjectPtrValues<FLazyObjectPtr>(ContainerAddress, EPropertyMemoryAccess::InContainer, &Ptr, ArrayIndex, 1);
}

bool FLazyObjectProperty::AllowCrossLevel() const
{
	return true;
}

uint32 FLazyObjectProperty::GetValueTypeHashInternal(const void* Src) const
{
	return GetTypeHash(GetPropertyValue(Src));
}

void FLazyObjectProperty::CopySingleValueToScriptVM( void* Dest, void const* Src ) const
{
	*(FObjectPtr*)Dest = ((const FLazyObjectPtr*)Src)->Get();
}

void FLazyObjectProperty::CopySingleValueFromScriptVM( void* Dest, void const* Src ) const
{
	*(FLazyObjectPtr*)Dest = *(UObject**)Src;
}

void FLazyObjectProperty::CopyCompleteValueToScriptVM( void* Dest, void const* Src ) const
{
	GetWrappedUObjectPtrValues<FLazyObjectPtr>((UObject**)Dest, Src, EPropertyMemoryAccess::Direct, 0, ArrayDim);
}

void FLazyObjectProperty::CopyCompleteValueFromScriptVM( void* Dest, void const* Src ) const
{
	SetWrappedUObjectPtrValues<FLazyObjectPtr>(Dest, EPropertyMemoryAccess::Direct, (UObject**)Src, 0, ArrayDim);
}

void FLazyObjectProperty::CopyCompleteValueToScriptVM_InContainer(void* OutValue, void const* InContainer) const
{
	GetWrappedUObjectPtrValues<FLazyObjectPtr>((UObject**)OutValue, InContainer, EPropertyMemoryAccess::InContainer, 0, ArrayDim);
}

void FLazyObjectProperty::CopyCompleteValueFromScriptVM_InContainer(void* OutContainer, void const* InValue) const
{
	SetWrappedUObjectPtrValues<FLazyObjectPtr>(OutContainer, EPropertyMemoryAccess::InContainer, (UObject**)InValue, 0, ArrayDim);
}
