// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "Misc/ScopeExit.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/PropertyHelper.h"
#include "UObject/PropertyPathFunctions.h"
#include "UObject/PropertyPathName.h"
#include "UObject/PropertyTag.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/CommonFNames.h"
#include "UObject/InstanceDataObjectUtils.h"

namespace UEMapProperty_Private
{
	/**
	 * Checks if any of the pairs in the map compare equal to the one passed.
	 *
	 * @param  MapHelper    The map to search through.
	 * @param  LogicalIndex The index in the map to start searching from.
	 * @param  Num          The number of elements to compare.
	 */
	bool AnyEqual(const FScriptMapHelper& MapHelper, const int32 LogicalIndex, int32 Num, const uint8* PairToCompare, const uint32 PortFlags)
	{
		const FProperty* KeyProp   = MapHelper.GetKeyProperty();
		const FProperty* ValueProp = MapHelper.GetValueProperty();
		const int32 ValueOffset = MapHelper.MapLayout.ValueOffset;

		FScriptMapHelper::FIterator IteratorA(MapHelper, LogicalIndex);
		for (; IteratorA && Num; --Num, ++IteratorA)
		{
			if (KeyProp->Identical(MapHelper.GetPairPtr(IteratorA), PairToCompare, PortFlags) && ValueProp->Identical(MapHelper.GetPairPtr(IteratorA) + ValueOffset, PairToCompare + ValueOffset, PortFlags))
			{
				return true;
			}
		}

		return false;
	}

	bool RangesContainSameAmountsOfVal(const FScriptMapHelper& MapHelperA, const int32 LogicalIndexA, const FScriptMapHelper& MapHelperB, const int32 LogicalIndexB, int32 Num, const uint8* PairToCompare, const uint32 PortFlags)
	{
		const FProperty* KeyProp   = MapHelperA.GetKeyProperty();
		const FProperty* ValueProp = MapHelperA.GetValueProperty();

		// Ensure that both maps are the same type
		check(KeyProp   == MapHelperB.GetKeyProperty());
		check(ValueProp == MapHelperB.GetValueProperty());

		const int32 ValueOffset = MapHelperA.MapLayout.ValueOffset;

		FScriptMapHelper::FIterator IteratorA(MapHelperA, LogicalIndexA);
		FScriptMapHelper::FIterator IteratorB(MapHelperB, LogicalIndexB);

		int32 CountA = 0;
		int32 CountB = 0;
		for (;;)
		{
			if (Num == 0)
			{
				return CountA == CountB;
			}

			const uint8* PairA = MapHelperA.GetPairPtr(IteratorA);
			const uint8* PairB = MapHelperB.GetPairPtr(IteratorB);
			if (PairA == PairToCompare || (KeyProp->Identical(PairA, PairToCompare, PortFlags) && ValueProp->Identical(PairA + ValueOffset, PairToCompare + ValueOffset, PortFlags)))
			{
				++CountA;
			}

			if (PairB == PairToCompare || (KeyProp->Identical(PairB, PairToCompare, PortFlags) && ValueProp->Identical(PairB + ValueOffset, PairToCompare + ValueOffset, PortFlags)))
			{
				++CountB;
			}

			++IteratorA;
			++IteratorB;
			--Num;
		}
	}

	bool IsPermutation(const FScriptMapHelper& MapHelperA, const FScriptMapHelper& MapHelperB, const uint32 PortFlags)
	{
		const FProperty* KeyProp   = MapHelperA.GetKeyProperty();
		const FProperty* ValueProp = MapHelperA.GetValueProperty();

		// Ensure that both maps are the same type
		check(KeyProp   == MapHelperB.GetKeyProperty());
		check(ValueProp == MapHelperB.GetValueProperty());

		int32 Num = MapHelperA.Num();
		if (Num != MapHelperB.Num())
		{
			return false;
		}

		const int32 ValueOffset = MapHelperA.MapLayout.ValueOffset;

		// Skip over common initial sequence
		FScriptMapHelper::FIterator IteratorA(MapHelperA);
		FScriptMapHelper::FIterator IteratorB(MapHelperB);
		for (;;)
		{
			if (Num == 0)
			{
				return true;
			}

			const uint8* PairA = MapHelperA.GetPairPtr(IteratorA);
			const uint8* PairB = MapHelperB.GetPairPtr(IteratorB);
			if (!KeyProp->Identical(PairA, PairB, PortFlags))
			{
				break;
			}

			if (!ValueProp->Identical(PairA + ValueOffset, PairB + ValueOffset, PortFlags))
			{
				break;
			}

			++IteratorA;
			++IteratorB;
			--Num;
		}

		const int32 FirstIndexA = IteratorA.GetLogicalIndex();
		const int32 FirstIndexB = IteratorB.GetLogicalIndex();
		const int32 FirstNum    = Num;
		for (;;)
		{
			const uint8* PairA = MapHelperA.GetPairPtr(IteratorA);
			if (!AnyEqual(MapHelperA, FirstIndexA, FirstNum - Num, PairA, PortFlags) && !RangesContainSameAmountsOfVal(MapHelperA, FirstIndexA, MapHelperB, FirstIndexB, FirstNum, PairA, PortFlags))
			{
				return false;
			}

			--Num;
			if (Num == 0)
			{
				return true;
			}

			++IteratorA;
		}
	}
}

IMPLEMENT_FIELD(FMapProperty)

FMapProperty::FMapProperty(FFieldVariant InOwner, const FName& InName, EMapPropertyFlags InMapFlags)
	: Super(InOwner, InName)
{
	// These are expected to be set post-construction by AddCppProperty
	KeyProp = nullptr;
	ValueProp = nullptr;

	MapFlags = InMapFlags;
}

FMapProperty::FMapProperty(FFieldVariant InOwner, const UECodeGen_Private::FMapPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
	// These are expected to be set post-construction by AddCppProperty
	KeyProp = nullptr;
	ValueProp = nullptr;

	MapFlags = Prop.MapFlags;
}

#if WITH_EDITORONLY_DATA
FMapProperty::FMapProperty(UField* InField)
	: Super(InField)
	, MapFlags(EMapPropertyFlags::None)
{
	UMapProperty* SourceProperty = CastChecked<UMapProperty>(InField);
	MapLayout = SourceProperty->MapLayout;

	KeyProp = CastField<FProperty>(SourceProperty->KeyProp->GetAssociatedFField());
	if (!KeyProp)
	{
		KeyProp = CastField<FProperty>(CreateFromUField(SourceProperty->KeyProp));
		SourceProperty->KeyProp->SetAssociatedFField(KeyProp);
	}

	ValueProp = CastField<FProperty>(SourceProperty->ValueProp->GetAssociatedFField());
	if (!ValueProp)
	{
		ValueProp = CastField<FProperty>(CreateFromUField(SourceProperty->ValueProp));
		SourceProperty->ValueProp->SetAssociatedFField(ValueProp);
	}
}
#endif // WITH_EDITORONLY_DATA

FMapProperty::~FMapProperty()
{
	delete KeyProp;
	KeyProp = nullptr;
	delete ValueProp;
	ValueProp = nullptr;
}

void FMapProperty::PostDuplicate(const FField& InField)
{
	const FMapProperty& Source = static_cast<const FMapProperty&>(InField);
	KeyProp = CastFieldChecked<FProperty>(FField::Duplicate(Source.KeyProp, this));
	ValueProp = CastFieldChecked<FProperty>(FField::Duplicate(Source.ValueProp, this));
	MapLayout = Source.MapLayout;
	Super::PostDuplicate(InField);
}

void FMapProperty::LinkInternal(FArchive& Ar)
{
	check(KeyProp && ValueProp);

	KeyProp  ->Link(Ar);
	ValueProp->Link(Ar);

	int32 KeySize        = KeyProp  ->GetSize();
	int32 ValueSize      = ValueProp->GetSize();
	int32 KeyAlignment   = KeyProp  ->GetMinAlignment();
	int32 ValueAlignment = ValueProp->GetMinAlignment();

	MapLayout = FScriptMap::GetScriptLayout(KeySize, KeyAlignment, ValueSize, ValueAlignment);

	ValueProp->SetOffset_Internal(MapLayout.ValueOffset);

	Super::LinkInternal(Ar);
}

bool FMapProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	checkSlow(KeyProp);
	checkSlow(ValueProp);

	FScriptMapHelper MapHelperA(this, A);

	int32 ANum = MapHelperA.Num();

	if (!B)
	{
		return ANum == 0;
	}

	FScriptMapHelper MapHelperB(this, B);
	if (ANum != MapHelperB.Num())
	{
		return false;
	}

	return UEMapProperty_Private::IsPermutation(MapHelperA, MapHelperB, PortFlags);
}

void FMapProperty::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	if (KeyProp)
	{
		KeyProp->GetPreloadDependencies(OutDeps);
	}
	if (ValueProp)
	{
		ValueProp->GetPreloadDependencies(OutDeps);
	}
}

void FMapProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, const void* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	const bool bUPS = UnderlyingArchive.UseUnversionedPropertySerialization();
	bool bExperimentalOverridableLogic = HasAnyPropertyFlags(CPF_ExperimentalOverridableLogic);
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	// Map containers must be serialized as a "whole" value, which means that we need to serialize every field for struct-typed entries.
	// When using a custom property list, we need to temporarily bypass this logic to ensure that all map elements are fully serialized.
	const bool bIsUsingCustomPropertyList = !!UnderlyingArchive.ArUseCustomPropertyList;
	UnderlyingArchive.ArUseCustomPropertyList = false;
	ON_SCOPE_EXIT
	{
		UnderlyingArchive.ArUseCustomPropertyList = bIsUsingCustomPropertyList;
	};

	// If we're doing delta serialization within this property, act as if there are no defaults
	if (!UnderlyingArchive.DoIntraPropertyDelta() && !bExperimentalOverridableLogic)
	{
		Defaults = nullptr;
	}

	// Ar related calls in this function must be mirrored in FMapProperty::ConvertFromType
	checkSlow(KeyProp);
	checkSlow(ValueProp);

	FUObjectSerializeContext* Context = FUObjectThreadContext::Get().GetSerializeContext();

	FScriptMapHelper MapHelper(this, Value);

	// *** Experimental *** Special serialization path for map with overridable serialization logic
	if (!bUPS)
	{
		// Make sure the container is reloading accordingly to the value set in the property tag if any
		if (UnderlyingArchive.IsLoading() && FPropertyTagScope::GetCurrentPropertyTag())
		{
			bExperimentalOverridableLogic = FPropertyTagScope::GetCurrentPropertyTag()->bExperimentalOverridableLogic;
		}

		if (bExperimentalOverridableLogic)
		{
			checkf(!UnderlyingArchive.ArUseCustomPropertyList, TEXT("Using custom property list is not supported by overridable serialization"));

			if (UnderlyingArchive.IsLoading())
			{
				int32 NumReplaced = 0;
				FStructuredArchive::FArray ReplacedArray = Record.EnterArray(TEXT("Replaced"), NumReplaced);
				if (NumReplaced != INDEX_NONE)
				{
					MapHelper.EmptyValues(NumReplaced);
					for (int32 i = 0; i < NumReplaced; i++)
					{
						FStructuredArchive::FRecord EntryRecord = ReplacedArray.EnterElement().EnterRecord();
						int32 Index = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
						uint8* PairPtr = MapHelper.GetPairPtr(Index);
						{
							UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
							FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
							KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), PairPtr);
						}
						{
							UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
							FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
							ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), PairPtr + MapLayout.ValueOffset);
						}
					}
					MapHelper.Rehash();
				}
				else
				{
					FOverriddenPropertySet* OverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties();

					// This is not fully implemented yet and not a priority right now, so just trying to prevent it as the result could be random
					checkf(!KeyProp->HasAnyPropertyFlags(CPF_PersistentInstance) || CastField<FClassProperty>(KeyProp) || !CastField<FObjectProperty>(KeyProp), TEXT("The key as an instanced sub object is NYI"));

					uint8* TempKeyValueStorage = nullptr;
					ON_SCOPE_EXIT
					{
						if (TempKeyValueStorage)
						{
							KeyProp->DestroyValue(TempKeyValueStorage);
							ValueProp->DestroyValue(TempKeyValueStorage + MapLayout.ValueOffset);
							FMemory::Free(TempKeyValueStorage);
						}
					};

					int32 NumRemoved = 0;
					FStructuredArchive::FArray RemovedArray = Record.EnterArray(TEXT("Removed"), NumRemoved);
					if (NumRemoved != 0)
					{
						TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
						KeyProp->InitializeValue(TempKeyValueStorage);
						ValueProp->InitializeValue(TempKeyValueStorage + MapLayout.ValueOffset);

						for (int32 i = 0; i < NumRemoved; ++i)
						{
							{
								UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
								FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
								KeyProp->SerializeItem(RemovedArray.EnterElement().EnterRecord().EnterField(TEXT("Key")), TempKeyValueStorage);
							}

							bool bPersistOverride = MapHelper.RemovePair(TempKeyValueStorage);

#if WITH_EDITORONLY_DATA
							// remove operations for unknown properties should be persisted regardless of whether they can be found in the map because it's possible
							// the lack of CDO constructor is causing an element to be missing that otherwise would've been present
							if (!bPersistOverride)
							{
								bPersistOverride |= UE::IsSerializingUnknownData(UnderlyingArchive, this);
							}
#endif
							if (bPersistOverride)
							{
								// Need to fetch the MapOverriddenPropertyNode every loop as the previous might have reallocated the node.
								if (FOverriddenPropertyNode* MapOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, UnderlyingArchive.GetSerializedPropertyChain(), /*Property*/nullptr) : nullptr)
								{
									// Rebuild the overridden info
									FOverriddenPropertyNodeID RemovedKeyID = FOverriddenPropertyNodeID::FromMapKey(KeyProp, TempKeyValueStorage);
									OverriddenProperties->RestoreSubPropertyOperation(EOverriddenPropertyOperation::Remove, *MapOverriddenPropertyNode, RemovedKeyID);
								}
							}
						}
					}

					int32 NumModified = 0;
					FStructuredArchive::FArray ModifiedArray = Record.EnterArray(TEXT("Modified"), NumModified);
					if (NumModified != 0)
					{
						if (!TempKeyValueStorage)
						{
							TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
							KeyProp->InitializeValue(TempKeyValueStorage);
							ValueProp->InitializeValue(TempKeyValueStorage + MapLayout.ValueOffset);
						}
						for (int32 i = 0; i < NumModified; ++i)
						{
							FStructuredArchive::FRecord EntryRecord = ModifiedArray.EnterElement().EnterRecord();

							// Read key into temporary storage
							{
								UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
								FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
								KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), TempKeyValueStorage);
							}

							const int32 Index = MapHelper.FindMapPairIndexFromHash(TempKeyValueStorage);
							uint8* ValuePtr = Index != INDEX_NONE ? MapHelper.GetValuePtr(Index) : TempKeyValueStorage + MapLayout.ValueOffset;

							// Deserialize value into hash map-owned memory
							{
								UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
								FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
								ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), ValuePtr);
							}

							// Track only if we found the key in the array. Otherwise, skip it.
							if (Index != INDEX_NONE)
							{
								// Need to fetch the MapOverriddenPropertyNode every loop as the previous might have reallocated the node.
								if (FOverriddenPropertyNode* MapOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, UnderlyingArchive.GetSerializedPropertyChain(), /*Property*/nullptr) : nullptr)
								{
									// Rebuild the overridden info
									FOverriddenPropertyNodeID ModifiedKeyID = FOverriddenPropertyNodeID::FromMapKey(KeyProp, TempKeyValueStorage);
									OverriddenProperties->RestoreSubPropertyOperation(EOverriddenPropertyOperation::Modified, *MapOverriddenPropertyNode, ModifiedKeyID);
								}
							}
						}
					}

					// Support of subobject shadowed serialization
					if (UnderlyingArchive.UEVer() >= EUnrealEngineObjectUE5Version::OS_SUB_OBJECT_SHADOW_SERIALIZATION)
					{
						int32 NumShadowed = 0;
						FStructuredArchive::FArray ShadowedArray = Record.EnterArray(TEXT("Shadowed"), NumShadowed);
						if (NumShadowed != 0)
						{
							if (!TempKeyValueStorage)
							{
								TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
								KeyProp->InitializeValue(TempKeyValueStorage);
								ValueProp->InitializeValue(TempKeyValueStorage + MapLayout.ValueOffset);
							}
							for (int32 i = 0; i < NumShadowed; ++i)
							{
								FStructuredArchive::FRecord EntryRecord = ShadowedArray.EnterElement().EnterRecord();

								// Read key into temporary storage
								{
									UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
									FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
									KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), TempKeyValueStorage);
								}

								// Only modifying property when loading loose properties or placeholders, oterwise load in temp storage
								void* ValuePtr = 
#if WITH_EDITORONLY_DATA
								Context->bImpersonateProperties ? MapHelper.FindOrAdd(TempKeyValueStorage) :
#endif // WITH_EDITORONLY_DATA
								TempKeyValueStorage + MapLayout.ValueOffset;

								// Deserialize value
								{
									UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
									FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
									ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), ValuePtr);
								}
							}
						}
					}

					int32 NumAdded = 0;
					FStructuredArchive::FArray AddedArray = Record.EnterArray(TEXT("Added"), NumAdded);
					if (NumAdded != 0)
					{
						if (!TempKeyValueStorage)
						{
							TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
							KeyProp->InitializeValue(TempKeyValueStorage);
							ValueProp->InitializeValue(TempKeyValueStorage + MapLayout.ValueOffset);
						}

						for (int32 i = 0; i < NumAdded; ++i)
						{
							FStructuredArchive::FRecord EntryRecord = AddedArray.EnterElement().EnterRecord();

							// Read key into temporary storage
							{
								UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
								FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
								KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), TempKeyValueStorage);
							}

							void* ValuePtr = MapHelper.FindOrAdd(TempKeyValueStorage);

							// Deserialize value into hash map-owned memory
							{
								UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
								FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
								ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), ValuePtr);
							}

							// Need to fetch the MapOverriddenPropertyNode every loop as the previous might have reallocated the node.
							if (FOverriddenPropertyNode* MapOverriddenPropertyNode = OverriddenProperties ? OverriddenProperties->RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation::Modified, UnderlyingArchive.GetSerializedPropertyChain(), /*Property*/nullptr) : nullptr)
							{
								// Rebuild the overridden info
								FOverriddenPropertyNodeID AddedKeyID = FOverriddenPropertyNodeID::FromMapKey(KeyProp, TempKeyValueStorage);
								OverriddenProperties->RestoreSubPropertyOperation(EOverriddenPropertyOperation::Add, *MapOverriddenPropertyNode, AddedKeyID);
							}
						}
					}
				}
			}
			else
			{
				// Container for temporarily tracking some indices
				TArray<FScopedPropertyValue> RemovedKeys;
				TArray<int32> AddedIndices;
				TSet<int32> ModifiedIndices;
				TSet<int32> ShadowedIndices;

				bool bReplaceMap = false;
				if (!Defaults || !UnderlyingArchive.DoDelta() || UnderlyingArchive.IsTransacting())
				{
					bReplaceMap = true;
				}
				else 
				{
					EOverriddenPropertyOperation MapOverrideOp = EOverriddenPropertyOperation::None;
					FOverriddenPropertySet* OverriddenProperties = FOverridableSerializationLogic::GetOverriddenProperties();
					if (OverriddenProperties)
					{
						MapOverrideOp = OverriddenProperties->GetOverriddenPropertyOperation(UnderlyingArchive.GetSerializedPropertyChain(), /*Property*/nullptr);
						bReplaceMap = MapOverrideOp == EOverriddenPropertyOperation::Replace;
					}
					else
					{
						// In the case we do not have overridable serialization enable, let write the entire content of the array
						bReplaceMap = true;
					}

					if(!bReplaceMap)
					{
						checkf(!KeyProp->HasAnyPropertyFlags(CPF_PersistentInstance) || CastField<FClassProperty>(KeyProp) || !CastField<FObjectProperty>(KeyProp), TEXT("The key as an instanced sub object is NYI"));

						if (FOverridableSerializationLogic::ShouldPropertyShadowSerializeSubObject(this))
						{
							for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
							{
								ShadowedIndices.Add(It.GetInternalIndex());
							}
						}

						if (OverriddenProperties)
						{
							checkf(Defaults, TEXT("Expecting overridable serialization to have defaults to compare to"));
							FScriptMapHelper DefaultsMapHelper(this, Defaults);

							if (const FOverriddenPropertyNode* MapOverriddenPropertyNode = OverriddenProperties->GetOverriddenPropertyNode(UnderlyingArchive.GetSerializedPropertyChain()))
							{
								// Figure out the modifications of the map
								for (const FOverriddenPropertyNode& SubNode : MapOverriddenPropertyNode->GetSubPropertyNodes())
								{
									switch (SubNode.GetOperation())
									{
										case EOverriddenPropertyOperation::Remove:
										{
											if (FScopedPropertyValue Found = SubNode.GetNodeID().ToMapKey(DefaultsMapHelper))
											{
												RemovedKeys.Emplace(MoveTemp(Found));
											}
											break;
										}
										case EOverriddenPropertyOperation::Add:
										{
											const int32 InternalIndex = SubNode.GetNodeID().ToMapInternalIndex(MapHelper);
											if (InternalIndex != INDEX_NONE)
											{
												AddedIndices.Add(InternalIndex);
												ShadowedIndices.Remove(InternalIndex);
											}
											break;
										}
										case EOverriddenPropertyOperation::Modified:
										{
											const int32 InternalIndex = SubNode.GetNodeID().ToMapInternalIndex(MapHelper);
											if (InternalIndex != INDEX_NONE)
											{
												ModifiedIndices.Add(InternalIndex);
												ShadowedIndices.Remove(InternalIndex);
											}
											break;
										}
									default:
										checkf(false, TEXT("Unsupported map operation"));
										break;
									}
								}
							}
						}
					}
				}

				auto SerializePair = [this, Context, &UnderlyingArchive](FStructuredArchive::FArray& Array, uint8* PairPtr)
				{
					FStructuredArchive::FRecord EntryRecord = Array.EnterElement().EnterRecord();
					{
						UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
						FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
						KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), PairPtr);
					}
					{
						UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
						FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
						ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), PairPtr + MapLayout.ValueOffset);
					}
				};

				int32 NumReplaced = bReplaceMap ? MapHelper.Num() : INDEX_NONE;
				FStructuredArchive::FArray ReplacedArray = Record.EnterArray(TEXT("Replaced"), NumReplaced);
				if (bReplaceMap)
				{
					for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
					{
						SerializePair(ReplacedArray, MapHelper.GetPairPtr(It.GetInternalIndex()));
					}
				}
				else
				{
					checkf(Defaults, TEXT("Expecting overridable serialization to have defaults to compare to"));
					FScriptMapHelper DefaultsMapHelper(this, Defaults);

					int32 NumRemoved = bReplaceMap ? INDEX_NONE : RemovedKeys.Num();
					FStructuredArchive::FArray RemovedArray = Record.EnterArray(TEXT("Removed"), NumRemoved);
					for (const FScopedPropertyValue& Key : RemovedKeys)
					{
						FStructuredArchive::FRecord EntryRecord = RemovedArray.EnterElement().EnterRecord();
						UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
						FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
						KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), Key.GetData());
					}

					int32 NumModified = ModifiedIndices.Num();
					FStructuredArchive::FArray ModifiedArray = Record.EnterArray(TEXT("Modified"), NumModified);
					for (int32 i : ModifiedIndices)
					{
						SerializePair(ModifiedArray, MapHelper.GetPairPtr(i));
					}

					// Support of subobject shadowed serialization
					// Introduced from EUnrealEngineObjectUE5Version::OS_SUB_OBJECT_SHADOW_SERIALIZATION
					int32 NumShadowed = ShadowedIndices.Num();
					FStructuredArchive::FArray ShadowedArray = Record.EnterArray(TEXT("Shadowed"), NumShadowed);
					for (int32 i : ShadowedIndices)
					{
						SerializePair(ShadowedArray, MapHelper.GetPairPtr(i));
					}

					// Added keys
					int32 NumAdded = AddedIndices.Num();
					FStructuredArchive::FArray AddedArray = Record.EnterArray(TEXT("Added"), NumAdded);
					for (int32 i : AddedIndices)
					{
						SerializePair(AddedArray, MapHelper.GetPairPtr(i));
					}
				}
			}

			return;
		}
	}

	if (UnderlyingArchive.IsLoading())
	{
		// Delete any explicitly-removed elements
		int32 NumKeysToRemove = 0;
		FStructuredArchive::FArray KeysToRemoveArray = Record.EnterArray(TEXT("KeysToRemove"), NumKeysToRemove);
		const bool bReplaceMap = NumKeysToRemove == INDEX_NONE;

		if (Defaults && !bReplaceMap)
		{
			CopyValuesInternal(Value, Defaults, 1);
		}

		const bool bIsLoadingToUnknownPropertyTree =
#if WITH_EDITORONLY_DATA
			FUObjectThreadContext::Get().GetSerializeContext()->bIsLoadingToUnknownPropertyTree;
#else
			false;
#endif
		uint8* TempKeyValueStorage = nullptr;
		uint8* TempElementValueStorage = nullptr;
		ON_SCOPE_EXIT
		{
			if (TempKeyValueStorage)
			{
				if (TempElementValueStorage)
				{
					ValueProp->DestroyValue(TempElementValueStorage);
				}
				KeyProp->DestroyValue(TempKeyValueStorage);
				FMemory::Free(TempKeyValueStorage);
			}
		};
		if (bIsLoadingToUnknownPropertyTree)
		{
			// When loading into the unknown property tree the key type may be an unknown struct and therefore unhashable. In that case we can't add it to the map. 
			// Luckily we don't actually care about truly loading the data since we're just harvesting type info. We'll skip adding it and instead serialize into a temp location
			TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
			KeyProp->InitializeValue(TempKeyValueStorage);
			TempElementValueStorage = TempKeyValueStorage + MapLayout.ValueOffset;
			ValueProp->InitializeValue(TempElementValueStorage);
		}

		if (!Defaults || MapHelper.Num() == 0 || bReplaceMap) // Faster loading path when loading into an empty map or replacing the entire map
		{
			if (NumKeysToRemove && !bReplaceMap)
			{
				// Load and discard keys to remove, map is empty
				if (!TempKeyValueStorage)
				{
					TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
					KeyProp->InitializeValue(TempKeyValueStorage);
				}

				FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
				for (; NumKeysToRemove; --NumKeysToRemove)
				{
					KeyProp->SerializeItem(KeysToRemoveArray.EnterElement(), TempKeyValueStorage);
				}
			}

			int32 NumEntries = 0;
			FStructuredArchive::FArray EntriesArray = Record.EnterArray(TEXT("Entries"), NumEntries);

			// Empty and reserve then deserialize pairs directly into map memory
			MapHelper.EmptyValues(NumEntries);
			for (; NumEntries; --NumEntries)
			{
				FStructuredArchive::FRecord EntryRecord = EntriesArray.EnterElement().EnterRecord();


				void* KeyPtr;
				void* ValuePtr;
				if (bIsLoadingToUnknownPropertyTree)
				{
					KeyPtr = TempKeyValueStorage;
					ValuePtr = TempElementValueStorage;
				}
				else
				{
					int32 Index = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
					KeyPtr = MapHelper.GetKeyPtr(Index);
					ValuePtr = MapHelper.GetValuePtr(Index);
				}

				{
					UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
					KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), KeyPtr);
				}
				{
					UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
					ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), ValuePtr);
				}
			}

#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			// In UE_INTERNAL_UNIVERSAL_PLACEHOLDERS environs we may lose user types
			// and contained data will thus be unhashable - we could fix this
			// with even better placeholder support
			if(KeyProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash)) 
#endif
			if (!bIsLoadingToUnknownPropertyTree)
			{
				MapHelper.Rehash();
			}
		}
		else // Slower loading path that mutates non-empty map
		{

			if (NumKeysToRemove)
			{
				if (!TempKeyValueStorage)
				{
					TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
					KeyProp->InitializeValue(TempKeyValueStorage);
				}

				UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
				FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
				for (; NumKeysToRemove; --NumKeysToRemove)
				{
					// Read key into temporary storage
					KeyProp->SerializeItem(KeysToRemoveArray.EnterElement(), TempKeyValueStorage);

					// If the key is in the map, remove it
					if (uint8* PairPtr = MapHelper.FindMapPairPtrFromHash(TempKeyValueStorage))
					{
						MapHelper.RemovePair(PairPtr);
					}
				}
			}

			int32 NumEntries = 0;
			FStructuredArchive::FArray EntriesArray = Record.EnterArray(TEXT("Entries"), NumEntries);

			// Allocate temporary key space if we haven't allocated it already above
			if (NumEntries != 0 && !TempKeyValueStorage)
			{
				TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
				KeyProp->InitializeValue(TempKeyValueStorage);
			}

			// Read remaining items into container
			for (; NumEntries; --NumEntries)
			{
				FStructuredArchive::FRecord EntryRecord = EntriesArray.EnterElement().EnterRecord();

				// Read key into temporary storage
				{
					UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
					KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), TempKeyValueStorage);
				}

				void* ValuePtr = bIsLoadingToUnknownPropertyTree ? TempElementValueStorage : MapHelper.FindOrAdd(TempKeyValueStorage);

				// Deserialize value into hash map-owned memory
				{
					UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
					ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), ValuePtr);
				}
			}
		}
	}
	else
	{
		FScriptMapHelper DefaultsHelper(this, Defaults);

		// Override logic should only supports replacing the entire array
		const bool bReplaceMap = FOverridableSerializationLogic::GetOverriddenProperties() != nullptr;

		// Container for temporarily tracking some indices
		TSet<int32> Indices;

		// Determine how many keys are missing from the object
		if (Defaults && !bReplaceMap)
		{
			for (FScriptMapHelper::FIterator Iterator(DefaultsHelper); Iterator; ++Iterator)
			{
				uint8* DefaultPairPtr = DefaultsHelper.GetPairPtr(Iterator);
				if (!MapHelper.FindMapPairPtrWithKey(DefaultPairPtr))
				{
					Indices.Add(Iterator.GetInternalIndex());
				}
			}
		}

		// Write out the missing keys
		int32 MissingKeysNum = bReplaceMap ? INDEX_NONE : Indices.Num();
		FStructuredArchive::FArray KeysToRemoveArray = Record.EnterArray(TEXT("KeysToRemove"), MissingKeysNum);
		{
			FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
			for (int32 Index : Indices)
			{
				KeyProp->SerializeItem(KeysToRemoveArray.EnterElement(), DefaultsHelper.GetPairPtr(Index));
			}
		}

		// Write out differences from defaults
		if (Defaults && !bReplaceMap)
		{
			Indices.Empty(Indices.Num());
			for (FScriptMapHelper::FIterator Iterator(MapHelper); Iterator; ++Iterator)
			{
				uint8* ValuePairPtr   = MapHelper.GetPairPtr(Iterator);
				uint8* DefaultPairPtr = DefaultsHelper.FindMapPairPtrWithKey(ValuePairPtr);

				if (!DefaultPairPtr || !ValueProp->Identical(ValuePairPtr + MapLayout.ValueOffset, DefaultPairPtr + MapLayout.ValueOffset))
				{
					Indices.Add(Iterator.GetInternalIndex());
				}
			}

			// Write out differences from defaults
			int32 Num = Indices.Num();
			FStructuredArchive::FArray EntriesArray = Record.EnterArray(TEXT("Entries"), Num);
			for (int32 Index : Indices)
			{
				uint8* ValuePairPtr = MapHelper.GetPairPtrWithoutCheck(Index);
				FStructuredArchive::FRecord EntryRecord = EntriesArray.EnterElement().EnterRecord();

				{
					UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
					KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), ValuePairPtr);
				}
				{
					UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
					ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), ValuePairPtr + MapLayout.ValueOffset);
				}
			}
		}
		else
		{
			int32 Num = MapHelper.Num();
			FStructuredArchive::FArray EntriesArray = Record.EnterArray(TEXT("Entries"), Num);

			for (FScriptMapHelper::FIterator Iterator(MapHelper); Iterator; ++Iterator)
			{
				FStructuredArchive::FRecord EntryRecord = EntriesArray.EnterElement().EnterRecord();

				uint8* ValuePairPtr = MapHelper.GetPairPtr(Iterator);
				{
					UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Key});
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, KeyProp, this);
					KeyProp->SerializeItem(EntryRecord.EnterField(TEXT("Key")), ValuePairPtr);
				}
				{
					UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {UE::NAME_Value});
					FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ValueProp, this);
					ValueProp->SerializeItem(EntryRecord.EnterField(TEXT("Value")), ValuePairPtr + MapLayout.ValueOffset);
				}
			}
		}
	}
}

bool FMapProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData) const
{
	UE_LOGF( LogProperty, Error, "Replicated TMaps are not supported." );
	return 1;
}

void FMapProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	SerializeSingleField(Ar, KeyProp, this);
	SerializeSingleField(Ar, ValueProp, this);
}

void FMapProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
	if (KeyProp)
	{
		KeyProp->AddReferencedObjects(Collector);
	}
	if (ValueProp)
	{
		ValueProp->AddReferencedObjects(Collector);
	}
}

FString FMapProperty::GetCPPTypeCustom(FString* ExtendedTypeText, uint32 CPPExportFlags, const FString& KeyTypeText, const FString& InKeyExtendedTypeText, const FString& ValueTypeText, const FString& InValueExtendedTypeText) const
{
	if (ExtendedTypeText)
	{
		// if property type is a template class, add a space between the closing brackets
		FString KeyExtendedTypeText = InKeyExtendedTypeText;
		if ((KeyExtendedTypeText.Len() && KeyExtendedTypeText.Right(1) == TEXT(">"))
			|| (!KeyExtendedTypeText.Len() && KeyTypeText.Len() && KeyTypeText.Right(1) == TEXT(">")))
		{
			KeyExtendedTypeText += TEXT(" ");
		}

		// if property type is a template class, add a space between the closing brackets
		FString ValueExtendedTypeText = InValueExtendedTypeText;
		if ((ValueExtendedTypeText.Len() && ValueExtendedTypeText.Right(1) == TEXT(">"))
			|| (!ValueExtendedTypeText.Len() && ValueTypeText.Len() && ValueTypeText.Right(1) == TEXT(">")))
		{
			ValueExtendedTypeText += TEXT(" ");
		}

		*ExtendedTypeText = FString::Printf(TEXT("<%s%s,%s%s>"), *KeyTypeText, *KeyExtendedTypeText, *ValueTypeText, *ValueExtendedTypeText);
	}

	return TEXT("TMap");
}

FString FMapProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const
{
	checkSlow(KeyProp);
	checkSlow(ValueProp);

	FString KeyTypeText, KeyExtendedTypeText;
	FString ValueTypeText, ValueExtendedTypeText;

	if (ExtendedTypeText)
	{
		KeyTypeText = KeyProp->GetCPPType(&KeyExtendedTypeText, CPPExportFlags & ~CPPF_ArgumentOrReturnValue); // we won't consider map keys to be "arguments or return values"
		ValueTypeText = ValueProp->GetCPPType(&ValueExtendedTypeText, CPPExportFlags & ~CPPF_ArgumentOrReturnValue); // we won't consider map values to be "arguments or return values"
	}

	return GetCPPTypeCustom(ExtendedTypeText, CPPExportFlags, KeyTypeText, KeyExtendedTypeText, ValueTypeText, ValueExtendedTypeText);
}

FString FMapProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	checkSlow(KeyProp);
	checkSlow(ValueProp);
	ExtendedTypeText = FString::Printf(TEXT("%s,%s"), *KeyProp->GetCPPType(), *ValueProp->GetCPPType());
	return TEXT("TMAP");
}

void FMapProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	checkSlow(KeyProp);
	checkSlow(ValueProp);

	uint8* TempMapStorage = nullptr;
	void* PropertyValuePtr = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		// Allocate temporary map as we first need to initialize it with the value provided by the getter function and then export it
		TempMapStorage = (uint8*)AllocateAndInitializeValue();
		PropertyValuePtr = TempMapStorage;
		FProperty::GetValue_InContainer(ContainerOrPropertyPtr, PropertyValuePtr);
	}
	else
	{
		PropertyValuePtr = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
	}

	ON_SCOPE_EXIT
	{
		DestroyAndFreeValue(TempMapStorage);
	};

	FScriptMapHelper MapHelper(this, PropertyValuePtr);

	if (MapHelper.Num() == 0)
	{
		ValueStr += TEXT("()");
		return;
	}

	const bool bExternalEditor = (0 != (PPF_ExternalEditor & PortFlags));

	uint8* StructDefaults = nullptr;
	if (FStructProperty* StructValueProp = CastField<FStructProperty>(ValueProp))
	{
		checkSlow(StructValueProp->Struct);

		if (!bExternalEditor)
		{
			// For external editor, we always export all fields
			StructDefaults = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
			ValueProp->InitializeValue(StructDefaults + MapLayout.ValueOffset);
		}
	}
	ON_SCOPE_EXIT
	{
		if (StructDefaults)
		{
			ValueProp->DestroyValue(StructDefaults + MapLayout.ValueOffset);
			FMemory::Free(StructDefaults);
		}
	};

	FScriptMapHelper DefaultMapHelper(this, DefaultValue);

	uint8* PropData = MapHelper.GetPairPtrWithoutCheck(0);
	if (PortFlags & PPF_BlueprintDebugView)
	{
		int32 Index  = 0;
		bool  bFirst = true;
		for (int32 Count = MapHelper.Num(); Count; PropData += MapLayout.SetLayout.Size, ++Index)
		{
			if (MapHelper.IsValidIndex(Index))
			{
				if (bFirst)
				{
					bFirst = false;
				}
				else
				{
					ValueStr += TCHAR('\n');
				}

				ValueStr += TEXT("[");
				KeyProp->ExportText_Internal(ValueStr, PropData, EPropertyPointerType::Direct, nullptr, Parent, PortFlags | PPF_Delimited, ExportRootScope);
				ValueStr += TEXT("] ");

				// Always use struct defaults if the inner is a struct, for symmetry with the import of array inner struct defaults
				uint8* PropDefault = StructDefaults ? StructDefaults : DefaultValue ? DefaultMapHelper.FindMapPairPtrWithKey(PropData) : nullptr;

				if (bExternalEditor)
				{
					// For external editor, always write
					PropDefault = PropData;
				}

				ValueProp->ExportText_Internal(ValueStr, PropData + MapLayout.ValueOffset, EPropertyPointerType::Direct, PropDefault + MapLayout.ValueOffset, Parent, PortFlags | PPF_Delimited, ExportRootScope);

				--Count;
			}
		}
	}
	else
	{
		int32 Index  = 0;
		bool  bFirst = true;
		for (int32 Count = MapHelper.Num(); Count; PropData += MapLayout.SetLayout.Size, ++Index)
		{
			if (MapHelper.IsValidIndex(Index))
			{
				if (bFirst)
				{
					ValueStr += TCHAR('(');
					bFirst = false;
				}
				else
				{
					ValueStr += TCHAR(',');
				}

				ValueStr += TEXT("(");

				KeyProp->ExportText_Internal(ValueStr, PropData, EPropertyPointerType::Direct, nullptr, Parent, PortFlags | PPF_Delimited, ExportRootScope);

				ValueStr += TEXT(", ");

				// Always use struct defaults if the inner is a struct, for symmetry with the import of array inner struct defaults
				uint8* PropDefault = StructDefaults ? StructDefaults : DefaultValue ? DefaultMapHelper.FindMapPairPtrWithKey(PropData) : nullptr;

				if (bExternalEditor)
				{
					// For external editor, always write
					PropDefault = PropData;
				}

				ValueProp->ExportText_Internal(ValueStr, PropData + MapLayout.ValueOffset, EPropertyPointerType::Direct, PropDefault + MapLayout.ValueOffset, Parent, PortFlags | PPF_Delimited, ExportRootScope);

				ValueStr += TEXT(")");

				--Count;
			}
		}

		ValueStr += TEXT(")");
	}
}

const TCHAR* FMapProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	checkSlow(KeyProp);
	checkSlow(ValueProp);

	FScriptMapHelper MapHelper(this, PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType));
	uint8* TempMapStorage = nullptr;

	ON_SCOPE_EXIT
	{
		if (TempMapStorage)
		{
			// TempMap is used by property setter so if it was allocated call the setter now
			FProperty::SetValue_InContainer(ContainerOrPropertyPtr, TempMapStorage);

			// Destroy and free the temp map used by property setter
			DestroyAndFreeValue(TempMapStorage);
		}
	};

	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		// Allocate temporary map as we first need to initialize it with the parsed items and then use the setter to update the property
		TempMapStorage = (uint8*)AllocateAndInitializeValue();
		// Reinitialize the map helper with the temp value
		MapHelper = FScriptMapHelper(this, TempMapStorage);
	}

	MapHelper.EmptyValues();

	// If we export an empty array we export an empty string, so ensure that if we're passed an empty string
	// we interpret it as an empty array.
	if (*Buffer++ != TCHAR('('))
	{
		return nullptr;
	}

	SkipWhitespace(Buffer);
	if (*Buffer == TCHAR(')'))
	{
		return Buffer + 1;
	}

	uint8* TempPairStorage   = (uint8*)FMemory::Malloc(MapLayout.ValueOffset + ValueProp->GetElementSize());

	bool bSuccess = false;
	ON_SCOPE_EXIT
	{
		FMemory::Free(TempPairStorage);

		// If we are returning because of an error, remove any already-added elements from the map before returning
		// to ensure we're not left with a partial state.
		if (!bSuccess)
		{
			MapHelper.EmptyValues();
		}
	};

	for (;;)
	{
		KeyProp->InitializeValue(TempPairStorage);
		ValueProp->InitializeValue(TempPairStorage + MapLayout.ValueOffset);
		ON_SCOPE_EXIT
		{
			ValueProp->DestroyValue(TempPairStorage + MapLayout.ValueOffset);
			KeyProp->DestroyValue(TempPairStorage);
		};

		if (*Buffer++ != TCHAR('('))
		{
			return nullptr;
		}

		// Parse the key
		SkipWhitespace(Buffer);
		Buffer = KeyProp->ImportText_Internal(Buffer, TempPairStorage, EPropertyPointerType::Direct, Parent, PortFlags | PPF_Delimited, ErrorText);
		if (!Buffer)
		{
			return nullptr;
		}

		// Skip this element if it's already in the map
		bool bSkip = MapHelper.FindMapIndexWithKey(TempPairStorage) != INDEX_NONE;

		SkipWhitespace(Buffer);
		if (*Buffer++ != TCHAR(','))
		{
			return nullptr;
		}

		// Parse the value
		SkipWhitespace(Buffer);
		Buffer = ValueProp->ImportText_Internal(Buffer, TempPairStorage + MapLayout.ValueOffset, EPropertyPointerType::Direct, Parent, PortFlags | PPF_Delimited, ErrorText);
		if (!Buffer)
		{
			return nullptr;
		}

		SkipWhitespace(Buffer);
		if (*Buffer++ != TCHAR(')'))
		{
			return nullptr;
		}

		if (!bSkip)
		{
			int32  Index   = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
			uint8* PairPtr = MapHelper.GetPairPtrWithoutCheck(Index);

			// Copy over imported key and value from temporary storage
			KeyProp  ->CopyCompleteValue_InContainer(PairPtr, TempPairStorage);
			ValueProp->CopyCompleteValue_InContainer(PairPtr, TempPairStorage);
		}

		SkipWhitespace(Buffer);
		switch (*Buffer++)
		{
			case TCHAR(')'):
				MapHelper.Rehash();
				bSuccess = true;
				return Buffer;

			case TCHAR(','):
				SkipWhitespace(Buffer);
				break;

			default:
				return nullptr;
		}
	}
}

void FMapProperty::AddCppProperty(FProperty* Property)
{
	check(Property);

	if (!KeyProp)
	{
		// If the key is unset, assume it's the key
		check(!KeyProp);
		ensureAlwaysMsgf(Property->HasAllPropertyFlags(CPF_HasGetValueTypeHash), TEXT("Attempting to create Map Property with unhashable key type: %s - Provide a GetTypeHash function!"), *Property->GetName());
		KeyProp = Property;
	}
	else
	{
		// Otherwise assume it's the value
		check(!ValueProp);
		ValueProp = Property;
	}
}

void FMapProperty::CopyValuesInternal(void* Dest, void const* Src, int32 Count) const
{
	check(Count == 1);

	if (Dest == Src)
	{
		return;
	}

	FScriptMapHelper SrcMapHelper (this, Src);
	FScriptMapHelper DestMapHelper(this, Dest);

	int32 Num = SrcMapHelper.Num();
	DestMapHelper.EmptyValues(Num);

	if (Num == 0)
	{
		return;
	}

	checkf(SrcMapHelper.GetMaxIndex() >= Num, TEXT("Corrupt source map: Num = %d, MaxIndex = %d"), Num, SrcMapHelper.GetMaxIndex());

	for (int32 SrcIndex = 0; Num; ++SrcIndex)
	{
		if (SrcMapHelper.IsValidIndex(SrcIndex))
		{
			int32 DestIndex = DestMapHelper.AddDefaultValue_Invalid_NeedsRehash();

			uint8* SrcData  = SrcMapHelper .GetPairPtrWithoutCheck(SrcIndex);
			uint8* DestData = DestMapHelper.GetPairPtrWithoutCheck(DestIndex);

			KeyProp  ->CopyCompleteValue_InContainer(DestData, SrcData);
			ValueProp->CopyCompleteValue_InContainer(DestData, SrcData);

			--Num;
		}
	}

	DestMapHelper.Rehash();
}

void FMapProperty::ClearValueInternal(void* Data) const
{
	FScriptMapHelper MapHelper(this, Data);
	MapHelper.EmptyValues();
}

void FMapProperty::DestroyValueInternal(void* Data) const
{
	FScriptMapHelper MapHelper(this, Data);
	MapHelper.EmptyValues();

	//@todo UE potential double destroy later from this...would be ok for a script map, but still
	((FScriptMap*)Data)->~FScriptMap();
}

bool FMapProperty::ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const
{
	check(KeyProp);
	check(ValueProp);
	return KeyProp->ContainsFinishDestroy(EncounteredStructProps) || ValueProp->ContainsFinishDestroy(EncounteredStructProps);
}

void FMapProperty::FinishDestroyInternal( void* Data ) const
{
	if (!Data)
	{
		return;
	}
	
	check(KeyProp);
	check(ValueProp);

	const bool bMayHaveFinishDestroyKey   = (KeyProp->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)) == 0;
	const bool bMayHaveFinishDestroyValue = (ValueProp->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)) == 0;

	if (bMayHaveFinishDestroyKey
		|| bMayHaveFinishDestroyValue)
	{
		FScriptMapHelper MapHelper(this, Data);
		for (FScriptMapHelper::FIterator It(MapHelper.CreateIterator()); It; ++It)
		{
			uint8* PairPtr = MapHelper.GetPairPtr(It);
			if (bMayHaveFinishDestroyKey)
			{
				KeyProp->FinishDestroy(PairPtr);
			}
			if (bMayHaveFinishDestroyValue)
			{
				ValueProp->FinishDestroy(PairPtr + MapLayout.ValueOffset);
			}
		}
	}
}

/**
 * Creates new copies of components
 * 
 * @param	Data				pointer to the address of the instanced object referenced by this UComponentProperty
 * @param	DefaultData			pointer to the address of the default value of the instanced object referenced by this UComponentProperty
 * @param	InOwner				the object that contains this property's data
 * @param	InstanceGraph		contains the mappings of instanced objects and components to their templates
 */
void FMapProperty::InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, FObjectInstancingGraph* InstanceGraph)
{
	if (!Data)
	{
		return;
	}

	const bool bUsesDynamicInstancing = InOwner->GetClass()->ShouldUseDynamicSubobjectInstancing();
	const bool bInstancedKey   = KeyProp  ->ContainsInstancedObjectProperty() || bUsesDynamicInstancing;
	const bool bInstancedValue = ValueProp->ContainsInstancedObjectProperty() || bUsesDynamicInstancing;

	if (!bInstancedKey && !bInstancedValue)
	{
		return;
	}

	FScriptMapHelper MapHelper(this, Data);

	if (DefaultData)
	{
		FScriptMapHelper DefaultMapHelper(this, DefaultData);
		for (FScriptMapHelper::FIterator It(MapHelper.CreateIterator()); It; ++It)
		{
			uint8* PairPtr = MapHelper.GetPairPtr(It);
			const uint8* DefaultPairPtr = DefaultMapHelper.FindMapPairPtrWithKey(PairPtr, /*IndexHint*/ It.GetLogicalIndex());

			if (bInstancedKey)
			{
				KeyProp->InstanceSubobjects(PairPtr, DefaultPairPtr, InOwner, InstanceGraph);
			}

			if (bInstancedValue)
			{
				ValueProp->InstanceSubobjects(PairPtr + MapLayout.ValueOffset, DefaultPairPtr ? DefaultPairPtr + MapLayout.ValueOffset : nullptr, InOwner, InstanceGraph);
			}
		}
	}
	else
	{
		for (FScriptMapHelper::FIterator It(MapHelper.CreateIterator()); It; ++It)
		{
			uint8* PairPtr = MapHelper.GetPairPtr(It);

			if (bInstancedKey)
			{
				KeyProp->InstanceSubobjects(PairPtr, nullptr, InOwner, InstanceGraph);
			}

			if (bInstancedValue)
			{
				ValueProp->InstanceSubobjects(PairPtr + MapLayout.ValueOffset, nullptr, InOwner, InstanceGraph);
			}
		}
	}

	// Instancing keys will likely have invalidated the hash, so rehash
	if (bInstancedKey)
	{
		MapHelper.Rehash();
	}
}

bool FMapProperty::SameType(const FProperty* Other) const
{
	FMapProperty* MapProp = (FMapProperty*)Other;
	return Super::SameType(Other) && KeyProp && ValueProp && KeyProp->SameType(MapProp->KeyProp) && ValueProp->SameType(MapProp->ValueProp);
}

EConvertFromTypeResult FMapProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	// Ar related calls in this function must be mirrored in FMapProperty::SerializeItem
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	checkSlow(KeyProp);
	checkSlow(ValueProp);

	if (FStructProperty* KeyPropAsStruct = CastField<FStructProperty>(KeyProp))
	{
		if (!KeyPropAsStruct->Struct || (KeyPropAsStruct->Struct->GetCppStructOps() && !KeyPropAsStruct->Struct->GetCppStructOps()->HasGetTypeHash()))
		{
			// If the type we contain is no longer hashable, we're going to drop the saved data here.
			// This can happen if the native GetTypeHash function is removed.
			ensureMsgf(false, TEXT("Map Property %s has an unhashable key type %s and will lose its saved data. Package: %s"),
				*Tag.Name.ToString(), *KeyPropAsStruct->Struct->GetFName().ToString(), *UnderlyingArchive.GetArchiveName());

			FScriptMapHelper ScriptMapHelper(this, ContainerPtrToValuePtr<void>(Data));
			ScriptMapHelper.EmptyValues();

			return EConvertFromTypeResult::CannotConvert;
		}
	}

	if (Tag.Type != NAME_MapProperty)
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}

	const UE::FPropertyTypeName KeyType = Tag.GetType().GetParameter(0);
	const UE::FPropertyTypeName ValueType = Tag.GetType().GetParameter(1);
	const FName KeyTypeName = KeyType.GetName();
	const FName ValueTypeName = ValueType.GetName();
	bool bCanSerializeKey;
	bool bCanSerializeValue;

	const FPackageFileVersion Version = UnderlyingArchive.UEVer();
	if (Version >= EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
	{
		bCanSerializeKey = KeyProp->CanSerializeFromTypeName(KeyType);
		bCanSerializeValue = ValueProp->CanSerializeFromTypeName(ValueType);
		if (bCanSerializeKey && bCanSerializeValue)
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}
	}
	else
	{
		const auto IsPotentiallyUnsafeByteProperty = []( const FProperty* P )
		{
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			if(const FByteProperty* ByteProperty = CastField<FByteProperty>(P))
			{
				// byte properties with enums are unambiguously enum properties
				// but if no enum was declared we cannot know if it's a pure
				// byte property or an enum property that lost its enum:
				// disaster recovery is possible, but better to just resave the asset
				// post PROPERTY_TAG_COMPLETE_TYPE_NAME:
				return ByteProperty->Enum == nullptr;
			}
#endif // UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			return false;
		};
		bCanSerializeKey = (KeyTypeName == KeyProp->GetID());
		if(bCanSerializeKey && IsPotentiallyUnsafeByteProperty(KeyProp))
		{
			return EConvertFromTypeResult::CannotConvert;
		}
		bCanSerializeValue = (ValueTypeName == ValueProp->GetID());
		if(bCanSerializeValue && IsPotentiallyUnsafeByteProperty(ValueProp))
		{
			return EConvertFromTypeResult::CannotConvert;
		}
		if ((bCanSerializeKey || KeyTypeName.IsNone()) && (bCanSerializeValue || ValueTypeName.IsNone()))
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}
	}

	if (Tag.bExperimentalOverridableLogic)
	{
		return EConvertFromTypeResult::CannotConvert;
	}

	const auto SerializeOrConvert = [Context = FUObjectThreadContext::Get().GetSerializeContext()](bool bCanSerialize, FProperty* Inner, const FPropertyTag& InnerTag, FName InnerName, FStructuredArchive::FSlot InnerSlot, uint8* InnerData, const UStruct* InnerDefaultsStruct) -> bool
	{
		UE::FSerializedPropertyPathScope SerializedPropertyPath(Context, {InnerName});

		if (!bCanSerialize)
		{
			switch (Inner->ConvertFromType(InnerTag, InnerSlot, InnerData, InnerDefaultsStruct, nullptr))
			{
				case EConvertFromTypeResult::Converted:
				case EConvertFromTypeResult::Serialized:
					return true;
				case EConvertFromTypeResult::CannotConvert:
					return false;
				case EConvertFromTypeResult::UseSerializeItem:
					if (InnerTag.Type != Inner->GetID())
					{
						return false;
					}
					// Fall through to default SerializeItem
					break;
				default:
					checkNoEntry();
					return false;
			}
		}

		uint8* DestAddress = Inner->ContainerPtrToValuePtr<uint8>(InnerData, InnerTag.ArrayIndex);
		Inner->SerializeItem(InnerSlot, DestAddress);
		return true;
	};

	FScriptMapHelper MapHelper(this, ContainerPtrToValuePtr<void>(Data));

	uint8* TempKeyValueStorage = nullptr;
	ON_SCOPE_EXIT
	{
		if (TempKeyValueStorage)
		{
			KeyProp->DestroyValue(TempKeyValueStorage);
			FMemory::Free(TempKeyValueStorage);
		}
	};

	FPropertyTag KeyPropertyTag;
	KeyPropertyTag.SetProperty(KeyProp);
	KeyPropertyTag.SetType(KeyType);
	KeyPropertyTag.Name = Tag.Name;
	KeyPropertyTag.ArrayIndex = 0;

	FPropertyTag ValuePropertyTag;
	ValuePropertyTag.SetProperty(ValueProp);
	ValuePropertyTag.SetType(ValueType);
	ValuePropertyTag.Name = Tag.Name;
	ValuePropertyTag.ArrayIndex = 0;

	bool bConversionSucceeded = true;

	FStructuredArchive::FRecord ValueRecord = Slot.EnterRecord();

	// When we saved this instance we wrote out any elements that were in the 'Default' instance but not in the 
	// instance that was being written. Presumably we were constructed from our defaults and must now remove 
	// any of the elements that were not present when we saved this Map:
	int32 NumKeysToRemove = 0;
	FStructuredArchive::FArray KeysToRemoveArray = ValueRecord.EnterArray(TEXT("KeysToRemove"), NumKeysToRemove);
	const bool bReplaceMap = NumKeysToRemove == INDEX_NONE;

	if (NumKeysToRemove && !bReplaceMap)
	{
		TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
		KeyProp->InitializeValue(TempKeyValueStorage);

		if (SerializeOrConvert(bCanSerializeKey, KeyProp, KeyPropertyTag, UE::NAME_Key, KeysToRemoveArray.EnterElement(), TempKeyValueStorage, DefaultsStruct))
		{
			// If the key is in the map, remove it
			int32 Found = MapHelper.FindMapIndexWithKey(TempKeyValueStorage);
			if (Found != INDEX_NONE)
			{
				MapHelper.RemoveAt(Found);
			}

			// things are going fine, remove the rest of the keys:
			for (int32 I = 1; I < NumKeysToRemove; ++I)
			{
				verify(SerializeOrConvert(bCanSerializeKey, KeyProp, KeyPropertyTag, UE::NAME_Key, KeysToRemoveArray.EnterElement(), TempKeyValueStorage, DefaultsStruct));
				Found = MapHelper.FindMapIndexWithKey(TempKeyValueStorage);
				if (Found != INDEX_NONE)
				{
					MapHelper.RemoveAt(Found);
				}
			}
		}
		else
		{
			bConversionSucceeded = false;
		}
	}

	int32 Num = 0;
	FStructuredArchive::FArray EntriesArray = ValueRecord.EnterArray(TEXT("Entries"), Num);

	if (bConversionSucceeded)
	{
		if (bReplaceMap)
		{
			MapHelper.EmptyValues(Num);
		}

		if (Num != 0)
		{
			if (TempKeyValueStorage == nullptr)
			{
				TempKeyValueStorage = (uint8*)FMemory::Malloc(MapLayout.SetLayout.Size);
				KeyProp->InitializeValue(TempKeyValueStorage);
			}

			FStructuredArchive::FRecord FirstPropertyRecord = EntriesArray.EnterElement().EnterRecord();

			if (SerializeOrConvert(bCanSerializeKey, KeyProp, KeyPropertyTag, UE::NAME_Key, FirstPropertyRecord.EnterField(TEXT("Key")), TempKeyValueStorage, DefaultsStruct))
			{
				// Add a new default value if the key doesn't currently exist in the map
				bool bKeyAlreadyPresent = true;
				int32 NextPairIndex = MapHelper.FindMapIndexWithKey(TempKeyValueStorage);
				if (NextPairIndex == INDEX_NONE)
				{
					bKeyAlreadyPresent = false;
					NextPairIndex = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
				}

				uint8* NextPairPtr = MapHelper.GetPairPtrWithoutCheck(NextPairIndex);
				// This copy is unnecessary when the key was already in the map:
				KeyProp->CopyCompleteValue_InContainer(NextPairPtr, TempKeyValueStorage);

				// Deserialize value
				if (SerializeOrConvert(bCanSerializeValue, ValueProp, ValuePropertyTag, UE::NAME_Value, FirstPropertyRecord.EnterField(TEXT("Value")), NextPairPtr, DefaultsStruct))
				{
					// first entry went fine, convert the rest:
					for (int32 I = 1; I < Num; ++I)
					{
						FStructuredArchive::FRecord PropertyRecord = EntriesArray.EnterElement().EnterRecord();

						verify(SerializeOrConvert(bCanSerializeKey, KeyProp, KeyPropertyTag, UE::NAME_Key, PropertyRecord.EnterField(TEXT("Key")), TempKeyValueStorage, DefaultsStruct));
						NextPairIndex = MapHelper.FindMapIndexWithKey(TempKeyValueStorage);
						if (NextPairIndex == INDEX_NONE)
						{
							NextPairIndex = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
						}

						NextPairPtr = MapHelper.GetPairPtrWithoutCheck(NextPairIndex);
						// This copy is unnecessary when the key was already in the map:
						KeyProp->CopyCompleteValue_InContainer(NextPairPtr, TempKeyValueStorage);
						verify(SerializeOrConvert(bCanSerializeValue, ValueProp, ValuePropertyTag, UE::NAME_Value, PropertyRecord.EnterField(TEXT("Value")), NextPairPtr, DefaultsStruct));
					}
				}
				else
				{
					if (!bKeyAlreadyPresent)
					{
						MapHelper.EmptyValues();
					}

					bConversionSucceeded = false;
				}
			}
			else
			{
				bConversionSucceeded = false;
			}

			MapHelper.Rehash();
		}
	}

	// if we could not convert the property ourself, then indicate that calling code needs to advance the property
	if (!bConversionSucceeded)
	{
		UE_LOGF(LogClass, Warning,
			"Map Element Type mismatch in %ls - Previous (%ls to %ls) Current (%ls to %ls) for package: %ls",
			*WriteToString<32>(Tag.Name),
			*WriteToString<32>(KeyPropertyTag.GetType()),
			*WriteToString<32>(ValuePropertyTag.GetType()),
			*WriteToString<32>(UE::FPropertyTypeName(KeyProp)),
			*WriteToString<32>(UE::FPropertyTypeName(ValueProp)),
			*UnderlyingArchive.GetArchiveName());
	}

	return bConversionSucceeded ? EConvertFromTypeResult::Converted : EConvertFromTypeResult::CannotConvert;
}

#if WITH_EDITORONLY_DATA
void FMapProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (KeyProp)
	{
		KeyProp->AppendSchemaHash(Builder, bSkipEditorOnly);
	}
	if (ValueProp)
	{
		ValueProp->AppendSchemaHash(Builder, bSkipEditorOnly);
	}
}
#endif

void FScriptMapHelper::Rehash()
{
	WithScriptMap([this](auto* Map)
	{
		// Moved out-of-line to maybe fix a weird link error
		Map->Rehash(MapLayout, [this](const void* Src) {
			return KeyProp->GetValueTypeHash(Src);
		});
	});
}

FField* FMapProperty::GetInnerFieldByName(const FName& InName)
{
	if (KeyProp && KeyProp->GetFName() == InName)
	{
		return KeyProp;
	}
	else if (ValueProp && ValueProp->GetFName() == InName)
	{
		return ValueProp;
	}
	return nullptr;
}

void FMapProperty::GetInnerFields(TArray<FField*>& OutFields)
{
	if (KeyProp)
	{
		OutFields.Add(KeyProp);
		KeyProp->GetInnerFields(OutFields);
	}
	if (ValueProp)
	{
		OutFields.Add(ValueProp);
		ValueProp->GetInnerFields(OutFields);
	}
}

void* FMapProperty::GetValueAddressAtIndex_Direct(const FProperty* Inner, void* InValueAddress, const int32 LogicalIndex) const
{
	checkf(Inner == KeyProp || Inner == ValueProp, TEXT("Inner property must be either KeyProp or ValueProp"));

	FScriptMapHelper MapHelper(this, InValueAddress);
	const int32 InternalIndex = MapHelper.FindInternalIndex(LogicalIndex);
	if (InternalIndex != INDEX_NONE)
	{
		if (Inner == KeyProp)
		{
			return MapHelper.GetKeyPtr(InternalIndex);
		}

		return MapHelper.GetValuePtr(InternalIndex);
	}
	return nullptr;
}

bool FMapProperty::UseBinaryOrNativeSerialization(const FArchive& Ar) const
{
	if (Super::UseBinaryOrNativeSerialization(Ar))
	{
		return true;
	}

	const FProperty* LocalKeyProp = KeyProp;
	const FProperty* LocalValueProp = ValueProp;
	check(LocalKeyProp);
	check(LocalValueProp);
	return LocalKeyProp->UseBinaryOrNativeSerialization(Ar) || LocalValueProp->UseBinaryOrNativeSerialization(Ar);
}

bool FMapProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	if (!Super::LoadTypeName(Type, Tag))
	{
		return false;
	}

	const UE::FPropertyTypeName KeyType = Type.GetParameter(0);
	const UE::FPropertyTypeName ValueType = Type.GetParameter(1);
	FField* KeyField = FField::TryConstruct(KeyType.GetName(), this, GetFName());
	FField* ValueField = FField::TryConstruct(ValueType.GetName(), this, GetFName());
	FProperty* KeyProperty = CastField<FProperty>(KeyField);
	FProperty* ValueProperty = CastField<FProperty>(ValueField);
	if (KeyProperty && ValueProperty && KeyProperty->LoadTypeName(KeyType, Tag) && ValueProperty->LoadTypeName(ValueType, Tag))
	{
		KeyProp = KeyProperty;
		ValueProp = ValueProperty;
		return true;
	}
	delete KeyField;
	delete ValueField;
	return false;
}

void FMapProperty::SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const
{
	Super::SaveTypeName(Type);

	const FProperty* LocalKeyProp = KeyProp;
	const FProperty* LocalValueProp = ValueProp;
	check(LocalKeyProp);
	check(LocalValueProp);
	Type.BeginParameters();
	LocalKeyProp->SaveTypeName(Type);
	LocalValueProp->SaveTypeName(Type);
	Type.EndParameters();
}

bool FMapProperty::CanSerializeFromTypeName(UE::FPropertyTypeName Type) const
{
	if (!Super::CanSerializeFromTypeName(Type))
	{
		return false;
	}

	const FProperty* LocalKeyProp = KeyProp;
	const FProperty* LocalValueProp = ValueProp;
	check(LocalKeyProp);
	check(LocalValueProp);
	return LocalKeyProp->CanSerializeFromTypeName(Type.GetParameter(0)) && LocalValueProp->CanSerializeFromTypeName(Type.GetParameter(1));
}

EPropertyVisitorControlFlow FMapProperty::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const
{
	FScriptMapHelper MapHelper(this, Context.Data.PropertyData);

	// Indicate in the path that this property contains inner properties
	Context.Path.Top().bContainsInnerProperties = MapHelper.Num() > 0;

	EPropertyVisitorControlFlow RetVal = Super::Visit(Context, InFunc);

	if (RetVal == EPropertyVisitorControlFlow::StepInto)
	{
		checkf(KeyProp && ValueProp, TEXT("Expecting a valid inner property type"));

		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			{
				// Visit Key
				FPropertyVisitorScope Scope(Context.Path, FPropertyVisitorInfo(KeyProp, It.GetLogicalIndex(), EPropertyVisitorInfoType::MapKey));
				FPropertyVisitorContext SubContext = Context.VisitPropertyData(MapHelper.GetKeyPtr(It));

				RetVal = KeyProp->Visit(SubContext, InFunc);
				if (RetVal == EPropertyVisitorControlFlow::Stop)
				{
					return EPropertyVisitorControlFlow::Stop;
				}
				if (RetVal == EPropertyVisitorControlFlow::StepOut)
				{
					return EPropertyVisitorControlFlow::StepOver;
				}
			}

			{
				// Visit Value
				FPropertyVisitorScope Scope(Context.Path, FPropertyVisitorInfo(ValueProp, It.GetLogicalIndex(), EPropertyVisitorInfoType::MapValue));
				FPropertyVisitorContext SubContext = Context.VisitPropertyData(MapHelper.GetValuePtr(It));

				RetVal = ValueProp->Visit(SubContext, InFunc);
				if (RetVal == EPropertyVisitorControlFlow::Stop)
				{
					return EPropertyVisitorControlFlow::Stop;
				}
				if (RetVal == EPropertyVisitorControlFlow::StepOut)
				{
					return EPropertyVisitorControlFlow::StepOver;
				}
			}
		}
	}
	return RetVal;
}

void* FMapProperty::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	if ((Info.PropertyInfo == EPropertyVisitorInfoType::MapKey && Info.Property == KeyProp) || (Info.PropertyInfo == EPropertyVisitorInfoType::MapValue && Info.Property == ValueProp))
	{
		return GetValueAddressAtIndex_Direct(Info.Property, Data, Info.Index);
	}

	return nullptr;
}

bool FMapProperty::HasIntrusiveUnsetOptionalState() const
{
	return true;
}

void FMapProperty::InitializeIntrusiveUnsetOptionalValue(void* Data) const
{
	// FScriptMap's unset state constructor is good enough
	Super::InitializeIntrusiveUnsetOptionalValue(Data);
}

bool FMapProperty::IsIntrusiveOptionalValueSet(const void* Data) const
{
	// FScriptMap's unset state comparison is good enough
	return Super::IsIntrusiveOptionalValueSet(Data);
}

void FMapProperty::ClearIntrusiveOptionalValue(void* Data) const
{
	// Destroy any inner elements first, because FScriptMap's destructor will only free memory
	if (IsIntrusiveOptionalValueSet(Data))
	{
		FScriptMapHelper MapHelper(this, Data);
		MapHelper.EmptyValues();

		// Call Super to actually reset the optional to the unset state, now that any elements have been destroyed
		Super::ClearIntrusiveOptionalValue(Data);
	}
}
