// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "UObject/LinkerPlaceholderFunction.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/PropertyHelper.h"
#include "UObject/PropertyTypeName.h"
#include "UObject/UnrealTypePrivate.h"

FMulticastScriptDelegate FMulticastDelegateProperty::EmptyDelegate;

FMulticastDelegateProperty::FMulticastDelegateProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, SignatureFunction(nullptr)
{
}

FMulticastDelegateProperty::FMulticastDelegateProperty(FFieldVariant InOwner, const UECodeGen_Private::FMulticastDelegatePropertyParams& Prop, EPropertyFlags AdditionalPropertyFlags /*= CPF_None*/)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop, AdditionalPropertyFlags)
{
	SignatureFunction = Prop.SignatureFunctionFunc ? Prop.SignatureFunctionFunc(ETypeConstructPhase::Outer) : nullptr;
}

#if WITH_EDITORONLY_DATA
FMulticastDelegateProperty::FMulticastDelegateProperty(UField* InField)
	: Super(InField)
{
	UMulticastDelegateProperty* SourceProperty = CastChecked<UMulticastDelegateProperty>(InField);
	SignatureFunction = SourceProperty->SignatureFunction;
}
#endif // WITH_EDITORONLY_DATA

void FMulticastDelegateProperty::PostDuplicate(const FField& InField)
{
	const FMulticastDelegateProperty& Source = static_cast<const FMulticastDelegateProperty&>(InField);
	SignatureFunction = Source.SignatureFunction;
	Super::PostDuplicate(InField);
}

/*-----------------------------------------------------------------------------
	FMulticastDelegateProperty.
-----------------------------------------------------------------------------*/
void FMulticastDelegateProperty::InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, FObjectInstancingGraph* InstanceGraph )
{
	check(Data);
	if (DefaultData)
	{
		for( int32 i=0; i<ArrayDim; i++ )
		{
			// Fix up references to the class default object (if necessary)

			FMulticastScriptDelegate& Delegate = GetMulticastScriptDelegate((uint8*)Data, i);
			FMulticastScriptDelegate& DefaultDelegate = GetMulticastScriptDelegate((uint8*)DefaultData, i);

			if (&Delegate == &EmptyDelegate || &DefaultDelegate == &EmptyDelegate)
			{
				continue;
			}

			FMulticastScriptDelegate::FWriteAccessScope DelegateWriteScope = Delegate.GetWriteAccessScope();
			FMulticastScriptDelegate::FReadAccessScope DefaultDelegateReadScope = DefaultDelegate.GetReadAccessScope();

			FMulticastScriptDelegate::InvocationListType::TIterator CurInvocation(Delegate.InvocationList);
			FMulticastScriptDelegate::InvocationListType::TIterator DefaultInvocation(DefaultDelegate.InvocationList);
			for(; CurInvocation && DefaultInvocation; ++CurInvocation, ++DefaultInvocation )
			{
				FMulticastScriptDelegate::UnicastDelegateType& DestDelegateInvocation = *CurInvocation;
				UObject* CurrentUObject = DestDelegateInvocation.GetUObject();

				if (CurrentUObject)
				{
					FMulticastScriptDelegate::UnicastDelegateType& DefaultDelegateInvocation = *DefaultInvocation;
					UObject *Template = DefaultDelegateInvocation.GetUObject();
					UObject* NewUObject = InstanceGraph->InstancePropertyValue(Template, CurrentUObject, InOwner, EInstancePropertyValueFlags::AllowSelfReference | EInstancePropertyValueFlags::DoNotCreateNewInstance);
					DestDelegateInvocation.BindUFunction(NewUObject, DestDelegateInvocation.GetFunctionName());
				}
			}
			// now finish up the ones for which there is no default
			for(; CurInvocation; ++CurInvocation )
			{
				FMulticastScriptDelegate::UnicastDelegateType& DestDelegateInvocation = *CurInvocation;
				UObject* CurrentUObject = DestDelegateInvocation.GetUObject();

				if (CurrentUObject)
				{
					UObject* NewUObject = InstanceGraph->InstancePropertyValue(NULL, CurrentUObject, InOwner, EInstancePropertyValueFlags::AllowSelfReference | EInstancePropertyValueFlags::DoNotCreateNewInstance);
					DestDelegateInvocation.BindUFunction(NewUObject, DestDelegateInvocation.GetFunctionName());
				}
			}
		}
	}
	else // no default data 
	{
		for( int32 i=0; i<ArrayDim; i++ )
		{
			FMulticastScriptDelegate& Delegate = GetMulticastScriptDelegate((uint8*)Data, i);
			if (&Delegate == &EmptyDelegate)
			{
				continue;
			}

			FMulticastScriptDelegate::FWriteAccessScope DelegateWriteScope = Delegate.GetWriteAccessScope();

			for (FMulticastScriptDelegate::InvocationListType::TIterator CurInvocation(Delegate.InvocationList); CurInvocation; ++CurInvocation)
			{
				FMulticastScriptDelegate::UnicastDelegateType& DestDelegateInvocation = *CurInvocation;
				UObject* CurrentUObject = DestDelegateInvocation.GetUObject();

				if (CurrentUObject)
				{
					UObject* NewUObject = InstanceGraph->InstancePropertyValue(NULL, CurrentUObject, InOwner, EInstancePropertyValueFlags::AllowSelfReference | EInstancePropertyValueFlags::DoNotCreateNewInstance);
					DestDelegateInvocation.BindUFunction(NewUObject, DestDelegateInvocation.GetFunctionName());
				}
			}
		}
	}
}

bool FMulticastDelegateProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	if (!B)
	{
		return A == B;
	}

	const FMulticastScriptDelegate& DelegateA = GetMulticastScriptDelegate(A, 0);
	const FMulticastScriptDelegate& DelegateB = GetMulticastScriptDelegate(B, 0);

	if (&DelegateA == &DelegateB)
	{
		return true;
	}

	FMulticastScriptDelegate::FReadAccessScope DelegateA_ReadScope = DelegateA.GetReadAccessScope();
	FMulticastScriptDelegate::FReadAccessScope DelegateB_ReadScope = DelegateB.GetReadAccessScope();


	const int32 ListASize = DelegateA.InvocationList.Num();
	if (ListASize != DelegateB.InvocationList.Num())
	{
		return false;
	}

	for (int32 CurInvocationIndex = 0; CurInvocationIndex != ListASize; ++CurInvocationIndex)
	{
		const FMulticastScriptDelegate::UnicastDelegateType& BindingA = DelegateA.InvocationList[CurInvocationIndex];
		const FMulticastScriptDelegate::UnicastDelegateType& BindingB = DelegateB.InvocationList[CurInvocationIndex];

#if UE_WITH_REMOTE_OBJECT_HANDLE
		if ((PortFlags & PPF_AvoidRemoteObjectMigration) != 0)
		{
			// Before comparing the actual object pointers, compare their remote ids to prevent dereferencing objects that may be remote
			if (BindingA.GetRemoteId() != BindingB.GetRemoteId())
			{
				return false;
			}
		}
		else
#endif

		if (BindingA.GetFunctionName() != BindingB.GetFunctionName())
		{
			return false;
		}

		// Check UObject pointer comparison last.  This is to avoid unnecessary resolution of the
		// delegates' weak bindings back to pointers, particularly when objects are remote.
		if (BindingA.GetUObject() != BindingB.GetUObject())
		{
			return false;
		}
	}

	return true;
}

bool FMulticastDelegateProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	// Do not allow replication of delegates, as there is no way to make this secure (it allows the execution of any function in any object, on the remote client/server)
	return 1;
}


FString FMulticastDelegateProperty::GetCPPType( FString* ExtendedTypeText/*=NULL*/, uint32 CPPExportFlags/*=0*/ ) const
{
	// We have this test because sometimes the delegate hasn't been set up by FixupDelegateProperties at the time
	// we need the type for an error message.  We deliberately format it so that it's unambiguously not CPP code, but is still human-readable.
	if (!SignatureFunction)
	{
		return FString("{multicast delegate type}");
	}

	FString UnmangledFunctionName = SignatureFunction->GetName().LeftChop( FString( HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX ).Len() );
	const UClass* OwnerClass = SignatureFunction->GetOwnerClass();

	const bool bBlueprintCppBackend = (0 != (CPPExportFlags & EPropertyExportCPPFlags::CPPF_BlueprintCppBackend));
	const bool bNative = SignatureFunction->IsNative();
	if (bBlueprintCppBackend && bNative)
	{
		UStruct* StructOwner = Cast<UStruct>(SignatureFunction->GetOuter());
		if (StructOwner)
		{
			return FString::Printf(TEXT("%s%s::F%s"), StructOwner->GetPrefixCPP(), *StructOwner->GetName(), *UnmangledFunctionName);
		}
	}
	else
	{
		if ((0 != (CPPExportFlags & EPropertyExportCPPFlags::CPPF_BlueprintCppBackend)) && OwnerClass && !OwnerClass->HasAnyClassFlags(CLASS_Native))
		{
			// The name must be valid, this removes spaces, ?, etc from the user's function name. It could
			// be slightly shorter because the postfix ("__pf") is not needed here because we further post-
			// pend to the string. Normally the postfix is needed to make sure we don't mangle to a valid
			// identifier and collide:
			UnmangledFunctionName = UnicodeToCPPIdentifier(UnmangledFunctionName, false, TEXT(""));
			// the name must be unique
			const FString OwnerName = UnicodeToCPPIdentifier(OwnerClass->GetName(), false, TEXT(""));
			const FString NewUnmangledFunctionName = FString::Printf(TEXT("%s__%s"), *UnmangledFunctionName, *OwnerName);
			UnmangledFunctionName = NewUnmangledFunctionName;
		}
		if (0 != (CPPExportFlags & EPropertyExportCPPFlags::CPPF_CustomTypeName))
		{
			UnmangledFunctionName += TEXT("__MulticastDelegate");
		}
	}
	return FString("F") + UnmangledFunctionName;
}


void FMulticastDelegateProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	FMulticastScriptDelegate DelegateInContainer;
	FMulticastScriptDelegate* Delegate;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		GetValue_InContainer(PropertyValueOrContainer, &DelegateInContainer);
		Delegate = &GetMulticastScriptDelegate(&DelegateInContainer, 0);
	}
	else
	{
		Delegate = &GetMulticastScriptDelegate(PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType), 0);
	}

	// Start delegate array with open paren
	ValueStr += TEXT( "(" );

	if (Delegate && Delegate != &EmptyDelegate)
	{
		FMulticastScriptDelegate::FReadAccessScope DelegateReadScope = Delegate->GetReadAccessScope();

		bool bIsFirstFunction = true;
		for (FMulticastScriptDelegate::InvocationListType::TConstIterator CurInvocation(Delegate->InvocationList); CurInvocation; ++CurInvocation)
		{
			if (CurInvocation->IsBound())
			{
				if (!bIsFirstFunction)
				{
					ValueStr += TEXT(",");
				}
				bIsFirstFunction = false;

				bool bDelegateHasValue = CurInvocation->GetFunctionName() != NAME_None;
				ValueStr += FString::Printf(TEXT("%s.%s"),
					CurInvocation->GetUObject() != NULL ? *CurInvocation->GetUObject()->GetPathName() : TEXT("(null)"),
					*CurInvocation->GetFunctionName().ToString());
			}
		}
	}

	// Close the array (NOTE: It could be empty, but that's fine.)
	ValueStr += TEXT( ")" );
}

const TCHAR* FMulticastDelegateProperty::ImportDelegateFromText( FMulticastScriptDelegate& MulticastDelegate, const TCHAR* Buffer, UObject* Parent, FOutputDevice* ErrorText ) const
{
	// Multi-cast delegates always expect an opening parenthesis when using assignment syntax, so that
	// users don't accidentally blow away already-bound delegates in DefaultProperties.  This also helps
	// to differentiate between single-cast and multi-cast delegates
	if( *Buffer != TCHAR( '(' ) )
	{
		return NULL;
	}

	// Clear the existing delegate
	MulticastDelegate.Clear();

	// process opening parenthesis
	++Buffer;
	SkipWhitespace(Buffer);

	// Empty Multi-cast delegates is still valid.
	if (*Buffer == TCHAR(')'))
	{
		return Buffer;
	}

	do
	{
		// Parse the delegate
		FScriptDelegate ImportedDelegate;
		Buffer = DelegatePropertyTools::ImportDelegateFromText( ImportedDelegate, SignatureFunction, Buffer, Parent, ErrorText );
		if( Buffer == NULL )
		{
			return NULL;
		}

		// Add this delegate to our multicast delegate's invocation list
		MulticastDelegate.AddUnique( ImportedDelegate );

		SkipWhitespace(Buffer);
	}
	while( *Buffer == TCHAR(',') && Buffer++ );


	// We expect a closing paren
	if( *( Buffer++ ) != TCHAR(')') )
	{
		return NULL;
	}

	return MulticastDelegate.IsBound() ? Buffer : NULL;
}


const TCHAR* FMulticastDelegateProperty::ImportText_Add( const TCHAR* Buffer, void* PropertyValue, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText ) const
{
	if ( !ValidateImportFlags(PortFlags,ErrorText) )
	{
		return NULL;
	}

	// Parse the delegate
	FScriptDelegate ImportedDelegate;
	Buffer = DelegatePropertyTools::ImportDelegateFromText( ImportedDelegate, SignatureFunction, Buffer, Parent, ErrorText );
	if( Buffer == NULL )
	{
		return NULL;
	}

	// Add this delegate to our multicast delegate's invocation list
	AddDelegate(MoveTemp(ImportedDelegate), Parent, PropertyValue);

	SkipWhitespace(Buffer);

	return Buffer;
}


const TCHAR* FMulticastDelegateProperty::ImportText_Remove( const TCHAR* Buffer, void* PropertyValue, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText ) const
{
	if ( !ValidateImportFlags(PortFlags,ErrorText) )
	{
		return NULL;
	}

	// Parse the delegate
	FScriptDelegate ImportedDelegate;
	Buffer = DelegatePropertyTools::ImportDelegateFromText( ImportedDelegate, SignatureFunction, Buffer, Parent, ErrorText );
	if( Buffer == NULL )
	{
		return NULL;
	}

	// Remove this delegate from our multicast delegate's invocation list
	RemoveDelegate(ImportedDelegate, Parent, PropertyValue);

	SkipWhitespace(Buffer);

	return Buffer;
}


void FMulticastDelegateProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar << SignatureFunction;

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (auto PlaceholderFunc = Cast<ULinkerPlaceholderFunction>(SignatureFunction))
		{
			PlaceholderFunc->AddReferencingProperty(this);
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
}

bool FMulticastDelegateProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && (SignatureFunction == ((FMulticastDelegateProperty*)Other)->SignatureFunction);
}

EConvertFromTypeResult FMulticastDelegateProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	// Multicast delegate properties are serialization compatible
	if (Tag.Type == NAME_MulticastDelegateProperty || Tag.Type == FMulticastInlineDelegateProperty::StaticClass()->GetFName() || Tag.Type == FMulticastSparseDelegateProperty::StaticClass()->GetFName())
	{
		uint8* DestAddress = ContainerPtrToValuePtr<uint8>(Data, Tag.ArrayIndex);
		SerializeItem(Slot, DestAddress, nullptr);

		return EConvertFromTypeResult::Converted;
	}

	return EConvertFromTypeResult::UseSerializeItem;
}

void FMulticastDelegateProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (SignatureFunction && !SignatureFunction->IsA<ULinkerPlaceholderFunction>())
#endif//USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	{
		Collector.AddReferencedObject(SignatureFunction);
	}
	Super::AddReferencedObjects(Collector);
}


IMPLEMENT_FIELD(FMulticastDelegateProperty)

FMulticastInlineDelegateProperty::FMulticastInlineDelegateProperty(FFieldVariant InOwner, const UECodeGen_Private::FMulticastDelegatePropertyParams& Prop)
	: TProperty_MulticastDelegate(InOwner, Prop)
{
}

const FMulticastScriptDelegate* FMulticastInlineDelegateProperty::GetMulticastDelegate(const void* PropertyValue) const
{
	return (const FMulticastScriptDelegate*)PropertyValue;
}

void FMulticastInlineDelegateProperty::SetMulticastDelegate(void* PropertyValue, FMulticastScriptDelegate ScriptDelegate) const
{
	*(FMulticastScriptDelegate*)PropertyValue = MoveTemp(ScriptDelegate);
}

FMulticastScriptDelegate& FMulticastInlineDelegateProperty::GetMulticastScriptDelegate(const void* PropertyValue, int32 Index) const
{
	return (PropertyValue ? *((FMulticastScriptDelegate*)PropertyValue + Index) : EmptyDelegate);
}

void FMulticastInlineDelegateProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FArchiveUObjectFromStructuredArchive Adapter(Slot);
	FArchive& Ar = Adapter.GetArchive();
	Ar << *GetPropertyValuePtr(Value);
	Adapter.Close();
}

const TCHAR* FMulticastInlineDelegateProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	const TCHAR* Result = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		FMulticastScriptDelegate MulticastDelegate;		
		Result = ImportDelegateFromText(MulticastDelegate, Buffer, Parent, ErrorText);
		if (Result)
		{
			SetValue_InContainer(ContainerOrPropertyPtr, MulticastDelegate);
		}
	}
	else
	{
		FMulticastScriptDelegate& MulticastDelegate = *(FMulticastScriptDelegate*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
		Result = ImportDelegateFromText(MulticastDelegate, Buffer, Parent, ErrorText);
	}
	return Result;	
}

void ResolveDelegateReference(const FMulticastInlineDelegateProperty* InlineProperty, UObject*& Parent, void*& PropertyValue)
{
	if (PropertyValue == nullptr)
	{
		checkf(Parent, TEXT("Must specify at least one of Parent or PropertyValue"));
		PropertyValue = InlineProperty->GetPropertyValuePtr_InContainer(Parent);
	}
	// Owner doesn't matter for inline delegates, so we don't worry about the Owner == nullptr case
}

void FMulticastInlineDelegateProperty::AddDelegate(FScriptDelegate ScriptDelegate, UObject* Parent, void* PropertyValue) const
{
	ResolveDelegateReference(this, Parent, PropertyValue);

	FMulticastScriptDelegate& MulticastDelegate = (*(FMulticastScriptDelegate*)PropertyValue);

	// Add this delegate to our multicast delegate's invocation list
	MulticastDelegate.AddUnique(MoveTemp(ScriptDelegate));
}

void FMulticastInlineDelegateProperty::RemoveDelegate(const FScriptDelegate& ScriptDelegate, UObject* Parent, void* PropertyValue) const
{
	ResolveDelegateReference(this, Parent, PropertyValue);

	FMulticastScriptDelegate& MulticastDelegate = (*(FMulticastScriptDelegate*)PropertyValue);

	// Remove this delegate from our multicast delegate's invocation list
	MulticastDelegate.Remove(ScriptDelegate);
}

void FMulticastInlineDelegateProperty::ClearDelegate(UObject* Parent, void* PropertyValue) const
{
	ResolveDelegateReference(this, Parent, PropertyValue);

	FMulticastScriptDelegate& MulticastDelegate = (*(FMulticastScriptDelegate*)PropertyValue);
	MulticastDelegate.Clear();
}

IMPLEMENT_FIELD(FMulticastInlineDelegateProperty)

FMulticastSparseDelegateProperty::FMulticastSparseDelegateProperty(FFieldVariant InOwner, const UECodeGen_Private::FMulticastDelegatePropertyParams& Prop)
	: TProperty_MulticastDelegate(InOwner, Prop)
{
}

const FMulticastScriptDelegate* FMulticastSparseDelegateProperty::GetMulticastDelegate(const void* PropertyValue) const
{
	const FSparseDelegate* SparseDelegate = (const FSparseDelegate*)PropertyValue;
	if (SparseDelegate->IsBound())
	{
		USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
		UObject* OwningObject = FSparseDelegateStorage::ResolveSparseOwner(*SparseDelegate, SparseDelegateFunc->OwningClassName, SparseDelegateFunc->DelegateName);
		return FSparseDelegateStorage::GetMulticastDelegate(OwningObject, SparseDelegateFunc->DelegateName);
	}

	return nullptr;
}

void FMulticastSparseDelegateProperty::SetMulticastDelegate(void* PropertyValue, FMulticastScriptDelegate ScriptDelegate) const
{
	FSparseDelegate& SparseDelegate = *(FSparseDelegate*)PropertyValue;

	USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
	UObject* OwningObject = FSparseDelegateStorage::ResolveSparseOwner(SparseDelegate, SparseDelegateFunc->OwningClassName, SparseDelegateFunc->DelegateName);

	if (ScriptDelegate.IsBound())
	{
		FSparseDelegateStorage::SetMulticastDelegate(OwningObject, SparseDelegateFunc->DelegateName, MoveTemp(ScriptDelegate));
		SparseDelegate.bIsBound = true;
	}
	else if (SparseDelegate.bIsBound)
	{
		FSparseDelegateStorage::Clear(OwningObject, SparseDelegateFunc->DelegateName);
		SparseDelegate.bIsBound = false;
	}
}


FMulticastScriptDelegate& FMulticastSparseDelegateProperty::GetMulticastScriptDelegate(const void* PropertyValue, int32 Index) const
{
	if (FSparseDelegate* SparseDelegate = (FSparseDelegate*)PropertyValue)
	{
		SparseDelegate += Index;
		if (SparseDelegate->IsBound())
		{
			USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
			UObject* OwningObject = FSparseDelegateStorage::ResolveSparseOwner(*SparseDelegate, SparseDelegateFunc->OwningClassName, SparseDelegateFunc->DelegateName);
			if (FMulticastScriptDelegate* Delegate = FSparseDelegateStorage::GetMulticastDelegate(OwningObject, SparseDelegateFunc->DelegateName))
			{
				return *Delegate;
			}
		}
	}
	return EmptyDelegate;
}

void FMulticastSparseDelegateProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FArchiveUObjectFromStructuredArchive Adapter(Slot);
	SerializeItemInternal(Adapter.GetArchive(), Value, Defaults);
	Adapter.Close();
}

void FMulticastSparseDelegateProperty::SerializeItemInternal(FArchive& Ar, void* Value, void const* Defaults) const
{
	FSparseDelegate& SparseDelegate = *(FSparseDelegate*)Value;

	if (Ar.IsLoading())
	{
		FMulticastScriptDelegate Delegate;
		Ar << Delegate;

		if (Delegate.IsBound())
		{
			USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
			UObject* OwningObject = FSparseDelegateStorage::ResolveSparseOwner(SparseDelegate, SparseDelegateFunc->OwningClassName, SparseDelegateFunc->DelegateName);
			FSparseDelegateStorage::SetMulticastDelegate(OwningObject, SparseDelegateFunc->DelegateName, MoveTemp(Delegate));
			SparseDelegate.bIsBound = true;
		}
		else if (SparseDelegate.bIsBound)
		{
			USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
			UObject* OwningObject = FSparseDelegateStorage::ResolveSparseOwner(SparseDelegate, SparseDelegateFunc->OwningClassName, SparseDelegateFunc->DelegateName);
			FSparseDelegateStorage::Clear(OwningObject, SparseDelegateFunc->DelegateName);
			SparseDelegate.bIsBound = false;
		}
	} 
	else
	{
		if (SparseDelegate.IsBound())
		{
			USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
			UObject* OwningObject = FSparseDelegateStorage::ResolveSparseOwner(SparseDelegate, SparseDelegateFunc->OwningClassName, SparseDelegateFunc->DelegateName);
			if (FMulticastScriptDelegate* Delegate = FSparseDelegateStorage::GetMulticastDelegate(OwningObject, SparseDelegateFunc->DelegateName))
			{
				Ar << *Delegate;
			}
			else
			{
				FMulticastScriptDelegate DummyDelegate;
				Ar << DummyDelegate;
			}
		}
		else
		{
			FMulticastScriptDelegate DummyDelegate;
			Ar << DummyDelegate;
		}
	}
}

const TCHAR* FMulticastSparseDelegateProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	FMulticastScriptDelegate Delegate;
	const TCHAR* Result = ImportDelegateFromText(Delegate, Buffer, Parent, ErrorText);

	if (Result)
	{
		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			FProperty::SetValue_InContainer(ContainerOrPropertyPtr, &Delegate);
		}
		else
		{
			FSparseDelegate& SparseDelegate = *(FSparseDelegate*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
			USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);

			if (Delegate.IsBound())
			{
				FSparseDelegateStorage::SetMulticastDelegate(Parent, SparseDelegateFunc->DelegateName, MoveTemp(Delegate));
				SparseDelegate.bIsBound = true;
			}
			else
			{
				FSparseDelegateStorage::Clear(Parent, SparseDelegateFunc->DelegateName);
				SparseDelegate.bIsBound = false;
			}
		}
	}

	return Result;
}

void ResolveDelegateReference(const FMulticastSparseDelegateProperty* SparseProperty, UObject*& Parent, void*& PropertyValue)
{
	USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SparseProperty->SignatureFunction);

	if (Parent == nullptr)
	{
		checkf(PropertyValue, TEXT("Must specify at least one of Parent or PropertyValue"));
		Parent = FSparseDelegateStorage::ResolveSparseOwner(*(FSparseDelegate*)PropertyValue, SparseDelegateFunc->OwningClassName, SparseDelegateFunc->DelegateName);
	}
	else if (PropertyValue)
	{
		checkSlow(Parent == FSparseDelegateStorage::ResolveSparseOwner(*(FSparseDelegate*)PropertyValue, SparseDelegateFunc->OwningClassName, SparseDelegateFunc->DelegateName));
	}
	else
	{
		PropertyValue = SparseProperty->GetPropertyValuePtr_InContainer(Parent);
	}
}

void FMulticastSparseDelegateProperty::AddDelegate(FScriptDelegate ScriptDelegate, UObject* Parent, void* PropertyValue) const
{
	ResolveDelegateReference(this, Parent, PropertyValue);
	USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
	FSparseDelegate& SparseDelegate = *(FSparseDelegate*)PropertyValue;
	SparseDelegate.__Internal_AddUnique(Parent, SparseDelegateFunc->DelegateName, MoveTemp(ScriptDelegate));
}

void FMulticastSparseDelegateProperty::RemoveDelegate(const FScriptDelegate& ScriptDelegate, UObject* Parent, void* PropertyValue) const
{
	ResolveDelegateReference(this, Parent, PropertyValue);
	USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
	FSparseDelegate& SparseDelegate = *(FSparseDelegate*)PropertyValue;
	SparseDelegate.__Internal_Remove(Parent, SparseDelegateFunc->DelegateName, ScriptDelegate);
}

void FMulticastSparseDelegateProperty::ClearDelegate(UObject* Parent, void* PropertyValue) const
{
	ResolveDelegateReference(this, Parent, PropertyValue);
	USparseDelegateFunction* SparseDelegateFunc = CastChecked<USparseDelegateFunction>(SignatureFunction);
	FSparseDelegate& SparseDelegate = *(FSparseDelegate*)PropertyValue;
	SparseDelegate.__Internal_Clear(Parent, SparseDelegateFunc->DelegateName);
}

bool FMulticastSparseDelegateProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	if (!Super::LoadTypeName(Type, Tag))
	{
		return false;
	}

	// This cannot be used without its SignatureFunction and the tag lacks the information needed to load it.
	return false;
}

IMPLEMENT_FIELD(FMulticastSparseDelegateProperty)
