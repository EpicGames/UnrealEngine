// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "Misc/StringBuilder.h"

/*-----------------------------------------------------------------------------
	FNameProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FNameProperty)

FNameProperty::FNameProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FNameProperty::FNameProperty(FFieldVariant InOwner, const UECodeGen_Private::FNamePropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
FNameProperty::FNameProperty(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA

void FNameProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	FName Temp;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		GetValue_InContainer(PropertyValueOrContainer, &Temp);
	}
	else
	{
		Temp = *(FName*)PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType);
	}
	if( !(PortFlags & PPF_Delimited) )
	{
		ValueStr += Temp.ToString();
	}
	else if ( Temp != NAME_None )
	{
		ValueStr += FString::Printf( TEXT("\"%s\""), *Temp.ToString().ReplaceCharWithEscapedChar() );
	}
	else
	{
		ValueStr += TEXT("\"\"");
	}
}
const TCHAR* FNameProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	FName ImportedName;
	if (!(PortFlags & PPF_Delimited))
	{
		ImportedName = FName(Buffer);		

		// in order to indicate that the value was successfully imported, advance the buffer past the last character that was imported
		Buffer += FCString::Strlen(Buffer);
	}
	else
	{
		TStringBuilder<256> Token;
		Buffer = FPropertyHelpers::ReadToken(Buffer, /* out */ Token, true);
		if (!Buffer)
		{
			return nullptr;
		}

		ImportedName = FName(Token);
	}
	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		SetValue_InContainer(ContainerOrPropertyPtr, ImportedName);
	}
	else
	{
		*(FName*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType) = ImportedName;
	}

	return Buffer;
}

EConvertFromTypeResult FNameProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	if (Tag.Type == NAME_StrProperty)
	{
		FString str;
		Slot << str;
		SetPropertyValue_InContainer(Data, FName(*str), Tag.ArrayIndex);
		return EConvertFromTypeResult::Converted;
	}

	// Convert serialized text to name.
	if (Tag.Type == NAME_TextProperty)
	{ 
		FText Text;  
		Slot << Text;
		const FName Name = FName(*Text.ToString());
		SetPropertyValue_InContainer(Data, Name, Tag.ArrayIndex);
		return EConvertFromTypeResult::Converted;
	}

	return EConvertFromTypeResult::UseSerializeItem;
}

uint32 FNameProperty::GetValueTypeHashInternal(const void* Src) const
{
	return GetTypeHash(*(const FName*)Src);
}
