// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyNames.h"

extern const FName PropertyNames::PropertyVisibilityOverrideName(TEXT("PropertyVisibilityOverride"));
