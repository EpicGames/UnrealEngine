// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

IMPLEMENT_FIELD(FNumericProperty)
IMPLEMENT_FIELD(FInt8Property)
IMPLEMENT_FIELD(FInt16Property)
IMPLEMENT_FIELD(FIntProperty)
IMPLEMENT_FIELD(FInt64Property)
IMPLEMENT_FIELD(FUInt16Property)
IMPLEMENT_FIELD(FUInt32Property)
IMPLEMENT_FIELD(FUInt64Property)
IMPLEMENT_FIELD(FFloatProperty)
IMPLEMENT_FIELD(FDoubleProperty)

FNumericProperty::FNumericProperty(FFieldVariant InOwner, const FName& InName)
	: FProperty(InOwner, InName)
{
}

FNumericProperty::FNumericProperty(FFieldVariant InOwner, const UECodeGen_Private::FPropertyParamsBaseWithOffset& Prop, EPropertyFlags AdditionalPropertyFlags /*= CPF_None*/)
	: Super(InOwner, Prop, AdditionalPropertyFlags)
{
}

#if WITH_EDITORONLY_DATA
FNumericProperty::FNumericProperty(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA

int64 FNumericProperty::ReadEnumAsInt64(FStructuredArchive::FSlot Slot, const UStruct* DefaultsStruct, const FPropertyTag& Tag)
{
	//@warning: mirrors loading code in FByteProperty::SerializeItem() and FEnumProperty::SerializeItem()
	FName EnumName;
	Slot << EnumName;

	FName EnumTypeName = Tag.GetType().GetParameterName(0);

	UEnum* Enum = nullptr;
	if (DefaultsStruct)
	{
		Enum = FindUField<UEnum>(DefaultsStruct->IsA<UClass>() ? (UClass*)DefaultsStruct : DefaultsStruct->GetTypedOuter<UClass>(), EnumTypeName);
	}
	if (!Enum)
	{
		// Enums (at least native) are stored as short names (for now) so find the Tag enum by name
		Enum = FindFirstObject<UEnum>(*WriteToString<128>(EnumTypeName), EFindFirstObjectOptions::NativeFirst);
	}

	if (!Enum)
	{
		UE_LOGF(LogClass, Warning, "Failed to find enum '%ls' when converting property '%ls' during property loading - setting to 0", *EnumTypeName.ToString(), *Tag.Name.ToString());
		return 0;
	}

	Slot.GetUnderlyingArchive().Preload(Enum);

	// This handles redirects internally
	int64 Result = Enum->GetValueOrBitfieldFromString(EnumName.ToString());
	if (!Enum->IsValidEnumValueOrBitfield(Result))
	{
		int64 ReplacementValue = Enum->HasAnyEnumFlags(EEnumFlags::Flags) ? 0 : Enum->GetMaxEnumValue();

		UE_LOGF(
			LogClass,
			Warning,
			"Failed to find valid enum value '%ls' for enum type '%ls' when converting property '%ls' during property loading - setting to '%ls'",
			*EnumName.ToString(),
			*Enum->GetName(),
			*Tag.Name.ToString(),
			*Enum->GetNameByValue(ReplacementValue).ToString()
			);

		return ReplacementValue;
	}

	return Result;
}

const TCHAR* FNumericProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	if ( Buffer != NULL )
	{
		const TCHAR* Start = Buffer;
		if (IsInteger())
		{
			if (FChar::IsAlpha(*Buffer))
			{
				int64 EnumValue = UEnum::ParseEnum(Buffer);
				if (EnumValue != INDEX_NONE)
				{
					if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
					{
						SetValue_InContainer(ContainerOrPropertyPtr, &EnumValue);
					}
					else
					{
						SetIntPropertyValue(PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType), EnumValue);
					}
					return Buffer;
				}
				else
				{
					return NULL;
				}
			}
			else
			{
				if ( !FCString::Strnicmp(Start,TEXT("0x"),2) )
				{
					Buffer+=2;
					while ( Buffer && (FParse::HexDigit(*Buffer) != 0 || *Buffer == TCHAR('0')) )
					{
						Buffer++;
					}
				}
				else
				{
					while ( Buffer && (*Buffer == TCHAR('-') || *Buffer == TCHAR('+')) )
					{
						Buffer++;
					}

					while ( Buffer &&  FChar::IsDigit(*Buffer) )
					{
						Buffer++;
					}
				}

				if (Start == Buffer)
				{
					// import failure
					return NULL;
				}
			}
		}
		else
		{
			check(IsFloatingPoint());
			// floating point
			while( *Buffer == TCHAR('+') || *Buffer == TCHAR('-') || *Buffer == TCHAR('.') || (*Buffer >= TCHAR('0') && *Buffer <= TCHAR('9')) )
			{
				Buffer++;
			}
			if ( *Buffer == TCHAR('f') || *Buffer == TCHAR('F') )
			{
				Buffer++;
			}
		}
		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			SetNumericPropertyValueFromString_InContainer(ContainerOrPropertyPtr, Start);
		}
		else
		{
			SetNumericPropertyValueFromString(PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType), Start);
		}
	}
	return Buffer;
}

void FNumericProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		ValueStr += GetNumericPropertyValueToString_InContainer(PropertyValueOrContainer);
	}
	else
	{
		ValueStr += GetNumericPropertyValueToString(PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType));
	}
}

bool FNumericProperty::IsFloatingPoint() const
{
	return false;
}

bool FNumericProperty::IsInteger() const
{
	return true;
}

UEnum* FNumericProperty::GetIntPropertyEnum() const
{
	return nullptr;
}

/** 
	* Set the value of an unsigned integral property type
	* @param Data - pointer to property data to set
	* @param Value - Value to set data to
**/
void FNumericProperty::SetIntPropertyValue(void* Data, uint64 Value) const
{
	check(0);
}

/** 
	* Set the value of a signed integral property type
	* @param Data - pointer to property data to set
	* @param Value - Value to set data to
**/
void FNumericProperty::SetIntPropertyValue(void* Data, int64 Value) const
{
	check(0);
}

/** 
	* Set the value of a floating point property type
	* @param Data - pointer to property data to set
	* @param Value - Value to set data to
**/
void FNumericProperty::SetFloatingPointPropertyValue(void* Data, double Value) const
{
	check(0);
}

/** 
	* Set the value of any numeric type from a string point
	* @param Data - pointer to property data to set
	* @param Value - Value (as a string) to set 
	* CAUTION: This routine does not do enum name conversion
**/
void FNumericProperty::SetNumericPropertyValueFromString(void* Data, TCHAR const* Value) const
{
	check(0);
}
void FNumericProperty::SetNumericPropertyValueFromString_InContainer(void* Container, TCHAR const* Value) const
{
	check(0);
}
/** 
	* Gets the value of a signed integral property type
	* @param Data - pointer to property data to get
	* @return Data as a signed int
**/
int64 FNumericProperty::GetSignedIntPropertyValue(void const* Data) const
{
	check(0);
	return 0;
}
int64 FNumericProperty::GetSignedIntPropertyValue_InContainer(void const* Container) const
{
	check(0);
	return 0;
}

/** 
	* Gets the value of an unsigned integral property type
	* @param Data - pointer to property data to get
	* @return Data as an unsigned int
**/
uint64 FNumericProperty::GetUnsignedIntPropertyValue(void const* Data) const
{
	check(0);
	return 0;
}
uint64 FNumericProperty::GetUnsignedIntPropertyValue_InContainer(void const* Data) const
{
	check(0);
	return 0;
}

/** 
	* Gets the value of an floating point property type
	* @param Data - pointer to property data to get
	* @return Data as a double
**/
double FNumericProperty::GetFloatingPointPropertyValue(void const* Data) const
{
	check(0);
	return 0.0;
}

/** 
	* Get the value of any numeric type and return it as a string
	* @param Data - pointer to property data to get
	* @return Data as a string
	* CAUTION: This routine does not do enum name conversion
**/
FString FNumericProperty::GetNumericPropertyValueToString(void const* Data) const
{
	check(0);
	return FString();
}

FString FNumericProperty::GetNumericPropertyValueToString_InContainer(void const* Container) const
{
	check(0);
	return FString();
}
