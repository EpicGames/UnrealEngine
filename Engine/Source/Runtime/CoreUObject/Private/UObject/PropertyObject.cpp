// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealType.h"
#include "Blueprint/BlueprintSupport.h"
#include "UObject/ICookInfo.h"
#include "UObject/LinkerPlaceholderBase.h"
#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/PropertyHelper.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/CoreNet.h"
#include "UObject/ObjectPathId.h"
#include "UObject/PackedObjectRef.h"

/*-----------------------------------------------------------------------------
	FObjectProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FObjectProperty)

FObjectProperty::FObjectProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FObjectProperty::FObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParams& Prop)
	: Super(InOwner, Prop)
{
}

#if WITH_EDITORONLY_DATA
FObjectProperty::FObjectProperty(UField* InField)
	: TFObjectPropertyBase(InField)
{
}
#endif // WITH_EDITORONLY_DATA

FString FObjectProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags)  const
{
	check(PropertyClass);
	if (HasAnyPropertyFlags(CPF_TObjectPtr) && !(CPPExportFlags & CPPF_NoTObjectPtr))
	{
		return FString::Printf(TEXT("TObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
	}
	return FString::Printf(TEXT("%s%s*"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
}

FString FObjectProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	if (HasAnyPropertyFlags(CPF_TObjectPtr))
	{
		ExtendedTypeText = FString::Printf(TEXT("TObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
		return TEXT("OBJECTPTR");
	}
	ExtendedTypeText = FString::Printf(TEXT("%s%s"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
	return TEXT("OBJECT");
}

EConvertFromTypeResult FObjectProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	using namespace UE::CoreUObject::Private;
	static FName NAME_AssetObjectProperty = "AssetObjectProperty"; // old name of soft object property

	if (Tag.Type == NAME_SoftObjectProperty || Tag.Type == NAME_AssetObjectProperty)
	{
		// This property used to be a TSoftObjectPtr<Foo> but is now a raw FObjectProperty Foo*, we can convert without loss of data
		FSoftObjectPtr PreviousValue;
		Slot << PreviousValue;
		ECookLoadType CookLoadType = this->IsEditorOnlyProperty() ? ECookLoadType::EditorOnly : ECookLoadType::UsedInGame;

		UObject* PreviousValueObj = nullptr;

		bool bLazyResolve = false;
		// If we're async loading, only zenloader supports synchronous loads from the async loading thread.
		// Other loaders are not safe to do a sync load because they may crash or fail to set the variable, so throw an error if it's not already in memory
		ELoaderType LoaderType = GetLoaderType();
		if (!IsInGameThread())
		{
			PreviousValueObj = PreviousValue.Get();

			if (!PreviousValueObj && !PreviousValue.IsNull())
			{
				if (LoaderType != ELoaderType::ZenLoader)
				{
					UE_LOGF(LogClass, Error, "Failed to convert soft path %ls to unloaded object as this is not supported during async loading with the currently active loader '%ls'. Load and resave %ls in the editor to fix!", *PreviousValue.ToString(), LexToString(LoaderType), *Slot.GetUnderlyingArchive().GetArchiveName());
				}
#if WITH_EDITOR && UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
				// Only for the parallel loading path first so that we can do some opt-in testing
				else if (IsInParallelLoadingThread())
				{
					const FSoftObjectPath SoftObjectPath = PreviousValue.ToSoftObjectPath();

					const FObjectRef ImportRef(SoftObjectPath.GetLongPackageFName(), NAME_None, NAME_None, FObjectPathId(PreviousValue.GetAssetName()));
					const FPackedObjectRef PackedObjectRef = MakePackedObjectRef(ImportRef);
					PreviousValueObj = (UObject*)PackedObjectRef.EncodedRef;

					UE_LOGF(LogProperty, Log, "ConvertFromType is replacing softobjectpath %ls with lazyref %p", *SoftObjectPath.GetLongPackageFName().ToString(), PreviousValueObj);

					SoftObjectPath.LoadAsync(
						FLoadSoftObjectPathAsyncDelegate(),
						FLoadAssetAsyncOptionalParams
						{
							.PackagePriority = MAX_int32
						}
					);
					bLazyResolve = true;
				}
#endif
				else
				{
					FCookLoadScope CookLoadScope(CookLoadType);
					PreviousValueObj = PreviousValue.LoadSynchronous();
				}
			}
		}
		else
		{
			FCookLoadScope CookLoadScope(CookLoadType);
			PreviousValueObj = PreviousValue.LoadSynchronous();
		}

		// Now copy the value into the object's address space
		SetPropertyValue_InContainer(Data, PreviousValueObj, Tag.ArrayIndex);

		if (!bLazyResolve)
		{
			// Validate the type is proper
			CheckValidObject(GetPropertyValuePtr_InContainer(Data, Tag.ArrayIndex), PreviousValueObj, Defaults);
		}

		return EConvertFromTypeResult::Converted;
	}
	else if (Tag.Type == NAME_InterfaceProperty)
	{
		UObject* ObjectValue;
		Slot << ObjectValue;

		if (ObjectValue && !ObjectValue->IsA(PropertyClass))
		{
			UE_LOGF(LogClass, Warning, "Failed to convert interface property %ls of %ls from Interface to %ls", *this->GetName(), *Slot.GetUnderlyingArchive().GetArchiveName(), *PropertyClass->GetName());
			return EConvertFromTypeResult::CannotConvert;
		}

		SetPropertyValue_InContainer(Data, ObjectValue, Tag.ArrayIndex);
		CheckValidObject(GetPropertyValuePtr_InContainer(Data, Tag.ArrayIndex), ObjectValue, Defaults);
		return EConvertFromTypeResult::Converted;
	}

	return EConvertFromTypeResult::UseSerializeItem;
}
bool FObjectProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	// We never return Identical when duplicating for PIE because we want to be sure to serialize everything. An example is the LevelScriptActor being serialized against its CDO,
	// which contains actor references. We want to serialize those references so they are fixed up.
	if ((PortFlags & PPF_DuplicateForPIE) != 0)
	{
		return false;
	}

	const TObjectPtr<UObject> NullValue = nullptr;
	const TObjectPtr<UObject>& ObjectA = GetPropertyValue(A);
	const TObjectPtr<UObject>& ObjectB = B ? GetPropertyValue(B) : NullValue;

	if (ObjectA == ObjectB)
	{
		return true;
	}

	if (!ObjectA || !ObjectB)
	{
		return false;
	}

	// If a deep comparison is required, resolve the object handles and run the deep comparison logic
	// If a deep comparison is not required, avoid resolving the object handles because resolving declares
	// a cook dependency.
	if ((PortFlags & (PPF_DeepCompareInstances | PPF_DeepComparison)) != 0)
	{
		bool bPerformDeepComparison = true;
		if ((PortFlags & PPF_DeepCompareDSOsOnly) != 0)
		{
			UClass* ClassA = ObjectA.GetClass();
			UObject* DSO = ClassA ? ClassA->GetDefaultSubobjectByName(ObjectA.GetFName()) : nullptr;
			bPerformDeepComparison = DSO != nullptr;
		}
		if (bPerformDeepComparison && ObjectA.GetClass() == ObjectB.GetClass() && ObjectA.GetFName() == ObjectB.GetFName())
		{
			return FObjectPropertyBase::StaticIdentical(ObjectA.Get(), ObjectB.Get(), PortFlags);
		}
	}
	return false;
}

bool FObjectProperty::AllowCrossLevel() const
{
	return HasAnyPropertyFlags(CPF_InstancedReference);
}

void FObjectProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	TObjectPtr<UObject>* ObjectPtr = GetPropertyValuePtr(Value);
	if (UnderlyingArchive.IsObjectReferenceCollector())
	{
		TObjectPtr<UObject> CurrentValue = *ObjectPtr;
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_REMOTE_OBJECT_HANDLE
		if (HasAnyPropertyFlags(CPF_TObjectPtr))
		{
			FObjectPtr* Ptr = (FObjectPtr*)ObjectPtr;
			Slot << *Ptr;
		}
		else
#endif
		{
			// Serialize in place
			Slot << (*ObjectPtr);
		}
#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING) 
		if (!UnderlyingArchive.IsSaving() && !UnderlyingArchive.HasAnyPortFlags(PPF_AvoidRemoteObjectMigration))
		{
			CheckValidObject(Value, CurrentValue, Defaults);
		}
#endif
	}
	else
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_REMOTE_OBJECT_HANDLE
		if (HasAnyPropertyFlags(CPF_TObjectPtr) || UnderlyingArchive.HasAnyPortFlags(PPF_AvoidRemoteObjectMigration))
		{
			FObjectHandle OriginalHandle = ObjectPtr->GetHandle();
			Slot << *ObjectPtr;

			FObjectHandle CurrentHandle = ObjectPtr->GetHandle();
			if ((OriginalHandle != CurrentHandle) && IsObjectHandleResolved(CurrentHandle))
			{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
				if (ObjectPtr->IsA<ULinkerPlaceholderExportObject>())
				{
					//resolve the handle with no read to avoid trigger a handle read. 
					ULinkerPlaceholderExportObject* PlaceholderVal = static_cast<ULinkerPlaceholderExportObject*>(UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(CurrentHandle));
					PlaceholderVal->AddReferencingPropertyValue(this, Value);
				}
				else if (ObjectPtr->IsA<ULinkerPlaceholderClass>())
				{
					//resolve the handle with no read to avoid trigger a handle read. 
					ULinkerPlaceholderClass* PlaceholderClass = static_cast<ULinkerPlaceholderClass*>(UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(CurrentHandle));
					PlaceholderClass->AddReferencingPropertyValue(this, Value);
				}
				// NOTE: we don't remove this from CurrentValue if it is a 
				//       ULinkerPlaceholderExportObject; this is because this property 
				//       could be an array inner, and another member of that array (also 
				//       referenced through this property)... if this becomes a problem,
				//       then we could inc/decrement a ref count per referencing property 
				//
				// @TODO: if this becomes problematic (because ObjectValue doesn't match 
				//        this property's PropertyClass), then we could spawn another
				//        placeholder object (of PropertyClass's type), or use null; but
				//        we'd have to modify ULinkerPlaceholderExportObject::ReplaceReferencingObjectValues()
				//        to accommodate this (as it depends on finding itself as the set value)
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

				if (!UnderlyingArchive.IsSaving())
				{
					TObjectPtr<UObject> CurrentValue(*ObjectPtr);
					CheckValidObject(Value, CurrentValue, Defaults);
				}
			}
		}
		else
#endif
		{
			TObjectPtr<UObject> ObjectValuePtr = GetObjectPtrPropertyValue(Value);
			check(ObjectValuePtr.IsResolved());
			UObject* ObjectValue = UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(ObjectValuePtr.GetHandle());

			Slot << ObjectValue;

			TObjectPtr<UObject> CurrentValuePtr = GetObjectPtrPropertyValue(Value);
			check(CurrentValuePtr.IsResolved());
			UObject* CurrentValue = UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(CurrentValuePtr.GetHandle());
			PostSerializeObjectItem(UnderlyingArchive, Value, CurrentValue, ObjectValue, EObjectPropertyOptions::None, Defaults);
		}
	}
}

void FObjectProperty::PostSerializeObjectItem(FArchive& SerializingArchive, void* Value, UObject* CurrentValue, UObject* ObjectValue, EObjectPropertyOptions Options /*= EObjectPropertyOptions::None*/, const void* Defaults /*= nullptr*/) const
{
	// Query the thread context to see if we're serializing an IDO.
	auto IsSerializingInstanceDataObject = []()
	{
#if WITH_EDITORONLY_DATA
		const FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
		return SerializeContext ? SerializeContext->bImpersonateProperties : false;
#else
		return false;
#endif
	};

	// Make sure non-nullable properties don't end up with null values.
	if (!(Options & EObjectPropertyOptions::AllowNullValuesOnNonNullableProperty) &&
		!ObjectValue && HasAnyPropertyFlags(CPF_NonNullable) &&
		!SerializingArchive.IsSerializingDefaults() && // null values when Serializing CDOs are allowed, they will be fixed up later
		!SerializingArchive.IsSaving() && // Constructing new objects when saving may confuse package saving code (new import/export created after import/export collection pass)
		!SerializingArchive.IsTransacting() && // Don't create new objects when loading from the transaction buffer
		!SerializingArchive.IsCountingMemory() &&
		!IsSerializingInstanceDataObject()) // Don't create new objects when serializing IDOs, they instead mirror the instanced object's graph
	{
		UObject* DefaultValue = nullptr;

		using UE::CoreUObject::Private::ENonNullableBehavior;
		using UE::CoreUObject::Private::GetNonNullableBehavior;

		ENonNullableBehavior NonNullableBehavior = GetNonNullableBehavior();
		FString DefaultValueFailureReason;
		if (NonNullableBehavior == ENonNullableBehavior::CreateDefaultObjectIfPossible)
		{
			DefaultValue = ConstructDefaultObjectValueIfNecessary(CurrentValue, &DefaultValueFailureReason, Defaults);
			if (!DefaultValueFailureReason.IsEmpty())
			{
				DefaultValueFailureReason.InsertAt(0, TEXT(" as "));
			}
		}

		if (DefaultValue)
		{
			UE_LOGF(LogProperty, Warning,
				"Failed to serialize value for non-nullable property %ls (previously: %ls). Reference will be defaulted to %ls.",
				*GetFullName(),
				*GetFullNameSafe(CurrentValue),
				*GetFullNameSafe(DefaultValue)
			);
		}
		else if (NonNullableBehavior == ENonNullableBehavior::LogWarning)
		{
			UE_LOGF(LogProperty, Warning,
				"Failed to serialize value for non-nullable property %ls (previously: %ls). Reference will be nulled%ls - will cause a runtime error if accessed.",
				*GetFullName(),
				*GetFullNameSafe(CurrentValue),
				*DefaultValueFailureReason
			);
		}
		else
		{
			UE_LOGF(LogProperty, Error,
				"Failed to serialize value for non-nullable property %ls (previously: %ls). Reference will be nulled%ls - will cause a runtime error if accessed.",
				*GetFullName(),
				*GetFullNameSafe(CurrentValue),
				*DefaultValueFailureReason
			);
		}

		SetObjectPropertyValueUnchecked(Value, DefaultValue);
		ObjectValue = DefaultValue;
	}

	if (ObjectValue != CurrentValue)
	{
		SetObjectPropertyValue(Value, ObjectValue);

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
		if (ULinkerPlaceholderExportObject* PlaceholderVal = Cast<ULinkerPlaceholderExportObject>(ObjectValue))
		{
			PlaceholderVal->AddReferencingPropertyValue(this, Value);
		}
		else if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(ObjectValue))
		{
			PlaceholderClass->AddReferencingPropertyValue(this, Value);
		}
		// NOTE: we don't remove this from CurrentValue if it is a 
		//       ULinkerPlaceholderExportObject; this is because this property 
		//       could be an array inner, and another member of that array (also 
		//       referenced through this property)... if this becomes a problem,
		//       then we could inc/decrement a ref count per referencing property 
		//
		// @TODO: if this becomes problematic (because ObjectValue doesn't match 
		//        this property's PropertyClass), then we could spawn another
		//        placeholder object (of PropertyClass's type), or use null; but
		//        we'd have to modify ULinkerPlaceholderExportObject::ReplaceReferencingObjectValues()
		//        to accommodate this (as it depends on finding itself as the set value)
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

		if (!SerializingArchive.IsSaving() && !UE::IsSavingPackage(nullptr))
		{
			CheckValidObject(Value, ObjectValue, Defaults);
		}
	}
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
bool FObjectProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	const bool bUseRemoteObjectReference = Map && Map->IsUsingRemoteObjectReferences() && HasAnyPropertyFlags(CPF_TObjectPtr);
	if (bUseRemoteObjectReference)
	{
		TObjectPtr<UObject>* ObjectPtr = GetPropertyValuePtr(Data);

		FRemoteObjectReference RemoteReference(*ObjectPtr);
		bool bResult = false;
		RemoteReference.NetSerialize(Ar, Map, bResult);

		if (Ar.IsLoading())
		{
			*ObjectPtr = TObjectPtr<UObject>(RemoteReference.ToObjectPtr());
		}
		return bResult;
	}
	return Super::NetSerializeItem(Ar, Map, Data, MetaData);
}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

const TCHAR* FObjectProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const
{
	const TCHAR* Result = TFObjectPropertyBase<TObjectPtr<UObject>>::ImportText_Internal(Buffer, ContainerOrPropertyPtr, PropertyPointerType, OwnerObject, PortFlags, ErrorText);
	if (Result)
	{
		void* Data = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
		TObjectPtr<UObject> ObjectValue = GetObjectPtrPropertyValue(Data);
		CheckValidObject(Data, ObjectValue);

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING		
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(ObjectValue))
		{
			// we use this tracker mechanism to help record the instance that is
			// referencing the placeholder (so we can replace it later on fixup)
			FScopedPlaceholderContainerTracker ImportingObjTracker(OwnerObject);

			PlaceholderClass->AddReferencingPropertyValue(this, Data);
		}
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		else
		{
			// as far as we know, ULinkerPlaceholderClass is the only type we have to handle through ImportText()
			check(!FBlueprintSupport::IsDeferredDependencyPlaceholder(ObjectValue));
		}
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	}
	return Result;
}

uint32 FObjectProperty::GetValueTypeHashInternal(const void* Src) const
{
	return GetTypeHash(GetPropertyValue(Src));
}

UObject* FObjectProperty::GetObjectPropertyValue(const void* PropertyValueAddress) const
{
	return GetPropertyValue(PropertyValueAddress).Get();
}

TObjectPtr<UObject> FObjectProperty::GetObjectPtrPropertyValue(const void* PropertyValueAddress) const
{
	return GetPropertyValue(PropertyValueAddress);
}

UObject* FObjectProperty::GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	UObject* Result = nullptr;
	GetWrappedUObjectPtrValues<FObjectPtr>(&Result, ContainerAddress, EPropertyMemoryAccess::InContainer, ArrayIndex, 1);
	return Result;
}

TObjectPtr<UObject> FObjectProperty::GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	TObjectPtr<UObject> Result = nullptr;
	GetWrappedUObjectPtrValues<FObjectPtr>(&Result, ContainerAddress, EPropertyMemoryAccess::InContainer, ArrayIndex, 1);
	return Result;
}

void FObjectProperty::SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const
{
	SetPropertyValue(PropertyValueAddress, Ptr);
}

void FObjectProperty::SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const
{
	SetPropertyValue(PropertyValueAddress, Value);
}

void FObjectProperty::SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex) const
{
	SetWrappedUObjectPtrValues<FObjectPtr>(ContainerAddress, EPropertyMemoryAccess::InContainer, &Ptr, ArrayIndex, 1);
}

void FObjectProperty::SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex) const
{
	SetWrappedUObjectPtrValues<FObjectPtr>(ContainerAddress, EPropertyMemoryAccess::InContainer, &Value, ArrayIndex, 1);
}

void FObjectProperty::CopySingleValueToScriptVM(void* Dest, const void* Src) const
{
	*(FObjectPtr*)Dest = ((const FObjectPtr*)Src)->Get();
}

void FObjectProperty::CopySingleValueFromScriptVM(void* Dest, const void* Src) const
{
	*(FObjectPtr*)Dest = *(UObject**)Src;
}

void FObjectProperty::CopyCompleteValueToScriptVM(void* Dest, const void* Src) const
{
	GetWrappedUObjectPtrValues<FObjectPtr>((UObject**)Dest, Src, EPropertyMemoryAccess::Direct, 0, ArrayDim);
}

void FObjectProperty::CopyCompleteValueFromScriptVM(void* Dest, const void* Src) const
{
	SetWrappedUObjectPtrValues<FObjectPtr>(Dest, EPropertyMemoryAccess::Direct, (UObject**)Src, 0, ArrayDim);
}

void FObjectProperty::CopyCompleteValueToScriptVM_InContainer(void* OutValue, void const* InContainer) const
{
	GetWrappedUObjectPtrValues<FObjectPtr>((UObject**)OutValue, InContainer, EPropertyMemoryAccess::InContainer, 0, ArrayDim);
}

void FObjectProperty::CopyCompleteValueFromScriptVM_InContainer(void* OutContainer, void const* InValue) const
{
	SetWrappedUObjectPtrValues<FObjectPtr>(OutContainer, EPropertyMemoryAccess::InContainer, (UObject**)InValue, 0, ArrayDim);
}

void FObjectProperty::CopyValuesInternal(void* Dest, void const* Src, int32 Count) const
{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_REMOTE_OBJECT_HANDLE
	if (HasAllPropertyFlags(CPF_TObjectPtr))
	{
		for (int32 Index = 0; Index < Count; Index++)
		{
			GetPropertyValuePtr(Dest)[Index] = GetPropertyValuePtr(Src)[Index];
		}
	}
	else
#endif	
	{
		for (int32 Index = 0; Index < Count; Index++)
		{
			GetPropertyValuePtr(Dest)[Index] = GetPropertyValuePtr(Src)[Index].Get();
		}
	}
	
}
