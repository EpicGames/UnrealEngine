// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyOptional.h"

#include "Misc/Guid.h"
#include "Serialization/CustomVersion.h"
#include "String/LexFromString.h"
#include "UObject/GarbageCollectionSchema.h"
#include "UObject/LinkerLoad.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/PropertyHelper.h"
#include "UObject/PropertyTypeName.h"

DEFINE_LOG_CATEGORY_STATIC(LogOptionalProperty, Log, All);

IMPLEMENT_FIELD(FOptionalProperty)

// Custom serialization version for the legacy FOptionProperty
// Note that this is deprecated, and new versions should not be added.
// It's only used to support loading old versions of serialized data.
struct FLegacyFOptionPropertyCustomVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0,
		ConditionalSerializeValue = 1,
		AlwaysSavingIsSetForFObjectProperty = 2,
		RemoveIsValueNonNullablePointerHack = 3
	};

	const FGuid GUID;
	FCustomVersionRegistration Registration;
		
	FLegacyFOptionPropertyCustomVersion()
	: GUID{0x68C409FC, 0x70954986, 0x8963ACD2, 0xC4865183}
	, Registration(GUID, RemoveIsValueNonNullablePointerHack, TEXT("FPropertyOption"))
	{}
};
static FLegacyFOptionPropertyCustomVersion LegacyFOptionPropertyCustomVersion;
static const FString InitString = "__INIT__";

FOptionalProperty::FOptionalProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FOptionalProperty::FOptionalProperty(FFieldVariant InOwner, const UECodeGen_Private::FGenericPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

FOptionalProperty::~FOptionalProperty()
{
	if (ValueProperty)
	{
		delete ValueProperty;
		ValueProperty = nullptr;
	}
}

void FOptionalProperty::SetValueProperty(FProperty* InValueProperty)
{
	check(!ValueProperty);
	check(InValueProperty);
	ValueProperty = InValueProperty;
}

bool FOptionalProperty::HasIntrusiveUnsetOptionalState() const
{
	// TOptional<TOptional<T>> doesn't have an intrusive unset state even if the innermost type does.
	return false;
}

bool FOptionalProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && ValueProperty && ValueProperty->SameType(CastFieldChecked<FOptionalProperty>(Other)->ValueProperty);
}

void FOptionalProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
	if (ValueProperty)
	{
		ValueProperty->AddReferencedObjects(Collector);
	}
}

void FOptionalProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	SerializeSingleField(Ar, ValueProperty, this);
	checkSlow(ValueProperty);
}

void FOptionalProperty::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	checkSlow(ValueProperty);
	ValueProperty->GetPreloadDependencies(OutDeps);
}

void FOptionalProperty::PostDuplicate(const FField& InField)
{
	const FOptionalProperty& Source = static_cast<const FOptionalProperty&>(InField);
	ValueProperty = CastFieldChecked<FProperty>(FField::Duplicate(Source.ValueProperty, this));
	Super::PostDuplicate(InField);
}

FField* FOptionalProperty::GetInnerFieldByName(const FName& InName)
{
	checkSlow(ValueProperty);
	if (ValueProperty->GetFName() == InName)
	{
		return ValueProperty;
	}
	return nullptr;
}

void FOptionalProperty::GetInnerFields(TArray<FField*>& OutFields)
{
	checkSlow(ValueProperty);
	OutFields.Add(ValueProperty);
	ValueProperty->GetInnerFields(OutFields);
}

void FOptionalProperty::AddCppProperty(FProperty* Property)
{
	SetValueProperty(Property);
}

FString FOptionalProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const
{
	checkSlow(ValueProperty);
	if (ExtendedTypeText != nullptr)
	{
		FString ValueExtendedTypeText;
		FString ValueTypeText = ValueProperty->GetCPPType(&ValueExtendedTypeText, CPPExportFlags & ~CPPF_ArgumentOrReturnValue); // we won't consider the optional's inner value type to be "arguments or return values"
		*ExtendedTypeText = FString::Printf(TEXT("<%s%s>"), *ValueTypeText, *ValueExtendedTypeText);
	}
	return TEXT("TOptional");
}

FString FOptionalProperty::GetCPPMacroType(FString& ExtendedTypeText) const
{
	checkSlow(ValueProperty);
	ExtendedTypeText = ValueProperty->GetCPPType();
	return TEXT("TOPTIONAL");
}

void FOptionalProperty::LinkInternal(FArchive& Ar)
{
	checkSlow(ValueProperty);
	ValueProperty->Link(Ar);
	
	// After ValueProperty's size has been computed, compute the size of this property.
	SetElementSize(CalcSize());

	// Standard non-intrusive optional properties can always be initialized by zeroing memory.
	if (!ValueProperty->HasIntrusiveUnsetOptionalState())
	{
		PropertyFlags |= CPF_ZeroConstructor;
	}

	// Propagate CPF_NoDestructor, CPF_IsPlainOldData, and CPF_HasGetValueTypeHash from the value property.
	PropertyFlags |= (ValueProperty->PropertyFlags & (CPF_NoDestructor|CPF_IsPlainOldData|CPF_HasGetValueTypeHash));
	if (ValueProperty->ContainsInstancedObjectProperty())
	{
		PropertyFlags |= CPF_ContainsInstancedReference;
	}
}

bool FOptionalProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	checkSlow(ValueProperty);

	if (B == nullptr)
	{
		return !IsSet(A);
	}

	const bool bIsSetA = IsSet(A);
	const bool bIsSetB = IsSet(B);
	if (bIsSetA != bIsSetB)
	{
		return false;
	}
	else if (!bIsSetA)
	{
		return true;
	}

	return ValueProperty->Identical(GetValuePointerForRead(A), GetValuePointerForRead(B), PortFlags);
}

void FOptionalProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Data, void const* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	const bool bIsLoading = UnderlyingArchive.IsLoading();

	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	// Use an optional field slot to encode whether the optional was set or not.
	TOptional<FStructuredArchiveSlot> MaybeValueSlot = Record.TryEnterField(TEXT("Value"), IsSet(Data));
	if (MaybeValueSlot.IsSet())
	{
		FStructuredArchive::FSlot ValueSlot = MaybeValueSlot.GetValue();
		
		const void* ValueDefaults = Defaults ? GetValuePointerForReadIfSet(Defaults) : nullptr;

		if (Slot.GetArchiveState().UseUnversionedPropertySerialization() ||
			UnderlyingArchive.UEVer() >= EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
		{
			// Simply serialize the inner value if using unversioned property serialization.
			void* ValueData = bIsLoading
				? MarkSetAndGetInitializedValuePointerToReplace(Data)
				: GetValuePointerForReadOrReplace(Data);
			GetValueProperty()->SerializeItem(ValueSlot, ValueData, ValueDefaults);
		}
		else if (bIsLoading)
		{
			FPropertyTag ValueTag;

			// Deserializing a FPropertyTag from text won't deserialize the ArrayIndex, leaving it uninitialized as INDEX_NONE.
			ValueTag.ArrayIndex = 0;

			// Serialize the value's tag.
			ValueSlot << ValueTag;

			if (UE::FPropertyTypeName NewTypeName = ApplyRedirectsToPropertyType(ValueTag.GetType(), ValueProperty); !NewTypeName.IsEmpty())
			{
				ValueTag.SetType(NewTypeName);
			}

			// Deserialize/convert the value.
			void* ValueData = MarkSetAndGetInitializedValuePointerToReplace(Data);

			const int64 ValueStartOffset = UnderlyingArchive.Tell();
			bool bSuccessfullyDeserialized = false;
			switch (GetValueProperty()->ConvertFromType(ValueTag, ValueSlot, static_cast<uint8*>(ValueData), GetOwnerStruct(), static_cast<const uint8*>(ValueDefaults)))
			{
				case EConvertFromTypeResult::Converted:
				case EConvertFromTypeResult::Serialized:
					bSuccessfullyDeserialized = true;
					break;
				case EConvertFromTypeResult::UseSerializeItem:
					if (ValueTag.Type == GetValueProperty()->GetID())
					{
						ValueTag.SerializeTaggedProperty(ValueSlot, GetValueProperty(), (uint8*)ValueData, (const uint8*)ValueDefaults);
						bSuccessfullyDeserialized = !UnderlyingArchive.IsCriticalError();
					}
					break;
				case EConvertFromTypeResult::CannotConvert:
					break;
				default:
					check(false);
			}

			if (!bSuccessfullyDeserialized)
			{
				UnderlyingArchive.Seek(ValueStartOffset + ValueTag.Size);
				
				// If the deserialization or conversion of the value failed, behave as if an unset optional value was deserialized.
				MarkUnset(Data);
			}
			else
			{
				const int64 NumLoadedBytes = UnderlyingArchive.Tell() - ValueStartOffset;
				checkf(
					ValueTag.Size == NumLoadedBytes,
					TEXT("Value tagged with ID %s was saved as %i bytes, but loading it as a %s loaded %" INT64_FMT " bytes."),
					*ValueTag.Type.ToString(),
					ValueTag.Size,
					*GetValueProperty()->GetClass()->GetFName().ToString(),
					NumLoadedBytes);
			}
		}
		else
		{
			void* ValueData = GetValuePointerForReadOrReplace(Data);

			// Construct and serialize a tag for the value.
			FPropertyTag ValueTag(GetValueProperty(), 0, static_cast<uint8*>(ValueData));
			ValueSlot << ValueTag;

			// Serialize the value.
			int64 ValueStartOffset = UnderlyingArchive.Tell();
			ValueTag.SerializeTaggedProperty(ValueSlot, GetValueProperty(), static_cast<uint8*>(ValueData), static_cast<const uint8*>(ValueDefaults));
			
			// If saving to a non-text archive, save the size of the serialized value so it can be skipped over on load if it's the wrong type.
			const int64 ValueEndOffset = UnderlyingArchive.Tell();
			ValueTag.Size = IntCastChecked<int32>(ValueEndOffset - ValueStartOffset);
			if (ValueTag.Size > 0 && !UnderlyingArchive.IsTextFormat())
			{
				// The tag serialization set ValueTag.SizeOffset to the archive offset where a placeholder size field was serialized.
				// Now that we know the size of the serialized value, seek back to the size field and serialize the true value over the placeholder.
				UnderlyingArchive.Seek(ValueTag.SizeOffset);
				UnderlyingArchive << ValueTag.Size;
				UnderlyingArchive.Seek(ValueEndOffset);
			}
		}
	}
	else if (bIsLoading)
	{
		// If the field wasn't present, the serialized optional was unset.
		MarkUnset(Data);
	}
}

bool FOptionalProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData /*= nullptr*/) const
{
	bool bSuccess = true;
	bool bIsLoading = Ar.IsLoading();

	uint8 IsSetValue = IsSet(Data);
	Ar.SerializeBits(&IsSetValue, 1);

	if (IsSetValue > 0)
	{
		void* ValueData = bIsLoading
			? MarkSetAndGetInitializedValuePointerToReplace(Data)
			: GetValuePointerForReadOrReplace(Data);

		// The NetSerializeItem code path is not supported by all property types and they will not be supported within replicated optionals yet either
		bSuccess = ValueProperty->NetSerializeItem(Ar, Map, ValueData, MetaData);
	}
	
	if (bIsLoading && (!bSuccess || !IsSetValue))
	{
		MarkUnset(Data);
	}

	return bSuccess;
}

bool FOptionalProperty::SupportsNetSharedSerialization() const
{
	return ValueProperty->SupportsNetSharedSerialization();
}

void FOptionalProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	checkSlow(ValueProperty);
	void* OptionalValuePointer = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
	// Output (Value) or ()
	ValueStr += TEXT("(");
	if (const void* ValuePointer = GetValuePointerForReadIfSet(OptionalValuePointer))
	{
		ValueProperty->ExportTextItem_Direct(ValueStr, ValuePointer, DefaultValue ? GetValuePointerForReadIfSet(DefaultValue) : nullptr, Parent, PortFlags, ExportRootScope);

		// If we got no value back from our ValueProperty's text export but we are SET (ie: an empty array, map, set, etc), 
		// we set `__init__` to the exported text to so we know to do an initialize on importing (See ImportText_Internal).
		if (ValueStr == TEXT("("))
		{
			ValueStr = InitString;
			return;
		}
	}
	ValueStr += TEXT(")");
}

const TCHAR* FOptionalProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	checkSlow(ValueProperty);
	void* Data = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);

	if (!Buffer)
	{
		return nullptr;
	}

	// INIT
	if (FString(Buffer) == InitString)
	{
		// We are set but have no text to import for our value property.
		MarkUnset(Data);
		MarkSetAndGetInitializedValuePointerToReplace(Data);
		return Buffer;
	}

	if (*Buffer++ != TCHAR('('))
	{
		return nullptr;
	}
	
	SkipWhitespace(Buffer);

	// Check if there is a value
	if (*Buffer != TCHAR(')'))
	{
		// Yes, parse it
		void* ValuePointer = MarkSetAndGetInitializedValuePointerToReplace(Data);
		Buffer = ValueProperty->ImportText_Direct(Buffer, ValuePointer, Parent, PortFlags, ErrorText);
		if (!Buffer)
		{
			MarkUnset(Data);
			return nullptr;
		}
	}
	else
	{
		MarkUnset(Data);
	}
	
	SkipWhitespace(Buffer);

	if (*Buffer++ != TCHAR(')'))
	{
		return nullptr;
	}

	return Buffer;
}

void FOptionalProperty::CopyValuesInternal(void* Dest, void const* Src, int32 Count) const
{
	int32 Size = GetSize();
	for (int32 Index = 0; Index < Count; ++Index, (uint8*&)Dest += Size, (const uint8*&)Src += Size)
	{
		if (const void* SrcValue = GetValuePointerForReadIfSet(Src))
		{
			void* DestValue = MarkSetAndGetInitializedValuePointerToReplace(Dest);
			ValueProperty->CopySingleValue(DestValue, SrcValue);
		}
		else
		{
			MarkUnset(Dest);
		}
	}
}

void FOptionalProperty::ClearValueInternal(void* Data) const
{
	MarkUnset(Data);
}

void FOptionalProperty::InitializeValueInternal(void* Data) const
{
	if (ValueProperty->HasIntrusiveUnsetOptionalState())
	{
		ValueProperty->InitializeIntrusiveUnsetOptionalValue(Data);
	}
	else
	{
		*GetIsSetPointer(Data) = false;
	}
}

void FOptionalProperty::DestroyValueInternal(void* Data) const
{
	MarkUnset(Data);
	if (ValueProperty->HasIntrusiveUnsetOptionalState())
	{
		ValueProperty->DestroyValue(Data);
	}
}

bool FOptionalProperty::ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const
{
	check(ValueProperty);
	return ValueProperty->ContainsFinishDestroy(EncounteredStructProps);
}

void FOptionalProperty::FinishDestroyInternal( void* Data ) const
{
	check(ValueProperty);
	if (void* Value = GetValuePointerForReplaceIfSet(Data))
	{
		ValueProperty->FinishDestroy(Value);
	}
}

void FOptionalProperty::InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, struct FObjectInstancingGraph* InstanceGraph)
{
	if (Data
		&& IsSet(Data)
		&& (ValueProperty->ContainsInstancedObjectProperty() || InOwner->GetClass()->ShouldUseDynamicSubobjectInstancing()))
	{
		ValueProperty->InstanceSubobjects(Data, DefaultData ? GetValuePointerForReadIfSet(DefaultData) : nullptr, InOwner, InstanceGraph);
	}
}

int32 FOptionalProperty::GetMinAlignment() const
{
	return ValueProperty->GetMinAlignment();
}

EConvertFromTypeResult FOptionalProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* ContainerData, const UStruct* DefaultsStruct, const uint8* DefaultsContainer)
{
	if (Slot.GetArchiveState().UEVer() >= EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
	{
		if (CanSerializeFromTypeName(Tag.GetType()))
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}

		FPropertyTag ValueTag = Tag;
		TOptional<FStructuredArchive::FSlot> MaybeValueSlot;
		if (Tag.Type == NAME_OptionalProperty)
		{
			FStructuredArchive::FRecord Record = Slot.EnterRecord();
			MaybeValueSlot = Record.TryEnterField(TEXT("Value"), /*bEnterForSaving*/ false);
			if (!MaybeValueSlot)
			{
				MarkUnset(ContainerPtrToValuePtr<void>(ContainerData));
				return EConvertFromTypeResult::Converted;
			}
			ValueTag.SetProperty(GetValueProperty());
			ValueTag.SetType(Tag.GetType().GetParameter(0));

			// Mimic FPropertyTag::SerializeTaggedProperty storing bool values in the tag because implementations of ConvertFromType expect it there.
			if (ValueTag.Type == NAME_BoolProperty)
			{
				*MaybeValueSlot << ValueTag.BoolVal;
			}
		}

		FStructuredArchive::FSlot ValueSlot = MaybeValueSlot.Get(Slot);
		uint8* ValueData = (uint8*)MarkSetAndGetInitializedValuePointerToReplace(ContainerPtrToValuePtr<void>(ContainerData));
		switch (GetValueProperty()->ConvertFromType(ValueTag, ValueSlot, ValueData, nullptr, nullptr))
		{
		case EConvertFromTypeResult::Converted:
			return EConvertFromTypeResult::Converted;
		case EConvertFromTypeResult::Serialized:
			return EConvertFromTypeResult::Serialized;
		case EConvertFromTypeResult::CannotConvert:
			return EConvertFromTypeResult::CannotConvert;
		case EConvertFromTypeResult::UseSerializeItem:
			if (ValueTag.Type == GetValueProperty()->GetID())
			{
				GetValueProperty()->SerializeItem(ValueSlot, ValueData);
				return EConvertFromTypeResult::Serialized;
			}
			return EConvertFromTypeResult::CannotConvert;
		default:
			checkNoEntry();
			return EConvertFromTypeResult::CannotConvert;
		}
	}

	static const FName NAME_OptionProperty("OptionProperty");
	if (Tag.Type != NAME_OptionProperty)
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}
	const int32 FPropertyOptionVersion = Slot.GetArchiveState().CustomVer(LegacyFOptionPropertyCustomVersion.GUID);

	checkSlow(ValueProperty);

	void* Data = ContainerPtrToValuePtr<void>(ContainerData, Tag.ArrayIndex);
	const void* Defaults = DefaultsStruct ? ContainerPtrToValuePtrForDefaults<void>(DefaultsStruct, DefaultsContainer, Tag.ArrayIndex) : nullptr;
	
	bool bIsValueNonNullablePointer = ValueProperty->HasAllPropertyFlags(CPF_NonNullable);
	if (FPropertyOptionVersion < FLegacyFOptionPropertyCustomVersion::RemoveIsValueNonNullablePointerHack)
	{
		bIsValueNonNullablePointer =
			  ValueProperty->IsA<FClassProperty>() ? false
			: ValueProperty->IsA<FObjectProperty>() ? true
			: (ValueProperty->GetPropertyFlags() & CPF_NonNullable) != 0;
	}

	if (bIsValueNonNullablePointer && FPropertyOptionVersion < FLegacyFOptionPropertyCustomVersion::AlwaysSavingIsSetForFObjectProperty)
	{
		// Code below is duplicated from FObjectProperty::SerializeItem with the exception of passing AllowNullValuesOnNonNullableProperty to PostSerializeObjectItem
		// in order to maintain backwards compatibility with packages saved prior to non-nullable properties properly constructing new objects when failing to load
		// valid (non-null) object values. We also don't need to make sure the duplicated code matches the most recent version of FObjectProperty::SerializeItem
		// because we only run the duplicated code for packages created before the introduction of FLegacyFOptionPropertyCustomVersion::AlwaysSavingIsSetForFObjectProperty.

		// We're also not duplicating 'if (UnderlyingArchive.IsObjectReferenceCollector())' code path from FObjectProperty::SerializeItem because reference collectors
		// always have the latest custom version set. Sanity check below.
		checkf(!Slot.GetArchiveState().IsObjectReferenceCollector(), TEXT("Reference collector archive (%s) serializing with an old FLegacyFOptionPropertyCustomVersion (%d)"),
			*Slot.GetArchiveState().GetArchiveName(), FPropertyOptionVersion);

		// Another sanity check since the duplicated code comes from FObjectProperty we are assuming ValueProperty is in fact an FObjectProperty or FClassProperty which shares SerializeItem code with FObjectProperty
		// Note that we only support FObjectProperty and FClassProperty because these are the only two object referencing property types that have so far been used with FOptionalProperty. 
		checkf(ValueProperty && (ValueProperty->GetClass() == FObjectProperty::StaticClass() || ValueProperty->GetClass() == FClassProperty::StaticClass()), 
			TEXT("Unexpected value property type when deserializing optional non-nullable property %s: %s"),
			*GetFullName(), ValueProperty ? *ValueProperty->GetClass()->GetName() : TEXT("None"));

		FObjectProperty* ObjectValueProperty = CastFieldChecked<FObjectProperty>(ValueProperty);

		// Begin duplicated code (no need to keep up to date with FObjectProperty::SerializeItem code, see comment above)
		TObjectPtr<UObject> ObjectValuePtr = ObjectValueProperty->GetObjectPtrPropertyValue(Data);
		check(ObjectValuePtr.IsResolved());
		UObject* ObjectValue = UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(ObjectValuePtr.GetHandle());

		Slot << ObjectValue;

		TObjectPtr<UObject> CurrentValuePtr = ObjectValueProperty->GetObjectPtrPropertyValue(Data);
		check(CurrentValuePtr.IsResolved());
		UObject* CurrentValue = UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(CurrentValuePtr.GetHandle());

		ObjectValueProperty->PostSerializeObjectItem(Slot.GetUnderlyingArchive(), Data, CurrentValue, ObjectValue, 
			EObjectPropertyOptions::AllowNullValuesOnNonNullableProperty); // The original code was passing EObjectPropertySerializeOptions::None here
		// End duplicated code
	}
	else if (FPropertyOptionVersion < FLegacyFOptionPropertyCustomVersion::ConditionalSerializeValue)
	{
		// Legacy: Serialize the inner FProperty whether it is set or not.
		FStructuredArchive::FStream Stream = Slot.EnterStream();
		bool bLoadedIsSet = false;
		Stream.EnterElement() << bLoadedIsSet;
		void* ValueData = MarkSetAndGetInitializedValuePointerToReplace(Data);
		ValueProperty->SerializeItem(Stream.EnterElement(), ValueData, Defaults ? GetValuePointerForReadIfSet(Defaults) : nullptr);
		if (!bLoadedIsSet)
		{
			MarkUnset(Data);
		}
	}
	else
	{
		FStructuredArchive::FStream Stream = Slot.EnterStream();
		bool bIsSetValue = IsSet(Data);
		Stream.EnterElement() << bIsSetValue;

		if (bIsSetValue)
		{
			void* ValuePointer = MarkSetAndGetInitializedValuePointerToReplace(Data);
			ValueProperty->SerializeItem(Stream.EnterElement(), ValuePointer, Defaults ? GetValuePointerForReadIfSet(Defaults) : nullptr);
		}
		else
		{
			MarkUnset(Data);
		}
	}
	return EConvertFromTypeResult::Converted;
}

uint32 FOptionalProperty::GetValueTypeHashInternal(const void* Src) const
{
	// If the option is set, return the value's hash, otherwise return 0.
	if (const void* ValuePointer = GetValuePointerForReadIfSet(Src))
	{
		return ValueProperty->GetValueTypeHash(ValuePointer);
	}
	else
	{
		return 0;
	}
}

bool FOptionalProperty::UseBinaryOrNativeSerialization(const FArchive& Ar) const
{
	if (Super::UseBinaryOrNativeSerialization(Ar))
	{
		return true;
	}

	const FProperty* LocalValueProperty = ValueProperty;
	check(LocalValueProperty);
	return LocalValueProperty->UseBinaryOrNativeSerialization(Ar);
}

bool FOptionalProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	if (!Super::LoadTypeName(Type, Tag))
	{
		return false;
	}

	const UE::FPropertyTypeName ValueType = Type.GetParameter(0);
	FField* Field = FField::TryConstruct(ValueType.GetName(), this, GetFName());
	if (FProperty* Property = CastField<FProperty>(Field); Property && Property->LoadTypeName(ValueType, Tag))
	{
		ValueProperty = Property;
		return true;
	}
	delete Field;
	return false;
}

void FOptionalProperty::SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const
{
	Super::SaveTypeName(Type);

	const FProperty* LocalValueProperty = ValueProperty;
	check(LocalValueProperty);
	Type.BeginParameters();
	LocalValueProperty->SaveTypeName(Type);
	Type.EndParameters();
}

bool FOptionalProperty::CanSerializeFromTypeName(UE::FPropertyTypeName Type) const
{
	if (!Super::CanSerializeFromTypeName(Type))
	{
		return false;
	}

	const FProperty* LocalValueProperty = ValueProperty;
	check(LocalValueProperty);
	return LocalValueProperty->CanSerializeFromTypeName(Type.GetParameter(0));
}

EPropertyVisitorControlFlow FOptionalProperty::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const
{
	// Indicate in the path that this property contains inner properties
	Context.Path.Top().bContainsInnerProperties = IsSet(Context.Data.PropertyData);

	EPropertyVisitorControlFlow RetVal = Super::Visit(Context, InFunc);

	if (RetVal == EPropertyVisitorControlFlow::StepInto && IsSet(Context.Data.PropertyData))
	{
		if(!IsSet(Context.Data.PropertyData))
		{
			// There is nothing to StepInto, so continue to next property
			return EPropertyVisitorControlFlow::StepOver;
		}
		checkf(ValueProperty, TEXT("Expecting a valid property value"));

		FPropertyVisitorScope Scope(Context.Path, FPropertyVisitorInfo(ValueProperty));
		RetVal = ValueProperty->Visit(Context, InFunc);
	}
	return RetVal;
}

void* FOptionalProperty::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	if (Info.Property == ValueProperty && IsSet(Data))
	{
		return Data;
	}

	return nullptr;
}
