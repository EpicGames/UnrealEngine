// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyPathFunctions.h"

#include "UObject/Class.h"
#include "UObject/PropertyOptional.h"
#include "UObject/PropertyPathName.h"
#include "UObject/UnrealType.h"
#include "Templates/FunctionWithContext.h"

namespace UE
{

const FName NAME_Key(ANSITEXTVIEW("Key"));
const FName NAME_Value(ANSITEXTVIEW("Value"));

FProperty* FindPropertyByNameAndTypeName(const UStruct* Struct, FName Name, FPropertyTypeName TypeName)
{
	for (FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (Property->GetFName() == Name && Property->CanSerializeFromTypeName(TypeName))
		{
			return Property;
		}
	}
	return nullptr;
}

inline static const UStruct* FindStructFromProperty(const FProperty* Property)
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		return StructProperty->Struct;
	}
	return nullptr;
}

FPropertyValueInContainer TryResolvePropertyPath(const FPropertyPathName& Path, UObject* Object)
{
	FPropertyPathNameResolver Resolver(Path, Object);
	return Resolver.Value;
}

FPropertyPathNameResolver::FPropertyPathNameResolver(const FPropertyPathName& Path, UObject* Object)
	: Path(Path)
	, Object(Object)
{
	Resolve();
}

FPropertyPathNameResolver& FPropertyPathNameResolver::Next()
{
	while (!ActiveContainerIterators.IsEmpty())
	{
		FContainerIterator& Info = ActiveContainerIterators.Last();
		++Info.CurIndex;
		if (Info.Property->ArrayDim > 1) // static array
		{
			if (Info.CurIndex < Info.Property->ArrayDim)
			{
				return Resolve();
			}
		}
		else
		{
			void* DataPtr = Info.Property->ContainerPtrToValuePtr<uint8>(Info.NextContainer);
			if (const FArrayProperty* AsArrayProperty = CastField<FArrayProperty>(Info.Property))
			{
				FScriptArrayHelper Helper(AsArrayProperty, DataPtr);
				if (Info.CurIndex < Helper.Num())
				{
					return Resolve();
				}
			}
			else if (const FSetProperty* AsSetProperty = CastField<FSetProperty>(Info.Property))
			{
				FScriptSetHelper Helper(AsSetProperty, DataPtr);
				if (Info.CurIndex < Helper.Num())
				{
					return Resolve();
				}
			}
			else if (const FMapProperty* AsMapProperty = CastField<FMapProperty>(Info.Property))
			{
				FScriptMapHelper Helper(AsMapProperty, DataPtr);
				if (Info.CurIndex < Helper.Num())
				{
					return Resolve();
				}
			}
		}

		ActiveContainerIterators.Pop();
	}
	Value = {};
	return *this;
}

bool FPropertyPathNameResolver::BuildChangeEvent(TSharedPtr<FPropertyChangedChainEvent>& OutEvent, int32 ChangeType)
{
	if (Value)
	{
		EventArrayIndices.Empty();
		EventArrayIndices.Emplace();
		for (int32 PathIndex = 0, SegmentCount = Path.GetSegmentCount(); PathIndex < SegmentCount; ++PathIndex)
		{
			FPropertyPathNameSegment Segment = Path.GetSegment(PathIndex);
			if (Segment.Index != INDEX_NONE)
			{
				EventArrayIndices[0].Add(Segment.Name.ToString(), Segment.Index);
			}
		}
		for (const FContainerIterator& WildcardInfo : ActiveContainerIterators)
		{
			EventArrayIndices[0].Add(WildcardInfo.Property->GetName(), WildcardInfo.CurIndex);
		}

		FPropertyChangedEvent InnerEvent(const_cast<FProperty*>(Value.Property), ChangeType, { Object });
		InnerEvent.ObjectIteratorIndex = 0;
		OutEvent = MakeShared<FPropertyChangedChainEvent>(EventChain, InnerEvent);
		OutEvent->SetArrayIndexPerObject(EventArrayIndices);
		return true;
	}
	return false;
}

void FPropertyPathNameResolver::Reset()
{
	ActiveContainerIterators.Reset();
	Resolve();
}

FPropertyPathNameResolver& FPropertyPathNameResolver::Resolve()
{
	const UStruct* NextStruct;
	void* NextContainer;
	int32 PathIndex;
	if (ActiveContainerIterators.IsEmpty())
	{
		NextStruct = Object ? Object->GetClass() : nullptr;
		NextContainer = Object;
		PathIndex = 0;
		EventChain.Empty();
		ContainerValues.Reset();
	}
	else
	{
		// jump to the current container being entered
		NextStruct = ActiveContainerIterators.Last().NextStruct;
		NextContainer = ActiveContainerIterators.Last().NextContainer;
		PathIndex = ActiveContainerIterators.Last().PathIndex;
		while (EventChain.Num() > ActiveContainerIterators.Last().ChainLength)
		{
			EventChain.RemoveNode(EventChain.GetTail());
		}
		int32 NewContainerValuesNum = ActiveContainerIterators.Last().ContainerValuesNum;
		if (ContainerValues.Num() > NewContainerValuesNum)
		{
			ContainerValues.RemoveAt(NewContainerValuesNum, ContainerValues.Num() - NewContainerValuesNum, EAllowShrinking::No);
		}
		
	}

	for (const int32 SegmentCount = Path.GetSegmentCount(); PathIndex < SegmentCount; ++PathIndex)
	{
		// Fail if the previous segment failed to resolve the struct or container for this segment.
		if (!NextStruct || !NextContainer)
		{
			Value = {};
			return *this;
		}

		FPropertyPathNameSegment Segment = Path.GetSegment(PathIndex);
		const FProperty* Property = FindPropertyByNameAndTypeName(NextStruct, Segment.Name, Segment.Type);

		if (!Property)
		{
			Value = {};
			return *this;
		}

		// Support for wildcard Indices
		if (PathIndex < SegmentCount - 1) // don't use wildcards for the last element in the path
		{
			if (Segment.Index == INDEX_NONE)
			{
				if (!ActiveContainerIterators.IsEmpty() && ActiveContainerIterators.Last().PathIndex == PathIndex)
				{
					// wildcard element detected! use the current index!
					Segment.Index = ActiveContainerIterators.Last().CurIndex;
				}
				else if (Property->ArrayDim > 1 || Property->IsA<FArrayProperty>() || Property->IsA<FSetProperty>() || Property->IsA<FMapProperty>())
				{
					Segment.Index = 0;
					ActiveContainerIterators.Emplace(PathIndex, Property, NextContainer, NextStruct, EventChain.Num(), ContainerValues.Num(), Segment.Index);
				}
			}
		}

		if (Value)
		{
			ContainerValues.Push(Value);
		}
		Value.Property = Property;
		Value.Struct = NextStruct;
		Value.Container = NextContainer;
		Value.ArrayIndex = 0;

		// Check the bounds and assign the index for static arrays.
		if (Property->ArrayDim > 1)
		{
			if (Segment.Index < 0 || Segment.Index >= Property->ArrayDim)
			{
				Value = {};
				return *this;
			}
			Value.ArrayIndex = Segment.Index;
		}

		// Resolve the struct and container for the next segment if there is one.
		NextStruct = FindStructFromProperty(Property);
		NextContainer = Property->ContainerPtrToValuePtr<uint8>(NextContainer, Value.ArrayIndex);

		// Resolve optionals to the struct and container of their value if they have one.
		if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
		{
			FOptionalPropertyLayout OptionalHelper(OptionalProperty->GetValueProperty());
			NextStruct = FindStructFromProperty(OptionalHelper.GetValueProperty());
			NextContainer = OptionalHelper.GetValuePointerForReadOrReplaceIfSet(NextContainer);
			if (!NextContainer && PathIndex < SegmentCount - 1)
			{
				return Next(); // this optional is unset. Try the next one in the list of wildcards
			}
		}

		EventChain.AddTail(const_cast<FProperty*>(Value.Property));

		// Scalar values and static containers are finished resolving.
		if (Property->ArrayDim > 1 || Segment.Index == INDEX_NONE)
		{
			continue;
		}

		// Resolve dynamic containers, which have no struct when resolving directly to an element.
		Value.Struct = nullptr;

		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			FScriptArrayHelper ArrayHelper(ArrayProperty, NextContainer);
			if (!ArrayHelper.IsValidIndex(Segment.Index))
			{
				return Next(); // this index isn't in the array. Try the next one in the list of wildcards
			}

			NextStruct = FindStructFromProperty(ArrayProperty->Inner);
			NextContainer = ArrayHelper.GetRawPtr(Segment.Index);
			Value.Property = ArrayProperty->Inner;
			Value.Container = NextContainer;
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
		{
			FScriptSetHelper SetHelper(SetProperty, NextContainer);
			NextStruct = FindStructFromProperty(SetProperty->ElementProp);
			NextContainer = SetHelper.FindNthElementPtr(Segment.Index);
			if (!NextContainer)
			{
				return Next(); // this index isn't in the set. Try the next one in the list of wildcards
			}
			Value.Property = SetProperty->ElementProp;
			Value.Container = NextContainer;
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
		{
			FScriptMapHelper MapHelper(MapProperty, NextContainer);
			if (++PathIndex == SegmentCount)
			{
				Value = {};
				return *this;
			}

			// A Key or Value segment with no type or index is required to distinguish which property to resolve.
			const FPropertyPathNameSegment MapSegment = Path.GetSegment(PathIndex);
			if (!MapSegment.Type.IsEmpty() || MapSegment.Index != INDEX_NONE)
			{
				Value = {};
				return *this;
			}

			if (MapSegment.Name == NAME_Key)
			{
				NextStruct = FindStructFromProperty(MapProperty->KeyProp);
				NextContainer = MapHelper.FindNthKeyPtr(Segment.Index);
				if (!NextContainer)
				{
					return Next(); // this index isn't in the map. Try the next one in the list of wildcards
				}
				Value.Property = MapProperty->KeyProp;
			}
			else if (MapSegment.Name == NAME_Value)
			{
				NextStruct = FindStructFromProperty(MapProperty->ValueProp);
				NextContainer = MapHelper.FindNthValuePtr(Segment.Index);
				if (!NextContainer)
				{
					return Next(); // this index isn't in the map. Try the next one in the list of wildcards
				}
				Value.Property = MapProperty->ValueProp;
			}
			else
			{
				Value = {};
				return *this;
			}

			// The key and value property both have an offset relative to the pair.
			Value.Container = MapHelper.FindNthPairPtr(Segment.Index);
		}
		else
		{
			Value = {};
			return *this;
		}
	}
	return *this;
}



} // UE
