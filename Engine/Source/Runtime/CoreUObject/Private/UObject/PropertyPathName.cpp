// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyPathName.h"

#include "Misc/StringBuilder.h"
#include "Templates/TypeHash.h"
#include "Internationalization/Regex.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "UObject/PackedObjectRef.h"

namespace UE
{

inline bool FPropertyPathName::FSegment::operator==(const FSegment& Segment) const
{
	return NameWithIndex == Segment.NameWithIndex && Type == Segment.Type;
}

inline bool FPropertyPathName::FSegment::operator<(const FSegment& Segment) const
{
	return Compare(Segment) < 0;
}

inline int32 FPropertyPathName::FSegment::Compare(const FSegment& Segment) const
{
	if (int32 CompareNameWithIndex = NameWithIndex.Compare(Segment.NameWithIndex))
	{
		return CompareNameWithIndex;
	}
	return (Type == Segment.Type) ? 0 : (Type < Segment.Type ? -1 : 1);
}

bool FPropertyPathName::operator==(const FPropertyPathName& Path) const
{
	const int32 SegmentCount = Segments.Num();
	if (SegmentCount != Path.Segments.Num())
	{
		return false;
	}

	for (int32 SegmentIndex = SegmentCount - 1; SegmentIndex >= 0; --SegmentIndex)
	{
		if (!(Segments[SegmentIndex] == Path.Segments[SegmentIndex]))
		{
			return false;
		}
	}

	return true;
}

bool FPropertyPathName::operator<(const FPropertyPathName& Path) const
{
	const int32 SegmentCountA = Segments.Num();
	const int32 SegmentCountB = Path.Segments.Num();
	const int32 SegmentCountMin = FPlatformMath::Min(SegmentCountA, SegmentCountB);

	for (int32 SegmentIndex = 0; SegmentIndex < SegmentCountMin; ++SegmentIndex)
	{
		if (const int32 Compare = Segments[SegmentIndex].Compare(Path.Segments[SegmentIndex]))
		{
			return Compare < 0;
		}
	}

	return SegmentCountA < SegmentCountB;
}

void FPropertyPathName::ToString(FStringBuilderBase& Out, FStringView Separator) const
{
	bool bFirst = true;
	for (const FSegment& Segment : Segments)
	{
		if (bFirst)
		{
			bFirst = false;
		}
		else
		{
			Out.Append(Separator);
		}

		FPropertyPathNameSegment UnpackedSegment = Segment.Unpack();
		Out << UnpackedSegment.Name;

		if (const int32 Index = UnpackedSegment.Index; Index != INDEX_NONE)
		{
			Out << TEXT('[') << Index << TEXT(']');
		}

		if (!UnpackedSegment.Type.IsEmpty())
		{
			Out << TEXTVIEW(" (") << UnpackedSegment.Type << TEXT(')');
		}
	}
}

TSharedRef<FJsonValue> FPropertyPathName::ToJsonValue() const
{
	TArray<TSharedPtr<FJsonValue>> PathArray;
	for (const FSegment& Segment : Segments)
	{
		FPropertyPathNameSegment UnpackedSegment = Segment.Unpack();
		PathArray.Add(UnpackedSegment.ToJsonValue());
	}

	return StaticCastSharedRef<FJsonValue>(MakeShared<FJsonValueArray>(PathArray));
}

bool FPropertyPathName::FromJsonValue(const TSharedPtr<FJsonValue>& JsonValue, FPropertyPathName& OutPath)
{
	TArray<TSharedPtr<FJsonValue>>* PathArray = nullptr;
	if (JsonValue && JsonValue->TryGetArray(PathArray) && PathArray)
	{
		for (TSharedPtr<FJsonValue> SegmentValue : *PathArray)
		{
			FPropertyPathNameSegment UnpackedSegment;
			if (FPropertyPathNameSegment::FromJsonValue(SegmentValue, UnpackedSegment))
			{
				OutPath.Segments.Add(FSegment::Pack(UnpackedSegment));
			}
			else
			{
				return false; // segment failed to parse
			}
		}
		return true;
	}
	return false;
}

[[nodiscard]] uint32 GetTypeHash(const FPropertyPathName& Path)
{
	uint32 Hash = 0;
	for (const FPropertyPathName::FSegment& Segment : Path.Segments)
	{
		Hash = HashCombineFast(Hash, GetTypeHash(Segment.NameWithIndex));
		Hash = HashCombineFast(Hash, GetTypeHash(Segment.Type));
	}
	return Hash;
}

TSharedRef<FJsonValue> FPropertyPathNameSegment::ToJsonValue() const
{
	TSharedRef<FJsonObject> Result = MakeShared<FJsonObject>();
	Result->SetStringField(TEXT("Name"), Name.ToString());
	Result->SetStringField(TEXT("Type"), *WriteToString<128>(Type));
	if (Index != INDEX_NONE)
	{
		Result->SetNumberField(TEXT("Index"), Index);
	}
	return StaticCastSharedRef<FJsonValue>(MakeShared<FJsonValueObject>(Result));
}

bool FPropertyPathNameSegment::FromJsonValue(const TSharedPtr<FJsonValue>& JsonValue, FPropertyPathNameSegment& OutSegment)
{
	TSharedPtr<FJsonObject>* AsObject = nullptr;
	if (!JsonValue || !JsonValue->TryGetObject(AsObject) || !AsObject || !AsObject->IsValid())
	{
		return false; // value must be a non-null object
	}

	// Name Field (required)
	FString Name;
	if (!(*AsObject)->TryGetStringField(TEXT("Name"), Name))
	{
		return false; // Name field is required
	}
	OutSegment.Name = FName(Name);

	// Type Field (required)
	FString Type;
	if (!(*AsObject)->TryGetStringField(TEXT("Type"), Type))
	{
		return false; // Type field is required
	}
	FPropertyTypeNameBuilder Builder;
	if (!Builder.TryParse(Type))
	{
		return false; // type failed to parse
	}
	OutSegment.Type = Builder.Build();
	

	// Index Field (optional)
	if (!(*AsObject)->TryGetNumberField(TEXT("Index"), OutSegment.Index))
	{
		OutSegment.Index = INDEX_NONE;
	}
	return true;
}

} // UE
