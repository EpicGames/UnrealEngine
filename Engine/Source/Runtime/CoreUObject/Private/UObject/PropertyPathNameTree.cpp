// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyPathNameTree.h"

#include "Hash/Blake3.h"
#include "Misc/StringBuilder.h"
#include "Templates/TypeHash.h"

namespace UE
{

void FPropertyPathNameTree::Empty()
{
	Nodes.Empty();
}

FPropertyPathNameTree::FNode FPropertyPathNameTree::Add(const FPropertyPathName& Path, int32 StartIndex)
{
	if (const int32 SegmentCount = Path.GetSegmentCount(); StartIndex < SegmentCount)
	{
		FPropertyPathNameSegment Segment = Path.GetSegment(StartIndex);
		FValue& Child = Nodes.FindOrAdd({Segment.Name, Segment.Type});
		if (++StartIndex < SegmentCount)
		{
			if (!Child.SubTree)
			{
				Child.SubTree = MakeUnique<FPropertyPathNameTree>();
			}
			return Child.SubTree->Add(Path, StartIndex);
		}
		return {&Child};
	}
	return {};
}

bool FPropertyPathNameTree::Remove(const FPropertyPathName& Path, int32 StartIndex)
{
	if (const int32 SegmentCount = Path.GetSegmentCount(); StartIndex < SegmentCount)
	{
		FPropertyPathNameSegment Segment = Path.GetSegment(StartIndex);
		if (FValue* Child = Nodes.Find({ Segment.Name, Segment.Type }))
		{
			if (++StartIndex >= SegmentCount)
			{
				Nodes.Remove({ Segment.Name, Segment.Type });
				return true;
			}
			else if (Child->SubTree && Child->SubTree->Remove(Path, StartIndex))
			{
				// if the node is tagless and childless after the operation, remove it too
				if (Child->Tag.Name.IsNone() && Child->SubTree->Nodes.IsEmpty())
				{
					Nodes.Remove({ Segment.Name, Segment.Type });
				}
				return true;
			}
		}
	}
	return false;
}

FPropertyPathNameTree::FNode FPropertyPathNameTree::Find(const FPropertyPathName& Path, int32 StartIndex)
{
	if (const int32 SegmentCount = Path.GetSegmentCount(); StartIndex < SegmentCount)
	{
		FPropertyPathNameSegment Segment = Path.GetSegment(StartIndex);
		if (FValue* Child = Nodes.Find({Segment.Name, Segment.Type}))
		{
			if (++StartIndex >= SegmentCount)
			{
				return {Child};
			}
			if (Child->SubTree)
			{
				return Child->SubTree->Find(Path, StartIndex);
			}
		}
	}
	return {};
}

void FPropertyPathNameTree::FNode::SetTag(const FPropertyTag& Tag)
{
	if (FValue* LocalValue = const_cast<FValue*>(Value))
	{
		if (LocalValue->Tag.Name.IsNone())
		{
			// Copy the tag and reset values that may vary between tags with the same path.
			LocalValue->Tag = Tag;
			LocalValue->Tag.Size = 0;
			LocalValue->Tag.ArrayIndex = INDEX_NONE;
			LocalValue->Tag.SizeOffset = INDEX_NONE;
			LocalValue->Tag.BoolVal = 0;
		}
		else
		{
			ensureMsgf(LocalValue->Tag.GetType() == Tag.GetType() && LocalValue->Tag.SerializeType == Tag.SerializeType,
				TEXT("Tag mismatch in property path name tree for property %s of type %s."),
				*WriteToString<32>(Tag.Name), *WriteToString<64>(Tag.GetType()));
		}
	}
}

void AppendHash(FBlake3& Builder, const FPropertyPathNameTree& Tree)
{
	TArray<FPropertyPathNameTree::FKey, TInlineAllocator<16>> Keys;
	Tree.Nodes.GetKeys(Keys);
	Keys.Sort();

	for (const FPropertyPathNameTree::FKey& Key : Keys)
	{
		AppendHash(Builder, Key.Name);
		AppendHash(Builder, Key.Type);

		const FPropertyPathNameTree::FValue& Value = Tree.Nodes.FindChecked(Key);

		// Most of the tag is represented in the key and does not need to be hashed twice.
		Builder.Update(&Value.Tag.PropertyGuid, sizeof(Value.Tag.PropertyGuid));
		Builder.Update(&Value.Tag.SerializeType, sizeof(Value.Tag.SerializeType));

		if (Value.SubTree)
		{
			AppendHash(Builder, *Value.SubTree);
		}
	}
}

} // UE
