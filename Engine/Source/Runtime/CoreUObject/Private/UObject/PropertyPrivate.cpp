// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Property.cpp: UProperty implementation
=============================================================================*/

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Math/RandomStream.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Class.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "UObject/PropertyHelper.h"
#include "UObject/CoreRedirects.h"
#include "UObject/SoftObjectPath.h"
#include "Math/Box2D.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/GarbageCollectionSchema.h"
#include "UObject/LinkerPlaceholderFunction.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "Async/UniqueLock.h"

/*-----------------------------------------------------------------------------
	UProperty implementation.
-----------------------------------------------------------------------------*/

//
// Constructors.
//
UProperty::UProperty(const FObjectInitializer& ObjectInitializer)
	: UField(ObjectInitializer)
	, ArrayDim(1)
#if WITH_EDITORONLY_DATA
	, AssociatedField(nullptr)
#endif // WITH_EDITORONLY_DATA
{
}

UProperty::UProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UField(FObjectInitializer::Get())
	, ArrayDim(1)
	, PropertyFlags(InFlags)
	, Offset_Internal(InOffset)
#if WITH_EDITORONLY_DATA
	, AssociatedField(nullptr)
#endif // WITH_EDITORONLY_DATA
{
}

UProperty::UProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags )
: UField(ObjectInitializer)	
, ArrayDim(1)
, PropertyFlags(InFlags)
, Offset_Internal(InOffset)
#if WITH_EDITORONLY_DATA
, AssociatedField(nullptr)
#endif // WITH_EDITORONLY_DATA
{
}

//
// Serializer.
//
void UProperty::Serialize( FArchive& Ar )
{
	// Make sure that we aren't saving a property to a package that shouldn't be serialised.
#if WITH_EDITORONLY_DATA
	check( !Ar.IsFilterEditorOnly() || !IsEditorOnlyProperty() );
#endif // WITH_EDITORONLY_DATA

	Super::Serialize( Ar );

	EPropertyFlags SaveFlags = PropertyFlags & ~CPF_ComputedFlags;
	// Archive the basic info.
	Ar << ArrayDim << (uint64&)SaveFlags;
	if (Ar.IsLoading())
	{
		PropertyFlags = (SaveFlags & ~CPF_ComputedFlags) | (PropertyFlags & CPF_ComputedFlags);
	}
	
	if (FPlatformProperties::HasEditorOnlyData() == false)
	{
		// Make sure that we aren't saving a property to a package that shouldn't be serialised.
		check( !IsEditorOnlyProperty() );
	}

	Ar << RepNotifyFunc;

	if( Ar.IsLoading() )
	{
		Offset_Internal = 0;
		DestructorLinkNext = NULL;
	}

	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);
	if (Ar.IsSaving() || Ar.CustomVer(FReleaseObjectVersion::GUID) >= FReleaseObjectVersion::PropertiesSerializeRepCondition)
	{
		Ar << BlueprintReplicationCondition;
	}
}

#if WITH_EDITORONLY_DATA
FField* UProperty::GetAssociatedFField()
{
	return AssociatedField;
}

void UProperty::SetAssociatedFField(FField* InField)
{
	checkf(!AssociatedField || !InField || AssociatedField == InField, TEXT("Setting new associated field for %s but it already has %s associated with it"), *GetPathName(), *AssociatedField->GetPathName());
	AssociatedField = InField;
}
#endif // WITH_EDITORONLY_DATA





UEnumProperty::UEnumProperty(const FObjectInitializer& ObjectInitializer, UEnum* InEnum)
	: UProperty(ObjectInitializer)
	, Enum(InEnum)
{
	// This is expected to be set post-construction by AddCppProperty
	UnderlyingProp = nullptr;
}

UEnumProperty::UEnumProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UEnum* InEnum)
	: UProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags | CPF_HasGetValueTypeHash)
	, Enum(InEnum)
{
	// This is expected to be set post-construction by AddCppProperty
	UnderlyingProp = nullptr;
}

void UEnumProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Enum;
	if (Enum != nullptr)
	{
		Ar.Preload(Enum);
	}
	Ar << UnderlyingProp;
	if (UnderlyingProp != nullptr)
	{
		Ar.Preload(UnderlyingProp);
	}
}

void UArrayProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Inner;
	checkSlow(Inner || HasAnyFlags(RF_ClassDefaultObject) || !IsValidChecked(this));
}

// Ensure that TArray and FScriptArray are interchangeable, as FScriptArray will be used to access a native array property
// from script that is declared as a TArray in C++.
static_assert(sizeof(FScriptArray) == sizeof(TArray<uint8>), "FScriptArray and TArray<uint8> must be interchangable.");


void UObjectPropertyBase::BeginDestroy()
{
#if USE_UPROPERTY_LOAD_DEFERRING
	if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(PropertyClass))
	{
		PlaceholderClass->RemoveReferencingProperty(this);
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING

	Super::BeginDestroy();
}

void UObjectPropertyBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << PropertyClass;

#if USE_UPROPERTY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(PropertyClass))
		{
			PlaceholderClass->AddReferencingProperty(this);
		}
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING
}

#if USE_UPROPERTY_LOAD_DEFERRING
void UObjectPropertyBase::SetPropertyClass(UClass* NewPropertyClass)
{
	if (ULinkerPlaceholderClass* NewPlaceholderClass = Cast<ULinkerPlaceholderClass>(NewPropertyClass))
	{
		NewPlaceholderClass->AddReferencingProperty(this);
	}

	if (ULinkerPlaceholderClass* OldPlaceholderClass = Cast<ULinkerPlaceholderClass>(PropertyClass))
	{
		OldPlaceholderClass->RemoveReferencingProperty(this);
	}
	PropertyClass = NewPropertyClass;
}
#endif // USE_UPROPERTY_LOAD_DEFERRING



UBoolProperty::UBoolProperty(const FObjectInitializer& ObjectInitializer)
	: UProperty(ObjectInitializer)
	, FieldSize(0)
	, ByteOffset(0)
	, ByteMask(1)
	, FieldMask(1)
{
	SetBoolSize(1, false, 1);
}

UBoolProperty::UBoolProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, uint32 InBitMask, uint32 InElementSize, bool bIsNativeBool)
	: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InFlags | CPF_HasGetValueTypeHash)
	, FieldSize(0)
	, ByteOffset(0)
	, ByteMask(1)
	, FieldMask(1)
{
	SetBoolSize(InElementSize, bIsNativeBool, InBitMask);
}

UBoolProperty::UBoolProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, uint32 InBitMask, uint32 InElementSize, bool bIsNativeBool)
	: UProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags | CPF_HasGetValueTypeHash)
	, FieldSize(0)
	, ByteOffset(0)
	, ByteMask(1)
	, FieldMask(1)
{
	SetBoolSize(InElementSize, bIsNativeBool, InBitMask);
}

void UBoolProperty::SetBoolSize(const uint32 InSize, const bool bIsNativeBool, const uint32 InBitMask /*= 0*/)
{
	if (bIsNativeBool)
	{
		PropertyFlags |= (CPF_IsPlainOldData | CPF_NoDestructor | CPF_ZeroConstructor);
	}
	else
	{
		PropertyFlags &= ~(CPF_IsPlainOldData | CPF_ZeroConstructor);
		PropertyFlags |= CPF_NoDestructor;
	}
	uint32 TestBitmask = InBitMask ? InBitMask : 1;
	ElementSize = InSize;
	FieldSize = (uint8)ElementSize;
	ByteOffset = 0;
	if (bIsNativeBool)
	{
		ByteMask = true;
		FieldMask = 255;
	}
	else
	{
		// Calculate ByteOffset and get ByteMask.
		for (ByteOffset = 0; ByteOffset < InSize && ((ByteMask = *((uint8*)&TestBitmask + ByteOffset)) == 0); ByteOffset++);
		FieldMask = ByteMask;
	}
	check((int32)FieldSize == ElementSize);
	check(ElementSize != 0);
	check(FieldMask != 0);
	check(ByteMask != 0);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UBoolProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Serialize additional flags which will help to identify this UBoolProperty type and size.
	uint8 BoolSize = (uint8)ElementSize;
	Ar << BoolSize;
	uint8 NativeBool = false;
	if (Ar.IsLoading())
	{
		Ar << NativeBool;
		if (IsValidChecked(this))
		{
			SetBoolSize(BoolSize, !!NativeBool);
		}
	}
	else
	{
		NativeBool = (!HasAnyFlags(RF_ClassDefaultObject) && IsValidChecked(this) && Ar.IsSaving()) ? (IsNativeBool() ? 1 : 0) : 0;
		Ar << NativeBool;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UByteProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << Enum;
	if (Enum != nullptr)
	{
		Ar.Preload(Enum);
	}
}

void UClassProperty::BeginDestroy()
{
#if USE_UPROPERTY_LOAD_DEFERRING
	if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
	{
		PlaceholderClass->RemoveReferencingProperty(this);
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING

	Super::BeginDestroy();
}

void UClassProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << MetaClass;

#if USE_UPROPERTY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
		{
			PlaceholderClass->AddReferencingProperty(this);
		}
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING

	if (!(MetaClass || HasAnyFlags(RF_ClassDefaultObject)))
	{
		// If we failed to load the MetaClass and we're not a CDO, that means we relied on a class that has been removed or doesn't exist.
		// The most likely cause for this is either an incomplete recompile, or if content was migrated between games that had native class dependencies
		// that do not exist in this game.  We allow blueprint classes to continue, because compile on load will error out, and stub the class that was using it
		UClass* TestClass = dynamic_cast<UClass*>(GetOwnerStruct());
		if (TestClass && TestClass->HasAllClassFlags(CLASS_Native) && !TestClass->HasAllClassFlags(CLASS_NewerVersionExists) && (TestClass->GetOutermost() != GetTransientPackage()))
		{
			checkf(false, TEXT("Class property tried to serialize a missing class.  Did you remove a native class and not fully recompile?"));
		}
	}
}

#if USE_UPROPERTY_LOAD_DEFERRING
void UClassProperty::SetMetaClass(UClass* NewMetaClass)
{
	if (ULinkerPlaceholderClass* NewPlaceholderClass = Cast<ULinkerPlaceholderClass>(NewMetaClass))
	{
		NewPlaceholderClass->AddReferencingProperty(this);
	}

	if (ULinkerPlaceholderClass* OldPlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
	{
		OldPlaceholderClass->RemoveReferencingProperty(this);
	}
	MetaClass = NewMetaClass;
}
#endif // USE_UPROPERTY_LOAD_DEFERRING

void UDelegateProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << SignatureFunction;

#if USE_UPROPERTY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (auto PlaceholderFunc = Cast<ULinkerPlaceholderFunction>(SignatureFunction))
		{
			PlaceholderFunc->AddReferencingProperty(this);
		}
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING
}

void UDelegateProperty::BeginDestroy()
{
#if USE_UPROPERTY_LOAD_DEFERRING
	if (auto PlaceholderFunc = Cast<ULinkerPlaceholderFunction>(SignatureFunction))
	{
		PlaceholderFunc->RemoveReferencingProperty(this);
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING

	Super::BeginDestroy();
}





void UInterfaceProperty::BeginDestroy()
{
#if USE_UPROPERTY_LOAD_DEFERRING
	if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(InterfaceClass))
	{
		PlaceholderClass->RemoveReferencingProperty(this);
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	Super::BeginDestroy();
}

void UInterfaceProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar << InterfaceClass;

#if USE_UPROPERTY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(InterfaceClass))
		{
			PlaceholderClass->AddReferencingProperty(this);
		}
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING

	if (!InterfaceClass && !HasAnyFlags(RF_ClassDefaultObject))
	{
		// If we failed to load the InterfaceClass and we're not a CDO, that means we relied on a class that has been removed or doesn't exist.
		// The most likely cause for this is either an incomplete recompile, or if content was migrated between games that had native class dependencies
		// that do not exist in this game.  We allow blueprint classes to continue, because compile-on-load will error out, and stub the class that was using it
		UClass* TestClass = dynamic_cast<UClass*>(GetOwnerStruct());
		if (TestClass && TestClass->HasAllClassFlags(CLASS_Native) && !TestClass->HasAllClassFlags(CLASS_NewerVersionExists) && (TestClass->GetOutermost() != GetTransientPackage()))
		{
			checkf(false, TEXT("Interface property tried to serialize a missing interface.  Did you remove a native class and not fully recompile?"));
		}
	}
}

#if USE_UPROPERTY_LOAD_DEFERRING
void UInterfaceProperty::SetInterfaceClass(UClass* NewInterfaceClass)
{
	if (ULinkerPlaceholderClass* NewPlaceholderClass = Cast<ULinkerPlaceholderClass>(NewInterfaceClass))
	{
		NewPlaceholderClass->AddReferencingProperty(this);
	}

	if (ULinkerPlaceholderClass* OldPlaceholderClass = Cast<ULinkerPlaceholderClass>(InterfaceClass))
	{
		OldPlaceholderClass->RemoveReferencingProperty(this);
	}
	InterfaceClass = NewInterfaceClass;
}
#endif // USE_UPROPERTY_LOAD_DEFERRING


UMapProperty::UMapProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags)
{
	// These are expected to be set post-construction by AddCppProperty
	KeyProp = nullptr;
	ValueProp = nullptr;
}

void UMapProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << KeyProp;
	Ar << ValueProp;
}

// Ensure that TArray and FScriptMap are interchangeable, as FScriptMap will be used to access a native array property
// from script that is declared as a TArray in C++.
static_assert(sizeof(FScriptMap) == sizeof(TMap<uint32, uint8>), "FScriptMap and TMap<uint32, uint8> must be interchangable.");

void UMulticastDelegateProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << SignatureFunction;

#if USE_UPROPERTY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (auto PlaceholderFunc = Cast<ULinkerPlaceholderFunction>(SignatureFunction))
		{
			PlaceholderFunc->AddReferencingProperty(this);
		}
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING
}

void UMulticastDelegateProperty::BeginDestroy()
{
#if USE_UPROPERTY_LOAD_DEFERRING
	if (auto PlaceholderFunc = Cast<ULinkerPlaceholderFunction>(SignatureFunction))
	{
		PlaceholderFunc->RemoveReferencingProperty(this);
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING

	Super::BeginDestroy();
}


USetProperty::USetProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags)
	: UProperty(ObjectInitializer, EC_CppProperty, InOffset, InFlags)
{
	// This is expected to be set post-construction by AddCppProperty
	ElementProp = nullptr;
}

void USetProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << ElementProp;
}

// Ensure that TArray and FScriptMap are interchangeable, as FScriptMap will be used to access a native array property
// from script that is declared as a TArray in C++.
static_assert(sizeof(FScriptSet) == sizeof(TMap<uint32, uint8>), "FScriptSet and TSet<uint32, uint8> must be interchangable.");


void USoftClassProperty::BeginDestroy()
{
#if USE_UPROPERTY_LOAD_DEFERRING
	if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
	{
		PlaceholderClass->RemoveReferencingProperty(this);
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING

	Super::BeginDestroy();
}

void USoftClassProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar << MetaClass;

#if USE_UPROPERTY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
		{
			PlaceholderClass->AddReferencingProperty(this);
		}
	}
#endif // USE_UPROPERTY_LOAD_DEFERRING

	if (!(MetaClass || HasAnyFlags(RF_ClassDefaultObject)))
	{
		// If we failed to load the MetaClass and we're not a CDO, that means we relied on a class that has been removed or doesn't exist.
		// The most likely cause for this is either an incomplete recompile, or if content was migrated between games that had native class dependencies
		// that do not exist in this game.  We allow blueprint classes to continue, because compile on load will error out, and stub the class that was using it
		UClass* TestClass = dynamic_cast<UClass*>(GetOwnerStruct());
		if (TestClass && TestClass->HasAllClassFlags(CLASS_Native) && !TestClass->HasAllClassFlags(CLASS_NewerVersionExists) && (TestClass->GetOutermost() != GetTransientPackage()))
		{
			checkf(false, TEXT("Class property tried to serialize a missing class.  Did you remove a native class and not fully recompile?"));
		}
	}
}

#if USE_UPROPERTY_LOAD_DEFERRING
void USoftClassProperty::SetMetaClass(UClass* NewMetaClass)
{
	if (ULinkerPlaceholderClass* NewPlaceholderClass = Cast<ULinkerPlaceholderClass>(NewMetaClass))
	{
		NewPlaceholderClass->AddReferencingProperty(this);
	}

	if (ULinkerPlaceholderClass* OldPlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
	{
		OldPlaceholderClass->RemoveReferencingProperty(this);
	}
	MetaClass = NewMetaClass;
}
#endif // USE_UPROPERTY_LOAD_DEFERRING


static inline void PreloadInnerStructMembers(UStructProperty* StructProperty)
{
#if USE_UPROPERTY_LOAD_DEFERRING
	uint32 PropagatedLoadFlags = 0;
	if (FLinkerLoad* Linker = StructProperty->GetLinker())
	{
		PropagatedLoadFlags |= (Linker->LoadFlags & LOAD_DeferDependencyLoads);
	}

	if (UScriptStruct* Struct = StructProperty->Struct)
	{
		if (FLinkerLoad* StructLinker = Struct->GetLinker())
		{
			UE::TUniqueLock ScopeLock(StructLinker->Mutex);
			TGuardValue<uint32> LoadFlagGuard(StructLinker->LoadFlags, StructLinker->LoadFlags | PropagatedLoadFlags);
			Struct->RecursivelyPreload();
		}
	}
#else // USE_UPROPERTY_LOAD_DEFERRING
	StructProperty->Struct->RecursivelyPreload();
#endif // USE_UPROPERTY_LOAD_DEFERRING
}

/*-----------------------------------------------------------------------------
	UStructProperty.
-----------------------------------------------------------------------------*/

UStructProperty::UStructProperty(ECppProperty, int32 InOffset, EPropertyFlags InFlags, UScriptStruct* InStruct)
	: UProperty(FObjectInitializer::Get(), EC_CppProperty, InOffset, InStruct->GetCppStructOps() ? InStruct->GetCppStructOps()->GetComputedPropertyFlags() | InFlags : InFlags)
	, Struct(InStruct)
{
	ElementSize = Struct->PropertiesSize;
}

UStructProperty::UStructProperty(const FObjectInitializer& ObjectInitializer, ECppProperty, int32 InOffset, EPropertyFlags InFlags, UScriptStruct* InStruct)
	: UProperty(ObjectInitializer, EC_CppProperty, InOffset, InStruct->GetCppStructOps() ? InStruct->GetCppStructOps()->GetComputedPropertyFlags() | InFlags : InFlags)
	, Struct(InStruct)
{
	ElementSize = Struct->PropertiesSize;
}

void UStructProperty::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	static UScriptStruct* FallbackStruct = GetFallbackStruct();

	if (Ar.IsPersistent() && Ar.GetLinker() && Ar.IsLoading() && !Struct)
	{
		// It's necessary to solve circular dependency problems, when serializing the Struct causes linking of the Property.
		Struct = FallbackStruct;
	}

	Ar << Struct;
#if WITH_EDITOR
	if (Ar.IsPersistent() && Ar.GetLinker())
	{
		if (!Struct && Ar.IsLoading())
		{
			UE_LOGF(LogProperty, Error, "UStructProperty::Serialize Loading: Property '%ls'. Unknown structure.", *GetFullName());
			Struct = FallbackStruct;
		}
		else if ((FallbackStruct == Struct) && Ar.IsSaving())
		{
			UE_LOGF(LogProperty, Error, "UStructProperty::Serialize Saving: Property '%ls'. FallbackStruct structure.", *GetFullName());
		}
	}
#endif // WITH_EDITOR
	if (Struct)
	{
		PreloadInnerStructMembers(this);
	}
	else
	{
		ensure(true);
	}
}

