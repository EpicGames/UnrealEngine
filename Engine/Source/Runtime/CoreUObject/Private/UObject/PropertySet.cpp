// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "Misc/ScopeExit.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/PropertyHelper.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectThreadContext.h"

namespace UESetProperty_Private
{
	/**
	 * Checks if any of the elements in the set compare equal to the one passed.
	 *
	 * @param  SetHelper    The set to search through.
	 * @param  LogicalIndex The index in the set to start searching from.
	 * @param  Num          The number of elements to compare.
	 */
	bool AnyEqual(const FScriptSetHelper& SetHelper, const int32 LogicalIndex, int32 Num, const uint8* ElementToCompare, uint32 PortFlags)
	{
		const FProperty* ElementProp = SetHelper.GetElementProperty();

		FScriptSetHelper::FIterator Iterator(SetHelper, LogicalIndex);
		for (; Iterator && Num; --Num, ++Iterator)
		{
			if (ElementProp->Identical(SetHelper.GetElementPtr(Iterator), ElementToCompare, PortFlags))
			{
				return true;
			}
		}

		return false;
	}

	bool RangesContainSameAmountsOfVal(const FScriptSetHelper& SetHelperA, const int32 LogicalIndexA, const FScriptSetHelper& SetHelperB, const int32 LogicalIndexB, int32 Num, const uint8* ElementToCompare, uint32 PortFlags)
	{
		const FProperty* ElementProp = SetHelperA.GetElementProperty();

		// Ensure that both sets are the same type
		check(ElementProp == SetHelperB.GetElementProperty());

		FScriptSetHelper::FIterator IteratorA(SetHelperA, LogicalIndexA);
		FScriptSetHelper::FIterator IteratorB(SetHelperB, LogicalIndexB);

		int32 CountA = 0;
		int32 CountB = 0;
		for (;;)
		{
			if (Num == 0)
			{
				return CountA == CountB;
			}

			const uint8* ElementA = SetHelperA.GetElementPtr(IteratorA);
			const uint8* ElementB = SetHelperB.GetElementPtr(IteratorB);
			if (ElementProp->Identical(ElementA, ElementToCompare, PortFlags))
			{
				++CountA;
			}

			if (ElementProp->Identical(ElementB, ElementToCompare, PortFlags))
			{
				++CountB;
			}

			++IteratorA;
			++IteratorB;
			--Num;
		}
	}

	bool IsPermutation(const FScriptSetHelper& SetHelperA, const FScriptSetHelper& SetHelperB, const uint32 PortFlags)
	{
		const FProperty* ElementProp = SetHelperA.GetElementProperty();

		// Ensure that both maps are the same type
		check(ElementProp == SetHelperB.GetElementProperty());

		int32 Num = SetHelperA.Num();
		if (Num != SetHelperB.Num())
		{
			return false;
		}

		// Skip over common initial sequence
		FScriptSetHelper::FIterator IteratorA(SetHelperA);
		FScriptSetHelper::FIterator IteratorB(SetHelperB);
		for (;;)
		{
			if (Num == 0)
			{
				return true;
			}

			const uint8* ElementA = SetHelperA.GetElementPtr(IteratorA);
			const uint8* ElementB = SetHelperB.GetElementPtr(IteratorB);
			if (!ElementProp->Identical(ElementA, ElementB, PortFlags))
			{
				break;
			}

			++IteratorA;
			++IteratorB;
			--Num;
		}

		const int32 FirstIndexA = IteratorA.GetLogicalIndex();
		const int32 FirstIndexB = IteratorB.GetLogicalIndex();
		const int32 FirstNum    = Num;
		for (;;)
		{
			const uint8* ElementA = SetHelperA.GetElementPtr(IteratorA);
			if (!AnyEqual(SetHelperA, FirstIndexA, FirstNum - Num, ElementA, PortFlags) && !RangesContainSameAmountsOfVal(SetHelperA, FirstIndexA, SetHelperB, FirstIndexB, FirstNum, ElementA, PortFlags))
			{
				return false;
			}

			--Num;
			if (Num == 0)
			{
				return true;
			}

			++IteratorA;
		}
	}
}

IMPLEMENT_FIELD(FSetProperty)

FSetProperty::FSetProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
	// This is expected to be set post-construction by AddCppProperty
	ElementProp = nullptr;
}

FSetProperty::FSetProperty(FFieldVariant InOwner, const UECodeGen_Private::FSetPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
	// This is expected to be set post-construction by AddCppProperty
	ElementProp = nullptr;
}

#if WITH_EDITORONLY_DATA
FSetProperty::FSetProperty(UField* InField)
	: Super(InField)
{
	USetProperty* SourceProperty = CastChecked<USetProperty>(InField);
	SetLayout = SourceProperty->SetLayout;

	ElementProp = CastField<FProperty>(SourceProperty->ElementProp->GetAssociatedFField());
	if (!ElementProp)
	{
		ElementProp = CastField<FProperty>(CreateFromUField(SourceProperty->ElementProp));
		SourceProperty->ElementProp->SetAssociatedFField(ElementProp);
	}
}
#endif // WITH_EDITORONLY_DATA

FSetProperty::~FSetProperty()
{
	delete ElementProp;
	ElementProp = nullptr;
}

void FSetProperty::PostDuplicate(const FField& InField)
{
	const FSetProperty& Source = static_cast<const FSetProperty&>(InField);
	ElementProp = CastFieldChecked<FProperty>(FField::Duplicate(Source.ElementProp, this));
	SetLayout = Source.SetLayout;
	Super::PostDuplicate(InField);
}

void FSetProperty::LinkInternal(FArchive& Ar)
{
	check(ElementProp);

	ElementProp->Link(Ar);

	const int32 ElementPropSize = ElementProp->GetSize();
	const int32 ElementPropAlignment = ElementProp->GetMinAlignment();

	SetLayout = FScriptSet::GetScriptLayout(ElementPropSize, ElementPropAlignment);

	Super::LinkInternal(Ar);
}

bool FSetProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	checkSlow(ElementProp);

	FScriptSetHelper SetHelperA(this, A);

	const int32 ANum = SetHelperA.Num();

	if (!B)
	{
		return ANum == 0;
	}

	FScriptSetHelper SetHelperB(this, B);
	if (ANum != SetHelperB.Num())
	{
		return false;
	}

	return UESetProperty_Private::IsPermutation(SetHelperA, SetHelperB, PortFlags);
}

void FSetProperty::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	if (ElementProp)
	{
		ElementProp->GetPreloadDependencies(OutDeps);
	}
}

void FSetProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, const void* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	// Set containers must be serialized as a "whole" value, which means that we need to serialize every field for struct-typed entries.
	// When using a custom property list, we need to temporarily bypass this logic to ensure that all set elements are fully serialized.
	const bool bIsUsingCustomPropertyList = !!UnderlyingArchive.ArUseCustomPropertyList;
	UnderlyingArchive.ArUseCustomPropertyList = false;
	ON_SCOPE_EXIT
	{
		UnderlyingArchive.ArUseCustomPropertyList = bIsUsingCustomPropertyList;
	};

	// If we're doing delta serialization within this property, act as if there are no defaults
	if (!UnderlyingArchive.DoIntraPropertyDelta())
	{
		Defaults = nullptr;
	}

	// Ar related calls in this function must be mirrored in FSetProperty::ConvertFromType
	checkSlow(ElementProp);

	// Ensure that the element property has been loaded before calling SerializeItem() on it
	//UnderlyingArchive.Preload(ElementProp);

	FScriptSetHelper SetHelper(this, Value);

	if (UnderlyingArchive.IsLoading())
	{
		FUObjectSerializeContext* Context = FUObjectThreadContext::Get().GetSerializeContext();

		if (Defaults)
		{
			CopyValuesInternal(Value, Defaults, 1);
		}

		// Delete any explicitly-removed elements
		int32 NumElementsToRemove = 0;
		FStructuredArchive::FArray ElementsToRemoveArray = Record.EnterArray(TEXT("ElementsToRemove"), NumElementsToRemove);

		if (!Defaults || SetHelper.Num() == 0) // Faster loading path when loading elements into an empty set
		{
			if (NumElementsToRemove)
			{
				// Load and discard elements to remove, set is empty
				void* TempElementStorage = FMemory::Malloc(SetLayout.Size);
				ElementProp->InitializeValue(TempElementStorage);

				FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ElementProp, this);
				for (; NumElementsToRemove; --NumElementsToRemove)
				{
					ElementProp->SerializeItem(ElementsToRemoveArray.EnterElement(), TempElementStorage);
				}

				ElementProp->DestroyValue(TempElementStorage);
				FMemory::Free(TempElementStorage);
			}

			int32 Num = 0;
			FStructuredArchive::FArray ElementsArray = Record.EnterArray(TEXT("Elements"), Num);

			// Empty and reserve then deserialize elements directly into set memory
			SetHelper.EmptyElements(Num);
			for (; Num; --Num)
			{
				int32 Index = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
				ElementProp->SerializeItem(ElementsArray.EnterElement(), SetHelper.GetElementPtrWithoutCheck(Index));
			}
		}
		else // Slower loading path that mutates non-empty set
		{
			uint8* TempElementStorage = nullptr;
			ON_SCOPE_EXIT
			{
				if (TempElementStorage)
				{
					ElementProp->DestroyValue(TempElementStorage);
					FMemory::Free(TempElementStorage);
				}
			};

			if (NumElementsToRemove)
			{
				TempElementStorage = (uint8*)FMemory::Malloc(SetLayout.Size);
				ElementProp->InitializeValue(TempElementStorage);

				FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ElementProp, this);
				for (; NumElementsToRemove; --NumElementsToRemove)
				{
					// Read key into temporary storage
					ElementProp->SerializeItem(ElementsToRemoveArray.EnterElement(), TempElementStorage);

					// If the key is in the map, remove it
					const int32 Found = SetHelper.FindElementIndex(TempElementStorage);
					if (Found != INDEX_NONE)
					{
						SetHelper.RemoveAt(Found);
					}
				}
			}

			int32 Num = 0;
			FStructuredArchive::FArray ElementsArray = Record.EnterArray(TEXT("Elements"), Num);

			// Allocate temporary key space if we haven't allocated it already above
			if (Num != 0 && !TempElementStorage)
			{
				TempElementStorage = (uint8*)FMemory::Malloc(SetLayout.Size);
				ElementProp->InitializeValue(TempElementStorage);
			}

			FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ElementProp, this);
			// Read remaining items into container
			for (; Num; --Num)
			{
				// Read key into temporary storage
				ElementProp->SerializeItem(ElementsArray.EnterElement(), TempElementStorage);

				// Add a new entry if the element doesn't currently exist in the set
				if (SetHelper.FindElementIndex(TempElementStorage) == INDEX_NONE)
				{
					const int32 NewElementIndex = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
					uint8* NewElementPtr = SetHelper.GetElementPtrWithoutCheck(NewElementIndex);

					// Copy over deserialized key from temporary storage
					ElementProp->CopyCompleteValue_InContainer(NewElementPtr, TempElementStorage);
				}
			}
		}

#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
		// In UE_INTERNAL_UNIVERSAL_PLACEHOLDERS environs we may lose user types
		// and contained data will thus be unhashable - we could fix this
		// with even better placeholder support
		if(ElementProp->HasAnyPropertyFlags(CPF_HasGetValueTypeHash)) 
#endif
		{
			SetHelper.Rehash();
		}
	}
	else
	{
		FScriptSetHelper DefaultsHelper(this, Defaults);

		// Container for temporarily tracking some indices
		TSet<int32> Indices;

		// Determine how many keys are missing from the object
		if (Defaults)
		{
			for (FScriptSetHelper::FIterator Iterator(DefaultsHelper); Iterator; ++Iterator)
			{
				const uint8* DefaultElementPtr = DefaultsHelper.GetElementPtr(Iterator);
				if (SetHelper.FindElementIndex(DefaultElementPtr) == INDEX_NONE)
				{
					Indices.Add(Iterator.GetInternalIndex());
				}
			}
		}

		// Write out the removed elements
		int32 RemovedElementsNum = Indices.Num();
		FStructuredArchive::FArray RemovedElementsArray = Record.EnterArray(TEXT("ElementsToRemove"), RemovedElementsNum);
		
		{
			FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ElementProp, this);
			for (int32 Index : Indices)
			{
				ElementProp->SerializeItem(RemovedElementsArray.EnterElement(), DefaultsHelper.GetElementPtr(Index));
			}
		}

		// Write out added elements
		if (Defaults)
		{
			Indices.Reset();
			for (FScriptSetHelper::FIterator Iterator(SetHelper); Iterator; ++Iterator)
			{
				const uint8* ValueElement = SetHelper.GetElementPtr(Iterator);
				const uint8* DefaultElement = DefaultsHelper.FindElementPtr(ValueElement);

				if (!DefaultElement)
				{
					Indices.Add(Iterator.GetInternalIndex());
				}
			}

			// Write out differences from defaults
			int32 Num = Indices.Num();
			FStructuredArchive::FArray ElementsArray = Record.EnterArray(TEXT("Elements"), Num);

			FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ElementProp, this);
			for (int32 Index : Indices)
			{
				uint8* ElementPtr = SetHelper.GetElementPtrWithoutCheck(Index);

				ElementProp->SerializeItem(ElementsArray.EnterElement(), ElementPtr);
			}
		}
		else
		{
			int32 Num = SetHelper.Num();
			FStructuredArchive::FArray ElementsArray = Record.EnterArray(TEXT("Elements"), Num);

			FSerializedPropertyScope SerializedProperty(UnderlyingArchive, ElementProp, this);

			for (FScriptSetHelper::FIterator Iterator(SetHelper); Iterator; ++Iterator)
			{
				uint8* ElementPtr = SetHelper.GetElementPtr(Iterator);
				ElementProp->SerializeItem(ElementsArray.EnterElement(), ElementPtr);
			}
		}
	}
}

bool FSetProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	UE_LOGF( LogProperty, Error, "Replicated TSets are not supported." );
	return 1;
}

void FSetProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	SerializeSingleField(Ar, ElementProp, this);
}

void FSetProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(Collector);
	if (ElementProp)
	{
		ElementProp->AddReferencedObjects(Collector);
	}
}

FString FSetProperty::GetCPPMacroType(FString& ExtendedTypeText) const
{
	checkSlow(ElementProp);
	ExtendedTypeText = FString::Printf(TEXT("%s"), *ElementProp->GetCPPType());
	return TEXT("TSET");
}

FString FSetProperty::GetCPPTypeCustom(FString* ExtendedTypeText, uint32 CPPExportFlags, const FString& ElementTypeText, const FString& InElementExtendedTypeText) const
{
	if (ExtendedTypeText)
	{
		// if property type is a template class, add a space between the closing brackets
		FString ElementExtendedTypeText = InElementExtendedTypeText;
		if ((ElementExtendedTypeText.Len() && ElementExtendedTypeText.Right(1) == TEXT(">"))
			|| (!ElementExtendedTypeText.Len() && ElementTypeText.Len() && ElementTypeText.Right(1) == TEXT(">")))
		{
			ElementExtendedTypeText += TEXT(" ");
		}

		*ExtendedTypeText = FString::Printf(TEXT("<%s%s>"), *ElementTypeText, *ElementExtendedTypeText);
	}

	return TEXT("TSet");
}

FString FSetProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const
{
	checkSlow(ElementProp);

	FString ElementTypeText;
	FString ElementExtendedTypeText;

	if (ExtendedTypeText)
	{
		ElementTypeText = ElementProp->GetCPPType(&ElementExtendedTypeText, CPPExportFlags & ~CPPF_ArgumentOrReturnValue); // we won't consider set elements to be "arguments or return values"
	}

	return GetCPPTypeCustom(ExtendedTypeText, CPPExportFlags, ElementTypeText, ElementExtendedTypeText);
}

void FSetProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	checkSlow(ElementProp);

	uint8* TempSetStorage = nullptr;
	void* PropertyValuePtr = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		// Allocate temporary map as we first need to initialize it with the value provided by the getter function and then export it
		TempSetStorage = (uint8*)AllocateAndInitializeValue();
		PropertyValuePtr = TempSetStorage;
		FProperty::GetValue_InContainer(ContainerOrPropertyPtr, PropertyValuePtr);
	}
	else
	{
		PropertyValuePtr = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
	}

	ON_SCOPE_EXIT
	{
		DestroyAndFreeValue(TempSetStorage);
	};

	FScriptSetHelper SetHelper(this, PropertyValuePtr);

	if (SetHelper.Num() == 0)
	{
		ValueStr += TEXT("()");
		return;
	}

	const bool bExternalEditor = (0 != (PPF_ExternalEditor & PortFlags));

	uint8* StructDefaults = nullptr;
	if (FStructProperty* StructElementProp = CastField<FStructProperty>(ElementProp))
	{
		checkSlow(StructElementProp->Struct);

		if (!bExternalEditor)
		{
			// For external editor, we always export all fields
			StructDefaults = (uint8*)FMemory::Malloc(SetLayout.Size);
			ElementProp->InitializeValue(StructDefaults);
		}
	}

	ON_SCOPE_EXIT
	{
		if (StructDefaults)
		{
			ElementProp->DestroyValue(StructDefaults);
			FMemory::Free(StructDefaults);
		}
	};

	FScriptSetHelper DefaultSetHelper(this, DefaultValue);

	uint8* PropData = SetHelper.GetElementPtrWithoutCheck(0);
	if (PortFlags & PPF_BlueprintDebugView)
	{
		int32 Index  = 0;
		bool  bFirst = true;
		for (int32 Count = SetHelper.Num(); Count; PropData += SetLayout.Size, ++Index)
		{
			if (SetHelper.IsValidIndex(Index))
			{
				if (bFirst)
				{
					bFirst = false;
				}
				else
				{
					ValueStr += TCHAR('\n');
				}

				// Always use struct defaults if the element is a struct, for symmetry with the import of array inner struct defaults
				uint8* PropDefault = StructDefaults ? StructDefaults : DefaultValue ? DefaultSetHelper.FindElementPtr(PropData) : nullptr;

				if (bExternalEditor)
				{
					// For external editor, always write
					PropDefault = PropData;
				}

				ElementProp->ExportTextItem_Direct(ValueStr, PropData, PropDefault, Parent, PortFlags | PPF_Delimited, ExportRootScope);

				--Count;
			}
		}
	}
	else
	{
		int32 Index  = 0;
		bool  bFirst = true;
		for (int32 Count = SetHelper.Num(); Count; PropData += SetLayout.Size, ++Index)
		{
			if (SetHelper.IsValidIndex(Index))
			{
				if (bFirst)
				{
					ValueStr += TCHAR('(');
					bFirst = false;
				}
				else
				{
					ValueStr += TCHAR(',');
				}

				uint8* PropDefault = nullptr;

				if (bExternalEditor)
				{
					// For external editor, always write
					PropDefault = PropData;
				}

				ElementProp->ExportTextItem_Direct(ValueStr, PropData, PropDefault, Parent, PortFlags | PPF_Delimited, ExportRootScope);

				--Count;
			}
		}

		ValueStr += TEXT(")");
	}
}

const TCHAR* FSetProperty::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	checkSlow(ElementProp);

	FScriptSetHelper SetHelper(this, PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType));
	uint8* TempSetStorage = nullptr;
	uint8* TempElementStorage = nullptr;
	bool bSuccess = true;

	ON_SCOPE_EXIT
	{
		FMemory::Free(TempElementStorage);

		// If we are returning because of an error, remove any already-added elements from the map before returning
		// to ensure we're not left with a partial state.
		if (!bSuccess)
		{
			SetHelper.EmptyElements();
		}

		if (TempSetStorage)
		{
			// TempSet is used by property setter so if it was allocated call the setter now
			FProperty::SetValue_InContainer(ContainerOrPropertyPtr, TempSetStorage);

			// Destroy and free the temp set used by property setter
			DestroyAndFreeValue(TempSetStorage);
		}
	};

	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		// Allocate temporary set as we first need to initialize it with the parsed items and then use the setter to update the property
		TempSetStorage = (uint8*)AllocateAndInitializeValue();
		// Reinitialize the set helper with the temp value
		SetHelper = FScriptSetHelper(this, TempSetStorage);
	}

	SetHelper.EmptyElements();

	// If we export an empty array we export an empty string, so ensure that if we're passed an empty string
	// we interpret it as an empty array.
	if (*Buffer++ != TCHAR('('))
	{
		return nullptr;
	}

	SkipWhitespace(Buffer);
	if (*Buffer == TCHAR(')'))
	{
		return Buffer + 1;
	}

	TempElementStorage = (uint8*)FMemory::Malloc(ElementProp->GetElementSize());
	// From this point failure should empty the set
	bSuccess = false;

	check(ElementProp->GetOffset_ForInternal() == 0);
	for (;;)
	{
		ElementProp->InitializeValue(TempElementStorage);
		ON_SCOPE_EXIT
		{
			ElementProp->DestroyValue(TempElementStorage);
		};

		// Read key into temporary storage
		Buffer = ElementProp->ImportText_Direct(Buffer, TempElementStorage, Parent, PortFlags | PPF_Delimited, ErrorText);
		if (!Buffer)
		{
			return nullptr;
		}

		// If the key isn't in the map yet, add it
		if (SetHelper.FindElementIndex(TempElementStorage) == INDEX_NONE)
		{
			const int32 NewElementIndex = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
			uint8* NewElementPtr = SetHelper.GetElementPtrWithoutCheck(NewElementIndex);

			// Copy over imported key from temporary storage
			ElementProp->CopyCompleteValue_InContainer(NewElementPtr, TempElementStorage);
		}

		// Parse the element

		SkipWhitespace(Buffer);

		switch (*Buffer++)
		{
		case TCHAR(')'):
			SetHelper.Rehash();
			bSuccess = true;
			return Buffer;

		case TCHAR(','):
			SkipWhitespace(Buffer);
			break;

		default:
			return nullptr;
		}
	}
}

void FSetProperty::AddCppProperty(FProperty* Property)
{
	check(!ElementProp);
	check(Property);
	ensureAlwaysMsgf(Property->HasAllPropertyFlags(CPF_HasGetValueTypeHash), TEXT("Attempting to create Set Property with unhashable element type: %s - Provide a GetTypeHash function!"), *Property->GetName());

	ElementProp = Property;
}

void FSetProperty::CopyValuesInternal(void* Dest, void const* Src, int32 Count) const
{
	check(Count == 1);
	
	if (Dest == Src)
	{
		return;
	}

	FScriptSetHelper SrcSetHelper (this, Src);
	FScriptSetHelper DestSetHelper(this, Dest);

	int32 Num = SrcSetHelper.Num();
	DestSetHelper.EmptyElements(Num);

	if (Num == 0)
	{
		return;
	}

	checkf(SrcSetHelper.GetMaxIndex() >= Num, TEXT("Corrupt source set: Num = %d, MaxIndex = %d"), Num, SrcSetHelper.GetMaxIndex());

	for (int32 SrcIndex = 0; Num; ++SrcIndex)
	{
		if (SrcSetHelper.IsValidIndex(SrcIndex))
		{
			const int32 DestIndex = DestSetHelper.AddDefaultValue_Invalid_NeedsRehash();

			uint8* SrcData  = SrcSetHelper.GetElementPtrWithoutCheck(SrcIndex);
			uint8* DestData = DestSetHelper.GetElementPtrWithoutCheck(DestIndex);

			ElementProp->CopyCompleteValue_InContainer(DestData, SrcData);

			--Num;
		}
	}

	DestSetHelper.Rehash();
}

void FSetProperty::ClearValueInternal(void* Data) const
{
	FScriptSetHelper SetHelper(this, Data);
	SetHelper.EmptyElements();
}

void FSetProperty::DestroyValueInternal(void* Data) const
{
	FScriptSetHelper SetHelper(this, Data);
	SetHelper.EmptyElements();

	//@todo UE potential double destroy later from this...would be ok for a script set, but still
	((FScriptSet*)Data)->~FScriptSet();
}

bool FSetProperty::ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const
{
	check(ElementProp);
	return ElementProp->ContainsFinishDestroy(EncounteredStructProps);
}

void FSetProperty::FinishDestroyInternal( void* Data ) const
{
	if ((ElementProp->PropertyFlags & (CPF_IsPlainOldData | CPF_NoDestructor)) == 0)
	{
		FScriptSetHelper SetHelper(this, Data);
		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			uint8* ElementPtr = SetHelper.GetElementPtr(It);
			ElementProp->FinishDestroy(ElementPtr);
		}
	}
}

/**
 * Creates new copies of components
 * 
 * @param	Data				pointer to the address of the instanced object referenced by this UComponentProperty
 * @param	DefaultData			pointer to the address of the default value of the instanced object referenced by this UComponentProperty
 * @param	Owner				the object that contains this property's data
 * @param	InstanceGraph		contains the mappings of instanced objects and components to their templates
 */
void FSetProperty::InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, FObjectInstancingGraph* InstanceGraph)
{
	if (!Data)
	{
		return;
	}

	const bool bUsesDynamicInstancing = InOwner->GetClass()->ShouldUseDynamicSubobjectInstancing();
	const bool bInstancedElement = ElementProp->ContainsInstancedObjectProperty() || bUsesDynamicInstancing;

	if (!bInstancedElement)
	{
		return;
	}

	FScriptSetHelper SetHelper(this, Data);

	if (DefaultData)
	{
		FScriptSetHelper DefaultSetHelper(this, DefaultData);

		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			uint8* ElementPtr = SetHelper.GetElementPtr(It);
			const uint8* DefaultElementPtr = DefaultSetHelper.FindElementPtr(ElementPtr, It.GetLogicalIndex());

			ElementProp->InstanceSubobjects(ElementPtr, DefaultElementPtr, InOwner, InstanceGraph);
		}
	}
	else
	{
		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			uint8* ElementPtr = SetHelper.GetElementPtr(It);
			ElementProp->InstanceSubobjects(ElementPtr, nullptr, InOwner, InstanceGraph);
		}
	}


	// Instancing keys will likely have invalidated the hash, so rehash
	SetHelper.Rehash();
}

bool FSetProperty::SameType(const FProperty* Other) const
{
	FSetProperty* SetProp = (FSetProperty*)Other;
	return Super::SameType(Other) && ElementProp && ElementProp->SameType(SetProp->ElementProp);
}

EConvertFromTypeResult FSetProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	// Ar related calls in this function must be mirrored in FSetProperty::SerializeItem
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	checkSlow(ElementProp);

	if (FStructProperty* ElementPropAsStruct = CastField<FStructProperty>(ElementProp))
	{
		if (!ElementPropAsStruct->Struct || (ElementPropAsStruct->Struct->GetCppStructOps() && !ElementPropAsStruct->Struct->GetCppStructOps()->HasGetTypeHash()))
		{
			// If the type we contain is no longer hashable, we're going to drop the saved data here.
			// This can happen if the native GetTypeHash function is removed.
			ensureMsgf(false, TEXT("Set Property %s has an unhashable type %s and will lose its saved data. Package: %s"),
				*Tag.Name.ToString(), *ElementPropAsStruct->Struct->GetFName().ToString(), *UnderlyingArchive.GetArchiveName());

			FScriptSetHelper ScriptSetHelper(this, ContainerPtrToValuePtr<void>(Data));
			ScriptSetHelper.EmptyElements();

			return EConvertFromTypeResult::CannotConvert;
		}
	}

	if (Tag.Type != NAME_SetProperty)
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}

	const FPackageFileVersion Version = UnderlyingArchive.UEVer();
	if (Version >= EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME)
	{
		if (CanSerializeFromTypeName(Tag.GetType()))
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}
	}
	else
	{
		const FName InnerTypeName = Tag.GetType().GetParameterName(0);
		if (InnerTypeName.IsNone() || InnerTypeName == ElementProp->GetID())
		{
			return EConvertFromTypeResult::UseSerializeItem;
		}
	}

	FScriptSetHelper ScriptSetHelper(this, ContainerPtrToValuePtr<void>(Data));

	uint8* TempElementStorage = nullptr;
	ON_SCOPE_EXIT
	{
		if (TempElementStorage)
		{
			ElementProp->DestroyAndFreeValue(TempElementStorage);
		}
	};

	FPropertyTag InnerPropertyTag;
	InnerPropertyTag.SetProperty(ElementProp);
	InnerPropertyTag.SetType(Tag.GetType().GetParameter(0));
	InnerPropertyTag.Name = Tag.Name;
	InnerPropertyTag.ArrayIndex = 0;

	bool bConversionSucceeded = true;

	FStructuredArchive::FRecord ValueRecord = Slot.EnterRecord();

	// When we saved this instance we wrote out any elements that were in the 'Default' instance but not in the
	// instance that was being written. Presumably we were constructed from our defaults and must now remove
	// any of the elements that were not present when we saved this Set:
	int32 NumElementsToRemove = 0;
	FStructuredArchive::FArray ElementsToRemoveArray = ValueRecord.EnterArray(TEXT("ElementsToRemove"), NumElementsToRemove);

	if (NumElementsToRemove)
	{
		TempElementStorage = (uint8*)ElementProp->AllocateAndInitializeValue();

		EConvertFromTypeResult ConvertResult = ElementProp->ConvertFromType(InnerPropertyTag, ElementsToRemoveArray.EnterElement(), TempElementStorage, DefaultsStruct, nullptr);
		if (ConvertResult == EConvertFromTypeResult::Converted || ConvertResult == EConvertFromTypeResult::Serialized)
		{
			int32 Found = ScriptSetHelper.FindElementIndex(TempElementStorage);
			if (Found != INDEX_NONE)
			{
				ScriptSetHelper.RemoveAt(Found);
			}

			for (int32 I = 1; I < NumElementsToRemove; ++I)
			{
				ConvertResult = ElementProp->ConvertFromType(InnerPropertyTag, ElementsToRemoveArray.EnterElement(), TempElementStorage, DefaultsStruct, nullptr);
				check(ConvertResult == EConvertFromTypeResult::Converted || ConvertResult == EConvertFromTypeResult::Serialized);

				Found = ScriptSetHelper.FindElementIndex(TempElementStorage);
				if (Found != INDEX_NONE)
				{
					ScriptSetHelper.RemoveAt(Found);
				}
			}
		}
		else
		{
			bConversionSucceeded = false;
		}
	}

	int32 Num = 0;
	FStructuredArchive::FArray ElementsArray = ValueRecord.EnterArray(TEXT("Elements"), Num);

	if (bConversionSucceeded)
	{
		if (Num != 0)
		{
			// Allocate temporary key space if we haven't allocated it already above
			if (TempElementStorage == nullptr)
			{
				TempElementStorage = (uint8*)ElementProp->AllocateAndInitializeValue();
			}

			// and read the first entry, we have to check for conversion possibility again because 
			// NumElementsToRemove may not have run (in fact, it likely did not):
			EConvertFromTypeResult ConvertResult = ElementProp->ConvertFromType(InnerPropertyTag, ElementsArray.EnterElement(), TempElementStorage, DefaultsStruct, nullptr);
			if (ConvertResult == EConvertFromTypeResult::Converted || ConvertResult == EConvertFromTypeResult::Serialized)
			{
				if (ScriptSetHelper.FindElementIndex(TempElementStorage) == INDEX_NONE)
				{
					const int32 NewElementIndex = ScriptSetHelper.AddDefaultValue_Invalid_NeedsRehash();
					uint8* NewElementPtr = ScriptSetHelper.GetElementPtrWithoutCheck(NewElementIndex);

					// Copy over deserialized key from temporary storage
					ElementProp->CopyCompleteValue_InContainer(NewElementPtr, TempElementStorage);
				}

				// Read remaining items into container
				for (int32 I = 1; I < Num; ++I)
				{
					// Read key into temporary storage
					ConvertResult = ElementProp->ConvertFromType(InnerPropertyTag, ElementsArray.EnterElement(), TempElementStorage, DefaultsStruct, nullptr);
					check(ConvertResult == EConvertFromTypeResult::Converted || ConvertResult == EConvertFromTypeResult::Serialized);

					// Add a new entry if the element doesn't currently exist in the set
					if (ScriptSetHelper.FindElementIndex(TempElementStorage) == INDEX_NONE)
					{
						const int32 NewElementIndex = ScriptSetHelper.AddDefaultValue_Invalid_NeedsRehash();
						uint8* NewElementPtr = ScriptSetHelper.GetElementPtrWithoutCheck(NewElementIndex);

						// Copy over deserialized key from temporary storage
						ElementProp->CopyCompleteValue_InContainer(NewElementPtr, TempElementStorage);
					}
				}
			}
			else
			{
				bConversionSucceeded = false;
			}

			ScriptSetHelper.Rehash();
		}
	}

	// if we could not convert the property ourself, then indicate that calling code needs to advance the property
	if (!bConversionSucceeded)
	{
		UE_LOGF(LogClass, Warning, "Set Element Type mismatch in %ls - Previous (%ls) Current (%ls) for package: %ls",
			*WriteToString<32>(Tag.Name), *WriteToString<32>(InnerPropertyTag.GetType()), *WriteToString<32>(UE::FPropertyTypeName(ElementProp)), *UnderlyingArchive.GetArchiveName());
	}

	return bConversionSucceeded ? EConvertFromTypeResult::Converted : EConvertFromTypeResult::CannotConvert;
}

#if WITH_EDITORONLY_DATA
void FSetProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (ElementProp)
	{
		ElementProp->AppendSchemaHash(Builder, bSkipEditorOnly);
	}
}
#endif

void FScriptSetHelper::Rehash()
{
	// Moved out-of-line to maybe fix a weird link error
	Set->Rehash(SetLayout, [this](const void* Src) {
		return ElementProp->GetValueTypeHash(Src);
	});
}

FField* FSetProperty::GetInnerFieldByName(const FName& InName)
{
	if (ElementProp && ElementProp->GetFName() == InName)
	{
		return ElementProp;
	}
	return nullptr;
}

void FSetProperty::GetInnerFields(TArray<FField*>& OutFields)
{
	if (ElementProp)
	{
		OutFields.Add(ElementProp);
		ElementProp->GetInnerFields(OutFields);
	}
}

void* FSetProperty::GetValueAddressAtIndex_Direct(const FProperty* Inner, void* InValueAddress, const int32 LogicalIndex) const
{
	checkf(Inner == ElementProp, TEXT("Inner property must be identical to ElementProp"));

	FScriptSetHelper SetHelper(this, InValueAddress);
	const int32 InternalIndex = SetHelper.FindInternalIndex(LogicalIndex);
	if (InternalIndex != INDEX_NONE)
	{
		return SetHelper.GetElementPtrWithoutCheck(InternalIndex);
	}

	return nullptr;
}

bool FSetProperty::UseBinaryOrNativeSerialization(const FArchive& Ar) const
{
	if (Super::UseBinaryOrNativeSerialization(Ar))
	{
		return true;
	}

	const FProperty* LocalElementProp = ElementProp;
	check(LocalElementProp);
	return LocalElementProp->UseBinaryOrNativeSerialization(Ar);
}

bool FSetProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	if (!Super::LoadTypeName(Type, Tag))
	{
		return false;
	}

	const UE::FPropertyTypeName ElementType = Type.GetParameter(0);
	FField* Field = FField::TryConstruct(ElementType.GetName(), this, GetFName());
	if (FProperty* Property = CastField<FProperty>(Field); Property && Property->LoadTypeName(ElementType, Tag))
	{
		ElementProp = Property;
		return true;
	}
	delete Field;
	return false;
}

void FSetProperty::SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const
{
	Super::SaveTypeName(Type);

	const FProperty* LocalElementProp = ElementProp;
	check(LocalElementProp);
	Type.BeginParameters();
	LocalElementProp->SaveTypeName(Type);
	Type.EndParameters();
}

bool FSetProperty::CanSerializeFromTypeName(UE::FPropertyTypeName Type) const
{
	if (!Super::CanSerializeFromTypeName(Type))
	{
		return false;
	}

	const FProperty* LocalElementProp = ElementProp;
	check(LocalElementProp);
	return LocalElementProp->CanSerializeFromTypeName(Type.GetParameter(0));
}

EPropertyVisitorControlFlow FSetProperty::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Conmtext*/)> InFunc) const
{
	FScriptSetHelper SetHelper(this, Context.Data.PropertyData);

	// Indicate in the path that this property contains inner properties
	Context.Path.Top().bContainsInnerProperties = SetHelper.Num() > 0;

	EPropertyVisitorControlFlow RetVal = Super::Visit(Context, InFunc);

	if (RetVal == EPropertyVisitorControlFlow::StepInto)
	{
		checkf(ElementProp, TEXT("Expecting a valid inner property type"));

		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			// visit element
			FPropertyVisitorScope Scope(Context.Path, FPropertyVisitorInfo(ElementProp, It.GetLogicalIndex(), EPropertyVisitorInfoType::ContainerIndex));
			FPropertyVisitorContext SubContext = Context.VisitPropertyData(SetHelper.GetElementPtr(It));

			RetVal = ElementProp->Visit(SubContext, InFunc);
			if (RetVal == EPropertyVisitorControlFlow::Stop)
			{
				return EPropertyVisitorControlFlow::Stop;
			}
			if (RetVal == EPropertyVisitorControlFlow::StepOut)
			{
				return EPropertyVisitorControlFlow::StepOver;
			}
		}
	}
	return RetVal;
}

void* FSetProperty::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	if (Info.PropertyInfo == EPropertyVisitorInfoType::ContainerIndex && Info.Property == ElementProp)
	{
		return GetValueAddressAtIndex_Direct(Info.Property, Data, Info.Index);
	}

	return nullptr;
}

bool FSetProperty::HasIntrusiveUnsetOptionalState() const
{
	return true;
}

void FSetProperty::InitializeIntrusiveUnsetOptionalValue(void* Data) const
{
	// FScriptSet's unset state constructor is good enough
	Super::InitializeIntrusiveUnsetOptionalValue(Data);
}

bool FSetProperty::IsIntrusiveOptionalValueSet(const void* Data) const
{
	// FScriptSet's unset state comparison is good enough
	return Super::IsIntrusiveOptionalValueSet(Data);
}

void FSetProperty::ClearIntrusiveOptionalValue(void* Data) const
{
	// Destroy any inner elements first, because FScriptSet's destructor will only free memory
	if (IsIntrusiveOptionalValueSet(Data))
	{
		FScriptSetHelper SetHelper(this, Data);
		SetHelper.EmptyElements();

		// Call Super to actually reset the optional to the unset state, now that any elements have been destroyed
		Super::ClearIntrusiveOptionalValue(Data);
	}
}
