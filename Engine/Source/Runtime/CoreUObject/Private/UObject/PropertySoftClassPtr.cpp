// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/LinkerPlaceholderClass.h"

/*-----------------------------------------------------------------------------
	FSoftClassProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FSoftClassProperty)

FSoftClassProperty::FSoftClassProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, MetaClass(nullptr)
{
}

FSoftClassProperty::FSoftClassProperty(FFieldVariant InOwner, const UECodeGen_Private::FSoftClassPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FObjectPropertyParamsWithoutClass&)Prop, UClass::StaticClass())
{
	this->MetaClass = Prop.MetaClassFunc ? Prop.MetaClassFunc(ETypeConstructPhase::Inner) : nullptr;
}

#if WITH_EDITORONLY_DATA
FSoftClassProperty::FSoftClassProperty(UField* InField)
	: Super(InField)
{
	USoftClassProperty* SourceProperty = CastChecked<USoftClassProperty>(InField);
	MetaClass = SourceProperty->MetaClass;
}
#endif // WITH_EDITORONLY_DATA

void FSoftClassProperty::PostDuplicate(const FField& InField)
{
	const FSoftClassProperty& Source = static_cast<const FSoftClassProperty&>(InField);
	MetaClass = Source.MetaClass;
	Super::PostDuplicate(InField);
}

FString FSoftClassProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const
{
	check(MetaClass);
	return FString::Printf(TEXT("TSoftClassPtr<%s%s> "), MetaClass->GetPrefixCPP(), *MetaClass->GetName());
}

FString FSoftClassProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	ExtendedTypeText = FString::Printf(TEXT("TSoftClassPtr<%s%s> "),MetaClass->GetPrefixCPP(),*MetaClass->GetName());
	return TEXT("SOFTCLASS");
}

void FSoftClassProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );
	Ar << MetaClass;

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (Ar.IsLoading() || Ar.IsObjectReferenceCollector())
	{
		if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
		{
			PlaceholderClass->AddReferencingProperty(this);
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	if( !MetaClass )
	{
		// If we failed to load the MetaClass and we're not a CDO, that means we relied on a class that has been removed or doesn't exist.
		// The most likely cause for this is either an incomplete recompile, or if content was migrated between games that had native class dependencies
		// that do not exist in this game.  We allow blueprint classes to continue, because compile on load will error out, and stub the class that was using it
		UClass* TestClass = dynamic_cast<UClass*>(GetOwnerStruct());
		if( TestClass && TestClass->HasAllClassFlags(CLASS_Native) && !TestClass->HasAllClassFlags(CLASS_NewerVersionExists) && (TestClass->GetOutermost() != GetTransientPackage()) )
		{
			checkf(false, TEXT("Class property tried to serialize a missing class.  Did you remove a native class and not fully recompile?"));
		}
	}
}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
void FSoftClassProperty::SetMetaClass(UClass* NewMetaClass)
{
	if (ULinkerPlaceholderClass* NewPlaceholderClass = Cast<ULinkerPlaceholderClass>(NewMetaClass))
	{
		NewPlaceholderClass->AddReferencingProperty(this);
	}

	if (ULinkerPlaceholderClass* OldPlaceholderClass = Cast<ULinkerPlaceholderClass>(MetaClass))
	{
		OldPlaceholderClass->RemoveReferencingProperty(this);
	}
	MetaClass = NewMetaClass;
}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

void FSoftClassProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject( MetaClass );
	Super::AddReferencedObjects( Collector );
}

bool FSoftClassProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && (MetaClass == ((FSoftClassProperty*)Other)->MetaClass);
}
