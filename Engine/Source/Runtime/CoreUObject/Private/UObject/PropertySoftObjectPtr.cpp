// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "UObject/LinkerLoad.h"
#include "UObject/SoftObjectPtr.h"
#if WITH_EDITOR
#include "Misc/EditorPathHelper.h"
#endif

/*-----------------------------------------------------------------------------
	FSoftObjectProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FSoftObjectProperty)

FSoftObjectProperty::FSoftObjectProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FSoftObjectProperty::FSoftObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FSoftObjectPropertyParams& Prop)
	: Super(InOwner, Prop)
{
}

FSoftObjectProperty::FSoftObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FObjectPropertyParamsWithoutClass& Prop, UClass* InClass)
	: Super(InOwner, Prop, InClass)
{
}

#if WITH_EDITORONLY_DATA
FSoftObjectProperty::FSoftObjectProperty(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA

FString FSoftObjectProperty::GetCPPType(FString* ExtendedTypeText, uint32 CPPExportFlags) const
{
	if (ensureMsgf(PropertyClass, TEXT("Soft object property missing PropertyClass: %s"), *GetFullNameSafe(this)))
	{
		return FString::Printf(TEXT("TSoftObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
	}
	else
	{
		return TEXT("TSoftObjectPtr<UObject>");
	}
}

FString FSoftObjectProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	ExtendedTypeText = FString::Printf(TEXT("TSoftObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
	return TEXT("SOFTOBJECT");
}

FName FSoftObjectProperty::GetID() const
{
	// SoftClass shares the same tag, they are binary compatible
	return NAME_SoftObjectProperty;
}

// this is always shallow, can't see that we would want it any other way
bool FSoftObjectProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	FSoftObjectPtr ObjectA = *(FSoftObjectPtr*)NotNullGet(A);
	FSoftObjectPtr ObjectB = B ? *((FSoftObjectPtr*)B) : FSoftObjectPtr();

	return ObjectA.GetUniqueID() == ObjectB.GetUniqueID();
}

void FSoftObjectProperty::LinkInternal(FArchive& Ar)
{
	checkf(!HasAnyPropertyFlags(CPF_NonNullable), TEXT("Soft Object Properties can't be non nullable but \"%s\" is marked as CPF_NonNullable"), *GetFullName());
	Super::LinkInternal(Ar);
}

void FSoftObjectProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	// We never serialize our reference while the garbage collector is harvesting references
	// to objects, because we don't want soft object pointers to keep objects from being garbage collected
	// Allow persistent archives so they can keep track of string references. (e.g. FArchiveSaveTagImports)
	if( !UnderlyingArchive.IsObjectReferenceCollector() || UnderlyingArchive.IsModifyingWeakAndStrongReferences() || UnderlyingArchive.IsPersistent() )
	{
		Slot << *(FSoftObjectPtr*)NotNullGet(Value);
	}
	else
	{
		// TODO: This isn't correct, but it keeps binary serialization happy. We should ALWAYS be serializing the pointer
		// to the archive in this function, and allowing the underlying archive to ignore it if necessary
		Slot.EnterStream();
	}
}

bool FSoftObjectProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	// Serialize directly, will use FBitWriter/Reader
	Ar << *(FSoftObjectPtr*)NotNullGet(Data);

	return true;
}

void FSoftObjectProperty::ExportText_Internal( FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope ) const
{
	FSoftObjectPtr SoftObjectPtr;
	
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		GetValue_InContainer(PropertyValueOrContainer, &SoftObjectPtr);
	}
	else
	{
		SoftObjectPtr = *(FSoftObjectPtr*)PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType);
	}

	FSoftObjectPath SoftObjectPath;
	UObject *Object = SoftObjectPtr.Get();

	if (Object)
	{
#if WITH_EDITOR
		// Use object in case name has changed. Export editor path if feature is enabled.
		SoftObjectPath = FEditorPathHelper::GetEditorPathFromReferencer(Object, Parent);
#else
		// Use object in case name has changed.
		SoftObjectPath = FSoftObjectPath(Object);
#endif
	}
	else
	{
		SoftObjectPath = SoftObjectPtr.GetUniqueID();
	}

	SoftObjectPath.ExportTextItem(ValueStr, SoftObjectPath, Parent, PortFlags, ExportRootScope);
}

const TCHAR* FSoftObjectProperty::ImportText_Internal(const TCHAR* InBuffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	FSoftObjectPath SoftObjectPath;

	bool bImportTextSuccess = false;
#if WITH_EDITOR
	if (HasAnyPropertyFlags(CPF_EditorOnly))
	{
		FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None, ESoftObjectPathCollectType::EditorOnlyCollect, ESoftObjectPathSerializeType::AlwaysSerialize);
		bImportTextSuccess = SoftObjectPath.ImportTextItem(InBuffer, PortFlags, Parent, ErrorText, GetLinker());
	}
	else
#endif // WITH_EDITOR
	{
		bImportTextSuccess = SoftObjectPath.ImportTextItem(InBuffer, PortFlags, Parent, ErrorText, GetLinker());
	}

	if (bImportTextSuccess)
	{
#if WITH_EDITOR
		// If EditorPath feature is enabled. Make sure we import a proper Editor Path if Object has a EditorPathOwner
		if (FEditorPathHelper::IsEnabled())
		{
			if (UObject* Object = SoftObjectPath.ResolveObject())
			{
				SoftObjectPath = FEditorPathHelper::GetEditorPathFromReferencer(Object, Parent);
			}
		}
#endif

		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			FSoftObjectPtr SoftObjectPtr(SoftObjectPath);
			SetValue_InContainer(ContainerOrPropertyPtr, SoftObjectPtr);
		}
		else
		{
			FSoftObjectPtr& SoftObjectPtr = *(FSoftObjectPtr*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
			SoftObjectPtr = SoftObjectPath;
		}
		return InBuffer;
	}
	else
	{
		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			FSoftObjectPtr NullPtr(nullptr);
			SetValue_InContainer(ContainerOrPropertyPtr, NullPtr);
		}
		else
		{
			FSoftObjectPtr& SoftObjectPtr = *(FSoftObjectPtr*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
			SoftObjectPtr = nullptr;
		}
		return nullptr;
	}
}

EConvertFromTypeResult FSoftObjectProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	static FName NAME_AssetObjectProperty = "AssetObjectProperty";
	static FName NAME_SoftObjectPath = "SoftObjectPath";
	static FName NAME_SoftClassPath = "SoftClassPath";
	static FName NAME_StringAssetReference = "StringAssetReference";
	static FName NAME_StringClassReference = "StringClassReference";

	FArchive& Archive = Slot.GetUnderlyingArchive();

	if (Tag.Type == NAME_AssetObjectProperty)
	{
		// Old name of soft object property, serialize normally
		uint8* DestAddress = ContainerPtrToValuePtr<uint8>(Data, Tag.ArrayIndex);

		Tag.SerializeTaggedProperty(Slot, this, DestAddress, nullptr);

		if (Archive.IsCriticalError())
		{
			return EConvertFromTypeResult::CannotConvert;
		}

		return EConvertFromTypeResult::Converted;
	}
	else if (Tag.Type == NAME_ObjectProperty)
	{
		// This property used to be a raw FObjectProperty Foo* but is now a TSoftObjectPtr<Foo>;
		// Serialize from mismatched tag directly into the FSoftObjectPtr's soft object path to ensure that the delegates needed for cooking
		// are fired
		FSoftObjectPtr* PropertyValue = GetPropertyValuePtr_InContainer(Data, Tag.ArrayIndex);
		check(PropertyValue);

		FSerializedPropertyScope SerializedProperty(Archive, this);
		return PropertyValue->GetUniqueID().SerializeFromMismatchedTag(Tag, Slot) ? EConvertFromTypeResult::Converted : EConvertFromTypeResult::UseSerializeItem;
	}
	else if (Tag.Type == NAME_StructProperty)
	{
		const FName StructName = Tag.GetType().GetParameterName(0);
		if (StructName == NAME_SoftObjectPath || StructName == NAME_SoftClassPath || StructName == NAME_StringAssetReference || StructName == NAME_StringClassReference)
		{
			// This property used to be a FSoftObjectPath but is now a TSoftObjectPtr<Foo>
			FSoftObjectPath PreviousValue;
			// explicitly call Serialize to ensure that the various delegates needed for cooking are fired
			FSerializedPropertyScope SerializedProperty(Archive, this);
			PreviousValue.Serialize(Slot);

			// now copy the value into the object's address space
			FSoftObjectPtr PreviousValueSoftObjectPtr;
			PreviousValueSoftObjectPtr = PreviousValue;
			SetPropertyValue_InContainer(Data, PreviousValueSoftObjectPtr, Tag.ArrayIndex);

			return EConvertFromTypeResult::Converted;
		}
	}

	return EConvertFromTypeResult::UseSerializeItem;
}

UObject* FSoftObjectProperty::LoadObjectPropertyValue(const void* PropertyValueAddress) const
{
	return GetPropertyValue(PropertyValueAddress).LoadSynchronous();
}

UObject* FSoftObjectProperty::GetObjectPropertyValue(const void* PropertyValueAddress) const
{
	return GetPropertyValue(PropertyValueAddress).Get();
}

TObjectPtr<UObject> FSoftObjectProperty::GetObjectPtrPropertyValue(const void* PropertyValueAddress) const
{
	return TObjectPtr<UObject>(GetPropertyValue(PropertyValueAddress).Get());
}

UObject* FSoftObjectProperty::GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	UObject* Result = nullptr;
	GetWrappedUObjectPtrValues<FSoftObjectPtr>(&Result, ContainerAddress, EPropertyMemoryAccess::InContainer, ArrayIndex, 1);
	return Result;
}

TObjectPtr<UObject> FSoftObjectProperty::GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	TObjectPtr<UObject> Result = nullptr;
	GetWrappedUObjectPtrValues<FSoftObjectPtr>(&Result, ContainerAddress, EPropertyMemoryAccess::InContainer, ArrayIndex, 1);
	return Result;
}

void FSoftObjectProperty::SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const
{
	SetPropertyValue(PropertyValueAddress, TCppType(Value));
}

void FSoftObjectProperty::SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const
{
	SetPropertyValue(PropertyValueAddress, TCppType(Ptr));
}

void FSoftObjectProperty::SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex) const
{
	SetWrappedUObjectPtrValues<FSoftObjectPtr>(ContainerAddress, EPropertyMemoryAccess::InContainer, &Value, ArrayIndex, 1);
}

void FSoftObjectProperty::SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex) const
{
	SetWrappedUObjectPtrValues<FSoftObjectPtr>(ContainerAddress, EPropertyMemoryAccess::InContainer, &Ptr, ArrayIndex, 1);
}

bool FSoftObjectProperty::AllowCrossLevel() const
{
	return true;
}

uint32 FSoftObjectProperty::GetValueTypeHashInternal(const void* Src) const
{
	return GetTypeHash(GetPropertyValue(Src));
}
