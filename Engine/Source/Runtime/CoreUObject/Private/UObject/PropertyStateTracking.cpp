// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyStateTracking.h"

#if WITH_EDITORONLY_DATA

#include "Algo/Copy.h"
#include "Algo/Sort.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Hash/Blake3.h"
#include "String/ParseTokens.h"
#include "Templates/SharedPointer.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/PropertyPathNameTree.h"
#include "UObject/PropertyTypeName.h"
#include "UObject/UObjectAnnotation.h"

namespace UE
{

struct FUnknownPropertyTreeAnnotation
{
	bool IsDefault() const
	{
		return !Tree.IsValid();
	}

	TSharedPtr<FPropertyPathNameTree> Tree;
};

using FUnknownPropertyTreeAnnotationStore = FUObjectAnnotationSparse<FUnknownPropertyTreeAnnotation, /*bAutoRemove*/ true>;

UE_AUTORTFM_ALWAYS_OPEN FUnknownPropertyTreeAnnotationStore& GetUnknownPropertyTreeAnnotations()
{
	static FUnknownPropertyTreeAnnotationStore Annotations;
	return Annotations;
}

FUnknownPropertyTree::FUnknownPropertyTree(const UObject* InOwner)
	: Owner(InOwner)
{
}

TSharedPtr<FPropertyPathNameTree> FUnknownPropertyTree::Find() const
{
	return GetUnknownPropertyTreeAnnotations().GetAnnotation(Owner).Tree;
}

TSharedPtr<FPropertyPathNameTree> FUnknownPropertyTree::FindOrCreate()
{
	FUnknownPropertyTreeAnnotationStore& Store = GetUnknownPropertyTreeAnnotations();
	TSharedPtr<FPropertyPathNameTree> Tree = Store.GetAnnotation(Owner).Tree;
	if (!Tree)
	{
		Tree = MakeShared<FPropertyPathNameTree>();
		Store.AddAnnotation(Owner, {Tree});
	}
	return Tree;
}

void FUnknownPropertyTree::Destroy()
{
	GetUnknownPropertyTreeAnnotations().RemoveAnnotation(Owner);
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FUnknownEnumNamesAnnotation
{
	bool IsDefault() const
	{
		return !Enums.IsValid();
	}

	struct FInfo
	{
		TSet<FName> Names;
		bool bHasFlags = false;
	};

	TSharedPtr<TMap<FPropertyTypeName, FInfo>> Enums;
};

using FUnknownEnumNamesAnnotationStore = FUObjectAnnotationSparse<FUnknownEnumNamesAnnotation, /*bAutoRemove*/ true>;

UE_AUTORTFM_ALWAYS_OPEN FUnknownEnumNamesAnnotationStore& GetUnknownEnumNamesAnnotations()
{
	static FUnknownEnumNamesAnnotationStore Annotations;
	return Annotations;
}

FUnknownEnumNames::FUnknownEnumNames(const UObject* InOwner)
	: Owner(InOwner)
{
}

void FUnknownEnumNames::Add(const UEnum* Enum, FPropertyTypeName EnumTypeName, FName EnumValueName)
{
	checkf(Enum || !EnumTypeName.IsEmpty(), TEXT("FUnknownEnumNames::Add() requires an enum or its type name. Owner: %s"), *Owner->GetPathName());

	if (EnumTypeName.IsEmpty())
	{
		FPropertyTypeNameBuilder Builder;
		Builder.AddPath(Enum);
		EnumTypeName = Builder.Build();
	}

	FUnknownEnumNamesAnnotationStore& Store = GetUnknownEnumNamesAnnotations();
	TSharedPtr<TMap<FPropertyTypeName, FUnknownEnumNamesAnnotation::FInfo>> Enums = Store.GetAnnotation(Owner).Enums;
	if (!Enums)
	{
		Enums = MakeShared<TMap<FPropertyTypeName, FUnknownEnumNamesAnnotation::FInfo>>();
		Store.AddAnnotation(Owner, {Enums});
	}

	FUnknownEnumNamesAnnotation::FInfo& Info = Enums->FindOrAdd(EnumTypeName);

	TStringBuilder<128> EnumValueString(InPlace, EnumValueName);
	if (String::FindFirstChar(EnumValueString, TEXT('|')) == INDEX_NONE)
	{
		if (int32 ColonIndex = String::FindFirst(EnumValueString, TEXTVIEW("::")); ColonIndex != INDEX_NONE)
		{
			Info.Names.Emplace(EnumValueString.ToView().RightChop(ColonIndex + TEXTVIEW("::").Len()));
		}
		else
		{
			Info.Names.Add(EnumValueName);
		}
	}
	else
	{
		Info.bHasFlags = true;
		String::ParseTokens(EnumValueString, TEXT('|'), [&Info, Enum](FStringView Token)
		{
			FName Name(Token);
			if (!Enum || Enum->GetIndexByName(Name) == INDEX_NONE)
			{
				Info.Names.Add(Name);
			}
		}, String::EParseTokensOptions::SkipEmpty | String::EParseTokensOptions::Trim);
	}

	if (!Info.bHasFlags && Enum && Enum->HasAnyEnumFlags(EEnumFlags::Flags))
	{
		Info.bHasFlags = true;
	}
}

void FUnknownEnumNames::Find(FPropertyTypeName EnumTypeName, TArray<FName>& OutNames, bool& bOutHasFlags) const
{
	checkf(!EnumTypeName.IsEmpty(), TEXT("FUnknownEnumNames::Find() requires an enum type name. Owner: %s"), *Owner->GetPathName());

	OutNames.Empty();
	bOutHasFlags = false;

	if (TSharedPtr<TMap<FPropertyTypeName, FUnknownEnumNamesAnnotation::FInfo>> Enums = GetUnknownEnumNamesAnnotations().GetAnnotation(Owner).Enums)
	{
		if (const FUnknownEnumNamesAnnotation::FInfo* Info = Enums->Find(EnumTypeName))
		{
			OutNames = Info->Names.Array();
			bOutHasFlags = Info->bHasFlags;
		}
	}
}

bool FUnknownEnumNames::IsEmpty() const
{
	return GetUnknownEnumNamesAnnotations().GetAnnotation(Owner).IsDefault();
}

void FUnknownEnumNames::Destroy()
{
	GetUnknownEnumNamesAnnotations().RemoveAnnotation(Owner);
}

void AppendHash(FBlake3& Builder, const FUnknownEnumNames& EnumNames)
{
	if (TSharedPtr<TMap<FPropertyTypeName, FUnknownEnumNamesAnnotation::FInfo>> Enums = GetUnknownEnumNamesAnnotations().GetAnnotation(EnumNames.Owner).Enums)
	{
		TArray<FPropertyTypeName, TInlineAllocator<4>> Keys;
		Enums->GetKeys(Keys);
		Algo::Sort(Keys);

		for (const FPropertyTypeName& Key : Keys)
		{
			AppendHash(Builder, Key);
			const FUnknownEnumNamesAnnotation::FInfo& Info = Enums->FindChecked(Key);
			TArray<FName, TInlineAllocator<4>> Names;
			Names.Reserve(Info.Names.Num());
			Algo::Copy(Info.Names, Names);
			Algo::Sort(Names, FNameLexicalLess());
			for (const FName& Name : Names)
			{
				AppendHash(Builder, Name);
			}
			Builder.Update(&Info.bHasFlags, sizeof(Info.bHasFlags));
		}
	}
}

} // UE

#endif // WITH_EDITORONLY_DATA
