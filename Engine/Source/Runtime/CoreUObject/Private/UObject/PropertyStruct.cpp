// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"

#include "Hash/Blake3.h"
#include "IO/IoHash.h"
#include "Misc/StringBuilder.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerPlaceholderBase.h"
#include "UObject/PropertyHelper.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectThreadContext.h"
#include "Async/UniqueLock.h"

static inline void PreloadInnerStructMembers(FStructProperty* StructProperty)
{
	if (UseCircularDependencyLoadDeferring())
	{
		uint32 PropagatedLoadFlags = 0;
		if (FLinkerLoad* Linker = StructProperty->GetLinker())
		{
			PropagatedLoadFlags |= (Linker->LoadFlags & LOAD_DeferDependencyLoads);
		}

		if (UScriptStruct* Struct = StructProperty->Struct)
		{
			if (FLinkerLoad* StructLinker = Struct->GetLinker())
			{
				UE::TUniqueLock ScopeLock(StructLinker->Mutex);
				TGuardValue<uint32> LoadFlagGuard(StructLinker->LoadFlags, StructLinker->LoadFlags | PropagatedLoadFlags);
				Struct->RecursivelyPreload();
			}
		}
	}
	else
	{
		StructProperty->Struct->RecursivelyPreload();
	}
}

/*-----------------------------------------------------------------------------
	FStructProperty.
-----------------------------------------------------------------------------*/

IMPLEMENT_FIELD(FStructProperty)

FStructProperty::FStructProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
	, Struct(nullptr)
{
	SetElementSize(0);
}

static EPropertyFlags GetStructComputedPropertyFlags(const UECodeGen_Private::FStructPropertyParams& Prop)
{
	EPropertyFlags ComputedPropertyFlags = CPF_None;
	UScriptStruct* Struct = Prop.ScriptStructFunc ? Prop.ScriptStructFunc(ETypeConstructPhase::Outer) : nullptr;
	if (Struct && Struct->GetCppStructOps())
	{
		ComputedPropertyFlags = Struct->GetCppStructOps()->GetComputedPropertyFlags();
	}
	return ComputedPropertyFlags;
}

FStructProperty::FStructProperty(FFieldVariant InOwner, const UECodeGen_Private::FStructPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop, GetStructComputedPropertyFlags(Prop))
{
	Struct = Prop.ScriptStructFunc ? Prop.ScriptStructFunc(ETypeConstructPhase::Outer) : nullptr;
}

#if WITH_EDITORONLY_DATA
FStructProperty::FStructProperty(UField* InField)
	: Super(InField)
{
	UStructProperty* SourceProperty = CastChecked<UStructProperty>(InField);
	Struct = SourceProperty->Struct;
	check(GetElementSize() == SourceProperty->ElementSize); // this should've been set by FProperty
}
#endif // WITH_EDITORONLY_DATA

int32 FStructProperty::GetMinAlignment() const
{
	return Struct->GetMinAlignment();
}

void FStructProperty::PostDuplicate(const FField& InField)
{
	const FStructProperty& Source = static_cast<const FStructProperty&>(InField);
	Struct = Source.Struct;
	Super::PostDuplicate(InField);
}

void FStructProperty::LinkInternal(FArchive& Ar)
{
	// We potentially have to preload the property itself here, if we were the inner of an array property
	//if(HasAnyFlags(RF_NeedLoad))
	//{
	//	GetLinker()->Preload(this);
	//}

	if (Struct)
	{
		// Preload is required here in order to load the value of Struct->PropertiesSize
		Ar.Preload(Struct);
	}
	else
	{
		UE_LOGF(LogProperty, Error, "Struct type unknown for property '%ls'; perhaps the USTRUCT() was renamed or deleted?", *GetFullName());
		Struct = GetFallbackStruct();
	}
	PreloadInnerStructMembers(this);
	
	SetElementSize(Align(Struct->PropertiesSize, Struct->GetMinAlignment()));
	if(UScriptStruct::ICppStructOps* Ops = Struct->GetCppStructOps())
	{
		PropertyFlags |= Ops->GetComputedPropertyFlags();
	}
	else
	{
		// User Defined structs won't have UScriptStruct::ICppStructOps. Setting their flags here.
		PropertyFlags |= CPF_HasGetValueTypeHash;
	}

	if (Struct->StructFlags & STRUCT_ZeroConstructor)
	{
		PropertyFlags |= CPF_ZeroConstructor;
	}
	if (Struct->StructFlags & STRUCT_IsPlainOldData)
	{
		PropertyFlags |= CPF_IsPlainOldData;
	}
	if (Struct->StructFlags & STRUCT_NoDestructor)
	{
		PropertyFlags |= CPF_NoDestructor;
	}
}

bool FStructProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	return Struct->CompareScriptStruct(A, B, PortFlags);
}

bool FStructProperty::UseBinaryOrNativeSerialization(const FArchive& Ar) const
{
	if (Super::UseBinaryOrNativeSerialization(Ar))
	{
		return true;
	}

	check(Struct);
	const bool bUseBinarySerialization = Struct->UseBinarySerialization(Ar);
	const bool bUseNativeSerialization = Struct->UseNativeSerialization();
	return bUseBinarySerialization || bUseNativeSerialization;
}

bool FStructProperty::FindInnerPropertyInstance(FName PropertyName, const void* Data, const FProperty*& OutProp, const void*& OutData) const
{
	return Struct->FindInnerPropertyInstance(PropertyName, Data, OutProp, OutData);
}

uint32 FStructProperty::GetValueTypeHashInternal(const void* Src) const
{
	check(Struct);
	return Struct->GetStructTypeHash(Src);
}

void FStructProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	FScopedPlaceholderPropertyTracker ImportPropertyTracker(this);

	Struct->SerializeItem(Slot, Value, Defaults);
}

bool FStructProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData) const
{
	//------------------------------------------------
	//	Custom NetSerialization
	//------------------------------------------------
	if (Struct->StructFlags & STRUCT_NetSerializeNative)
	{
		UScriptStruct::ICppStructOps* CppStructOps = Struct->GetCppStructOps();
		check(CppStructOps); // else should not have STRUCT_NetSerializeNative
		bool bSuccess = true;
		bool bMapped = CppStructOps->NetSerialize(Ar, Map, bSuccess, Data);
		if (!bSuccess)
		{
			UE_LOGF(LogProperty, Warning, "Native NetSerialize %ls (%ls) failed.", *GetFullName(), *Struct->GetFullName() );
		}
		return bMapped;
	}

	UE_LOGF( LogProperty, Fatal, "Deprecated code path" );

	return 1;
}

bool FStructProperty::SupportsNetSharedSerialization() const
{
	return !(Struct->StructFlags & STRUCT_NetSerializeNative) || (Struct->StructFlags & STRUCT_NetSharedSerialization);
}

void FStructProperty::GetPreloadDependencies(TArray<UObject*>& OutDeps)
{
	Super::GetPreloadDependencies(OutDeps);
	OutDeps.Add(Struct);
}

void FStructProperty::Serialize( FArchive& Ar )
{
	Super::Serialize( Ar );

	static UScriptStruct* FallbackStruct = GetFallbackStruct();
	
	if (Ar.IsPersistent() && Ar.GetLinker() && Ar.IsLoading() && !Struct)
	{
		// It's necessary to solve circular dependency problems, when serializing the Struct causes linking of the Property.
		Struct = FallbackStruct;
	}

	Ar << Struct;
#if WITH_EDITOR
	if (Ar.IsPersistent() && Ar.GetLinker())
	{
		if (!Struct && Ar.IsLoading())
		{
			UE_LOGF(LogProperty, Error, "FStructProperty::Serialize Loading: Property '%ls'. Unknown structure.", *GetFullName());
			Struct = FallbackStruct;
		}
		else if ((FallbackStruct == Struct) && Ar.IsSaving())
		{
			UE_LOGF(LogProperty, Error, "FStructProperty::Serialize Saving: Property '%ls'. FallbackStruct structure.", *GetFullName());
		}
	}
#endif // WITH_EDITOR
	if (Struct)
	{
		PreloadInnerStructMembers(this);
	}
	else
	{
		ensure(true);
	}
}
void FStructProperty::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Struct);
	Super::AddReferencedObjects(Collector);
}

FString FStructProperty::GetCPPType( FString* ExtendedTypeText/*=NULL*/, uint32 CPPExportFlags/*=0*/ ) const
{
	return Struct->GetStructCPPName(CPPExportFlags);
}

FString FStructProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	ExtendedTypeText = GetCPPType(NULL, CPPF_None);
	return TEXT("STRUCT");
}

void FStructProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	void* StructData = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		int32 RequiredAllocSize = Struct->GetStructureSize();
		StructData = FMemory::Malloc(RequiredAllocSize);
		Struct->InitializeStruct(StructData);
		GetValue_InContainer(PropertyValueOrContainer, StructData);
	}
	else
	{
		StructData = PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType);
	}

	Struct->ExportText(ValueStr, StructData, DefaultValue, Parent, PortFlags, ExportRootScope, true);

	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		Struct->DestroyStruct(StructData);
		FMemory::Free(StructData);
		StructData = nullptr;
	}
}

const TCHAR* FStructProperty::ImportText_Internal(const TCHAR* InBuffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	FScopedPlaceholderPropertyTracker ImportPropertyTracker(this);

	uint32 PropagatedLoadFlags = 0;
	if (FLinkerLoad* Linker = GetLinker())
	{
		PropagatedLoadFlags |= (Linker->LoadFlags & LOAD_DeferDependencyLoads);
	}

	uint32 OldFlags = 0;
	FLinkerLoad* StructLinker = Struct->GetLinker();
	TOptional<UE::TUniqueLock<UE::FRecursiveMutex>> OptionalMutex;
	if (StructLinker)
	{
		OptionalMutex.Emplace(StructLinker->Mutex);
		OldFlags = StructLinker->LoadFlags;
		StructLinker->LoadFlags |= OldFlags | PropagatedLoadFlags;
	}
#endif 
	void* StructData = nullptr;
	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		int32 RequiredAllocSize = Struct->GetStructureSize();
		StructData = FMemory::Malloc(RequiredAllocSize);
		Struct->InitializeStruct(StructData);
		GetValue_InContainer(ContainerOrPropertyPtr, StructData);
	}
	else
	{
		StructData = PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType);
	}

	const TCHAR* Result = Struct->ImportText(InBuffer, StructData, Parent, PortFlags, ErrorText, [this]() { return GetName(); }, true);

	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		SetValue_InContainer(ContainerOrPropertyPtr, StructData);
		Struct->DestroyStruct(StructData);
		FMemory::Free(StructData);
		StructData = nullptr;
	}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (StructLinker)
	{
		StructLinker->LoadFlags = OldFlags;
	}
#endif

	return Result;
}

void FStructProperty::CopyValuesInternal( void* Dest, void const* Src, int32 Count  ) const
{
	Struct->CopyScriptStruct(Dest, Src, Count);
}

void FStructProperty::InitializeValueInternal( void* InDest ) const
{
	Struct->InitializeStruct(InDest, ArrayDim);
}

void FStructProperty::ClearValueInternal( void* Data ) const
{
	Struct->ClearScriptStruct(Data, 1); // clear only does one value
}

void FStructProperty::DestroyValueInternal( void* Dest ) const
{
	Struct->DestroyStruct(Dest, ArrayDim);
}

bool FStructProperty::ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const
{
	// Skip if already being processed.
	if (EncounteredStructProps.Contains(this))
	{
		return false;
	}

	if (!Struct)
	{
		UE_LOGF(LogGarbage, Warning, "Broken FStructProperty does not have a UStruct: %ls", *GetFullName() );
		return false;
	}

	if (const UScriptStruct::ICppStructOps* CppStructOps = Struct->GetCppStructOps())
	{
		if (CppStructOps->HasClearOnFinishDestroy())
		{
			return true;
		}
	}

	EncounteredStructProps.Add(this);

	bool bValue = false;
	for (const FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (Property->ContainsFinishDestroy(EncounteredStructProps))
		{
			bValue = true;
			break;
		}
	}

	EncounteredStructProps.RemoveSingleSwap(this, EAllowShrinking::No);
	
	return bValue;
}

void FStructProperty::FinishDestroyInternal( void* Data ) const
{
	const int32 Stride = Struct->GetStructureSize();
	
	if (UScriptStruct::ICppStructOps* CppStructOps = Struct->GetCppStructOps())
	{
		if (CppStructOps->HasClearOnFinishDestroy())
		{
			Struct->ClearScriptStruct(Data, ArrayDim);
			return;
		}
	}
	
	for (int32 ArrayIndex = 0; ArrayIndex < ArrayDim; ArrayIndex++)
	{
		uint8* ItemData = (uint8*)Data + ArrayIndex * Stride;
		for (const FProperty* Property = Struct->PropertyLink; Property; Property = Property->PropertyLinkNext)
		{
			Property->FinishDestroy(Property->ContainerPtrToValuePtr<void>(ItemData));
		}
	}
}

bool FStructProperty::HasIntrusiveUnsetOptionalState() const  
{ 
	if (UScriptStruct::ICppStructOps* CppStructOps = Struct->GetCppStructOps())
	{
		return CppStructOps->HasIntrusiveUnsetOptionalState();
	}
	return false;
}

void FStructProperty::InitializeIntrusiveUnsetOptionalValue(void* Data) const 
{
	if (UScriptStruct::ICppStructOps* CppStructOps = Struct->GetCppStructOps())
	{
		CppStructOps->InitializeIntrusiveUnsetOptionalValue(Data);
		return;
	}
	checkf(false, TEXT("This should only be called when there is an intrusive unset state, which requires CppStructOps"));
}

bool FStructProperty::IsIntrusiveOptionalValueSet(const void* Data) const 
{
	if (UScriptStruct::ICppStructOps* CppStructOps = Struct->GetCppStructOps())
	{
		return CppStructOps->IsIntrusiveOptionalValueSet(Data);
	}
	checkf(false, TEXT("This should only be called when there is an intrusive unset state, which requires CppStructOps"));
	return false;
}

void FStructProperty::ClearIntrusiveOptionalValue(void* Data) const 
{
	if (UScriptStruct::ICppStructOps* CppStructOps = Struct->GetCppStructOps())
	{
		CppStructOps->ClearIntrusiveOptionalValue(Data);
		return;
	}
	checkf(false, TEXT("This should only be called when there is an intrusive unset state, which requires CppStructOps"));
}

/**
 * Creates new copies of components
 * 
 * @param	Data				pointer to the address of the instanced object referenced by this UComponentProperty
 * @param	DefaultData			pointer to the address of the default value of the instanced object referenced by this UComponentProperty
 * @param	Owner				the object that contains this property's data
 * @param	InstanceGraph		contains the mappings of instanced objects and components to their templates
 */
void FStructProperty::InstanceSubobjects( void* Data, void const* DefaultData, TNotNull<UObject*> InOwner, FObjectInstancingGraph* InstanceGraph )
{
	for (int32 Index = 0; Index < ArrayDim; Index++)
	{
		Struct->InstanceSubobjectTemplates( (uint8*)Data + GetElementSize() * Index, DefaultData ? (uint8*)DefaultData + GetElementSize() * Index : NULL, Struct, InOwner, InstanceGraph );
	}
}

bool FStructProperty::SameType(const FProperty* Other) const
{
	return Super::SameType(Other) && (Struct == ((FStructProperty*)Other)->Struct);
}

EConvertFromTypeResult FStructProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	if (Struct)
	{
		FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
		const bool bCanSerialize = CanSerializeFromTypeName(Tag.GetType());

		if ((Struct->StructFlags & STRUCT_SerializeFromMismatchedTag) && !bCanSerialize)
		{
			UScriptStruct::ICppStructOps* CppStructOps = Struct->GetCppStructOps();
			check(CppStructOps && (CppStructOps->HasSerializeFromMismatchedTag() || CppStructOps->HasStructuredSerializeFromMismatchedTag())); // else should not have STRUCT_SerializeFromMismatchedTag
			void* DestAddress = ContainerPtrToValuePtr<void>(Data, Tag.ArrayIndex);
			if (CppStructOps->HasStructuredSerializeFromMismatchedTag() && CppStructOps->StructuredSerializeFromMismatchedTag(Tag, Slot, DestAddress))
			{
				return EConvertFromTypeResult::Converted;
			}
			else
			{
				FArchiveUObjectFromStructuredArchive Adapter(Slot);
				if (CppStructOps->HasSerializeFromMismatchedTag() && CppStructOps->SerializeFromMismatchedTag(Tag, Adapter.GetArchive(), DestAddress))
				{
					return EConvertFromTypeResult::Converted;
				}
				else if (((Struct->StructFlags & STRUCT_SerializeNative) == 0) && CppStructOps->HasSerializeFromMismatchedTag() && CppStructOps->IsUECoreVariant())
				{
					// Special case for Transform, as the f/d variants are immutable whilst the default is not, so we must call SerializeTaggedProperties directly to perform the conversion.
					if (Tag.GetType().GetParameterName(0) == NAME_Transform)
					{
						Struct->SerializeTaggedProperties(Slot, (uint8*)DestAddress, Struct, nullptr);
						return EConvertFromTypeResult::Converted;
					}
					// If a core variant without a native serializer returns false from SerializeFromMismatchedTag fall back to standard SerializeItem.
					// We rely on all properties within the variant supporting SerializeFromMismatchedTag to perform the conversion per property.
					return EConvertFromTypeResult::UseSerializeItem;
				}
				else
				{
					UE_LOGF(LogClass, Warning, "SerializeFromMismatchedTag failed: Type mismatch in %ls - Previous (%ls) Current(%ls) in package: %ls",
						*WriteToString<32>(Tag.Name), *WriteToString<64>(Tag.GetType()), *WriteToString<64>(UE::FPropertyTypeName(this)), *UnderlyingArchive.GetArchiveName());
					return EConvertFromTypeResult::CannotConvert;
				}
			}
		}

		if (Tag.Type == NAME_StructProperty && !bCanSerialize && (UnderlyingArchive.UEVer() >= VER_UE4_STRUCT_GUID_IN_PROPERTY_TAG || !UseBinaryOrNativeSerialization(UnderlyingArchive)))
		{
			UE_LOGF(LogClass, Warning, "Struct Property %ls has a struct type mismatch (tag %ls != prop %ls) in package: %ls. If that struct got renamed, add an entry to ActiveStructRedirects.",
				*WriteToString<32>(Tag.Name), *WriteToString<64>(Tag.GetType().GetParameter(0)), *WriteToString<64>(UE::FPropertyTypeName(this).GetParameter(0)), *UnderlyingArchive.GetArchiveName());
			return EConvertFromTypeResult::CannotConvert;
		}
	}
	return EConvertFromTypeResult::UseSerializeItem;
}

#if WITH_EDITORONLY_DATA
void FStructProperty::AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const
{
	Super::AppendSchemaHash(Builder, bSkipEditorOnly);
	if (Struct)
	{
		const FIoHash& StructSchemaHash = Struct->GetSchemaHash(bSkipEditorOnly);
		Builder.Update(&StructSchemaHash, sizeof(StructSchemaHash));
	}
}
#endif

bool FStructProperty::LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag)
{
	if (!Super::LoadTypeName(Type, Tag))
	{
		return false;
	}

	const UE::FPropertyTypeName TypePath = Type.GetParameter(0);
	if ((Struct = UE::FindObjectByTypePath<UScriptStruct>(TypePath)))
	{
		return true;
	}

	// TODO: Look up the struct based on the guid.
	//const FName StructGuidName = Type.GetParameterName(1);
	//if (FGuid StructGuid; !StructGuidName.IsNone() && FGuid::Parse(StructGuidName.ToString(), StructGuid) && StructGuid.IsValid())
	//{
	//}

#if WITH_EDITORONLY_DATA
	if (Tag && Tag->SerializeType == EPropertyTagSerializeType::Property)
	{
		Struct = GetFallbackStruct();
		SetMetaData(UE::NAME_OriginalType, *WriteToString<256>(TypePath));
		return true;
	}
#endif // WITH_EDITORONLY_DATA

	return false;
}

void FStructProperty::SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const
{
	Super::SaveTypeName(Type);

	const UScriptStruct* LocalStruct = Struct;
	check(LocalStruct);

	Type.BeginParameters();
#if WITH_EDITORONLY_DATA
	if (const UE::FPropertyTypeName OriginalType = UE::FindOriginalType(this); !OriginalType.IsEmpty())
	{
		Type.AddType(OriginalType);
	}
	else
#endif // WITH_EDITORONLY_DATA
	{
		Type.AddPath(LocalStruct);
	}
	if (const FGuid StructGuid = LocalStruct->GetCustomGuid(); StructGuid.IsValid())
	{
		Type.AddGuid(StructGuid);
	}
	Type.EndParameters();
}

bool FStructProperty::CanSerializeFromTypeName(UE::FPropertyTypeName Type) const
{
	if (!Super::CanSerializeFromTypeName(Type))
	{
		return false;
	}

	const UScriptStruct* LocalStruct = Struct;
	check(LocalStruct);

	const FName StructName = Type.GetParameterName(0);
	if (StructName == LocalStruct->GetFName())
	{
		return true;
	}

	const FName StructGuidName = Type.GetParameterName(1);
	if (FGuid StructGuid; !StructGuidName.IsNone() && FGuid::Parse(StructGuidName.ToString(), StructGuid) &&
		StructGuid.IsValid() && StructGuid == LocalStruct->GetCustomGuid())
	{
		return true;
	}

#if WITH_EDITORONLY_DATA
	if (const UE::FPropertyTypeName OriginalType = UE::FindOriginalType(this);
		!OriginalType.IsEmpty() && StructName == OriginalType.GetName())
	{
		return true;
	}
#endif // WITH_EDITORONLY_DATA

	return false;
}

EPropertyVisitorControlFlow FStructProperty::Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const
{
	// Indicate in the path that this property contains inner properties
	Context.Path.Top().bContainsInnerProperties = true;

	EPropertyVisitorControlFlow RetVal = Super::Visit(Context, InFunc);

	if (RetVal == EPropertyVisitorControlFlow::StepInto)
	{
		RetVal = Struct->Visit(Context, InFunc);
	}
	return RetVal;
}

void* FStructProperty::ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
{
	return Struct->ResolveVisitedPathInfo(Data, Info);
}
