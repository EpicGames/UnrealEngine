// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyTag.h"

#include "Misc/AsciiSet.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "Serialization/SerializedPropertyScope.h"
#include "String/ParseTokens.h"
#include "String/Split.h"
#include "UObject/BlueprintsObjectVersion.h"
#include "UObject/DebugSerializationFlags.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/OverriddenObjectsExternalHandler.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectThreadContext.h"

#ifndef UE_ENABLE_SERIALIZE_TAGGED_PROPERTY_TRACING
#define UE_ENABLE_SERIALIZE_TAGGED_PROPERTY_TRACING 0
#endif

#if UE_ENABLE_SERIALIZE_TAGGED_PROPERTY_TRACING
#include "Trace/Trace.inl"
#endif

/** Flags used for serialization of FPropertyTag. DO NOT EDIT THESE VALUES! */
enum class EPropertyTagFlags : uint8
{
	None						= 0x00,
	HasArrayIndex				= 0x01,
	HasPropertyGuid				= 0x02,
	HasPropertyExtensions		= 0x04,
	HasBinaryOrNativeSerialize	= 0x08,
	BoolTrue					= 0x10,
	SkippedSerialize			= 0x20,
};

ENUM_CLASS_FLAGS(EPropertyTagFlags);

/**
 * Enum flags that indicate that additional data was serialized for that property tag.
 * Registered flags should be serialized in ascending order.
 */
enum class EPropertyTagExtension : uint8
{
	NoExtension					= 0x00,
	ReserveForFutureUse			= 0x01, // Can be used to add a next group of extensions

	////////////////////////////////////////////////
	// First extension group
	OverridableInformation		= 0x02,
	HasExternalsObjects			= 0x04,

	//
	// Add more extensions for the first group here
	//
};

ENUM_CLASS_FLAGS(EPropertyTagExtension);

thread_local const FPropertyTag* FPropertyTagScope::CurrentPropertyTag = nullptr;

/*-----------------------------------------------------------------------------
FPropertyTag
-----------------------------------------------------------------------------*/
FPropertyTag::FPropertyTag()
	: OverrideOperation(EOverriddenPropertyOperation::None)
{
}

FPropertyTag::FPropertyTag(FProperty* Property, int32 InIndex, uint8* Value)
	: Prop(Property)
	, Name(Property->GetFName())
	, ArrayIndex(InIndex)
	, OverrideOperation(EOverriddenPropertyOperation::None)
{
	SetType(UE::FPropertyTypeName(Property));

	if (FBoolProperty* Bool = CastField<FBoolProperty>(Property))
	{
		BoolVal = Bool->GetPropertyValue(Value);
	}
}

void FPropertyTag::SetProperty(FProperty* Property)
{
	Prop = Property;
}

void FPropertyTag::SetPropertyGuid(const FGuid& InPropertyGuid)
{
	HasPropertyGuid = InPropertyGuid.IsValid();
	PropertyGuid = InPropertyGuid;
}

void FPropertyTag::SetType(UE::FPropertyTypeName InFullType)
{
	TypeName = InFullType;
	Type = TypeName.GetName();
}

static EPropertyTagExtension CalculatePropertyExtensionFlags(FArchive& UnderlyingArchive, FPropertyTag& Tag)
{
	EPropertyTagExtension PropertyTagExtensions = EPropertyTagExtension::NoExtension;

	if (UnderlyingArchive.IsSaving())
	{
		if(FOverridableSerializationLogic::IsEnabled())
		{
			Tag.OverrideOperation = FOverridableSerializationLogic::GetOverriddenPropertyOperation(UnderlyingArchive, Tag.GetProperty());
			PropertyTagExtensions |= EPropertyTagExtension::OverridableInformation;
		}
		else
		{
			Tag.OverrideOperation = EOverriddenPropertyOperation::None;
		}

		if (Tag.GetProperty()->HasAnyPropertyFlags(CPF_ExperimentalOverridableLogic))
		{
			Tag.bExperimentalOverridableLogic = true;
			PropertyTagExtensions |= EPropertyTagExtension::OverridableInformation;
		}
		else
		{
			Tag.bExperimentalOverridableLogic = false;
		}
		
		Tag.bExperimentalExternalObjects = false;
#if WITH_EDITOR
		if (Tag.GetProperty()->HasAllPropertyFlags(CPF_ExperimentalOverridableLogic|CPF_ExperimentalExternalObjects) && !UnderlyingArchive.IsCooking())
		{
			FUObjectSerializeContext* Context = FUObjectThreadContext::Get().GetSerializeContext();
			if (Context->SerializedObject)
			{
				Tag.bExperimentalExternalObjects = FPropertyArrayExternalObjectHandler::Get().IsSavingExternal(Context->SerializedObject, Tag.GetProperty());
			}
			
			PropertyTagExtensions |= EPropertyTagExtension::HasExternalsObjects;
		}		
#endif // WITH_EDITOR
	}

	return PropertyTagExtensions;
}

static void SerializePropertyExtensions(FStructuredArchive::FSlot Slot, EPropertyTagExtension PropertyTagExtensions, FPropertyTag& Tag)
{
	// Serialize tag extensions, consider doing an init function and context as `EClassSerializationControlExtension` if we add more extensions
	if (Slot.GetArchiveState().IsTextFormat())
	{
		Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("PropertyExtensions"), PropertyTagExtensions, EPropertyTagExtension::NoExtension);
	}
	else
	{
		Slot << SA_ATTRIBUTE(TEXT("PropertyExtensions"), PropertyTagExtensions);
	}

	// OverridableInformation
	if (EnumHasAnyFlags(PropertyTagExtensions, EPropertyTagExtension::OverridableInformation))
	{
		Slot << SA_ATTRIBUTE(TEXT("OverriddenPropertyOperation"), Tag.OverrideOperation);
		Slot << SA_ATTRIBUTE(TEXT("ExperimentalOverridableLogic"), Tag.bExperimentalOverridableLogic);
	}

	if (EnumHasAnyFlags(PropertyTagExtensions, EPropertyTagExtension::HasExternalsObjects))
	{
		bool bValue = Tag.bExperimentalExternalObjects.Get(false);
		Slot << SA_ATTRIBUTE(TEXT("ExperimentalExternalObjects"), bValue);
		Tag.bExperimentalExternalObjects = bValue;
	}
}

static void ParsePathName(FStringView Path, UE::FPropertyTypeNameBuilder& Builder)
{
	FStringView OuterChain;
	FStringView ObjectName;
	if (UE::String::SplitLastOfAnyChar(Path, {TEXT('.'), TEXT(':')}, OuterChain, ObjectName))
	{
		Builder.AddName(FName(ObjectName));
		Builder.BeginParameters();
		UE::String::ParseTokensMultiple(OuterChain, {TEXT('.'), TEXT(':')}, [&Builder](FStringView Outer)
		{
			Builder.AddName(FName(Outer));
		});
		Builder.EndParameters();
	}
	else
	{
		Builder.AddName(FName(Path));
	}
}

FORCENOINLINE void LoadPropertyTagNoFullType(FStructuredArchive::FSlot Slot, FPropertyTag& Tag)
{
	const auto AddEnumPath = [](UE::FPropertyTypeNameBuilder& Builder, FName EnumName)
	{
		TStringBuilder<256> EnumPath(InPlace, EnumName);
		if (FAsciiSet::HasNone(EnumPath.ToView(), ".:"))
		{
			Builder.AddName(EnumName);
		}
		else
		{
			ParsePathName(EnumPath, Builder);
		}
	};

	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	const bool bIsTextFormat = UnderlyingArchive.IsTextFormat();
	const FPackageFileVersion Version = UnderlyingArchive.UEVer();

	check(UnderlyingArchive.IsLoading());

	if (!bIsTextFormat)
	{
		Slot << SA_ATTRIBUTE(TEXT("Name"), Tag.Name);
		if (Tag.Name.IsNone())
		{
			return;
		}
	}

	Slot << SA_ATTRIBUTE(TEXT("Type"), Tag.Type);

	if (!bIsTextFormat)
	{
		Slot << SA_ATTRIBUTE(TEXT("Size"), Tag.Size);
		Slot << SA_ATTRIBUTE(TEXT("ArrayIndex"), Tag.ArrayIndex);
	}

	if (Tag.Type.GetNumber() == NAME_NO_NUMBER_INTERNAL)
	{
		// Build Tag.TypeName from the partial type name that was saved in older versions.
		UE::FPropertyTypeNameBuilder TypeBuilder;
		TypeBuilder.AddName(Tag.Type);

		const FNameEntryId TagType = Tag.Type.GetComparisonIndex();

		// only need to serialize this for structs
		if (TagType == NAME_StructProperty)
		{
			TypeBuilder.BeginParameters();
			FName StructName;
			FGuid StructGuid;
			Slot << SA_ATTRIBUTE(TEXT("StructName"), StructName);
			TypeBuilder.AddName(StructName);
			if (Version >= VER_UE4_STRUCT_GUID_IN_PROPERTY_TAG)
			{
				if (bIsTextFormat)
				{
					Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("StructGuid"), StructGuid, FGuid());
				}
				else
				{
					Slot << SA_ATTRIBUTE(TEXT("StructGuid"), StructGuid);
				}
				if (StructGuid.IsValid())
				{
					TypeBuilder.AddGuid(StructGuid);
				}
			}
			TypeBuilder.EndParameters();
		}
		// only need to serialize this for bools
		else if (TagType == NAME_BoolProperty && !UnderlyingArchive.IsTextFormat())
		{
			if (UnderlyingArchive.IsSaving())
			{
				FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Tag.GetProperty());
				Slot << SA_ATTRIBUTE(TEXT("BoolVal"), Tag.BoolVal);
			}
			else
			{
				Slot << SA_ATTRIBUTE(TEXT("BoolVal"), Tag.BoolVal);
			}
		}
		// only need to serialize this for bytes/enums
		else if (TagType == NAME_ByteProperty)
		{
			FName EnumName;
			if (UnderlyingArchive.IsTextFormat())
			{
				Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("EnumName"), EnumName, NAME_None);
			}
			else
			{
				Slot << SA_ATTRIBUTE(TEXT("EnumName"), EnumName);
			}
			if (!EnumName.IsNone())
			{
				TypeBuilder.BeginParameters();
				AddEnumPath(TypeBuilder, EnumName);
				TypeBuilder.EndParameters();
			}
		}
		else if (TagType == NAME_EnumProperty)
		{
			FName EnumName;
			Slot << SA_ATTRIBUTE(TEXT("EnumName"), EnumName);
			TypeBuilder.BeginParameters();
			AddEnumPath(TypeBuilder, EnumName);
			TypeBuilder.AddName(NAME_ByteProperty);
			TypeBuilder.EndParameters();
		}
		// need to serialize the InnerType for arrays
		else if (TagType == NAME_ArrayProperty)
		{
			FName InnerType;
			if (Version >= VAR_UE4_ARRAY_PROPERTY_INNER_TAGS)
			{
				Slot << SA_ATTRIBUTE(TEXT("InnerType"), InnerType);
			}
			TypeBuilder.BeginParameters();
			TypeBuilder.AddName(InnerType);
			TypeBuilder.EndParameters();
		}
		// need to serialize the InnerType for optionals.
		else if (TagType == NAME_OptionalProperty)
		{
			FName InnerType;
			Slot << SA_ATTRIBUTE(TEXT("InnerType"), InnerType);
			TypeBuilder.BeginParameters();
			TypeBuilder.AddName(InnerType);
			TypeBuilder.EndParameters();
		}
		else if (Version >= VER_UE4_PROPERTY_TAG_SET_MAP_SUPPORT)
		{
			if (TagType == NAME_SetProperty)
			{
				FName InnerType;
				Slot << SA_ATTRIBUTE(TEXT("InnerType"), InnerType);
				TypeBuilder.BeginParameters();
				TypeBuilder.AddName(InnerType);
				TypeBuilder.EndParameters();
			}
			else if (TagType == NAME_MapProperty)
			{
				FName InnerType;
				FName ValueType;
				Slot << SA_ATTRIBUTE(TEXT("InnerType"), InnerType);
				Slot << SA_ATTRIBUTE(TEXT("ValueType"), ValueType);
				TypeBuilder.BeginParameters();
				TypeBuilder.AddName(InnerType);
				TypeBuilder.AddName(ValueType);
				TypeBuilder.EndParameters();
			}
		}

		Tag.TypeName = TypeBuilder.Build();
	}

	// Property tags to handle renamed blueprint properties effectively.
	if (Version >= VER_UE4_PROPERTY_GUID_IN_PROPERTY_TAG)
	{
		if (bIsTextFormat)
		{
			Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("PropertyGuid"), Tag.PropertyGuid, FGuid());
			Tag.HasPropertyGuid = Tag.PropertyGuid.IsValid();
		}
		else
		{
			Slot << SA_ATTRIBUTE(TEXT("HasPropertyGuid"), Tag.HasPropertyGuid);
			if (Tag.HasPropertyGuid)
			{
				Slot << SA_ATTRIBUTE(TEXT("PropertyGuid"), Tag.PropertyGuid);
			}
		}
	}

	if (Version >= EUnrealEngineObjectUE5Version::PROPERTY_TAG_EXTENSION_AND_OVERRIDABLE_SERIALIZATION)
	{
		EPropertyTagExtension PropertyTagExtensions = CalculatePropertyExtensionFlags(UnderlyingArchive, Tag);
		SerializePropertyExtensions(Slot, PropertyTagExtensions, Tag);
	}

	Tag.SerializeType = EPropertyTagSerializeType::Unknown;
}

FORCENOINLINE void SerializePropertyTagAsText(FStructuredArchive::FSlot Slot, FPropertyTag& Tag)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	Slot << SA_ATTRIBUTE(TEXT("Type"), Tag.TypeName);
	if (UnderlyingArchive.IsLoading())
	{
		Tag.SetType(Tag.TypeName);
	}

	bool bHasBinaryOrNativeSerialize = (Tag.SerializeType == EPropertyTagSerializeType::BinaryOrNative);
	bool bSkippedSerialize = (Tag.SerializeType == EPropertyTagSerializeType::Skipped);

	Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("PropertyGuid"), Tag.PropertyGuid, FGuid());
	Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("HasBinaryOrNativeSerialize"), bHasBinaryOrNativeSerialize, false);
	Slot << SA_OPTIONAL_ATTRIBUTE(TEXT("SkippedSerialize"), bSkippedSerialize, false);

	Tag.HasPropertyGuid = Tag.PropertyGuid.IsValid();
	Tag.SerializeType = bSkippedSerialize ? EPropertyTagSerializeType::Skipped
		: (bHasBinaryOrNativeSerialize ? EPropertyTagSerializeType::BinaryOrNative : EPropertyTagSerializeType::Property);

	EPropertyTagExtension PropertyTagExtensions = CalculatePropertyExtensionFlags(UnderlyingArchive, Tag);
	SerializePropertyExtensions(Slot, PropertyTagExtensions, Tag);
}

// Serializer.
FArchive& operator<<(FArchive& Ar, FPropertyTag& Tag)
{
	FStructuredArchiveFromArchive(Ar).GetSlot() << Tag;
	return Ar;
}

void operator<<(FStructuredArchive::FSlot Slot, FPropertyTag& Tag)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	const bool bIsTextFormat = UnderlyingArchive.IsTextFormat();
	const FPackageFileVersion Version = UnderlyingArchive.UEVer();

	check(!UnderlyingArchive.GetArchiveState().UseUnversionedPropertySerialization());
	checkf(!UnderlyingArchive.IsSaving() || Tag.GetProperty(), TEXT("FPropertyTag must be constructed with a valid property when used for saving data!"));

	if (UNLIKELY(Version < EUnrealEngineObjectUE5Version::PROPERTY_TAG_COMPLETE_TYPE_NAME))
	{
		LoadPropertyTagNoFullType(Slot, Tag);
		return;
	}

	if (UnderlyingArchive.IsSaving())
	{
		Tag.SerializeType = Tag.SerializeType == EPropertyTagSerializeType::Skipped ? Tag.SerializeType
			: (Tag.GetProperty()->UseBinaryOrNativeSerialization(UnderlyingArchive)
				? EPropertyTagSerializeType::BinaryOrNative : EPropertyTagSerializeType::Property);
	}

	if (UNLIKELY(bIsTextFormat))
	{
		SerializePropertyTagAsText(Slot, Tag);
		return;
	}

	Slot << SA_ATTRIBUTE(TEXT("Name"), Tag.Name);
	if (Tag.Name.IsNone())
	{
		return;
	}

	Slot << SA_ATTRIBUTE(TEXT("Type"), Tag.TypeName);
	if (UnderlyingArchive.IsLoading())
	{
		Tag.SetType(Tag.TypeName);
	}

	if (UnderlyingArchive.IsSaving())
	{
		// Store the serialized offset of the Size field.
		// UStruct::SerializeTaggedProperties will rewrite it after the property has been serialized.
		Tag.SizeOffset = UnderlyingArchive.Tell();
	}
	Slot << SA_ATTRIBUTE(TEXT("Size"), Tag.Size);

	EPropertyTagFlags PropertyTagFlags = EPropertyTagFlags::None;
	EPropertyTagExtension PropertyTagExtensions = CalculatePropertyExtensionFlags(UnderlyingArchive, Tag);

	if (UnderlyingArchive.IsSaving())
	{
		if (Tag.ArrayIndex != 0)
		{
			PropertyTagFlags |= EPropertyTagFlags::HasArrayIndex;
		}
		if (Tag.HasPropertyGuid)
		{
			PropertyTagFlags |= EPropertyTagFlags::HasPropertyGuid;
		}
		if (PropertyTagExtensions != EPropertyTagExtension::NoExtension)
		{
			PropertyTagFlags |= EPropertyTagFlags::HasPropertyExtensions;
		}
		if (Tag.SerializeType == EPropertyTagSerializeType::BinaryOrNative)
		{
			PropertyTagFlags |= EPropertyTagFlags::HasBinaryOrNativeSerialize;
		}
		if (Tag.SerializeType == EPropertyTagSerializeType::Skipped)
		{
			PropertyTagFlags |= EPropertyTagFlags::SkippedSerialize;
		}
		if (Tag.BoolVal && Tag.Type == NAME_BoolProperty)
		{
			PropertyTagFlags |= EPropertyTagFlags::BoolTrue;
		}
	}

	Slot << SA_ATTRIBUTE(TEXT("Flags"), PropertyTagFlags);

	if (UnderlyingArchive.IsLoading())
	{
		Tag.HasPropertyGuid = EnumHasAnyFlags(PropertyTagFlags, EPropertyTagFlags::HasPropertyGuid);
		Tag.SerializeType = EnumHasAnyFlags(PropertyTagFlags, EPropertyTagFlags::SkippedSerialize)
			? EPropertyTagSerializeType::Skipped
			: (EnumHasAnyFlags(PropertyTagFlags, EPropertyTagFlags::HasBinaryOrNativeSerialize)
				? EPropertyTagSerializeType::BinaryOrNative : EPropertyTagSerializeType::Property);
		Tag.BoolVal = EnumHasAnyFlags(PropertyTagFlags, EPropertyTagFlags::BoolTrue);
	}

	if (EnumHasAnyFlags(PropertyTagFlags, EPropertyTagFlags::HasArrayIndex))
	{
		Slot << SA_ATTRIBUTE(TEXT("ArrayIndex"), Tag.ArrayIndex);
	}
	else
	{
		Tag.ArrayIndex = 0;
	}

	if (EnumHasAnyFlags(PropertyTagFlags, EPropertyTagFlags::HasPropertyGuid))
	{
		Slot << SA_ATTRIBUTE(TEXT("PropertyGuid"), Tag.PropertyGuid);
	}

	if (EnumHasAnyFlags(PropertyTagFlags, EPropertyTagFlags::HasPropertyExtensions))
	{
		SerializePropertyExtensions(Slot, PropertyTagExtensions, Tag);
	}
}

// Property serializer.
void FPropertyTag::SerializeTaggedProperty(FArchive& Ar, FProperty* Property, uint8* Value, const uint8* Defaults) const
{
	SerializeTaggedProperty(FStructuredArchiveFromArchive(Ar).GetSlot(), Property, Value, Defaults);
}

#if UE_ENABLE_SERIALIZE_TAGGED_PROPERTY_TRACING
UE_TRACE_EVENT_BEGIN(Cpu, SerializeTaggedProperty, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PropertyName)
UE_TRACE_EVENT_END()
#endif

void FPropertyTag::SerializeTaggedProperty(FStructuredArchive::FSlot Slot, FProperty* Property, uint8* Value, const uint8* Defaults) const
{
#if UE_ENABLE_SERIALIZE_TAGGED_PROPERTY_TRACING
	UE_TRACE_LOG_SCOPED_T(Cpu, SerializeTaggedProperty, CpuChannel)
		<< SerializeTaggedProperty.PropertyName(*WriteToString<256>(Property->GetFName()));
#endif

	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();
	const int64 StartOfProperty = UnderlyingArchive.Tell();

	if (!UnderlyingArchive.IsTextFormat() && Property->GetClass() == FBoolProperty::StaticClass())
	{
		// ensure that the property scope gets recorded for boolean properties even though the data is stored in the tag
		FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Property);
		UnderlyingArchive.Serialize(nullptr, 0); 

		FBoolProperty* Bool = (FBoolProperty*)Property;
		if (UnderlyingArchive.IsLoading())
		{
			Bool->SetPropertyValue(Value, BoolVal != 0);
		}

		Slot.EnterStream();	// Effectively discard
	}
	else
	{
#if WITH_EDITOR
		static const FName NAME_SerializeTaggedProperty = FName("SerializeTaggedProperty");
		FArchive::FScopeAddDebugData P(UnderlyingArchive, NAME_SerializeTaggedProperty);
		FArchive::FScopeAddDebugData A(UnderlyingArchive, Property->GetFName());
#endif
		FSerializedPropertyScope SerializedProperty(UnderlyingArchive, Property);
		FPropertyTagScope CurrentPropertyTagScope(this);
		
#if UE_INTERNAL_LINKER_BOUNDS
		// Update archive bounds with tagged property known size:
		FArchiveBoundsScope ArchiveBounds(UnderlyingArchive.SetBounds(Size)); 
#endif // UE_INTERNAL_LINKER_BOUNDS

		Property->SerializeItem(Slot, Value, Defaults);
	}

	// Ensure that we serialize what we expected to serialize.
	const int64 EndOfProperty = UnderlyingArchive.Tell();
	if ((Size && (EndOfProperty - StartOfProperty != Size))
#if UE_INTERNAL_LINKER_BOUNDS
	|| UnderlyingArchive.IsCriticalError()
#endif//UE_INTERNAL_LINKER_BOUNDS
	)
	{
		UE_LOG(LogClass, Error, TEXT("Failed loading tagged %s. Read %" INT64_FMT "B, expected %dB. Package: %s"), *GetFullNameSafe(Property), EndOfProperty - StartOfProperty, Size, *UnderlyingArchive.GetArchiveName());
		UnderlyingArchive.Seek(StartOfProperty + Size);
		Property->ClearValue(Value);
#if UE_INTERNAL_LINKER_BOUNDS
		UnderlyingArchive.ClearCriticalError();
#endif//UE_INTERNAL_LINKER_BOUNDS
	}
}
