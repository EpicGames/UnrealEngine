// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyTempVal.h"

FPropertyTempVal::FPropertyTempVal(FProperty* InProp)
{
	// Allocate space for the owned object and construct it
	void* LocalValue = FMemory::Malloc(InProp->GetSize(), InProp->GetMinAlignment());
	InProp->InitializeValue(LocalValue);

	// Store fields
	Prop  = InProp;
	Value = LocalValue;
}

FPropertyTempVal::~FPropertyTempVal()
{
	void* LocalValue = Value;

	// Destroy the object and free the memory
	Prop->DestroyValue(LocalValue);
	FMemory::Free(LocalValue);
}

void FPropertyTempVal::Serialize(FArchive& Ar, const void* Defaults)
{
	Prop->SerializeItem(FStructuredArchiveFromArchive(Ar).GetSlot(), Value, Defaults);
}

void FPropertyTempVal::ExportText(FString& ValueStr, const void* Defaults, UObject* Parent, int32 PortFlags, UObject* ExportRootScope)
{
	Prop->ExportTextItem_Direct(ValueStr, Value, Defaults, Parent, PortFlags, ExportRootScope);
}
