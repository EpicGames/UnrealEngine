// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyTypeName.h"

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "AutoRTFM.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "HAL/LowLevelMemTracker.h"
#include "Hash/Blake3.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "String/Find.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

namespace UE
{

static_assert(sizeof(FPropertyTypeNameNode) <= 16);

constexpr static int32 GPropertyTypeNameBlockBits = 13;
constexpr static int32 GPropertyTypeNameBlockOffsetBits = 13;

constexpr static int32 GPropertyTypeNameBlockOffsetCount = 1 << GPropertyTypeNameBlockOffsetBits;
constexpr static int32 GPropertyTypeNameBlockOffsetMask = (1 << GPropertyTypeNameBlockOffsetBits) - 1;

constexpr static int32 GPropertyTypeNameBlockSize = GPropertyTypeNameBlockOffsetCount * sizeof(FPropertyTypeNameNode);
constexpr static int32 GPropertyTypeNameBlockCount = 1 << GPropertyTypeNameBlockBits;

LLM_DEFINE_TAG(FPropertyTypeName);

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

inline uint32 GetTypeHash(const FPropertyTypeNameNode& Node)
{
	return HashCombineFast(GetTypeHashHelper(Node.Name), GetTypeHashHelper(Node.InnerCount));
}

inline FArchive& operator<<(FArchive& Ar, FPropertyTypeNameNode& Node)
{
	return Ar << Node.Name << Node.InnerCount;
}

inline void operator<<(FStructuredArchiveSlot Slot, FPropertyTypeNameNode& Node)
{
	FStructuredArchiveRecord Record = Slot.EnterRecord();
	Record.EnterField(TEXT("Name")) << Node.Name;
	Record.EnterField(TEXT("InnerCount")) << Node.InnerCount;
}

inline const FPropertyTypeNameNode* AppendNode(FStringBuilderBase& Builder, const FPropertyTypeNameNode* Node)
{
	Builder << Node->Name;

	if (int32 Remaining = Node++->InnerCount)
	{
		Builder.AppendChar(TEXT('('));
		for (; Remaining > 0; --Remaining)
		{
			Node = AppendNode(Builder, Node);
			Builder.AppendChar(TEXT(','));
		}
		Builder.RemoveSuffix(1);
		Builder.AppendChar(TEXT(')'));
	}

	return Node;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

struct FPropertyTypeNameNodeProxy
{
	const FPropertyTypeNameNode* First = nullptr;

	inline void HashAndMeasure(uint32& OutHash, int32& OutCount) const
	{
		uint32 Hash = 0;
		const FPropertyTypeNameNode* Node = First;
		for (int32 Remaining = 1; Remaining > 0; --Remaining, ++Node)
		{
			Hash = HashCombineFast(Hash, GetTypeHash(*Node));
			Remaining += Node->InnerCount;
		}
		OutHash = Hash;
		OutCount = UE_PTRDIFF_TO_INT32(Node - First);
	}

	friend inline uint32 GetTypeHash(const FPropertyTypeNameNodeProxy& Proxy)
	{
		uint32 Hash = 0;
		int32 Count = 0;
		Proxy.HashAndMeasure(Hash, Count);
		return Hash;
	}

	friend inline void AppendHash(FBlake3& Builder, const FPropertyTypeNameNodeProxy& Proxy)
	{
		const FPropertyTypeNameNode* Node = Proxy.First;
		for (int32 Remaining = 1; Remaining > 0; --Remaining, ++Node)
		{
			AppendHash(Builder, Node->Name);
			Builder.Update(&Node->InnerCount, sizeof(Node->InnerCount));
			Remaining += Node->InnerCount;
		}
	}

	friend inline bool operator==(const FPropertyTypeNameNodeProxy& Lhs, const FPropertyTypeNameNodeProxy& Rhs)
	{
		const FPropertyTypeNameNode* LhsNode = Lhs.First;
		const FPropertyTypeNameNode* RhsNode = Rhs.First;
		for (int32 Remaining = 1; Remaining > 0; --Remaining, ++LhsNode, ++RhsNode)
		{
			if (LhsNode->Name == RhsNode->Name && LhsNode->InnerCount == RhsNode->InnerCount)
			{
				Remaining += LhsNode->InnerCount;
				continue;
			}
			return false;
		}
		return true;
	}

	friend inline bool operator<(const FPropertyTypeNameNodeProxy& Lhs, const FPropertyTypeNameNodeProxy& Rhs)
	{
		const FPropertyTypeNameNode* LhsNode = Lhs.First;
		const FPropertyTypeNameNode* RhsNode = Rhs.First;
		for (int32 Remaining = 1; Remaining > 0; --Remaining, ++LhsNode, ++RhsNode)
		{
			if (const int32 Compare = LhsNode->Name.Compare(RhsNode->Name))
			{
				return Compare < 0;
			}
			if (const int32 Compare = LhsNode->InnerCount - RhsNode->InnerCount)
			{
				return Compare < 0;
			}
			Remaining += LhsNode->InnerCount;
		}
		return true;
	}
};

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

class FPropertyTypeNameTable
{
public:
	FPropertyTypeNameTable();

	int32 FindOrAddByName(const FPropertyTypeNameNode* First);

	const FPropertyTypeNameNode* ResolveByIndex(int32 Index) const;

private:
	int32 StoreByIndex(const FPropertyTypeNameNode* Nodes, int32 Count);

	FPropertyTypeNameNode* AllocateBlock(int32 BlockIndex);

	// TODO: Shard the hash table and stop using TMap.
	mutable FRWLock NameMapMutex;
	TMap<FPropertyTypeNameNodeProxy, int32> NameMap;

	std::atomic<int32> NextIndex = 1; // 1 leaves a "None" entry at 0
	std::atomic<FPropertyTypeNameNode*> Blocks[GPropertyTypeNameBlockCount]{};
	FMutex BlockAllocationMutex;
};

FPropertyTypeNameTable GPropertyTypeNameTable;

FPropertyTypeNameTable::FPropertyTypeNameTable()
{
	AllocateBlock(0);
}

// Open because property type name table is a global cached object.
UE_AUTORTFM_ALWAYS_OPEN
int32 FPropertyTypeNameTable::FindOrAddByName(const FPropertyTypeNameNode* First)
{
	const FPropertyTypeNameNodeProxy Proxy{First};

	uint32 Hash;
	int32 Count;
	Proxy.HashAndMeasure(Hash, Count);

	if (FReadScopeLock Lock(NameMapMutex); const int32* Index = NameMap.FindByHash(Hash, Proxy))
	{
		return *Index;
	}

	LLM_SCOPE_BYTAG(FPropertyTypeName);
	FWriteScopeLock Lock(NameMapMutex);
	const int32 Index = StoreByIndex(First, Count);
	NameMap.AddByHash(Hash, {ResolveByIndex(Index)}, Index);
	return Index;
}

// Open because property type name table is a global cached object.
UE_AUTORTFM_ALWAYS_OPEN
const FPropertyTypeNameNode* FPropertyTypeNameTable::ResolveByIndex(int32 Index) const
{
	const int32 BlockIndex = Index >> GPropertyTypeNameBlockOffsetBits;
	const int32 BlockOffset = Index & GPropertyTypeNameBlockOffsetMask;
	return Blocks[BlockIndex].load(std::memory_order_relaxed) + BlockOffset;
}

// Open because property type name table is a global cached object.
UE_AUTORTFM_ALWAYS_OPEN
int32 FPropertyTypeNameTable::StoreByIndex(const FPropertyTypeNameNode* Nodes, int32 Count)
{
	if (Count == 1 && Nodes->Name.IsNone())
	{
		return 0;
	}

	UE_CLOGF(Count > GPropertyTypeNameBlockOffsetCount, LogCore, Fatal,
		"Invalid property type name with %d nodes. This can happen when serializing from a corrupt or invalid archive.", Count);

	int32 Index;
	for (int32 BaseIndex = NextIndex.load(std::memory_order_relaxed);;)
	{
		const int32 RemainingCountInBlock = GPropertyTypeNameBlockOffsetCount - (BaseIndex & GPropertyTypeNameBlockOffsetMask);
		Index = BaseIndex + (RemainingCountInBlock <= Count ? RemainingCountInBlock : 0);
		if (LIKELY(NextIndex.compare_exchange_weak(BaseIndex, Index + Count, std::memory_order_relaxed)))
		{
			break;
		}
	}

	UE_CLOGF(Index + Count > GPropertyTypeNameBlockCount * GPropertyTypeNameBlockOffsetCount, LogCore, Fatal,
		"Exceeded property type name capacity of %d nodes when storing %d nodes.",
		GPropertyTypeNameBlockCount * GPropertyTypeNameBlockOffsetCount, Count);

	const int32 BlockIndex = Index >> GPropertyTypeNameBlockOffsetBits;
	FPropertyTypeNameNode* Block = Blocks[BlockIndex].load(std::memory_order_acquire);
	if (UNLIKELY(!Block))
	{
		Block = AllocateBlock(BlockIndex);
	}
	const int32 BlockOffset = Index & GPropertyTypeNameBlockOffsetMask;
	for (FPropertyTypeNameNode* Target = Block + BlockOffset; Count > 0; --Count)
	{
		*Target++ = *Nodes++;
	}
	return Index;
}

// Open because property type name table is a global cached object.
UE_AUTORTFM_ALWAYS_OPEN
FPropertyTypeNameNode* FPropertyTypeNameTable::AllocateBlock(int32 BlockIndex)
{
	TUniqueLock Lock(BlockAllocationMutex);
	FPropertyTypeNameNode* Block = Blocks[BlockIndex].load(std::memory_order_acquire);
	if (!Block)
	{
		Block = (FPropertyTypeNameNode*)FMemory::MallocZeroed(GPropertyTypeNameBlockSize, alignof(FPropertyTypeNameNode));
		Blocks[BlockIndex].store(Block, std::memory_order_release);
	}
	return Block;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FPropertyTypeName::FPropertyTypeName(const FProperty* Property)
	: FPropertyTypeName([](const FProperty* P)
	{
		FPropertyTypeNameBuilder Builder;
		P->SaveTypeName(Builder);
		return Builder.Build();
	}(Property))
{
}

FName FPropertyTypeName::GetName() const
{
	const FPropertyTypeNameNode* First = GPropertyTypeNameTable.ResolveByIndex(Index);
	return First->Name;
}

int32 FPropertyTypeName::GetParameterCount() const
{
	const FPropertyTypeNameNode* First = GPropertyTypeNameTable.ResolveByIndex(Index);
	return First->InnerCount;
}

FPropertyTypeName FPropertyTypeName::GetParameter(int32 ParamIndex) const
{
	const FPropertyTypeNameNode* First = GPropertyTypeNameTable.ResolveByIndex(Index);
	if (UNLIKELY(ParamIndex < 0 || ParamIndex >= First->InnerCount))
	{
		return {};
	}
	const FPropertyTypeNameNode* Param = First + 1;
	for (int32 Skip = ParamIndex; Skip > 0; --Skip, ++Param)
	{
		Skip += Param->InnerCount;
	}
	FPropertyTypeName Type = *this;
	Type.Index += UE_PTRDIFF_TO_INT32(Param - First);
	return Type;
}

bool FPropertyTypeName::IsStruct(FName StructName) const
{
	const FPropertyTypeNameNode* First = GPropertyTypeNameTable.ResolveByIndex(Index);
	return First->InnerCount > 0 && First->Name == NAME_StructProperty && First[1].Name == StructName;
}

bool FPropertyTypeName::IsEnum(FName EnumName) const
{
	const FPropertyTypeNameNode* First = GPropertyTypeNameTable.ResolveByIndex(Index);
	return First->InnerCount > 0 && (First->Name == NAME_EnumProperty || First->Name == NAME_ByteProperty) && First[1].Name == EnumName;
}

uint32 GetTypeHash(const FPropertyTypeName& TypeName)
{
	const FPropertyTypeNameNode* First = GPropertyTypeNameTable.ResolveByIndex(TypeName.Index);
	return GetTypeHash(FPropertyTypeNameNodeProxy{First});
}

void AppendHash(FBlake3& Builder, const FPropertyTypeName& TypeName)
{
	const FPropertyTypeNameNode* First = GPropertyTypeNameTable.ResolveByIndex(TypeName.Index);
	AppendHash(Builder, FPropertyTypeNameNodeProxy{First});
}

bool operator==(const FPropertyTypeName& Lhs, const FPropertyTypeName& Rhs)
{
	if (Lhs.Index == Rhs.Index)
	{
		return true;
	}

	const FPropertyTypeNameNode* LhsNode = GPropertyTypeNameTable.ResolveByIndex(Lhs.Index);
	const FPropertyTypeNameNode* RhsNode = GPropertyTypeNameTable.ResolveByIndex(Rhs.Index);
	return FPropertyTypeNameNodeProxy{LhsNode} == FPropertyTypeNameNodeProxy{RhsNode};
}

bool operator<(const FPropertyTypeName& Lhs, const FPropertyTypeName& Rhs)
{
	if (Lhs.Index == Rhs.Index)
	{
		return false;
	}

	const FPropertyTypeNameNode* LhsNode = GPropertyTypeNameTable.ResolveByIndex(Lhs.Index);
	const FPropertyTypeNameNode* RhsNode = GPropertyTypeNameTable.ResolveByIndex(Rhs.Index);
	return FPropertyTypeNameNodeProxy{LhsNode} < FPropertyTypeNameNodeProxy{RhsNode};
}

FArchive& operator<<(FArchive& Ar, FPropertyTypeName& TypeName)
{
	if (Ar.IsLoading())
	{
		TArray<FPropertyTypeNameNode, TInlineAllocator<16>> Nodes;
		int32 Remaining = 1;
		do
		{
			FPropertyTypeNameNode& Node = Nodes.AddDefaulted_GetRef();
			Ar << Node;
			Remaining += Node.InnerCount - 1;
		}
		while (Remaining > 0);
		TypeName.Index = GPropertyTypeNameTable.FindOrAddByName(Nodes.GetData());
	}
	else
	{
		const FPropertyTypeNameNode* Node = GPropertyTypeNameTable.ResolveByIndex(TypeName.Index);
		for (int32 Remaining = 1; Remaining > 0; --Remaining, ++Node)
		{
			FPropertyTypeNameNode NodeCopy = *Node;
			Ar << NodeCopy;
			Remaining += NodeCopy.InnerCount;
		}
	}
	return Ar;
}

void operator<<(FStructuredArchiveSlot Slot, FPropertyTypeName& TypeName)
{
	const FArchiveState& State = Slot.GetArchiveState();
	if (State.IsLoading())
	{
		if (State.IsTextFormat())
		{
			FString Text;
			Slot << Text;
			FPropertyTypeNameBuilder Builder;
			if (Builder.TryParse(Text))
			{
				TypeName = Builder.Build();
			}
			else
			{
				TypeName.Reset();
				Slot.GetUnderlyingArchive().SetError();
			}
		}
		else
		{
			FStructuredArchiveStream Stream = Slot.EnterStream();
			TArray<FPropertyTypeNameNode, TInlineAllocator<16>> Nodes;
			int32 Remaining = 1;
			do
			{
				FPropertyTypeNameNode& Node = Nodes.AddDefaulted_GetRef();
				Stream.EnterElement() << Node;
				Remaining += Node.InnerCount - 1;

				if (State.IsCriticalError())
				{
					break;
				}
			}
			while (Remaining > 0);
			TypeName.Index = GPropertyTypeNameTable.FindOrAddByName(Nodes.GetData());
		}
	}
	else
	{
		if (State.IsTextFormat())
		{
			FString Text(WriteToString<256>(TypeName));
			Slot << Text;
		}
		else
		{
			FStructuredArchiveStream Stream = Slot.EnterStream();
			const FPropertyTypeNameNode* Node = GPropertyTypeNameTable.ResolveByIndex(TypeName.Index);
			for (int32 Remaining = 1; Remaining > 0; --Remaining, ++Node)
			{
				FPropertyTypeNameNode NodeCopy = *Node;
				Stream.EnterElement() << NodeCopy;
				Remaining += NodeCopy.InnerCount;

				if (State.IsCriticalError())
				{
					break;
				}
			}
		}
	}
}

FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FPropertyTypeName& TypeName)
{
	const FPropertyTypeNameNode* Node = GPropertyTypeNameTable.ResolveByIndex(TypeName.Index);
	AppendNode(Builder, Node);
	return Builder;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

void FPropertyTypeNameBuilder::BeginParameters()
{
	checkf(!Nodes.IsEmpty(), TEXT("A type name must be added prior to setting type parameters."));
	ActiveIndex = Nodes.Num() - 1;
}

void FPropertyTypeNameBuilder::EndParameters()
{
	ActiveIndex = OuterNodeIndex[ActiveIndex];
}

void FPropertyTypeNameBuilder::AddName(FName Name)
{
	checkf(ActiveIndex >= 0 || Nodes.IsEmpty(), TEXT("Only one type name may be added as the root node."));

	FPropertyTypeNameNode& Node = Nodes.AddDefaulted_GetRef();
	Node.Name = Name;
	Node.InnerCount = 0;

	OuterNodeIndex.Add(ActiveIndex);
	if (ActiveIndex >= 0)
	{
		++Nodes[ActiveIndex].InnerCount;
	}
}

void FPropertyTypeNameBuilder::AddGuid(const FGuid& Guid)
{
	AddName(FName(WriteToString<48>(Guid)));
}

void FPropertyTypeNameBuilder::AddPath(const UField* Field)
{
	if (!Field)
	{
		AddName(NAME_None);
		return;
	}

	AddName(Field->GetFName());

	TArray<UObject*, TInlineAllocator<8>> OuterChain;
	for (UObject* Outer = Field->GetOuter(); Outer; Outer = Outer->GetOuter())
	{
		OuterChain.Add(Outer);
	}
	if (!OuterChain.IsEmpty())
	{
		BeginParameters();
		for (UObject* Outer : ReverseIterate(OuterChain))
		{
			AddName(Outer->GetFName());
		}
		EndParameters();
	}
}

void FPropertyTypeNameBuilder::AddType(FPropertyTypeName Name)
{
	AddName(Name.GetName());
	if (const int32 Count = Name.GetParameterCount())
	{
		BeginParameters();
		for (int32 Index = 0; Index < Count; ++Index)
		{
			AddType(Name.GetParameter(Index));
		}
		EndParameters();
	}
}

bool FPropertyTypeNameBuilder::TryParse(FStringView Name)
{
	const auto ResetToInitial = [this, InitialCount = Nodes.Num(), InitialActiveIndex = ActiveIndex]
	{
		Nodes.SetNum(InitialCount, EAllowShrinking::No);
		OuterNodeIndex.SetNum(InitialCount, EAllowShrinking::No);
		ActiveIndex = InitialActiveIndex;
	};

	int32 Index = 0;
	int32 Depth = 0;
	bool bAllowName = true;
	bool bAllowBegin = false;
	for (FStringView Remaining = Name;; Remaining.RightChopInline(Index + 1))
	{
		Index = String::FindFirstOfAnyChar(Remaining, {TEXT('('), TEXT(','), TEXT(')')});

		if (Index != 0)
		{
			FStringView Type = (Index > 0 ? Remaining.Left(Index) : Remaining).TrimStartAndEnd();
			if (!Type.IsEmpty())
			{
				if (!bAllowName)
				{
					// Names must follow '(' or ',' or be at the root.
					break;
				}
				AddName(FName(Type));
				bAllowBegin = true;
				bAllowName = false;
			}
		}

		if (Index == INDEX_NONE)
		{
			if (Depth == 0 && !bAllowName)
			{
				return true;
			}
			// Missing a ')' and/or missing a name.
			break;
		}

		if (bAllowName)
		{
			// Missing a name.
			break;
		}

		const TCHAR C = Remaining[Index];
		if (C == TEXT('('))
		{
			if (!bAllowBegin)
			{
				// '(' must follow a name.
				break;
			}
			++Depth;
			BeginParameters();
			bAllowName = true;
		}
		else if (C == TEXT(')'))
		{
			if (Depth <= 0)
			{
				// ')' must have a matching '('.
				break;
			}
			EndParameters();
			--Depth;
		}
		else // if (C == TEXT(','))
		{
			if (Depth <= 0)
			{
				// ',' must not be used at the root.
				break;
			}
			bAllowName = true;
		}

		bAllowBegin = false;
	}

	ResetToInitial();
	return false;
}

FPropertyTypeName FPropertyTypeNameBuilder::Build() const
{
	FPropertyTypeName Type;
	if (!Nodes.IsEmpty())
	{
		Type.Index = GPropertyTypeNameTable.FindOrAddByName(Nodes.GetData());
	}
	return Type;
}

void FPropertyTypeNameBuilder::Reset()
{
	Nodes.Reset();
	OuterNodeIndex.Reset();
	ActiveIndex = INDEX_NONE;
}

} // UE
