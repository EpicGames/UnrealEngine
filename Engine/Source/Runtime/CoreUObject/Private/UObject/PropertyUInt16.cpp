// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	FUInt16Property.
-----------------------------------------------------------------------------*/

FUInt16Property::FUInt16Property(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FUInt16Property::FUInt16Property(FFieldVariant InOwner, const UECodeGen_Private::FUInt16PropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
FUInt16Property::FUInt16Property(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA
