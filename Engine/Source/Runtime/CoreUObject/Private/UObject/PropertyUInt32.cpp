// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	FUInt32Property.
-----------------------------------------------------------------------------*/

FUInt32Property::FUInt32Property(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FUInt32Property::FUInt32Property(FFieldVariant InOwner, const UECodeGen_Private::FUInt32PropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
FUInt32Property::FUInt32Property(UField* InField)
	: TProperty_Numeric(InField)
{
}
#endif // WITH_EDITORONLY_DATA
