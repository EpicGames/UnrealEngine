// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/*-----------------------------------------------------------------------------
	FUInt64Property.
-----------------------------------------------------------------------------*/

FUInt64Property::FUInt64Property(FFieldVariant InOwner, const FName& InName)
	: TProperty_Numeric(InOwner, InName)
{
}

FUInt64Property::FUInt64Property(FFieldVariant InOwner, const UECodeGen_Private::FUInt64PropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
FUInt64Property::FUInt64Property(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA
