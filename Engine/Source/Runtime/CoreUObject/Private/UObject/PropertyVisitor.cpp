// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/PropertyVisitor.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"

//----------------------------------------------------------------------//
// FPropertyVisitorInfo 
//----------------------------------------------------------------------//

bool FPropertyVisitorInfo::Identical(const FPropertyVisitorInfo& Other) const
{
	return Property == Other.Property
		&& ParentStructType == Other.ParentStructType
		&& Index == Other.Index
		&& PropertyInfo == Other.PropertyInfo
		&& bContainsInnerProperties == Other.bContainsInnerProperties;
}

//----------------------------------------------------------------------//
// FPropertyVisitorPath 
//----------------------------------------------------------------------//

FPropertyVisitorPath::Iterator FPropertyVisitorPath::InvalidIterator()
{
	static FPropertyVisitorPath EmptyPath;
	return Iterator(EmptyPath.GetPath());
}

FPropertyVisitorPath::FPropertyVisitorPath(const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain)
{
	const FEditPropertyChain::TDoubleLinkedListNode* PropertyIterator = PropertyChain.GetActiveMemberNode() ? PropertyChain.GetActiveMemberNode() : PropertyChain.GetHead();
	while(PropertyIterator)
	{
		const FProperty* CurrentProperty = PropertyIterator->GetValue();
		int32 ArrayIndex = PropertyEvent.GetArrayIndex(CurrentProperty->GetName());
		if (ArrayIndex == INDEX_NONE)
		{
			Push(FPropertyVisitorInfo{CurrentProperty});
		}
		else if (CurrentProperty->ArrayDim > 1)
		{
			Push(FPropertyVisitorInfo{CurrentProperty, ArrayIndex, EPropertyVisitorInfoType::StaticArrayIndex});
		}
		else
		{
			// Only container is left, no easy way yet to know if we are editing a Key/value of a map
			Push(FPropertyVisitorInfo{CurrentProperty, ArrayIndex, EPropertyVisitorInfoType::ContainerIndex});
		}
		PropertyIterator = PropertyIterator->GetNextNode();
	}
}

FPropertyVisitorPath::FPropertyVisitorPath(const FArchiveSerializedPropertyChain& PropertyChain)
{
	TArray<class FProperty*, TInlineAllocator<8>>::TConstIterator PropertyIterator = PropertyChain.GetRootIterator();
	while (PropertyIterator)
	{
		const FProperty* CurrentProperty = (*PropertyIterator);

		// @todo add container index eventually if we ever have this information
		Push(FPropertyVisitorInfo{CurrentProperty});

		++PropertyIterator;
	}
}

FString FPropertyVisitorPath::ToString(const TCHAR* Separator /*= TEXT(".")*/) const
{
	return PropertyVisitorHelpers::PathToString(Path, Separator);
}

void FPropertyVisitorPath::ToString(FStringBuilderBase& Out, const TCHAR* Separator /*= TEXT(".")*/) const
{
	PropertyVisitorHelpers::PathToString(Path, Out, Separator);
}

void FPropertyVisitorPath::AppendString(FStringBuilderBase& Out, const TCHAR* Separator /*= TEXT(".")*/) const
{
	PropertyVisitorHelpers::PathAppendString(Path, Out, Separator);
}

bool FPropertyVisitorPath::Contained(const FPropertyVisitorPath& Other, bool* bIsEqual) const
{
	return PropertyVisitorHelpers::PathIsContainedWithin(Path, Other.Path, bIsEqual);
}

void* FPropertyVisitorPath::GetPropertyDataPtr(UObject* Object) const
{
	checkf(Object, TEXT("Expecting an valid object"));
	return PropertyVisitorHelpers::ResolveVisitedPath(Object->GetClass(), Object, *this);
}

FArchiveSerializedPropertyChain FPropertyVisitorPath::ToSerializedPropertyChain() const
{
	return PropertyVisitorHelpers::PathToSerializedPropertyChain(Path);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EPropertyVisitorControlFlow PropertyVisitorHelpers::VisitProperty(const UStruct* PropertyOwner, const FProperty* Property, FPropertyVisitorPath& Path, const FPropertyVisitorData& InData, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorPath& /*Path*/, const FPropertyVisitorData& /*Data*/)> InFunc)
{
	FPropertyVisitorContext Context(Path, InData);
	return VisitProperty(PropertyOwner, Property, Context, [&InFunc](const FPropertyVisitorContext& Context) -> EPropertyVisitorControlFlow
	{
		return InFunc(Context.Path, Context.Data);
	});
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

EPropertyVisitorControlFlow PropertyVisitorHelpers::VisitProperty(const UStruct* PropertyOwner, const FProperty* Property, FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc)
{
	EPropertyVisitorControlFlow RetVal = EPropertyVisitorControlFlow::StepOver;

	// ArrayDim of one means it is just a single property, not a static array
	if (Property->ArrayDim == 1)
	{
		FPropertyVisitorScope Scope(Context.Path, FPropertyVisitorInfo(Property, PropertyOwner));
		FPropertyVisitorData Data(Property->ContainerPtrToValuePtr<void>(Context.Data.PropertyData), /*ParentStructData*/Context.Data.PropertyData);
		FPropertyVisitorContext SubContext(Context.Path, Data, Context.Scope);

		RetVal = Property->Visit(SubContext, InFunc);
	}
	else
	{
		// In the case of a static array, we need to setup the EPropertyVisitorInfoType
		for (int32 StaticArrayIndex = 0; StaticArrayIndex < Property->ArrayDim; ++StaticArrayIndex)
		{
			FPropertyVisitorScope Scope(Context.Path, FPropertyVisitorInfo(Property, StaticArrayIndex, EPropertyVisitorInfoType::StaticArrayIndex, PropertyOwner));

			FPropertyVisitorData Data(Property->ContainerPtrToValuePtr<void>(Context.Data.PropertyData, StaticArrayIndex), /*ParentStructData*/Context.Data.PropertyData);
			FPropertyVisitorContext SubContext(Context.Path, Data, Context.Scope);

			RetVal = Property->Visit(SubContext, InFunc);

			if (RetVal == EPropertyVisitorControlFlow::Stop
				|| RetVal == EPropertyVisitorControlFlow::StepOut)
			{
				break;
			}
		}
	}
	return RetVal;
}

FString PropertyVisitorHelpers::PathToString(TArrayView<const FPropertyVisitorInfo> Path, const TCHAR* Separator)
{
	TStringBuilder<FName::StringBufferSize> PropertyPath;
	PathAppendString(Path, PropertyPath, Separator);
	return PropertyPath.ToString();
}

void PropertyVisitorHelpers::PathToString(TArrayView<const FPropertyVisitorInfo> Path, FStringBuilderBase& Out, const TCHAR* Separator)
{
	Out.Reset();
	PathAppendString(Path, Out, Separator);
}

void PropertyVisitorHelpers::PathAppendString(TArrayView<const FPropertyVisitorInfo> Path, FStringBuilderBase& Out, const TCHAR* Separator)
{
	bool bFirstEntry = true;
	for (const FPropertyVisitorInfo& Entry : Path)
	{
		bool bDisplayPropertyName = false;
		bool bDisplayIndex = false;
		FString Suffix;
		switch (Entry.PropertyInfo)
		{
			case EPropertyVisitorInfoType::None:
				bDisplayPropertyName = true;
				break;
			case EPropertyVisitorInfoType::StaticArrayIndex:
				bDisplayPropertyName = true;
				bDisplayIndex = true;
				break;
			case EPropertyVisitorInfoType::ContainerIndex:
				bDisplayIndex = true;
				break;
			case EPropertyVisitorInfoType::MapKey:
				bDisplayIndex = true;
				Suffix = TEXT("Key");
				break;
			case EPropertyVisitorInfoType::MapValue:
				bDisplayIndex = true;
				Suffix = TEXT("Value");
				break;
			default:
				checkf(false, TEXT("Unsupported enum value"));
				break;
		}
		if (bDisplayPropertyName)
		{
			if (bFirstEntry)
			{
				bFirstEntry = false;
			}
			else
			{
				Out.Append(Separator);
			}
			Out.Append(Entry.Property->GetAuthoredName());
		}
		if (bDisplayIndex)
		{
			checkf(Entry.Index != INDEX_NONE, TEXT("Expecting the index to be valid"));
			Out.Appendf(TEXT("[%d]"), Entry.Index);
		}
		if (Suffix.Len())
		{
			Out.Append(Separator);
			Out.Append(Suffix);
		}
	}
}

bool PropertyVisitorHelpers::PathIsContainedWithin(TArrayView<const FPropertyVisitorInfo> Path, TArrayView<const FPropertyVisitorInfo> OtherPath, bool* bIsEqual)
{
	int32 i;
	for (i = 0; i < Path.Num(); ++i)
	{
		if (i >= OtherPath.Num())
		{
			// The other path is smaller than this one, so not contained in
			break;
		}

		const FPropertyVisitorInfo& PathInfo = Path[i];
		const FPropertyVisitorInfo& OtherPathInfo = OtherPath[i];

		if (PathInfo.Property != OtherPathInfo.Property)
		{
			// The property is different, so not contained in
			break;
		}

		if (PathInfo.PropertyInfo != OtherPathInfo.PropertyInfo)
		{
			if (PathInfo.PropertyInfo != EPropertyVisitorInfoType::None)
			{
				// The property info type is different and it is not none, so not contained in
				break;
			}
		}
		else if (PathInfo.Index != OtherPathInfo.Index)
		{
			// The index is different, so not contained in
			break;
		}
	}
	if (bIsEqual)
	{
		*bIsEqual = i == OtherPath.Num();
	}
	return i == Path.Num();
}

FArchiveSerializedPropertyChain PropertyVisitorHelpers::PathToSerializedPropertyChain(TArrayView<const FPropertyVisitorInfo> Path)
{
	FArchiveSerializedPropertyChain Chain;
	for (const FPropertyVisitorInfo& Info : Path)
	{
		// @todo add container index eventually if we ever have this information
		Chain.PushProperty(const_cast<FProperty*>(Info.Property), Info.Property->IsEditorOnlyProperty());
	}
	return Chain;
}
