// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "UObject/LinkerPlaceholderExportObject.h"
#include "UObject/LinkerPlaceholderClass.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/CoreNet.h"

/*-----------------------------------------------------------------------------
	FWeakObjectProperty.
-----------------------------------------------------------------------------*/
IMPLEMENT_FIELD(FWeakObjectProperty)

FWeakObjectProperty::FWeakObjectProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FWeakObjectProperty::FWeakObjectProperty(FFieldVariant InOwner, const UECodeGen_Private::FWeakObjectPropertyParams& Prop)
	: Super(InOwner, Prop)
{
}

#if WITH_EDITORONLY_DATA
FWeakObjectProperty::FWeakObjectProperty(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA

FString FWeakObjectProperty::GetCPPType( FString* ExtendedTypeText/*=NULL*/, uint32 CPPExportFlags/*=0*/ ) const
{
	check(PropertyClass);
	if (PropertyFlags & CPF_AutoWeak)
	{
		return FString::Printf(TEXT("TAutoWeakObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass.GetName());
	}
	return FString::Printf(TEXT("TWeakObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass.GetName());
}

FString FWeakObjectProperty::GetCPPMacroType( FString& ExtendedTypeText ) const
{
	if (PropertyFlags & CPF_AutoWeak)
	{
		ExtendedTypeText = FString::Printf(TEXT("TAutoWeakObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
		return TEXT("AUTOWEAKOBJECT");
	}
	ExtendedTypeText = FString::Printf(TEXT("TWeakObjectPtr<%s%s>"), PropertyClass->GetPrefixCPP(), *PropertyClass->GetName());
	return TEXT("WEAKOBJECT");
}

void FWeakObjectProperty::LinkInternal(FArchive& Ar)
{
	checkf(!HasAnyPropertyFlags(CPF_NonNullable), TEXT("Weak Object Properties can't be non nullable but \"%s\" is marked as CPF_NonNullable"), *GetFullName());
	Super::LinkInternal(Ar);
}

void FWeakObjectProperty::SerializeItem( FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults ) const
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	UObject* OldObjectValue = nullptr;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (!UnderlyingArchive.HasAnyPortFlags(PPF_AvoidRemoteObjectMigration))
#endif
	{
		OldObjectValue = GetObjectPropertyValue(Value);
	}

	Slot << *(FWeakObjectPtr*)NotNullGet(Value);

	if (
#if UE_WITH_REMOTE_OBJECT_HANDLE
	!UnderlyingArchive.HasAnyPortFlags(PPF_AvoidRemoteObjectMigration) &&
#endif
	   (UnderlyingArchive.IsLoading() || UnderlyingArchive.IsModifyingWeakAndStrongReferences()))
	{
		UObject* NewObjectValue = GetObjectPropertyValue(Value);

		if (OldObjectValue != NewObjectValue)
		{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			if (UnderlyingArchive.IsLoading() && !UnderlyingArchive.IsObjectReferenceCollector())
			{
				if (ULinkerPlaceholderExportObject* PlaceholderVal = Cast<ULinkerPlaceholderExportObject>(NewObjectValue))
				{
					PlaceholderVal->AddReferencingPropertyValue(this, Value);
				}
				else if (ULinkerPlaceholderClass* PlaceholderClass = Cast<ULinkerPlaceholderClass>(NewObjectValue))
				{
					PlaceholderClass->AddReferencingPropertyValue(this, Value);
				}
			}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

			CheckValidObject(Value, nullptr); // FWeakObjectProperty is never non-nullable at this point so it's ok to pass null as the current value
		}
	}
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
bool FWeakObjectProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	if ((PortFlags & PPF_AvoidRemoteObjectMigration) != 0)
	{
		// With remote object handles enabled, weak pointers are equal if the globally unique remote ids of the objects they point to are identical
		// which is the default behavior of FWeakObjectPtr::operator==
		// This way we don't need to resolve the actual objects these weak pointers reference
		FWeakObjectPtr ObjectA = GetPropertyValue(A);
		FWeakObjectPtr ObjectB = B ? GetPropertyValue(B) : FWeakObjectPtr();

		return ObjectA == ObjectB;
	}
	return Super::Identical(A, B, PortFlags);
}

bool FWeakObjectProperty::NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData) const
{
	const bool bUseRemoteObjectReference = Map && Map->IsUsingRemoteObjectReferences();
	if (bUseRemoteObjectReference)
	{
		FWeakObjectPtr* WeakPtr = GetPropertyValuePtr(Data);

		FRemoteObjectReference RemoteReference(FObjectPtr(WeakPtr->GetRemoteId()));
		bool bResult = false;
		RemoteReference.NetSerialize(Ar, Map, bResult);

		if (Ar.IsLoading())
		{
			*WeakPtr = RemoteReference.ToWeakPtr();
		}
		return bResult;
	}
	return Super::NetSerializeItem(Ar, Map, Data, MetaData);
}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

UObject* FWeakObjectProperty::GetObjectPropertyValue(const void* PropertyValueAddress) const
{
	return GetPropertyValue(PropertyValueAddress).Get();
}

TObjectPtr<UObject> FWeakObjectProperty::GetObjectPtrPropertyValue(const void* PropertyValueAddress) const
{
	return TObjectPtr<UObject>(GetPropertyValue(PropertyValueAddress).Get());
}

UObject* FWeakObjectProperty::GetObjectPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	UObject* Result = nullptr;
	GetWrappedUObjectPtrValues<FWeakObjectPtr>(&Result, ContainerAddress, EPropertyMemoryAccess::InContainer, ArrayIndex, 1);
	return Result;
}

TObjectPtr<UObject> FWeakObjectProperty::GetObjectPtrPropertyValue_InContainer(const void* ContainerAddress, int32 ArrayIndex) const
{
	TObjectPtr<UObject> Result = nullptr;
	GetWrappedUObjectPtrValues<FWeakObjectPtr>(&Result, ContainerAddress, EPropertyMemoryAccess::InContainer, ArrayIndex, 1);
	return Result;
}

void FWeakObjectProperty::SetObjectPropertyValueUnchecked(void* PropertyValueAddress, UObject* Value) const
{
	SetPropertyValue(PropertyValueAddress, TCppType(Value));
}

void FWeakObjectProperty::SetObjectPtrPropertyValueUnchecked(void* PropertyValueAddress, TObjectPtr<UObject> Ptr) const
{
	SetPropertyValue(PropertyValueAddress, TCppType(Ptr));
}

void FWeakObjectProperty::SetObjectPropertyValueUnchecked_InContainer(void* ContainerAddress, UObject* Value, int32 ArrayIndex) const
{
	SetWrappedUObjectPtrValues<FWeakObjectPtr>(ContainerAddress, EPropertyMemoryAccess::InContainer, &Value, ArrayIndex, 1);
}

void FWeakObjectProperty::SetObjectPtrPropertyValueUnchecked_InContainer(void* ContainerAddress, TObjectPtr<UObject> Ptr, int32 ArrayIndex) const
{
	SetWrappedUObjectPtrValues<FWeakObjectPtr>(ContainerAddress, EPropertyMemoryAccess::InContainer, &Ptr, ArrayIndex, 1);
}

uint32 FWeakObjectProperty::GetValueTypeHashInternal(const void* Src) const
{
	return GetTypeHash(*(FWeakObjectPtr*)Src);
}

void FWeakObjectProperty::CopySingleValueToScriptVM( void* Dest, void const* Src ) const
{
	#if UE_GC_RUN_WEAKPTR_BARRIERS
	*(FObjectPtr*)Dest = ((const FWeakObjectPtr*)Src)->Get();
	#else
	*(UObject**)Dest = ((const FWeakObjectPtr*)Src)->Get();	
	#endif
}

void FWeakObjectProperty::CopySingleValueFromScriptVM( void* Dest, void const* Src ) const
{
	*(FWeakObjectPtr*)Dest = *(UObject**)Src;
}

void FWeakObjectProperty::CopyCompleteValueToScriptVM( void* Dest, void const* Src ) const
{
	GetWrappedUObjectPtrValues<FWeakObjectPtr>((UObject**)Dest, Src, EPropertyMemoryAccess::Direct, 0, ArrayDim);
}

void FWeakObjectProperty::CopyCompleteValueFromScriptVM( void* Dest, void const* Src ) const
{
	SetWrappedUObjectPtrValues<FWeakObjectPtr>(Dest, EPropertyMemoryAccess::Direct, (UObject**)Src, 0, ArrayDim);
}

void FWeakObjectProperty::CopyCompleteValueToScriptVM_InContainer(void* OutValue, void const* InContainer) const
{
	GetWrappedUObjectPtrValues<FWeakObjectPtr>((UObject**)OutValue, InContainer, EPropertyMemoryAccess::InContainer, 0, ArrayDim);
}

void FWeakObjectProperty::CopyCompleteValueFromScriptVM_InContainer(void* OutContainer, void const* InValue) const
{
	SetWrappedUObjectPtrValues<FWeakObjectPtr>(OutContainer, EPropertyMemoryAccess::InContainer, (UObject**)InValue, 0, ArrayDim);
}
