// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReachabilityAnalysisState.cpp: Incremental reachability analysis support.
=============================================================================*/

#include "UObject/ReachabilityAnalysisState.h"
#include "UObject/ReachabilityAnalysis.h"
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/VVMContext.h"
#include "VerseVM/VVMHeap.h"
#endif

namespace UE::GC
{

void FReachabilityAnalysisState::Init()
{
	NumIterations = 0;
}

void FReachabilityAnalysisState::SetupWorkers(int32 InNumWorkers)
{
	NumWorkers = InNumWorkers;
	Stats = {};
	bIsSuspended = false;
}

void FReachabilityAnalysisState::UpdateStats(const FProcessorStats& InStats)
{
	Stats.AddStats(InStats);
}

void FReachabilityAnalysisState::ResetWorkers()
{
	NumWorkers = 0;
}

void FReachabilityAnalysisState::FinishIteration()
{
	NumIterations++;
}

static bool VerseGCIsTerminationPending()
{
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	if (GIsFrankenGCCollecting)
	{
		Verse::FIOContext Context = Verse::FIOContextPromise{};
		return Verse::FHeap::IsGCTerminationPendingExternalSignal(Context);
	}
#endif

	return true;
}

bool FReachabilityAnalysisState::CheckIfAnyContextIsSuspended()
{
	bIsSuspended = false;
	for (int32 WorkerIndex = 0; WorkerIndex < NumWorkers && !bIsSuspended; ++WorkerIndex)
	{
		bIsSuspended = Contexts[WorkerIndex]->bIsSuspended;
	}
	if (!bIsSuspended)
	{
		bIsSuspended = !VerseGCIsTerminationPending();
	}
	return bIsSuspended;
}

} // namespce UE::GC

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::GC::Private
{

FString FIterationTimerStat::ToString() const
{
	return FString::Printf(TEXT("%7.3fms, Iterations: %4d, Max: %7.3fms (%4d)"), Total * 1000, NumIterations, Max * 1000, SlowestIteration);
}

void FStats::DumpToLog()
{	
#define UE_TRUE_FALSE(b) (b ? TEXT("true") : TEXT("false"))
	UE_LOGF(LogGarbage, Log, "Garbage Collection Total  %7.3fms%ls", TotalTime * 1000, bInProgress ? TEXT(" (still in progress, results may be incomplete)") : TEXT(""));
	UE_LOGF(LogGarbage, Log, "  Verify                  %7.3fms", VerifyTime * 1000);	
	UE_LOGF(LogGarbage, Log, "  Reachability            %ls", *ReachabilityTime.ToString());
	UE_LOGF(LogGarbage, Log, "    MarkAsUnreachable     %7.3fms", MarkObjectsAsUnreachableTime * 1000);
	UE_LOGF(LogGarbage, Log, "    Reference Collection  %ls", *ReferenceCollectionTime.ToString());
	UE_LOGF(LogGarbage, Log, "  GarbageTracking         %7.3fms", GarbageTrackingTime * 1000);
	UE_LOGF(LogGarbage, Log, "  DissolveClusters        %7.3fms", DissolveUnreachableClustersTime * 1000);
	UE_LOGF(LogGarbage, Log, "  GatherUnreachable       %ls", *GatherUnreachableTime.ToString());
	UE_LOGF(LogGarbage, Log, "  NotifyUnreachable       %7.3fms", NotifyUnreachableTime * 1000);
	UE_LOGF(LogGarbage, Log, "  VerifyNoUnreachable     %7.3fms", VerifyNoUnreachableTime * 1000);	
	UE_LOGF(LogGarbage, Log, "  Unhashing               %ls", *UnhashingTime.ToString());
	UE_LOGF(LogGarbage, Log, "  DestroyGarbage          %ls", *DestroyGarbageTime.ToString());

	UE_LOGF(LogGarbage, Log, "Pre GC:     Objects: %7d, Roots : %7d, Clusters : %7d, Clustered Objects : %7d", NumObjects, NumRoots, NumClusters, NumClusteredObjects);
	UE_LOGF(LogGarbage, Log, "Destroyed:  Objects: %7d, including        Clusters : %7d, Clustered Objects : %7d", NumUnreachableObjects, NumDissolvedClusters, NumUnreachableClusteredObjects);
	UE_LOGF(LogGarbage, Log, "Number of barrier objects: %d", NumBarrierObjects);
	UE_LOGF(LogGarbage, Log, "Number of weak references for clearing %d and objects that need weak reference clearing: %d", NumWeakReferencesForClearing, NumObjectsThatNeedWeakReferenceClearing);
	UE_LOGF(LogGarbage, Log, "Started as full purge: %ls, finished as full purge: %ls", UE_TRUE_FALSE(bStartedAsFullPurge), UE_TRUE_FALSE(bFinishedAsFullPurge));
	UE_LOGF(LogGarbage, Log, "Flushed async loading: %ls", UE_TRUE_FALSE(bFlushedAsyncLoading));
	UE_LOGF(LogGarbage, Log, "Purged previous GC objects: %ls", UE_TRUE_FALSE(bPurgedPreviousGCObjects));
#undef UE_TRUE_FALSE
}

} // namespce UE::GC::Private
