// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReachabilityAnalysisState.h: Support for incremental reachability analysis
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectBase.h"
#include "UObject/Object.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/ReachabilityAnalysis.h"

namespace UE::GC
{

/**
* Reachability analysis state holds information about suspended worker contexts.
*/
class AUTORTFM_DISABLE FReachabilityAnalysisState
{
public:

	// Work-stealing algorithms are O(N^2), everyone steals from everyone.
	// Might want to improve that before going much wider.
	static constexpr int32 MaxWorkers = 16;

private:
	/** Number of workers threads */
	int32 NumWorkers = 0;
	/** Worker thread contexts */
	FWorkerContext* Contexts[MaxWorkers] = { nullptr };

	/** Additional object flags for rooting UObjects reachability analysis was kicked off with */
	EObjectFlags ObjectKeepFlags = RF_NoFlags;
	/** True if reachability analysis started as part of full GC */
	bool bPerformFullPurge = false;
	/** True if reachability analysis was paused due to time limit */
	bool bIsSuspended = false;
	/** Stats of the last reachability analysis run (all iterations) */
	UE::GC::FProcessorStats Stats;

	/** Total time of all reachability analysis iterations */
	double IncrementalMarkPhaseTotalTime = 0.0;

	/** Total time of the actual reference traversal */
	double ReferenceProcessingTotalTime = 0.0;

	/** Number of reachability analysis iterations performed during reachability analysis */
	int32 NumIterations = 0;

	/** Number of reachability analysis iterations to skip when running with gc.DelayReachabilityIterations */
	int32 NumRechabilityIterationsToSkip = 0;

	alignas (PLATFORM_CACHE_LINE_SIZE) double IterationStartTime = 0.0;
	alignas (PLATFORM_CACHE_LINE_SIZE) double IterationTimeLimit = 0.0;

public:

	FReachabilityAnalysisState() = default;

	void StartTimer(double InTimeLimit)
	{
		IterationTimeLimit = InTimeLimit;
		IterationStartTime = FPlatformTime::Seconds();
	}

	const UE::GC::FProcessorStats& GetStats() const
	{
		return Stats;
	}

	bool IsSuspended() const
	{
		return bIsSuspended;
	}

	/** Kicks off new Garbage Collection cycle */
	void CollectGarbage(EObjectFlags KeepFlags, bool bFullPurge);

	/** Performs Reachability Analysis (also incrementally) and destroys objects (also incrementally) */
	void PerformReachabilityAnalysisAndConditionallyPurgeGarbage(bool bReachabilityUsingTimeLimit);

	/** Checks if Time Limit has been exceeded. This function needs to be super fast as it's called very frequently. */
	FORCEINLINE bool IsTimeLimitExceeded() const
	{
		if (IterationTimeLimit > 0.0)
		{
			return (FPlatformTime::Seconds() - IterationStartTime) >= IterationTimeLimit;
		}
		return false;
	}

	FORCEINLINE int32 GetNumWorkers() const
	{
		return NumWorkers;
	}

	FORCEINLINE FWorkerContext** GetContextArray()
	{
		return Contexts;
	}

	FORCEINLINE int32 GetNumIterations() const
	{
		return NumIterations;
	}

	/** Initializes reachability analysis */
	void Init();

	/** Initializes expected number of worker threads */
	void SetupWorkers(int32 InNumWorkers);

	/** Main Garbage Collection function executed on Reachability Analysis Thread when StartEvent has been triggered */
	void PerformReachabilityAnalysis();

	/** Updates reachability analysis state after RA iteration */
	void UpdateStats(const UE::GC::FProcessorStats& InStats);

	/** Checks if any of the active worker contexts is suspended and updates the IsSuspended state */
	bool CheckIfAnyContextIsSuspended();

	/** Resets workers after reachability analysis is fully complete */
	void ResetWorkers();

	/** Marks the end of reachability iteration */
	void FinishIteration();
};


/** Settings for GatherUnreachableObjects function */
enum class EGatherOptions : uint32
{
	None = 0,
	Parallel = 1,	 // Use threading to increase performance
};
ENUM_CLASS_FLAGS(EGatherOptions);

inline int32 GetNumThreadsForGather(const EGatherOptions Options, const int32 NumObjects)
{
	if (!!(Options & EGatherOptions::Parallel) && NumObjects > 0)
	{
		constexpr int32 MinNumObjectsPerThread = 100;
		const int32 MaxNumThreads = FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads());
		const int32 NumThreadsForNumObjects = (NumObjects + MinNumObjectsPerThread - 1) / MinNumObjectsPerThread;
		return FMath::Min(NumThreadsForNumObjects, MaxNumThreads);
	}
	return 1;
}

/** Structure that holds the current state of Thread Iteration */
template <typename PayloadType>
struct TGatherIterator
{
	/** Initial start index */
	int32 StartIndex = 0;
	/** Current index of an object */
	int32 Index = 0;
	/** Estimated number of objects to process */
	int32 Num = 0;
	/** Last index of object to process */
	int32 LastIndex = 0;
	/** Data gathered by the iteration */
	PayloadType Payload;
};

template <typename PayloadType>
struct TDefaultPayloadOps
{
	inline static int32 Num(const PayloadType& InPayload)
	{
		return InPayload.Num();
	}

	inline static bool Reserve(const TArray<TGatherIterator<PayloadType>, TInlineAllocator<32>>& ThreadIterators, PayloadType& OutPayload)
	{
		int32 SizeToReserve = 0;
		for (const TGatherIterator<PayloadType>& It : ThreadIterators)
		{
			SizeToReserve += TDefaultPayloadOps<PayloadType>::Num(It.Payload);
		}
		OutPayload.Reserve(TDefaultPayloadOps<PayloadType>::Num(OutPayload) + SizeToReserve);
		return SizeToReserve > 0;
	}

	inline static void Append(const PayloadType& InSource, PayloadType& OutDest)
	{
		OutDest.Append(InSource);
	}
};

/** Helper class that holds the current state of Object Gathering phases */
template <typename PayloadType, typename PayloadOps = TDefaultPayloadOps<PayloadType>>
class AUTORTFM_DISABLE TThreadedGather
{
public:

	typedef TGatherIterator<PayloadType> FIterator;
	typedef TArray<FIterator, TInlineAllocator<32>> FThreadIterators;

	FORCEINLINE FThreadIterators& GetThreadIterators()
	{
		return ThreadIterators;
	}

	FORCEINLINE bool IsSuspended() const
	{
		return ThreadIterators.Num() > 0;
	}

	AUTORTFM_ENABLE FORCEINLINE bool IsPending() const
	{
		return bIsPending;
	}

	void Init()
	{
		bIsPending = true;
	}

	FORCENOINLINE void Start(const EGatherOptions Options, const int32 TotalNumObjects, const int32 FirstIndex = 0, const int32 DesiredNumThreads = 0)
	{
		int32 NumObjectsToProcess = TotalNumObjects - FirstIndex;
		int32 NumThreads = DesiredNumThreads > 0 ? DesiredNumThreads : GetNumThreadsForGather(Options, NumObjectsToProcess);
		int32 NumberOfObjectsPerThread = (NumObjectsToProcess / NumThreads) + 1;

		ThreadIterators.AddDefaulted(NumThreads);

		for (int32 ThreadIndex = 0; ThreadIndex < NumThreads; ++ThreadIndex)
		{
			FIterator& Iterator = ThreadIterators[ThreadIndex];
			Iterator.StartIndex = ThreadIndex * NumberOfObjectsPerThread + FirstIndex;
			Iterator.Index = Iterator.StartIndex;
			Iterator.Num = (ThreadIndex < (NumThreads - 1)) ? NumberOfObjectsPerThread : (TotalNumObjects - (NumThreads - 1) * NumberOfObjectsPerThread);
			Iterator.LastIndex = FMath::Min(TotalNumObjects - 1, Iterator.Index + Iterator.Num - 1);
		}
	}

	FORCENOINLINE void Finish(PayloadType& OutGatheredObjects)
	{
		if (ThreadIterators.Num() == 1 && PayloadOps::Num(OutGatheredObjects) == 0)
		{
			OutGatheredObjects = MoveTemp(ThreadIterators[0].Payload);
		}
		else if (PayloadOps::Reserve(ThreadIterators, OutGatheredObjects))
		{
			for (FIterator& Iterator : ThreadIterators)
			{
				PayloadOps::Append(Iterator.Payload, OutGatheredObjects);
			}
		}
		ThreadIterators.Reset();
		bIsPending = false;
	}

	FORCEINLINE int32 NumWorkerThreads() const
	{
		return ThreadIterators.Num();
	}

	int32 NumScanned() const
	{
		int32 NumScanned = 0;
		for (const FIterator& ThreadState : ThreadIterators)
		{
			NumScanned += ThreadState.Index - ThreadState.StartIndex;
		}
		return NumScanned;
	}

	int32 NumGathered() const
	{
		int32 NumGathered = 0;
		for (const FIterator& ThreadState : ThreadIterators)
		{
			NumGathered += PayloadOps::Num(ThreadState.Payload);
		}
		return NumGathered;
	}

private:

	FThreadIterators ThreadIterators;
	bool bIsPending = false;
};

/** Timer that can be used with ParallelFor that distributes TimeLimit checks evenly across multiple threads */
class FTimeSlicer
{
	const int32 TimeLimitPollInterval = 0;	
	int32 TimeLimitTimePollCounter = 0;
	const double StartTime = 0.0;
	const double TimeLimit = 0.0;
	std::atomic<int32>& TimeLimitFlag;
public:

	FTimeSlicer(int32 InPollInterval, int32 InitialCounter, double InStartTime, double InTimeLimit, std::atomic<int32>& InTimeLimitFlag)
		: TimeLimitPollInterval(InPollInterval)		
		, TimeLimitTimePollCounter(InitialCounter)
		, StartTime(InStartTime)
		, TimeLimit(InTimeLimit)
		, TimeLimitFlag(InTimeLimitFlag)
	{
	}

	FORCEINLINE bool IsTimeLimitExceeded()
	{
		if (TimeLimit > 0.0)
		{
			const bool bPollTimeLimit = ((TimeLimitTimePollCounter++) % TimeLimitPollInterval == 0);
			if (bPollTimeLimit)
			{
				// It's this thread's time to check if time limit has been exceeded
				if ((FPlatformTime::Seconds() - StartTime) >= TimeLimit)
				{
					TimeLimitFlag.store(1, std::memory_order_relaxed);
					return true;
				}
			}
			else
			{
				// Check if any other thread signaled that time limit has been exceeded
				return !!TimeLimitFlag.load(std::memory_order_relaxed);
			}
		}
		return false;
	}
};

} // namespace UE::GC

namespace UE::GC::Private
{
	/**
	 * Returns statistics of the last Garbage Collecion cycle.
	 *
	 * @return	Statistics of the last Garabage Collection cycle.
	 */
	COREUOBJECT_API UE::GC::Private::FStats GetGarbageCollectionStats();
}