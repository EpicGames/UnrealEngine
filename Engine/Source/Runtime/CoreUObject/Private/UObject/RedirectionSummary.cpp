// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/CoreRedirects/RedirectionSummary.h"

#if WITH_EDITOR

#include "Misc/ScopeRWLock.h"

FRedirectionSummary::FRedirectionSummary(FRedirectionSummary&& Other)
{
	*this = MoveTemp(Other);
}

FRedirectionSummary& FRedirectionSummary::operator=(FRedirectionSummary&& Other)
{
	if (&Other == this)
	{
		return *this;
	}

	// Technical correctness for holding multiple critical sections: enter the locks in the same order
	// from all possible callsites, in this case, from two calls to operator=.
	TOptional<FWriteScopeLock> ScopeA;
	TOptional<FWriteScopeLock> ScopeB;
	if (&Other < this)
	{
		ScopeA.Emplace(Other.Lock);
		ScopeB.Emplace(Lock);
	}
	else
	{
		ScopeA.Emplace(Lock);
		ScopeB.Emplace(Other.Lock);
	}

	RedirectsForPackage = MoveTemp(Other.RedirectsForPackage);
	GlobalRedirects = MoveTemp(Other.GlobalRedirects);
	PrefixedRedirects = MoveTemp(Other.PrefixedRedirects);

	Other.RedirectsForPackage.Empty();
	Other.GlobalRedirects.Empty();
	Other.PrefixedRedirects.Empty();

	return *this;
}

void FRedirectionSummary::Add(const FCoreRedirect& CoreRedirect)
{
	FWriteScopeLock ScopeLock(Lock);

	if (CoreRedirect.IsPrefixMatch() && !CoreRedirect.OldName.PackageName.IsNone())
	{
		PrefixedRedirects.Add(FCoreRedirect(CoreRedirect));
	}
	else if (CoreRedirect.IsWildcardMatch() || CoreRedirect.OldName.PackageName.IsNone())
	{
		GlobalRedirects.Add(FCoreRedirect(CoreRedirect));

		OnGlobalCoreRedirectAdded.Broadcast(CoreRedirect);
	}
	else
	{
		for (FName PackageName : GetAffectedPackages(CoreRedirect))
		{
			RedirectsForPackage.FindOrAdd(PackageName).Add(FCoreRedirect(CoreRedirect));
		}
	}
}

void FRedirectionSummary::Remove(const FCoreRedirect& CoreRedirect)
{
	FWriteScopeLock ScopeLock(Lock);

	if (CoreRedirect.IsPrefixMatch() && !CoreRedirect.OldName.PackageName.IsNone())
	{
		PrefixedRedirects.Remove(CoreRedirect);
	}
	else if (CoreRedirect.IsWildcardMatch() || CoreRedirect.OldName.PackageName.IsNone())
	{
		GlobalRedirects.Remove(CoreRedirect);
	}
	else
	{
		for (FName PackageName : GetAffectedPackages(CoreRedirect))
		{
			FRedirectContainer* Container = RedirectsForPackage.Find(PackageName);
			if (Container)
			{
				Container->Remove(CoreRedirect);
				if (Container->IsEmpty())
				{
					RedirectsForPackage.Remove(PackageName);
				}
			}
		}
	}
}

void FRedirectionSummary::AddObjectRedirector(const FCoreRedirect& ObjectRedirect)
{
	FWriteScopeLock ScopeLock(Lock);
	ObjectRedirectsForPackage.FindOrAdd(ObjectRedirect.OldName.PackageName).Add(FCoreRedirect(ObjectRedirect));
}

void FRedirectionSummary::RemoveObjectRedirector(const FCoreRedirect& ObjectRedirect)
{
	FWriteScopeLock ScopeLock(Lock);

	FRedirectContainer* Container = ObjectRedirectsForPackage.Find(ObjectRedirect.OldName.PackageName);
	if (!Container)
	{
		return;
	}

	Container->Remove(ObjectRedirect);
	if (Container->IsEmpty())
	{
		ObjectRedirectsForPackage.Remove(ObjectRedirect.OldName.PackageName);
	}
}

void FRedirectionSummary::GetObjectRedirectHashAffectingPackage(const FName& PackageName, FBlake3& Hasher)
{
	{
		FReadScopeLock ScopeLock(Lock);

		const FRedirectContainer* Container = ObjectRedirectsForPackage.Find(PackageName);
		if (!Container)
		{
			return;
		}

		bool bSuccess = Container->TryAppendHashInReadLock(Hasher);
		if (bSuccess)
		{
			return;
		}
	}

	{
		FWriteScopeLock WriteLock(Lock);
		if (FRedirectContainer* Container = ObjectRedirectsForPackage.Find(PackageName))
		{
			Container->AppendHashInWriteLock(Hasher);
		}
	}
}

void FRedirectionSummary::GetHashAffectingPackages(const TConstArrayView<FName>& PackageNames, TArray<FBlake3Hash>& Hashes)
{
	check(Hashes.Num() == PackageNames.Num());

	if (PackageNames.IsEmpty())
	{
		return;
	}

	int32 Index = 0;
	const FName* PackageNamesData = PackageNames.GetData();
	int32 PackageNamesNum = PackageNames.Num();
	bool bNeedWriteLock = false;

	{
		FReadScopeLock ScopeLock(Lock);
		while (!bNeedWriteLock && Index < PackageNamesNum)
		{
			FBlake3 Hasher;
			FRedirectContainer* Container = RedirectsForPackage.Find(PackageNamesData[Index]);
			if (!Container)
			{
				++Index;
			}
			else if (Container->TryAppendHashInReadLock(Hasher))
			{
				Hashes[Index] = Hasher.Finalize();
				++Index;
			}
			else
			{
				bNeedWriteLock = true;
			}
		}
	}

	if (bNeedWriteLock)
	{

		FWriteScopeLock WriteLock(Lock);
		while (Index < PackageNamesNum)
		{
			FRedirectContainer* Container = RedirectsForPackage.Find(PackageNamesData[Index]);
			if (Container)
			{
				FBlake3 Hasher;
				Container->AppendHashInWriteLock(Hasher);
				Hashes[Index] = Hasher.Finalize();
			}
			++Index;
		}
	}

	//Now add prefixed redirects.
	for (int32 PackageIndex = 0; PackageIndex < PackageNamesNum; ++PackageIndex)
	{
		const FName& PackageName = PackageNamesData[PackageIndex];

		bool bPrefixedRedirectFound = false;
		FBlake3 Hasher;
		for (const TPair<FCoreRedirect, bool>& Pair : PrefixedRedirects)
		{
			FCoreRedirectObjectName ObjectName(PackageName.ToString());

			const FCoreRedirect& Redirect = Pair.Key;
			bool bMatch = Redirect.Matches(ObjectName);
			if (bMatch)
			{
				Redirect.AppendHash(Hasher);
				bPrefixedRedirectFound = true;
			}
		}

		if (bPrefixedRedirectFound)
		{
			Hasher.Update(&Hashes[PackageIndex], sizeof(FBlake3Hash));
			Hashes[PackageIndex] = Hasher.Finalize();
		}
	}
}

void FRedirectionSummary::AppendHashGlobal(FBlake3& Hasher)
{
	{
		FReadScopeLock ScopeLock(Lock);
		if (GlobalRedirects.TryAppendHashInReadLock(Hasher))
		{
			return;
		}
	}
	FWriteScopeLock WriteLock(Lock);
	GlobalRedirects.AppendHashInWriteLock(Hasher);
}

FCoreRedirects::FGlobalCoreRedirectAddedEvent& FRedirectionSummary::GetOnGlobalCoreRedirectAdded()
{
	return OnGlobalCoreRedirectAdded;
}

void FRedirectionSummary::DebugPrintGlobalCoreRedirects() const
{
	GlobalRedirects.DebugPrint();
}

void FRedirectionSummary::FRedirectContainer::Add(FCoreRedirect&& Redirect)
{
	Redirects.Add(MoveTemp(Redirect));
	bHashDirty = true;
}

void FRedirectionSummary::FRedirectContainer::Remove(const FCoreRedirect& Redirect)
{
	Redirects.Remove(Redirect);
	bHashDirty = true;
}

bool FRedirectionSummary::FRedirectContainer::IsEmpty() const
{
	return Redirects.IsEmpty();
}

void FRedirectionSummary::FRedirectContainer::Empty()
{
	Redirects.Empty();
	Hash = FBlake3Hash();
	bHashDirty = false;
}

bool FRedirectionSummary::FRedirectContainer::TryAppendHashInReadLock(FBlake3& Hasher) const
{
	if (bHashDirty)
	{
		return false;
	}
	AppendHashWithoutDirtyCheck(Hasher);
	return true;
}

void FRedirectionSummary::FRedirectContainer::AppendHashInWriteLock(FBlake3& Hasher)
{
	if (bHashDirty)
	{
		CalculateHash();
		bHashDirty = false;
	}
	AppendHashWithoutDirtyCheck(Hasher);
}

void FRedirectionSummary::FRedirectContainer::DebugPrint() const
{
	for (const TPair<FCoreRedirect, bool>& Pair : Redirects)
	{
		const FCoreRedirect& Redirect = Pair.Key;
		UE_LOGF(LogCoreRedirects, Display, "%ls->%ls (%d)", *Redirect.OldName.ToString(), *Redirect.NewName.ToString(), EnumToUnderlyingType(Redirect.RedirectFlags));
	}
}

void FRedirectionSummary::FRedirectContainer::CalculateHash()
{
	// For hashing we need determinism across multiple runs, so since our hash is dirty
	// we have to sort the redirects on keys now to ensure we have a deterministic hash.
	Redirects.KeyStableSort(TLess<>());

	FBlake3 Hasher;
	for (const TPair<FCoreRedirect,bool>& Pair: Redirects)
	{
		const FCoreRedirect& Redirect = Pair.Key;
		Redirect.AppendHash(Hasher);
	}
	Hash = Hasher.Finalize();
}

void FRedirectionSummary::FRedirectContainer::AppendHashWithoutDirtyCheck(FBlake3& Hasher) const
{
	Hasher.Update(&Hash.GetBytes(), sizeof(Hash.GetBytes()));
}

TArray<FName, TInlineAllocator<2>> FRedirectionSummary::GetAffectedPackages(const FCoreRedirect& Redirect)
{
	// Note on why we need the redirects that redirect both from and to the package:
	// When a redirector from X to Y changes to redirect from X to Z, package A that references X will now need to
	// write Z instead of Y into its saved imports.
	// When a redirector from X to Y changes to redirect from W to Y, or is deleted, package A that references X will
	// now need to write X instead of Y into its saved imports.
	TArray<FName, TInlineAllocator<2>> Result;
	if (!Redirect.OldName.PackageName.IsNone())
	{
		Result.Add(Redirect.OldName.PackageName);
	}
	if (!Redirect.NewName.PackageName.IsNone() && Redirect.NewName.PackageName != Redirect.OldName.PackageName)
	{
		Result.Add(Redirect.NewName.PackageName);
	}
	return Result;
}

bool FRedirectionSummary::FCompareRedirect::operator()(const FCoreRedirect& A, const FCoreRedirect& B)
{
	return A.Compare(B) < 0;
}

#endif
