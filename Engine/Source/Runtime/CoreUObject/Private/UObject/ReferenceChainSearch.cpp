// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ReferenceChainSearch.h"

#include "Algo/AnyOf.h"
#include "Algo/RemoveIf.h"
#include "Algo/Reverse.h"
#include "Experimental/Graph/GraphConvert.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/ThreadHeartBeat.h"
#include "Misc/FeedbackContext.h"
#include "Templates/Greater.h"
#include "Misc/StringOutputDevice.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/GCObject.h"
#include "UObject/GarbageCollectionInternalFlags.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UnrealType.h"

DEFINE_LOG_CATEGORY_STATIC(LogReferenceChain, Log, All);

namespace UE::ReferenceChainSearch
{
	using UE::Graph::FGraph;
	using UE::Graph::FVertex;

	// This policy causes us to allocate a dense graph in all cases.
	// If memory usage is a concern for minimal/direct search cases, or when excluding DisregardForGC objects, we
	// could build a sparse list of relevant objects and map them to vertices that way rather than using GUObjectArray.
	struct FPolicyUObjectHeap
	{
		using ObjectType = UObject*; // FGCObjectInfo
		using ConstObjectType = const UObject*;

		TMap<const UObject*, FGCObjectInfo*>& ObjectToInfoMap;
		TMap<FGCObjectInfo*, FReferenceChainSearch::FGraphNode*>& AllNodes;
		bool bGCOnly;

		FPolicyUObjectHeap(
			TMap<const UObject*, FGCObjectInfo*>& InObjectToInfoMap,
			TMap<FGCObjectInfo*, FReferenceChainSearch::FGraphNode*>& InAllNodes,
			EReferenceChainSearchMode Mode)
			: ObjectToInfoMap(InObjectToInfoMap)
			, AllNodes(InAllNodes)
		{
			bGCOnly = !!(Mode & EReferenceChainSearchMode::GCOnly);
		}

		FVertex GetGCObjectReferencerVertex() const
		{
			return FGCObject::GGCObjectReferencer ? ObjectToVertex(FGCObject::GGCObjectReferencer) : UE::Graph::InvalidVertex;
		}
		int32 GetFirstVertexIndex() const
		{
			return bGCOnly ? GUObjectArray.GetFirstGCIndex() : 0;
		}
		int32 GetNumVertices() const
		{
			return GUObjectArray.GetObjectArrayNum();
		}
		FVertex ObjectToVertex(ConstObjectType Object) const
		{
			FVertex Vertex = GUObjectArray.ObjectToIndex(Object);
			UE_CLOGF(Vertex < 0 || Vertex >= GetNumVertices(), LogReferenceChain, Fatal, "Invalid object index in reference chain search %d", Vertex);
			return Vertex;
		}
		ObjectType VertexToObject(FVertex Vertex) const
		{
			return static_cast<UObject*>(GUObjectArray.IndexToObject(Vertex)->GetObject());
		}

		bool IsIn(ConstObjectType Outer, ConstObjectType PotentialInner) const
		{
			return PotentialInner->IsIn(Outer);
		}
		bool IsIn(FVertex Outer, FVertex PotentialInner) const
		{
			return VertexToObject(PotentialInner)->IsIn(VertexToObject(Outer));
		}

		PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
		bool IsRoot(FVertex Vertex, EReferenceChainSearchMode SearchMode) const
		{
			const UObject* Object = VertexToObject(Vertex);
			return Object->HasAnyInternalFlags(EInternalObjectFlags_RootFlags)
				|| Object->GetRefCount() != 0
				|| (GARBAGE_COLLECTION_KEEPFLAGS != RF_NoFlags && Object->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS)
					&& !(SearchMode & EReferenceChainSearchMode::FullChain));
		}
		PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS

		bool IsGarbage(FVertex Vertex) const
		{
			return GUObjectArray.IndexToObject(Vertex)->HasAnyFlags(EInternalObjectFlags::Garbage);
		}

		FGCObjectInfo* FindOrCreateObjectInfoAndGraphNode(FVertex InVertex)
		{
			ObjectType Object = VertexToObject(InVertex);
			FGCObjectInfo* Info = ObjectToInfoMap.FindRef(Object);
			if (!Info)
			{
				Info = FGCObjectInfo::FindOrAddInfoHelper(Object, ObjectToInfoMap);
			}
			if (!AllNodes.Contains(Info))
			{
				FReferenceChainSearch::FGraphNode* Node = new FReferenceChainSearch::FGraphNode;
				Node->ObjectInfo = Info;
				AllNodes.Add(Info, Node);
			}
			return Info;
		}

		FReferenceChainSearch::FGraphNode* VertexToGraphNode(FVertex InVertex) const
		{
			ObjectType Object = VertexToObject(InVertex);
			FGCObjectInfo* ObjectInfo = ObjectToInfoMap.FindChecked(Object);
			return AllNodes.FindChecked(ObjectInfo);
		}

		/**
		 * @param InOutReferenceInfo a map whose keys are vertices we want to find references FROM and whose valuesa re maps whose keys we want to
		 * find references TO.
		 */
		void GatherReferenceInfo(TMap<FVertex, TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>>& InOutReferenceInfo);
	};

#if ENABLE_GC_HISTORY
	struct FPolicyGCHistory
	{
		using ObjectType = FGCObjectInfo*;

		const FGCSnapshot& Snapshot;
		TMap<FGCObjectInfo*, FReferenceChainSearch::FGraphNode*>& AllNodes;

		TMap<FGCObjectInfo*, FVertex> ObjectInfoToVertex;
		int32 NumVertexIndices = 0;

		FPolicyGCHistory(const FGCSnapshot& InSnapshot, TMap<FGCObjectInfo*, FReferenceChainSearch::FGraphNode*>& InAllNodes)
			: Snapshot(InSnapshot)
			, AllNodes(InAllNodes)
		{
			ObjectInfoToVertex.Reserve(Snapshot.ObjectToInfoMap.GetMaxIndex() + 1);
			for (auto It = Snapshot.ObjectToInfoMap.CreateConstIterator(); It; ++It)
			{
				int32 Index = It.GetId().AsInteger();
				ObjectInfoToVertex.Add(It.Value(), Index);
				NumVertexIndices = FMath::Max(Index + 1, NumVertexIndices);
			}
		}

		FVertex GetGCObjectReferencerVertex() const
		{
			if (FGCObject::GGCObjectReferencer)
			{
				if (FGCObjectInfo* Info = Snapshot.ObjectToInfoMap.FindRef(FGCObject::GGCObjectReferencer))
				{
					return ObjectToVertex(Info);
				}
			}
			return UE::Graph::InvalidVertex;
		}

		int32 GetFirstVertexIndex() const
		{
			return 0;
		}
		int32 GetNumVertices() const
		{
			return Snapshot.ObjectToInfoMap.GetMaxIndex() + 1;
		}

		FVertex ObjectToVertex(ObjectType Object) const
		{
			const FVertex* Vertex = ObjectInfoToVertex.Find(Object);
			UE_CLOGF(!Vertex || *Vertex < 0 || *Vertex >= GetNumVertices(), LogReferenceChain, Fatal, "Invalid object index in reference chain search %d", Vertex ? *Vertex : -1);
			return *Vertex;
		}

		ObjectType VertexToObject(FVertex Vertex) const
		{
			return Snapshot.ObjectToInfoMap.Get(FSetElementId::FromInteger(Vertex)).Value;
		}

		bool IsIn(FGCObjectInfo* Outer, FGCObjectInfo* PotentialInner) const
		{
			return PotentialInner->IsIn(Outer);
		}
		bool IsIn(FVertex Outer, FVertex PotentialInner) const
		{
			return VertexToObject(PotentialInner)->IsIn(VertexToObject(Outer));
		}

		PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS
		bool IsRoot(FVertex Vertex, EReferenceChainSearchMode SearchMode) const
		{
			ObjectType Object = VertexToObject(Vertex);
			return Object->HasAnyInternalFlags(EInternalObjectFlags_RootFlags)
				|| Object->GetRefCount() != 0
				|| (GARBAGE_COLLECTION_KEEPFLAGS != RF_NoFlags && Object->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS)
					&& !(SearchMode & EReferenceChainSearchMode::FullChain));
		}
		PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS

		bool IsGarbage(FVertex Vertex) const
		{
			return VertexToObject(Vertex)->IsGarbage();
		}

		FGCObjectInfo* FindOrCreateObjectInfoAndGraphNode(FVertex InVertex)
		{
			FGCObjectInfo* Info = VertexToObject(InVertex);
			if (!AllNodes.Contains(Info))
			{
				FReferenceChainSearch::FGraphNode* Node = new FReferenceChainSearch::FGraphNode;
				Node->ObjectInfo = Info;
				AllNodes.Add(Info, Node);
			}
			return Info;
		}

		FReferenceChainSearch::FGraphNode* VertexToGraphNode(FVertex InVertex) const
		{
			ObjectType Object = VertexToObject(InVertex);
			return AllNodes.FindChecked(Object);
		}

		/**
		 * @param InOutReferenceInfo a map whose keys are vertices we want to find references FROM and whose valuesa re maps whose keys we want to
		 * find references TO.
		 */
		void GatherReferenceInfo(TMap<FVertex, TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>>& InOutReferenceInfo);
	};
#endif // ENABLE_GC_HISTORY

	template<typename Derived>
	struct TReferenceSearchBase
	{
		// We don't actually implement reference processing on the processor type, we just store a thread index and delegate to the derived
		// class
		struct FProcessor
		{
			static constexpr EGCOptions Options = EGCOptions::None;
			int32 ThreadIndex;
			Derived* This;

			void BeginTimingObject(UObject* CurrentObject) { }
			void UpdateDetailedStats(UObject* CurrentObject) { }
			void LogDetailedStatsSummary() { }
			void HandleTokenStreamObjectReference(UE::GC::FWorkerContext& Context,
				UObject* ReferencingObject,
				UObject*& Object,
				UE::GC::FMemberId MemberId,
				UE::GC::EOrigin Origin,
				bool bAllowReferenceElimination)
			{
				if (Object)
				{
					ReferencingObject = Context.GetReferencingObject();
					if (!ReferencingObject)
					{
						ReferencingObject = FGCObject::GGCObjectReferencer;
					}
					check(ReferencingObject);

					if (Object != ReferencingObject)
					{
						This->HandleObjectReference(Context, ThreadIndex, Object, ReferencingObject, MemberId, Origin);
					}
				}
			}
			FORCEINLINE bool IsTimeLimitExceeded() const
			{
				return false;
			}
		};

		template<bool bNeedsPropertyReferencer = false>
		class FCollector final: public FReferenceCollector
		{
		protected:
			FProcessor& Processor;
			UE::GC::FWorkerContext& Context;

		public:
			FCollector(FProcessor& InProcessor, FGCArrayStruct& InContext)
				: Processor(InProcessor)
				, Context(InContext) { }

			virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
			{
				Processor.HandleTokenStreamObjectReference(Context,
					const_cast<UObject*>(ReferencingObject),
					Object,
					UE::GC::EMemberlessId::Collector,
					UE::GC::EOrigin::Other,
					false);
			}
			virtual void HandleObjectReferences(UObject** Objects, int32 Num, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
			{
				for (UObject*& Object : MakeArrayView(Objects, Num))
				{
					HandleObjectReference(Object, ReferencingObject, ReferencingProperty);
				}
			}

			virtual bool NeedsPropertyReferencer() const
			{
				return bNeedsPropertyReferencer;
			}
			virtual bool IsIgnoringArchetypeRef() const override
			{
				return false;
			}
			virtual bool IsIgnoringTransient() const override
			{
				return false;
			}
			virtual bool MarkWeakObjectReferenceForClearing(UObject** WeakReference, UObject* ReferenceOwner)
			{
				// To avoid false positives we need to implement this method just like GC does
				// as these references will be treated as weak and should not be reported
				return true;
			}
		};

		// Stores a set of edge lists for objects starting at StartVertex.
		struct FThreadData
		{
			UObject* PreviousReferencingObject = nullptr;
			TSet<FVertex> CurrentEdgeList;

			TArray64<FVertex> GraphBuffer;
			TArray<TArrayView<FVertex>> EdgeLists;
			FVertex StartVertex;

			SIZE_T GetAllocatedSize() const
			{
				return CurrentEdgeList.GetAllocatedSize() + GraphBuffer.GetAllocatedSize() + EdgeLists.GetAllocatedSize();
			}

			void FlushEdgeList(const FPolicyUObjectHeap& InPolicy)
			{
				if (PreviousReferencingObject && CurrentEdgeList.Num() > 0)
				{
					// Flush current edge list to graph buffer
					FVertex SourceVertex = InPolicy.ObjectToVertex(PreviousReferencingObject);
					TArray<FVertex> EdgeList = CurrentEdgeList.Array();
					const int64 BufferStartIndex = GraphBuffer.Num();
					// During construction store edge lists relative to null to be fixed up later
					FVertex* StartAddress = nullptr;
					GraphBuffer.Append(EdgeList);
					EdgeLists[SourceVertex - StartVertex] = TArrayView<FVertex>(StartAddress + BufferStartIndex, EdgeList.Num()); //-V769
					CurrentEdgeList.Reset();
				}
			}
		};
		FPolicyUObjectHeap& Policy;
		TArray<FThreadData> AllThreadData; // Order of threads does not matter for MergeGraph

		SIZE_T GetAllocatedSize() const
		{
			SIZE_T Size = 0;
			for (const FThreadData& Thread : AllThreadData)
			{
				Size += Thread.GetAllocatedSize();
			}
			return Size + AllThreadData.GetAllocatedSize();
		}

		void SetNumThreads(int32 InNumThreads, int32 NumberOfObjectsPerThread, int32 GlobalStartIndex, int32 MaxNumberOfObjects, int32 ObjectReferencerIndex)
		{
			AllThreadData.Reset();
			AllThreadData.AddDefaulted(InNumThreads);
			for (int32 i = 0; i < InNumThreads; ++i)
			{
				FThreadData& Thread = AllThreadData[i];
				Thread.StartVertex = GlobalStartIndex + i * NumberOfObjectsPerThread;
				const int32 NumObjects =
					(i < (InNumThreads - 1)) ? NumberOfObjectsPerThread : (MaxNumberOfObjects - (InNumThreads - 1) * NumberOfObjectsPerThread);
				Thread.EdgeLists.Reserve(NumObjects);
				Thread.EdgeLists.SetNumZeroed(NumObjects);
			}

			if (ObjectReferencerIndex != INDEX_NONE)
			{
				FThreadData& ObjectReferencerThreadData = AllThreadData.AddDefaulted_GetRef();
				ObjectReferencerThreadData.StartVertex = ObjectReferencerIndex;
				ObjectReferencerThreadData.EdgeLists.Reserve(1);
				ObjectReferencerThreadData.EdgeLists.SetNumZeroed(1);
			}
		}

		void CollectAllReferences(bool bGCOnly)
		{
			constexpr bool bParallel = Derived::bParallel;
			Derived* This = static_cast<Derived*>(this);
			const int32 GlobalStartObjectIndex = bGCOnly ? GUObjectArray.GetFirstGCIndex() : 0;
			int32 ObjectReferencerVertex = bGCOnly && FGCObject::GGCObjectReferencer ? Policy.ObjectToVertex(FGCObject::GGCObjectReferencer) : INDEX_NONE;
			if (ObjectReferencerVertex >= GlobalStartObjectIndex)
			{
				ObjectReferencerVertex = INDEX_NONE;
			}

			const int32 MaxNumberOfObjects = GUObjectArray.GetObjectArrayNum() - GlobalStartObjectIndex;
			
			if constexpr (bParallel)
			{
				const int32 NumThreads = GetNumCollectReferenceWorkers();
				const int32 NumberOfObjectsPerThread = (MaxNumberOfObjects / NumThreads) + 1;

				SetNumThreads(NumThreads, NumberOfObjectsPerThread, GlobalStartObjectIndex, MaxNumberOfObjects, ObjectReferencerVertex);

				ParallelFor(NumThreads,
					[this, bGCOnly, This, GlobalStartObjectIndex, MaxNumberOfObjects, NumThreads, NumberOfObjectsPerThread](int32 ThreadIndex) {
						FProcessor Processor{ ThreadIndex, This };
						TArray<UObject*> ObjectsToSerialize;
						
						ObjectsToSerialize.Reserve(NumberOfObjectsPerThread);

						const int32 FirstObjectIndex = GlobalStartObjectIndex + ThreadIndex * NumberOfObjectsPerThread;
						const int32 NumObjects = (ThreadIndex < (NumThreads - 1))
												   ? NumberOfObjectsPerThread
												   : (MaxNumberOfObjects - (NumThreads - 1) * NumberOfObjectsPerThread);

						// First cache all potential referencers
						for (int32 ObjectIndex = 0; ObjectIndex < NumObjects; ++ObjectIndex)
						{
							FUObjectItem& ObjectItem = GUObjectArray.GetObjectItemArrayUnsafe()[FirstObjectIndex + ObjectIndex];
							UObject* Object = static_cast<UObject*>(ObjectItem.GetObject());
							if (Object && !ObjectItem.IsUnreachable() && !This->ShouldSkipReferencer(ThreadIndex, Object))
							{
								ObjectsToSerialize.Add(Object);
							}
						}

						UE::GC::FWorkerContext Context;
						Context.SetInitialObjectsUnpadded(ObjectsToSerialize);
						CollectReferences<FCollector<>>(Processor, Context);
					});
			}
			else 
			{
				SetNumThreads(1, MaxNumberOfObjects, GlobalStartObjectIndex, MaxNumberOfObjects, ObjectReferencerVertex);
				FProcessor Processor{ 0, This };
				TArray<UObject*> ObjectsToProcess;
				for (FRawObjectIterator It; It; ++It)
				{
					FUObjectItem* ObjItem = *It;
					UObject* Object = static_cast<UObject*>(ObjItem->GetObject());

					// // We can't ask the iterator for only GC objects because that would skip the GC Object referencer
					// if (bGCOnly && GUObjectArray.IsDisregardForGC(Object) && (Object != FGCObject::GGCObjectReferencer))
					// {
					// 	continue;
					// }

					if (This->ShouldSkipReferencer(0, Object))
					{
						continue;
					}

					// Find direct references
					UE::GC::FWorkerContext Context;
					ObjectsToProcess = { Object };
					Context.SetInitialObjectsUnpadded(ObjectsToProcess);
					CollectReferences<FCollector<>>(Processor, Context);
				}
			}
			
			if (ObjectReferencerVertex != INDEX_NONE)
			{
				FProcessor Processor{ AllThreadData.Num() - 1, This };
				TArray<UObject*> ObjectsToProcess;
				// Find direct references
				UE::GC::FWorkerContext Context;
				ObjectsToProcess = { FGCObject::GGCObjectReferencer };
				Context.SetInitialObjectsUnpadded(ObjectsToProcess);
				CollectReferences<FCollector<>>(Processor, Context);
			}
		}

		void CollectReferencesFromObject(UObject* FromObject)
		{
			Derived* This = static_cast<Derived*>(this);
			FProcessor Processor{ 0, This };
			// Find direct references
			UE::GC::FWorkerContext Context;
			TArray<UObject*> ObjectsToProcess;
			ObjectsToProcess = { FromObject };
			Context.SetInitialObjectsUnpadded(ObjectsToProcess);
			CollectReferences<FCollector<true /* detailed property info */>>(Processor, Context);
		}

		TReferenceSearchBase(FPolicyUObjectHeap& InPolicy)
			: Policy(InPolicy) { }
	};

	struct FDirectReferenceSearch : public TReferenceSearchBase<FDirectReferenceSearch>
	{
		using Super = TReferenceSearchBase<FDirectReferenceSearch>;
		static constexpr bool bParallel = true;

		FDirectReferenceSearch(FPolicyUObjectHeap& InPolicy)
			: Super(InPolicy) { }

		bool ShouldSkipReferencer(int32 ThreadIndex, UObject* Object)
		{
			return false;
		}

		void HandleObjectReference(
			UE::GC::FWorkerContext& Context,
			int32 ThreadIndex, UObject* Object, UObject* ReferencingObject, UE::GC::FTokenId TokenIndex, UE::GC::EOrigin TokenType)
		{
			FThreadData& ThreadData = AllThreadData[ThreadIndex];
			if (ThreadData.PreviousReferencingObject != ReferencingObject)
			{
				ThreadData.FlushEdgeList(Policy);
			}

			ThreadData.PreviousReferencingObject = ReferencingObject;
			ThreadData.CurrentEdgeList.Add(Policy.ObjectToVertex(Object));
		}

		// Merge all edge lists into a single edge list, leaving the actual data in individual buffers
		void MergeGraph(TArray<TConstArrayView<FVertex>>& OutMergedEdgeLists)
		{
			FVertex* OldBase = nullptr;
			for (FThreadData& Thread : AllThreadData)
			{
				Thread.FlushEdgeList(Policy);
				FVertex* EdgeListBase = Thread.GraphBuffer.GetData();
				for (int32 i = 0; i < Thread.EdgeLists.Num(); ++i)
				{
					TConstArrayView<FVertex> TempEdgeList = Thread.EdgeLists[i];
					FVertex SourceVertex = Thread.StartVertex + i;
					TConstArrayView<FVertex> EdgeList{ TempEdgeList.GetData() - OldBase + EdgeListBase, TempEdgeList.Num() };
					OutMergedEdgeLists[SourceVertex] = EdgeList;
				}
			}
		}
	};

	struct FMinimalReferenceSearch : public TReferenceSearchBase<FMinimalReferenceSearch>
	{
		using Super = TReferenceSearchBase<FMinimalReferenceSearch>;
		static constexpr bool bParallel = true;
		TSet<const UObject*> TargetObjects;
		TSet<UObject*> FoundTargets;

		FMinimalReferenceSearch(FPolicyUObjectHeap& InPolicy)
			: Super(InPolicy) { }

		SIZE_T GetAllocatedSize() const
		{
			return TargetObjects.GetAllocatedSize() + FoundTargets.GetAllocatedSize() + Super::GetAllocatedSize();
		}

		bool ShouldSkipReferencer(int32 ThreadIndex, UObject* Object)
		{
			// Skip objects which are in any of what we're looking for so we only report reference external to that entire group
			for (UObject* OuterObject = Object; OuterObject; OuterObject = OuterObject->GetOuter())
			{
				if (TargetObjects.Contains(OuterObject) && Object->GetOuter() != nullptr)
				{
					FThreadData& ThreadData = AllThreadData[ThreadIndex];
					// Add an edge between this object and its outer so the outer object may be reachable via this object
					ThreadData.FlushEdgeList(Policy);
					ThreadData.PreviousReferencingObject = Object;
					ThreadData.CurrentEdgeList.Add(Policy.ObjectToVertex(Object->GetOuter()));
					return true;
				}
			}
			return false;
		}

		void HandleObjectReference(
			UE::GC::FWorkerContext& Context,
			int32 ThreadIndex,
			UObject* Object,
			UObject* ReferencingObject,
			UE::GC::FTokenId TokenIndex,
			UE::GC::EOrigin TokenType)
		{
			FThreadData& ThreadData = AllThreadData[ThreadIndex];
			if (!TargetObjects.Contains(Object))
			{
				UObject* OuterObject = Object->GetOuter();
				for (; OuterObject; OuterObject = OuterObject->GetOuter())
				{
					if (TargetObjects.Contains(OuterObject))
					{
						break;
					}
				}

				// Only track references to target objects or their inners
				if (!OuterObject)
				{
					return;
				}
			}

			// Add an edge from ReferencingObject to Object
			if (ThreadData.PreviousReferencingObject != ReferencingObject)
			{
				ThreadData.FlushEdgeList(Policy);
			}

			ThreadData.PreviousReferencingObject = ReferencingObject;
			ThreadData.CurrentEdgeList.Add(Policy.ObjectToVertex(Object));
			if (TargetObjects.Contains(Object))
			{
				FoundTargets.Add(Object);
			}
		}

		// Merge all edge lists into a single edge list, leaving the actual data in individual buffers
		// For each edge list, if it contains any direct references to the targets, remove any other references
		void MergeGraph(TArray<TConstArrayView<FVertex>>& OutMergedEdgeLists)
		{
			for (FThreadData& Thread : AllThreadData)
			{
				Thread.FlushEdgeList(Policy);
			}

			TSet<FVertex> TargetVertices;
			for (const UObject* TargetObject : TargetObjects)
			{
				TargetVertices.Add(Policy.ObjectToVertex(TargetObject));
			}

			FVertex* OldBase = nullptr;
			for (FThreadData& Thread : AllThreadData)
			{
				FVertex* EdgeListBase = Thread.GraphBuffer.GetData();
				for (int32 i = 0; i < Thread.EdgeLists.Num(); ++i)
				{
					TArrayView<FVertex> EdgeList{ Thread.EdgeLists[i].GetData() - OldBase + EdgeListBase, Thread.EdgeLists[i].Num() };
					int32 NumDirectRefs = Algo::RemoveIf(EdgeList, [&](FVertex Vertex) { return !TargetVertices.Contains(Vertex); });
					if (NumDirectRefs != 0)
					{
						EdgeList = EdgeList.Slice(0, NumDirectRefs);
					}

					FVertex SourceVertex = Thread.StartVertex + i;
					OutMergedEdgeLists[SourceVertex] = EdgeList;
				}
			}
		}
	};

	/**
	 * Create an initial graph from all UObject references on the heap for later analysis.
	 * Do not record any additional info such as property names/callstacks in this pass.
	 *
	 */
	void PerformInitialGatherFromLiveUObjectHeap(FPolicyUObjectHeap Policy,
		TConstArrayView<const UObject*> ObjectsToFindReferencesTo,
		EReferenceChainSearchMode Mode,
		UE::Graph::FGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::ReferenceChainSearch::PerformInitialGatherFromLiveUObjectHeap);
		// Minimal mode tries to minimize the amount of heap memory used by searching for a mixture of direct references and references via
		// outer chains
		const bool bMinimal = !!(Mode & EReferenceChainSearchMode::Minimal);
		const bool bGCOnly = !!(Mode & EReferenceChainSearchMode::GCOnly);
		const bool bDirectOnly = !!(Mode & EReferenceChainSearchMode::Direct);

		if (bMinimal)
		{
			FMinimalReferenceSearch Search(Policy);
			for (const UObject* Object : ObjectsToFindReferencesTo)
			{
				Search.TargetObjects.Add(Object);
			}

			Search.CollectAllReferences(bGCOnly);
			TArray<TConstArrayView<FVertex>> MergedEdgeLists;
			MergedEdgeLists.Reserve(Policy.GetNumVertices());
			if (bGCOnly)
			{
				MergedEdgeLists.AddZeroed(Policy.GetFirstVertexIndex());
			}
			MergedEdgeLists.SetNumUninitialized(Policy.GetNumVertices());
			Search.MergeGraph(MergedEdgeLists);

			OutGraph = UE::Graph::ConstructTransposeGraph(MergedEdgeLists);
			UE_LOGF(LogReferenceChain, Display, "InitialGather memory usage: %.2f", static_cast<double>(OutGraph.GetAllocatedSize() + Search.GetAllocatedSize()) / 1024.0 / 1024.0);
		}
		else
		{
			FDirectReferenceSearch Search(Policy);
			Search.CollectAllReferences(bGCOnly);
			TArray<TConstArrayView<FVertex>> MergedEdgeLists;
			MergedEdgeLists.Reserve(Policy.GetNumVertices());
			if (bGCOnly)
			{
				MergedEdgeLists.AddZeroed(Policy.GetFirstVertexIndex());
			}
			MergedEdgeLists.SetNumUninitialized(Policy.GetNumVertices());
			Search.MergeGraph(MergedEdgeLists);

			OutGraph = UE::Graph::ConstructTransposeGraph(MergedEdgeLists);
			UE_LOGF(LogReferenceChain, Display, "InitialGather memory usage: %.2f", static_cast<double>(OutGraph.GetAllocatedSize() + Search.GetAllocatedSize()) / 1024.0 / 1024.0);
		}
	}

#if ENABLE_GC_HISTORY
	void PerformInitialGatherFromGCHistory(const FPolicyGCHistory& Policy, FGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::ReferenceChainSearch::PerformInitialGatherFromGCHistory);
		int64 TotalEdges = 0;
		for (const TPair<FReferenceToken, TArray<FGCDirectReference>*>& Pair : Policy.Snapshot.DirectReferences)
		{
			if (Pair.Key.IsGCObjectInfo())
			{
				FVertex FromVertex = Policy.ObjectToVertex(Pair.Key.AsGCObjectInfo());
				TotalEdges += Pair.Value->Num();
			}
		}

		FGraph TempGraph;
		TempGraph.Buffer.Reserve(TotalEdges);
		TempGraph.EdgeLists.Reserve(Policy.GetNumVertices());
		TempGraph.EdgeLists.SetNumZeroed(Policy.GetNumVertices());

		for (const TPair<FReferenceToken, TArray<FGCDirectReference>*>& Pair : Policy.Snapshot.DirectReferences)
		{
			if (Pair.Key.IsGCObjectInfo())
			{
				int64 StartOffset = TempGraph.Buffer.Num();
				FVertex FromVertex = Policy.ObjectToVertex(Pair.Key.AsGCObjectInfo());
				for (FGCDirectReference& ReferenceInfo : *Pair.Value)
				{
					if (ReferenceInfo.Reference.IsGCObjectInfo())
					{
						FVertex ToVertex = Policy.ObjectToVertex(ReferenceInfo.Reference.AsGCObjectInfo());
						check(TempGraph.EdgeLists.IsValidIndex(ToVertex));
						TempGraph.Buffer.Add(ToVertex);
					}
				}
				TempGraph.EdgeLists[FromVertex] =
					TConstArrayView<int32>(TempGraph.Buffer.GetData() + StartOffset, static_cast<int32>(TempGraph.Buffer.Num() - StartOffset));
			}
		}

		OutGraph = UE::Graph::ConstructTransposeGraph(TempGraph.EdgeLists);
	}
#endif // ENABLE_GC_HISTORY

	// Temporary storage for a reference chain before gathering information
	struct FGraphPath
	{
		FVertex Target;
		// Reversed after construction.
		// 	During construction:
		// 		First element is root
		// 	After construction:
		//		First element is target, last element is root
		TArray<FVertex> Vertices;
	};

	template<typename PolicyType>
	int32 BuildGraphPathRecursive(PolicyType& Policy,
		const UE::Graph::FGraph& Graph,
		TMap<FVertex, int32>& VisitCounts,
		FVertex ThisVertex,
		TArray<FGraphPath>& ProducedPaths,
		int32 ChainDepth, // For reserving space in new chains so we don't reallocate as we return from recursion
		const int32 VisitCounter,
		EReferenceChainSearchMode Mode,
		FVertex GCObjReferencerVertex)
	{
		int32 ProducedPathsCount = 0; // ?

		// Always create a chain for the FGCObject referencer node even if we've found another path to it, because it handles many types of
		// references
		if (ThisVertex == GCObjReferencerVertex)
		{
			FGraphPath& Path = ProducedPaths.AddDefaulted_GetRef();
			Path.Vertices.Reserve(ChainDepth);
			Path.Vertices.Add(ThisVertex);
			ProducedPathsCount = 1;
			return ProducedPathsCount;
		}

		if (int32& ThisVertexVisitCount = VisitCounts.FindOrAdd(ThisVertex); ThisVertexVisitCount != VisitCounter)
		{
			ThisVertexVisitCount = VisitCounter;

			const bool bIsRoot = Policy.IsRoot(ThisVertex, Mode);
			if (bIsRoot)
			{
				FGraphPath& Path = ProducedPaths.AddDefaulted_GetRef();
				Path.Vertices.Reserve(ChainDepth);
				Path.Vertices.Add(ThisVertex);
				ProducedPathsCount = 1;
			}
			else
			{
				for (FVertex ReferencedByVertex : Graph.EdgeLists[ThisVertex])
				{
					// For each of the referencers of this node, duplicate the current chain and continue processing
					if (ReferencedByVertex == GCObjReferencerVertex || VisitCounts.FindOrAdd(ReferencedByVertex) != VisitCounter)
					{
						int32 OldCount = ProducedPaths.Num();
						int32 NewCount = BuildGraphPathRecursive(Policy,
							Graph,
							VisitCounts,
							ReferencedByVertex,
							ProducedPaths,
							ChainDepth + 1,
							VisitCounter,
							Mode,
							GCObjReferencerVertex);
						for (int32 NewIndex = OldCount; NewIndex < (NewCount + OldCount); ++NewIndex)
						{
							FGraphPath& Path = ProducedPaths[NewIndex];
							Path.Vertices.Add(ThisVertex);
						}
						ProducedPathsCount += NewCount;
					}
				}
			}
		}

		return ProducedPathsCount;
	}

	template<typename PolicyType>
	void RemoveDuplicateGarbageChains(PolicyType& Policy, TArray<FGraphPath>& InOutGraphPaths, FVertex GCObjectReferencerVertex)
	{
		typedef TPair<FVertex, FVertex> FGarbageChainKey;
		TMap<FGarbageChainKey, FGraphPath*> GarbageChains;
		TArray<FGraphPath*> NoGarbageChains;

		for (FGraphPath& Path : InOutGraphPaths)
		{
			const int32 GarbageIndex = Path.Vertices.FindLastByPredicate([&Policy](FVertex Vertex) { return Policy.IsGarbage(Vertex); });
			if (GarbageIndex == INDEX_NONE)
			{
				NoGarbageChains.Add(&Path);
				continue;
			}

			FVertex Garbage = Path.Vertices[GarbageIndex];
			FVertex RootReferencer = Path.Vertices.Last();
			if (RootReferencer == GCObjectReferencerVertex && Path.Vertices.Num() > 2)
			{
				RootReferencer = Path.Vertices[Path.Vertices.Num() - 2];
			}

			FGarbageChainKey Key(Garbage, RootReferencer);
			if (FGraphPath* Existing = GarbageChains.FindRef(Key))
			{
				int32 ExistingLen = Existing->Vertices.Num() - Existing->Vertices.Find(Garbage);
				int32 NewLen = Path.Vertices.Num() - Path.Vertices.Find(Garbage);
				if (NewLen < ExistingLen)
				{
					GarbageChains.FindOrAdd(Key, &Path);
				}
			}
			else
			{
				GarbageChains.Add(Key, &Path);
			}
		}

		// Copy relevant paths out of InOutGraphPaths before ovewriting its contents.
		TArray<FGraphPath> OutPaths;
		OutPaths.Reserve(GarbageChains.Num() + NoGarbageChains.Num());
		for (const TPair<FGarbageChainKey, FGraphPath*>& Pair : GarbageChains)
		{
			OutPaths.Emplace(MoveTemp(*Pair.Value));
		}
		for (FGraphPath* Path : NoGarbageChains)
		{
			OutPaths.Emplace(MoveTemp(*Path));
		}
		InOutGraphPaths = MoveTemp(OutPaths);
	}

	void RemoveChainsWithDuplicatedRoots(TArray<FGraphPath>& InOutPaths, FVertex GCObjectReferencerVertex)
	{
		const auto Predicate = [GCObjectReferencerVertex](const FVertex& V) { return V != GCObjectReferencerVertex; };
		// This is going to be rather slow but it depends on the number of chains which shouldn't be too bad (usually)
		for (int32 FirstChainIndex = 0; FirstChainIndex < InOutPaths.Num(); ++FirstChainIndex)
		{
			const FGraphPath& FirstPath = InOutPaths[FirstChainIndex];
			const FVertex RootNode = FirstPath.Vertices[FirstPath.Vertices.FindLastByPredicate(Predicate)];
			for (int32 SecondChainIndex = InOutPaths.Num() - 1; SecondChainIndex > FirstChainIndex; --SecondChainIndex)
			{
				const FGraphPath& SecondPath = InOutPaths[SecondChainIndex];
				if (SecondPath.Vertices[SecondPath.Vertices.FindLastByPredicate(Predicate)] == RootNode)
				{
					InOutPaths.RemoveAt(SecondChainIndex);
				}
			}
		}
	}

	void RemoveDuplicatedChains(TArray<FGraphPath>& InOutPaths, FVertex GCObjectReferencerVertex)
	{
		// We consider chains identical if the direct referencer of the target node and the root node are identical
		typedef TTuple<FVertex, FVertex, FVertex> FChainDedupKey;
		TSparseMap<FChainDedupKey, FGraphPath*> UniqueChains;

		for (int32 ChainIndex = 0; ChainIndex < InOutPaths.Num(); ++ChainIndex)
		{
			FGraphPath& Path = InOutPaths[ChainIndex];
			FVertex DirectReferencer = Path.Vertices[1];

			// If the chain referencers is the GUObjectReferencer then go one down to find a more unique object to use for deduplication
			FVertex RootReferencer = Path.Vertices.Last();
			if (RootReferencer == GCObjectReferencerVertex && Path.Vertices.Num() > 2)
			{
				RootReferencer = Path.Vertices[Path.Vertices.Num() - 2];
			}

			FChainDedupKey ChainRootAndReferencer(Path.Target, DirectReferencer, RootReferencer);
			if (FGraphPath* ExistingChain = UniqueChains.FindRef(ChainRootAndReferencer))
			{
				if (ExistingChain->Vertices.Num() > Path.Vertices.Num())
				{
					UniqueChains[ChainRootAndReferencer] = &Path;
				}
			}
			else
			{
				UniqueChains.Add(ChainRootAndReferencer, &Path);
			}
		}

		TArray<FGraphPath> OutPaths;
		for (const TPair<FChainDedupKey, FGraphPath*>& Pair : UniqueChains)
		{
			OutPaths.Emplace(MoveTemp(*Pair.Value));
		}
		InOutPaths = MoveTemp(OutPaths);
	}

	/**
	 * Search a transpose object graph for paths from the target objects to the roots.
	 *
	 * @param Graph reverse object reference graph i.e. edges are from objects to the objects that reference them.
	 */
	template<typename PolicyType>
	void PerformSearch(PolicyType& Policy,
		TConstArrayView<typename PolicyType::ObjectType> ObjectsToFindReferencesTo,
		const UE::Graph::FGraph& Graph,
		EReferenceChainSearchMode Mode,
		TArray<FGraphPath>& OutGraphPaths)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::ReferenceChainSearch::PerformSearch);
		const bool bDirectOnly = !!(Mode & EReferenceChainSearchMode::Direct);
		const bool bExternalOnly = !!(Mode & EReferenceChainSearchMode::ExternalOnly);
		const bool bLongest = !!(Mode & EReferenceChainSearchMode::Longest);
		const bool bShortest = !!(Mode & EReferenceChainSearchMode::Shortest);
		const bool bShortestToGarbage = !!(Mode & EReferenceChainSearchMode::ShortestToGarbage);

		using ObjectType = typename PolicyType::ObjectType;
		if (bDirectOnly)
		{
			for (ObjectType TargetObject : ObjectsToFindReferencesTo)
			{
				FVertex TargetVertex = Policy.ObjectToVertex(TargetObject);
				TConstArrayView<FVertex> InEdges = Graph.EdgeLists[TargetVertex];
				for (FVertex ReferencerVertex : InEdges)
				{
					ObjectType Referencer = Policy.VertexToObject(ReferencerVertex);
					if (!bExternalOnly || !Policy.IsIn(TargetObject, Referencer))
					{
						FGraphPath Path;
						Path.Target = TargetVertex;
						Path.Vertices.Add(TargetVertex);
						Path.Vertices.Add(ReferencerVertex);
						OutGraphPaths.Emplace(MoveTemp(Path));
					}
				}
			}
		}
		else
		{
			int32 VisitCounter = 0;
			// Sparse mapping of visit count for relevant nodes
			TMap<FVertex, int32> VisitCounts;
			TArray<FGraphPath> TempChains;
			FVertex GCObjReferencerVertex = Policy.GetGCObjectReferencerVertex();
			for (ObjectType TargetObject : ObjectsToFindReferencesTo)
			{
				const FVertex TargetVertex = Policy.ObjectToVertex(TargetObject);
				TArray<FGraphPath> ThisTargetChains;
				for (FVertex ReferencedByVertex : Graph.EdgeLists[TargetVertex])
				{
					VisitCounts.FindOrAdd(TargetVertex) = ++VisitCounter;
					TempChains.Reset();
					const int32 MinChainDepth = 2;
					BuildGraphPathRecursive(Policy,
						Graph,
						VisitCounts,
						ReferencedByVertex,
						TempChains,
						MinChainDepth,
						VisitCounter,
						Mode,
						GCObjReferencerVertex);
					for (FGraphPath& Chain : TempChains)
					{
						Chain.Target = TargetVertex;
						Chain.Vertices.Add(TargetVertex);
						Algo::Reverse(Chain.Vertices);
					}

					if (bExternalOnly)
					{
						auto IsExternal = [](const FGraphPath& Chain) { return false; };
						TempChains.RemoveAllSwap([&IsExternal](const FGraphPath& Chain) { return !IsExternal(Chain); });
					}

					ThisTargetChains.Append(TempChains);
				}

				if (bLongest)
				{
					Algo::SortBy(
						ThisTargetChains,
						[](const FGraphPath& Chain) { return Chain.Vertices.Num(); },
						TGreater<int32>{});
				}
				else
				{
					Algo::SortBy(
						ThisTargetChains,
						[](const FGraphPath& Chain) { return Chain.Vertices.Num(); },
						TLess<int32>{});
				}

				if (bShortestToGarbage)
				{
					RemoveDuplicateGarbageChains(Policy, ThisTargetChains, GCObjReferencerVertex);
				}
				else if (bShortest || bLongest)
				{
					RemoveChainsWithDuplicatedRoots(ThisTargetChains, GCObjReferencerVertex);
				}

				OutGraphPaths.Append(MoveTemp(ThisTargetChains));
			}

			if (!bLongest && !bShortest && !bShortestToGarbage)
			{
				RemoveDuplicatedChains(OutGraphPaths, GCObjReferencerVertex);
			}
		}

		// TODO: Improvements to relevance of emitted paths using edge-list based algorithms
		//  Condense graph and search for paths through there, then pick arbitrary paths through cyclical components?
		//  Should we remove vertices that can't participate in any relevant chains to simplify condensation?
	}

	struct FReferenceInfoSearch : public TReferenceSearchBase<FReferenceInfoSearch>
	{
		using Super = TReferenceSearchBase<FReferenceInfoSearch>;
		const TMap<const UObject*, FGCObjectInfo*>& ObjectToInfoMap;

		TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>* ReferenceInfoMap;

		FReferenceInfoSearch(FPolicyUObjectHeap& InPolicy, const TMap<const UObject*, FGCObjectInfo*>& InObjectToInfoMap)
			: Super(InPolicy)
			, ObjectToInfoMap(InObjectToInfoMap)
		{
		}

		void Reset(TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>* InReferenceInfoMap)
		{
			ReferenceInfoMap = InReferenceInfoMap;
		}

		void HandleObjectReference(
			UE::GC::FWorkerContext& Context,
			int32 ThreadIndex, UObject* Object, UObject* ReferencingObject, UE::GC::FMemberId MemberId, UE::GC::EOrigin Origin)
		{
			if (!Object || Object == ReferencingObject) // Skip self-references just in case
			{
				return;
			}

			if (!ReferencingObject)
			{
				ReferencingObject = FGCObject::GGCObjectReferencer;
			}

			FReferenceChainSearch::FObjectReferenceInfo* RefInfo = ReferenceInfoMap->Find(Policy.ObjectToVertex(Object));
			if (!RefInfo)
			{
				return; // Irrelevant reference
			}
			if (RefInfo->Object != nullptr)
			{
				return; // Already found a ref from this object to this target
			}

			FGCObjectInfo* ObjectInfo = ObjectToInfoMap.FindRef(Object);
			*RefInfo = FReferenceChainSearch::FObjectReferenceInfo(ObjectInfo);
			if (MemberId != UE::GC::EMemberlessId::Collector)
			{
				RefInfo->ReferencerName = Context.SchemaStack->ToString();
				RefInfo->Type = FReferenceChainSearch::EReferenceType::Property;
			}
			else
			{
				RefInfo->Type = FReferenceChainSearch::EReferenceType::AddReferencedObjects;
				RefInfo->StackFrames.AddUninitialized(FReferenceChainSearch::FObjectReferenceInfo::MaxStackFrames);
				int32 NumStackFrames = FPlatformStackWalk::CaptureStackBackTrace(RefInfo->StackFrames.GetData(), RefInfo->StackFrames.Num());
				RefInfo->StackFrames.SetNum(NumStackFrames);

				if (FGCObject::GGCObjectReferencer && (!ReferencingObject || ReferencingObject == FGCObject::GGCObjectReferencer))
				{
					if (FGCObject::GGCObjectReferencer->GetReferencerName(Object, RefInfo->ReferencerName, true))
					{
					}
					else if (FGCObject* ReferencingFGCObject = FGCObject::GGCObjectReferencer->GetCurrentlySerializingObject())
					{
						RefInfo->ReferencerName = ReferencingFGCObject->GetReferencerName();
					}
				}
				else if (ReferencingObject)
				{
					RefInfo->ReferencerName = ReferencingObject->GetFullName();
				}
			}
		}
	};

	void FPolicyUObjectHeap::GatherReferenceInfo(TMap<FVertex, TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>>& InOutReferenceInfo)
	{
		FReferenceInfoSearch Search(*this, ObjectToInfoMap);
		for (TPair<FVertex, TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>>& SourcePair : InOutReferenceInfo)
		{
			FVertex SourceVertex = SourcePair.Key;
			UObject* Object = VertexToObject(SourceVertex);
			Search.Reset(&SourcePair.Value);
			Search.CollectReferencesFromObject(Object);
		}
	}

#if ENABLE_GC_HISTORY
	void FPolicyGCHistory::GatherReferenceInfo(TMap<FVertex, TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>>& InOutReferenceInfo)
	{
		FGCObjectInfo* GCObjReferencerInfo = Snapshot.ObjectToInfoMap.FindRef(FGCObject::GGCObjectReferencer);
		FVertex GCObjReferencerVertex = ObjectToVertex(GCObjReferencerInfo);
		for (TPair<FVertex, TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>>& SourcePair : InOutReferenceInfo)
		{
			FVertex SourceVertex = SourcePair.Key;
			TArray<FGCDirectReference>* DirectReferences = Snapshot.DirectReferences.FindRef(FReferenceToken(VertexToObject(SourceVertex)));
			if (DirectReferences)
			{
				for (TPair<FVertex, FReferenceChainSearch::FObjectReferenceInfo>& TargetPair : SourcePair.Value)
				{
					FVertex TargetVertex = TargetPair.Key;
					FGCObjectInfo* TargetInfo = VertexToObject(TargetVertex);
					// Linear search is not ideal, maybe build a map if we're looking for many references
					FGCDirectReference* RefInfo = DirectReferences->FindByPredicate(
						[TargetInfo](const FGCDirectReference& RefInfo) { return RefInfo.Reference == FReferenceToken(TargetInfo); });
					if (RefInfo)
					{
						FReferenceChainSearch::EReferenceType ReferenceType = FReferenceChainSearch::EReferenceType::Unknown;
						if (SourceVertex == GCObjReferencerVertex || RefInfo->ReferencerName.IsNone())
						{
							ReferenceType = FReferenceChainSearch::EReferenceType::AddReferencedObjects;
						}
						else
						{
							ReferenceType = FReferenceChainSearch::EReferenceType::Property;
						}
						TargetPair.Value = FReferenceChainSearch::FObjectReferenceInfo(TargetInfo, ReferenceType, RefInfo->ReferencerName.ToString());
					}
				}
			}
		}
	}
#endif // ENABLE_GC_HISTORY

	template<typename PolicyType>
	void PopulateReferenceInfo(PolicyType& Policy,
		const UE::Graph::FGraph& Graph,
		EReferenceChainSearchMode Mode,
		const TArray<FGraphPath>& GraphPaths,
		TArray<FReferenceChainSearch::FReferenceChain*>& OutReferenceChains)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::ReferenceChainSearch::PopulateReferenceInfo);
		// Construct the node/info types the public API expects
		for (const FGraphPath& Path : GraphPaths)
		{
			for (FVertex Vertex : Path.Vertices)
			{
				Policy.FindOrCreateObjectInfoAndGraphNode(Vertex);
			}
		}

		// Only gather callstacks/info for relevant edges
		TMap<FVertex, TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>> ReferenceInfoMap;
		for (const FGraphPath& Path : GraphPaths)
		{
			// From rooted object to target object
			for (int32 i = Path.Vertices.Num() - 1; i > 0; --i)
			{
				FVertex From = Path.Vertices[i];
				FVertex To = Path.Vertices[i - 1];
				ReferenceInfoMap.FindOrAdd(From).FindOrAdd(To);
			}
		}

		// Re-gather references for each object we care about to gather detailed info
		Policy.GatherReferenceInfo(ReferenceInfoMap);
		for (auto It = ReferenceInfoMap.CreateIterator(); It; ++It)
		{
			FVertex SourceVertex = It.Key();
			FReferenceChainSearch::FGraphNode* SourceNode = Policy.VertexToGraphNode(SourceVertex);
			const TMap<FVertex, FReferenceChainSearch::FObjectReferenceInfo>& ReferencedObjects = It.Value();
			for (const TPair<FVertex, FReferenceChainSearch::FObjectReferenceInfo>& RefereePair : ReferencedObjects)
			{
				FVertex ToVertex = RefereePair.Key;
				const FReferenceChainSearch::FObjectReferenceInfo& Info = RefereePair.Value;
				if (ensureAlwaysMsgf(Info.Object,
						TEXT("Failed to find expected reference from %s to %s"),
						*Policy.VertexToObject(SourceVertex)->GetPathName(),
						*Policy.VertexToObject(ToVertex)->GetPathName()))
				{
					FReferenceChainSearch::FGraphNode* TargetNode = Policy.VertexToGraphNode(ToVertex);
					TargetNode->ReferencedByObjects.Add(SourceNode);
					SourceNode->ReferencedObjects.Emplace(
						FReferenceChainSearch::FNodeReferenceInfo(TargetNode, Info.Type, Info.ReferencerName, Info.StackFrames));
				}
			}
		}

		for (const FGraphPath& Path : GraphPaths)
		{
			FReferenceChainSearch::FGraphNode* TargetNode = Policy.VertexToGraphNode(Path.Target);
			TArray<FReferenceChainSearch::FGraphNode*> Nodes;
			Nodes.Reserve(Path.Vertices.Num());
			TArray<const FReferenceChainSearch::FNodeReferenceInfo*> ReferenceInfos;
			ReferenceInfos.Reserve(Path.Vertices.Num());

			for (FVertex Vertex : Path.Vertices)
			{
				Nodes.Add(Policy.VertexToGraphNode(Vertex));
			}

			ReferenceInfos.SetNumUninitialized(Path.Vertices.Num());
			ReferenceInfos[0] = nullptr; // Target references nothing
			for (int32 i = Path.Vertices.Num() - 1; i > 0; --i)
			{
				const FReferenceChainSearch::FNodeReferenceInfo* Info =
					Nodes[i]->ReferencedObjects.Find(FReferenceChainSearch::FNodeReferenceInfo(Nodes[i - 1]));
				ReferenceInfos[i] = Info;
			}
			FReferenceChainSearch::FReferenceChain* Chain =
				new FReferenceChainSearch::FReferenceChain(TargetNode, MoveTemp(Nodes), MoveTemp(ReferenceInfos));
			OutReferenceChains.Add(Chain);
		}
	}
} // namespace UE::ReferenceChainSearch

FString FReferenceChainSearch::GetObjectFlags(const FGCObjectInfo& InObject)
{
	using namespace UE::GC::Private;

	FString Flags;

	if (!InObject.IsDisregardForGC())
	{
		if (!InObject.HasAnyInternalFlags(EInternalObjectFlags_ReachabilityFlags))
		{
			Flags += TEXT("(Error: No reachability flag) ");
		}
	}
	else if (InObject.HasAnyInternalFlags(FGCFlags::GetReachableFlagValue_ForGC()))
	{
		Flags += TEXT("(Error: Reachable but NeverGCed) ");
	}

	if (InObject.HasAnyInternalFlags(FGCFlags::GetMaybeUnreachableFlagValue_ForGC()))
	{
		Flags += FString::Printf(TEXT("(MaybeUnreachable<%d>) "), uint32{EnumToUnderlyingType(FGCFlags::GetMaybeUnreachableFlagValue_ForGC())});
	}

	if (InObject.HasAnyInternalFlags(EInternalObjectFlags::Unreachable))
	{
		Flags += TEXT("(Unreachable) ");
	}

	if (InObject.GetRefCount())
	{
		Flags += FString::Printf(TEXT("(refcounted<%d>) "), InObject.GetRefCount());
	}

	if (InObject.IsRooted())
	{
		Flags += TEXT("(root) ");
	}

	CA_SUPPRESS(6011)
	if (InObject.IsNative())
	{
		Flags += TEXT("(native) ");
	}

	if (InObject.HasAnyInternalFlags(EInternalObjectFlags::Garbage))
	{
		Flags += TEXT("(Garbage) ");
	}

	if (InObject.HasAnyFlags(RF_Standalone))
	{
		Flags += TEXT("(standalone) ");
	}

	if (InObject.HasAnyInternalFlags(EInternalObjectFlags::Async))
	{
		Flags += TEXT("(async) ");
	}

	if (InObject.HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
	{
		Flags += TEXT("(asyncloading) ");
	}

	if (InObject.HasAnyInternalFlags(EInternalObjectFlags::LoaderImport))
	{
		Flags += TEXT("(loaderimport) ");
	}

	if (InObject.IsDisregardForGC())
	{
		Flags += TEXT("(NeverGCed) ");
	}

	if (InObject.HasAnyInternalFlags(EInternalObjectFlags::ClusterRoot))
	{
		Flags += TEXT("(ClusterRoot) ");
	}

	if (InObject.GetOwnerIndex() > 0)
	{
		Flags += TEXT("(Clustered) ");
	}
	return Flags;
}

static void ConvertStackFramesToCallstack(
	const uint64* StackFrames,
	int32 NumStackFrames,
	int32 Indent,
	FOutputDevice& Out,
	TMap<uint64, FString>& Cache,
	ELogVerbosity::Type VerbosityForPrint)
{
	// Convert the stack trace to text
	FStringView View;
	for (int32 Idx = 0; Idx < NumStackFrames; Idx++)
	{
		if (FString* Existing = Cache.Find(StackFrames[Idx]))
		{
			View = FStringView(**Existing, Existing->Len());
		}
		else
		{
			ANSICHAR Buffer[1024];
			Buffer[0] = '\0';
			FPlatformStackWalk::ProgramCounterToHumanReadableString(Idx, StackFrames[Idx], Buffer, sizeof(Buffer));
			View = Cache.FindOrAdd(StackFrames[Idx], FString(Buffer));
		}

		if (View.Contains(TEXTVIEW("TFastReferenceCollector")))
		{
			break;
		}

		if (!View.Contains(TEXTVIEW("FWindowsPlatformStackWalk")) && !View.Contains(TEXTVIEW("FDirectReferenceProcessor"))
			&& !View.Contains(TEXTVIEW("FReferenceInfoProcessor")))
		{
			if (int32 Index = View.Find(TEXTVIEW("!")); Index != INDEX_NONE)
			{
				View = View.RightChop(Index + 1);
			}
			Out.Logf(VerbosityForPrint, TEXT("%*s   ^ %.*s"), Indent, TEXT(""), View.Len(), View.GetData());
		}
	}
}

void FReferenceChainSearch::DumpChain(FReferenceChainSearch::FReferenceChain* Chain,
	TFunctionRef<bool(FCallbackParams& Params)> ReferenceCallback,
	TMap<uint64, FString>& CallstackCache,
	FOutputDevice& Out,
	ELogVerbosity::Type InVerbosityForPrint)
{
	if (Chain->Num())
	{
		bool bPostCallbackContinue = true;
		const int32 RootIndex = Chain->Num() - 1;
		const FNodeReferenceInfo* ReferenceInfo = Chain->GetReferenceInfo(RootIndex);
		FGCObjectInfo* ReferencerObject = Chain->GetNode(RootIndex)->ObjectInfo;
		{
			FCallbackParams Params;
			Params.Referencer = nullptr;
			Params.Object = ReferencerObject;
			Params.ReferenceInfo = nullptr;
			Params.Indent = FMath::Min<int32>(TCStringSpcHelper<TCHAR>::MAX_SPACES, Chain->Num() - RootIndex);
			Params.Out = &Out;

			Out.Logf(InVerbosityForPrint, TEXT("%s%s %s"),
				FCString::Spc(Params.Indent),
				*GetObjectFlags(*ReferencerObject),
				*ReferencerObject->GetFullName());

			bPostCallbackContinue = ReferenceCallback(Params);
		}

		// Roots are at the end so iterate from the last to the first node
		for (int32 NodeIndex = RootIndex - 1; NodeIndex >= 0 && bPostCallbackContinue; --NodeIndex)
		{
			FGCObjectInfo* Object = Chain->GetNode(NodeIndex)->ObjectInfo;

			FCallbackParams Params;
			Params.Referencer = ReferencerObject;
			Params.Object = Object;
			Params.ReferenceInfo = ReferenceInfo;
			Params.Indent = FMath::Min<int32>(TCStringSpcHelper<TCHAR>::MAX_SPACES, Chain->Num() - NodeIndex - 1);
			Params.Out = &Out;

			if (ReferenceInfo && ReferenceInfo->Type == EReferenceType::Property)
			{
				FString ReferencingPropertyName;
				UClass* ReferencerClass = Cast<UClass>(ReferencerObject->GetClass()->TryResolveObject());
				TArray<FProperty*> ReferencingProperties;

				if (ReferencerClass && UE::GC::FPropertyStack::ConvertPathToProperties(ReferencerClass, ReferenceInfo->ReferencerName, ReferencingProperties))
				{
					FProperty* InnermostProperty = ReferencingProperties.Last();
					FProperty* OutermostProperty = ReferencingProperties[0];

					ReferencingPropertyName = FString::Printf(TEXT("%s %s%s::%s"),
						*InnermostProperty->GetCPPType(),
						OutermostProperty->GetOwnerClass()->GetPrefixCPP(),
						*OutermostProperty->GetOwnerClass()->GetName(),
						*ReferenceInfo->ReferencerName);
				}
				else
				{
					// Handle base UObject referencer info (it's only exposed to the GC token stream and not to the reflection system)
					static const FStringView ClassPropertyName = TEXTVIEW("Class");
					static const FStringView OuterPropertyName = TEXTVIEW("Outer");

					FString ClassName;
					if (ReferenceInfo->ReferencerName == ClassPropertyName || ReferenceInfo->ReferencerName == OuterPropertyName)
					{
						ClassName = TEXT("UObject");
					}
					else if (ReferencerClass)
					{
						// Use the native class name when possible
						ClassName = ReferencerClass->GetPrefixCPP();
						ClassName += ReferencerClass->GetName();
					}
					else
					{
						// Revert to the internal class name if not
						ClassName = ReferencerObject->GetClassName();
					}
					ReferencingPropertyName = FString::Printf(TEXT("UObject* %s::%s"), *ClassName, *ReferenceInfo->ReferencerName);
				}

				Out.Logf(InVerbosityForPrint, TEXT("%s-> %s = %s %s"),
					FCString::Spc(Params.Indent),
					*ReferencingPropertyName,
					*GetObjectFlags(*Object),
					*Object->GetFullName());
			}
			else if (ReferenceInfo && ReferenceInfo->Type == EReferenceType::AddReferencedObjects)
			{
				FString UObjectOrGCObjectName;
				if (ReferenceInfo->ReferencerName.IsEmpty())
				{
					UClass* ReferencerClass = Cast<UClass>(ReferencerObject->GetClass()->TryResolveObject());
					if (ReferencerClass)
					{
						UObjectOrGCObjectName = ReferencerClass->GetPrefixCPP();
						UObjectOrGCObjectName += ReferencerClass->GetName();
					}
					else
					{
						UObjectOrGCObjectName += ReferencerObject->GetClassName();
					}
				}
				else
				{
					UObjectOrGCObjectName = ReferenceInfo->ReferencerName;
				}

				Out.Logf(InVerbosityForPrint, TEXT("%s-> %s::AddReferencedObjects(%s %s)"),
					FCString::Spc(Params.Indent),
					*UObjectOrGCObjectName,
					*GetObjectFlags(*Object),
					*Object->GetFullName());

				if (ReferenceInfo->StackFrames.Num())
				{
					ConvertStackFramesToCallstack(ReferenceInfo->StackFrames.GetData(),
						ReferenceInfo->StackFrames.Num(),
						Params.Indent,
						Out,
						CallstackCache,
						InVerbosityForPrint);
				}
			}
			else if (ReferenceInfo && ReferenceInfo->Type == EReferenceType::OuterChain)
			{
				Out.Logf(InVerbosityForPrint, TEXT("%s-> %s = %s %s"),
					FCString::Spc(Params.Indent),
					TEXT("Outer Chain"),
					*GetObjectFlags(*Object),
					*Object->GetFullName());
			}
			else
			{
				Out.Logf(InVerbosityForPrint, TEXT("%s-> %s = %s %s"),
					FCString::Spc(Params.Indent),
					TEXT("UNKNOWN"),
					*GetObjectFlags(*Object),
					*Object->GetFullName());
			}

			bPostCallbackContinue = ReferenceCallback(Params);

			ReferencerObject = Object;
			ReferenceInfo = Chain->GetReferenceInfo(NodeIndex);
		}
		Out.Logf(InVerbosityForPrint, TEXT("  "));
	}
}

bool FReferenceChainSearch::FReferenceChain::IsExternal() const
{
	if (Nodes.Num() > 1)
	{
		// Reference is external if the root (the last node) is not in the first node (target)
		return !Nodes.Last()->ObjectInfo->IsIn(Nodes[0]->ObjectInfo);
	}
	else
	{
		return false;
	}
}

FReferenceChainSearch::FReferenceChainSearch(UObject* InObjectToFindReferencesTo,
	EReferenceChainSearchMode Mode /*= EReferenceChainSearchMode::PrintResults*/,
	ELogVerbosity::Type InVerbosityForPrint /* = ELogVerbosity::Log */)
	: FReferenceChainSearch(TConstArrayView<UObject*>(&InObjectToFindReferencesTo, 1), Mode, InVerbosityForPrint)
{
}

FReferenceChainSearch::FReferenceChainSearch(TConstArrayView<UObject*> InObjectsToFindReferencesTo,
	EReferenceChainSearchMode Mode /*= EReferenceChainSearchMode::PrintResults*/,
	ELogVerbosity::Type InVerbosityForPrint /* = ELogVerbosity::Log */)
	: SearchMode(Mode)
	, VerbosityForPrint(InVerbosityForPrint)
{
	FSlowHeartBeatScope DisableHangDetection; // This function can be very slow

	// Lock the global array so that nothing can add UObjects while we're iterating over it
	GUObjectArray.LockInternalArray();

	for (const UObject* Object : InObjectsToFindReferencesTo)
	{
		ObjectInfosToFindReferencesTo.Add(FGCObjectInfo::FindOrAddInfoHelper(Object, ObjectToInfoMap));
	}

	UE::ReferenceChainSearch::FPolicyUObjectHeap Policy(ObjectToInfoMap, AllNodes, Mode);

	UE::Graph::FGraph ReferenceGraph;
	UE::ReferenceChainSearch::PerformInitialGatherFromLiveUObjectHeap(Policy, InObjectsToFindReferencesTo, SearchMode, ReferenceGraph);

	TArray<UE::ReferenceChainSearch::FGraphPath> Paths;
	UE::ReferenceChainSearch::PerformSearch(Policy, InObjectsToFindReferencesTo, ReferenceGraph, Mode, Paths);
	UE::ReferenceChainSearch::PopulateReferenceInfo(Policy, ReferenceGraph, Mode, Paths, ReferenceChains);

	if (!!(Mode & (EReferenceChainSearchMode::PrintResults|EReferenceChainSearchMode::PrintAllResults)))
	{
		PrintResults(!!(Mode & EReferenceChainSearchMode::PrintAllResults), /*TargetObject*/nullptr, VerbosityForPrint);
	}

	UE_LOGF(LogReferenceChain, Display, "Post-search memory usage: %.2f", static_cast<double>(ReferenceGraph.GetAllocatedSize() + Paths.GetAllocatedSize() + GetAllocatedSize()) / 1024.0 / 1024.0);

	GUObjectArray.UnlockInternalArray();
}

FReferenceChainSearch::FReferenceChainSearch(EReferenceChainSearchMode Mode)
	: SearchMode(Mode)
{
}

FReferenceChainSearch::~FReferenceChainSearch()
{
	Cleanup();
}

int64 FReferenceChainSearch::GetAllocatedSize() const
{
	int64 Size = 0;
	// Size += ObjectInfosToFindReferencesTo.GetAllocatedSize();
	Size += ReferenceChains.GetAllocatedSize();
	for (const FReferenceChain* Chain : ReferenceChains)
	{
		Size += Chain->GetAllocatedSize();
	}
	Size += AllNodes.GetAllocatedSize();
	Size += AllNodes.Num() * sizeof(FGCObjectInfo); // GC object infos are heap allocated and owned by this structure
	for (const TPair<FGCObjectInfo*, FGraphNode*> Pair : AllNodes)
	{
		// FGCObjectInfo makes no allocations
		Size += Pair.Value->GetAllocatedSize();
	}
	Size += ObjectToInfoMap.GetAllocatedSize();
	return Size;
}

#if ENABLE_GC_HISTORY
void FReferenceChainSearch::PerformSearchFromGCSnapshot(UObject* InObjectToFindReferencesTo, FGCSnapshot& InSnapshot)
{
	PerformSearchFromGCSnapshot(MakeArrayView(&InObjectToFindReferencesTo, 1), InSnapshot);
}

void FReferenceChainSearch::PerformSearchFromGCSnapshot(TConstArrayView<UObject*> InObjectsToFindReferencesTo, FGCSnapshot& InSnapshot)
{
	FSlowHeartBeatScope DisableHangDetection; // This function can be very slow

	Cleanup();

	UE::ReferenceChainSearch::FPolicyGCHistory Policy(InSnapshot, AllNodes);

	for (const UObject* Obj : InObjectsToFindReferencesTo)
	{
		FGCObjectInfo* ObjInfo = InSnapshot.ObjectToInfoMap.FindRef(Obj);
		if (ObjInfo)
		{
			ObjectInfosToFindReferencesTo.Add(ObjInfo);
		}
		else
		{
			UE_LOGF(LogReferenceChain, Warning, "Target object %ls was not present in GC snapshot.", *Obj->GetPathName());
		}
	}

	for (TPair<FReferenceToken, TArray<FGCDirectReference>*>& Pair : InSnapshot.DirectReferences)
	{
		for (const FGCDirectReference& RefInfo : *Pair.Value)
		{
			if (RefInfo.Reference.IsGCObjectInfo() && ObjectInfosToFindReferencesTo.Contains(RefInfo.Reference.AsGCObjectInfo()))
			{
				UE_LOGF(LogReferenceChain,
					Display,
					"Direct ref in GC history from %ls to %ls",
					*Pair.Key.GetDescription(),
					*RefInfo.Reference.GetDescription());
			}
		}
	}

	UE::Graph::FGraph ReferenceGraph;
	UE::ReferenceChainSearch::PerformInitialGatherFromGCHistory(Policy, ReferenceGraph);

	TArray<UE::ReferenceChainSearch::FGraphPath> Paths;
	UE::ReferenceChainSearch::PerformSearch(Policy, ObjectInfosToFindReferencesTo, ReferenceGraph, SearchMode, Paths);
	UE::ReferenceChainSearch::PopulateReferenceInfo(Policy, ReferenceGraph, SearchMode, Paths, ReferenceChains);

	if (!!(SearchMode & (EReferenceChainSearchMode::PrintResults | EReferenceChainSearchMode::PrintAllResults)))
	{
		PrintResults(!!(SearchMode & EReferenceChainSearchMode::PrintAllResults), /*TargetObject*/nullptr, VerbosityForPrint);
	}
}
#endif // ENABLE_GC_HISTORY

void FReferenceChainSearch::SetVerbosityForPrint(ELogVerbosity::Type Verbosity)
{
	VerbosityForPrint = Verbosity;
}

int32 FReferenceChainSearch::PrintResults(bool bDumpAllChains /*= false*/, UObject* TargetObject /*= nullptr*/, ELogVerbosity::Type InVerbosityForPrint /*= ELogVerbosity::Log*/) const
{
	return PrintResults([](FCallbackParams& Params) { return true; }, bDumpAllChains, TargetObject, InVerbosityForPrint);
}

static TAutoConsoleVariable CVarMaxReferenceChainsToPrint(
	TEXT("gc.MaxReferenceChainsToPrint"),
	100,
	TEXT("Sets the limit of how many reference chains ReferenceChainSearch utility will print"),
	ECVF_Default
);

int32 FReferenceChainSearch::PrintResults(TFunctionRef<bool(FCallbackParams& Params)> ReferenceCallback, bool bDumpAllChains /*= false*/, UObject* TargetObject /*= nullptr*/, ELogVerbosity::Type InVerbosityForPrint /*= ELogVerbosity::Log*/) const
{
	FSlowHeartBeatScope DisableHangDetection; // This function can be very slow

	const int32 MaxChainsToPrint = CVarMaxReferenceChainsToPrint.GetValueOnAnyThread();
	int32 NumPrintedChains = 0;

	TMap<uint64, FString> CallstackCache;
	for (FReferenceChain* Chain : ReferenceChains)
	{
		if (TargetObject != nullptr && Chain->TargetNode->ObjectInfo->TryResolveObject() != TargetObject)
		{
			continue;
		}

		if (bDumpAllChains || NumPrintedChains < MaxChainsToPrint)
		{
			DumpChain(Chain, ReferenceCallback, CallstackCache, *GWarn, InVerbosityForPrint);
			NumPrintedChains++;
		}
		else
		{
#if !NO_LOGGING
			GWarn->CategorizedLogf(LogReferenceChain.GetCategoryName(), InVerbosityForPrint,
				TEXT("Referenced by %d more reference chain(s)."), ReferenceChains.Num() - NumPrintedChains);
#endif
			break;
		}
	}

	if (NumPrintedChains == 0)
	{
		auto LogUnreachableObject = [InVerbosityForPrint, this](const FGCObjectInfo& ObjInfo) {
			if (ObjInfo.HasAnyInternalFlags(EInternalObjectFlags_RootFlags))
			{
#if !NO_LOGGING
				GWarn->CategorizedLogf(LogReferenceChain.GetCategoryName(), InVerbosityForPrint,
					TEXT("%s%s is not currently reachable but it does have some of EInternalObjectFlags_RootFlags set."), *GetObjectFlags(ObjInfo), *ObjInfo.GetFullName());
#endif
			}
			else if (ObjInfo.HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS))
			{
#if !NO_LOGGING
				GWarn->CategorizedLogf(LogReferenceChain.GetCategoryName(), InVerbosityForPrint,
					TEXT("%s%s is not currently reachable but it does have some of GARBAGE_COLLECTION_KEEPFLAGS set."), *GetObjectFlags(ObjInfo), *ObjInfo.GetFullName());
#endif
			}
			else if (ObjInfo.GetRefCount() != 0)
			{
#if !NO_LOGGING
				GWarn->CategorizedLogf(LogReferenceChain.GetCategoryName(), InVerbosityForPrint,
					TEXT("%s%s is not currently reachable but it does have %d refcount set."), *GetObjectFlags(ObjInfo), *ObjInfo.GetFullName(), ObjInfo.GetRefCount());
#endif
			}
			else
			{
#if !NO_LOGGING
				GWarn->CategorizedLogf(LogReferenceChain.GetCategoryName(), InVerbosityForPrint,
					TEXT("%s%s is not currently reachable. Try using GC history to debug transient leaks with 'gc.historysize 1'"), *GetObjectFlags(ObjInfo), *ObjInfo.GetFullName());
#endif
			}
		};
		if (TargetObject)
		{
			FGCObjectInfo* ObjInfo = FGCObjectInfo::FindOrAddInfoHelper(TargetObject, const_cast<TMap<const UObject*, FGCObjectInfo*>&>(ObjectToInfoMap));
			check(ObjInfo);
			LogUnreachableObject(*ObjInfo);
		}
		else
		{
			for (FGCObjectInfo* ObjInfo : ObjectInfosToFindReferencesTo)
			{
				LogUnreachableObject(*ObjInfo);
			}
		}
	}

	return NumPrintedChains;
}

FString FReferenceChainSearch::GetRootPath(UObject* TargetObject /*= nullptr*/) const
{
	return GetRootPath([](FCallbackParams& Params) { return true; }, TargetObject);
}

FString FReferenceChainSearch::GetRootPath(TFunctionRef<bool(FCallbackParams& Params)> ReferenceCallback, UObject* TargetObject /*= nullptr*/) const
{
	FReferenceChain* Chain = nullptr;
	if (ReferenceChains.Num())
	{
		Chain = ReferenceChains[0];
		if( TargetObject )
		{
			int32 Index = ReferenceChains.IndexOfByPredicate([=](const FReferenceChain* Chain){return Chain->TargetNode->ObjectInfo->TryResolveObject() == TargetObject; });
			Chain = Index != INDEX_NONE ? ReferenceChains[Index] : nullptr;
		}
	}

	if (Chain)
	{
		FStringOutputDevice OutString;
		OutString.SetAutoEmitLineTerminator(true);
		TMap<uint64, FString> CallstackCache;
		DumpChain(Chain, ReferenceCallback, CallstackCache, OutString, VerbosityForPrint);
		return MoveTemp(OutString);
	}
	else
	{
		if (TargetObject)
		{
			FGCObjectInfo ObjectInfo(TargetObject);
			return FString::Printf(TEXT("%s%s is not currently reachable."),
				*GetObjectFlags(ObjectInfo),
				*ObjectInfo.GetFullName()
			);
		}
		else
		{
			return TEXT("No target objects are currently reachable.");
		}
	}
}

void FReferenceChainSearch::Cleanup()
{
	ObjectInfosToFindReferencesTo.Empty();
	for (int32 ChainIndex = 0; ChainIndex < ReferenceChains.Num(); ++ChainIndex)
	{
		delete ReferenceChains[ChainIndex];
	}
	ReferenceChains.Empty();

	for (TPair<FGCObjectInfo*, FGraphNode*>& ObjectNodePair : AllNodes)
	{
		delete ObjectNodePair.Value;
	}
	AllNodes.Empty();

	for (TPair<const UObject*, FGCObjectInfo*>& ObjectToInfoPair : ObjectToInfoMap)
	{
		delete ObjectToInfoPair.Value;
	}
	ObjectToInfoMap.Empty();
}

static FORCEINLINE bool HasGarbageCollectionKeepFlags(FGCObjectInfo* ObjectInfo)
{
	return ObjectInfo && (ObjectInfo->HasAnyFlags(GARBAGE_COLLECTION_KEEPFLAGS) || ObjectInfo->HasAnyInternalFlags(EInternalObjectFlags_RootFlags) || ObjectInfo->GetRefCount() != 0);
}

static bool PrintStaleReferenceChainsAndFindReferencingObjects(UObject* ObjectToFindReferencesTo, FReferenceChainSearch& RefChainSearch, FGCObjectInfo*& OutGarbageObject, FGCObjectInfo*& OutReferencingObject, ELogVerbosity::Type Verbosity)
{
#if !NO_LOGGING
	if (Verbosity != ELogVerbosity::NoLogging)
	{
		FMsg::Logf(__FILE__, __LINE__, LogLoad.GetCategoryName(), Verbosity, TEXT("Printing reference chains leading to %s: "), *ObjectToFindReferencesTo->GetFullName());
	}
#endif
	return RefChainSearch.PrintResults([&ObjectToFindReferencesTo, &OutGarbageObject, &OutReferencingObject](FReferenceChainSearch::FCallbackParams& Params)
		{
			check(Params.Object);
			if (!Params.Object->IsValid())
			{
				// We may find many chains that lead to the leak but for brevity we only report the first one in the fatal error below
				if (!OutGarbageObject)
				{
					OutGarbageObject = Params.Object;
					OutReferencingObject = Params.Referencer;
				}
				return true;
			}
			else
			{
				UObject* ResolvedObject = Params.Object->TryResolveObject();
				if (ResolvedObject != ObjectToFindReferencesTo)
				{
					// Keep track of the last referencing object, we may need it if there's no garbage referencers
					OutReferencingObject = Params.Object;
				}
				return true;
			}
		}, false, ObjectToFindReferencesTo, Verbosity) != 0;
}

static FString GetPathToStaleObjectReferencer(UObject* ObjectToFindReferencesTo, FReferenceChainSearch& RefChainSearch)
{
	bool bFirstGarbageObjectFound = false;
	FString PathToCulprit = RefChainSearch.GetRootPath([&bFirstGarbageObjectFound, ObjectToFindReferencesTo](FReferenceChainSearch::FCallbackParams& Params)
		{
			check(Params.Object);
			// Mark the first garbage reference or the target object reference as the culprit
			if ((!Params.Object->IsValid() || Params.Object->TryResolveObject() == ObjectToFindReferencesTo) && !bFirstGarbageObjectFound)
			{
				bFirstGarbageObjectFound = true;
				check(Params.Out);
				Params.Out->Logf(ELogVerbosity::Log, TEXT("%s    ^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^^"), FCString::Spc(Params.Indent));
				Params.Out->Logf(ELogVerbosity::Log, TEXT("%s    ^ This reference is preventing the old %s from being GC'd ^"), FCString::Spc(Params.Indent), *ObjectToFindReferencesTo->GetClass()->GetName());
			}
			return true;
		}, ObjectToFindReferencesTo);
	return PathToCulprit;
}

FString FReferenceChainSearch::FindAndPrintStaleReferencesToObject(UObject* ObjectToFindReferencesTo, EPrintStaleReferencesOptions Options)
{
	return FindAndPrintStaleReferencesToObjects(TConstArrayView<UObject*>(&ObjectToFindReferencesTo, 1), Options)[0];
}

TArray<FString> FReferenceChainSearch::FindAndPrintStaleReferencesToObjects(TConstArrayView<UObject*> ObjectsToFindReferencesTo, EPrintStaleReferencesOptions Options)
{
	TArray<FString> Paths;
	checkf(ObjectsToFindReferencesTo.Num() > 0, TEXT("ObjectsToFindReferencesTo cannot be empty"));
	bool bAnyGarbage = false;
	for (UObject* Obj : ObjectsToFindReferencesTo)
	{
		checkf(Obj, TEXT("Object to find references to cannot be null"));
		bAnyGarbage = bAnyGarbage || Obj->HasAnyInternalFlags(EInternalObjectFlags::Garbage);
	}

	ELogVerbosity::Type Verbosity = ELogVerbosity::NoLogging;
	switch (Options & EPrintStaleReferencesOptions::VerbosityMask)
	{
	case EPrintStaleReferencesOptions::None:
		Verbosity = ELogVerbosity::NoLogging;
		break;
	case EPrintStaleReferencesOptions::Log:
		Verbosity = ELogVerbosity::Log;
		break;
	case EPrintStaleReferencesOptions::Display:
		Verbosity = ELogVerbosity::Display;
		break;
	case EPrintStaleReferencesOptions::Warning:
		Verbosity = ELogVerbosity::Warning;
		break;
	case EPrintStaleReferencesOptions::Error:
	case EPrintStaleReferencesOptions::Fatal: // Use verbosity error for reference chains so we can log all errors before crashing 
	default:
		Verbosity = ELogVerbosity::Error;
		break;
	}

	UE_LOGF(LogLoad, Log, "Beginning reference chain search...");
	for (UObject* Obj : ObjectsToFindReferencesTo)
	{
		UE_LOGF(LogLoad, Log, " - %ls", *Obj->GetFullName());
	}

	EReferenceChainSearchMode SearchMode = EReferenceChainSearchMode::GCOnly;
	if (!!(Options & EPrintStaleReferencesOptions::Minimal))
	{
		SearchMode |= EReferenceChainSearchMode::Minimal;
	}
	else
	{
		SearchMode |= bAnyGarbage ? EReferenceChainSearchMode::ShortestToGarbage : EReferenceChainSearchMode::Shortest;
	}

	FReferenceChainSearch RefChainSearch(ObjectsToFindReferencesTo, SearchMode);
#if ENABLE_GC_HISTORY
	// If we have the full history already, don't only search for direct referencers 
	FReferenceChainSearch HistorySearch(EReferenceChainSearchMode::ShortestToGarbage); // we may not need it but it has to live in this scope so that FGCObjectInfos are being kept alive
#endif

	for (UObject* ObjectToFindReferencesTo : ObjectsToFindReferencesTo)
	{
		FGCObjectInfo* OutGarbageObject = nullptr;
		FGCObjectInfo* OutReferencingObject = nullptr;

		bool bReferenceChainFound = PrintStaleReferenceChainsAndFindReferencingObjects(ObjectToFindReferencesTo, RefChainSearch, OutGarbageObject, OutReferencingObject, Verbosity);

		FString PathToCulprit;
		FString GarbageErrorMessage;
		if (bReferenceChainFound)
		{
			PathToCulprit = GetPathToStaleObjectReferencer(ObjectToFindReferencesTo, RefChainSearch);
			FString GarbageObjectName = OutGarbageObject ? OutGarbageObject->GetFullName() : ObjectToFindReferencesTo->GetFullName();
			checkf(OutReferencingObject || HasGarbageCollectionKeepFlags(OutGarbageObject ? OutGarbageObject : OutReferencingObject), TEXT("No object referencing %s found even though we have a valid reference chain"), *ObjectToFindReferencesTo->GetPathName());
			GarbageErrorMessage = FString::Printf(TEXT("Object %s is being referenced by %s"), *GarbageObjectName,
				OutReferencingObject ? *OutReferencingObject->GetFullName() : (HasGarbageCollectionKeepFlags(OutGarbageObject) ? TEXT("GarbageCollectionKeepFlags") : TEXT("NULL")));
		}
#if ENABLE_GC_HISTORY
		else
		{
			FGCSnapshot* LastGCSnapshot = FGCHistory::Get().GetLastSnapshot();
			if (LastGCSnapshot)
			{
				UE_LOGF(LogLoad, Log, "Looking for references to %ls in the last GC run history...", *ObjectToFindReferencesTo->GetFullName());

				HistorySearch.PerformSearchFromGCSnapshot(ObjectsToFindReferencesTo, *LastGCSnapshot);

				OutGarbageObject = nullptr;
				OutReferencingObject = nullptr;
				bReferenceChainFound = PrintStaleReferenceChainsAndFindReferencingObjects(ObjectToFindReferencesTo, HistorySearch, OutGarbageObject, OutReferencingObject, Verbosity);
				if (bReferenceChainFound)
				{
					bReferenceChainFound = true;
					PathToCulprit = GetPathToStaleObjectReferencer(ObjectToFindReferencesTo, HistorySearch);
					FString GarbageObjectName = OutGarbageObject ? OutGarbageObject->GetFullName() : ObjectToFindReferencesTo->GetFullName();
					checkf(OutReferencingObject || HasGarbageCollectionKeepFlags(OutGarbageObject ? OutGarbageObject : OutReferencingObject), TEXT("No object referencing %s found even though we have a valid reference chain"), *ObjectToFindReferencesTo->GetPathName());
					GarbageErrorMessage = FString::Printf(TEXT("Garbage object %s was previously being referenced by %s"), *GarbageObjectName, OutReferencingObject ? *OutReferencingObject->GetFullName() : TEXT("NULL"));
				}
			}
		}
#endif // ENABLE_GC_HISTORY

		if (!bReferenceChainFound)
		{
			if (OutGarbageObject)
			{
				GarbageErrorMessage = FString::Printf(TEXT("Garbage object %s is not referenced so it may have a flag set that's preventing it from being destroyed (see log for details)"), *OutGarbageObject->GetFullName());
				PathToCulprit = OutGarbageObject->GetFullName();
			}
			else
			{
				GarbageErrorMessage = TEXT("However it's not referenced by any object. It may have a flag set that's preventing it from being destroyed (see log for details)");
			}
		}

		Paths.Add(PathToCulprit);

#if DO_ENSURE
		if ((Options & EPrintStaleReferencesOptions::Ensure) == EPrintStaleReferencesOptions::Ensure)
		{
			ensureAlwaysMsgf(false, TEXT("Old %s not cleaned up by GC! %s:") LINE_TERMINATOR TEXT("%s"),
				*ObjectToFindReferencesTo->GetFullName(),
				*GarbageErrorMessage,
				*PathToCulprit);
		}
		else
#endif
			if (Verbosity == ELogVerbosity::Fatal)
			{
				UE_LOG(LogLoad, Fatal, TEXT("Old %s not cleaned up by GC! %s:") LINE_TERMINATOR TEXT("%s"),
					*ObjectToFindReferencesTo->GetFullName(),
					*GarbageErrorMessage,
					*PathToCulprit);
			}
			else
			{
				GLog->CategorizedLogf(TEXT("LogLoad"), Verbosity, TEXT("Old %s not cleaned up by GC! %s:") LINE_TERMINATOR TEXT("%s"),
					*ObjectToFindReferencesTo->GetFullName(),
					*GarbageErrorMessage,
					*PathToCulprit);
			}
	}

	if ((Options & EPrintStaleReferencesOptions::Fatal) == EPrintStaleReferencesOptions::Fatal)
	{
		// If you change the format of this string, update the corresponding AutomationUtils/Matcher/WorldLeakEventMatcher.cs
		
		// Emit an error and then a fatal error since Gauntlet can truncate Fatal Error messages.
		UE_LOG(LogLoad, Error, 
			TEXT("====Fatal World Leaks====") LINE_TERMINATOR
			TEXT("Logging first error, check logs for additional information") LINE_TERMINATOR 
			TEXT("%s")
			TEXT("====End Fatal World Leaks====") LINE_TERMINATOR,
			Paths.Num() ? *Paths[0] : TEXT("No paths to leaked objects found!"));
		UE_LOGF(LogLoad, Fatal, "Crashing due to uobject leaks (world leaks). See log above for details.");
	}

	return Paths;
}