// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ReferenceToken.h"
#include "UObject/GCObject.h"
#include "UObject/GCObjectInfo.h"
#include "UObject/GCVerseCellInfo.h"
#include "UObject/GarbageCollectionSchema.h"

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/VVMCollectionCycleRequest.h"
#include "VerseVM/VVMContext.h"
#include "VerseVM/VVMHeap.h"
#include "VerseVM/VVMMarkStack.h"
#include "VerseVM/VVMValue.h"
#include "VerseVM/VVMWriteBarrier.h"
#endif

FString FReferenceToken::GetDescription() const
{
	switch (GetType())
	{
	case EReferenceTokenType::None:
		return FString("None");
	case EReferenceTokenType::Barrier:
		return FString("Barrier");
	case EReferenceTokenType::Object:
		return AsObject()->GetFullName();
	case EReferenceTokenType::GCObjectInfo:
		return AsGCObjectInfo()->GetFullName();
	case EReferenceTokenType::GCObject:
	{
		FGCObject* GCObject = AsGCObject();
		return GCObject ? GCObject->GetReferencerName() : TEXT("Unknown");
	}
	case EReferenceTokenType::VerseCell:
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		return AsVerseCell()->DebugName();
#else
		unimplemented();
		return FString();
#endif
	case EReferenceTokenType::GCVerseCellInfo:
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		return AsGCVerseCellInfo()->GetDebugName().ToString();
#else
		unimplemented();
		return FString();
#endif
	default:
		unimplemented();
		return FString();
	}
}

FString FReferenceToken::GetDescription(FName PropertyName, const TCHAR* DefaultPropertyName) const
{
	return FString::Printf(TEXT("%s->%s"), *GetDescription(), PropertyName != NAME_None ? *PropertyName.ToString() : DefaultPropertyName);
}

FString FReferenceToken::GetMemberName(UE::GC::FMemberId& MemberId) const
{
	switch (GetType())
	{
	case EReferenceTokenType::Object:
	{
		UObject* Object = AsObject();
		UClass* Class = Object ? Object->GetClass() : nullptr;
		return Class ? GetMemberDebugInfo(Class->ReferenceSchema.Get(), MemberId).Name.ToString() : TEXT("N/A");
	}

	default:
		return TEXT("N/A");
	}
}
