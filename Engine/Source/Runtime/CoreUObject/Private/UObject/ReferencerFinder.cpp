// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ReferencerFinder.h"

#include "Async/ParallelFor.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectGlobals.h"

class FAllReferencesProcessor : public FSimpleReferenceProcessorBase
{
	const TSparseSet<UObject*>& PotentiallyReferencedObjects;
	TSparseSet<UObject*>& ReferencingObjects;
	EReferencerFinderFlags Flags;

public:
	FAllReferencesProcessor(const TSparseSet<UObject*>& InPotentiallyReferencedObjects, EReferencerFinderFlags InFlags, TSparseSet<UObject*>& OutReferencingObjects)
		: PotentiallyReferencedObjects(InPotentiallyReferencedObjects)
		, ReferencingObjects(OutReferencingObjects)
		, Flags(InFlags)
	{
	}
	FORCEINLINE_DEBUGGABLE void HandleTokenStreamObjectReference(FGCArrayStruct& ObjectsToSerializeStruct, UObject* ReferencingObject, UObject*& Object, UE::GC::FTokenId, EGCTokenType, bool)
	{
		if (!ReferencingObject)
		{
			ReferencingObject = ObjectsToSerializeStruct.GetReferencingObject();
		}
		if (Object && ReferencingObject && Object != ReferencingObject)
		{
			if (PotentiallyReferencedObjects.Contains(Object))
			{
				if ((Flags & EReferencerFinderFlags::SkipInnerReferences) != EReferencerFinderFlags::None)
				{
					if (ReferencingObject->IsIn(Object))
					{
						return;
					}
				}
				ReferencingObjects.Add(ReferencingObject);
			}
		}
	}
};

class FAllReferencesCollector : public UE::GC::TDefaultCollector<FAllReferencesProcessor>
{
	using Super = UE::GC::TDefaultCollector<FAllReferencesProcessor>;
public:
	using Super::Super;

	// MarkWeakObjectReferenceForClearing is a strange function, it's used by raw pointers that need 
	// weak semantics. by returning true here we indicate that the reference does not need to be reported
	// to the collector. That's the case for us because the GC information no longer contains tokens
	// describing the location of weak references, so if we want to process weak references we need
	// to use the reflection data:
	virtual bool MarkWeakObjectReferenceForClearing(UObject** WeakReference, UObject* ReferenceOwner) override
	{
		return true;
	}
};

// Allow parallel reference collection to be overridden to single threaded via console command.
static int32 GAllowParallelReferenceCollection = 1;
static FAutoConsoleVariableRef CVarAllowParallelReferenceCollection(
	TEXT("ref.AllowParallelCollection"),
	GAllowParallelReferenceCollection,
	TEXT("Used to control parallel reference collection."),
	ECVF_Default
);

// Until all native UObject classes have been registered it's unsafe to run FReferencerFinder on multiple threads
static bool GUObjectRegistrationComplete = false;

void FReferencerFinder::NotifyRegistrationComplete()
{
	GUObjectRegistrationComplete = true;
}

namespace UE::Private
{

struct FReferencerFinderArchive : public FArchiveUObject
								, public FReferenceCollector
{
private:
	UObject* CurrentObject = nullptr;
	const TSparseSet<UObject*>& Targets;
	bool bDoesReferenceAnyTargets = false;
	const bool bSkipInnerReferences;

public:
	FReferencerFinderArchive(const TSparseSet<UObject*>& InTargets, bool bInSkipInnerReferences)
		: FArchiveUObject()
		, Targets(InTargets)
		, bDoesReferenceAnyTargets(false)
		, bSkipInnerReferences(bInSkipInnerReferences)
	{
		SetIsSaving(true);
		SetShouldSkipCompilingAssets(true);
		SetWantBinaryPropertySerialization(true);
		SetUseUnversionedPropertySerialization(true);
		SetShouldSkipUpdateCustomVersion(true);
		// we aren't modifying, but we are searching weak references,
		// which per the comment on IsModifyingWeakAndStrongReferences() 
		// is a valid use of this flag. This pattern has been trailblazed
		// by FFindReferencersArchive and FFindLightmapsArchive:
		ArIsModifyingWeakAndStrongReferences = true;
		ArIsObjectReferenceCollector = true;
		ArShouldSkipBulkData = true;
	}

	// Crawls the provided object, returning true if we find any references to the 
	// objects in Targets:
	bool SearchForReferencesToTargets(UObject* Obj)
	{
		checkSlow(Obj);
		CurrentObject = Obj;
		bDoesReferenceAnyTargets = false;
		// This could miss some things in a user serialize functions,
		// but there exist many user Serialize routines that are not threadsafe
		// and deadlocking would be disastrous.
		Obj->GetClass()->SerializeBin(*this, Obj);

		return bDoesReferenceAnyTargets;
	}
	
private:
	// FReferenceCollector:
	virtual bool IsIgnoringArchetypeRef() const override { return false; }
	virtual bool IsIgnoringTransient() const override { return false; }
	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty)
	{
		if (InObject != nullptr && 
			Targets.Contains(InObject) )
		{
			ProcessTargetObject(InObject);
		}
	}

	// FArchiveUObject
	virtual FArchive& operator<<(UObject*& ObjRef) override
	{
		if (ObjRef != nullptr && 
			Targets.Contains(ObjRef) )
		{
			ProcessTargetObject(ObjRef);
		}

		return *this;
	}

	void ProcessTargetObject(UObject* Object)
	{
		if (bSkipInnerReferences && CurrentObject->IsIn(Object))
		{
			return;
		}

		bDoesReferenceAnyTargets = true;
	}
};

static TArray<UObject*> GetAllReferencersIncludingWeak(
	const TSparseSet<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore, const bool bSkipInnerReferences)
{
	// Lock the global array so that nothing can add UObjects while we're iterating over it
	GUObjectArray.LockInternalArray();

	const int32 MaxNumberOfObjects = GUObjectArray.GetObjectArrayNum();
	const int32 NumThreads = GetNumCollectReferenceWorkers();
	const int32 NumberOfObjectsPerThread = (MaxNumberOfObjects / NumThreads) + 1;
	
	// Allocate per thread results, in this case each thread will produce a list of referencers
	TUniquePtr<TSparseSet<UObject*>[]> ThreadResultsAlloc = MakeUnique<TSparseSet<UObject*>[]>(NumThreads);
	TArrayView<TSparseSet<UObject*>> ThreadResults(ThreadResultsAlloc.Get(), NumThreads);

	ParallelFor(NumThreads, 
		[&Referencees, 
		ObjectsToIgnore, 
		&ThreadResults,
		NumberOfObjectsPerThread, 
		NumThreads, 
		MaxNumberOfObjects, 
		bSkipInnerReferences](int32 ThreadIndex)
	{
		const EReferencerFinderFlags Flags = bSkipInnerReferences ? EReferencerFinderFlags::SkipInnerReferences : EReferencerFinderFlags::None;
		FReferencerFinderArchive ReferenceFinderArchive(Referencees, bSkipInnerReferences);
		TSparseSet<UObject*>& ThreadResult = ThreadResults[ThreadIndex];
		TArray<UObject*> ObjectsToSearch;
		ObjectsToSearch.Reserve(NumberOfObjectsPerThread);
		
		// Process the block of objects assigned to this thread:
		const int32 FirstObjectIndex = ThreadIndex * NumberOfObjectsPerThread;
		const int32 EndIndex = FMath::Min(FirstObjectIndex + NumberOfObjectsPerThread, MaxNumberOfObjects);
		for (int32 Index = FirstObjectIndex; Index < EndIndex; ++Index)
		{
			FUObjectItem& ObjectItem = GUObjectArray.GetObjectItemArrayUnsafe()[Index];
			// Skip any objects still being deserialized because it is not safe to access them until they are constructed correctly.
			if (!ObjectItem.GetObject() || ObjectItem.IsUnreachable() || ObjectItem.HasAnyFlags(EInternalObjectFlags::AsyncLoadingPhase1))
			{
				continue;
			}
			UObject* PotentialReferencer = static_cast<UObject*>(ObjectItem.GetObject());
			
			if(!IsValid(PotentialReferencer))
			{
				continue;
			}

			const bool bIsIgnored = ObjectsToIgnore && ObjectsToIgnore->Contains(PotentialReferencer);
			const bool bIsReferencee = Referencees.Contains(PotentialReferencer);
			if ( bIsIgnored || bIsReferencee )
			{
				continue;
			}

			// We could skip this for objects with no reflected object references
			// as mostly this is redundant to the faster path below (but both are 
			// mostly going to return false so no speedup reversing the order)
			if (ReferenceFinderArchive.SearchForReferencesToTargets(PotentialReferencer))
			{
				ThreadResult.Add(PotentialReferencer);
			}
			else
			{
				ObjectsToSearch.Add(PotentialReferencer);
			}
		}
		
		FAllReferencesProcessor Processor(Referencees, Flags, ThreadResult);
		FGCArrayStruct ArrayStruct;
		ArrayStruct.SetInitialObjectsUnpadded(ObjectsToSearch);
		{
			FGCScopeGuard GCGuard;
			CollectReferences(Processor, ArrayStruct);
		}
	},  (GUObjectRegistrationComplete && GAllowParallelReferenceCollection) ? 
			EParallelForFlags::None : EParallelForFlags::ForceSingleThread);
			
	// Release the global array lock
	GUObjectArray.UnlockInternalArray();

	TArray<UObject*> FinalResult;
	for(const TSparseSet<UObject*>& ThreadResult : ThreadResults)
	{
		FinalResult.Append(ThreadResult.Array());
	}

	return FinalResult;
}

static TArray<UObject*> GetAllReferencersExcludingWeak(
	const TSparseSet<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore, const EReferencerFinderFlags Flags)
{
	// We can use the faster GC schema to find references when we don't need to include weak 
	// references:
	TArray<UObject*> Ret;
	FCriticalSection ResultCritical;

	// Lock the global array so that nothing can add UObjects while we're iterating over it
	GUObjectArray.LockInternalArray();

	const int32 MaxNumberOfObjects = GUObjectArray.GetObjectArrayNum();
	const int32 NumThreads = GetNumCollectReferenceWorkers();
	const int32 NumberOfObjectsPerThread = (MaxNumberOfObjects / NumThreads) + 1;

	ParallelFor(NumThreads, [&Referencees, ObjectsToIgnore, &ResultCritical, &Ret, NumberOfObjectsPerThread, NumThreads, MaxNumberOfObjects, Flags](int32 ThreadIndex)
	{
		TSparseSet<UObject*> ThreadResult;
		FAllReferencesProcessor Processor(Referencees, Flags, ThreadResult);
		TArray<UObject*> ObjectsToSerialize;
		ObjectsToSerialize.Reserve(NumberOfObjectsPerThread);
		
		const int32 FirstObjectIndex = ThreadIndex * NumberOfObjectsPerThread;
		const int32 EndIndex = FMath::Min(FirstObjectIndex + NumberOfObjectsPerThread, MaxNumberOfObjects);
		
		for (int32 Index = FirstObjectIndex; Index < EndIndex; ++Index)
		{
			FUObjectItem& ObjectItem = GUObjectArray.GetObjectItemArrayUnsafe()[Index];
			if (ObjectItem.GetObject() && !ObjectItem.IsUnreachable())
			{
				UObject* PotentialReferencer = static_cast<UObject*>(ObjectItem.GetObject());
				if (ObjectsToIgnore && ObjectsToIgnore->Contains(PotentialReferencer))
				{
					continue;
				}

				if (!Referencees.Contains(PotentialReferencer))
				{
					ObjectsToSerialize.Add(PotentialReferencer);
				}
			}
		}

		FGCArrayStruct ArrayStruct;
		ArrayStruct.SetInitialObjectsUnpadded(ObjectsToSerialize);
			
		{
			// Since ReferenceCollector is configured to automatically assemble reference token streams
			// for classes that require it, make sure GC is locked because UClass::AssembleReferenceTokenStream requires it
			FGCScopeGuard GCGuard;
			// Now check if any of the potential referencers is referencing any of the referencees
			CollectReferences(Processor, ArrayStruct);
		}

		if (ThreadResult.Num())
		{
			// We found objects referencing some of the referencees so add them to the final results array
			FScopeLock ResultLock(&ResultCritical);
			Ret.Append(ThreadResult.Array());
		}
	}, (GUObjectRegistrationComplete && GAllowParallelReferenceCollection) ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread);

	// Release the global array lock
	GUObjectArray.UnlockInternalArray();
	return Ret;
}

// Feature flag for disabling this reworked serialize based reference finding, as there is a risk
// of undiscovered race conditions:
static bool GUseSerializeToFindWeakReferencers = true;
static FAutoConsoleVariableRef CVar_UseSerializeToFindWeakReferencers(
	TEXT("UObject.UseSerializeToFindWeakReferencers"),
	GUseSerializeToFindWeakReferencers,
	TEXT("If true use Serialize routines to find objects that are referencing only via weak references - set to false to only use GC schema for reference finding")
);

TArray<UObject*> GetAllReferencers(
	const TSparseSet<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore, EReferencerFinderFlags Flags)
{
	if(Referencees.Num() == 0)
	{
		return TArray<UObject*>();
	}

	const bool bSkipWeakReferences = (Flags & EReferencerFinderFlags::SkipWeakReferences) 
		!= EReferencerFinderFlags::None;
	const bool bSkipInnerReferences = (Flags & EReferencerFinderFlags::SkipInnerReferences) 
		!= EReferencerFinderFlags::None;
	if(!bSkipWeakReferences && UE::Private::GUseSerializeToFindWeakReferencers)
	{
		return UE::Private::GetAllReferencersIncludingWeak(
			Referencees, 
			ObjectsToIgnore, 
			bSkipInnerReferences
		);
	}
	else
	{
		return UE::Private::GetAllReferencersExcludingWeak(
			Referencees,
			ObjectsToIgnore,
			Flags
		);
	}
}
}// namespace UE::Private

TArray<UObject*> FReferencerFinder::GetAllReferencers(
	const TArray<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore, EReferencerFinderFlags Flags)
{
	return UE::Private::GetAllReferencers(TSparseSet<UObject*>(Referencees), ObjectsToIgnore, Flags);
}

TArray<UObject*> FReferencerFinder::GetAllReferencers(
	const TSet<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore, EReferencerFinderFlags Flags)
{
	TSparseSet<UObject*> AsSparse;
	for(UObject* Obj: Referencees)
	{
		AsSparse.Add(Obj);
	}
	return UE::Private::GetAllReferencers(AsSparse, ObjectsToIgnore, Flags);
}