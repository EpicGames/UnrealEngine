// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

FReferencerInformation::FReferencerInformation( UObject* inReferencer, int32 InReferences, const TArray<const FProperty*>& InProperties )
: Referencer(inReferencer), TotalReferences(InReferences), ReferencingProperties(InProperties)
{
}

FReferencerInformation::FReferencerInformation( UObject* inReferencer )
: Referencer(inReferencer), TotalReferences(0)
{
}

FReferencerInformationList::FReferencerInformationList()
{
}
FReferencerInformationList::FReferencerInformationList( const TArray<FReferencerInformation>& InternalRefs, const TArray<FReferencerInformation>& ExternalRefs )
: InternalReferences(InternalRefs), ExternalReferences(ExternalRefs)
{
}
