// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/RemoteExecutor.h"
#include "AutoRTFM.h"
#include "UObject/RemoteObject.h"

DEFINE_LOG_CATEGORY(LogRemoteExec);

int32 GRemoteExecutorForcedNetPumpIterationCount = 0;
static FAutoConsoleVariableRef CVarRemoteExecutorForcedNetPumpIterationCount(
	TEXT("remoteexecutor.ForcedNetPumpIterationCount"),
	GRemoteExecutorForcedNetPumpIterationCount,
	TEXT("How many execution pumps can we perform before we force pump the network? 0 = disabled"));

bool GRemoteExecutorNewNetPumpBehavior = false;
static FAutoConsoleVariableRef CVarRemoteExecutorNewNetPumpBehavior(
	TEXT("remoteexecutor.NewNetPumpBehavior"),
	GRemoteExecutorNewNetPumpBehavior,
	TEXT("Enable the 'new' net pump behavior"));

bool GRemoteExecutorAllowSpeculationMode = true;
static FAutoConsoleVariableRef CVarRemoteExecutorAllowSpeculationMode(
	TEXT("remoteexecutor.AllowSpeculationMode"),
	GRemoteExecutorAllowSpeculationMode,
	TEXT("Allow speculation mode (late abort) when transacting"));

namespace UE::RemoteExecutor
{
#define DEFINE_REMOTE_EXECUTOR_DELEGATE(x) decltype(x) x
	DEFINE_REMOTE_EXECUTOR_DELEGATE(OnTransactionQueuedDelegate);
	DEFINE_REMOTE_EXECUTOR_DELEGATE(OnTransactionStartingDelegate);
	DEFINE_REMOTE_EXECUTOR_DELEGATE(OnTransactionCompletedDelegate);
	DEFINE_REMOTE_EXECUTOR_DELEGATE(OnTransactionAbortedDelegate);
	DEFINE_REMOTE_EXECUTOR_DELEGATE(OnTransactionReleasedDelegate);

	DEFINE_REMOTE_EXECUTOR_DELEGATE(OnRegionBeginDelegate);
	DEFINE_REMOTE_EXECUTOR_DELEGATE(OnRegionEndDelegate);
#undef DEFINE_REMOTE_EXECUTOR_DELEGATE

const TCHAR* EnumToString(ERemoteExecutorAbortReason AbortReason)
{
	switch (AbortReason)
	{
	case ERemoteExecutorAbortReason::Unspecified:
		return TEXT("Unspecified");
	case ERemoteExecutorAbortReason::RequiresDependencies:
		return TEXT("RequiresDependencies");
	case ERemoteExecutorAbortReason::AbandonWork:
		return TEXT("AbandonWork");
	default:
		checkf(false, TEXT("EnumToString: Unknown abort reason"));
		break;
	};
	return TEXT("");
}
}

FArchive& operator<<(FArchive& Ar, FRemoteTransactionId& Id)
{
	Ar << Id.Id;
	return Ar;
}

struct FRemoteExecutorWorkList
{
	// bWorking is false until the executor starts working on it
	bool bWorking = false;

	FName Name;
	FRemoteWorkPriority Priority;
	bool bIsTransactional = false;

	TArray<TUniqueFunction<void(void)>> PendingWork;

	void Reset()
	{
		bWorking = false;
		Name = FName{};
		Priority = FRemoteWorkPriority{};
		bIsTransactional = false;
		PendingWork.Empty();
	}
};

struct FRemoteExecutorWork
{
	FName Name;
	FRemoteTransactionId RequestId;
	FRemoteWorkPriority Priority;
	bool bIsTransactional = false;

	TUniqueFunction<void(void)> Work;

	TArray<void*> SubsystemRequests;
	bool bRequestCreated = false;
	
	uint32 ExecutionAttempts = 0;

	// If non-zero, then the current work will abort rather than commit
	uint32 NumAbortsRequested = 0;
	
	bool bRequiresMultiServerCommit = false;
	FString RequiresMultiServerCommitReason;

    // for work that belongs to a work list, this lets 
	// us find the worklist in order to enqueue the next item
	int32 WorkListIndex = INDEX_NONE;
};

class FRemoteExecutor
{
	TArray<FRemoteSubsystemBase*> Subsystems;

	uint32 NextTransactionRequestId = 0;
	uint32 NextUniqueId = 0;

	TArray<FRemoteExecutorWork> PendingWorks;
	TArray<FRemoteExecutorWork> PendingMigrationWorks;
	TArray<FRemoteExecutorWorkList> PendingWorkLists;

	// GRemoteExecutorForcedNetPumpIterationCount controls how many iterations
	// we're allowed to go before we unconditionally try to pump the network
	// (to reduce latency of servicing requests of other servers)
	// Here we keep track of how many iterations it has been since the last
	// network pump happened.
	uint32 IterationsSinceNetworkPump = 0;

public:
	FRemoteServerId LocalServerId;
	FRemoteExecutorDelegates* Delegates = nullptr;
	int32 SpeculationModeScopeCount = 0;

	FRemoteExecutor() = default;

	FRemoteExecutor(FRemoteServerId InLocalServerId, FRemoteExecutorDelegates* InDelegates)
		: LocalServerId(InLocalServerId)
		, Delegates(InDelegates)
	{
	}

	~FRemoteExecutor()
	{
		for (FRemoteSubsystemBase* Subsystem : Subsystems)
		{
			delete Subsystem;
		}
		Subsystems.Empty();
	}

	FRemoteServerId GetLocalServerId() const { return LocalServerId; }

	bool HasPendingWork() const
	{
		return PendingWorks.Num() > 0
			|| PendingMigrationWorks.Num() > 0
			|| PendingWorkLists.Num() > 0
			|| ActiveRemoteMultiServerCommitRequestId.IsValid();
	}

	FRemoteExecutorWork* ExecutingWork = nullptr;

	// Pump the network next chance we get
	bool bShouldPumpNetwork = false;

	// When true, ExecutePendingWork breaks out of its outer loop after each iteration,
	// allowing the caller to pump other executors before continuing.
	bool bSingleStepModeForTesting = false;

	// These should only ever be accessed in an AUTORTFM_OPEN scope
	UE::RemoteExecutor::ERemoteExecutorAbortReason AbortReason = UE::RemoteExecutor::ERemoteExecutorAbortReason::Unspecified;
	FString AbortReasonDescription;

	// tracking data for servicing a remote multi-server commit
	FRemoteServerId ActiveRemoteMultiServerCommitServerId;
	FRemoteTransactionId ActiveRemoteMultiServerCommitRequestId;
	FRemoteWorkPriority ActiveRemoteMultiServerCommitPriority;
	TArray<TUniqueFunction<void()>> ActiveRemoteMultiServerCommitDeferredActions;
	bool bActiveRemoteMultiServerCommitReady = false;

	// tracking data for executing our local multi-server commit
	FRemoteTransactionId MultiServerCommitRequestId;
	TArray<FRemoteServerId> MultiServerCommitReadyServers;
	bool bMultiServerCommitRequiresAbort = false;

	void RegisterSubsystem(FRemoteSubsystemBase* Subsystem)
	{
		Subsystems.Add(Subsystem);
	}

	void ActivateSubsystemsForTesting()
	{
		for (FRemoteSubsystemBase* Subsystem : Subsystems)
		{
			Subsystem->ActivateForTesting();
		}
	}

	void DeactivateSubsystemsForTesting()
	{
		for (FRemoteSubsystemBase* Subsystem : Subsystems)
		{
			Subsystem->DeactivateForTesting();
		}
	}

	FRemoteTransactionId GenerateNextTransactionId()
	{
		uint32 RawRequestId = NextTransactionRequestId++;
		return FRemoteTransactionId((RawRequestId++ % 0x800000u) + 1000u);
	}

	uint32 GenerateNextUniqueId()
	{
		// pack the local server id into the top bits
		uint32 UniqueIdMask = (1u << (32 - REMOTE_OBJECT_SERVER_ID_BIT_SIZE)) - 1;
		
		uint32 UniqueId = (NextUniqueId++) & UniqueIdMask;
		UniqueId |= LocalServerId.GetIdNumber() << (32 - REMOTE_OBJECT_SERVER_ID_BIT_SIZE);
		return UniqueId;
	}

	void EnqueueWork(FName InWorkName, FRemoteWorkPriority InWorkPriority, bool bIsTransactional, int32 InWorkListIndex, TUniqueFunction<void(void)>&& InWork)
	{
		FRemoteExecutorWork& NewWork = InWorkPriority.GetWorkType() == ERemoteWorkType::Normal ? PendingWorks.Emplace_GetRef() : PendingMigrationWorks.Emplace_GetRef();
		NewWork.Name = InWorkName;
		NewWork.Priority = InWorkPriority;
		NewWork.Work = MoveTemp(InWork);
		NewWork.bIsTransactional = bIsTransactional;
		NewWork.RequestId = GenerateNextTransactionId();
		NewWork.WorkListIndex = InWorkListIndex;

		if (bIsTransactional)
		{
			UE::RemoteExecutor::OnTransactionQueuedDelegate.Broadcast(NewWork.RequestId, NewWork.Name);
		}
	}

	void EnqueueWorkList(FName WorkName, FRemoteWorkPriority InWorkPriority, bool bIsTransactional, TArray<TUniqueFunction<void(void)>>&& InWorkList)
	{
		FRemoteExecutorWorkList& NewWorkList = PendingWorkLists.Emplace_GetRef();

		NewWorkList.Name = WorkName;
		NewWorkList.Priority = InWorkPriority;
		NewWorkList.bIsTransactional = bIsTransactional;
		NewWorkList.PendingWork = MoveTemp(InWorkList);
	}

private:

	void EnqueueNextWorkFromList(int32 WorkListIndex)
	{
		if (WorkListIndex != INDEX_NONE)
		{
			FRemoteExecutorWorkList& WorkList = PendingWorkLists[WorkListIndex];
			check(WorkList.bWorking);

			if (WorkList.PendingWork.Num() > 0)
			{
				EnqueueWork(WorkList.Name, WorkList.Priority, WorkList.bIsTransactional, WorkListIndex, MoveTemp(WorkList.PendingWork[0]));
				WorkList.PendingWork.RemoveAt(0, EAllowShrinking::No);
			}
		}
	};

	void ConditionallyPumpNetwork(bool bForcePumpNetwork)
	{
		if (GRemoteExecutorNewNetPumpBehavior)
		{
			// Always pump if we're trying this transaction multiple times
			bool bUnconditionallyPumpNetwork = bForcePumpNetwork || bShouldPumpNetwork;

			// if GRemoteExecutorForcedNetPumpIterationCount is nonzero, we may need to unconditionally pump the network
			if (GRemoteExecutorForcedNetPumpIterationCount && !bUnconditionallyPumpNetwork)
			{
				IterationsSinceNetworkPump++;
				bUnconditionallyPumpNetwork = (IterationsSinceNetworkPump % GRemoteExecutorForcedNetPumpIterationCount) == 0;
			}

			if (bUnconditionallyPumpNetwork)
			{
				// pump the network, reset our state variables
				IterationsSinceNetworkPump = 0;
				bShouldPumpNetwork = false;
				Delegates->TickNetworkDelegate.ExecuteIfBound();
			}
		}
	};

public:

	void ExecutePendingWork()
	{
		check(ExecutingWork == nullptr);

		if ((PendingWorks.Num() > 0) || (PendingWorkLists.Num() > 0) || (PendingMigrationWorks.Num() > 0))
		{
			UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork BEGIN with %d work items, %d migration items and %d work lists", PendingWorks.Num(), PendingMigrationWorks.Num(), PendingWorkLists.Num());
		}

		auto EnqueueFirstWorkFromList = [this](int32 WorkListIndex) -> void
			{
				if (WorkListIndex != INDEX_NONE)
				{
					FRemoteExecutorWorkList& WorkList = PendingWorkLists[WorkListIndex];

					// is bWorking is already true then we already have work enqueued from this list
					if (!WorkList.bWorking)
					{
						WorkList.bWorking = true;

						if (WorkList.PendingWork.Num() > 0)
						{
							EnqueueWork(WorkList.Name, WorkList.Priority, WorkList.bIsTransactional, WorkListIndex, MoveTemp(WorkList.PendingWork[0]));
							WorkList.PendingWork.RemoveAt(0, EAllowShrinking::No);
						}
					}
				}
			};

		double LastStallPrintTime = FPlatformTime::Seconds();

		for (int32 LocalIterationNumber = 0; ; LocalIterationNumber++)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ResumeExecutePendingWork)

			double Now = FPlatformTime::Seconds();
			bool bPrintStall = false;

#if !UE_BUILD_SHIPPING && !NO_LOGGING
			ELogVerbosity::Type InitialLogVerbosity = LogRemoteExec.GetVerbosity();
			ON_SCOPE_EXIT
			{
				LogRemoteExec.SetVerbosity(InitialLogVerbosity);
			};

			constexpr int32 MaxIterationsBeforeStall = 100000;
			if (LocalIterationNumber >= MaxIterationsBeforeStall && !(LocalIterationNumber % MaxIterationsBeforeStall))
			{
				bPrintStall = true;
				UE_SET_LOG_VERBOSITY(LogRemoteExec, VeryVerbose);
			}
#endif

			if ((Now - LastStallPrintTime) > 1.0)
			{
				LastStallPrintTime = Now;
				bPrintStall = true;
			}

			if (!GRemoteExecutorNewNetPumpBehavior)
			{
				// pump the network
				Delegates->TickNetworkDelegate.ExecuteIfBound();
			}

			// check if we're actively servicing a remote multi-server commit
			if (ActiveRemoteMultiServerCommitRequestId.IsValid())
			{
				check(ActiveRemoteMultiServerCommitServerId.IsValid());
				check(ActiveRemoteMultiServerCommitPriority.IsValid());

				if (bPrintStall)
				{
					UE_LOGF(LogRemoteExec, Verbose, "ExecutePendingWork[LocalIter %d] Waiting on handling remote multi server commit %ls %ls",
						LocalIterationNumber,
						*ActiveRemoteMultiServerCommitServerId.ToString(),
						*ActiveRemoteMultiServerCommitRequestId.ToString());
				}

				// if we have an active remote multi-server commit that we are servicing, we pump the network and 
				// pause executing any local work until that is complete
				if (GRemoteExecutorNewNetPumpBehavior)
				{
					IterationsSinceNetworkPump = 0;
					Delegates->TickNetworkDelegate.ExecuteIfBound();
				}

				continue;
			}

			// enqueue the first work item from any new work lists
			for (int32 WorkListIndex = 0; WorkListIndex < PendingWorkLists.Num(); WorkListIndex++)
			{
				EnqueueFirstWorkFromList(WorkListIndex);
			}

			// subsystem ticking
			for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
			{
				FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
				Subsystem->TickSubsystem();
			}

			//
			// if we get here and there is no work pending, break out and finish
			//
			if (PendingWorks.Num() == 0 && PendingMigrationWorks.Num() == 0)
			{
				break;
			}

			// Maybe pump the network
			{
				const bool bForcePumpNetwork = (LocalIterationNumber > 0);
				ConditionallyPumpNetwork(bForcePumpNetwork);
			}

			ExecutePendingWork(PendingMigrationWorks, LastStallPrintTime);
			ExecutePendingWork(PendingWorks, LastStallPrintTime);

			if (bSingleStepModeForTesting)
			{
				break;
			}
		}

		if (!HasPendingWork())
		{
			PendingWorkLists.Empty();
			AbortReason = UE::RemoteExecutor::ERemoteExecutorAbortReason::Unspecified;
			AbortReasonDescription.Reset();
		}
	}

private:

	void ExecutePendingWork(TArray<FRemoteExecutorWork>& WorksToExecute, double& LastStallPrintTime)
	{
		if (UE_LOG_ACTIVE(LogRemoteExec, VeryVerbose))
		{
			UE_CLOGF(WorksToExecute.Num() > 0, LogRemoteExec, VeryVerbose, "ExecutePendingWork Entries: ");

			for (int32 PendingWorkIndex = 0; PendingWorkIndex < WorksToExecute.Num(); PendingWorkIndex++)
			{
				FRemoteExecutorWork& PendingWork = WorksToExecute[PendingWorkIndex];

				UE_LOGF(LogRemoteExec, VeryVerbose, "   PendingWork[%d] '%ls' RequestId %ls %ls",
					   PendingWorkIndex,
					   *PendingWork.Name.ToString(),
					   *PendingWork.RequestId.ToString(),
					   *PendingWork.Priority.ToString());
			}
		}

		bool bPrintStall = false;

		// round robin through executing all pending work
		for (int32 PendingWorkIndex = 0; PendingWorkIndex < WorksToExecute.Num(); )
		{
			check(ExecutingWork == nullptr);
			TGuardValue<FRemoteExecutorWork*> ExecutingWorkSave(ExecutingWork, &WorksToExecute[PendingWorkIndex]);
			check(ExecutingWork);

			UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : Executing request %ls %ls --",
				*ExecutingWork->RequestId.ToString(),
				*ExecutingWork->Name.ToString(),
				*ExecutingWork->Priority.ToString()); // -V522

			if (ExecutingWork->bIsTransactional)
			{
				if (!ExecutingWork->bRequestCreated)
				{
					// new request
					ExecutingWork->bRequestCreated = true;

					for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
					{
						FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
						UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : Creating request[%ls]",
							*ExecutingWork->RequestId.ToString(),
							Subsystem->NameForDebug());
						Subsystem->CreateRequest(ExecutingWork->RequestId, ExecutingWork->Priority);

						Subsystem->SetActiveRequest(ExecutingWork->RequestId);

						Subsystem->BeginRequest();
					}
				}
				else
				{
					for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
					{
						FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
						Subsystem->SetActiveRequest(ExecutingWork->RequestId);
					}
				}

				double Now = FPlatformTime::Seconds();
				if ((Now - LastStallPrintTime) > 1.0)
				{
					LastStallPrintTime = Now;
					bPrintStall = true;
				}

				// tick the subsystems
				for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
				{
					FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
					Subsystem->TickRequest();
				}

				int32 SubsystemReadyCount = 0;
				for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
				{
					FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
					if (Subsystem->AreDependenciesSatisfied())
					{
						SubsystemReadyCount++;
					}
					else
					{
						if (bPrintStall)
						{
							UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : subsystem %ls not ready...",
								*ExecutingWork->RequestId.ToString(),
								Subsystem->NameForDebug());
						}
					}
				}

				if (SubsystemReadyCount != Subsystems.Num())
				{
					if (bPrintStall)
					{
						UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : %d of %d subsystems not ready...",
							*ExecutingWork->RequestId.ToString(),
							Subsystems.Num() - SubsystemReadyCount, Subsystems.Num());
					}

					PendingWorkIndex++;
					continue;
				}

				// all of the subsystems are ready, try to perform the work
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(ExecutingWork->Name)

					FRemoteTransactionId RequestId = ExecutingWork->RequestId;
				UE::RemoteExecutor::OnTransactionStartingDelegate.Broadcast(RequestId, ExecutingWork->Name);
				AutoRTFM::ETransactionResult WorkTransactResult =
					AutoRTFM::Transact(
						[this]()
						{
							UE_AUTORTFM_OPEN
							{
								AbortReason = UE::RemoteExecutor::ERemoteExecutorAbortReason::Unspecified;
								AbortReasonDescription.Reset();
								ExecutingWork->NumAbortsRequested = 0;

								ExecutingWork->ExecutionAttempts++;
							};

							// You can set a breakpoint inside the abort handler below to see what caused the abort,
							// or you can rely on the log output which only catches livelocks
							const uint32 ExecutionAttempts = ExecutingWork->ExecutionAttempts;
							UE_AUTORTFM_ONABORT(ExecutionAttempts, this)
							{
								constexpr uint32 MaxExecutionAttemptsBeforeLivelockWarning = 500;
								if ((ExecutionAttempts >= MaxExecutionAttemptsBeforeLivelockWarning) && (ExecutionAttempts % MaxExecutionAttemptsBeforeLivelockWarning) == 0)
								{
									UE_LOGF(LogRemoteExec, Display, "vvv Transaction Aborted %u Times. Dumping Callstack. vvv", ExecutionAttempts);
									FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
									UE_LOGF(LogRemoteExec, Display, "^^^ Transaction Aborted %u Times ^^^", ExecutionAttempts);
								}
							};

							ExecutingWork->Work();

							// Did we do something that should late-abort this transaction?
							if (ExecutingWork->NumAbortsRequested > 0)
							{
								UE_LOGF(LogRemoteExec, Warning, "We ended a transaction marked for abort. We expected to abort before this point (%d abort requests).", ExecutingWork->NumAbortsRequested);
								AutoRTFM::AbortTransaction();
							}
							else
							{
								ensureMsgf(AbortReason == UE::RemoteExecutor::ERemoteExecutorAbortReason::Unspecified && AbortReasonDescription.IsEmpty(), TEXT("RemoteExecutor: AbortReason %s specified during transaction without aborting"), *AbortReasonDescription);
							}

							if (ExecutingWork->bRequiresMultiServerCommit)
							{
								TRACE_CPUPROFILER_EVENT_SCOPE(MultiServerCommit)

									// we're done with the work and about to commit - 
									// first we need to send borrowed objects back
									// to their owner and ask them if they are ready
									// to commit or not...
									UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : STARTING multi-server commit because %ls",
										*ExecutingWork->RequestId.ToString(),
										*ExecutingWork->RequiresMultiServerCommitReason);

								check(!MultiServerCommitRequestId.IsValid());
								check(MultiServerCommitReadyServers.Num() == 0);
								check(bMultiServerCommitRequiresAbort == false);

								UE_AUTORTFM_OPEN
								{
									MultiServerCommitRequestId = ExecutingWork->RequestId;
									bMultiServerCommitRequiresAbort = false;
								};

								// first collect the list of servers that need to be involved
								TArray<FRemoteServerId> MultiServerCommitRemoteServers;
								for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
								{
									UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : BeginMultiServerCommit subsystem %d...",
										*ExecutingWork->RequestId.ToString(),
										SubsystemIndex);

									FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
									Subsystem->BeginMultiServerCommit(MultiServerCommitRemoteServers);
								}

								UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : BeginMultiServerCommit DONE (%d servers)",
									*ExecutingWork->RequestId.ToString(),
									MultiServerCommitRemoteServers.Num());

								for (FRemoteServerId ServerId : MultiServerCommitRemoteServers)
								{
									UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : BeginMultiServerCommit server %ls",
										*ExecutingWork->RequestId.ToString(),
										*ServerId.ToString());
								}

								// signal to the relevant servers that they need to help us do a multi-server commit
								UE_AUTORTFM_OPEN
								{
									Delegates->BeginMultiServerCommitDelegate.ExecuteIfBound(ExecutingWork->RequestId, ExecutingWork->Priority, MultiServerCommitRemoteServers);
								};

								// ask each subsystem to send any necessary data as part of this commit
								for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
								{
									UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : ExecuteMultiServerCommit subsystem %d...",
										*ExecutingWork->RequestId.ToString(),
										SubsystemIndex);

									FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
									Subsystem->ExecuteMultiServerCommit();
								}

								UE_AUTORTFM_OPEN
								{
									// tell each server that we're done sending commit data and we are waiting for them
									// to respond with whether they are ready or we need to abort and retry
									Delegates->ReadyMultiServerCommitDelegate.ExecuteIfBound(ExecutingWork->RequestId, MultiServerCommitRemoteServers);
								};

								// tick the network until everything is ready
								double LastPrintTime = FPlatformTime::Seconds();

								for (;;)
								{
									double Now = FPlatformTime::Seconds();
									if ((Now - LastPrintTime) > 1.0)
									{
										LastPrintTime = Now;
										UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : WAITING for multi-server commit...",
											*ExecutingWork->RequestId.ToString());
									}

									if (MultiServerCommitRemoteServers.Num() == MultiServerCommitReadyServers.Num())
									{
										// we got a response from each server, do a sanity check to ensure the list of ready
										// servers is the list we expected to have										
										UE_AUTORTFM_OPEN
										{
											MultiServerCommitRemoteServers.Sort();
											MultiServerCommitReadyServers.Sort();
										};

										for (int32 ServerIndex = 0; ServerIndex < MultiServerCommitRemoteServers.Num(); ServerIndex++)
										{
											FRemoteServerId ServerId = MultiServerCommitRemoteServers[ServerIndex];
											FRemoteServerId ReadyServerId = MultiServerCommitReadyServers[ServerIndex];

											if (ServerId != ReadyServerId)
											{
												UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : Multi-server commit expected server %ls but got %ls",
													*ExecutingWork->RequestId.ToString(),
													*ServerId.ToString(),
													*ReadyServerId.ToString());
											}

											check(ServerId == ReadyServerId);
										}

										if (bMultiServerCommitRequiresAbort)
										{
											UE_LOGF(LogRemoteExec, Verbose, "ExecutePendingWork[%ls] : Multi-server commit ALL servers READY, but we are flagged with ABORT",
												*ExecutingWork->RequestId.ToString());

											UE_AUTORTFM_OPEN
											{
												for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
												{
													FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
													Subsystem->AbortMultiServerCommit();
												}

												// aborted, tell all servers to abandon this commit
												Delegates->AbandonMultiServerCommitDelegate.ExecuteIfBound(ExecutingWork->RequestId, MultiServerCommitRemoteServers);

												MultiServerCommitRequestId = FRemoteTransactionId::Invalid();
												MultiServerCommitReadyServers.Reset();
												bMultiServerCommitRequiresAbort = false;
											};

											using namespace UE::RemoteExecutor;
											AbortCurrentTransaction(EAbortTransactionBehavior::Immediately, ERemoteExecutorAbortReason::RequiresDependencies, TEXT("bMultiServerCommitRequiresAbort"));

											return;
										}
										else
										{
											// the commit was accepted by every server, notify each subsystem that we are committing
											UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : Multi-server commit ALL servers READY, COMMITTING",
												*ExecutingWork->RequestId.ToString());

											for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
											{
												FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
												Subsystem->CommitMultiServerCommit();
											}

											break;
										}
									}

									UE_AUTORTFM_OPEN
									{
										Delegates->TickNetworkDelegate.ExecuteIfBound();
									};
								}

								UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : DONE with multi-server commit", *ExecutingWork->RequestId.ToString());

								UE_AUTORTFM_OPEN
								{
									MultiServerCommitRequestId = FRemoteTransactionId::Invalid();
									MultiServerCommitReadyServers.Reset();
									bMultiServerCommitRequiresAbort = false;
								};

								UE_AUTORTFM_OPEN
								{
									// tell each server that we are committed and they should commit
									Delegates->EndMultiServerCommitDelegate.ExecuteIfBound(ExecutingWork->RequestId, MultiServerCommitRemoteServers);
								};
							};
						});

				bool bWorkComplete = false;
				bool bWorkTransactionAborted = false;

				if (WorkTransactResult == AutoRTFM::ETransactionResult::AbortedByRequest)
				{
					using namespace UE::RemoteExecutor;

					UE_LOGF(LogRemoteExec, Verbose, "ExecutePendingWork[%ls] (%ls): Attempt %u ABORTED. Reason: %ls (%ls)",
							*ExecutingWork->RequestId.ToString(),
							*ExecutingWork->Name.ToString(),
							ExecutingWork->ExecutionAttempts,
							EnumToString(AbortReason),
							*AbortReasonDescription);

					bWorkTransactionAborted = true;
					UE::RemoteExecutor::OnTransactionAbortedDelegate.Broadcast(RequestId, ExecutingWork->ExecutionAttempts, AbortReasonDescription);

					if (AbortReason == ERemoteExecutorAbortReason::AbandonWork) // -V547
					{
						bWorkComplete = true;
					}
					else 
					{
						for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
						{
							FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
							Subsystem->TickAbortedRequest();
						}
					}
				}
				else
				{
					check(AbortReason == UE::RemoteExecutor::ERemoteExecutorAbortReason::Unspecified);

					UE_CLOGF(ExecutingWork->ExecutionAttempts > 1, LogRemoteExec, Verbose, "ExecutePendingWork[%ls] (%ls): COMPLETED after %u attempts",
							*ExecutingWork->RequestId.ToString(),
							*ExecutingWork->Name.ToString(),
							ExecutingWork->ExecutionAttempts);

					UE_CLOGF(ExecutingWork->ExecutionAttempts == 1, LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] (%ls): COMPLETED (first attempt)",
							 *ExecutingWork->RequestId.ToString(),
							 *ExecutingWork->Name.ToString());

					UE::RemoteExecutor::OnTransactionCompletedDelegate.Broadcast(RequestId, ExecutingWork->ExecutionAttempts);

					bWorkComplete = true;
				}

				if (bWorkComplete)
				{
					for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
					{
						FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
						UE_LOGF(LogRemoteExec, VeryVerbose, "ExecutePendingWork[%ls] : End request[%ls]",
							*ExecutingWork->RequestId.ToString(),
							Subsystem->NameForDebug());
						Subsystem->EndRequest(!bWorkTransactionAborted);
						Subsystem->ClearActiveRequest();
					}

					for (int32 SubsystemIndex = 0; SubsystemIndex < Subsystems.Num(); SubsystemIndex++)
					{
						FRemoteSubsystemBase* Subsystem = Subsystems[SubsystemIndex];
						Subsystem->DestroyRequest(ExecutingWork->RequestId);
					}

					UE::RemoteExecutor::OnTransactionReleasedDelegate.Broadcast(ExecutingWork->RequestId);

					// remove this work from the list and do not increment PendingWorkIndex
					EnqueueNextWorkFromList(ExecutingWork->WorkListIndex);
					WorksToExecute.RemoveAt(PendingWorkIndex, EAllowShrinking::No);
				}
				else
				{
					// we will revisit this later, move on to the next work
					PendingWorkIndex++;
				}
			}
			else
			{
				UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(ExecutingWork->Name);

				// execute non-transactional work
				ExecutingWork->Work();

				// remove this work from the list and do not increment PendingWorkIndex
				EnqueueNextWorkFromList(ExecutingWork->WorkListIndex);
				WorksToExecute.RemoveAt(PendingWorkIndex, EAllowShrinking::No);

				UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(TEXT(""));
			}

			// it's possible this work ended with trying to perform a multi-server
			// commit that got aborted because a remote multi-server commit of higher
			// priority came in. if so, we need to break out of processing further work
			// so we can service the remote multi-server commit
			if (ActiveRemoteMultiServerCommitRequestId.IsValid())
			{
				break;
			}

			// Finally, see if we have any requests waiting that need servicing
			ConditionallyPumpNetwork(false);
		}
	}	
};

FRemoteExecutorDelegates GDefaultDelegates;
FRemoteExecutor* GRemoteExecutor = nullptr;

void ExecuteTransactionalInternal(FName WorkName, FRemoteWorkPriority InWorkPriority, const TFunctionRef<void(void)>& InWork)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	check(!AutoRTFM::IsClosed());

	if (GRemoteExecutor)
	{
		GRemoteExecutor->EnqueueWork(WorkName, InWorkPriority, true, INDEX_NONE, [&InWork]() { InWork(); });
		UE::RemoteExecutor::ExecutePendingWork();
	}
	else
	{
		InWork();
	}
#else
	InWork();
#endif
}

namespace UE::RemoteExecutor
{
FRemoteExecutor* CreateRemoteExecutor(FRemoteServerId LocalServerId, FRemoteExecutorDelegates* Delegates)
{
	GRemoteExecutor = new FRemoteExecutor(LocalServerId, Delegates);
	return GRemoteExecutor;
}

void DestroyRemoteExecutor(FRemoteExecutor* Executor)
{
	if (GRemoteExecutor == Executor)
	{
		GRemoteExecutor = nullptr;
	}

	delete Executor;
}

void SetRemoteExecutorForTesting(FRemoteExecutor* Executor)
{
	if (GRemoteExecutor)
	{
		GRemoteExecutor->DeactivateSubsystemsForTesting();
	}

	GRemoteExecutor = Executor;
	if (GRemoteExecutor)
	{
		GRemoteExecutor->ActivateSubsystemsForTesting();
	}
}

void SetSingleStepModeForTesting(bool bEnable)
{
	check(GRemoteExecutor);
	GRemoteExecutor->bSingleStepModeForTesting = bEnable;
}

FSpeculationExecutionScope::FSpeculationExecutionScope()
{
	if (GRemoteExecutor && GRemoteExecutorAllowSpeculationMode)
	{
		++GRemoteExecutor->SpeculationModeScopeCount;
	}
}

FSpeculationExecutionScope::~FSpeculationExecutionScope()
{
	if (GRemoteExecutor)
	{
		if (GRemoteExecutorAllowSpeculationMode)
		{
			--GRemoteExecutor->SpeculationModeScopeCount;
			check(GRemoteExecutor->SpeculationModeScopeCount >= 0);
		}
		else
		{
			GRemoteExecutor->SpeculationModeScopeCount = 0;
		}

#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (GRemoteExecutor->SpeculationModeScopeCount == 0)
		{
			if (GRemoteExecutor->ExecutingWork && GRemoteExecutor->ExecutingWork->NumAbortsRequested)
			{
				AutoRTFM::AbortTransaction();
			}
		}
#endif
	}
}

bool IsRemoteExecutorActive()
{
	return GRemoteExecutor != nullptr;
}

bool IsSpeculationMode()
{
	check(GRemoteExecutor && GRemoteExecutor->SpeculationModeScopeCount >= 0);
	return GRemoteExecutor && GRemoteExecutorAllowSpeculationMode && (GRemoteExecutor->SpeculationModeScopeCount > 0);
}

void RegisterRemoteSubsystem(FRemoteExecutor* Executor, FRemoteSubsystemBase* Subsystem)
{
	Executor->RegisterSubsystem(Subsystem);
}

void NotifyNetworkIsWaiting()
{
	GRemoteExecutor->bShouldPumpNetwork = true;
}

void AbortCurrentTransaction(EAbortTransactionBehavior AbortBehavior, ERemoteExecutorAbortReason AbortReason, FStringView AbortDescription)
{
	check(AutoRTFM::IsTransactional());

	UE_AUTORTFM_OPEN
	{
		GRemoteExecutor->AbortReason = AbortReason;
		GRemoteExecutor->AbortReasonDescription = AbortDescription;

		FRemoteExecutorWork* CurrentWorkItem = GRemoteExecutor->ExecutingWork;
		if (ensure(CurrentWorkItem))
		{
			CurrentWorkItem->NumAbortsRequested++;
		}
	};

	if (AutoRTFM::IsClosed())
	{
		if (AbortBehavior == EAbortTransactionBehavior::Immediately)
		{
			AutoRTFM::AbortTransaction();
		}
	}
}

void AbortTransactionRequiresDependencies(FStringView Description)
{
	EAbortTransactionBehavior AbortBehavior = IsSpeculationMode() ? EAbortTransactionBehavior::BeforeCommit : EAbortTransactionBehavior::Immediately;
	AbortCurrentTransaction(AbortBehavior, ERemoteExecutorAbortReason::RequiresDependencies, Description);
}

void AbortTransactionAndAbandonWork(FStringView Description)
{
	AbortCurrentTransaction(EAbortTransactionBehavior::Immediately, ERemoteExecutorAbortReason::AbandonWork, Description);
}

void TransactionRequiresMultiServerCommit(FStringView Description)
{
	if (!GRemoteExecutor)
	{
		return;
	}

	if (GRemoteExecutor->ExecutingWork && !GRemoteExecutor->ExecutingWork->bRequiresMultiServerCommit)
	{
		GRemoteExecutor->ExecutingWork->bRequiresMultiServerCommit = true;
		GRemoteExecutor->ExecutingWork->RequiresMultiServerCommitReason = Description;

		UE_LOGF(LogRemoteExec, VeryVerbose, "TransactionRequiresMultiServerCommit ACTIVATED because: %ls", *GRemoteExecutor->ExecutingWork->RequiresMultiServerCommitReason);
	}
}

void BeginRemoteMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId, FRemoteWorkPriority RequestPriority)
{
	bool bAcceptRequest = false;
	bool bAbortLocalCommit = false;

	if (GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId.IsValid())
	{
		if (GRemoteExecutor->bActiveRemoteMultiServerCommitReady)
		{
			// we already told the remote server that we are READY, so we are locked in for a moment until we finish
			UE_LOGF(LogRemoteExec, VeryVerbose, "BeginRemoteMultiServerCommit %ls from %ls %ls DENYING because we are already READY with remote multi-server commit %ls %ls",
				*RequestId.ToString(),
				*ServerId.ToString(),
				*RequestPriority.ToString(),
				*GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId.ToString(),
				*GRemoteExecutor->ActiveRemoteMultiServerCommitPriority.ToString());
		}
		else if (IsHigherPriority(RequestPriority, GRemoteExecutor->ActiveRemoteMultiServerCommitPriority))
		{
			UE_LOGF(LogRemoteExec, VeryVerbose, "BeginRemoteMultiServerCommit %ls from %ls %ls ACCEPTING because our remote multi-server commit %ls is lower priority %ls",
				*RequestId.ToString(),
				*ServerId.ToString(),
				*RequestPriority.ToString(),
				*GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId.ToString(),
				*GRemoteExecutor->ActiveRemoteMultiServerCommitPriority.ToString());

			bAcceptRequest = true;
		}
		else
		{
			UE_LOGF(LogRemoteExec, VeryVerbose, "BeginRemoteMultiServerCommit %ls from %ls %ls DENYING because we are servicing higher priority remote multi-server commit %ls %ls",
				*RequestId.ToString(),
				*ServerId.ToString(),
				*RequestPriority.ToString(),
				*GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId.ToString(),
				*GRemoteExecutor->ActiveRemoteMultiServerCommitPriority.ToString());
		}
	}
	else if (GRemoteExecutor->MultiServerCommitRequestId.IsValid())
	{
		// we are in a local multi-server commit - should we abort it in favor of this remote one?
		check(GRemoteExecutor->ExecutingWork);
		if (IsHigherPriority(RequestPriority, GRemoteExecutor->ExecutingWork->Priority))
		{
			UE_LOGF(LogRemoteExec, VeryVerbose, "BeginRemoteMultiServerCommit %ls from %ls %ls ACCEPTING because our local multi-server commit %ls is lower priority %ls",
				*RequestId.ToString(),
				*ServerId.ToString(),
				*RequestPriority.ToString(),
				*GRemoteExecutor->MultiServerCommitRequestId.ToString(),
				*GRemoteExecutor->ExecutingWork->Priority.ToString());

			bAcceptRequest = true;
			bAbortLocalCommit = true;
		}
		else
		{
			UE_LOGF(LogRemoteExec, VeryVerbose, "BeginRemoteMultiServerCommit %ls from %ls %ls DENYING because we are locally in multi-server commit %ls %ls",
				*RequestId.ToString(),
				*ServerId.ToString(),
				*RequestPriority.ToString(),
				*GRemoteExecutor->MultiServerCommitRequestId.ToString(),
				*GRemoteExecutor->ExecutingWork->Priority.ToString());
		}
	}
	else
	{
		// we aren't currently in a local multi-server commit, and we aren't
		// servicing a remote multi-server commit, so accept this
		UE_LOGF(LogRemoteExec, VeryVerbose, "BeginRemoteMultiServerCommit %ls %ls from %ls ACCEPTED",
			*RequestId.ToString(),
			*RequestPriority.ToString(),
			*ServerId.ToString());

		bAcceptRequest = true;
	}

	if (bAcceptRequest)
	{
		// do we have to first abandon one we're working on?
		if (GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId.IsValid())
		{
			GRemoteExecutor->Delegates->AbortRemoteMultiServerCommitDelegate.Execute(GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId, GRemoteExecutor->ActiveRemoteMultiServerCommitServerId);

			GRemoteExecutor->ActiveRemoteMultiServerCommitServerId = FRemoteServerId{};
			GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId = FRemoteTransactionId::Invalid();
			GRemoteExecutor->ActiveRemoteMultiServerCommitPriority = FRemoteWorkPriority{};
			GRemoteExecutor->bActiveRemoteMultiServerCommitReady = false;
			GRemoteExecutor->ActiveRemoteMultiServerCommitDeferredActions.Reset();
		}

		check(!GRemoteExecutor->ActiveRemoteMultiServerCommitServerId.IsValid());
		check(!GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId.IsValid());
		check(!GRemoteExecutor->ActiveRemoteMultiServerCommitPriority.IsValid());
		check(!GRemoteExecutor->bActiveRemoteMultiServerCommitReady);
		check(GRemoteExecutor->ActiveRemoteMultiServerCommitDeferredActions.Num() == 0);

		GRemoteExecutor->ActiveRemoteMultiServerCommitServerId = ServerId;
		GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId = RequestId;
		GRemoteExecutor->ActiveRemoteMultiServerCommitPriority = RequestPriority;
	}
	else
	{
		GRemoteExecutor->Delegates->AbortRemoteMultiServerCommitDelegate.Execute(RequestId, ServerId);
	}

	if (bAbortLocalCommit)
	{
		// we can't immediately abort the transaction, set this flag for it to properly shut down gracefully
		GRemoteExecutor->bMultiServerCommitRequiresAbort = true;
	}
}

void EndRemoteMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId)
{
	UE_LOGF(LogRemoteExec, VeryVerbose, "EndRemoteMultiServerCommit : %ls %ls", *ServerId.ToString(), *RequestId.ToString());

	check(GRemoteExecutor->ActiveRemoteMultiServerCommitServerId == ServerId);
	check(GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId == RequestId);

	for (TUniqueFunction<void()>& DeferredAction : GRemoteExecutor->ActiveRemoteMultiServerCommitDeferredActions)
	{
		DeferredAction();
	}

	GRemoteExecutor->ActiveRemoteMultiServerCommitServerId = FRemoteServerId{};
	GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId = FRemoteTransactionId::Invalid();
	GRemoteExecutor->ActiveRemoteMultiServerCommitPriority = FRemoteWorkPriority{};
	GRemoteExecutor->bActiveRemoteMultiServerCommitReady = false;
	GRemoteExecutor->ActiveRemoteMultiServerCommitDeferredActions.Reset();
}

void AbandonRemoteMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId)
{
	if ((GRemoteExecutor->ActiveRemoteMultiServerCommitServerId == ServerId) &&
		GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId == RequestId)
	{
		UE_LOGF(LogRemoteExec, VeryVerbose, "AbandonRemoteMultiServerCommit : %ls %ls", *ServerId.ToString(), *RequestId.ToString());

		GRemoteExecutor->ActiveRemoteMultiServerCommitServerId = FRemoteServerId{};
		GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId = FRemoteTransactionId::Invalid();
		GRemoteExecutor->ActiveRemoteMultiServerCommitPriority = FRemoteWorkPriority{};
		GRemoteExecutor->bActiveRemoteMultiServerCommitReady = false;
		GRemoteExecutor->ActiveRemoteMultiServerCommitDeferredActions.Reset();
	}
	else
	{
		UE_LOGF(LogRemoteExec, VeryVerbose, "AbandonRemoteMultiServerCommit : %ls %ls IGNORING", *ServerId.ToString(), *RequestId.ToString());
	}
}

void EnqueueRemoteMultiServerCommitAction(FRemoteServerId ServerId, FRemoteTransactionId RequestId, TUniqueFunction<void()>&& Action)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	// THIS has to sit in a holding pen...
	if ((GRemoteExecutor->ActiveRemoteMultiServerCommitServerId == ServerId) &&
		GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId == RequestId)
	{
		UE_LOGF(LogRemoteExec, Verbose, "EnqueueRemoteMultiServerCommitAction Enqueueing action from %ls request %ls", *ServerId.ToString(), *RequestId.ToString());

		GRemoteExecutor->ActiveRemoteMultiServerCommitDeferredActions.Add(MoveTemp(Action));
	}
	else
	{
		// else we've received this message for a different multi-server commit that we no longer have active, so just abandon
		// since they would have been told previously that we were aborting it
		UE_LOGF(LogRemoteExec, VeryVerbose, "EnqueueRemoteMultiServerCommitAction : IGNORING action from %ls request %ls", *ServerId.ToString(), *RequestId.ToString());
	}
#endif
}

void ReadyMultiServerCommitResponse(FRemoteServerId ServerId, FRemoteTransactionId RequestId)
{
	if (GRemoteExecutor->MultiServerCommitRequestId == RequestId)
	{
		UE_LOGF(LogRemoteExec, VeryVerbose, "ReadyMultiServerCommitResponse got ready server: %ls", *ServerId.ToString());
		GRemoteExecutor->MultiServerCommitReadyServers.Add(ServerId);
	}
	else
	{
		UE_LOGF(LogRemoteExec, VeryVerbose, "ReadyMultiServerCommitResponse ignoring %ls because we are working on %ls", *RequestId.ToString(), *GRemoteExecutor->MultiServerCommitRequestId.ToString());
	}
}

void AbortMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId)
{
	if (GRemoteExecutor->MultiServerCommitRequestId == RequestId)
	{
		UE_LOGF(LogRemoteExec, VeryVerbose, "AbortMultiServerCommit got valid: %ls", *RequestId.ToString());
		GRemoteExecutor->bMultiServerCommitRequiresAbort = true;
		GRemoteExecutor->MultiServerCommitReadyServers.Add(ServerId);
	}
	else
	{
		UE_LOGF(LogRemoteExec, VeryVerbose, "AbortMultiServerCommit ignoring %ls because we are working on %ls", *RequestId.ToString(), *GRemoteExecutor->MultiServerCommitRequestId.ToString());
	}
}

void ReadyRemoteMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId)
{
	if ((GRemoteExecutor->ActiveRemoteMultiServerCommitServerId == ServerId) &&
		(GRemoteExecutor->ActiveRemoteMultiServerCommitRequestId == RequestId))
	{
		UE_LOGF(LogRemoteExec, VeryVerbose, "ReadyRemoteMultiServerCommit %ls from %ls", *RequestId.ToString(), *ServerId.ToString());

		check(!GRemoteExecutor->bActiveRemoteMultiServerCommitReady);
		GRemoteExecutor->bActiveRemoteMultiServerCommitReady = true;

		// if everything passed OK we need to send a message back saying we're ready
		// TODO: this is where we'll need to hook code in that verifies everything
		// we received is able to be accepted
		GRemoteExecutor->Delegates->ReadyRemoteMultiServerCommitDelegate.Execute(RequestId, ServerId);
	}
}

FRemoteWorkPriority CreateRootWorkPriority()
{
	FRemoteWorkPriority RootWorkPriority = FRemoteWorkPriority::CreateRootWorkPriority(GRemoteExecutor->GetLocalServerId());
	return RootWorkPriority;
}

void ExecutePendingWork()
{
	if (GRemoteExecutor)
	{
		GRemoteExecutor->ExecutePendingWork();
	}
}

bool HasPendingWork()
{
	if (GRemoteExecutor)
	{
		return GRemoteExecutor->HasPendingWork();
	}
	return false;
}

void ExecuteTransactionalWithExplicitPriority(FName WorkName, FRemoteWorkPriority WorkPriority, const TFunctionRef<void(void)>& Work)
{
	ExecuteTransactionalInternal(WorkName, WorkPriority, Work);
}

void ExecuteTransactional(FName WorkName, const TFunctionRef<void(void)>& Work)
{
	if (GRemoteExecutor)
	{
		FRemoteWorkPriority RootWorkPriority = FRemoteWorkPriority::CreateRootWorkPriority(GRemoteExecutor->GetLocalServerId());
		ExecuteTransactionalWithExplicitPriority(WorkName, RootWorkPriority, Work);
	}
	else
	{
		Work();
	}
}

void EnqueueWorkWithExplicitPriority(FName WorkName, FRemoteWorkPriority WorkPriority, bool bIsTransactional, TUniqueFunction<void(void)>&& InWork)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	if (GRemoteExecutor)
	{
		GRemoteExecutor->EnqueueWork(WorkName, WorkPriority, bIsTransactional, INDEX_NONE, MoveTemp(InWork));
	}
	else
	{
		InWork();
	}
#else
	InWork();
#endif
}

void EnqueueWork(FName WorkName, bool bIsTransactional, TUniqueFunction<void(void)>&& InWork)
{
	if (GRemoteExecutor)
	{
		FRemoteWorkPriority RootWorkPriority = FRemoteWorkPriority::CreateRootWorkPriority(GRemoteExecutor->GetLocalServerId());
		EnqueueWorkWithExplicitPriority(WorkName, RootWorkPriority, bIsTransactional, MoveTemp(InWork));
	}
	else
	{
		InWork();
	}
}

void EnqueueMigrationWork(FName WorkName, TUniqueFunction<void(void)>&& InWork)
{
	if (GRemoteExecutor)
	{
		FRemoteWorkPriority RootWorkPriority = FRemoteWorkPriority::CreateRootWorkPriority(GRemoteExecutor->GetLocalServerId(), ERemoteWorkType::Migration);
		EnqueueWorkWithExplicitPriority(WorkName, RootWorkPriority, /*bIsTransactional=*/ true, MoveTemp(InWork));
	}
	else
	{
		InWork();
	}
}


void EnqueueWorkListWithExplicitPriority(FName WorkName, FRemoteWorkPriority WorkPriority, bool bIsTransactional, TArray<TUniqueFunction<void(void)>>&& InWorkList)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (GRemoteExecutor)
	{
		GRemoteExecutor->EnqueueWorkList(WorkName, WorkPriority, bIsTransactional, MoveTemp(InWorkList));
	}
	else
	{
#endif
	for(int32 WorkIndex = 0; WorkIndex < InWorkList.Num(); WorkIndex++)
	{
		const TUniqueFunction<void(void)>& Work = InWorkList[WorkIndex];
		Work();
	}
#if UE_WITH_REMOTE_OBJECT_HANDLE
	}
#endif
}

void EnqueueWorkList(FName WorkName, bool bIsTransactional, TArray<TUniqueFunction<void(void)>>&& InWorkList)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (GRemoteExecutor)
	{
		FRemoteWorkPriority RootWorkPriority = FRemoteWorkPriority::CreateRootWorkPriority(GRemoteExecutor->GetLocalServerId());
		EnqueueWorkListWithExplicitPriority(WorkName, RootWorkPriority, bIsTransactional, MoveTemp(InWorkList));
	}
	else
	{
#endif
	for(int32 WorkIndex = 0; WorkIndex < InWorkList.Num(); WorkIndex++)
	{
		const TUniqueFunction<void(void)>& Work = InWorkList[WorkIndex];
		Work();
	}
#if UE_WITH_REMOTE_OBJECT_HANDLE
	}
#endif
}

} // namespace UE::RemoteExecutor

const TCHAR* EnumToString(ERemoteWorkType WorkType)
{
	switch (WorkType)
	{
	case ERemoteWorkType::Migration:
		return TEXT("Migration");
	case ERemoteWorkType::Normal:
		return TEXT("Normal");
	default:
		checkf(false, TEXT("Unknown ERemoteWorkType %d"), (int32)WorkType);
		break;
	}
	return TEXT("");
}

FString FRemoteWorkPriority::ToString() const
{
	return FString::Format(TEXT("[pri: type {0} rsi {1} depth {2} id {3} ]"),
		{ EnumToString(GetWorkType()),
		*GetRootServerId().ToString(),
		GetWorkDepth(),
		GetUniqueId() });
}

FRemoteWorkPriority FRemoteWorkPriority::CreateRootWorkPriority(FRemoteServerId ServerId, ERemoteWorkType WorkType /*= ERemoteWorkType::Normal*/)
{
	FRemoteWorkPriority Result;

	uint64 RawWorkType = (uint64)WorkType;
	uint64 RawServerId = ServerId.GetIdNumber();
	uint64 RawWorkDepth = 0xFFull;
	uint64 RawUniqueId = GRemoteExecutor->GenerateNextUniqueId();

	Result.PackedData = PackPriority(RawWorkType, RawServerId, RawWorkDepth, RawUniqueId);

	checkf(WorkType == Result.GetWorkType(), TEXT("Work type (%u) was not properly encoded (%d)"), (uint32)WorkType, (uint32)Result.GetWorkType());
	checkf(ServerId == Result.GetRootServerId(), TEXT("Server id (%s) was not properly encoded (%s)"), *ServerId.ToString(), *Result.GetRootServerId().ToString());
	checkf(RawWorkDepth == Result.GetWorkDepth(), TEXT("Work depth (%llu) was not properly encoded (%u)"), RawWorkDepth, Result.GetWorkDepth());
	checkf(RawUniqueId == Result.GetUniqueId(), TEXT("Unique id (%llu) was not properly encoded (%u)"), RawUniqueId, Result.GetUniqueId());

	return Result;
}

FRemoteWorkPriority FRemoteWorkPriority::CreateDependentWorkPriority() const
{
	FRemoteWorkPriority Result;

	uint64 RawWorkType = (uint64)GetWorkType();
	uint64 RawServerId = GetRootServerId().GetIdNumber();
	uint64 RawWorkDepth = GetWorkDepth() - 1;
	uint64 RawUniqueId = GRemoteExecutor->GenerateNextUniqueId();

	Result.PackedData = PackPriority(RawWorkType, RawServerId, RawWorkDepth, RawUniqueId);

	checkf(GetWorkType() == Result.GetWorkType(), TEXT("Work type (%u) was not properly encoded (%d)"), (uint32)GetWorkType(), (uint32)Result.GetWorkType());
	checkf(GetRootServerId() == Result.GetRootServerId(), TEXT("Server id (%s) was not properly encoded (%s)"), *GetRootServerId().ToString(), *Result.GetRootServerId().ToString());
	checkf(RawWorkDepth == Result.GetWorkDepth(), TEXT("Work depth (%llu) was not properly encoded (%u)"), RawWorkDepth, Result.GetWorkDepth());
	checkf(RawUniqueId == Result.GetUniqueId(), TEXT("Unique id (%llu) was not properly encoded (%u)"), RawUniqueId, Result.GetUniqueId());

	return Result;
}

// returns if Lhs is higher priority than Rhs
bool IsHigherPriority(FRemoteWorkPriority Lhs, FRemoteWorkPriority Rhs)
{
	return Lhs.PackedData < Rhs.PackedData;
}

bool IsEqualPriority(FRemoteWorkPriority Lhs, FRemoteWorkPriority Rhs)
{
	// they're equal if neither is higher than the other
	return !IsHigherPriority(Lhs, Rhs) && !IsHigherPriority(Rhs, Lhs);
}

bool IsHigherOrEqualPriority(FRemoteWorkPriority Lhs, FRemoteWorkPriority Rhs)
{
	return IsHigherPriority(Lhs, Rhs) || IsEqualPriority(Lhs, Rhs);
}

bool operator==(FRemoteWorkPriority Lhs, FRemoteWorkPriority Rhs)
{
	return Lhs.PackedData == Rhs.PackedData;
}

FArchive& operator<<(FArchive& Ar, FRemoteWorkPriority& Priority)
{
	Ar << Priority.PackedData;
	return Ar;
}
