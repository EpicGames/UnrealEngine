// Copyright Epic Games, Inc. All Rights Reserved.
#include "UObject/RemoteHeap.h"
#include "Templates/UnrealTemplate.h"
#include "AutoRTFM.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
#include "jit_heap_ue.h"
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogRemoteHeap, Display, All);

DEFINE_LOG_CATEGORY(LogRemoteHeap);

const uint64 RemoteHeapBaseAddress   = 0xAA00000000000000ull; // as signed, must be negative

const uint64 RemoteHeapReservedBitCount = 8;
const uint64 RemoteHeapServerIdBitCount = (REMOTE_OBJECT_SERVER_ID_BIT_SIZE + 3ull) & ~3ull;
const uint64 RemoteHeapServerIdMask = (1ull << RemoteHeapServerIdBitCount) - 1ull;

const uint64 RemoteHeapLineSize = 128;
const uint64 RemoteHeapLineMask = ~(RemoteHeapLineSize-1ull);

// Prevent us from having to match this definition to the declaration in the header just to initialize the delegate
#define DEFINE_REMOTE_HEAP_DELEGATE(x) decltype(x) x
	DEFINE_REMOTE_HEAP_DELEGATE(UE::RemoteHeapSendLineDelegate);
	DEFINE_REMOTE_HEAP_DELEGATE(UE::RemoteHeapRequestLineReadDelegate);
	DEFINE_REMOTE_HEAP_DELEGATE(UE::RemoteHeapRequestLineExclusiveDelegate);
	DEFINE_REMOTE_HEAP_DELEGATE(UE::RemoteHeapExclusiveAckDelegate);
	DEFINE_REMOTE_HEAP_DELEGATE(UE::RemoteHeapDeallocateDelegate);
	DEFINE_REMOTE_HEAP_DELEGATE(UE::RemoteHeapNotifyLineDeallocatedDelegate);
#undef DEFINE_REMOTE_HEAP_DELEGATE

#if WITH_DEV_AUTOMATION_TESTS

static uint64 GRemoteHeapCoverage = 0;

namespace UE
{
void FRemoteHeapCoverage::Reset()
{
	GRemoteHeapCoverage = 0;
}

void FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow Flow)
{
	GRemoteHeapCoverage |= 1ull << (uint32)Flow;
}

uint64 FRemoteHeapCoverage::GetCoveredFlows()
{
	return GRemoteHeapCoverage;
}

bool FRemoteHeapCoverage::IsFlowCovered(ERemoteHeapCoverageFlow Flow)
{
	return (GRemoteHeapCoverage & (1ull << (uint32)Flow)) != 0;
}

uint64 FRemoteHeapCoverage::GetFullCoverageMask()
{
	static_assert((uint32)ERemoteHeapCoverageFlow::Count <= 64,
		"ERemoteHeapCoverageFlow has more than 64 entries; switch to a wider type");
	return (1ull << (uint32)ERemoteHeapCoverageFlow::Count) - 1ull;
}
} // namespace UE

#endif // WITH_DEV_AUTOMATION_TESTS

static FRemoteServerId RemoteHeapGetOwnerServerId(uint64 Address)
{
	uint32 ServerIdNum = (uint32)((Address >> (64 - RemoteHeapReservedBitCount - RemoteHeapServerIdBitCount)) & RemoteHeapServerIdMask);
	return FRemoteServerId::FromIdNumber(ServerIdNum);
}

namespace UE
{

uint64 FRemoteHeap::GetRemoteHeapBaseAddressForServerId(FRemoteServerId ServerId)
{
	uint64 ServerIdNum = ServerId.GetIdNumber();

	uint64 BaseAddress = RemoteHeapBaseAddress;
	BaseAddress |= (ServerIdNum << (64 - RemoteHeapReservedBitCount - RemoteHeapServerIdBitCount));

	return BaseAddress;
}

void RemoteHeapRedirectedLoad(void* DestPointer, uint64_t Size, uint64_t SourceAddress, bool bWillWriteHint)
{
	switch(Size)
	{
	case 8: *(uint64*)DestPointer = FRemoteHeap::Instance->RemoteHeapRead8(SourceAddress, bWillWriteHint); break;
	case 4: *(uint32*)DestPointer = FRemoteHeap::Instance->RemoteHeapRead4(SourceAddress, bWillWriteHint); break;
	case 2: *(uint16*)DestPointer = FRemoteHeap::Instance->RemoteHeapRead2(SourceAddress, bWillWriteHint); break;
	case 1: *(uint8*)DestPointer = FRemoteHeap::Instance->RemoteHeapRead1(SourceAddress, bWillWriteHint); break;
	default: FRemoteHeap::Instance->RemoteHeapRead(DestPointer, Size, SourceAddress, bWillWriteHint); break;
	}
}

void RemoteHeapRedirectedLoad8(void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint)
{
	*(uint64*)DestPointer = FRemoteHeap::Instance->RemoteHeapRead8(SourceAddress, bWillWriteHint);
}

void RemoteHeapRedirectedLoad4(void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint)
{
	*(uint32*)DestPointer = FRemoteHeap::Instance->RemoteHeapRead4(SourceAddress, bWillWriteHint);
}

void RemoteHeapRedirectedLoad2(void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint)
{
	*(uint16*)DestPointer = FRemoteHeap::Instance->RemoteHeapRead2(SourceAddress, bWillWriteHint);
}

void RemoteHeapRedirectedLoad1(void* DestPointer, uint64_t SourceAddress, bool bWillWriteHint)
{
	*(uint8*)DestPointer = FRemoteHeap::Instance->RemoteHeapRead1(SourceAddress, bWillWriteHint);
}

void RemoteHeapRedirectedStore(uint64_t DestAddress, uint64_t Size, const void* SourcePointer)
{
	switch(Size)
	{
	case 8: FRemoteHeap::Instance->RemoteHeapWrite8(DestAddress, *(uint64*)SourcePointer); break;
	case 4: FRemoteHeap::Instance->RemoteHeapWrite4(DestAddress, *(uint32*)SourcePointer); break;
	case 2: FRemoteHeap::Instance->RemoteHeapWrite2(DestAddress, *(uint16*)SourcePointer); break;
	case 1: FRemoteHeap::Instance->RemoteHeapWrite1(DestAddress, *(uint8*)SourcePointer); break;
	default: FRemoteHeap::Instance->RemoteHeapWrite(DestAddress, Size, SourcePointer); break;
	}
}

void RemoteHeapRedirectedStore8(uint64_t DestAddress, const void* SourcePointer)
{
	FRemoteHeap::Instance->RemoteHeapWrite8(DestAddress, *(uint64*)SourcePointer);
}

void RemoteHeapRedirectedStore4(uint64_t DestAddress, const void* SourcePointer)
{
	FRemoteHeap::Instance->RemoteHeapWrite4(DestAddress, *(uint32*)SourcePointer);
}

void RemoteHeapRedirectedStore2(uint64_t DestAddress, const void* SourcePointer)
{
	FRemoteHeap::Instance->RemoteHeapWrite2(DestAddress, *(uint16*)SourcePointer);
}

void RemoteHeapRedirectedStore1(uint64_t DestAddress, const void* SourcePointer)
{
	FRemoteHeap::Instance->RemoteHeapWrite1(DestAddress, *(uint8*)SourcePointer);
}

void FRemoteHeap::CreateInstance(FRemoteExecutor* Executor, FRemoteServerId LocalServerId, FRemoteHeapAllocatorOverride* AllocatorOverride)
{
	FRemoteHeap* Heap = new FRemoteHeap(LocalServerId);
	Heap->AllocatorOverride = AllocatorOverride;
	Instance = Heap;

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	RemoteExecutor::RegisterRemoteSubsystem(Executor, Heap);

	uint64 LocalServerHeapBaseAddress = GetRemoteHeapBaseAddressForServerId(LocalServerId);

	if (!AllocatorOverride)
	{
		pas_range heap_range = pas_range_create(LocalServerHeapBaseAddress, LocalServerHeapBaseAddress + RemoteHeapSizePerServer);
		jit_heap_add_fresh_memory(heap_range);
	}

	// Register the heap redirect callbacks once (they route through FRemoteHeap::Instance)
	static bool bCallbacksRegistered = false;
	if (!bCallbacksRegistered)
	{
		AutoRTFM::FHeapRedirectCallbacks Callbacks;
		// the address space is encoded as the bottom nibble of the top byte
		Callbacks.AddressSpace = (RemoteHeapBaseAddress >> 56) & 0x0F;
		Callbacks.RedirectedLoad = &RemoteHeapRedirectedLoad;
		Callbacks.RedirectedLoad8 = &RemoteHeapRedirectedLoad8;
		Callbacks.RedirectedLoad4 = &RemoteHeapRedirectedLoad4;
		Callbacks.RedirectedLoad2 = &RemoteHeapRedirectedLoad2;
		Callbacks.RedirectedLoad1 = &RemoteHeapRedirectedLoad1;
		Callbacks.RedirectedStore = &RemoteHeapRedirectedStore;
		Callbacks.RedirectedStore8 = &RemoteHeapRedirectedStore8;
		Callbacks.RedirectedStore4 = &RemoteHeapRedirectedStore4;
		Callbacks.RedirectedStore2 = &RemoteHeapRedirectedStore2;
		Callbacks.RedirectedStore1 = &RemoteHeapRedirectedStore1;
		AutoRTFM::RegisterHeapRedirectCallbacks(Callbacks);
		bCallbacksRegistered = true;
	}

	UE_LOGF(LogRemoteHeap, Display, "FRemoteHeap Initialized: BaseAddress %llx", LocalServerHeapBaseAddress);
#endif
}

void FRemoteHeap::BeginRequest()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_LOGF(LogRemoteHeap, VeryVerbose, "BeginRequest: request %ls %ls",
		*ActiveRequest->RequestId.ToString(),
		*ActiveRequest->Priority.ToString());
#endif
}

void FRemoteHeap::ForwardIndividualRequests(FRemoteServerId DestinationServer, const TArray<FRemoteHeapOutgoingLineRequest>& IndividualRequests, bool bForceNeedsData)
{
	for (int32 LineRequestIndex = 0; LineRequestIndex < IndividualRequests.Num(); LineRequestIndex++)
	{
		const FRemoteHeapOutgoingLineRequest& LineRequest = IndividualRequests[LineRequestIndex];

		if (DestinationServer != LineRequest.DestinationServerId)
		{
			if (LineRequest.RequestedAccess == ERemoteHeapLineState::Exclusive)
			{
				RemoteHeapRequestLineExclusiveDelegate.ExecuteIfBound(LineRequest.Address, DestinationServer, LineRequest.RequestPriority, LineRequest.DestinationServerId, bForceNeedsData ? true : LineRequest.bNeedsData);
			}
			else
			{
				RemoteHeapRequestLineReadDelegate.ExecuteIfBound(LineRequest.Address, DestinationServer, LineRequest.RequestPriority, LineRequest.DestinationServerId);
			}

			UE_LOGF(LogRemoteHeap, VeryVerbose, "Forwarding request for line %llx to server %ls for %ls",
				LineRequest.Address,
				*DestinationServer.ToString(),
				*LineRequest.DestinationServerId.ToString());
		}
		else
		{
			UE_LOGF(LogRemoteHeap, VeryVerbose, "Dropping request-to-itself for line %llx to server %ls",
				LineRequest.Address,
				*DestinationServer.ToString());
		}

	}
}

void FRemoteHeap::NotifyIndividualRequestsDeallocated(const TArray<FRemoteHeapOutgoingLineRequest>& IndividualRequests)
{
	for (int32 LineRequestIndex = 0; LineRequestIndex < IndividualRequests.Num(); LineRequestIndex++)
	{
		const FRemoteHeapOutgoingLineRequest& LineRequest = IndividualRequests[LineRequestIndex];

		RemoteHeapNotifyLineDeallocatedDelegate.ExecuteIfBound(LineRequest.Address, LineRequest.DestinationServerId);

		UE_LOGF(LogRemoteHeap, Verbose, "Line %llx deallocated: notifying requesting server %ls",
			LineRequest.Address,
			*LineRequest.DestinationServerId.ToString());
	}
}


SRemoteHeapLineEntry* FRemoteHeap::FindHeapLine(uint64 Address)
{
	for (SRemoteHeapLineEntry& HeapLine : HeapLines)
	{
		if (HeapLine.Address == Address)
		{
			return &HeapLine;
		}
	}
	return nullptr;
}

const SRemoteHeapLineEntry* FRemoteHeap::FindHeapLine(uint64 Address) const
{
	for (const SRemoteHeapLineEntry& HeapLine : HeapLines)
	{
		if (HeapLine.Address == Address)
		{
			return &HeapLine;
		}
	}
	return nullptr;
}

FRemoteHeap::ELineRequestAction FRemoteHeap::ProcessExclusiveLineRequest(
	FRemoteHeapOutgoingLineRequests& LineRequests,
	SRemoteHeapLineEntry& LineEntry,
	const FRemoteHeapOutgoingLineRequest& HighestPriorityRequest)
{
	//
	// they are looking for exclusive access - are they higher priority than all of our requests for the line?
	//
	bool bLineLocked = false;

	for (int32 RequestIndex = 0; RequestIndex < GetRequestCount(); RequestIndex++)
	{
		FRemoteHeapRequest* LocalRequest = GetRequestByIndex(RequestIndex);

		if (IsHigherPriority(LocalRequest->Priority, HighestPriorityRequest.RequestPriority))
		{
			bLineLocked = LocalRequest->RequestedLines.Contains(LineEntry.Address);

			if (bLineLocked)
			{
				UE_LOGF(LogRemoteHeap, VeryVerbose, "TickSubsystem line %llx locked: remote exclusive request %ls is lower priority than local request %ls",
					HighestPriorityRequest.Address,
					*HighestPriorityRequest.RequestPriority.ToString(),
					*LocalRequest->Priority.ToString());
				break;
			}
		}
	}

	if (bLineLocked)
	{
#if WITH_DEV_AUTOMATION_TESTS
		FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_ExclusiveRequest_LineLocked);
#endif
		return ELineRequestAction::Advance;
	}

	//
	// we have the line data (Shared or Exclusive) and we are giving it up to an Exclusive requestor
	//

	// Send line data if requested — the destination is the new last mutator
	if (HighestPriorityRequest.bNeedsData)
	{
		RemoteHeapSendLineDelegate.ExecuteIfBound(LineEntry.Address,
			HighestPriorityRequest.DestinationServerId, LineEntry.Data, HighestPriorityRequest.DestinationServerId,
			LineEntry.LineVersion, LineEntry.AllocationBase, LineEntry.AllocationSize);
	}

	// now build the ForwardedServers list for them (the set of servers that the requestor needs
	// to hear an Ack from before they can consider the line exclusive to them)
	//
	// three situations:
	//
	// - we have the line Exclusive, ack it with empty ForwardedServers
	//
	// - we have the line Shared AND we have a write request in-progress:
	//   - ForwardedServers is just the servers we're waiting on Acks from
	//
	// - we have the line Shared and do NOT have a write request in-progress:
	//   - ForwardedServers is the LastMutator plus the servers we shared it with
	TArray<FRemoteServerId> ForwardedServers;

	if (LineEntry.State == ERemoteHeapLineState::Exclusive)
	{
#if WITH_DEV_AUTOMATION_TESTS
		FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_ExclusiveRequest_FromExclusive);
#endif
		UE_LOGF(LogRemoteHeap, VeryVerbose, "TickSubsystem evicting line %llx (v%llu) for exclusive request from %ls, forwarded to %d children",
			LineEntry.Address,
			LineEntry.LineVersion,
			*HighestPriorityRequest.DestinationServerId.ToString(),
			ForwardedServers.Num());

		RemoteHeapExclusiveAckDelegate.ExecuteIfBound(HighestPriorityRequest.DestinationServerId,
			LineEntry.Address, LineEntry.LineVersion, ForwardedServers);
	}
	else
	{
		check(LineEntry.State == ERemoteHeapLineState::Shared);

		FRemoteHeapLineRequest* PendingWriteRequest = nullptr;

		for (int32 PendingIndex = 0; PendingIndex < PendingLineRequests.Num(); PendingIndex++)
		{
			FRemoteHeapLineRequest& PendingRequest = PendingLineRequests[PendingIndex];

			if (PendingRequest.Address == LineEntry.Address)
			{
				check(PendingRequest.RequestedAccess == ERemoteHeapLineState::Exclusive);

				PendingWriteRequest = &PendingRequest;

				// we have an in-progress request to acquire Exclusive ownership of this line
				// but we are now yielding to this requester
				//
				// we need to forward their request to all of the NeededAcks servers
				for (const FRemoteServerId& NeededAckServer : PendingRequest.NeededAcks)
				{
					if (!PendingRequest.ReceivedAcks.Contains(NeededAckServer) &&
						(NeededAckServer != HighestPriorityRequest.DestinationServerId))
					{
						ForwardedServers.Add(NeededAckServer);

						// pass bNeedsData as false here because they can get the data from us
						RemoteHeapRequestLineExclusiveDelegate.ExecuteIfBound(LineEntry.Address, NeededAckServer,
							HighestPriorityRequest.RequestPriority, HighestPriorityRequest.DestinationServerId, false);
					}
				}
			}
		}

		if (!PendingWriteRequest)
		{
#if WITH_DEV_AUTOMATION_TESTS
			FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_ExclusiveRequest_FromShared_NoPendingWrite);
#endif
			if ((LineEntry.LastMutatorServerId != HighestPriorityRequest.DestinationServerId) &&
				(LineEntry.LastMutatorServerId != LocalServerId))
			{
				// if they are not themselves the last mutator, tell them who is
				ForwardedServers.Add(LineEntry.LastMutatorServerId);

				// pass bNeedsData as false here because they can get the data from us
				RemoteHeapRequestLineExclusiveDelegate.ExecuteIfBound(LineEntry.Address, LineEntry.LastMutatorServerId,
					HighestPriorityRequest.RequestPriority, HighestPriorityRequest.DestinationServerId, false);
			}

			// tell them about everyone we shared the line with and forward their request to them
			for (const FRemoteServerId& ChildServer : LineEntry.RemoteServers)
			{
				if (ChildServer != HighestPriorityRequest.DestinationServerId)
				{
					ForwardedServers.Add(ChildServer);

					// pass bNeedsData as false here because they can get the data from us
					RemoteHeapRequestLineExclusiveDelegate.ExecuteIfBound(LineEntry.Address, ChildServer,
						HighestPriorityRequest.RequestPriority, HighestPriorityRequest.DestinationServerId, false);
				}
			}
		}

		// Send ack with forwarded servers list
		RemoteHeapExclusiveAckDelegate.ExecuteIfBound(HighestPriorityRequest.DestinationServerId,
			LineEntry.Address, LineEntry.LineVersion, ForwardedServers);

		UE_LOGF(LogRemoteHeap, VeryVerbose, "TickSubsystem evicting line %llx (v%llu) for exclusive request from %ls, forwarded to %d children",
			LineEntry.Address,
			LineEntry.LineVersion,
			*HighestPriorityRequest.DestinationServerId.ToString(),
			ForwardedServers.Num());

		if (PendingWriteRequest)
		{
#if WITH_DEV_AUTOMATION_TESTS
			FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_ExclusiveRequest_FromShared_PendingWrite);
#endif
			// if we have a pending write request, clear our NeededAcks, replace it with
			// this requestor, and forward the request to them
			PendingWriteRequest->NeededAcks.Reset();

			PendingWriteRequest->NeededAcks.Add(HighestPriorityRequest.DestinationServerId);

			// we are giving the line up, so bNeedsData = true
			RemoteHeapRequestLineExclusiveDelegate.ExecuteIfBound(LineEntry.Address, HighestPriorityRequest.DestinationServerId,
				PendingWriteRequest->RequestPriority, LocalServerId, true);
		}
	}

	// Transition to Remote
	LineEntry.State = ERemoteHeapLineState::Remote;
	LineEntry.Data.Reset();
	LineEntry.LastMutatorServerId = HighestPriorityRequest.DestinationServerId;
	LineEntry.RemoteServers.Reset();

	// Forward remaining requests to the winner:

	// first remove the highest pri from the individual requests
	LineRequests.IndividualRequests.RemoveAt(0);

	// force bNeedsData to true in this case because we expect the data
	// to be mutated before it arrives at these servers
	ForwardIndividualRequests(HighestPriorityRequest.DestinationServerId, LineRequests.IndividualRequests, true);

	return ELineRequestAction::Consumed;
}

FRemoteHeap::ELineRequestAction FRemoteHeap::ProcessSharedLineRequest(
	FRemoteHeapOutgoingLineRequests& LineRequests,
	SRemoteHeapLineEntry& LineEntry,
	const FRemoteHeapOutgoingLineRequest& HighestPriorityRequest)
{
	//
	// they are looking for shared access - we can share it unless
	// we have a higher priority Write request
	//
	bool bLineLocked = false;

	for (int32 RequestIndex = 0; RequestIndex < GetRequestCount(); RequestIndex++)
	{
		FRemoteHeapRequest* ExistingRequest = GetRequestByIndex(RequestIndex);

		if (IsHigherPriority(ExistingRequest->Priority, HighestPriorityRequest.RequestPriority))
		{
			const ERemoteHeapLineState* LocalRequestedAccess = ExistingRequest->RequestedLines.Find(LineEntry.Address);
			if (LocalRequestedAccess && *LocalRequestedAccess == ERemoteHeapLineState::Exclusive)
			{
				bLineLocked = true;

				UE_LOGF(LogRemoteHeap, VeryVerbose, "TickSubsystem line %llx locked: remote read request %ls is lower priority than local write work %ls",
					HighestPriorityRequest.Address,
					*HighestPriorityRequest.RequestPriority.ToString(),
					*ExistingRequest->Priority.ToString());
				break;
			}
		}
	}

	if (bLineLocked)
	{
#if WITH_DEV_AUTOMATION_TESTS
		FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_SharedRequest_LineLocked);
#endif
		return ELineRequestAction::Advance;
	}

	UE_LOGF(LogRemoteHeap, VeryVerbose, "TickSubsystem sending copy of line %llx to %ls %ls",
		HighestPriorityRequest.Address,
		*HighestPriorityRequest.DestinationServerId.ToString(),
		*HighestPriorityRequest.RequestPriority.ToString());

	check(HighestPriorityRequest.bNeedsData);

#if WITH_DEV_AUTOMATION_TESTS
	FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_SharedRequest_Served);
#endif

	RemoteHeapSendLineDelegate.ExecuteIfBound(LineEntry.Address,
		HighestPriorityRequest.DestinationServerId, LineEntry.Data, LineEntry.LastMutatorServerId,
		LineEntry.LineVersion, LineEntry.AllocationBase, LineEntry.AllocationSize);

	// Track the new child. If we have a pending exclusive upgrade,
	// add to NeededAcks (which is the authoritative set during acquisition)
	// and immediately tell them to evict. Otherwise add to RemoteServers.
	for (FRemoteHeapLineRequest& PendingRequest : PendingLineRequests)
	{
		if ((PendingRequest.Address == LineEntry.Address) && (PendingRequest.RequestedAccess == ERemoteHeapLineState::Exclusive))
		{
#if WITH_DEV_AUTOMATION_TESTS
			FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_SharedRequest_HasPendingExclusiveUpgrade);
#endif
			PendingRequest.NeededAcks.Add(HighestPriorityRequest.DestinationServerId);

			RemoteHeapRequestLineExclusiveDelegate.ExecuteIfBound(LineEntry.Address,
				HighestPriorityRequest.DestinationServerId,
				PendingRequest.RequestPriority, LocalServerId, false);
			break;
		}
	}

	LineEntry.RemoteServers.Add(HighestPriorityRequest.DestinationServerId);

	// If we were Exclusive, downgrade to Shared
	if (LineEntry.State == ERemoteHeapLineState::Exclusive)
	{
		LineEntry.State = ERemoteHeapLineState::Shared;
	}

	// satisfied this individual request; loop back if more remain
	LineRequests.IndividualRequests.RemoveAt(0, EAllowShrinking::No);
	return LineRequests.IndividualRequests.Num() > 0 ? ELineRequestAction::Retry : ELineRequestAction::Consumed;
}

void FRemoteHeap::TickSubsystem()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	//
	// go through the pending outgoing line requests and see if we can satisfy any of them
	//
	for (int32 Index = 0; Index < OutgoingLineRequests.Num(); )
	{
		FRemoteHeapOutgoingLineRequests& LineRequests = OutgoingLineRequests[Index];

		UE_LOGF(LogRemoteHeap, VeryVerbose, "TickSubsystem processing (%d) requests for line %llx",
			LineRequests.IndividualRequests.Num(),
			LineRequests.Address);

		SRemoteHeapLineEntry* LineEntry = FindHeapLine(LineRequests.Address);

		if (!LineEntry)
		{
			//
			// we have no record of the line locally - we should be the allocating server
			// (other servers would have this line as Remote, not missing, and have to ask us)
			//
			FRemoteServerId AllocatingServer = RemoteHeapGetOwnerServerId(LineRequests.Address);
			check(AllocatingServer == LocalServerId);

			// We are the allocating server and don't have this line — it must have been deallocated.
			// Notify each requesting server so they can clean up.
			// Note that we don't need to maintain the LineVersion across deallocation because
			// all of the RPCs are ordered, and we are the only server able to allocate the memory
			// at this address, and deallocation requires Exclusive access, so there is no
			// chance of an in-flight message for this memory line at this point, and the
			// next one will originate from us (the LineVersion will start back at 0)
#if WITH_DEV_AUTOMATION_TESTS
			FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_LineNotFound_AllocatingServer);
#endif
			NotifyIndividualRequestsDeallocated(LineRequests.IndividualRequests);
			OutgoingLineRequests.RemoveAt(Index, EAllowShrinking::No);
			continue;
		}

		if (LineEntry->State == ERemoteHeapLineState::Remote)
		{
			//
			// We have a record of the line but don't have the contents locally - forward
			// all requests to the LastMutator or the AllocatingServer
			//
#if WITH_DEV_AUTOMATION_TESTS
			FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickSubsystem_LineRemote_Forward);
#endif
			ForwardIndividualRequests(LineEntry->LastMutatorServerId, LineRequests.IndividualRequests, false);
			OutgoingLineRequests.RemoveAt(Index, EAllowShrinking::No);
			continue;
		}

		//
		// We have the line (Exclusive or Shared) - IndividualRequests is sorted by priority so let's
		// consider the highest priority
		//
		// HighestPriorityRequest is taken by value here because the process functions may modify
		// LineRequests.IndividualRequests (e.g. RemoveAt(0)) before they finish using it
		//
		FRemoteHeapOutgoingLineRequest HighestPriorityRequest = LineRequests.IndividualRequests[0];

		ELineRequestAction Action;
		if (HighestPriorityRequest.RequestedAccess == ERemoteHeapLineState::Exclusive)
		{
			Action = ProcessExclusiveLineRequest(LineRequests, *LineEntry, HighestPriorityRequest);
		}
		else
		{
			Action = ProcessSharedLineRequest(LineRequests, *LineEntry, HighestPriorityRequest);
		}

		switch (Action)
		{
			case ELineRequestAction::Consumed:
				OutgoingLineRequests.RemoveAt(Index, EAllowShrinking::No);
				break; // Index now points at the next entry

			case ELineRequestAction::Advance:
				Index++;
				break;

			case ELineRequestAction::Retry:
				break; // same Index, next IndividualRequest is now at [0]
		}
	}
#endif
}

void FRemoteHeap::TickRequest()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ActiveRequest->LocalMutatedLines.Reset();

	// go through the list of RequestedLines and see if we need to send off requests for them
	for (const auto& RequestedLinePair : ActiveRequest->RequestedLines)
	{
		uint64 Address = RequestedLinePair.Key;
		ERemoteHeapLineState RequestedAccess = RequestedLinePair.Value;
		check((RequestedAccess == ERemoteHeapLineState::Exclusive) || (RequestedAccess == ERemoteHeapLineState::Shared));

		const SRemoteHeapLineEntry* HeapLineEntry = FindHeapLine(Address);

		if (HeapLineEntry)
		{
			//
			// we have a record of the line locally, check to see if the request
			// is already satisfied by what we have
			//
			bool bSatisfied = false;

			if (RequestedAccess == ERemoteHeapLineState::Exclusive)
			{
				bSatisfied = (HeapLineEntry->State == ERemoteHeapLineState::Exclusive);
			}
			else
			{
				bSatisfied = (HeapLineEntry->State == ERemoteHeapLineState::Exclusive) || (HeapLineEntry->State == ERemoteHeapLineState::Shared);
			}

			if (bSatisfied)
			{
#if WITH_DEV_AUTOMATION_TESTS
				FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickRequest_AlreadySatisfied);
#endif
				UE_LOGF(LogRemoteHeap, VeryVerbose, "TickRequest: line %llx already satisfied", Address);
				continue;
			}
		}

		//
		// Request is not satisfied - determine the server to request it from
		//
		FRemoteServerId ExpectedOwnerServerId;

		if (HeapLineEntry)
		{
			// if we have a record of the line, then we might have a more accurate server to ask
			// (the last server that we think mutated it)
			ExpectedOwnerServerId = HeapLineEntry->LastMutatorServerId;
		}
		else
		{
			// we have no record of the line, the originating allocating server will
			// always have some record of where it might be if it isn't deallocated
			ExpectedOwnerServerId = RemoteHeapGetOwnerServerId(Address);
		}


		//
		// before we consider sending out a request for the line, let's find
		// the highest priority local request for this line
		//
		FRemoteWorkPriority HighestRequestPriority = ActiveRequest->Priority;
		for (int32 RequestIndex = 0; RequestIndex < GetRequestCount(); RequestIndex++)
		{
			FRemoteHeapRequest* ExistingRequest = GetRequestByIndex(RequestIndex);
			const ERemoteHeapLineState* ExistingRequestedAccess = ExistingRequest->RequestedLines.Find(Address);
			if (ExistingRequestedAccess && (*ExistingRequestedAccess == ERemoteHeapLineState::Exclusive) && (RequestedAccess == ERemoteHeapLineState::Exclusive))
			{
				if (IsHigherPriority(ExistingRequest->Priority, HighestRequestPriority))
				{
					HighestRequestPriority = ExistingRequest->Priority;
				}
			}
		}

		if (RequestedAccess == ERemoteHeapLineState::Exclusive)
		{
			bool bNeedsData = (!HeapLineEntry) || (HeapLineEntry->State == ERemoteHeapLineState::Remote);

			//
			// see if we have an existing (pending) line request for Exclusive
			//
			FRemoteHeapLineRequest* WriteRequest = nullptr;
			for (FRemoteHeapLineRequest& ExistingRequest : PendingLineRequests)
			{
				if ((ExistingRequest.Address == Address) && (ExistingRequest.RequestedAccess == ERemoteHeapLineState::Exclusive))
				{
					WriteRequest = &ExistingRequest;
					break;
				}
			}

			if (WriteRequest)
			{
				if (IsHigherPriority(HighestRequestPriority, WriteRequest->RequestPriority))
				{
					//
					// we have a new request for this line at a higher priority - we need to
					// re-issue requests to all servers that we expect to hear from so
					// they can reconsider our request
					//
#if WITH_DEV_AUTOMATION_TESTS
					FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickRequest_BoostExclusiveRequest);
#endif
					UE_LOGF(LogRemoteHeap, VeryVerbose, "TickRequest: boosting exclusive request for %llx to %ls",
						Address, *HighestRequestPriority.ToString());

					WriteRequest->RequestPriority = HighestRequestPriority;

					// Re-send at higher priority to all servers we're still waiting on
					for (const FRemoteServerId& NeededServer : WriteRequest->NeededAcks)
					{
						if (!WriteRequest->ReceivedAcks.Contains(NeededServer))
						{
							RemoteHeapRequestLineExclusiveDelegate.Execute(Address, NeededServer,
								WriteRequest->RequestPriority, LocalServerId, bNeedsData);
						}
					}
				}
			}
			else
			{
				//
				// we don't have an existing Write request, so we need to send it out
				//
#if WITH_DEV_AUTOMATION_TESTS
				FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickRequest_NewExclusiveRequest);
#endif

				//
				// we first check if there is an existing Read request - if it
				// is lower priority , we can discard it
				//
				for (int32 PendingIndex = 0; PendingIndex < PendingLineRequests.Num(); PendingIndex++)
				{
					FRemoteHeapLineRequest& PendingRequest = PendingLineRequests[PendingIndex];

					if ((PendingRequest.Address == Address) && (PendingRequest.RequestedAccess == ERemoteHeapLineState::Shared))
					{
						if (!IsHigherPriority(PendingRequest.RequestPriority, HighestRequestPriority))
						{
							PendingLineRequests.RemoveAtSwap(PendingIndex, EAllowShrinking::No);
						}
						break;
					}
				}

				UE_LOGF(LogRemoteHeap, VeryVerbose, "TickRequest: requesting exclusive for %llx from %ls at %ls",
					Address, *ExpectedOwnerServerId.ToString(), *HighestRequestPriority.ToString());

				WriteRequest = &PendingLineRequests.Emplace_GetRef();
				WriteRequest->Address = Address;
				WriteRequest->RequestPriority = HighestRequestPriority;
				WriteRequest->RequestedAccess = ERemoteHeapLineState::Exclusive;

				//
				// seed the NeededAcks list with the list of known servers we need to
				// receive Acks from before we can consider the line Exclusive to us.
				// each Ack that we receive may add more NeededAcks
				//
				if (ExpectedOwnerServerId != LocalServerId)
				{
					WriteRequest->NeededAcks.Add(ExpectedOwnerServerId);
					RemoteHeapRequestLineExclusiveDelegate.Execute(Address, ExpectedOwnerServerId,
						WriteRequest->RequestPriority, LocalServerId, bNeedsData);
				}

				if (HeapLineEntry)
				{
					if (HeapLineEntry->RemoteServers.Num() > 0)
					{
						// if we have RemoteServers that should mean that the 
						// line is currently Shared and we don't need the data ourselves
						check (HeapLineEntry->State == ERemoteHeapLineState::Shared);
						check (bNeedsData == false);
					}

					for (FRemoteServerId ChildServer : HeapLineEntry->RemoteServers)
					{
						WriteRequest->NeededAcks.Add(ChildServer);
						RemoteHeapRequestLineExclusiveDelegate.Execute(Address, ChildServer,
							WriteRequest->RequestPriority, LocalServerId, bNeedsData);
					}
				}
			}
		}
		else
		{
			//
			// Read-only access
			//

			// Find existing Read pending request
			FRemoteHeapLineRequest* ReadRequest = nullptr;
			for (FRemoteHeapLineRequest& ExistingRequest : PendingLineRequests)
			{
				if ((ExistingRequest.Address == Address) && (ExistingRequest.RequestedAccess == ERemoteHeapLineState::Shared))
				{
					ReadRequest = &ExistingRequest;
					break;
				}
			}

			if (ReadRequest)
			{
				//
				// we haven an existing request - do we need to bump the priority?
				//
				if (IsHigherPriority(HighestRequestPriority, ReadRequest->RequestPriority))
				{
#if WITH_DEV_AUTOMATION_TESTS
					FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickRequest_BoostReadRequest);
#endif
					UE_LOGF(LogRemoteHeap, VeryVerbose, "TickRequest: boosting read request for %llx to %ls",
						Address, *HighestRequestPriority.ToString());

					ReadRequest->RequestPriority = HighestRequestPriority;
					RemoteHeapRequestLineReadDelegate.Execute(Address, ExpectedOwnerServerId,
						ReadRequest->RequestPriority, LocalServerId);
				}
			}
			else
			{
				//
				// we don't have an existing read request, but we should only create one
				// if there isn't already a higher priority write request
				//
				bool bHigherPriorityWriteRequestExists = false;
				for (const FRemoteHeapLineRequest& ExistingRequest : PendingLineRequests)
				{
					if ((ExistingRequest.Address == Address) && (ExistingRequest.RequestedAccess == ERemoteHeapLineState::Exclusive))
					{
						bHigherPriorityWriteRequestExists = IsHigherPriority(ExistingRequest.RequestPriority, HighestRequestPriority);
						break;
					}
				}

				if (bHigherPriorityWriteRequestExists)
				{
#if WITH_DEV_AUTOMATION_TESTS
					FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickRequest_ReadSkipped_HigherPriorityWrite);
#endif
				}
				else
				{
#if WITH_DEV_AUTOMATION_TESTS
					FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::TickRequest_NewReadRequest);
#endif
					UE_LOGF(LogRemoteHeap, VeryVerbose, "TickRequest: requesting read for %llx from %ls at %ls",
						Address, *ExpectedOwnerServerId.ToString(), *HighestRequestPriority.ToString());

					ReadRequest = &PendingLineRequests.Emplace_GetRef();
					ReadRequest->Address = Address;
					ReadRequest->RequestPriority = HighestRequestPriority;
					ReadRequest->RequestedAccess = ERemoteHeapLineState::Shared;

					RemoteHeapRequestLineReadDelegate.Execute(Address, ExpectedOwnerServerId,
						ReadRequest->RequestPriority, LocalServerId);
				}
			}
		}
	}
#endif
}

void FRemoteHeap::TickAbortedRequest()
{
}

bool FRemoteHeap::AreDependenciesSatisfied() const
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	for (const auto& RequestedLinePair : ActiveRequest->RequestedLines)
	{
		uint64 Address = RequestedLinePair.Key;
		ERemoteHeapLineState NeededState = RequestedLinePair.Value;

		const SRemoteHeapLineEntry* HeapLine = FindHeapLine(Address);

		// Exclusive satisfies both Exclusive and Shared needs.
		// Shared satisfies only Shared needs.
		bool bFound = HeapLine &&
			((HeapLine->State == ERemoteHeapLineState::Exclusive) ||
			 (HeapLine->State == ERemoteHeapLineState::Shared && NeededState == ERemoteHeapLineState::Shared));

		if (!bFound)
		{
			return false;
		}
	}
#endif

	return true;
}

void FRemoteHeap::EndRequest(bool bTransactionCommitted)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (bTransactionCommitted)
	{
		// commit the mutated lines into the line cache
		for (SRemoteHeapLineData& MutatedLine : ActiveRequest->LocalMutatedLines)
		{
			if (SRemoteHeapLineEntry* HeapLine = FindHeapLine(MutatedLine.Address))
			{
				HeapLine->Data = MoveTempIfPossible(MutatedLine.Data);
			}
		}

		ActiveRequest->LocalMutatedLines.Reset();
	}
#endif
}

void FRemoteHeap::BeginMultiServerCommit(TArray<FRemoteServerId>& OutMultiServerCommitRemoteServers)
{
}

void FRemoteHeap::ExecuteMultiServerCommit()
{
}

void FRemoteHeap::AbortMultiServerCommit()
{
}

void FRemoteHeap::CommitMultiServerCommit()
{
}

void FRemoteHeap::OnLineReceived(uint64 Address, const TArray<uint8>& Data,
	FRemoteServerId InLastMutatorServerId, uint64 LineVersion,
	FRemoteServerId SenderServerId, uint64 AllocationBase, uint64 AllocationSize)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_LOGF(LogRemoteHeap, VeryVerbose, "OnLineReceived: %llx (v%llu) from LastMutator %ls", Address, LineVersion, *InLastMutatorServerId.ToString());

	SRemoteHeapLineEntry* LineEntry = FindHeapLine(Address);

	if (LineEntry != nullptr)
	{
		UE_LOGF(LogRemoteHeap, VeryVerbose, "OnLineReceived: %llx : found existing line entry", Address);
	}
	else
	{
		LineEntry = &HeapLines.Emplace_GetRef();
		LineEntry->State = ERemoteHeapLineState::Shared;
		LineEntry->Address = Address;
		UE_LOGF(LogRemoteHeap, VeryVerbose, "OnLineReceived: %llx : created new line entry", Address);
	}

	ensure(Data.Num() == RemoteHeapLineSize);
	LineEntry->Data = Data;
	LineEntry->LastMutatorServerId = InLastMutatorServerId;
	LineEntry->LineVersion = LineVersion;
	LineEntry->AllocationBase = AllocationBase;
	LineEntry->AllocationSize = AllocationSize;

	//
	// check our pending requests
	//

	// the line is at least Shared in both the read-satisfied and write-waiting cases
	LineEntry->State = ERemoteHeapLineState::Shared;

	// If a pending read request exists for this line it is now satisfied — remove it
	for (int32 PendingIndex = 0; PendingIndex < PendingLineRequests.Num(); PendingIndex++)
	{
		if (PendingLineRequests[PendingIndex].Address == Address &&
			PendingLineRequests[PendingIndex].RequestedAccess == ERemoteHeapLineState::Shared)
		{
#if WITH_DEV_AUTOMATION_TESTS
			FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::OnLineReceived_SharedRequestSatisfied);
#endif
			PendingLineRequests.RemoveAtSwap(PendingIndex, EAllowShrinking::No);
			break;
		}
	}

	// If a pending write request exists for this line, update its ack tracking
	for (FRemoteHeapLineRequest& PendingRequest : PendingLineRequests)
	{
		if ((PendingRequest.Address != Address) || (PendingRequest.RequestedAccess != ERemoteHeapLineState::Exclusive))
			continue;

		// If this line version is newer, reset ack tracking
		if (LineVersion > PendingRequest.AckVersion)
		{
			PendingRequest.NeededAcks.Reset();
			PendingRequest.ReceivedAcks.Reset();
			PendingRequest.AckVersion = LineVersion;
		}

		// The server that sent us this data must also ack (confirming it evicted its copy)
		// before we can claim exclusive ownership
		PendingRequest.NeededAcks.Add(SenderServerId);

		// I don't believe it's possible to reach this point and have all acks already
		// satisfied, because the Ack from the server sending the line always comes after
		// the sending of the line, so we definitely don't have an ack from this server yet
		check(!PendingRequest.AreAcksSatisfied());

#if WITH_DEV_AUTOMATION_TESTS
		FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::OnLineReceived_ExclusiveRequest_WaitingForAcks);
#endif
		break; // at most one Exclusive request per address
	}
#endif
}

static void AddOutgoingLineRequest(TArray<FRemoteHeapOutgoingLineRequests>& OutgoingLineRequests,
	uint64 Address, FRemoteWorkPriority Priority, FRemoteServerId Destination,
	ERemoteHeapLineState RequestedAccess, bool bNeedsData)
{
	FRemoteHeapOutgoingLineRequests* LineRequests = nullptr;

	for (FRemoteHeapOutgoingLineRequests& ExistingLineRequests : OutgoingLineRequests)
	{
		if (ExistingLineRequests.Address == Address)
		{
			LineRequests = &ExistingLineRequests;
		}
	}

	if (!LineRequests)
	{
		LineRequests = &OutgoingLineRequests.Emplace_GetRef();
		LineRequests->Address = Address;
	}

	FRemoteHeapOutgoingLineRequest* LineRequest = nullptr;

	for (FRemoteHeapOutgoingLineRequest& ExistingLineRequest : LineRequests->IndividualRequests)
	{
		if (ExistingLineRequest.DestinationServerId == Destination)
		{
			LineRequest = &ExistingLineRequest;

			// only upgrade priority, access, and bNeedsData — never downgrade
			if (IsHigherPriority(Priority, LineRequest->RequestPriority))
			{
				LineRequest->RequestPriority = Priority;
			}
			if (RequestedAccess == ERemoteHeapLineState::Exclusive)
			{
				LineRequest->RequestedAccess = ERemoteHeapLineState::Exclusive;
			}
			LineRequest->bNeedsData = LineRequest->bNeedsData || bNeedsData;
			break;
		}
	}

	if (LineRequest == nullptr)
	{
		LineRequest = &LineRequests->IndividualRequests.Emplace_GetRef();
		LineRequest->Address = Address;
		LineRequest->DestinationServerId = Destination;
		LineRequest->RequestPriority = Priority;
		LineRequest->RequestedAccess = RequestedAccess;
		LineRequest->bNeedsData = bNeedsData;
	}

	// re-sort the list by priority (either this is a new request,
	// or we may have updated the priority of an existing request)
	// Sort exclusive requests before read requests at the same priority
	LineRequests->IndividualRequests.Sort([](const FRemoteHeapOutgoingLineRequest& Lhs, const FRemoteHeapOutgoingLineRequest& Rhs) -> bool
		{
			if (IsHigherPriority(Lhs.RequestPriority, Rhs.RequestPriority))
				return true;
			if (IsHigherPriority(Rhs.RequestPriority, Lhs.RequestPriority))
				return false;

			// Same priority, sort exclusive requests first
			return (Lhs.RequestedAccess == ERemoteHeapLineState::Exclusive);
		});
}

void FRemoteHeap::OnReadLineRequested(uint64 Address, FRemoteWorkPriority Priority, FRemoteServerId Destination)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_LOGF(LogRemoteHeap, Verbose, "OnReadLineRequested: %llx %ls %ls", Address, *Priority.ToString(), *Destination.ToString());

	AddOutgoingLineRequest(OutgoingLineRequests, Address, Priority, Destination, ERemoteHeapLineState::Shared, true);
#endif
}

void FRemoteHeap::OnExclusiveRequested(uint64 Address, FRemoteWorkPriority Priority, FRemoteServerId Destination, bool bNeedsData)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_LOGF(LogRemoteHeap, Verbose, "OnExclusiveRequested: %llx %ls %ls bNeedsData=%d", Address, *Priority.ToString(), *Destination.ToString(), bNeedsData);

	AddOutgoingLineRequest(OutgoingLineRequests, Address, Priority, Destination, ERemoteHeapLineState::Exclusive, bNeedsData);
#endif
}

void FRemoteHeap::OnExclusiveAck(uint64 Address, uint64 LineVersion,
	FRemoteServerId AckingServer,
	const TArray<FRemoteServerId>& ForwardedServers)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	{
		TArray<FString> ForwardedStrings;
		ForwardedStrings.Reserve(ForwardedServers.Num());
		for (const FRemoteServerId& Server : ForwardedServers)
		{
			ForwardedStrings.Add(Server.ToString());
		}
		UE_LOGF(LogRemoteHeap, VeryVerbose, "OnExclusiveAck: %llx from %ls, forwarded %d servers [%ls]",
			Address, *AckingServer.ToString(), ForwardedServers.Num(), *FString::Join(ForwardedStrings, TEXT(", ")));
	}

	const SRemoteHeapLineEntry* HeapLineEntry = FindHeapLine(Address);

	// Find the pending exclusive request for this address
	FRemoteHeapLineRequest* MatchingRequest = nullptr;
	int32 MatchingIndex = -1;

	for (int32 PendingIndex = 0; PendingIndex < PendingLineRequests.Num(); PendingIndex++)
	{
		FRemoteHeapLineRequest& PendingRequest = PendingLineRequests[PendingIndex];
		if ((PendingRequest.Address == Address) && (PendingRequest.RequestedAccess == ERemoteHeapLineState::Exclusive))
		{
			MatchingRequest = &PendingRequest;
			MatchingIndex = PendingIndex;
			break;
		}
	}

	if (MatchingIndex == -1)
	{
		// this server has yielded the line to us and we expect more acks
		// from the list of ForwardedServers, but we are no longer requesting
		// write access to the line, so we need to forward this ack on to
		// either the last mutator or the allocating server
#if WITH_DEV_AUTOMATION_TESTS
		FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::OnExclusiveAck_Abandoned);
#endif

		FRemoteServerId ForwardServerId = HeapLineEntry ? HeapLineEntry->LastMutatorServerId : RemoteHeapGetOwnerServerId(Address);

		if (LocalServerId != ForwardServerId)
		{
			TArray<FRemoteServerId> CombinedForwarded = ForwardedServers;
			CombinedForwarded.Add(AckingServer);

			RemoteHeapExclusiveAckDelegate.ExecuteIfBound(ForwardServerId, Address, LineVersion, CombinedForwarded);

			UE_LOGF(LogRemoteHeap, VeryVerbose, "OnExclusiveAck: forwarding abandoned ack for %llx (v%llu) to %ls",
				Address, LineVersion, *ForwardServerId.ToString());
		}
	}
	else
	{
		if (LineVersion > MatchingRequest->AckVersion)
		{
			// Newer version: reset tracking and start fresh
#if WITH_DEV_AUTOMATION_TESTS
			FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::OnExclusiveAck_NewerVersion);
#endif
			UE_LOGF(LogRemoteHeap, VeryVerbose, "OnExclusiveAck: %llx newer version reset (v%llu -> v%llu) from %ls",
				Address, MatchingRequest->AckVersion, LineVersion, *AckingServer.ToString());

			MatchingRequest->NeededAcks.Reset();
			MatchingRequest->ReceivedAcks.Reset();
			MatchingRequest->AckVersion = LineVersion;
			MatchingRequest->NeededAcks = TSet<FRemoteServerId>(ForwardedServers);
			MatchingRequest->ReceivedAcks.Add(AckingServer);
		}
		else if (LineVersion == MatchingRequest->AckVersion)
		{
			// Same version: accumulate
			MatchingRequest->ReceivedAcks.Add(AckingServer);
			for (const FRemoteServerId& ForwardedServer : ForwardedServers)
			{
				MatchingRequest->NeededAcks.Add(ForwardedServer);
			}
		}
		else
		{
			// it is believed to be impossible to validly get a 'stale' ack because if
			// a server sends an ack, then it understands it has yielded the line to
			// an exclusive requester, so in order for anyone to full acquire that
			// line, all acks need to resolve. Only after all acks resolve can the
			// version be bumped, so it should not be possible to see an old ack version
			check(false); 

			UE_LOGF(LogRemoteHeap, VeryVerbose, "OnExclusiveAck: ignoring stale ack for %llx (v%llu < v%llu)",
				Address, LineVersion, MatchingRequest->AckVersion);
		}

		{
			TArray<FString> NeededStrings;
			for (const FRemoteServerId& Server : MatchingRequest->NeededAcks)
			{
				NeededStrings.Add(Server.ToString());
			}
			TArray<FString> ReceivedStrings;
			for (const FRemoteServerId& Server : MatchingRequest->ReceivedAcks)
			{
				ReceivedStrings.Add(Server.ToString());
			}

			UE_LOGF(LogRemoteHeap, VeryVerbose, "OnExclusiveAck: %llx NeededAcks=[%ls] ReceivedAcks=[%ls]",
				Address, *FString::Join(NeededStrings, TEXT(", ")), *FString::Join(ReceivedStrings, TEXT(", ")));
		}

		// Check if all acks collected and we have the data
		if (MatchingRequest->AreAcksSatisfied())
		{
			SRemoteHeapLineEntry* LineEntry = FindHeapLine(Address);

			// all acks are satisfied - the line data arrives before the ack from any particular
			// server, so we should already have the data
			check(LineEntry && LineEntry->Data.Num() > 0);

			// Promote to Exclusive
#if WITH_DEV_AUTOMATION_TESTS
			FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::OnExclusiveAck_SameVersion_Promoted);
#endif
			LineEntry->State = ERemoteHeapLineState::Exclusive;
			LineEntry->LineVersion++;
			LineEntry->LastMutatorServerId = LocalServerId;
			LineEntry->RemoteServers.Reset();

			PendingLineRequests.RemoveAtSwap(MatchingIndex, EAllowShrinking::No);

			UE_LOGF(LogRemoteHeap, VeryVerbose, "OnExclusiveAck: line %llx promoted to Exclusive", Address);
		}
	}
#endif
}

void FRemoteHeap::OnLineDeallocated(uint64 Address)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_LOGF(LogRemoteHeap, Verbose, "OnLineDeallocated: %llx", Address);
#if WITH_DEV_AUTOMATION_TESTS
	FRemoteHeapCoverage::RecordFlow(ERemoteHeapCoverageFlow::OnLineDeallocated);
#endif

	// Remove from HeapLines
	for (int32 LineIndex = 0; LineIndex < HeapLines.Num(); LineIndex++)
	{
		if (HeapLines[LineIndex].Address == Address)
		{
			HeapLines.RemoveAtSwap(LineIndex, EAllowShrinking::No);
			break;
		}
	}

	// Remove all pending requests for this address (both Read and Write)
	for (int32 PendingIndex = 0; PendingIndex < PendingLineRequests.Num(); PendingIndex++)
	{
		int32 PendingIndexReversed = PendingLineRequests.Num() - PendingIndex - 1;
		if (PendingLineRequests[PendingIndexReversed].Address == Address)
		{
			PendingLineRequests.RemoveAtSwap(PendingIndexReversed, EAllowShrinking::No);
		}
	}

	// Remove from all active requests' RequestedLines maps so AreDependenciesSatisfied() no longer blocks
	for (int32 RequestIndex = 0; RequestIndex < GetRequestCount(); RequestIndex++)
	{
		FRemoteHeapRequest* Request = GetRequestByIndex(RequestIndex);
		Request->RequestedLines.Remove(Address);
	}
#endif
}

void FRemoteHeap::ExtractHeapRangeReadonly(uint64 StartAddress, uint64 Size, void* DstData, bool bWillWriteHint)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	uint64 LineAddressCursor = StartAddress & RemoteHeapLineMask;
	FRemoteServerId OwnerServerId = RemoteHeapGetOwnerServerId(StartAddress); 

	UE_LOGF(LogRemoteHeap, VeryVerbose, "ExtractHeapRangeReadonly StartAddress: %llx Size %llu | Owner %ls | bWillWriteHint %d", StartAddress, Size, *OwnerServerId.ToString(), bWillWriteHint);

	uint8* DstCursor = (uint8*)DstData;
	uint64 DstAddressCursor = StartAddress;
	uint64 DstSizeRemaining = Size;

	TArray<uint64, TInlineAllocator<8>> PendingUnavailable;

	while (DstSizeRemaining)
	{
		uint64 LineOffset = DstAddressCursor - LineAddressCursor;
		uint64 SizeToCopy = FMath::Min(DstSizeRemaining, RemoteHeapLineSize - LineOffset);
		check(SizeToCopy > 0);
		check(SizeToCopy <= Size);

		bool bFound = false;

		// is this line locally mutable?
		for(SRemoteHeapLineData& HeapLine : ActiveRequest->LocalMutatedLines)
		{
			if (HeapLine.Address == LineAddressCursor)
			{
				UE_LOGF(LogRemoteHeap, VeryVerbose, "ExtractHeapRangeReadonly: line %llx found in mutable section", LineAddressCursor);
					
				// found the data in the mutable section
				FMemory::Memcpy(DstCursor, &HeapLine.Data[(uint32)LineOffset], SizeToCopy);

				bFound = true;
				break;
			}
		}

		if (!bFound)
		{
			// check the global cache
			if (SRemoteHeapLineEntry* HeapLine = FindHeapLine(LineAddressCursor))
			{
				if (HeapLine->State == ERemoteHeapLineState::Exclusive ||
					HeapLine->State == ERemoteHeapLineState::Shared)
				{
					UE_LOGF(LogRemoteHeap, VeryVerbose, "ExtractHeapRangeReadonly: line %llx found in readonly section", LineAddressCursor);
					FMemory::Memcpy(DstCursor, &HeapLine->Data[(uint32)LineOffset], SizeToCopy);

					bFound = true;
				}
			}
		}

		if (!bFound)
		{
			PendingUnavailable.Add(LineAddressCursor);
		}

		LineAddressCursor += RemoteHeapLineSize;

		DstCursor += SizeToCopy;
		DstAddressCursor += SizeToCopy;
		DstSizeRemaining -= SizeToCopy;
	}

	if (PendingUnavailable.Num() > 0)
	{
		UE_AUTORTFM_OPEN
		{
			for (uint64 UnavailableLine : PendingUnavailable)
			{
				if (bWillWriteHint)
				{
					// Write subsumes Read — set Write directly
					ActiveRequest->RequestedLines.FindOrAdd(UnavailableLine) = ERemoteHeapLineState::Exclusive;
				}
				else
				{
					// if the line is already requested, don't overwrite it (it might be requested for
					// write, which we don't want to downgrade)
					ERemoteHeapLineState* Existing = ActiveRequest->RequestedLines.Find(UnavailableLine);
					if (!Existing)
					{
						ActiveRequest->RequestedLines.Add(UnavailableLine, ERemoteHeapLineState::Shared);
					}
				}
			}
		};

		if (bWillWriteHint)
		{
			RemoteExecutor::AbortTransactionRequiresDependencies(TEXT("remote heap read (with write hint)"));
		}
		else
		{
			RemoteExecutor::AbortTransactionRequiresDependencies(TEXT("remote heap read"));
		}
	}
#endif
}

void FRemoteHeap::InsertHeapRangeMutable(uint64 StartAddress, uint64 Size, const void* SrcData)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	uint64 LineAddressCursor = StartAddress & RemoteHeapLineMask;
	FRemoteServerId OwnerServerId = RemoteHeapGetOwnerServerId(StartAddress);

	UE_LOGF(LogRemoteHeap, VeryVerbose, "InsertHeapRangeMutable StartAddress: %llx Size %llu | Owner %ls", StartAddress, Size, *OwnerServerId.ToString());

	const uint8* SrcCursor = (const uint8*)SrcData;
	uint64 SrcAddressCursor = StartAddress;
	uint64 SrcSizeRemaining = Size;

	TArray<uint64, TInlineAllocator<8>> PendingUnavailable;

	while (SrcSizeRemaining)
	{
		uint64 LineOffset = SrcAddressCursor - LineAddressCursor;
		uint64 SizeToCopy = FMath::Min(SrcSizeRemaining, RemoteHeapLineSize - LineOffset);
		check(SizeToCopy > 0);
		check(SizeToCopy <= Size);

		bool bFound = false;

		// is this line locally mutable?
		for(SRemoteHeapLineData& MutableHeapLine : ActiveRequest->LocalMutatedLines)
		{
			if (MutableHeapLine.Address == LineAddressCursor)
			{
				// found the data in the mutable section

				UE_LOGF(LogRemoteHeap, VeryVerbose, "InsertHeapRangeMutable: line %llx found in mutable section", LineAddressCursor);
				FMemory::Memcpy(&MutableHeapLine.Data[(uint32)LineOffset], SrcCursor, SizeToCopy);

				bFound = true;
				break;
			}
		}

		if (!bFound)
		{
			// can we pull a line into the mutable section?
			if (SRemoteHeapLineEntry* HeapLine = FindHeapLine(LineAddressCursor))
			{
				if (HeapLine->State == ERemoteHeapLineState::Exclusive)
				{
					UE_LOGF(LogRemoteHeap, VeryVerbose, "InsertHeapRangeMutable: line %llx found in readonly section, moving to mutable", LineAddressCursor);

					// make a copy of the cached line into the mutable section
					SRemoteHeapLineData& MutableHeapLine = ActiveRequest->LocalMutatedLines.Emplace_GetRef(*HeapLine);
					FMemory::Memcpy(&MutableHeapLine.Data[(uint32)LineOffset], SrcCursor, SizeToCopy);

					bFound = true;
				}
			}
		}

		if (!bFound)
		{
			PendingUnavailable.Add(LineAddressCursor);
		}

		LineAddressCursor += RemoteHeapLineSize;

		SrcCursor += SizeToCopy;
		SrcAddressCursor += SizeToCopy;
		SrcSizeRemaining -= SizeToCopy;
	}

	if (PendingUnavailable.Num() > 0)
	{
		UE_AUTORTFM_OPEN
		{
			for (uint64 UnavailableLine : PendingUnavailable)
			{
				// Write subsumes Read — set Write directly
				ActiveRequest->RequestedLines.FindOrAdd(UnavailableLine) = ERemoteHeapLineState::Exclusive;
			}
		};

		RemoteExecutor::AbortTransactionRequiresDependencies(TEXT("remote heap write"));
	}
#endif
}

uint8 FRemoteHeap::RemoteHeapRead1(uint64 Addr, bool bWillWriteHint)
{
	uint8 Result = 0;
	ExtractHeapRangeReadonly(Addr, sizeof(Result), &Result, bWillWriteHint);

	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapRead1: %llx %02x", Addr, (uint32)Result);
	return Result;
}

void FRemoteHeap::RemoteHeapWrite1(uint64 Addr, uint8 Value)
{
	InsertHeapRangeMutable(Addr, sizeof(Value), &Value);
	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapWrite1: %llx %02x", Addr, (uint32)Value);
}

uint16 FRemoteHeap::RemoteHeapRead2(uint64 Addr, bool bWillWriteHint)
{
	uint16 Result = 0;
	ExtractHeapRangeReadonly(Addr, sizeof(Result), &Result, bWillWriteHint);

	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapRead2: %llx %04x", Addr, Result);
	return Result;
}

void FRemoteHeap::RemoteHeapWrite2(uint64 Addr, uint16 Value)
{
	InsertHeapRangeMutable(Addr, sizeof(Value), &Value);
	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapWrite2: %llx %04x", Addr, Value);
}

uint32 FRemoteHeap::RemoteHeapRead4(uint64 Addr, bool bWillWriteHint)
{
	uint32 Result = 0;
	ExtractHeapRangeReadonly(Addr, sizeof(Result), &Result, bWillWriteHint);

	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapRead4: %llx %08x", Addr, Result);
	return Result;
}

void FRemoteHeap::RemoteHeapWrite4(uint64 Addr, uint32 Value)
{
	InsertHeapRangeMutable(Addr, sizeof(Value), &Value);
	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapWrite4: %llx %08x", Addr, Value);
}

uint64 FRemoteHeap::RemoteHeapRead8(uint64 Addr, bool bWillWriteHint)
{
	uint64 Result = 0;
	ExtractHeapRangeReadonly(Addr, sizeof(Result), &Result, bWillWriteHint);

	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapRead8: %llx %016llx", Addr, Result);
	return Result;
}

void FRemoteHeap::RemoteHeapWrite8(uint64 Addr, uint64 Value)
{
	InsertHeapRangeMutable(Addr, sizeof(Value), &Value);
	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapWrite8: %llx %016llx", Addr, Value);
}

void FRemoteHeap::RemoteHeapRead(void* DestPointer, uint64 Size, uint64 SourceAddress, bool bWillWriteHint)
{
	ExtractHeapRangeReadonly(SourceAddress, Size, DestPointer, bWillWriteHint);
	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapRead: %llx (size %llu)", SourceAddress, Size);
}

void FRemoteHeap::RemoteHeapWrite(uint64 DestAddress, uint64 Size, const void* SourcePointer)
{
	InsertHeapRangeMutable(DestAddress, Size, SourcePointer);
	UE_LOGF(LogRemoteHeap, VeryVerbose, "RemoteHeapWrite: %llx (size %llu)", DestAddress, Size);
}

void* FRemoteHeap::RemoteHeapAllocate(uint64 Size)
{
	void* Result = nullptr;

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	// pad allocations to a multiple of the line size
	// so separate allocations never get shared and can
	// be migrated separately
	Size = (Size + RemoteHeapLineSize-1) & RemoteHeapLineMask;

	UE_AUTORTFM_OPEN
	{
		if (AllocatorOverride)
			Result = AllocatorOverride->Allocate(AllocatorOverride->Context, Size);
		else
			Result = jit_heap_try_allocate(Size);
	};

	UE_AUTORTFM_ONABORT(this, Result)
	{
		if (AllocatorOverride)
			AllocatorOverride->Deallocate(AllocatorOverride->Context, Result);
		else
			jit_heap_deallocate(Result);
	};

	uint64 AllocationBase = (uint64)Result;
	uint64 LineAddressCursor = AllocationBase;
	uint64 LineAddressEnd = LineAddressCursor + Size;

	while(LineAddressCursor < LineAddressEnd)
	{
		SRemoteHeapLineEntry& NewHeapLine = FRemoteHeap::Instance->HeapLines.Emplace_GetRef();
		NewHeapLine.Address = LineAddressCursor;
		NewHeapLine.State = ERemoteHeapLineState::Exclusive;
		NewHeapLine.LineVersion = 1;
		NewHeapLine.LastMutatorServerId = FRemoteHeap::Instance->LocalServerId;
		NewHeapLine.Data.SetNum(RemoteHeapLineSize);
		NewHeapLine.AllocationBase = AllocationBase;
		NewHeapLine.AllocationSize = Size;

		LineAddressCursor += RemoteHeapLineSize;
	}

	UE_LOGF(LogRemoteHeap, Verbose, "RemoteHeapAllocate: %llx (size %llu)", Result, Size);

#endif

	return Result;
}

void FRemoteHeap::RemoteHeapDeallocate(void* Ptr)
{
	if (Ptr == nullptr)
		return;

#if UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM
	uint64 Address = 0;
	FMemory::Memcpy(&Address, &Ptr, sizeof(uint64));

	uint64 LineAddress = Address & RemoteHeapLineMask;
	check(Address == LineAddress);
	FRemoteServerId OwnerServer = RemoteHeapGetOwnerServerId(Address);

	if (OwnerServer != LocalServerId)
	{
		// We are not the allocating server - forward the deallocation to the owner.
		// The allocating server is responsible for acquiring Exclusive access to all
		// lines before freeing, so we don't need to do that here. Our HeapLines
		// entries will be cleaned up via OnLineDeallocated once the allocating server
		// completes the deallocation.
		UE_AUTORTFM_ONCOMMIT(this, Address, OwnerServer)
		{
			UE_LOGF(LogRemoteHeap, Verbose, "RemoteHeapDeallocate (remote %ls): %llx", *OwnerServer.ToString(), Address);
			RemoteHeapDeallocateDelegate.ExecuteIfBound(Address, OwnerServer);
		};
		return;
	}


	// Find the first line to get the allocation metadata
	const SRemoteHeapLineEntry* FirstLine = FindHeapLine(LineAddress);
	check(FirstLine != nullptr);
	uint64 AllocBase = FirstLine->AllocationBase;
	uint64 AllocSize = FirstLine->AllocationSize;

	check(AllocBase != 0 && AllocSize != 0);
	check(Address == AllocBase);

	// We are the allocating server - require Exclusive access on ALL lines before
	// freeing to ensure all caches are evicted across all servers, otherwise we risk
	// stale data if the same address range is allocated again later
	bool bAllExclusive = true;

	for (uint64 LineCursor = AllocBase; LineCursor < AllocBase + AllocSize; LineCursor += RemoteHeapLineSize)
	{
		const SRemoteHeapLineEntry* HeapLine = FindHeapLine(LineCursor);
		if (!HeapLine || HeapLine->State != ERemoteHeapLineState::Exclusive)
		{
			bAllExclusive = false;
			break;
		}
	}

	if (!bAllExclusive)
	{
		// Request Write access for ALL lines in the allocation, not just the ones
		// currently non-Exclusive, because Exclusive lines could be evicted before retry
		UE_AUTORTFM_OPEN
		{
			for (uint64 LineCursor = AllocBase; LineCursor < AllocBase + AllocSize; LineCursor += RemoteHeapLineSize)
			{
				ActiveRequest->RequestedLines.FindOrAdd(LineCursor) = ERemoteHeapLineState::Exclusive;
			}
		};
		RemoteExecutor::AbortTransactionRequiresDependencies(TEXT("remote heap deallocate requires exclusive"));
		return;
	}

	UE_AUTORTFM_ONCOMMIT(this, Ptr, Address, AllocBase, AllocSize)
	{
		UE_LOGF(LogRemoteHeap, Verbose, "RemoteHeapDeallocate (local): %llx", Address);
		if (AllocatorOverride)
		{
			AllocatorOverride->Deallocate(AllocatorOverride->Context, (void*)Ptr);
		}
		else
		{
			jit_heap_deallocate((void*)Ptr);
		}

		// Remove HeapLines entries for all lines in this allocation
		for (int32 LineIndex = 0; LineIndex < HeapLines.Num(); LineIndex++)
		{
			int32 LineIndexReversed = HeapLines.Num() - LineIndex - 1;
			if (HeapLines[LineIndexReversed].AllocationBase == AllocBase)
			{
				HeapLines.RemoveAtSwap(LineIndexReversed, EAllowShrinking::No);
			}
		}
	};

#endif
}

void RemoteHeapOnLineReceived(uint64 Address, const TArray<uint8>& Data,
	FRemoteServerId LastMutatorServerId, uint64 LineVersion,
	FRemoteServerId SenderServerId,
	uint64 AllocationBase, uint64 AllocationSize)
{
	FRemoteHeap::Instance->OnLineReceived(Address, Data, LastMutatorServerId, LineVersion, SenderServerId, AllocationBase, AllocationSize);
}

void RemoteHeapOnLineRequested(uint64 Address, FRemoteWorkPriority Priority, FRemoteServerId Destination)
{
	FRemoteHeap::Instance->OnReadLineRequested(Address, Priority, Destination);
}

void RemoteHeapOnExclusiveRequested(uint64 Address, FRemoteWorkPriority Priority,
	FRemoteServerId Destination, bool bNeedsData)
{
	FRemoteHeap::Instance->OnExclusiveRequested(Address, Priority, Destination, bNeedsData);
}

void RemoteHeapOnExclusiveAck(uint64 Address, uint64 LineVersion,
	FRemoteServerId AckingServer,
	const TArray<FRemoteServerId>& ForwardedServers)
{
	FRemoteHeap::Instance->OnExclusiveAck(Address, LineVersion, AckingServer, ForwardedServers);
}

void RemoteHeapOnLineDeallocated(uint64 Address)
{
	FRemoteHeap::Instance->OnLineDeallocated(Address);
}

void RemoteHeapDeallocate(uint64 Address)
{
	FRemoteHeap::Instance->RemoteHeapDeallocate((void*)Address);
}

void FRemoteHeapAllocator::ForAnyElementType::ResizeAllocation(FRemoteHeapAllocator::SizeType CurrentNum, FRemoteHeapAllocator::SizeType NewMax, FRemoteHeapAllocator::SizeType NumBytesPerElement)
{
	if (NewMax == 0)
	{
		FRemoteHeap::Instance->RemoteHeapDeallocate(Allocation);
		Allocation = nullptr;
	}
	else
	{
		void* NewMemory = FRemoteHeap::Instance->RemoteHeapAllocate(NewMax * NumBytesPerElement);

		if (CurrentNum != 0)
		{
			FMemory::Memcpy(NewMemory, Allocation, CurrentNum * NumBytesPerElement); //-V575
		}

		FRemoteHeap::Instance->RemoteHeapDeallocate(Allocation);
		Allocation = NewMemory;
	}
}

FRemoteHeapAllocator::ForAnyElementType::~ForAnyElementType()
{
	FRemoteHeap::Instance->RemoteHeapDeallocate(Allocation);
	Allocation = nullptr;
}

bool RemoteObject::IsPointerOnRemoteHeap(const void* Pointer)
{
	union
	{
		const void* P;
		uint64 Value;
	};

	P = Pointer;
	return (Value & 0xFF00000000000000ull) == RemoteHeapBaseAddress;
}

} // namespace UE
