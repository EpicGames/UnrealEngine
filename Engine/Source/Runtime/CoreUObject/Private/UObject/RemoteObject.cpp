// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/RemoteObject.h"

#include "UObject/RemoteExecutor.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Class.h"
#include "UObject/GCObject.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "UObject/GarbageCollection.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/RemoteObjectSerialization.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/ObjectHandlePrivate.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectMigrationContext.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/ObjectHandle.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "String/LexFromString.h"
#include "Templates/Casts.h"
#include "Hash/CityHash.h"
#include "Modules/VisualizerDebuggingState.h"
#include <atomic>

DEFINE_LOG_CATEGORY(LogRemoteObject);



int32 GRemoteIdToStringVerbosity = (int32)ERemoteIdToStringVerbosity::Id;
static FAutoConsoleVariableRef CVarRemoteIdToStringVerbosity(
	TEXT("ro.IdToStringVerbosity"),
	GRemoteIdToStringVerbosity,
	TEXT("Sets the verbosity FRemoteObjectId::ToString() prints the id to log with (1: id only, 2: id + name, 3: id + pathname, 4: id + fullname, 5: id + fullname + attributes"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
		{
			if (GRemoteIdToStringVerbosity <= (int32)ERemoteIdToStringVerbosity::Default || GRemoteIdToStringVerbosity > (int32)ERemoteIdToStringVerbosity::Max)
			{
				GRemoteIdToStringVerbosity = (int32)ERemoteIdToStringVerbosity::Id;
			}
		}));

FRemoteServerId FRemoteServerId::GlobalServerId(ERemoteServerIdConstants::Local); // Here Local means uninitialized

void AssignGlobalServerIdDebuggingState()
{
	// CF8B6D3D-3185-453C-AF12-88EB19245359 => cf8b6d3d3185453caf1288eb19245359
	constexpr FGuid GGlobalServerIdGuid = FGuid(0xCF8B6D3D, 0x3185453C, 0xAF1288EB, 0x19245359);

	(void)::UE::Core::FVisualizerDebuggingState::Assign(GGlobalServerIdGuid, &FRemoteServerId::GlobalServerId);
}

void FRemoteServerId::InitGlobalServerId(FRemoteServerId Id)
{
	// Guard against re-initializing the global id unless remote object support is disabled and the id has been initialized to invalid value
	checkf(GlobalServerId.Id == (uint32)ERemoteServerIdConstants::Local || (!FRemoteObjectId::RemoteObjectSupportCompiledIn && !GlobalServerId.IsValid()), 
		TEXT("Global server id has already been initialized (%s)"), *GlobalServerId.ToString());
	GlobalServerId = Id;
}

bool FRemoteServerId::IsGlobalServerIdInitialized()
{
	// This is the only place we check if the id is valid.
	// We can be running a build with remote object support compiled in but disabled in which case we initialize GlobalServerId to an invalid value
	// but we still want to return false in this case because of existing use cases that check this value to determine if remote objects are enabled
	return GlobalServerId.Id != (uint32)ERemoteServerIdConstants::Local;
}


namespace UE::RemoteObject::Private
{

std::atomic<uint64> RemoteObjectSerialNumber(1);
std::atomic<uint64> AssetObjectSerialNumber(1);
int32 UnsafeToMigrateObjects = 0; // This should go into TLS
int32 GForceReturnObjectHandles = 0; // This should go into TLS
int32 GInitRemoteServerIdBeforeUObjectInit = 0; // Initializes remote server id before UObject system is initialized
thread_local bool GIsTransactionallyPostMigratingObjects = false;

class FRemoteObjectStubMap : private TMap<FRemoteObjectId, UE::RemoteObject::Handle::FRemoteObjectStub*>
{
	using Super = TMap<FRemoteObjectId, UE::RemoteObject::Handle::FRemoteObjectStub*>;

public:

	using Super::begin;
	using Super::end;

	virtual ~FRemoteObjectStubMap()
	{
		for (TPair<FRemoteObjectId, UE::RemoteObject::Handle::FRemoteObjectStub*>& Pair : *this)
		{
			delete Pair.Value;
		}
	}

	UE::RemoteObject::Handle::FRemoteObjectStub*& FindOrAdd(FRemoteObjectId Id)
	{		
		return Super::FindOrAdd(UE::RemoteObject::Private::FRemoteIdLocalizationHelper::GetLocalized(Id));
	}

	UE::RemoteObject::Handle::FRemoteObjectStub* Find(FRemoteObjectId Id) const
	{
		using namespace UE::RemoteObject::Handle;

		UE::RemoteObject::Handle::FRemoteObjectStub* const* ExistingStub = Super::Find(UE::RemoteObject::Private::FRemoteIdLocalizationHelper::GetLocalized(Id));
		return ExistingStub ? *ExistingStub : nullptr;
	}
};

class FRemoteObjectMaps
{
	mutable FTransactionallySafeCriticalSection ObjectMapCritical;
	// Maps remote object id to to a stub. Note that at the moment stubs are only destroyed on exit (this is required by FRemoteObjectHandlePrivate)
	FRemoteObjectStubMap RemoteObjects;
	// Maps remote object id to to a pathname. Note that at the moment these pathnames are only destroyed on exit (this is required by FRemoteObjectClass)
	TMap<FRemoteObjectId, FRemoteObjectPathName*> AssetPaths;

public:

	virtual ~FRemoteObjectMaps()
	{
		for (TPair<FRemoteObjectId, FRemoteObjectPathName*>& Pair : AssetPaths)
		{
			delete Pair.Value;
		}

	}
	UE::RemoteObject::Handle::FRemoteObjectStub* FindStub(FRemoteObjectId Id)
	{
		using namespace UE::RemoteObject::Handle;

		UE::TScopeLock MapLock(ObjectMapCritical);
		return RemoteObjects.Find(Id);
	}

	UE::RemoteObject::Handle::FRemoteObjectStub* FindOrAddStub(FRemoteObjectId Id, FRemoteServerId ResidentServerId = FRemoteServerId())
	{
		using namespace UE::RemoteObject::Handle;

		UE::TScopeLock MapLock(ObjectMapCritical);
		FRemoteObjectStub*& Stub = RemoteObjects.FindOrAdd(Id);
		if (!Stub)
		{
			Stub = new FRemoteObjectStub();
			Stub->Id = Id;

			// if we are creating the stub, then this object's owner is deduced from its ID
			// if the server ID is invalid then it's a local native object that was created before the local server had its ID assigned
			FRemoteServerId ObjectServerId = Id.GetServerId();
			Stub->OwningServerId = ObjectServerId.IsValid() ? ObjectServerId : FRemoteServerId::GetLocalServerId();
			Stub->ResidentServerId = ResidentServerId.IsValid() ? ResidentServerId : Stub->OwningServerId;
		}
		else if (ResidentServerId.IsValid())
		{
			// Update the ResidentServerId only if it's valid
			Stub->ResidentServerId = ResidentServerId;
		}
		return Stub;
	}

	UE::RemoteObject::Handle::FRemoteObjectStub* FindOrAddStub(const UObject* Object, FRemoteServerId ResidentServerId = FRemoteServerId())
	{
		using namespace UE::RemoteObject::Handle;
		using namespace UE::CoreUObject::Private;

		checkf(Object, TEXT("Unable to add a stub for a null object"));

		UE::TScopeLock MapLock(ObjectMapCritical);
		FRemoteObjectStub* Stub = FindOrAddStub(FRemoteObjectId(Object), ResidentServerId);

		// Always try and update the stub with the most recent information about the object
		FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
		checkf(ObjectItem, TEXT("Attempting to get a serial number for an object that does not exist in the global UObject array (it's possible GUObjectArray is not initialized yet, ObjectIndex=%d)"), GUObjectArray.ObjectToIndex(Object));
		Stub->SerialNumber = ObjectItem->GetSerialNumber();
		Stub->Name = Object->GetFName();
		Stub->bWasGarbage = !IsValid(Object);
		if (UObject* Outer = FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Object))
		{
			Stub->OuterId = FObjectHandleUtils::GetRemoteId(Outer);
		}
		// The class of an object never changes so we only need to set it once
		if (!Stub->Class.IsValid())
		{
			Stub->Class = FRemoteObjectClass(Object->GetClass());
		}

		return Stub;
	}

	FRemoteObjectPathName* StoreAssetPath(UObject* InObject)
	{
		FRemoteObjectId ObjectId(InObject);
		FRemoteObjectPathName*& Path = AssetPaths.FindOrAdd(ObjectId);
		if (!Path)
		{
			Path = new FRemoteObjectPathName(InObject);
		}
		return Path;
	}

	FRemoteObjectPathName* FindAssetPath(FRemoteObjectId ObjectId)
	{
		FRemoteObjectPathName** PathName = AssetPaths.Find(ObjectId);
		return PathName ? *PathName : nullptr;
	}

	void ForEachStubWithMigratedData(TFunctionRef<bool(UE::RemoteObject::Handle::FRemoteObjectStub*)> Operation)
	{
		using namespace UE::RemoteObject::Handle;

		UE::TScopeLock MapLock(ObjectMapCritical);
		for (TPair<FRemoteObjectId,FRemoteObjectStub*>& Pair : RemoteObjects)
		{
			FRemoteObjectStub* Stub = Pair.Value;
			if (Stub->MigratedData && Stub->MigratedData->State != EMigratedDataState::Completed)
			{
				if (!Operation(Stub))
				{
					return;
				}
			}
		}
	}
};
FRemoteObjectMaps* ObjectMaps = nullptr;

void InitServerId()
{
	FRemoteServerId GlobalServerId;
	FString ServerId;
	const TCHAR* CommandLine = FCommandLine::Get();
	if (!FParse::Value(CommandLine, TEXT("MultiServerLocalId="), ServerId))
	{
		if (!FParse::Value(CommandLine, TEXT("LocalPeerId="), ServerId))
		{
			int ListenPort = 0;
			if (FParse::Param(CommandLine, TEXT("MultiServerLocalHost")) && FParse::Value(CommandLine, TEXT("Port="), ListenPort))
			{
				if (ListenPort > 0)
				{
					ServerId = FString::FromInt(ListenPort % 1000);
				}
			}
		}
	}
	if (!ServerId.IsEmpty())
	{
		GlobalServerId = FRemoteServerId::FromString(ServerId);
		checkf(GlobalServerId.IsValid(), TEXT("Remote ServerId is not valid"));

		FRemoteServerId::InitGlobalServerId(GlobalServerId);
	}

	UE_LOGF(LogRemoteObject, Display, "Global Server Id: %ls", *FRemoteServerId::GetLocalServerId().ToString());
}

void InitRemoteObjects()
{
	AssignGlobalServerIdDebuggingState();

	if (!FRemoteObjectId::RemoteObjectSupportCompiledIn)
	{
		FRemoteServerId::InitGlobalServerId(FRemoteServerId());
		// Always init global server id debug visualizer support but early out if remote object support is not compiled in (UE_WITH_REMOTE_OBJECT_HANDLE is 0)
		return;
	}

	ObjectMaps = new FRemoteObjectMaps();
	
	if (GConfig)
	{
		// Get the config value now before it's first used
		GConfig->GetInt(TEXT("Core.System.Experimental"), TEXT("ro.InitRemoteServerIdBeforeUObjectInit"), GInitRemoteServerIdBeforeUObjectInit, GEngineIni);
	}
	if (GInitRemoteServerIdBeforeUObjectInit)
	{
		InitServerId();
	}

	if (!UE::RemoteObject::Transfer::RemoteObjectTransferDelegate.IsBound())
	{
		UE::RemoteObject::Transfer::RemoteObjectTransferDelegate.BindStatic(&UE::RemoteObject::Serialization::Disk::SaveObjectToDisk);
	}
	if (!UE::RemoteObject::Transfer::RequestRemoteObjectDelegate.IsBound())
	{
		UE::RemoteObject::Transfer::RequestRemoteObjectDelegate.BindLambda(
			[](FRemoteWorkPriority RequestPriority, FRemoteObjectId ObjectId, FRemoteServerId LastKnownResidentServerId, FRemoteServerId DestinationServerId)
			{
				// Turns a request into an immediate load
				FUObjectMigrationContext MigrationContext {
					.ObjectId = ObjectId, .RemoteServerId = DestinationServerId, .OwnerServerId = LastKnownResidentServerId,
					.MigrationSide = EObjectMigrationSide::Receive
				};
				UE::RemoteObject::Serialization::Disk::LoadObjectFromDisk(MigrationContext);
			}
		);
	}
	if (!UE::RemoteObject::Transfer::StoreRemoteObjectDataDelegate.IsBound())
	{
		UE::RemoteObject::Transfer::StoreRemoteObjectDataDelegate.BindStatic(&UE::RemoteObject::Serialization::Disk::SaveObjectToDisk);
	}
	if (!UE::RemoteObject::Transfer::RestoreRemoteObjectDataDelegate.IsBound())
	{
		UE::RemoteObject::Transfer::RestoreRemoteObjectDataDelegate.BindStatic(&UE::RemoteObject::Serialization::Disk::LoadObjectFromDisk);
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	// We rely on garbage elimination being disabled because we don't allow it inside of borrowed objects
	if (UObjectBaseUtility::IsGarbageEliminationEnabled())
	{
		UE_LOGF(LogRemoteObject, Warning, "Disabling garbage elimination support because remote object support is enabled and these features are mutually exclusive");
		UObjectBaseUtility::SetGarbageEliminationEnabled(false);
	}	
#endif
}

void ShutdownRemoteObjects()
{
	delete ObjectMaps;
	ObjectMaps = nullptr;
}

UE::RemoteObject::Handle::FRemoteObjectStub* FindOrAddRemoteObjectStub(FRemoteObjectId ObjectId, FRemoteServerId ResidentServerId)
{
	checkf(ObjectMaps, TEXT("FindOrAddRemoteObjectStub(%s, %s): Remote object system is not initialized"), *ObjectId.ToString(), *ResidentServerId.ToString());
	return ObjectMaps->FindOrAddStub(ObjectId, ResidentServerId);
}

void RegisterSharedObject(UObject* Object)
{
	MarkAsRemoteReference(Object);
}

void SetResidence(UObject* Object, EResidence Residence, FRemoteServerId ResidentServerId)
{
	using namespace UE::RemoteObject::Handle;

	static_assert(sizeof(FObjectHandle) == sizeof(UObject*));

	checkf(ObjectMaps, TEXT("Trying to set residence for %s to %s but the remote object system is not initialized!"), *GetPathNameSafe(Object), EnumToString(Residence));
	checkf(Object, TEXT("Trying to set residence for a null object!"));
	checkf(Residence == EResidence::Remote || ResidentServerId.IsLocal(), TEXT("Setting residence for %s to %s with a non-local server id (%s)"), *GetPathNameSafe(Object), EnumToString(Residence), *ResidentServerId.ToString());
	ensureMsgf(!Object->HasAnyFlags(EObjectFlags::RF_ClassDefaultObject | EObjectFlags::RF_ArchetypeObject), TEXT("We're about to set an ArchetypeObject %s as remote reference"), *GetNameSafe(Object));
	
	EInternalObjectFlags ClearFlags = EInternalObjectFlags::None;
	EInternalObjectFlags SetFlags = EInternalObjectFlags::None;

	switch (Residence)
	{
	case EResidence::Local:
		{
			ClearFlags = (EInternalObjectFlags::Remote | EInternalObjectFlags::LocalNotReady);
			SetFlags = EInternalObjectFlags::RemoteReference;
		}
		break;
	case EResidence::LocalNotReady:
		{
			ClearFlags = EInternalObjectFlags::Remote;
			SetFlags = (EInternalObjectFlags::RemoteReference | EInternalObjectFlags::LocalNotReady);
		}
		break;
	case EResidence::Remote:
		{
			ClearFlags = (EInternalObjectFlags_RootFlags | EInternalObjectFlags::RemoteReference | EInternalObjectFlags::Borrowed | EInternalObjectFlags::LocalNotReady);
			SetFlags = EInternalObjectFlags::Remote;			
		}
		break;
	default:
		checkf(false, TEXT("Attempted to set an unknown residence (%d) for %s"), (int32)Residence, *GetPathNameSafe(Object));
		break;
	}

	FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
	ObjectItem->ClearFlags(ClearFlags);
	ObjectItem->SetFlags(SetFlags);
	if (FRemoteObjectStub* Stub = ObjectMaps->FindOrAddStub(Object, ResidentServerId)) // FindOrAddStub always needs to be called. It's not only for debugging purposes
	{
		UE_LOGF(LogRemoteObject, VeryVerbose, "Setting residence for %ls %ls to %ls", *Stub->Id.ToString(), *GetPathNameSafe(Object), EnumToString(Residence));

		checkf(Residence != EResidence::LocalNotReady || Stub->MigratedData, TEXT("Setting residence for %s to LocalNotReady is only possible if the stub references migrated data"), 
			*Stub->Id.ToString(ERemoteIdToStringVerbosity::PathName));
		checkf(Residence == EResidence::LocalNotReady || !Stub->MigratedData, TEXT("Setting residence for %s to %s is only possible if the stub does NOT reference migrated data"), 
			*Stub->Id.ToString(ERemoteIdToStringVerbosity::PathName), EnumToString(Residence));
	}
}

void MarkAsRemote(UObject* Object, FRemoteServerId ResidentServerId)
{
	SetResidence(Object, EResidence::Remote, ResidentServerId);
}

void MarkAsRemoteReference(const UObject* Object)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
	ObjectItem->SetFlags(EInternalObjectFlags::RemoteReference);
#endif
}

bool IsRemoteReference(const UObject* Object)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return Object->HasAnyInternalFlags(EInternalObjectFlags::RemoteReference);
#else
	return false;
#endif
}

void MarkAsBorrowed(UObject* Object)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
	ObjectItem->SetFlags(EInternalObjectFlags::Borrowed);
#endif
}

bool IsBorrowed(const UObject* Object)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return Object->HasAnyInternalFlags(EInternalObjectFlags::Borrowed);
#else
	return false;
#endif
}

void MarkAsLocal(UObject* Object)
{
	SetResidence(Object, EResidence::Local, FRemoteServerId::GetLocalServerId());
}

void StoreAssetPath(UObject* Object)
{
	// Make sure the asset has a stub and that the stub knows the owner if this asset is the asset server (disk / content)
	ObjectMaps->FindOrAddStub(Object, FRemoteServerId(ERemoteServerIdConstants::Asset));
	ObjectMaps->StoreAssetPath(Object);
}

FRemoteObjectPathName* FindAssetPath(FRemoteObjectId RemoteId)
{
	return ObjectMaps->FindAssetPath(RemoteId);
}

UE::RemoteObject::Handle::FRemoteObjectStub* FindRemoteObjectStub(FRemoteObjectId ObjectId)
{
	return ObjectMaps ? ObjectMaps->FindStub(ObjectId) : nullptr;
}

FName GetServerBaseNameForUniqueName(const UClass* Class)
{
	using namespace UE::RemoteObject;

	checkf(Class, TEXT("Unable to generate base name for a unique object name without the object's Class"));

	// Packages follow different naming rules than other UObjects and ATM we're not migrating packages so fall back to Class->GetFName()
	if (FRemoteServerId::IsGlobalServerIdInitialized() && Class->GetFName() != NAME_Package)
	{
		return *FString::Printf(TEXT("%s_S%s"), *Class->GetFName().GetPlainNameString(), *FRemoteServerId::GetLocalServerId().ToString());
	}
	return Class->GetFName();
}

FUnsafeToMigrateScope::FUnsafeToMigrateScope()
{
	UnsafeToMigrateObjects++;
}
FUnsafeToMigrateScope::~FUnsafeToMigrateScope()
{
	UnsafeToMigrateObjects--;
	check(UnsafeToMigrateObjects >= 0);
}

bool IsSafeToMigrateObjects()
{
	// Not a thread safe test but atm we assume we're running single-threaded
	return !(GIsGarbageCollecting || UnsafeToMigrateObjects);
}

bool IsForceReturnObjectHandles()
{
	return GForceReturnObjectHandles > 0;
}

bool IsRemoteObjectSystemInitialized()
{
	return !!ObjectMaps;
}

FScopedForceReturnObjectHandles::FScopedForceReturnObjectHandles()
{
	Private::GForceReturnObjectHandles++;
}

FScopedForceReturnObjectHandles::~FScopedForceReturnObjectHandles()
{
	Private::GForceReturnObjectHandles--;
	check(Private::GForceReturnObjectHandles >= 0);
}

void ForEachStubWithMigratedData(TFunctionRef<bool(UE::RemoteObject::Handle::FRemoteObjectStub*)> Operation)
{
	if (ObjectMaps)
	{
		ObjectMaps->ForEachStubWithMigratedData(Operation);
	}
}

void SetTransactionallyPostMigratingObjects(bool bIsPostMigrating)
{
	checkf(IsInGameThread(), TEXT("Starting to transactionally post migrate objects outside of the game thread. This is not supported."))
	GIsTransactionallyPostMigratingObjects = bIsPostMigrating;
}

} // namespace UE::RemoteObject::Private

namespace UE::RemoteObject::Handle
{

FRemoteObjectClass::FRemoteObjectClass(UClass* InClass)
{
	checkf(InClass, TEXT("FRemoteClassStub requires a valid class"));
	
	if (InClass->IsNative())
	{
		// Native classes are never GC'd so we can just store a raw pointer to a class object
		PathNameOrClass = UPTRINT(InClass);
	}
	else
	{
		// Blueprints and Verse classes are assets that can be GC'd after an object of such class is sent to another server
		// Since we don't want a strong reference to an asset class we store a pathname instead (here we take advantage of the fact that pathnames are never destroyed, see FRemoteObjectMaps)
		// We could cache the class object here for faster access but it wouldn't remove the requirement of storing its pathname (because the class may get GC'd and we want to be able to load it)
		// An alternative to this approach would be to store the class object's remote id but that would require one more lookup (id -> pathname)
		// It's also important to keep this structure small so that FRemoteObjectStub is also small because stubs are never destroyed atm
		PathNameOrClass = UPTRINT(UE::RemoteObject::Private::ObjectMaps->StoreAssetPath(InClass)) | 1;
	}
}

UClass* FRemoteObjectClass::GetClass() const
{
	if (IsNative())
	{
		return reinterpret_cast<UClass*>(PathNameOrClass);
	}
	else
	{
		const FRemoteObjectPathName* PathName = reinterpret_cast<const FRemoteObjectPathName*>(PathNameOrClass & ~UPTRINT(1));
		// Resolve() will first attempt to find the class in memory and if it doesn't exist, load it.
		return Cast<UClass>(PathName->Resolve());
	}
}

FRemoteObjectStub::FRemoteObjectStub(UObject* Object)
{
	using namespace UE::CoreUObject::Private;

	if (Object)
	{
		Id = FObjectHandleUtils::GetRemoteId(Object);
		Class = FRemoteObjectClass(Object->GetClass());
		bWasGarbage = !IsValid(Object);
		if (UObject* Outer = FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Object))
		{
			OuterId = FObjectHandleUtils::GetRemoteId(Outer);
		}
	}
}

EResidence GetResidence(FRemoteObjectId ObjectId)
{
	using namespace UE::RemoteObject::Private;

	if (!ObjectId.IsValid() || !ObjectMaps)
	{
		return EResidence::Local;
	}

	if (UObject* Object = StaticFindObjectFastInternal(ObjectId))
	{
		return GetResidence(Object);
	}

	// if a stub exists but we don't have the object's memory (the UObject doesn't exist) then the object must be remote
	if (UE::RemoteObject::Handle::FRemoteObjectStub* Stub = ObjectMaps->FindStub(ObjectId))
	{
		return Stub->MigratedData ? EResidence::LocalNotReady : EResidence::Remote;
	}

	const FRemoteServerId ServerId = ObjectId.GetServerId();
	// Invalid server Id means local native classes which are created before a server has a chance to have an id assigned
	return (ServerId.IsValid() && !ServerId.IsLocal() && !ServerId.IsAsset()) ? EResidence::Remote : EResidence::Local;
}

EResidence GetResidence(const FRemoteObjectStub* Stub)
{
	if (!Stub)
	{
		return EResidence::Local;
	}

	if (UObject* Object = StaticFindObjectFastInternal(Stub->Id))
	{
		return GetResidence(Object);
	}

	// Similar to the above GetResidence(ObjectId): if a stub exists but we don't have the object's memory (the UObject doesn't exist) then the object must be remote
	return Stub->MigratedData ? EResidence::LocalNotReady : EResidence::Remote;
}

EResidence GetResidence(const UObject* Object)
{
	using namespace UE::RemoteObject::Private;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Object && ObjectMaps)
	{
		const int32 InternalIndex = GUObjectArray.ObjectToIndex(Object);
		const FUObjectItem* ObjectItem = InternalIndex >= 0 ? GUObjectArray.IndexToObject(InternalIndex) : nullptr;
		if (ObjectItem && !ObjectItem->HasAnyFlags(EInternalObjectFlags::PendingConstruction))
		{
			if (ObjectItem->HasAnyFlags(EInternalObjectFlags::LocalNotReady))
			{
				return EResidence::LocalNotReady;
			}
			else if (ObjectItem->HasAnyFlags(EInternalObjectFlags::Remote))
			{
				return EResidence::Remote;
			}
		}
		return EResidence::Local;
	}
#endif
	return EResidence::Local;
}

bool IsRemote(FRemoteObjectId ObjectId)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return GetResidence(ObjectId) == EResidence::Remote;
#else
	return false;
#endif
}

bool IsRemote(const FRemoteObjectStub* Stub)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return GetResidence(Stub) == EResidence::Remote;
#else
	return false;
#endif
}

bool IsRemote(const UObject* Object)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return GetResidence(Object) == EResidence::Remote;
#else
	return false;
#endif
}

bool IsOwned(const UObject* Object)
{
	using namespace UE::CoreUObject::Private;
	return IsOwned(FObjectHandleUtils::GetRemoteId(Object));
}

bool IsOwned(FRemoteObjectId ObjectId)
{
	using namespace UE::CoreUObject::Private;
	using namespace UE::RemoteObject::Private;

	bool bResult = true;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE::RemoteObject::Handle::FRemoteObjectStub* RemoteStub = FindRemoteObjectStub(ObjectId);
	if (RemoteStub)
	{
		bResult = (RemoteStub->OwningServerId == FRemoteServerId::GetLocalServerId() || RemoteStub->OwningServerId.IsAsset());
	}
	else
	{
		const FRemoteServerId ServerId = ObjectId.GetServerId();
		// Invalid server Id means local native objects which were created before the local server had a chance to have an id assigned
		bResult = (!ServerId.IsValid() || ServerId.IsAsset() || ServerId.IsLocal());
	}
#endif
	return bResult;
}

bool IsValid(FRemoteObjectId ObjectId)
{
	using namespace UE::RemoteObject::Private;

	// Note: this function needs to reflect the logic of ResolveObject(...) functions

	if (UObject* Object = StaticFindObjectFastInternal(ObjectId))
	{		
		return IsValid(Object);
	}

	if (FRemoteObjectStub* Stub = ObjectMaps->FindStub(ObjectId))
	{
		return !Stub->bWasGarbage;
	}

	// ObjectId represents an object that has never been migrated
	return false;
}

FRemoteServerId GetOwnerServerId(FRemoteObjectId ObjectId)
{
	using namespace UE::CoreUObject::Private;
	using namespace UE::RemoteObject::Private;

	FRemoteServerId Result;
	if (FRemoteObjectStub* RemoteStub = FindRemoteObjectStub(ObjectId))
	{
		Result = RemoteStub->OwningServerId;
	}
	else
	{
		// if the object wasn't received or ever migrated, use the encoded server id. This may not reflect the actual state of the object since we don't track ownership across servers if we don't own the object
		Result = ObjectId.GetServerId();
	}
	return Result;
}

FRemoteServerId GetOwnerServerId(const UObject* Object)
{
	return GetOwnerServerId(FRemoteObjectId(Object));
}

void ChangeOwnerServerId(const UObject* Object, FRemoteServerId NewOwnerServerId)
{
	using namespace UE::CoreUObject::Private;
	using namespace UE::RemoteObject::Private;
	
#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE::RemoteObject::Handle::FRemoteObjectStub* RemoteStub = FindRemoteObjectStub(FObjectHandleUtils::GetRemoteId(Object));

	// The remote stub is always expected to be found for this object.
	if (ensureMsgf(RemoteStub, TEXT("Missing stub for %s (%s / 0x%016llx)"), *GetPathNameSafe(Object), *FRemoteObjectId(Object).ToString(), (int64)(PTRINT)Object))
	{
		RemoteStub->OwningServerId = NewOwnerServerId;
	}
#endif
}

FRemoteObjectId GetRemoteObjectId(TFunctionRef<const UObject*()> CodeThatReturnsRawUObjectPtr)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	Private::FScopedForceReturnObjectHandles DisallowMigrationsAndForceHandlesToBeReturned;
	
	FObjectHandle ObjectHandle;
	ObjectHandle.AssignFromRawPayload( BitCast<UPTRINT>(CodeThatReturnsRawUObjectPtr()) );
	return ObjectHandle.GetRemoteId();
#else
	return FRemoteObjectId{};
#endif
}

UObject* ResolveObject(FRemoteObjectStub* Stub, ERemoteReferenceType RefType /*= ERemoteReferenceType::Strong*/)
{
	using namespace UE::RemoteExecutor;
	using namespace UE::RemoteObject::Transfer;
	using namespace UE::RemoteObject::Private;

	// This is a slightly faster version of IsRemote(FRemoteObjectId) because we already know a Stub exists and we are going to re-use the Object pointer
	UObject* Object = StaticFindObjectFastInternal(Stub->Id);

	if (!Object && Stub->OwningServerId == FRemoteServerId(ERemoteServerIdConstants::Asset))
	{
		if (FRemoteObjectPathName* AssetPath = FindAssetPath(Stub->Id))
		{
			Object = AssetPath->Resolve();
		}
	}

	if (!IsSafeToMigrateObjects() && (Object || RefType == ERemoteReferenceType::Weak)) // Not a thread-safe test
	{
		// Begin/FinishDestroy overrides may attempt to access subobjects of objects that have been migrated in which case we
		// don't want to accidentally migrate them back mid-purge and if the Object memory is still valid (but has EInternalObjectFlags::Remote flag)
		// we can just return it and let the owner finish its cleanup.
		// In case of weak object ptrs it's relatively safe to just return null if the object doesn't exist on this server (see CanResolveObject)
		TouchResidentObject(Object);
		return Object;
	}

	if (Stub->MigratedData)
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (IsForceReturnObjectHandles())
		{
			// Re-encode this as the FObjectHandle (which has bitfield rules)
			return reinterpret_cast<UObject*>(FObjectHandle(Stub).Payload);
		}
#endif

		TransactionallyMigrateObjects(Stub->MigratedData);
	
		Object = StaticFindObjectFastInternal(Stub->Id);
		checkf(Object, TEXT("Tried to migrate %s from migrated data but the object does not exist"), *Stub->Id.ToString());
		return Object;
	}

	bool bRemoteObject = !Object || GetResidence(Object) == EResidence::Remote;
	if (bRemoteObject)
	{
		#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (IsForceReturnObjectHandles())
		{
			// Re-encode this as the FObjectHandle (which has bitfield rules)
			return reinterpret_cast<UObject*>(FObjectHandle(Stub).Payload);
		}
		#endif

		checkf(!GIsGarbageCollecting, TEXT("Resolving remote objects while collecting garbage is not allowed (trying to resolve object %s (%s)"), *Stub->Id.ToString(), *Stub->Name.ToString());

		MigrateObjectFromRemoteServer(Stub->Id, Stub->ResidentServerId);

		// if running transactionally (and not speculating), we will have aborted and not reached here
		// if running transactionally (and speculating), we will use a stale object to speculate and abort later.

		// if running non-transactionally, object migrated immediately, so we just re-resolve
		Object = StaticFindObjectFastInternal(Stub->Id);
		bRemoteObject = !Object || GetResidence(Object) == EResidence::Remote;
		checkf(IsSpeculationMode() || !bRemoteObject, TEXT("Failed to resolve remote object %s, either this code is not running in a transaction and should be, or the transaction failed to abort"), *Stub->Id.ToString());
	}

	return Object;
}

UObject* ResolveObject(UObject* Object, ERemoteReferenceType RefType /*= ERemoteReferenceType::Strong*/)
{
	using namespace UE::RemoteObject::Transfer;
	using namespace UE::RemoteObject::Private;

	// Begin/FinishDestroy overrides may attempt to access subobjects of objects that have been migrated in which case we
	// don't want to accidentally migrate them back mid-purge and if the Object memory is still valid (but has EInternalObjectFlags::Remote flag)
	// we can just return it and let the owner finish its cleanup.
	if (IsSafeToMigrateObjects()) // Note: this is not a thread-safe check
	{
		FRemoteObjectStub* Stub = ObjectMaps->FindStub(UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(Object));
		checkf(Stub, TEXT("Failed to find remote object stub for %s"), *GetPathNameSafe(Object));
		return ResolveObject(Stub, RefType);
	}

	TouchResidentObject(Object);
	return Object;
}

void TouchResidentObject(UObject* Object)
{
	UE::RemoteObject::Transfer::TouchResidentObject(Object);
}

bool CanResolveObject(FRemoteObjectId ObjectId, ECanResolveObjectBehavior CanResolveObjectBehavior)
{
	using namespace UE::RemoteObject::Private;

	// Note: this function needs to reflect the logic of ResolveObject(...) functions
	
	if (UObject* Object = StaticFindObjectFastInternal(ObjectId))
	{
		// If we want to treat garbage objects as unresolvable, check the flag
		if (CanResolveObjectBehavior == ECanResolveObjectBehavior::CannotResolveGarbageObjects)
		{
			return IsValid(Object);
		}

		// Object memory is local and even if it's already been migrated we can resolve it
		return true;
	}

	if (!ObjectMaps)
	{
		return false;
	}

	if (FRemoteObjectStub* Stub = ObjectMaps->FindStub(ObjectId))
	{
		if (Stub->MigratedData)
		{
			return true;
		}

		// A stub exists so the object memory is not local but we can (attempt to) migrate it back if:
		// 1. It's not already Garbage (unless we're allowing a migration of Garbage objects)
		if (CanResolveObjectBehavior == ECanResolveObjectBehavior::CannotResolveGarbageObjects && Stub->bWasGarbage)
		{
			return false;
		}

		// 2. We're not speculating (in that case Get() can return nullptr, so make sure this is false)
		if (UE::RemoteExecutor::IsSpeculationMode())
		{
			// We also need to request the object here, or code that branches on IsValid would be wrong.
			using namespace UE::RemoteObject::Transfer;
			MigrateObjectFromRemoteServer(Stub->Id, Stub->ResidentServerId);

			return false;
		}

		// 3. We're not explicitly saying it's unsafe to migrate objects (FUnsafeToMigrateScope)
		// 4. During Garbage Collection. Note: GIsGarbageCollecting checks are not thread safe
		return IsSafeToMigrateObjects();
	}

	// ObjectId is local or represents an object that has never been migrated
	return false;
}

UClass* GetClass(FRemoteObjectId ObjectId, ERemoteObjectGetClassBehavior GetClassBehavior)
{
	if (FRemoteObjectStub* Stub = Private::FindRemoteObjectStub(ObjectId))
	{
		if (Stub->Class.IsValid())
		{
			return Stub->Class.GetClass();
		}
		else if (GetClassBehavior == ERemoteObjectGetClassBehavior::ReturnNullIfNeverLocal)
		{
			return nullptr;
		}
		else if (GetClassBehavior == ERemoteObjectGetClassBehavior::MigrateIfNeverLocal)
		{
			UObject* ResolvedObject = ResolveObject(Stub, ERemoteReferenceType::Strong);
			return ResolvedObject->GetClass();
		}
	}

	return nullptr;
}

bool IsTransactionallyPostMigratingObjects()
{
	return UE::RemoteObject::Private::GIsTransactionallyPostMigratingObjects;
}

} // namespace UE::RemoteObject::Handle

namespace UE::CoreUObject::Private
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	void FObjectHandleUtils::ChangeRemoteId(UObjectBase* Object, FRemoteObjectId Id)
	{
		using namespace UE::RemoteObject::Private;
		UnhashObject(Object);
		FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
		// ObjectItem may not exist when the UObject system hasn't been initialized yet but theoretically this function should only get called when
		// something attempts to re-construct a default subobject that already exists so ObjectItem should always be valid
		checkf(ObjectItem, TEXT("Attempting to change remote ID for an object that does not exist in the global UObject array (it's possible GUObjectArray is not initialized yet, ObjectIndex=%d)"), GUObjectArray.ObjectToIndex(Object));
		ObjectItem->SetRemoteId(Id);
		HashObject(Object);
	}

	FRemoteObjectId FRemoteObjectHandlePrivate::GetRemoteId() const
	{
		if (ExtractHandleType() == ERemoteObjectHandleType::RemoteId)
		{
			return ExtractRemoteId();
		}
		else if (ExtractHandleType() == ERemoteObjectHandleType::StubPointer)
		{
			return ExtractStubPointer()->Id;
		}
		else if (ExtractHandleType() == ERemoteObjectHandleType::ObjectPointer)
		{
			return FRemoteObjectId(ExtractObjectPointer());
		}

		return FRemoteObjectId{};
	}

	FRemoteObjectHandlePrivate FRemoteObjectHandlePrivate::ConvertToRemoteHandle(UObject* Object)
	{
		using namespace UE::RemoteObject::Handle;
		using namespace UE::RemoteObject::Private;		

		FRemoteObjectStub* Stub = ObjectMaps->FindStub(UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(Object));
		checkf(Stub, TEXT("Failed to find remote object stub for %s"), *GetPathNameSafe(Object));

		FRemoteObjectHandlePrivate Handle;
		Handle.AssignFromStubPointer(Stub);
		return Handle;
	}

	FRemoteObjectHandlePrivate FRemoteObjectHandlePrivate::FromIdNoResolve(FRemoteObjectId ObjectId)
	{
		using namespace UE::RemoteObject::Handle;
		using namespace UE::RemoteObject::Private;

		UObject* Obj = nullptr;
		if (ObjectId.IsValid())
		{
			Obj = StaticFindObjectFastInternal(ObjectId);
			if (Obj && !Obj->HasAnyInternalFlags(EInternalObjectFlags::Remote))
			{
				return FRemoteObjectHandlePrivate(Obj);
			}
			else if (const FRemoteObjectStub* const FoundStub = ObjectMaps->FindStub(ObjectId))
			{
				return FRemoteObjectHandlePrivate(FoundStub);
			}
			else if (!Obj && ObjectId.IsLocal())
			{
				// If we don't have an existing stub or object, and the ID is local, we can assume
				// the object was never migrated or referenced by another server and is now destroyed.
				// Create a null handle instead of using the RemoteId because there's no remote object to find.
				return FRemoteObjectHandlePrivate(Obj);
			}
			else if (FRemoteObjectStub* Stub = ObjectMaps->FindOrAddStub(ObjectId))
			{
				return FRemoteObjectHandlePrivate(Stub);
			}
			check(false);
		}
		return FRemoteObjectHandlePrivate(Obj);
	}

	void FRemoteObjectHandlePrivate::AssignFromRawPayload(UPTRINT InPayload)
	{
		Payload = InPayload;
	}

	void FRemoteObjectHandlePrivate::AssignFromStubPointer(const UE::RemoteObject::Handle::FRemoteObjectStub* Value)
	{
		if (RemoteObject::IsPointerOnRemoteHeap(this))
		{
			Payload = ((uint8)ERemoteObjectHandleType::RemoteId & HandleTypeMask) | (BitCast<UPTRINT>(Value->Id) & PayloadMask);
		}
		else
		{
			Payload = ((uint8)ERemoteObjectHandleType::StubPointer & HandleTypeMask) | (BitCast<UPTRINT>(Value) & PayloadMask);
		}
	}

	void FRemoteObjectHandlePrivate::AssignFromObjectPointer(const UObject* Value)
	{
		if (RemoteObject::IsPointerOnRemoteHeap(this))
		{
			Payload = ((uint8)ERemoteObjectHandleType::RemoteId & HandleTypeMask) | (BitCast<UPTRINT>(FObjectHandleUtils::GetRemoteId(Value)) & PayloadMask);
		}
		else
		{
			Payload = ((uint8)ERemoteObjectHandleType::ObjectPointer & HandleTypeMask) | (BitCast<UPTRINT>(Value) & PayloadMask);
		}
	}

	void FRemoteObjectHandlePrivate::AssignFromRemoteId(FRemoteObjectId Value)
	{
		Payload = ((uint8)ERemoteObjectHandleType::RemoteId & HandleTypeMask) | (BitCast<UPTRINT>(Value) & PayloadMask);
	}

	bool UE::CoreUObject::Private::FRemoteObjectHandlePrivate::UEOpEquals(FRemoteObjectHandlePrivate RHS) const
	{
		using namespace UE::CoreUObject::Private;
		using namespace UE::RemoteObject::Handle;

		// if Lhs and Rhs are both ObjectPointers, we compare the actual pointers
		// as not every object will have a valid FRemoteObjectId
		if (ExtractHandleType() == ERemoteObjectHandleType::ObjectPointer)
		{
			if (RHS.ExtractHandleType() == ERemoteObjectHandleType::ObjectPointer)
			{
				return ExtractObjectPointer() == RHS.ExtractObjectPointer();
			}
		}

		// however, if one of the two is not an ObjectPointer, then at least that one has
		// a FRemoteObjectId and we can try to get a remote id from both of them to compare
		return GetRemoteId() == RHS.GetRemoteId();
	}

	UE::RemoteObject::Handle::FRemoteObjectStub* UE::CoreUObject::Private::FRemoteObjectHandlePrivate::ToStub() const
	{
		using namespace UE::RemoteObject::Handle;

		if (ExtractHandleType() == ERemoteObjectHandleType::RemoteId)
		{
			FRemoteObjectId RemoteId = ExtractRemoteId();
			FRemoteObjectStub* StubPointer = RemoteObject::Private::ObjectMaps->FindOrAddStub(RemoteId, FRemoteServerId{});
			return StubPointer;
		}
		else if (ExtractHandleType() == ERemoteObjectHandleType::StubPointer)
		{
			return ExtractStubPointer();
		}
		else if (ExtractHandleType() == ERemoteObjectHandleType::ObjectPointer)
		{
			FRemoteObjectHandlePrivate StubHandle = ConvertToRemoteHandle(const_cast<UObject*>(ExtractObjectPointer()));
			return StubHandle.ToStub();
		}

		return nullptr;
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

} // namespace UE::CoreUObject::Private

#ifndef UE_WITH_REMOTE_ASSET_ID
#define UE_WITH_REMOTE_ASSET_ID 1 // set this to 0 to disable remote asset IDs
#endif

bool FRemoteObjectId::IsMigratingAsset(UObjectBase* InObject)
{
	using namespace UE::CoreUObject::Private;

	// An object is a migrating asset if:
	// its class or any of its outers' class is marked as MigratingAsset
	// AND it's NOT an archetype or subobject of an archetype 
	// AND it's NOT a subobject of a UStruct (class).
	// In other words we don't want CDOs or Blueprint Classes to be migrating even when they are marked as migrating assets
	bool bIsMarkedAsMigratingAsset = false;
	bool bIsClassOrArchetype = false;

	for (UObjectBase* OuterIt = InObject; OuterIt; OuterIt = FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(OuterIt))
	{
		UClass* Class = OuterIt->GetClass();
		if (!!(OuterIt->GetFlags() & RF_ArchetypeObject) || Class->IsChildOf(UStruct::StaticClass()))
		{
			bIsClassOrArchetype = true;
			break;
		}
		if (!!(OuterIt->GetFlags() & RF_MigratingAsset))
		{
			bIsMarkedAsMigratingAsset = true;
		}
	}

	return bIsMarkedAsMigratingAsset && !bIsClassOrArchetype;
}

bool FRemoteObjectId::IsAssetInternal(UObjectBase* InObject, EInternalObjectFlags InInitialFlags)
{
	bool bIsAsset = false;

	if (GIsInitialLoad || !!(InInitialFlags & EInternalObjectFlags::Native) || !!(InObject->GetFlags() & RF_ArchetypeObject))
	{
		// Native objects (classes, CDOs etc) are always in memory and are considered assets any server can find locally
		// Note that this first condition can not touch too much of UObject API because we might literarlly be constructing the first StaticClass() etc.
		// Hopefull GIsInitialLoad and the Native flag will filter most of the initially created objects and most of the native objects will be constructed before we ever hit the 'else' block
		bIsAsset = true;
	}
	else
	{
		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		// Check if we're currently inside of a callstack where something is being loaded or the object has the RF_WasLoaded flag
		if (ThreadContext.AsyncPackageLoader || (ThreadContext.GetSerializeContext() && ThreadContext.GetSerializeContext()->GetBeginLoadCount() > 0) || !!(InObject->GetFlags() & RF_WasLoaded))
		{
			bIsAsset = !IsMigratingAsset(InObject);
		}
	}

	return bIsAsset;
}

namespace
{
	/**
	* Utility function that generates a similar output to GetObjectPathNameSafe(InObject) but for UObjectBase which may not even have a name or outer set
	* @param InObject Object to generate the name for
	* @param InName Name of the object (identical to InObject->GetFName().ToString() but the object may not even have the name assigned yet)
	* @param InOuterPathName Path name of the outer of the specified object (optional, used only when the Outer is not set yet)
	* @return PathName of the object or "None" if the specified object is null
	*/
	FString GetUObjectBasePathNameSafe(UObjectBase* InObject, const TCHAR* InName, const TCHAR* InOuterPathName)
	{
		FString Result;
		if (InObject)
		{
			TStringBuilder<256> ObjectPathBuilder;
			if (InObject->GetOuter())
			{
				ObjectPathBuilder << InObject->GetOuter()->GetPathName();
			}
			else if (InOuterPathName)
			{
				ObjectPathBuilder << InOuterPathName;
			}

			if (ObjectPathBuilder.Len() > 0)
			{
				ObjectPathBuilder << SUBOBJECT_DELIMITER_CHAR;
			}
			ObjectPathBuilder << InName;
			Result += ObjectPathBuilder.ToView();
		}
		else
		{
			// Similar to what GetObjectNameSafe(nullptr) would return
			Result = TEXT("None");
		}
		
		return Result;
	}

	uint64 GenerateSerialNumberFromPathName(UObjectBase* InObject, const TCHAR* InName, const TCHAR* InOuterPathName)
	{
		// An asset must have the same remote object id on each game server. In order to achieve this the object's full path is converted into a 53-bit hash (the lower
		// 53 bits of a 64-bit hash) and used as the remote id.
		FString ObjectPathName = GetUObjectBasePathNameSafe(InObject, InName, InOuterPathName);

		const uint64 ObjectPathHash64 = CityHash64(
			reinterpret_cast<const char*>(*ObjectPathName),
			ObjectPathName.Len() * sizeof(FString::ElementType)
		);

		const uint64 MaskLowBits = (1ULL << REMOTE_OBJECT_SERIAL_NUMBER_BIT_SIZE) - 1;
		const uint64 ObjectPathHash53 = ObjectPathHash64 & MaskLowBits;

		return ObjectPathHash53;
	}
}

FRemoteObjectId FRemoteObjectId::Generate(UObjectBase* InObject, const TCHAR* InName, const TCHAR* InOuterPathName, EInternalObjectFlags InInitialFlags /*= EInternalObjectFlags::None*/)
{
	using namespace UE::RemoteObject::Private;
	using namespace UE::CoreUObject::Private;

	bool bIsAsset = false;
#if UE_WITH_REMOTE_ASSET_ID
	bIsAsset = IsAssetInternal(InObject, InInitialFlags);
#endif // UE_WITH_REMOTE_ASSET_ID

	FRemoteObjectId Result;

	if (!bIsAsset)
	{
		Result = FRemoteObjectId::Generate(ERemoteServerIdConstants::Local);
	}
	else
	{
		Result = FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::Asset), GenerateSerialNumberFromPathName(InObject, InName, InOuterPathName));
	}

	UE_LOGF(LogRemoteObject, Verbose, "FRemoteObjectId::Generate: Object=%ls RemoteId=%ls", *GetUObjectBasePathNameSafe(InObject, InName, InOuterPathName), *Result.ToString(ERemoteIdToStringVerbosity::Id));

	return Result;
}

FRemoteObjectId FRemoteObjectId::Generate(UObjectBase* InObject, const TCHAR* InName, EInternalObjectFlags InInitialFlags /*= EInternalObjectFlags::None*/)
{
	return Generate(InObject, InName, nullptr, InInitialFlags);
}

FRemoteObjectId FRemoteObjectId::Generate(ERemoteServerIdConstants ServerId)
{
	using namespace UE::RemoteObject::Private;

	switch (ServerId)
	{
	case ERemoteServerIdConstants::Asset:
		return FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::Asset), AssetObjectSerialNumber.fetch_add(1));

	case ERemoteServerIdConstants::Local:
		return FRemoteObjectId(FRemoteServerId(ERemoteServerIdConstants::Local), RemoteObjectSerialNumber.fetch_add(1));

	default:
		UE_LOGF(LogRemoteObject, Fatal, "Unable to generate an object id for server id: %u", (uint32)ServerId);
	};
	return FRemoteObjectId();
}

FString FRemoteObjectId::ToString(ERemoteIdToStringVerbosity InVerbosityOverride /*= ERemoteIdToStringVerbosity::Default*/) const
{
	using namespace UE::RemoteObject::Handle;

	const int32 Verbosity = (InVerbosityOverride == ERemoteIdToStringVerbosity::Default) ? 
		FMath::Clamp(GRemoteIdToStringVerbosity, (int32)ERemoteIdToStringVerbosity::Id, (int32)ERemoteIdToStringVerbosity::Max) : (int32)InVerbosityOverride;

	if (Verbosity <= (int32)ERemoteIdToStringVerbosity::Id)
	{
		return FString::Printf(TEXT("%s-%llu"), *GetServerId().ToString(), SerialNumber);
	}
	else
	{
		FString AdditionalInfo;
		bool bStubOnly = false;

		if (UObject* ExistingObject = StaticFindObjectFastInternal(*this))
		{
			switch ((ERemoteIdToStringVerbosity)Verbosity)
			{
			case ERemoteIdToStringVerbosity::Name:
				AdditionalInfo = ExistingObject->GetName();
				break;
			case ERemoteIdToStringVerbosity::PathName:
				AdditionalInfo = ExistingObject->GetPathName();
				break;
			case ERemoteIdToStringVerbosity::FullName:
			case ERemoteIdToStringVerbosity::FullNameAttributes:
				AdditionalInfo = ExistingObject->GetFullName();
				break;
			}
		}
		else
		{
			FRemoteObjectPathName RemotePathName(*this);
			if (RemotePathName.Num())
			{
				bStubOnly = true;
				switch ((ERemoteIdToStringVerbosity)Verbosity)
				{
				case ERemoteIdToStringVerbosity::Name:
					AdditionalInfo = RemotePathName.GetObjectName().ToString();
					break;
				case ERemoteIdToStringVerbosity::PathName:
					AdditionalInfo = RemotePathName.ToString();
					break;
				case ERemoteIdToStringVerbosity::FullName:
				case ERemoteIdToStringVerbosity::FullNameAttributes:
				{
					UClass* Class = GetClass(*this, ERemoteObjectGetClassBehavior::ReturnNullIfNeverLocal);
					AdditionalInfo = FString::Printf(TEXT("%s %s"), *GetNameSafe(Class), *RemotePathName.ToString());
				}
				break;
				}
			}
			else
			{
				AdditionalInfo = TEXT("Unknown object");
			}
		}

		if (Verbosity >= (int32)ERemoteIdToStringVerbosity::FullNameAttributes)
		{
			EResidence Residence = GetResidence(*this);
			if (Residence != EResidence::Local)
			{
				AdditionalInfo += FString::Printf(TEXT(" (%s)"), EnumToString(Residence));
			}
			if (IsOwned(*this))
			{
				AdditionalInfo += TEXT(" (owned)");
			}
			if (bStubOnly)
			{
				AdditionalInfo += TEXT(" (stub)");
			}
		}

		return FString::Printf(TEXT("%s-%llu %s"), *GetServerId().ToString(), SerialNumber, *AdditionalInfo);
	}
}

FRemoteObjectId FRemoteObjectId::FromString(const FStringView& InText)
{
	int32 ServerDelimiterIndex = -1;
	if (InText.FindChar('-', ServerDelimiterIndex))
	{
		// Parse formatted id (as returned from FRemoteObjectId::ToString()): 'ServerId-SerialNumber', e.g. 'Asset-12345'
		FStringView ServerIdText(InText.GetData(), ServerDelimiterIndex);
		const int32 ObjectSerialStartIndex = ServerDelimiterIndex + 1;
		FStringView ObjectSerialText(InText.GetData() + ObjectSerialStartIndex, InText.Len() - ObjectSerialStartIndex);
		FRemoteServerId ObjectServerId = FRemoteServerId::FromString(ServerIdText);
		uint64 ObjectSerial = 0;
		LexFromString(ObjectSerial, ObjectSerialText);

		return FRemoteObjectId(ObjectServerId, ObjectSerial);
	}
	else
	{
		// Parse the string as a number as if it was the memory image of an id
		FRemoteObjectId RemoteId;
		LexFromString(RemoteId.Id, InText);
		if (RemoteId.ServerId != (uint32)ERemoteServerIdConstants::Invalid)
		{
			return RemoteId;
		}
	}		
	return FRemoteObjectId();
}

FRemoteObjectId::FRemoteObjectId(const UObjectBase* Object)
	: FRemoteObjectId(UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(Object))
{
}

bool FRemoteObjectId::Serialize(FArchive& Ar)
{
	static_assert(sizeof(uint64) == sizeof(Id));

	uint64 SerializedID = Id;
	if (Ar.IsSaving())
	{
		FRemoteObjectId GlobalizedId = GetGlobalized();
		SerializedID = GlobalizedId.Id;
	}

	Ar << SerializedID;
	{
		FRemoteObjectId LocalizedId;
		LocalizedId.Id = SerializedID;
		checkf(LocalizedId.GetServerId().Id != (uint32)ERemoteServerIdConstants::Local, TEXT("Local server id should never be serialized"));
	}

	if (Ar.IsLoading())
	{
		FRemoteObjectId LocalizedId;
		LocalizedId.Id = SerializedID;
		checkf(LocalizedId.GetServerId().Id != (uint32)ERemoteServerIdConstants::Local, TEXT("Local server id should never be serialized"));
		*this = LocalizedId.GetLocalized();
	}

	return true;
}

bool FRemoteObjectId::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	return Serialize(Ar);
}

FArchive& operator<<(FArchive& Ar, FRemoteObjectId& Id)
{
	Id.Serialize(Ar);
	return Ar;
}

FRemoteServerId FRemoteServerId::FromIdNumber(uint32 InNumber)
{
	checkf(InNumber <= (uint32)ERemoteServerIdConstants::Max, TEXT("Remote server id can not be greater than %u, got: %u"), (uint32)ERemoteServerIdConstants::Max, InNumber);
	FRemoteServerId Result;
	Result.Id = InNumber;
	return Result;
}

FRemoteServerId FRemoteServerId::FromString(const FStringView& InText)
{
	FRemoteServerId Result;

	if (InText == TEXT("Invalid"))
	{
		Result.Id = (uint32)ERemoteServerIdConstants::Invalid;
	}
	else if (InText == TEXT("Asset"))
	{
		Result.Id = (uint32)ERemoteServerIdConstants::Asset;
	}
	else if (InText == TEXT("Database"))
	{
		Result.Id = (uint32)ERemoteServerIdConstants::Database;
	}
	else if (InText == TEXT("Local"))
	{
		Result.Id = (uint32)ERemoteServerIdConstants::Local;
	}
	else
	{
		uint32 ServerIdNumber = (uint32)ERemoteServerIdConstants::Invalid;
		LexFromString(ServerIdNumber, InText);
		if (ensureMsgf(ServerIdNumber <= (uint32)ERemoteServerIdConstants::Max, TEXT("Parsed Remote Server Id value %u that is bigger than allowed max %u"), ServerIdNumber, (uint32)ERemoteServerIdConstants::Max))
		{
			Result.Id = ServerIdNumber;
		}
		else
		{
			UE_LOGF(LogRemoteObject, Warning, "Clamping ServerId number %u to the maximum allowed %u", ServerIdNumber, (uint32)ERemoteServerIdConstants::Max);
			Result.Id = (uint32)ERemoteServerIdConstants::Max;
		}		
	}

	return Result;
}

FString FRemoteServerId::ToString() const
{
	switch (Id)
	{
		case (uint32)ERemoteServerIdConstants::Asset:
			return TEXT("Asset");

		case (uint32)ERemoteServerIdConstants::Database:
			return TEXT("Database");

		case (uint32)ERemoteServerIdConstants::Local:
			return IsGlobalServerIdInitialized() ? GetGlobalized().ToString() : TEXT("Local");

		default:
			return FString::FromInt(Id);
	}
}

bool FRemoteServerId::Serialize(FArchive& Ar)
{
	static_assert(sizeof(uint32) == sizeof(Id));

	uint32 SerializedId = Id;
	if (Ar.IsSaving())
	{
		FRemoteServerId GlobalizedId = GetGlobalized();
		SerializedId = GlobalizedId.Id;
	}

	Ar << SerializedId;
	checkf(SerializedId != (uint32)ERemoteServerIdConstants::Local, TEXT("Local server id should never be serialized"));

	if (Ar.IsLoading())
	{
		FRemoteServerId LocalizedId;
		LocalizedId.Id = SerializedId;
		*this = LocalizedId.GetLocalized();
	}	

	return true;
}

bool FRemoteServerId::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	return Serialize(Ar);
}

FArchive& operator<<(FArchive& Ar, FRemoteServerId& Id)
{
	Id.Serialize(Ar);
	return Ar;
}

// Debugging functionality to help us find these objects in debug builds
#if !UE_BUILD_SHIPPING && UE_WITH_REMOTE_OBJECT_HANDLE
/**
 * Put this in a Debug Watch Window on a specific UObject.  You may have to forcibly cast the UObject to UObjectBase*
 * e.g. DebugFindRemoteObjectStub((UObjectBase*)Header.Class.DebugPtr)
 */
COREUOBJECT_API UE::RemoteObject::Handle::FRemoteObjectStub* DebugFindRemoteObjectStub(const UObjectBase* Object)
{
	if (!Object)
	{
		return nullptr;
	}

	uintptr_t Pointer = reinterpret_cast<uintptr_t>(Object);
	if (Pointer & 0x1)
	{
		return reinterpret_cast<UE::RemoteObject::Handle::FRemoteObjectStub*>(Pointer & ~UPTRINT(1));
	}

	FRemoteObjectId ObjId { Object };
	return UE::RemoteObject::Private::ObjectMaps->FindStub(ObjId);
}

/**
 * Put this in a Debug Watch Window to get the FRemoteObjectStub of a given ObjectId.
 * e.g. DebugFindRemoteObjectStub(2,123)
 */
COREUOBJECT_API UE::RemoteObject::Handle::FRemoteObjectStub* DebugFindRemoteObjectStub(uint32 ServerId, uint64 SerialNumber)
{
	// Sanitize inputs
	ServerId = (ServerId <= MAX_REMOTE_OBJECT_SERVER_ID) ? ServerId : 0;
	SerialNumber = (SerialNumber <= MAX_REMOTE_OBJECT_SERIAL_NUMBER) ? SerialNumber : 0;

	const FRemoteObjectId ObjId { FRemoteServerId::FromIdNumber(ServerId), SerialNumber };
	return UE::RemoteObject::Private::ObjectMaps->FindStub(ObjId);
}

/**
 * Attempt to find a UObject in the currently debugged process by its FRemoteObjectId constituents.
 * Once you know a FRemoteObjectId, take its ServerId and SerialNumber and pass them into Debug Watch Window as arguments (in that order)
 * e.g. DebugFindObjectLocallyFromRemoteId( 2, 1234 )
 */
COREUOBJECT_API UObject* DebugFindObjectLocallyFromRemoteId(uint16 ServerId, uint64 SerialNumber)
{
	return StaticFindObjectFastInternal(FRemoteObjectId(FRemoteServerId::FromIdNumber(static_cast<uint32>(ServerId)), SerialNumber));
}

/**
 * Attempt to find a UObject in the currently debugged process by its FRemoteObjectId's Full uint64 Id
 * Once you find a FRemoteObjectId, copy its Id and pass them into Debug Watch Window as the argument
 * e.g. DebugFindObjectLocallyFromRemoteId( 1234567890 )
 */
COREUOBJECT_API UObject* DebugFindObjectLocallyFromRemoteId(uint64 FullId)
{
	constexpr uint64 SerialBitMask = (1ull << 54ull) - 1ull;
	FRemoteObjectId RemoteId(FRemoteServerId::FromIdNumber(static_cast<uint32>((FullId >> 54ull) & 1023ull)), FullId & SerialBitMask);
	ensure(RemoteId.GetIdNumber() == FullId);

	return StaticFindObjectFastInternal(RemoteId);
}
#endif // !UE_BUILD_SHIPPING && UE_WITH_REMOTE_OBJECT_HANDLE
