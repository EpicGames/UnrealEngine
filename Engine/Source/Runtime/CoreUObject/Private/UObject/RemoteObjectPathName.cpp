// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/RemoteObjectPathName.h"
#include "UObject/RemoteObject.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/ObjectPtr.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectMigrationContext.h"
#include "UObject/UnrealType.h"
#include "UObject/Package.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Templates/Casts.h"
#include "UObject/ObjectHandlePrivate.h"
#include "Misc/Paths.h"

namespace UE::RemoteObject::Private
{

template <typename PathType>
FORCENOINLINE FString RemotePathNameToString(const PathType& InPathName, const FRemoteObjectTables& InTables, int32 InMinPathNameIndex = 0)
{
	TStringBuilder<256> Result;
	for (int32 PathSegment = InPathName.Num() - 1; PathSegment >= InMinPathNameIndex; --PathSegment)
	{
		InPathName.GetSegmentName(PathSegment, InTables).AppendString(Result);
		if (PathSegment > InMinPathNameIndex)
		{
			if (PathSegment != (InPathName.Num() - 2))
			{
				Result << TEXT('.');
			}
			else
			{
				// at this point the constructed path consists of /Package/Name.ObjectName so we need to use the subobject delimiter
				// see UObjectBaseUtility::GetPathName
				Result << SUBOBJECT_DELIMITER_CHAR;
			}
		}
	}
	return Result.ToString();
}

template <typename PathType>
FORCENOINLINE UObject* LoadRemotePathName(const PathType& InPathName, const FRemoteObjectTables& InTables, int32 InNameIndex)
{
	FString LoadPathName = RemotePathNameToString<PathType>(InPathName, InTables, InNameIndex);
	const int32 PackageNameIndex = InPathName.Num() - 1;
	UObject* Object = nullptr;
	if (InNameIndex == PackageNameIndex)
	{
		// We're loading the Outermost (Package) so we can't use StaticLoadObject because it assumes the name of the object we're trying to load 
		// is just short package name (/Root/PackageName.PackageName) which is not true in case of Blueprint classes (/Root/PackageName.PackageName_C)
		// Loading the package will load all objects inside of it but that's also true for StaticLoadObject.
		Object = LoadPackage(nullptr, *LoadPathName, LOAD_None);
	}
	else
	{
		Object = StaticLoadObject(UObject::StaticClass(), nullptr, *LoadPathName);
	}
	if (!Object)
	{
		if (InNameIndex == 0)
		{
			UE_LOGF(LogRemoteObject, Warning, "Failed to load asset object %ls (%ls)", *LoadPathName, *InPathName.GetSegmentId(InNameIndex, InTables).ToString());
		}
		else
		{
			UE_LOGF(LogRemoteObject, Warning, "Failed to load asset object %ls which is an outer of remote object %ls (%ls)",
				*LoadPathName,
				*RemotePathNameToString<PathType>(InPathName, InTables),
				*InPathName.GetSegmentId(0, InTables).ToString());
		}
	}
	return Object;
}

template <typename PathType>
FORCENOINLINE UObject* ResolveRemotePathName(const PathType& InPathName, const FRemoteObjectTables& InTables)
{
	// Resolve a remote object path name starting with the outermost object and working towards the innermost one
	UObject* Outer = nullptr;
	UObject* Object = nullptr;
	for (int32 PathNameIndex = InPathName.Num() - 1; PathNameIndex >= 0; --PathNameIndex)
	{
		Object = StaticFindObjectFast(UObject::StaticClass(), Outer, InPathName.GetSegmentName(PathNameIndex, InTables));
		if (!Object)
		{			
			FRemoteObjectId RemoteId = InPathName.GetSegmentId(PathNameIndex, InTables);
			if (RemoteId.IsAsset())
			{
				Object = LoadRemotePathName(InPathName, InTables, PathNameIndex);
				if (!Object)
				{
					break;
				}
			}
		}
		Outer = Object;
	}
	return Object;
}

} // namespace UE::RemoteObject::Private

void FRemoteObjectTables::Serialize(FArchive& Ar)
{
	Ar << Names;
	Ar << RemoteIds;
}

UObject* FPackedRemoteObjectPathName::Resolve(const FRemoteObjectTables& InTables) const
{
	using namespace UE::RemoteObject::Private;
	return ResolveRemotePathName(*this, InTables);
}

void FPackedRemoteObjectPathName::Serialize(FArchive& Ar)
{
	Ar << Names;
	Ar << RemoteIds;
}

FString FPackedRemoteObjectPathName::ToString(const FRemoteObjectTables& InTables, int32 InMinPathSegmentIndex /*= 0*/) const
{
	using namespace UE::RemoteObject::Private;
	checkf(InMinPathSegmentIndex >= 0, TEXT("Minimum path segment index must be > 0 (got %d)"), InMinPathSegmentIndex);
	return RemotePathNameToString(*this, InTables, InMinPathSegmentIndex);
}

void AppendObjectToPathName(UObject* InObject, FRemoteObjectPathName& OutPathName)
{
	for (UObject* PathNameObject = InObject; PathNameObject; PathNameObject = PathNameObject->GetOuter())
	{
		OutPathName.Names.Add(PathNameObject->GetFName());
		OutPathName.RemoteIds.Add(FRemoteObjectId(PathNameObject));
	}
}

FRemoteObjectPathName::FRemoteObjectPathName(UObject* InObject)
{
	AppendObjectToPathName(InObject, *this);
}

FRemoteObjectPathName::FRemoteObjectPathName(FRemoteObjectId RemoteId)
{
	using namespace UE::RemoteObject::Private;
	using namespace UE::RemoteObject::Handle;

	while (RemoteId.IsValid())
	{
		UObject* Object = StaticFindObjectFastInternal(RemoteId);
		if (Object)
		{
			AppendObjectToPathName(Object, *this);
			// We can abort now because once we encounter an existing object in the outer chain we can fully construct the pathname
			break;
		}
		else
		{
			FRemoteObjectStub* Stub = FindRemoteObjectStub(RemoteId);
			if (Stub && !Stub->Name.IsNone() && Stub->OuterId.IsValid())
			{
				Names.Add(Stub->Name);
				RemoteIds.Add(Stub->Id);
				RemoteId = Stub->OuterId;
			}
			else
			{
				// If the object could not be found by its id and the object does not have a stub, we can't fully construct the pathname so reset everything and abort
				// We also need to early out if the OuterId of a stub is invalid because at the moment we can distinguish between a stub that represents a package or an incomplete stub
				// and since we don't normally migrate packages (yet) we must assume it's the latter
				Names.Empty();
				RemoteIds.Empty();
				break;
			}
		}
	}
}

UObject* FRemoteObjectPathName::Resolve() const
{
	using namespace UE::RemoteObject::Private;
	return ResolveRemotePathName(*this, *this);
}

FString FRemoteObjectPathName::ToString(int32 InMinPathSegmentIndex /*= 0*/) const
{
	using namespace UE::RemoteObject::Private;
	checkf(InMinPathSegmentIndex >= 0, TEXT("Minimum path segment index must be > 0 (got %d)"), InMinPathSegmentIndex);
	return RemotePathNameToString(*this, *this, InMinPathSegmentIndex);
}