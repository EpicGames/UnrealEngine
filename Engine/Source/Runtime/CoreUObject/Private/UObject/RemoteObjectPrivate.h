// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/UObjectMigrationContext.h"
#include "UObject/RemoteObjectTransfer.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRemoteObject, Display, All);

struct FRemoteObjectPathName;

enum class EMigratedDataState
{
	Uninitialized = 0,
	Received = 1,
	Deserialized = 2,
	PostMigrated = 3,
	Completed = 4
};
const TCHAR* EnumToString(EMigratedDataState InState);

/** Structure that contains object data and migration context for deserializing and PostMigrating objects inside of a transaction */
struct FTransactionalMigrationData : public FRefCountedObject
{
	FTransactionalMigrationData() = default;
	FTransactionalMigrationData(FRemoteObjectData& InData, FUObjectMigrationContext& InContext);

	/** Serialized data of the migrated objects */
	FRemoteObjectData Data;
	/** The context for the migrated objects */
	FUObjectMigrationContext Context;
	
	/** List of remote ids of the received objects */
	TArray<FRemoteObjectId> ReceivedObjectRemoteIds;
	/** List of received objects */
	TArray<UObject*> ReceivedObjects;
	/** The index of the object requested by the migration request from the received objects list */
	int32 RequestedObjectIndex = -1;

	/** The current state of the migrated data */
	EMigratedDataState State = EMigratedDataState::Uninitialized;

	void SetReceivedObjectsResidence(EResidence Residence);
};

namespace UE::RemoteObject::Private
{
	/**
	* Initializes remote objects subsystems
	*/
	void InitRemoteObjects();

	/**
	* Frees memory associated with remote objects subsystems
	*/
	void ShutdownRemoteObjects();

	/**
	* Sets residence of an object
	* @param Object Object which residence will be set
	* @param Residence Residence to set
	* @param ResidentServerId Server where the object is currently resident (should be FRemoteServerId::GetLocalServerId() for EResidence::Local and LocalNotReady)
	*/
	void SetResidence(UObject* Object, EResidence Residence, FRemoteServerId ResidentServerId);

	/**
	* Marks object memory as remote and creates its stub
	*/
	UE_DEPRECATED(5.8, "Use SetResidence(Object, EResidence::Remote, ResidentServerId) instead.")
	void MarkAsRemote(UObject* Object, FRemoteServerId DestinationServerId);

	/**
	* Marks object as referenced by a remote object
	*/
	void MarkAsRemoteReference(const UObject* Object);

	/**
	* @return true if object is marked as remote reference
	*/
	bool IsRemoteReference(const UObject* Object);

	/**
	* Marks object memory as local
	*/
	UE_DEPRECATED(5.8, "Use SetResidence(Object, EResidence::Local, FRemoteServerId::GetLocalServerId()) instead.")
	void MarkAsLocal(UObject* Object);

	/**
	* Marks object as borrowed
	*/
	void MarkAsBorrowed(UObject* Object);

	/**
	* Returns true if the specified object is marked as borrowed
	*/
	bool IsBorrowed(const UObject* Object);

	/**
	* Finds or adds a stub for a remote object that is known to be resident on a specific server and updates its ResidentServerId
	* @param ObjectId Remote object id to find or create a stub for
	* @param ResidentServerId (optional) New resident server id to set for the new or existing stub (can be invalid in which case the resident server id will be set to local for a new stub or will not be updated for an existing stub)
	* @return Stub for the specified remote object id
	*/
	UE::RemoteObject::Handle::FRemoteObjectStub* FindOrAddRemoteObjectStub(FRemoteObjectId ObjectId, FRemoteServerId ResidentServerId = FRemoteServerId());

	/**
	* Registers object for sharing, marking it as owned by the current server
	*/
	void RegisterSharedObject(UObject* Object);

	/**
	* Finds remote object stub
	*/
	UE::RemoteObject::Handle::FRemoteObjectStub* FindRemoteObjectStub(FRemoteObjectId ObjectId);

	/**
	* Gets a base FName that will be used to generate a unique object name (see MakeUniqueObjectName)
	*/
	FName GetServerBaseNameForUniqueName(const UClass* Class);

	/**
	* Stores FRemoteObjectPath for a remotely referenced asset that's about to be destroyed so that the engine knows it should load the asset when something requests it
	*/
	void StoreAssetPath(UObject* Object);

	/**
	* Attempts to find an FRemoteObjectPath for an object id representing an asset
	*/
	FRemoteObjectPathName* FindAssetPath(FRemoteObjectId RemoteId);

	struct FUnsafeToMigrateScope
	{
		FUnsafeToMigrateScope();
		~FUnsafeToMigrateScope();
	};

	struct FScopedForceReturnObjectHandles : FNoncopyable
	{
		FScopedForceReturnObjectHandles();
		~FScopedForceReturnObjectHandles();
	};

	struct FRemoteIdLocalizationHelper
	{
		inline static FRemoteServerId GetLocalized(FRemoteServerId InId)
		{
			return InId.GetLocalized();
		}
		inline static FRemoteServerId GetGlobalized(FRemoteServerId InId)
		{
			return InId.GetGlobalized();
		}
		inline static FRemoteObjectId GetLocalized(FRemoteObjectId InId)
		{
			return InId.GetLocalized();
		}
		inline static FRemoteObjectId GetGlobalized(FRemoteObjectId InId)
		{
			return InId.GetGlobalized();
		}
	};

	/**
	* Returns true if UE::RemoteObject::Handle::Resolve() functions are forced to return FObjectHandles instead of the actual UObject pointers
	*/
	bool IsForceReturnObjectHandles();

	/**
	* Returns true if the internal structures used by the remote object system are initialized
	*/
	bool IsRemoteObjectSystemInitialized();

	/**
	* Returns true if the remote object handle support is compiled in and internal structures used by the remote object system are initialized
	*/
	inline bool IsRemoteObjectSystemCompiledInAndInitialized()
	{
		if (FRemoteObjectId::RemoteObjectSupportCompiledIn)
		{
			return IsRemoteObjectSystemInitialized();
		}
		return false;
	}

	/**
	* Calls the provided function for each of the internally stored stubs that references valid migration data
	*/
	void ForEachStubWithMigratedData(TFunctionRef<bool(UE::RemoteObject::Handle::FRemoteObjectStub*)> Operation);

	/**
	* Sets an internal flag indicating if the current thread is transactionally deserializing and PostMigrating objects
	*/
	void SetTransactionallyPostMigratingObjects(bool bIsPostMigrating);
}

namespace UE::RemoteObject::Transfer::Private
{


	/**
	* Stores remotely referenced unreachable objects to database
	*/
	void StoreUnreachableRemoteObjectsToDatabase(const TArrayView<FUObjectItem*>& UnreachableObjects);
}
