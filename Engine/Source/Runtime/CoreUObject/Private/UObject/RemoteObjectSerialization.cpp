// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/RemoteObjectSerialization.h"
#include "UObject/RemoteObject.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/ObjectHandle.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectMigrationContext.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/UnrealType.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Templates/Casts.h"
#include "Async/Async.h"
#include "UObject/ObjectHandlePrivate.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include <atomic>

int32 GRemoteObjectsMigrateFullHierarchy = 1;
static FAutoConsoleVariableRef CVarRemoteObjectsMigrateFullHierarchy(
	TEXT("ro.MigrateFullHierarchy"),
	GRemoteObjectsMigrateFullHierarchy,
	TEXT("Whether remote objects that are default subobjects should be always migrated with their parent objects"));

int32 GResetBorrowedObjects = 1;
static FAutoConsoleVariableRef CVarResetBorrowedObjects(
	TEXT("ro.ResetBorrowedObjects"),
	GResetBorrowedObjects,
	TEXT("Whether remote objects that were borrowed should be reset upon returning to their owner server instead of being reconstructed"));

int32 GUseImmutableArchetypes = 1;
static FAutoConsoleVariableRef CVarUseImmutableArchetypes(
	TEXT("ro.UseImmutableArchetypes"),
	GUseImmutableArchetypes,
	TEXT("Whether to use immutable archetypes when serializing remote object data"));

#ifndef UE_WITH_REMOTEOBJECT_SERIALIZATION_VERIFICATION
#define UE_WITH_REMOTEOBJECT_SERIALIZATION_VERIFICATION DO_CHECK && !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
#endif

#if UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING

int32 GDebugRemoteObjectSerialization = 0;
static FAutoConsoleVariableRef CVarDebugRemoteObjectSerialization(
	TEXT("ro.DebugRemoteObjectSerialization"),
	GDebugRemoteObjectSerialization,
	TEXT("Whether to dump detailed remote object serialization stats to log"));

#endif // UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING

const UObject* FindImmutableArchetype(const UObject* InObj);

using FNameIndexType = FRemoteObjectTables::FNameIndexType;

DEFINE_LOG_CATEGORY_STATIC(LogRemoteSerialization, Log, All);

FArchive& operator<<(FArchive& Ar, FRemoteObjectBytes& Chunk)
{
	Ar << Chunk.Bytes;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FPackedRemoteObjectHeader& Header)
{
	Ar << Header.NameIndex;
	Ar << Header.IdIndex;
	Ar << Header.ClassIndex;

	return Ar;
}

namespace UE::RemoteObject::Serialization::Private
{
	/*
	* The migration serial number is shared between 'Local' server id and 'Database'.
	* The reason for that is that we can't have a globally unique database serial number as it's stored locally
	* and effectively it's the current server that restores the object from the database.
	*/
	std::atomic<uint64> MigrationSerialNumber(1);

	FRemoteObjectId GetNextMigrationId(FRemoteServerId ServerId = FRemoteServerId::GetLocalServerId())
	{
		checkf(ServerId.IsLocal() || ServerId.IsDatabase(), TEXT("Migration id can only be generated for the Local server or Database, requested for %s"), *ServerId.ToString());
		uint64 SerialNumber = 0;
		UE_AUTORTFM_OPEN
		{
			uint64 CurrentSerialNumber = MigrationSerialNumber.load(std::memory_order_relaxed);
			do
			{
				SerialNumber = (CurrentSerialNumber <= MAX_REMOTE_OBJECT_SERIAL_NUMBER) ? CurrentSerialNumber : 1;
			} while (!MigrationSerialNumber.compare_exchange_weak(CurrentSerialNumber, SerialNumber + 1, std::memory_order_relaxed));
		};
		return FRemoteObjectId(ServerId, SerialNumber);
	}
}

namespace UE::RemoteObject::Serialization::Disk
{

enum class ERemoteObjectFileType : int32
{
	Unknown = 0,
	Data = 1,
	Redirector = 2,

	MaxPlusOne,
	Max = MaxPlusOne - 1
};

struct FRemoteObjectFileHeader
{
	ERemoteObjectFileType FileType = ERemoteObjectFileType::Unknown;
	FRemoteObjectId RootObjectId;
};

void SerializeFileHeader(FArchive& Ar, FRemoteObjectFileHeader& InHeader)
{
	int32 SerializedFileType = (int32)InHeader.FileType;
	Ar << SerializedFileType;
	Ar << InHeader.RootObjectId;
}

void DeserializeFileHeader(FArchive& Ar, FRemoteObjectFileHeader& OutHeader)
{
	int32 SerializedFileType = -1;
	Ar << SerializedFileType;
	checkf(SerializedFileType >= (int32)ERemoteObjectFileType::Data && SerializedFileType <= (int32)ERemoteObjectFileType::Max, TEXT("Unknown remote object file type %d"), SerializedFileType);
	
	OutHeader.FileType = (ERemoteObjectFileType)SerializedFileType;
	Ar << OutHeader.RootObjectId;
}

void SerializeNameTables(FArchive& Ar, const FRemoteObjectData& InObjectData)
{
	FRemoteObjectId MigrationId = InObjectData.MigrationId;
	Ar << MigrationId;

	FNameIndexType NumNames = IntCastChecked<FNameIndexType>(InObjectData.Tables.Names.Num());
	Ar << NumNames;
	for (const FName& Name : InObjectData.Tables.Names)
	{
		FString NameString = Name.ToString();
		Ar << NameString;
	}
	Ar << const_cast<TArray<FRemoteObjectId>&>(InObjectData.Tables.RemoteIds);
	Ar << const_cast<TArray<FPackedRemoteObjectPathName>&>(InObjectData.PathNames);
	Ar << const_cast<TArray<FPackedRemoteObjectHeader>&>(InObjectData.SerializedObjectHeaders);
}

void DeserializeNameTables(FArchive& Ar, FRemoteObjectData& OutObjectData)
{
	Ar << OutObjectData.MigrationId;
	OutObjectData.Tables.Names.Reset();
	FNameIndexType NumNames = 0;
	Ar << NumNames;
	for (FNameIndexType NameIndex = 0; NameIndex < NumNames; ++NameIndex)
	{
		FString NameString;
		Ar << NameString;
		OutObjectData.Tables.Names.Add(FName(NameString, FNAME_Add));
	}
	Ar << OutObjectData.Tables.RemoteIds;
	Ar << OutObjectData.PathNames;
	Ar << OutObjectData.SerializedObjectHeaders;
}

FString GenerateRemoteObjectFilename(FRemoteObjectId ObjectId, FRemoteServerId OwnerServerId)
{
	return FPaths::Combine(*FPaths::ProjectSavedDir(), *(FString::Printf(TEXT("%s-%s_%s.remote"), *FRemoteServerId::GetLocalServerId().ToString(), *ObjectId.ToString(ERemoteIdToStringVerbosity::Id), *OwnerServerId.ToString())));
}

void LoadObjectFromDiskInternal(const FUObjectMigrationContext& MigrationContext, FRemoteObjectId RootObjectId, FRemoteObjectData& OutData, ERemoteObjectFileType PreviousFileType)
{
	FString Filename(GenerateRemoteObjectFilename(RootObjectId, MigrationContext.OwnerServerId));
	TUniquePtr<FArchive> FileReader{ IFileManager::Get().CreateFileReader(*Filename) };
	checkf(FileReader.IsValid(), TEXT("Unable to create file reader for remote object %s"), *MigrationContext.ObjectId.ToString());

	FRemoteObjectFileHeader Header;
	DeserializeFileHeader(*FileReader, Header);

	if (Header.FileType == ERemoteObjectFileType::Redirector)
	{
		// We allow only one level of recursion: a redirector must point to an actual data file
		checkf(PreviousFileType != ERemoteObjectFileType::Redirector, TEXT("A remote object file redirector redirected to another redirector (Context Object ID: %s, RootObjectId: %s)"),
			*MigrationContext.ObjectId.ToString(), *RootObjectId.ToString());
		LoadObjectFromDiskInternal(MigrationContext, Header.RootObjectId, OutData, Header.FileType);
	}
	else
	{
		DeserializeNameTables(*FileReader, OutData);
		*FileReader << OutData.Bytes;
	}

	FileReader->Close();
	IFileManager::Get().Delete(*Filename, false, true, true);
}

void LoadObjectFromDisk(const FUObjectMigrationContext& MigrationContext)
{
	FRemoteObjectData ObjectData;
	LoadObjectFromDiskInternal(MigrationContext, MigrationContext.ObjectId, ObjectData, ERemoteObjectFileType::Unknown);

	// Loading the object from disk will restore the MigrationId the data was serialized with but to make things consistent (and not confusing as to where the data came from) 
	// we want a new migration id with the server id set to Database
	ObjectData.MigrationId = UE::RemoteObject::Serialization::Private::GetNextMigrationId(FRemoteServerId(ERemoteServerIdConstants::Database));

	FRemoteObjectId RootObjectId(ObjectData.GetRootSerializedObjectId());
	for (FRemoteObjectId SerializedObjectId : ObjectData)
	{
		if (SerializedObjectId != RootObjectId)
		{
			FString Filename(GenerateRemoteObjectFilename(SerializedObjectId, MigrationContext.OwnerServerId));
			IFileManager::Get().Delete(*Filename, false, true, true);
		}
	}

	// We have transferred ownership from the Database to the local server
	FRemoteServerId LocalServerId = FRemoteServerId::GetLocalServerId();
	const FRemoteServerId& DatabaseId = UE::RemoteObject::Transfer::DatabaseId;
	UE::RemoteObject::Transfer::OnObjectDataReceived(LocalServerId, MigrationContext.ObjectId, DatabaseId, ObjectData);
}

void SaveObjectToDisk(const UE::RemoteObject::Transfer::FMigrateSendParams& Params)
{
	checkf(Params.MigrationContext.ObjectId == Params.ObjectData.GetRootSerializedObjectId(), TEXT("Saving a subobject %s to disk. Only its root %s can be saved directly."),
		*Params.MigrationContext.ObjectId.ToString(), *Params.ObjectData.GetRootSerializedObjectId().ToString());

	{
		TUniquePtr<FArchive> RootFileWriter{ IFileManager::Get().CreateFileWriter(*GenerateRemoteObjectFilename(Params.MigrationContext.ObjectId, Params.MigrationContext.OwnerServerId)) };
		checkf(RootFileWriter.IsValid(), TEXT("Unable to create file writer for remote object %s"), *Params.MigrationContext.ObjectId.ToString());

		FRemoteObjectFileHeader RootHeader{ ERemoteObjectFileType::Data, Params.MigrationContext.ObjectId };
		SerializeFileHeader(*RootFileWriter, RootHeader);
		SerializeNameTables(*RootFileWriter, Params.ObjectData);
		*RootFileWriter << const_cast<TArray<FRemoteObjectBytes>&>(Params.ObjectData.Bytes);
		RootFileWriter->Close();
	}

	FRemoteObjectFileHeader SubobjectHeader{ ERemoteObjectFileType::Redirector, Params.MigrationContext.ObjectId };
	for (FRemoteObjectId SerializedObjectId : Params.ObjectData)
	{
		if (SerializedObjectId != Params.MigrationContext.ObjectId)
		{
			TUniquePtr<FArchive> SubobjectFileWriter{ IFileManager::Get().CreateFileWriter(*GenerateRemoteObjectFilename(SerializedObjectId, Params.MigrationContext.OwnerServerId)) };
			checkf(SubobjectFileWriter.IsValid(), TEXT("Unable to create file writer for remote object redirector %s"), *SerializedObjectId.ToString());
			SerializeFileHeader(*SubobjectFileWriter, SubobjectHeader);
			SubobjectFileWriter->Close();
		}
	}
}

} // namespace UE::RemoteObject::Serialization::Disk

namespace UE::RemoteObject::Serialization
{

UObject* FindArchetype(const UObject* InObj)
{
	using namespace UE::CoreUObject::Private;

	if (GUseImmutableArchetypes)
	{
		bool bNativeObject = true;
		// No need to get the immutable CDO for BP class instance or their subobjects as BP classes are assets themselves and although their CDOs can still technically be modified at runtime
		// they never are because they can be GC'd and reset to their original state when they're reloaded so they're not a persistent storage like the native CDOs
		for (const UObject* OuterIt = InObj; OuterIt && bNativeObject; OuterIt = FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(OuterIt))
		{
			UClass* ObjectClass = OuterIt->GetClass();
			bNativeObject = !ObjectClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint);
		}
		if (bNativeObject)
		{
			return const_cast<UObject*>(FindImmutableArchetype(InObj));
		}
	}
	return InObj->GetArchetype();
}

const FRemoteObjectConstructionParams* FRemoteObjectConstructionOverrides::Find(FName InName, UObject* InOuter) const
{
	const FRemoteObjectId OuterId(InOuter);
	// At the moment the number of serialized objects is usually pretty low (< 10) so no need for hash table lookups
	for (const FRemoteObjectConstructionParams& RemoteObjectParams : Overrides)
	{
		if (RemoteObjectParams.Name == InName && RemoteObjectParams.OuterId == OuterId)
		{
			return &RemoteObjectParams;
		}
	}
	return nullptr;
}

FRemoteObjectConstructionOverridesStack& FRemoteObjectConstructionOverridesStack::Get()
{
	checkf(IsInGameThread(), TEXT("Currently FRemoteObjectConstructionOverridesStack singleton can only be accessed from the game thread"));
	static FRemoteObjectConstructionOverridesStack Singleton;
	return Singleton;
}

FRemoteObjectConstructionOverridesStack::~FRemoteObjectConstructionOverridesStack()
{
	checkf(Stack.Num() == 0, TEXT("RemoteObjectConstructionOverridesStack still contains overrides"));
}

const FRemoteObjectConstructionParams* FRemoteObjectConstructionOverridesStack::Find(FName InName, UObject* InOuter) const
{
	const FRemoteObjectConstructionParams* Result = nullptr;
	for (int32 StackIndex = Stack.Num() - 1; !Result && StackIndex >= 0; --StackIndex)
	{
		Result = Stack[StackIndex]->Find(InName, InOuter);
	}
	return Result;
}

enum class ERemoteReferenceType : uint8
{
	None = 0,
	IdOnly = 1,
	PathName = 2
};

FArchive& operator<<(FArchive& Ar, ERemoteReferenceType& RefType)
{
	uint8 SerialziedType = (uint8)RefType;
	Ar << SerialziedType;
	RefType = (ERemoteReferenceType)SerialziedType;
	return Ar;
}

/**
* Structure that holds information about a reference to an object
* Helpful in avoiding calculating the same reference properties multiple times
*/
struct FRemoteObjectReferenceInfo
{
	UObject* Object = nullptr;
	FRemoteObjectId Id;
	ERemoteReferenceType Type = ERemoteReferenceType::None;
	bool bIsSubobject = false;
};

/**
* Base archive for serializing object data for migration
* Implements serialization debugging functionality (see UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING)
*/
template <typename T>
class TArchiveRemoteObjectBase : public T
{
protected:

	UObject* RootObject = nullptr;
	FRemoteObjectData& ObjectData;
	const FUObjectMigrationContext* MigrationContext = nullptr;
	TArray<uint8> SerializedBytes;
	FString ArchiveName;

#if UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING
	struct FMigratedPropertyStats
	{
		int64 Size = 0;
		int64 Count = 0;
	};

	FString SerializationScope;
	TMap<FString, TMap<FProperty*, FMigratedPropertyStats>> ObjectPropertyStats;

	void DumpStatsToLog()
	{
		if (!GDebugRemoteObjectSerialization)
		{
			return;
		}

		int64 NameTableSize = 0;
		{
			TArray<uint8> NameTableData;
			FMemoryWriter NameTableWriter(NameTableData);
			UE::RemoteObject::Serialization::Disk::SerializeNameTables(NameTableWriter, ObjectData);
			NameTableSize = NameTableData.Num();
		}

		const int64 TotalSize = NameTableSize + ObjectData.GetNumBytes();

		UE_LOGF(LogRemoteSerialization, Log, "%ls Object Data stats for %ls %ls (Object Data toal: %d, total: %lld):", *GetArchiveName(), *FRemoteObjectId(RootObject).ToString(), *GetFullNameSafe(RootObject), ObjectData.GetNumBytes(), TotalSize);
		UE_LOGF(LogRemoteSerialization, Log, "  Name Table total size: %lld (Names: %d, RemoteIds: %d, Paths: %d)", NameTableSize, ObjectData.Tables.Names.Num(), ObjectData.Tables.RemoteIds.Num(), ObjectData.PathNames.Num());

		TArray<FString> SortedPathNames;
		for (const FPackedRemoteObjectPathName& PathName : ObjectData.PathNames)
		{
			SortedPathNames.Add(PathName.ToString(ObjectData.Tables));
		}
		SortedPathNames.Sort();
		for (const FString& PathName : SortedPathNames)
		{
			UE_LOGF(LogRemoteSerialization, Log, "    %ls", *PathName);
		}

		for (const TPair<FString, TMap<FProperty*, FMigratedPropertyStats>>& ObjectStatsPair : ObjectPropertyStats)
		{
			const FString& Obj = ObjectStatsPair.Key;
			const TMap<FProperty*, FMigratedPropertyStats>& PropStats = ObjectStatsPair.Value;

			int64 Total = 0;
			for (const TPair<FProperty*, FMigratedPropertyStats>& PropertyPair : PropStats)
			{
				const FMigratedPropertyStats& Stats = PropertyPair.Value;
				Total += Stats.Size;
			}

			UE_LOGF(LogRemoteSerialization, Log, "  Data serialized for %ls (Total: %lld):", Obj.Len() ? *Obj : TEXT("Native Serialize"), Total);

			for (const TPair<FProperty*, FMigratedPropertyStats>& PropertyPair : PropStats)
			{
				FProperty* Prop = PropertyPair.Key;
				const FMigratedPropertyStats& Stats = PropertyPair.Value;
				UE_LOGF(LogRemoteSerialization, Log, "    %ls: size: %lld, count: %lld", Prop ? *GetFullNameSafe(Prop) : TEXT("Native Serialize"), Stats.Size, Stats.Count);
			}
		}
	}
#endif // #if UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING

public:

	TArchiveRemoteObjectBase(FRemoteObjectData& InObjectData, const FUObjectMigrationContext* InContext, const TCHAR* InArchiveName)
		: T(SerializedBytes)
		, ObjectData(InObjectData)
		, MigrationContext(InContext)
		, ArchiveName(InArchiveName)
	{
		T::SetIsPersistent(false);
		T::SetUseUnversionedPropertySerialization(true);
		T::SetPortFlags(PPF_AvoidRemoteObjectMigration);

		if (T::IsLoading())
		{
			int32 NumSerializedBytes = InObjectData.GetNumBytes();
			SerializedBytes.Reserve(NumSerializedBytes);
			for (const FRemoteObjectBytes& Chunk : ObjectData.Bytes)
			{
				SerializedBytes.Append(Chunk.Bytes);
			}			
		}
	}

	virtual ~TArchiveRemoteObjectBase()
	{
		if (SerializedBytes.Num() > ObjectData.GetNumBytes())
		{
			ObjectData.Bytes.Empty();

			constexpr int32 MaxChunkSize = int32(TNumericLimits<uint16>::Max()) - 1;
			const uint8* RawBytes = SerializedBytes.GetData();
			int32 NumBytes = SerializedBytes.Num();
			int32 NumChunks = (NumBytes + MaxChunkSize - 1) / MaxChunkSize;
			ObjectData.Bytes.AddDefaulted(NumChunks);

			for (FRemoteObjectBytes& Chunk : ObjectData.Bytes)
			{
				int32 ChunkSize = FMath::Min(NumBytes, MaxChunkSize);
				if (ChunkSize > 0)
				{
					Chunk.Bytes.AddZeroed(ChunkSize);
					FMemory::Memcpy(Chunk.Bytes.GetData(), RawBytes, ChunkSize);
					RawBytes += ChunkSize;
					NumBytes -= ChunkSize;
				}
			}
		}
#if UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING
		DumpStatsToLog();
#endif
	}

	virtual FString GetArchiveName() const override
	{
		return ArchiveName;
	}

	virtual const FUObjectMigrationContext* GetMigrationContext() const override
	{
		return MigrationContext;
	}

	void SetRootObject(UObject* InRoot)
	{
		RootObject = InRoot;
	}

	UObject* GetRootObject() const
	{
		return RootObject;
	}

#if UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING
	void SetSerializationScope(const TCHAR* InScope)
	{
		if (GDebugRemoteObjectSerialization)
		{
			SerializationScope = InScope;
		}
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		if (!GDebugRemoteObjectSerialization)
		{
			T::Serialize(Data, Num);
		}
		else
		{
			FProperty* CurrentProperty = nullptr;
			if (const FArchiveSerializedPropertyChain* PropChain = T::GetSerializedPropertyChain())
			{
				if (PropChain->GetNumProperties())
				{
					CurrentProperty = PropChain->GetPropertyFromRoot(0);
				}
			}

			TMap<FProperty*, FMigratedPropertyStats>& ObjectStats = ObjectPropertyStats.FindOrAdd(SerializationScope);
			FMigratedPropertyStats& PropertyStats = ObjectStats.FindOrAdd(CurrentProperty);

			const int64 StartPos = T::Tell();

			T::Serialize(Data, Num);

			PropertyStats.Size += T::Tell() - StartPos;
			PropertyStats.Count++;
		}
	}
#endif // UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING
};

class FArchiveRemoteObjectWriter : public TArchiveRemoteObjectBase<FMemoryWriter>
{
protected:

	TArray<UObject*> ObjectsToSerialize;
	TSet<UObject*> SerializedObjects;
	TMap<FName, FNameIndexType> NameMap;
	TMap<UObject*, FNameIndexType> PathNameMap;
	TMap<FRemoteObjectId, FNameIndexType> RemoteIdMap;
	TSet<UObject*>* ReferencedObjectsSet = nullptr;
	ERemoteObjectSerializationFlags RemoteSerializationFlags = ERemoteObjectSerializationFlags::None;
public:

	static FRemoteObjectReferenceInfo GetReferenceInfo(const FObjectPtr& ObjPtr, UObject* InRootObject)
	{
		using namespace UE::RemoteObject;
		using namespace UE::RemoteObject::Handle;

		FRemoteObjectReferenceInfo Info;
		Info.Id = ObjPtr.GetRemoteId();
		if (ObjPtr)
		{
			if (ObjPtr.GetResidence() == EResidence::Local)
			{
				Info.Object = ObjPtr.Get();
				Info.bIsSubobject = Info.Object->IsIn(InRootObject);

				if (Info.Object == InRootObject || Info.bIsSubobject || (Info.Id.GetServerId().IsValid() && Info.Id.GetServerId() != FRemoteServerId::GetLocalServerId() && !Info.Id.IsAsset()))
				{
					Info.Type = ERemoteReferenceType::IdOnly;
				}
				else
				{

					Info.Type = ERemoteReferenceType::PathName;
				}
			}
			else
			{
				Info.Type = ERemoteReferenceType::IdOnly;
			}
		}
		return Info;
	}

	virtual bool PopulateObjectHeader(UObject* Object, FRemoteObjectHeader& OutHeader)
	{
		using namespace UE::RemoteObject;
		using namespace UE::RemoteObject::Private;
		using namespace UE::CoreUObject::Private;

		OutHeader.Name = AddNameToNameMap(Object->GetFName());
		OutHeader.RemoteId = AddRemoteIdToIdMap(FRemoteObjectId(Object));
		OutHeader.Class = Object->GetClass();
		UObject* Outer = FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Object);
		OutHeader.Outer = Outer;
		OutHeader.Archetype = Object->GetArchetype();
		OutHeader.InternalFlags = (int32)(Object->GetInternalFlags() & EInternalObjectFlags::Garbage);

		return true;
	}

	void CreatePackedObjectHeaders(const TArray<FRemoteObjectHeader>& InHeaders)
	{
		checkf(ObjectData.SerializedObjectHeaders.Num() == 0, TEXT("Packed object headers already created"));

		for (const FRemoteObjectHeader& Header : InHeaders)
		{
			FPackedRemoteObjectHeader& PackedHeader = ObjectData.SerializedObjectHeaders.Emplace_GetRef();
			PackedHeader.NameIndex = Header.Name;
			PackedHeader.IdIndex = Header.RemoteId;
			PackedHeader.ClassIndex = AddPathNameToNameMap(Header.Class);
		}
	}
protected:

	virtual FRemoteObjectReferenceInfo GetReferenceInfo(const FObjectPtr& ObjPtr) const
	{
		return GetReferenceInfo(ObjPtr, RootObject);
	}

	virtual void WriteObjectReference(const FRemoteObjectReferenceInfo& RefInfo)
	{
		using namespace UE::CoreUObject::Private;

		*this << RefInfo.Type;
		// Always serialize unique id as objects may not exist on the other server and then we may need to pull them from this server
		FNameIndexType IdIndex = 0;
		if (RefInfo.Type != ERemoteReferenceType::None)
		{
			IdIndex = AddRemoteIdToIdMap(RefInfo.Id);
			*this << IdIndex;

			if (RefInfo.Type == ERemoteReferenceType::PathName)
			{
				FNameIndexType PathNameIndex = AddPathNameToNameMap(RefInfo.Object);
				*this << PathNameIndex;
			}
		}		
	}

	void WriteObjectPtr(const FObjectPtr& ObjPtr)
	{
		using namespace UE::CoreUObject::Private;
		using namespace UE::RemoteObject::Private;
		using namespace UE::RemoteObject::Handle;

		FRemoteObjectReferenceInfo Info = GetReferenceInfo(ObjPtr);
		WriteObjectReference(Info);

		if (ObjPtr && ObjPtr.GetResidence() == EResidence::Local)
		{
			UObject* Object = ObjPtr.Get();

			// Anything can be marked as a remote reference, even assets in which case we rely on this flag to be set so that GC calls
			// StoreObjectToDatabase for any remotely referenced asset (and only for remotely referenced assets) that's about to be GC'd
			if (!IsRemoteReference(Object))
			{
				if (ReferencedObjectsSet)
				{
					UE_AUTORTFM_OPEN
					{
						ReferencedObjectsSet->Add(Object);
					};
				}
			}

			if (Info.Type == ERemoteReferenceType::IdOnly && Info.bIsSubobject && !EnumHasAnyFlags(RemoteSerializationFlags, ERemoteObjectSerializationFlags::Resetting))
			{
				checkf(ObjectsToSerialize.Contains(Object), TEXT("Serializing an object reference (%s) that is not in the ObjectsToSerialize list for root %s"), *GetPathNameSafe(Object), *GetPathNameSafe(RootObject));
			}
		}
	}

public:

	FArchiveRemoteObjectWriter(
		UObject* InRootObject, 
		FRemoteObjectData& OutObjectData, 
		const FUObjectMigrationContext* InMigrationContext, 
		const TCHAR* ArchiveName = nullptr, TSet<UObject*>* OutReferencedObjectsSet = nullptr,
		ERemoteObjectSerializationFlags Flags = ERemoteObjectSerializationFlags::None)
		: TArchiveRemoteObjectBase<FMemoryWriter>(OutObjectData, InMigrationContext, ArchiveName ? ArchiveName : TEXT("RemoteObjectWriter"))
		, RemoteSerializationFlags(Flags)
	{
		static_assert(sizeof(FNameIndexType) == sizeof(FRemoteObjectTables::FNameIndexType), "Name Index type must match the type used for storing remote object path name indices");

		SetRootObject(InRootObject);
		ObjectsToSerialize.Add(InRootObject);

		ReferencedObjectsSet = OutReferencedObjectsSet;
	}

	virtual UObject* GetArchetypeFromLoader(const UObject* Obj) override
	{
		return FindArchetype(Obj);
	}

	FNameIndexType AddNameToNameMap(const FName& Name)
	{
		FNameIndexType NameIndex = TNumericLimits<FNameIndexType>::Max();
		if (FNameIndexType* ExistingNameIndex = NameMap.Find(Name))
		{
			NameIndex = *ExistingNameIndex;
		}
		else
		{
			int32 NewIndex = ObjectData.Tables.Names.Add(Name);
			NameIndex = IntCastChecked<FNameIndexType>(NewIndex);
			NameMap.Add(Name, NameIndex);
		}
		return NameIndex;
	}

	FNameIndexType AddRemoteIdToIdMap(FRemoteObjectId RemoteId)
	{
		FNameIndexType IdIndex = TNumericLimits<FNameIndexType>::Max();
		if (FNameIndexType* ExistingIdIndex = RemoteIdMap.Find(RemoteId))
		{
			IdIndex = *ExistingIdIndex;
		}
		else
		{
			int32 NewIndex = ObjectData.Tables.RemoteIds.Add(RemoteId);
			IdIndex = IntCastChecked<FNameIndexType>(NewIndex);
			RemoteIdMap.Add(RemoteId, IdIndex);
		}
		return IdIndex;
	}

	FNameIndexType AddPathNameToNameMap(UObject* Object)
	{
		FNameIndexType PathNameIndex = TNumericLimits<FNameIndexType>::Max();
		if (FNameIndexType* ExistingPathNameIndex = PathNameMap.Find(Object))
		{
			PathNameIndex = *ExistingPathNameIndex;
		}
		else
		{
			int32 NewPathNameIndex = ObjectData.PathNames.AddDefaulted();
			PathNameIndex = IntCastChecked<FNameIndexType>(NewPathNameIndex);

			FPackedRemoteObjectPathName& NewPathName = ObjectData.PathNames[PathNameIndex];

			// Store individual indices of FNames of every object in this object's outer chain
			for (UObject* OuterChain = Object; OuterChain; OuterChain = UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(OuterChain))
			{
				NewPathName.RemoteIds.Add(AddRemoteIdToIdMap(FRemoteObjectId(OuterChain)));
				NewPathName.Names.Add(AddNameToNameMap(OuterChain->GetFName()));
			}
			PathNameMap.Add(Object, PathNameIndex);
		}
		return PathNameIndex;
	}

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		WriteObjectPtr(FObjectPtr(Obj));
		return *this;
	}

	virtual FArchive& operator<<(FName& Name) override
	{
		FNameIndexType NameIndex = AddNameToNameMap(Name);
		*this << NameIndex;
		return *this;
	}

	virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FObjectPtr& Value) override 
	{ 
		WriteObjectPtr(Value);
		return *this;
	}
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return FArchiveUObject::SerializeSoftObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override { return FArchiveUObject::SerializeSoftObjectPath(*this, Value); }
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override
	{
		using namespace UE::RemoteObject::Handle;
		FObjectPtr Ptr;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		const FRemoteObjectId WeakPtrId = Value.GetRemoteId();
		if (GetResidence(WeakPtrId) != EResidence::Local)
		{
			Ptr = FObjectPtr(WeakPtrId);
		}
		else
#endif
		{
			Ptr = Value.Get(/** bEvenIfGarbage */ true);
		}
		WriteObjectPtr(Ptr);

		return *this;
	}

	TArray<UObject*>& GetObjectsToSerialize()
	{
		return ObjectsToSerialize;
	}

	virtual void SerializeRemoteObject(UObject* Object, const FRemoteObjectHeader& Header)
	{		
		Object->Serialize(*this);
	}

	FRemoteObjectId GetRemoteObjectId(FNameIndexType IdIndex) const
	{
		return ObjectData.Tables.RemoteIds[IdIndex];
	}
};

template <typename T>
struct TRemoteObjectArchiveScope
{
	TArchiveRemoteObjectBase<T>& Ar;

	TRemoteObjectArchiveScope(TArchiveRemoteObjectBase<T>& InAr, const TCHAR* InScope)
		: Ar(InAr)
	{
#if UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING
		Ar.SetSerializationScope(InScope);
#endif
	}
	TRemoteObjectArchiveScope(TArchiveRemoteObjectBase<T>& InAr, UObject* InObjectScope)
		: Ar(InAr)
	{
#if UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING
		UObject* Root = Ar.GetRootObject();
		UObject* RootOuter = UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Root);
		Ar.SetSerializationScope(*InObjectScope->GetFullName(RootOuter));
#endif
	}
	~TRemoteObjectArchiveScope()
	{
#if UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING
		Ar.SetSerializationScope(nullptr);
#endif
	}
};

using FRemoteObjectWriterScope = TRemoteObjectArchiveScope<FMemoryWriter>;
using FRemoteObjectReaderScope = TRemoteObjectArchiveScope<FMemoryReader>;

/**
* Helper archive that serializes the difference between archetypes and their instances
* This is achieved using delta serialization but the data we serialize against is coming from instances of the archetypes
* Effectively this is the opposite how delta serialization normally works which serializes instances of archetypes against the archetypes
* In other words this archive is used to serialize archetypes, not their instances (more specifically the delta between the archetype and its instance)
*/
class FArchetypeDeltaWriter : public FArchiveRemoteObjectWriter
{
	UObject* InstanceScope = nullptr;
	const UObject* ArchetypeScope = nullptr;

public:

	FArchetypeDeltaWriter(UObject* InRootObject, FRemoteObjectData& OutObjectData)
		: FArchiveRemoteObjectWriter(InRootObject, OutObjectData, /*MigrationContext*/ nullptr, TEXT("RemoteArchetypeDeltaWriter"), nullptr, ERemoteObjectSerializationFlags::Resetting)
	{
	}

	virtual UObject* GetArchetypeFromLoader(const UObject* InArchetype) override
	{
		checkf(!InArchetype || InArchetype->IsTemplate(), TEXT("FArchetypeDeltaWriter should only be serializing archetypes but is trying to find an instance for %s"), *GetPathNameSafe(InArchetype));
		
		// Since this archive serializes the archetype we want the archetype of that archetype to be its instance (effectively reversing the object -> archetype relationship)
		// this way we will delta serialize the difference between the archetype and its instance
		if (InstanceScope)
		{
			checkf(InArchetype == ArchetypeScope, TEXT("Serializing an instance %s without a matching archetype (expected: %s)"), *GetPathNameSafe(InstanceScope), *GetPathNameSafe(ArchetypeScope));
			return InstanceScope;
		}

		UE_LOGF(LogRemoteSerialization, Warning, "FArchetypeDeltaWriter does not contain an archetype mapping for %ls", *GetPathNameSafe(InArchetype));
		return FArchiveRemoteObjectWriter::GetArchetypeFromLoader(InArchetype);
	}

	virtual void SerializeRemoteObject(UObject* Object, const FRemoteObjectHeader& Header) override
	{		
		UObject* ObjectToSerialize = nullptr;
		ArchetypeScope = nullptr;

		if (const UObject* Archetype = FindArchetype(Object))
		{
			ObjectToSerialize = const_cast<UObject*>(Archetype);
			ArchetypeScope = Archetype;
		}
		checkf(ObjectToSerialize, TEXT("Serializing an instance %s without matching archetype"), *GetPathNameSafe(Object));

		InstanceScope = Object;
		ObjectToSerialize->Serialize(*this);
		InstanceScope = nullptr;
		ArchetypeScope = nullptr;
	}

protected:

	virtual void WriteObjectReference(const FRemoteObjectReferenceInfo& RefInfo) override
	{
		using namespace UE::CoreUObject::Private;

		FRemoteObjectReferenceInfo ReplacementInfo(RefInfo);

		if (InstanceScope && ArchetypeScope)
		{
			// If we're serializing a reference to an archetype try to replace it with a reference to its instance
			// This way the produced delta between the archetype and its instance will be correctly pointing to the instances of default subobjects
			// we can then deserialize over the archetype instance.
			if (RefInfo.Object == ArchetypeScope)
			{
				ReplacementInfo.Object = InstanceScope;
				ReplacementInfo.Id = FRemoteObjectId(InstanceScope);
			}
		}

		FArchiveRemoteObjectWriter::WriteObjectReference(ReplacementInfo);
	}
};


FArchive& operator<<(FArchive& Ar, FRemoteObjectHeader& Header)
{
	Ar << Header.Name;
	Ar << Header.RemoteId;
	Ar << Header.Class;
	Ar << Header.Outer;
	Ar << Header.Archetype;
	Ar << Header.InternalFlags;
	Ar << Header.StartOffset;

	return Ar;
}

class FArchiveRemoteObjectReader : public TArchiveRemoteObjectBase<FMemoryReader>
{
	const TArray<FName>& Names;
	const TArray<FRemoteObjectId>& RemoteIds;
	TArray<UObject*> ResolvedPathNameObjects;
	const ERemoteObjectSerializationFlags DeserializeFlags = ERemoteObjectSerializationFlags::None;

	void ResolvePathNames(const FRemoteObjectData& InObjectData, TArray<UObject*>& OutResolvedObjects)
	{
		OutResolvedObjects.Reset(InObjectData.PathNames.Num());
		for (const FPackedRemoteObjectPathName& PathName : InObjectData.PathNames)
		{
			UObject* Object = PathName.Resolve(InObjectData.Tables);
			OutResolvedObjects.Add(Object);
		}
	}

	ERemoteReferenceType ReadObjectReference(FObjectPtr& Value)
	{
		using namespace UE::RemoteObject::Private;
		using namespace UE::RemoteObject::Handle;

		FObjectPtr NewValue;
		ERemoteReferenceType Type;		
		*this << Type;

		if (Type != ERemoteReferenceType::None)
		{
			FRemoteObjectId ObjId;
			{
				FNameIndexType IdIndex;
				*this << IdIndex;
				ObjId = RemoteIds[IdIndex];
			}

			bool bNeedsResolvingWithId = true;
			if (Type == ERemoteReferenceType::PathName)
			{
				FNameIndexType PathNameIndex = TNumericLimits<FNameIndexType>::Max();
				*this << PathNameIndex;

				// In some situations (like resetting an object to its archetype state) we might want to preserve
				// references to remote objects because we might end up migrating them mid-deserialization 
				// Overwriting them could also potentially discard any changes made to them on another server
				if (EnumHasAnyFlags(DeserializeFlags, ERemoteObjectSerializationFlags::PreserveRemoteReferences) && Value.GetResidence() != EResidence::Local)
				{
					bNeedsResolvingWithId = false;
				}
				else
				{
					// Try to resolve path name immediately as we expect the object to exist in memory.
					UObject* Obj = ResolvedPathNameObjects[PathNameIndex];
					if (Obj)
					{
						MarkAsRemoteReference(Obj);
						NewValue = FObjectPtr(Obj);
						bNeedsResolvingWithId = false;
					}
				}
			}
			if (bNeedsResolvingWithId)
			{
				// If the serialized reference was not found in memory or if the reference was serialized as id-only 
				// keep it as an unresolved ObjectPtr and store a pointer to it so that we can try to resolve it after all objects have been deserialize
#if UE_WITH_REMOTE_OBJECT_HANDLE
				if (!EnumHasAnyFlags(DeserializeFlags, ERemoteObjectSerializationFlags::PreserveRemoteReferences) || Value.GetResidence() == EResidence::Local)
				{
					FObjectHandle Handle = FObjectHandle::FromIdNoResolve(ObjId);
					NewValue = FObjectPtr(Handle);
				}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
			}
		}

		SetObjectReference(Value, NewValue);

		return Type;
	}

protected:

	virtual void SetObjectReference(FObjectPtr& CurrentValue, FObjectPtr NewValue)
	{
		CurrentValue = NewValue;
	}

public:

	/**
	 * @param InObjectData:  The serialized object data we are should deserialize
	 * @param InResolvedPathNames:  The existing Resolved Objects that corresponds to InObjectData.PathNames
	 * @param bInAssignedOwnership:  We must take ownership of the objects we deserialize. We could already have ownership of those objects.
	 * @param InDeserializeFlags:  The flags for how we should treat references during deserialization.
	 */
	FArchiveRemoteObjectReader(FRemoteObjectData& InObjectData, const FUObjectMigrationContext* InMigrationContext, ERemoteObjectSerializationFlags InDeserializeFlags, const TCHAR* ArchiveName = nullptr)
		: TArchiveRemoteObjectBase<FMemoryReader>(InObjectData, InMigrationContext, ArchiveName ? ArchiveName : TEXT("RemoteObjectReader"))
		, Names(InObjectData.Tables.Names)
		, RemoteIds(InObjectData.Tables.RemoteIds)
		, DeserializeFlags(InDeserializeFlags)
	{
		ResolvePathNames(InObjectData, ResolvedPathNameObjects);
	}

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		FObjectPtr Value;
		ReadObjectReference(Value);
		check(Value.GetResidence() == EResidence::Local);
		Obj = Value.Get();
		return *this;
	}

	virtual FArchive& operator<<(FName& Name) override
	{
		FNameIndexType NameIndex = 0;
		*this << NameIndex;
		Name = Names[NameIndex];
		return *this;
	}

	virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FObjectPtr& Value) override 
	{
		ReadObjectReference(Value);
		return *this;
	}

	virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return FArchiveUObject::SerializeSoftObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override { return FArchiveUObject::SerializeSoftObjectPath(*this, Value); }
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override 
	{
		using namespace UE::RemoteObject::Handle;

		FObjectPtr Ptr;
		ReadObjectReference(Ptr);

		if (Ptr.GetResidence() == EResidence::Local)
		{
			Value = Ptr.Get();
		}
#if UE_WITH_REMOTE_OBJECT_HANDLE
		else
		{
			Value = FWeakObjectPtr(Ptr.GetRemoteId());
		}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

		return *this;
	}

	virtual void SerializeRemoteObject(UObject* Object)
	{
		Object->Serialize(*this);
	}
};

FRemoteObjectConstructionOverrides::FRemoteObjectConstructionOverrides(const FRemoteObjectData& ObjectData, const TArray<FRemoteObjectHeader>& InObjectHeaders)
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;

	Overrides.Reserve(InObjectHeaders.Num());
	for (const FRemoteObjectHeader& Header : InObjectHeaders)
	{
		FRemoteObjectConstructionParams& Params = Overrides.Emplace_GetRef();
		Params.Name = ObjectData.GetName(Header.Name);
#if UE_WITH_REMOTE_OBJECT_HANDLE
		Params.OuterId = ::GetRemoteObjectId(Header.Outer.GetHandle());
#endif
		Params.RemoteId = ObjectData.GetRemoteObjectId(Header.RemoteId);
		Params.SerialNumber = Header.SerialNumber;
	}
}

UObject* ConstructRemoteObject(const FRemoteObjectData& ObjectData, const FRemoteObjectHeader& Header, ERemoteObjectSerializationFlags DeserializeFlags, const FUObjectMigrationContext* MigrationContext)
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;

	UClass* Class = CastChecked<UClass>(Header.Class.Get());
	UObject* Outer = nullptr;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Header.Outer.GetResidence() == EResidence::Local)
	{
		Outer = Header.Outer.Get();
	}
	else
	{
		Outer = StaticFindObjectFastInternal(Header.Outer.GetRemoteId());
		UE_CLOGF(!Outer, LogRemoteSerialization, Fatal, "Failed to resolve an Outer when constructing remote object")
	}
#endif
	FName Name = ObjectData.GetName(Header.Name);
	FRemoteObjectId SerializedObjectId = ObjectData.GetRemoteObjectId(Header.RemoteId);

	// The object may already exist in memory (it could be a default subobject of an object we've just created)
	UObject* Object = StaticFindObjectFast(Class, Outer, Name);
	if (Object)
	{
		if (FRemoteObjectId(Object) != SerializedObjectId)
		{
			UE_LOGF(LogRemoteSerialization, Warning, "Received remote object %ls with identical pathname (%ls) as a local object %ls. Remote object will be renamed.", *SerializedObjectId.ToString(), *Object->GetPathName(), *FRemoteObjectId(Object).ToString());
			Name = FName();
			Object = nullptr;
		}
	}
	else
	{
		Object = StaticFindObjectFastInternal(SerializedObjectId);
		if (Object)
		{
			// The object already exists on this server but has been renamed
			Name = Object->GetFName();
		}
	}

	// Note that although we expect all deserialized objects to be remote or to not exist, we might be deserializing a default subobject (component) created in it's owner (Outer) constructor 
	// in which case it would not be marked as remote. In this case we don't need to re-construct an object that was created in one of the previous calls to ConstructRemoteObject.
	const bool bIsRemote = (MigrationContext ? MigrationContext->GetResidence(SerializedObjectId) : GetResidence(Object)) == EResidence::Remote;
	// If not or the object is marked as remote (which means we brought it back before it was GC'd) (re)construct it.	
	// Unless we explcitly want to re-use existing (valid / not marked as garbage) objects and skip re-construction to avoid side-effects
	// Note that even then an object may not exist on this server (it could've been constructed on a different server when its owner was migrated or was simply GC'd)
	const bool bSkipConstruction = IsValid(Object) && (!bIsRemote || EnumHasAnyFlags(DeserializeFlags, ERemoteObjectSerializationFlags::UseExistingObjects));
	if (!bSkipConstruction)
	{
		FStaticConstructObjectParameters Params(Class);
		Params.Outer = Outer;
		Params.Name = Name;
		Params.SerialNumber = Header.SerialNumber;
		Params.Template = Header.Archetype.Get();
#if UE_WITH_REMOTE_OBJECT_HANDLE
		Params.RemoteId = SerializedObjectId;
#endif

		{
			// In case we're allocating on top of existing object that's marked as remote don't try to resolve any of its references since they all are going to be destroyed anyway
			FUnsafeToMigrateScope UnsafeToMigrateScope;

			// Using StaticConstructObject_Internal to pass the extra parameters (RemoteId and SubobjectOverrides) which are not exposed to normal APIs
			Object = StaticConstructObject_Internal(Params);
		}
	}
	else if (MigrationContext)
	{
		UE_CLOGF(MigrationContext->GetObjectMigrationRecvType(SerializedObjectId) == EObjectMigrationRecvType::Borrowed,
			LogRemoteSerialization, Warning, "Received a borrowed object that was not reconstructed %ls (IsValid: %ls, Residence: %ls, UseExistingObjects: %ls)", *SerializedObjectId.ToString(),
			IsValid(Object) ? TEXT("yes") : TEXT("no"),
			EnumToString(MigrationContext->GetResidence(SerializedObjectId)),
			EnumHasAnyFlags(DeserializeFlags, ERemoteObjectSerializationFlags::UseExistingObjects) ? TEXT("yes") : TEXT("no")
		);
	}

	if (!Object)
	{
		return nullptr;
	}

	checkf(FRemoteObjectId(Object) == SerializedObjectId, TEXT("Created an object with a different ID:%s than requested:%s"), *FRemoteObjectId(Object).ToString(), *SerializedObjectId.ToString());

	if (UE::RemoteObject::Handle::FRemoteObjectStub* Stub = UE::RemoteObject::Private::FindRemoteObjectStub(SerializedObjectId))
	{
		// If we don't reset the migrated data immediatetly we will enter an infinite recursion when trying to resolve this object's inner object's outer (this Object)
		Stub->MigratedData = nullptr;
	}
	SetResidence(Object, EResidence::Local, FRemoteServerId::GetLocalServerId());

	// Update internal flags on the migrated object. It's possible the object being migrated already existed in memory on this server and had the EInternalObjectFlags::Garbage flag set.
	// Unless the migrated version also had this flag set we need to clear it.
	// It's also possible that the local object didn't have this flag set but the migrated one has so we need to set it on this server too (it's not impossible to migrate objects marked as garbage)
	EInternalObjectFlags InternalFlags = (EInternalObjectFlags)Header.InternalFlags;
	// Clearing and setting the garbage flag needs to happen through dedicated functions
	if (!(InternalFlags & EInternalObjectFlags::Garbage))
	{
		Object->ClearGarbage();
	}
	else
	{
		Object->MarkAsGarbage();
		InternalFlags &= ~EInternalObjectFlags::Garbage;
	}
	// Any other internal flags can be set with SetInternalFlags 
	if (InternalFlags != EInternalObjectFlags::None)
	{
		Object->SetInternalFlags(InternalFlags);
	}

	return Object;
}

FORCENOINLINE void SortRemoteObjectHeadersByOuter(FArchiveRemoteObjectWriter& Ar, TArray<FRemoteObjectHeader>& ObjectHeaders)
{
	using namespace UE::CoreUObject::Private;

	struct FOuterSortPredicate
	{
		TMap<UObject*, int32> OuterLevelMap;

		bool operator()(const FRemoteObjectHeader& A, const FRemoteObjectHeader& B) const
		{
			int32 OuterALevel = OuterLevelMap.FindChecked(NoResolveObjectHandleNoRead(A.Outer.GetHandle()));
			int32 OuterBLevel = OuterLevelMap.FindChecked(NoResolveObjectHandleNoRead(B.Outer.GetHandle()));
			return OuterALevel > OuterBLevel;
		}
	} Predicate;

	// Assign a level index to each of the Serialized Objects' Outers. Indices grow from the Innermost to the Outermost Outer
	// e.g.                     World (3)
	//                             |
	//                      PersistentLevel (2)
	//                             |
	//                          Actor (1)
	//                         /         \
	//                ComponentA (0)    ComponentB (0)
	//                    |                  |
	//                SubobjectA        SubobjectB (leaf = no index)

	for (FRemoteObjectHeader& Header : ObjectHeaders)
	{
		int32 CurrentObjectOuterLevel = 0;
		for (UObject* Outer = NoResolveObjectHandleNoRead(Header.Outer.GetHandle()); Outer; Outer = FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Outer))
		{
			int32& OuterIndex = Predicate.OuterLevelMap.FindOrAdd(Outer, CurrentObjectOuterLevel);
			if (OuterIndex < CurrentObjectOuterLevel)
			{
				OuterIndex = CurrentObjectOuterLevel;
			}
			CurrentObjectOuterLevel++;
		}
	}

	// Sort object headers from the outermost to the innermost object
	if (Predicate.OuterLevelMap.Num() > 1)
	{
		ObjectHeaders.Sort<FOuterSortPredicate>(Predicate);
	}

	// Make sure the root object header is still the first header in the array
	checkf(FRemoteObjectId(Ar.GetRootObject()) == Ar.GetRemoteObjectId(ObjectHeaders[0].RemoteId), TEXT("Root object id (%s) does not match the first object header id (%s)."),
		*FRemoteObjectId(Ar.GetRootObject()).ToString(), *Ar.GetRemoteObjectId(ObjectHeaders[0].RemoteId).ToString());

#if UE_WITH_REMOTEOBJECT_SERIALIZATION_VERIFICATION && UE_WITH_REMOTE_OBJECT_HANDLE
	// Make sure the header array is sorted
	for (int32 HeaderIndex = 0; HeaderIndex < ObjectHeaders.Num(); ++HeaderIndex)
	{
		FRemoteObjectHeader& Header = ObjectHeaders[HeaderIndex];
		FRemoteObjectId OuterId = Header.Outer.GetRemoteId();
		int32 OuterHeaderIndex = ObjectHeaders.IndexOfByPredicate([OuterId, &Ar](const FRemoteObjectHeader& MaybeOuterHeader)
			{
				return Ar.GetRemoteObjectId(MaybeOuterHeader.RemoteId) == OuterId;
			});
		checkf(OuterHeaderIndex < HeaderIndex, TEXT("Serialized object's[%d] %s Outer[%d] %s is serialized after the object"),
			HeaderIndex, *Ar.GetRemoteObjectId(Header.RemoteId).ToString(ERemoteIdToStringVerbosity::PathName), OuterHeaderIndex, *OuterId.ToString(ERemoteIdToStringVerbosity::PathName));
	}
#endif // UE_WITH_REMOTEOBJECT_SERIALIZATION_VERIFICATION
}

void SerializeObjectDataInternal(FArchiveRemoteObjectWriter& Ar, UObject* RequestedObject, FRemoteObjectId RequestedObjectId, TSet<UObject*>& OutObjects, ERemoteObjectSerializationFlags SerializationFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SerializeRemoteObjectDataInternal);

	using namespace UE::RemoteObject::Handle;

	TArray<FRemoteObjectHeader> ObjectHeaders;
	int32 Version = 0;
	int64 HeaderOffset = 0;

	int64 OffsetOfHeaderOffset = 0;
	{
		FRemoteObjectWriterScope Scope(Ar, TEXT("Header"));
		Ar << Version;
		Ar << RequestedObjectId;
		OffsetOfHeaderOffset = Ar.Tell();
		Ar << HeaderOffset;
	}

	int32 SerializedObjectIndex = 0;
	bool bSerializedRequestedObject = false;

	TSet<UObject*> ProcessedObjects;
	do
	{
		for (; SerializedObjectIndex < Ar.GetObjectsToSerialize().Num(); ++SerializedObjectIndex)
		{
			UObject* ObjectToSerialize = Ar.GetObjectsToSerialize()[SerializedObjectIndex];
			if (!ProcessedObjects.Contains(ObjectToSerialize))
			{
				ProcessedObjects.Add(ObjectToSerialize);

				FRemoteObjectWriterScope Scope(Ar, ObjectToSerialize);				
				FRemoteObjectHeader Header;
				if (Ar.PopulateObjectHeader(ObjectToSerialize, Header))
				{					
					FRemoteObjectHeader& SerializedHeader = ObjectHeaders.Emplace_GetRef(Header);
					OutObjects.Add(ObjectToSerialize);

					SerializedHeader.StartOffset = Ar.Tell();

					Ar.SerializeRemoteObject(ObjectToSerialize, SerializedHeader);
				}
				else
				{
					UE_LOGF(LogRemoteSerialization, Warning, "Unable to serialize object (asset: %ls) %ls", 
						FRemoteObjectId(ObjectToSerialize).IsAsset() ? TEXT("yes") : TEXT("no"),
						*ObjectToSerialize->GetPathName());
				}
			}
		}

		bSerializedRequestedObject = OutObjects.Contains(RequestedObject);
		if (!bSerializedRequestedObject)
		{
			checkf(!ProcessedObjects.Contains(RequestedObject), TEXT("%s couldn't be serialized"), *RequestedObject->GetPathName());

			// InObject was a default subobject (see GRemoteObjectsMigrateFullHierarchy) but when we serialized its parent 
			// it turned out that the parent had no direct reference to InObject in which case we need to manually add InObject to ObjectsToSerialize list
			Ar.GetObjectsToSerialize().Add(RequestedObject);
		}
	} while (!bSerializedRequestedObject);

	HeaderOffset = Ar.Tell();
	Ar.Seek(OffsetOfHeaderOffset);
	Ar << HeaderOffset;
	Ar.Seek(HeaderOffset);

	// Sort objects from the outermost to the innermost. 
	// Skip sorting if there's just one object or we're resetting a borrowed object because CDO delta serialization substitutes serialized objects (archetypes) with their instances and the outers don't match
	if (OutObjects.Num() > 1 && !EnumHasAnyFlags(SerializationFlags, ERemoteObjectSerializationFlags::Resetting))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SerializeRemoteObjectDataSortHeaders);
		SortRemoteObjectHeadersByOuter(Ar, ObjectHeaders);
	}

	Ar.CreatePackedObjectHeaders(ObjectHeaders);

	{
		FRemoteObjectWriterScope Scope(Ar, TEXT("ObjectHeaders"));
		Ar << ObjectHeaders;
	}
}

FRemoteObjectData SerializeObjectData(UObject* InObject, TSet<UObject*>& OutObjects, TSet<UObject*>& OutReferencedObjects, const FUObjectMigrationContext* MigrationContext, ERemoteObjectSerializationFlags InFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SerializeRemoteObjectData);

	UObject* Object = InObject;
	if (GRemoteObjectsMigrateFullHierarchy && !EnumHasAnyFlags(InFlags, ERemoteObjectSerializationFlags::SkipCanonicalRootSearch))
	{
		Object = FindCanonicalRootObjectForSerialization(Object);
	}

	FRemoteObjectData ObjectData;
	ObjectData.MigrationId = UE::RemoteObject::Serialization::Private::GetNextMigrationId();
	FRemoteObjectId RequestedObjectId(InObject);

	{
		FArchiveRemoteObjectWriter Ar(Object, ObjectData, MigrationContext, nullptr, &OutReferencedObjects);
		Ar.SetMigratingRemoteObjects(true);

		EGetObjectsFlags GetObjectsFlags = EGetObjectsFlags::IncludeNestedObjects;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (MigrationContext && MigrationContext->RemoteServerId.IsDatabase())
		{
			GetObjectsFlags |= EGetObjectsFlags::EvenIfUnreachable;
		}
#endif
		GetObjectsWithOuter(Ar.GetRootObject(), Ar.GetObjectsToSerialize(), GetObjectsFlags);

		SerializeObjectDataInternal(Ar, InObject, RequestedObjectId, OutObjects, InFlags);
	}
	checkf(ObjectData.SerializedObjectHeaders.Num(), TEXT("Serialized object data for object %s is empty"), *RequestedObjectId.ToString());

#if UE_WITH_REMOTEOBJECT_SERIALIZATION_VERIFICATION
	// We are doing an aggressive warning log here because these missing reference issues are hard to track down:
	TArray<UObject*> AllSubobjects;
	GetObjectsWithOuter(InObject, AllSubobjects);
	AllSubobjects.RemoveAll([OutObjects](UObject* SubObj) { return OutObjects.Contains(SubObj); });

	auto WrappedGetNameSafe = [](const UObject* O) { return FString::Printf(TEXT("%s (%s)"), *GetNameSafe(O), *(FObjectPtr{ const_cast<UObject*>(O) }.GetRemoteId().ToString())); };
	UE_CLOGF(!AllSubobjects.IsEmpty(), LogRemoteSerialization, Warning, "SerializeObjectData for %ls did not serialize all of its SubObjects. Missed:\n  %ls", *GetNameSafe(InObject), *FString::JoinBy(AllSubobjects, TEXT(", "), WrappedGetNameSafe));
#endif

	return MoveTemp(ObjectData);
}

int32 DeserializeObjectDataInternal(FArchiveRemoteObjectReader& Ar,
	FRemoteObjectData& ObjectData,
	const FUObjectMigrationContext* MigrationContext,
	TArray<FRemoteObjectId>& OutObjectRemoteIds,
	TArray<UObject*>& OutObjects,
	ERemoteObjectSerializationFlags DeserializeFlags);

void ResetRemoteObjects(const TArray<UObject*>& ObjectHierarchy)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ResetRemoteObjects)

	for (UObject* ObjectToReset : ObjectHierarchy)
	{
		// This code is effectively copying the delta between the archetype and its instance (ObjectToReset) to the instance so that the instance (ObjectToReset) is identical to the archetype.
		// At the end of this operation ObjectToReset will be referencing the archetype's subobjects but that's ok because references to the instances of subobjects
		// will be restored when deserializing the migrated data from another server.
		FRemoteObjectData ArchetypeDelta;
		{
			FRemoteObjectId RequestedObjectId(ObjectToReset);
			FArchetypeDeltaWriter Ar(ObjectToReset, ArchetypeDelta);

			// Even though we pass the Object to reset to SerializeObjectDataInternal, the FArchetypeDeltaWriter will actually serialize the differences between the archetypes' data and their respective instances
			// This will produce a delta between the archetypes and their instances which will then be used to restore the instances' state to the archetypes' default state
			TSet<UObject*> SerializedObjects;
			SerializeObjectDataInternal(Ar, ObjectToReset, RequestedObjectId, SerializedObjects, ERemoteObjectSerializationFlags::Resetting | ERemoteObjectSerializationFlags::SkipCanonicalRootSearch);
		}

		{
			// Deserialize archetype data on top of the object to reset to restore its state to the archetype values
			TArray<UObject*> DeserializedObjects;
			TArray<FRemoteObjectId> DeserializedIds;
			// When deserializing archetype delta we want to:
			// Preserve any references to remote objects that have not been migrated yet (this is because we can't generate archetype delta for them because they don't exist on this server and we don't have their data)
			// Additionally we don't want to recursively re-enter this function so let the deserialization process know we're already resetting migrated object(s)
			ERemoteObjectSerializationFlags DeserializationFlags = ERemoteObjectSerializationFlags::PreserveRemoteReferences | ERemoteObjectSerializationFlags::UseExistingObjects | ERemoteObjectSerializationFlags::Resetting;
			FArchiveRemoteObjectReader Ar(ArchetypeDelta, /*MigrationContext*/ nullptr, DeserializationFlags, TEXT("RemoteArchetypeDeltaReader"));
			DeserializeObjectDataInternal(Ar, ArchetypeDelta, /*MigrationContext*/ nullptr, DeserializedIds, DeserializedObjects, DeserializationFlags);
			check(DeserializedObjects.Num() == 1 && DeserializedObjects[0] ==ObjectToReset);
		}
	}
}

int32 DeserializeObjectDataInternal(FArchiveRemoteObjectReader& Ar,
							FRemoteObjectData& ObjectData,
							const FUObjectMigrationContext* MigrationContext,
							TArray<FRemoteObjectId>& OutObjectRemoteIds,
							TArray<UObject*>& OutObjects,
							ERemoteObjectSerializationFlags DeserializeFlags)
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;
	TRACE_CPUPROFILER_EVENT_SCOPE(DeserializeObjectData);

	int32 Version = 0;
	FRemoteObjectId RequestedObjectId;
	int64 HeaderOffset = 0;
	TArray<FRemoteObjectHeader> ObjectHeaders;
	TArray<UObject*> ResolvedPathNameObjects;
	int32 RequestedObjectIndex = -1;
	const bool bResetting = EnumHasAnyFlags(DeserializeFlags, ERemoteObjectSerializationFlags::Resetting);

	// If we are being assigned ownership, we *must* take ownership of the objects.  Note: We also may already have ownership of those objects. 

	// If we're calling this function to reset an object to its archetype state then we don't want FArchiveRemoteObjectReader to be marked as migrating remote objects (Which it is by default)
	Ar.SetMigratingRemoteObjects(!bResetting);

	{
		FRemoteObjectReaderScope Scope(Ar, TEXT("Header"));
		Ar << Version;
		Ar << RequestedObjectId;
		Ar << HeaderOffset;
	}

	const int64 ObjectDataOffset = Ar.Tell();

	Ar.Seek(HeaderOffset);
	{
		FRemoteObjectReaderScope Scope(Ar, TEXT("ObjectHeaders"));
		Ar << ObjectHeaders;
	}
	Ar.Seek(ObjectDataOffset);	

	if (ObjectHeaders.Num())
	{
		const FRemoteServerId LocalServerId = FRemoteServerId::GetLocalServerId();

		// Try to find any existing WeakObject serial numbers for the objects that are about to be constructed
		for (FRemoteObjectHeader& ObjectHeader : ObjectHeaders)
		{
			FRemoteObjectId SerializedObjectId = ObjectData.GetRemoteObjectId(ObjectHeader.RemoteId);
			if (FRemoteObjectStub* Stub = FindRemoteObjectStub(SerializedObjectId))
			{
				ObjectHeader.SerialNumber = Stub->SerialNumber;
				if (!Stub->Name.IsNone())
				{
					// Remote object could've been renamed when it was migrated so always make sure that it has the same name locally
					ObjectHeader.Name = ObjectData.Tables.AddUniqueName(Stub->Name);
				}
			}
		}

		const bool bReturningBorrowedObject = GResetBorrowedObjects && (MigrationContext && MigrationContext->IsOwned(ObjectData.GetRootSerializedObjectId()));
		if (bReturningBorrowedObject)
		{
			// Root object is owned by this server so we're receiving an object that was borrowed by another server.
			// In this case we don't need to reconstruct anything and we can re-use the objects that are already in memory.
			DeserializeFlags |= ERemoteObjectSerializationFlags::UseExistingObjects;
		}

		// Construct all objects first
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConstructRemoteObjects);

			FRemoteObjectConstructionOverrides ConstructionOverrides(ObjectData, ObjectHeaders);
			FRemoteObjectConstructionOverridesScope OverridesScope(&ConstructionOverrides);

			OutObjectRemoteIds.Reserve(ObjectHeaders.Num());
			OutObjects.Reserve(ObjectHeaders.Num());
			for (FRemoteObjectHeader& ObjectHeader : ObjectHeaders)
			{
				FRemoteObjectId SerializedObjectId = ObjectData.GetRemoteObjectId(ObjectHeader.RemoteId);
				OutObjectRemoteIds.Add(SerializedObjectId);
				OutObjects.Add(ConstructRemoteObject(ObjectData, ObjectHeader, DeserializeFlags, MigrationContext));
			}

			checkf(OutObjectRemoteIds.Num() == ObjectData.SerializedObjectHeaders.Num(), TEXT("The number of constructed object ids (%d) does not match the number of serialized object headers (%d)"), OutObjectRemoteIds.Num(), ObjectData.SerializedObjectHeaders.Num());
#if UE_WITH_REMOTEOBJECT_SERIALIZATION_VERIFICATION
			for (int32 ObjectIndex = 0; ObjectIndex < OutObjectRemoteIds.Num(); ++ObjectIndex)
			{
				FRemoteObjectId ConstructedObjectId = OutObjectRemoteIds[ObjectIndex];
				FRemoteObjectId SerializedObjectId = ObjectData.GetSerializedObjectId(ObjectIndex);
				checkf(ConstructedObjectId == SerializedObjectId, TEXT("Constructed object id[%d] (%s) does not match the serialized object id (%s)"), ObjectIndex, *ConstructedObjectId.ToString(), *SerializedObjectId.ToString());
			}
			for (FSerializedRemoteObjectIterator It(ObjectData); It; ++It)
			{
				UObject* SerializedObject = OutObjects[It.GetIndex()];
				UClass* HeaderClass = It.GetClass();
				checkf(SerializedObject->GetClass() == HeaderClass, TEXT("Constructed object's [%d] (%s) class does not match the serialized object header class (%s)"), It.GetIndex(), *SerializedObject->GetFullName(), *HeaderClass->GetPathName());
			}
#endif
		}

		UObject* RootObject = OutObjects.Num() ? OutObjects[0] : nullptr;
		if (!ensureMsgf(RootObject, TEXT("%hs had objects to construct but could not reconstruct them"), __func__))
		{
			return RequestedObjectIndex;
		}

		// If we're already resetting a borrowed object we don't want to change the ownership (see below for an exception) until the object is deserialized using remote server data
		// and we also don't want to recursively reset the object
		if (!bResetting && RootObject)
		{
			TArray<UObject*> CreatedObjects;
			CreatedObjects.Reserve(OutObjects.Num());
			CreatedObjects.Add(RootObject);
			GetObjectsWithOuter(RootObject, CreatedObjects);

			// ensure remote object stubs are created (ownership will be assigned after PostMigrate)
			for (UObject* Object : CreatedObjects)
			{
				FRemoteObjectId ObjectId(Object);
				// Call FindOrAddRemoteObjectStub instead of RegisterRemoteObjectId to always update the resident server id
				if (FRemoteObjectStub* Stub = FindOrAddRemoteObjectStub(ObjectId, LocalServerId))
				{
					// It's possible we created more objects than we received (maybe one of the subobjects has already been GC'd on the other side)
					// In this case make sure to assign the new object ownership to the server that owns the root object
					if (MigrationContext && !MigrationContext->TryGetOwningServerId(ObjectId).IsValid())
					{
						Stub->OwningServerId = MigrationContext->TryGetOwningServerId(OutObjectRemoteIds[0]);
					}
				}
			}

#if UE_WITH_REMOTEOBJECT_SERIALIZATION_VERIFICATION
			for (UObject* DeserializedObject : OutObjects)
			{
				checkf(!DeserializedObject || CreatedObjects.Contains(DeserializedObject), TEXT("Deserialized an object (%s) that's outside of the root object hierarchy"), *GetPathNameSafe(DeserializedObject));
			}
#endif

			if (bReturningBorrowedObject)
			{
				checkf(RootObject == OutObjects[0], TEXT("Wrong RootObject: %s, expected: %s"), *GetPathNameSafe(RootObject), *GetPathNameSafe(OutObjects[0]));
				ResetRemoteObjects(CreatedObjects);
			}
		}

		// Deserialize all objects
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(DeserializeIntoCreatedObjects);

			Ar.SetRootObject(OutObjects.Num() ? OutObjects[0] : nullptr);
			for (int32 ObjectIndex = 0; ObjectIndex < OutObjects.Num(); ++ObjectIndex)
			{
				if (UObject* Object = OutObjects[ObjectIndex])
				{
					FRemoteObjectReaderScope Scope(Ar, Object);
					Ar.Seek(ObjectHeaders[ObjectIndex].StartOffset);
					Ar.SerializeRemoteObject(Object);
					if (RequestedObjectIndex == -1 && RequestedObjectId == FRemoteObjectId(Object))
					{
						RequestedObjectIndex = ObjectIndex;
					}
				}
			}
		}
	}
	checkf(RequestedObjectIndex >= 0, TEXT("Received remote object data but the requested object (%s) was not deserialized"), *RequestedObjectId.ToString());
	return RequestedObjectIndex;
}

int32 DeserializeObjectData(FRemoteObjectData& ObjectData,
	const FUObjectMigrationContext* MigrationContext,
	TArray<FRemoteObjectId>& OutObjectRemoteIds,
	TArray<UObject*>& OutObjects,
	ERemoteObjectSerializationFlags DeserializeFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(DeserializeRemoteObjectData);

	FArchiveRemoteObjectReader Ar(ObjectData, MigrationContext, DeserializeFlags);
	return DeserializeObjectDataInternal(Ar, ObjectData, MigrationContext, OutObjectRemoteIds, OutObjects, DeserializeFlags);
}

UObject* FindCanonicalRootObjectForSerialization(UObject* Object)
{
	// find the outermost migration root
	UObject* Cursor = Object;

	// if we walk the outer chain and don't happen to find 
	// any migration roots, default to using the object itself
	UObject* Result = Object;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	// walk the cursor up the entire Outer chain and update
	// Result with the outermost Outer that is a migration root
	// (this covers the case where we find a migration root
	// nested in another, we pick the outermost one)
	while (Cursor)
	{
		if (Cursor->IsMigrationRoot())
		{
			Result = Cursor;
		}

		Cursor = Cursor->GetOuter();
	}
#endif

	return Result;
}

} // namespace UE::RemoteObject::Serialization

namespace UE::RemoteObject::Serialization::Network
{
	// List of RPCs while borrowed per-object (Annotations are temporary data you can attach to UObjects)
	COREUOBJECT_API FUObjectAnnotationSparse<FBorrowedRpcAnnotations, /*bAutoRemove=*/true> RemoteObjects_QueuedRPCs;

	class FRemoteObjectReferenceWriter : public FMemoryWriter
	{
	public:
		FRemoteObjectReferenceWriter(TArray<uint8>& InBytes)
			: FMemoryWriter(InBytes)
		{
		}

		virtual FArchive& operator<<(UObject*& Obj) override
		{
			FRemoteObjectReference RemoteReference { FObjectPtr(Obj) };
			RemoteReference.Serialize(*this);
			return *this;
		}

		virtual FArchive& operator<<(FObjectPtr& Value) override 
		{
			FRemoteObjectReference RemoteReference(Value);
			RemoteReference.Serialize(*this);
			return *this;
		}

		virtual FArchive& operator<<(FWeakObjectPtr& Value) override
		{
			FRemoteObjectReference RemoteReference(Value);
			RemoteReference.Serialize(*this);
			return *this;
		}

		virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); }
		virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return FArchiveUObject::SerializeSoftObjectPtr(*this, Value); }
		virtual FArchive& operator<<(FSoftObjectPath& Value) override { return FArchiveUObject::SerializeSoftObjectPath(*this, Value); }
	};

	class FRemoteObjectReferenceReader : public FMemoryReader
	{
	public:
		FRemoteObjectReferenceReader(TArray<uint8>& InBytes)
			: FMemoryReader(InBytes)
		{
		}

		virtual FArchive& operator<<(UObject*& Obj) override
		{
			FRemoteObjectReference RemoteReference;
			RemoteReference.Serialize(*this);
			Obj = RemoteReference.Resolve();
			return *this;
		}

		virtual FArchive& operator<<(FObjectPtr& Value) override 
		{
			FRemoteObjectReference RemoteReference;
			RemoteReference.Serialize(*this);
			Value = RemoteReference.ToObjectPtr();
			return *this;
		}

		virtual FArchive& operator<<(FWeakObjectPtr& Value) override
		{
			FRemoteObjectReference RemoteReference;
			RemoteReference.Serialize(*this);
			Value = RemoteReference.ToWeakPtr();
			return *this;
		}

		virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return FArchiveUObject::SerializeLazyObjectPtr(*this, Value); }
		virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return FArchiveUObject::SerializeSoftObjectPtr(*this, Value); }
		virtual FArchive& operator<<(FSoftObjectPath& Value) override { return FArchiveUObject::SerializeSoftObjectPath(*this, Value); }
	};

	void EnqueueRPC(UObject* RootObject, UObject* SubObject, UFunction* Function, void* Parameters)
	{
		// Serialize Parameters, taking into account objects need to be stored as remote object references
		TArray<uint8> SerializedParams;
		{
			FRemoteObjectReferenceWriter RemoteObjectReferenceWriter(SerializedParams);
			FStructuredArchiveFromArchive StructuredRemoteObjectReferenceArchive(RemoteObjectReferenceWriter);

			for (TFieldIterator<FProperty> PropertyIt(Function); PropertyIt && PropertyIt->HasAnyPropertyFlags(CPF_Parm); ++PropertyIt)
			{
				uint8* Param = PropertyIt->ContainerPtrToValuePtr<uint8>(Parameters);
				PropertyIt->SerializeItem(StructuredRemoteObjectReferenceArchive.GetSlot(), Param);
			}
		}

		// Append the data to the existing annotation (or create a new one)
		FBorrowedRpcAnnotations AnnotationInstance = RemoteObjects_QueuedRPCs.GetAndRemoveAnnotation(RootObject);
		FBorrowedRpcAnnotations::FBorrowedSerializedRPC& BorrowedRPC = AnnotationInstance.SerializedRPCs.Emplace_GetRef();

		// Create an entry in the annotations for this RPC
		BorrowedRPC.SubObject = SubObject;
		BorrowedRPC.FunctionName = Function->GetFName();
		BorrowedRPC.Params = MoveTemp(SerializedParams);

		// Store the modified annotation back
		RemoteObjects_QueuedRPCs.AddAnnotation(RootObject, MoveTemp(AnnotationInstance));

		UE_LOGF(LogRemoteSerialization, Verbose, "Enqueing RPC %ls on root object %ls and sub-object %ls", *Function->GetName(), *GetNameSafe(RootObject), *GetNameSafe(SubObject));
	}

	void SerializeRPCQueue(UObject* RootObject, FArchive& Ar)
	{
		// Remove if saving (or create if none exists / loading).
		FBorrowedRpcAnnotations AnnotationInstance = RemoteObjects_QueuedRPCs.GetAndRemoveAnnotation(RootObject);
		AnnotationInstance.Serialize(Ar);

		// If we're loading, add our annotation back for PostMigrate to handle it correctly
		if (Ar.IsLoading() && !AnnotationInstance.IsDefault())
		{
			RemoteObjects_QueuedRPCs.AddAnnotation(RootObject, MoveTemp(AnnotationInstance));
		}
	}

	void ProcessRPC(UObject* TargetObject, FBorrowedRpcAnnotations::FBorrowedSerializedRPC& RPC)
	{
		UFunction* Function = TargetObject->FindFunction(RPC.FunctionName);
		if (!Function)
		{
			UE_LOGF(LogRemoteSerialization, Warning, "Cannot find function %ls on object %ls", *RPC.FunctionName.ToString(), *TargetObject->GetName());
			return;
		}

		uint8* Params = static_cast<uint8*>(FMemory_Alloca(Function->ParmsSize));
		FMemory::Memzero(Params, Function->ParmsSize);

		for (TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->InitializeValue_InContainer(Params);
		}

		// Deserailize RPC arguments into memory.
		FRemoteObjectReferenceReader RemoteObjectReferenceReader(RPC.Params);
		FStructuredArchiveFromArchive StructuredRemoteObjectReferenceArchive(RemoteObjectReferenceReader);

		for (TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			uint8* PropertyMemory = It->ContainerPtrToValuePtr<uint8>(Params);
			It->SerializeItem(StructuredRemoteObjectReferenceArchive.GetSlot(), PropertyMemory);
		}

		// Call the RPC function.
		UE_LOGF(LogRemoteSerialization, Verbose, "Processing queued RPC %ls on %ls", *Function->GetName(), *TargetObject->GetName());
		ensureMsgf(UE::RemoteObject::Handle::IsOwned(TargetObject), TEXT("%hs: %s is Not Owned so we cannot process the RPC!"), __FUNCTION__, *GetNameSafe(TargetObject));
		TargetObject->ProcessEvent(Function, Params);

		for (TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
		{
			It->DestroyValue_InContainer(Params);
		}
	}

	void ProcessRPCQueue(UObject* RootObject)
	{
		FBorrowedRpcAnnotations AnnotationInstance = RemoteObjects_QueuedRPCs.GetAndRemoveAnnotation(RootObject);

		for (FBorrowedRpcAnnotations::FBorrowedSerializedRPC& RPC : AnnotationInstance.SerializedRPCs)
		{
			UObject* SubObject = RPC.SubObject.Get();
			UObject* TargetObject = SubObject ? SubObject : RootObject;

			if (!TargetObject)
			{
				UE_LOGF(LogRemoteSerialization, Error, "Cannot find the target object for function %ls on root object %ls", *RPC.FunctionName.ToString(), *GetNameSafe(RootObject));
				continue;
			}

			ProcessRPC(TargetObject, RPC);
		}
	}
}