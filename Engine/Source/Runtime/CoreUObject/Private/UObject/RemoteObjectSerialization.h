// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/RemoteObjectTransfer.h"

namespace UE::RemoteObject::Handle { struct FRemoteObjectStub; }

#ifndef UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING
#define UE_WITH_REMOTEOBJECTARCHIVE_DEBUGGING (!(UE_BUILD_SHIPPING || UE_BUILD_TEST) && !NO_LOGGING)
#endif

namespace UE::RemoteObject::Serialization
{
	enum class ERemoteObjectSerializationFlags : uint8
	{
		None = 0,
		UseExistingObjects = (1 << 0),			// If possible re-use existing UObjects and don't reconstruct them when deserializing object data
		PreserveRemoteReferences = (1 << 1),		// Don't overwrite references to objects that are remote
		Resetting = (1 << 2),						// Indicates that the serialization process is resetting an object to its archetype state
		SkipCanonicalRootSearch = (1 << 3),			// Don't attempt to find a canonical root for serialization
	};
	ENUM_CLASS_FLAGS(ERemoteObjectSerializationFlags);

	/**
	* Basic information needed to construct deserialized remote object (this information does not get serialized in UObject::Serialize())
	*/
	struct FRemoteObjectHeader
	{
		FRemoteObjectTables::FNameIndexType Name;
		FRemoteObjectTables::FNameIndexType RemoteId;
		TObjectPtr<UClass> Class;
		TObjectPtr<UObject> Outer;
		TObjectPtr<UObject> Archetype;
		int32 InternalFlags = 0;
		int64 StartOffset = 0;

		/** Transient (SerialNumber is local-only but it's stored here for convenience when deserializing object data) */
		int32 SerialNumber = 0;
	};

	/**
	* Basic information needed to construct remote (sub)object
	*/
	struct FRemoteObjectConstructionParams
	{
		FName Name;
		FRemoteObjectId OuterId;
		FRemoteObjectId RemoteId;
		int32 SerialNumber = 0;
	};

	/**
	* Stores basic information for constructing remote (sub)objects.
	* Prevents unnecessary calls to FRemoteObjectId::Generate() when constructing default subobjects (or in general objects constructed in remote objects' constructors)
	* Sets the SerialNumber during subobject construction so that any weak pointers also constructed in constructors that may point to a subobject, has the correct SerialNumber
	*/
	class FRemoteObjectConstructionOverrides
	{
		TArray<FRemoteObjectConstructionParams> Overrides;

	public:

		FRemoteObjectConstructionOverrides() = default;
		explicit FRemoteObjectConstructionOverrides(const FRemoteObjectData& ObjectData, const TArray<FRemoteObjectHeader>& InObjectHeaders);

		/**
		* Finds object construction overrides for an object that will be constricted with the specified Name and Outer
		*/
		const FRemoteObjectConstructionParams* Find(FName InName, UObject* InOuter) const;
	};

	/**
	* Singleton that stores the current stack of remote object construction overrides. 
	* This singleton can be used to access construction overrides when constructing remote objects when they're being deserialized from a remote server
	* or (in the future) when constructing objects from loaded packages with remote object ids baked in.
	* Note that atm this is a game-thread only object.
	*/
	class FRemoteObjectConstructionOverridesStack final
	{
		TArray<const FRemoteObjectConstructionOverrides*> Stack;

	public:

		FRemoteObjectConstructionOverridesStack() = default;
		~FRemoteObjectConstructionOverridesStack();

		COREUOBJECT_API static FRemoteObjectConstructionOverridesStack& Get();

		void Push(const FRemoteObjectConstructionOverrides& InOverrides)
		{
			Stack.Push(&InOverrides);
		}
		void Pop()
		{
			Stack.Pop(EAllowShrinking::No);
		}
		bool IsEmpty() const
		{
			return !Stack.Num();
		}
		COREUOBJECT_API const FRemoteObjectConstructionParams* Find(FName InName, UObject* InOuter) const;
	};

	/**
	* Pushes remote object construction overrides onto the FRemoteObjectConstructionOverridesStack on construction and pops them on destruction. 
	*/
	class FRemoteObjectConstructionOverridesScope final
	{
		FRemoteObjectConstructionOverridesStack& Stack;
		FRemoteObjectConstructionOverrides* Overrides = nullptr;

	public:
		explicit FRemoteObjectConstructionOverridesScope(FRemoteObjectConstructionOverrides* InOverrides)
			: Stack(FRemoteObjectConstructionOverridesStack::Get())
			, Overrides(InOverrides)
		{
			if (Overrides)
			{
				Stack.Push(*Overrides);
			}
		}
		~FRemoteObjectConstructionOverridesScope()
		{
			if (Overrides)
			{
				Stack.Pop();
			}
		}
	};

	/**
	* Serializes an object and its subobject (or if the object is a default subobject, its parent and the parent's subobjects)
	* @param InObject Object to be serialized
	* @param OutObjects All objects that have been serialized (Object and its subobjects and/or parent)
	* @param OutReferencedObjects Keeps track of all objects that need to be tagged with RemoteReference
	* @param MigrationContext Contains the meta data of the current migration request
	* @param InFlags Serialization flags
	* @return Remote object data representing the serialized objects
	*/
	FRemoteObjectData SerializeObjectData(UObject* InObject, TSet<UObject*>& OutObjects, TSet<UObject*>& OutReferencedObjects, const FUObjectMigrationContext* MigrationContext, ERemoteObjectSerializationFlags InFlags = ERemoteObjectSerializationFlags::None);

	/**
	* Deserializes remote object data
	* @param ObjectData the data to deserailize 
	* @param MigrationContext the Context (meta data) of the current migration that's causing the deserialization
	* @param OutObjectRemoteIds Remote IDs of the deserialized objects
	* @param OutReceivedObjects All deserialized objects
	* @param DeserializeFlags Flags modifying the behavior of the deserialization process
	* @return Index of an object in OutReceivedObjects that was the main object the migration request was triggered for 
	* (usually 0 but if a migration requests a default subobject then its parent is also migrated and the return value will be > 0)
	*/
	int32 DeserializeObjectData(FRemoteObjectData& ObjectData, const FUObjectMigrationContext* MigrationContext, TArray<FRemoteObjectId>& OutObjectRemoteIds, TArray<UObject*>& OutReceivedObjects, ERemoteObjectSerializationFlags DeserializeFlags = ERemoteObjectSerializationFlags::None);

	/**
	* Finds the canonical 'root' object that is used for remote object serialization
	* - we trace up the chain of Outer pointers until we reach the first non default subobject
	*/
	UObject* FindCanonicalRootObjectForSerialization(UObject* Object);

} // namespace UE::RemoveObject::Serialization


namespace UE::RemoteObject::Serialization::Disk
{
	void LoadObjectFromDisk(const FUObjectMigrationContext& MigrationContext);
	void SaveObjectToDisk(const UE::RemoteObject::Transfer::FMigrateSendParams& Params);
}

namespace UE::RemoteObject::Serialization::Network
{
	/*
	* We want to be able to store "annotations" (metadata) about an Object while it's borrowed in order to
	* execute it on the owning server when the loan is returned.
	*/
	struct FBorrowedRpcAnnotations
	{
		// The data for an individual RPC call
		struct FBorrowedSerializedRPC
		{
			/** The object to execute the RPC on */
			TWeakObjectPtr<UObject> SubObject;

			/** The function name of the RPC */
			FName FunctionName;

			/** The serialized parameters of the RPC */
			TArray<uint8> Params;

			friend FArchive& operator<<(FArchive& Ar, FBorrowedSerializedRPC& Value)
			{
				Ar << Value.SubObject;
				Ar << Value.FunctionName;
				Ar << Value.Params;

				return Ar;
			}
		};

		// Serialized RPC Data that were processed (not executed) while borrowed
		TArray<FBorrowedSerializedRPC> SerializedRPCs;

		// Annotation is default (empty) and removable if it has no RPCs
		bool IsDefault() const
		{
			return SerializedRPCs.IsEmpty();
		}

		void Serialize(FArchive& Ar)
		{
			TArrayPrivateFriend::Serialize(Ar, SerializedRPCs);
		}
	};
}

template<> struct TStructOpsTypeTraits<UE::RemoteObject::Serialization::Network::FBorrowedRpcAnnotations::FBorrowedSerializedRPC> : public TStructOpsTypeTraitsBase2<UE::RemoteObject::Serialization::Network::FBorrowedRpcAnnotations::FBorrowedSerializedRPC>
{
	enum { WithSerializer = true };
};
