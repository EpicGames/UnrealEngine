// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/RemoteObjectTransfer.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/RemoteObjectSerialization.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Class.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "UObject/RemoteObject.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/ObjectHandlePrivate.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectMigrationContext.h"
#include "HAL/CriticalSection.h"
#include "AutoRTFM.h"
#include "Templates/Atomic.h"


bool FRemoteObjectData::Compare(const FRemoteObjectData& InOtherData) const
{
	if (!Tables.Compare(InOtherData.Tables))
	{
		return false;
	}
	if (PathNames.Num() != InOtherData.PathNames.Num())
	{
		return false;
	}
	for (int32 Index = 0; Index < PathNames.Num(); ++Index)
	{
		if (!PathNames[Index].Compare(InOtherData.PathNames[Index]))
		{
			return false;
		}
	}
	if (SerializedObjectHeaders.Num() != InOtherData.SerializedObjectHeaders.Num())
	{
		return false;
	}
	for (int32 Index = 0; Index < SerializedObjectHeaders.Num(); ++Index)
	{
		if (!SerializedObjectHeaders[Index].Compare(InOtherData.SerializedObjectHeaders[Index]))
		{
			return false;
		}
	}
	if (Bytes.Num() != InOtherData.Bytes.Num())
	{
		return false;
	}
	for (int32 Index = 0; Index < Bytes.Num(); ++Index)
	{
		if (!Bytes[Index].Compare(InOtherData.Bytes[Index]))
		{
			return false;
		}
	}
	return true;
}

extern bool GAllowGetObjectsReturnUnreachableObjects;

namespace UE::RemoteObject::Transfer
{
const FRemoteServerId DatabaseId(ERemoteServerIdConstants::Database);
static TAutoConsoleVariable<bool> CVarRemoveMissingMigratedComponents(TEXT("DSTM.RemoveMissingMigratedComponents"), true, TEXT("When we receive less SubObjects (Components) than we expect, remove the extras that we have locally (assume they were removed on purpose)"));

int32 GTransactionalMigration = 1;
static FAutoConsoleVariableRef CVarTransactionalMigrationEnabled(
	TEXT("DSTM.TransactionalMigration"),
	GTransactionalMigration,
	TEXT("When non-zero, migrated objects will be PostMigrated inside of a transaction"));

int32 GTransactionalDeserialization = 0;
static FAutoConsoleVariableRef CVarTransactionalDeserializationEnabled(
	TEXT("DSTM.TransactionalDeserialization"),
	GTransactionalDeserialization,
	TEXT("When non-zero, migrated objects will be created and deserialized inside of a transaction"));

// Prevent us from having to match this definition to the declartion in the header just to initialize the delegate
#define DEFINE_MIGRATION_DELEGATE(x) decltype(x) x
	DEFINE_MIGRATION_DELEGATE(RemoteObjectTransferDelegate);
	DEFINE_MIGRATION_DELEGATE(RemoteObjectDeniedTransferDelegate);
	DEFINE_MIGRATION_DELEGATE(RequestRemoteObjectDelegate);
	DEFINE_MIGRATION_DELEGATE(OnObjectsReceivedDelegate);
	DEFINE_MIGRATION_DELEGATE(OnObjectsSentDelegate);
	DEFINE_MIGRATION_DELEGATE(OnObjectsLoadedFromDiskDelegate);
	DEFINE_MIGRATION_DELEGATE(OnObjectsSavedToDiskDelegate);
	DEFINE_MIGRATION_DELEGATE(OnObjectTouchedDelegate);
	DEFINE_MIGRATION_DELEGATE(StoreRemoteObjectDataDelegate);
	DEFINE_MIGRATION_DELEGATE(RestoreRemoteObjectDataDelegate);
#undef DEFINE_MIGRATION_DELEGATE

/**
* Stores data received from a remote server and a migration context in their respective objects' stubs (possibly creating these stubs in the process)
* Remote objects are not created or deserialized inside of this function but the resident server of the received objects is set to the local server id
* @param ObjectData Data received from a remote server
* @param MigrationContext Migration context
* @returns Structure that holds the migrated data and the migration context
*/
TRefCountPtr<FTransactionalMigrationData> StoreMigratedObjectData(FRemoteObjectData& ObjectData, FUObjectMigrationContext& MigrationContext);

/**
* Removes the specified migrated data from the stubs of objects serialized inside of the migrated data
*/
void ResetMigratedData(TRefCountPtr<FTransactionalMigrationData> MigratedData);

void DeserializeObjectsFromMigratedData(FTransactionalMigrationData& InOutMigratedData);

/**
* Creates, deserializes and PostMigrates an object(s) from object data received from another server
* @param ObjectData Data received from another server
* @param MigrationContext Migration context
* @return The object requested by the current server or pushed by the sending server (when transferring ownership)
*/
UObject* MigrateObjectFromObjectData(TRefCountPtr<FTransactionalMigrationData> MigratedData);

/**
 * Remote object transfer queue. 
 * Queued requests (both send and receive) are processed on the game thread since some systems (like RPCs) that are used for transfering object data are GT-only
 */

class FRemoteObjectRequest
{
public:
	FRemoteTransactionId RequestId;
	FRemoteWorkPriority Priority;

	// this set fills up with object ids that we have ever touched
	// while running the transaction
	TSet<FRemoteObjectId> RequiredObjects;
	TMap<FRemoteObjectId, FRemoteObjectId> RequiredObjectsCanonicalRootByObjectID;
	TSet<FRemoteObjectId> RequiredObjectsCanonicalRoots;

	// when a transaction aborts and requires an object, this gets
	// filled in with the object id
	FRemoteObjectId NewRequiredObject;

	// this set gets cleared before running the 
	// transaction and only tracks the objects touched 
	// during the most recent run
	TSet<FRemoteObjectId> UsedObjects;

	FRemoteTransactionId GetRequestId() const { return RequestId; }
	FRemoteWorkPriority GetPriority() const { return Priority; }
};

class FObjectMigrationRequest
{
public:
	FRemoteObjectId ObjectId;
	FRemoteServerId DestinationServerId;
	FRemoteWorkPriority RequestPriority;
};

class FObjectMigrationRequests
{
public:
	FRemoteObjectId ObjectId;

	// IndividualRequests is sorted by priority (highest first)
	TArray<FObjectMigrationRequest> IndividualRequests;
};

class FPendingObjectRequest
{
public:
	FRemoteObjectId ObjectId;
	FRemoteWorkPriority RequestPriority;
};

class FRemoteObjectTransferQueue : public FRemoteSubsystem<FRemoteObjectTransferQueue, FRemoteObjectRequest>
{
	// these are objects that we have outstanding requests for (across all requests)
	TArray<FPendingObjectRequest> PendingObjectRequests;

	TSet<UObject*> MultiServerCommitObjectsToReturn;
	TSet<UObject*> MultiServerCommitSentObjects;
	TSet<UObject*> MultiServerCommitReferencedObjects;

	// this is a list of objects that we have locally that 
	// other servers are asking for
	TArray<FObjectMigrationRequests> PendingObjectMigrationRequests;

public:
	bool bInMultiServerCommit = false;

	const TCHAR* NameForDebug() final { return TEXT("RemoteObjectTransferQueue"); }

	void BeginRequest() final
	{
	}

	void TickSubsystem() final
	{
		using namespace UE::RemoteObject::Private;
		using namespace UE::RemoteObject::Handle;
		using namespace UE::RemoteObject::Serialization;

		// go through the list of pending object migration requests and see if we can satisfy any of them
		for (int32 MigrationRequestsIndex = 0; MigrationRequestsIndex < PendingObjectMigrationRequests.Num(); )
		{
			FObjectMigrationRequests& MigrationRequests = PendingObjectMigrationRequests[MigrationRequestsIndex];

			UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations processing (%d) requests for obj %ls",
					MigrationRequests.IndividualRequests.Num(),
					*MigrationRequests.ObjectId.ToString());

			FRemoteObjectId CanonicalRootObjectId;
			FRemoteServerId OwnerServerId;		

			UObject* Object = StaticFindObjectFastInternal(MigrationRequests.ObjectId);
			TRefCountPtr<FTransactionalMigrationData> MigratedData;

			if (Object && GetResidence(Object) == EResidence::Local)
			{
				// find the canonical root object id to use for arbitration
				UObject* const RootObject = FindCanonicalRootObjectForSerialization(Object);
				CanonicalRootObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(RootObject);
				OwnerServerId = GetOwnerServerId(RootObject);
			}
			else if (FRemoteObjectStub* Stub = FindRemoteObjectStub(MigrationRequests.ObjectId))
			{
				MigratedData = Stub->MigratedData;
				if (MigratedData)
				{
					// Get the canonical root object id to use for arbitration
					CanonicalRootObjectId = Stub->MigratedData->Data.GetRootSerializedObjectId();
					FRemoteObjectStub* RootStub = Stub->Id == CanonicalRootObjectId ? Stub : FindRemoteObjectStub(CanonicalRootObjectId);
					check(RootStub);
					OwnerServerId = RootStub->OwningServerId;
				}
			}

			if (CanonicalRootObjectId.IsValid())
			{
				UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations from %ls: obj %ls (is local)", *FRemoteServerId::GetLocalServerId().ToString(),
					*MigrationRequests.ObjectId.ToString()); 

				// look at the list of requests for this object and pick the one with the highest priority for arbitration
				// (the list is sorted, so the first element is always the highest priority)
				FObjectMigrationRequest* HighestPriorityRequest = &MigrationRequests.IndividualRequests[0];

				// verify the list is sorted
				for (int32 MigrationRequestIndex = 1; MigrationRequestIndex < MigrationRequests.IndividualRequests.Num(); MigrationRequestIndex++)
				{
					FObjectMigrationRequest* Request = &MigrationRequests.IndividualRequests[MigrationRequestIndex];
					check(!IsHigherPriority(Request->RequestPriority, HighestPriorityRequest->RequestPriority));
				}

				UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations obj %ls highest priority request is %ls",
					*MigrationRequests.ObjectId.ToString(),
					*HighestPriorityRequest->RequestPriority.ToString());

				// now look over all active transactions and see if this request is higher priority than all of them
				bool bObjectLocked = false;

				for (int32 RequestIndex = 0; RequestIndex < GetRequestCount(); RequestIndex++)
				{
					FRemoteObjectRequest* ExistingRequest = GetRequestByIndex(RequestIndex);

					if (IsHigherPriority(ExistingRequest->GetPriority(), HighestPriorityRequest->RequestPriority))
					{
						// ExistingRequest is higher priority, now see if it actually needs this object
						bObjectLocked = ExistingRequest->RequiredObjectsCanonicalRoots.Contains(CanonicalRootObjectId);
					}

					if (bObjectLocked)
					{
						break;
					}
				}

				if (!bObjectLocked)
				{
					static FName SendObjectRegionName("ObjectTransferQueue_SendObject");
					UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(SendObjectRegionName);

					// send the object
					UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations sending obj %ls to %ls %ls",
						*HighestPriorityRequest->ObjectId.ToString(),
						*HighestPriorityRequest->DestinationServerId.ToString(),
						*HighestPriorityRequest->RequestPriority.ToString());

					FUObjectMigrationContext MigrationContext {
						.ObjectId = HighestPriorityRequest->ObjectId,
						.RemoteServerId = HighestPriorityRequest->DestinationServerId,
						.OwnerServerId = OwnerServerId,
						.MigrationSide = EObjectMigrationSide::Send
					};
					FScopedObjectMigrationContext ScopedMigrationContext(MigrationContext);
					
					TSet<TObjectPtr<UObject>> SentObjects;
					TSet<UObject*> ReferencedObjects;
					FRemoteObjectData ObjectData;
					if (!MigratedData)
					{
						TSet<UObject*> SerializedObjects;
						ObjectData = SerializeObjectData(Object, SerializedObjects, ReferencedObjects, &MigrationContext);
						SentObjects = ObjectPtrWrap(SerializedObjects);
					}
					else
					{
						checkf(MigratedData->State == EMigratedDataState::Received || (MigratedData->State == EMigratedDataState::Deserialized && !GTransactionalDeserialization),
							TEXT("Sending migrated data back without PostMigrating it but the data is in unexpected state (%s)"), EnumToString(MigratedData->State));
						// Pretend we PostMigrated the data for the reset function
						MigratedData->State = EMigratedDataState::PostMigrated;
						ResetMigratedData(MigratedData);
						ObjectData = MoveTemp(MigratedData->Data);						
						
						for (FRemoteObjectId SerializedId : ObjectData)
						{
							UObject* SerializedObject = StaticFindObjectFastInternal(SerializedId);
							if (SerializedObject)
							{
								SentObjects.Add(SerializedObject);
							}
							else
							{
#if UE_WITH_REMOTE_OBJECT_HANDLE
								FRemoteObjectStub* Stub = FindRemoteObjectStub(SerializedId);
								checkf(Stub, TEXT("Stub for %s must exist when sending the stored object data"), *SerializedId.ToString());
								FObjectHandle Handle(Stub);
								SentObjects.Add(TObjectPtr<UObject>(FObjectPtr(Handle)));
#endif
							}
						};
					}
					MigrationContext.CacheObjectStatus(ObjectData);

					checkf(SentObjects.Num() > 0, TEXT("unable to migrate object %s to %s"), *HighestPriorityRequest->ObjectId.ToString(), *HighestPriorityRequest->DestinationServerId.ToString());

					for (UObject* RO : ReferencedObjects)
					{
						MarkAsRemoteReference(RO);
					}

#if UE_WITH_REMOTE_OBJECT_HANDLE
					for (TObjectPtr<UObject> SO : SentObjects)
					{
						FRemoteObjectId SubObjectId = SO.GetRemoteId();
						UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations sending obj %ls sent subobject : %ls [%ls]",
							*HighestPriorityRequest->ObjectId.ToString(),
							*SubObjectId.ToString(),
							*SO.GetName());
					}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

					FMigrateSendParams SendRemoteObjectParams = { .MigrationContext = MigrationContext, .ObjectData = ObjectData };
					SendRemoteObject(SendRemoteObjectParams);

					{
						// PostMigrate callbacks may be manipulating migrated objects and may try to migrate them back so prevent that from happening since
						// they all will be destroyed anyway
						FUnsafeToMigrateScope UnsafeToMigrateScope;

						// Notify that objects have been migrated to a remote server
						OnObjectsSentDelegate.Broadcast(SentObjects, ObjectData, MigrationContext);

						for (TObjectPtr<UObject> SentObject : SentObjects)
						{
							if (IsObjectHandleResolved_ForGC(SentObject.GetHandle()))
							{
								UObject* ObjectToSend = SentObject;
#if UE_WITH_REMOTE_OBJECT_HANDLE
								ObjectToSend->PostMigrate(MigrationContext);
#endif								
								SetResidence(ObjectToSend, EResidence::Remote, HighestPriorityRequest->DestinationServerId);
								UE::RemoteObject::Handle::ChangeOwnerServerId(ObjectToSend, OwnerServerId);
							}
							else
							{
#if UE_WITH_REMOTE_OBJECT_HANDLE
								FRemoteObjectStub* Stub = SentObject.GetHandle().ToStub();
								checkf(Stub, TEXT("Stub must exist here"));

								Stub->ResidentServerId = HighestPriorityRequest->DestinationServerId;
								Stub->OwningServerId = OwnerServerId;
#endif
							}
						}
					}

					// for every other request, forward the request on to the new server
					for (int32 MigrationRequestIndex = 0; MigrationRequestIndex < MigrationRequests.IndividualRequests.Num(); MigrationRequestIndex++)
					{
						FObjectMigrationRequest* MigrationRequest = &MigrationRequests.IndividualRequests[MigrationRequestIndex];

						if (MigrationRequest != HighestPriorityRequest)
						{
							if (MigrationRequest->DestinationServerId != HighestPriorityRequest->DestinationServerId)
							{
								UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations obj %ls forwarding request for server %ls",
									*MigrationRequests.ObjectId.ToString(),
									*MigrationRequest->DestinationServerId.ToString());

								SendRemoteObjectRequest(MigrationRequest->RequestPriority, MigrationRequest->ObjectId, HighestPriorityRequest->DestinationServerId, MigrationRequest->DestinationServerId);
							}
							else
							{
								UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations obj %ls denying request to server %ls",
									*MigrationRequests.ObjectId.ToString(),
									*MigrationRequest->DestinationServerId.ToString());

								RemoteObjectDeniedTransferDelegate.Execute(MigrationRequest->ObjectId, MigrationRequest->DestinationServerId);
							}
						}
					}

					UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(FString::Format(TEXT("{0} -> {1}"), {*CanonicalRootObjectId.ToString(ERemoteIdToStringVerbosity::Name), *MigrationContext.RemoteServerId.ToString()}));

					// all done, delete all requests for this object
					PendingObjectMigrationRequests.RemoveAt(MigrationRequestsIndex, EAllowShrinking::No);
					// do not increment MigrationRequestsIndex
				}
				else
				{
					UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations obj %ls is locked, continuing",
						*MigrationRequests.ObjectId.ToString());

					// object is locked, do we need to report it?
					MigrationRequestsIndex++;
				}
			}
			else
			{
				UE_LOGF(LogRemoteObject, VeryVerbose, "TickObjectMigrations obj %ls is remote, forwarding requests",
					*MigrationRequests.ObjectId.ToString());

				// we found a request for an object we don't have, we need to forward all of these
				// requests to whatever server we think has the object
				FRemoteObjectStub* RemoteObjectStub = FindRemoteObjectStub(MigrationRequests.ObjectId);

				FRemoteServerId CurrentResidentServerId = RemoteObjectStub ? RemoteObjectStub->ResidentServerId : MigrationRequests.ObjectId.GetServerId();

				for (int32 MigrationRequestIndex = 0; MigrationRequestIndex < MigrationRequests.IndividualRequests.Num(); MigrationRequestIndex++)
				{
					FObjectMigrationRequest* MigrationRequest = &MigrationRequests.IndividualRequests[MigrationRequestIndex];
				
					if (CurrentResidentServerId != MigrationRequest->DestinationServerId)
					{
						SendRemoteObjectRequest(MigrationRequest->RequestPriority, MigrationRequest->ObjectId, CurrentResidentServerId, MigrationRequest->DestinationServerId);
					}
					else
					{
						RemoteObjectDeniedTransferDelegate.Execute(MigrationRequest->ObjectId, MigrationRequest->DestinationServerId);
					}
				}

				// all done, delete all requests for this object
				PendingObjectMigrationRequests.RemoveAt(MigrationRequestsIndex, EAllowShrinking::No);
				// do not increment MigrationRequestsIndex
			}
		}
	}

	void SendRequests()
	{
		using namespace UE::RemoteObject::Private;
		using namespace UE::RemoteObject::Handle;
		using namespace UE::RemoteObject::Serialization;

		for (const FRemoteObjectId& RequiredObjectId : ActiveRequest->RequiredObjects)
		{
			FRemoteObjectId* RequiredCanonicalRootObjectId = ActiveRequest->RequiredObjectsCanonicalRootByObjectID.Find(RequiredObjectId);
			FRemoteObjectId ObjectIdToRequest = RequiredCanonicalRootObjectId ? *RequiredCanonicalRootObjectId : RequiredObjectId;

			if (Handle::GetResidence(ObjectIdToRequest) == EResidence::Remote)
			{
				// find the highest priority request that needs this object
				FRemoteObjectRequest* HighestPriorityRequest = ActiveRequest;

				for (int32 RequestIndex = 0; RequestIndex < GetRequestCount(); RequestIndex++)
				{
					FRemoteObjectRequest* ExistingRequest = GetRequestByIndex(RequestIndex);
					bool bFoundObject =
						ExistingRequest->RequiredObjectsCanonicalRoots.Contains(ObjectIdToRequest)||
						ExistingRequest->RequiredObjects.Contains(ObjectIdToRequest);

					if (bFoundObject)
					{
						if (IsHigherPriority(ExistingRequest->Priority, HighestPriorityRequest->Priority))
						{
							HighestPriorityRequest = ExistingRequest;
						}
					}
				}

				// does this object id already exist in PendingObjectRequests?
				// if it does, but the current HighestPriorityRequest is of
				// a different priority, then we need to re-send the request
				// to update the priority at which we require this object

				FPendingObjectRequest* ExistingObjectRequest = nullptr;

				for (FPendingObjectRequest& PendingObjectRequest : PendingObjectRequests)
				{
					if (PendingObjectRequest.ObjectId == ObjectIdToRequest)
					{
						ExistingObjectRequest = &PendingObjectRequest;
						break;
					}
				}

				const bool bUpdatingPriority = (ExistingObjectRequest != nullptr);

				if (ExistingObjectRequest == nullptr)
				{
					ExistingObjectRequest = &PendingObjectRequests.Emplace_GetRef();
					ExistingObjectRequest->ObjectId = ObjectIdToRequest;
				}

				if (ExistingObjectRequest->RequestPriority != HighestPriorityRequest->GetPriority())
				{
					// send this request
					ExistingObjectRequest->RequestPriority = HighestPriorityRequest->GetPriority();

					FRemoteObjectStub* RemoteObjectStub = FindRemoteObjectStub(ObjectIdToRequest);
					FRemoteServerId CurrentResidentServerId = RemoteObjectStub ? RemoteObjectStub->ResidentServerId : ObjectIdToRequest.GetServerId();

					SendRemoteObjectRequest(
						HighestPriorityRequest->GetPriority(),
						ObjectIdToRequest,
						CurrentResidentServerId,
						FRemoteServerId::GetLocalServerId());
			
					if (bUpdatingPriority)
					{
						UE_LOGF(LogRemoteObject, Verbose, "FRemoteObjectTransferQueue: TickRequest(%ls) sent updated priority for %ls %ls to server %ls (%d pending requests)", *ActiveRequest->GetRequestId().ToString(), *RequiredObjectId.ToString(), *HighestPriorityRequest->Priority.ToString(), *CurrentResidentServerId.ToString(), PendingObjectRequests.Num());
					}
					else
					{
						UE_LOGF(LogRemoteObject, Verbose, "FRemoteObjectTransferQueue: TickRequest(%ls) sent request for %ls %ls to server %ls (%d pending requests)", *ActiveRequest->GetRequestId().ToString(), *RequiredObjectId.ToString(), *HighestPriorityRequest->Priority.ToString(), *CurrentResidentServerId.ToString(), PendingObjectRequests.Num());
					}
				}
			}
			else if (!RequiredCanonicalRootObjectId)
			{
				if (UObject* Object = StaticFindObjectFastInternal(RequiredObjectId))
				{
					UObject* const RootObject = FindCanonicalRootObjectForSerialization(Object);
					const FRemoteObjectId CanonicalRootObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(RootObject);
					ActiveRequest->RequiredObjectsCanonicalRootByObjectID.Add(RequiredObjectId, CanonicalRootObjectId);
					ActiveRequest->RequiredObjectsCanonicalRoots.Add(CanonicalRootObjectId);
				}
				else if (FRemoteObjectStub* Stub = FindRemoteObjectStub(RequiredObjectId))
				{
					if (Stub->MigratedData)
					{
						const FRemoteObjectId CanonicalRootObjectId = Stub->MigratedData->Data.GetRootSerializedObjectId();
						ActiveRequest->RequiredObjectsCanonicalRootByObjectID.Add(RequiredObjectId, CanonicalRootObjectId);
						ActiveRequest->RequiredObjectsCanonicalRoots.Add(CanonicalRootObjectId);
					}
				}
			}
		}
	}

	void TickRequest() final
	{
		using namespace UE::RemoteObject::Private;
		using namespace UE::RemoteObject::Handle;
		using namespace UE::RemoteObject::Serialization;

		// reset the list of used objects each time before we run
		ActiveRequest->UsedObjects.Reset();
		ActiveRequest->NewRequiredObject = FRemoteObjectId();

		UE_CLOGF(ActiveRequest->RequiredObjects.Num() || PendingObjectRequests.Num(), LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: TickRequest(%ls) %d required objs (%d pending requests):",
			*ActiveRequest->GetRequestId().ToString(), ActiveRequest->RequiredObjects.Num(), PendingObjectRequests.Num());

#if !NO_LOGGING
		if (UE_GET_LOG_VERBOSITY(LogRemoteObject) == ELogVerbosity::VeryVerbose)
		{
			for (const FRemoteObjectId& RequiredObjectId : ActiveRequest->RequiredObjects)
			{
				FRemoteObjectId* CanonicalObjectId = ActiveRequest->RequiredObjectsCanonicalRootByObjectID.Find(RequiredObjectId);
				UE_LOGF(LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: TickRequest(%ls) %ls [root %ls] <%ls>", 
					*ActiveRequest->GetRequestId().ToString(), 
					*RequiredObjectId.ToString(), 
					CanonicalObjectId ? *(CanonicalObjectId->ToString()) : TEXT("<unk>"), 
					EnumToString(Handle::GetResidence(RequiredObjectId)));
			}
		}
#endif
		SendRequests();
	}

	void TickAbortedRequest() final
	{
		using namespace UE::RemoteObject::Private;
		using namespace UE::RemoteObject::Handle;
		using namespace UE::RemoteObject::Serialization;

		if (ActiveRequest->NewRequiredObject.IsValid())
		{
			// add it to the active request's list of required objects
			ActiveRequest->RequiredObjects.Add(ActiveRequest->NewRequiredObject);

			SendRequests();
		}
	}

	bool AreDependenciesSatisfied() const final
	{
		for (const FRemoteObjectId& RequiredObjectId : ActiveRequest->RequiredObjects)
		{
			if (Handle::GetResidence(RequiredObjectId) == EResidence::Remote)
			{
				return false;
			}
		}

		UE_CLOGF(ActiveRequest->RequiredObjects.Num(), LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: TickRequest(%ls) all %d required objects are local", *ActiveRequest->GetRequestId().ToString(), ActiveRequest->RequiredObjects.Num());

		return true;
	}

	void BeginMultiServerCommit(TArray<FRemoteServerId>& OutMultiServerCommitRemoteServers) final
	{
		using namespace UE::RemoteObject::Serialization;

		check(ActiveRequest);
		check(!bInMultiServerCommit);
		bInMultiServerCommit = true;

		check(MultiServerCommitObjectsToReturn.Num() == 0);
		check(MultiServerCommitSentObjects.Num() == 0);
		check(MultiServerCommitReferencedObjects.Num() == 0);

		UE_LOGF(LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: BeginMultiServerCommit has %d used objects", ActiveRequest->UsedObjects.Num());

		// are any of our used objects borrowed?
		for (FRemoteObjectId UsedObjectId : ActiveRequest->UsedObjects)
		{
			if (UsedObjectId.IsAsset())
			{
				continue;
			}

			// UsedObject may have been destroyed since it was touched, so we may not be able to find it.
			UObject* UsedObject = StaticFindObjectFastInternal(UsedObjectId);
			if (!UsedObject)
			{
				continue;
			}

			// If we used it this run and we're about to commit, we expect that the object is local
			check(Handle::GetResidence(UsedObjectId) != EResidence::Remote);
			FRemoteServerId OwnerServerId = Handle::GetOwnerServerId(UsedObject);

			UObject* RootObject = FindCanonicalRootObjectForSerialization(UsedObject);
			check(RootObject);
			FRemoteObjectId RootObjectId = FRemoteObjectId(RootObject);
			FRemoteServerId RootOwnerServerId = Handle::GetOwnerServerId(RootObject);

			if (RootObjectId.IsAsset())
			{
				// if the used object is not an asset, but the root of it is, then we
				// expect that the used object owner is the local server
				check(OwnerServerId == FRemoteServerId::GetLocalServerId());

				continue;
			}

			if (RootOwnerServerId != FRemoteServerId::GetLocalServerId() && !RootOwnerServerId.IsAsset())
			{
				UE_LOGF(LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: borrowed obj: %ls (%ls) owner server: %ls (root: %ls (%ls) root owner: %ls)",
					*UsedObjectId.ToString(),
					*UsedObject->GetName(),
					*OwnerServerId.ToString(),
					*RootObjectId.ToString(),
					*RootObject->GetName(),
					*RootOwnerServerId.ToString());

				MultiServerCommitObjectsToReturn.Add(RootObject);

				if (OutMultiServerCommitRemoteServers.Find(RootOwnerServerId) == INDEX_NONE)
				{
					OutMultiServerCommitRemoteServers.Add(RootOwnerServerId);
				}
			}
		}
	}

	void ExecuteMultiServerCommit() final
	{
		using namespace UE::RemoteObject::Serialization;
		
		check(MultiServerCommitSentObjects.Num() == 0);
		check(MultiServerCommitReferencedObjects.Num() == 0);
		check(bInMultiServerCommit);

		for (UObject* Object : MultiServerCommitObjectsToReturn)
		{
			FRemoteObjectId ObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(Object);
			FRemoteServerId OwnerServerId = Handle::GetOwnerServerId(Object);

			UE_LOGF(LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: ExecuteMultiServerCommit sending root object %ls back to %ls", *ObjectId.ToString(), *OwnerServerId.ToString());

			// NOTE: we need to run SerializeObjectData in the CLOSED because it turns out some of the
			// serialization code ends up mutating the object, so we need to be able to undo those
			// mutations if we need to abort past this point
			FUObjectMigrationContext MigrationContext {
				.ObjectId = ObjectId,
				.RemoteServerId = OwnerServerId,
				.OwnerServerId = OwnerServerId,
				.MigrationSide = EObjectMigrationSide::Send,
				.MultiServerCommitRequestId = ActiveRequest->RequestId
			};
			FScopedObjectMigrationContext ScopedMigrationContext(MigrationContext);

			TSet<UObject*> SentObjects;
			FRemoteObjectData ObjectData = SerializeObjectData(Object, SentObjects, MultiServerCommitReferencedObjects, &MigrationContext);
			MigrationContext.CacheObjectStatus(ObjectData);
			FMigrateSendParams SendRemoteObjectParams = { .MigrationContext = MigrationContext, .ObjectData = ObjectData};

			UE_AUTORTFM_OPEN
			{
				SendRemoteObject(SendRemoteObjectParams);

				for (UObject* SO : SentObjects)
				{
					FRemoteObjectId SOId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(SO);
					UE_LOGF(LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: ExecuteMultiServerCommit sent object %ls (subobject %ls %ls)", *ObjectId.ToString(), *SOId.ToString(), *SO->GetName());
				}

				MultiServerCommitSentObjects.Append(SentObjects);
			};
		}
	}

	void AbortMultiServerCommit() final
	{
		UE_AUTORTFM_OPEN
		{
			MultiServerCommitObjectsToReturn.Reset();
			MultiServerCommitSentObjects.Reset();
			MultiServerCommitReferencedObjects.Reset();
		};

		bInMultiServerCommit = false;
	}

	void CommitMultiServerCommit() final
	{
		using namespace UE::RemoteObject::Serialization;
		using namespace UE::RemoteObject::Private;

		UE_LOGF(LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: CommitMultiServerCommit");

		for (UObject* ReferencedObject : MultiServerCommitReferencedObjects)
		{
			FRemoteObjectId ReferencedObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(ReferencedObject);
			UE_LOGF(LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: CommitMultiServerCommit setting RemoteReference %ls", *ReferencedObjectId.ToString());

			MarkAsRemoteReference(ReferencedObject);
		}

		UE_AUTORTFM_OPEN
		{
			MultiServerCommitReferencedObjects.Reset();
		};

		FUnsafeToMigrateScope UnsafeToMigrateScope;
		for (UObject* SentObject : MultiServerCommitSentObjects)
		{
			FRemoteServerId OwnerServerId = Handle::GetOwnerServerId(SentObject);
			FRemoteObjectId SentObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(SentObject);

			FUObjectMigrationContext MigrationContext {
				.ObjectId = SentObjectId,
				.RemoteServerId = OwnerServerId,
				.OwnerServerId = OwnerServerId,
				.MigrationSide = EObjectMigrationSide::Send
			};
			MigrationContext.CacheObjectStatus(SentObjectId);
			FScopedObjectMigrationContext ScopedMigrationContext(MigrationContext);

			UE_LOGF(LogRemoteObject, VeryVerbose, "FRemoteObjectTransferQueue: CommitMultiServerCommit postmigrate %ls", *SentObjectId.ToString());

#if UE_WITH_REMOTE_OBJECT_HANDLE
			SentObject->PostMigrate(MigrationContext);
#endif
			SetResidence(SentObject, EResidence::Remote, OwnerServerId);
		}

		UE_AUTORTFM_OPEN
		{
			MultiServerCommitObjectsToReturn.Reset();
			MultiServerCommitSentObjects.Reset();
		};

		bInMultiServerCommit = false;
	}

	void EndRequest(bool bTransactionCommitted) final
	{
	}

	void RequestObjectMigration(FRemoteWorkPriority RequestPriority, FRemoteObjectId ObjectId, FRemoteServerId DestinationServerId)
	{
		check(DestinationServerId != FRemoteServerId::GetLocalServerId());

		// enqueue this request
		FObjectMigrationRequests* ObjectRequests = nullptr;

		for (int32 RequestIndex = 0; RequestIndex < PendingObjectMigrationRequests.Num(); RequestIndex++)
		{
			FObjectMigrationRequests* ExistingObjectRequests = &PendingObjectMigrationRequests[RequestIndex];

			if (ExistingObjectRequests->ObjectId == ObjectId)
			{
				ObjectRequests = ExistingObjectRequests;
				break;
			}
		}
		
		if (ObjectRequests == nullptr)
		{
			ObjectRequests = &PendingObjectMigrationRequests.Emplace_GetRef();
			ObjectRequests->ObjectId = ObjectId;
		}

		// first search the list to see if we have a request with the same DestinationServerId - if so, overwrite the priority
		// otherwise, insert the request in priority sorted order
		FObjectMigrationRequest* Request = nullptr;

		for (FObjectMigrationRequest& ExistingRequest : ObjectRequests->IndividualRequests)
		{
			if (ExistingRequest.DestinationServerId == DestinationServerId)
			{
				Request = &ExistingRequest;
				break;
			}
		}

		if (!Request)
		{
			Request = &ObjectRequests->IndividualRequests.Emplace_GetRef();
			Request->ObjectId = ObjectId;
			Request->DestinationServerId = DestinationServerId;
		}

		Request->RequestPriority = RequestPriority;

		// re-sort the list by priority
		ObjectRequests->IndividualRequests.Sort([](const FObjectMigrationRequest& Lhs, const FObjectMigrationRequest& Rhs) -> bool
			{
				return IsHigherPriority(Lhs.RequestPriority, Rhs.RequestPriority);
			});
	}

	void SendRemoteObjectRequest(FRemoteWorkPriority RequestPriority, FRemoteObjectId ObjectId, FRemoteServerId LastKnownResidentServerId, FRemoteServerId DestinationServerId)
	{
		check(!AutoRTFM::IsClosed());

		// Request an object from remote server
		if (LastKnownResidentServerId != DatabaseId)
		{
			// We are forwarding a request for pull-migrate from the resident server id
			// This isn't technically a migration, it is actually a request for migration...
			RequestRemoteObjectDelegate.Execute(RequestPriority, ObjectId, LastKnownResidentServerId, DestinationServerId);
		}
		else
		{
			UE_LOGF(LogRemoteObject, Log, "%s: Restoring ObjectId %ls from Database and Forwarding to %ls", __func__, *ObjectId.ToString(), *DestinationServerId.ToString());
			// We are intercepting a request to pull-migrate since we believe the object resides in our database, migrate it "from" the database to us
			// Inside the database call, it will fix-up these migration context parameters to make us the owner
			FUObjectMigrationContext MigrationContext {
				.ObjectId = ObjectId,
				.RemoteServerId = DatabaseId,
				.OwnerServerId = DatabaseId,
				.MigrationSide = EObjectMigrationSide::Receive
			};
			MigrationContext.CacheObjectStatus(ObjectId);
			FScopedObjectMigrationContext ScopedMigrationContext(MigrationContext);
			RestoreRemoteObjectDataDelegate.Execute(MigrationContext);

#if UE_WITH_REMOTE_OBJECT_HANDLE
			// We assume we have it restored, now forward it
			TWeakObjectPtr WeakObjPtr { ObjectId };
			ensureMsgf(!WeakObjPtr.IsExplicitlyNull() && WeakObjPtr.IsValid(true) && WeakObjPtr.GetResidence() != EResidence::Remote, TEXT("Expected RemoteObjectId %s to be restored from Database and thus locally owned"), *ObjectId.ToString());
#endif

			// If we didn't expect it locally, send it on to the final destination
			if (DestinationServerId != FRemoteServerId::GetLocalServerId())
			{
				MigrateObjectToRemoteServer(ObjectId, DestinationServerId);
			}
		}

		// Tell the executor that we have RPCs waiting to send
		UE::RemoteExecutor::NotifyNetworkIsWaiting();
	}

	/**
	*/
	void SendRemoteObject(const FMigrateSendParams& Params)
	{
		check(!AutoRTFM::IsClosed());

		UE_LOGF(LogRemoteObject, Verbose, "FRemoteObjectTransferQueue(%ls) SendRemoteObject %ls (mid: %ls)",
			*FRemoteServerId::GetLocalServerId().ToString(),
			*Params.MigrationContext.ObjectId.ToString(),
			*Params.ObjectData.MigrationId.ToString(ERemoteIdToStringVerbosity::Id));

		RemoteObjectTransferDelegate.Execute(Params);
	}

	/**
	* Fullfils receive request
	* @param ObjectId Id of the object that was requested
	* @param Data Object data. Data ownership is transferred to the receive request (if it exists)
	*/
	FORCENOINLINE void FulfillReceiveRequest(FRemoteObjectData& ObjectData, FUObjectMigrationContext& MigrationContext)
	{
		using namespace UE::RemoteObject::Serialization;
		using namespace UE::RemoteObject::Private;
		using namespace UE::RemoteObject::Handle;
		using namespace UE::RemoteObject::Transfer;

		check(!AutoRTFM::IsClosed());

		int32 FoundPendingRequestIndex = INDEX_NONE;

		for (int32 PendingRequestIndex = 0; PendingRequestIndex < PendingObjectRequests.Num(); PendingRequestIndex++)
		{
			const FPendingObjectRequest& PendingObjectRequest = PendingObjectRequests[PendingRequestIndex];
			if (PendingObjectRequest.ObjectId == MigrationContext.ObjectId)
			{
				FoundPendingRequestIndex = PendingRequestIndex;
				break;
			}
		}

		const bool bFoundRequest = (FoundPendingRequestIndex != INDEX_NONE);

		checkf(ObjectData.MigrationId.IsValid(), TEXT("Received RemoteObjectData with an invalid migration id"));

		UE_LOGF(LogRemoteObject, Verbose, "FRemoteObjectTransferQueue(%ls): FulfillReceiveRequest %ls (mid: %ls, was requested: %ls, owner server id: %ls, transferring ownership: %ls)",
			*FRemoteServerId::GetLocalServerId().ToString(),
			*MigrationContext.ObjectId.ToString(),
			*ObjectData.MigrationId.ToString(ERemoteIdToStringVerbosity::Id),
			bFoundRequest ? TEXT("yes") : TEXT("no"),
			*MigrationContext.OwnerServerId.ToString(),
			(MigrationContext.GetObjectMigrationRecvType(MigrationContext.ObjectId) == EObjectMigrationRecvType::AssignedOwnership) ? TEXT("yes") : TEXT("no"));

#if UE_AUTORTFM
		if (FRemoteObjectId::RemoteObjectSupportCompiledIn && GTransactionalMigration)
		{
			TRefCountPtr<FTransactionalMigrationData> StoredData = StoreMigratedObjectData(ObjectData, MigrationContext);
			if (StoredData)
			{
				if (!GTransactionalDeserialization)
				{
					DeserializeObjectsFromMigratedData(*StoredData);
					// Deserialization sets the residence of the deserialized objects to Local as soon as they are created to avoid re-resolving the deserialized objects when accessing them as outers
					// or in Serialize() function overrides. Since we queue the PostMigrate step to be transactional we need to mark the objects as LocalNotReady again so that if something else attempts
					// to access the deserialized objects BEFORE the transactional PostMigrate runs the engine will attempt to resolve (PostMigrate) the deserialized objects
					StoredData->SetReceivedObjectsResidence(EResidence::LocalNotReady);
				}
				static FName TransactionalWorkName("TransactionallyMigrateObjects");
				UE::RemoteExecutor::EnqueueMigrationWork(TransactionalWorkName, [StoredData]()
					{
						TransactionallyMigrateObjects(StoredData);
					});
			}
			else
			{
				// We received a dupicate data so exit early without removing the request (this is a re-entry and the request will be removed from the function that migrated this data first)
				return;
			}
		}
		else
#endif
		{
			TRefCountPtr<FTransactionalMigrationData> MigratedData = new FTransactionalMigrationData(ObjectData, MigrationContext);
			MigrateObjectFromObjectData(MigratedData);
		}

		if (bFoundRequest)
		{
			PendingObjectRequests.RemoveAt(FoundPendingRequestIndex);
		}
	}

	void DenyReceiveRequest(FRemoteObjectId ObjectId)
	{
		check(!AutoRTFM::IsClosed());

		int32 FoundPendingRequestIndex = INDEX_NONE;

		for (int32 PendingRequestIndex = 0; PendingRequestIndex < PendingObjectRequests.Num(); PendingRequestIndex++)
		{
			const FPendingObjectRequest& PendingObjectRequest = PendingObjectRequests[PendingRequestIndex];
			if (PendingObjectRequest.ObjectId == ObjectId)
			{
				FoundPendingRequestIndex = PendingRequestIndex;
				break;
			}
		}

		const bool bFoundRequest = (FoundPendingRequestIndex != INDEX_NONE);

		UE_LOGF(LogRemoteObject, Verbose, "FRemoteObjectTransferQueue: DenyReceiveRequest %ls, (was requested: %d)", *ObjectId.ToString(), bFoundRequest);

		if (bFoundRequest)
		{
			PendingObjectRequests.RemoveAt(FoundPendingRequestIndex);
		}
	}
};

TRefCountPtr<FTransactionalMigrationData> StoreMigratedObjectData(FRemoteObjectData& ObjectData, FUObjectMigrationContext& MigrationContext)
{
	using namespace UE::RemoteObject::Serialization;
	using namespace UE::RemoteObject::Private;
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Transfer;

	FRemoteObjectStub* RootStub = FindOrAddRemoteObjectStub(MigrationContext.ObjectId, FRemoteServerId::GetLocalServerId());

	TRefCountPtr<FTransactionalMigrationData> MigratedData = new FTransactionalMigrationData(ObjectData, MigrationContext);

	UE_LOGF(LogRemoteObject, VeryVerbose, "%ls received %d objects", *FRemoteServerId::GetLocalServerId().ToString(), MigratedData->Data.SerializedObjectHeaders.Num());

	for (FSerializedRemoteObjectIterator It(MigratedData->Data); It; ++It)
	{
		FRemoteObjectId SerializedId = It.GetId();
		FRemoteObjectStub* Stub = FindOrAddRemoteObjectStub(SerializedId, FRemoteServerId::GetLocalServerId());

		if (Stub->MigratedData)
		{			
			// This can happen on re-entry from within the same callstack as the first store in which case both data structures should contain identical migrated object data
			UE_CLOGF(Stub->MigratedData->Data.MigrationId != MigratedData->Data.MigrationId, LogRemoteObject, Fatal, 
				"Received remote object %ls (%ls) from server %ls in mid: %ls bundled with %ls (%ls) but the latter object's data already exists locally (received from server %ls and bundled with %ls %ls, state: %ls, mid: %ls)",
				*MigratedData->Context.ObjectId.ToString(),
				*MigratedData->Data.GetName(MigratedData->Context.ObjectId).ToString(),
				*MigratedData->Context.RemoteServerId.ToString(),
				*MigratedData->Data.MigrationId.ToString(ERemoteIdToStringVerbosity::Id),
				*SerializedId.ToString(),
				*MigratedData->Data.GetName(SerializedId).ToString(),
				*Stub->MigratedData->Context.RemoteServerId.ToString(),
				*Stub->MigratedData->Context.ObjectId.ToString(),
				*Stub->MigratedData->Data.GetName(Stub->MigratedData->Context.ObjectId).ToString(),
				EnumToString(Stub->MigratedData->State),
				*Stub->MigratedData->Data.MigrationId.ToString(ERemoteIdToStringVerbosity::Id)
			);
		
			UE_LOGF(LogRemoteObject, Warning, "Received duplicate remote object data (mid: %ls) for the same object (%ls bundled with %ls).",
				*MigratedData->Data.MigrationId.ToString(ERemoteIdToStringVerbosity::Id), *SerializedId.ToString(), *MigratedData->Context.ObjectId.ToString());

			return nullptr;
		}		

		Stub->OwningServerId = MigratedData->Context.OwnerServerId;
		Stub->MigratedData = MigratedData;
		Stub->ResidentServerId = FRemoteServerId::GetLocalServerId();
		if (Stub->Name.IsNone())
		{
			Stub->Name = It.GetName();
		}
		if (!Stub->Class.IsValid())
		{
			Stub->Class = FRemoteObjectClass(It.GetClass());
		}
		if (UObject* Object = StaticFindObjectFastInternal(SerializedId))
		{
			SetResidence(Object, EResidence::LocalNotReady, FRemoteServerId::GetLocalServerId());
		}
	};

	checkf(RootStub->MigratedData == MigratedData, TEXT("Data migrated for %s was not correctly stored in its stub"), *RootStub->Id.ToString());

	FRemoteObjectTransferQueue* TransferQueue = FRemoteObjectTransferQueue::Instance;
	for (int32 RequestIndex = 0; RequestIndex < TransferQueue->GetRequestCount(); RequestIndex++)
	{
		FRemoteObjectRequest* ExistingRequest = TransferQueue->GetRequestByIndex(RequestIndex);

		for (FRemoteObjectId SerializedId : MigratedData->Data)
		{
			if (FRemoteObjectId* ExistingCanonicalRootForSubObject = ExistingRequest->RequiredObjectsCanonicalRootByObjectID.Find(SerializedId))
			{
				*ExistingCanonicalRootForSubObject = SerializedId;
			}
		}
	}

	checkf(RootStub->MigratedData == MigratedData, TEXT("Received migrated object data for %s but failed to set it on the root stub"), *MigrationContext.ObjectId.ToString());

	return MigratedData;
}

void DeserializeObjectsFromMigratedData(FTransactionalMigrationData& InOutMigratedData)
{
	using namespace UE::RemoteObject::Serialization;
	using namespace UE::RemoteObject::Private;

	checkf(InOutMigratedData.State == EMigratedDataState::Received, TEXT("Starting to deserialize remote object data but it's in an unexpected state %s (expected: %s)"), EnumToString(InOutMigratedData.State), EnumToString(EMigratedDataState::Received));

	ensure(!AutoRTFM::IsTransactional());
	static FName GetObjectRegionName("ObjectTransferQueue_RecvObject");
	UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(GetObjectRegionName);

	const FRemoteServerId LocalServerId = FRemoteServerId::GetLocalServerId();
	TArray<FRemoteObjectId>& ReceivedObjectRemoteIds = InOutMigratedData.ReceivedObjectRemoteIds;
	TArray<UObject*>& ReceivedObjects = InOutMigratedData.ReceivedObjects;
	FRemoteObjectData& ObjectData = InOutMigratedData.Data;
	const FUObjectMigrationContext& MigrationContext = InOutMigratedData.Context;

	InOutMigratedData.RequestedObjectIndex = DeserializeObjectData(ObjectData, &MigrationContext, ReceivedObjectRemoteIds, ReceivedObjects);
	InOutMigratedData.State = EMigratedDataState::Deserialized;

	// Debug section
	checkf(ReceivedObjects.Num() > 0, TEXT("PeerId:%s unable to deserialize object data (%d bytes)"), *FRemoteServerId::GetLocalServerId().ToString(), ObjectData.GetNumBytes());
	if (ReceivedObjects.Num() > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(CheckForNewOrRemovedSubObjects);

		if (UE_LOG_ACTIVE(LogRemoteObject, VeryVerbose))
		{
			UE_LOGF(LogRemoteObject, VeryVerbose, "PeerId:%ls deserialized object data (%d bytes)", *FRemoteServerId::GetLocalServerId().ToString(), ObjectData.GetNumBytes());
			for (int32 SubObjectIndex = 0; SubObjectIndex < ReceivedObjects.Num(); SubObjectIndex++)
			{
				FRemoteObjectId SubObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(ReceivedObjects[SubObjectIndex]);
				check(ReceivedObjectRemoteIds[SubObjectIndex] == SubObjectId);

				UE_LOGF(LogRemoteObject, VeryVerbose, "         subobject[%2d] : %ls %ls",
					SubObjectIndex,
					*SubObjectId.ToString(),
					(SubObjectIndex == InOutMigratedData.RequestedObjectIndex) ? TEXT("<requested>") : TEXT(""));
			}

			UE_LOGF(LogRemoteObject, VeryVerbose, "         <done listing subobjects>");
		}

		// Let's make sure we received data for all of the objects
		UObject* RootObject = ReceivedObjects[0];
		TArray<UObject*> AllSubObjects;
		constexpr EObjectFlags ExclusionFlags = EObjectFlags::RF_MirroredGarbage;
		GetObjectsWithOuter(RootObject, AllSubObjects, EGetObjectsFlags::IncludeNestedObjects, ExclusionFlags);

		// Apply the same filter to ReceivedObjects as when calling GetObjectsWithOuter()
		TArray<UObject*> FilteredReceivedObjects;
		FilteredReceivedObjects.Reserve(ReceivedObjects.Num());
		for (UObject* Obj : ReceivedObjects)
		{
			if (!Obj->HasAnyFlags(ExclusionFlags))
			{
				FilteredReceivedObjects.Add(Obj);
			}
		}
		const bool bSameNumberOfObjects = (1 + AllSubObjects.Num() == FilteredReceivedObjects.Num());
		if (!bSameNumberOfObjects)
		{
			UE_LOGF(LogRemoteObject, Warning, "While migrating '%ls' [%ls -> %ls] we received %d Objects whereas %d Objects got instantiated after deserialization.  Differences:", *GetNameSafe(RootObject), *MigrationContext.RemoteServerId.ToString(), *LocalServerId.ToString(), ReceivedObjects.Num(), 1 + AllSubObjects.Num());
			TArray<UObject*> ReceivedButNotInstanced;
			for (UObject* Obj : FilteredReceivedObjects)
			{
				if (AllSubObjects.Remove(Obj) < 1)
				{
					ReceivedButNotInstanced.Add(Obj);
				}
			}			
			if (!ReceivedButNotInstanced.IsEmpty())
			{
				// We'll always have the root object in this list due to AllSubObjects not containing the root object
				ReceivedButNotInstanced.Remove(RootObject);
			}

			auto WrappedGetNameSafe = [](const UObject* O) { return FString::Printf(TEXT("%s (%s)"), *GetNameSafe(O), *(FObjectPtr{ const_cast<UObject*>(O) }.GetRemoteId().ToString())); };
			UE_CLOGF(!ReceivedButNotInstanced.IsEmpty(), LogRemoteObject, Warning, " Received objects but they weren't instanced: %ls", *FString::JoinBy(ReceivedButNotInstanced, TEXT(", "), WrappedGetNameSafe));
			UE_CLOGF(!AllSubObjects.IsEmpty(), LogRemoteObject, Warning, " New (or Existing) instances not received: %ls", *FString::JoinBy(AllSubObjects, TEXT(", "), WrappedGetNameSafe));

			if (CVarRemoveMissingMigratedComponents.GetValueOnGameThread())
			{
				// Let's remove all of those instances that we shouldn't have
				for (UObject* Obj : AllSubObjects)
				{
					UE_LOGF(LogRemoteObject, Warning, " Removing: %ls", *GetNameSafe(Obj));
					Obj->MarkAsGarbage();
				}
			}
		}
	}

	UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(FString::Format(TEXT("{0} {1} from {2} ({3} bytes)"), { *ReceivedObjects[0]->GetName(), *ObjectData.GetRootSerializedObjectId().ToString(), *MigrationContext.RemoteServerId.ToString(), ObjectData.GetNumBytes()}));
}

UObject* PostMigrateObjectsFromMigratedData(FTransactionalMigrationData& InOutMigratedData)
{
	using namespace UE::RemoteObject::Serialization;
	using namespace UE::RemoteObject::Private;

	checkf(InOutMigratedData.State == EMigratedDataState::Deserialized, TEXT("Migrated data has not been deserialized yet (State: %s)"), EnumToString(InOutMigratedData.State));

	static FName PostMigrateRegionName("ObjectTransferQueue_PostMigrate");
	UE::RemoteExecutor::OnRegionBeginDelegate.ExecuteIfBound(PostMigrateRegionName);
	AutoRTFM::PushOnAbortHandler(&InOutMigratedData, []()
	{
		UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(TEXT("Abort"));
	});

	const FRemoteServerId LocalServerId = FRemoteServerId::GetLocalServerId();
	const FRemoteObjectData& ObjectData = InOutMigratedData.Data;
	const FUObjectMigrationContext& MigrationContext = InOutMigratedData.Context;
	const TArray<UObject*>& ReceivedObjects = InOutMigratedData.ReceivedObjects;
	const TArray<FRemoteObjectId>& ReceivedObjectRemoteIds = InOutMigratedData.ReceivedObjectRemoteIds;

	if (LocalServerId != MigrationContext.OwnerServerId)
	{
		for (UObject* ReceivedObject : ReceivedObjects)
		{
			MarkAsBorrowed(ReceivedObject);
		}
	}
#if UE_WITH_REMOTE_OBJECT_HANDLE
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(PostMigrateObjects);

		for (UObject* ReceivedObject : ReceivedObjects)
		{
			ReceivedObject->PostMigrate(MigrationContext);
		}
	}
#endif

	if (MigrationContext.RemoteServerId == DatabaseId)
	{
		// Let other systems know that we've loaded an object from storage
		OnObjectsLoadedFromDiskDelegate.Broadcast(ReceivedObjects, ObjectData, MigrationContext);
	}
	else
	{
		// Notify that objects have been migrated from a remote server
		OnObjectsReceivedDelegate.Broadcast(ReceivedObjects, ObjectData, MigrationContext);
	}

	// Set the ownership to the correct server id
	for (UObject* ReceivedObject : ReceivedObjects)
	{
		UE::RemoteObject::Handle::ChangeOwnerServerId(ReceivedObject, MigrationContext.OwnerServerId);
#if UE_WITH_REMOTE_OBJECT_HANDLE
		ensureAlwaysMsgf(ReceivedObject->HasAnyInternalFlags(EInternalObjectFlags::Borrowed) == !UE::RemoteObject::Handle::IsOwned(ReceivedObject),
			TEXT("Object %s (%s) is %s owned but is %s borrowed"),
			*ReceivedObject->GetPathName(),
			*FRemoteObjectId(ReceivedObject).ToString(),
			UE::RemoteObject::Handle::IsOwned(ReceivedObject) ? TEXT("") : TEXT("not"),
			IsBorrowed(ReceivedObject) ? TEXT("") : TEXT("not"));
#endif
	}

	// do any existing requests require any of the objects in this hierarchy?
	// if so, we can now update the canonical root id
	FRemoteObjectId RootObjectRemoteId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(ReceivedObjects[0]);

	FRemoteObjectTransferQueue* TransferQueue = FRemoteObjectTransferQueue::Instance;
	for (int32 RequestIndex = 0; RequestIndex < TransferQueue->GetRequestCount(); RequestIndex++)
	{
		FRemoteObjectRequest* ExistingRequest = TransferQueue->GetRequestByIndex(RequestIndex);

		for (FRemoteObjectId SubObjectId : ReceivedObjectRemoteIds)
		{
			if (FRemoteObjectId* ExistingCanonicalRootForSubObject = ExistingRequest->RequiredObjectsCanonicalRootByObjectID.Find(SubObjectId))
			{
				*ExistingCanonicalRootForSubObject = RootObjectRemoteId;
			}
		}
	}

	AutoRTFM::PopOnAbortHandler(&InOutMigratedData);
	UE::RemoteExecutor::OnRegionEndDelegate.ExecuteIfBound(FString::Format(TEXT("{0} {1} num objects: {2}"), { *ReceivedObjects[0]->GetName(), *RootObjectRemoteId.ToString(), ReceivedObjects.Num(),}));

	UObject* Result = ReceivedObjects[InOutMigratedData.RequestedObjectIndex];

	InOutMigratedData.State = EMigratedDataState::PostMigrated;

	checkf(ReceivedObjectRemoteIds.Contains(MigrationContext.ObjectId), TEXT("PeerId:%s requested object %s migration from PeerId:%s but received %s"),
		*FRemoteServerId::GetLocalServerId().ToString(), *MigrationContext.ObjectId.ToString(), *MigrationContext.RemoteServerId.ToString(), *UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(Result).ToString());
	checkf(MigrationContext.OwnerServerId == Handle::GetOwnerServerId(Result), TEXT("Expected owner id %s doesn't match post-deserialize owner: %s"), *MigrationContext.OwnerServerId.ToString(), *Handle::GetOwnerServerId(Result).ToString());

	return Result;
}

UObject* MigrateObjectFromObjectData(TRefCountPtr<FTransactionalMigrationData> MigratedData)
{
	if (MigratedData->State != EMigratedDataState::Deserialized)
	{
		DeserializeObjectsFromMigratedData(*MigratedData);
	}
	else
	{
		// If the migrated data has already been deserialized then we must be PostMigrating transactionally in which case
		// we need to set the migrated objects' residence back to local before we start calling PostMigrate functions
		// to avoid re-resolving the deserialized objects
		check(AutoRTFM::IsClosed());
		MigratedData->SetReceivedObjectsResidence(EResidence::Local);
	}
	return PostMigrateObjectsFromMigratedData(*MigratedData);
} 

void ResetMigratedData(TRefCountPtr<FTransactionalMigrationData> MigratedData)
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;

	checkf(MigratedData && MigratedData->Data.SerializedObjectHeaders.Num() > 0, TEXT("Trying to reset invalid migrated data"));
	checkf(MigratedData->State != EMigratedDataState::Completed, TEXT("Trying to reset data that has already been fully PostMigrated"));
	checkf(MigratedData->State == EMigratedDataState::PostMigrated, TEXT("Trying to reset data that has not yet been fully PostMigrated (state: %s)"), EnumToString(MigratedData->State));

	for (FRemoteObjectId SerializedId : MigratedData->Data)
	{
		FRemoteObjectStub* Stub = FindRemoteObjectStub(SerializedId);
		checkf(Stub, TEXT("Resetting migrated data for %s but its stub does not exist"), *SerializedId.ToString());
		checkf(Stub->MigratedData == MigratedData || !Stub->MigratedData, TEXT("Stub for serialized object id %s does not have the expected migrated object data"), *SerializedId.ToString());		
		Stub->MigratedData = nullptr;
	}
	MigratedData->State = EMigratedDataState::Completed;
}

void TransactionallyMigrateObjects(TRefCountPtr<FTransactionalMigrationData> MigratedData)
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;

	// It's possible that once object data is received and queued for transactional deserialization another transaction dereferences one of the migrated objects 
	// and deserializes their data before the queued TransactionallyMigrateObjects transaction is executed.
	// It's also possible that the migrated object data is sent to another server before we had a chance to deserialize it.
	// In that case (bCompleted == true) just skip the deserialization and run some extra checks (if enabled)
	if (MigratedData->State != EMigratedDataState::Completed)
	{
		SetTransactionallyPostMigratingObjects(true);
		MigrateObjectFromObjectData(MigratedData);
		SetTransactionallyPostMigratingObjects(false);

		ResetMigratedData(MigratedData);

#if DO_CHECK
		for (FRemoteObjectId SerializedId : MigratedData->Data)
		{
			FRemoteObjectStub* Stub = FindRemoteObjectStub(SerializedId);
			checkf(Stub, TEXT("Object %s has been fully migrated but its stub does not exist"), *SerializedId.ToString());
			checkf(Stub->ResidentServerId == FRemoteServerId::GetLocalServerId(), TEXT("Object %s has been fully migrated but its resident server is not local (%s)"), *SerializedId.ToString(), *Stub->ResidentServerId.ToString());
			checkf(Stub->MigratedData == nullptr, TEXT("Stub for serialized object %s still has its migrated data stored even though the data has been fully deserialized"), *SerializedId.ToString());
			UObject* Object = StaticFindObjectFastInternal(SerializedId);
			checkf(Object != nullptr, TEXT("Object %s has been migrated but its UObject does not exist locally"), *SerializedId.ToString());
			checkf(GetResidence(Object) == EResidence::Local, TEXT("Object %s %s has been fully migrated but its residence is not Local (%s)"), *SerializedId.ToString(), *GetPathNameSafe(Object), EnumToString(GetResidence(Object))); //-V547
		}
#endif
	}
#if DO_CHECK
	else
	{
		for (FRemoteObjectId SerializedId : MigratedData->Data)
		{
			FRemoteObjectStub* Stub = FindRemoteObjectStub(SerializedId);
			checkf(Stub->MigratedData != MigratedData, TEXT("Stub for serialized object %s still has its migrated data stored even though the data has been fully deserialized"), *SerializedId.ToString());
		}
	}
#endif
}

void MigrateObjectFromRemoteServer(FRemoteObjectId ObjectId, FRemoteServerId CurrentOwnerServerId)
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;

	UE_LOGF(LogRemoteObject, Verbose, "Aborting transaction, server %ls needs obj id %ls from server %ls",
		*FRemoteServerId::GetLocalServerId().ToString(),
		*ObjectId.ToString(),
		*CurrentOwnerServerId.ToString());

	FRemoteObjectStub* Stub = FindRemoteObjectStub(ObjectId);
	if (Stub && Stub->MigratedData)
	{
		// We already received the object's data from another server so deserialize and PostMigrate it now
		TransactionallyMigrateObjects(Stub->MigratedData);
		return;
	}

	FRemoteObjectTransferQueue* TransferQueue = FRemoteObjectTransferQueue::Instance;
#if !UE_AUTORTFM
	{
		FRemoteServerId LastKnownResidentServerId;
		if (Stub)
		{
			LastKnownResidentServerId = Stub->ResidentServerId;
		}
		TransferQueue->SendRemoteObjectRequest(FRemoteWorkPriority(), ObjectId, LastKnownResidentServerId, FRemoteServerId::GetLocalServerId());
	}
#else
	checkf(TransferQueue->ActiveRequest, TEXT("Attempting to access remote object %s but we are outside of a transaction"), *ObjectId.ToString());

	// ensure the request is added in the Open so after we abort it is preserved
	UE_AUTORTFM_OPEN
	{
		// check bInMultiServerCommit here to ensure the UsedObjects list doesn't 
		// accidentally mutate during the actual multi-server commit process
		if (!TransferQueue->bInMultiServerCommit && !TransferQueue->ActiveRequest->UsedObjects.Contains(ObjectId))
		{
			TransferQueue->ActiveRequest->UsedObjects.Add(ObjectId);

			OnObjectTouchedDelegate.Broadcast(TransferQueue->ActiveRequest->RequestId, ObjectId);
		}

		// add it to the active request's list of required objects
		bool bAlreadyAdded = false;
		TransferQueue->ActiveRequest->RequiredObjects.Add(ObjectId, &bAlreadyAdded);

		if (!bAlreadyAdded)
		{
			// does any other existing request also require this object? if so, try to
			// grab the cached canonical id from them
			for (int32 RequestIndex = 0; RequestIndex < TransferQueue->GetRequestCount(); RequestIndex++)
			{
				FRemoteObjectRequest* ExistingRequest = TransferQueue->GetRequestByIndex(RequestIndex);

				if (FRemoteObjectId* FoundRequiredRemoteObjectID = ExistingRequest->RequiredObjects.Find(ObjectId))
				{
					FRemoteObjectId* CanonicalRootObjectId = ExistingRequest->RequiredObjectsCanonicalRootByObjectID.Find(*FoundRequiredRemoteObjectID);

					if (CanonicalRootObjectId)
					{
						TransferQueue->ActiveRequest->RequiredObjectsCanonicalRootByObjectID.Add(ObjectId, *CanonicalRootObjectId);
						TransferQueue->ActiveRequest->RequiredObjectsCanonicalRoots.Add(*CanonicalRootObjectId);
						break;
					}
				}
			}
		}
		
		if (UE_LOG_ACTIVE(LogRemoteObject, VeryVerbose))
		{
			ANSICHAR HumanReadableString[8192] = { 0 };
			constexpr int32 SkipNumCalls = 5;
			FGenericPlatformStackWalk::StackWalkAndDump(HumanReadableString, sizeof(HumanReadableString), SkipNumCalls);
			UE_LOGF(LogRemoteObject, VeryVerbose, "Callstack: %s", HumanReadableString);
		}
	};

	// TODO: DO we want to log when we're pulling over an object we've NEVER known about?!
	// 
	// abort so the outer handler can renegotiate object transfers and retry
	UE::RemoteExecutor::AbortTransactionRequiresDependencies(
		FString::Format(TEXT("object {0} required from server {1}"),
			{*ObjectId.ToString(),
			*CurrentOwnerServerId.ToString()}));
#endif // UE_AUTORTFM
}

void TouchResidentObject(UObject* Object)
{
	using namespace UE::RemoteObject::Serialization;

	if (!Object)
	{
		return;
	}

	FRemoteObjectTransferQueue* TransferQueue = FRemoteObjectTransferQueue::Instance;

	if (!TransferQueue)
	{
		return;
	}

	// check bInMultiServerCommit here to ensure the UsedObjects list doesn't 
	// accidentally mutate during the actual multi-server commit process
	if (TransferQueue->bInMultiServerCommit)
	{
		return;
	}

	if (AutoRTFM::IsClosed() && TransferQueue->ActiveRequest)
	{
		UE_AUTORTFM_OPEN
		{
			const FRemoteObjectId ObjectId(Object);

			bool bAlreadyInUsedObjectsList = false;
			TransferQueue->ActiveRequest->UsedObjects.Add(ObjectId, &bAlreadyInUsedObjectsList);

			if (!bAlreadyInUsedObjectsList)
			{
				OnObjectTouchedDelegate.Broadcast(TransferQueue->ActiveRequest->RequestId, ObjectId);
			}

			TransferQueue->ActiveRequest->NewRequiredObject = ObjectId;
		};
	}
}

void TransferObjectOwnershipToRemoteServer(UObject* Object, FRemoteServerId DestinationServerId)
{
	using namespace UE::RemoteObject::Serialization;
	using namespace UE::RemoteObject::Private;

	check(!AutoRTFM::IsClosed());

	FRemoteObjectId ObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(Object);

	// find the canonical root object id to use for arbitration
	UObject* const RootObject = FindCanonicalRootObjectForSerialization(Object);
	const FRemoteObjectId CanonicalRootObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(RootObject);

	// Figure out the ownership semantics
	FUObjectMigrationContext MigrationContext {
		.ObjectId = ObjectId,
		.RemoteServerId = DestinationServerId,
		.OwnerServerId = DestinationServerId,
		.MigrationSide = EObjectMigrationSide::Send
	};
    FScopedObjectMigrationContext ScopedMigrationContext(MigrationContext);
	
	UE_LOGF(LogRemoteObject, Verbose, "TransferObjectOwnershipToRemoteServer %ls (%ls) root %ls (%ls) to %ls",
		*ObjectId.ToString(),
		*Object->GetName(),
		*CanonicalRootObjectId.ToString(),
		*RootObject->GetName(),
		*DestinationServerId.ToString());

	// send the object
	TSet<UObject*> SentObjects;
	TSet<UObject*> ReferencedObjects;
	FRemoteObjectData ObjectData = SerializeObjectData(Object, SentObjects, ReferencedObjects, &MigrationContext);
	MigrationContext.CacheObjectStatus(ObjectData);

	UE_LOGF(LogRemoteObject, VeryVerbose, "TransferObjectOwnershipToRemoteServer %ls serialization complete (%d bytes)",
		*ObjectId.ToString(),
		ObjectData.GetNumBytes());

	for (UObject* RO : ReferencedObjects)
	{
		MarkAsRemoteReference(RO);
	}

	for (UObject* SO : SentObjects)
	{
		FRemoteObjectId SubObjectId = UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(SO);
		UE_LOGF(LogRemoteObject, VeryVerbose, "TransferObjectOwnershipToRemoteServer %ls sent subobject : %ls (%ls)",
			*ObjectId.ToString(),
			*SubObjectId.ToString(),
			*GetNameSafe(SO));
	}

	FMigrateSendParams SendRemoteObjectParams = { .MigrationContext = MigrationContext, .ObjectData = ObjectData };
	FRemoteObjectTransferQueue::Instance->SendRemoteObject(SendRemoteObjectParams);

	checkf(SentObjects.Num() > 0, TEXT("PeerId:%s unable to migrate object %s to PeerId:%s"), *FRemoteServerId::GetLocalServerId().ToString(), *UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(Object).ToString(), *DestinationServerId.ToString());

	{
		// PostMigrate callbacks may be manipulating migrated objects and may try to migrate them back so prevent that from happening since
		// they all will be destroyed anyway
		FUnsafeToMigrateScope UnsafeToMigrateScope;

		// Notify that objects have been migrated to a remote server
		OnObjectsSentDelegate.Broadcast(ObjectPtrWrap(SentObjects), ObjectData, MigrationContext);

		for (UObject* SentObject : SentObjects)
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			SentObject->PostMigrate(MigrationContext);
#endif
			SetResidence(SentObject, EResidence::Remote, DestinationServerId);
			UE::RemoteObject::Handle::ChangeOwnerServerId(SentObject, DestinationServerId);
		}
	}
}

void RegisterSharedObject(UObject* Object)
{
	UE::RemoteObject::Private::RegisterSharedObject(Object);
}

void RegisterRemoteObjectId(FRemoteObjectId Id, FRemoteServerId ResidentServerId)
{
	using namespace UE::RemoteObject::Private;

	if (!FindRemoteObjectStub(Id))
	{
		// The reson why we don't simply always call FindOrAddRemoteObjectStub is that FindOrAddRemoteObjectStub always updates 
		// the ResidentServerId (even if the stub already exists) and we only want to update the ResidentServerId when the stub is created
		FindOrAddRemoteObjectStub(Id, ResidentServerId);
	}
}

void GetAllBorrowedObjects(TArray<FRemoteObjectId>& OutBorrowedObjectIds)
{
	FRemoteObjectTransferQueue* TransferQueue = FRemoteObjectTransferQueue::Instance;
	check(TransferQueue->ActiveRequest);

	for (const FRemoteObjectId& Id : TransferQueue->ActiveRequest->UsedObjects)
	{
		if (!UE::RemoteObject::Handle::IsOwned(Id))
		{
			OutBorrowedObjectIds.Add(Id);
		}
	}
}

void MigrateObjectToRemoteServer(FRemoteObjectId ObjectId, FRemoteServerId DestinationServerId)
{
	FRemoteWorkPriority RootWorkPriority = UE::RemoteExecutor::CreateRootWorkPriority();
	MigrateObjectToRemoteServerWithExplicitPriority(RootWorkPriority, ObjectId, DestinationServerId);
}

void MigrateObjectToRemoteServerWithExplicitPriority(FRemoteWorkPriority RequestPriority, FRemoteObjectId ObjectId, FRemoteServerId DestinationServerId)
{
	FRemoteObjectTransferQueue::Instance->RequestObjectMigration(RequestPriority, ObjectId, DestinationServerId);
}

int32 ReturnObjectDataToOwnedServers()
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;

	int32 NumReturnedObjects = 0;

	if (GTransactionalMigration)
	{
		ForEachStubWithMigratedData([&NumReturnedObjects](FRemoteObjectStub* Stub)
			{
				if (Stub->OwningServerId != FRemoteServerId::GetLocalServerId())
				{
					checkf(Stub->MigratedData, TEXT("Iterating over all stubs with migrated data but the stub for %s is not referencing any migrated data"), *Stub->Id.ToString());
					FRemoteObjectId MigratedDataRootId = Stub->MigratedData->Data.GetRootSerializedObjectId();
					if (Stub->Id == MigratedDataRootId)
					{
						MigrateObjectToRemoteServer(Stub->Id, Stub->OwningServerId);
						NumReturnedObjects++;
					}
				}
				return true;
			});
	}

	return NumReturnedObjects;
}

void OnObjectDataReceived(FRemoteServerId OwnerServerId, FRemoteObjectId ObjectId, FRemoteServerId RemoteServerId, FRemoteObjectData& Data)
{
	check(!AutoRTFM::IsClosed());

	FUObjectMigrationContext MigrationContext {
		.ObjectId = ObjectId,
		.RemoteServerId = RemoteServerId,
		.OwnerServerId = OwnerServerId,
		.MigrationSide = EObjectMigrationSide::Receive
	};
	MigrationContext.CacheObjectStatus(Data);
	FScopedObjectMigrationContext ScopedMigrationContext(MigrationContext);

	FRemoteObjectTransferQueue::Instance->FulfillReceiveRequest(Data, MigrationContext);
}

void OnObjectDataDenied(FRemoteObjectId ObjectId, FRemoteServerId RemoteServerId)
{
	check(!AutoRTFM::IsClosed());
	FRemoteObjectTransferQueue::Instance->DenyReceiveRequest(ObjectId);
}

void InitRemoteObjectTransfer(FRemoteExecutor* Executor)
{
	FRemoteObjectTransferQueue* Queue = new FRemoteObjectTransferQueue();
	FRemoteObjectTransferQueue::Instance = Queue;
	UE::RemoteExecutor::RegisterRemoteSubsystem(Executor, Queue);
}

} // namespace UE::RemoteObject::Transfer

namespace UE::RemoteObject::Transfer::Private
{

UObject* GetOutermostUnreachableRootObject(UObject* Object)
{
	UObject* OutermostUnreachableObject = Object;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	for (UObject* Outer = Object->GetOuter(); Outer; Outer = Outer->GetOuter())
	{
		FUObjectItem* OuterItem = GUObjectArray.ObjectToObjectItem(Outer);
		if (OuterItem->HasAllFlags(EInternalObjectFlags::Unreachable))
		{
			OutermostUnreachableObject = Outer;
			if (OutermostUnreachableObject->IsMigrationRoot())
			{
				// We don't want to go deeper than the migration root
				break;
			}
		}
		else
		{
			break;
		}
	}
#endif
	return OutermostUnreachableObject;
}

void StoreObjectToDatabase(UObject* Object, FRemoteObjectId ObjectId)
{
	using namespace UE::RemoteObject::Serialization;
	using namespace UE::RemoteObject::Private;
	using namespace UE::RemoteObject::Transfer;
	using namespace UE::RemoteObject::Handle;

	FRemoteServerId LocalServerId = FRemoteServerId::GetLocalServerId();
	FRemoteServerId OwnerServerId = UE::RemoteObject::Handle::GetOwnerServerId(Object);
	if (LocalServerId != OwnerServerId)
	{
		// We don't own this, and it's definitely remotely owned, just don't save it to the database
		EResidence Residence = GetResidence(Object);
		if (Residence == EResidence::Remote) //-V547
		{
			return;
		}

		// This ensure should never fire.  We shouldn't be able to store a non-locally-owned Object into the Database (that is the job of the Owning Server).  A scenario where I've seen this:
		// We borrowed an Object that was Marked As Garbage and didn't return it to the Owning Server before a GC.
		ensureMsgf(false, TEXT("%hs was about to store non-locally-owned Object %s (residence: %d) to Server %s's Database; making it remote instead"), __func__, *GetNameSafe(Object), int32{EnumToUnderlyingType(Residence)}, *LocalServerId.ToString());
		SetResidence(Object, EResidence::Remote, OwnerServerId);
		return;
	}

	UE_LOGF(LogRemoteObject, VeryVerbose, "Storing %ls %ls to database", *ObjectId.ToString(), *GetPathNameSafe(Object));
	do
	{
		// Object could be a subobject of its outer that is going to be destroyed later (and have this function invoked on)
		// so to make storing to database (disk) consistent with how objects are migrated (root object with its subobjects in one transfer), 
		// find this object's outermost object that is also unreachable (but we never go deeper than the migration root).
		// The Outer does not need to be marked as a remote reference because the act of storing its subobjects to database will mark it as such anyway and in this case we can end up
		// storing the Outer with its subobjects to database again. We also don't want to only rely on the remote reference flag when searching for the root to store to database because if the unreachable
		// outer was not marked as a remote reference and we only stored its subobjects to database then we wouldn't be able to restore them because the outer would never get stored to database.
		// If we re-entered this loop because Object has not been stored to disk with its OutermostUnreachableObject
		// we still want to try and get an Outer that might also have not been stored. If such outer does not exist GetOutermostUnreachableRootObject(Object) will return Object itself.
		UObject* OutermostUnreachableObject = GetOutermostUnreachableRootObject(Object);

		TSet<UObject*> SentObjects;
		TSet<UObject*> ReferencedObjects;
		{
			// Let's setup a fake migration context for storing this
			FRemoteObjectId RootObjectId = FRemoteObjectId { OutermostUnreachableObject };


			// Setup a migration context for "push migrating to the new owner: the Database Server"
			FUObjectMigrationContext StoreToDatabaseContext {
				.ObjectId = RootObjectId,
				.RemoteServerId = DatabaseId,
				.OwnerServerId = DatabaseId,
				.MigrationSide = EObjectMigrationSide::Send
			};
			FScopedObjectMigrationContext ScopedMigrationContext(StoreToDatabaseContext);

			// Do the actual serialization but skip canonical root object search since we've already done this with the outermost unreachable reference and
			// we don't want to find the root again if we're serializing a subobject that's unreachable from its root (Outer)
			FRemoteObjectData ObjectData = SerializeObjectData(OutermostUnreachableObject, SentObjects, ReferencedObjects, &StoreToDatabaseContext, ERemoteObjectSerializationFlags::SkipCanonicalRootSearch);
#if DO_CHECK
			// Make sure none of the sent objects is marked as remote at this point, otherwise we either stored a remote (an outdated) object memory to database or we stored the object to database twice
			for (UObject* SentObject : SentObjects)
			{
				FRemoteObjectId SentObjectId(SentObject);
				if (FRemoteObjectStub* Stub = FindRemoteObjectStub(SentObjectId))
				{
					checkf(GetResidence(SentObject) != EResidence::Remote, TEXT("Double store to database for %s (%s)? Object is remote on server %s"), //-V547
						*SentObjectId.ToString(ERemoteIdToStringVerbosity::Id),
						*GetPathNameSafe(SentObject),
						*Stub->ResidentServerId.ToString());
				}
				UE_LOGF(LogRemoteObject, VeryVerbose, "\tStored %ls %ls to database", *SentObjectId.ToString(), *GetPathNameSafe(SentObject));
			}
#endif

			const FMigrateSendParams Params { .MigrationContext = StoreToDatabaseContext, .ObjectData = ObjectData };
			StoreRemoteObjectDataDelegate.Execute(Params);

			// Let other systems know that we've saved out object(s)
			UE::RemoteObject::Transfer::OnObjectsSavedToDiskDelegate.Broadcast(SentObjects, Params.ObjectData, StoreToDatabaseContext);
		}

		checkf(SentObjects.Num() > 0, TEXT("PeerId:%s unable to store object %s to DatabaseId:%s"), *FRemoteServerId::GetLocalServerId().ToString(), *ObjectId.ToString(), *DatabaseId.ToString());

		for (UObject* ReferencedObject : ReferencedObjects)
		{
			MarkAsRemoteReference(ReferencedObject);
		}
		
		// Mark all stored objects as remote to make sure we don't accidentally store them again
		for (UObject* SentObject : SentObjects)
		{
			SetResidence(SentObject, EResidence::Remote, DatabaseId);
		}

		// It's possible that OutermostUnreachableObject no longer referenced this Object and Object hasn't been stored to disk
		// in which case we need to repeat this loop until Object is marked as remote (and is no longer marked as RemoteReference).
		
		// Assert if we're about to enter an infinite loop.
		checkf(!IsRemoteReference(Object) || OutermostUnreachableObject != Object,
			TEXT("PeerId:%s failed to store %s to database"), *FRemoteServerId::GetLocalServerId().ToString(), *Object->GetPathName());

	} while (IsRemoteReference(Object));
}

void StoreUnreachableRemoteObjectsToDatabase(const TArrayView<FUObjectItem*>& UnreachableObjects)
{	
	TGuardValue<bool> GuardAllowUnreachableObjects(GAllowGetObjectsReturnUnreachableObjects, true);

	for (const FUObjectItem* ObjectItem : UnreachableObjects)
	{
		if (ObjectItem->HasAnyFlags(EInternalObjectFlags::RemoteReference) && !ObjectItem->HasAnyFlags(EInternalObjectFlags::Remote))
		{
			FRemoteObjectId ObjectItemId;
			UObject* Object = (UObject*)ObjectItem->GetObject();
#if UE_WITH_REMOTE_OBJECT_HANDLE
			// We can avoid looking up the object id (which happens in FRemoteObjectId(Object)) since we already have the FUObjectItem where the id is stored
			ObjectItemId = ObjectItem->GetRemoteId();
#endif
			if (ObjectItemId.IsAsset())
			{
				// We don't need to store assets to database since they already exist on disk so we just store their pathname so that we can reload them if needed
				UE::RemoteObject::Private::StoreAssetPath(Object);
			}
			else
			{
				StoreObjectToDatabase(Object, ObjectItemId);
			}			
		}
	}
}

} // namespace UE::RemoteObject::Transfer::Private

const TCHAR* EnumToString(EMigratedDataState InState)
{
	switch (InState)
	{
	case EMigratedDataState::Uninitialized:
		return TEXT("Uninitialized");
	case EMigratedDataState::Received:
		return TEXT("Received");
	case EMigratedDataState::Deserialized:
		return TEXT("Deserialized");
	case EMigratedDataState::PostMigrated:
		return TEXT("PostMigrated");
	case EMigratedDataState::Completed:
		return TEXT("Completed");
	default:
		checkf(false, TEXT("EnumToString: Unknown EMigratedDataState"));
		break;
	};
	return TEXT("");
}

FTransactionalMigrationData::FTransactionalMigrationData(FRemoteObjectData& InData, FUObjectMigrationContext& InContext)
{
	Data = MoveTemp(InData);
	Context = MoveTemp(InContext);
	State = EMigratedDataState::Received;
}

void FTransactionalMigrationData::SetReceivedObjectsResidence(EResidence Residence)
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;

	checkf(State == EMigratedDataState::Deserialized, TEXT("Received objects residence can only be set when migrated data state is Deserialized"));
	checkf(Residence == EResidence::Local || Residence == EResidence::LocalNotReady, TEXT("Migrated object data can only set %s or %s residence (requested: %s)"), EnumToString(EResidence::Local), EnumToString(EResidence::LocalNotReady), EnumToString(Residence));

	FTransactionalMigrationData* DataToSet = (Residence == EResidence::LocalNotReady ? this : nullptr);
	for (int32 DeserializedObjectIndex = 0; DeserializedObjectIndex < ReceivedObjects.Num(); ++DeserializedObjectIndex)
	{
		UObject* DeserializedObject = ReceivedObjects[DeserializedObjectIndex];
		FRemoteObjectId DeserializedObjectId = ReceivedObjectRemoteIds[DeserializedObjectIndex];
		checkf(DeserializedObjectId == FRemoteObjectId(DeserializedObject), TEXT("Object id (%s) does not match the respective ReceivedObjectRemoteId (%s)"), *FRemoteObjectId(DeserializedObject).ToString(), *DeserializedObjectId.ToString());
		FRemoteObjectStub* Stub = FindRemoteObjectStub(DeserializedObjectId);
		checkf(Stub, TEXT("Stub for deserialized object %s found!"), *DeserializedObjectId.ToString());
		Stub->MigratedData = DataToSet;
		SetResidence(DeserializedObject, Residence, FRemoteServerId::GetLocalServerId());
	}
}

FRemoteObjectReference::FRemoteObjectReference(const FObjectPtr& Ptr)
{
	ObjectId = Ptr.GetRemoteId();
	if (ObjectId.IsValid())
	{
		if (Ptr.GetResidence() != EResidence::Remote)
		{
			ServerId = FRemoteServerId::GetLocalServerId();
		}
		else if (UE::RemoteObject::Handle::FRemoteObjectStub* Stub = UE::RemoteObject::Private::FindRemoteObjectStub(ObjectId))
		{
			ServerId = Stub->ResidentServerId;
		}
	}
}

FRemoteObjectReference::FRemoteObjectReference(const FWeakObjectPtr& WeakPtr)
#if UE_WITH_REMOTE_OBJECT_HANDLE
	: FRemoteObjectReference(FObjectPtr(WeakPtr.GetRemoteId()))
#endif
{
}

FObjectPtr FRemoteObjectReference::ToObjectPtr() const
{
	return FObjectPtr(ObjectId);
}

FWeakObjectPtr FRemoteObjectReference::ToWeakPtr() const
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	return FWeakObjectPtr(ObjectId);
#else
	return FWeakObjectPtr();
#endif
}

UObject* FRemoteObjectReference::Resolve() const
{
	return ToObjectPtr().Get();
}

bool FRemoteObjectReference::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	using namespace UE::RemoteObject::Serialization;
	using namespace UE::RemoteObject::Private;

	// Serialize remote object id
	Serialize(Ar);

	// Serialize the pathname of the object if possible which will then be used on the receiving end to find assets loaded by both servers in memory	
	FRemoteObjectPathName PathName;
	if (Ar.IsSaving() && ObjectId.IsValid() && IsRemoteObjectSystemCompiledInAndInitialized())
	{		
		if (UObject* ExistingObject = StaticFindObjectFastInternal(ObjectId))
		{
			UE::RemoteObject::Private::RegisterSharedObject(ExistingObject);
			PathName = FRemoteObjectPathName(ExistingObject);
		}
	}

	Ar << PathName;

	if (Ar.IsLoading() && PathName.Num() && IsRemoteObjectSystemCompiledInAndInitialized())
	{
		// We currently can't differentiate between assets that exist on both servers which we don't want to migrate
		// so try to find the object in memory first by its id and if it doesn't exist try to find the object by pathname
		// and if such object does exist use its remote id to resolve this reference to avoid migrating the asset from another server
		UObject* ExistingObject = StaticFindObjectFastInternal(ObjectId);
		if (!ExistingObject && (ObjectId.IsAsset() || ObjectId.IsLocal()))
		{
			ExistingObject = PathName.Resolve();
			if (ExistingObject)
			{
				ObjectId = FRemoteObjectId(ExistingObject);
			}
		}
	}
	bOutSuccess = true;
	return bOutSuccess;
}

void FRemoteObjectReference::NetDequantize(FRemoteObjectId InObjectId, FRemoteServerId InServerId, const FRemoteObjectPathName& InPath)
{
	using namespace UE::RemoteObject::Transfer;
	using namespace UE::RemoteObject::Private;

	ObjectId = InObjectId;
	ServerId = InServerId;

	if (IsRemoteObjectSystemCompiledInAndInitialized())
	{
		RegisterRemoteObjectId(ObjectId, ServerId);

		if (InPath.Num())
		{
			// We currently can't differentiate between assets that exist on both servers which we don't want to migrate
			// so try to find the object in memory first by its id and if it doesn't exist try to find the object by pathname
			// and if such object does exist use its remote id to resolve this reference to avoid migrating the asset from another server
			UObject* ExistingObject = StaticFindObjectFastInternal(ObjectId);
			if (!ExistingObject && (ObjectId.IsAsset() || ObjectId.IsLocal()))
			{
				ExistingObject = InPath.Resolve();
				if (ExistingObject)
				{
					ObjectId = FRemoteObjectId(ExistingObject);
				}
			}
		}
	}
}

bool FRemoteObjectReference::Serialize(FArchive& Ar)
{
	using namespace UE::RemoteObject::Transfer;
	using namespace UE::RemoteObject::Private;

	Ar << ObjectId;
	Ar << ServerId;

	if (Ar.IsLoading() && IsRemoteObjectSystemCompiledInAndInitialized())
	{
		RegisterRemoteObjectId(ObjectId, ServerId);
	}

	return true;
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
void UObject::PostMigrate(const FUObjectMigrationContext& MigrationContext)
{
	UE_LOGF(LogRemoteObject, VeryVerbose, "PostMigrate %ls %ls",
		MigrationContext.MigrationSide == EObjectMigrationSide::Receive ? TEXT("Receiving") : TEXT("Sending"),
		*FRemoteObjectId(this).ToString(ERemoteIdToStringVerbosity::PathName));

	checkf(UE::RemoteObject::Transfer::GTransactionalMigration == 0 || MigrationContext.MigrationSide != EObjectMigrationSide::Receive || AutoRTFM::IsClosed(), 
		TEXT("PostMigrate when receiving %s can only be called inside of a transaction"), *FRemoteObjectId(this).ToString(ERemoteIdToStringVerbosity::PathName));
}
#endif //UE_WITH_REMOTE_OBJECT_HANDLE