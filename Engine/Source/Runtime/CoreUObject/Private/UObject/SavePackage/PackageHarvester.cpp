// Copyright Epic Games, Inc. All Rights Reserved.


#include "UObject/SavePackage/PackageHarvester.h"

#include "Cooker/CookDependency.h"
#include "Interfaces/ITargetPlatform.h"
#include "Misc/PackageAccessTracking.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "UObject/OverridableManager.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/SavePackage/SaveContext.h"
#include "UObject/SavePackage/SavePackageUtilities.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectSerializeContext.h"
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/VVMPackage.h"
#endif

#if WITH_EDITORONLY_DATA
#include "UObject/Class.h"
#include "UObject/InstanceDataObjectUtils.h"
#endif

EObjectMark GenerateMarksForObject(const UObject* InObject, FSaveContext& SaveContext)
{
	using namespace UE::SavePackageUtilities;

	EObjectMark Marks = OBJECTMARK_NOMARKS;

	// CDOs must be included if their class are, so do not generate any marks for it here, defer exclusion to their outer and class
	if (InObject->HasAnyFlags(RF_ClassDefaultObject))
	{
		return Marks;
	}

	if (!InObject->NeedsLoadForClient())
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_NotForClient);
	}

	if (!InObject->NeedsLoadForServer())
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_NotForServer);
	}
#if WITH_ENGINE
	const ITargetPlatform* TargetPlatform = SaveContext.GetTargetPlatform();
	bool bCheckTargetPlatform = false;
	if (TargetPlatform != nullptr)
	{
		// NotForServer && NotForClient implies EditorOnly
		const bool bIsEditorOnlyObject = (Marks & OBJECTMARK_NotForServer) && (Marks & OBJECTMARK_NotForClient);
		const bool bTargetAllowsEditorObjects = TargetPlatform->AllowsEditorObjects();
		
		// no need to query the target platform if the object is editoronly and the targetplatform doesn't allow editor objects 
		bCheckTargetPlatform = !bIsEditorOnlyObject || bTargetAllowsEditorObjects;
	}
	if (bCheckTargetPlatform && TargetPlatform && 
		(!InObject->NeedsLoadForTargetPlatform(TargetPlatform) || !TargetPlatform->AllowObject(InObject)))
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_NotForTargetPlatform);
	}
#endif
	
	EEditorOnlyObjectFlags EditorOnlyObjectFlags = SaveContext.GetEditorOnlyObjectFlags();
	bool bApplyHasNonEditorOnlyReferences = EnumHasAnyFlags(EditorOnlyObjectFlags,
		EEditorOnlyObjectFlags::ApplyHasNonEditorOnlyReferences);
#if WITH_EDITORONLY_DATA
	bool bIsEditorOnlyObject = UE::SavePackageUtilities::IsEditorOnlyObjectInternal(InObject,
		EditorOnlyObjectFlags,
		SaveContext.GetFunctorReadCachedEditorOnlyObject(),
		SaveContext.GetFunctorWriteCachedEditorOnlyObject());
	bool bStrippableEditorOnlyObject = bIsEditorOnlyObject
		&& UE::SavePackageUtilities::CanStripEditorOnlyImportsAndExports();
	if (bStrippableEditorOnlyObject)
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_EditorOnly);
	}
	else
#endif
	// If NotForClient and NotForServer, it is implicitly editor only
	if ((Marks & OBJECTMARK_NotForClient) && (Marks & OBJECTMARK_NotForServer) &&
		(!bApplyHasNonEditorOnlyReferences || !InObject->HasNonEditorOnlyReferences()))
	{
		Marks = (EObjectMark)(Marks | OBJECTMARK_EditorOnly);
	}

	return Marks;
}

bool ConditionallyExcludeObjectForRealm(FSaveContext& SaveContext, TObjectPtr<UObject> ObjPtr,
	UE::SavePackageUtilities::FObjectStatus& ObjectStatus, ESaveRealm HarvestingContext)
{
	using namespace UE::SavePackageUtilities;

	if (!ObjPtr || ObjPtr.GetPackage().GetFName() == GLongCoreUObjectPackageName)
	{
		// No object or in CoreUObject, don't exclude
		return false;
	}
	FHarvestedRealm& RealmBeingChecked = SaveContext.GetHarvestedRealm(HarvestingContext);
	if (RealmBeingChecked.IsExcluded(ObjPtr))
	{
		return true;
	}
	if (RealmBeingChecked.IsIncluded(ObjPtr))
	{
		return false;
	}
	if (RealmBeingChecked.IsNotExcluded(ObjPtr))
	{
		return false;
	}

	const EObjectMark ExcludedObjectMarks = SaveContext.GetExcludedObjectMarks(HarvestingContext);
	const ITargetPlatform* TargetPlatform = SaveContext.GetTargetPlatform();

	UObject* Obj = SaveContext.ResolveForSave(ObjPtr, ObjectStatus);

	EObjectMark ObjectMarks = GenerateMarksForObject(Obj, SaveContext);
	if (!!(ObjectMarks & ExcludedObjectMarks))
	{
		RealmBeingChecked.AddExcluded(Obj);
		return true;
	}

	// If the object class is excluded, the object must be excluded too
	bool bApplyHasNonEditorOnlyReferences = EnumHasAnyFlags(SaveContext.GetEditorOnlyObjectFlags(),
		EEditorOnlyObjectFlags::ApplyHasNonEditorOnlyReferences);
	bool bIgnoreEditorOnlyClass = bApplyHasNonEditorOnlyReferences && Obj->HasNonEditorOnlyReferences();
	if (!bIgnoreEditorOnlyClass)
	{
		TObjectPtr<UObject> Class = Obj->GetClass();
		FObjectStatus& ClassStatus = SaveContext.GetCachedObjectStatus(Class);
		if (ConditionallyExcludeObjectForRealm(SaveContext, Class, ClassStatus, HarvestingContext))
		{
			RealmBeingChecked.AddExcluded(Obj);
			return true;
		}
	}

	// If the object outer is excluded, the object must be excluded too
	TObjectPtr<UObject> Outer = Obj->GetOuter();
	FObjectStatus& OuterStatus = SaveContext.GetCachedObjectStatus(Outer);
	if (ConditionallyExcludeObjectForRealm(SaveContext, Outer, OuterStatus, HarvestingContext))
	{
		RealmBeingChecked.AddExcluded(Obj);
		return true;
	}

	if (!bIgnoreEditorOnlyClass)
	{
		// Check parent struct if we have one
		UStruct* ThisStruct = Cast<UStruct>(Obj);
		if (ThisStruct)
		{
			UObject* SuperStruct = ThisStruct->GetSuperStruct();
			if (SuperStruct)
			{
				FObjectStatus& SuperStructStatus = SaveContext.GetCachedObjectStatus(SuperStruct);
				if (ConditionallyExcludeObjectForRealm(SaveContext, SuperStruct, SuperStructStatus, HarvestingContext))
				{
					RealmBeingChecked.AddExcluded(Obj);
					return true;
				}
			}
		}

		// Check archetype, this may not have been covered in the case of components
		UObject* Archetype = Obj->GetArchetype();
		if (Archetype)
		{
			FObjectStatus& ArchetypeStatus = SaveContext.GetCachedObjectStatus(Archetype);
			if (ConditionallyExcludeObjectForRealm(SaveContext, Archetype, ArchetypeStatus, HarvestingContext))
			{
				RealmBeingChecked.AddExcluded(Obj);
				return true;
			}
		}
	}

	RealmBeingChecked.AddNotExcluded(Obj);
	return false;
}

bool DoesObjectNeedLoadForEditorGame(UObject* InObject)
{
	check(InObject);
	bool bNeedsLoadForEditorGame = false;
	// NeedsLoadForEditor game is inherited to child objects, so check outer chain
	UObject* Outer = InObject;
	while (Outer && !bNeedsLoadForEditorGame)
	{
		bNeedsLoadForEditorGame = Outer->NeedsLoadForEditorGame();
		Outer = Outer->GetOuter();
	}

	if (InObject->HasAnyFlags(RF_ClassDefaultObject))
	{
		bNeedsLoadForEditorGame = bNeedsLoadForEditorGame || InObject->GetClass()->NeedsLoadForEditorGame();
	}
	return bNeedsLoadForEditorGame;
}

namespace UE::SavePackageUtilities::Private
{

FArchiveSavePackageCollector::FArchiveSavePackageCollector(FArchiveSavePackageData& InSavePackageData,
	bool bFilterEditorOnly, bool bSaveUnversioned, bool bCooking)
	: FArchiveUObject()
{
	SetArchiveFlags(InSavePackageData, bFilterEditorOnly, bSaveUnversioned, bCooking);
}

void FArchiveSavePackageCollector::SetArchiveFlags(FArchiveSavePackageData& InSavePackageData,
	bool bFilterEditorOnly, bool bSaveUnversioned, bool bCooking)
{
	this->SetIsSaving(true);
	this->SetIsPersistent(true);
	ArIsObjectReferenceCollector = true;
	ArShouldSkipBulkData = true;
	ArIgnoreClassGeneratedByRef = bCooking;

	this->SetPortFlags(GetSavePackagePortFlags());
	this->SetFilterEditorOnly(bFilterEditorOnly);
	this->SetSavePackageData(&InSavePackageData);
	this->SetUseUnversionedPropertySerialization(bSaveUnversioned);
}

uint32 GetSavePackagePortFlags()
{
	return PPF_DeepCompareInstances | PPF_DeepCompareDSOsOnly;
}

} // namespace UE::SavePackageUtilities::Private

FPackageHarvester::FExportScope::FExportScope(FPackageHarvester& InHarvester, const FExportWithContext& InToProcess)
	: Harvester(InHarvester)
	, PreviousExportHarvestingRealms(InHarvester.CurrentExportHarvestingRealms)
	, bPreviousFilterEditorOnly(InHarvester.IsFilterEditorOnly())
	, bPreviousSavingOptionalObject(InHarvester.IsSavingOptionalObject())
{
	check(!Harvester.HasAnyExportHarvestingRealms());
	Harvester.CurrentExportDependencies = { InToProcess.Export, InToProcess.CellExport };
	Harvester.CurrentExportHarvestingRealms = InToProcess.HarvestedFromRealms;

	if (Harvester.CurrentExportHarvestingRealms.Contains(ESaveRealm::Optional))
	{
		// The optional realm has to be harvested individually because we set flags separately for it
		check(Harvester.CurrentExportHarvestingRealms.Num() == 1);

		// if we are auto generating optional package, then do not filter editor properties for that harvest
		if (Harvester.SaveContext.IsSaveAutoOptional())
		{
			Harvester.SetFilterEditorOnly(false);
		}

		// when serializing for the harvester, and it's in the optional realm, mark the archive
		// so serialize functions have the context
		Harvester.SetSavingOptionalObject(true);

		// We cannot serialize for the optional realm at the same time as we serialize for other realms because native
		// serialize functions can change their behavior based on Ar.IsSavingOptionalObject, and the non-optional package 
		// could pull in extra imports or FNames that client builds don't want. Currently we guarantee that the
		// optional realm is serialized alone in FPackageHarvester::TryHarvestExport.
		checkf(Harvester.CurrentExportHarvestingRealms.Num() == 1, TEXT("Expecting only 1 realm when it's optional - if this trips, we will have to manage marking 'AllowedInOptional' object/fname properties to not bloat things"));
	}
}

FPackageHarvester::FExportScope::~FExportScope()
{
	Harvester.AppendCurrentExportDependencies();
	Harvester.CurrentExportHarvestingRealms = PreviousExportHarvestingRealms;
	Harvester.SetFilterEditorOnly(bPreviousFilterEditorOnly);
	Harvester.SetSavingOptionalObject(bPreviousSavingOptionalObject);
}

FPackageHarvester::FIgnoreDependenciesScope::FIgnoreDependenciesScope(FPackageHarvester& InHarvester)
	: Harvester(InHarvester)
	, bPreviousValue(Harvester.CurrentExportDependencies.bIgnoreDependencies)
{
	Harvester.CurrentExportDependencies.bIgnoreDependencies = true;
}

FPackageHarvester::FIgnoreDependenciesScope::~FIgnoreDependenciesScope()
{
	Harvester.CurrentExportDependencies.bIgnoreDependencies = bPreviousValue;
}

FPackageHarvester::FHarvestScope::FHarvestScope(FPackageHarvester& InHarvester)
	: Harvester(InHarvester)
	, PreviousExportHarvestingRealms(InHarvester.CurrentExportHarvestingRealms)
	, bActive(true)
{
}

FPackageHarvester::FHarvestScope::FHarvestScope(FHarvestScope&& Other)
	: Harvester(Other.Harvester)
	, PreviousExportHarvestingRealms(MoveTemp(Other.PreviousExportHarvestingRealms))
	, bActive(Other.bActive)
{
	Other.bActive = false;
}

FPackageHarvester::FHarvestScope::~FHarvestScope()
{
	if (bActive)
	{
		Harvester.CurrentExportHarvestingRealms = PreviousExportHarvestingRealms;
	}
}

bool FPackageHarvester::FHarvestScope::IsEmpty() const
{
	return Harvester.CurrentExportHarvestingRealms.IsEmpty();
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterRootReferencesScope()
{
	check(!HasAnyExportHarvestingRealms());
	FHarvestScope Scope(*this);

	CurrentExportHarvestingRealms.Add(ESaveRealm::Game);
	if (!SaveContext.IsCooking())
	{
		CurrentExportHarvestingRealms.Add(ESaveRealm::Editor);
	}
	return Scope;
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterRealmsArrayScope(FExportingRealmsArray& Array)
{
	FHarvestScope Scope(*this);
	CurrentExportHarvestingRealms = Array;
	return Scope;
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterConditionalEditorOnlyScope(bool bIsEditorOnly)
{
	FHarvestScope Scope(*this);
	if (bIsEditorOnly)
	{
		// When saving editor (no targetplatform), editoronly objects are stored in editor realm but not
		// game so that we can mark them as editoronly.
		// When saving cooked, the game realm is all we are saving, and most platforms do not export those
		// editor-only objects so we do not add them to the game realm for those platforms. But some cooked
		// platforms include editor-only objects (CookedEditor platforms) so for those platforms we do allow
		// editor-only objects to be added to the game realm.
		if (!SaveContext.GetTargetPlatform() || !SaveContext.GetTargetPlatform()->AllowsEditorObjects())
		{
			CurrentExportHarvestingRealms.RemoveSwap(ESaveRealm::Game, EAllowShrinking::No);
		}
	}
	return Scope;
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterConditionalOptionalObjectScope(TObjectPtr<UObject> Object)
{
	FHarvestScope Scope(*this);
	if (!HasAnyExportHarvestingRealms())
	{
		return Scope;
	}
	bool bIsOptionalObject = ShouldObjectBeHarvestedInOptionalRealm(Object, SaveContext);
	if (!bIsOptionalObject)
	{
		return Scope;
	}

	// It is illegal for a non-optional used-in-game object to reference an optional object
	// Give an error if the referer is not optional and is not editoronly.
	// An example of an editor-only object refererring to an optional object is a UObjectRedirector.
	UObject* Ref = CurrentExportDependencies.CurrentExport;
	if (Ref)
	{
		// TODO: Change IsEditorOnlyObject to also test NeedsLoadForClient and NeedsLoadForServer, 
		// and then change this location to call IsEditorOnlyObject
		bool bEditorOnly = Ref->IsEditorOnly() || (!Ref->NeedsLoadForClient() && !Ref->NeedsLoadForServer());
		if (!bEditorOnly)
		{
			ForEachExportHarvestingRealm([this, Ref, Object](ESaveRealm HarvestingRealm)
			{
				if (HarvestingRealm != ESaveRealm::Optional)
				{
					SaveContext.RecordIllegalReference(Ref, Object, EIllegalRefReason::ReferenceToOptional);
				}
			});
		}
	}

	// No matter the realm referring to it, optional objects are added into the optional realm
	CurrentExportHarvestingRealms.Reset();
	CurrentExportHarvestingRealms.Add(ESaveRealm::Optional);
	return Scope;
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterNewExportOnlyScope(UObject* Export)
{
	FHarvestScope Scope(*this);
	CurrentExportHarvestingRealms.RemoveAllSwap([this, Export](ESaveRealm HarvestingRealm)
		{
			return SaveContext.GetHarvestedRealm(HarvestingRealm).IsExport(Export);
		}, EAllowShrinking::No);
	return Scope;
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterNotExcludedScope(TObjectPtr<UObject> Object,
	UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	FHarvestScope Scope(*this);
	CurrentExportHarvestingRealms.RemoveAllSwap([this, Object, &ObjectStatus](ESaveRealm HarvestingRealm)
		{
			return ConditionallyExcludeObjectForRealm(SaveContext, Object, ObjectStatus, HarvestingRealm);
		}, EAllowShrinking::No);
	return Scope;
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterNotPreviouslyExcludedScope(TObjectPtr<UObject> Object)
{
	FHarvestScope Scope(*this);
	CurrentExportHarvestingRealms.RemoveAllSwap([this, Object](ESaveRealm HarvestingRealm)
		{
			return SaveContext.GetHarvestedRealm(HarvestingRealm).IsExcluded(Object); 
		}, EAllowShrinking::No);
	return Scope;
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterIncludedScope(TObjectPtr<UObject> Object)
{
	FHarvestScope Scope(*this);
	CurrentExportHarvestingRealms.RemoveAllSwap([this, Object](ESaveRealm HarvestingRealm)
		{
			return !SaveContext.GetHarvestedRealm(HarvestingRealm).IsIncluded(Object);
		}, EAllowShrinking::No);
	return Scope;
}

void FPackageHarvester::GetPreviouslyIncludedRealms(TObjectPtr<UObject> Object,
	FExportingRealmsArray& OutAlreadyIncluded, FExportingRealmsArray& OutNotAlreadyIncluded)
{
	OutAlreadyIncluded.Reset();
	OutNotAlreadyIncluded.Reset();
	for (ESaveRealm Realm : CurrentExportHarvestingRealms)
	{
		if (SaveContext.GetHarvestedRealm(Realm).IsIncluded(Object))
		{
			OutAlreadyIncluded.Add(Realm);
		}
		else
		{
			OutNotAlreadyIncluded.Add(Realm);
		}
	}
}

FPackageHarvester::FHarvestScope FPackageHarvester::EnterIncludedScope(Verse::VCell* Cell)
{
	FHarvestScope Scope(*this);
	CurrentExportHarvestingRealms.RemoveAllSwap([this, Cell](ESaveRealm HarvestingRealm)
		{
			return !SaveContext.GetHarvestedRealm(HarvestingRealm).IsCellIncluded(Cell);
		}, EAllowShrinking::No);
	return Scope;
}

void FPackageHarvester::GetPreviouslyIncludedRealms(Verse::VCell* Cell,
	FExportingRealmsArray& OutAlreadyIncluded, FExportingRealmsArray& OutNotAlreadyIncluded)
{
	OutAlreadyIncluded.Reset();
	OutNotAlreadyIncluded.Reset();
	for (ESaveRealm Realm : CurrentExportHarvestingRealms)
	{
		if (SaveContext.GetHarvestedRealm(Realm).IsCellIncluded(Cell))
		{
			OutAlreadyIncluded.Add(Realm);
		}
		else
		{
			OutNotAlreadyIncluded.Add(Realm);
		}
	}
}

bool FPackageHarvester::IsObjNative(UObject* InObj)
{
	bool bIsNative = InObj->IsNative();
	UObject* Outer = InObj->GetOuter();
	while (!bIsNative && Outer)
	{
		bIsNative |= Cast<UClass>(Outer) != nullptr && Outer->IsNative();
		Outer = Outer->GetOuter();
	}
	return bIsNative;
};

bool FPackageHarvester::ShouldObjectBeHarvestedInOptionalRealm(TObjectPtr<UObject> InObj, FSaveContext& InContext)
{
	if (!InContext.IsCooking())
	{
		return false;
	}
	return InObj.GetClass()->HasAnyClassFlags(CLASS_Optional);
}

template <typename CallbackType>
void FPackageHarvester::ForEachExportHarvestingRealm(CallbackType&& Callback)
{
	for (ESaveRealm HarvestingRealm : CurrentExportHarvestingRealms)
	{
		Callback(HarvestingRealm);
	}
}

bool FPackageHarvester::HasAnyExportHarvestingRealms()
{
	return !CurrentExportHarvestingRealms.IsEmpty();
}

FPackageHarvester::FPackageHarvester(FSaveContext& InContext)
	: UE::SavePackageUtilities::Private::FArchiveSavePackageCollector(InContext.GetArchiveSavePackageData(),
		InContext.IsFilterEditorOnly(), InContext.IsSaveUnversionedProperties(), InContext.IsCooking())
	, SaveContext(InContext)
{
	ResolveOverrides();
	// Clear the SaveContext's Saveable cache. It was used earlier during RoutePreSave, and the PreSave functions
	// may have modified RF_Transient or other flags on UObjects and therefore invalidated the cached result. It
	// can also be invalidated by SaveOverrides with bForceTransient.
	SaveContext.ClearSaveableCache();
}

FPackageHarvester::FExportWithContext FPackageHarvester::PopExportToProcess()
{
	FExportWithContext ExportToProcess;
	ExportsToProcess.Dequeue(ExportToProcess);
	return ExportToProcess;
}

void FPackageHarvester::ProcessExport(const FExportWithContext& InProcessContext)
{
	check(InProcessContext.Export && !InProcessContext.CellExport);

	UObject* Export = InProcessContext.Export;
	FExportScope HarvesterScope(*this, InProcessContext);
	UE::FScopedObjectSerializeContext ObjectSerializeContext(Export, *this);

	// Harvest its class; warn if class is not included
	UClass* Class = Export->GetClass();
	*this << Class;
	ForEachExportHarvestingRealm([this, Class, Export](ESaveRealm HarvestingRealm)
		{
			if (HarvestingRealm == ESaveRealm::Game && !SaveContext.IsCooking())
			{
				// During an EditorSave we mark objects as UsedInGame (!EditorOnly) by adding them to the Game realm.
				// But objects can be marked as UsedInGame even if their class is EditorOnly if they
				// return true from HasNonEditorOnlyReferences. So suppress this error that the class is
				// missing from the realm if the realm is Game and we are doing an editor save.
				return;
			}
			FHarvestedRealm& RealmData = SaveContext.GetHarvestedRealm(HarvestingRealm);
			if (!RealmData.IsIncluded(Class))
			{
				SaveContext.RecordIllegalReference(Export, Class, EIllegalRefReason::UnsaveableClass,
					GetUnsaveableReason(Class, HarvestingRealm));
			}
		});

	// Harvest the export outer
	if (UObject* Outer = Export->GetOuter())
	{
		// Harvest the outer as dependencies if the outer is not in the package or if the outer is a ref from optional to non optional object in an optional context
		auto HarvestOuter = [this, &Outer](bool bShouldHarvestOuterAsDependencies)
		{
			if (bShouldHarvestOuterAsDependencies)
			{
				*this << Outer;
			}
			else
			{
				// Legacy behavior does not add an export outer as a preload dependency if that outer is also an export since those are handled already by the EDL
				FIgnoreDependenciesScope IgnoreDependencies(*this);
				*this << Outer;
			}
		};

		bool bShouldHarvestOuterAsDependencies = !Outer->IsInPackage(SaveContext.GetPackage());
		if (!CurrentExportHarvestingRealms.Contains(ESaveRealm::Optional))
		{
			HarvestOuter(bShouldHarvestOuterAsDependencies);
		}
		else
		{
			// The optional realm has to be harvested individually because we set flags separately for it
			check(CurrentExportHarvestingRealms.Num() == 1);
			bool bShouldHarvestOuterAsDependenciesInOptional = bShouldHarvestOuterAsDependencies ||
				(Export->GetClass()->HasAnyClassFlags(CLASS_Optional) && !Outer->GetClass()->HasAnyClassFlags(CLASS_Optional));
			HarvestOuter(bShouldHarvestOuterAsDependenciesInOptional);
		}

		// Only packages or object having the currently saved package as outer are allowed to have no outer
		ForEachExportHarvestingRealm([this, Outer, Export](ESaveRealm HarvestingRealm)
			{
				FHarvestedRealm& RealmData = SaveContext.GetHarvestedRealm(HarvestingRealm);
				if (!RealmData.IsIncluded(Outer))
				{
					if (!Export->IsA<UPackage>() && Outer != SaveContext.GetPackage())
					{
						SaveContext.RecordIllegalReference(Export, Outer, EIllegalRefReason::UnsaveableOuter,
							GetUnsaveableReason(Outer, HarvestingRealm));
					}
				}
			});
	}

	// Harvest its template, if any
	UObject* Template = Export->GetArchetype();
	if (Template && (Template != Class->GetDefaultObject() || SaveContext.IsCooking()))
	{
		*this << Template;
	}

	// Generate ObjectStatus workaround for `ClassGeneratedBy` property if set as a validation workaround for export archive that don't set `ArIgnoreClassGeneratedByRef`
	if (UClass* ClassExport = Cast<UClass>(Export))
	{
#if WITH_EDITORONLY_DATA
		SaveContext.GetCachedObjectStatus(ClassExport->ClassGeneratedBy);
#endif
	}

	// Serialize the object or CDO
	if (Export->HasAnyFlags(RF_ClassDefaultObject))
	{
		Class->SerializeDefaultObject(Export, *this);
		//@ todo FH: I don't think recursing into the template subobject is necessary, serializing it should catch the necessary sub objects
		// GetCDOSubobjects??
	}

	// In the CDO case the above would serialize most of the references, including transient properties
	// but we still want to serialize the object using the normal path to collect all custom versions it might be using.
	{
		SCOPED_SAVETIMER_TEXT(*WriteToString<128>(GetClassTraceScope(Export), TEXT("_SaveSerialize")));
		Export->Serialize(*this);
	}

	{
		// Register the object for the post save serialization if requested
		FObjectSaveContextData& ObjectSaveContext = SaveContext.GetObjectSaveContext();
		if (ObjectSaveContext.bRequestPostSaveSerialization)
		{
			SaveContext.AddObjectToPostSaveSerialization(Export);
			ObjectSaveContext.bRequestPostSaveSerialization = false;
		}
	}

	// Gather object preload dependencies
	if (SaveContext.IsCooking())
	{
		TArray<UObject*> Deps;
		{
			// We want to tag these as imports, but not as dependencies, here since they are handled separately to the the DependsMap as SerializationBeforeSerializationDependencies instead of CreateBeforeSerializationDependencies 
			FIgnoreDependenciesScope IgnoreDependencies(*this);

			Export->GetPreloadDependencies(Deps);
			for (UObject* Dep : Deps)
			{
				// We assume nothing in coreuobject ever loads assets in a constructor
				if (Dep && Dep->GetOutermost()->GetFName() != GLongCoreUObjectPackageName)
				{
					*this << Dep;
				}
			}
		}

		if (SaveContext.IsProcessingPrestreamingRequests())
		{
			Deps.Reset();
			Export->GetPrestreamPackages(Deps);
			for (UObject* Dep : Deps)
			{
				if (Dep)
				{
					UPackage* Pkg = Dep->GetOutermost();
					if (ensureAlways(!Pkg->HasAnyPackageFlags(PKG_CompiledIn)))
					{
						SaveContext.AddPrestreamPackages(Pkg);
					}
				}
			}
		}
	}
}

void FPackageHarvester::TryHarvestExport(UObject* InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	check(!HasAnyExportHarvestingRealms());

	FHarvestScope ScopeSetHarvestingRealms(*this);
	FExportingRealmsArray HarvestingRealms;
	if (ShouldObjectBeHarvestedInOptionalRealm(InObject, SaveContext))
	{
		// if the object is optional and we are cooking, harvest in the Optional context
		HarvestingRealms.Add(ESaveRealm::Optional);
	}
	else if (SaveContext.CurrentHarvestingRealm == ESaveRealm::Optional)
	{
		// If we are automatically generating an optional package then the callsite will harvest all public objects into the optional
		// realm in addition to harvesting them normally. It indicates harvesting into the optional realm by setting CurrentHarvestingRealm.
		HarvestingRealms.Add(ESaveRealm::Optional);
	}
	else
	{
		// For regular objects not in the autoptional special harvest context, exports are added by default to the game realm, and if making
		// an editor save, also to the editor realm
		HarvestingRealms.Add(ESaveRealm::Game);
		if (!SaveContext.IsCooking())
		{
			HarvestingRealms.Add(ESaveRealm::Editor);
		}
	}
	CurrentExportHarvestingRealms = HarvestingRealms;

	TryHarvestExportInternal(InObject, ObjectStatus);
}

void FPackageHarvester::TryHarvestExportInternal(UObject* InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	using namespace UE::SavePackageUtilities;

	// Those should have been already validated
	check(InObject && ObjectStatus.IsInSavePackage(InObject, SaveContext.GetPackage()));

	// Switch to the Optional Realm if the Object is optional
	FHarvestScope OptionalObjectScope = EnterConditionalOptionalObjectScope(InObject);

#if WITH_EDITORONLY_DATA
	// Remove the Game realm if the object is editoronly and does not override it with HasNonEditorOnlyReferences
	bool bIsEditorOnlyObject = UE::SavePackageUtilities::IsEditorOnlyObjectInternal(InObject,
		SaveContext.GetEditorOnlyObjectFlags(),
		SaveContext.GetFunctorReadCachedEditorOnlyObject(),
		SaveContext.GetFunctorWriteCachedEditorOnlyObject());
	bool bStrippableEditorOnlyObject = bIsEditorOnlyObject
		&& !InObject->HasNonEditorOnlyReferences()
		&& UE::SavePackageUtilities::CanStripEditorOnlyImportsAndExports();
	FHarvestScope EditorOnlyScope = EnterConditionalEditorOnlyScope(bStrippableEditorOnlyObject);
#endif

	// Remove realms for which we have already harvested the export
	FHarvestScope NewExportOnlyScope = EnterNewExportOnlyScope(InObject);
	if (NewExportOnlyScope.IsEmpty())
	{
		return;
	}

	// Check whether the object is unsaveable and skip adding it as an export to any realm if so
	ObjectStatus.bAttemptedExport = true;
	if (SaveContext.IsUnsaveable(InObject, ObjectStatus))
	{
		return;
	}

	// Filter out any realms in which the export is excluded
	FHarvestScope NotExcludedScope = EnterNotExcludedScope(InObject, ObjectStatus);
	if (NotExcludedScope.IsEmpty())
	{
		return;
	}

	HarvestExport(InObject, ObjectStatus);

	// @todo: Remove this code once we know that the SubObjectsShadowSerialization works with SceneGraph and it is tested correctly.
	// Objects that has overridable serialization enabled, needs to separately export their subobject independently of the property serializing it.
	// This is because it needs to make a difference between the pointer that is overridden vs some properties inside the instanced sub object that are overridden.
	if (!FOverridableSerializationLogic::HasCapabilities(FOverridableSerializationLogic::ECapabilities::SubObjectsShadowSerialization) && FOverridableManager::Get().IsEnabled(InObject))
	{
		for (TPropertyValueIterator<FObjectProperty> ObjPropIt(InObject->GetClass(), InObject); ObjPropIt; ++ObjPropIt)
		{
			const FObjectProperty* ObjProp = ObjPropIt.Key();
			if (ShouldSkipProperty(ObjProp))
			{
				continue;
			}

			UObject* ObjValue = ObjProp->GetObjectPropertyValue(ObjPropIt.Value());
			if (!ObjValue || !ObjValue->IsInOuter(InObject))
			{
				continue;
			}
			FObjectStatus& ObjValueStatus = SaveContext.GetCachedObjectStatus(ObjValue);
			if (!ObjValueStatus.IsInSavePackage(ObjValue, SaveContext.GetPackage()))
			{
				continue;
			}

			TryHarvestExportInternal(ObjValue, ObjValueStatus);
		}
	}
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FPackageHarvester::ProcessCellExport(const FExportWithContext& InProcessContext)
{
	check(!InProcessContext.Export && InProcessContext.CellExport);

	Verse::VCell* Export = InProcessContext.CellExport;
	FExportScope HarvesterScope(*this, InProcessContext);

	Verse::FRunningContext Context = Verse::FRunningContextPromise{};
	FStructuredArchiveFromArchive StructuredArchive(*this);
	Verse::FStructuredArchiveVisitor Visitor(Context, StructuredArchive.GetSlot().EnterRecord());
	Export->GetCppClassInfo()->SerializeLayout(Context, Export, Visitor);
	Export->Serialize(Context, Visitor);
}

void FPackageHarvester::TryHarvestCellExport(Verse::VCell* InCell)
{
	check(!HasAnyExportHarvestingRealms());
	check(SaveContext.IsCooking());

	FHarvestScope ScopeSetHarvestingRealms(*this);
	FExportingRealmsArray HarvestingRealms;
	HarvestingRealms.Add(ESaveRealm::Game);
	CurrentExportHarvestingRealms = HarvestingRealms;

	HarvestCellExport(InCell);
}
#endif

void FPackageHarvester::TryHarvestImport(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	UE_COOK_RESULTPROJECTION_SCOPED(UE::Cook::ResultProjection::PackageAndClass);

	// Those should have been already validated
	check(InObject && !ObjectStatus.IsInSavePackage(InObject, SaveContext.GetPackage()))
	if (InObject == nullptr)
	{
		return;
	}

	// Do not add unsaveable imports to any realm
	if (SaveContext.IsUnsaveable(InObject, ObjectStatus))
	{
		return;
	}

	// Do not add imports in packages excluded from cooking to any realm
	if (FCoreUObjectDelegates::ShouldCookPackageForPlatform.IsBound())
	{
		if (!FCoreUObjectDelegates::ShouldCookPackageForPlatform.Execute(InObject.GetPackage(), CookingTarget()))
		{
			return;
		}
	}

	// Filter out any realms in which the import is excluded
	FHarvestScope NotExcludedScope = EnterNotExcludedScope(InObject, ObjectStatus);
	if (NotExcludedScope.IsEmpty())
	{
		return;
	}

	HarvestImport(InObject, ObjectStatus);
	ProcessImport(InObject, ObjectStatus);
}

void FPackageHarvester::ProcessImport(TObjectPtr<UObject> InObjPtr,
	UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	++CurrentExportDependencies.ProcessImportDepth;
	ON_SCOPE_EXIT{ --CurrentExportDependencies.ProcessImportDepth; };

	UObject* InObject = SaveContext.ResolveForSave(InObjPtr, ObjectStatus);
	bool bIsNative = IsObjNative(InObject);
	TObjectPtr<UObject> ObjOuter = InObject->GetOuter();
	UClass* ObjClass = InObject->GetClass();
	FName ObjName = InObject->GetFName();
	if (SaveContext.IsCooking())
	{
		// The ignore dependencies check is necessary not to have infinite recursive calls
		if (!bIsNative && !CurrentExportDependencies.bIgnoreDependencies)
		{
			UClass* ClassObj = Cast<UClass>(InObject);
			UObject* CDO = ClassObj ? ClassObj->GetDefaultObject() : nullptr;
			if (CDO)
			{
				FIgnoreDependenciesScope IgnoreDependencies(*this);

				// Gets all subobjects defined in a class, including the CDO, CDO components and blueprint-created components
				TArray<UObject*> ObjectTemplates;
				ObjectTemplates.Add(CDO);
				UE::SavePackageUtilities::GetCDOSubobjects(CDO, ObjectTemplates);
				for (UObject* ObjTemplate : ObjectTemplates)
				{
					// Recurse into templates
					if (ObjTemplate->HasAnyFlags(RF_Public))
					{
						*this << ObjTemplate;
					}
					else
					{
						// CDO Subobjects are supposed to be public; we rely on that because otherwise they could be garbage collected by
						// SoftGC during cooking and then not saved out, causing a missing export.
						UE_LOGF(LogSavePackage, Warning,
							"Invalid subobject on a CDO; we will skip importing it. Found when saving package %ls which imported the CDO containing subobject %ls.",
							*SaveContext.GetPackage()->GetName(), *ObjTemplate->GetPathName());
					}
				}
			}
		}
	}

	// Harvest the import name
	HarvestPackageHeaderName(ObjName);

	// Recurse into outer, package override and non native class
	if (ObjOuter)
	{
		*this << ObjOuter;
	}
	UPackage* Package = InObject->GetExternalPackage();
	if (Package && Package != InObject)
	{
		*this << Package;

		// The package needs to be included in the SaveContext's HarvestingRealm, or ValidateImports will log an error:
		// "Missing import package name...". Log it here instead with an explanation for why it's not included.
		// It's possible the package will be marked as editoronly by not including it in the game realm, so don't check
		// all CurrentExportHarvestingRealms, just check the SaveContext's HarvestingRealm.
		if (SaveContext.IsIncluded(InObjPtr) && !SaveContext.IsIncluded(Package))
		{
			SaveContext.RecordIllegalReference(CurrentExportDependencies.CurrentExport, InObjPtr,
				EIllegalRefReason::ExternalPackage, Package->GetName());
		}
	}
	else if (!IsFilterEditorOnly())
	{
		// operator<<(FStructuredArchive::FSlot Slot, FObjectImport& I) will need to write NAME_None for this empty ExternalPackage pointer
		HarvestPackageHeaderName(NAME_None);
	}

	// For things with a BP-created class we need to recurse into that class so the import ClassPackage will load properly
	// We don't do this for native classes to avoid bloating the import table, but we need to harvest their name and outer (package) name
	if (!ObjClass->IsNative())
	{
		*this << ObjClass; 
	}	
	else
	{
		HarvestPackageHeaderName(ObjClass->GetFName());
		HarvestPackageHeaderName(ObjClass->GetOuter()->GetFName());
	}

#if WITH_EDITORONLY_DATA
	if (!SaveContext.IsCooking())
	{
		HarvestImportTypeHierarchyNames(InObject);
	}
#endif
}

FString FPackageHarvester::GetArchiveName() const
{
	return FString::Printf(TEXT("PackageHarvester (%s)"), *SaveContext.GetPackage()->GetName());
}

void FPackageHarvester::MarkSearchableName(const TObjectPtr<const UObject>& TypeObject, const FName& ValueName) const
{
	if (TypeObject == nullptr)
	{
		return;
	}

	// Serialize object to make sure it ends up in import table
	// This is doing a const cast to avoid backward compatibility issues
	TObjectPtr<UObject> TempObject = TObjectPtr<UObject>(FObjectPtr(TypeObject.GetHandle()));
	FPackageHarvester* MutableArchive = const_cast<FPackageHarvester*>(this);
	MutableArchive->HarvestSearchableName(TempObject, ValueName);
}

FArchive& FPackageHarvester::operator<<(UObject*& Obj)
{
	FObjectPtr ObjectPtr(Obj);
	return *this << ObjectPtr;
}

FArchive& FPackageHarvester::operator<<(FObjectPtr& ObjBase)
{
	// if the object is null, skip the harvest
	if (!ObjBase)
	{
		return *this;
	}
	TObjectPtr<UObject> Obj(ObjBase);

#if WITH_EDITORONLY_DATA
	FHarvestScope EditorOnlyScope = EnterConditionalEditorOnlyScope(IsEditorOnlyPropertyOnTheStack());
#endif

	// if the object is already marked excluded, skip the harvest
	FHarvestScope NotPreviouslyExcludedScope = EnterNotPreviouslyExcludedScope(Obj);
	if (NotPreviouslyExcludedScope.IsEmpty())
	{
		return *this;
	}

	// if the referenced object is the package we are saving, just harvest its name
	if (Obj == SaveContext.GetPackage())
	{
		HarvestPackageHeaderName(Obj->GetFName());
		return *this;
	}

	// For realms in which the object has already been included as an import or export,
	// do reduced work - just the work that needs to be done per referencer that references it.
	// This is important for performance, because a full harvest of the object can be expensive.
	// For new realms that have not previously included the object, do a full harvest.
	FExportingRealmsArray PreviouslyIncludedRealms;
	FExportingRealmsArray NewRealms;
	GetPreviouslyIncludedRealms(Obj, PreviouslyIncludedRealms, NewRealms);

	UE::SavePackageUtilities::FObjectStatus& ObjectStatus = SaveContext.GetCachedObjectStatus(Obj);
	if (!NewRealms.IsEmpty())
	{
		// if the object is in the save context package, try to tag it as export for the new realms.
		// Otherwise visit the import for the new realms.
		FHarvestScope NewRealmsScope = EnterRealmsArrayScope(NewRealms);
		if (ObjectStatus.IsInSavePackage(Obj, SaveContext.GetPackage()))
		{
			TryHarvestExportInternal(Obj, ObjectStatus);
		}
		else
		{
			TryHarvestImport(Obj, ObjectStatus);
		}
	}

	// Work that needs to be done per referencing export. This work only needs to be done if Obj is an import or
	// export in any realm.
	FHarvestScope ObjIncludedScope = EnterIncludedScope(Obj);
	if (!ObjIncludedScope.IsEmpty())
	{
		UObject* ResolvedObject = SaveContext.ResolveForSave(Obj, ObjectStatus);

		// Add a dependency from the current referencing export to Obj
		HarvestDependency(Obj, IsObjNative(ResolvedObject));

		// Validate the reference is allowed (this validation is a side-effect of EnterConditionalOptionalObjectScope)
		EnterConditionalOptionalObjectScope(Obj);
	}

	return *this;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FArchive& FPackageHarvester::operator<<(Verse::VCell*& Cell)
{
	if (!Cell)
	{
		return *this;
	}

	FExportingRealmsArray PreviouslyIncludedRealms;
	FExportingRealmsArray NewRealms;
	GetPreviouslyIncludedRealms(Cell, PreviouslyIncludedRealms, NewRealms);
	if (!NewRealms.IsEmpty())
	{
		FHarvestScope NewRealmsScope = EnterRealmsArrayScope(NewRealms);

		Verse::FRunningContext Context = Verse::FRunningContextPromise{};
		Verse::VPackage* VersePackage = Context.PackageForCell(Cell);
		if (!Cell->GetCppClassInfo()->SerializeIdentity || VersePackage == nullptr || VersePackage->GetUPackage() == SaveContext.GetPackage())
		{
			HarvestCellExport(Cell);
		}
		else
		{
			// Do not add imports in packages excluded from cooking to any realm
			if (FCoreUObjectDelegates::ShouldCookPackageForPlatform.IsBound())
			{
				if (!FCoreUObjectDelegates::ShouldCookPackageForPlatform.Execute(VersePackage->GetUPackage(), CookingTarget()))
				{
					return *this;
				}
			}

			HarvestCellImport(Cell);

			UPackage* Package = VersePackage->GetUPackage();
			*this << Package;
		}
	}

	FHarvestScope CellIncludedScope = EnterIncludedScope(Cell);
	if (!CellIncludedScope.IsEmpty())
	{
		CurrentExportDependencies.CellReferences.Add(Cell);
	}

	return *this;
}
#endif

FArchive& FPackageHarvester::operator<<(struct FWeakObjectPtr& Value)
{
	UObject* Object = Value.Get(true);

	// If the referenced object share the same outer as the referencer but aren't in the same package, we treat this weak object
	// reference as a hard reference, to make sure they end up in the import table. This guarantee that they can be referenced by objects
	// saved in other packages,  such as different external actors from the same world.
	const UObject* CurrentExport = CurrentExportDependencies.CurrentExport;

	// @todo FH: Should we really force weak import in cooked builds?
	if (IsCooking() || (Object && !Object->IsInPackage(SaveContext.GetPackage()) && (Object->GetOutermostObject() == CurrentExport->GetOutermostObject())))
	{
		*this << Object;
	}
	else
	{
		FArchiveUObject::SerializeWeakObjectPtr(*this, Value);
	}

	return *this;
}

FArchive& FPackageHarvester::operator<<(FLazyObjectPtr& LazyObjectPtr)
{
	// @todo FH: Does this really do anything as far as tagging goes?
	FUniqueObjectGuid ID;
	ID = LazyObjectPtr.GetUniqueID();
	return *this << ID;
}

FArchive& FPackageHarvester::operator<<(FSoftObjectPath& Value)
{
	// We need to harvest NAME_None even if the path isn't valid
	Value.SerializePath(*this);
	// Add the soft object path to the realm's list. We need to map even invalid and uncollected soft object paths
	// so that they can be mapped to an index when serializing the exports
	ForEachExportHarvestingRealm([this, &Value](ESaveRealm HarvestingRealm)
		{
			SaveContext.GetHarvestedRealm(HarvestingRealm).GetSoftObjectPathList().Add(Value);
		});
	if (Value.IsValid())
	{
		FSoftObjectPathThreadContext& ThreadContext = FSoftObjectPathThreadContext::Get();
		FName ReferencingPackageName, ReferencingPropertyName;
		ESoftObjectPathCollectType CollectType = ESoftObjectPathCollectType::AlwaysCollect;
		ESoftObjectPathSerializeType SerializeType = ESoftObjectPathSerializeType::AlwaysSerialize;

		FString Path = Value.ToString();
		FName PackageName = FName(*FPackageName::ObjectPathToPackageName(Path));

		bool bRecordedRuntimeDependency = false;

		// Don't track as a soft reference to a package if this is a NeverCollect SoftObjectPath
		ThreadContext.GetSerializationOptions(ReferencingPackageName, ReferencingPropertyName, CollectType, SerializeType, this);
		if (UE::SoftObjectPath::IsCollectable(CollectType))
		{
#if WITH_EDITORONLY_DATA
			// Don't track as a used-in-game soft reference to a package if this is an EditorOnly SoftObjectPath
			// CollectType takes into account FSoftObjectPathSerializationScopes and
			// this->FArchiveState::IsEditorOnlyPropertyOnTheStack
			FHarvestScope EditorOnlyScope = EnterConditionalEditorOnlyScope(CollectType == ESoftObjectPathCollectType::EditorOnlyCollect);
#endif
			HarvestPackageHeaderName(PackageName);
			ForEachExportHarvestingRealm([this, PackageName, &bRecordedRuntimeDependency](ESaveRealm HarvestingRealm)
				{
					SaveContext.GetHarvestedRealm(HarvestingRealm).GetSoftPackageReferenceList().Add(PackageName);
					bRecordedRuntimeDependency = true;
				});
		}

		if (!bRecordedRuntimeDependency)
		{
			// Save the names of packages not recorded in the regular SoftPackageReferenceList in the UntrackedSoftPackageReferenceList.
			// When doing cook saves, redirections on untracked soft packages are added to build dependencies. This way 
			// when the redirection changes, a recook is triggered.
			ForEachExportHarvestingRealm([this, PackageName](ESaveRealm HarvestingRealm)
			{
				SaveContext.GetHarvestedRealm(HarvestingRealm).GetUntrackedSoftPackageReferenceList().Add(PackageName);
			});
		}
	}
	return *this;
}

FArchive& FPackageHarvester::operator<<(FName& Name)
{
	HarvestExportDataName(Name);
	return *this;
}

bool FPackageHarvester::ShouldSkipProperty(const FProperty* InProperty) const
{
	const TSet<FProperty*>* Props = TransientPropertyOverrides.Find(CurrentExportDependencies.CurrentExport);
	if (Props && Props->Contains(InProperty))
	{
		return true;
	}
	return false;
}

void FPackageHarvester::ResolveOverrides()
{
	for (auto& PairObjectOverrides : SaveContext.GetObjectSaveContext().SaveOverrides)
	{
		TSet<FProperty*> Props;
		for (const FPropertySaveOverride& Override : PairObjectOverrides.Value.PropOverrides)
		{
			if (FProperty* Prop = Override.bMarkTransient ? CastField<FProperty>(Override.PropertyPath.GetTyped(FProperty::StaticClass())) : nullptr)
			{
				FProperty* InnerProp = Prop;
				if (const FArrayProperty* ArrayProp = CastField<FArrayProperty>(Prop))
				{
					InnerProp = ArrayProp->Inner;
					checkf(InnerProp, TEXT("Missing InnerProp for ArrayProperty.Name: %s, Type: %s. Package: %s"), *Prop->GetName(), *Prop->GetClass()->GetName(), *GetNameSafe(SaveContext.GetPackage()));
				}

				// Harvest the name of the property, since it is only made transient for the purpose of the package harvest, it will be needed for the LinkerSave
				HarvestExportDataName(Prop->GetFName());

				Props.Add(Prop);

				if (Prop != InnerProp)
				{
					if (Prop->GetFName() != InnerProp->GetFName())
					{
						// Harvest the name of the inner property as well, since it is only made transient for the purpose of the package harvest, it will be needed for the LinkerSave
						HarvestExportDataName(InnerProp->GetFName());
					}

					Props.Add(InnerProp);
				}
			}
		}
		if (!Props.IsEmpty())
		{
			TransientPropertyOverrides.Add(PairObjectOverrides.Key, MoveTemp(Props));
		}
		if (!PairObjectOverrides.Key->HasAnyFlags(RF_Transient) && PairObjectOverrides.Value.bForceTransient)
		{
			SaveContext.GetCachedObjectStatus(PairObjectOverrides.Key).bSaveOverrideForcedTransient = true;
		}
	}
}

TMap<UObject*, TSet<FProperty*>> FPackageHarvester::ReleaseTransientPropertyOverrides()
{
	return MoveTemp(TransientPropertyOverrides);
}

void FPackageHarvester::HarvestDependency(TObjectPtr<UObject> InObj, bool bIsNative)
{
	if (CurrentExportDependencies.ProcessImportDepth > 0 || // Skip if we are processing a transitive import found inside ProcessImport
		CurrentExportDependencies.bIgnoreDependencies || // Skip in stack-specific cases that put FIgnoreDependenciesScope on the stack
		(InObj.GetOuter() == nullptr && InObj.GetClass()->GetFName() == NAME_Package)) // Skip if object is a package
	{
		return;
	}

	if (CurrentExportDependencies.CurrentExport)
	{
		if (bIsNative)
		{
			CurrentExportDependencies.NativeObjectReferences.Add(InObj);
		}
		else
		{
			CurrentExportDependencies.ObjectReferences.Add(InObj);
		}
	}
	else if (CurrentExportDependencies.CurrentCellExport)
	{
		CurrentExportDependencies.ObjectReferences.Add(InObj);
	}
}

bool FPackageHarvester::CurrentExportHasDependency(TObjectPtr<UObject> InObj, ESaveRealm HarvestingRealm) const
{
	FHarvestedRealm& RealmData = SaveContext.GetHarvestedRealm(HarvestingRealm);
	return RealmData.GetObjectDependencies().Contains(InObj) || RealmData.GetNativeObjectDependencies().Contains(InObj);
}

void FPackageHarvester::HarvestExportDataName(FName Name)
{
	ForEachExportHarvestingRealm([this, Name](ESaveRealm HarvestingRealm)
		{
			SaveContext.GetHarvestedRealm(HarvestingRealm).GetNamesReferencedFromExportData().Add(Name.GetDisplayIndex());
		});
}

void FPackageHarvester::HarvestPackageHeaderName(FName Name)
{
	ForEachExportHarvestingRealm([this, Name](ESaveRealm HarvestingRealm)
		{
			SaveContext.GetHarvestedRealm(HarvestingRealm).GetNamesReferencedFromPackageHeader().Add(Name.GetDisplayIndex());
		});
}

void FPackageHarvester::HarvestSearchableName(TObjectPtr<UObject> TypeObject, FName Name)
{
	// Searchable names are only ever saved into the editor scope, so don't harvest the type object for game saves
	FHarvestScope EditorOnlyScope = EnterConditionalEditorOnlyScope(true);
	
	// Make sure the object is tracked as a dependency
	bool bAlreadyTrackedInAllHarvestingRealms = true;
	ForEachExportHarvestingRealm([this, &bAlreadyTrackedInAllHarvestingRealms, TypeObject](ESaveRealm HarvestingRealm)
		{
			bAlreadyTrackedInAllHarvestingRealms &= CurrentExportHasDependency(TypeObject, HarvestingRealm);
		});
	if (!bAlreadyTrackedInAllHarvestingRealms)
	{
		(*this) << TypeObject;
	}

	HarvestPackageHeaderName(Name);
	ForEachExportHarvestingRealm([this, TypeObject, Name](ESaveRealm HarvestingRealm)
		{
			FHarvestedRealm& RealmData = SaveContext.GetHarvestedRealm(HarvestingRealm);
			RealmData.GetSearchableNamesObjectMap().FindOrAdd(TypeObject).AddUnique(Name);
		});
}

void FPackageHarvester::HarvestExport(UObject* InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	bool bFromOptionalRef = CurrentExportDependencies.CurrentExport &&
		CurrentExportDependencies.CurrentExport->GetClass()->HasAnyClassFlags(CLASS_Optional);
	FTaggedExport TaggedExport(InObject, !DoesObjectNeedLoadForEditorGame(InObject), bFromOptionalRef);

	ForEachExportHarvestingRealm([this, &TaggedExport, InObject, &ObjectStatus](ESaveRealm HarvestingRealm)
		{
			FHarvestedRealm& RealmData = SaveContext.GetHarvestedRealm(HarvestingRealm);
			RealmData.AddExport(TaggedExport);
			RealmData.GetNamesReferencedFromPackageHeader().Add(InObject->GetFName().GetDisplayIndex());
			SetInstigatorFromProcessingStack(ObjectStatus, HarvestingRealm);
		});
	ExportsToProcess.Enqueue({ InObject, nullptr, CurrentExportHarvestingRealms });
}

void FPackageHarvester::HarvestImport(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	ForEachExportHarvestingRealm([this, InObject, &ObjectStatus](ESaveRealm HarvestingRealm)
		{
			SaveContext.GetHarvestedRealm(HarvestingRealm).AddImport(InObject);
			if (CurrentExportDependencies.ProcessImportDepth == 0)
			{
				SaveContext.GetHarvestedRealm(HarvestingRealm).AddDirectImport(InObject);
			}
			SetInstigatorFromProcessingStack(ObjectStatus, HarvestingRealm);
		});
}

void FPackageHarvester::SetInstigatorFromProcessingStack(UE::SavePackageUtilities::FObjectStatus& ObjectStatus,
	ESaveRealm HarvestingRealm)
{
	using namespace UE::SavePackageUtilities;

	FRealmInstigator& Instigator = ObjectStatus.RealmInstigator[(uint32)HarvestingRealm];
	if (Instigator.Object)
	{
		return;
	}

	Instigator.Property = nullptr;
	if (CurrentExportDependencies.CurrentExport)
	{
		Instigator.Object = CurrentExportDependencies.CurrentExport;
		const FArchiveSerializedPropertyChain* PropertyChain = GetSerializedPropertyChain();
		if (PropertyChain && PropertyChain->GetNumProperties() > 0)
		{
			Instigator.Property = PropertyChain->GetPropertyFromRoot(0);
		}
	}
	else
	{
		Instigator.Object = SaveContext.GetPackage();
	}
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FPackageHarvester::HarvestCellExport(Verse::VCell* InCell)
{
	ForEachExportHarvestingRealm([this, InCell](ESaveRealm HarvestingRealm)
		{
			FHarvestedRealm& RealmData = SaveContext.GetHarvestedRealm(HarvestingRealm);
			RealmData.AddCellExport(InCell);

			FName CppTypeInfo(InCell->GetCppClassInfo()->Name);
			RealmData.GetNamesReferencedFromPackageHeader().Add(CppTypeInfo.GetDisplayIndex());
		});
	ExportsToProcess.Enqueue({ nullptr, InCell, CurrentExportHarvestingRealms });
}

void FPackageHarvester::HarvestCellImport(Verse::VCell* InCell)
{
	ForEachExportHarvestingRealm([this, InCell](ESaveRealm HarvestingRealm)
		{
			SaveContext.GetHarvestedRealm(HarvestingRealm).AddCellImport(InCell);
		});
}
#endif

#if WITH_EDITORONLY_DATA
// Harvests the names of an import Struct object's super type hierarchy (used in: FImportTypeHierarchy).
void FPackageHarvester::HarvestImportTypeHierarchyNames(const UObject* Import)
{
	const UStruct* Struct = Cast<UStruct>(Import);
	if (Struct)
	{
		// Harvest names for types that can be used to create placeholders with, or types from an export with RF_HasDynamicImports
		// See comment in SavePackage2.cpp.
		bool bRequiresNames = UE::CanCreatePropertyBagPlaceholderForType(Struct->GetClass(), Struct) ||
			(CurrentExportDependencies.CurrentExport && CurrentExportDependencies.CurrentExport->HasAnyFlags(RF_HasDynamicImports));
		if (bRequiresNames)
		{
			const UStruct* SuperStruct = Struct->GetSuperStruct();

			while (SuperStruct && (SuperStruct != UObject::StaticClass()))
			{
				const UClass* SuperStructClass = SuperStruct->GetClass();
				const UPackage* SuperStructPackage = SuperStruct->GetPackage();

				FName TypeName = SuperStruct->GetFName();
				FName PackageName = SuperStructPackage ? SuperStructPackage->GetFName() : NAME_None;

				// This is the Class of the Struct, possible values are "Class", "VerseClass", "BlueprintGeneratedClass", "UserDefinedStruct", etc.
				// We need to know the class of the Struct so we can create a placeholder type for it, when it is not available anymore (e.g., the type
				// was removed).
				FName ClassName = SuperStructClass ? SuperStructClass->GetFName() : NAME_None;
				FName ClassPackageName = (SuperStructClass && SuperStructClass->GetPackage()) ? SuperStructClass->GetPackage()->GetFName() : NAME_None;

				HarvestPackageHeaderName(TypeName);
				HarvestPackageHeaderName(PackageName);
				HarvestPackageHeaderName(ClassName);
				HarvestPackageHeaderName(ClassPackageName);

				SuperStruct = SuperStruct->GetSuperStruct();
			}
		}
	}
}
#endif

void FPackageHarvester::AppendCurrentExportDependencies()
{
	check(CurrentExportDependencies.CurrentExport || CurrentExportDependencies.CurrentCellExport);
	if (CurrentExportDependencies.CurrentExport)
	{
		ForEachExportHarvestingRealm([this](ESaveRealm HarvestingRealm)
			{
				FHarvestedRealm& RealmData = SaveContext.GetHarvestedRealm(HarvestingRealm);
				RealmData.GetObjectDependencies().Add(CurrentExportDependencies.CurrentExport,
					MoveTemp(CurrentExportDependencies.ObjectReferences));
				RealmData.GetNativeObjectDependencies().Add(CurrentExportDependencies.CurrentExport,
					MoveTemp(CurrentExportDependencies.NativeObjectReferences));
				RealmData.GetCellDependencies().Add(CurrentExportDependencies.CurrentExport,
					MoveTemp(CurrentExportDependencies.CellReferences));
			});
		CurrentExportDependencies.CurrentExport = nullptr;
	}
	else if (CurrentExportDependencies.CurrentCellExport)
	{
		ForEachExportHarvestingRealm([this](ESaveRealm HarvestingRealm)
			{
				FHarvestedRealm& RealmData = SaveContext.GetHarvestedRealm(HarvestingRealm);
				RealmData.GetCellObjectDependencies().Add(CurrentExportDependencies.CurrentCellExport,
					MoveTemp(CurrentExportDependencies.ObjectReferences));
				RealmData.GetCellCellDependencies().Add(CurrentExportDependencies.CurrentCellExport,
					MoveTemp(CurrentExportDependencies.CellReferences));
			});
		CurrentExportDependencies.CurrentCellExport = nullptr;
	}
}

FString FPackageHarvester::GetUnsaveableReason(UObject* Required, ESaveRealm RealmInWhichItIsUnsaveable)
{
	TObjectPtr<UObject> Culprit;
	FString Reason;
	ESaveableStatus Status = GetSaveableStatusForRealm(Required, RealmInWhichItIsUnsaveable, Culprit, Reason);
	if (Status != ESaveableStatus::Success)
	{
		return FString::Printf(TEXT("It %s."), *Reason);
	}

	bool bShouldBeExport = Required->IsInPackage(SaveContext.GetPackage());
	if (bShouldBeExport)
	{
		return FString(TEXTVIEW("It should be an export but was excluded for an unknown reason."));
	}
	else
	{
		return FString(TEXTVIEW("It should be an import but was excluded for an unknown reason."));

	}
}

ESaveableStatus FPackageHarvester::GetSaveableStatusForRealm(UObject* Obj, ESaveRealm RealmInWhichItIsUnsaveable,
	TObjectPtr<UObject>& OutCulprit, FString& OutReason)
{
	// Copy some of the code from operator<<(UObject*), TryHarvestExport, TryHarvestImport to find out why the Required object was not included
	if (!Obj || Obj->GetOutermost()->GetFName() == GLongCoreUObjectPackageName)
	{
		// No object or in CoreUObject, it is saveable
		return ESaveableStatus::Success;
	}

	UE::SavePackageUtilities::FObjectStatus& ObjectStatus = SaveContext.GetCachedObjectStatus(Obj);
	SaveContext.UpdateSaveableStatus(Obj, ObjectStatus);
	if (ObjectStatus.SaveableStatus != ESaveableStatus::Success)
	{
		if (ObjectStatus.SaveableStatus == ESaveableStatus::OuterUnsaveable)
		{
			check(ObjectStatus.SaveableStatusCulprit != nullptr &&
				ObjectStatus.SaveableStatusCulpritStatus != ESaveableStatus::Success);
			OutCulprit = ObjectStatus.SaveableStatusCulprit;
			OutReason = FString::Printf(TEXT("has outer %s which %s"), *OutCulprit.GetPathName(),
				LexToString(ObjectStatus.SaveableStatusCulpritStatus));
		}
		else
		{
			OutCulprit = Obj;
			OutReason = LexToString(ObjectStatus.SaveableStatus);
		}
		return ObjectStatus.SaveableStatus;
	}

	const EObjectMark ExcludedObjectMarks = SaveContext.GetExcludedObjectMarks(RealmInWhichItIsUnsaveable);
	const ITargetPlatform* TargetPlatform = SaveContext.GetTargetPlatform();
	EObjectMark ObjectMarks = GenerateMarksForObject(Obj, SaveContext);
	if (!!(ObjectMarks & ExcludedObjectMarks))
	{
		OutCulprit = Obj;
		OutReason = FString::Printf(TEXT("has ObjectMarks 0x%x that are excluded for the current cooking target"),
			(int32)(ObjectMarks & ExcludedObjectMarks));
		return ESaveableStatus::ExcludedByPlatform;
	}

	UObject* ObjOuter = Obj->GetOuter();
	FString RecursiveReason;
	ESaveableStatus RecursiveStatus;
	if (ObjOuter)
	{
		RecursiveStatus = GetSaveableStatusForRealm(ObjOuter, RealmInWhichItIsUnsaveable, OutCulprit, RecursiveReason);
		if (RecursiveStatus != ESaveableStatus::Success)
		{
			check(OutCulprit);
			if (RecursiveStatus == ESaveableStatus::OuterUnsaveable)
			{
				OutReason = MoveTemp(RecursiveReason);
			}
			else
			{
				OutReason = FString::Printf(TEXT("has outer %s which %s"), *OutCulprit->GetPathName(), *RecursiveReason);
			}
			return ESaveableStatus::OuterUnsaveable;
		}
	}

	UClass* ObjClass = Obj->GetClass();
	if (ObjClass)
	{
		RecursiveStatus = GetSaveableStatusForRealm(ObjClass, RealmInWhichItIsUnsaveable, OutCulprit, RecursiveReason);
		if (RecursiveStatus != ESaveableStatus::Success)
		{
			OutCulprit = ObjClass;
			OutReason = FString::Printf(TEXT("has class %s which %s"), *ObjClass->GetPathName(), *RecursiveReason);
			return ESaveableStatus::ClassUnsaveable;
		}
	}

	// Check parent struct if we have one
	UStruct* ThisStruct = Cast<UStruct>(Obj);
	UObject* SuperStruct = ThisStruct ? ThisStruct->GetSuperStruct() : nullptr;
	if (SuperStruct)
	{
		RecursiveStatus = GetSaveableStatusForRealm(SuperStruct, RealmInWhichItIsUnsaveable, OutCulprit, RecursiveReason);
		if (RecursiveStatus != ESaveableStatus::Success)
		{
			OutCulprit = SuperStruct;
			OutReason = FString::Printf(TEXT("has superclass %s which %s"), *SuperStruct->GetPathName(), *RecursiveReason);
			return ESaveableStatus::ClassUnsaveable;
		}
	}

	// Check archetype, this may not have been covered in the case of components
	UObject* Archetype = Obj->GetArchetype();
	if (Archetype)
	{
		RecursiveStatus = GetSaveableStatusForRealm(Archetype, RealmInWhichItIsUnsaveable, OutCulprit, RecursiveReason);
		if (RecursiveStatus != ESaveableStatus::Success)
		{
			OutCulprit = Archetype;
			OutReason = FString::Printf(TEXT("has archetype %s which %s"), *Archetype->GetPathName(), *RecursiveReason);
			return ESaveableStatus::ClassUnsaveable;
		}
	}

	bool bShouldBeExport = Obj->IsInPackage(SaveContext.GetPackage());
	if (!bShouldBeExport)
	{
		// Imports additionally can be excluded if their package is excluded for the platform
		bool bExcludePackageFromCook = FCoreUObjectDelegates::ShouldCookPackageForPlatform.IsBound() ?
			!FCoreUObjectDelegates::ShouldCookPackageForPlatform.Execute(Obj->GetOutermost(), CookingTarget()) : false;
		if (bExcludePackageFromCook)
		{
			OutCulprit = Obj;
			OutReason = FString::Printf(TEXT("is in package %s which is excluded from the cook by FCoreUObjectDelegates::ShouldCookPackageForPlatform."),
				*Obj->GetOutermost()->GetName());
			return ESaveableStatus::ExcludedByPlatform;
		}
	}

	return ESaveableStatus::Success;
}
