// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/Queue.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SavePackage.h"

enum class ESaveableStatus;
enum class ESaveRealm : uint32;
enum class EIllegalRefReason : uint8;
class FSaveContext;
class UObject;
namespace UE::SavePackageUtilities { struct FObjectStatus; }
struct FWeakObjectPtr;
struct FLazyObjectPtr;
struct FSoftObjectPath;
namespace Verse { struct VCell; }

/**
 * Helper class used to gather all package exports, imports and dependencies to build linker tables when saving packages
 * Gather Export, Imports, Referenced Names, SoftObjectPaths, Custom Object Versions
 */
class FPackageHarvester : public UE::SavePackageUtilities::Private::FArchiveSavePackageCollector
{
public:
	using FExportingRealmsArray = TArray<ESaveRealm, TInlineAllocator<2>>;
	struct FExportWithContext
	{
		/* Export to process. Only one of these two should be set. */
		UObject* Export = nullptr;
		Verse::VCell* CellExport = nullptr;
		/* The realms from which this export was harvested. */
		FExportingRealmsArray HarvestedFromRealms;

		operator bool() const
		{
			return Export != nullptr || CellExport != nullptr;
		}
	};

	/** Scope object constructed when changing the CurrentExportHarvestingRealms that restores the previous realms when done */
	class FHarvestScope
	{
	public:
		FHarvestScope(FPackageHarvester& InHarvester);
		FHarvestScope(FHarvestScope&& Other);
		FHarvestScope(const FHarvestScope& Other) = delete;
		FHarvestScope& operator=(FHarvestScope&& Other) = delete;
		FHarvestScope& operator=(const FHarvestScope& Other) = delete;
		~FHarvestScope();
		bool IsEmpty() const;
	private:
		FPackageHarvester& Harvester;
		FExportingRealmsArray PreviousExportHarvestingRealms;
		bool bActive;
	};

public:
	FPackageHarvester(FSaveContext& InContext);

	FExportWithContext PopExportToProcess();

	void TryHarvestExport(UObject* InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	/** Add objects/names/others that are referenced when an object is being saved */
	void ProcessExport(const FExportWithContext& InProcessContext);
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	void TryHarvestCellExport(Verse::VCell* InCell);
	AUTORTFM_DISABLE void ProcessCellExport(const FExportWithContext& InProcessContext);
#endif
	FHarvestScope EnterRootReferencesScope();

	void HarvestExportDataName(FName Name);
	void HarvestPackageHeaderName(FName Name);
	void HarvestSearchableName(TObjectPtr<UObject> TypeObject, FName Name);
	void HarvestDependency(TObjectPtr<UObject> InObj, bool bIsNative);

	bool CurrentExportHasDependency(TObjectPtr<UObject> InObj, ESaveRealm HarvestingRealm) const;
	TMap<UObject*, TSet<FProperty*>> ReleaseTransientPropertyOverrides();

	// FArchiveUObject implementation
	virtual FString GetArchiveName() const override;
	virtual void MarkSearchableName(const TObjectPtr<const UObject>& TypeObject, const FName& ValueName) const override;
	virtual FArchive& operator<<(UObject*& Obj) override;
	virtual FArchive& operator<<(struct FWeakObjectPtr& Value) override;
	virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	virtual FArchive& operator<<(FName& Name) override;
	virtual FArchive& operator<<(FObjectPtr& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Cell) override;
#endif

	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;

	/** Return whether the Object and all its Outers are native. */
	static bool IsObjNative(UObject* InObj);
	/** Report whether the object should be put in the optional realm even if harvesting realm is non-optional. */
	static bool ShouldObjectBeHarvestedInOptionalRealm(TObjectPtr<UObject> InObj, FSaveContext& InSaveContext);

private:
	/** Dependencies collected for the current export, and scope information about how to collect them. */
	struct FExportDependencies
	{
		UObject* CurrentExport = nullptr;
		Verse::VCell* CurrentCellExport = nullptr;
		TSet<TObjectPtr<UObject>> ObjectReferences;
		TSet<TObjectPtr<UObject>> NativeObjectReferences;
		TSet<Verse::VCell*> CellReferences;
		int32 ProcessImportDepth = 0;
		bool bIgnoreDependencies = false;
	};

	/**
	 * Set dependencies and other storage to write to the given export, and set
	 * CurrentExportHarvestingRealms and other data based on where the export was harvested.
	 */
	class FExportScope
	{
	public:
		FExportScope(FPackageHarvester& InHarvester, const FExportWithContext& InToProcess);
		~FExportScope();
	private:
		FPackageHarvester& Harvester;
		FExportingRealmsArray PreviousExportHarvestingRealms;
		bool bPreviousFilterEditorOnly;
		bool bPreviousSavingOptionalObject;
	};

	/** Mark that dependencies should be ignored when harvesting further imports/exports in this scope. */
	class FIgnoreDependenciesScope
	{
	public:
		FIgnoreDependenciesScope(FPackageHarvester& InHarvester);
		~FIgnoreDependenciesScope();
	private:
		FPackageHarvester& Harvester;
		bool bPreviousValue;
	};

private:

	/** Call void (ESaveRealm HarvestingRealm) on each Realm in CurrentExportHarvestingRealms */
	template <typename CallbackType>
	void ForEachExportHarvestingRealm(CallbackType&& Callback);
	bool HasAnyExportHarvestingRealms();

	void ResolveOverrides();

	void TryHarvestExportInternal(UObject* InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	void HarvestExport(UObject* InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	void TryHarvestImport(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	void HarvestImport(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	/** Add objects/names/others that are referenced even when an object is just being imported */
	void ProcessImport(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	void SetInstigatorFromProcessingStack(UE::SavePackageUtilities::FObjectStatus& ObjectStatus,
		ESaveRealm HarvestingRealm);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	void HarvestCellExport(Verse::VCell* InCell);
	void HarvestCellImport(Verse::VCell* InCell);
#endif

#if WITH_EDITORONLY_DATA
	void HarvestImportTypeHierarchyNames(const UObject* Import);
#endif

	void AppendCurrentExportDependencies();
	FString GetUnsaveableReason(UObject* Required, ESaveRealm RealmInWhichItIsUnsaveable);
	ESaveableStatus GetSaveableStatusForRealm(UObject* Obj, ESaveRealm RealmInWhichItIsUnsaveable,
		TObjectPtr<UObject>& OutCulprit, FString& OutReason);

	/** Set HarvestingRealms equal to the given Array. */
	FHarvestScope EnterRealmsArrayScope(FExportingRealmsArray& Array);
	/** If bIsEditorOnly is true, remove CurrentExportHarvestingRealms that do not follow editoronly references. */
	FHarvestScope EnterConditionalEditorOnlyScope(bool bIsEditorOnly);
	/** If the Object is optional, clear CurrentExportHarvestingRealms and populate it with the Optional Realm. */
	FHarvestScope EnterConditionalOptionalObjectScope(TObjectPtr<UObject> Object);
	/** Remove CurrentExportHarvestingRealms that already include Export. */
	FHarvestScope EnterNewExportOnlyScope(UObject* Export);
	/** Remove CurrentExportHarvestingRealms that return false for ConditionallyExcludeObjectForRealm(Object). */
	FHarvestScope EnterNotExcludedScope(TObjectPtr<UObject> Object,
		UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	/** Remove CurrentExportHarvestingRealms that return true for IsExcluded(Object). */
	FHarvestScope EnterNotPreviouslyExcludedScope(TObjectPtr<UObject> Object);
	/** Remove CurrentExportHarvestingRealms that return false for IsIncluded(Object). */
	FHarvestScope EnterIncludedScope(TObjectPtr<UObject> Object);
	/** Copy CurrentExportHarvestingRealms and split it into two arrays: AlreadyIncluded and NotAlreadyIncluded. */
	void GetPreviouslyIncludedRealms(TObjectPtr<UObject> Object, FExportingRealmsArray& OutAlreadyIncluded,
		FExportingRealmsArray& OutNotAlreadyIncluded);

	FHarvestScope EnterIncludedScope(Verse::VCell* Cell);
	void GetPreviouslyIncludedRealms(Verse::VCell* Cell, FExportingRealmsArray& OutAlreadyIncluded,
		FExportingRealmsArray& OutNotAlreadyIncluded);

	FSaveContext& SaveContext;
	TQueue<FExportWithContext> ExportsToProcess;
	FExportDependencies CurrentExportDependencies;
	TMap<UObject*, TSet<FProperty*>> TransientPropertyOverrides;
	FExportingRealmsArray CurrentExportHarvestingRealms;
};
