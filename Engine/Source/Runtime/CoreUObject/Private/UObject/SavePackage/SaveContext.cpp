// Copyright Epic Games, Inc. All Rights Reserved.

#include "SaveContext.h"

#include "Algo/Find.h"
#include "Algo/Unique.h"
#include "Cooker/CookDependency.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PackageAccessTracking.h"
#include "Serialization/PackageWriter.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UObjectGlobals.h"

TArray<ESaveRealm> FSaveContext::GetHarvestedRealmsToSave()
{
	TArray<ESaveRealm> HarvestedContextsToSave;
	if (IsCooking())
	{
		HarvestedContextsToSave.Add(ESaveRealm::Game);
		if (IsSaveOptional())
		{
			HarvestedContextsToSave.Add(ESaveRealm::Optional);
		}
	}
	else
	{
		HarvestedContextsToSave.Add(ESaveRealm::Editor);
	}
	return HarvestedContextsToSave;
}

bool FSaveContext::IsUnsaveable(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus,
	bool bEmitWarning)
{
	if (!InObject)
	{
		return false;
	}
	UpdateSaveableStatus(InObject, ObjectStatus);
	check(ObjectStatus.bSaveableStatusValid);

	if (bEmitWarning && ObjectStatus.SaveableStatus != ESaveableStatus::Success)
	{
		// if this is a class default object being exported, make sure it's not unsaveable for any reason,
		// as we need it to be saved to disk (unless it's associated with a transient generated class)
#if WITH_EDITORONLY_DATA
		ensureAlways(!ObjectStatus.bAttemptedExport || !InObject->HasAllFlags(RF_ClassDefaultObject) ||
			(InObject->GetClass()->ClassGeneratedBy != nullptr && InObject->GetClass()->HasAnyFlags(RF_Transient)));
#endif

		if (ObjectStatus.SaveableStatus == ESaveableStatus::OuterUnsaveable
			&& (ObjectStatus.SaveableStatusCulpritStatus == ESaveableStatus::AbstractClass
				|| ObjectStatus.SaveableStatusCulpritStatus == ESaveableStatus::DeprecatedClass
				|| ObjectStatus.SaveableStatusCulpritStatus == ESaveableStatus::NewerVersionExistsClass)
			&& InObject.GetPackage() == GetPackage())
		{
			check(ObjectStatus.SaveableStatusCulprit);
			UE_LOGF(LogSavePackage, Warning, "%ls has unsaveable outer %ls (outer is %ls), so it will not be saved.",
				*InObject.GetFullName(), *ObjectStatus.SaveableStatusCulprit->GetFullName(),
				LexToString(ObjectStatus.SaveableStatusCulpritStatus));
		}
	}

	return ObjectStatus.SaveableStatus != ESaveableStatus::Success;
}

void FSaveContext::UpdateSaveableStatus(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	if (ObjectStatus.bSaveableStatusValid)
	{
		return;
	}

	ObjectStatus.bSaveableStatusValid = true;
	ObjectStatus.SaveableStatus = ESaveableStatus::Success;

	ESaveableStatus StatusNoOuter = GetSaveableStatusNoOuter(InObject, ObjectStatus);
	if (StatusNoOuter != ESaveableStatus::Success)
	{
		check(StatusNoOuter != ESaveableStatus::OuterUnsaveable &&
			StatusNoOuter != ESaveableStatus::ClassUnsaveable);
		ObjectStatus.SaveableStatus = StatusNoOuter;
		return;
	}

	TObjectPtr<UObject> Outer = InObject.GetOuter();
	if (Outer)
	{
		UE::SavePackageUtilities::FObjectStatus& OuterStatus = GetCachedObjectStatus(Outer);
		UpdateSaveableStatus(Outer, OuterStatus);

		ObjectStatus.bSaveableStatusValid = true;
		ObjectStatus.SaveableStatus = ESaveableStatus::Success;
		if (OuterStatus.SaveableStatus != ESaveableStatus::Success)
		{
			ObjectStatus.SaveableStatus = ESaveableStatus::OuterUnsaveable;
			if (OuterStatus.SaveableStatus == ESaveableStatus::OuterUnsaveable)
			{
				check(OuterStatus.SaveableStatusCulprit);
				check(OuterStatus.SaveableStatusCulpritStatus != ESaveableStatus::Success);
				ObjectStatus.SaveableStatusCulprit = OuterStatus.SaveableStatusCulprit;
				ObjectStatus.SaveableStatusCulpritStatus = OuterStatus.SaveableStatusCulpritStatus;
			}
			else
			{
				ObjectStatus.SaveableStatusCulprit = Outer;
				ObjectStatus.SaveableStatusCulpritStatus = OuterStatus.SaveableStatus;
			}
		}
	}
}

ESaveableStatus FSaveContext::GetSaveableStatusNoOuter(TObjectPtr<UObject> ObjPtr,
	UE::SavePackageUtilities::FObjectStatus& ObjectStatus)
{
	UObject* Obj = ResolveForSave(ObjPtr, ObjectStatus);

	// pending kill objects are unsaveable
	if (!IsValidChecked(Obj))
	{
		return ESaveableStatus::PendingKill;
	}

	// transient objects are unsaveable if non-native
	if (!Obj->IsNative())
	{
		if (ObjectStatus.HasTransientFlag(Obj))
		{
			return ESaveableStatus::TransientFlag;
		}
		if (ObjectStatus.bSaveOverrideForcedTransient)
		{
			return ESaveableStatus::TransientOverride;
		}
	}

	UClass* Class = Obj->GetClass();
	// if the object class is abstract, has been marked as deprecated, there is a newer version that exist, or the class is marked transient, then the object is unsaveable
	// @note: Although object instances of a transient class should definitely be unsaveable, it results in discrepancies with the old save algorithm and currently load problems
	if (Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists /*| CLASS_Transient*/)
		&& !Obj->HasAnyFlags(RF_ClassDefaultObject))
	{
		// There used to be a check for reference if the class had the CLASS_HasInstancedReference,
		// but we don't need it because those references are outer-ed to the object being flagged as unsaveable, making them unsaveable as well without having to look for them
		return Class->HasAnyClassFlags(CLASS_Abstract) ? ESaveableStatus::AbstractClass :
			Class->HasAnyClassFlags(CLASS_Deprecated) ? ESaveableStatus::DeprecatedClass :
			ESaveableStatus::NewerVersionExistsClass;
	}

	return ESaveableStatus::Success;
}

UObject* FSaveContext::ResolveForSave(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& Status)
{
	using namespace UE::Cook;


	if (
		// SavePackage Build dependencies are only added when cooking. Build dependencies in editor saves mean
		// management propagation dependencies and are only added from system-specific OnCookEvent overrides.
		IsCooking()
		// No need to consider build dependencies when resolving an export; the package and class of the export
		// are already present as dependencies.
		&& !Status.IsInSavePackage(InObject, GetPackage())
		)
	{
		UObject* Object = nullptr;
		{
			// Disable the automated tracking of the resolve with the cooker.
			UE_COOK_RESULTPROJECTION_SCOPED(UE::Cook::ResultProjection::None);
			Object = InObject.Get();
		}
		if (!Status.bDeclaredSaveBuildDependency && Object)
		{
			// Declare the build dependencies necessary for our use of the object.
#if WITH_EDITOR
			TArray<FCookDependency>& SaveDependencies = ObjectSaveContext.BuildResultDependencies.FindOrAdd(BuildResult::NAME_Save);
			SaveDependencies.Add(FCookDependency::Package(Object->GetPackage()->GetFName()));
			SaveDependencies.Add(FCookDependency::NativeClass(Object->GetClass()));
#endif
			Status.bDeclaredSaveBuildDependency = true;
		}
		return Object;
	}
	else
	{
		return InObject.Get();
	}
}

bool FSaveContext::IsTransient(TObjectPtr<UObject> InObject)
{
	return IsTransient(InObject, GetCachedObjectStatus(InObject));
}

bool FSaveContext::IsTransient(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& Status)
{
	if (!InObject)
	{
		return false;
	}

	UObject* ObjectPtr = ResolveForSave(InObject, Status);
	if (ObjectPtr->HasAnyFlags(RF_Transient))
	{
		return true;
	}

	if (Status.bSaveOverrideForcedTransient)
	{
		return true;
	}
	if (Status.bAttemptedExport && Status.bSaveableStatusValid && Status.SaveableStatus != ESaveableStatus::Success)
	{
		// Exports found to be unsaveable are treated the same as transient objects for all the calls to IsTransient
		// in SavePackage.
		return true;
	}

	return false;
}

FSavePackageResultStruct FSaveContext::GetFinalResult()
{
	if (Result != ESavePackageResult::Success)
	{
		return Result;
	}

	ESavePackageResult FinalResult = IsStubRequested() ? ESavePackageResult::GenerateStub : ESavePackageResult::Success;
	FSavePackageResultStruct ResultData(FinalResult, TotalPackageSizeUncompressed, SerializedPackageFlags);

	ResultData.SavedAssets = MoveTemp(SavedAssets);
	UClass* PackageClass = UPackage::StaticClass();
	for (TObjectPtr<UObject> Import : GetImports())
	{
		if (Import.IsA(PackageClass))
		{
			ResultData.ImportPackages.Add(Import.GetFName());
		}
	}
	TSet<FName>& SoftPackageReferenceList = GetSoftPackageReferenceList();
	ResultData.SoftPackageReferences = SoftPackageReferenceList.Array();
	const TSet<FName>& UntrackedPackageReferenceList = GetUntrackedSoftPackageReferenceList();
	ResultData.UntrackedSoftPackageReferences = UntrackedPackageReferenceList.Array();

#if WITH_EDITOR
	for (const FSoftObjectPath& RuntimeDependency : ObjectSaveContext.CookRuntimeDependencies)
	{
		FName PackageDependency = RuntimeDependency.GetLongPackageFName();
		if (!PackageDependency.IsNone())
		{
			ResultData.SoftPackageReferences.Add(PackageDependency);
		}
	}
	ResultData.BuildResultDependencies = MoveTemp(ObjectSaveContext.BuildResultDependencies);

	if (IsCooking())
	{
		// Add the imports, exports, and PreloadDependencies for the CookImportsChecker

		// the package isn't actually in the export map, but that is ok, we add it as export anyway for error checking
		ResultData.Exports.Reserve(GetExports().Num() + 1);
		ResultData.Exports.Add(GetPackage());
		for (const FTaggedExport& ExportData : GetExports())
		{
			UObject* Export = ExportData.Obj.Get();
			check(Export);
			ResultData.Exports.Add(Export);
		}

		ResultData.Imports.Reserve(GetImports().Num());
		for (TObjectPtr<UObject> ImportPtr : GetImports())
		{
			UObject* Import = ImportPtr.Get();
			check(Import);
			ResultData.Imports.Add(Import);
		}

		ResultData.PreloadDependencies = MoveTemp(GetPreloadDependencies());
	}
#endif
	return ResultData;
}

UE::SavePackageUtilities::EEditorOnlyObjectFlags FSaveContext::GetEditorOnlyObjectFlags() const
{
	using namespace UE::SavePackageUtilities;

	// If doing an editor save, HasNonEditorOnlyReferences=true overrides NotForClient, NotForServer,
	// and virtual IsEditorOnly and marks it as UsedInGame
	bool bApplyHasNonEditorOnlyReferences = GetTargetPlatform() == nullptr;
	return
		EEditorOnlyObjectFlags::CheckRecursive |
		(bApplyHasNonEditorOnlyReferences
			? EEditorOnlyObjectFlags::ApplyHasNonEditorOnlyReferences
			: EEditorOnlyObjectFlags::None);
}

void FSaveContext::AddObjectToPostSaveSerialization(UObject* Object)
{
	PostSaveObjectsToSerialize.Add(Object);
}

const TSet<UObject*>& FSaveContext::GetPostSaveObjectsToSerialize() const
{
	return PostSaveObjectsToSerialize;
}

namespace
{
	// Use Soft Class Ptr to gather all asset types even if they are not yet loaded.
	TArray<TSoftClassPtr<UObject>> AutomaticOptionalInclusionAssetTypeList;
}

void FSaveContext::SetupHarvestingRealms()
{
	// Create the different harvesting realms
	HarvestedRealms.AddDefaulted((uint32)ESaveRealm::RealmCount);

	// if cooking the default harvesting context is Game, otherwise it's the editor context
	CurrentHarvestingRealm = IsCooking() ? ESaveRealm::Game : ESaveRealm::Editor;

	// Generate the automatic optional context inclusion asset list
	static bool bAssetListGenerated = [](TArray<TSoftClassPtr<UObject>>& OutAssetList)
		{
			UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;

			TArray<FString> AssetList;
			GConfig->GetArray(TEXT("CookSettings"), TEXT("AutomaticOptionalInclusionAssetType"), AssetList, GEditorIni);
			for (const FString& AssetType : AssetList)
			{
				TSoftClassPtr<UObject> SoftClassPath(AssetType);
				if (!SoftClassPath.IsNull())
				{
					OutAssetList.Add(MoveTemp(SoftClassPath));
				}
				else
				{
					UE_LOGF(LogSavePackage, Warning, "The asset type '%ls' found while building the allowlist for automatic optional data inclusion list is not a valid class path.", *AssetType);
				}
			}
			return true;
		}(AutomaticOptionalInclusionAssetTypeList);

	if (bAssetListGenerated && Asset)
	{
		// if the asset type itself is a class (ie. BP) use that to check for auto optional
		UClass* AssetType = Cast<UClass>(Asset);
		AssetType = AssetType ? AssetType : Asset->GetClass();
		bool bAllowedClass = Algo::FindByPredicate(AutomaticOptionalInclusionAssetTypeList, [AssetType](const TSoftClassPtr<UObject>& InAssetClass)
			{
				const UClass* AssetClass = InAssetClass.Get();
				return AssetClass && AssetType->IsChildOf(AssetClass);
			}) != nullptr;
		bIsSaveAutoOptional = IsCooking() && IsSaveOptional() && bAllowedClass;
	}
}

EObjectMark FSaveContext::GetExcludedObjectMarksForGameRealm(const ITargetPlatform* TargetPlatform)
{
	if (TargetPlatform)
	{
		return UE::SavePackageUtilities::GetExcludedObjectMarksForTargetPlatform(TargetPlatform);
	}
	else
	{
		return static_cast<EObjectMark>(OBJECTMARK_NotForTargetPlatform | OBJECTMARK_EditorOnly);
	}
}

void FSaveContext::UpdateEditorRealmPackageBuildDependencies()
{
	using namespace UE::Cook;

	PackageBuildDependencies.Empty();

	// PackageBuildDependencies are only recorded for non-cooked packages
	if (IsCooking())
	{
		return;
	}

#if WITH_EDITOR
	for (const TPair<FName, TArray<FCookDependency>>& BuildResultPair : ObjectSaveContext.BuildResultDependencies)
	{
		// Only consider Load and Save BuildResults; other BuildResults on the package do not contribute to the runtime references
		// of the cooked package (or if they do, then they are also added to the Save result), so we don't need to report them to
		// the AssetManager's chunk assignment graph traversal.
		if (BuildResultPair.Key != UE::Cook::BuildResult::NAME_Load && BuildResultPair.Key != UE::Cook::BuildResult::NAME_Save)
		{
			continue;
		}
		for (const UE::Cook::FCookDependency& CookDependency : BuildResultPair.Value)
		{
			FName PackageName = NAME_None;
			switch (CookDependency.GetType())
			{
			case ECookDependency::Package: [[fallthrough]];
			case ECookDependency::TransitiveBuild:
				PackageName = CookDependency.GetPackageName();
				break;
			default:
				break;
			}
			if (PackageName.IsNone())
			{
				continue;
			}
			PackageBuildDependencies.Add(PackageName);
		}
	}
	PackageBuildDependencies.Sort(FNameLexicalLess());
	PackageBuildDependencies.SetNum(Algo::Unique(PackageBuildDependencies));

	FHarvestedRealm& HarvestedRealm = GetHarvestedRealm(ESaveRealm::Editor);
	TSet<FNameEntryId>& NamesReferencedFromPackageHeader = HarvestedRealm.GetNamesReferencedFromPackageHeader();
	for (FName PackageBuildDependency : PackageBuildDependencies)
	{
		NamesReferencedFromPackageHeader.Add(PackageBuildDependency.GetDisplayIndex());
	}
#endif
}

void FSaveContext::AddExportedClassesToDependencies()
{
#if WITH_EDITOR

	//first find the list of the existing class in the editor package
	FName PackageName = GetPackage()->GetFName();
	FAssetPackageData AssetPackageData;
	IAssetRegistryInterface::GetPtr()->TryGetAssetPackageData(PackageName, AssetPackageData);

	//Retrieve all the UObjects saved in the package
	const TSet<FTaggedExport>& ExportsSet = GetHarvestedRealm(ESaveRealm::Game).GetExports();

	//Make a list of unique class saved in the cooked package but not present in the editor package.
	//Those are the one I want to add as dependencies.
	TStringBuilder<256> ClassPathName;
	TSet<FName> ImportedClassesSet(AssetPackageData.ImportedClasses);
	TSet<const UClass*> UniqueClassesToAdd;
	for (const FTaggedExport& Export : ExportsSet)
	{
		const UClass* ClassToExport = Export.Obj->GetClass();

		ClassPathName.Reset();
		ClassToExport->GetPathName(nullptr, ClassPathName);
		bool AlreadyInEditorPackage = ImportedClassesSet.Find(FName(ClassPathName)) != nullptr;
		if (!AlreadyInEditorPackage)
		{
			UniqueClassesToAdd.Add(ClassToExport);
		}
	}

	//Finally add those missing class to the dependencies
	if (!UniqueClassesToAdd.IsEmpty())
	{
		TArray<UE::Cook::FCookDependency>& CookSaveDependencies
			= GetObjectSaveContext().BuildResultDependencies.FindOrAdd(UE::Cook::BuildResult::NAME_Save);
		for (const UClass* Class : UniqueClassesToAdd)
		{
			CookSaveDependencies.Add(UE::Cook::FCookDependency::NativeClass(Class));
		}
	}
#endif // WITH_EDITOR
}

const TCHAR* LexToString(ESaveableStatus Status)
{
	static_assert(static_cast<int32>(ESaveableStatus::__Count) == 10);
	switch (Status)
	{
	case ESaveableStatus::Success: return TEXT("is saveable");
	case ESaveableStatus::PendingKill: return TEXT("is pendingkill");
	case ESaveableStatus::TransientFlag: return TEXT("is transient");
	case ESaveableStatus::TransientOverride: return TEXT("is Overriden as transient");
	case ESaveableStatus::AbstractClass: return TEXT("has a Class with CLASS_Abstract");
	case ESaveableStatus::DeprecatedClass: return TEXT("has a Class with CLASS_Deprecated");
	case ESaveableStatus::NewerVersionExistsClass: return TEXT("has a Class with CLASS_NewerVersionExists");
	case ESaveableStatus::OuterUnsaveable: return TEXT("has an unsaveable Outer");
	case ESaveableStatus::ClassUnsaveable: return TEXT("has an unsaveable Class");
	case ESaveableStatus::ExcludedByPlatform: return TEXT("is excluded by TargetPlatform");
	default: return TEXT("Unknown");
	}
}