// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "IO/IoHash.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "Serialization/ArchiveSavePackageData.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/UnversionedPropertySerialization.h"
#include "Templates/PimplPtr.h"
#include "Templates/UniquePtr.h"
#include "UObject/ArchiveCookContext.h"
#include "UObject/LinkerSave.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/SavePackage/SavePackageUtilities.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectThreadContext.h"

namespace Verse { struct VCell; }

/**
 * Wraps an object tagged as export along with some of its harvested settings
 */
struct FTaggedExport
{
	TObjectPtr<UObject> Obj;
	uint32 bNotAlwaysLoadedForEditorGame : 1;
	/**
	 * Indicate that this export should have a public hash even if it isn't marked as RF_Public
	 * This will artificially mark the object RF_Public in the linker tables so the iostore generates the public hash
	 */
	uint32 bGeneratePublicHash : 1;
	/**
	 * Indicate if the object that directly referenced this export was optional
	 * Used to determine mandatory objects in the game save realm
	 */
	uint32 bFromOptionalReference : 1;

	FTaggedExport()
		: Obj(nullptr)
		, bNotAlwaysLoadedForEditorGame(false)
		, bGeneratePublicHash(false)
		, bFromOptionalReference(false)
	{}

	FTaggedExport(TObjectPtr<UObject> InObj, bool bInNotAlwaysLoadedForEditorGame = true, bool bInFromOptionalReference = false)
		: Obj(InObj)
		, bNotAlwaysLoadedForEditorGame(bInNotAlwaysLoadedForEditorGame)
		, bGeneratePublicHash(false)
		, bFromOptionalReference(bInFromOptionalReference)
	{}

	inline bool operator == (const FTaggedExport& Other) const
	{
		return Obj == Other.Obj;
	}
};

inline uint32 GetTypeHash(const FTaggedExport& Export)
{
	return GetTypeHash(Export.Obj);
}

/** 
 * Available save realm during save package harvesting 
 * A realm is the set of objects gathered and referenced for a particular domain/context
 */
enum class ESaveRealm : uint32
{
	Game		= 0,
	Optional,
	Editor,
	RealmCount,
	None		= RealmCount
};

/** Reason for harvested illegal reference */
enum class EIllegalRefReason : uint8
{
	None = 0,
	ReferenceToOptional,
	ReferenceFromOptionalToMissingGameExport,
	UnsaveableClass,
	UnsaveableOuter,
	ExternalPackage,
};

/** Small struct to store illegal references harvested during save */
struct FIllegalReference
{
	UObject* From = nullptr;
	UObject* To = nullptr;
	EIllegalRefReason Reason;
	FString FormatStringArg;
};

enum class ESaveableStatus
{
	Success,
	PendingKill,
	TransientFlag,
	TransientOverride,
	AbstractClass,
	DeprecatedClass,
	NewerVersionExistsClass,
	OuterUnsaveable,
	ClassUnsaveable,
	ExcludedByPlatform,
	__Count,
};

namespace UE::SavePackageUtilities
{

/** Information about the instigator that caused an object to be added as import or export to a SaveRealm. */
struct FRealmInstigator
{
	UObject* Object = nullptr;
	FProperty* Property = nullptr;
};

/** Calculated flags about saveable status and other state for UObjects encountered during save. */
struct FObjectStatus
{
	FObjectStatus()
		: bSaveOverrideForcedTransient(false)
		, bSaveableStatusValid(false), bEditorOnlyValid(false), bEditorOnly(false), bAttemptedExport(false)
		, bIsInSavePackageValid(false), bIsInSavePackage(false)
		, bDeclaredSaveBuildDependency(false)
	{
	}
	FObjectStatus(const FObjectStatus&) = default;
	FObjectStatus(FObjectStatus&&) = default;
	FObjectStatus& operator=(const FObjectStatus&) = default;
	FObjectStatus& operator=(FObjectStatus&&) = default;
	bool HasTransientFlag(const UObject* InObject)
	{
		check(InObject);
		return InObject->HasAnyFlags(RF_Transient);
	}
	void ClearSaveableStatus()
	{
		SaveableStatus = ESaveableStatus::Success;
		bSaveableStatusValid = false;
		SaveableStatusCulprit = nullptr;
	}
	void SetIsInSavePackage(bool bInIsInSavePackage)
	{
		bIsInSavePackageValid = true;
		bIsInSavePackage = bInIsInSavePackage;
	}
	TOptional<bool> IsInSavePackage() const
	{
		return bIsInSavePackageValid ? TOptional<bool>(bIsInSavePackage) : TOptional<bool>();
	}
	bool IsInSavePackage(TObjectPtr<UObject> InObject, UPackage* SavePackage)
	{
		if (!bIsInSavePackageValid)
		{
			SetIsInSavePackage(InObject.GetPackage() == SavePackage);
		}
		return bIsInSavePackage;
	}

	FRealmInstigator RealmInstigator[(uint32)ESaveRealm::RealmCount];
	UObject* SaveableStatusCulprit = nullptr;
	ESaveableStatus SaveableStatus = ESaveableStatus::Success;
	ESaveableStatus SaveableStatusCulpritStatus = ESaveableStatus::Success;
	bool bSaveOverrideForcedTransient : 1;
	bool bSaveableStatusValid : 1;
	bool bEditorOnlyValid : 1;
	bool bEditorOnly : 1;
	bool bAttemptedExport : 1;
	bool bIsInSavePackageValid : 1;
	bool bIsInSavePackage : 1;
	bool bDeclaredSaveBuildDependency : 1;
};

}
/** Hold the harvested exports and imports for a realm */
struct FHarvestedRealm
{

	~FHarvestedRealm()
	{
		CloseLinkerArchives();

		if (TempFilename.IsSet())
		{
			IFileManager::Get().Delete(*TempFilename.GetValue());
		}
		if (TextFormatTempFilename.IsSet())
		{
			IFileManager::Get().Delete(*TextFormatTempFilename.GetValue());
		}
	}

	void AddDirectImport(TObjectPtr<UObject> InObject)
	{
		DirectImports.Add(InObject);
	}

	void AddImport(TObjectPtr<UObject> InObject)
	{
		Imports.Add(InObject);
	}

	void AddExport(FTaggedExport InTagObj)
	{
		Exports.Add(MoveTemp(InTagObj));
	}

	void AddExcluded(TObjectPtr<UObject> InObject)
	{
		Excluded.Add(InObject);
	}

	void AddNotExcluded(TObjectPtr<UObject> InObject)
	{
		NotExcluded.Add(InObject);
	}

	bool IsImport(TObjectPtr<UObject> InObject) const
	{
		return Imports.Contains(InObject);
	}

	bool IsExport(TObjectPtr<UObject> InObject) const
	{
		return Exports.Contains(InObject);
	}

	bool IsIncluded(TObjectPtr<UObject> InObject) const
	{
		return IsImport(InObject) || IsExport(InObject);
	}

	/**
	 * Used during harvesting to early exit from objects we have found referenced earlier but excluded because of
	 * editoronly or unsaveable or otherwise.
	 */
	bool IsExcluded(TObjectPtr<UObject> InObject) const
	{
		return Excluded.Contains(InObject);
	}

	bool IsNotExcluded(TObjectPtr<UObject> InObject) const
	{
		return NotExcluded.Contains(InObject);
	}

	void AddCellImport(Verse::VCell* InCell)
	{
		CellImports.Add(InCell);
	}

	void AddCellExport(Verse::VCell* InCell)
	{
		CellExports.Add(InCell);
	}

	bool IsCellImport(Verse::VCell* InCell) const
	{
		return CellImports.Contains(InCell);
	}

	bool IsCellExport(Verse::VCell* InCell) const
	{
		return CellExports.Contains(InCell);
	}

	bool IsCellIncluded(Verse::VCell* InCell) const
	{
		return IsCellImport(InCell) || IsCellExport(InCell);
	}

	TSet<FTaggedExport>& GetExports()
	{
		return Exports;
	}

	const TSet<FTaggedExport>& GetExports() const
	{
		return Exports;
	}

	const TArray<UE::SavePackageUtilities::FPreloadDependency>& GetPreloadDependencies() const
	{
		return PreloadDependencies;
	}

	TArray<UE::SavePackageUtilities::FPreloadDependency> &GetPreloadDependencies()
	{
		return PreloadDependencies;
	}

	const TSet<TObjectPtr<UObject>>& GetDirectImports() const
	{
		return DirectImports;
	}

	const TSet<TObjectPtr<UObject>>& GetImports() const
	{
		return Imports;
	}

	const TSet<Verse::VCell*>& GetCellExports() const
	{
		return CellExports;
	}

	const TSet<Verse::VCell*>& GetCellImports() const
	{
		return CellImports;
	}

	const TSet<FName>& GetSoftPackageReferenceList() const
	{
		return SoftPackageReferenceList;
	}

	TSet<FName>& GetSoftPackageReferenceList()
	{
		return SoftPackageReferenceList;
	}

	const TSet<FName>& GetUntrackedSoftPackageReferenceList() const
	{
		return UntrackedSoftPackageReferenceList;
	}

	TSet<FName>& GetUntrackedSoftPackageReferenceList()
	{
		return UntrackedSoftPackageReferenceList;
	}

	const TMap<TObjectPtr<UObject>, TArray<FName>>& GetSearchableNamesObjectMap() const
	{
		return SearchableNamesObjectMap;
	}

	TMap<TObjectPtr<UObject>, TArray<FName>>& GetSearchableNamesObjectMap()
	{
		return SearchableNamesObjectMap;
	}

	const TSet<FNameEntryId>& GetNamesReferencedFromExportData() const
	{
		return NamesReferencedFromExportData;
	}

	TSet<FNameEntryId>& GetNamesReferencedFromExportData()
	{
		return NamesReferencedFromExportData;
	}

	const TSet<FNameEntryId>& GetNamesReferencedFromPackageHeader() const
	{
		return NamesReferencedFromPackageHeader;
	}

	TSet<FNameEntryId>& GetNamesReferencedFromPackageHeader()
	{
		return NamesReferencedFromPackageHeader;
	}

	const TSet<FSoftObjectPath>& GetSoftObjectPathList() const
	{
		return SoftObjectPathList;
	}

	TSet<FSoftObjectPath>& GetSoftObjectPathList() 
	{
		return SoftObjectPathList;
	}

	const TMap<TObjectPtr<UObject>, TSet<TObjectPtr<UObject>>>& GetObjectDependencies() const
	{
		return ExportObjectDependencies;
	}

	TMap<TObjectPtr<UObject>, TSet<TObjectPtr<UObject>>>& GetObjectDependencies()
	{
		return ExportObjectDependencies;
	}

	const TMap<TObjectPtr<UObject>, TSet<TObjectPtr<UObject>>>& GetNativeObjectDependencies() const
	{
		return ExportNativeObjectDependencies;
	}

	TMap<TObjectPtr<UObject>, TSet<TObjectPtr<UObject>>>& GetNativeObjectDependencies()
	{
		return ExportNativeObjectDependencies;
	}

	const TMap<TObjectPtr<UObject>, TSet<Verse::VCell*>>& GetCellDependencies() const
	{
		return ExportCellDependencies;
	}

	TMap<TObjectPtr<UObject>, TSet<Verse::VCell*>>& GetCellDependencies()
	{
		return ExportCellDependencies;
	}

	const TMap<Verse::VCell*, TSet<TObjectPtr<UObject>>>& GetCellObjectDependencies() const
	{
		return CellExportObjectDependencies;
	}

	TMap<Verse::VCell*, TSet<TObjectPtr<UObject>>>& GetCellObjectDependencies()
	{
		return CellExportObjectDependencies;
	}

	const TMap<Verse::VCell*, TSet<Verse::VCell*>>& GetCellCellDependencies() const
	{
		return CellExportCellDependencies;
	}

	TMap<Verse::VCell*, TSet<Verse::VCell*>>& GetCellCellDependencies()
	{
		return CellExportCellDependencies;
	}

	bool NameExists(const FName& Name) const
	{
		// Normally FName comparisons would be case insensitive and done using the comparisonIndex however 
		// NamesReferencedFromExportData and NamesReferencedFromPackageHeader contain DisplayIndices and the passed in 'Name' 
		// comes from memory (rather than disk) where we will not have case-sensitive descrepancies. Using the more restrictive 
		// case-sensitive search in this case is valid and faster.
		const FNameEntryId DisplayId = Name.GetDisplayIndex();
		return NamesReferencedFromExportData.Find(DisplayId) || NamesReferencedFromPackageHeader.Find(DisplayId);
	}

	FLinkerSave* GetLinker() const
	{
		return Linker.Get();
	}

	void SetLinker(TPimplPtr<FLinkerSave> InLinker)
	{
		Linker = MoveTemp(InLinker);
	}

	bool CloseLinkerArchives()
	{
		bool bSuccess = true;
		if (Linker)
		{
			bSuccess = Linker->CloseAndDestroySaver();
		}
		StructuredArchive.Reset();
		Formatter.Reset();
		TextFormatArchive.Reset();
		return bSuccess;
	}

	FArchive* GetTextFormatArchive() const
	{
		return TextFormatArchive.Get();
	}

	void SetTextFormatArchive(TUniquePtr<FArchive> InTextArchive)
	{
		TextFormatArchive = MoveTemp(InTextArchive);
	}

	FArchiveFormatterType* GetFormatter() const
	{
		return Formatter.Get();
	}

	void SetFormatter(TUniquePtr<FArchiveFormatterType> InFormatter)
	{
		Formatter = MoveTemp(InFormatter);
	}

	FStructuredArchive* GetStructuredArchive() const
	{
		return StructuredArchive.Get();
	}

	void SetStructuredArchive(TUniquePtr<FStructuredArchive> InArchive)
	{
		StructuredArchive = MoveTemp(InArchive);
	}

	const TOptional<FString>& GetTempFilename() const
	{
		return TempFilename;
	}

	void SetTempFilename(TOptional<FString> InTemp)
	{
		TempFilename = MoveTemp(InTemp);
	}

	const TOptional<FString>& GetTextFormatTempFilename() const
	{
		return TextFormatTempFilename;
	}

	void SetTextFormatTempFilename(TOptional<FString> InTemp)
	{
		TextFormatTempFilename = MoveTemp(InTemp);
	}

private:
	friend class FSaveContext;

	/** Linker associated with this realm. */
	TPimplPtr<FLinkerSave> Linker;

	/** Archives associated with this linker and realm. */
	TUniquePtr<FArchive> TextFormatArchive;
	TUniquePtr<FArchiveFormatterType> Formatter;
	TUniquePtr<FStructuredArchive> StructuredArchive;

	/** Temp Filename for the archive. */
	TOptional<FString> TempFilename;
	TOptional<FString> TextFormatTempFilename;

	// Set of objects excluded (import or exports) through marks or otherwise (i.e. transient flags, etc)
	TSet<TObjectPtr<UObject>> Excluded;
	// Set of objects not excluded through marks or otherwise (i.e. transient flags, etc) while not being marked specifically included yet
	TSet<TObjectPtr<UObject>> NotExcluded;
	// Set of objects marked as export
	TSet<FTaggedExport> Exports;
	// Set of objects marked as import
	TSet<TObjectPtr<UObject>> Imports;
	// Set of EDL/ZenLoader dependencies that were written into the package for runtime loading
	TArray<UE::SavePackageUtilities::FPreloadDependency> PreloadDependencies;
	// Set of objects that were referenced directly from an export. Some imports are transitively added by
	// FPackageHarvester::ProcessImport (Outer, Class, CDO, CDO subobjects, others?) for long-standing reasons
	// (performance, loading behavior, other?)/ But some features such as allowed-import access warnings need
	// to consider only the direct imports.
	TSet<TObjectPtr<UObject>> DirectImports;
	// Set of Verse cells marked as export
	TSet<Verse::VCell*> CellExports;
	// Set of Verse cells marked as import
	TSet<Verse::VCell*> CellImports;
	// Set of names referenced from export serialization
	TSet<FNameEntryId> NamesReferencedFromExportData;
	// Set of names referenced from the package header (import and export table object names etc)
	TSet<FNameEntryId> NamesReferencedFromPackageHeader;
	// Set of SoftObjectPath harvested in this realm
	TSet<FSoftObjectPath> SoftObjectPathList;
	// List of soft package reference found
	TSet<FName> SoftPackageReferenceList;
	// List of soft package reference not in the SoftPackageReferenceList because they are NeverCollect or EditorOnly.  
	// They are saved here anyway in case there is a redirection. The redirection will be saved in the build dependencies so
	// incremental cook can detect when the redirection changed.
	TSet<FName> UntrackedSoftPackageReferenceList;
	// Map of objects to their list of searchable names
	TMap<TObjectPtr<UObject>, TArray<FName>> SearchableNamesObjectMap;
	// Map of objects to their dependencies
	TMap<TObjectPtr<UObject>, TSet<TObjectPtr<UObject>>> ExportObjectDependencies;
	// Map of objects to their native dependencies
	TMap<TObjectPtr<UObject>, TSet<TObjectPtr<UObject>>> ExportNativeObjectDependencies;
	// Map of objects to their cell dependencies
	TMap<TObjectPtr<UObject>, TSet<Verse::VCell*>> ExportCellDependencies;
	// Map of cells to their object dependencies
	TMap<Verse::VCell*, TSet<TObjectPtr<UObject>>> CellExportObjectDependencies;
	// Map of cells to their cell dependencies
	TMap<Verse::VCell*, TSet<Verse::VCell*>> CellExportCellDependencies;
};


/**
 * Helper class that encapsulate the full necessary context and intermediate result to save a package
 */
class FSaveContext
{
public:
	struct FSetSaveRealmToSaveScope
	{
		FSetSaveRealmToSaveScope(FSaveContext& InContext, ESaveRealm InHarvestingRealm)
			: Context(InContext)
			, PreviousHarvestingRealm(InContext.CurrentHarvestingRealm)
		{
			Context.CurrentHarvestingRealm = InHarvestingRealm;
		}

		~FSetSaveRealmToSaveScope()
		{
			Context.CurrentHarvestingRealm = PreviousHarvestingRealm;
		}

	private:
		FSaveContext& Context;
		ESaveRealm PreviousHarvestingRealm;
	};

public:
	FSaveContext(UPackage* InPackage, UObject* InAsset, const TCHAR* InFilename, const FSavePackageArgs& InSaveArgs)
		: Package(InPackage)
		, Asset(InAsset)
		, Filename(InFilename)
		, SaveArgs(InSaveArgs)
		, PackageWriter(InSaveArgs.SavePackageContext ? InSaveArgs.SavePackageContext->PackageWriter : nullptr)
		, ObjectSavePackageSerializeContext(ObjectSaveContext)
		, ArchiveSavePackageData(ObjectSavePackageSerializeContext, nullptr, nullptr)
		, GameRealmExcludedObjectMarks(GetExcludedObjectMarksForGameRealm(SaveArgs.GetTargetPlatform()))
	{
		// Assumptions & checks
		check(InPackage);
		check(InFilename);
		// if we are cooking we should be doing it in the editor and with a PackageWriter
		check(!IsCooking() || WITH_EDITOR);
		checkf(!IsCooking() || PackageWriter, TEXT("Cook saves require an IPackageWriter"));

		// Store initial state
		InitialPackageFlags = Package->GetPackageFlags();

		SaveArgs.TopLevelFlags = UE::SavePackageUtilities::NormalizeTopLevelFlags(SaveArgs.TopLevelFlags, IsCooking());
		if (PackageWriter)
		{
			bIgnoreHeaderDiffs = SaveArgs.SavePackageContext->PackageWriterCapabilities.bIgnoreHeaderDiffs;
		}

		// if the asset wasn't provided, fetch it from the package
		if (Asset == nullptr)
		{
			Asset = InPackage->FindAssetInPackage();
		}

		TargetPackagePath = FPackagePath::FromLocalPath(InFilename);
		if (TargetPackagePath.GetHeaderExtension() == EPackageExtension::Unspecified)
		{
			TargetPackagePath.SetHeaderExtension(EPackageExtension::EmptyString);
		}

		bCanUseUnversionedPropertySerialization = CanUseUnversionedPropertySerialization(SaveArgs.GetTargetPlatform());
		bTextFormat = FString(Filename).EndsWith(FPackageName::GetTextAssetPackageExtension()) || FString(Filename).EndsWith(FPackageName::GetTextMapPackageExtension());
		static const IConsoleVariable* ProcessPrestreamingRequests = IConsoleManager::Get().FindConsoleVariable(TEXT("s.ProcessPrestreamingRequests"));
		if (ProcessPrestreamingRequests)
		{
			bIsProcessingPrestreamPackages = ProcessPrestreamingRequests->GetInt() > 0;
		}
		static const IConsoleVariable* FixupStandaloneFlags = IConsoleManager::Get().FindConsoleVariable(TEXT("save.FixupStandaloneFlags"));
		if (FixupStandaloneFlags)
		{
			bIsFixupStandaloneFlags = FixupStandaloneFlags->GetInt() != 0;
		}

		ObjectSaveContext.Set(InPackage, GetTargetPlatform(), TargetPackagePath, SaveArgs.SaveFlags);
		if (SaveArgs.ArchiveCookData)
		{
			ObjectSaveContext.CookType = SaveArgs.ArchiveCookData->CookContext.GetCookType();
			ObjectSaveContext.CookingDLC = SaveArgs.ArchiveCookData->CookContext.GetCookingDLC();
#if WITH_EDITOR
			ObjectSaveContext.CookInfo = SaveArgs.ArchiveCookData->CookContext.GetCookInfo();
#endif
		}
		ArchiveSavePackageData.TargetPlatform = ObjectSaveContext.TargetPlatform;
		ArchiveSavePackageData.CookContext = SaveArgs.ArchiveCookData ? &SaveArgs.ArchiveCookData->CookContext : nullptr;
		if (SaveArgs.InOutSaveOverrides)
		{
			ObjectSaveContext.SaveOverrides = MoveTemp(*SaveArgs.InOutSaveOverrides);
		}
		ObjectSaveContext.PackageWriter = PackageWriter;
		if (PackageWriter)
		{
			ObjectSaveContext.bDeterminismDebug = SaveArgs.SavePackageContext->PackageWriterCapabilities.bDeterminismDebug;
		}

		// Setup the harvesting flags and generate the context for harvesting the package
		SetupHarvestingRealms();
	} 

	~FSaveContext()
	{
		if (bPostSaveRootRequired && Asset)
		{
			UE::SavePackageUtilities::CallPostSaveRoot(Asset, ObjectSaveContext, bNeedPreSaveCleanup);
		}

		// Move the SaveOverrides that we copied and/or modified onto our ObjectSaveContext back to the
		// InOut SaveOverrides parameter on the SaveArgs.
		if (SaveArgs.InOutSaveOverrides)
		{
			*SaveArgs.InOutSaveOverrides = MoveTemp(ObjectSaveContext.SaveOverrides);
		}
	}

	uint32 GetInitialPackageFlags() const
	{
		return InitialPackageFlags;
	}

	const FSavePackageArgs& GetSaveArgs() const
	{
		return SaveArgs;
	}

	FArchiveSavePackageData& GetArchiveSavePackageData()
	{
		return ArchiveSavePackageData;
	}

	const ITargetPlatform* GetTargetPlatform() const
	{
		return SaveArgs.GetTargetPlatform();
	}

	UPackage* GetPackage() const
	{
		return Package;
	}

	UObject* GetAsset() const
	{
		return Asset;
	}

	const TCHAR* GetFilename() const
	{
		return Filename;
	}

	const FPackagePath& GetTargetPackagePath() const
	{
		return TargetPackagePath;
	}

	EObjectMark GetExcludedObjectMarks(ESaveRealm HarvestingRealm) const
	{
		switch (HarvestingRealm)
		{
		case ESaveRealm::Optional:
			// When considering excluded objects for a platform, do not consider editor only object, not for platform, not for client and not for server objects in the optional context as excluded
			return (EObjectMark)(GameRealmExcludedObjectMarks & ~(EObjectMark::OBJECTMARK_EditorOnly | EObjectMark::OBJECTMARK_NotForTargetPlatform | EObjectMark::OBJECTMARK_NotForClient | EObjectMark::OBJECTMARK_NotForServer));
		case ESaveRealm::Game:
			return GameRealmExcludedObjectMarks;
		case ESaveRealm::Editor:
			return EObjectMark::OBJECTMARK_NOMARKS;
		default:
			checkNoEntry();
			return EObjectMark::OBJECTMARK_NOMARKS;
		}
	}

	EObjectFlags GetTopLevelFlags() const
	{
		return SaveArgs.TopLevelFlags;
	}

	bool IsUsingSlowTask() const
	{
		return SaveArgs.bSlowTask;
	}

	FOutputDevice* GetError() const
	{
		return SaveArgs.Error;
	}

	const FDateTime& GetFinalTimestamp() const
	{
		return SaveArgs.FinalTimeStamp;
	}

	FSavePackageContext* GetSavePackageContext() const
	{
		return SaveArgs.SavePackageContext;
	}

	bool IsCooking() const
	{
		return SaveArgs.IsCooking();
	}

	bool IsProceduralSave() const
	{
		return ObjectSaveContext.bProceduralSave;
	}

	bool IsUpdatingLoadedPath() const
	{
		return ObjectSaveContext.bUpdatingLoadedPath;
	}

	bool IsFilterEditorOnly() const
	{
		return Package->HasAnyPackageFlags(PKG_FilterEditorOnly);
	}

	bool IsStripEditorOnly() const
	{
		return !(SaveArgs.SaveFlags & ESaveFlags::SAVE_KeepEditorOnlyCookedPackages);
	}

	bool IsForceByteSwapping() const
	{
		return SaveArgs.bForceByteSwapping;
	}

	bool IsWarningLongFilename() const
	{
		return SaveArgs.bWarnOfLongFilename;
	}

	bool IsTextFormat() const
	{
		return bTextFormat;
	}

	bool IsFromAutoSave() const
	{
		return !!(SaveArgs.SaveFlags & SAVE_FromAutosave);
	}

	bool IsSaveToMemory() const
	{
		return !!(SaveArgs.SaveFlags & SAVE_Async) || PackageWriter;
	}

	bool IsGenerateSaveError() const
	{
		return !(SaveArgs.SaveFlags & SAVE_NoError);
	}

	bool IsKeepDirty() const
	{
		return !!(SaveArgs.SaveFlags & SAVE_KeepDirty);
	}

	bool IsSaveUnversionedNative() const
	{
		return !!(SaveArgs.SaveFlags & SAVE_Unversioned_Native);
	}

	bool IsSaveUnversionedProperties() const
	{
		return !!(SaveArgs.SaveFlags & SAVE_Unversioned_Properties) && bCanUseUnversionedPropertySerialization;
	}

	bool IsSaveOptional() const
	{
		return !!(SaveArgs.SaveFlags & SAVE_Optional);
	}

	bool IsSaveAutoOptional() const
	{
		return bIsSaveAutoOptional;
	}

	bool IsConcurrent() const
	{
		return !!(SaveArgs.SaveFlags & SAVE_Concurrent);
	}

	bool IsIgnoringHeaderDiff() const
	{
		return bIgnoreHeaderDiffs;
	}

	bool IsProcessingPrestreamingRequests() const
	{
		return bIsProcessingPrestreamPackages;
	}

	bool IsFixupStandaloneFlags() const
	{
		return bIsFixupStandaloneFlags;
	}

	bool ShouldRehydratePayloads() const
	{
		return (SaveArgs.SaveFlags & ESaveFlags::SAVE_RehydratePayloads) != 0;
	}

	bool ShouldKeepPersistentGUID() const
	{
		return IsProceduralSave() || IsFromAutoSave() || (SaveArgs.SaveFlags & ESaveFlags::SAVE_KeepPersistentGUID) != 0;
	}

	bool GetPostSaveRootRequired() const
	{
		return bPostSaveRootRequired;
	}

	void SetPostSaveRootRequired(bool bInPostSaveRootRequired)
	{
		bPostSaveRootRequired = bInPostSaveRootRequired;
	}

	bool GetPreSaveCleanup() const
	{
		return bNeedPreSaveCleanup;
	}

	void SetPreSaveCleanup(bool bInNeedPreSaveCleanup)
	{
		bNeedPreSaveCleanup = bInNeedPreSaveCleanup;
	}

	bool IsStubRequested() const
	{
		return bGenerateFileStub;
	}

	void RequestStubFile()
	{
		bGenerateFileStub = true;
	}

	ESaveRealm GetCurrentHarvestingRealm() const
	{
		return CurrentHarvestingRealm;
	}

	/** Returns which save context should be saved. */
	TArray<ESaveRealm> GetHarvestedRealmsToSave();

	/**
	 * Resolve a TObjectPtr into a raw UObject* for use during SavePackage. If the object is an import,
	 * this function avoids the normal automatic behavior of marking the object's package as a generic build
	 * dependency, and instead manually records a dependency on the PackageAndClass cook result projection,
	 * which are the only dependencies necessary for generic saving of imports.
	 */
	UObject* ResolveForSave(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& Status);

	bool IsTransient(TObjectPtr<UObject> InObject);
	bool IsTransient(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	bool IsUnsaveable(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus,
		bool bEmitWarning = true);
	void UpdateSaveableStatus(TObjectPtr<UObject> InObject, UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	UE::SavePackageUtilities::FObjectStatus& GetCachedObjectStatus(TObjectPtr<UObject> InObject)
	{
		using namespace UE::SavePackageUtilities;
		TUniquePtr<FObjectStatus>& ObjectStatus = ObjectStatusCache.FindOrAdd(InObject);
		if (!ObjectStatus)
		{
			ObjectStatus = MakeUnique<FObjectStatus>();
		}
		return *ObjectStatus;
	}

	UE::SavePackageUtilities::FObjectStatus* FindCachedObjectStatus(TObjectPtr<const UObject> InObject) const
	{
		using namespace UE::SavePackageUtilities;
		if (const TUniquePtr<FObjectStatus>* ObjectStatus = ObjectStatusCache.Find(InObject))
		{
			return ObjectStatus->Get();
		}
		return nullptr;
	}

	void RecordIllegalReference(UObject* InFrom, UObject* InTo, EIllegalRefReason InReason, FString&& InOptionalReasonText = FString())
	{
		HarvestedIllegalReferences.Add({ InFrom, InTo, InReason, MoveTemp(InOptionalReasonText) });
	}

	const TArray<FIllegalReference>& GetIllegalReferences() const
	{
		return HarvestedIllegalReferences;
	}
	
	void AddImport(UObject* InObject)
	{
		GetHarvestedRealm().AddImport(InObject);
	}

	void AddDirectImport(UObject* InObject)
	{
		GetHarvestedRealm().AddDirectImport(InObject);
	}

	void AddExport(FTaggedExport InTagObj)
	{
		GetHarvestedRealm().AddExport(MoveTemp(InTagObj));
	}

	void AddExcluded(UObject* InObject)
	{
		GetHarvestedRealm().AddExcluded(InObject);
	}

	bool IsImport(UObject* InObject) const
	{
		return GetHarvestedRealm().IsImport(InObject);
	}

	bool IsExport(UObject* InObject) const
	{
		return GetHarvestedRealm().IsExport(InObject);
	}

	bool IsIncluded(TObjectPtr<UObject> InObject) const
	{
		return GetHarvestedRealm().IsIncluded(InObject);
	}

	TSet<FTaggedExport>& GetExports()
	{
		return GetHarvestedRealm().GetExports();
	}

	TArray<UE::SavePackageUtilities::FPreloadDependency>& GetPreloadDependencies()
	{
		return GetHarvestedRealm().GetPreloadDependencies();
	}

	const TSet<TObjectPtr<UObject>>& GetImports() const
	{
		return GetHarvestedRealm().GetImports();
	}

	const TSet<TObjectPtr<UObject>>& GetDirectImports() const
	{
		return GetHarvestedRealm().GetDirectImports();
	}

	const TSet<TObjectPtr<UObject>>& GetImportsUsedInGame() const
	{
		return GetHarvestedRealm(ESaveRealm::Game).GetImports();
	}

	const TSet<Verse::VCell*>& GetCellExports() const
	{
		return GetHarvestedRealm().GetCellExports();
	}

	const TSet<Verse::VCell*>& GetCellImports() const
	{
		return GetHarvestedRealm().GetCellImports();
	}

	const TSet<FName>& GetSoftPackageReferenceList() const
	{
		return GetHarvestedRealm().GetSoftPackageReferenceList();
	}

	TSet<FName>& GetSoftPackageReferenceList()
	{
		return GetHarvestedRealm().GetSoftPackageReferenceList();
	}

	const TSet<FName>& GetSoftPackagesUsedInGame() const
	{
		return GetHarvestedRealm(ESaveRealm::Game).GetSoftPackageReferenceList();
	}

	TSet<FName>& GetSoftPackagesUsedInGame()
	{
		return GetHarvestedRealm(ESaveRealm::Game).GetSoftPackageReferenceList();
	}

	const TSet<FName>& GetUntrackedSoftPackageReferenceList() const
	{
		return GetHarvestedRealm().GetUntrackedSoftPackageReferenceList();
	}

	const TArray<FName>& GetPackageBuildDependencies(ESaveRealm SaveRealm = ESaveRealm::None)
	{
		SaveRealm = (SaveRealm == ESaveRealm::None) ? GetCurrentHarvestingRealm() : SaveRealm;
		if (SaveRealm == ESaveRealm::Editor)
		{
			return PackageBuildDependencies;
		}
		return EmptyList;
	}

	/**
	 * We copy manually-declared CookBuildDependencies from native class OnCookEvent functions into the AssetRegistry
	 * dependencies recorded for the editor save of a package. These build dependencies are used for the management
	 * database created by the AssetManager (UpdateManagementDatabase) that assigns chunks from PrimaryAssets out to
	 * all of the Assets that each PrimaryAsset depends on. We need to include CookBuildDependencies in that graph
	 * traversal because we can have some editoronly references that are read during cook to put data into the cooked
	 * version of the package that references them. We also use editoronly references to hide references that are used
	 * in game, but only on some platforms. To follow those references during the AssetManager's graph traversal, we
	 * need to report them in some other way than as used-in-game dependencies (because used-in-game dependencies
	 * would incorrectly mark them as used-in-game for all platforms). We implement that with build dependencies.
	 * See FMaterialOverrideNanite for an example of editor-only dependencies that are used-in-game on some platforms.
	*/
	void UpdateEditorRealmPackageBuildDependencies();

	void AddExportedClassesToDependencies();

	const TMap<TObjectPtr<UObject>, TArray<FName>>& GetSearchableNamesObjectMap() const
	{
		return GetHarvestedRealm().GetSearchableNamesObjectMap();
	}

	TMap<TObjectPtr<UObject>, TArray<FName>>& GetSearchableNamesObjectMap()
	{
		return GetHarvestedRealm().GetSearchableNamesObjectMap();
	}

	const TSet<FNameEntryId>& GetNamesReferencedFromExportData() const
	{
		return GetHarvestedRealm().GetNamesReferencedFromExportData();
	}

	const TSet<FNameEntryId>& GetNamesReferencedFromPackageHeader() const
	{
		return GetHarvestedRealm().GetNamesReferencedFromPackageHeader();
	}

	TSet<FNameEntryId>& GetNamesReferencedFromPackageHeader()
	{
		return GetHarvestedRealm().GetNamesReferencedFromPackageHeader();
	}

	const TSet<FSoftObjectPath>& GetSoftObjectPathList() const
	{
		return GetHarvestedRealm().GetSoftObjectPathList();
	}

	const TMap<TObjectPtr<UObject>, TSet<TObjectPtr<UObject>>>& GetObjectDependencies() const
	{
		return GetHarvestedRealm().GetObjectDependencies();
	}

	const TMap<TObjectPtr<UObject>, TSet<TObjectPtr<UObject>>>& GetNativeObjectDependencies() const
	{
		return GetHarvestedRealm().GetNativeObjectDependencies();
	}

	const TMap<TObjectPtr<UObject>, TSet<Verse::VCell*>>& GetCellDependencies() const
	{
		return GetHarvestedRealm().GetCellDependencies();
	}

	const TMap<Verse::VCell*, TSet<TObjectPtr<UObject>>>& GetCellObjectDependencies() const
	{
		return GetHarvestedRealm().GetCellObjectDependencies();
	}

	const TMap<Verse::VCell*, TSet<Verse::VCell*>>& GetCellCellDependeices() const
	{
		return GetHarvestedRealm().GetCellCellDependencies();
	}

	bool NameExists(const FName& Name) const
	{
		return GetHarvestedRealm().NameExists(Name);
	}

	const FCustomVersionContainer& GetCustomVersions() const
	{
		return CustomVersions;
	}

	const TSet<TObjectPtr<UPackage>>& GetPrestreamPackages() const
	{
		return PrestreamPackages;
	}

	TSet<TObjectPtr<UPackage>>& GetPrestreamPackages()
	{
		return PrestreamPackages;
	}

	bool IsPrestreamPackage(TObjectPtr<UPackage> InPackage) const
	{
		return PrestreamPackages.Contains(InPackage);
	}

	void AddPrestreamPackages(UPackage* InPackage)
	{
		PrestreamPackages.Add(InPackage);
	}

	void SetCustomVersions(FCustomVersionContainer InCustomVersions)
	{
		CustomVersions = MoveTemp(InCustomVersions);
	}

	TArray<FLinkerSave*> GetLinkers() const
	{
		TArray<FLinkerSave*> Linkers;
		for (const FHarvestedRealm& Realm : HarvestedRealms)
		{
			if (FLinkerSave* Linker = Realm.GetLinker())
			{
				Linkers.Add(Realm.GetLinker());
			}
		}
		return Linkers;
	}

	FLinkerSave* GetLinker() const
	{
		return GetHarvestedRealm().GetLinker();
	}

	void UpdatePackageLinkerVersions()
	{
		FLinkerSave* Linker = GetLinker();
		check(Linker);
		Package->SetLinkerPackageVersion(Linker->UEVer());
		Package->SetLinkerLicenseeVersion(Linker->LicenseeUEVer());
		Package->SetLinkerCustomVersions(Linker->GetCustomVersions());
	}

	void UpdatePackageFileSize(int64 InFileSize)
	{
		Package->SetFileSize(InFileSize);
	}

	void SetLinker(TPimplPtr<FLinkerSave> InLinker)
	{
		GetHarvestedRealm().SetLinker(MoveTemp(InLinker));
	}

	bool CloseLinkerArchives()
	{
		return GetHarvestedRealm().CloseLinkerArchives();
	}

	FArchive* GetTextFormatArchive() const
	{
		return GetHarvestedRealm().GetTextFormatArchive();
	}

	void SetTextFormatArchive(TUniquePtr<FArchive> InTextArchive)
	{
		GetHarvestedRealm().SetTextFormatArchive(MoveTemp(InTextArchive));
	}

	FArchiveFormatterType* GetFormatter() const
	{
		return GetHarvestedRealm().GetFormatter();
	}

	void SetFormatter(TUniquePtr<FArchiveFormatterType> InFormatter)
	{
		GetHarvestedRealm().SetFormatter(MoveTemp(InFormatter));
	}

	FStructuredArchive* GetStructuredArchive() const
	{
		return GetHarvestedRealm().GetStructuredArchive();
	}

	void SetStructuredArchive(TUniquePtr<FStructuredArchive> InArchive)
	{
		GetHarvestedRealm().SetStructuredArchive(MoveTemp(InArchive));
	}

	const TOptional<FString>& GetTempFilename() const
	{
		return GetHarvestedRealm().GetTempFilename();
	}

	void SetTempFilename(TOptional<FString> InTemp)
	{
		GetHarvestedRealm().SetTempFilename(MoveTemp(InTemp));
	}

	const TOptional<FString>& GetTextFormatTempFilename() const
	{
		return GetHarvestedRealm().GetTextFormatTempFilename();
	}

	void SetTextFormatTempFilename(TOptional<FString> InTemp)
	{
		GetHarvestedRealm().SetTextFormatTempFilename(MoveTemp(InTemp));
	}

	FSavePackageResultStruct GetFinalResult();

	FObjectSaveContextData& GetObjectSaveContext()
	{
		return ObjectSaveContext;
	}

	IPackageWriter* GetPackageWriter() const
	{
		return PackageWriter;
	}

	bool HasExternalImportValidations() const
	{
		return SaveArgs.SavePackageContext ? SaveArgs.SavePackageContext->GetExternalImportValidations().Num() > 0 : false;
	}

	const TArray<TFunction<FSavePackageSettings::ExternalImportValidationFunc>>& GetExternalImportValidations() const
	{
		check(SaveArgs.SavePackageContext);
		return SaveArgs.SavePackageContext->GetExternalImportValidations();
	}

	bool HasExternalExportValidations() const
	{
		return SaveArgs.SavePackageContext ? SaveArgs.SavePackageContext->GetExternalExportValidations().Num() > 0 : false;
	}

	const TArray<TFunction<FSavePackageSettings::ExternalExportValidationFunc>>& GetExternalExportValidations() const
	{
		check(SaveArgs.SavePackageContext);
		return SaveArgs.SavePackageContext->GetExternalExportValidations();
	}

	const FHarvestedRealm& GetHarvestedRealm(ESaveRealm Realm = ESaveRealm::None) const
	{
		return HarvestedRealms[(uint32)(Realm == ESaveRealm::None ? CurrentHarvestingRealm : Realm)];
	}
	FHarvestedRealm& GetHarvestedRealm(ESaveRealm Realm = ESaveRealm::None)
	{
		return HarvestedRealms[(uint32)(Realm == ESaveRealm::None ? CurrentHarvestingRealm : Realm)];
	}

	TArray<FAssetData>& GetSavedAssets()
	{
		return SavedAssets;
	}

	const TMap<UObject*, TSet<FProperty*>>& GetTransientPropertyOverrides()
	{
		return TransientPropertyOverrides;
	}

	void SetTransientPropertyOverrides(TMap<UObject*, TSet<FProperty*>>&& InTransientPropertyOverrides)
	{
		TransientPropertyOverrides = MoveTemp(InTransientPropertyOverrides);
	}

	void ClearSaveableCache()
	{
		for (TPair<TObjectPtr<UObject>, TUniquePtr<UE::SavePackageUtilities::FObjectStatus>>& Pair : ObjectStatusCache)
		{
			Pair.Value->ClearSaveableStatus();
		}
	}

#if WITH_EDITORONLY_DATA
	auto GetFunctorReadCachedEditorOnlyObject()
	{
		using namespace UE::SavePackageUtilities;
		return [this](const UObject* Obj)
			{
				FObjectStatus& Status = GetCachedObjectStatus(const_cast<UObject*>(Obj));
				return !Status.bEditorOnlyValid ? EEditorOnlyObjectResult::Uninitialized :
					Status.bEditorOnly ? EEditorOnlyObjectResult::EditorOnly :
					EEditorOnlyObjectResult::NonEditorOnly;
			};
	}
	auto GetFunctorWriteCachedEditorOnlyObject()
	{
		using namespace UE::SavePackageUtilities;
		return [this](const UObject* Obj, bool bEditorOnly)
			{
				FObjectStatus& Status = GetCachedObjectStatus(const_cast<UObject*>(Obj));
				Status.bEditorOnlyValid = true;
				Status.bEditorOnly = bEditorOnly;
			};
	}
#endif
	UE::SavePackageUtilities::EEditorOnlyObjectFlags GetEditorOnlyObjectFlags() const;

	void AddObjectToPostSaveSerialization(UObject* Object);

	const TSet<UObject*>& GetPostSaveObjectsToSerialize() const;

public:
	ESavePackageResult Result;

	EPropertyLocalizationGathererResultFlags GatherableTextResultFlags = EPropertyLocalizationGathererResultFlags::Empty;

	//@note FH: Most of these public members should be moved to harvested realm class 
	FIoHash PackageSavedHash;
	int64 PackageHeaderAndExportSize = 0;
	int64 TotalPackageSizeUncompressed = 0;
	int32 OffsetAfterPackageFileSummary = 0;
	int32 OffsetAfterImportMap = 0;
	int32 OffsetAfterExportMap = 0;
	int64 OffsetAfterPayloadToc = 0;
	int32 SerializedPackageFlags = 0;
	TArray<FLargeMemoryWriter, TInlineAllocator<4>> AdditionalFilesFromExports;
	FSavePackageOutputFileArray AdditionalPackageFiles;
private:

	// Create the harvesting contexts and automatic optional context gathering options
	void SetupHarvestingRealms();
	ESaveableStatus GetSaveableStatusNoOuter(TObjectPtr<UObject> Obj,
		UE::SavePackageUtilities::FObjectStatus& ObjectStatus);
	static EObjectMark GetExcludedObjectMarksForGameRealm(const ITargetPlatform* TargetPlatform);

	friend class FPackageHarvester;

	// Args
	UPackage* Package;
	UObject* Asset;
	FPackagePath TargetPackagePath;
	const TCHAR* Filename;
	FSavePackageArgs SaveArgs;
	IPackageWriter* PackageWriter;

	// State context
	FObjectSaveContextData ObjectSaveContext;
	FObjectSavePackageSerializeContext ObjectSavePackageSerializeContext;
	FArchiveSavePackageData ArchiveSavePackageData;
	bool bCanUseUnversionedPropertySerialization = false;
	bool bTextFormat = false;
	bool bIsProcessingPrestreamPackages = false;
	bool bIsFixupStandaloneFlags = false;
	bool bPostSaveRootRequired = false;
	bool bNeedPreSaveCleanup = false;
	bool bGenerateFileStub = false;
	bool bIgnoreHeaderDiffs = false;
	bool bIsSaveAutoOptional = false;

	// Mutated package state
	uint32 InitialPackageFlags;

	// Config classes shared with the old Save
	FCanSkipEditorReferencedPackagesWhenCooking SkipEditorRefCookingSetting;

	// An object matching any GameRealmExcludedObjectMarks should be excluded from imports or exports in the game realm
	const EObjectMark GameRealmExcludedObjectMarks;

	// Harvested custom versions
	FCustomVersionContainer CustomVersions;

	// The current default harvesting context being queried by the save context
	ESaveRealm CurrentHarvestingRealm = ESaveRealm::None;

	// List of harvested content split per harvesting context
	TArray<FHarvestedRealm> HarvestedRealms;

	// List of harvested illegal references
	TArray<FIllegalReference> HarvestedIllegalReferences;

	// Set of harvested prestream packages, should be deprecated
	TSet<TObjectPtr<UPackage>> PrestreamPackages;

	// Set of AssetDatas created for the Assets saved into the package
	TArray<FAssetData> SavedAssets;

	// Overridden properties for each export that should be treated as transient, and nulled out when serializing
	TMap<UObject*, TSet<FProperty*>> TransientPropertyOverrides;

	// Cache of FObjectStatus for every object encountered during the save
	TMap<TObjectPtr<UObject>, TUniquePtr<UE::SavePackageUtilities::FObjectStatus>> ObjectStatusCache;

	// List of package build dependencies reported from PreSave or Serialize functions
	TArray<FName> PackageBuildDependencies;
	// Empty list of FNames, used for functions that need to return a reference to an empty array.
	TArray<FName> EmptyList;

	// List of objects to serialize again during the post save phase.
	TSet<UObject*> PostSaveObjectsToSerialize;
};

const TCHAR* LexToString(ESaveableStatus Status);
