// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/SavePackage/SavePackageUtilities.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/CookTagList.h"
#include "Blueprint/BlueprintSupport.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookEvents.h"
#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Hash/Blake3.h"
#include "Interfaces/ITargetPlatform.h"
#include "IO/IoHash.h"
#include "Logging/StructuredLog.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopedSlowTask.h"
#include "SaveContext.h"
#include "Serialization/ArchiveSavePackageDataBuffer.h"
#include "Serialization/BulkData.h"
#include "Serialization/EditorBulkData.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/FileRegionArchive.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/PackageWriter.h"
#include "Tasks/Task.h"
#include "UObject/ArchiveCookContext.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Class.h"
#include "UObject/GCScopeLock.h"
#include "UObject/ImportExportCollector.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/LinkerSave.h"
#include "UObject/Object.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/UnrealType.h"

DEFINE_LOG_CATEGORY(LogSavePackage);
UE_TRACE_CHANNEL_DEFINE(SaveTimeChannel, "Adds extra cpu timing events that track serialization and package saving operations.");

#if ENABLE_COOK_STATS
#include "ProfilingDebugging/ScopedTimers.h"

int32 FSavePackageStats::NumPackagesSaved = 0;
double FSavePackageStats::SavePackageTimeSec = 0.0;
double FSavePackageStats::TagPackageExportsPresaveTimeSec = 0.0;
double FSavePackageStats::TagPackageExportsTimeSec = 0.0;
double FSavePackageStats::FullyLoadLoadersTimeSec = 0.0;
double FSavePackageStats::ResetLoadersTimeSec = 0.0;
double FSavePackageStats::TagPackageExportsGetObjectsWithOuter = 0.0;
double FSavePackageStats::TagPackageExportsGetObjectsWithMarks = 0.0;
double FSavePackageStats::SerializeImportsTimeSec = 0.0;
double FSavePackageStats::SortExportsSeekfreeInnerTimeSec = 0.0;
double FSavePackageStats::SerializeExportsTimeSec = 0.0;
double FSavePackageStats::SerializeBulkDataTimeSec = 0.0;
double FSavePackageStats::AsyncWriteTimeSec = 0.0;
double FSavePackageStats::MBWritten = 0.0;
TMap<FName, FArchiveDiffStats> FSavePackageStats::PackageDiffStats;
int32 FSavePackageStats::NumberOfDifferentPackages= 0;

FCookStatsManager::FAutoRegisterCallback FSavePackageStats::RegisterCookStats(FSavePackageStats::AddSavePackageStats);

void FSavePackageStats::AddSavePackageStats(FCookStatsManager::AddStatFuncRef AddStat)
{
	// Don't use FCookStatsManager::CreateKeyValueArray because there's just too many arguments. Don't need to overburden the compiler here.
	TArray<FCookStatsManager::StringKeyValue> StatsList;
	StatsList.Empty(15);
#define ADD_COOK_STAT(Name) StatsList.Emplace(TEXT(#Name), LexToString(Name))
	ADD_COOK_STAT(NumPackagesSaved);
	ADD_COOK_STAT(SavePackageTimeSec);
	ADD_COOK_STAT(TagPackageExportsPresaveTimeSec);
	ADD_COOK_STAT(TagPackageExportsTimeSec);
	ADD_COOK_STAT(FullyLoadLoadersTimeSec);
	ADD_COOK_STAT(ResetLoadersTimeSec);
	ADD_COOK_STAT(TagPackageExportsGetObjectsWithOuter);
	ADD_COOK_STAT(TagPackageExportsGetObjectsWithMarks);
	ADD_COOK_STAT(SerializeImportsTimeSec);
	ADD_COOK_STAT(SortExportsSeekfreeInnerTimeSec);
	ADD_COOK_STAT(SerializeExportsTimeSec);
	ADD_COOK_STAT(SerializeBulkDataTimeSec);
	ADD_COOK_STAT(AsyncWriteTimeSec);
	ADD_COOK_STAT(MBWritten);

	AddStat(TEXT("Package.Save"), StatsList);

	{
		PackageDiffStats.ValueSort([](const FArchiveDiffStats& Lhs, const FArchiveDiffStats& Rhs) { return Lhs.NewFileTotalSize > Rhs.NewFileTotalSize; });

		StatsList.Empty(15);
		for (const TPair<FName, FArchiveDiffStats>& Stat : PackageDiffStats)
		{
			StatsList.Emplace(Stat.Key.ToString(), LexToString((double)Stat.Value.NewFileTotalSize / 1024.0 / 1024.0));
		}

		AddStat(TEXT("Package.DifferentPackagesSizeMBPerAsset"), StatsList);
	}

	{
		PackageDiffStats.ValueSort([](const FArchiveDiffStats& Lhs, const FArchiveDiffStats& Rhs) { return Lhs.NumDiffs > Rhs.NumDiffs; });

		StatsList.Empty(15);
		for (const TPair<FName, FArchiveDiffStats>& Stat : PackageDiffStats)
		{
			StatsList.Emplace(Stat.Key.ToString(), LexToString(Stat.Value.NumDiffs));
		}

		AddStat(TEXT("Package.NumberOfDifferencesInPackagesPerAsset"), StatsList);
	}

	{
		PackageDiffStats.ValueSort([](const FArchiveDiffStats& Lhs, const FArchiveDiffStats& Rhs) { return Lhs.DiffSize > Rhs.DiffSize; });

		StatsList.Empty(15);
		for (const TPair<FName, FArchiveDiffStats>& Stat : PackageDiffStats)
		{
			StatsList.Emplace(Stat.Key.ToString(), LexToString((double)Stat.Value.DiffSize / 1024.0 / 1024.0));
		}

		AddStat(TEXT("Package.PackageDifferencesSizeMBPerAsset"), StatsList);
	}

	int64 NewFileTotalSize = 0;
	int64 NumDiffs = 0;
	int64 DiffSize = 0;
	for (const TPair<FName, FArchiveDiffStats>& PackageStat : PackageDiffStats)
	{
		NewFileTotalSize += PackageStat.Value.NewFileTotalSize;
		NumDiffs += PackageStat.Value.NumDiffs;
		DiffSize += PackageStat.Value.DiffSize;
	}

	const double DifferentPackagesSizeMB = (double)NewFileTotalSize / 1024.0 / 1024.0;
	const int64  NumberOfDifferencesInPackages = NumDiffs;
	const double PackageDifferencesSizeMB = (double)DiffSize / 1024.0 / 1024.0;

	StatsList.Empty(15);
	ADD_COOK_STAT(NumberOfDifferentPackages);
	ADD_COOK_STAT(DifferentPackagesSizeMB);
	ADD_COOK_STAT(NumberOfDifferencesInPackages);
	ADD_COOK_STAT(PackageDifferencesSizeMB);

	AddStat(TEXT("Package.DiffTotal"), StatsList);

#undef ADD_COOK_STAT		
	const FString TotalString = "Total";
}

void FSavePackageStats::MergeStats(const TMap<FName, FArchiveDiffStats>& ToMerge)
{
	for (const TPair<FName, FArchiveDiffStats>& Stat : ToMerge)
	{
		PackageDiffStats.FindOrAdd(Stat.Key).DiffSize += Stat.Value.DiffSize;
		PackageDiffStats.FindOrAdd(Stat.Key).NewFileTotalSize += Stat.Value.NewFileTotalSize;
		PackageDiffStats.FindOrAdd(Stat.Key).NumDiffs += Stat.Value.NumDiffs;
	}
};

#endif

static FThreadSafeCounter OutstandingAsyncWrites;


// Initialization for GIsSavingPackage
PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool GIsSavingPackage = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

// GIsSavingPackage is still accessible globally but is meant only for the game thread
// so provide another one for any other thread than GT.
thread_local bool GIsSavingPackageTLS = false;

namespace UE
{
	bool& GetIsSavingPackageRef()
	{
		if (IsInGameThread())
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return GIsSavingPackage;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		return GIsSavingPackageTLS;
	}
	
	bool IsSavingPackage()
	{
		return GetIsSavingPackageRef();
	}

	bool IsSavingPackage(UObject* InOuter)
	{
		if (InOuter == nullptr)
		{
			return IsSavingPackage();
		}
		return InOuter->GetPackage()->HasAnyPackageFlags(PKG_IsSaving);
	}

	void SetIsSavingPackage(bool bValue)
	{
		GetIsSavingPackageRef() = bValue;
	}
}

FSavePackageSettings& FSavePackageSettings::GetDefaultSettings()
{
	static FSavePackageSettings Default;
	return Default;
}

namespace UE::SavePackageUtilities
{

const FName NAME_World("World");
const FName NAME_Level("Level");
const FName NAME_PrestreamPackage("PrestreamPackage");

/**
* A utility that records the state of a package's files before we start moving and overwriting them. 
* This provides an easy way for us to restore the original state of the package incase of failures 
* while saving.
*/
class FPackageBackupUtility
{
public:
	FPackageBackupUtility(const FPackagePath& InPackagePath)
		: PackagePath(InPackagePath)
	{

	}

	/** 
	* Record a file that has been moved. These will need to be moved back to
	* restore the package.
	*/
	void RecordMovedFile( const FString& OriginalPath, const FString& NewLocation)
	{
		MovedOriginalFiles.Emplace(OriginalPath, NewLocation);
	}

	/** 
	* Record a newly created file that did not exist before. These will need
	* deleting to restore the package.
	*/
	void RecordNewFile(const FString& NewLocation)
	{
		NewFiles.Add(NewLocation);
	}

	/** Restores the package to it's original state */
	void RestorePackage()
	{
		IFileManager& FileSystem = IFileManager::Get();

		UE_LOGF(LogSavePackage, Verbose, "Restoring package '%ls'", *PackagePath.GetDebugName());

		// First we should delete any new file that has been saved for the package
		for (const FString& Entry : NewFiles)
		{
			if (!FileSystem.Delete(*Entry))
			{
				UE_LOGF(LogSavePackage, Error, "Failed to delete newly added file '%ls' when trying to restore the package state and the package could be unstable, please revert in revision control!", *Entry);
			}
		}

		// Now we can move back the original files
		for (const TPair<FString, FString>& Entry : MovedOriginalFiles)
		{
			if (!FileSystem.Move(*Entry.Key, *Entry.Value))
			{
				UE_LOGF(LogSavePackage, Error, "Failed to restore package '%ls', the file '%ls' is in an incorrect state and the package could be unstable, please revert in revision control!", *PackagePath.GetDebugName(), *Entry.Key);
			}
		}
	}

	/** Deletes the backed up files once they are no longer required. */
	void DiscardBackupFiles()
	{
		IFileManager& FileSystem = IFileManager::Get();

		// Note that we do not warn if we fail to delete a backup file as that is probably
		// the least of the users problems at the moment.
		for (const TPair<FString, FString>& Entry : MovedOriginalFiles)
		{
			FileSystem.Delete(*Entry.Value, /*RequireExists*/false, /*EvenReadOnly*/true);
		}
	}

private:
	const FPackagePath& PackagePath;

	TArray<FString> NewFiles;
	TArray<TPair<FString, FString>> MovedOriginalFiles;	
};

/**
 * Determines the set of object marks that should be excluded for the target platform
 *
 * @param TargetPlatform	The platform being saved for, or null for saving platform-agnostic version
 *
 * @return Excluded object marks specific for the particular target platform, objects with any of these marks will be rejected from the cook
 */
EObjectMark GetExcludedObjectMarksForTargetPlatform(const class ITargetPlatform* TargetPlatform)
{
	// we always want to exclude NotForTargetPlatform (in other words, later on, the target platform
	// can mark an object as NotForTargetPlatform, and then this will exlude that object and anything
	// inside it, from being saved out)
	EObjectMark ObjectMarks = OBJECTMARK_NotForTargetPlatform;

	if (TargetPlatform)
	{
		if (!TargetPlatform->AllowsEditorObjects())
		{
			ObjectMarks = (EObjectMark)(ObjectMarks | OBJECTMARK_EditorOnly);
		}

		const bool bIsServerOnly = TargetPlatform->IsServerOnly();
		const bool bIsClientOnly = TargetPlatform->IsClientOnly();

		if (bIsServerOnly)
		{
			ObjectMarks = (EObjectMark)(ObjectMarks | OBJECTMARK_NotForServer);
		}
		else if (bIsClientOnly)
		{
			ObjectMarks = (EObjectMark)(ObjectMarks | OBJECTMARK_NotForClient);
		}
	}

	return ObjectMarks;
}

/**
 * Find the most likely culprit that caused the objects in the passed in array to be considered illegal for saving.
 *
 * @param	BadObjects				Array of objects that are considered "bad" (e.g. non- RF_Public, in different map package, ...)
 * @param	SaveContext				SaveContext for the SavePackage call.
 * @param	HarvestedRealm				Which realm encountered the BadObjects.
 * @param	OutMostLikelyCulprit	UObject that is considered the most likely culprit causing the "bad" objects to be referenced or NULL
 * @param	OutReferencer			UObject referencing the most likely culprit
 * @param	OutReferencerProperty	Property (belonging to referencer) storing the offending reference
 * @param	OutIsCulpritArchetype	Is the most likely culprit an archetype object
 */
void FindMostLikelyCulprit(const TArray<UObject*>& BadObjects, FSaveContext& SaveContext,
	ESaveRealm HarvestedRealm, UObject*& OutMostLikelyCulprit, UObject*& OutReferencer,
	const FProperty*& OutReferencerProperty, bool& OutIsCulpritArchetype, FString& OutDiagnosticText)
{
	UObject* ArchetypeCulprit = nullptr;
	UObject* ReferencedCulprit = nullptr;
	const FProperty* CulpritReferencerProperty = nullptr;
	UObject* CulpritReferencer = nullptr;

	OutMostLikelyCulprit = nullptr;
	OutReferencer = nullptr;
	OutReferencerProperty = nullptr;
	OutIsCulpritArchetype = false;

	TStringBuilder<1024> DiagnosticText;
	DiagnosticText << TEXT("SavePackage Invalid Reference Diagnostics:");

	// Iterate over all objects that are marked as unserializable/ bad and print out their referencers.
	for (int32 BadObjIndex = 0; BadObjIndex < BadObjects.Num(); BadObjIndex++)
	{
		UObject* Obj = BadObjects[BadObjIndex];

		// SavePackage adds references to the class archetype manually; if this type is a class archetype and it is private, mark it as an error
		// for that reason rather than checking references. Class archetypes must be public since instances of their class in other packages can refer to them
		if (Obj->HasAnyFlags(RF_ArchetypeObject | RF_DefaultSubObject | RF_ClassDefaultObject))
		{
			DiagnosticText.Appendf(TEXT("\n\t%s is a private Archetype object"), *Obj->GetFullName());
			TArray<const TCHAR*> Flags;
			auto AddFlagIfPresent = [&Flags, Obj](EObjectFlags InFlag, const TCHAR* Descriptor)
			{
				if (Obj->HasAnyFlags(InFlag))
				{
					Flags.Add(Descriptor);
				}
			};
			AddFlagIfPresent(RF_ArchetypeObject, TEXT("RF_ArchetypeObject"));
			AddFlagIfPresent(RF_ClassDefaultObject, TEXT("RF_ClassDefaultObject"));
			AddFlagIfPresent(RF_DefaultSubObject, TEXT("RF_DefaultSubObject"));
			DiagnosticText.Appendf(TEXT("\n\t\tThis object is an archetype (flags include %s) but is private. This is a code error from the generator of the object. All archetype objects must be public."),
				*FString::Join(Flags, TEXT("|")));

			if (ArchetypeCulprit == nullptr)
			{
				ArchetypeCulprit = Obj;
			}
			continue;
		}

		FObjectStatus* ObjectStatus = SaveContext.FindCachedObjectStatus(Obj);
		FRealmInstigator* Instigator = ObjectStatus ? &ObjectStatus->RealmInstigator[(uint32)HarvestedRealm] : nullptr;
		DiagnosticText.Appendf(TEXT("\n\tInstigator of %s:"), *Obj->GetFullName());
		if (!Instigator || !Instigator->Object)
		{
			DiagnosticText << TEXT("\n\t\tObject: <unknown>");
			DiagnosticText << TEXT("\n\t\tProperty: <unknown>");
		}
		else
		{
			CulpritReferencer = Instigator->Object;
			CulpritReferencerProperty = Instigator->Property;
			DiagnosticText.Appendf(TEXT("\n\t\tObject: %s"), *Instigator->Object->GetFullName());
			DiagnosticText.Appendf(TEXT("\n\t\tProperty: %s"),
				Instigator->Property ? *Instigator->Property->GetPathName() : TEXT("<unknown>"));

			// Later ReferencedCulprits are higher priority than earlier culprits.
			// TODO: Not sure if this is an intentional behavior or if they choice was arbitrary.
			ReferencedCulprit = Obj;
		}
	}

	if (ArchetypeCulprit)
	{
		// ArchetypeCulprits are the most likely to be the problem; they are definitely a problem
		OutMostLikelyCulprit = ArchetypeCulprit;
		OutIsCulpritArchetype = true;
	}
	else
	{
		OutMostLikelyCulprit = ReferencedCulprit; // Might be null, in which case we didn't find one
		OutReferencer = CulpritReferencer;
		OutReferencerProperty = CulpritReferencerProperty;
	}

	if (OutMostLikelyCulprit == nullptr)
	{
		// Make sure we report something
		for (UObject* BadObject : BadObjects)
		{
			if (BadObject)
			{
				OutMostLikelyCulprit = BadObject;
				break;
			}
		}
	}
	OutDiagnosticText = DiagnosticText;
}

ESavePackageResult FinalizeTempOutputFiles(const FPackagePath& PackagePath, const FSavePackageOutputFileArray& OutputFiles, const FDateTime& FinalTimeStamp)
{
	UE_LOGF(LogSavePackage, Log,  "Moving output files for package: %ls", *PackagePath.GetDebugName());

	IFileManager& FileSystem = IFileManager::Get();
	FPackageBackupUtility OriginalPackageState(PackagePath);

	UE_LOGF(LogSavePackage, Verbose, "Moving existing files to the temp directory");

	TArray<bool, TInlineAllocator<4>> CanFileBeMoved;
	CanFileBeMoved.SetNum(OutputFiles.Num());

	// First check if any of the target files that already exist are read only still, if so we can fail the 
	// whole thing before we try to move any files
	for (int32 Index = 0; Index < OutputFiles.Num(); ++Index)
	{	
		const FSavePackageOutputFile& File = OutputFiles[Index];

		if (File.FileMemoryBuffer.IsValid())
		{
			ensureMsgf(false, TEXT("FinalizeTempOutputFiles does not handle async saving files! (%s)"), *PackagePath.GetDebugName());
			return ESavePackageResult::Error;
		}

		if (!File.TempFilePath.IsEmpty())
		{
			FFileStatData FileStats = FileSystem.GetStatData(*File.TargetPath);
			if (FileStats.bIsValid && FileStats.bIsReadOnly)
			{
				UE_LOGF(LogSavePackage, Error, "Cannot remove '%ls' as it is read only!", *File.TargetPath);
				return ESavePackageResult::Error;
			}
			CanFileBeMoved[Index] = FileStats.bIsValid;
		}
		else
		{
			CanFileBeMoved[Index] = false;
		}
	}

	// Now we need to move all of the files that we are going to overwrite (if any) so that we 
	// can restore them if anything goes wrong.
	for (int32 Index = 0; Index < OutputFiles.Num(); ++Index)
	{
		if (CanFileBeMoved[Index]) 
		{
			const FSavePackageOutputFile& File = OutputFiles[Index];

			const FString BaseFilename = FPaths::GetBaseFilename(File.TargetPath);
			const FString TempFilePath = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), *BaseFilename.Left(32));

			if (FileSystem.Move(*TempFilePath, *File.TargetPath))
			{
				OriginalPackageState.RecordMovedFile(File.TargetPath, TempFilePath);
			}
			else
			{
				UE_LOGF(LogSavePackage, Warning, "Failed to move '%ls' to temp directory", *File.TargetPath);
				OriginalPackageState.RestorePackage();

				return ESavePackageResult::Error;
			}
		}
	}

	// Now attempt to move the new files from the temp location to the final location
	for (const FSavePackageOutputFile& File : OutputFiles)
	{
		if (!File.TempFilePath.IsEmpty()) // Only try to move output files that were saved to temp files
		{
			UE_LOGF(LogSavePackage, Log, "Moving '%ls' to '%ls'", *File.TempFilePath, *File.TargetPath);

			if (FileSystem.Move(*File.TargetPath, *File.TempFilePath))
			{
				OriginalPackageState.RecordNewFile(File.TargetPath);
			}
			else
			{
				UE_LOGF(LogSavePackage, Warning, "Failed to move '%ls' from temp directory", *File.TargetPath);
				OriginalPackageState.RestorePackage();

				return ESavePackageResult::Error;
			}

			if (FinalTimeStamp != FDateTime::MinValue())
			{
				FileSystem.SetTimeStamp(*File.TargetPath, FinalTimeStamp);
			}
		}
	}

	// Finally we can clean up the temp files as we do not need to restore them (failure to delete them will not be considered an error)
	OriginalPackageState.DiscardBackupFiles();

	return ESavePackageResult::Success;
}

void WriteToFile(const FString& Filename, const uint8* InDataPtr, int64 InDataSize)
{
	IFileManager& FileManager = IFileManager::Get();

	for (int tries = 0; tries < 3; ++tries)
	{
		if (FArchive* Ar = FileManager.CreateFileWriter(*Filename))
		{
			Ar->Serialize(/* grrr */ const_cast<uint8*>(InDataPtr), InDataSize);
			bool bArchiveError = Ar->IsError();
			delete Ar;

			int64 ActualSize = FileManager.FileSize(*Filename);
			if (ActualSize != InDataSize)
			{
				FileManager.Delete(*Filename);

				UE_LOG(LogSavePackage, Fatal, TEXT("Could not save to %s! Tried to write %" INT64_FMT " bytes but resultant size was %" INT64_FMT ".%s"),
					*Filename, InDataSize, ActualSize, bArchiveError ? TEXT(" Ar->Serialize failed.") : TEXT(""));
			}
			return;
		}
	}

	UE_LOGF(LogSavePackage, Fatal, "Could not write to %ls!", *Filename);
}

void AsyncWriteFile(FLargeMemoryPtr Data, const int64 DataSize, const TCHAR* Filename, EAsyncWriteOptions Options, TArrayView<const FFileRegion> InFileRegions)
{
	OutstandingAsyncWrites.Increment();
	FString OutputFilename(Filename);

	UE::Tasks::Launch(TEXT("PackageAsyncFileWrite"), [Data = MoveTemp(Data), DataSize, OutputFilename = MoveTemp(OutputFilename), Options, FileRegions = TArray<FFileRegion>(InFileRegions)]() mutable
	{
		WriteToFile(OutputFilename, Data.Get(), DataSize);

		if (FileRegions.Num() > 0)
		{
			TArray<uint8> Memory;
			FMemoryWriter Ar(Memory);
			FFileRegion::SerializeFileRegions(Ar, FileRegions);

			WriteToFile(OutputFilename + FFileRegion::RegionsFileExtension, Memory.GetData(), Memory.Num());
		}

		OutstandingAsyncWrites.Decrement();
	});
}

void AsyncWriteFile(EAsyncWriteOptions Options, FSavePackageOutputFile& File)
{
	checkf(File.TempFilePath.IsEmpty(), TEXT("AsyncWriteFile does not handle temp files!"));
	AsyncWriteFile(FLargeMemoryPtr(File.FileMemoryBuffer.Release()), File.DataSize, *File.TargetPath, Options, File.FileRegions);
}

/** For a CDO get all of the subobjects templates nested inside it or it's class */
void GetCDOSubobjects(UObject* CDO, TArray<UObject*>& Subobjects)
{
	TArray<UObject*> CurrentSubobjects;
	TArray<UObject*> NextSubobjects;

	// Recursively search for subobjects. Only care about ones that have a full subobject chain as some nested objects are set wrong
	GetObjectsWithOuter(CDO->GetClass(), NextSubobjects, EGetObjectsFlags::None);
	GetObjectsWithOuter(CDO, NextSubobjects, EGetObjectsFlags::None);

	while (NextSubobjects.Num() > 0)
	{
		CurrentSubobjects = NextSubobjects;
		NextSubobjects.Empty();
		for (UObject* SubObj : CurrentSubobjects)
		{
			if (SubObj->HasAnyFlags(RF_DefaultSubObject | RF_ArchetypeObject))
			{
				Subobjects.Add(SubObj);
				GetObjectsWithOuter(SubObj, NextSubobjects, EGetObjectsFlags::None);
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
bool CanStripEditorOnlyImportsAndExports()
{
	// Configurable via ini setting
	static struct FCanStripEditorOnlyExportsAndImports
	{
		bool bCanStripEditorOnlyObjects;
		FCanStripEditorOnlyExportsAndImports()
			: bCanStripEditorOnlyObjects(true)
		{
			UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;
			GConfig->GetBool(TEXT("Core.System"), TEXT("CanStripEditorOnlyExportsAndImports"), bCanStripEditorOnlyObjects, GEngineIni);
		}
		FORCEINLINE operator bool() const { return bCanStripEditorOnlyObjects; }
	} CanStripEditorOnlyExportsAndImportsData;
	return CanStripEditorOnlyExportsAndImportsData;
}
#endif

bool IsUpdatingLoadedPath(bool bIsCooking, const FPackagePath& TargetPackagePath, uint32 SaveFlags)
{
#if WITH_EDITOR
	return !bIsCooking &&							// Do not update the loadedpath if we're cooking
		TargetPackagePath.IsMountedPath() &&		// Do not update the loadedpath if the new path is not a viable mounted path
		!(SaveFlags & SAVE_BulkDataByReference) &&	// Do not update the loadedpath if it's an EditorDomainSave. TODO: Change the name of this flag.
		!(SaveFlags & SAVE_FromAutosave);			// Do not update the loadedpath if it's an autosave.
#else
	return false; // Saving when not in editor never updates the LoadedPath
#endif
}

bool IsProceduralSave(bool bIsCooking, const FPackagePath& TargetPackagePath, uint32 SaveFlags)
{
#if WITH_EDITOR
	return bIsCooking ||							// Cooking is a procedural save
		(SaveFlags & SAVE_BulkDataByReference);		// EditorDomainSave is a procedural save. TODO: Change the name of this flag.
#else
	return false; // Saving when not in editor never has user changes
#endif
}

void CallPreSave(UObject* Object, FObjectSaveContextData& ObjectSaveContext)
{
	SCOPED_SAVETIMER_TEXT(*WriteToString<256>(GetClassTraceScope(Object), TEXTVIEW("_PreSave")));
	// Reset the scope so that untagged memory allocated during PreSave shows up as Untagged rather than the caller of SavePackage
	LLM_SCOPE(ELLMTag::Untagged);

	FObjectPreSaveContext ObjectPreSaveContext(ObjectSaveContext);
	ObjectSaveContext.bBaseClassCalled = false;
	ObjectSaveContext.NumRefPasses = 0;
	Object->PreSave(ObjectPreSaveContext);
	if (!ObjectSaveContext.bBaseClassCalled)
	{
		UE_LOGF(LogSavePackage, Warning, "Class %ls did not call Super::PreSave", *Object->GetClass()->GetName());
	}
	// When we deprecate PreSave, and need to take different actions based on the PreSave, remove this bAllowPreSave variable
	constexpr bool bAllowPreSave = true;
	if (!bAllowPreSave && ObjectSaveContext.NumRefPasses > 1)
	{
		UE_LOGF(LogSavePackage, Warning, "Class %ls overrides the deprecated PreSave function", *Object->GetClass()->GetName());
	}
}

#if WITH_EDITOR
void CallCookEventPlatformCookDependencies(UObject* Object, FObjectSaveContextData& ObjectSaveContext)
{
	SCOPED_SAVETIMER_TEXT(*WriteToString<256>(GetClassTraceScope(Object), TEXTVIEW("_PlatformCookDependencies")));

	using namespace UE::Cook;
	FCookEventContext CookEventContext(ObjectSaveContext);
	Object->OnCookEvent(ECookEvent::PlatformCookDependencies, CookEventContext);
}
#endif

void CallPreSaveRoot(UObject* Object, FObjectSaveContextData& ObjectSaveContext)
{
	SCOPED_SAVETIMER_TEXT(*WriteToString<256>(GetClassTraceScope(Object), TEXTVIEW("_PreSave")));

	ObjectSaveContext.bCleanupRequired = false;
	ObjectSaveContext.Object = Object;
	Object->PreSaveRoot(FObjectPreSaveRootContext(ObjectSaveContext));
}

void CallPostSaveRoot(UObject* Object, FObjectSaveContextData& ObjectSaveContext, bool bNeedsCleanup)
{
	SCOPED_SAVETIMER_TEXT(*WriteToString<256>(GetClassTraceScope(Object), TEXTVIEW("_PreSave")));

	ObjectSaveContext.Object = Object;
	ObjectSaveContext.bCleanupRequired = bNeedsCleanup;
	Object->PostSaveRoot(FObjectPostSaveRootContext(ObjectSaveContext));
}

EObjectFlags NormalizeTopLevelFlags(EObjectFlags TopLevelFlags, bool bIsCooking)
{
	// if we aren't cooking and top level flags aren't empty, add RF_HasExternalPackage to them to catch external packages data
	if (TopLevelFlags != RF_NoFlags && !bIsCooking)
	{
		TopLevelFlags |= RF_HasExternalPackage;
	}
	return TopLevelFlags;
}

void IncrementOutstandingAsyncWrites()
{
	OutstandingAsyncWrites.Increment();
}

void DecrementOutstandingAsyncWrites()
{
	OutstandingAsyncWrites.Decrement();
}

void ResetCookStats()
{
#if ENABLE_COOK_STATS
	FSavePackageStats::NumPackagesSaved = 0;
#endif
}

int32 GetNumPackagesSaved()
{
#if ENABLE_COOK_STATS
	return FSavePackageStats::NumPackagesSaved;
#else
	return 0;
#endif
}

#if WITH_EDITOR
FAddResaveOnDemandPackage OnAddResaveOnDemandPackage;
#endif

} // end namespace UE::SavePackageUtilities

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FObjectSaveContextData::FObjectSaveContextData() = default;
FObjectSaveContextData::~FObjectSaveContextData() = default;
FObjectSaveContextData::FObjectSaveContextData(const FObjectSaveContextData& Other) = default;
FObjectSaveContextData::FObjectSaveContextData(FObjectSaveContextData&& Other) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FObjectSaveContextData::FObjectSaveContextData(UPackage* Package, const ITargetPlatform* InTargetPlatform, const TCHAR* InTargetFilename, uint32 InSaveFlags)
{
	Set(Package, InTargetPlatform, InTargetFilename, InSaveFlags);
}

FObjectSaveContextData::FObjectSaveContextData(UPackage* Package, const ITargetPlatform* InTargetPlatform, const FPackagePath& TargetPath, uint32 InSaveFlags)
{
	Set(Package, InTargetPlatform, TargetPath, InSaveFlags);
}

#if WITH_EDITOR
namespace UE::SavePackageUtilities
{

void HarvestCookRuntimeDependencies(FObjectSaveContextData& Data, UObject* HarvestReferencesFrom)
{
	if (!HarvestReferencesFrom)
	{
		return;
	}
	if (!Data.TargetPlatform)
	{
		return;
	}

	UPackage* PackageBeingSaved = nullptr; // We don't have a pointer for this, so set it to null
	// Don't store the input Data on the ArchiveSavePackageData; we just want to harvest the serialized
	// FSoftObjectPaths, not allow direct writes to its cookdependencies or other data. We only set
	// ArchiveSavePackageData to provide access to the CookContext.
	FArchiveCookContext CookContext(PackageBeingSaved, Data.CookType, Data.CookingDLC, Data.TargetPlatform, Data.CookInfo);
	FArchiveSavePackageDataBuffer SavePackageData(CookContext);

	FImportExportCollector Collector(HarvestReferencesFrom->GetPackage());
	Collector.SetSavePackageData(&SavePackageData);
	Collector.SerializeObjectAndReferencedExports(HarvestReferencesFrom);
	for (const TPair<FName, ESoftObjectPathCollectType>& Pair : Collector.GetImportedPackages())
	{
		if (Pair.Value != ESoftObjectPathCollectType::AlwaysCollect)
		{
			continue;
		}
		FName PackageName = Pair.Key;
		if (FPackageName::IsScriptPackage(WriteToString<256>(PackageName)))
		{
			// Ignore native imports; we don't need to mark them for cooking
			continue;
		}
		FSoftObjectPath PackageSoftPath(FTopLevelAssetPath(PackageName, NAME_None));
		Data.CookRuntimeDependencies.Add(MoveTemp(PackageSoftPath));
	}
}

} // namespace UE::SavePackageUtilities

bool FObjectPreSaveContext::IsDeterminismDebug() const
{
	return Data.bDeterminismDebug;
}

void FObjectPreSaveContext::RegisterDeterminismHelper(
	const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)
{
	if (Data.PackageWriter)
	{
		Data.PackageWriter->RegisterDeterminismHelper(Data.Object, DeterminismHelper);
	}
}

bool FObjectSavePackageSerializeContext::IsHarvestingCookDependencies() const
{
	return (Data.ObjectSaveContextPhase == EObjectSaveContextPhase::Harvest)
		| (Data.ObjectSaveContextPhase == EObjectSaveContextPhase::CookDependencyHarvest);
}
void FObjectSavePackageSerializeContext::AddCookLoadDependency(UE::Cook::FCookDependency BuildDependency)
{
	if (!IsHarvestingCookDependencies())
	{
		UE_LOGF(LogSavePackage, Error,
			"AddCookLoadDependency called when !IsHarvestingCookDependencies(). This is invalid and will be ignored.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}
	Data.BuildResultDependencies.Add(UE::Cook::BuildResult::NAME_Load, MoveTemp(BuildDependency));
}
void FObjectSavePackageSerializeContext::AddCookSaveDependency(UE::Cook::FCookDependency BuildDependency)
{
	if (!IsHarvestingCookDependencies())
	{
		UE_LOGF(LogSavePackage, Error,
			"AddCookSaveDependency called when !IsHarvestingCookDependencies(). This is invalid and will be ignored.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}
	Data.BuildResultDependencies.Add(UE::Cook::BuildResult::NAME_Save, MoveTemp(BuildDependency));
}
void FObjectSavePackageSerializeContext::AddCookRuntimeDependency(FSoftObjectPath RuntimeDependency)
{
	if (!IsHarvestingCookDependencies())
	{
		UE_LOGF(LogSavePackage, Error,
			"AddCookRuntimeDependency called when !IsHarvestingCookDependencies(). This is invalid and will be ignored.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}
	Data.CookRuntimeDependencies.Add(MoveTemp(RuntimeDependency));
}
void FObjectSavePackageSerializeContext::HarvestCookRuntimeDependencies(UObject* HarvestReferencesFrom)
{
	if (GetPhase() != EObjectSaveContextPhase::Harvest)
	{
		UE_LOGF(LogSavePackage, Error,
			"HarvestCookRuntimeDependencies called when GetPhase() != EObjectSaveContextPhase::Harvest. This is invalid and will be ignored.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}
	UE::SavePackageUtilities::HarvestCookRuntimeDependencies(Data, HarvestReferencesFrom);
}

bool FObjectSavePackageSerializeContext::IsDeterminismDebug() const
{
	return Data.bDeterminismDebug;
}

void FObjectSavePackageSerializeContext::RegisterDeterminismHelper(
	const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper)
{
	if (GetPhase() != EObjectSaveContextPhase::Harvest)
	{
		UE_LOGF(LogSavePackage, Error,
			"RegisterDeterminismHelper called when GetPhase() != EObjectSaveContextPhase::Harvest. This is invalid and will be ignored.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Warning);
		return;
	}
	if (Data.PackageWriter)
	{
		Data.PackageWriter->RegisterDeterminismHelper(Data.Object, DeterminismHelper);
	}
}

void FObjectSavePackageSerializeContext::RequestPostSaveSerialization()
{
	if (GetPhase() != EObjectSaveContextPhase::Harvest || !IsCooking())
	{
		UE_LOGF(LogSavePackage, Error,
			"RequestPostSaveSerialization called when GetPhase() != EObjectSaveContextPhase::Harvest or !IsCooking(). This is invalid and will be ignored.");
		FDebug::DumpStackTraceToLog(ELogVerbosity::Error);
		return;
	}
	Data.bRequestPostSaveSerialization = true;
}

#endif

void FObjectSaveContextData::Set(UPackage* Package, const ITargetPlatform* InTargetPlatform, const TCHAR* InTargetFilename, uint32 InSaveFlags)
{
	FPackagePath PackagePath(FPackagePath::FromLocalPath(InTargetFilename));
	if (PackagePath.GetHeaderExtension() == EPackageExtension::Unspecified)
	{
		PackagePath.SetHeaderExtension(EPackageExtension::EmptyString);
	}
	Set(Package, InTargetPlatform, PackagePath, InSaveFlags);
}

void FObjectSaveContextData::Set(UPackage* Package, const ITargetPlatform* InTargetPlatform, const FPackagePath& TargetPath, uint32 InSaveFlags)
{
	TargetFilename = TargetPath.GetLocalFullPath();
	TargetPlatform = InTargetPlatform;
	SaveFlags = InSaveFlags;
	OriginalPackageFlags = Package ? Package->GetPackageFlags() : 0;
	bProceduralSave = UE::SavePackageUtilities::IsProceduralSave(InTargetPlatform != nullptr, TargetPath, InSaveFlags);
	bUpdatingLoadedPath = UE::SavePackageUtilities::IsUpdatingLoadedPath(InTargetPlatform != nullptr, TargetPath, InSaveFlags);
}

#if WITH_EDITORONLY_DATA
bool IsEditorOnlyObject(const UObject* InObject, bool bCheckRecursive)
{
	using namespace UE::SavePackageUtilities;
	EEditorOnlyObjectFlags Flags = EEditorOnlyObjectFlags::None;
	Flags |= bCheckRecursive ? EEditorOnlyObjectFlags::CheckRecursive : EEditorOnlyObjectFlags::None;
	return IsEditorOnlyObjectInternal(InObject, Flags);
}

bool IsEditorOnlyObject(const UObject* InObject, bool bCheckRecursive,
	TFunctionRef<UE::SavePackageUtilities::EEditorOnlyObjectResult(const UObject*)> LookupInCache,
	TFunctionRef<void(const UObject*, bool)> AddToCache)
{
	using namespace UE::SavePackageUtilities;
	EEditorOnlyObjectFlags Flags = EEditorOnlyObjectFlags::None;
	Flags |= bCheckRecursive ? EEditorOnlyObjectFlags::CheckRecursive : EEditorOnlyObjectFlags::None;
	return IsEditorOnlyObjectInternal(InObject, Flags, LookupInCache, AddToCache);
}

namespace UE::SavePackageUtilities
{

bool IsEditorOnlyObjectWithoutWritingCache(const UObject* InObject, EEditorOnlyObjectFlags Flags,
	TFunctionRef<EEditorOnlyObjectResult(const UObject*)> LookupInCache,
	TFunctionRef<void(const UObject*, bool)> AddToCache)
{
	check(InObject);

	bool bCheckRecursive = EnumHasAnyFlags(Flags, EEditorOnlyObjectFlags::CheckRecursive);
	bool bIgnoreEditorOnlyClass = EnumHasAnyFlags(Flags, EEditorOnlyObjectFlags::ApplyHasNonEditorOnlyReferences) &&
		InObject->HasNonEditorOnlyReferences();

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("IsEditorOnlyObject"), STAT_IsEditorOnlyObject, STATGROUP_LoadTime);

	// CDOs must be included if their class and archetype and outer are included.
	// Ignore their value of IsEditorOnly
	if (!InObject->HasAnyFlags(RF_ClassDefaultObject))
	{
		if (!bIgnoreEditorOnlyClass && InObject->IsEditorOnly())
		{
			return true;
		}
	}

	// If this is a package that is editor only or the object is in editor-only package,
	// the object is editor-only too.
	const bool bIsAPackage = InObject->IsA<UPackage>();
	const UPackage* Package;
	if (bIsAPackage)
	{
		if (InObject->HasAnyFlags(RF_ClassDefaultObject))
		{
			// The default package is not editor-only, and it is part of a cycle that would cause infinite recursion:
			// DefaultPackage -> GetOuter() -> Package:/Script/CoreUObject -> GetArchetype() -> DefaultPackage
			return false;
		}
		Package = static_cast<const UPackage*>(InObject);
	}
	else
	{
		// In the case that the object is an external object, we want to use its host package, rather than its
		// external package when testing editor-only. All external packages are editor-only, but the objects in
		// the external package logically belong to the host package, and are editoronly if and only if the host is.
		// So use GetOutermostObject()->GetPackage(). This will be the same as GetPackage for non-external objects.
		UObject* HostObject = InObject->GetOutermostObject();
		Package = HostObject->GetPackage();
	}
	
	if (Package && Package->HasAnyPackageFlags(PKG_EditorOnly))
	{
		return true;
	}

	if (bCheckRecursive && !InObject->IsNative())
	{
		UObject* Outer = InObject->GetOuter();
		if (Outer && Outer != Package)
		{
			if (IsEditorOnlyObjectInternal(Outer, Flags, LookupInCache, AddToCache))
			{
				return true;
			}
		}
		if (!bIgnoreEditorOnlyClass)
		{
			const UStruct* InStruct = Cast<UStruct>(InObject);
			if (InStruct)
			{
				const UStruct* SuperStruct = InStruct->GetSuperStruct();
				if (SuperStruct && IsEditorOnlyObjectInternal(SuperStruct, Flags, LookupInCache, AddToCache))
				{
					return true;
				}
			}
			else
			{
				if (IsEditorOnlyObjectInternal(InObject->GetClass(), Flags, LookupInCache, AddToCache))
				{
					return true;
				}

				UObject* Archetype = InObject->GetArchetype();
				if (Archetype && IsEditorOnlyObjectInternal(Archetype, Flags, LookupInCache, AddToCache))
				{
					return true;
				}
			}
		}
	}
	return false;
}

bool IsEditorOnlyObjectInternal(const UObject* InObject, EEditorOnlyObjectFlags Flags)
{
	return IsEditorOnlyObjectInternal(InObject, Flags,
		[](const UObject* Object)
		{
			return EEditorOnlyObjectResult::Uninitialized;
		},
		[](const UObject* Object, bool bEditorOnly)
		{
		});
}

bool IsEditorOnlyObjectInternal(const UObject* InObject, EEditorOnlyObjectFlags Flags,
	TFunctionRef<EEditorOnlyObjectResult(const UObject*)> LookupInCache,
	TFunctionRef<void(const UObject*, bool)> AddToCache)
{
	EEditorOnlyObjectResult Result = LookupInCache(InObject);
	if (Result != EEditorOnlyObjectResult::Uninitialized)
	{
		return Result == EEditorOnlyObjectResult::EditorOnly;
	}

	bool bResult = IsEditorOnlyObjectWithoutWritingCache(InObject, Flags, LookupInCache, AddToCache);
	AddToCache(InObject, bResult);
	return bResult;
}

} // namespace UE::SavePackageUtilities

#endif // WITH_EDITORONLY_DATA

void FObjectImportSortHelper::SortImports(FLinkerSave* Linker)
{
	TArray<FObjectImport>& Imports = Linker->ImportMap;
	if (Imports.IsEmpty())
	{
		return;
	}

	// Map of UObject => full name; optimization for sorting.
	TMap<TObjectPtr<UObject>, FString> ObjectToFullNameMap;
	ObjectToFullNameMap.Reserve(Imports.Num());

	for (const FObjectImport& Import : Imports)
	{
		if (Import.XObject)
		{
			ObjectToFullNameMap.Add(Import.XObject, Import.XObject->GetFullName());
		}
	}

	auto CompareObjectImports = [&ObjectToFullNameMap](const FObjectImport& A, const FObjectImport& B)
	{
		int32 Result = 0;
		if (A.XObject == nullptr)
		{
			Result = 1;
		}
		else if (B.XObject == nullptr)
		{
			Result = -1;
		}
		else
		{
			const FString* FullNameA = ObjectToFullNameMap.Find(A.XObject);
			const FString* FullNameB = ObjectToFullNameMap.Find(B.XObject);
			checkSlow(FullNameA);
			checkSlow(FullNameB);

			Result = FCString::Stricmp(**FullNameA, **FullNameB);
		}

		return Result < 0;
	};
	
	Algo::Sort(Linker->ImportMap, CompareObjectImports);
}

void FObjectExportSortHelper::SortExports(FLinkerSave* Linker)
{
	TArray<FObjectExport>& Exports = Linker->ExportMap;
	if (Exports.IsEmpty())
	{
		return;
	}
	
	// Map of UObject => full name; optimization for sorting.
	TMap<UObject*, FString> ObjectToFullNameMap;
	ObjectToFullNameMap.Reserve(Exports.Num());

	for (const FObjectExport& Export : Exports)
	{
		if (Export.Object)
		{
			ObjectToFullNameMap.Add(Export.Object, Export.Object->GetFullName());
		}
	}

	auto CompareObjectExports = [&ObjectToFullNameMap](const FObjectExport& A, const FObjectExport& B)
	{
		int32 Result = 0;
		if (A.Object == nullptr)
		{
			Result = 1;
		}
		else if (B.Object == nullptr)
		{
			Result = -1;
		}
		else
		{
			const FString* FullNameA = ObjectToFullNameMap.Find(A.Object);
			const FString* FullNameB = ObjectToFullNameMap.Find(B.Object);
			checkSlow(FullNameA);
			checkSlow(FullNameB);

			Result = FCString::Stricmp(**FullNameA, **FullNameB);
		}

		return Result < 0;
	};
	
	Algo::Sort(Linker->ExportMap, CompareObjectExports);
}

FScopedSavingFlag::FScopedSavingFlag(bool InSavingConcurrent, UPackage* InSavedPackage)
	: bSavingConcurrent(InSavingConcurrent)
	, SavedPackage(InSavedPackage)
{
	check(!IsGarbageCollecting());

	if (IsInGameThread())
	{
		// Don't need to take the GC Lock here since we've already validated we're not in the middle of GC
		// with the check above. GIsSavingPackage provides the proper protection for avoiding object find during save (i.e. StaticFindObject)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GIsSavingPackage = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		// Make sure the GC cannot run while we're saving
		FGCCSyncObject::Get().LockAsync();
		GIsSavingPackageTLS = true;
	}

	// Mark the package as being saved 
	if (SavedPackage)
	{
		SavedPackage->SetPackageFlags(PKG_IsSaving);
	}
}

FScopedSavingFlag::~FScopedSavingFlag()
{
	if (IsInGameThread())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		GIsSavingPackage = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		GIsSavingPackageTLS = false;
		FGCCSyncObject::Get().UnlockAsync();
	}

	if (SavedPackage)
	{
		SavedPackage->ClearPackageFlags(PKG_IsSaving);
	}
}

FCanSkipEditorReferencedPackagesWhenCooking::FCanSkipEditorReferencedPackagesWhenCooking()
{
	//UE_DEPRECATED(5.5, TEXT("No longer used; skiponlyeditoronly is used instead and tracks editoronly references via savepackage results."))
	static bool bWarned = false;
	if (!bWarned)
	{
		bWarned = true;
		bool bResult = true;

		UE::ConfigAccessTracking::FIgnoreScope IgnoreScope;

		GConfig->GetBool(TEXT("Core.System"), TEXT("CanSkipEditorReferencedPackagesWhenCooking"), bResult, GEngineIni);
		if (bResult)
		{
			UE_LOGF(LogSavePackage, Warning,
				"Engine.ini:[Core.System]:CanSkipEditorReferencedPackagesWhenCooking is deprecated; it is replaced by Editor.ini:[CookSettings]:SkipOnlyEditorOnly. Remove this setting from your inis.");
		}
	}
}

namespace UE::SavePackageUtilities
{

/**
 * Static: Saves thumbnail data for the specified package outer and linker
 *
 * @param	InOuter							the outer to use for the new package
 * @param	Linker							linker we're currently saving with
 * @param	Slot							structed archive slot we are saving too (temporary)
 */
void SaveThumbnails(UPackage* InOuter, FLinkerSave* Linker, FStructuredArchive::FSlot Slot)
{
	FStructuredArchive::FRecord Record = Slot.EnterRecord();

	Linker->Summary.ThumbnailTableOffset = 0;

#if WITH_EDITORONLY_DATA
	// Do we have any thumbnails to save?
	if( !(Linker->Summary.GetPackageFlags() & PKG_FilterEditorOnly) && InOuter->HasThumbnailMap() )
	{
		const FThumbnailMap& PackageThumbnailMap = InOuter->GetThumbnailMap();


		// Figure out which objects have thumbnails.  Note that we only want to save thumbnails
		// for objects that are actually in the export map.  This is so that we avoid saving out
		// thumbnails that were cached for deleted objects and such.
		TArray< FObjectFullNameAndThumbnail > ObjectsWithThumbnails;

		if (PackageThumbnailMap.Num())
		{
			for( int32 i=0; i<Linker->ExportMap.Num(); i++ )
			{
				FObjectExport& Export = Linker->ExportMap[i];
				if( Export.Object )
				{
					const FName ObjectFullName( *Export.Object->GetFullName(), FNAME_Find );
					const FObjectThumbnail* ObjectThumbnail = nullptr;
					// If the FName does not exist, then we know it is not in the map and do not need to search
					if (!ObjectFullName.IsNone())
					{
						ObjectThumbnail = PackageThumbnailMap.Find(ObjectFullName);
					}
		
					// if we didn't find the object via full name, try again with ??? as the class name, to support having
					// loaded old packages without going through the editor (ie cooking old packages)
					if (ObjectThumbnail == nullptr)
					{
						// can't overwrite ObjectFullName, so that we add it properly to the map
						FName OldPackageStyleObjectFullName = FName(*FString::Printf(TEXT("??? %s"), *Export.Object->GetPathName()), FNAME_Find);
						if (!OldPackageStyleObjectFullName.IsNone())
						{
							ObjectThumbnail = PackageThumbnailMap.Find(OldPackageStyleObjectFullName);
						}
					}
					if( ObjectThumbnail != nullptr )
					{
						// IMPORTANT: We save all thumbnails here, even if they are a shared (empty) thumbnail!
						// Empty thumbnails let us know that an asset is in a package without having to
						// make a linker for it.
						ObjectsWithThumbnails.Add( FObjectFullNameAndThumbnail( ObjectFullName, ObjectThumbnail ) );
					}
				}
			}
		}

		// preserve thumbnail rendered for the level
		const FObjectThumbnail* ObjectThumbnail = PackageThumbnailMap.Find(FName(*InOuter->GetFullName()));
		if (ObjectThumbnail != nullptr)
		{
			ObjectsWithThumbnails.Add( FObjectFullNameAndThumbnail(FName(*InOuter->GetFullName()), ObjectThumbnail ) );
		}
		
		// Do we have any thumbnails?  If so, we'll save them out along with a table of contents
		if( ObjectsWithThumbnails.Num() > 0 )
		{
			// Save out the image data for the thumbnails
			FStructuredArchive::FStream ThumbnailStream = Record.EnterStream(TEXT("Thumbnails"));

			for( int32 CurObjectIndex = 0; CurObjectIndex < ObjectsWithThumbnails.Num(); ++CurObjectIndex )
			{
				FObjectFullNameAndThumbnail& CurObjectThumb = ObjectsWithThumbnails[ CurObjectIndex ];

				// Store the file offset to this thumbnail
				CurObjectThumb.FileOffset = (int32)Linker->Tell();

				// Serialize the thumbnail!
				FObjectThumbnail* SerializableThumbnail = const_cast< FObjectThumbnail* >( CurObjectThumb.ObjectThumbnail );
				SerializableThumbnail->Serialize(ThumbnailStream.EnterElement());
			}


			// Store the thumbnail table of contents
			{
				Linker->Summary.ThumbnailTableOffset = (int32)Linker->Tell();

				// Save number of thumbnails
				int32 ThumbnailCount = ObjectsWithThumbnails.Num();
				FStructuredArchive::FArray IndexArray = Record.EnterField(TEXT("Index")).EnterArray(ThumbnailCount);

				// Store a list of object names along with the offset in the file where the thumbnail is stored
				for( int32 CurObjectIndex = 0; CurObjectIndex < ObjectsWithThumbnails.Num(); ++CurObjectIndex )
				{
					const FObjectFullNameAndThumbnail& CurObjectThumb = ObjectsWithThumbnails[ CurObjectIndex ];

					// Object name
					const FString ObjectFullName = CurObjectThumb.ObjectFullName.ToString();

					// Break the full name into it's class and path name parts
					const int32 FirstSpaceIndex = ObjectFullName.Find( TEXT( " " ) );
					check( FirstSpaceIndex != INDEX_NONE && FirstSpaceIndex > 0 );
					FString ObjectClassName = ObjectFullName.Left( FirstSpaceIndex );
					const FString ObjectPath = ObjectFullName.Mid( FirstSpaceIndex + 1 );

					// Remove the package name from the object path since that will be implicit based
					// on the package file name
					FString ObjectPathWithoutPackageName = ObjectPath.Mid( ObjectPath.Find( TEXT( "." ) ) + 1 );

					// File offset for the thumbnail (already saved out.)
					int32 FileOffset = CurObjectThumb.FileOffset;

					IndexArray.EnterElement().EnterRecord()
						<< SA_VALUE(TEXT("ObjectClassName"), ObjectClassName)
						<< SA_VALUE(TEXT("ObjectPathWithoutPackageName"), ObjectPathWithoutPackageName)
						<< SA_VALUE(TEXT("FileOffset"), FileOffset);
				}
			}
		}
	}

	// if content browser isn't enabled, clear the thumbnail map so we're not using additional memory for nothing
	if ( !GIsEditor || IsRunningCommandlet() )
	{
		InOuter->SetThumbnailMap(nullptr);
	}
#endif
}

ESavePackageResult AppendAdditionalData(FLinkerSave& Linker, int64& InOutDataStartOffset, FSavePackageContext* SavePackageContext)
{
	if (Linker.AdditionalDataToAppend.Num() == 0)
	{
		return ESavePackageResult::Success;
	}

	IPackageWriter* PackageWriter = SavePackageContext ? SavePackageContext->PackageWriter : nullptr;
	if (PackageWriter)
	{
		bool bDeclareRegionForEachAdditionalFile = SavePackageContext->PackageWriterCapabilities.bDeclareRegionForEachAdditionalFile;
		FFileRegionMemoryWriter DataArchive;
		for (FLinkerSave::AdditionalDataCallback& Callback : Linker.AdditionalDataToAppend)
		{
			if (bDeclareRegionForEachAdditionalFile)
			{
				DataArchive.PushFileRegionType(EFileRegionType::None);
			}
			Callback(Linker, DataArchive, InOutDataStartOffset + DataArchive.Tell());
			if (bDeclareRegionForEachAdditionalFile)
			{
				DataArchive.PopFileRegionType();
			}
		}
		IPackageWriter::FLinkerAdditionalDataInfo DataInfo{ Linker.LinkerRoot->GetFName() };
		int64 DataSize = DataArchive.TotalSize();
		FIoBuffer DataBuffer = FIoBuffer(FIoBuffer::AssumeOwnership, DataArchive.ReleaseOwnership(), DataSize);
		PackageWriter->WriteLinkerAdditionalData(DataInfo, DataBuffer, DataArchive.GetFileRegions());
		InOutDataStartOffset += DataSize;
	}
	else
	{
		int64 LinkerStart = Linker.Tell();
		FArchive& Ar = Linker;
		for (FLinkerSave::AdditionalDataCallback& Callback : Linker.AdditionalDataToAppend)
		{
			Callback(Linker, Linker, Linker.Tell());
		}
		InOutDataStartOffset += Linker.Tell() - LinkerStart;
	}

	Linker.AdditionalDataToAppend.Empty();

	// Note that we currently have no failure condition here, but we return a ESavePackageResult
	// in case one needs to be added in future code.
	return ESavePackageResult::Success;
}

ESavePackageResult CreatePayloadSidecarFile(FLinkerSave& Linker, const FPackagePath& PackagePath, const bool bSaveToMemory, FSavePackageOutputFileArray& AdditionalPackageFiles, FSavePackageContext* SavePackageContext)
{
	if (Linker.SidecarDataToAppend.IsEmpty())
	{
		return ESavePackageResult::Success;
	}

	// Note since we only allow sidecar file generation when saving a package and not when cooking 
	// we know that we don't need to generate the hash or check if we should write the file, since those 
	// operations are cooking only. However we still accept the parameters and check against them for 
	// safety in case someone tries to add support in the future.
	// We could add support but it is difficult to test and would be better left for a proper clean up pass
	// once we enable SavePackage2 only. 
	checkf(!Linker.IsCooking(), TEXT("Cannot write a sidecar file during cooking! (%s)"), *PackagePath.GetDebugName());
	IPackageWriter* PackageWriter = SavePackageContext ? SavePackageContext->PackageWriter : nullptr;

	UE::FPackageTrailerBuilder Builder;

	for (FLinkerSave::FSidecarStorageInfo& Info : Linker.SidecarDataToAppend)
	{
		Builder.AddPayload(Info.Identifier, Info.Payload, UE::Virtualization::EPayloadFilterReason::None);
	}

	Linker.SidecarDataToAppend.Empty();

	FLargeMemoryWriter Ar(0, true /* bIsPersistent */);
	if (!Builder.BuildAndAppendTrailer(nullptr, Ar))
	{
		UE_LOGF(LogSavePackage, Error, "Failed to build sidecar package trailer for '%ls'", *PackagePath.GetDebugName());
		return ESavePackageResult::Error;
	}

	const int64 DataSize = Ar.TotalSize();
	checkf(DataSize > 0, TEXT("Sidecar archive should not be empty! (%s)"), *PackagePath.GetDebugName());

	FString TargetFilePath = PackagePath.GetLocalFullPath(EPackageSegment::PayloadSidecar);

	if (PackageWriter)
	{
		IPackageWriter::FAdditionalFileInfo SidecarSegmentInfo;
		SidecarSegmentInfo.PackageName = PackagePath.GetPackageFName();
		SidecarSegmentInfo.Filename = MoveTemp(TargetFilePath);
		FIoBuffer FileData(FIoBuffer::AssumeOwnership, Ar.ReleaseOwnership(), DataSize);
		PackageWriter->WriteAdditionalFile(SidecarSegmentInfo, FileData);
	}
	else if (bSaveToMemory)
	{
		AdditionalPackageFiles.Emplace(MoveTemp(TargetFilePath), FLargeMemoryPtr(Ar.ReleaseOwnership()), TArray<FFileRegion>(), DataSize);
	}
	else
	{
		const FString BaseFilename = FPaths::GetBaseFilename(TargetFilePath);
		FString TempFilePath = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), *BaseFilename.Left(32));
			
		SavePackageUtilities::WriteToFile(TempFilePath, Ar.GetData(), DataSize); // TODO: Check the error handling here!
		UE_LOGF(LogSavePackage, Verbose, "Saved '%ls' as temp file '%ls'", *TargetFilePath, *TempFilePath);

		AdditionalPackageFiles.Emplace(MoveTemp(TargetFilePath), MoveTemp(TempFilePath), DataSize);		
	}

	return ESavePackageResult::Success;
}

void SaveMetaData(UPackage* InOuter, FLinkerSave* Linker, FStructuredArchive::FRecord Record)
{
	Linker->Summary.MetaDataOffset = (int32)Linker->Tell();

#if WITH_METADATA
	FStructuredArchive::FRecord MetaDataRecord = Record.EnterRecord(TEXT("MetaData"));

	FMetaData& PackageMetaData = InOuter->GetMetaData();
	{
		int32 NumObjectMetaDataMap = PackageMetaData.ObjectMetaDataMap.Num();
		MetaDataRecord << SA_VALUE(TEXT("NumObjectMetaDataMap"), NumObjectMetaDataMap);

		int32 NumRootMetaDataMap = PackageMetaData.RootMetaDataMap.Num();
		MetaDataRecord << SA_VALUE(TEXT("NumRootMetaDataMap"), NumRootMetaDataMap);

		{
			FStructuredArchive::FStream ObjectMetaDataMapStream = MetaDataRecord.EnterStream(TEXT("ObjectMetaDataMap"));
			for (TPair<FSoftObjectPath, TMap<FName, FString>>& ObjectMetaData : PackageMetaData.ObjectMetaDataMap)
			{
				ObjectMetaDataMapStream.EnterElement() << ObjectMetaData;
			}
		}

		{
			FStructuredArchive::FStream RootMetaDataMapStream = MetaDataRecord.EnterStream(TEXT("RootMetaDataMap"));
			for (TPair<FName, FString>& RootMetaData : PackageMetaData.RootMetaDataMap)
			{
				RootMetaDataMapStream.EnterElement() << RootMetaData;
			}
		}
	}
#endif
}

void SaveWorldLevelInfo(UPackage* InOuter, FLinkerSave* Linker, FStructuredArchive::FRecord Record)
{
	Linker->Summary.WorldTileInfoDataOffset = 0;
	
	if(FWorldTileInfo* WorldTileInfo = InOuter->GetWorldTileInfo())
	{
		Linker->Summary.WorldTileInfoDataOffset = (int32)Linker->Tell();
		Record << SA_VALUE(TEXT("WorldLevelInfo"), *WorldTileInfo);
	}
}

bool TryHashFile(FStringView Filename, FIoHash& OutHash, int64 Offset, int64 Size)
{
	FBlake3 Builder;
	if (!TryHashFile(Filename, Builder, Offset, Size))
	{
		OutHash = FIoHash::Zero;
		return false;
	}

	OutHash = FIoHash(Builder.Finalize());
	return true;
}

} // end namespace UE::SavePackageUtilities

void UPackage::WaitForAsyncFileWrites()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPackage::WaitForAsyncFileWrites);

	while (OutstandingAsyncWrites.GetValue())
	{
		FPlatformProcess::Sleep(0.0f);
	}
}

bool UPackage::HasAsyncFileWrites()
{
	return OutstandingAsyncWrites.GetValue() > 0;
}

bool UPackage::IsEmptyPackage(UPackage* Package, const UObject* LastReferencer)
{
	// Don't count null or volatile packages as empty, just let them be NULL or get GCed
	if ( Package != nullptr )
	{
		// Make sure the package is fully loaded before determining if it is empty
		if( !Package->IsFullyLoaded() )
		{
			Package->FullyLoad();
		}

		bool bIsEmpty = true;
		ForEachObjectWithPackage(Package, [LastReferencer, &bIsEmpty](UObject* InObject)
		{
			// if the package contains at least one object that has asset registry data and isn't the `LastReferencer` consider it not empty
			if (InObject->IsAsset() && InObject != LastReferencer)
			{
				bIsEmpty = false;
				// we can break out of the iteration as soon as we find one valid object
				return false;
			}
			return true;
		// Don't consider transient, class default or garbage objects
		}, EGetObjectsFlags::None, RF_Transient | RF_ClassDefaultObject, EInternalObjectFlags::Garbage);
		return bIsEmpty;
	}

	// Invalid package
	return false;
}


namespace UE::AssetRegistry
{

void WritePackageData(FStructuredArchiveRecord& ParentRecord, bool bIsCooking, const UPackage* Package,
	FLinkerSave* Linker, const TSet<TObjectPtr<UObject>>& ImportsUsedInGame, const TSet<FName>& SoftPackagesUsedInGame,
	const ITargetPlatform* TargetPlatform, TArray<FAssetData>* OutAssetDatas)
{
	if (TargetPlatform)
	{
		FArchiveCookContext CookContext(const_cast<UPackage*>(Package), UE::Cook::ECookType::Unknown,
			UE::Cook::ECookingDLC::Unknown, TargetPlatform, nullptr /* CookInfo */);
		WritePackageData(ParentRecord, &CookContext, Package, Linker, ImportsUsedInGame, SoftPackagesUsedInGame,
			OutAssetDatas, true /* bProceduralSave */);
	}
	else
	{
		WritePackageData(ParentRecord, nullptr, Package, Linker, ImportsUsedInGame, SoftPackagesUsedInGame,
			OutAssetDatas, false/* bProceduralSave */);
	}
}

void WritePackageData(FStructuredArchiveRecord& ParentRecord, FArchiveCookContext* CookContext, const UPackage* Package,
	FLinkerSave* Linker, const TSet<TObjectPtr<UObject>>& ImportsUsedInGame, const TSet<FName>& SoftPackagesUsedInGame,
	TArray<FAssetData>* OutAssetDatas, bool bProceduralSave)
{
	FWritePackageDataArgs Args;
	Args.ParentRecord = &ParentRecord;
	Args.Package = Package;
	Args.Linker = Linker;
	Args.ImportsUsedInGame = &ImportsUsedInGame;
	Args.SoftPackagesUsedInGame = &SoftPackagesUsedInGame;
	Args.bProceduralSave = bProceduralSave;
	Args.CookContext = CookContext;
	Args.OutAssetDatas = OutAssetDatas;
	TArray<FName> PackageBuildDependencies;
	Args.PackageBuildDependencies = &PackageBuildDependencies;
	WritePackageData(Args);
}

// See the corresponding ReadPackageDataMain and ReadPackageDataDependencies defined in PackageReader.cpp in AssetRegistry module
void WritePackageData(FWritePackageDataArgs& Args)
{
	Args.bProceduralSave = Args.bProceduralSave || (Args.CookContext != nullptr);
	FLinkerSave* Linker = Args.Linker;
	IAssetRegistryInterface* AssetRegistry = IAssetRegistryInterface::GetPtr();

	// To avoid large patch sizes, we have frozen cooked package format at the format before VER_UE4_ASSETREGISTRY_DEPENDENCYFLAGS
	// Non-cooked saves do a full update. Orthogonally, they also store the assets in the package in addition to the output variable.
	// Cooked saves do an additive update and do not store the assets in the package.
	bool bPreDependencyFormat = false;
	bool bWriteAssetsToPackage = true;
	// Editor saves do a full update, but procedural saves (including cook) do not
	bool bFullUpdate = !Args.bProceduralSave;
	FCookTagList* CookTagList = nullptr;
	if (Args.CookContext)
	{
		bPreDependencyFormat = true;
		bWriteAssetsToPackage = false;
		CookTagList = Args.CookContext->GetCookTagList();
	}	

	// WritePackageData is currently only called if not bTextFormat; we rely on that to save offsets
	FArchive& BinaryArchive = Args.ParentRecord->GetUnderlyingArchive();
	check(!BinaryArchive.IsTextFormat());

	// Store the asset registry offset in the file and enter a record for the asset registry data
	Linker->Summary.AssetRegistryDataOffset = (int32)BinaryArchive.Tell();
	FStructuredArchiveRecord AssetRegistryRecord = Args.ParentRecord->EnterField(TEXT("AssetRegistry")).EnterRecord();

	// Offset to Dependencies
	int64 OffsetToAssetRegistryDependencyDataOffset = INDEX_NONE;
	if (!bPreDependencyFormat)
	{
		// Write placeholder data for the offset to the separately-serialized AssetRegistryDependencyData
		OffsetToAssetRegistryDependencyDataOffset = BinaryArchive.Tell();
		int64 AssetRegistryDependencyDataOffset = 0;
		AssetRegistryRecord << SA_VALUE(TEXT("AssetRegistryDependencyDataOffset"), AssetRegistryDependencyDataOffset);
		check(BinaryArchive.Tell() == OffsetToAssetRegistryDependencyDataOffset + sizeof(AssetRegistryDependencyDataOffset));
	}

	TArray<UObject*> AssetObjects;
	for (int32 i = 0; i < Linker->ExportMap.Num(); i++)
	{
		FObjectExport& Export = Linker->ExportMap[i];
		if (Export.Object && Export.Object->IsAsset())
		{
#if WITH_EDITOR
			if (Args.CookContext)
			{
				TArray<UObject*> AdditionalObjects;
				Export.Object->GetAdditionalAssetDataObjectsForCook(*Args.CookContext, AdditionalObjects);
				for (UObject* Object : AdditionalObjects)
				{
					if (Object->IsAsset())
					{
						AssetObjects.Add(Object);
					}
				}
			}
#endif
			AssetObjects.Add(Export.Object);
		}
	}

	int32 ObjectCountInPackage = bWriteAssetsToPackage ? AssetObjects.Num() : 0;
	FStructuredArchive::FArray AssetArray = AssetRegistryRecord.EnterArray(TEXT("TagMap"), ObjectCountInPackage);
	FString PackageName = Args.Package->GetName();

	for (int32 ObjectIdx = 0; ObjectIdx < AssetObjects.Num(); ++ObjectIdx)
	{
		const UObject* Object = AssetObjects[ObjectIdx];

		// Exclude the package name in the object path, we just need to know the path relative to the package we are saving
		FString ObjectPath = Object->GetPathName(Args.Package);
		FString ObjectClassName = Object->GetClass()->GetPathName();

		FAssetRegistryTagsContextData TagsContextData(Object, EAssetRegistryTagsCaller::SavePackage);
		TagsContextData.bProceduralSave = Args.bProceduralSave;
		TagsContextData.TargetPlatform = nullptr;
		if (Args.CookContext)
		{
			TagsContextData.TargetPlatform = Args.CookContext->GetTargetPlatform();
			TagsContextData.CookType = Args.CookContext->GetCookType();
			TagsContextData.CookingDLC = Args.CookContext->GetCookingDLC();
			TagsContextData.bWantsCookTags = TagsContextData.CookType == UE::Cook::ECookType::ByTheBook;
		}
		TagsContextData.bFullUpdateRequested = bFullUpdate;
		FAssetRegistryTagsContext TagsContext(TagsContextData);

		if (AssetRegistry && !TagsContextData.bFullUpdateRequested)
		{
			FAssetData ExistingAssetData;
			if (AssetRegistry->TryGetAssetByObjectPath(FSoftObjectPath(Object), ExistingAssetData)
				== UE::AssetRegistry::EExists::Exists)
			{
				TagsContextData.Tags.Reserve(ExistingAssetData.TagsAndValues.Num());
				ExistingAssetData.TagsAndValues.ForEach(
					[&TagsContextData](const TPair<FName, FAssetTagValueRef>& Pair)
					{
						TagsContextData.Tags.Add(Pair.Key, UObject::FAssetRegistryTag(Pair.Key, Pair.Value.GetStorageString(),
							UObject::FAssetRegistryTag::TT_Alphabetical));
					});
			}
		}
		if (CookTagList)
		{
			TArray<FCookTagList::FTagNameValuePair>* CookTags = CookTagList->ObjectToTags.Find(Object);
			if (CookTags)
			{
				for (FCookTagList::FTagNameValuePair& Pair : *CookTags)
				{
					TagsContext.AddCookTag(UObject::FAssetRegistryTag(Pair.Key, Pair.Value,
						UObject::FAssetRegistryTag::TT_Alphabetical));
				}
			}
		}
		
		Object->GetAssetRegistryTags(TagsContext);

		int32 TagCount = TagsContextData.Tags.Num();
		TagsContextData.Tags.KeySort(FNameLexicalLess());

		if (bWriteAssetsToPackage)
		{
			FStructuredArchive::FRecord AssetRecord = AssetArray.EnterElement().EnterRecord();
			AssetRecord << SA_VALUE(TEXT("Path"), ObjectPath) << SA_VALUE(TEXT("Class"), ObjectClassName);

			FStructuredArchive::FMap TagMap = AssetRecord.EnterField(TEXT("Tags")).EnterMap(TagCount);

			for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContextData.Tags)
			{
				const UObject::FAssetRegistryTag& Tag = TagPair.Value;
				FString Key = Tag.Name.ToString();
				FString Value = Tag.Value;

				TagMap.EnterElement(Key) << Value;
			}
		}

		if (Args.OutAssetDatas)
		{
			FAssetDataTagMap TagsAndValues;
			for (TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContextData.Tags)
			{
				UObject::FAssetRegistryTag& Tag = TagPair.Value;
				if (!Tag.Name.IsNone() && !Tag.Value.IsEmpty())
				{
					TagsAndValues.Add(Tag.Name, MoveTemp(Tag.Value));
				}
			}
			// if we do not have a full object path already, build it
			const bool bFullObjectPath = ObjectPath.StartsWith(TEXT("/"), ESearchCase::CaseSensitive);
			if (!bFullObjectPath)
			{
				// if we do not have a full object path, ensure that we have a top level object for the package and not a sub object
				if (!ensureMsgf(!ObjectPath.Contains(TEXT("."), ESearchCase::CaseSensitive),
					TEXT("Cannot make FAssetData for sub object %s in package %s!"), *ObjectPath, *PackageName))
				{
					continue;
				}
				ObjectPath = PackageName + TEXT(".") + ObjectPath;
			}

			Args.OutAssetDatas->Emplace(PackageName, ObjectPath, FTopLevelAssetPath(ObjectClassName),
				MoveTemp(TagsAndValues), Args.Package->GetChunkIDs(), Args.Package->GetPackageFlags());
		}
	}
	if (bPreDependencyFormat)
	{
		// The legacy format did not write the other sections, or the offsets to those other sections
		return;
	}

	// Overwrite the placeholder offset for the AssetRegistryDependencyData and enter a record for the asset registry dependency data
	{
		int64 AssetRegistryDependencyDataOffset = Linker->Tell();
		BinaryArchive.Seek(OffsetToAssetRegistryDependencyDataOffset);
		BinaryArchive << AssetRegistryDependencyDataOffset;
		BinaryArchive.Seek(AssetRegistryDependencyDataOffset);
	}
	FStructuredArchiveRecord DependencyDataRecord = Args.ParentRecord->EnterField(TEXT("AssetRegistryDependencyData")).EnterRecord();

	// Convert the IsUsedInGame sets into a bitarray with a value per import/softpackagereference
	TBitArray<> ImportUsedInGameBits;
	TBitArray<> SoftPackageUsedInGameBits;
	ImportUsedInGameBits.Reserve(Linker->ImportMap.Num());
	for (int32 ImportIndex = 0; ImportIndex < Linker->ImportMap.Num(); ++ImportIndex)
	{
		ImportUsedInGameBits.Add(Args.ImportsUsedInGame->Contains(Linker->ImportMap[ImportIndex].XObject));
	}
	SoftPackageUsedInGameBits.Reserve(Linker->SoftPackageReferenceList.Num());
	for (int32 SoftPackageIndex = 0; SoftPackageIndex < Linker->SoftPackageReferenceList.Num(); ++SoftPackageIndex)
	{
		SoftPackageUsedInGameBits.Add(Args.SoftPackagesUsedInGame->Contains(
			Linker->SoftPackageReferenceList[SoftPackageIndex]));
	}

	// Serialize the Dependency section
	DependencyDataRecord << SA_VALUE(TEXT("ImportUsedInGame"), ImportUsedInGameBits);
	DependencyDataRecord << SA_VALUE(TEXT("SoftPackageUsedInGame"), SoftPackageUsedInGameBits);

	// Currently the only type of ExtraPackageDependencies we have are the collected build dependencies,
	// which have both the Build and PropagateManage flags. Store them as pairs of { PackageName, EExtraDepencyFlags }
	// even though we only have one possible value for EExtraDepencyFlags, so that we can extend the types of dependencies
	// later without needing to change EUnrealEngineObjectUE5Version.
	TArray<TPair<FName, uint32>> ExtraPackageDependencies;
	ExtraPackageDependencies.Reserve(Args.PackageBuildDependencies->Num());
	constexpr EExtraDependencyFlags BuildAndPropagate =
		EExtraDependencyFlags::Build | EExtraDependencyFlags::PropagateManage;
	for (FName ExtraPackageName : *Args.PackageBuildDependencies)
	{
		ExtraPackageDependencies.Add({ ExtraPackageName, static_cast<uint32>(BuildAndPropagate) });
	}
	DependencyDataRecord << SA_VALUE(TEXT("ExtraPackageDependencies"), ExtraPackageDependencies);
}

}
