// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/FileHelper.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/ArchiveObjectCrc32.h"
#include "Serialization/ArchiveStackTrace.h"
#include "Serialization/FileRegions.h"
#include "UObject/NameTypes.h"
#include "UObject/Package.h"
#include "UObject/UObjectMarks.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SavePackage.h"
#include "UObject/UObjectGlobals.h"

// This file contains private utilities used by UPackage::Save

class FCbFieldView;
class FCbWriter;
class FPackagePath;
class FSaveContext;
class FSavePackageContext;
class IPackageWriter;
class ITargetPlatform;
enum class ESavePackageResult;
enum class ESaveRealm : uint32;
struct FIoHash;

// Save Time trace
#if UE_TRACE_ENABLED && !UE_BUILD_SHIPPING
UE_TRACE_CHANNEL_EXTERN(SaveTimeChannel)
#define SCOPED_SAVETIMER(TimerName) TRACE_CPUPROFILER_EVENT_SCOPE_ON_CHANNEL(TimerName, SaveTimeChannel)
#define SCOPED_SAVETIMER_TEXT(TimerName) TRACE_CPUPROFILER_EVENT_SCOPE_TEXT_ON_CHANNEL(TimerName, SaveTimeChannel)
#else
#define SCOPED_SAVETIMER(TimerName)
#define SCOPED_SAVETIMER_TEXT(TimerName)
#endif

struct FLargeMemoryDelete
{
	void operator()(uint8* Ptr) const
	{
		if (Ptr)
		{
			FMemory::Free(Ptr);
		}
	}
};
typedef TUniquePtr<uint8, FLargeMemoryDelete> FLargeMemoryPtr;

enum class EAsyncWriteOptions
{
	None = 0
};
ENUM_CLASS_FLAGS(EAsyncWriteOptions)

struct FScopedSavingFlag
{
	FScopedSavingFlag(bool InSavingConcurrent, UPackage* InSavedPackage);
	~FScopedSavingFlag();

	bool bSavingConcurrent;

private:
	// The package being saved
	UPackage* SavedPackage = nullptr;
};

struct FCanSkipEditorReferencedPackagesWhenCooking
{
	FCanSkipEditorReferencedPackagesWhenCooking();
};


/** Represents an output file from the package when saving */
struct FSavePackageOutputFile
{
	/** Constructor used for async saving */
	FSavePackageOutputFile(const FString& InTargetPath, FLargeMemoryPtr&& MemoryBuffer, const TArray<FFileRegion>& InFileRegions, int64 InDataSize)
		: TargetPath(InTargetPath)
		, FileMemoryBuffer(MoveTemp(MemoryBuffer))
		, FileRegions(InFileRegions)
		, DataSize(InDataSize)
	{

	}

	/** Constructor used for saving first to a temp file which can be later moved to the target directory */
	FSavePackageOutputFile(const FString& InTargetPath, const FString& InTempFilePath, int64 InDataSize)
		: TargetPath(InTargetPath)
		, TempFilePath(InTempFilePath)
		, DataSize(InDataSize)
	{

	}

	/** The final target location of the file once all saving operations are completed */
	FString TargetPath;

	/** The temp location (if any) that the file is stored at, pending a move to the TargetPath */
	FString TempFilePath;

	/** The entire file stored as a memory buffer for the async saving path */
	FLargeMemoryPtr FileMemoryBuffer;
	/** An array of file regions in FileMemoryBuffer generated during cooking */
	TArray<FFileRegion> FileRegions;

	/** The size of the file in bytes */
	int64 DataSize;
};

// Currently we only expect to store up to 2 files in this, so set the inline capacity to double of this
using FSavePackageOutputFileArray = TArray<FSavePackageOutputFile, TInlineAllocator<4>>;

 /**
  * Helper structure to encapsulate sorting a linker's import table alphabetically
  * @note We should be able to remove this and use less expensive sort order.
  */
struct FObjectImportSortHelper
{
	/**
	 * Sorts imports according to the order in which they occur in the list of imports.
	 *
	 * @param	Linker				linker containing the imports that need to be sorted
	 */
	static void SortImports(FLinkerSave* Linker);
};

/**
 * Helper structure to encapsulate sorting a linker's export table alphabetically
  * @note We should be able to remove this and use less expensive sort order.
 */
struct FObjectExportSortHelper
{
	/**
	 * Sorts exports alphabetically.
	 *
	 * @param	Linker				linker containing the exports that need to be sorted
	 */
	static void SortExports(FLinkerSave* Linker);
};

// Utility functions used by UPackage::Save
namespace UE::SavePackageUtilities
{

extern const FName NAME_World;
extern const FName NAME_Level;
extern const FName NAME_PrestreamPackage;

void SaveThumbnails(UPackage* InOuter, FLinkerSave* Linker, FStructuredArchive::FSlot Slot);

/**
	* Used to append additional data to the end of the package file by invoking callbacks stored in the linker.
	* They may be saved to the end of the file, or to a separate archive passed into the PackageWriter.
	 
	* @param Linker The linker containing the exports. Provides the list of AdditionalData, and the data may write to it as their target archive.
	* @param InOutStartOffset In value is the offset in the Linker's archive where the datas will be put. If SavePackageContext settings direct
	*        the datas to write in a separate archive that will be combined after the linker, the value is the offset after the Linker archive's 
	*        totalsize and after any previous post-Linker archive data such as BulkDatas.
	*        Output value is incremented by the number of bytes written the Linker or the separate archive at the end of the linker.
	* @param SavePackageContext If non-null and configured to require it, data is passed to this PackageWriter on this context rather than appended to the Linker archive.
	*/
ESavePackageResult AppendAdditionalData(FLinkerSave& Linker, int64& InOutDataStartOffset, FSavePackageContext* SavePackageContext);
	
/** Used to create the sidecar file (.upayload) from payloads that have been added to the linker */
ESavePackageResult CreatePayloadSidecarFile(FLinkerSave& Linker, const FPackagePath& PackagePath, const bool bSaveToMemory,
	FSavePackageOutputFileArray& AdditionalPackageFiles, FSavePackageContext* SavePackageContext);

void SaveMetaData(UPackage* InOuter, FLinkerSave* Linker, FStructuredArchive::FRecord Record);
void SaveWorldLevelInfo(UPackage* InOuter, FLinkerSave* Linker, FStructuredArchive::FRecord Record);
EObjectMark GetExcludedObjectMarksForTargetPlatform(const ITargetPlatform* TargetPlatform);
void FindMostLikelyCulprit(const TArray<UObject*>& BadObjects, FSaveContext& SaveContext,
	ESaveRealm HarvestedRealm, UObject*& OutMostLikelyCulprit, UObject*& OutReferencer,
	const FProperty*& OutReferencerProperty, bool& OutIsCulpritArchetype, FString& OutDiagnosticText);
	
/** 
	* Search 'OutputFiles' for output files that were saved to the temp directory and move those files
	* to their final location. Output files that were not saved to the temp directory will be ignored.
	* 
	* If errors are encountered then the original state of the package will be restored and should continue to work.
	*/
ESavePackageResult FinalizeTempOutputFiles(const FPackagePath& PackagePath, const FSavePackageOutputFileArray& OutputFiles, const FDateTime& FinalTimeStamp);

void WriteToFile(const FString& Filename, const uint8* InDataPtr, int64 InDataSize);
void AsyncWriteFile(FLargeMemoryPtr Data, const int64 DataSize, const TCHAR* Filename, EAsyncWriteOptions Options, TArrayView<const FFileRegion> InFileRegions);
void AsyncWriteFile(EAsyncWriteOptions Options, FSavePackageOutputFile& File);

template <typename HashBuilderType>
bool TryHashFile(FStringView Filename, HashBuilderType& Builder, int64 Offset = 0, int64 Size = -1);
bool TryHashFile(FStringView Filename, FIoHash& OutHash, int64 Offset = 0, int64 Size = -1);

void GetCDOSubobjects(UObject* CDO, TArray<UObject*>& Subobjects);

enum class EEditorOnlyObjectFlags
{
	None = 0,
	CheckRecursive = 1 << 1,
	ApplyHasNonEditorOnlyReferences = 1 << 2,

};
ENUM_CLASS_FLAGS(EEditorOnlyObjectFlags);

#if WITH_EDITORONLY_DATA
bool CanStripEditorOnlyImportsAndExports();

/** Returns result of IsEditorOnlyObjectInternal if Engine:[Core.System]:CanStripEditorOnlyExportsAndImports (ini) is set to true */
bool IsEditorOnlyObjectInternal(const UObject* InObject, EEditorOnlyObjectFlags Flags,
	TFunctionRef<UE::SavePackageUtilities::EEditorOnlyObjectResult(const UObject*)> LookupInCache,
	TFunctionRef<void(const UObject*, bool)> AddToCache);
bool IsEditorOnlyObjectInternal(const UObject* InObject, EEditorOnlyObjectFlags Flags);
#endif

void HarvestCookRuntimeDependencies(FObjectSaveContextData& Data, UObject* HarvestReferencesFrom);

} // namespace UE::SavePackageUtilities

#if ENABLE_COOK_STATS
struct FSavePackageStats
{
	static int32 NumPackagesSaved;
	static double SavePackageTimeSec;
	static double TagPackageExportsPresaveTimeSec;
	static double TagPackageExportsTimeSec;
	static double FullyLoadLoadersTimeSec;
	static double ResetLoadersTimeSec;
	static double TagPackageExportsGetObjectsWithOuter;
	static double TagPackageExportsGetObjectsWithMarks;
	static double SerializeImportsTimeSec;
	static double SortExportsSeekfreeInnerTimeSec;
	static double SerializeExportsTimeSec;
	static double SerializeBulkDataTimeSec;
	static double AsyncWriteTimeSec;
	static double MBWritten;
	static TMap<FName, FArchiveDiffStats> PackageDiffStats;
	static int32 NumberOfDifferentPackages;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats;
	static void AddSavePackageStats(FCookStatsManager::AddStatFuncRef AddStat);
	static void MergeStats(const TMap<FName, FArchiveDiffStats>& ToMerge);
};
#endif


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


namespace UE::SavePackageUtilities
{

template <typename HashBuilderType>
bool TryHashFile(FStringView Filename, HashBuilderType& Builder, int64 Offset, int64 Size)
{
	return FFileHelper::LoadFileInBlocks(Filename, [&Builder](FMemoryView Block)
		{
			Builder.Update(Block);
		}, Offset, Size);
}

}