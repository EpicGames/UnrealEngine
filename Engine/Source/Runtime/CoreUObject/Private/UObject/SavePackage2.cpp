// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutoRTFM.h"
#include "UObject/SavePackage.h"
#include "UObject/LinkerLoad.h"

#if UE_WITH_SAVEPACKAGE
#include "AssetRegistry/AssetData.h"
#include "Async/ParallelFor.h"
#include "Blueprint/BlueprintSupport.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookEvents.h"
#include "HAL/FileManager.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "IO/IoDispatcher.h"
#include "Logging/StructuredLog.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Misc/MessageDialog.h"
#include "Misc/Optional.h"
#include "Misc/PackageAccessTracking.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "Misc/RedirectCollector.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/SecureHash.h"
#include "ProfilingDebugging/CookStats.h"
#include "Serialization/ArchiveProxy.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "Serialization/ArchiveStackTrace.h"
#include "Serialization/EditorBulkData.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/PackageWriter.h"
#include "Serialization/PropertyLocalizationDataGathering.h"
#include "Serialization/UnversionedPropertySerialization.h"
#include "Serialization/ImportTypeHierarchy.h"
#include "UObject/ArchiveCookContext.h"
#include "UObject/DebugSerializationFlags.h"
#include "UObject/EditorObjectVersion.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/LinkerSave.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/Package.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/SavePackage/PackageHarvester.h"
#include "UObject/SavePackage/SaveContext.h"
#include "UObject/SavePackage/SavePackageUtilities.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectSerializeContext.h"
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/Inline/VVMClassInline.h"
#include "VerseVM/VVMGlobalProgram.h"
#include "VerseVM/VVMMap.h"
#include "VerseVM/VVMPackage.h"
#include "VerseVM/VVMRef.h"
#include "VerseVM/VVMValueObject.h"
#include "VerseVM/VVMVerseClass.h"
#include "VerseVM/VVMVerseStruct.h"
#endif

#if ENABLE_COOK_STATS
#include "ProfilingDebugging/ScopedTimers.h"
#endif

bool UPackage::SavePackage(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename, const FSavePackageArgs& SaveArgs)
{
	const FSavePackageResultStruct Result = Save(InOuter, InAsset, Filename, SaveArgs);
	return Result == ESavePackageResult::Success;
}

FSavePackageContext::~FSavePackageContext()
{
	delete PackageWriter;
}

// defined in UObjectGlobals.cpp
extern const FName NAME_UniqueObjectNameForCooking;
COREUOBJECT_API extern bool GetOutputCookingWarnings();

namespace
{

int32 GFixupStandaloneFlags = 0;
static FAutoConsoleVariableRef CVar_FixupStandaloneFlags(
	TEXT("save.FixupStandaloneFlags"),
	GFixupStandaloneFlags,
	TEXT("If non-zero, when the UAsset of a package is missing RF_Standalone, the flag is added. If zero, the flags are not changed and the save fails.")
);

int32 GMemoryMappedBulkDataAlignment = 1;
static FAutoConsoleVariableRef CVar_MemoryMappedBulkDataAlignment(
	TEXT("cook.MemoryMappedBulkDataAlignment"),
	GMemoryMappedBulkDataAlignment,
	TEXT("0 - Target platform memory mapping alignment is respected, might result in big zero-padding")
	TEXT("1 - No special alignment required, 1-byte alignment used (no padding, default)")
	TEXT("Any other non-zero value - overrides the memory mapped data alignment, implies padding")
);

bool GValidateVirtualExportsFileOffset = true;
static FAutoConsoleVariableRef CVarValidateVirtualExportsFileOffset(
	TEXT("save.ValidateVirtualExportsFileOffset"),
	GValidateVirtualExportsFileOffset,
	TEXT("When true additional validation will be run when saving a package file."));

ESavePackageResult WriteAdditionalFiles(FSaveContext& SaveContext, FScopedSlowTask& SlowTask, int64& VirtualExportsFileOffset);

ESavePackageResult ReturnSuccessOrCancel()
{
	return !GWarn->ReceivedUserCancel() ? ESavePackageResult::Success : ESavePackageResult::Canceled;
}

ESavePackageResult ValidatePackage(FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_ValidatePackage);

	// Platform can't save the package
	if (!FPlatformProperties::HasEditorOnlyData())
	{
		return ESavePackageResult::Error;
	}

	// Check recursive save package call
	if (UE::IsSavingPackage() && !SaveContext.IsConcurrent())
	{
		ensureMsgf(false, TEXT("Recursive SavePackage() is not supported"));
		return ESavePackageResult::Error;
	}

	FString FilenameStr(SaveContext.GetFilename());

	/// Cooking checks
	if (SaveContext.IsCooking())
	{
#if WITH_EDITORONLY_DATA
		// if we strip editor only data, validate the package isn't referenced only by editor data
		// This check has to be done prior to validating the asset, because invalid state in the
		// package after stripping editoronly objects is okay if we're going to skip saving the whole package.
		if (SaveContext.IsStripEditorOnly())
		{
			// Don't save packages marked as editor-only.
			if (SaveContext.GetPackage()->HasAnyPackageFlags(PKG_EditorOnly))
			{
				UE_CLOGF(SaveContext.IsGenerateSaveError(), LogSavePackage, Verbose, "Package marked as editor-only: %ls. Package will not be saved.", *SaveContext.GetPackage()->GetName());
				return ESavePackageResult::ContainsEditorOnlyData;
			}
		}
#endif
	}

	UObject* Asset = SaveContext.GetAsset();
	if (Asset)
	{
		// If an asset is provided, validate it is in the package
		if (!Asset->IsInPackage(SaveContext.GetPackage()))
		{
			if (SaveContext.IsGenerateSaveError() && SaveContext.GetError())
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Name"), FText::FromString(FilenameStr));
				FText ErrorText = FText::Format(NSLOCTEXT("SavePackage2", "AssetSaveNotInPackage", "The Asset '{Name}' being saved is not in the provided package."), Arguments);
				SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *ErrorText.ToString());
			}
			return ESavePackageResult::Error;
		}

		// If An asset is provided, validate it has the requested TopLevelFlags. This is necessary to prevent dataloss, but only
		// when saving packages to the WorkspaceDomain
		EObjectFlags TopLevelFlags = SaveContext.GetTopLevelFlags();
		if (!SaveContext.IsCooking() && TopLevelFlags != RF_NoFlags && !Asset->HasAnyFlags(TopLevelFlags))
		{
			if (SaveContext.IsFixupStandaloneFlags() && !Asset->GetExternalPackage() && EnumHasAnyFlags(TopLevelFlags, RF_Standalone))
			{
				UE_LOGF(LogSavePackage, Warning, "The Asset %ls being saved is missing the RF_Standalone flag; adding it.", *Asset->GetPathName());
				Asset->SetFlags(RF_Standalone);
				check(Asset->HasAnyFlags(TopLevelFlags));
			}
			else
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Name"), FText::FromString(Asset->GetPathName()));
				static_assert(sizeof(TopLevelFlags) <= sizeof(uint32), "Expect EObjectFlags to be uint32");
				Arguments.Add(TEXT("Flags"), FText::FromString(FString::Printf(TEXT("%x"), (uint32)TopLevelFlags)));
				FText ErrorText;
				if (!Asset->GetExternalPackage() && EnumHasAnyFlags(TopLevelFlags, RF_Standalone))
				{
					ErrorText = FText::Format(NSLOCTEXT("SavePackage2", "AssetSaveMissingStandaloneFlag", "The Asset {Name} being saved does not have any of the provided object flags (0x{Flags}); saving the package would cause data loss. Run with -dpcvars=save.FixupStandaloneFlags=1 to add the RF_Standalone flag."),
						Arguments);
				}
				else
				{
					ErrorText = FText::Format(NSLOCTEXT("SavePackage2", "AssetSaveMissingTopLevelFlags", "The Asset {Name} being saved does not have any of the provided object flags (0x{Flags}); saving the package would cause data loss."),
						Arguments);
				}
				if (SaveContext.IsGenerateSaveError() && SaveContext.GetError())
				{
					SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *ErrorText.ToString());
				}
				else
				{
					UE_LOGF(LogSavePackage, Warning, "%ls", *ErrorText.ToString());
				}

				return ESavePackageResult::Error;
			}
		}
	}

	// Make sure package is allowed to be saved.
	if (!SaveContext.IsCooking() && FCoreUObjectDelegates::IsPackageOKToSaveDelegate.IsBound())
	{
		bool bIsOKToSave = FCoreUObjectDelegates::IsPackageOKToSaveDelegate.Execute(SaveContext.GetPackage(), SaveContext.GetFilename(), SaveContext.GetError());
		if (!bIsOKToSave)
		{
			if (SaveContext.IsGenerateSaveError())
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Name"), FText::FromString(FilenameStr));
				FText FormatText = SaveContext.GetPackage()->ContainsMap()
					? NSLOCTEXT("SavePackage2", "MapSaveNotAllowed", "Map '{Name}' is not allowed to save (see log for reason)")
					: NSLOCTEXT("SavePackage2", "AssetSaveNotAllowed", "Asset '{Name}' is not allowed to save (see log for reason)");
				FText ErrorText = FText::Format(FormatText, Arguments);
				SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *ErrorText.ToString());
			}
			return ESavePackageResult::Error;
		}
	}

	// Check if the package is fully loaded
	if (!SaveContext.GetPackage()->IsFullyLoaded())
	{
		if (SaveContext.IsGenerateSaveError())
		{
			// We cannot save packages that aren't fully loaded as it would clobber existing not loaded content.
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Name"), FText::FromString(FilenameStr));
			FText FormatText = SaveContext.GetPackage()->ContainsMap()
				? NSLOCTEXT("SavePackage2", "CannotSaveMapPartiallyLoaded", "Map '{Name}' cannot be saved as it has only been partially loaded")
				: NSLOCTEXT("SavePackage2", "CannotSaveAssetPartiallyLoaded", "Asset '{Name}' cannot be saved as it has only been partially loaded");
			FText ErrorText = FText::Format(FormatText, Arguments);
			SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *ErrorText.ToString());
		}
		return ESavePackageResult::Error;
	}

	// Warn about long package names, which may be bad for consoles with limited filename lengths.
	if (SaveContext.IsWarningLongFilename())
	{
		int32 MaxFilenameLength = FPlatformMisc::GetMaxPathLength();

		// If the name is of the form "_LOC_xxx.ext", remove the loc data before the length check
		FString BaseFilename = FPaths::GetBaseFilename(FilenameStr);
		FString CleanBaseFilename = BaseFilename;
		if (CleanBaseFilename.Find("_LOC_") == BaseFilename.Len() - 8)
		{
			CleanBaseFilename = BaseFilename.LeftChop(8);
		}
		if (CleanBaseFilename.Len() > MaxFilenameLength)
		{
			if (SaveContext.IsGenerateSaveError())
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("FileName"), FText::FromString(BaseFilename));
				Arguments.Add(TEXT("MaxLength"), FText::AsNumber(MaxFilenameLength));
				SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *FText::Format(NSLOCTEXT("Core", "Error_FilenameIsTooLongForCooking", "Filename '{FileName}' is too long; this may interfere with cooking for consoles. Unreal filenames should be no longer than {MaxLength} characters."), Arguments).ToString());
			}
			else
			{
				UE_LOGF(LogSavePackage, Warning, "%ls", *FString::Printf(TEXT("Filename is too long (%d characters); this may interfere with cooking for consoles. Unreal filenames should be no longer than %d characters. Filename value: %s"), BaseFilename.Len(), MaxFilenameLength, *BaseFilename));
			}
		}
	}
	return ReturnSuccessOrCancel();
}

void EnsurePackageLocalization(UPackage* InPackage, const FSaveContext& InSaveContext)
{
#if USE_STABLE_LOCALIZATION_KEYS
	if (GIsEditor)
	{
		bool bAlwaysResetPackageLocIdOnSave = false;
		GConfig->GetBool(TEXT("Internationalization"), TEXT("bAlwaysResetPackageLocIdOnSave"), bAlwaysResetPackageLocIdOnSave, GEditorIni);
		if (bAlwaysResetPackageLocIdOnSave && !InSaveContext.IsCooking())
		{
			// Package localization IDs are meant to be stable by default. But to automatically fix conflicts, having them reset on save can be useful in some contexts.
			// The downside being that text keys might get regenerated (they might feel unstable). Having translation memory with external tools can mitigate that downside though.
			TextNamespaceUtil::ResetPackageNamespace(InPackage);
		}
		else
		{
			// In any case, we need to ensure that we have a package localization namespace as the package loading will need it
			// We need to do this before entering the UE::IsSavingPackage() block as it may change the package meta-data
			TextNamespaceUtil::EnsurePackageNamespace(InPackage);
		}
	}
#endif // USE_STABLE_LOCALIZATION_KEYS
}

void PreSavePackage(FSaveContext& SaveContext)
{
#if WITH_EDITOR
	// If the in memory package filename is different the filename we are saving it to and if the save context allows for it,
	// regenerate a new persistent id for the package.
	UPackage* Package = SaveContext.GetPackage();
	if (!SaveContext.ShouldKeepPersistentGUID() && !Package->GetLoadedPath().IsEmpty() && Package->GetLoadedPath() != SaveContext.GetTargetPackagePath())
	{
		Package->SetPersistentGuid(FGuid::NewGuid());
	}
#endif //WITH_EDITOR
}

ESavePackageResult RoutePresave(FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_RoutePresave);

	// Just route presave on all objects in the package while skipping unsaveable objects. 
	// This should be more efficient then trying to restrict to just the actual export, 
	// objects likely to not be export will probably not care about PreSave and should be mainly noop
	TArray<UObject*> ObjectsInPackage;
	GetObjectsWithPackage(SaveContext.GetPackage(), ObjectsInPackage);

	FObjectSaveContextData& ObjectSaveContext = SaveContext.GetObjectSaveContext();
	for (UObject* Object : ObjectsInPackage)
	{
		// Do not emit unsaveable warning while routing presave. 
		// this is to prevent warning on objects which won't be harvested later since they are unreferenced
		if (!SaveContext.IsUnsaveable(Object, SaveContext.GetCachedObjectStatus(Object), false/*bEmitWarning*/))
		{
			ObjectSaveContext.Object = Object;
			if (SaveContext.IsCooking() && Object->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
			{
				FArchiveObjectCrc32 CrcArchive;
				CrcArchive.ArIsFilterEditorOnly = true;
				FString PathNameBefore = Object->GetPathName();
				int32 Before = CrcArchive.Crc32(Object);
				UE::SavePackageUtilities::CallPreSave(Object, ObjectSaveContext);
#if WITH_EDITOR
				UE::SavePackageUtilities::CallCookEventPlatformCookDependencies(Object, ObjectSaveContext);
#endif
				int32 After = CrcArchive.Crc32(Object);

				if (Before != After)
				{
					FString PathNameAfter = Object->GetPathName();
					if (PathNameBefore == PathNameAfter)
					{
						UE_ASSET_LOG(
							LogSavePackage,
							Warning,
							Object,
							TEXT("Non-deterministic cook warning - PreSave() has modified %s '%s' - a resave may be required"),
							Object->HasAnyFlags(RF_ClassDefaultObject) ? TEXT("CDO") : TEXT("archetype"),
							*Object->GetName()
						);
					}
					else
					{
						UObject* ObjectUsedToReport = Object->IsInPackage(SaveContext.GetPackage()) ? Object : SaveContext.GetPackage();
						UE_ASSET_LOG(
							LogSavePackage,
							Warning,
							ObjectUsedToReport,
							TEXT("Non-deterministic cook warning - PreSave() has renamed %s '%s' to %s - a resave may be required"),
							Object->HasAnyFlags(RF_ClassDefaultObject) ? TEXT("CDO") : TEXT("archetype"),
							*PathNameBefore, *PathNameAfter
						);
					}

				}
			}
			else
			{
				UE::SavePackageUtilities::CallPreSave(Object, ObjectSaveContext);
#if WITH_EDITOR
				UE::SavePackageUtilities::CallCookEventPlatformCookDependencies(Object, ObjectSaveContext);
#endif
			}
		}
	}

	return ReturnSuccessOrCancel();
}

AUTORTFM_DISABLE ESavePackageResult HarvestPackage(FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_Save_HarvestPackage);

	FPackageHarvester Harvester(SaveContext);
	EObjectFlags TopLevelFlags = SaveContext.GetTopLevelFlags();
	UObject* Asset = SaveContext.GetAsset();

	auto TryHarvestRootObject = [&Harvester, &SaveContext](UObject* InRoot)
	{
		// ForEachObjectWithPackage will filter out Transient objects but not objects with the bForceTransient Save Override
		// Perform one more check to cover this case
		UE::SavePackageUtilities::FObjectStatus& ObjectStatus = SaveContext.GetCachedObjectStatus(InRoot);
		if (SaveContext.IsTransient(InRoot, ObjectStatus))
		{
			return;
		}

		Harvester.TryHarvestExport(InRoot, ObjectStatus);
		// if we are automatically generating an optional package output, re-harvest objects with that realm as the default
		if (SaveContext.IsSaveAutoOptional())
		{
			//@todo: FH: the top level might have to be altered to use the editor one when auto harvesting optional data
			FSaveContext::FSetSaveRealmToSaveScope RealmScope(SaveContext, ESaveRealm::Optional);
			Harvester.TryHarvestExport(InRoot, ObjectStatus);
		}
	};

	// if no top level flags are passed, only use the provided package asset as root
	if (TopLevelFlags == RF_NoFlags)
	{
		TryHarvestRootObject(Asset);
	}
	// Otherwise use all objects which have the relevant flags
	else
	{
		ForEachObjectWithPackage(SaveContext.GetPackage(), [&TryHarvestRootObject, TopLevelFlags](UObject* InObject)
			{
				if (InObject->HasAnyFlags(TopLevelFlags))
				{
					TryHarvestRootObject(InObject);
				}
				return true;
			}, EGetObjectsFlags::IncludeNestedObjects, RF_Transient);
	}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
	if (Verse::VPackage* VersePackage = Verse::GlobalProgram->LookupPackage(Context, SaveContext.GetPackage()))
	{
		Harvester.TryHarvestCellExport(VersePackage);
	}
#endif
	// Now process harvested roots
	while (FPackageHarvester::FExportWithContext ExportContext = Harvester.PopExportToProcess())
	{
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		if (ExportContext.Export)
		{
			Harvester.ProcessExport(ExportContext);
		}
		else if (ExportContext.CellExport)
		{
			Harvester.ProcessCellExport(ExportContext);
		}
#else
		Harvester.ProcessExport(ExportContext);
#endif
	}

	// If we have a valid optional context and we are saving it,
	// transform any harvested non optional export into imports
	// Mark other optional import package as well
	if (!SaveContext.IsSaveAutoOptional() &&
		SaveContext.IsSaveOptional() &&
		SaveContext.IsCooking() &&
		SaveContext.GetHarvestedRealm(ESaveRealm::Optional).GetExports().Num() &&
		SaveContext.GetHarvestedRealm(ESaveRealm::Game).GetExports().Num())
	{
		bool bHasNonOptionalSelfReference = false;
		FHarvestedRealm& OptionalContext = SaveContext.GetHarvestedRealm(ESaveRealm::Optional);
		for (auto It = OptionalContext.GetExports().CreateIterator(); It; ++It)
		{
			if (!FPackageHarvester::ShouldObjectBeHarvestedInOptionalRealm(It->Obj, SaveContext))
			{
				// Make sure the export is found in the game context as well
				if (FTaggedExport* GameExport = SaveContext.GetHarvestedRealm(ESaveRealm::Game).GetExports().Find(It->Obj))
				{
					// Flagging the export in the game context to generate it's public hash isn't necessary anymore
					//GameExport->bGeneratePublicHash = true; 

					// Transform the export as an import
					OptionalContext.AddImport(It->Obj);
					// Flag the package itself to be an import
					bHasNonOptionalSelfReference = true;
					It.RemoveCurrent();
				}
				// if not found in the game context and the reference directly came from an optional object, record an illegal reference
				else if (It->bFromOptionalReference)
				{
					// TODO: Suppress this error and leave the object in the optional realm on Server builds, because it does not work in
					// some cases on multiplatform cooks. EditorOptional is a global commandline setting that applies to all platforms in
					// a multiplatform cook, but on some projects the project cooks Client and Server together and wants editoroptional
					// only on client, so it cooks both of them as editoroptional. But then server doesn't support editoroptional for
					// objects that are in ClassesExcludedOnDedicatedServer but that are referenced both from optional and from gamerealm
					// objects; we get this error because the object is stripped from the game realm but remains in the optional realm.
					// 
					// Change EditorOptional to be specifiable per platform, and remove this suppression.
					if (!SaveContext.GetTargetPlatform()->IsServerOnly())
					{
						SaveContext.RecordIllegalReference(nullptr, It->Obj, EIllegalRefReason::ReferenceFromOptionalToMissingGameExport);
						It.RemoveCurrent();
					}
				}
				else
				{
					It.RemoveCurrent();
				}
			}
		}
		// Also add the current package itself as an import if we are referencing any non optional export
		if (bHasNonOptionalSelfReference)
		{
			OptionalContext.AddImport(SaveContext.GetPackage());
		}
	}

	{
		FPackageHarvester::FHarvestScope RootReferenceScope = Harvester.EnterRootReferencesScope();
		// Trim PrestreamPackage list 
		TSet<TObjectPtr<UPackage>>& PrestreamPackages = SaveContext.GetPrestreamPackages();
		TSet<TObjectPtr<UPackage>> KeptPrestreamPackages;
		for (TObjectPtr<UPackage> Pkg : PrestreamPackages)
		{
			// If the prestream package hasn't been otherwise already marked as an import, keep it as such and mark it as an import
			if (!SaveContext.IsImport(Pkg))
			{
				KeptPrestreamPackages.Add(Pkg);
				Harvester << Pkg;
			}
		}
		Exchange(PrestreamPackages, KeptPrestreamPackages);

		// Harvest the PrestreamPackage class name if needed
		if (PrestreamPackages.Num() > 0)
		{
			Harvester.HarvestPackageHeaderName(UE::SavePackageUtilities::NAME_PrestreamPackage);
		}

#if WITH_METADATA
		// Harvest package MetaData
		if (!SaveContext.IsCooking())
		{
			FMetaData& PackageMetaData = SaveContext.GetPackage()->GetMetaData();
			Harvester << PackageMetaData;
		}
#endif

		// if we have a WorldTileInfo, we need to harvest its dependencies as well, i.e. Custom Version
		if (FWorldTileInfo* WorldTileInfo = SaveContext.GetPackage()->GetWorldTileInfo())
		{
			Harvester << *WorldTileInfo;
		}
	}

	// The Editor version is used as part of the check to see if a package is too old to use the gather cache, so we always have to add it if we have gathered loc for this asset
	// We need to set the editor custom version before we copy the version container to the summary, otherwise we may end up with corrupt assets
	// because we later do it on the Linker when actually gathering loc data
	if (!SaveContext.IsFilterEditorOnly())
	{
		Harvester.UsingCustomVersion(FEditorObjectVersion::GUID);
	}
	SaveContext.SetCustomVersions(Harvester.GetCustomVersions());
	SaveContext.SetTransientPropertyOverrides(Harvester.ReleaseTransientPropertyOverrides());

	if (SaveContext.IsCooking())
	{
		SaveContext.AddExportedClassesToDependencies();
	}

	// Contractually all BuildResultDependencies (which come from PreSave or from Serialize) must be
	// declared by this point. Copy PackageBuildDependencies from those lists and implement our contract that
	// PackageBuildDependencies are in the name map.
	SaveContext.UpdateEditorRealmPackageBuildDependencies();

	return ReturnSuccessOrCancel();
}

ESavePackageResult ValidateRealms(FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_Save_ValidateRealms);
	if (SaveContext.GetIllegalReferences().Num())
	{
		for (const FIllegalReference& Reference : SaveContext.GetIllegalReferences())
		{
			FString ErrorMessage;
			switch (Reference.Reason)
			{
			case EIllegalRefReason::ReferenceToOptional:
				ErrorMessage = FString::Printf(TEXT("Can't save %s: Non-optional object (%s) has a reference to optional object (%s). Only optional objects can refer to other optional objects."),
					SaveContext.GetFilename(),
					Reference.From ? *Reference.From->GetPathName() : TEXT("Unknown"),
					Reference.To ? *Reference.To->GetPathName() : TEXT("Unknown"));
				break;
			case EIllegalRefReason::ReferenceFromOptionalToMissingGameExport:
				ErrorMessage = FString::Printf(TEXT("Can't save %s: Optional object (%s) has a reference to cooked object (%s) which is missing. Non optional objects referenced by optional objects needs to be present in cooked data."),
					SaveContext.GetFilename(),
					Reference.From ? *Reference.From->GetPathName() : TEXT("Unknown"),
					Reference.To ? *Reference.To->GetPathName() : TEXT("Unknown"));
				break;
			case EIllegalRefReason::UnsaveableClass:
				ErrorMessage = FString::Printf(TEXT("Can't save %s: Object (%s) is an export but is an instance of class (%s) which is unsaveable: %s"),
					SaveContext.GetFilename(),
					Reference.From ? *Reference.From->GetPathName() : TEXT("Unknown"),
					Reference.To ? *Reference.To->GetPathName() : TEXT("Unknown"),
					*Reference.FormatStringArg);
				break;
			case EIllegalRefReason::UnsaveableOuter:
				ErrorMessage = FString::Printf(TEXT("Can't save %s: Object (%s) is an export but is a subobject of (%s) which is unsaveable: %s"),
					SaveContext.GetFilename(),
					Reference.From ? *Reference.From->GetPathName() : TEXT("Unknown"),
					Reference.To ? *Reference.To->GetPathName() : TEXT("Unknown"),
					*Reference.FormatStringArg);
				break;
			case EIllegalRefReason::ExternalPackage:
				if (Reference.From && Reference.To && Reference.From->GetOutermostObject() == Reference.To->GetOutermostObject() && SaveContext.IsCooking())
				{
					ErrorMessage = FString::Printf(TEXT("Can't save %s: export (%s) has a reference to export (%s) which still has its external package set to (%s)."),
						SaveContext.GetFilename(),
						Reference.From ? *Reference.From->GetPathName() : TEXT("Unknown"),
						Reference.To ? *Reference.To->GetPathName() : TEXT("Unknown"),
						!Reference.FormatStringArg.IsEmpty() ? *Reference.FormatStringArg : TEXT("Unknown"));
				}
				else
				{
					ErrorMessage = FString::Printf(TEXT("Can't save %s: export (%s) has a reference to import (%s), but the import is in ExternalPackage (%s) which was marked unsaveable."),
						SaveContext.GetFilename(),
						Reference.From ? *Reference.From->GetPathName() : TEXT("Unknown"),
						Reference.To ? *Reference.To->GetPathName() : TEXT("Unknown"),
						!Reference.FormatStringArg.IsEmpty() ? *Reference.FormatStringArg : TEXT("Unknown"));
				}
				break;
			default:
				ErrorMessage = FString::Printf(TEXT("Can't save %s: Unknown Illegal reference from object (%s) to object (%s)"),
					SaveContext.GetFilename(),
					Reference.From ? *Reference.From->GetPathName() : TEXT("Unknown"),
					Reference.To ? *Reference.To->GetPathName() : TEXT("Unknown"));
			}

			if (SaveContext.IsGenerateSaveError())
			{
				SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *ErrorMessage);
			}
			else
			{
				UE_LOGF(LogSavePackage, Error, "%ls", *ErrorMessage);
			}

		}
		return ESavePackageResult::Error;
	}
	return ReturnSuccessOrCancel();
}

ESavePackageResult ValidateExports(FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_Save_ValidateExports);

	/// Export Validation for optional realm
	if (SaveContext.GetCurrentHarvestingRealm() == ESaveRealm::Optional)
	{
		// return empty realm to skip processing if no export are found
		if (SaveContext.GetExports().Num() == 0)
		{
			return ESavePackageResult::EmptyRealm;
		}
		return ReturnSuccessOrCancel();
	}

	/// Export validation for game/editor realm
	// Check if we gathered any exports
	if (SaveContext.GetExports().Num() == 0 && SaveContext.GetCellExports().Num() == 0)
	{
		UE_CLOGF(SaveContext.IsGenerateSaveError(), LogSavePackage, Verbose, "No exports found (or all exports are editor-only) for %ls. Package will not be saved.", SaveContext.GetFilename());
		return SaveContext.IsCooking() ? ESavePackageResult::ContainsEditorOnlyData : ESavePackageResult::Error;
	}

	// Validate that if an asset was provided it had the proper flags to be present in the exports.
	UObject* Asset = SaveContext.GetAsset();
	if (Asset)
	{
		if (!SaveContext.GetExports().Contains(FTaggedExport{ Asset }) &&
			SaveContext.GetTopLevelFlags() != RF_NoFlags && !Asset->HasAnyFlags(SaveContext.GetTopLevelFlags()))
		{
			FString ErrorMessage = FString::Printf(
				TEXT("The asset to save %s in package %s does not contain any of the provided object flags."),
				*Asset->GetName(), *SaveContext.GetPackage()->GetName());
			if (SaveContext.IsGenerateSaveError())
			{
				SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *ErrorMessage);
			}
			else
			{
				UE_LOGF(LogSavePackage, Error, "%ls", *ErrorMessage);
			}
			return ESavePackageResult::Error;
		}
	}

	// If this is a map package, make sure there is a world or level in the export map.
	if (SaveContext.GetPackage()->ContainsMap())
	{
		bool bContainsMap = false;
		for (const FTaggedExport& Export : SaveContext.GetExports())
		{
			UObject* Object = Export.Obj;
			// Consider redirectors to world/levels as map packages too.
			while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(Object))
			{
				Object = Redirector->DestinationObject;
			}
			if (Object)
			{
				FName ClassName = Object->GetClass()->GetFName();
				bContainsMap |= (ClassName == UE::SavePackageUtilities::NAME_World || ClassName == UE::SavePackageUtilities::NAME_Level);
			}
		}
		if (!bContainsMap)
		{
			ensureMsgf(false, TEXT("Attempting to save a map package '%s' that does not contain a map object."), *SaveContext.GetPackage()->GetName());
			UE_LOGF(LogSavePackage, Error, "Attempting to save a map package '%ls' that does not contain a map object.", *SaveContext.GetPackage()->GetName());

			if (SaveContext.IsGenerateSaveError())
			{
				SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *FText::Format(NSLOCTEXT("Core", "SavePackageNoMap", "Attempting to save a map asset '{0}' that does not contain a map object"), FText::FromString(SaveContext.GetFilename())).ToString());
			}
			return ESavePackageResult::Error;
		}
	}

	// Validate there's no placeholder object in the export list
#if WITH_EDITOR
	if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObjectSupportEnabled())
	{
		if (!UE::FPropertyBagRepository::IsPropertyBagPlaceholderObjectFeatureEnabled(UE::FPropertyBagRepository::EPlaceholderObjectFeature::AllowSavingPlaceholderExports))
		{
			TArray<const UObject*, TInlineAllocator<16>> PlaceholderExports;
			const TSet<FTaggedExport>& AllExports = SaveContext.GetExports();
			for (const FTaggedExport& Export : AllExports)
			{
				if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderObject(Export.Obj))
				{
					PlaceholderExports.Add(Export.Obj);
				}
				else if (const UStruct* TypeExport = Cast<UStruct>(Export.Obj))
				{
					if (UE::FPropertyBagRepository::IsPropertyBagPlaceholderType(TypeExport))
					{
						PlaceholderExports.Add(TypeExport);
					}
				}
			}

			if (PlaceholderExports.Num() == AllExports.Num())
			{
				// If all exports are placeholders, allow this to pass as editor-only data (e.g. a placeholder type package).
				return ESavePackageResult::ContainsEditorOnlyData;
			}
			else if (PlaceholderExports.Num() > 0)
			{
				for (const UObject* PlaceholderExport : PlaceholderExports)
				{
					check(PlaceholderExport);
					UObject* Outer = PlaceholderExport->GetOuter();
					FString ErrorMessage = FString::Printf(TEXT("Saving illegal placeholder object in package '%s'. Object Name '%s', Class '%s', Outer '%s', Outer Class '%s', Full Path '%s'. You might have unresolved classes or compilation failures to fix."), *SaveContext.GetPackage()->GetName(), *PlaceholderExport->GetName(), *PlaceholderExport->GetClass()->GetName(), Outer ? *Outer->GetName() : TEXT("None"), Outer ? *Outer->GetClass()->GetName() : TEXT("None"), *PlaceholderExport->GetFullName());
					if (SaveContext.IsGenerateSaveError())
					{
						SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *ErrorMessage);
					}
					else
					{
						UE_LOGF(LogSavePackage, Error, "%ls", *ErrorMessage);
					}
				}

				return ESavePackageResult::Error;
			}
		}
	}
#endif

	// Validate External Export Rules
	if (SaveContext.HasExternalExportValidations())
	{
		TSet<UObject*> Exports;
		Algo::Transform(SaveContext.GetExports(), Exports, [](const FTaggedExport& InExport) { return InExport.Obj; });

		FExportsValidationContext::EFlags Flags = SaveContext.IsCooking()
			? FExportsValidationContext::EFlags::IsCooking
			: FExportsValidationContext::EFlags::None;
		FOutputDevice* OutputDevice = SaveContext.IsGenerateSaveError() ? SaveContext.GetError() : nullptr;
		for (const TFunction<FSavePackageSettings::ExternalExportValidationFunc>& ValidateExport : SaveContext.GetExternalExportValidations())
		{
			SaveContext.Result = ValidateExport({ SaveContext.GetPackage(), Exports, SaveContext.GetObjectSaveContext().SaveOverrides, Flags, SaveContext.GetTargetPlatform(), OutputDevice });
			if (SaveContext.Result != ESavePackageResult::Success)
			{
				return SaveContext.Result;
			}
		}
	}

	// Cooking checks
#if WITH_EDITOR && !UE_FNAME_OUTLINE_NUMBER
	if (GetOutputCookingWarnings())
	{
		// check the name list for UniqueObjectNameForCooking cooking. Since check is a fast check for any NAME_UniqueObjectNameForCooking
		// we can only perform this check when UE_FNAME_OUTLINE_NUMBER=0 as otherwise all prefixed names will have a unique ComparisonIndex
		if (SaveContext.NameExists(NAME_UniqueObjectNameForCooking))
		{
			const FNameEntryId UniqueObjectNameForCookingComparisonId = NAME_UniqueObjectNameForCooking.GetComparisonIndex();
			for (const FTaggedExport& Export : SaveContext.GetExports())
			{
				const FName NameInUse = Export.Obj->GetFName();
				if (NameInUse.GetComparisonIndex() == UniqueObjectNameForCookingComparisonId)
				{
					UObject* Outer = Export.Obj->GetOuter();
					UE_LOGF(LogSavePackage, Warning, "Saving object into cooked package %ls which was created at cook time, Object Name %ls, Full Path %ls, Class %ls, Outer %ls, Outer class %ls", SaveContext.GetFilename(), *NameInUse.ToString(), *Export.Obj->GetFullName(), *Export.Obj->GetClass()->GetName(), Outer ? *Outer->GetName() : TEXT("None"), Outer ? *Outer->GetClass()->GetName() : TEXT("None"));
				}
			}
		}
	}
#endif

	return ReturnSuccessOrCancel();
}

ESavePackageResult ValidateIllegalReferences(FSaveContext& SaveContext, TArray<UObject*>& PrivateObjects,
	TArray<UObject*>& PrivateContentObjects, TArray<UObject*>& ObjectsInOtherMaps,
	ESaveRealm HarvestedRealm)
{
	FFormatNamedArguments Args;

	TArray<UObject*>* ObjectsToCheck = nullptr;
	FString ErrorPrologue;
	FString ErrorEpilogue;

	// Illegal objects in other map warning
	if (ObjectsInOtherMaps.Num() > 0)
	{
		ObjectsToCheck = &ObjectsInOtherMaps;
		ErrorPrologue = FString::Printf(TEXT("Can't save '%s': Illegal reference to private object: "), SaveContext.GetFilename());
		ErrorEpilogue = FString(" (private object belongs to an external map).");
	}
	else if (PrivateObjects.Num() > 0)
	{
		ObjectsToCheck = &PrivateObjects;
		ErrorPrologue = FString::Printf(TEXT("Can't save '%s': Illegal reference to private object: "), SaveContext.GetFilename());
		ErrorEpilogue = FString(".");
	}
	else if (PrivateContentObjects.Num() > 0)
	{
		ObjectsToCheck = &PrivateContentObjects;
		ErrorPrologue = FString::Printf(TEXT("Can't save package: PKG_NotExternallyReferenceable: Package '%s' imports object: "), SaveContext.GetFilename());
		ErrorEpilogue = FString(", which is in a different mount point and its package is marked as PKG_NotExternallyReferenceable.");
	}

	if (ObjectsToCheck)
	{
		UObject* MostLikelyCulprit = nullptr;
		UObject* Referencer = nullptr;
		const FProperty* ReferencerProperty = nullptr;
		bool IsCulpritArchetype;
		FString CulpritDiagnostics;
		UE::SavePackageUtilities::FindMostLikelyCulprit(*ObjectsToCheck, SaveContext, HarvestedRealm, MostLikelyCulprit,
			Referencer, ReferencerProperty, IsCulpritArchetype, CulpritDiagnostics);

		UObject* ReferencerOuter = Referencer ? Referencer->GetOuter() : nullptr;

		FString ReferencerStr = Referencer ? Referencer->GetName() : TEXT("Unknown referencer");
		FString ReferencerPropertyStr = ReferencerProperty ? *ReferencerProperty->GetName() : TEXT("Unknown property");
		FString CulpritStr = MostLikelyCulprit ? *MostLikelyCulprit->GetFullName() : TEXT("Unknown");
		FString ReferencerOuterStr = ReferencerOuter ? ReferencerOuter->GetPathName() : TEXT("Unknown owner");

		FString ReferenceDescription;

		if (IsCulpritArchetype)
		{
			ReferenceDescription = FString::Printf(TEXT("'%s' referenced because it is an archetype object"), *CulpritStr);
		}
		else
		{
			ReferenceDescription = FString::Printf(TEXT("'%s' referenced by '%s' (at '%s') in its '%s' property"),
				*CulpritStr, *ReferencerStr, *ReferencerOuterStr, *ReferencerPropertyStr);
		}

		FString ErrorMessage = FString::Printf(TEXT("%s%s%s"), *ErrorPrologue, *ReferenceDescription, *ErrorEpilogue);

		UE_LOGF(LogSavePackage, Warning, "%ls", *CulpritDiagnostics);
		if (SaveContext.IsGenerateSaveError())
		{
			SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *ErrorMessage);
		}
		else
		{
			UE_LOGF(LogSavePackage, Error, "%ls", *ErrorMessage);
		}
		return ESavePackageResult::Error;
	}
	else
	{
		return ReturnSuccessOrCancel();
	}
}

ESavePackageResult ValidateImports(FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_Save_ValidateImports);

	TArray<UObject*> TopLevelObjects;
	UPackage* Package = SaveContext.GetPackage();
	GetObjectsWithPackage(Package, TopLevelObjects, EGetObjectsFlags::None);
	auto IsInAnyTopLevelObject = [&TopLevelObjects](UObject* InObject) -> bool
	{
		for (UObject* TopObject : TopLevelObjects)
		{
			if (InObject->IsInOuter(TopObject))
			{
				return true;
			}
		}
		return false;
	};
	auto AnyTopLevelObjectIsIn = [&TopLevelObjects](UObject* InObject) -> bool
	{
		for (UObject* TopObject : TopLevelObjects)
		{
			if (TopObject->IsInOuter(InObject))
			{
				return true;
			}
		}
		return false;
	};
	auto AnyTopLevelObjectHasSameOutermostObject = [&TopLevelObjects](UObject* InObject) -> bool
	{
		UObject* Outermost = InObject->GetOutermostObject();
		for (UObject* TopObject : TopLevelObjects)
		{
			if (TopObject->GetOutermostObject() == Outermost)
			{
				return true;
			}
		}
		return false;

	};
	auto IsSourcePackageReferenceAllowed = [](UPackage* InSourcePackage, UPackage* InImportPackage) -> bool
	{
#if WITH_EDITORONLY_DATA
		// Generated packages must have the same persistent GUID as their source package
		if (InSourcePackage->GetPersistentGuid() == InImportPackage->GetPersistentGuid())
		{
			// Generated packages can reference into their source package, or into other generated packages of the same source
			if (InSourcePackage->HasAnyPackageFlags(PKG_CookGenerated))
			{
				return true;
			}
		}
#endif
		return false;
	};

	auto IsMapReferenceAllowed = [&SaveContext](UObject* InImport) -> bool
	{
		// If we have at least one export that is outered to an object in the import's package, consider the reference as allowed.
		// Ideally, we would like to only allow imports from exports that shares an outer in the same package, but this will be ok
		// for now.
		for (const FTaggedExport& Export : SaveContext.GetExports())
		{
			if (Export.Obj->GetOuter()->GetPackage() == InImport->GetPackage())
			{
				return true;
			}
		}

		if (!InImport->HasAnyFlags(RF_Public))
		{
			return false;
		}

		// If we have a public import from a map (i.e. the world) only redirector are allowed to have a hard reference
		for (const auto& Pair : SaveContext.GetObjectDependencies())
		{
			if (Pair.Key->GetClass() != UObjectRedirector::StaticClass())
			{
				if (Pair.Value.Contains(InImport))
				{
					return false;
				}
			}
		}
		return true;
	};

	FString PackageName = Package->GetName();

	// Warn for private objects & map object references
	TArray<UObject*> PrivateObjects;
	TArray<UObject*> PrivateContentObjects;
	TArray<UObject*> ObjectsInOtherMaps;
	const TSet<TObjectPtr<UObject>>& Imports = SaveContext.GetImports();
	ESaveRealm HarvestedRealm = SaveContext.GetCurrentHarvestingRealm();
	const TSet<TObjectPtr<UObject>>& DirectImports = SaveContext.GetDirectImports();
	UE_COOK_RESULTPROJECTION_SCOPED(UE::Cook::ResultProjection::PackageAndClass);
	for (const TObjectPtr<UObject>& Import : Imports)
	{
		TObjectPtr<UPackage> ImportPackage = Import.GetPackage();
		// All names should be properly harvested at this point
		ensureAlwaysMsgf(SaveContext.NameExists(Import.GetFName())
			, TEXT("Missing import name %s while saving package %s. Did you rename an import during serialization?"), *Import->GetName(), *PackageName);
		ensureAlwaysMsgf(SaveContext.NameExists(ImportPackage.GetFName())
			, TEXT("Missing import package name %s while saving package %s. Did you rename an import during serialization?"), *ImportPackage->GetName(), *PackageName);
		ensureAlwaysMsgf(SaveContext.NameExists(Import.GetClass()->GetFName())
			, TEXT("Missing import class name %s while saving package %s"), *Import->GetClass()->GetName(), *PackageName);
		ensureAlwaysMsgf(SaveContext.NameExists(Import.GetClass()->GetOuter()->GetFName())
			, TEXT("Missing import class package name %s while saving package %s"), *Import.GetClass()->GetOuter()->GetName(), *PackageName);

		// if the import is marked as a prestream package, we dont need to validate further
		if (SaveContext.IsPrestreamPackage(ImportPackage))
		{
			ensureAlwaysMsgf(Import == ImportPackage, TEXT("Found an import refrence %s in a prestream package %s while saving package %s"), *Import.GetName(), *ImportPackage.GetName(), *PackageName);
			// These are not errors
			UE_LOGF(LogSavePackage, Display, "Prestreaming package %ls ", *ImportPackage.GetPathName());
			continue;
		}

		// if an import outer is an export and that import doesn't have a specific package set then, there's an error
		const bool bWrongImport = Import->GetOuter()
			&& Import->GetOuter()->IsInPackage(SaveContext.GetPackage())
			&& Import->GetExternalPackage() == nullptr
			// The optional context will have import that are actually in the same package, similar to external package
			&& SaveContext.GetCurrentHarvestingRealm() != ESaveRealm::Optional;
		if (bWrongImport)
		{
			if (!SaveContext.IsTransient(Import) || !Import->IsNative())
			{
				UE_LOGF(LogSavePackage, Warning, "Bad Object=%ls", *Import->GetFullName());
			}
			else
			{
				// if an object is marked RF_Transient and native, it is either an intrinsic class or
				// a property of an intrinsic class.  Only properties of intrinsic classes will have
				// an Outer that passes the check for "GetOuter()->IsInPackage(InOuter)" (thus ending up in this
				// block of code).  Just verify that the Outer for this property is also marked RF_Transient and Native
				check(SaveContext.IsTransient(Import->GetOuter()) && Import->GetOuter()->IsNative());
			}
		}
		check(!bWrongImport || SaveContext.IsTransient(Import) || Import->IsNative());

		// if this import shares a outer with top level object of this package then the reference is acceptable
		if ((!SaveContext.IsCooking() || SaveContext.GetCurrentHarvestingRealm() == ESaveRealm::Optional) &&
			(IsInAnyTopLevelObject(Import) || AnyTopLevelObjectIsIn(Import) || AnyTopLevelObjectHasSameOutermostObject(Import)))
		{
			continue;
		}

		// Allow private imports for split packages into their source package
		if (!IsSourcePackageReferenceAllowed(Package, ImportPackage))
		{
			// See whether the object we are referencing is in another map package and if it is allowed (i.e. from redirector).
			if (ImportPackage->ContainsMap() && !IsMapReferenceAllowed(Import))
			{
				ObjectsInOtherMaps.Add(Import);
			}

			if (!Import->HasAnyFlags(RF_Public) && (!SaveContext.IsCooking() || !ImportPackage->HasAnyPackageFlags(PKG_CompiledIn)))
			{
				PrivateObjects.Add(Import);
			}
		}

		// Enforce that Private content can only be directly referenced by something within the same Mount Point
		// This only applies to direct imports. Transitive imports (A -> PackageB.B, B -> B.Class -> PackageC) are
		// allowed even for private content.
		if ((ImportPackage->GetAssetAccessSpecifier() == EAssetAccessSpecifier::Private) && DirectImports.Contains(Import))
		{
			FName MountPointName = FPackageName::GetPackageMountPoint(PackageName);

			FName ImportMountPointName = FPackageName::GetPackageMountPoint(FNameBuilder(ImportPackage->GetFName()).ToView());

			if (!MountPointName.IsNone() && !ImportMountPointName.IsNone())
			{
				if (MountPointName != ImportMountPointName)
				{
					PrivateContentObjects.Add(Import);
				}
			}
			else
			{
				PrivateContentObjects.Add(Import);
			}
		}
	}
	if (PrivateObjects.Num() > 0 || PrivateContentObjects.Num() > 0 || ObjectsInOtherMaps.Num() > 0)
	{
		return ValidateIllegalReferences(SaveContext, PrivateObjects, PrivateContentObjects, ObjectsInOtherMaps, HarvestedRealm);
	}

	// Validate External Import Rules
	if (SaveContext.HasExternalImportValidations())
	{
		for (const TFunction<FSavePackageSettings::ExternalImportValidationFunc>& ValidateImport : SaveContext.GetExternalImportValidations())
		{
			SaveContext.Result = ValidateImport({ SaveContext.GetPackage(), SaveContext.GetImports(), SaveContext.IsGenerateSaveError() ? SaveContext.GetError() : nullptr });
			if (SaveContext.Result != ESavePackageResult::Success)
			{
				return SaveContext.Result;
			}
		}
	}

	return ReturnSuccessOrCancel();
}

ESavePackageResult CreateLinker(FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_Save_CreateLinkerSave);
	const FString BaseFilename = FPaths::GetBaseFilename(SaveContext.GetFilename());

	const bool bIsOptionalRealm = SaveContext.GetCurrentHarvestingRealm() == ESaveRealm::Optional;
	const bool bByteSwapping = SaveContext.IsForceByteSwapping();
	const bool bSaveUnversioned = SaveContext.IsSaveUnversionedNative();
	UPackage* const Package = SaveContext.GetPackage();
	const FName PackageName = Package->GetFName();
	IPackageWriter* PackageWriter = SaveContext.GetPackageWriter();

	TPimplPtr<FLinkerSave> LinkerSave = MakePimpl<FLinkerSave>(Package);
	if (SaveContext.IsGenerateSaveError())
	{
		LinkerSave->SetOutputDevice(SaveContext.GetError());
	}
	LinkerSave->AssignSaveContext(&SaveContext);
	LinkerSave->SetTransientPropertyOverrides(SaveContext.GetTransientPropertyOverrides());
	LinkerSave->bUpdatingLoadedPath = SaveContext.IsUpdatingLoadedPath();
	LinkerSave->bProceduralSave = SaveContext.IsProceduralSave();
	LinkerSave->bRehydratePayloads = SaveContext.ShouldRehydratePayloads();

	if (PackageWriter)
	{
		check(!SaveContext.IsTextFormat()); // TextFormat not supported with a PackageWriter

		uint16 MultiOutputIndex = bIsOptionalRealm ? 1 : 0;
		TUniquePtr<FLargeMemoryWriter> ExportsArchive =
			PackageWriter->CreateLinkerArchive(PackageName, SaveContext.GetAsset(), MultiOutputIndex);
		LinkerSave->AssignSaver(ExportsArchive.Release(), bByteSwapping, bSaveUnversioned);
		SaveContext.SetFormatter(MakeUnique<FBinaryArchiveFormatter>(*LinkerSave));
	}
	else
	{
		check(!SaveContext.IsCooking()); // IsCooking requires a PackageWriter
		check(!bIsOptionalRealm); // OptionalRealm is only supported when cooking.

		if (SaveContext.IsSaveToMemory())
		{
			check(!SaveContext.IsTextFormat()); // TextFormat not supported with SaveToMemory

			TUniquePtr<FLargeMemoryWriter> ExportsArchive(new FLargeMemoryWriter(
				0 /* PreAllocateBytes */,
				false /* bPersistent */, // LinkerSave wraps this archive and reports Persistent=true
				*PackageName.ToString()));
			LinkerSave->AssignSaver(ExportsArchive.Release(), bByteSwapping, bSaveUnversioned);
			SaveContext.SetFormatter(MakeUnique<FBinaryArchiveFormatter>(*LinkerSave));
		}
		else if (!SaveContext.IsTextFormat())
		{
			// Make a temp filename. CreateTempFilename guarantees unique, non-existing filename.
			// The temp file will be saved in the game save folder to not have to deal with potentially too long paths.
			// Since the temp filename may include a 32 character GUID as well, limit the user prefix to 32 characters.
			FString TempFilename = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), *BaseFilename.Left(32));
			SaveContext.SetTempFilename(TempFilename);
			if (!LinkerSave->TryAssignFileSaver(TempFilename, bByteSwapping, bSaveUnversioned))
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("Name"), FText::FromString(*TempFilename));
				FText ErrorText = FText::Format(
					NSLOCTEXT("SavePackage", "CouldNotCreateSaveFile", "Could not create temporary save filename {Name}."),
					Arguments);
				UE_LOGF(LogSavePackage, Error, "%ls", *ErrorText.ToString());
				if (SaveContext.IsGenerateSaveError())
				{
					SaveContext.GetError()->Logf(ELogVerbosity::Error, TEXT("%s"), *ErrorText.ToString());
				}
				return ESavePackageResult::Error;
			}
			SaveContext.SetFormatter(MakeUnique<FBinaryArchiveFormatter>(*LinkerSave));
		}
		else // IsTextFormat
		{
#if !WITH_TEXT_ARCHIVE_SUPPORT
			UE_LOGF(LogSavePackage, Error, "Could not save package to text format. Text Archive Support is not compiled into this version of the editor executable.");
			return ESavePackageResult::Error;
#else
			// TODO: Remove the use of the file FArchive in LinkerSave in the text format case; it is unused because
			// we serialize into the TextFormatArchive instead of into FLinkerSave. But some SavePackage and LinkerSave
			// functions still expect it to be set.
			FString TempFilename = FPaths::CreateTempFilename(*FPaths::ProjectSavedDir(), *BaseFilename.Left(32));
			SaveContext.SetTempFilename(TempFilename);
			if (!LinkerSave->TryAssignFileSaver(TempFilename, bByteSwapping, bSaveUnversioned))
			{
				UE_LOGF(LogSavePackage, Error, "Could not create temporary save filename %ls.", *TempFilename);
				return ESavePackageResult::Error;
			}

			FString TextFormatTempFilename = TempFilename + FPackageName::GetTextAssetPackageExtension();
			SaveContext.SetTextFormatTempFilename(TextFormatTempFilename);
			SaveContext.SetTextFormatArchive(TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*TextFormatTempFilename)));
			TUniquePtr<FJsonArchiveOutputFormatter> OutputFormatter = MakeUnique<FJsonArchiveOutputFormatter>(*SaveContext.GetTextFormatArchive());
			OutputFormatter->SetObjectIndicesMap(&LinkerSave->ObjectIndicesMap);
			SaveContext.SetFormatter(MoveTemp(OutputFormatter));
#endif
		}
	}

	// Create the PackageTrailer for the Saver types that support it.
	if ((SaveContext.GetSaveArgs().SaveFlags & SAVE_BulkDataByReference) != 0)
	{
		if (const FLinkerLoad* LinkerLoad = FLinkerLoad::FindExistingLinkerForPackage(Package))
		{
			const UE::FPackageTrailer* Trailer = LinkerLoad->GetPackageTrailer();
			if (Trailer && Trailer->GetNumPayloads(UE::EPayloadStorageType::Any) > 0)
			{
				LinkerSave->PackageTrailerBuilder = UE::FPackageTrailerBuilder::CreateReferenceToTrailer(
					*Trailer, PackageName.ToString());
			}
		}
	}
	// PackageTrailer is not supported yet for cook or other procedural saves, unless using SAVE_BulkDataByReference.
	// PackageTrailer is not supported for text based assets yet.
	else if (!SaveContext.IsProceduralSave() && !SaveContext.IsTextFormat())
	{
		LinkerSave->PackageTrailerBuilder = MakeUnique<UE::FPackageTrailerBuilder>(PackageName.ToString());
	}

	// Assign Linker to the SaveContext
	SaveContext.SetLinker(MoveTemp(LinkerSave));
	SaveContext.SetStructuredArchive(MakeUnique<FStructuredArchive>(*SaveContext.GetFormatter()));
	return ReturnSuccessOrCancel();
}

struct FNameEntryIdSortHelper
{
	/** Comparison function used when sorting Names in the package's name table. */
	bool operator()(FNameEntryId A, FNameEntryId B) const
	{
		if (A == B) return false;
		// Sort by ignore-case first, then by case-sensitive.
		// So we will get { 'AAA', 'Aaa', 'aaa', 'BBB', 'Bbb', 'bbb' }
		if (int32 Compare = A.CompareLexical(B)) return Compare < 0;
		return A.CompareLexicalSensitive(B) < 0;
	}
};

ESavePackageResult BuildLinker(FSaveContext& SaveContext)
{
	// Setup Linker 
	FLinkerSave* Linker = SaveContext.GetLinker();
	{
		// Use the custom versions we harvested from the dependency harvesting pass
		Linker->Summary.SetCustomVersionContainer(SaveContext.GetCustomVersions());

		Linker->SetPortFlags(UE::SavePackageUtilities::Private::GetSavePackagePortFlags());
		if (SaveContext.IsSaveAutoOptional() && SaveContext.GetCurrentHarvestingRealm() == ESaveRealm::Optional)
		{
			// Do not filter editor only data when automatically creating optional data using full uncooked objects
			Linker->SetFilterEditorOnly(false);
		}
		else
		{
			Linker->SetFilterEditorOnly(SaveContext.IsFilterEditorOnly());
		}
		Linker->SetSavePackageData(&SaveContext.GetArchiveSavePackageData());

		bool bUseUnversionedProperties = SaveContext.IsSaveUnversionedProperties();
		Linker->SetUseUnversionedPropertySerialization(bUseUnversionedProperties);

#if WITH_EDITOR
		if (SaveContext.IsCooking())
		{
			Linker->SetDebugSerializationFlags(DSF_EnableCookerWarnings | Linker->GetDebugSerializationFlags());
			Linker->SetSaveBulkDataToSeparateFiles(true);

			if (const ITargetPlatform* TargetPlatform = SaveContext.GetTargetPlatform())
			{
				if (TargetPlatform->SupportsFeature(ETargetPlatformFeatures::MemoryMappedFiles))
				{
					int64 MemoryMapAlignment = GMemoryMappedBulkDataAlignment > 0 ? GMemoryMappedBulkDataAlignment : TargetPlatform->GetMemoryMappingAlignment();
					Linker->SetMemoryMapAlignment(MemoryMapAlignment);
				}
				Linker->SetFileRegionsEnabled(TargetPlatform->SupportsFeature(ETargetPlatformFeatures::CookFileRegionMetadata));
			}
		}
		else if (SaveContext.GetPackageWriter() && SaveContext.GetSavePackageContext() &&
			SaveContext.GetSavePackageContext()->PackageWriterCapabilities.bDeclareRegionForEachAdditionalFile)
		{
			Linker->SetFileRegionsEnabled(true);
			Linker->SetDeclareRegionForEachAdditionalFile(true);
		}
#endif
		if (SaveContext.IsCooking() == false)
		{
			const bool bSaveBulkDataByReference = (SaveContext.GetSaveArgs().SaveFlags & SAVE_BulkDataByReference) != 0;
			Linker->SetSaveBulkDataByReference(bSaveBulkDataByReference);
		}

		// Make sure the package has the same version as the linker
		SaveContext.UpdatePackageLinkerVersions();
	}

#if WITH_EDITORONLY_DATA
	Linker->Summary.PersistentGuid = SaveContext.GetPackage()->GetPersistentGuid();
#endif
	Linker->Summary.Generations = TArray<FGenerationInfo>{ FGenerationInfo(0, 0) };
	if (SaveContext.IsProceduralSave())
	{
		// Procedural saves should be deterministic, so we have to clear the EngineVersion fields
		// to avoid indeterminism when it changes
		Linker->Summary.SavedByEngineVersion = FEngineVersion();
		Linker->Summary.CompatibleWithEngineVersion = FEngineVersion();
	}

	// Build Name Map
	{
		SCOPED_SAVETIMER(UPackage_Save_BuildNameMap);
		const TSet<FNameEntryId>& NamesReferencedFromExportData = SaveContext.GetNamesReferencedFromExportData();
		TSet<FNameEntryId>& NamesReferencedFromPackageHeader = SaveContext.GetNamesReferencedFromPackageHeader();

		Linker->NameMap.Reserve(NamesReferencedFromExportData.Num() + NamesReferencedFromPackageHeader.Num());
		for (FNameEntryId NameEntryId : NamesReferencedFromExportData)
		{
			Linker->NameMap.Add(NameEntryId);
		}
		for (FNameEntryId NameEntryId : NamesReferencedFromPackageHeader)
		{
			if (!NamesReferencedFromExportData.Contains(NameEntryId))
			{
				Linker->NameMap.Add(NameEntryId);
			}
		}
		Linker->Summary.NameOffset = 0;
		Linker->Summary.NameCount = Linker->NameMap.Num();
		Linker->Summary.NamesReferencedFromExportDataCount = NamesReferencedFromExportData.Num();

		Algo::Sort(MakeArrayView(Linker->NameMap.GetData(), Linker->Summary.NamesReferencedFromExportDataCount), FNameEntryIdSortHelper());
		Algo::Sort(MakeArrayView(Linker->NameMap.GetData() + Linker->Summary.NamesReferencedFromExportDataCount,
			Linker->Summary.NameCount - Linker->Summary.NamesReferencedFromExportDataCount), FNameEntryIdSortHelper());

		if (!SaveContext.IsTextFormat())
		{
			for (int32 Index = 0; Index < Linker->NameMap.Num(); ++Index)
			{
				Linker->NameIndices.Add(Linker->NameMap[Index], Index);
			}
		}
	}

	// Build SoftObjectPathList
	{
		SCOPED_SAVETIMER(UPackage_Save_BuildSoftObjectPathList);
		Linker->Summary.SoftObjectPathsOffset = 0;
		Linker->Summary.SoftObjectPathsCount = 0;

		// Do not serialize a soft object path list when cooking.
		// iostore for example does not keep that list as part of its header information
		if (!SaveContext.IsCooking())
		{
			Linker->SoftObjectPathList = SaveContext.GetSoftObjectPathList().Array();
		}

		if (!SaveContext.IsTextFormat())
		{
			for (int32 Index = 0; Index < Linker->SoftObjectPathList.Num(); ++Index)
			{
				Linker->SoftObjectPathIndices.Add(Linker->SoftObjectPathList[Index], Index);
			}
		}
	}

	// Build GatherableText
	{
		Linker->Summary.GatherableTextDataOffset = 0;
		Linker->Summary.GatherableTextDataCount = 0;
		if (!SaveContext.IsFilterEditorOnly())
		{
			SCOPED_SAVETIMER(UPackage_Save_BuildGatherableTextData);

			// Gathers from the given package
			SaveContext.GatherableTextResultFlags = EPropertyLocalizationGathererResultFlags::Empty;
			FPropertyLocalizationDataGatherer(Linker->GatherableTextDataMap, SaveContext.GetPackage(), SaveContext.GatherableTextResultFlags);
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	// Gather VersePaths
	Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
	TMap<Verse::VCell*, Verse::VUniqueString*> VersePathMap;
	{
		TSet<Verse::VPackage*> VersePackages;
		for (Verse::VCell* Import : SaveContext.GetCellImports())
		{
			VersePackages.Add(Context.PackageForCell(Import));
		}
		for (Verse::VCell* Export : SaveContext.GetCellExports())
		{
			if (Export->GetCppClassInfo()->SerializeIdentity)
			{
				if (Verse::VPackage* VersePackage = Context.PackageForCell(Export))
				{
					VersePackages.Add(VersePackage);
				}
			}
		}

		VersePathMap.Reserve(SaveContext.GetCellImports().Num() + SaveContext.GetCellExports().Num());
		for (Verse::VPackage* VersePackage : VersePackages)
		{
			VersePathMap.Add(VersePackage, &VersePackage->GetRootPath());
			for (int32 Index = 0; Index < VersePackage->NumDefinitions(); ++Index)
			{
				if (Verse::VCell* Cell = VersePackage->GetDefinition(Index).ExtractCell())
				{
					// Packages can re-export cells from other packages; only use their original Verse path here.
					if (Context.PackageForCell(Cell) == VersePackage)
					{
						Verse::VUniqueString* VersePath = &VersePackage->GetDefinitionName(Index);
						VersePathMap.Add(Cell, VersePath);
					}
				}
			}
		}
	}

	auto ProjectVersePath = [](FCellResource& CellResource)
	{
		return CellResource.VersePath->AsStringView();
	};
#endif

	// Build ImportMap
	{
		SCOPED_SAVETIMER(UPackage_Save_BuildImportMap);
		UE_COOK_RESULTPROJECTION_SCOPED(UE::Cook::ResultProjection::PackageAndClass);

		for (TObjectPtr<UObject> Import : SaveContext.GetImports())
		{
			UClass* ImportClass = Import.GetClass();
			FName ReplacedName = NAME_None;
			FObjectImport& ObjectImport = Linker->ImportMap.Add_GetRef(FObjectImport(Import, ImportClass));

			// flag the import as optional
			if (ImportClass->HasAnyClassFlags(CLASS_Optional))
			{
				ObjectImport.bImportOptional = true;
			}

			// If the package import is a prestream package, mark it as such by hacking its class name
			if (SaveContext.IsPrestreamPackage(Cast<UPackage>(Import)))
			{
				ObjectImport.ClassName = UE::SavePackageUtilities::NAME_PrestreamPackage;
			}

			if (ReplacedName != NAME_None)
			{
				ObjectImport.ObjectName = ReplacedName;
			}
		}
		//Algo::SortBy(Linker->ImportMap, &FObjectResource::ObjectName, FNameLexicalLess());

		// @todo: To stay consistent with the old save and prevent binary diff between the algo, use the old import sort for now
		// a future cvar could allow projects use the less expensive sort in their own time down the line
		FObjectImportSortHelper ImportSortHelper;
		{
			SCOPED_SAVETIMER(UPackage_Save_SortImports);
			ImportSortHelper.SortImports(Linker);
		}
		Linker->Summary.ImportCount = Linker->ImportMap.Num();

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		for (Verse::VCell* const& Import : SaveContext.GetCellImports())
		{
			FCellImport& CellImport = Linker->CellImportMap.Add_GetRef(FCellImport(Import));
			ensure(Import->GetCppClassInfo()->SerializeIdentity);
			CellImport.VersePath = VersePathMap[Import];
		}
		Algo::SortBy(Linker->CellImportMap, ProjectVersePath);
#endif
		Linker->Summary.CellImportCount = Linker->CellImportMap.Num();
	}

	// Build ExportMap & Package Netplay data
	{
		SCOPED_SAVETIMER(UPackage_Save_BuildExportMap);
		for (const FTaggedExport& TaggedExport : SaveContext.GetExports())
		{
			FObjectExport& Export = Linker->ExportMap.Add_GetRef(FObjectExport(TaggedExport.Obj, TaggedExport.bNotAlwaysLoadedForEditorGame));
			Export.bGeneratePublicHash = TaggedExport.bGeneratePublicHash;

			if (UPackage* Package = Cast<UPackage>(Export.Object))
			{
				Export.PackageFlags = Package->GetPackageFlags();
			}
		}
		//Algo::SortBy(Linker->ExportMap, &FObjectResource::ObjectName, FNameLexicalLess());

		// @todo: To stay consistent with the old save and prevent binary diff between the algo, use the old import sort for now
		// a future cvar could allow projects use the less expensive sort in their own time down the line
		// Also, currently the export sort order matters in an incidental manner where it should be properly tracked with dependencies instead.
		// for example where FAnimInstanceProxy PostLoad actually depends on UAnimBlueprintGeneratedClass PostLoad to be properly initialized.
		FObjectExportSortHelper ExportSortHelper;
		{
			SCOPED_SAVETIMER(UPackage_Save_SortExports);
			ExportSortHelper.SortExports(Linker);
		}
		Linker->Summary.ExportCount = Linker->ExportMap.Num();

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		Verse::VUniqueString& EmptyVersePath = Verse::VUniqueString::New(Context, "");
		for (Verse::VCell* const& Export : SaveContext.GetCellExports())
		{
			FCellExport& CellExport = Linker->CellExportMap.Add_GetRef(FCellExport(Export));
			if (Verse::VUniqueString** VersePath = VersePathMap.Find(Export))
			{
				CellExport.VersePath = *VersePath;
			}
			else
			{
				CellExport.VersePath = &EmptyVersePath;
			}
		}
		Algo::SortBy(Linker->CellExportMap, ProjectVersePath);
#endif
		Linker->Summary.CellExportCount = Linker->CellExportMap.Num();
	}

	// Build Linker Reverse Mapping
	{
		for (int32 ExportIndex = 0; ExportIndex < Linker->ExportMap.Num(); ++ExportIndex)
		{
			UObject* Object = Linker->ExportMap[ExportIndex].Object;
			check(Object);
			Linker->ObjectIndicesMap.Add(Object, FPackageIndex::FromExport(ExportIndex));
		}
		for (int32 ImportIndex = 0; ImportIndex < Linker->ImportMap.Num(); ++ImportIndex)
		{
			UObject* Object = Linker->ImportMap[ImportIndex].XObject;
			check(Object);
			Linker->ObjectIndicesMap.Add(Object, FPackageIndex::FromImport(ImportIndex));
		}

		int32 CellExportOffset = Linker->ExportMap.Num();
		for (int32 CellExportIndex = 0; CellExportIndex < Linker->CellExportMap.Num(); ++CellExportIndex)
		{
			Verse::VCell* Cell = Linker->CellExportMap[CellExportIndex].Cell;
			check(Cell);
			Linker->CellIndicesMap.Add(Cell, FPackageIndex::FromExport(CellExportOffset + CellExportIndex));
		}
		int32 CellImportOffset = Linker->ImportMap.Num();
		for (int32 CellImportIndex = 0; CellImportIndex < Linker->CellImportMap.Num(); ++CellImportIndex)
		{
			Verse::VCell* Cell = Linker->CellImportMap[CellImportIndex].Cell;
			check(Cell);
			Linker->CellIndicesMap.Add(Cell, FPackageIndex::FromImport(CellImportOffset + CellImportIndex));
		}
	}

	// Build DependsMap
	{
		SCOPED_SAVETIMER(UPackage_Save_BuildExportDependsMap);

		Linker->DependsMap.AddZeroed(Linker->ExportMap.Num());
		for (int32 ExpIndex = 0; ExpIndex < Linker->ExportMap.Num(); ++ExpIndex)
		{
			UObject* Object = Linker->ExportMap[ExpIndex].Object;
			TArray<FPackageIndex>& DependIndices = Linker->DependsMap[ExpIndex];
			const TSet<TObjectPtr<UObject>>* SrcDepends = SaveContext.GetObjectDependencies().Find(Object);
			checkf(SrcDepends, TEXT("Couldn't find dependency map for %s"), *Object->GetFullName());
			DependIndices.Reserve(SrcDepends->Num());

			for (TObjectPtr<UObject> DependentObject : *SrcDepends)
			{
				FPackageIndex DependencyIndex = Linker->ObjectIndicesMap.FindRef(DependentObject);

				// if we didn't find it (FindRef returns 0 on failure, which is good in this case), then we are in trouble, something went wrong somewhere
				checkf(!DependencyIndex.IsNull(), TEXT("Failed to find dependency index for %s (%s)"), *DependentObject->GetFullName(), *Object->GetFullName());

				// add the import as an import for this export
				DependIndices.Add(DependencyIndex);
			}
		}
	}

	// Build SoftPackageReference & Searchable Name Map
	{
		SCOPED_SAVETIMER(UPackage_Save_BuildSoftPackagesAndSearchableNames);
		Linker->SoftPackageReferenceList = SaveContext.GetSoftPackageReferenceList().Array();

		if (!SaveContext.IsFilterEditorOnly())
		{
			// Convert the searchable names map from UObject to packageindex
			for (TPair<TObjectPtr<UObject>, TArray<FName>>& SearchableNamePair : SaveContext.GetSearchableNamesObjectMap())
			{
				const FPackageIndex PackageIndex = Linker->MapObject(SearchableNamePair.Key);
				// This should always be in the imports already
				if (ensureMsgf(!PackageIndex.IsNull(), TEXT("%s: Searchable object %s with is not mapped"), *Linker->LinkerRoot->GetFullName(), *SearchableNamePair.Key->GetFullName()))
				{
					Linker->SearchableNamesMap.FindOrAdd(PackageIndex) = MoveTemp(SearchableNamePair.Value);
				}
			}
		}
		SaveContext.GetSearchableNamesObjectMap().Empty();
	}

	// Map Export Indices
	{
		SCOPED_SAVETIMER(UPackage_Save_MapExportIndices);

		for (FObjectExport& Export : Linker->ExportMap)
		{
			// Set class index.
			// If this is *exactly* a UClass, store null instead; for anything else, including UClass-derived classes, map it
			UClass* ObjClass = Export.Object->GetClass();
			if (ObjClass != UClass::StaticClass())
			{
				Export.ClassIndex = Linker->MapObject(ObjClass);
				// The class should be mappable because it was checked in FPackageHarvester::ProcessExport and the save early-exited if not
				checkf(!Export.ClassIndex.IsNull(), TEXT("Export %s class is not mapped when saving %s"), *Export.Object->GetFullName(), *Linker->LinkerRoot->GetName());
			}
			else
			{
				Export.ClassIndex = FPackageIndex();
			}

			if (SaveContext.IsCooking())
			{
				UObject* Archetype = Export.Object->GetArchetype();
				check(Archetype);
				check(Archetype->IsA(Export.Object->HasAnyFlags(RF_ClassDefaultObject) ? ObjClass->GetSuperClass() : ObjClass));
				Export.TemplateIndex = Linker->MapObject(Archetype);
				UE_CLOGF(Export.TemplateIndex.IsNull(), LogSavePackage, Fatal, "%ls was an archetype of %ls but returned a null index mapping the object.", *Archetype->GetFullName(), *Export.Object->GetFullName());
				check(!Export.TemplateIndex.IsNull());
			}

			// Set the parent index, if this export represents a UStruct-derived object
			UStruct* Struct = Cast<UStruct>(Export.Object);
			if (Struct && Struct->GetSuperStruct() != nullptr)
			{
				Export.SuperIndex = Linker->MapObject(Struct->GetSuperStruct());
				checkf(!Export.SuperIndex.IsNull(),
					TEXT("Export Struct (%s) of type (%s) inheriting from (%s) of type (%s) has not mapped super struct."),
					*GetPathNameSafe(Struct),
					*(Struct->GetClass()->GetName()),
					*GetPathNameSafe(Struct->GetSuperStruct()),
					*(Struct->GetSuperStruct()->GetClass()->GetName())
				);
			}
			else
			{
				Export.SuperIndex = FPackageIndex();
			}

			// Set FPackageIndex for this export's Outer. If the export's Outer
			// is the UPackage corresponding to this package's LinkerRoot, leave it null
			Export.OuterIndex = Export.Object->GetOuter() != SaveContext.GetPackage() ? Linker->MapObject(Export.Object->GetOuter()) : FPackageIndex();

			// Only packages or object having the currently saved package as outer are allowed to have no outer
			ensureMsgf(Export.OuterIndex != FPackageIndex() || Export.Object->IsA(UPackage::StaticClass()) || Export.Object->GetOuter() == SaveContext.GetPackage(), TEXT("Export %s has no valid outer!"), *Export.Object->GetPathName());
		}

		for (FObjectImport& Import : Linker->ImportMap)
		{
			if (Import.XObject)
			{
				// Set the package index.
				if (Import.XObject->GetOuter())
				{
					Import.OuterIndex = Linker->MapObject(Import.XObject->GetOuter());

					// if the import has a package set, set it up
					if (UPackage* ImportPackage = Import.XObject->GetExternalPackage())
					{
						Import.SetPackageName(ImportPackage->GetFName());
					}

					if (SaveContext.IsCooking())
					{
						// Only package imports are allowed to have no outer
						ensureMsgf(Import.OuterIndex != FPackageIndex() || Import.ClassName == NAME_Package, TEXT("Import %s has no valid outer when cooking!"), *Import.XObject->GetPathName());
					}
				}
			}
			else
			{
				checkf(false, TEXT("NULL XObject for import - Object: %s Class: %s"), *Import.ObjectName.ToString(), *Import.ClassName.ToString());
			}
		}

#if WITH_EDITORONLY_DATA
		// Build ImportTypeHierarchies
		if (!SaveContext.IsCooking())
		{
			SCOPED_SAVETIMER(UPackage_Save_BuildImportTypeHierarchies);

			for (FObjectExport& Export : Linker->ExportMap)
			{
				if (!Export.ClassIndex.IsNull())
				{
					UObject* ClassObject = nullptr;
					if (Export.ClassIndex.IsExport())
					{
						FObjectExport& ClassExport = Linker->ExportMap[Export.ClassIndex.ToExport()];
						ClassObject = ClassExport.Object;
					}
					else
					{
						FObjectImport& ClassImport = Linker->ImportMap[Export.ClassIndex.ToImport()];
						ClassObject = ClassImport.XObject;
					}
					
					if (ClassObject)
					{
						UStruct* Struct = Cast<UStruct>(ClassObject);
						if (Struct)
						{
							const FPackageIndex PackageIndex = Linker->MapObject(ClassObject);
							// This should always be in the linker tables already.
							if (ensureMsgf(!PackageIndex.IsNull(), TEXT("%s: Struct object: '%s' is not mapped"), *Linker->LinkerRoot->GetFullName(), *Struct->GetFullName()))
							{
								// Skip already processed class imports.
								if (!Linker->ImportTypeHierarchies.Contains(PackageIndex))
								{
									// To avoid bloating the ImportTypeHierarchy table only create entries for types 
									// that support PropertyBag placeholders, or when we will have dynamic imports
									const bool bExportHasDynamicImports = ((Export.ObjectFlags & RF_HasDynamicImports) != 0);
									if (bExportHasDynamicImports || UE::CanCreatePropertyBagPlaceholderForType(Struct->GetClass(), Struct))
									{
										UE::Serialization::Private::FImportTypeHierarchy& ImportTypeHierarchy = Linker->ImportTypeHierarchies.Add(PackageIndex);

										UStruct* SuperStruct = Struct->GetSuperStruct();

										while (SuperStruct && (SuperStruct != UObject::StaticClass()))
										{
											UClass* SuperStructClass = SuperStruct->GetClass();
											UPackage* SuperStructPackage = SuperStruct->GetPackage();

											UE::Serialization::Private::FTypeResource SuperType;
											SuperType.TypeName = SuperStruct->GetFName();
											SuperType.PackageName = SuperStructPackage ? SuperStructPackage->GetFName() : NAME_None;
											SuperType.ClassName = SuperStructClass ? SuperStructClass->GetFName() : NAME_None;
											SuperType.ClassPackageName = (SuperStructClass && SuperStructClass->GetPackage()) ? SuperStructClass->GetPackage()->GetFName() : NAME_None;

											ImportTypeHierarchy.AddSuperType(MoveTemp(SuperType));

											SuperStruct = SuperStruct->GetSuperStruct();
										}
									}
								}
							}
						}
					}
				}
			}
		}
#endif

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		for (FCellImport& CellImport : Linker->CellImportMap)
		{
			Verse::VPackage* VersePackage = Context.PackageForCell(CellImport.Cell);
			CellImport.PackageIndex = Linker->MapObject(VersePackage->GetUPackage());
			ensure(!CellImport.PackageIndex.IsNull());
		}
#endif
	}
	return ReturnSuccessOrCancel();
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void GetCellsWithOuter(Verse::FAllocationContext Context, FSaveContext& SaveContext, UObject* Outer, TArray<Verse::VCell*>& Stack, TSet<Verse::VCell*>& Results)
{
	struct FSubCellVisitor : UE::SavePackageUtilities::Private::FArchiveSavePackageCollector
	{
		TSet<Verse::VCell*>& SubCells;
		TArray<Verse::VCell*>& SubCellStack;

		FSubCellVisitor(FSaveContext& InContext, TSet<Verse::VCell*>& InSubCells, TArray<Verse::VCell*>& InSubCellStack)
			: FArchiveSavePackageCollector(InContext.GetArchiveSavePackageData(),
				InContext.IsFilterEditorOnly(), InContext.IsSaveUnversionedProperties(), InContext.IsCooking())
			, SubCells(InSubCells)
			, SubCellStack(InSubCellStack)
		{
		}

		virtual FArchive& operator<<(Verse::VCell*& InCell) override
		{
			if (InCell && InCell->GetCppClassInfo()->InstancedCell)
			{
				// VObjects representing class instances are not currently instanced from CDOs,
				// so there is no need to traverse into them here. See VValueObject::InstancedCell.
				if (Verse::VObject* Object = InCell->DynamicCast<Verse::VObject>(); Object && !Object->IsStruct())
				{
					return *this;
				}

				bool bIsAlreadyInSet = false;
				SubCells.Add(InCell, &bIsAlreadyInSet);
				if (!bIsAlreadyInSet)
				{
					SubCellStack.Push(InCell);
				}
			}
			return *this;
		}
	} Visitor{ SaveContext, Results, Stack };
	Outer->Serialize(Visitor);
	while (Stack.Num() > 0)
	{
		Verse::VCell* SubCell = Stack.Pop(EAllowShrinking::No);
		SubCell->Serialize(Context, Visitor);
	}
}
#endif

AUTORTFM_DISABLE void SavePreloadDependencies(FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext)
{
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
#endif

	FLinkerSave* Linker = SaveContext.GetLinker();
	TArray<UE::SavePackageUtilities::FPreloadDependency>& PreloadDependencies = SaveContext.GetPreloadDependencies();

	auto IncludeObjectAsDependency = [Linker, &SaveContext](int32 CallSite, TSet<FPackageIndex>& AddTo, TObjectPtr<UObject> ToTest, UObject* ForObj, bool bMandatory, bool bOnlyIfInLinkerTable)
	{
		// Skip transient, editor only, and excluded client/server objects
		if (ToTest)
		{
			TObjectPtr<UPackage> Outermost = ToTest.GetPackage();
			check(Outermost);
			if (Outermost.GetFName() == GLongCoreUObjectPackageName)
			{
				return; // We assume nothing in coreuobject ever loads assets in a constructor
			}
			FPackageIndex Index = Linker->MapObject(ToTest, false);
			if (Index.IsNull() && bOnlyIfInLinkerTable)
			{
				return;
			}
			if (!Index.IsNull() && (SaveContext.IsTransient(ToTest) && !ToTest->IsNative()))
			{
				UE_LOGF(LogSavePackage, Warning, "A dependency '%ls' of '%ls' is in the linker table, but is transient. We will keep the dependency anyway (%d).", *ToTest.GetFullName(), *ForObj->GetFullName(), CallSite);
			}
			if (!Index.IsNull() && !IsValid(ToTest))
			{
				UE_LOGF(LogSavePackage, Warning, "A dependency '%ls' of '%ls' is in the linker table, but is pending kill or garbage. We will keep the dependency anyway (%d).", *ToTest.GetFullName(), *ForObj->GetFullName(), CallSite);
			}
			bool bIncludedInHarvest = SaveContext.IsIncluded(ToTest);
			if (bMandatory && !bIncludedInHarvest)
			{
				UE_LOGF(LogSavePackage, Warning, "A dependency '%ls' of '%ls' was filtered, but is mandatory. This indicates a problem with editor only stripping. We will keep the dependency anyway (%d).", *ToTest.GetFullName(), *ForObj->GetFullName(), CallSite);
				bIncludedInHarvest = true;
			}
			if (bIncludedInHarvest)
			{
				if (!Index.IsNull())
				{
					AddTo.Add(Index);
					return;
				}
				else if (!SaveContext.IsUnsaveable(ToTest, SaveContext.GetCachedObjectStatus(ToTest)))
				{
					UE_CLOGF(Outermost->HasAnyPackageFlags(PKG_CompiledIn), LogSavePackage, Verbose, "A compiled in dependency '%ls' of '%ls' was not actually in the linker tables and so will be ignored (%d).", *ToTest->GetFullName(), *ForObj->GetFullName(), CallSite);
					UE_CLOGF(!Outermost->HasAnyPackageFlags(PKG_CompiledIn), LogSavePackage, Fatal, "A dependency '%ls' of '%ls' was not actually in the linker tables and so will be ignored (%d).", *ToTest->GetFullName(), *ForObj->GetFullName(), CallSite);
				}
			}
			check(!bMandatory);
		}
	};

	auto IncludeIndexAsDependency = [Linker](TSet<FPackageIndex>& AddTo, FPackageIndex Dep)
	{
		if (!Dep.IsNull())
		{
			UObject* ToTest = Dep.IsExport() ? Linker->Exp(Dep).Object : Linker->Imp(Dep).XObject;
			if (ToTest)
			{
				UPackage* Outermost = ToTest->GetOutermost();
				if (Outermost && Outermost->GetFName() != GLongCoreUObjectPackageName) // We assume nothing in coreuobject ever loads assets in a constructor
				{
					AddTo.Add(Dep);
				}
			}
		}
	};

	Linker->Summary.PreloadDependencyOffset = (int32)Linker->Tell();
	Linker->Summary.PreloadDependencyCount = -1;

	if (SaveContext.IsCooking())
	{
		Linker->Summary.PreloadDependencyCount = 0;

		FStructuredArchive::FStream DepedenciesStream = StructuredArchiveRoot.EnterStream(TEXT("PreloadDependencies"));

		TArray<UObject*> Subobjects;
		TArray<UObject*> Deps;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		TSet<Verse::VCell*> SubCells;
		TArray<Verse::VCell*> SubCellStack;
#endif
		TSet<FPackageIndex> SerializationBeforeCreateDependencies;
		TSet<FPackageIndex> SerializationBeforeSerializationDependencies;
		TSet<FPackageIndex> CreateBeforeSerializationDependencies;
		TSet<FPackageIndex> CreateBeforeCreateDependencies;

		for (int32 ExportIndex = 0; ExportIndex < Linker->ExportMap.Num(); ++ExportIndex)
		{
			FObjectExport& Export = Linker->ExportMap[ExportIndex];
			check(Export.Object);
			{
				SerializationBeforeCreateDependencies.Reset();
				IncludeIndexAsDependency(SerializationBeforeCreateDependencies, Export.ClassIndex);
				UObject* CDO = Export.Object->GetArchetype();
				IncludeObjectAsDependency(1, SerializationBeforeCreateDependencies, CDO, Export.Object, true, false);
				Subobjects.Reset();
				GetObjectsWithOuter(CDO, Subobjects);
				for (UObject* SubObj : Subobjects)
				{
					// Only include subobject archetypes
					if (SubObj->HasAnyFlags(RF_DefaultSubObject | RF_ArchetypeObject))
					{
						while (SaveContext.IsTransient(SubObj)) // transient components are stripped by the ICH, so find the one it will really use at runtime
						{
							UObject* SubObjArch = SubObj->GetArchetype();
							if (SubObjArch->GetClass()->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
							{
								break;
							}
							SubObj = SubObjArch;
						}
						if (IsValid(SubObj))
						{
							IncludeObjectAsDependency(2, SerializationBeforeCreateDependencies, SubObj, Export.Object, false, false);
						}
					}
				}
			}
			{
				SerializationBeforeSerializationDependencies.Reset();
				Deps.Reset();
				Export.Object->GetPreloadDependencies(Deps);

				for (UObject* Obj : Deps)
				{
					IncludeObjectAsDependency(3, SerializationBeforeSerializationDependencies, Obj, Export.Object, false, true);
				}
				if (Export.Object->HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
				{
					UObject* Outer = Export.Object->GetOuter();
					if (!Outer->IsA(UPackage::StaticClass()))
					{
						IncludeObjectAsDependency(4, SerializationBeforeSerializationDependencies, Outer, Export.Object, true, false);
					}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
					// These VCells must be serialized before this CDO is used as an archetype.
					// This is similar to the GetObjectsWithOuter(CDO) call above. We handle CDO VCells here instead because:
					//   1) Not all these CDO VCells are importable, but they are all in the CDO's package.
					//   2) This lets us add these dependency edges once rather than repeated for all instances.
					GetCellsWithOuter(Context, SaveContext, Export.Object, SubCellStack, SubCells);
					for (Verse::VCell* SubCell : SubCells)
					{
						SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[SubCell]);
					}
					SubCells.Reset();
#endif
				}
				if (Export.Object->IsA(UClass::StaticClass()))
				{
					// we need to load archetypes of our subobjects before we load the class
					UObject* CDO = CastChecked<UClass>(Export.Object)->GetDefaultObject();
					Subobjects.Reset();
					GetObjectsWithOuter(CDO, Subobjects);
					for (UObject* SubObj : Subobjects)
					{
						// Only include subobject archetypes
						if (SubObj->HasAnyFlags(RF_DefaultSubObject | RF_ArchetypeObject))
						{
							// Don't include the archetype of SubObjects that were not included in the harvesting phase; we didn't add their archetypes
							if (!SaveContext.IsIncluded(SubObj))
							{
								continue;
							}

							SubObj = SubObj->GetArchetype();
							while (SaveContext.IsTransient(SubObj)) // transient components are stripped by the ICH, so find the one it will really use at runtime
							{
								UObject* SubObjArch = SubObj->GetArchetype();
								if (SubObjArch->GetClass()->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic))
								{
									break;
								}
								SubObj = SubObjArch;
							}
							if (IsValid(SubObj))
							{
								IncludeObjectAsDependency(5, SerializationBeforeSerializationDependencies, SubObj, Export.Object, false, false);
							}
						}
					}
				}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
				// TODO: This may make more sense behind something like GetPreloadCellDependencies
				Verse::VClass* Class = nullptr;
				if (UVerseClass* VerseClass = Cast<UVerseClass>(Export.Object))
				{
					Class = VerseClass->Class.Get();
					check(Class != nullptr);
				}
				if (UVerseStruct* VerseStruct = Cast<UVerseStruct>(Export.Object))
				{
					Class = VerseStruct->Class.Get();
					if (VerseStruct->IsTuple())
					{
						check(Class == nullptr);
						FUtf8String MangledNameString = VerseStruct->GetFName().ToUtf8String();
						Verse::VUniqueString& MangledName = Verse::VUniqueString::New(Context, MangledNameString);
						Verse::VTupleType* TupleType = Verse::GlobalProgram->LookupTupleType(Context, MangledName);
						SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[TupleType]);
					}
					else
					{
						check(Class != nullptr);
					}
				}
				if (Class != nullptr)
				{
					// VClass is used to construct VShape during linking.
					SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[Class]);

					// Native members are bound via the VPackage during linking.
					// Their names are VUniqueStrings, which are fully deserialized at creation time.
					if (Class->IsNativeBound())
					{
						Verse::VPackage* VersePackage = &Class->GetPackage();
						SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[VersePackage]);
					}
				}
#endif
			}
			{
				CreateBeforeSerializationDependencies.Reset();
				UClass* Class = Cast<UClass>(Export.Object);
				UObject* ClassCDO = Class ? Class->GetDefaultObject() : nullptr;
				{
					TArray<FPackageIndex>& Depends = Linker->DependsMap[ExportIndex];
					for (FPackageIndex Dep : Depends)
					{
						UObject* ToTest = Dep.IsExport() ? Linker->Exp(Dep).Object : Linker->Imp(Dep).XObject;
						if (ToTest != ClassCDO)
						{
							IncludeIndexAsDependency(CreateBeforeSerializationDependencies, Dep);
						}
					}
				}
				{
					const TSet<TObjectPtr<UObject>>& NativeDeps = SaveContext.GetNativeObjectDependencies()[Export.Object];
					for (TObjectPtr<UObject> ToTest : NativeDeps)
					{
						if (ToTest != ClassCDO)
						{
							IncludeObjectAsDependency(6, CreateBeforeSerializationDependencies, ToTest, Export.Object, false, true);
						}
					}
				}
				{
					const TSet<Verse::VCell*>& CellDeps = SaveContext.GetCellDependencies()[Export.Object];
					for (Verse::VCell* Cell : CellDeps)
					{
						FPackageIndex Dep = Linker->CellIndicesMap.FindRef(Cell);
						CreateBeforeSerializationDependencies.Add(Dep);
					}
				}
			}
			{
				CreateBeforeCreateDependencies.Reset();
				IncludeIndexAsDependency(CreateBeforeCreateDependencies, Export.OuterIndex);
				IncludeIndexAsDependency(CreateBeforeCreateDependencies, Export.SuperIndex);
			}
			auto AddArcForDepChecking = [&Linker, &Export, &PreloadDependencies](bool bExportIsSerialize, FPackageIndex Dep, bool bDepIsSerialize)
			{
				check(Export.Object);
				check(!Dep.IsNull());
				UObject* DepObject = nullptr;
				if (Dep.IsExport())
				{
					if (Linker->ExportMap.IsValidIndex(Dep.ToExport()))
					{
						DepObject = Linker->Exp(Dep).Object;
						check(DepObject);
					}
					else
					{
						int32 CellExport = Dep.ToExport() - Linker->ExportMap.Num();
						check(Linker->CellExportMap.IsValidIndex(CellExport));
						check(Linker->CellExportMap[CellExport].Cell);
					}
				}
				else
				{
					if (Linker->ImportMap.IsValidIndex(Dep.ToImport()))
					{
						DepObject = Linker->Imp(Dep).XObject;
						check(DepObject);
					}
					else
					{
						int32 CellImport = Dep.ToImport() - Linker->ImportMap.Num();
						check(Linker->CellImportMap.IsValidIndex(CellImport));
						check(Linker->CellImportMap[CellImport].Cell);
					}
				}

				Linker->DepListForErrorChecking.Add(Dep);
				PreloadDependencies.Add({
					.SourceObject = Export.Object,
					.TargetObject = DepObject,
					.bSourceIsSerialize = bExportIsSerialize,
					.bTargetIsSerialize = bDepIsSerialize
					});
			};

			for (FPackageIndex Index : SerializationBeforeSerializationDependencies)
			{
				if (SerializationBeforeCreateDependencies.Contains(Index))
				{
					continue; // if the other thing must be serialized before we create, then this is a redundant dep
				}
				if (Export.FirstExportDependency == -1)
				{
					Export.FirstExportDependency = Linker->Summary.PreloadDependencyCount;
					check(Export.SerializationBeforeSerializationDependencies == 0 && Export.CreateBeforeSerializationDependencies == 0 && Export.SerializationBeforeCreateDependencies == 0 && Export.CreateBeforeCreateDependencies == 0);
				}
				Linker->Summary.PreloadDependencyCount++;
				Export.SerializationBeforeSerializationDependencies++;
				DepedenciesStream.EnterElement() << Index;
				AddArcForDepChecking(true, Index, true);
			}
			for (FPackageIndex Index : CreateBeforeSerializationDependencies)
			{
				if (SerializationBeforeCreateDependencies.Contains(Index))
				{
					continue; // if the other thing must be serialized before we create, then this is a redundant dep
				}
				if (SerializationBeforeSerializationDependencies.Contains(Index))
				{
					continue; // if the other thing must be serialized before we serialize, then this is a redundant dep
				}
				if (CreateBeforeCreateDependencies.Contains(Index))
				{
					continue; // if the other thing must be created before we are created, then this is a redundant dep
				}
				if (Export.FirstExportDependency == -1)
				{
					Export.FirstExportDependency = Linker->Summary.PreloadDependencyCount;
					check(Export.SerializationBeforeSerializationDependencies == 0 && Export.CreateBeforeSerializationDependencies == 0 && Export.SerializationBeforeCreateDependencies == 0 && Export.CreateBeforeCreateDependencies == 0);
				}
				Linker->Summary.PreloadDependencyCount++;
				Export.CreateBeforeSerializationDependencies++;
				DepedenciesStream.EnterElement() << Index;
				AddArcForDepChecking(true, Index, false);
			}
			for (FPackageIndex Index : SerializationBeforeCreateDependencies)
			{
				if (Export.FirstExportDependency == -1)
				{
					Export.FirstExportDependency = Linker->Summary.PreloadDependencyCount;
					check(Export.SerializationBeforeSerializationDependencies == 0 && Export.CreateBeforeSerializationDependencies == 0 && Export.SerializationBeforeCreateDependencies == 0 && Export.CreateBeforeCreateDependencies == 0);
				}
				Linker->Summary.PreloadDependencyCount++;
				Export.SerializationBeforeCreateDependencies++;
				DepedenciesStream.EnterElement() << Index;
				AddArcForDepChecking(false, Index, true);
			}
			for (FPackageIndex Index : CreateBeforeCreateDependencies)
			{
				if (Export.FirstExportDependency == -1)
				{
					Export.FirstExportDependency = Linker->Summary.PreloadDependencyCount;
					check(Export.SerializationBeforeSerializationDependencies == 0 && Export.CreateBeforeSerializationDependencies == 0 && Export.SerializationBeforeCreateDependencies == 0 && Export.CreateBeforeCreateDependencies == 0);
				}
				Linker->Summary.PreloadDependencyCount++;
				Export.CreateBeforeCreateDependencies++;
				DepedenciesStream.EnterElement() << Index;
				AddArcForDepChecking(false, Index, false);
			}
		}
		UE_LOGF(LogSavePackage, VeryVerbose, "Saved %d dependencies for %d exports.", Linker->Summary.PreloadDependencyCount, Linker->ExportMap.Num());

		for (int32 CellExportIndex = 0; CellExportIndex < Linker->CellExportMap.Num(); ++CellExportIndex)
		{
			FCellExport& Export = Linker->CellExportMap[CellExportIndex];
			check(Export.Cell);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
			{
				// TODO: These may make more sense behind something like GetPreloadCellDependencies
				SerializationBeforeSerializationDependencies.Reset();
				if (Verse::VMapBase* Map = Export.Cell->DynamicCast<Verse::VMapBase>())
				{
					// Map keys are hashed and compared during insertion.
					// TODO: Recursively add the elements of composite keys (structs, tuples, arrays, maps, options, rationals).
					for (TPair<Verse::VValue, Verse::VValue> Pair : *Map)
					{
						if (Verse::VCell* Key = Pair.Key.ExtractCell())
						{
							SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[Key]);
						}
					}
				}
				if (Verse::VClass* Class = Export.Cell->DynamicCast<Verse::VClass>())
				{
					// VClass and VArchetype hierarchy is used when constructing VShapes.
					// This is done both by UVerseClass/UVerseStruct linking and VValueObject serialization.
					for (Verse::TWriteBarrier<Verse::VClass>& Inherited : Class->GetInherited())
					{
						SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[Inherited.Get()]);
					}
					Verse::VArchetype& Archetype = Class->GetArchetype();
					SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[&Archetype]);
					for (int32 Index = 0; Index < Archetype.NumEntries; ++Index)
					{
						Verse::VArchetype::VEntry& Entry = Archetype.Entries[Index];
						// TODO: VTypes also depend on their component types (array element, etc.)
						Verse::VValue Type = Entry.Type.Follow();
						if (Type.IsCellOfType<Verse::VType>())
						{
							SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[Type.ExtractCell()]);
						}
						// Constructing a VShape inspects the values to determine whether they are methods.
						Verse::VValue Value = Entry.Value.Follow();
						if (Value.IsCellOfType<Verse::VFunction>())
						{
							SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[Value.ExtractCell()]);
						}
					}
				}
				if (Verse::VValueObject* Object = Export.Cell->DynamicCast<Verse::VValueObject>())
				{
					Verse::VClass& Class = Object->GetEmergentType()->Type->StaticCast<Verse::VClass>();
					SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[&Class]);
				}
				if (Verse::VNativeStruct* NativeStruct = Export.Cell->DynamicCast<Verse::VNativeStruct>())
				{
					Verse::VClass& Class = NativeStruct->GetEmergentType()->Type->StaticCast<Verse::VClass>();
					SerializationBeforeSerializationDependencies.Add(Linker->CellIndicesMap[&Class]);

					FPackageIndex Struct = Linker->ObjectIndicesMap.FindRef(Class.GetUEType<UObject>());
					IncludeIndexAsDependency(SerializationBeforeSerializationDependencies, Struct);
				}
			}
#endif
			{
				CreateBeforeSerializationDependencies.Reset();
				{
					const TSet<TObjectPtr<UObject>>& Depends = SaveContext.GetCellObjectDependencies()[Export.Cell];
					for (TObjectPtr<UObject> Object : Depends)
					{
						FPackageIndex Dep = Linker->ObjectIndicesMap.FindRef(Object);
						IncludeIndexAsDependency(CreateBeforeSerializationDependencies, Dep);
					}
				}
				{
					const TSet<Verse::VCell*>& CellDeps = SaveContext.GetCellCellDependeices()[Export.Cell];
					for (Verse::VCell* Cell : CellDeps)
					{
						FPackageIndex Dep = Linker->CellIndicesMap.FindRef(Cell);
						CreateBeforeSerializationDependencies.Add(Dep);
					}
				}
			}

			for (FPackageIndex Index : SerializationBeforeSerializationDependencies)
			{
				if (Export.FirstExportDependency == -1)
				{
					Export.FirstExportDependency = Linker->Summary.PreloadDependencyCount;
					check(Export.SerializationBeforeSerializationDependencies == 0 && Export.CreateBeforeSerializationDependencies == 0);
				}
				Linker->Summary.PreloadDependencyCount++;
				Export.SerializationBeforeSerializationDependencies++;
				DepedenciesStream.EnterElement() << Index;
			}
			for (FPackageIndex Index : CreateBeforeSerializationDependencies)
			{
				if (SerializationBeforeSerializationDependencies.Contains(Index))
				{
					continue;
				}
				if (Export.FirstExportDependency == -1)
				{
					Export.FirstExportDependency = Linker->Summary.PreloadDependencyCount;
					check(Export.SerializationBeforeSerializationDependencies == 0 && Export.CreateBeforeSerializationDependencies == 0);
				}
				Linker->Summary.PreloadDependencyCount++;
				Export.CreateBeforeSerializationDependencies++;
				DepedenciesStream.EnterElement() << Index;
			}
		}
	}
}

void WriteGatherableText(FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext)
{
	FStructuredArchive::FStream Stream = StructuredArchiveRoot.EnterStream(TEXT("GatherableTextData"));
	// Do not gather text data during cooking since the data isn't only scrubbed off of editor packages
	if (!SaveContext.IsCooking()
		&& !SaveContext.IsFilterEditorOnly()
		// We can only cache packages that:
		//	1) Don't contain script data, as script data is very volatile and can only be safely gathered after it's been compiled (which happens automatically on asset load).
		//	2) Don't contain text keyed with an incorrect package localization ID, as these keys will be changed later during save.
		//	3) Don't contain text initialized from a string which includes text properties pasted into the property matrix 
		&& !EnumHasAnyFlags(SaveContext.GatherableTextResultFlags, EPropertyLocalizationGathererResultFlags::HasScript | EPropertyLocalizationGathererResultFlags::HasTextWithInvalidPackageLocalizationID | EPropertyLocalizationGathererResultFlags::HasTextInitializedFromString))
	{
		FLinkerSave* Linker = SaveContext.GetLinker();

		// The Editor version is used as part of the check to see if a package is too old to use the gather cache, so we always have to add it if we have gathered loc for this asset
		// Note that using custom version here only works because we already added it to the export tagger before the package summary was serialized
		Linker->UsingCustomVersion(FEditorObjectVersion::GUID);

		Linker->Summary.GatherableTextDataOffset = (int32)Linker->Tell();
		Linker->Summary.GatherableTextDataCount = Linker->GatherableTextDataMap.Num();
		for (FGatherableTextData& GatherableTextData : Linker->GatherableTextDataMap)
		{
			Stream.EnterElement() << GatherableTextData;
		}
	}
}

int64 WriteObjectDataResources(TArray<FObjectDataResource>& DataResources, FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext)
{
	check(SaveContext.GetLinker());
	FLinkerSave& Linker = *SaveContext.GetLinker();

	// The data resource table is only saved for cooked output
	if (Linker.IsCooking() == false || DataResources.IsEmpty())
	{
		Linker.Summary.DataResourceOffset = -1;
		return 0;
	}

	Linker.Summary.DataResourceOffset = (int32)Linker.Tell();
	FObjectDataResource::Serialize(StructuredArchiveRoot.EnterField(TEXT("DataResources")), DataResources);

	return Linker.Tell() - Linker.Summary.DataResourceOffset;
}

/**
 * Utility for safely setting the TotalHeaderSize member of FPackageFileSummary with a int64 value.
 * 
 * FPackageFileSummary uses int32 for a lot of offsets but a lot of our package writing code is
 * capable of handling files that exceed MAX_int32 so the final size is calculated as a int64.
 * If this value is truncated when storing in FPackageFileSummary then the package will not read
 * in correctly and most likely cause a crash. Rather than allowing the user to save bad data we
 * can use this utility to catch the error and log it so that the user can take action.
 */
ESavePackageResult SetSummaryTotalHeaderSize(FSaveContext& SaveContext, int64 TotalHeaderSize)
{
	FLinkerSave* Linker = SaveContext.GetLinker();

	if (TotalHeaderSize <= MAX_int32)
	{
		Linker->Summary.TotalHeaderSize = (int32)TotalHeaderSize;
		return ESavePackageResult::Success;
	}
	else
	{
		UE_LOG(LogSavePackage, Error, TEXT("Package header for '%s' is too large (%" UINT64_FMT " bytes), some package file summary offsets will be truncated when stored as a int32"),
			*SaveContext.GetPackage()->GetName(), TotalHeaderSize);

		return ESavePackageResult::Error;
	}
}

AUTORTFM_DISABLE ESavePackageResult WritePackageHeader(FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext)
{
	FLinkerSave* Linker = SaveContext.GetLinker();
#if WITH_EDITOR
	FArchiveStackTraceIgnoreScope IgnoreDiffScope(SaveContext.IsIgnoringHeaderDiff());
#endif

	// Write Dummy Summary
	{
		StructuredArchiveRoot.GetUnderlyingArchive() << Linker->Summary;
	}
	SaveContext.OffsetAfterPackageFileSummary = (int32)Linker->Tell();

	// Write Name Map
	Linker->Summary.NameOffset = SaveContext.OffsetAfterPackageFileSummary;
	{
		SCOPED_SAVETIMER(UPackage_Save_BuildNameMap);
		checkf(Linker->Summary.NameCount == Linker->NameMap.Num(), TEXT("Summary NameCount didn't match linker name map count when saving package header for '%s'"), *Linker->LinkerRoot->GetName());
		for (const FNameEntryId NameEntryId : Linker->NameMap)
		{
			FName::GetEntry(NameEntryId)->Write(*Linker);
		}
	}

	// Write Soft Object Paths
	{
		TGuardValue<bool> Guard(Linker->bIsWritingHeaderSoftObjectPaths, true);

		SCOPED_SAVETIMER(UPackage_Save_SaveSoftObjectPaths);
		// Save soft object paths references
		Linker->Summary.SoftObjectPathsOffset = (int32)Linker->Tell();
		Linker->Summary.SoftObjectPathsCount = Linker->SoftObjectPathList.Num();
		// Do not map soft object path during the table serialization itself
		FStructuredArchive::FStream SoftObjectPathListStream = StructuredArchiveRoot.EnterStream(TEXT("SoftObjectPathList"));
		for (FSoftObjectPath& Path : Linker->SoftObjectPathList)
		{
			SoftObjectPathListStream.EnterElement() << Path;
		}
	}

	// Write GatherableText
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteGatherableTextData);
		WriteGatherableText(StructuredArchiveRoot, SaveContext);
	}

#if WITH_METADATA
	// Save MetaData
	if (!SaveContext.IsCooking())
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteMetaData);
		UE::SavePackageUtilities::SaveMetaData(SaveContext.GetPackage(), Linker, StructuredArchiveRoot);
	}
#endif

	// Save Dummy Import Map, overwritten later.
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteDummyImportMap);
		Linker->Summary.ImportOffset = (int32)Linker->Tell();
		for (FObjectImport& Import : Linker->ImportMap)
		{
			StructuredArchiveRoot.GetUnderlyingArchive() << Import;
		}
	}
	SaveContext.OffsetAfterImportMap = (int32)Linker->Tell();

	// Save Dummy Export Map, overwritten later.
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteDummyExportMap);
		Linker->Summary.ExportOffset = (int32)Linker->Tell();
		for (FObjectExport& Export : Linker->ExportMap)
		{
			*Linker << Export;
		}
	}
	SaveContext.OffsetAfterExportMap = (int32)Linker->Tell();

	// Save Dummy Cell Import Map, overwritten later.
	{
		Linker->Summary.CellImportOffset = (int32)Linker->Tell();
		for (FCellImport& CellImport : Linker->CellImportMap)
		{
			*Linker << CellImport;
		}
	}

	// Save Dummy Cell Export Map, overwritten later.
	{
		Linker->Summary.CellExportOffset = (int32)Linker->Tell();
		for (FCellExport& CellExport : Linker->CellExportMap)
		{
			*Linker << CellExport;
		}
	}

	// Save Depend Map
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteDependsMap);

		FStructuredArchive::FStream DependsStream = StructuredArchiveRoot.EnterStream(TEXT("DependsMap"));
		Linker->Summary.DependsOffset = (int32)Linker->Tell();
		if (SaveContext.IsCooking())
		{
			//@todo optimization, this should just be stripped entirely from cooked packages
			TArray<FPackageIndex> Depends; // empty array
			for (int32 ExportIndex = 0; ExportIndex < Linker->ExportMap.Num(); ++ExportIndex)
			{
				DependsStream.EnterElement() << Depends;
			}
		}
		else
		{
			// save depends map (no need for later patching)
			check(Linker->DependsMap.Num() == Linker->ExportMap.Num());
			for (TArray<FPackageIndex>& Depends : Linker->DependsMap)
			{
				DependsStream.EnterElement() << Depends;
			}
		}
	}

	// Write Soft Package references & Searchable Names
	const bool bCookSoftPackagesReferences = SaveContext.IsCooking() && SaveContext.GetSaveArgs().SaveFlags & SAVE_CookSoftPackageReferences;
	if (!SaveContext.IsFilterEditorOnly() || bCookSoftPackagesReferences)
	{
		SCOPED_SAVETIMER(UPackage_Save_SaveSoftPackagesAndSearchableNames);

		// Save soft package references
		Linker->Summary.SoftPackageReferencesOffset = (int32)Linker->Tell();
		Linker->Summary.SoftPackageReferencesCount = Linker->SoftPackageReferenceList.Num();
		{
			FStructuredArchive::FStream SoftReferenceStream = StructuredArchiveRoot.EnterStream(TEXT("SoftReferences"));
			for (FName& SoftPackageName : Linker->SoftPackageReferenceList)
			{
				SoftReferenceStream.EnterElement() << SoftPackageName;
			}

			// Save searchable names map
			if (!SaveContext.IsFilterEditorOnly())
			{
				Linker->Summary.SearchableNamesOffset = (int32)Linker->Tell();
				Linker->SerializeSearchableNamesMap(StructuredArchiveRoot.EnterField(TEXT("SearchableNames")));
			}
			else
			{
				Linker->Summary.SearchableNamesOffset = 0;
			}
		}
	}
	else
	{
		Linker->Summary.SoftPackageReferencesCount = 0;
		Linker->Summary.SoftPackageReferencesOffset = 0;
		Linker->Summary.SearchableNamesOffset = 0;
	}

	// Write ImportTypeHierarchies
#if WITH_EDITORONLY_DATA
	Linker->Summary.ImportTypeHierarchiesCount = 0;
	Linker->Summary.ImportTypeHierarchiesOffset = 0;

	if (!SaveContext.IsCooking())
	{
		SCOPED_SAVETIMER(UPackage_Save_SaveImportTypeHierarchies);

		Linker->Summary.ImportTypeHierarchiesOffset = (int32)Linker->Tell();
		Linker->Summary.ImportTypeHierarchiesCount = Linker->ImportTypeHierarchies.Num();

		// Sort before saving to keep the entry order consistent.
		Linker->ImportTypeHierarchies.KeySort(TLess<FPackageIndex>());

		FStructuredArchive::FStream ImportTypeHierarchyStream = StructuredArchiveRoot.EnterStream(TEXT("ImportTypeHierarchies"));

		ImportTypeHierarchyStream << Linker->ImportTypeHierarchies;
	}
#endif

	// Save thumbnails
	{
		SCOPED_SAVETIMER(UPackage_Save_SaveThumbnails);
		UE::SavePackageUtilities::SaveThumbnails(SaveContext.GetPackage(), Linker, StructuredArchiveRoot.EnterField(TEXT("Thumbnails")));
	}
	{
		// Save asset registry data so the editor can search for information about assets in this package
		SCOPED_SAVETIMER(UPackage_Save_SaveAssetRegistryData);
		FArchiveSavePackageData& ArchiveSavePackageData = SaveContext.GetArchiveSavePackageData();
		UE::AssetRegistry::FWritePackageDataArgs WriteARArgs;
		WriteARArgs.ParentRecord = &StructuredArchiveRoot;
		WriteARArgs.Package = SaveContext.GetPackage();
		WriteARArgs.Linker = Linker;
		WriteARArgs.ImportsUsedInGame = &SaveContext.GetImportsUsedInGame();
		WriteARArgs.SoftPackagesUsedInGame = &SaveContext.GetSoftPackagesUsedInGame();
		WriteARArgs.PackageBuildDependencies = &SaveContext.GetPackageBuildDependencies();
		WriteARArgs.bProceduralSave = SaveContext.IsProceduralSave();
		WriteARArgs.CookContext = ArchiveSavePackageData.CookContext;
		WriteARArgs.OutAssetDatas = &SaveContext.GetSavedAssets();
		UE::AssetRegistry::WritePackageData(WriteARArgs);
	}
	// Save level information used by World browser
	{
		SCOPED_SAVETIMER(UPackage_Save_WorldLevelData);
		UE::SavePackageUtilities::SaveWorldLevelInfo(SaveContext.GetPackage(), Linker, StructuredArchiveRoot);
	}

	// Write Preload Dependencies
	{
		SCOPED_SAVETIMER(UPackage_Save_PreloadDependencies);
		SavePreloadDependencies(StructuredArchiveRoot, SaveContext);
	}

	// Rather than check if an offset is truncated every time we assign one, we can just check the final TotalHeaderSize to see if it is truncated.
	// Checking every time an offset is assigned would let us fail quicker but a) relies that new code follows the convention b) bloats the code a fair bit.
	const ESavePackageResult Result = SetSummaryTotalHeaderSize(SaveContext, Linker->Tell());
	if (Result != ESavePackageResult::Success)
	{
		return Result;
	}

	return ReturnSuccessOrCancel();
}

ESavePackageResult WritePackageTextHeader(FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext)
{
	FLinkerSave* Linker = SaveContext.GetLinker();

	// Write GatherableText
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteGatherableTextData);
		WriteGatherableText(StructuredArchiveRoot, SaveContext);
	}

	// Write ImportTable
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteImportTable);
		FStructuredArchive::FStream ImportTableStream = StructuredArchiveRoot.EnterStream(TEXT("ImportTable"));
		for (FObjectImport& Import : Linker->ImportMap)
		{
			ImportTableStream.EnterElement() << Import;
		}
	}

	// Write ExportTable
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteExportTable);
		FStructuredArchive::FStream ExportTableStream = StructuredArchiveRoot.EnterStream(TEXT("ExportTable"));
		for (FObjectExport& Export : Linker->ExportMap)
		{
			ExportTableStream.EnterElement() << Export;
		}
	}

	// Save thumbnails
	{
		SCOPED_SAVETIMER(UPackage_Save_SaveThumbnails);
		UE::SavePackageUtilities::SaveThumbnails(SaveContext.GetPackage(), Linker, StructuredArchiveRoot.EnterField(TEXT("Thumbnails")));
	}

	// Save level information used by World browser
	{
		SCOPED_SAVETIMER(UPackage_Save_WorldLevelData);
		UE::SavePackageUtilities::SaveWorldLevelInfo(SaveContext.GetPackage(), Linker, StructuredArchiveRoot);
	}

#if WITH_METADATA
	// Save meta data information
	{
		SCOPED_SAVETIMER(UPackage_Save_WriteMetaData);
		UE::SavePackageUtilities::SaveMetaData(SaveContext.GetPackage(), Linker, StructuredArchiveRoot);
	}
#endif

	return ReturnSuccessOrCancel();
}

[[nodiscard]] ESavePackageResult WriteCookedExports(FArchive& ExportsArchive, FSaveContext& SaveContext)
{
	// Used to make any serialized offset during export serialization relative to the beginning of the export.
	struct FExportProxyArchive
		: public FArchiveProxy
	{
		FExportProxyArchive(FArchive& Inner)
			: FArchiveProxy(Inner)
			, Offset(Inner.Tell())
		{
		}

		virtual void Seek(int64 Pos)
		{
			InnerArchive.Seek(Offset + Pos);
		}
		virtual int64 Tell()
		{
			return InnerArchive.Tell() - Offset;
		}
		virtual int64 TotalSize()
		{
			return InnerArchive.TotalSize() - Offset;
		}

		int64 Offset;
	};

	SCOPED_SAVETIMER(UPackage_Save_SaveExports);

	check(SaveContext.GetLinker() && SaveContext.GetLinker()->IsCooking());
	FLinkerSave& Linker = *SaveContext.GetLinker();
	FScopedSlowTask SlowTask((float)Linker.ExportMap.Num(), FText(), SaveContext.IsUsingSlowTask());

	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();

	for (int32 ExportIndex = 0; ExportIndex < Linker.ExportMap.Num(); ExportIndex++)
	{
		if (GWarn->ReceivedUserCancel())
		{
			return ESavePackageResult::Canceled;
		}
		SlowTask.EnterProgressFrame();

		FObjectExport& Export = Linker.ExportMap[ExportIndex];
		if (Export.Object == nullptr)
		{
			continue;
		}

		SCOPED_SAVETIMER(UPackage_Save_SaveExport);
		SCOPED_SAVETIMER_TEXT(*WriteToString<128>(GetClassTraceScope(Export.Object), TEXT("_SaveSerialize")));

		UE::FScopedObjectSerializeContext ObjectSerializeContext(Export.Object, Linker);

		Export.SerialOffset = ExportsArchive.Tell();
		Linker.CurrentlySavingExport = FPackageIndex::FromExport(ExportIndex);
		Linker.CurrentlySavingExportObject = Export.Object;

		FExportProxyArchive Ar(ExportsArchive);
		TGuardValue<FArchive*> GuardSaver(Linker.Saver, &Ar);

		if (Export.Object->HasAnyFlags(RF_ClassDefaultObject))
		{
			Export.Object->GetClass()->SerializeDefaultObject(Export.Object, Linker);
		}
		else
		{
			Export.Object->Serialize(Linker);
#if WITH_EDITOR
			Export.Object->CookAdditionalFiles(SaveContext.GetFilename(), SaveContext.GetTargetPlatform(),
				[&SaveContext](const TCHAR* Filename, void* Data, int64 Size)
				{
					FLargeMemoryWriter& Writer = SaveContext.AdditionalFilesFromExports.Emplace_GetRef(0, true, Filename);
					Writer.Serialize(Data, Size);
				});
#endif
		}

		Linker.CurrentlySavingExport = FPackageIndex();
		Linker.CurrentlySavingExportObject = nullptr;
		Export.SerialSize = ExportsArchive.Tell() - Export.SerialOffset;

		if (Export.ScriptSerializationEndOffset - Export.ScriptSerializationStartOffset > 0)
		{
			// Offset is already relative to export offset because of FExportProxyArchive 
			check(Export.ScriptSerializationStartOffset >= 0);
			check(Export.ScriptSerializationEndOffset <= Export.SerialSize);
		}
		else
		{
			check(Export.ScriptSerializationEndOffset == 0);
			check(Export.ScriptSerializationStartOffset == 0);
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
	for (int32 CellExportIndex = 0; CellExportIndex < Linker.CellExportMap.Num(); CellExportIndex++)
	{
		FCellExport& Export = Linker.CellExportMap[CellExportIndex];

		Export.SerialOffset = ExportsArchive.Tell();

		FExportProxyArchive Ar(ExportsArchive);
		TGuardValue<FArchive*> GuardSaver(Linker.Saver, &Ar);

		{
			FStructuredArchiveFromArchive StructuredArchive(Linker);
			Verse::FStructuredArchiveVisitor Visitor(Context, StructuredArchive.GetSlot().EnterRecord());
			Export.CppClassInfo->SerializeLayout(Context, Export.Cell, Visitor);
		}

		Export.SerialLayoutSize = ExportsArchive.Tell() - Export.SerialOffset;

		{
			FStructuredArchiveFromArchive StructuredArchive(Linker);
			Verse::FStructuredArchiveVisitor Visitor(Context, StructuredArchive.GetSlot().EnterRecord());
			Export.Cell->Serialize(Context, Visitor);
		}

		Export.SerialSize = ExportsArchive.Tell() - Export.SerialOffset;
	}
#endif

	return Linker.IsError() ? ESavePackageResult::Error : ReturnSuccessOrCancel();
}

ESavePackageResult WriteExports(FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext)
{
	//COOK_STAT(FScopedDurationTimer SaveTimer(FSavePackageStats::SerializeExportsTimeSec));
	SCOPED_SAVETIMER(UPackage_Save_SaveExports);
	FLinkerSave* Linker = SaveContext.GetLinker();
	FScopedSlowTask SlowTask((float)Linker->ExportMap.Num(), FText(), SaveContext.IsUsingSlowTask());

	FStructuredArchive::FRecord ExportsRecord = StructuredArchiveRoot.EnterRecord(TEXT("Exports"));

	// Save exports.
	int32 LastExportSaveStep = 0;
	for (int32 i = 0; i < Linker->ExportMap.Num(); i++)
	{
		if (GWarn->ReceivedUserCancel())
		{
			return ESavePackageResult::Canceled;
		}
		SlowTask.EnterProgressFrame();

		FObjectExport& Export = Linker->ExportMap[i];
		if (Export.Object)
		{
			SCOPED_SAVETIMER(UPackage_Save_SaveExport);

			UE::FScopedObjectSerializeContext ObjectSerializeContext(Export.Object, ExportsRecord.GetUnderlyingArchive());

			// Save the object data.
			Export.SerialOffset = Linker->Tell();
			Linker->CurrentlySavingExport = FPackageIndex::FromExport(i);
			Linker->CurrentlySavingExportObject = Export.Object;

			FString ObjectName = Export.Object->GetPathName(SaveContext.GetPackage());
			FStructuredArchive::FSlot ExportSlot = ExportsRecord.EnterField(*ObjectName);

#if WITH_EDITOR
			bool bSupportsText = UClass::IsSafeToSerializeToStructuredArchives(Export.Object->GetClass());
#else
			bool bSupportsText = false;
#endif

			SCOPED_SAVETIMER_TEXT(*WriteToString<128>(GetClassTraceScope(Export.Object), TEXT("_SaveSerialize")));
			if (Export.Object->HasAnyFlags(RF_ClassDefaultObject))
			{
				if (bSupportsText)
				{
					Export.Object->GetClass()->SerializeDefaultObject(Export.Object, ExportSlot);
				}
				else
				{
					FArchiveUObjectFromStructuredArchive Adapter(ExportSlot);
					Export.Object->GetClass()->SerializeDefaultObject(Export.Object, Adapter.GetArchive());
					Adapter.Close();
				}
			}
			else
			{
				if (bSupportsText)
				{
					FStructuredArchive::FRecord ExportRecord = ExportSlot.EnterRecord();
					Export.Object->Serialize(ExportRecord);
				}
				else
				{
					FArchiveUObjectFromStructuredArchive Adapter(ExportSlot);
					Export.Object->Serialize(Adapter.GetArchive());
					Adapter.Close();
				}

#if WITH_EDITOR
				if (Linker->IsCooking())
				{
					Export.Object->CookAdditionalFiles(SaveContext.GetFilename(), SaveContext.GetTargetPlatform(),
						[&SaveContext](const TCHAR* Filename, void* Data, int64 Size)
						{
							FLargeMemoryWriter& Writer = SaveContext.AdditionalFilesFromExports.Emplace_GetRef(0, true, Filename);
							Writer.Serialize(Data, Size);
						});
				}
#endif
			}
			Linker->CurrentlySavingExport = FPackageIndex();
			Linker->CurrentlySavingExportObject = nullptr;
			Export.SerialSize = Linker->Tell() - Export.SerialOffset;

			if (Export.ScriptSerializationEndOffset - Export.ScriptSerializationStartOffset > 0)
			{
				Export.ScriptSerializationStartOffset -= Export.SerialOffset;
				Export.ScriptSerializationEndOffset -= Export.SerialOffset;
				check(Export.ScriptSerializationStartOffset >= 0);
				check(Export.ScriptSerializationEndOffset <= Export.SerialSize);
			}
			else
			{
				check(Export.ScriptSerializationEndOffset == 0);
				check(Export.ScriptSerializationStartOffset == 0);
			}
		}
	}
	// if an error occurred on the linker while serializing exports, return an error
	return Linker->IsError() ? ESavePackageResult::Error : ReturnSuccessOrCancel();
}

[[nodiscard]] ESavePackageResult WriteBulkData(FSaveContext& SaveContext, int64& VirtualExportsFileOffset)
{
	COOK_STAT(FScopedDurationTimer SaveTimer(FSavePackageStats::SerializeBulkDataTimeSec));

	FLinkerSave& Linker = *SaveContext.GetLinker();

	Linker.Summary.BulkDataStartOffset = Linker.Tell();

	if (Linker.IsCooking() == false)
	{
		check(!Linker.HasCookedIndexBulkData());
		VirtualExportsFileOffset += Linker.GetBulkDataArchive(FBulkDataCookedIndex::Default).TotalSize();
	}

	const bool bIsOptionalRealm = SaveContext.GetCurrentHarvestingRealm() == ESaveRealm::Optional;

	IPackageWriter* PackageWriter = SaveContext.GetPackageWriter();

	if (PackageWriter == nullptr)
	{
		// Saving non-inline bulk data to the end of the package (Editor)

		check(!Linker.HasCookedIndexBulkData());
		check(Linker.IsCooking() == false);
		check(bIsOptionalRealm == false);

		FFileRegionMemoryWriter& Ar = Linker.GetBulkDataArchive(FBulkDataCookedIndex::Default);
		if (const int64 TotalSize = Ar.TotalSize(); TotalSize > 0)
		{
			FIoBuffer Buffer(FIoBuffer::AssumeOwnership, Ar.ReleaseOwnership(), TotalSize);
			Linker.Serialize(Buffer.GetData(), Buffer.GetSize());
			SaveContext.TotalPackageSizeUncompressed += TotalSize;
		}

		return ESavePackageResult::Success;
	}

	// Saving non-inline bulk data to separate file(s) (Cooking/EditorDomain)

	if (Linker.IsCooking() == false)
	{
		const bool bSaveBulkDataByReference = SaveContext.GetSaveArgs().SaveFlags & SAVE_BulkDataByReference;
		if (bSaveBulkDataByReference)
		{
			if (Linker.bUpdatingLoadedPath)
			{
				UE_LOGF(LogSavePackage, Error, "Save bulk data '%ls' FAILED, reason '%ls'",
					SaveContext.GetFilename(), TEXT("SAVE_BulkDataByReference is incompatible with bUpdatingLoadedPath"));
				return ESavePackageResult::Error;
			}
		}
	}

	const FName PackageName = SaveContext.GetPackage()->GetFName();
	const FPackageId PackageId = FPackageId::FromName(PackageName);
	const uint16 MultiOutputIndex = bIsOptionalRealm ? 1 : 0;

	auto GetFilePath = [&SaveContext, bIsOptionalRealm](EPackageExtension Ext, FBulkDataCookedIndex CookedIndex = FBulkDataCookedIndex::Default) -> FString
	{
		if (CookedIndex.IsDefault())
		{
			FString FileExt = bIsOptionalRealm ? FString(".o") + LexToString(Ext) : LexToString(Ext);
			return FPathViews::ChangeExtension(SaveContext.GetFilename(), FileExt);
		}
		else
		{
			FString FileExt = CookedIndex.GetAsExtension() + LexToString(Ext);
			return FPathViews::ChangeExtension(SaveContext.GetFilename(), FileExt);
		}
	};

	auto WriteToPackageWriter = [&SaveContext, PackageWriter, bIsOptionalRealm](FFileRegionMemoryWriter& Ar, IPackageWriter::FBulkDataInfo Info) -> int64
	{
		if (int64 TotalSize = Ar.TotalSize(); TotalSize > 0)
		{
			checkf(bIsOptionalRealm == false || Info.BulkDataType != IPackageWriter::FBulkDataInfo::Mmap,
				TEXT("Memory mapped bulk data is currently not supported for optional package '%s'"), *SaveContext.GetPackage()->GetName());

			FIoBuffer Buffer(FIoBuffer::AssumeOwnership, Ar.ReleaseOwnership(), TotalSize);
			PackageWriter->WriteBulkData(Info, Buffer, Ar.GetFileRegions());
			return TotalSize;
		}
		return 0;
	};

	FScopedSlowTask Feedback(3.0f);

	Linker.ForEachBulkDataCookedIndex([&SaveContext, &PackageName, &PackageId, &WriteToPackageWriter, &GetFilePath, MultiOutputIndex](FBulkDataCookedIndex CookedIndex, FFileRegionMemoryWriter& Ar)
		{
			SaveContext.TotalPackageSizeUncompressed += WriteToPackageWriter(Ar, IPackageWriter::FBulkDataInfo
				{
					PackageName,
					IPackageWriter::FBulkDataInfo::BulkSegment,
					GetFilePath(EPackageExtension::BulkDataDefault, CookedIndex),
					CreateBulkDataIoChunkId(PackageId.Value(), MultiOutputIndex, CookedIndex.GetValue(), EIoChunkType::BulkData),
					MultiOutputIndex
				});
		}, EBulkDataPayloadType::BulkSegment);

	Feedback.EnterProgressFrame();

	// @note FH: temporarily do not handle optional bulk data into editor optional packages, proper support will be added soon
	if (bIsOptionalRealm == false)
	{
		Linker.ForEachBulkDataCookedIndex([&SaveContext, &PackageName, &PackageId, &WriteToPackageWriter, &GetFilePath, MultiOutputIndex](FBulkDataCookedIndex CookedIndex, FFileRegionMemoryWriter& Ar)
			{
				SaveContext.TotalPackageSizeUncompressed += WriteToPackageWriter(Ar, IPackageWriter::FBulkDataInfo
					{
						PackageName,
						IPackageWriter::FBulkDataInfo::Optional,
						GetFilePath(EPackageExtension::BulkDataOptional, CookedIndex),
						CreateBulkDataIoChunkId(PackageId.Value(), MultiOutputIndex, CookedIndex.GetValue(), EIoChunkType::OptionalBulkData),
						MultiOutputIndex
					});
			}, EBulkDataPayloadType::Optional);
	}

	Feedback.EnterProgressFrame();

	Linker.ForEachBulkDataCookedIndex([&SaveContext, &PackageName, &PackageId, &WriteToPackageWriter, &GetFilePath, MultiOutputIndex](FBulkDataCookedIndex CookedIndex, FFileRegionMemoryWriter& Ar)
		{
			SaveContext.TotalPackageSizeUncompressed += WriteToPackageWriter(Ar, IPackageWriter::FBulkDataInfo
				{
					PackageName,
					IPackageWriter::FBulkDataInfo::Mmap,
					GetFilePath(EPackageExtension::BulkDataMemoryMapped),
					CreateBulkDataIoChunkId(PackageId.Value(), MultiOutputIndex, CookedIndex.GetValue(), EIoChunkType::MemoryMappedBulkData),
					MultiOutputIndex
				});
		}, EBulkDataPayloadType::MemoryMapped);

	Feedback.EnterProgressFrame();

	return ESavePackageResult::Success;
}

[[nodiscard]] ESavePackageResult BuildAndWriteTrailer(IPackageWriter* PackageWriter, FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext, int64& InOutCurrentOffset)
{
	SaveContext.GetLinker()->Summary.PayloadTocOffset = INDEX_NONE;

	if (SaveContext.GetLinker()->PackageTrailerBuilder.IsValid())
	{
		// At the moment we assume that we cannot have reference payloads in the trailer if SAVE_BulkDataByReference is not set and we
		// cannot have locally stored payloads if SAVE_BulkDataByReference is set.
		checkf((SaveContext.GetSaveArgs().SaveFlags & SAVE_BulkDataByReference) != 0 || SaveContext.GetLinker()->PackageTrailerBuilder->GetNumReferencedPayloads() == 0,
			TEXT("Attempting to build a package trailer with referenced payloads but the SAVE_BulkDataByReference flag is not set. '%s'"), *SaveContext.GetPackage()->GetName());

		checkf((SaveContext.GetSaveArgs().SaveFlags & SAVE_BulkDataByReference) == 0 || SaveContext.GetLinker()->PackageTrailerBuilder->GetNumLocalPayloads() == 0,
			TEXT("Attempting to build a package trailer with local payloads but the SAVE_BulkDataByReference flag is set. '%s'"), *SaveContext.GetPackage()->GetName());

		checkf(SaveContext.IsTextFormat() == false, TEXT("Attempting to build a package trailer for text based asset '%s', this is not supported!"), *SaveContext.GetPackage()->GetName());

		SaveContext.GetLinker()->Summary.PayloadTocOffset = InOutCurrentOffset;
		if (!PackageWriter)
		{
			if (!SaveContext.GetLinker()->PackageTrailerBuilder->BuildAndAppendTrailer(SaveContext.GetLinker(), *SaveContext.GetLinker(), InOutCurrentOffset))
			{
				return ESavePackageResult::Error;
			}
		}
		else
		{
			FLargeMemoryWriter TrailerData(0, true /* IsPersistent */);
			if (!SaveContext.GetLinker()->PackageTrailerBuilder->BuildAndAppendTrailer(SaveContext.GetLinker(), TrailerData, InOutCurrentOffset))
			{
				return ESavePackageResult::Error;
			}

			IPackageWriter::FPackageTrailerInfo TrailerInfo;
			TrailerInfo.PackageName = SaveContext.GetPackage()->GetFName();
			PackageWriter->WritePackageTrailer(TrailerInfo, FIoBuffer(FIoBuffer::AssumeOwnership, TrailerData.ReleaseOwnership(), TrailerData.TotalSize()));
		}

		SaveContext.GetLinker()->PackageTrailerBuilder.Reset();
	}

	return ESavePackageResult::Success;
}

ESavePackageResult WriteAdditionalExportFiles(FSaveContext& SaveContext)
{
	FSavePackageContext* SavePackageContext = SaveContext.GetSavePackageContext();

	if (SaveContext.IsCooking() && SaveContext.AdditionalFilesFromExports.Num() > 0)
	{
		checkf(SaveContext.GetCurrentHarvestingRealm() != ESaveRealm::Optional, TEXT("Addtional export files is currently unsupported with optional package multi output, Package %s"), *SaveContext.GetPackage()->GetName());
		IPackageWriter* PackageWriter = SavePackageContext ? SavePackageContext->PackageWriter : nullptr;
		checkf(PackageWriter, TEXT("Cooking requires a PackageWriter"));
		for (FLargeMemoryWriter& Writer : SaveContext.AdditionalFilesFromExports)
		{
			const int64 Size = Writer.TotalSize();
			SaveContext.TotalPackageSizeUncompressed += Size;

			IPackageWriter::FAdditionalFileInfo FileInfo;
			FileInfo.PackageName = SaveContext.GetPackage()->GetFName();
			FileInfo.Filename = *Writer.GetArchiveName();

			UE_LOGF(LogSavePackage, Verbose, "SavePackage writing additional file '%ls' due to package '%ls'.", *FileInfo.Filename, *FileInfo.PackageName.ToString());

			FIoBuffer FileData(FIoBuffer::AssumeOwnership, Writer.ReleaseOwnership(), Size);

			// This might not actuallly write the file, but instead add it to a queue to write later.
			// (See TPackageWriterToSharedBuffer).
			PackageWriter->WriteAdditionalFile(FileInfo, FileData);
		}
		SaveContext.AdditionalFilesFromExports.Empty();
	}
	return ReturnSuccessOrCancel();
}

ESavePackageResult UpdatePackageHeader(FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_Save_UpdatePackageHeader);

	FLinkerSave* Linker = SaveContext.GetLinker();
	int64 OffsetBeforeUpdates = Linker->Tell();
#if WITH_EDITOR
	FArchiveStackTraceIgnoreScope IgnoreDiffScope(SaveContext.IsIgnoringHeaderDiff());
#endif

	// Write Real Import Map
	if (!SaveContext.IsTextFormat())
	{
		Linker->Seek(Linker->Summary.ImportOffset);
		FStructuredArchive::FStream ImportTableStream = StructuredArchiveRoot.EnterStream(TEXT("ImportTable"));
		for (FObjectImport& Import : Linker->ImportMap)
		{
			ImportTableStream.EnterElement() << Import;
		}
	}
	// Write Real Export Map
	if (!SaveContext.IsTextFormat())
	{
		check(Linker->Tell() == SaveContext.OffsetAfterImportMap);
		Linker->Seek(Linker->Summary.ExportOffset);
		FStructuredArchive::FStream ExportTableStream = StructuredArchiveRoot.EnterStream(TEXT("ExportTable"));

		for (FObjectExport& Export : Linker->ExportMap)
		{
			ExportTableStream.EnterElement() << Export;
		}
		check(Linker->Tell() == SaveContext.OffsetAfterExportMap);
	}

	// Write Real Cell Import Map
	if (!SaveContext.IsTextFormat())
	{
		Linker->Seek(Linker->Summary.CellImportOffset);
		FStructuredArchive::FStream CellExportTableStream = StructuredArchiveRoot.EnterStream(TEXT("CellImportTable"));

		for (FCellImport& CellImport : Linker->CellImportMap)
		{
			CellExportTableStream.EnterElement() << CellImport;
		}
	}
	// Write Real Cell Export Map
	if (!SaveContext.IsTextFormat())
	{
		Linker->Seek(Linker->Summary.CellExportOffset);
		FStructuredArchive::FStream CellExportTableStream = StructuredArchiveRoot.EnterStream(TEXT("CellExportTable"));

		for (FCellExport& CellExport : Linker->CellExportMap)
		{
			CellExportTableStream.EnterElement() << CellExport;
		}
	}

	// Figure out if at leat one export is marked as an asset
	bool bContainsAsset = false;
	for (FObjectExport& Export : Linker->ExportMap)
	{
		bContainsAsset |= Export.bIsAsset;
	}

	// Update Summary
	// Write Real Summary
	{
		//@todo: remove ExportCount and NameCount - no longer used
		Linker->Summary.Generations.Last().ExportCount = Linker->Summary.ExportCount;
		Linker->Summary.Generations.Last().NameCount = Linker->Summary.NameCount;

		// create the package source (based on developer or user created)
#if (UE_BUILD_SHIPPING && WITH_EDITOR)
		Linker->Summary.PackageSource = FMath::Rand() * FMath::Rand();
#else
		Linker->Summary.PackageSource = FCrc::StrCrc_DEPRECATED(*FPaths::GetBaseFilename(SaveContext.GetFilename()).ToUpper());
#endif

		// Flag package as requiring localization gather if the archive requires localization gathering.
		Linker->LinkerRoot->ThisRequiresLocalizationGather(Linker->RequiresLocalizationGather());

		// Update package flags from package, in case serialization has modified package flags.
		uint32 PackageFlags = Linker->LinkerRoot->GetPackageFlags();
		if (SaveContext.IsCooking() && !bContainsAsset)
		{
			PackageFlags |= PKG_ContainsNoAsset;
		}
		else
		{
			PackageFlags &= ~PKG_ContainsNoAsset;
		}
		// Take the Linker FilterEditorOnlyData setting over the package flags to set this flag in the summary
		if (Linker->IsFilterEditorOnly())
		{
			PackageFlags |= PKG_FilterEditorOnly;
		}
		else
		{
			PackageFlags &= ~PKG_FilterEditorOnly;
		}
		Linker->Summary.SetPackageFlags(PackageFlags);

		// @todo: custom versions: when can this be checked?
		{
			// Verify that the final serialization pass hasn't added any new custom versions. Otherwise this will result in crashes when loading the package.
			bool bNewCustomVersionsUsed = false;
			for (const FCustomVersion& LinkerCustomVer : Linker->GetCustomVersions().GetAllVersions())
			{
				if (Linker->Summary.GetCustomVersionContainer().GetVersion(LinkerCustomVer.Key) == nullptr)
				{
					UE_LOGF(LogSavePackage, Error,
						"Unexpected custom version \"%ls\" found when saving %ls. This usually happens when export tagging and final serialization paths differ. Package will not be saved.",
						*LinkerCustomVer.GetFriendlyName().ToString(), *Linker->LinkerRoot->GetName());
					bNewCustomVersionsUsed = true;
				}
			}
			if (bNewCustomVersionsUsed)
			{
				return ESavePackageResult::Error;
			}
		}

		if (!SaveContext.IsTextFormat())
		{
			// Seek to beginning of the file to overwrite the previously-written placeholder Summary.
			Linker->Seek(0);
			// For the upcoming calculation of the SavedHash, set a placeholder SavedHash of 0.
			// We overwrite this in FinalizeFile.
#if WITH_EDITORONLY_DATA
			Linker->Summary.SetSavedHash(FIoHash());
#endif
		}
		else
		{
			// Calculating a SaveHash for text-based format is not yet implemented, and it's not clear what it should
			// be; should it be a hash of the text file? Should we save the file to binary to calculate the SaveHash
			// and then save it to text and write in the calculated SaveHash?
			// For now, just copy the original SaveHash from the package.
#if WITH_EDITORONLY_DATA
			FIoHash SavedHash = SaveContext.GetPackage()->GetSavedHash();
			SaveContext.PackageSavedHash = SavedHash;
			Linker->Summary.SetSavedHash(SavedHash);
#endif
		}
		{
			StructuredArchiveRoot.EnterField(TEXT("Summary")) << Linker->Summary;
		}

		if (!SaveContext.IsTextFormat())
		{
			check(Linker->Tell() == SaveContext.OffsetAfterPackageFileSummary);
		}
	}
	if (!SaveContext.IsTextFormat())
	{
		Linker->Seek(OffsetBeforeUpdates); // Return Linker Pos to the end; some packagewriters need it there
	}
	return ReturnSuccessOrCancel();
}

static bool TryCalculateSavedHashAndWriteItBackToMemoryWriter(FLargeMemoryWriter* Writer,
	FPackageFileSummary& Summary, FStringView PackageDebugName, FIoHash& OutSavedHash)
{
#if WITH_EDITORONLY_DATA
	// We only hash the Header+Exports archive up to PayloadTocOffset. The package trailer is written after that point,
	// and can be stored in other files rather than appended to the end of the Header+Exports file depending on user
	// settings. The hashes for the contents of the packageheader are embedded in the BulkDatas that store their bytes
	// in it, so its bytes are incorporated into our hash of the Header+Exports bytes which include those hashes.
	//
	// Our SavedHash does not yet include the data in the CreatePayloadSidecarFile file, that feature is not currently
	// used and we expect to remove it.
	//
	// Our SaveHash does not include data in the separate .ubulk file created when cooking; this is okay for the same
	// reason as the packagetrailer: the hash of the bytes of the bulk data are included in the bulkdata structs that
	// we do serialize.
	int64 HashEnd = Summary.PayloadTocOffset > 0 ? Summary.PayloadTocOffset : Writer->TotalSize();

	OutSavedHash = FIoHash::HashBuffer(Writer->GetData(), HashEnd);
	Summary.SetSavedHash(OutSavedHash);

	int64 SavedHashOffset = Summary.GetSavedHashRelativeOffset();
	if (SavedHashOffset < 0
		|| SavedHashOffset + static_cast<int64>(sizeof(FIoHash)) > Writer->TotalSize())
	{
		UE_LOG(LogSavePackage, Error,
			TEXT("Error writing PackageSaveHash for '%.*s': GetSavedHashRelativeOffset out of range: %" INT64_FMT " not in (0, %" INT64_FMT ")."),
			PackageDebugName.Len(), PackageDebugName.GetData(), SavedHashOffset, Writer->TotalSize());
		return false;
	}

	int64 SavedWriteOffset = Writer->Tell();
	Writer->Seek(SavedHashOffset);
	*Writer << OutSavedHash;
	Writer->Seek(SavedWriteOffset);
	return true;
#else
	OutSavedHash = FIoHash::Zero;
	return true;
#endif
}

static bool TryCalculateSavedHashAndWriteItBackToFile(const FString& Filename,
	FPackageFileSummary& Summary, bool bByteSwapping, FStringView PackageDebugName, FIoHash& OutSavedHash)
{
#if WITH_EDITORONLY_DATA
	bool bWroteSavedHash = false;

	// See comment on PayloadTocOffset in TryCalculateSavedHashAndWriteItBackToMemoryWriter.

	int64 HashEnd = Summary.PayloadTocOffset > 0 ? Summary.PayloadTocOffset : -1;
	if (!UE::SavePackageUtilities::TryHashFile(Filename, OutSavedHash, 0 /* RangeStart */, HashEnd))
	{
		UE_LOGF(LogSavePackage, Error,
			"Error writing PackageSaveHash for '%.*ls': Could not hash file %ls.",
			PackageDebugName.Len(), PackageDebugName.GetData(), *Filename);
		return false;
	}
	Summary.SetSavedHash(OutSavedHash);

	TUniquePtr<FArchive> ReopenedFile(IFileManager::Get().CreateFileWriter(*Filename, FILEWRITE_Append));
	if (!ReopenedFile)
	{
		UE_LOGF(LogSavePackage, Error,
			"Error writing PackageSaveHash for '%.*ls': Could not open for writing file %ls.",
			PackageDebugName.Len(), PackageDebugName.GetData(), *Filename);
		return false;
	}

	int64 SavedHashOffset = Summary.GetSavedHashRelativeOffset();
	if (SavedHashOffset < 0
		|| SavedHashOffset + static_cast<int64>(sizeof(FIoHash)) > ReopenedFile->TotalSize())
	{
		UE_LOG(LogSavePackage, Error,
			TEXT("Error writing PackageSaveHash for '%.*s': GetSavedHashRelativeOffset out of range: %" INT64_FMT " not in (0, %" INT64_FMT ")."),
			PackageDebugName.Len(), PackageDebugName.GetData(), SavedHashOffset, ReopenedFile->TotalSize());
		return false;
	}

	ReopenedFile->SetByteSwapping(bByteSwapping);
	ReopenedFile->Seek(SavedHashOffset);
	*ReopenedFile << OutSavedHash;

	if (ReopenedFile->IsError())
	{
		UE_LOGF(LogSavePackage, Error,
			"Error writing PackageSaveHash for '%.*ls': Write failed to file %ls.",
			PackageDebugName.Len(), PackageDebugName.GetData(), *Filename);
		return false;
	}

	return true;
#else
	OutSavedHash = FIoHash::Zero;
	return true;
#endif
}

ESavePackageResult FinalizeFile(FStructuredArchive::FRecord& StructuredArchiveRoot, FSaveContext& SaveContext)
{
	SCOPED_SAVETIMER(UPackage_Save_FinalizeFile);

	FIoHash SavedHash;

	FSavePackageContext* SavePackageContext = SaveContext.GetSavePackageContext();
	IPackageWriter* PackageWriter = SavePackageContext ? SavePackageContext->PackageWriter : nullptr;
	bool bIsOptionalRealm = SaveContext.GetCurrentHarvestingRealm() == ESaveRealm::Optional;
	FLinkerSave* Linker = SaveContext.GetLinker();
	if (PackageWriter)
	{
		FLargeMemoryWriter* Writer = static_cast<FLargeMemoryWriter*>(Linker->Saver);
		if (!TryCalculateSavedHashAndWriteItBackToMemoryWriter(Writer, Linker->Summary,
			SaveContext.GetFilename(), SavedHash))
		{
			return ESavePackageResult::Error;
		}

		IPackageWriter::FPackageInfo PackageInfo;
		PackageInfo.PackageName = SaveContext.GetPackage()->GetFName();
		// Adjust LooseFilePath if needed
		if (bIsOptionalRealm)
		{
			// Optional output have the form PackagePath.o.ext
			PackageInfo.LooseFilePath = FPathViews::ChangeExtension(SaveContext.GetFilename(), TEXT("o.") + FPaths::GetExtension(SaveContext.GetFilename()));
			PackageInfo.MultiOutputIndex = 1;
		}
		else
		{
			PackageInfo.LooseFilePath = SaveContext.GetFilename();
		}
		PackageInfo.HeaderSize = Linker->Summary.TotalHeaderSize;

		FPackageId PackageId = FPackageId::FromName(PackageInfo.PackageName);
		PackageInfo.ChunkId = CreateIoChunkId(PackageId.Value(), PackageInfo.MultiOutputIndex, EIoChunkType::ExportBundleData);

		PackageWriter->WritePackageData(PackageInfo, *Writer, Linker->FileRegions);
		SaveContext.CloseLinkerArchives();
	}
	else
	{
		check(!SaveContext.IsCooking()); // Cooking requires a PackageWriter
		check(!bIsOptionalRealm); // OptionalRealm is only supported when cooking.

		if (SaveContext.IsSaveToMemory())
		{
			FLargeMemoryWriter* Writer = static_cast<FLargeMemoryWriter*>(Linker->Saver);
			if (!TryCalculateSavedHashAndWriteItBackToMemoryWriter(Writer, Linker->Summary,
				SaveContext.GetFilename(), SavedHash))
			{
				return ESavePackageResult::Error;
			}

			EAsyncWriteOptions WriteOptions(EAsyncWriteOptions::None);

			// Add the uasset file to the list of output files
			int64 DataSize = Writer->TotalSize();
			SaveContext.AdditionalPackageFiles.Emplace(SaveContext.GetFilename(),
				FLargeMemoryPtr(Writer->ReleaseOwnership()), Linker->FileRegions, DataSize);

			for (FSavePackageOutputFile& Entry : SaveContext.AdditionalPackageFiles)
			{
				UE::SavePackageUtilities::AsyncWriteFile(WriteOptions, Entry);
			}
			SaveContext.CloseLinkerArchives();
		}
		else if (!SaveContext.IsTextFormat())
		{
			checkf(SaveContext.GetTempFilename().IsSet(),
				TEXT("The package should've been saved to a tmp file first! (%s)"), SaveContext.GetFilename());
			const FString* TempFilename = SaveContext.GetTempFilename().GetPtrOrNull();

			// Destroy archives used for saving, closing file handle.
			if (SaveContext.CloseLinkerArchives() == false)
			{
				UE_LOGF(LogSavePackage, Error, "Error writing temp file '%ls' for '%ls'",
					**TempFilename, SaveContext.GetFilename());
				return ESavePackageResult::Error;
			}

			if (!TryCalculateSavedHashAndWriteItBackToFile(*TempFilename, Linker->Summary,
				Linker->IsByteSwapping(), SaveContext.GetFilename(), SavedHash))
			{
				return ESavePackageResult::Error;
			}

			// Move file to its real destination

			// Add the .uasset file to the list of output files
			// TODO: Fix the 0 size, it isn't used after this point but needs to be cleaned up.
			SaveContext.AdditionalPackageFiles.Emplace(SaveContext.GetFilename(), *TempFilename, 0);

			ESavePackageResult FinalizeResult = UE::SavePackageUtilities::FinalizeTempOutputFiles(
				SaveContext.GetTargetPackagePath(), SaveContext.AdditionalPackageFiles, SaveContext.GetFinalTimestamp());

			SaveContext.SetTempFilename(TOptional<FString>());
			TempFilename = nullptr;

			if (FinalizeResult != ESavePackageResult::Success)
			{
				if (SaveContext.IsGenerateSaveError())
				{
					UE_LOGF(LogSavePackage, Error, "%ls",
						*FString::Printf(TEXT("Error saving '%s'"), SaveContext.GetFilename()));
					SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"),
						*FText::Format(NSLOCTEXT("Core", "SaveWarning", "Error saving '{0}'"),
							FText::FromString(FString(SaveContext.GetFilename()))).ToString());
				}
				else
				{
					UE_LOGF(LogSavePackage, Warning, "Error saving '%ls'", SaveContext.GetFilename());
				}
				return FinalizeResult;
			}
		}
		else
		{
			// Destroy archives used for saving, closing file handle.
			if (SaveContext.CloseLinkerArchives() == false)
			{
				UE_LOGF(LogSavePackage, Error, "Error writing temp file '%ls' for '%ls'",
					SaveContext.GetTempFilename().IsSet() ? *SaveContext.GetTempFilename().GetValue() : TEXT("UNKNOWN"), SaveContext.GetFilename());
				return ESavePackageResult::Error;
			}

			// SaveHash was written when writing the PackageSummary in UpdatePackageHeader.
			SavedHash = SaveContext.PackageSavedHash;

			// When saving in text format we will have two temp files, so we need to manually delete the non-textbased one
			check(SaveContext.GetTempFilename().IsSet());
			check(SaveContext.GetTextFormatTempFilename().IsSet());
			IFileManager::Get().Delete(*SaveContext.GetTempFilename().GetValue());
			SaveContext.SetTempFilename(SaveContext.GetTextFormatTempFilename());
			SaveContext.SetTextFormatTempFilename(TOptional<FString>());

			// Add the .utxt file to the list of output files
			// TODO: Fix the 0 size, it isn't used after this point but needs to be cleaned up.
			SaveContext.AdditionalPackageFiles.Emplace(SaveContext.GetFilename(), SaveContext.GetTempFilename().GetValue(), 0);

			ESavePackageResult FinalizeResult = UE::SavePackageUtilities::FinalizeTempOutputFiles(SaveContext.GetTargetPackagePath(), SaveContext.AdditionalPackageFiles,
				SaveContext.GetFinalTimestamp());

			SaveContext.SetTempFilename(TOptional<FString>());

			if (FinalizeResult != ESavePackageResult::Success)
			{
				if (SaveContext.IsGenerateSaveError())
				{
					UE_LOGF(LogSavePackage, Error, "%ls", *FString::Printf(TEXT("Error saving '%s'"), SaveContext.GetFilename()));
					SaveContext.GetError()->Logf(ELogVerbosity::Warning, TEXT("%s"), *FText::Format(NSLOCTEXT("Core", "SaveWarning", "Error saving '{0}'"), FText::FromString(FString(SaveContext.GetFilename()))).ToString());
				}
				else
				{
					UE_LOGF(LogSavePackage, Warning, "%ls", *FString::Printf(TEXT("Error saving '%s'"), SaveContext.GetFilename()));
				}
				return FinalizeResult;
			}
		}
	}

	if (SaveContext.GetCurrentHarvestingRealm() != ESaveRealm::Optional)
	{
		SaveContext.PackageSavedHash = SavedHash;
	}

	return ESavePackageResult::Success;
}

ESavePackageResult BeginCachePlatformCookedData(FSaveContext& SaveContext)
{
#if WITH_EDITOR
	if (!SaveContext.IsCooking() || SaveContext.IsConcurrent())
	{
		// BeginCacheForCookedPlatformData is not called if not cooking
		// When saving concurrently the cooker has called it ahead of time (because it is not threadsafe)
		return ESavePackageResult::Success;
	}
	ICookedPackageWriter* CookedWriter = SaveContext.GetPackageWriter()->AsCookedPackageWriter();
	check(CookedWriter); // Cooking requires a CookedPackageWriter

	// Find the saveable objects
	TArray<UObject*> ObjectsInPackage;
	UPackage* Package = SaveContext.GetPackage();
	GetObjectsWithPackage(Package, ObjectsInPackage);
	ObjectsInPackage.RemoveAllSwap([&SaveContext](UObject* Object)
		{
			return SaveContext.IsUnsaveable(Object, SaveContext.GetCachedObjectStatus(Object));
		});

	// Call the PackageWriter to dispatch the BeginCache calls
	ICookedPackageWriter::FBeginCacheForCookedPlatformDataInfo Info{
		Package->GetFName(), SaveContext.GetTargetPlatform(), TConstArrayView<UObject*>(ObjectsInPackage),
		SaveContext.GetSaveArgs().SaveFlags
	};
	EPackageWriterResult Result = CookedWriter->BeginCacheForCookedPlatformData(Info);
	switch (Result)
	{
	case EPackageWriterResult::Success: return ESavePackageResult::Success;
	case EPackageWriterResult::Error: return ESavePackageResult::Error;
	case EPackageWriterResult::Timeout: return ESavePackageResult::Timeout;
	default: return ESavePackageResult::Error;
	}
#else
	return ESavePackageResult::Success;
#endif
}

/**
 * Archive used during post save serialization.
 * It ignores serialization of primitive types. It is just used to provide a callback hook for UStruct::Serialize and UObject::Serialize to execute custom code.
 */
class FPostSaveArchive : public UE::SavePackageUtilities::Private::FArchiveSavePackageCollector
{
public:
	FPostSaveArchive(FSaveContext& InContext)
		: FArchiveSavePackageCollector(InContext.GetArchiveSavePackageData(), InContext.IsFilterEditorOnly(),
			InContext.IsSaveUnversionedProperties(), InContext.IsCooking())
	{
		//disable the delta serialization to make sure the objects are serialized no matter what.
		ArNoDelta = true;
		ArNoIntraPropertyDelta = true;
	}
};

void PostSavePackage(FSaveContext& SaveContext)
{
	UPackage* Package = SaveContext.GetPackage();
	// restore initial packages flags since the save can currently mutate the package flags in an undesirable fashion, (i.e. clearing/adding editor only filtering)
	Package->SetPackageFlagsTo(SaveContext.GetInitialPackageFlags());

	// Call PostSave serialization
	{
		FPostSaveArchive PostSaveArchive(SaveContext);

		const TSet<UObject*>& ObjectsToSerialize = SaveContext.GetPostSaveObjectsToSerialize();
		for (UObject* Object : ObjectsToSerialize)
		{
			Object->Serialize(PostSaveArchive);
		}
	}

	// If the save wasn't successful, do not run operations that update the in-memory package to match the on-disk package, and do not broadcast
	// that the package was saved.
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return;
	}

	// Clear dirty flag if desired
	if (!SaveContext.IsKeepDirty())
	{
		SaveContext.GetPackage()->SetDirtyFlag(false);
	}

	// then adjust flags that should be modified at the outcome of a save
	if (!SaveContext.IsFromAutoSave() && !SaveContext.IsProceduralSave())
	{
		// Package has been saved, so unmark the NewlyCreated flag.
		Package->ClearPackageFlags(PKG_NewlyCreated);
	}

	// Copy and modify the output SerializedPackageFlags from the PackageFlags written into the default realm summary
	uint32 SerializedPackageFlags = SaveContext.GetLinker()->Summary.GetPackageFlags();
	// Consider all output packages when reflecting PKG_ContainsNoAsset to the single entry in SerializedPackageFlags and the asset registry
	bool bContainsNoAsset = true;
	for (FLinkerSave* Linker : SaveContext.GetLinkers())
	{
		// GetLinkers shouldn't return null linker
		check(Linker);

		const bool bLinkerContainsNoAsset = Linker->Summary.GetPackageFlags() & PKG_ContainsNoAsset;
		bContainsNoAsset &= bLinkerContainsNoAsset;

		// Call the linker post save callbacks
		FObjectSaveContextData& ObjectSaveContext = SaveContext.GetObjectSaveContext();
		ObjectSaveContext.Object = nullptr;
		Linker->OnPostSave(SaveContext.GetTargetPackagePath(), FObjectPostSaveContext(SaveContext.GetObjectSaveContext()));
	}

	if (bContainsNoAsset)
	{
		SerializedPackageFlags |= PKG_ContainsNoAsset;
	}
	else
	{
		SerializedPackageFlags &= ~PKG_ContainsNoAsset;
	}
	SaveContext.SerializedPackageFlags = SerializedPackageFlags;

	// Notify the soft reference collector about our harvested soft references during save. 
	// This is currently needed only for cooking which does not require editor-only references 
#if WITH_EDITOR
	if (SaveContext.IsCooking())
	{
		GRedirectCollector.CollectSavedSoftPackageReferences(Package->GetFName(), SaveContext.GetSoftPackagesUsedInGame(), false);
	}
#endif

	// update data on the Package about its disk data (path, savedhash, etc) if we're saving to a valid mounted path
	// and we aren't currently cooking
	const FPackagePath& PackagePath = SaveContext.GetTargetPackagePath();
	if (SaveContext.IsUpdatingLoadedPath())
	{
#if WITH_EDITOR
		Package->SetLoadedPath(PackagePath);
#endif
		// Calling a function on SaveContext is required because SetFileSize is private and SaveContext is a friend.
		SaveContext.UpdatePackageFileSize(SaveContext.PackageHeaderAndExportSize);
#if WITH_EDITORONLY_DATA
		Package->SetSavedHash(SaveContext.PackageSavedHash);
#endif
	}

	// Send a message that the package was saved
	FObjectSaveContextData& ObjectSaveContext = SaveContext.GetObjectSaveContext();
	ObjectSaveContext.Object = nullptr;
	UPackage::PackageSavedWithContextEvent.Broadcast(SaveContext.GetFilename(), Package, FObjectPostSaveContext(ObjectSaveContext));
}

AUTORTFM_DISABLE ESavePackageResult SaveHarvestedRealms(FSaveContext& SaveContext, ESaveRealm HarvestingContextToSave)
{
	// Set the current harvested context to save
	FSaveContext::FSetSaveRealmToSaveScope Scope(SaveContext, HarvestingContextToSave);

	// Create slow task dialog if needed
	const int32 TotalSaveSteps = 12;
	FScopedSlowTask SlowTask(TotalSaveSteps, FText(), SaveContext.IsUsingSlowTask());

	// Validate Exports
	SlowTask.EnterProgressFrame();
	SaveContext.Result = ValidateExports(SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		// if we are skipping processing due to an empty realm, consider the save succesful since that only used internally for optional realm
		SaveContext.Result = SaveContext.Result == ESavePackageResult::EmptyRealm ? ESavePackageResult::Success : SaveContext.Result;
		return SaveContext.Result;
	}

	// Validate Imports
	SlowTask.EnterProgressFrame();
	SaveContext.Result = ValidateImports(SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	// Create Linker
	SlowTask.EnterProgressFrame();
	SaveContext.Result = CreateLinker(SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	// Build Linker
	SlowTask.EnterProgressFrame();
	SaveContext.Result = BuildLinker(SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	FStructuredArchive::FRecord StructuredArchiveRoot = SaveContext.GetStructuredArchive()->Open().EnterRecord();

	// Write Header
	SlowTask.EnterProgressFrame();
	SaveContext.Result = !SaveContext.IsTextFormat() ? WritePackageHeader(StructuredArchiveRoot, SaveContext) : WritePackageTextHeader(StructuredArchiveRoot, SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	// Write Exports 
	{
		SlowTask.EnterProgressFrame();
		FLinkerSave& Linker = *SaveContext.GetLinker();

		// mark the linker archive as currently serializing Optional data
		bool bOriginalSavingOptionalObject = Linker.IsSavingOptionalObject();
		Linker.SetSavingOptionalObject(HarvestingContextToSave == ESaveRealm::Optional);

		if (Linker.IsCooking())
		{
			// Write the exports into a separate archive
			const bool bIsOptionalRealm = SaveContext.GetCurrentHarvestingRealm() == ESaveRealm::Optional;
			TUniquePtr<FLargeMemoryWriter> ExportsArchive = SaveContext.GetPackageWriter()->CreateLinkerExportsArchive(
				SaveContext.GetPackage()->GetFName(), SaveContext.GetAsset(), bIsOptionalRealm ? 1 : 0);
			SaveContext.Result = WriteCookedExports(*ExportsArchive, SaveContext);

			if (SaveContext.Result == ESavePackageResult::Success)
			{
				// Write the data resource table to the header section before appending the export(s)
				const int64 DataResourceSize = WriteObjectDataResources(Linker.DataResourceMap, StructuredArchiveRoot, SaveContext);
				check(DataResourceSize >= 0);

				// Check to make sure that the package header is not too large
				SaveContext.Result = SetSummaryTotalHeaderSize(SaveContext, (int64)Linker.Summary.TotalHeaderSize + DataResourceSize);
				if (SaveContext.Result != ESavePackageResult::Success)
				{
					return SaveContext.Result;
				}

				{
					// Disables writing stack trace data when appending the exports data
					FArchiveStackTraceDisabledScope _;
					Linker.Serialize(ExportsArchive->GetData(), ExportsArchive->TotalSize());
				}

				// Adjust the export offsets with the total header size
				for (FObjectExport& Export : Linker.ExportMap)
				{
					if (Export.Object)
					{
						Export.SerialOffset += Linker.Summary.TotalHeaderSize;
					}
				}
				for (FCellExport& CellExport : Linker.CellExportMap)
				{
					if (CellExport.Cell)
					{
						CellExport.SerialOffset += Linker.Summary.TotalHeaderSize;
					}
				}
			}
		}
		else
		{
			SaveContext.Result = WriteExports(StructuredArchiveRoot, SaveContext);
		}

		if (SaveContext.Result != ESavePackageResult::Success)
		{
			return SaveContext.Result;
		}

		Linker.SetSavingOptionalObject(bOriginalSavingOptionalObject);
	}

	const int64 EndOfExportsOffset = SaveContext.GetLinker()->Tell();

	// When not using a PackageWriter, VirtualExportsFileOffset is identical to the offset in the
	// Exports archive: SaveContext.GetLinker()->Tell. When using a PackageWriter however, additional
	// blobs such as bulkdata are not written into the exports archive, they are stored as separate archives
	// in the PackageWriter. But various structs need to know the "offset" in the combined file that would
	// be created by appending all of these blobs after the exports. VirtualExportsFileOffset holds that value.
	int64 VirtualExportsFileOffset = EndOfExportsOffset;

	// Write bulk data
	{
		SlowTask.EnterProgressFrame();

		SaveContext.Result = WriteBulkData(SaveContext, VirtualExportsFileOffset);
		if (SaveContext.Result != ESavePackageResult::Success)
		{
			return SaveContext.Result;
		}
		SaveContext.GetLinker()->OnPostSaveBulkData();
	}

	IPackageWriter* PackageWriter = SaveContext.GetPackageWriter();
	SaveContext.Result = WriteAdditionalFiles(SaveContext, SlowTask, VirtualExportsFileOffset /* In/Out */);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	// Write out a tag at the end of the exports and AdditionalFiles
	if (PackageWriter)
	{
		VirtualExportsFileOffset += PackageWriter->GetExportsFooterSize();
	}
	else
	{
		if (!SaveContext.IsTextFormat())
		{
			uint32 Tag = PACKAGE_FILE_TAG;
			StructuredArchiveRoot.GetUnderlyingArchive() << Tag;
			VirtualExportsFileOffset += sizeof(Tag);
		}
	}

	// Now that the package is written out we can write the package trailer that is appended
	// to the file. This should be the last thing written to the file!
	SlowTask.EnterProgressFrame();
	SaveContext.Result = BuildAndWriteTrailer(PackageWriter, StructuredArchiveRoot, SaveContext, VirtualExportsFileOffset);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	if (PackageWriter)
	{
		checkf(SaveContext.GetLinker()->Tell() == EndOfExportsOffset,
			TEXT("The writing of additional files is not allowed to append to the LinkerSave when using a PackageWriter."));
	}
	else
	{
		ensureMsgf(!GValidateVirtualExportsFileOffset || SaveContext.GetLinker()->Tell() == VirtualExportsFileOffset,
			TEXT("VirtualExportsFileOffset is no longer in sync (%lld vs %lld) when writing out '%s'"),
			VirtualExportsFileOffset, SaveContext.GetLinker()->Tell(), * SaveContext.GetPackage()->GetName());
	}

	// Store the package header and export size of the non optional realm
	if (SaveContext.GetCurrentHarvestingRealm() != ESaveRealm::Optional)
	{
		SaveContext.PackageHeaderAndExportSize = VirtualExportsFileOffset;
	}
	SaveContext.TotalPackageSizeUncompressed += VirtualExportsFileOffset;
	for (const FSavePackageOutputFile& File : SaveContext.AdditionalPackageFiles)
	{
		SaveContext.TotalPackageSizeUncompressed += File.DataSize;
	}

	// Update Package Header
	SlowTask.EnterProgressFrame();
	SaveContext.Result = UpdatePackageHeader(StructuredArchiveRoot, SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	// Finalize File Write
	SlowTask.EnterProgressFrame();
	SaveContext.Result = FinalizeFile(StructuredArchiveRoot, SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	//COOK_STAT(FSavePackageStats::MBWritten += ((double)SaveContext.TotalPackageSizeUncompressed) / 1024.0 / 1024.0);
	{
		SCOPED_SAVETIMER(UPackage_Save_MarkExportLoaded);
		FLinkerSave* Linker = SaveContext.GetLinker();
		// Mark exports and the package as RF_Loaded after they've been serialized
		// This is to ensue that newly created packages are properly marked as loaded (since they now exist on disk and 
		// in memory in the exact same state).

		// Nobody should be touching those objects beside us while we are saving them here as this can potentially be executed from another thread
		for (auto& Export : Linker->ExportMap)
		{
			if (Export.Object)
			{
				Export.Object->SetFlags(RF_WasLoaded | RF_LoadCompleted);
			}
		}
	}
	return SaveContext.Result;
}

/**
 * InnerSave is the portion of Save that can be safely run concurrently
 */
AUTORTFM_DISABLE ESavePackageResult InnerSaveInternal(FSaveContext& SaveContext)
{
	// Create slow task dialog if needed
	const int32 TotalSaveSteps = 3;
	FScopedSlowTask SlowTask(TotalSaveSteps, FText(), SaveContext.IsUsingSlowTask());
	SlowTask.MakeDialogDelayed(3.0f, SaveContext.IsFromAutoSave());

	// Harvest Package
	SlowTask.EnterProgressFrame();
	SaveContext.GetObjectSaveContext().ObjectSaveContextPhase = EObjectSaveContextPhase::Harvest;
	SaveContext.Result = HarvestPackage(SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	SlowTask.EnterProgressFrame();
	SaveContext.Result = ValidateRealms(SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	// @todo: Need to adjust UE::IsSavingPackage() to properly prevent generating reference once, package harvesting is done
	// UE::IsSavingPackage() is too harsh however, since it should be scope only to the current package.
	//
	// Note: ValidateIllegalReferences will temporarily override FScopedSavingFlag's setting of UE::IsSavingPackage()
	// when illegal references are detected.
	FScopedSavingFlag IsSavingFlag(SaveContext.IsConcurrent(), SaveContext.GetPackage());

	// Split the save context into its harvested contexts,
	// This essentially mean that a package can produce multiple package output. 
	// This is different then the multiple file outputs a package can already produce
	// since each harvested context will produce those multiple file outputs i.e:
	// Input package -> Main cooked package -> .uasset
	//										-> .uexp
	//										-> .ubulk
	//										-> etc
	//					Sub cooked package	-> .o.uasset
	//										-> .o.uexp
	//										-> .o.ubulk
	//										-> etc
	SlowTask.EnterProgressFrame();
	SaveContext.GetObjectSaveContext().ObjectSaveContextPhase = EObjectSaveContextPhase::Write;
	for (ESaveRealm HarvestingContext : SaveContext.GetHarvestedRealmsToSave())
	{
		SaveContext.Result = SaveHarvestedRealms(SaveContext, HarvestingContext);
		if (SaveContext.Result != ESavePackageResult::Success)
		{
			return SaveContext.Result;
		}
	}
	// Mark the Package RF_Loaded after its been serialized. 
	// This was already done for each object in the package in SaveHarvestedRealms
	SaveContext.GetPackage()->SetFlags(RF_WasLoaded | RF_LoadCompleted);
#if WITH_EDITORONLY_DATA
	SaveContext.GetPackage()->SetHasBeenEndLoaded(true);
#endif // WITH_EDITORONLY_DATA

	return SaveContext.Result;
}

/** WriteAdditionalPayloads writes BulkData, PayloadSidecarFiles, and other data that is separate from exports.
 * Depending on settings, these additional files may be appended to the LinkerSave after the exports, or they
 * may be written into separate sidecar files.
 *
 * @param SaveContext The context for the overall save, including data about the additional payloads
 * @param LinkerSize If the Linker has finished writing, this is the size of the Linker's archive: Linker->Tell(). Otherwise it is -1.
 */
ESavePackageResult WriteAdditionalFiles(FSaveContext& SaveContext, FScopedSlowTask& SlowTask, int64& VirtualExportsFileOffset)
{
	SlowTask.EnterProgressFrame();

	// Add any pending data blobs to the end of the file by invoking the callbacks
	ESavePackageResult Result = UE::SavePackageUtilities::AppendAdditionalData(*SaveContext.GetLinker(), VirtualExportsFileOffset, SaveContext.GetSavePackageContext());
	if (Result != ESavePackageResult::Success)
	{
		return Result;
	}

	// Create the payload side car file (if needed)
	Result = UE::SavePackageUtilities::CreatePayloadSidecarFile(*SaveContext.GetLinker(), SaveContext.GetTargetPackagePath(), SaveContext.IsSaveToMemory(),
		SaveContext.AdditionalPackageFiles, SaveContext.GetSavePackageContext());
	if (Result != ESavePackageResult::Success)
	{
		return Result;
	}

	// Write Additional files from export
	SlowTask.EnterProgressFrame();
	Result = WriteAdditionalExportFiles(SaveContext);
	if (Result != ESavePackageResult::Success)
	{
		return Result;
	}
	return ESavePackageResult::Success;
}

FText GetSlowTaskStatusMessage(const FSaveContext& SaveContext)
{
	FString CleanFilename = FPaths::GetCleanFilename(SaveContext.GetFilename());
	FFormatNamedArguments Args;
	Args.Add(TEXT("CleanFilename"), FText::FromString(CleanFilename));
	return FText::Format(NSLOCTEXT("Core", "SavingFile", "Saving file: {CleanFilename}..."), Args);
}

void BeginSave(FSaveContext& SaveContext, FScopedSlowTask* SlowTask = nullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPackage::BeginSave);

	if (SlowTask)
	{
		SlowTask->EnterProgressFrame();
	}

	SaveContext.Result = ValidatePackage(SaveContext);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return;
	}

	// Ensures
	if (SlowTask)
	{
		SlowTask->EnterProgressFrame();
	}
	{
		if (!SaveContext.IsConcurrent())
		{
			// We need to make sure to flush any pending request that may involve the existing linker of this package as we want to reset it
			// to release any handle on the file prior to overwriting it.
			UPackage* Package = SaveContext.GetPackage();
			ConditionalFlushAsyncLoadingForSave(Package);
			(*GFlushStreamingFunc)();

			EnsurePackageLocalization(Package, SaveContext);

			// FullyLoad the package's Loader, so that anything we need to serialize (bulkdata, thumbnails) is available
			EnsureLoadingComplete(Package);
		}
	}

	// PreSave Asset
	if (SlowTask)
	{
		SlowTask->EnterProgressFrame();
	}
	PreSavePackage(SaveContext);
	if (SaveContext.GetAsset() && !SaveContext.IsConcurrent())
	{
		FObjectSaveContextData& ObjectSaveContext = SaveContext.GetObjectSaveContext();
		UE::SavePackageUtilities::CallPreSaveRoot(SaveContext.GetAsset(), ObjectSaveContext);
		SaveContext.SetPostSaveRootRequired(true);
		SaveContext.SetPreSaveCleanup(ObjectSaveContext.bCleanupRequired);
	}

	if (SlowTask)
	{
		SlowTask->EnterProgressFrame();
	}
	if (!SaveContext.IsConcurrent())
	{
		// Route Presave only if not calling concurrently or if the PackageWriter claims already completed, in those case they should be handled separately already
		IPackageWriter* PackageWriter = SaveContext.GetPackageWriter();
		if (!PackageWriter || !PackageWriter->IsPreSaveCompleted())
		{
			SaveContext.GetObjectSaveContext().ObjectSaveContextPhase = EObjectSaveContextPhase::PreSave;
			SaveContext.Result = RoutePresave(SaveContext);
			if (SaveContext.Result != ESavePackageResult::Success)
			{
				return;
			}
		}

		// Trigger platform cooked data caching after PreSave but before package harvesting 
		// After PreSave because objects can be created during PreSave and we need to cache them
		// Before package harvesting because it might modify some property and hence affect the harvested
		// property name of a tagged property for example
		SaveContext.Result = BeginCachePlatformCookedData(SaveContext);
		if (SaveContext.Result != ESavePackageResult::Success)
		{
			return;
		}

		// If we're writing to the existing file call ResetLoaders on the Package so that we drop the handle to the file on disk and can write to it
		// This might end flushing async loading for this package
		ResetLoadersForSave(SaveContext.GetPackage(), SaveContext.GetFilename());
	}
}

AUTORTFM_DISABLE void InnerSave(FSaveContext& SaveContext, FScopedSlowTask* SlowTask = nullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPackage::InnerSave);

	SaveContext.Result = InnerSaveInternal(SaveContext);

	// in case of failure or cancellation, do not exit here, still run cleanup (e.g. PostSaveRoot)
}

void EndSave(FSaveContext& SaveContext, FScopedSlowTask* SlowTask = nullptr)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPackage::EndSave);

	// PostSave Package - edit in memory package and send events if save was successful
	if (SlowTask)
	{
		SlowTask->EnterProgressFrame();
	}

	if (SaveContext.GetPostSaveRootRequired() && SaveContext.GetAsset())
	{
		SaveContext.GetObjectSaveContext().ObjectSaveContextPhase = EObjectSaveContextPhase::PostSave;
		UE::SavePackageUtilities::CallPostSaveRoot(SaveContext.GetAsset(), SaveContext.GetObjectSaveContext(), SaveContext.GetPreSaveCleanup());
		SaveContext.SetPostSaveRootRequired(false);
	}

	// PostSave Package - edit in memory package and send events if save was successful
	if (SlowTask)
	{
		SlowTask->EnterProgressFrame();
	}
	PostSavePackage(SaveContext);
}

} // end namespace

#define COOK_SAVEPACKAGE_SCOPE(Package) \
	UE_LOG_CONTEXT("Package", FSoftObjectPath(Package)); \
	COOK_STAT(FScopedDurationTimer FuncSaveTimer(FSavePackageStats::SavePackageTimeSec)); \
	UE_SCOPED_COOK_STAT(Package->GetFName(), EPackageEventStatType::SavePackage); \
	UE_TRACK_REFERENCING_PACKAGE_SCOPED(Package, PackageAccessTrackingOps::NAME_Save);

UE::Tasks::TTask<FSavePackageResultStruct> UPackage::SaveAsync(UPackage* InPackage, UObject* InAsset, const TCHAR* InFilename,
	const FSavePackageArgs& SaveArgs)
{
	AutoRTFM::UnreachableIfClosed("TODO: Mark this function as AUTORTFM_DISABLED and fixup callers");

	COOK_STAT(FSavePackageStats::NumPackagesSaved++);

	auto SaveContext = MakeShared<FSaveContext>(InPackage, InAsset, InFilename, SaveArgs);

	// It's important to know if the request was aborted after BeginSave. We cannot
	// just look for errors in the context because if BeginSave didn't error out,
	// we need to execute both InnerSave and EndSave even if InnerSave fails.
	auto AbortFlag = MakeShared<bool>(false);
	auto BeginTask =
		UE::Tasks::Launch(
			TEXT("BeginSave"),
			[SaveContext, AbortFlag]()
			{
				AutoRTFM::UnreachableIfClosed("This should never get called under autortfm");

				COOK_SAVEPACKAGE_SCOPE(SaveContext->GetPackage());

				BeginSave(*SaveContext);

				*AbortFlag = (SaveContext->Result != ESavePackageResult::Success);
			},
			LowLevelTasks::ETaskPriority::Normal,
			UE::Tasks::EExtendedTaskPriority::GameThreadHiPri
		);

	// Can run on any thread
	auto InnerTask =
		UE::Tasks::Launch(
			TEXT("InnerSave"),
			[SaveContext, AbortFlag]()
			{
				if (!*AbortFlag)
				{
					AutoRTFM::UnreachableIfClosed("This should never get called under autortfm");

					COOK_SAVEPACKAGE_SCOPE(SaveContext->GetPackage());

					InnerSave(*SaveContext);
				}
			},
			BeginTask
		);
	
	auto EndTask =
		UE::Tasks::Launch(
			TEXT("EndSave"),
			[SaveContext, AbortFlag]()
			{
				if (!*AbortFlag)
				{
					AutoRTFM::UnreachableIfClosed("This should never get called under autortfm");

					COOK_SAVEPACKAGE_SCOPE(SaveContext->GetPackage());

					EndSave(*SaveContext);
				}

				return SaveContext->GetFinalResult();
			},
			InnerTask,
			LowLevelTasks::ETaskPriority::Normal,
			UE::Tasks::EExtendedTaskPriority::GameThreadHiPri
		);

	return EndTask;
}

FSavePackageResultStruct UPackage::Save(UPackage* InPackage, UObject* InAsset, const TCHAR* InFilename,
	const FSavePackageArgs& SaveArgs)
{
	AutoRTFM::UnreachableIfClosed("TODO: Mark this function as AUTORTFM_DISABLED and fixup callers");

	TRACE_CPUPROFILER_EVENT_SCOPE(UPackage::Save);

	COOK_SAVEPACKAGE_SCOPE(InPackage);
	COOK_STAT(FSavePackageStats::NumPackagesSaved++);

	FSaveContext SaveContext(InPackage, InAsset, InFilename, SaveArgs);

	// Create the slow task dialog if needed
	const int32 TotalSaveSteps = 8;
	FScopedSlowTask SlowTask(TotalSaveSteps, GetSlowTaskStatusMessage(SaveContext), SaveContext.IsUsingSlowTask());
	SlowTask.MakeDialogDelayed(3.0f, SaveContext.IsFromAutoSave());

	BeginSave(SaveContext, &SlowTask);
	if (SaveContext.Result != ESavePackageResult::Success)
	{
		return SaveContext.Result;
	}

	SlowTask.EnterProgressFrame();
	InnerSave(SaveContext, &SlowTask);
	EndSave(SaveContext, &SlowTask);

	return SaveContext.GetFinalResult();
}

#undef COOK_SAVEPACKAGE_SCOPE

ESavePackageResult UPackage::SaveConcurrent(TArrayView<FPackageSaveInfo> InPackages, const FSavePackageArgs& SaveArgs, TArray<FSavePackageResultStruct>& OutResults)
{
	AutoRTFM::UnreachableIfClosed("TODO: Mark this function as AUTORTFM_DISABLED and fixup callers");

	const int32 TotalSaveSteps = 4;
	FScopedSlowTask SlowTask(TotalSaveSteps, NSLOCTEXT("Core", "SavingFiles", "Saving files..."), SaveArgs.bSlowTask);
	SlowTask.MakeDialogDelayed(3.0f, !!(SaveArgs.SaveFlags & SAVE_FromAutosave));

	// Create all the package save context and run pre save
	SlowTask.EnterProgressFrame();
	TArray<FSaveContext> PackageSaveContexts;
	{
		SCOPED_SAVETIMER(UPackage_SaveConcurrent_PreSave);
		for (FPackageSaveInfo& PackageSaveInfo : InPackages)
		{
			FSaveContext& SaveContext = PackageSaveContexts.Emplace_GetRef(PackageSaveInfo.Package, PackageSaveInfo.Package->FindAssetInPackage(), *PackageSaveInfo.Filename, SaveArgs);

			// Validation
			SaveContext.Result = ValidatePackage(SaveContext);
			if (SaveContext.Result != ESavePackageResult::Success)
			{
				continue;
			}

			// Ensures
			EnsurePackageLocalization(SaveContext.GetPackage(), SaveContext);
			EnsureLoadingComplete(SaveContext.GetPackage()); //@todo: needed??

			// PreSave Asset
			PreSavePackage(SaveContext);
			if (SaveContext.GetAsset())
			{
				SCOPED_SAVETIMER(UPackage_SaveConcurrent_PreSaveRoot);
				FObjectSaveContextData& ObjectSaveContext = SaveContext.GetObjectSaveContext();
				UE::SavePackageUtilities::CallPreSaveRoot(SaveContext.GetAsset(), ObjectSaveContext);
				SaveContext.SetPreSaveCleanup(ObjectSaveContext.bCleanupRequired);
			}

			// Route Presave
			SaveContext.Result = RoutePresave(SaveContext);
			if (SaveContext.Result != ESavePackageResult::Success)
			{
				continue;
			}
		}
	}

	SlowTask.EnterProgressFrame();
	{
		// Flush async loading and reset loaders
		SCOPED_SAVETIMER(UPackage_SaveConcurrent_ResetLoadersForSave);
		ResetLoadersForSave(InPackages);
	}

	SlowTask.EnterProgressFrame();
	{
		SCOPED_SAVETIMER(UPackage_SaveConcurrent);
		// @todo: Once UE::IsSavingPackage() is reworked we should reinstore the saving flag here for the gc lock
		// Passing in false here so that UE::IsSavingPackage() is set to true on top of locking the GC
		//FScopedSavingFlag IsSavingFlag(false);

		// Concurrent Part
		ParallelFor(PackageSaveContexts.Num(), [&PackageSaveContexts] AUTORTFM_DISABLE (int32 PackageIdx)
			{
				InnerSaveInternal(PackageSaveContexts[PackageIdx]);
			});
	}

	// Run Post Concurrent Save
	SlowTask.EnterProgressFrame();
	{
		SCOPED_SAVETIMER(UPackage_SaveConcurrent_PostSave);
		for (FSaveContext& SaveContext : PackageSaveContexts)
		{
			// PostSave Asset
			if (SaveContext.GetAsset())
			{
				UE::SavePackageUtilities::CallPostSaveRoot(SaveContext.GetAsset(), SaveContext.GetObjectSaveContext(), SaveContext.GetPreSaveCleanup());
				SaveContext.SetPreSaveCleanup(false);
			}

			// PostSave Package - edit in memory package and send events
			PostSavePackage(SaveContext);
			OutResults.Add(SaveContext.GetFinalResult());
		}
	}

	return ESavePackageResult::Success;
}

#endif // UE_WITH_SAVEPACKAGE
