// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScriptCore.cpp: Kismet VM execution and support code.
=============================================================================*/

#include "Containers/StaticArray.h"
#include "CoreMinimal.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMisc.h"
#include "Misc/CommandLine.h"
#include "Logging/LogScopedCategoryAndVerbosityOverride.h"
#include "Stats/Stats.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Modules/ModuleManager.h"
#include "UObject/ScriptInterface.h"
#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrProperty.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Object.h"
#include "UObject/CoreNative.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "Templates/Casts.h"
#include "Serialization/NullArchive.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "UObject/Stack.h"
#include "UObject/Reload.h"
#include "Blueprint/BlueprintSupport.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "UObject/ScriptMacros.h"
#include "UObject/ScriptTimeLimiter.h"
#include "UObject/UObjectThreadContext.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "AutoRTFM.h"

#if WITH_VERSE_VM
#include "VerseVM/VVMContext.h"
#include "VerseVM/VVMFrame.h"
#include "VerseVM/Inline/VVMContextImplInline.h"
#endif

DEFINE_LOG_CATEGORY(LogScriptFrame);
DEFINE_LOG_CATEGORY_STATIC(LogScriptCore, Log, All);

DECLARE_CYCLE_STAT(TEXT("Blueprint Time"), STAT_BlueprintTime, STATGROUP_Game);

#define LOCTEXT_NAMESPACE "ScriptCore"

#if TOTAL_OVERHEAD_SCRIPT_STATS
DEFINE_STAT(STAT_ScriptVmTime_Total);
DEFINE_STAT(STAT_ScriptNativeTime_Total);
#endif //TOTAL_OVERHEAD_SCRIPT_STATS

static int32 GVerboseScriptStats = 0;
static FAutoConsoleVariableRef CVarVerboseScriptStats(
	TEXT("bp.VerboseStats"),
	GVerboseScriptStats,
	TEXT("Create additional stats for Blueprint execution.\n"),
	ECVF_Default
);

static int32 GShortScriptWarnings = 0;
static FAutoConsoleVariableRef CVarShortScriptWarnings(
	TEXT("bp.ShortScriptWarnings"),
	GShortScriptWarnings,
	TEXT("Shorten the blueprint exception logs.\n"),
	ECVF_Default
);

static int32 GScriptRecurseLimit = 120;
static FAutoConsoleVariableRef CVarScriptRecurseLimit(
	TEXT("bp.ScriptRecurseLimit"),
	GScriptRecurseLimit,
	TEXT("Sets the number of recursions before script is considered in an infinite loop.\n"),
	ECVF_Default
);

#if PER_FUNCTION_SCRIPT_STATS
static int32 GMaxFunctionStatDepth = MAX_uint8;
static FAutoConsoleVariableRef CVarMaxFunctionStatDepth(
	TEXT("bp.MaxFunctionStatDepth"),
	GMaxFunctionStatDepth,
	TEXT("Script stack threshold for recording per function stats.\n"),
	ECVF_Default
);
#endif

/*-----------------------------------------------------------------------------
	Globals.
-----------------------------------------------------------------------------*/

//
// Native function table.
//
COREUOBJECT_API constinit TStaticArray<FNativeFuncPtr, EX_Max> GNatives(InPlace, &UObject::execUndefined);
COREUOBJECT_API int32 GNativeDuplicate=0;

COREUOBJECT_API int32 GMaximumScriptLoopIterations = 1000000;

thread_local static FFrame* GTopTrackingStackFrame = nullptr;

#if DO_BLUEPRINT_GUARD

static FORCEINLINE void CheckRunaway()
{
	FBlueprintContextTracker& ContextTracker = FBlueprintContextTracker::Get();
	int32 RunawayCount = ContextTracker.AddRunaway(); 

	// We need to periodically check that we're still within the script time limit.
	// For every 1 out of 256 runaway counter increments, we do a timeout check here.
	if (UNLIKELY((RunawayCount & 0xFF) == 0))
	{
		ContextTracker.EnforceScriptTimeLimit();
	}
}

COREUOBJECT_API void GInitRunaway() 
{
	FBlueprintContextTracker::Get().ResetRunaway();
}

#else

static FORCEINLINE void CheckRunaway() {}
COREUOBJECT_API void GInitRunaway() {}

#endif // DO_BLUEPRINT_GUARD

#define STORE_INSTRUCTION_NAMES SCRIPT_AUDIT_ROUTINES

#if STORE_INSTRUCTION_NAMES
const char* GNativeFuncNames[EX_Max];

#define STORE_INSTRUCTION_NAME(inst) \
static struct F##inst##Registrar \
{ \
	F##inst##Registrar() \
	{ \
		GNativeFuncNames[inst] = #inst; \
	} \
} inst##RegistrarInst;
#else
#define STORE_INSTRUCTION_NAME(inst)
#endif//STORE_INSTRUCTION_NAMES

#define IMPLEMENT_FUNCTION(func) \
	static FNativeFunctionRegistrar UObject##func##Registar(UObject::StaticClass(),#func,&UObject::func);

#define IMPLEMENT_CAST_FUNCTION(CastIndex, func) \
	IMPLEMENT_FUNCTION(func); \
	static uint8 UObject##func##CastTemp = GRegisterCast( CastIndex, &UObject::func );

#define IMPLEMENT_VM_FUNCTION(BytecodeIndex, func) \
	STORE_INSTRUCTION_NAME(BytecodeIndex) \
	IMPLEMENT_FUNCTION(func) \
	static uint8 UObject##func##BytecodeTemp = GRegisterNative( BytecodeIndex, &UObject::func );

//////////////////////////////////////////////////////////////////////////
// FBlueprintCoreDelegates

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FBlueprintCoreDelegates::FOnScriptDebuggingEvent FBlueprintCoreDelegates::OnScriptException;
FBlueprintCoreDelegates::FOnScriptInstrumentEvent FBlueprintCoreDelegates::OnScriptProfilingEvent;
FBlueprintCoreDelegates::FOnToggleScriptProfiler FBlueprintCoreDelegates::OnToggleScriptProfiler;

PRAGMA_ENABLE_DEPRECATION_WARNINGS

static int32 BlueprintContextVirtualStackAllocatorSize = 8 * 1024 * 1024;
FAutoConsoleVariableRef CVarBlueprintContextVirtualStackAllocatorStackSize(
	TEXT("r.FBlueprintContext.VirtualStackAllocatorStackSize"),
	BlueprintContextVirtualStackAllocatorSize,
	TEXT("Default size for FBlueprintContext's FVirtualStackAllocator"),
	ECVF_ReadOnly
);

static int BlueprintContextVirtualStackAllocatorDecommitMode = (int)EVirtualStackAllocatorDecommitMode::AllOnDestruction;
FAutoConsoleVariableRef CVarBlueprintContextVirtualStackAllocatorDecommitMode(
	TEXT("r.FBlueprintContext.VirtualStackAllocator.DecommitMode"),
	BlueprintContextVirtualStackAllocatorDecommitMode,
	TEXT("Specifies DecommitMode for FVirtualStackAllocator when used through its ThreadSingleton. Values are from EVirtualStackAllocatorDecommitMode."),
	ECVF_ReadOnly
);

FBlueprintContext::FBlueprintContext()
#if UE_USE_VIRTUAL_STACK_ALLOCATOR_FOR_SCRIPT_VM
	: VirtualStackAllocator(
		BlueprintContextVirtualStackAllocatorSize,
		static_cast<EVirtualStackAllocatorDecommitMode>(BlueprintContextVirtualStackAllocatorDecommitMode))
#else
	: VirtualStackAllocator(0, EVirtualStackAllocatorDecommitMode::AllOnDestruction)
#endif
{
	ensure(BlueprintContextVirtualStackAllocatorDecommitMode >= 0 && BlueprintContextVirtualStackAllocatorDecommitMode < (int)EVirtualStackAllocatorDecommitMode::NumModes);
}

// pulled the thread_local into a separate function to workaround
// a compile error complaining about having it live local to the
// lambda below
FBlueprintContext* FBlueprintContextGetThreadSingletonImpl()
{
	static thread_local FBlueprintContext ThreadLocalContext;
	return &ThreadLocalContext;
}

FBlueprintContext* FBlueprintContext::GetThreadSingleton()
{
	FBlueprintContext* Result;

	// Running this open without validation because a) it is hit often and b) it is just
	// initializing a thread-local variable which has no hazard.
	UE_AUTORTFM_OPEN_NO_VALIDATION { Result = FBlueprintContextGetThreadSingletonImpl(); };
	return Result;
}

void FBlueprintCoreDelegates::ThrowScriptException(const UObject* ActiveObject, FFrame& StackFrame, const FBlueprintExceptionInfo& Info)
{
	bool bShouldLogWarning = true;

	switch (Info.GetType())
	{
	case EBlueprintExceptionType::Breakpoint:
	case EBlueprintExceptionType::Tracepoint:
	case EBlueprintExceptionType::WireTracepoint:
		// These shouldn't warn (they're just to pass the exception into the editor via the delegate below)
		bShouldLogWarning = false;
		break;
#if WITH_EDITOR && DO_BLUEPRINT_GUARD
	case EBlueprintExceptionType::AccessViolation:
		bShouldLogWarning = FBlueprintContextTracker::Get().RecordAccessViolation(ActiveObject);
		break;
#endif // WITH_EDITOR && DO_BLUEPRINT_GUARD
	default:
		// Other unhandled cases should always emit a warning
		break;
	}

	if (bShouldLogWarning)
	{
		UE_SUPPRESS(LogScript, Warning, const_cast<FFrame*>(&StackFrame)->Logf(TEXT("%s"), *(Info.GetDescription().ToString())));
	}

	// cant fire arbitrary delegates here off the game thread
	if (IsInGameThread())
	{
#if DO_BLUEPRINT_GUARD
		// If nothing is bound, show warnings so something is left in the log.
		if (bShouldLogWarning && (OnScriptException.IsBound() == false) && !GShortScriptWarnings)
		{
			UE_LOGF(LogScript, Warning, "%ls", *StackFrame.GetStackTrace());
		}
#endif
		OnScriptException.Broadcast(ActiveObject, StackFrame, Info);
	}

	if (Info.GetType() == EBlueprintExceptionType::AbortExecution)
	{
		// abort errors halt further execution
		StackFrame.bAbortingExecution = true;
	}

	if (Info.GetType() == EBlueprintExceptionType::FatalError)
	{
		// Crash maybe?
	}
}

void FBlueprintCoreDelegates::InstrumentScriptEvent(const FScriptInstrumentationSignal& Info)
{
	OnScriptProfilingEvent.Broadcast(Info);
}

void FBlueprintCoreDelegates::SetScriptMaximumLoopIterations( const int32 MaximumLoopIterations )
{
	if (ensure(MaximumLoopIterations > 0))
	{
		GMaximumScriptLoopIterations = MaximumLoopIterations;
	}
}

bool FBlueprintCoreDelegates::IsDebuggingEnabled()
{
#if WITH_EDITORONLY_DATA
	return GIsEditor;
#else
	return FBlueprintCoreDelegates::OnScriptException.IsBound();
#endif
}

#if DO_BLUEPRINT_GUARD

FBlueprintContextTracker::FOnEnterScriptContext FBlueprintContextTracker::OnEnterScriptContext;
FBlueprintContextTracker::FOnExitScriptContext FBlueprintContextTracker::OnExitScriptContext;

FBlueprintContextTracker& FBlueprintContextTracker::Get()
{
	return TThreadSingleton<FBlueprintContextTracker>::Get();
}

const FBlueprintContextTracker* FBlueprintContextTracker::TryGet()
{
	return TThreadSingleton<FBlueprintContextTracker>::TryGet();
}

void FBlueprintContextTracker::ResetRunaway()
{
	Runaway = 0;
	Recurse = 0;
	bRanaway = false;
	bScriptTimedOut = false;
}

void FBlueprintContextTracker::EnterScriptContext(const UObject* ContextObject, const UFunction* ContextFunction)
{
	ScriptEntryTag++;

	if (IsInGameThread())
	{
		// Multicast delegate broadcast is not safe, this will be refactored later to completely disable in other threads
		OnEnterScriptContext.Broadcast(*this, ContextObject, ContextFunction);
	}
}

void FBlueprintContextTracker::ExitScriptContext()
{
	if (IsInGameThread())
	{
		OnExitScriptContext.Broadcast(*this);
	}

	ScriptEntryTag--;

	check(ScriptEntryTag >= 0);
}

void FBlueprintContextTracker::EnforceScriptTimeLimit()
{
	if (GMaximumScriptLoopIterations < INT32_MAX && UE::FScriptTimeLimiter::Get().HasExceededTimeLimit())
	{
		// Force the existing runaway checks to trigger.
		Runaway = GMaximumScriptLoopIterations + 1;
		bRanaway = true;
		bScriptTimedOut = true;
	}
}

bool FBlueprintContextTracker::RecordAccessViolation(const UObject* Object)
{
	// Determine if the access none should warn or not (we suppress warnings beyond a certain count for each object to avoid per-frame spaminess)
	struct FIntConfigValueHelper
	{
		int32 Value;

		FIntConfigValueHelper() : Value(0)
		{
			GConfig->GetInt(TEXT("ScriptErrorLog"), TEXT("MaxNumOfAccessViolation"), Value, GEditorIni);
		}
	};

	static const FIntConfigValueHelper MaxNumOfAccessViolation;
	if (MaxNumOfAccessViolation.Value > 0)
	{
		const FName ActiveObjectName = Object ? Object->GetFName() : FName();
		int32& Num = DisplayedWarningsMap.FindOrAdd(ActiveObjectName);
		Num++;
		if (Num > MaxNumOfAccessViolation.Value)
		{
			// Skip the generic warning, we've hit this one too many times
			return false;
		}
	}
	return true;
}

// This is meant to be called from the immediate mode, and for confusing reasons the optimized code isn't always safe in that case
UE_DISABLE_OPTIMIZATION_SHIP

void PrintScriptCallStackImpl()
{
	TStringBuilder<4096> ScriptStack;

#if WITH_VERSE_VM
	FString VerseVMStack = AutoRTFM::Open([] AUTORTFM_DISABLE
	{
		TStringBuilder<4096> VerseVMBuilder;

		if (const Verse::FNativeFrame* NativeFrame = Verse::FContext::NativeFrame())
		{
			VerseVMBuilder << TEXT("\n\nVerse VM Call Stack:\n");

			// We walk from the innermost native-frame boundary, not the interpreter's live PC/Frame/Task.
			// Those live on the interpreter's stack frame (FInterpreter::State) and aren't stored anywhere
			// the crash path can read. If a crash originates mid-bytecode in the interpreter loop, one or more
			// leaf VFrames will be missing from the printed stack.
			NativeFrame->WalkCallstackFrames(
				/*PC*/ nullptr,
				/*Frame*/ nullptr,
				/*Task*/ nullptr,
				[&VerseVMBuilder] AUTORTFM_DISABLE (const Verse::FOp* PC, const Verse::VFrame* Frame)
				{
					Frame->GetStackDescription(VerseVMBuilder, PC);
				},
				[&VerseVMBuilder] AUTORTFM_DISABLE (const Verse::FNativeFrame* Frame)
				{
					Frame->GetStackDescription(VerseVMBuilder);
				});
		}

		return FString(VerseVMBuilder);
	});

	ScriptStack << MoveTemp(VerseVMStack);
#endif

	if (const FBlueprintContextTracker* BlueprintExceptionTracker = FBlueprintContextTracker::TryGet())
	{
		TArrayView<const FFrame* const> RawStack = BlueprintExceptionTracker->GetCurrentScriptStack();
		ScriptStack << TEXT("\n\nScript Stack (") << RawStack.Num() << TEXT(" frames) :\n");

		for (int32 FrameIdx = RawStack.Num() - 1; FrameIdx >= 0; --FrameIdx)
		{
			RawStack[FrameIdx]->GetStackDescription(ScriptStack);
			ScriptStack << TEXT("\n");
		}
	}

	if (ScriptStack.Len() > 0)
	{
		UE_LOGF(LogOutputDevice, Warning, "%ls", *ScriptStack);
	}
}

UE_ENABLE_OPTIMIZATION_SHIP

extern CORE_API void (*GPrintScriptCallStackFn)();

#endif // DO_BLUEPRINT_GUARD

//////////////////////////////////////////////////////////////////////////
// FEditorScriptExecutionGuard
FEditorScriptExecutionGuard::FEditorScriptExecutionGuard()
	: bOldGAllowScriptExecutionInEditor(GAllowActorScriptExecutionInEditor)
{
	check(IsInGameThread());

	GAllowActorScriptExecutionInEditor = true;

	if( GIsEditor && !FApp::IsGame() )
	{
		GInitRunaway();
	}
}

FEditorScriptExecutionGuard::~FEditorScriptExecutionGuard()
{
	GAllowActorScriptExecutionInEditor = bOldGAllowScriptExecutionInEditor;
}

bool IsValidCPPIdentifierChar(TCHAR Char)
{
	return Char == TCHAR('_')
		|| (Char >= TCHAR('a') && Char <= TCHAR('z'))
		|| (Char >= TCHAR('A') && Char <= TCHAR('Z'))
		|| (Char >= TCHAR('0') && Char <= TCHAR('9'));
}

FString ToValidCPPIdentifierChars(TCHAR Char)
{
	FString Ret;
	int32 RawValue = Char;
	int32 Counter = 0;
	while (RawValue != 0)
	{
		int32 Digit = RawValue % 63;
		RawValue = (RawValue - Digit) / 63;

		TCHAR SafeChar;
		if (Digit <= 25)
		{
			SafeChar = TCHAR(TCHAR('a') + (25 - Digit));
		}
		else if (Digit <= 51)
		{
			SafeChar = TCHAR(TCHAR('A') + (51 - Digit));
		}
		else if (Digit <= 61)
		{
			SafeChar = TCHAR(TCHAR('0') + (61 - Digit));
		}
		else
		{
			check(Digit == 62);
			SafeChar = TCHAR('_');
		}

		Ret.AppendChar(SafeChar);
	}
	return Ret;
}

FString UnicodeToCPPIdentifier(const FString& InName, bool bDeprecated, const TCHAR* Prefix)
{
	// FName's can contain unicode characters or collide with other CPP identifiers or keywords. This function 
	// returns a string that will have a prefix which is unlikely to collide with existing identifiers and
	// converts unicode characters in place to valid ascii characters. Strictly speaking a C++ compiler *could*
	// support unicode identifiers in source files, but I am not comfortable relying on this behavior.


	FString Ret = InName;
	// Initialize postfix with a unique identifier. This prevents potential collisions between names that have unicode
	// characters and those that do not. The drawback is that it is not safe to put '__pf' in a blueprint name.
	FString Postfix = "__pf";
	for (auto& Char : Ret)
	{
		// if the character is not a valid character for a c++ identifier, then we need to encode it using valid characters:
		if (!IsValidCPPIdentifierChar(Char))
		{
			// deterministically map char to a valid ascii character, we have 63 characters available (aA-zZ, 0-9, and _)
			// so the optimal encoding would be base 63:
			Postfix.Append(ToValidCPPIdentifierChars(Char));
			Char = TCHAR('x');
		}
	}

	FString PrefixStr(Prefix);
	//fix for error C2059: syntax error : 'bad suffix on number'
	if (!PrefixStr.Len() && Ret.Len() && FChar::IsDigit(Ret[0]))
	{
		Ret.InsertAt(0, TCHAR('_'));
	}
	Ret = PrefixStr + Ret + Postfix;

	// Workaround for a strange compiler error
	if (InName == TEXT("Replicate to server"))
	{
		Ret = TEXT("MagicNameWorkaround");
	}

	return bDeprecated ? Ret + TEXT("_DEPRECATED") : Ret;
}

/*-----------------------------------------------------------------------------
	FFrame implementation.
-----------------------------------------------------------------------------*/

void FFrame::Step(UObject* Context, RESULT_DECL)
{
	int32 B = *Code++;
	(GNatives[B])(Context,*this,RESULT_PARAM);
}

void FFrame::StepExplicitProperty(void*const Result, FProperty* Property)
{
	checkSlow(Result != NULL);

	if (Property->PropertyFlags & CPF_OutParm)
	{
		// look through the out parameter infos and find the one that has the address of this property
		FOutParmRec* Out = OutParms;
		checkSlow(Out);
		while (Out->Property != Property)
		{
			Out = Out->NextOutParm;
			checkSlow(Out);
		}
		MostRecentPropertyAddress = Out->PropAddr;
		MostRecentPropertyContainer = nullptr;
		// no need to copy property value, since the caller is just looking for MostRecentPropertyAddress
	}
	else
	{
		MostRecentPropertyAddress = Property->ContainerPtrToValuePtr<uint8>(Locals);
		MostRecentPropertyContainer = Locals;
		Property->CopyCompleteValueToScriptVM_InContainer(Result, MostRecentPropertyContainer);
	}
}

//
// Helper function that checks commandline and Engine ini to see whether
// script stack should be shown on warnings
static bool ShowKismetScriptStackOnWarnings()
{
	static bool ShowScriptStackForScriptWarning = false;
	static bool CheckScriptWarningOptions = false;

	if (!CheckScriptWarningOptions)
	{
		GConfig->GetBool(TEXT("Kismet"), TEXT("ScriptStackOnWarnings"), ShowScriptStackForScriptWarning, GEngineIni);

		if (FParse::Param(FCommandLine::Get(), TEXT("SCRIPTSTACKONWARNINGS")))
		{
			ShowScriptStackForScriptWarning = true;
		}

		CheckScriptWarningOptions = true;
	}

	return ShowScriptStackForScriptWarning;
}

FString FFrame::GetScriptCallstack(bool bReturnEmpty, bool bTopOfStackOnly)
{
	TStringBuilder<4096> ScriptStack;
	GetScriptCallstack(ScriptStack, bReturnEmpty, bTopOfStackOnly);
	return FString(ScriptStack);
}

void FFrame::GetScriptCallstack(FStringBuilderBase& ScriptStack, bool bReturnEmpty, bool bTopOfStackOnly)
{
#if DO_BLUEPRINT_GUARD
	FBlueprintContextTracker& BlueprintExceptionTracker = FBlueprintContextTracker::Get();
	if (BlueprintExceptionTracker.ScriptStack.Num() > 0)
	{
		const bool bDisplayArrow = (BlueprintExceptionTracker.ScriptStack.Num() > 1) && !bTopOfStackOnly;
		const int32 TopOfStackIndex = BlueprintExceptionTracker.ScriptStack.Num() - 1;
		int32 i = TopOfStackIndex;

		do
		{
			ScriptStack << TEXT("\t");
			BlueprintExceptionTracker.ScriptStack[i]->GetStackDescription(ScriptStack);
			if ((i == TopOfStackIndex) && bDisplayArrow)
			{
				ScriptStack << TEXT(" <---");
			}
			ScriptStack << TEXT("\n");
			--i;
		} while ((i >= 0) && !bTopOfStackOnly);
	}
	else if (!bReturnEmpty)
	{
		ScriptStack << TEXT("\t[Empty] (FFrame::GetScriptCallstack() called from native code)");
	}
#else
	if (!bReturnEmpty)
	{
		ScriptStack << TEXT("Unable to display Script Callstack. Compile with DO_BLUEPRINT_GUARD=1");
	}
#endif
}

void FFrame::GetStackDescription(FStringBuilderBase& StringBuilder) const
{
	Node->GetOuter()->GetPathName(nullptr, StringBuilder);
	StringBuilder << TEXT(".") << Node->GetName();
}

#if DO_BLUEPRINT_GUARD
void FFrame::InitPrintScriptCallstack()
{
	GPrintScriptCallStackFn = &PrintScriptCallStackImpl;
}
#endif

COREUOBJECT_API FFrame* FFrame::PushThreadLocalTopStackFrame(FFrame* NewTopStackFrame)
{
	FFrame* Result = GTopTrackingStackFrame;
	GTopTrackingStackFrame = NewTopStackFrame;
	return Result;
}

COREUOBJECT_API void FFrame::PopThreadLocalTopStackFrame(FFrame* NewTopStackFrame)
{
	GTopTrackingStackFrame = NewTopStackFrame;
}

COREUOBJECT_API FFrame* FFrame::GetThreadLocalTopStackFrame()
{
	return GTopTrackingStackFrame;
}
//
// Error or warning handler.
//
//@TODO: This function should take more information in, or be able to gather it from the callstack!
void FFrame::KismetExecutionMessage(const TCHAR* Message, ELogVerbosity::Type Verbosity, FName WarningId)
{
#if !UE_BUILD_SHIPPING
	// Optionally always treat errors/warnings as bad
	if (Verbosity <= ELogVerbosity::Warning && FParse::Param(FCommandLine::Get(), TEXT("FATALSCRIPTWARNINGS")))
	{
		Verbosity = ELogVerbosity::Fatal;
	}
	else if(Verbosity == ELogVerbosity::Warning && WarningId != FName())
	{
		// check to see if this specific warning has been elevated to an error:
		if( FBlueprintSupport::ShouldTreatWarningAsError(WarningId) )
		{
			Verbosity = ELogVerbosity::Error;
		}
		else if(FBlueprintSupport::ShouldSuppressWarning(WarningId))
		{
			return;
		}
	}
#endif

	TStringBuilder<4096> ScriptStack;

	// Tracking down some places that display warnings but no message..
	ensureAlways(Verbosity > ELogVerbosity::Warning || FCString::Strlen(Message) > 0);

#if !UE_BUILD_SHIPPING
	// Show the stack for fatal/error, and on warning if that option is enabled
	auto PopulateStackString = [](TStringBuilder<4096>& ScriptStack, ELogVerbosity::Type Verbosity)
	{
#if DO_BLUEPRINT_GUARD
		if (Verbosity <= ELogVerbosity::Error || ShowKismetScriptStackOnWarnings())
	{
		ScriptStack = TEXT("Script call stack:\n");
		GetScriptCallstack(ScriptStack);
			return;
	}
#endif

		if (const FFrame* CurrentFrame = GetThreadLocalTopStackFrame())
		{
			ScriptStack = TEXT("Script Msg called by: ");
			ScriptStack << CurrentFrame->Object->GetFullName();
			return;
		}
	};
	if (Verbosity <= ELogVerbosity::Warning)
	{
		PopulateStackString(ScriptStack, Verbosity);
	}
#endif

	if (Verbosity == ELogVerbosity::Fatal)
	{
		UE_LOGF(LogScriptCore, Fatal, "Script Msg: %ls\n%ls", Message, *ScriptStack);
	}
#if NO_LOGGING
	else
#else
	else if (!LogScriptCore.IsSuppressed(Verbosity))
#endif
	{
		FScriptExceptionHandler::Get().HandleException(Verbosity, Message, *ScriptStack);
	}
}

void FFrame::Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category )
{
	// Treat errors/warnings as bad
	if (Verbosity == ELogVerbosity::Warning)
	{
#if !UE_BUILD_SHIPPING
		static bool GTreatScriptWarningsFatal = FParse::Param(FCommandLine::Get(),TEXT("FATALSCRIPTWARNINGS"));
		if (GTreatScriptWarningsFatal)
		{
			Verbosity = ELogVerbosity::Error;
		}
#endif
	}
	if( Verbosity==ELogVerbosity::Error )
	{
		UE_LOG(LogScriptCore, Fatal,
			TEXT("%s\r\n\t%s\r\n\t%s:%04" SIZE_T_X_FMT "\r\n\t%s"),
			V,
			*Object->GetFullName(),
			*Node->GetFullName(),
			Code - Node->Script.GetData(),
			*GetStackTrace()
		);
	}
	else
	{
#if DO_BLUEPRINT_GUARD
		if (GShortScriptWarnings)
		{
			UE_LOG(LogScript, Warning,
				TEXT("%s Object(%s)  %s:%04" SIZE_T_X_FMT),
				V,
				*Object->GetName(),
				*Node->GetName(),
				Code - Node->Script.GetData()
			);
		}
		else
		{
			UE_LOG(LogScript, Warning,
				TEXT("%s\r\n\t%s\r\n\t%s:%04" SIZE_T_X_FMT "%s"),
				V,
				*Object->GetFullName(),
				*Node->GetFullName(),
				Code - Node->Script.GetData(),
				ShowKismetScriptStackOnWarnings() ? *(FString("\r\n") + GetStackTrace()) : TEXT("")
			);
		}
#endif
	}
}
FString FFrame::GetStackTrace() const
{
	TStringBuilder<4096> Result;
	GetStackTrace(Result);
	return FString(Result);
}

void FFrame::GetStackTrace(FStringBuilderBase& Result) const
{
	// travel down the stack recording the frames
	TArray<const FFrame*> FrameStack;
	const FFrame* CurrFrame = this;
	while (CurrFrame != NULL)
	{
		FrameStack.Add(CurrFrame);
		CurrFrame = CurrFrame->PreviousFrame;
	}
	
	// and then dump them to a string
	if (FrameStack.Num() > 0)
	{
		Result << TEXT("Script call stack:\n");
		for (const FFrame* Frame : ReverseIterate(FrameStack))
		{
			Result << TEXT("\t");
			Frame->Node->GetFullName(Result);
			Result << TEXT("\n");
		}
	}
	else
	{
		Result << TEXT("Script call stack: [Empty] (FFrame::GetStackTrace() called from native code)");
	}
}

//////////////////////////////////////////////////////////////////////////
// FScriptInstrumentationSignal

FScriptInstrumentationSignal::FScriptInstrumentationSignal(EScriptInstrumentation::Type InEventType, const UObject* InContextObject, const struct FFrame& InStackFrame, const FName EventNameIn)
	: EventType(InEventType)
	, ContextObject(InContextObject)
	, Function(InStackFrame.Node)
	, EventName(EventNameIn)
	, StackFramePtr(&InStackFrame)
	, LatentLinkId(INDEX_NONE)
{
}

const UClass* FScriptInstrumentationSignal::GetClass() const
{
	return ContextObject ? ContextObject->GetClass() : nullptr;
}

const UClass* FScriptInstrumentationSignal::GetFunctionClassScope() const
{
	return Function->GetOuterUClass();
}

FName FScriptInstrumentationSignal::GetFunctionName() const
{
	return EventName.IsNone() ? Function->GetFName() : EventName;
}

int32 FScriptInstrumentationSignal::GetScriptCodeOffset() const
{
	int32 CodeOffset = INDEX_NONE;
	if (EventType == EScriptInstrumentation::ResumeEvent)
	{
		// Resume events require the link id rather than script code offset
		CodeOffset = LatentLinkId;
	}
	else if (StackFramePtr != nullptr)
	{
		// StackFramePtr->Code should never be outside the TArray StackFramePtr->Node->Script
		// and so the resulting pointer calculation should always fit in a int32 (the max size 
		// TArray stores) so it is safe to just cast.
		CodeOffset = static_cast<int32>(StackFramePtr->Code - StackFramePtr->Node->Script.GetData() - 1);
	}
	return CodeOffset;
}

/*-----------------------------------------------------------------------------
	Global script execution functions.
-----------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------
	Native registry.
-----------------------------------------------------------------------------*/

//
// Register a native function.
// Warning: Called at startup time, before engine initialization.
//
COREUOBJECT_API uint8 GRegisterNative(int32 NativeBytecodeIndex, const FNativeFuncPtr& Func)
{
	if( NativeBytecodeIndex != INDEX_NONE )
	{
		if( NativeBytecodeIndex<0 || NativeBytecodeIndex > GNatives.Num() || GNatives[NativeBytecodeIndex] != &UObject::execUndefined) 
		{
			CA_SUPPRESS(6385)
			if (!ReloadNotifyFunctionRemap(Func, GNatives[NativeBytecodeIndex]))
			{
				GNativeDuplicate = NativeBytecodeIndex;
			}
		}
		CA_SUPPRESS(6386)
		GNatives[NativeBytecodeIndex] = Func;
	}

	return 0;
}

static constinit TStaticArray<FNativeFuncPtr, CST_Max> GCasts(InPlace, &UObject::execUndefined);

static uint8 GRegisterCast(ECastToken CastCode, const FNativeFuncPtr& Func)
{
	if (CastCode != CST_Max)
	{
		GCasts[CastCode] = Func;
	}
	return 0;
}

void UObject::SkipFunction(FFrame& Stack, RESULT_DECL, UFunction* Function)
{
	// allocate temporary memory on the stack for evaluating parameters
	UE_VSTACK_MAKE_FRAME(SkipFunctionBookmark, Stack.CachedThreadVirtualStackAllocator);

	uint8* Frame = (uint8*)UE_VSTACK_ALLOC_ALIGNED(Stack.CachedThreadVirtualStackAllocator, Function->PropertiesSize, Function->GetMinAlignment());
	FMemory::Memzero(Frame, Function->PropertiesSize);
	for (FProperty* Property = (FProperty*)(Function->ChildProperties); *Stack.Code != EX_EndFunctionParms; Property = (FProperty*)Property->Next)
	{
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		// evaluate the expression into our temporary memory space
		// it'd be nice to be able to skip the copy, but most native functions assume a non-NULL Result pointer
		// so we can only do that if we know the expression is an l-value (out parameter)
		Stack.Step(Stack.Object, (Property->PropertyFlags & CPF_OutParm) ? NULL : Property->ContainerPtrToValuePtr<uint8>(Frame));
	}

	// advance the code past EX_EndFunctionParms
	Stack.Code++;

	// destruct properties requiring it for which we had to use our temporary memory 
	// @warning: conditions for skipping DestroyValue() here must match conditions for passing NULL to Stack.Step() above
	for (FProperty* Destruct = Function->DestructorLink; Destruct; Destruct = Destruct->DestructorLinkNext)
	{
		if (!Destruct->HasAnyPropertyFlags(CPF_OutParm))
		{
			Destruct->DestroyValue_InContainer(Frame);
		}
	}

	FProperty* ReturnProp = Function->GetReturnProperty();
	if (ReturnProp != NULL)
	{
		// destroy old value if necessary
		ReturnProp->DestroyValue(RESULT_PARAM);
		// copy zero value for return property into Result
		FMemory::Memzero(RESULT_PARAM, ReturnProp->ArrayDim * ReturnProp->GetElementSize());
	}
}

#ifdef _MSC_VER
#pragma warning (push)
#pragma warning (disable : 4750) // warning C4750: function with _alloca() inlined into a loop
#endif

// Helper function to set up a script function, and then execute it using ExecFtor. This is 
// a template function because we use alloca to allocate space for parameters and results,
// but we also have two hotpaths: normal function calls which must call GetFunctionCallspace,
// and normal bytecode functions that are local only!
template<typename Exec>
void ProcessScriptFunction(UObject* Context, UFunction* Function, FFrame& Stack, RESULT_DECL, Exec ExecFtor)
{
	check(!Function->HasAnyFunctionFlags(FUNC_Native));

	// Allocate any temporary memory the script may need via AllocA. This AllocA dependency, along with
	// the desire to inline calls to our Execution function are the reason for this template function:
	FFrame NewStack(Context, Function, nullptr, &Stack, Function->ChildProperties);
	UE_VSTACK_MAKE_FRAME(ProcessScriptFunctionBookmark, NewStack.CachedThreadVirtualStackAllocator);

	uint8* FrameMemory = Function->GetOuterUClassUnchecked()->GetPersistentUberGraphFrame(Context, Function);

	bool bUsePersistentFrame = (nullptr != FrameMemory);
	if (!bUsePersistentFrame)
	{
		FrameMemory = (uint8*)UE_VSTACK_ALLOC_ALIGNED(NewStack.CachedThreadVirtualStackAllocator, Function->PropertiesSize, Function->GetMinAlignment());
		if (Function->PropertiesSize)
		{
			FMemory::Memzero(FrameMemory, Function->PropertiesSize);
		}
	}

	/* 
		Allocate space for return value bookkeeping - rarely used by bytecode functions, 
		but necessary in cases where a bytecode function's signature needs to match 
		a native function:
	 */
  	if( Function->ReturnValueOffset != MAX_uint16 )
 	{
		FProperty* ReturnProperty = Function->GetReturnProperty();
		if(ensure(ReturnProperty))
		{
 			FOutParmRec* RetVal = (FOutParmRec*)UE_VSTACK_ALLOC(NewStack.CachedThreadVirtualStackAllocator, sizeof(FOutParmRec));

 			/* Our context should be that we're in a variable assignment to the return value, so ensure that we have a valid property to return to */
 			check(RESULT_PARAM != NULL);
 			RetVal->PropAddr = (uint8*)RESULT_PARAM;
 			RetVal->Property = ReturnProperty;
			NewStack.OutParms = RetVal;
		}
 	}
	
	NewStack.Locals = FrameMemory;
	FOutParmRec** LastOut = &NewStack.OutParms;
		
	for (FProperty* Property = (FProperty*)(Function->ChildProperties); *Stack.Code != EX_EndFunctionParms; Property = (FProperty*)Property->Next)
	{
		checkfSlow(Property, TEXT("NULL Property in Function %s"), *Function->GetPathName()); 

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;

		// Skip the return parameter case, as we've already handled it above
		const bool bIsReturnParam = ((Property->PropertyFlags & CPF_ReturnParm) != 0);
		if( bIsReturnParam )
		{
			continue;
		}

		if (Property->PropertyFlags & CPF_OutParm)
		{
			// evaluate the expression for this parameter, which sets Stack.MostRecentPropertyAddress to the address of the property accessed
			Stack.Step(Stack.Object, NULL);

			CA_SUPPRESS(6263)
			FOutParmRec* Out = (FOutParmRec*)UE_VSTACK_ALLOC(NewStack.CachedThreadVirtualStackAllocator, sizeof(FOutParmRec));
			// set the address and property in the out param info
			// warning: Stack.MostRecentPropertyAddress could be NULL for optional out parameters
			// if that's the case, we use the extra memory allocated for the out param in the function's locals
			// so there's always a valid address
			ensureMsgf(Stack.MostRecentPropertyAddress, TEXT("MostRecentPropertyAddress was null. Blueprint callstack:\n%s"), *Stack.GetScriptCallstack()); // possible problem - output param values on local stack are neither initialized nor cleaned.
			Out->PropAddr = (Stack.MostRecentPropertyAddress != NULL) ? Stack.MostRecentPropertyAddress : Property->ContainerPtrToValuePtr<uint8>(NewStack.Locals);
			Out->Property = Property;

			// add the new out param info to the stack frame's linked list
			if (*LastOut)
			{
				(*LastOut)->NextOutParm = Out;
				LastOut = &(*LastOut)->NextOutParm;
			}
			else
			{
				*LastOut = Out;
			}
		}
		else
		{
			// copy the result of the expression for this parameter into the appropriate part of the local variable space
			uint8* Param = Property->ContainerPtrToValuePtr<uint8>(NewStack.Locals);
			checkSlow(Param);

			Property->InitializeValue_InContainer(NewStack.Locals);

			Stack.Step(Stack.Object, Param);
		}
	}
	Stack.Code++;
	// set the next pointer of the last item to NULL to mark the end of the list
	if (*LastOut)
	{
		(*LastOut)->NextOutParm = NULL;
	}

	if (!bUsePersistentFrame)
	{
		// Initialize any local properties that aren't CPF_ZeroConstruct:
		for (FProperty* LocalProp = Function->FirstPropertyToInit; LocalProp != nullptr; LocalProp = (FProperty*)(LocalProp->PostConstructLinkNext))
		{
			LocalProp->InitializeValue_InContainer(NewStack.Locals);
		}
	}

	if( Function->Script.Num() > 0)
	{
		// Execute the code.
		ExecFtor( Context, NewStack, RESULT_PARAM );
	}

	if (!bUsePersistentFrame)
	{
		// destruct properties on the stack, except for out params since we know we didn't use that memory
		for (FProperty* Destruct = Function->DestructorLink; Destruct; Destruct = Destruct->DestructorLinkNext)
		{
			if (!Destruct->HasAnyPropertyFlags(CPF_OutParm))
			{
				Destruct->DestroyValue_InContainer(NewStack.Locals);
			}
		}
	}

	// propagate abort flag up the stack
	Stack.bAbortingExecution |= NewStack.bAbortingExecution;
}

DEFINE_FUNCTION(UObject::execCallMathFunction)
{
	UFunction* Function = (UFunction*)Stack.ReadObject();
	checkSlow(Function);
	checkSlow(Function->FunctionFlags & FUNC_Native);
	// ProcessContext is the arbiter of net callspace, so we can't call net functions using this instruction:
	checkSlow(!Function->HasAnyFunctionFlags(FUNC_NetFuncFlags|FUNC_BlueprintAuthorityOnly|FUNC_BlueprintCosmetic|FUNC_NetRequest|FUNC_NetResponse));
	UObject* NewContext = Function->GetOuterUClassUnchecked()->GetDefaultObject(false);
	checkSlow(NewContext);
	{
#if PER_FUNCTION_SCRIPT_STATS
		const bool bShouldTrackFunction = Stack.DepthCounter <= GMaxFunctionStatDepth;
		FScopeCycleCounterUObject FunctionScope(bShouldTrackFunction ? Function : nullptr);
#endif // PER_FUNCTION_SCRIPT_STATS

		// CurrentNativeFunction is used so far only by FLuaContext::InvokeScriptFunction
		// TGuardValue<UFunction*> NativeFuncGuard(Stack.CurrentNativeFunction, Function);
		
		FNativeFuncPtr Func = Function->GetNativeFunc();
		checkSlow(Func);
		(*Func)(NewContext, Stack, RESULT_PARAM);
	}
}
IMPLEMENT_VM_FUNCTION(EX_CallMath, execCallMathFunction);

void UObject::CallFunction( FFrame& Stack, RESULT_DECL, UFunction* Function )
{
#if PER_FUNCTION_SCRIPT_STATS
	const bool bShouldTrackFunction = (Stack.DepthCounter <= GMaxFunctionStatDepth);
	SCOPE_CYCLE_UOBJECT(FunctionScope, bShouldTrackFunction ? Function : nullptr);
#endif // PER_FUNCTION_SCRIPT_STATS

	SCOPE_CYCLE_UOBJECT(ContextScope, GVerboseScriptStats ? this : nullptr);
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYNAME(TEXT("UObject/UObjectInternals"));
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET(this, GetClass());

	checkSlow(Function);

	if (Function->FunctionFlags & FUNC_Native)
	{
		const bool bNetFunction = Function->HasAnyFunctionFlags(FUNC_NetFuncFlags|FUNC_BlueprintAuthorityOnly|FUNC_BlueprintCosmetic|FUNC_NetRequest|FUNC_NetResponse);
		const int32 FunctionCallspace = bNetFunction ? GetFunctionCallspace( Function, &Stack ) : FunctionCallspace::Local;

		uint8* SavedCode = NULL;
		if (FunctionCallspace & FunctionCallspace::Remote)
		{
			// Call native networkable function.
			uint8* Buffer = (uint8*)UE_VSTACK_ALLOC_ALIGNED(Stack.CachedThreadVirtualStackAllocator, Function->ParmsSize, Function->GetMinAlignment());

			SavedCode = Stack.Code; // Since this is native, we need to rollback the stack if we are calling both remotely and locally

			FMemory::Memzero( Buffer, Function->ParmsSize );

			// Form the RPC parameters.
			for (TFieldIterator<FProperty> It(Function); It && (It->PropertyFlags & (CPF_Parm|CPF_ReturnParm))==CPF_Parm; ++It)
			{
				uint8* CurrentPropAddr = It->ContainerPtrToValuePtr<uint8>(Buffer);
				if (CastField<FBoolProperty>(*It) && It->ArrayDim == 1)
				{
					uint8 TempValue = 0;
					Stack.Step(Stack.Object, &TempValue);
					const bool NewBoolValue = (TempValue != 0);

					if (NewBoolValue)
					{
						((FBoolProperty*)*It)->SetPropertyValue(CurrentPropAddr, true);
					}
				}
				else
				{
					Stack.Step(Stack.Object, CurrentPropAddr);
				}
			}
			checkSlow(*Stack.Code==EX_EndFunctionParms);

			CallRemoteFunction(Function, Buffer, Stack.OutParms, &Stack);
		}

		if (FunctionCallspace & FunctionCallspace::Local)
		{
			if (SavedCode)
			{
				Stack.Code = SavedCode;
			}

			// Call regular native function.
			FScopeCycleCounterUObject NativeContextScope(GVerboseScriptStats ? Stack.Object : nullptr);
			Function->Invoke(this, Stack, RESULT_PARAM);
		}
		else
		{
			// Eat up the remaining parameters in the stream.
			SkipFunction(Stack, RESULT_PARAM, Function);
		}
	}
	else
	{
		ProcessScriptFunction(this, Function, Stack, RESULT_PARAM, ProcessInternal);
	}
}

/** Helper function to zero the return value in case of a fatal (runaway / infinite recursion) error */
void ClearReturnValue(FProperty* ReturnProp, RESULT_DECL)
{
	if (ReturnProp != NULL)
	{
		uint8* Data = (uint8*)RESULT_PARAM;
		for (int32 ArrayIdx = 0; ArrayIdx < ReturnProp->ArrayDim; ArrayIdx++, Data += ReturnProp->GetElementSize())
		{
			// Clear the property. This assumes that it has already been initialized, and that the caller will destroy it.
			ReturnProp->ClearValue(Data);
		}
	}
}

void ProcessLocalScriptFunction(UObject* Context, FFrame& Stack, RESULT_DECL)
{
	UFunction* Function = (UFunction*)Stack.Node;
	// No POD struct can ever be stored in this buffer. 
	MS_ALIGN(16) uint8 Buffer[MAX_SIMPLE_RETURN_VALUE_SIZE] GCC_ALIGN(16);

#if DO_BLUEPRINT_GUARD
	FBlueprintContextTracker& BpET = FBlueprintContextTracker::Get();
	if (BpET.bRanaway)
	{
		// If we have a return property, return a zeroed value in it, to try and save execution as much as possible
		FProperty* ReturnProp = (Function)->GetReturnProperty();
		ClearReturnValue(ReturnProp, RESULT_PARAM);
		return;
	}
	else if (++BpET.Recurse == GScriptRecurseLimit)
	{
		// If we have a return property, return a zeroed value in it, to try and save execution as much as possible
		FProperty* ReturnProp = (Function)->GetReturnProperty();
		ClearReturnValue(ReturnProp, RESULT_PARAM);

		// Notify anyone who cares that we've had a fatal error, so we can shut down PIE, etc
		FBlueprintExceptionInfo InfiniteRecursionExceptionInfo(
			EBlueprintExceptionType::InfiniteLoop, 
			FText::Format(
				LOCTEXT("InfiniteLoop", "Infinite script recursion ({0} calls) detected - see log for stack trace"), 
				FText::AsNumber(GScriptRecurseLimit)
			)
		);
		FBlueprintCoreDelegates::ThrowScriptException(Context, Stack, InfiniteRecursionExceptionInfo);

		// This flag prevents repeated warnings of infinite loop, script exception handler 
		// is expected to have terminated execution appropriately:
		BpET.bRanaway = true;

		return;
	}
#endif
	// Execute the bytecode
	while (*Stack.Code != EX_Return && !Stack.bAbortingExecution)
	{
#if DO_BLUEPRINT_GUARD
		if (BpET.Runaway > GMaximumScriptLoopIterations)
		{
			// If we have a return property, return a zeroed value in it, to try and save execution as much as possible
			FProperty* ReturnProp = (Function)->GetReturnProperty();
			ClearReturnValue(ReturnProp, RESULT_PARAM);

			// Notify anyone who cares that we've had a fatal error, so we can shut down PIE, etc
			FText ExceptionMessage;
			if (BpET.bScriptTimedOut)
			{
				ExceptionMessage = LOCTEXT("TimedOut", "Computation timed out - see log for stack trace");
			}
			else
			{
				ExceptionMessage = FText::Format(
					LOCTEXT("RunawayLoop", "Runaway loop detected (over {0} iterations) - see log for stack trace"),
					FText::AsNumber(GMaximumScriptLoopIterations)
				);
			}

			// Need to reset Runaway counter BEFORE throwing script exception, because the exception causes a modal dialog,
			// and other scripts running will then erroneously think they are also "runaway".
			BpET.Runaway = 0;

			FBlueprintExceptionInfo RunawayLoopExceptionInfo(EBlueprintExceptionType::InfiniteLoop, ExceptionMessage);
			FBlueprintCoreDelegates::ThrowScriptException(Context, Stack, RunawayLoopExceptionInfo);
			return;
		}
#endif

		Stack.Step(Stack.Object, Buffer);
	}

	if (!Stack.bAbortingExecution)
	{
		// Step over the return statement and evaluate the result expression
		Stack.Code++;

		if (*Stack.Code != EX_Nothing)
		{
			Stack.Step(Stack.Object, RESULT_PARAM);
		}
		else
		{
			Stack.Code++;
		}
	}
	else
	{
		// If we have a return property, return a zeroed value in it
		FProperty* ReturnProp = (Function)->GetReturnProperty();
		ClearReturnValue(ReturnProp, RESULT_PARAM);
	}

#if DO_BLUEPRINT_GUARD
	--BpET.Recurse;
#endif
}

void ProcessLocalFunction(UObject* Context, UFunction* Fn, FFrame& Stack, RESULT_DECL)
{
	checkSlow(Fn);

	auto ContinueProcessLocalFuntionInner = [&]() {
		if(Fn->HasAnyFunctionFlags(FUNC_Native))
		{
			FScopeCycleCounterUObject NativeContextScope(GVerboseScriptStats ? Context : nullptr);
			Fn->Invoke(Context, Stack, RESULT_PARAM);
		}
		else
		{
	#if PER_FUNCTION_SCRIPT_STATS
			const bool bShouldTrackFunction = (Stack.DepthCounter <= GMaxFunctionStatDepth);
			SCOPE_CYCLE_UOBJECT(FunctionScope, bShouldTrackFunction ? Fn : nullptr);
	#endif // PER_FUNCTION_SCRIPT_STATS
			ProcessScriptFunction(Context, Fn, Stack, RESULT_PARAM, ProcessLocalScriptFunction);
		}
	};

#if ENABLE_LOW_LEVEL_MEM_TRACKER
	if (Context && FLowLevelMemTracker::IsEnabled())
	{
		LLM_SCOPE(ELLMTag::UObject);
		LLM_SCOPE_BYNAME(TEXT("UObject/UObjectInternals"));
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Context->GetPackage(), ELLMTagSet::Assets);
		LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(Context->GetClass(), ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_SCOPE_ASSET(Context, Context->GetClass());
		return ContinueProcessLocalFuntionInner();
	}
	else
#endif
	{
		return ContinueProcessLocalFuntionInner();
	}
}

DEFINE_FUNCTION(UObject::ProcessInternal)
{
#if DO_BLUEPRINT_GUARD
	// remove later when stable
	if (P_THIS->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists))
	{
		if (!GIsReinstancing)
		{
			ensureMsgf(!P_THIS->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists), TEXT("Object '%s' is being used for execution, but its class is out of date and has been replaced with a recompiled class!"), *P_THIS->GetFullName());
		}
		return;
	}
#endif

	UFunction* Function = (UFunction*)Stack.Node;
	int32 FunctionCallspace = P_THIS->GetFunctionCallspace(Function, NULL);
	if (FunctionCallspace & FunctionCallspace::Remote)
	{
		P_THIS->CallRemoteFunction(Function, Stack.Locals, Stack.OutParms, NULL);
	}

	if (FunctionCallspace & FunctionCallspace::Local)
	{
		ProcessLocalScriptFunction(Context, Stack, RESULT_PARAM);
	}
	else
	{
		FProperty* ReturnProp = (Function)->GetReturnProperty();
		ClearReturnValue(ReturnProp, RESULT_PARAM);
	}
}

bool UObject::CallFunctionByNameWithArguments(const TCHAR* Str, FOutputDevice& Ar, UObject* Executor, bool bForceCallWithNonExec/*=false*/)
{
	// Find an exec function.
	FString MsgStr;
	if(!FParse::Token(Str,MsgStr,true))
	{
		UE_LOGF(LogScriptCore, Verbose, "CallFunctionByNameWithArguments: Not Parsed '%ls'", Str);
		return false;
	}
	const FName Message = FName(*MsgStr,FNAME_Find);
	if(Message == NAME_None)
	{
		UE_LOGF(LogScriptCore, Verbose, "CallFunctionByNameWithArguments: Name not found '%ls'", Str);
		return false;
	}
	UFunction* Function = FindFunction(Message);
	if(nullptr == Function)
	{
		UE_LOGF(LogScriptCore, Verbose, "CallFunctionByNameWithArguments: Function not found '%ls'", Str);
		return false;
	}
	if(0 == (Function->FunctionFlags & FUNC_Exec) && !bForceCallWithNonExec)
	{
		UE_LOGF(LogScriptCore, Verbose, "CallFunctionByNameWithArguments: Function not executable '%ls'", Str);
		return false;
	}

	FProperty* LastParameter = nullptr;

	// find the last parameter
	for ( TFieldIterator<FProperty> It(Function); It && (It->PropertyFlags&(CPF_Parm|CPF_ReturnParm)) == CPF_Parm; ++It )
	{
		LastParameter = *It;
	}

	// Parse all function parameters.
	uint8* Parms = (uint8*)FMemory_Alloca_Aligned(Function->ParmsSize, Function->GetMinAlignment());
	FMemory::Memzero( Parms, Function->ParmsSize );

	for (TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
	{
		FProperty* LocalProp = *It;
		checkSlow(LocalProp);
		if (!LocalProp->HasAnyPropertyFlags(CPF_ZeroConstructor))
		{
			LocalProp->InitializeValue_InContainer(Parms);
		}
	}

	const uint32 ExportFlags = PPF_None;
	bool bFailed = 0;
	int32 NumParamsEvaluated = 0;
	for( TFieldIterator<FProperty> It(Function); It && (It->PropertyFlags & (CPF_Parm|CPF_ReturnParm))==CPF_Parm; ++It, NumParamsEvaluated++ )
	{
		FProperty* PropertyParam = *It;
		checkSlow(PropertyParam); // Fix static analysis warning
		if (NumParamsEvaluated == 0 && Executor)
		{
			FObjectPropertyBase* Op = CastField<FObjectPropertyBase>(*It);
			if( Op && Executor->IsA(Op->PropertyClass) )
			{
				// First parameter is implicit reference to object executing the command.
				Op->SetObjectPropertyValue(Op->ContainerPtrToValuePtr<uint8>(Parms), Executor);
				continue;
			}
		}

		// Keep old string around in case we need to pass the whole remaining string
		const TCHAR* RemainingStr = Str;

		// Parse a new argument out of Str
		FString ArgStr;
		FParse::Token(Str, ArgStr, true);

		// if ArgStr is empty but we have more params to read parse the function to see if these have defaults, if so set them
		bool bFoundDefault = false;
		bool bFailedImport = true;
#if WITH_EDITOR
		if (!FCString::Strcmp(*ArgStr, TEXT("")))
		{
			const FName DefaultPropertyKey(*(FString("CPP_Default_") + PropertyParam->GetName()));
			const FString& PropertyDefaultValue = Function->GetMetaData(DefaultPropertyKey);
			if (!PropertyDefaultValue.IsEmpty()) 
			{
				bFoundDefault = true;

				const TCHAR* Result = It->ImportText_InContainer(*PropertyDefaultValue, Parms, nullptr, ExportFlags);
				bFailedImport = (Result == nullptr);
			}
		}
#endif

		if (!bFoundDefault)
		{
			// if this is the last string property and we have remaining arguments to process, we have to assume that this
			// is a sub-command that will be passed to another exec (like "cheat giveall weapons", for example). Therefore
			// we need to use the whole remaining string as an argument, regardless of quotes, spaces etc.
			if (PropertyParam == LastParameter && PropertyParam->IsA<FStrProperty>() && FCString::Strcmp(Str, TEXT("")) != 0)
			{
				ArgStr = FString(RemainingStr).TrimStart();
			}

			const TCHAR* Result = It->ImportText_InContainer(*ArgStr, Parms, nullptr, ExportFlags);
			bFailedImport = (Result == nullptr);
		}
		
		if( bFailedImport )
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("Message"), FText::FromName( Message ));
			Arguments.Add(TEXT("PropertyName"), FText::FromName(It->GetFName()));
			Arguments.Add(TEXT("FunctionName"), FText::FromName(Function->GetFName()));
			Ar.Logf( TEXT("%s"), *FText::Format( NSLOCTEXT( "Core", "BadProperty", "'{Message}': Bad or missing property '{PropertyName}' when trying to call {FunctionName}" ), Arguments ).ToString() );
			bFailed = true;

			break;
		}
	}

	if( !bFailed )
	{
		ProcessEvent( Function, Parms );
	}

	//!!destructframe see also UObject::ProcessEvent
	for( TFieldIterator<FProperty> It(Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It )
	{
		It->DestroyValue_InContainer(Parms);
	}

	// Success.
	return true;
}

UFunction* UObject::FindFunction( FName InName ) const
{
	return GetClass()->FindFunctionByName(InName);
}

UFunction* UObject::FindFunctionChecked( FName InName ) const
{
	UFunction* Result = FindFunction(InName);
	if (Result == NULL)
	{
		UE_LOGF(LogScriptCore, Fatal, "Failed to find function %ls in %ls", *InName.ToString(), *GetFullName());
	}
	return Result;
}

#if TOTAL_OVERHEAD_SCRIPT_STATS
void FBlueprintEventTimer::FPausableScopeTimer::Start()
{
	FPausableScopeTimer*& ActiveTimer = FThreadedTimerManager::Get().ActiveTimer;

	double CurrentTime = FPlatformTime::Seconds();
	if (ActiveTimer)
	{
		ActiveTimer->Pause(CurrentTime);
	}

	PreviouslyActiveTimer = ActiveTimer;
	StartTime = CurrentTime;
	TotalTime = 0.0;

	ActiveTimer = this;
}

double FBlueprintEventTimer::FPausableScopeTimer::Stop()
{
	if (PreviouslyActiveTimer)
	{
		PreviouslyActiveTimer->Resume();
	}
	FThreadedTimerManager::Get().ActiveTimer = PreviouslyActiveTimer;
	return TotalTime + (FPlatformTime::Seconds() - StartTime);
}

FBlueprintEventTimer::FScopedVMTimer::FScopedVMTimer()
	: Timer()
	, VMParent(nullptr)
{
	if (IsInGameThread())
	{
		FScopedVMTimer*& ActiveVMTimer = FThreadedTimerManager::Get().ActiveVMScope;
		VMParent = ActiveVMTimer;

		ActiveVMTimer = this;
		Timer.Start();
	}
}

FBlueprintEventTimer::FScopedVMTimer::~FScopedVMTimer()
{
	if (IsInGameThread())
	{
		INC_FLOAT_STAT_BY(STAT_ScriptVmTime_Total, Timer.Stop() * 1000.0);
		FThreadedTimerManager::Get().ActiveVMScope = VMParent;
	}
}

FBlueprintEventTimer::FScopedNativeTimer::FScopedNativeTimer()
	: Timer()
{
	if (IsInGameThread())
	{
		Timer.Start();
	}
}

FBlueprintEventTimer::FScopedNativeTimer::~FScopedNativeTimer()
{
	if (IsInGameThread())
	{
		if (FThreadedTimerManager::Get().ActiveVMScope)
		{
			if (IsInGameThread())
			{
				INC_FLOAT_STAT_BY(STAT_ScriptNativeTime_Total, Timer.Stop()* 1000.0);
			}
		}
	}
}

#endif

#if SCRIPT_AUDIT_ROUTINES

// heap would be more time efficient:
template<typename T>
void NBest(TArray<T>& OutBest, const T& NewEntry, TFunctionRef<bool(const T&, const T&)> IsBetter)
{
	if(IsBetter(NewEntry, OutBest.Last()))
	{
		// find insertion point:
		int32 InsertIdx = INDEX_NONE;
		// O(n):
		for(int32 I = 0; I < OutBest.Num(); ++I)
		{
			if(IsBetter( NewEntry, OutBest[I] ))
			{
				InsertIdx = I;
				break;
			}
		}

		// O(n):
		OutBest.Insert(NewEntry, InsertIdx);
		OutBest.Pop();
	}
}

static void OutputLongestFunctions(FOutputDevice& Ar, int32 Num)
{
	// max heap would be more efficient
	TArray<UFunction*> LongestFunctions;
	LongestFunctions.AddDefaulted(Num);

	for( TObjectIterator<UClass> It; It; ++It)
	{
		UClass* BPGC = *It;
		for(TFieldIterator<UFunction> FuncIt(BPGC, EFieldIteratorFlags::ExcludeSuper); FuncIt; ++FuncIt)
		{
			UFunction* Fn = *FuncIt;
			int32 LenScript = Fn->Script.Num();

			NBest<UFunction*>(LongestFunctions, Fn, 
				[LenScript](UFunction* A, UFunction* B)
				{
					return B == nullptr || LenScript > B->Script.Num();
				}
			);
		}
	}

	if(LongestFunctions.Num() == 0)
	{
		Ar.Log(TEXT("No script functions found when looking for longest functions."));
	}
	else
	{
		for(UFunction* Fn : LongestFunctions)
		{
			if(!Fn)
			{
				break;
			}

			Ar.Logf(TEXT("%s %s %d"), *Fn->GetName(), *Fn->GetOuter()->GetName(), Fn->Script.Num());
		}
	}
}

static void OutputMostFrequentlyCalledFunctions(FOutputDevice& OutputAr, int32 Num)
{
	// Script serialization is recursive and requires certain symbols (e.g. Script, a reference
	// to the bytecode), so we declare a type so that we have some scope:
	struct FCallFrequencyCounter
	{
		FCallFrequencyCounter(TArray<uint8>& InScript)
			: Script(InScript)
		{
		}

		TArray<uint8>& Script;
		TMap<UFunction*, int32>* FunctionCallCounts = nullptr;
		// Could try and get more context on vcalls, but for
		// this macro auditing tool name should be enough:
		TMap<FName, int32>* VirtualFunctionCallCounts = nullptr;

		void* GetLinker() { return nullptr; }

		EExprToken SerializeExpr(int32& iCode, FArchive& Ar)
		{
			#define SERIALIZEEXPR_INC
			#define SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
			
				if(iCode < Script.Num())
				{
					switch((EExprToken)Script[iCode])
					{
					case EX_CallMath:
					case EX_LocalFinalFunction:
					case EX_FinalFunction:
						{
							// peak UFunction*:
							if(FunctionCallCounts)
							{
								UFunction* Fn = nullptr;
								FMemory::Memcpy( &Fn, &Script[iCode+1], sizeof(UFunction*) );
								if(ensure(Fn))
								{
									check(Fn->IsValidLowLevel());
									FunctionCallCounts->FindOrAdd(Fn)++;
								}
							}
						}
						break;
					case EX_VirtualFunction:
					case EX_LocalVirtualFunction:
						{
							// peak function name:
							if(VirtualFunctionCallCounts)
							{
								FScriptName ScriptName;
								FMemory::Memcpy( &ScriptName, &Script[iCode+1], sizeof(FScriptName) );
								VirtualFunctionCallCounts->FindOrAdd(ScriptNameToName(ScriptName))++;
							}
						}
						break;

					}
				}

			#include "UObject/ScriptSerialization.inl"
				return Expr;
			#undef SERIALIZEEXPR_INC
			#undef SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
		}

		void CountCalls(TMap<UFunction*, int32>* InFunctionCallCounts, TMap<FName, int32>* InVirtualFunctionCallCounts)
		{
			FunctionCallCounts = InFunctionCallCounts;
			VirtualFunctionCallCounts = InVirtualFunctionCallCounts;

			int32 iCode = 0;
			const int32 ScriptSizeBytes = Script.Num();
			FNullArchive DummyArchive;

			while (iCode < ScriptSizeBytes)
			{
				SerializeExpr(iCode, DummyArchive);
			}
		}
	};

	TMap<UFunction*, int32> FunctionCallCounts;
	TMap<FName, int32> VirtualFunctionCallCounts;

	for( TObjectIterator<UClass> It; It; ++It)
	{
		UClass* BPGC = *It;
		for(TFieldIterator<UFunction> FuncIt(BPGC, EFieldIteratorFlags::ExcludeSuper); FuncIt; ++FuncIt)
		{
			UFunction* Fn = *FuncIt;

			// disassem and log function calls:
			FCallFrequencyCounter Counter(Fn->Script);
			Counter.CountCalls(&FunctionCallCounts, &VirtualFunctionCallCounts);
		}
	}

	// sort by # calls:
	{
		TArray< TPair<UFunction*, int32> > FunctionCallsSorted;
		FunctionCallsSorted.AddDefaulted(Num);
		for(const TPair<UFunction*, int32>& Calls : FunctionCallCounts )
		{
			NBest<TPair<UFunction*, int32>>(FunctionCallsSorted, Calls, 
				[](const TPair<UFunction*, int32>& A, const TPair<UFunction*, int32>& B) -> bool
				{
					return B.Key == nullptr || A.Value > B.Value;
				}
			);
		}

		if(FunctionCallsSorted.Num())
		{
			OutputAr.Logf(TEXT("Top %d function call targets"), FunctionCallsSorted.Num());
			for(TPair<UFunction*, int32>& Calls : FunctionCallsSorted )
			{
				if(Calls.Key == nullptr)
				{
					break;
				}

				OutputAr.Logf(TEXT("%s %s %d"), *Calls.Key->GetName(), *Calls.Key->GetOuter()->GetName(), Calls.Value);
			}
		}
		else
		{
			OutputAr.Log(TEXT("No function call instructions found in memory"));
		}
	}

	{
		TArray< TPair<FName, int32>> VirtualFunctionCallsSorted;
		VirtualFunctionCallsSorted.AddDefaulted(Num);
		for(const TPair<FName, int32>& Calls : VirtualFunctionCallCounts )
		{
			NBest<TPair<FName, int32>>(VirtualFunctionCallsSorted, Calls, 
				[](const TPair<FName, int32>& A, const TPair<FName, int32>& B) -> bool
				{
					return B.Key == FName() || A.Value > B.Value;
				}
			);
		}

		if(VirtualFunctionCallsSorted.Num())
		{
			OutputAr.Logf(TEXT("Top %d virtual function call targets"), VirtualFunctionCallsSorted.Num());
			for(TPair<FName, int32>& Calls : VirtualFunctionCallsSorted )
			{
				if(Calls.Key == FName())
				{
					break;
				}

				OutputAr.Logf(TEXT("%s %d"), *(Calls.Key.ToString()), Calls.Value);
			}
		}
		else
		{
			OutputAr.Log(TEXT("No virtual function call instructions in memory"));
		}
	}
}

static void OutputMostFrequentlyUsedInstructions(FOutputDevice& OutputAr, int32 Num)
{
	// Script serialization is recursive and requires certain symbols (e.g. Script, a reference
	// to the bytecode), so we declare a type so that we have some scope:
	struct FInstructionFrequencyCounter
	{
		FInstructionFrequencyCounter(TArray<uint8>& InScript)
			: Script(InScript)
		{
		}
		
		TArray<uint8>& Script;
		TMap<EExprToken, int32>* InstructionCallCounts;

		void* GetLinker() { return nullptr; }

		
		EExprToken SerializeExpr(int32& iCode, FArchive& Ar)
		{
			#define SERIALIZEEXPR_INC
			#define SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
			
				if(iCode < Script.Num())
				{
					if(InstructionCallCounts)
					{
						InstructionCallCounts->FindOrAdd((EExprToken)Script[iCode])++;
					}
				}

			#include "UObject/ScriptSerialization.inl"
				return Expr;
			#undef SERIALIZEEXPR_INC
			#undef SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
		}

		void CountInstructions(TMap<EExprToken, int32>* InInstructionCallCounts)
		{
			InstructionCallCounts = InInstructionCallCounts;

			int32 iCode = 0;
			const int32 ScriptSizeBytes = Script.Num();
			FNullArchive DummyArchive;

			while (iCode < ScriptSizeBytes)
			{
				SerializeExpr(iCode, DummyArchive);
			}
		}
	};

	
	TMap<EExprToken, int32> InstructionCallCounts;

	for( TObjectIterator<UClass> It; It; ++It)
	{
		UClass* BPGC = *It;
		for(TFieldIterator<UFunction> FuncIt(BPGC, EFieldIteratorFlags::ExcludeSuper); FuncIt; ++FuncIt)
		{
			UFunction* Fn = *FuncIt;

			// disassem and log function calls:
			FInstructionFrequencyCounter Counter(Fn->Script);
			Counter.CountInstructions(&InstructionCallCounts);
		}
	}

	// sort by #:
	{
		TArray<TPair<EExprToken, int32>> InstructionCountsSorted;
		InstructionCountsSorted.AddDefaulted(Num);
		
		for(const TPair<EExprToken, int32>& Instruction : InstructionCallCounts )
		{
			NBest<TPair<EExprToken, int32>>(InstructionCountsSorted, Instruction, 
				[](const TPair<EExprToken, int32>& A, const TPair<EExprToken, int32>& B) -> bool
				{
					return A.Value > B.Value;
				}
			);
		}

		if(InstructionCountsSorted.Num())
		{
			OutputAr.Logf(TEXT("Top %d bytecode instructions"), InstructionCountsSorted.Num());
			for(TPair<EExprToken, int32>& Instruction : InstructionCountsSorted )
			{
				if(Instruction.Value == 0)
				{
					break;
				}

#if STORE_INSTRUCTION_NAMES
				if(GNativeFuncNames[Instruction.Key])
				{
					FString AsString = GNativeFuncNames[Instruction.Key];
					OutputAr.Logf(TEXT("%s %d"), *AsString, Instruction.Value);
				}
				else
				{
					OutputAr.Logf(TEXT("0x%x %d"), Instruction.Key, Instruction.Value);
				}
#else
				OutputAr.Logf(TEXT("0x%x %d"), Instruction.Key, Instruction.Value);
#endif
			}
		}
		else
		{
			OutputAr.Log(TEXT("No instructions found in memory"));
		}
	}
}

static void OutputTotalBytecodeSize(FOutputDevice& Ar)
{
	uint32 TotalSize = 0;
	
	for( TObjectIterator<UClass> It; It; ++It)
	{
		UClass* BPGC = *It;
		for(TFieldIterator<UFunction> FuncIt(BPGC, EFieldIteratorFlags::ExcludeSuper); FuncIt; ++FuncIt)
		{
			UFunction* Fn = *FuncIt;

			TotalSize += Fn->Script.Num();
		}
	}

	Ar.Logf(TEXT("Total bytecode size: %d"), TotalSize);
}

struct FScriptAuditExec 
	: public FSelfRegisteringExec
{
protected:
	// FSelfRegisteringExec:
	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;
} ScriptAudit;

bool FScriptAuditExec::Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (FParse::Command(&Cmd, TEXT("ScriptAudit")))
	{
		FString ParsedCommand = FParse::Token(Cmd, 0);

		if (ParsedCommand.Equals(TEXT("LongestFunctions"),ESearchCase::IgnoreCase))
		{
			int32 NumToOutput = 20;

			FString Num = FParse::Token(Cmd, 0);
			if(!Num.IsEmpty())
			{
				NumToOutput = FCString::Atoi(*Num);
			}
			OutputLongestFunctions(Ar, NumToOutput);
			return true;
		}
		else if (ParsedCommand.Equals(TEXT("FrequentFunctionsCalled"),ESearchCase::IgnoreCase))
		{
			int32 NumToOutput = 20;

			FString Num = FParse::Token(Cmd, 0);
			if(!Num.IsEmpty())
			{
				NumToOutput = FCString::Atoi(*Num);
			}
			OutputMostFrequentlyCalledFunctions(Ar, NumToOutput);
			return true;
		}
		else if (ParsedCommand.Equals(TEXT("FrequentInstructions"),ESearchCase::IgnoreCase))
		{
			int32 NumToOutput = 20;

			FString Num = FParse::Token(Cmd, 0);
			if(!Num.IsEmpty())
			{
				NumToOutput = FCString::Atoi(*Num);
			}
			OutputMostFrequentlyUsedInstructions(Ar, NumToOutput);
			return true;
		}
		else if (ParsedCommand.Equals(TEXT("TotalBytecodeSize"),ESearchCase::IgnoreCase))
		{
			OutputTotalBytecodeSize(Ar);
			return true;
		}
	}

	return false;
}

#endif //SCRIPT_AUDIT_ROUTINES

// Switch for a lightweight process event counter, useful when disabling the blueprint guard
// which can taint profiling results:
#define LIGHTWEIGHT_PROCESS_EVENT_COUNTER 0 && !DO_BLUEPRINT_GUARD

#if LIGHTWEIGHT_PROCESS_EVENT_COUNTER
thread_local int32 ProcessEventCounter = 0;
#endif

void UObject::ProcessEvent( UFunction* Function, void* Parms )
{
	// Unreachable objects are either about to be destroyed by garbage collection or temporarily marked as unreachable during GC reachability analysis on the GameThread.
	// UObject functions are unsafe to call off-GameThread unless precautions are taken not to coincide with garbage collection (analysis) on the GameThread.
	checkf(!IsUnreachable(), TEXT("Function '%s' called on Object '%s' that was marked unreachable. Object is possibly about to be garbage collected due to not being referenced. %s"),
		*Function->GetPathName(), *GetFullName(), !IsInGameThread() ? TEXT("Alternatively, this function was called from a non-GameThread which is unsafe.") : TEXT(""));
	checkf(!FUObjectThreadContext::Get().IsRoutingPostLoad, TEXT("Cannot call UnrealScript (%s - %s) while PostLoading objects"), *GetFullName(), *Function->GetFullName());

#if TOTAL_OVERHEAD_SCRIPT_STATS
	FBlueprintEventTimer::FScopedVMTimer VMTime;
#endif // TOTAL_OVERHEAD_SCRIPT_STATS

	// Reject.
	if (!IsValidChecked(this))
	{
		return;
	}
	
#if WITH_EDITORONLY_DATA
	// Cannot invoke script events when the game thread is paused for debugging.
	if(GIntraFrameDebuggingGameThread)
	{
		if(GFirstFrameIntraFrameDebugging)
		{
			UE_LOGF(LogScriptCore, Warning, "Cannot call UnrealScript (%ls - %ls) while stopped at a breakpoint.", *GetFullName(), *Function->GetFullName());
		}

		return;
	}
#endif	// WITH_EDITORONLY_DATA

	if ((Function->FunctionFlags & FUNC_Native) != 0)
	{
		int32 FunctionCallspace = GetFunctionCallspace(Function, NULL);
		if (FunctionCallspace & FunctionCallspace::Remote)
		{
			CallRemoteFunction(Function, Parms, NULL, NULL);
		}

		if ((FunctionCallspace & FunctionCallspace::Local) == 0)
		{
			return;
		}
	}
	else if (Function->Script.Num() == 0)
	{
		return;
	}
	checkSlow((Function->ParmsSize == 0) || (Parms != NULL));

#if PER_FUNCTION_SCRIPT_STATS
	SCOPE_CYCLE_UOBJECT(FunctionScope, Function);
#endif // PER_FUNCTION_SCRIPT_STATS

	SCOPE_CYCLE_UOBJECT(ContextScope, GVerboseScriptStats ? this : nullptr);

	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYNAME(TEXT("UObject/UObjectInternals"));
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetPackage(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(GetClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET(this, GetClass());

#if LIGHTWEIGHT_PROCESS_EVENT_COUNTER
	TGuardValue<int32> PECounter(ProcessEventCounter, ProcessEventCounter + 1);
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_BlueprintTime, IsInGameThread() && ProcessEventCounter == 1);
#endif

#if DO_BLUEPRINT_GUARD
	FBlueprintContextTracker& BlueprintContextTracker = FBlueprintContextTracker::Get();
	const int32 ProcessEventDepth = BlueprintContextTracker.GetScriptEntryTag();
	BlueprintContextTracker.EnterScriptContext(this, Function);

	// Only start stat if this is the top level context
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_BlueprintTime, IsInGameThread() && BlueprintContextTracker.GetScriptEntryTag() == 1);
#endif

#if UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
	// Fast path for ubergraph calls
	int32 EventGraphParams;
	if (Function->EventGraphFunction != nullptr)
	{
		// Call directly into the event graph, skipping the stub thunk function
		EventGraphParams = Function->EventGraphCallOffset;
		Parms = &EventGraphParams;
		Function = Function->EventGraphFunction;

		// Validate assumptions required for this optimized path (EventGraphFunction should have only been filled out if these held)
		checkSlow(Function->ParmsSize == sizeof(EventGraphParams));
		checkSlow(Function->FirstPropertyToInit == nullptr);
		checkSlow(Function->PostConstructLink == nullptr);
	}
#endif

	// Scope required for scoped script stats.
	{
		uint8* Frame = nullptr;
		if (Function->HasAnyFunctionFlags(FUNC_UbergraphFunction))
		{
			Frame = Function->GetOuterUClassUnchecked()->GetPersistentUberGraphFrame(this, Function);
		}

		FVirtualStackAllocator* VirtualStackAllocator = FBlueprintContext::GetThreadSingleton()->GetVirtualStackAllocator();
		UE_VSTACK_MAKE_FRAME(ProcessEventBookmark, VirtualStackAllocator);
		const bool bUsePersistentFrame = (nullptr != Frame);
		if (!bUsePersistentFrame)
		{
			Frame = (uint8*)UE_VSTACK_ALLOC_ALIGNED(VirtualStackAllocator, Function->PropertiesSize, Function->GetMinAlignment());
			// zero the local property memory
			const int32 NonParmsPropertiesSize = Function->PropertiesSize - Function->ParmsSize;
			if (NonParmsPropertiesSize)
			{
				FMemory::Memzero(Frame + Function->ParmsSize, NonParmsPropertiesSize);
			}
		}

		// initialize the parameter properties
		if (Function->ParmsSize)
		{
			FMemory::Memcpy(Frame, Parms, Function->ParmsSize);
		}

		// Create a new local execution stack.
		FFrame NewStack(this, Function, Frame, NULL, Function->ChildProperties);

		checkSlow(NewStack.Locals || Function->ParmsSize == 0);



		// if the function has out parameters, fill the stack frame's out parameter info with the info for those params 
		if ( Function->HasAnyFunctionFlags(FUNC_HasOutParms) )
		{
			FOutParmRec** LastOut = &NewStack.OutParms;
			for ( FProperty* Property = (FProperty*)(Function->ChildProperties); Property && (Property->PropertyFlags&(CPF_Parm)) == CPF_Parm; Property = (FProperty*)Property->Next )
			{
				// this is used for optional parameters - the destination address for out parameter values is the address of the calling function
				// so we'll need to know which address to use if we need to evaluate the default parm value expression located in the new function's
				// bytecode
				if ( Property->HasAnyPropertyFlags(CPF_OutParm) )
				{
					CA_SUPPRESS(6263)
					FOutParmRec* Out = (FOutParmRec*)UE_VSTACK_ALLOC(VirtualStackAllocator, sizeof(FOutParmRec));
					// set the address and property in the out param info
					// note that since C++ doesn't support "optional out" we can ignore that here
					Out->PropAddr = Property->ContainerPtrToValuePtr<uint8>(Parms);
					Out->Property = Property;

					// add the new out param info to the stack frame's linked list
					if (*LastOut)
					{
						(*LastOut)->NextOutParm = Out;
						LastOut = &(*LastOut)->NextOutParm;
					}
					else
					{
						*LastOut = Out;
					}
				}
			}

			// set the next pointer of the last item to NULL to mark the end of the list
			if (*LastOut)
			{
				(*LastOut)->NextOutParm = NULL;
			}
		}

		if (!bUsePersistentFrame)
		{
			for (FProperty* LocalProp = Function->FirstPropertyToInit; LocalProp != nullptr; LocalProp = (FProperty*)LocalProp->PostConstructLinkNext)
			{
				LocalProp->InitializeValue_InContainer(NewStack.Locals);
			}
		}

		// Call native function or UObject::ProcessInternal.
		const bool bHasReturnParam = Function->ReturnValueOffset != MAX_uint16;
		uint8* ReturnValueAddress = bHasReturnParam ? ((uint8*)Parms + Function->ReturnValueOffset) : nullptr;
		Function->Invoke(this, NewStack, ReturnValueAddress);

		if (!bUsePersistentFrame)
		{
			// Destroy local variables except function parameters.!! see also UObject::CallFunctionByNameWithArguments
			// also copy back constructed value parms here so the correct copy is destroyed when the event function returns
			for (FProperty* P = Function->DestructorLink; P; P = P->DestructorLinkNext)
			{
				if (!P->IsInContainer(Function->ParmsSize))
				{
					P->DestroyValue_InContainer(NewStack.Locals);
				}
				else if (!(P->PropertyFlags & CPF_OutParm))
				{
					FMemory::Memcpy(P->ContainerPtrToValuePtr<uint8>(Parms), P->ContainerPtrToValuePtr<uint8>(NewStack.Locals), P->ArrayDim * P->GetElementSize());
				}
			}
		}
	}

#if DO_BLUEPRINT_GUARD
	BlueprintContextTracker.ExitScriptContext();
#endif
}

#ifdef _MSC_VER
#pragma warning (pop)
#endif

DEFINE_FUNCTION(UObject::execUndefined)
{
	const FText LocalizedErrorMessage = FText::Format(
		LOCTEXT("UndefinedOpcode", "Encountered an undefined opcode ({0}) at byte offset {1}. The compiler may have generated an instruction sequence that was unexpected or incomplete."),
		FText::FromString(FString::Printf(TEXT("0x%02X"), Stack.Code[-1])),
		FText::AsNumber(Stack.Node ? reinterpret_cast<ScriptPointerType>(&Stack.Code[-1]) - reinterpret_cast<ScriptPointerType>(&Stack.Node->Script[0]) : 0)
	);

	Stack.Log(ELogVerbosity::Error, LocalizedErrorMessage.ToString());
}

DEFINE_FUNCTION(UObject::execLocalVariable)
{
	checkSlow(Stack.Object == P_THIS);
	checkSlow(Stack.Locals != NULL);

	FProperty* VarProperty = Stack.ReadProperty();
	if (VarProperty == nullptr)
	{
		FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::AccessViolation, LOCTEXT("MissingLocalVariable", "Attempted to access missing local variable. If this is a packaged/cooked build, are you attempting to use an editor-only property?"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
	}
	else
	{
		Stack.MostRecentPropertyAddress = VarProperty->ContainerPtrToValuePtr<uint8>(Stack.Locals);
		Stack.MostRecentPropertyContainer = Stack.Locals;

		if (RESULT_PARAM)
		{
			VarProperty->CopyCompleteValueToScriptVM_InContainer(RESULT_PARAM, Stack.MostRecentPropertyContainer);
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_LocalVariable, execLocalVariable );

DEFINE_FUNCTION(UObject::execInstanceVariable)
{
	FProperty* VarProperty = (FProperty*)Stack.ReadPropertyUnchecked();

	if (VarProperty == nullptr || !P_THIS->IsA((UClass*)VarProperty->InternalGetOwnerAsUObjectUnsafe()))
	{
		FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::AccessViolation, FText::Format(LOCTEXT("MissingProperty", "Attempted to access missing property '{0}'. If this is a packaged/cooked build, are you attempting to use an editor-only property?"), FText::FromString(GetNameSafe(VarProperty))));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
	}
	else
	{
		Stack.MostRecentPropertyAddress = VarProperty->ContainerPtrToValuePtr<uint8>(P_THIS);
		Stack.MostRecentPropertyContainer = (uint8*)P_THIS;
		if (RESULT_PARAM)
		{
			VarProperty->CopyCompleteValueToScriptVM_InContainer(RESULT_PARAM, Stack.MostRecentPropertyContainer);
		}
	}

	
}
IMPLEMENT_VM_FUNCTION( EX_InstanceVariable, execInstanceVariable );

DEFINE_FUNCTION(UObject::execClassSparseDataVariable)
{
	FProperty* VarProperty = (FProperty*)Stack.ReadPropertyUnchecked();

	if (VarProperty == nullptr || P_THIS->GetSparseClassDataStruct() == nullptr)
	{
		FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::AccessViolation, FText::Format(LOCTEXT("MissingSparseProperty", "Attempted to access missing sparse property '{0}' {1}, {2}. If this is a packaged/cooked build, are you attempting to use an editor-only property?"), FText::FromString(GetNameSafe(VarProperty)), FText::FromString(GetNameSafe(P_THIS->GetSparseClassDataStruct())), FText::FromString(GetNameSafe(VarProperty ? VarProperty->GetOwner<UClass>() : nullptr))));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
	}
	else
	{
		void* SparseDataBaseAddress = const_cast<void*>(P_THIS->GetClass()->GetSparseClassData(EGetSparseClassDataMethod::ArchetypeIfNull));
		Stack.MostRecentPropertyAddress = VarProperty->ContainerPtrToValuePtr<uint8>(SparseDataBaseAddress);
		Stack.MostRecentPropertyContainer = (uint8*)SparseDataBaseAddress;

		if (RESULT_PARAM)
		{
			VarProperty->CopyCompleteValueToScriptVM_InContainer(RESULT_PARAM, Stack.MostRecentPropertyContainer);
		}
	}
}
IMPLEMENT_VM_FUNCTION(EX_ClassSparseDataVariable, execClassSparseDataVariable);

DEFINE_FUNCTION(UObject::execDefaultVariable)
{
	FProperty* VarProperty = (FProperty*)Stack.ReadPropertyUnchecked();
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;

	UObject* DefaultObject = nullptr;
	if (P_THIS->HasAnyFlags(RF_ClassDefaultObject))
	{
		DefaultObject = P_THIS;
	}
	else
	{
		// @todo - allow access to archetype properties through object references?
	}

	if (VarProperty == nullptr || (DefaultObject && !DefaultObject->IsA((UClass*)VarProperty->InternalGetOwnerAsUObjectUnsafe())))
	{
		FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::AccessViolation, LOCTEXT("MissingPropertyDefaultObject", "Attempted to access a missing property on a CDO. If this is a packaged/cooked build, are you attempting to use an editor-only property?"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
	else
	{
		if(DefaultObject != nullptr)
		{
			Stack.MostRecentPropertyAddress = VarProperty->ContainerPtrToValuePtr<uint8>(DefaultObject);
			Stack.MostRecentPropertyContainer = (uint8*)DefaultObject;

			if(RESULT_PARAM)
			{
				VarProperty->CopyCompleteValueToScriptVM_InContainer(RESULT_PARAM, Stack.MostRecentPropertyContainer);
			}
		}
		else
		{
			FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::AccessViolation, LOCTEXT("AccessNoneDefaultObject", "Accessed None attempting to read a default property"));
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_DefaultVariable, execDefaultVariable );

DEFINE_FUNCTION(UObject::execLocalOutVariable)
{
	checkSlow(Stack.Object == P_THIS);

	// get the property we need to find
	FProperty* VarProperty = Stack.ReadProperty();
	
	// look through the out parameter infos and find the one that has the address of this property
	FOutParmRec* Out = Stack.OutParms;
	checkSlow(Out);
	while (Out->Property != VarProperty)
	{
		Out = Out->NextOutParm;
		checkSlow(Out);
	}
	Stack.MostRecentPropertyAddress = Out->PropAddr;

	// if desired, copy the value in that address to Result
	if (RESULT_PARAM && RESULT_PARAM != Stack.MostRecentPropertyAddress)
	{
		VarProperty->CopyCompleteValueToScriptVM(RESULT_PARAM, Stack.MostRecentPropertyAddress);
	}
}
IMPLEMENT_VM_FUNCTION(EX_LocalOutVariable, execLocalOutVariable);

DEFINE_FUNCTION(UObject::execInterfaceContext)
{
	// get the value of the interface variable
	FScriptInterface InterfaceValue;
	Stack.Step(P_THIS, &InterfaceValue);

	if (RESULT_PARAM != NULL)
	{
		// copy the UObject pointer to Result
		*(UObject**)RESULT_PARAM = InterfaceValue.GetObject();
	}
}
IMPLEMENT_VM_FUNCTION( EX_InterfaceContext, execInterfaceContext );

DEFINE_FUNCTION(UObject::execClassContext)
{
	// Get class expression.
	UClass* ClassContext = NULL;
	Stack.Step(P_THIS, &ClassContext);

	// Execute expression in class context.
	if(IsValid(ClassContext))
	{
		UObject* DefaultObject = ClassContext->GetDefaultObject();
		check(DefaultObject != NULL);

		Stack.Code += sizeof(CodeSkipSizeType)	// Code offset for NULL expressions.
			+ sizeof(ScriptPointerType);		// Property corresponding to the r-value data, in case the l-value needs to be cleared
		Stack.Step(DefaultObject, RESULT_PARAM);
	}
	else
	{
		if (Stack.MostRecentProperty != NULL)
		{
			FBlueprintExceptionInfo ExceptionInfo(
				EBlueprintExceptionType::AccessViolation, 
				FText::Format(
					LOCTEXT("AccessedNoneClass", "Accessed None trying to read Class from property {0}"), 
					FText::FromString(Stack.MostRecentProperty->GetName())
				)
			);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}
		else
		{
			FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::AccessViolation, LOCTEXT("AccessedNoneClassUnknownProperty", "Accessed None reading a Class"));
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
		}

		const CodeSkipSizeType wSkip = Stack.ReadCodeSkipCount(); // Code offset for NULL expressions. Code += sizeof(CodeSkipSizeType)
		FProperty* RValueProperty = nullptr;
		const VariableSizeType bSize = Stack.ReadVariableSize(&RValueProperty); // Code += sizeof(ScriptPointerType) + sizeof(uint8)
		Stack.Code += wSkip;
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.MostRecentProperty = nullptr;

		if (RESULT_PARAM && RValueProperty)
		{
			RValueProperty->ClearValue(RESULT_PARAM);
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_ClassContext, execClassContext );

DEFINE_FUNCTION(UObject::execEndOfScript)
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		UE_LOGF(LogScriptCore, Warning, "--- Dumping bytecode for %ls on %ls ---", *Stack.Node->GetFullName(), *Stack.Object->GetFullName());
		const UFunction* Func = Stack.Node;
		for(int32 i = 0; i < Func->Script.Num(); ++i)
		{
			UE_LOGF(LogScriptCore, Log, "0x%x", Func->Script[i]);
		}
	}
#endif //WITH_EDITOR

	UE_LOGF(LogScriptCore, Fatal,"Execution beyond end of script in %ls on %ls", *Stack.Node->GetFullName(), *Stack.Object->GetFullName());
}
IMPLEMENT_VM_FUNCTION( EX_EndOfScript, execEndOfScript );

DEFINE_FUNCTION(UObject::execNothing)
{
	// Do nothing.
}
IMPLEMENT_VM_FUNCTION( EX_Nothing, execNothing );

DEFINE_FUNCTION(UObject::execNothingInt32)
{
	int32 Value = Stack.ReadInt<int32>();
}
IMPLEMENT_VM_FUNCTION( EX_NothingInt32, execNothingInt32 );

DEFINE_FUNCTION(UObject::execNothingOp4a)
{
	// Do nothing.
}
IMPLEMENT_VM_FUNCTION( EX_DeprecatedOp4A, execNothingOp4a );

DEFINE_FUNCTION(UObject::execBreakpoint)
{
	if (FBlueprintCoreDelegates::IsDebuggingEnabled())
	{
		FBlueprintExceptionInfo BreakpointExceptionInfo(EBlueprintExceptionType::Breakpoint);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, BreakpointExceptionInfo);
	}
}
IMPLEMENT_VM_FUNCTION( EX_Breakpoint, execBreakpoint );

DEFINE_FUNCTION(UObject::execTracepoint)
{
	if (FBlueprintCoreDelegates::IsDebuggingEnabled())
	{
		FBlueprintExceptionInfo TracepointExceptionInfo(EBlueprintExceptionType::Tracepoint);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, TracepointExceptionInfo);
	}
}
IMPLEMENT_VM_FUNCTION( EX_Tracepoint, execTracepoint );

DEFINE_FUNCTION(UObject::execWireTracepoint)
{
	if (FBlueprintCoreDelegates::IsDebuggingEnabled())
	{
		FBlueprintExceptionInfo TracepointExceptionInfo(EBlueprintExceptionType::WireTracepoint);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, TracepointExceptionInfo);
	}
}
IMPLEMENT_VM_FUNCTION( EX_WireTracepoint, execWireTracepoint );

DEFINE_FUNCTION(UObject::execInstrumentation)
{
#if !UE_BUILD_SHIPPING
	const EScriptInstrumentation::Type EventType = static_cast<EScriptInstrumentation::Type>(Stack.PeekCode());
#if WITH_EDITORONLY_DATA
	if (GIsEditor)
	{
		if (EventType == EScriptInstrumentation::NodeEntry)
		{
			FBlueprintExceptionInfo TracepointExceptionInfo(EBlueprintExceptionType::Tracepoint);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, TracepointExceptionInfo);
		}
		else if (EventType == EScriptInstrumentation::NodeExit)
		{
			FBlueprintExceptionInfo WiretraceExceptionInfo(EBlueprintExceptionType::WireTracepoint);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, WiretraceExceptionInfo);
		}
		else if (EventType == EScriptInstrumentation::NodeDebugSite)
		{
			FBlueprintExceptionInfo TracepointExceptionInfo(EBlueprintExceptionType::Breakpoint);
			FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, TracepointExceptionInfo);
		}
	}
#endif
	if (EventType == EScriptInstrumentation::InlineEvent)
	{
		const FScriptName& EventName = *reinterpret_cast<FScriptName*>(&Stack.Code[1]);
		FScriptInstrumentationSignal InstrumentationEventInfo(EventType, P_THIS, Stack, ScriptNameToName(EventName));
		FBlueprintCoreDelegates::InstrumentScriptEvent(InstrumentationEventInfo);
		Stack.SkipCode(sizeof(FScriptName) + 1);
	}
	else
	{
		FScriptInstrumentationSignal InstrumentationEventInfo(EventType, P_THIS, Stack);
		FBlueprintCoreDelegates::InstrumentScriptEvent(InstrumentationEventInfo);
		Stack.SkipCode(1);
	}
#endif
}
IMPLEMENT_VM_FUNCTION( EX_InstrumentationEvent, execInstrumentation );

DEFINE_FUNCTION(UObject::execEndFunctionParms)
{
	// For skipping over optional function parms without values specified.
	Stack.Code--;
}
IMPLEMENT_VM_FUNCTION( EX_EndFunctionParms, execEndFunctionParms );


DEFINE_FUNCTION(UObject::execJump)
{
	CheckRunaway();

	// Jump immediate.
	CodeSkipSizeType Offset = Stack.ReadCodeSkipCount();
	Stack.Code = &Stack.Node->Script[Offset];
}
IMPLEMENT_VM_FUNCTION( EX_Jump, execJump );

DEFINE_FUNCTION(UObject::execComputedJump)
{
	CheckRunaway();

	// Get the jump offset expression
	int32 ComputedOffset = 0;
	Stack.Step( Stack.Object, &ComputedOffset );
	check((ComputedOffset < Stack.Node->Script.Num()) && (ComputedOffset >= 0));

	// Jump to the new offset
	Stack.Code = &Stack.Node->Script[ComputedOffset];
}
IMPLEMENT_VM_FUNCTION( EX_ComputedJump, execComputedJump );
	
DEFINE_FUNCTION(UObject::execJumpIfNot)
{
	CheckRunaway();

	// Get code offset.
	CodeSkipSizeType Offset = Stack.ReadCodeSkipCount();

	uint8 TempValue = 0;
	Stack.Step(Stack.Object, &TempValue);
	const bool NewBoolValue = (TempValue != 0);

	// Jump if false.
	if (!NewBoolValue)
	{
		Stack.Code = &Stack.Node->Script[Offset];
	}
}
IMPLEMENT_VM_FUNCTION( EX_JumpIfNot, execJumpIfNot );

DEFINE_FUNCTION(UObject::execAssert)
{
	// Get line number.
	int32 wLine = Stack.ReadWord();

	// find out whether we are in debug mode and therefore should crash on failure
	uint8 bDebug = *(uint8*)Stack.Code++;

	// Get boolean assert value.
	uint32 Value=0;
	Stack.Step( Stack.Object, &Value );

	// Check it.
	if( !Value )
	{
		Stack.Logf(TEXT("%s"), *Stack.GetStackTrace());
		if (bDebug)
		{
			Stack.Logf(ELogVerbosity::Error, TEXT("Assertion failed, line %i"), wLine);
		}
		else
		{
			UE_SUPPRESS(LogScript, Warning, Stack.Logf(TEXT("Assertion failed, line %i"), wLine));
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_Assert, execAssert );

DEFINE_FUNCTION(UObject::execPushExecutionFlow)
{
	// Read a code offset and push it onto the flow stack
	CodeSkipSizeType Offset = Stack.ReadCodeSkipCount();
	Stack.FlowStack.Push(Offset);
}
IMPLEMENT_VM_FUNCTION( EX_PushExecutionFlow, execPushExecutionFlow );

DEFINE_FUNCTION(UObject::execPopExecutionFlow)
{
	// Since this is a branch function, check for runaway script execution
	CheckRunaway();

	// Try to pop an entry off the stack and go there
	if (Stack.FlowStack.Num())
	{
		CodeSkipSizeType Offset = Stack.FlowStack.Pop(EAllowShrinking::No);
		Stack.Code = &Stack.Node->Script[ Offset ];
	}
	else
	{
		UE_LOGF(LogScriptCore, Log, "%ls", *Stack.GetStackTrace());
		Stack.Logf(ELogVerbosity::Error, TEXT("Tried to pop from an empty flow stack"));
	}
}
IMPLEMENT_VM_FUNCTION( EX_PopExecutionFlow, execPopExecutionFlow );

DEFINE_FUNCTION(UObject::execPopExecutionFlowIfNot)
{
	// Since this is a branch function, check for runaway script execution
	CheckRunaway();

	uint8 TempValue = 0;
	Stack.Step(Stack.Object, &TempValue);
	const bool NewBoolValue = (TempValue != 0);

	if (!NewBoolValue)
	{
		// Try to pop an entry off the stack and go there
		if (Stack.FlowStack.Num())
		{
			CodeSkipSizeType Offset = Stack.FlowStack.Pop(EAllowShrinking::No);
			Stack.Code = &Stack.Node->Script[ Offset ];
		}
		else
		{
			UE_LOGF(LogScriptCore, Log, "%ls", *Stack.GetStackTrace());
			Stack.Logf(ELogVerbosity::Error, TEXT("Tried to pop from an empty flow stack"));
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_PopExecutionFlowIfNot, execPopExecutionFlowIfNot );

DEFINE_FUNCTION(UObject::execLetValueOnPersistentFrame)
{
	Stack.MostRecentProperty = nullptr;
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;

	FProperty* DestProperty = Stack.ReadProperty();
	checkSlow(DestProperty);
	UFunction* UberGraphFunction = CastChecked<UFunction>(DestProperty->GetOwnerStruct());
	checkSlow(Stack.Object->GetClass()->IsChildOf(UberGraphFunction->GetOuterUClassUnchecked()));
	uint8* FrameBase = UberGraphFunction->GetOuterUClassUnchecked()->GetPersistentUberGraphFrame(Stack.Object, UberGraphFunction);
	checkSlow(FrameBase);
	uint8* DestAddress = DestProperty->ContainerPtrToValuePtr<uint8>(FrameBase);

	Stack.Step(Stack.Object, DestAddress);
}
IMPLEMENT_VM_FUNCTION(EX_LetValueOnPersistentFrame, execLetValueOnPersistentFrame);

DEFINE_FUNCTION(UObject::execSwitchValue)
{
	const int32 NumCases = Stack.ReadWord();
	const CodeSkipSizeType OffsetToEnd = Stack.ReadCodeSkipCount();

	Stack.MostRecentProperty = nullptr;
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.Step(Stack.Object, nullptr);

	FProperty* IndexProperty = Stack.MostRecentProperty;
	checkSlow(IndexProperty);

	uint8* IndexAdress = Stack.MostRecentPropertyAddress;
	if (!ensure(IndexAdress))
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError, 
			FText::Format(
				LOCTEXT("SwitchValueIndex", "Switch statement failed to read property for index value for index property {0}"),
				FText::FromString(IndexProperty->GetName())
			)
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	bool bProperCaseUsed = false;
	{
		auto LocalTempIndexMem = (uint8*)UE_VSTACK_ALLOC_ALIGNED(Stack.CachedThreadVirtualStackAllocator, IndexProperty->GetSize(), IndexProperty->GetMinAlignment());
		IndexProperty->InitializeValue(LocalTempIndexMem);
		for (int32 CaseIndex = 0; CaseIndex < NumCases; ++CaseIndex)
		{
			Stack.Step(Stack.Object, LocalTempIndexMem); // case index value
			const CodeSkipSizeType OffsetToNextCase = Stack.ReadCodeSkipCount();

			if (IndexAdress && IndexProperty->Identical(IndexAdress, LocalTempIndexMem))
			{
				Stack.Step(Stack.Object, RESULT_PARAM);
				bProperCaseUsed = true;
				break;
			}

			// skip to the next case
			Stack.Code = &Stack.Node->Script[OffsetToNextCase];
		}
		IndexProperty->DestroyValue(LocalTempIndexMem);
	}

	if (bProperCaseUsed)
	{
		Stack.Code = &Stack.Node->Script[OffsetToEnd];
	}
	else
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::NonFatalError, 
			FText::Format(
				LOCTEXT("SwitchValueOutOfBounds", "Switch statement failed to match case for index property {0}"),
				FText::FromString(IndexProperty->GetName())
			)
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);

		// get default value
		Stack.Step(Stack.Object, RESULT_PARAM);
	}
}
IMPLEMENT_VM_FUNCTION(EX_SwitchValue, execSwitchValue);

DEFINE_FUNCTION(UObject::execArrayGetByRef)
{
	// Get variable address.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.Step( Stack.Object, NULL ); // Evaluate variable.

	if (Stack.MostRecentPropertyAddress == NULL)
	{
		static FBlueprintExceptionInfo ExceptionInfo(EBlueprintExceptionType::AccessViolation, LOCTEXT("ArrayGetRefException", "Attempt to assign variable through None"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	void* ArrayAddr = Stack.MostRecentPropertyAddress;
	FArrayProperty* ArrayProperty = ExactCastField<FArrayProperty>(Stack.MostRecentProperty);

 	int32 ArrayIndex;
 	Stack.Step( Stack.Object, &ArrayIndex);
	
	if (ArrayProperty == nullptr)
	{
		Stack.bArrayContextFailed = true;
		return;
	}

	FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayAddr);
	Stack.MostRecentProperty = ArrayProperty->Inner;

	// Add a little safety for Blueprints to not hard crash
	if (ArrayHelper.IsValidIndex(ArrayIndex))
	{
		Stack.MostRecentPropertyAddress = ArrayHelper.GetRawPtr(ArrayIndex);
		Stack.MostRecentPropertyContainer = nullptr;

		if (RESULT_PARAM)
		{
			ArrayProperty->Inner->CopyCompleteValueToScriptVM(RESULT_PARAM, ArrayHelper.GetRawPtr(ArrayIndex));
		}
	}
	else
	{
		// clear so other methods don't try to use a stale value (depends on this method succeeding)
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		// sometimes other exec functions guard on MostRecentProperty, and expect 
		// MostRecentPropertyAddress to be filled out; since this was a failure
		// clear this too (so all reliant execs can properly detect)
		Stack.MostRecentProperty = nullptr;

		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			FText::Format(
			LOCTEXT("ArrayGetOutofBounds", "Attempted to access index {0} from array {1} of length {2}!"),
			FText::AsNumber(ArrayIndex),
			FText::FromString(*ArrayProperty->GetName()),
			FText::AsNumber(ArrayHelper.Num())
			)
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
}
IMPLEMENT_VM_FUNCTION(EX_ArrayGetByRef, execArrayGetByRef);

DEFINE_FUNCTION(UObject::execLet)
{
	Stack.MostRecentProperty = nullptr;
	FProperty* LocallyKnownProperty = Stack.ReadPropertyUnchecked();

	// Get variable address.
	Stack.MostRecentProperty = nullptr;
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.Step(Stack.Object, nullptr); // Evaluate variable.

	uint8* LocalTempResult = nullptr;
	uint8* PreviousPropertyAddress = nullptr;
	uint8* LocalPropertyContainer = Stack.MostRecentPropertyContainer;

	if (Stack.MostRecentPropertyAddress == nullptr)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			LOCTEXT("LetAccessNone", "Attempted to assign to None"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);

		if (LocallyKnownProperty)
		{
			LocalTempResult = (uint8*)UE_VSTACK_ALLOC_ALIGNED(Stack.CachedThreadVirtualStackAllocator, LocallyKnownProperty->GetSize(), LocallyKnownProperty->GetMinAlignment());
			LocallyKnownProperty->InitializeValue(LocalTempResult);
			Stack.MostRecentPropertyAddress = LocalTempResult;
		}
		else
		{
			Stack.MostRecentPropertyAddress = (uint8*)UE_VSTACK_ALLOC(Stack.CachedThreadVirtualStackAllocator, 1024);
			FMemory::Memzero(Stack.MostRecentPropertyAddress, sizeof(FString));
		}
	}
	else if (LocallyKnownProperty && LocallyKnownProperty->HasSetter())
	{
		// We can't assign a value directly to a property if it's got a setter or getter
		LocalTempResult = (uint8*)UE_VSTACK_ALLOC_ALIGNED(Stack.CachedThreadVirtualStackAllocator, LocallyKnownProperty->GetSize(), LocallyKnownProperty->GetMinAlignment());
		LocallyKnownProperty->InitializeValue(LocalTempResult);
		PreviousPropertyAddress = Stack.MostRecentPropertyAddress;
		Stack.MostRecentPropertyAddress = LocalTempResult;
	}

	// Evaluate expression into variable.
	Stack.Step(Stack.Object, Stack.MostRecentPropertyAddress);

	if (LocallyKnownProperty)
	{
		// LocalPropertyContainer will be nullptr if we raised LetAccessNone above
		if (LocallyKnownProperty->HasSetter() && LocalPropertyContainer)
		{
			LocallyKnownProperty->SetValue_InContainer(LocalPropertyContainer, LocalTempResult);
			Stack.MostRecentPropertyAddress = PreviousPropertyAddress;
		}

		if (LocalTempResult)
		{
			LocallyKnownProperty->DestroyValue(LocalTempResult);
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_Let, execLet );

DEFINE_FUNCTION(UObject::execLetObj)
{
	// Get variable address.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.Step( Stack.Object, NULL ); // Evaluate variable.

	if (Stack.MostRecentPropertyAddress == NULL)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			LOCTEXT("LetObjAccessNone", "Accessed None attempting to assign variable on an object"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	void* ObjAddr = Stack.MostRecentPropertyAddress;
	void* PropertyContainer = Stack.MostRecentPropertyContainer;
	FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Stack.MostRecentProperty);
	if (ObjectProperty == NULL)
	{
		FArrayProperty* ArrayProp = ExactCastField<FArrayProperty>(Stack.MostRecentProperty);
		if (ArrayProp != NULL)
		{
			ObjectProperty = CastField<FObjectPropertyBase>(ArrayProp->Inner);
		}
	}

	UObject* NewValue = NULL;
	// evaluate the r-value for this expression into Value
	Stack.Step( Stack.Object, &NewValue );

	if (ObjAddr)
	{
		checkSlow(ObjectProperty);
		if (ObjectProperty->HasSetter())
		{
			check(PropertyContainer != nullptr);
			ObjectProperty->SetValue_InContainer(PropertyContainer, &NewValue);
		}
		else
		{
			ObjectProperty->SetObjectPropertyValue(ObjAddr, NewValue);
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_LetObj, execLetObj );

DEFINE_FUNCTION(UObject::execLetWeakObjPtr)
{
	// Get variable address.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.Step( Stack.Object, NULL ); // Evaluate variable.

	if (Stack.MostRecentPropertyAddress == NULL)
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("LetWeakObjAccessNone", "Accessed None attempting to assign variable on a weakly referenced object"));
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}

	void* ObjAddr = Stack.MostRecentPropertyAddress;
	void* PropertyContainer = Stack.MostRecentPropertyContainer;
	FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Stack.MostRecentProperty);
	if (ObjectProperty == NULL)
	{
		FArrayProperty* ArrayProp = ExactCastField<FArrayProperty>(Stack.MostRecentProperty);
		if (ArrayProp != NULL)
		{
			ObjectProperty = CastField<FObjectPropertyBase>(ArrayProp->Inner);
		}
	}
	
	UObject* NewValue = NULL;
	// evaluate the r-value for this expression into Value
	Stack.Step( Stack.Object, &NewValue );

	if (ObjAddr)
	{
		checkSlow(ObjectProperty);
		if (ObjectProperty->HasSetter())
		{
			check(PropertyContainer != nullptr);
			FWeakObjectPtr NewWeakPtrValue(NewValue);
			ObjectProperty->SetValue_InContainer(PropertyContainer, &NewWeakPtrValue);
		}
		else
		{
			ObjectProperty->SetObjectPropertyValue(ObjAddr, NewValue);
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_LetWeakObjPtr, execLetWeakObjPtr );

DEFINE_FUNCTION(UObject::execLetBool)
{
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;

	// Get the variable and address to place the data.
	Stack.Step( Stack.Object, NULL );

	/*
		Class bool properties are packed together as bitfields, so in order 
		to set the value on the correct bool, we need to mask it against
		the bool property's BitMask.

		Local bool properties (declared inside functions) are not packed, thus
		their bitmask is always 1.

		Bool properties inside dynamic arrays and tmaps are also not packed together.
		If the bool property we're accessing is an element in a dynamic array, Stack.MostRecentProperty
		will be pointing to the dynamic array that has a FBoolProperty as its inner, so
		we'll need to check for that.
	*/
	uint8* BoolAddr = (uint8*)Stack.MostRecentPropertyAddress;
	void* PropertyContainer = Stack.MostRecentPropertyContainer;
	FBoolProperty* BoolProperty = ExactCastField<FBoolProperty>(Stack.MostRecentProperty);
	if (BoolProperty == NULL)
	{
		FArrayProperty* ArrayProp = ExactCastField<FArrayProperty>(Stack.MostRecentProperty);
		if (ArrayProp != NULL)
		{
			BoolProperty = ExactCastField<FBoolProperty>(ArrayProp->Inner);
		}
	}

	// Evaluate the r-value for this expression into Value
	// Note that a bool property can represent bitfields,
	// which is why a uint8 is needed here.

	uint8 TempValue = 0;
	Stack.Step(Stack.Object, &TempValue);

	if (BoolAddr)
	{
		const bool NewBoolValue = (TempValue != 0);

		checkSlow(CastField<FBoolProperty>(BoolProperty));
		if (BoolProperty->HasSetter())
		{
			check(PropertyContainer != nullptr);
			BoolProperty->SetValue_InContainer(PropertyContainer, &NewBoolValue);
		}
		else
		{
			BoolProperty->SetPropertyValue(BoolAddr, NewBoolValue);
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_LetBool, execLetBool );


DEFINE_FUNCTION(UObject::execLetDelegate)
{
	// Get variable address.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, NULL ); // Variable.

	FScriptDelegate* DelegateAddr = (FScriptDelegate*)Stack.MostRecentPropertyAddress;
	FScriptDelegate Delegate;
	Stack.Step( Stack.Object, &Delegate );

	if (DelegateAddr != NULL)
	{
		DelegateAddr->BindUFunction( Delegate.GetUObject(), Delegate.GetFunctionName() );
	}
}
IMPLEMENT_VM_FUNCTION( EX_LetDelegate, execLetDelegate );


DEFINE_FUNCTION(UObject::execLetMulticastDelegate)
{
	// Get variable address.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, NULL ); // Variable.

	FMulticastDelegateProperty* DelegateProp = CastFieldCheckedNullAllowed<FMulticastDelegateProperty>(Stack.MostRecentProperty);
	void* DelegateAddr = Stack.MostRecentPropertyAddress;
	FMulticastScriptDelegate Delegate;
	Stack.Step( Stack.Object, &Delegate );

	if (DelegateProp && DelegateAddr)
	{
		DelegateProp->SetMulticastDelegate(DelegateAddr, MoveTemp(Delegate));
	}
}
IMPLEMENT_VM_FUNCTION( EX_LetMulticastDelegate, execLetMulticastDelegate );


DEFINE_FUNCTION(UObject::execSelf)
{
	// Get Self actor for this context.
	if (RESULT_PARAM != nullptr)
	{
		*(UObject**)RESULT_PARAM = P_THIS;
	}
	// likely it's expecting us to fill out Stack.MostRecentProperty, which you 
	// cannot because 'self' is not a FProperty (it is essentially a constant)
	else 
	{
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation,
			LOCTEXT("AccessSelfAddress", "Attempted to reference 'self' as an addressable property.")
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);
	}
}
IMPLEMENT_VM_FUNCTION( EX_Self, execSelf );

DEFINE_FUNCTION(UObject::execContext)
{
	P_THIS->ProcessContextOpcode(Stack, RESULT_PARAM, /*bCanFailSilently=*/ false);
}
IMPLEMENT_VM_FUNCTION( EX_Context, execContext );

DEFINE_FUNCTION(UObject::execContext_FailSilent)
{
	P_THIS->ProcessContextOpcode(Stack, RESULT_PARAM, /*bCanFailSilently=*/ true);
}
IMPLEMENT_VM_FUNCTION( EX_Context_FailSilent, execContext_FailSilent );

void UObject::ProcessContextOpcode( FFrame& Stack, RESULT_DECL, bool bCanFailSilently )
{
	Stack.MostRecentProperty = NULL;
	
	// Get object variable.
	UObject* NewContext = NULL;
	Stack.Step( this, &NewContext );

	uint8* const OriginalCode = Stack.Code;
	const bool bValidContext = IsValid(NewContext);
	// Execute or skip the following expression in the object's context.
	if (bValidContext)
	{
		Stack.Code += sizeof(CodeSkipSizeType)	// Code offset for NULL expressions.
			+ sizeof(ScriptPointerType);		// Property corresponding to the r-value data, in case the l-value needs to be cleared
		Stack.Step( NewContext, RESULT_PARAM );
	}

	if (!bValidContext || Stack.bArrayContextFailed)
	{
		if (Stack.bArrayContextFailed)
		{
			Stack.bArrayContextFailed = false;
			Stack.Code = OriginalCode;
		}

		if (!bCanFailSilently)
		{
			if (NewContext && !IsValid(NewContext))
			{
				FBlueprintExceptionInfo ExceptionInfo(
					EBlueprintExceptionType::AccessViolation,
					FText::Format(
						LOCTEXT("AccessPendingKill", "Attempted to access {0} via property {1}, but {0} is not valid (pending kill or garbage)"),
						FText::FromString(GetNameSafe(NewContext)),
						FText::FromString(GetNameSafe(Stack.MostRecentProperty))
					)
				);
				FBlueprintCoreDelegates::ThrowScriptException(this, Stack, ExceptionInfo);
			}
			else if (Stack.MostRecentProperty != NULL)
			{
				FBlueprintExceptionInfo ExceptionInfo(
					EBlueprintExceptionType::AccessViolation,
					Stack.MostRecentProperty->GetOwner<UClass>()
					? FText::Format(
						LOCTEXT("AccessNoneUClassContext", "Accessed None trying to read {2} property {0} in {1}"),
						FText::FromString(Stack.MostRecentProperty->GetName()),
						FText::FromString(Stack.MostRecentProperty->GetOwner<UClass>()->GetPathName()),
						FText::FromString(Stack.MostRecentProperty->HasAllPropertyFlags(CPF_Virtual)?"(virtual)":"(real)"))
					: FText::Format(
						LOCTEXT("AccessNoneContext", "Accessed None trying to read {1} property {0} in not an UClass"),
						FText::FromString(Stack.MostRecentProperty->GetName()),
						FText::FromString(Stack.MostRecentProperty->HasAllPropertyFlags(CPF_Virtual) ? "(virtual)" : "(real)"))
				);
				FBlueprintCoreDelegates::ThrowScriptException(this, Stack, ExceptionInfo);
			}
			else
			{
				// Stack.MostRecentProperty will be NULL under the following conditions:
				//   1. the context expression was a function call which returned an object
				//   2. the context expression was a literal object reference
				//   3. the context expression was an instance variable that no longer exists (it was editor-only, etc.)
				FBlueprintExceptionInfo ExceptionInfo(
					EBlueprintExceptionType::AccessViolation,
					LOCTEXT("AccessNoneNoContext", "Accessed None")
				);
				FBlueprintCoreDelegates::ThrowScriptException(this, Stack, ExceptionInfo);
			}
		}

		const CodeSkipSizeType wSkip = Stack.ReadCodeSkipCount(); // Code offset for NULL expressions. Code += sizeof(CodeSkipSizeType)
		FProperty* RValueProperty = nullptr;
		const VariableSizeType bSize = Stack.ReadVariableSize(&RValueProperty); // Code += sizeof(ScriptPointerType) + sizeof(uint8)
		Stack.Code += wSkip;
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.MostRecentProperty = nullptr;

		if (RESULT_PARAM && RValueProperty)
		{
			RValueProperty->ClearValue(RESULT_PARAM);
		}
	}
}

DEFINE_FUNCTION(UObject::execStructMemberContext)
{
	// Get the structure element we care about
	FProperty* StructProperty = Stack.ReadProperty();
	checkSlow(StructProperty);

	// Evaluate an expression leading to the struct.
	Stack.MostRecentProperty = nullptr;
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.Step(Stack.Object, nullptr);

	if (Stack.MostRecentProperty != nullptr)
	{
		// Offset into the specific member
		Stack.MostRecentPropertyContainer = Stack.MostRecentPropertyAddress;
		Stack.MostRecentPropertyAddress = StructProperty->ContainerPtrToValuePtr<uint8>(Stack.MostRecentPropertyAddress);		
		Stack.MostRecentProperty = StructProperty;

		// Handle variable reads
		if (RESULT_PARAM)
		{
			StructProperty->CopyCompleteValueToScriptVM_InContainer(RESULT_PARAM, Stack.MostRecentPropertyContainer);
		}
	}
	else
	{
		// Access none
		FBlueprintExceptionInfo ExceptionInfo(
			EBlueprintExceptionType::AccessViolation, 
			FText::Format(
				LOCTEXT("AccessNoneStructure", "Accessed None reading structure {0}"),
				FText::FromString(StructProperty->GetName())
			)
		);
		FBlueprintCoreDelegates::ThrowScriptException(P_THIS, Stack, ExceptionInfo);

		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.MostRecentProperty = nullptr;
	}
}
IMPLEMENT_VM_FUNCTION( EX_StructMemberContext, execStructMemberContext );

DEFINE_FUNCTION(UObject::execVirtualFunction)
{
	// Call the virtual function.
	P_THIS->CallFunction( Stack, RESULT_PARAM, P_THIS->FindFunctionChecked(Stack.ReadName()) );
}
IMPLEMENT_VM_FUNCTION( EX_VirtualFunction, execVirtualFunction );

DEFINE_FUNCTION(UObject::execFinalFunction)
{
	// Call the final function.
	P_THIS->CallFunction( Stack, RESULT_PARAM, (UFunction*)Stack.ReadObject() );
}
IMPLEMENT_VM_FUNCTION( EX_FinalFunction, execFinalFunction );

DEFINE_FUNCTION(UObject::execLocalVirtualFunction)
{
	// Call the virtual function.
	ProcessLocalFunction(Context, P_THIS->FindFunctionChecked(Stack.ReadName()), Stack, RESULT_PARAM);
}
IMPLEMENT_VM_FUNCTION( EX_LocalVirtualFunction, execLocalVirtualFunction );

DEFINE_FUNCTION(UObject::execLocalFinalFunction)
{
	// Call the final function.
	ProcessLocalFunction(Context, (UFunction*)Stack.ReadObject(), Stack, RESULT_PARAM);
}
IMPLEMENT_VM_FUNCTION( EX_LocalFinalFunction, execLocalFinalFunction );

class FCallDelegateHelper
{
public:
	static void CallMulticastDelegate(FFrame& Stack)
	{
		//Get delegate
		UFunction* SignatureFunction = CastChecked<UFunction>(Stack.ReadObject());
		Stack.MostRecentPropertyAddress = nullptr;
		Stack.MostRecentPropertyContainer = nullptr;
		Stack.MostRecentProperty = nullptr;
		Stack.Step( Stack.Object, nullptr );
		FMulticastDelegateProperty* DelegateProp = CastFieldCheckedNullAllowed<FMulticastDelegateProperty>(Stack.MostRecentProperty);
		const FMulticastScriptDelegate* DelegateAddr = (DelegateProp ? DelegateProp->GetMulticastDelegate(Stack.MostRecentPropertyAddress) : nullptr);

		//Fill parameters
		uint8* Parameters = (uint8*)UE_VSTACK_ALLOC_ALIGNED(Stack.CachedThreadVirtualStackAllocator, SignatureFunction->ParmsSize, SignatureFunction->GetMinAlignment());
		FMemory::Memzero(Parameters, SignatureFunction->ParmsSize);
		for (FProperty* Property = (FProperty*)(SignatureFunction->ChildProperties); *Stack.Code != EX_EndFunctionParms; Property = (FProperty*)Property->Next)
		{
			Stack.MostRecentPropertyAddress = nullptr;
			Stack.MostRecentPropertyContainer = nullptr;
			if (Property->PropertyFlags & CPF_OutParm)
			{
				Stack.Step(Stack.Object, NULL);
				if(NULL != Stack.MostRecentPropertyAddress)
				{
					check(Property->IsInContainer(SignatureFunction->ParmsSize));
					uint8* ConstRefCopyParamAddress = Property->ContainerPtrToValuePtr<uint8>(Parameters);
					Property->CopyCompleteValueToScriptVM(ConstRefCopyParamAddress, Stack.MostRecentPropertyAddress);
				}
			}
			else
			{
				uint8* Param = Property->ContainerPtrToValuePtr<uint8>(Parameters);
				checkSlow(Param);
				Property->InitializeValue_InContainer(Parameters);
				Stack.Step(Stack.Object, Param);
			}
		}
		Stack.Code++;

		//Process delegate
		if (DelegateAddr)
		{
			DelegateAddr->ProcessDelegate<UObject>(Parameters);
		}
		
		//Clean parameters
		for (FProperty* Destruct = SignatureFunction->DestructorLink; Destruct; Destruct = Destruct->DestructorLinkNext)
		{
			Destruct->DestroyValue_InContainer(Parameters);
		}
	}
};

DEFINE_FUNCTION(UObject::execCallMulticastDelegate)
{
	FCallDelegateHelper::CallMulticastDelegate(Stack);
}
IMPLEMENT_VM_FUNCTION( EX_CallMulticastDelegate, execCallMulticastDelegate );

DEFINE_FUNCTION(UObject::execAddMulticastDelegate)
{
	// Get variable address.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, nullptr ); // Variable.

	FMulticastDelegateProperty* DelegateProp = CastFieldCheckedNullAllowed<FMulticastDelegateProperty>(Stack.MostRecentProperty);
	void* DelegateAddr = Stack.MostRecentPropertyAddress;

	FScriptDelegate Delegate;
	Stack.Step( Stack.Object, &Delegate );

	if (DelegateProp && DelegateAddr)
	{
		DelegateProp->AddDelegate(MoveTemp(Delegate), nullptr, DelegateAddr);
	}
}
IMPLEMENT_VM_FUNCTION( EX_AddMulticastDelegate, execAddMulticastDelegate );

DEFINE_FUNCTION(UObject::execRemoveMulticastDelegate)
{
	// Get variable address.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, nullptr ); // Variable.

	FMulticastDelegateProperty* DelegateProp = CastFieldCheckedNullAllowed<FMulticastDelegateProperty>(Stack.MostRecentProperty);
	void* DelegateAddr = Stack.MostRecentPropertyAddress;

	FScriptDelegate Delegate;
	Stack.Step( Stack.Object, &Delegate );

	if (DelegateProp && DelegateAddr)
	{
		DelegateProp->RemoveDelegate(Delegate, nullptr, DelegateAddr);
	}
}
IMPLEMENT_VM_FUNCTION( EX_RemoveMulticastDelegate, execRemoveMulticastDelegate );

DEFINE_FUNCTION(UObject::execClearMulticastDelegate)
{
	// Get the delegate address
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, nullptr );

	FMulticastDelegateProperty* DelegateProp = CastFieldCheckedNullAllowed<FMulticastDelegateProperty>(Stack.MostRecentProperty);
	void* DelegateAddr = Stack.MostRecentPropertyAddress;

	if (DelegateProp && DelegateAddr)
	{
		DelegateProp->ClearDelegate(nullptr, DelegateAddr);
	}
}
IMPLEMENT_VM_FUNCTION( EX_ClearMulticastDelegate, execClearMulticastDelegate );

DEFINE_FUNCTION(UObject::execIntConst)
{
	*(int32*)RESULT_PARAM = Stack.ReadInt<int32>();
}
IMPLEMENT_VM_FUNCTION( EX_IntConst, execIntConst );

DEFINE_FUNCTION(UObject::execInt64Const)
{
	*(int64*)RESULT_PARAM = Stack.ReadInt<int64>();
}
IMPLEMENT_VM_FUNCTION(EX_Int64Const, execInt64Const);

DEFINE_FUNCTION(UObject::execUInt64Const)
{
	*(uint64*)RESULT_PARAM = Stack.ReadInt<uint64>();
}
IMPLEMENT_VM_FUNCTION(EX_UInt64Const, execUInt64Const);

DEFINE_FUNCTION(UObject::execSkipOffsetConst)
{
	CodeSkipSizeType Literal = Stack.ReadCodeSkipCount();
	*(int32*)RESULT_PARAM = Literal;
}
IMPLEMENT_VM_FUNCTION( EX_SkipOffsetConst, execSkipOffsetConst );

DEFINE_FUNCTION(UObject::execFloatConst)
{
	*(float*)RESULT_PARAM = Stack.ReadFloat();
}
IMPLEMENT_VM_FUNCTION( EX_FloatConst, execFloatConst );

// Disable false positive buffer overrun warning during pgoprofile linking step
PGO_LINK_DISABLE_WARNINGS
DEFINE_FUNCTION(UObject::execDoubleConst)
{
	*(double*)RESULT_PARAM = Stack.ReadInt<double>();
}
IMPLEMENT_VM_FUNCTION( EX_DoubleConst, execDoubleConst );
PGO_LINK_ENABLE_WARNINGS

DEFINE_FUNCTION(UObject::execStringConst)
{
	*(FString*)RESULT_PARAM = (ANSICHAR*)Stack.Code;
	while( *Stack.Code )
		Stack.Code++;
	Stack.Code++;
}
IMPLEMENT_VM_FUNCTION( EX_StringConst, execStringConst );

DEFINE_FUNCTION(UObject::execUnicodeStringConst)
{
	FString& ResultStr = *(FString*)RESULT_PARAM;
	ResultStr = FString((UCS2CHAR*)Stack.Code);

	while( *(uint16*)Stack.Code )
	{
		Stack.Code+=sizeof(uint16);
	}
	Stack.Code+=sizeof(uint16);
}
IMPLEMENT_VM_FUNCTION( EX_UnicodeStringConst, execUnicodeStringConst );

DEFINE_FUNCTION(UObject::execTextConst)
{
	// What kind of text are we dealing with?
	const EBlueprintTextLiteralType TextLiteralType = (EBlueprintTextLiteralType)*Stack.Code++;

	switch (TextLiteralType)
	{
	case EBlueprintTextLiteralType::Empty:
		{
			*(FText*)RESULT_PARAM = FText::GetEmpty();
		}
		break;

	case EBlueprintTextLiteralType::LocalizedText:
	case EBlueprintTextLiteralType::LocalizedTextWithNotes:
		{
			FString SourceString;
			Stack.Step(Stack.Object, &SourceString);

			FString KeyString;
			Stack.Step(Stack.Object, &KeyString);
			
			FString Namespace;
			Stack.Step(Stack.Object, &Namespace);

			FString DevNotes;
			if (TextLiteralType == EBlueprintTextLiteralType::LocalizedTextWithNotes)
			{
				Stack.Step(Stack.Object, &DevNotes);
			}
#if WITH_EDITORONLY_DATA
			*(FText*)RESULT_PARAM = FText::AsLocalizable_Advanced(Namespace, KeyString, MoveTemp(SourceString), DevNotes);
#else
			*(FText*)RESULT_PARAM = FText::AsLocalizable_Advanced(Namespace, KeyString, MoveTemp(SourceString));
#endif
		}
		break;

	case EBlueprintTextLiteralType::InvariantText:
		{
			FString SourceString;
			Stack.Step(Stack.Object, &SourceString);

			*(FText*)RESULT_PARAM = FText::AsCultureInvariant(MoveTemp(SourceString));
		}
		break;

	case EBlueprintTextLiteralType::LiteralString:
		{
			FString SourceString;
			Stack.Step(Stack.Object, &SourceString);

			*(FText*)RESULT_PARAM = FText::FromString(MoveTemp(SourceString));
		}
		break;

	case EBlueprintTextLiteralType::StringTableEntry:
		{
			Stack.ReadObject(); // String Table asset (if any)

			FString TableIdString;
			Stack.Step(Stack.Object, &TableIdString);

			FString KeyString;
			Stack.Step(Stack.Object, &KeyString);

			*(FText*)RESULT_PARAM = FText::FromStringTable(FName(*TableIdString), KeyString);
		}
		break;

	default:
		checkf(false, TEXT("Unknown EBlueprintTextLiteralType! Please update UObject::execTextConst to handle this type of text."));
		break;
	}
}
IMPLEMENT_VM_FUNCTION( EX_TextConst, execTextConst );

DEFINE_FUNCTION(UObject::execPropertyConst)
{
	*(FProperty**)RESULT_PARAM = (FProperty*)Stack.ReadPropertyUnchecked();
}
IMPLEMENT_VM_FUNCTION(EX_PropertyConst, execPropertyConst);

DEFINE_FUNCTION(UObject::execObjectConst)
{
	*(UObject**)RESULT_PARAM = (UObject*)Stack.ReadObject();
}
IMPLEMENT_VM_FUNCTION( EX_ObjectConst, execObjectConst );

DEFINE_FUNCTION(UObject::execSoftObjectConst)
{
	FString LongPath;
	Stack.Step(Stack.Object, &LongPath);
	*(FSoftObjectPtr*)RESULT_PARAM = FSoftObjectPath(LongPath);
}
IMPLEMENT_VM_FUNCTION( EX_SoftObjectConst, execSoftObjectConst);

DEFINE_FUNCTION(UObject::execFieldPathConst)
{
	FString StringPath;
	Stack.Step(Stack.Object, &StringPath);
	FFieldPath FieldPath;
	FieldPath.Generate(*StringPath);
	*(FFieldPath*)RESULT_PARAM = FieldPath;
}
IMPLEMENT_VM_FUNCTION(EX_FieldPathConst, execFieldPathConst);

DEFINE_FUNCTION(UObject::execInstanceDelegate)
{
	FName FunctionName = Stack.ReadName();
	((FScriptDelegate*)RESULT_PARAM)->BindUFunction( (FunctionName == NAME_None) ? NULL : P_THIS, FunctionName );
}
IMPLEMENT_VM_FUNCTION( EX_InstanceDelegate, execInstanceDelegate );

DEFINE_FUNCTION(UObject::execBindDelegate)
{
	FName FunctionName = Stack.ReadName();

	// Get delegate address.
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, nullptr ); // Variable.

	FScriptDelegate* DelegateAddr = (FScriptDelegate*)Stack.MostRecentPropertyAddress;

	UObject* ObjectForDelegate = NULL;
	Stack.Step(Stack.Object, &ObjectForDelegate);

	if (DelegateAddr)
	{
		DelegateAddr->BindUFunction(ObjectForDelegate, FunctionName);
	}
}
IMPLEMENT_VM_FUNCTION( EX_BindDelegate, execBindDelegate );

DEFINE_FUNCTION(UObject::execNameConst)
{
	*(FName*)RESULT_PARAM = Stack.ReadName();
}
IMPLEMENT_VM_FUNCTION( EX_NameConst, execNameConst );

DEFINE_FUNCTION(UObject::execByteConst)
{
	*(uint8*)RESULT_PARAM = *Stack.Code++;
}
IMPLEMENT_VM_FUNCTION( EX_ByteConst, execByteConst );

DEFINE_FUNCTION(UObject::execRotationConst)
{
	((FRotator*)RESULT_PARAM)->Pitch = Stack.ReadDouble();
	((FRotator*)RESULT_PARAM)->Yaw   = Stack.ReadDouble();
	((FRotator*)RESULT_PARAM)->Roll  = Stack.ReadDouble();
}
IMPLEMENT_VM_FUNCTION( EX_RotationConst, execRotationConst );

DEFINE_FUNCTION(UObject::execVectorConst)
{
	((FVector*)RESULT_PARAM)->X = Stack.ReadDouble();
	((FVector*)RESULT_PARAM)->Y = Stack.ReadDouble();
	((FVector*)RESULT_PARAM)->Z = Stack.ReadDouble();
}
IMPLEMENT_VM_FUNCTION( EX_VectorConst, execVectorConst );

DEFINE_FUNCTION(UObject::execVector3fConst)
{
	((FVector3f*)RESULT_PARAM)->X = Stack.ReadFloat();
	((FVector3f*)RESULT_PARAM)->Y = Stack.ReadFloat();
	((FVector3f*)RESULT_PARAM)->Z = Stack.ReadFloat();
}
IMPLEMENT_VM_FUNCTION(EX_Vector3fConst, execVector3fConst);

DEFINE_FUNCTION(UObject::execTransformConst)
{
	// Rotation
	FQuat TmpRotation;
	TmpRotation.X = Stack.ReadDouble();
	TmpRotation.Y = Stack.ReadDouble();
	TmpRotation.Z = Stack.ReadDouble();
	TmpRotation.W = Stack.ReadDouble();

	// Translation
	FVector TmpTranslation;
	TmpTranslation.X = Stack.ReadDouble();
	TmpTranslation.Y = Stack.ReadDouble();
	TmpTranslation.Z = Stack.ReadDouble();

	// Scale
	FVector TmpScale;
	TmpScale.X = Stack.ReadDouble();
	TmpScale.Y = Stack.ReadDouble();
	TmpScale.Z = Stack.ReadDouble();

	((FTransform*)RESULT_PARAM)->SetComponents(TmpRotation, TmpTranslation, TmpScale);
}
IMPLEMENT_VM_FUNCTION( EX_TransformConst, execTransformConst );

DEFINE_FUNCTION(UObject::execStructConst)
{
	UScriptStruct* ScriptStruct = CastChecked<UScriptStruct>(Stack.ReadObject());
	int32 SerializedSize = Stack.ReadInt<int32>();
	
	// TODO: Change this once structs/classes can be declared as explicitly editor only
	bool bIsEditorOnlyStruct = false;

	for( FProperty* StructProp = ScriptStruct->PropertyLink; StructProp; StructProp = StructProp->PropertyLinkNext )
	{
		// Skip transient and editor only properties, this needs to be synched with KismetCompilerVMBackend
		if (StructProp->PropertyFlags & CPF_Transient || (!bIsEditorOnlyStruct && StructProp->PropertyFlags & CPF_EditorOnly))
		{
			continue;
		}

		for (int32 ArrayIter = 0; ArrayIter < StructProp->ArrayDim; ++ArrayIter)
		{
			Stack.Step(Stack.Object, StructProp->ContainerPtrToValuePtr<uint8>(RESULT_PARAM, ArrayIter));
		}
	}

	if (ScriptStruct->StructFlags & STRUCT_PostScriptConstruct)
	{
		UScriptStruct::ICppStructOps* TheCppStructOps = ScriptStruct->GetCppStructOps();
		check(TheCppStructOps); // else should not have STRUCT_PostScriptConstruct
		TheCppStructOps->PostScriptConstruct(RESULT_PARAM);
	}

	P_FINISH;	// EX_EndStructConst
}
IMPLEMENT_VM_FUNCTION( EX_StructConst, execStructConst );

DEFINE_FUNCTION(UObject::execSetArray)
{
	// Get the array address
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, nullptr ); // Array to set
	
	FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Stack.MostRecentProperty);
 	FScriptArrayHelper ArrayHelper(ArrayProperty, Stack.MostRecentPropertyAddress);
 	ArrayHelper.EmptyValues();
 
 	// Read in the parameters one at a time
 	int32 i = 0;
 	while(*Stack.Code != EX_EndArray)
 	{
 		ArrayHelper.AddValues(1);
 		Stack.Step(Stack.Object, ArrayHelper.GetRawPtr(i++));
 	}
 
 	P_FINISH;
}
IMPLEMENT_VM_FUNCTION( EX_SetArray, execSetArray );

DEFINE_FUNCTION(UObject::execSetSet)
{
	// Get the set address
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, nullptr ); // Set to set
	const int32 Num = Stack.ReadInt<int32>();

	FSetProperty* SetProperty = CastFieldChecked<FSetProperty>(Stack.MostRecentProperty);
 	FScriptSetHelper SetHelper(SetProperty, Stack.MostRecentPropertyAddress);
 	SetHelper.EmptyElements(Num);
 
	if (Num > 0)
	{
		FDefaultConstructedPropertyElement TempElement(SetProperty->ElementProp);

		// Read in the parameters one at a time
		while (*Stack.Code != EX_EndSet)
		{
			// needs to be an initialized/constructed value, in case the op is a literal that gets assigned over  
			Stack.Step(Stack.Object, TempElement.GetObjAddress());
			SetHelper.AddElement(TempElement.GetObjAddress());
		}
	}
	else
	{
		check(*Stack.Code == EX_EndSet);
	}
 
 	P_FINISH;
}
IMPLEMENT_VM_FUNCTION( EX_SetSet, execSetSet );

DEFINE_FUNCTION(UObject::execSetMap)
{
	// Get the map address
	Stack.MostRecentPropertyAddress = nullptr;
	Stack.MostRecentPropertyContainer = nullptr;
	Stack.MostRecentProperty = nullptr;
	Stack.Step( Stack.Object, nullptr ); // Map to set
	const int32 Num = Stack.ReadInt<int32>();
	
	FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(Stack.MostRecentProperty);
 	FScriptMapHelper MapHelper(MapProperty, Stack.MostRecentPropertyAddress);
 	MapHelper.EmptyValues(Num);
 
	if (Num > 0)
	{
		FDefaultConstructedPropertyElement TempKey(MapProperty->KeyProp);
		FDefaultConstructedPropertyElement TempValue(MapProperty->ValueProp);

		// Read in the parameters one at a time
		while (*Stack.Code != EX_EndMap)
		{
			Stack.Step(Stack.Object, TempKey.GetObjAddress());
			Stack.Step(Stack.Object, TempValue.GetObjAddress());
			MapHelper.AddPair(TempKey.GetObjAddress(), TempValue.GetObjAddress());
		}
	}
	else
	{
		check(*Stack.Code == EX_EndMap);
	}
 
 	P_FINISH;
}
IMPLEMENT_VM_FUNCTION( EX_SetMap, execSetMap );

DEFINE_FUNCTION(UObject::execArrayConst)
{
	FProperty* InnerProperty = CastFieldChecked<FProperty>((FField*)Stack.ReadPropertyUnchecked());
	int32 Num = Stack.ReadInt<int32>();
	check(RESULT_PARAM);
	FScriptArrayHelper ArrayHelper = FScriptArrayHelper::CreateHelperFormInnerProperty(InnerProperty, RESULT_PARAM);
	ArrayHelper.EmptyValues(Num);

	int32 i = 0;
	while (*Stack.Code != EX_EndArrayConst)
	{
		ArrayHelper.AddValues(1);
		Stack.Step(Stack.Object, ArrayHelper.GetRawPtr(i++));
	}
	ensure(i == Num);

	P_FINISH;	// EX_EndArrayConst
}
IMPLEMENT_VM_FUNCTION(EX_ArrayConst, execArrayConst);

DEFINE_FUNCTION(UObject::execSetConst)
{
	FProperty* InnerProperty = CastFieldChecked<FProperty>((FField*)Stack.ReadPropertyUnchecked());
	int32 Num = Stack.ReadInt<int32>();
	check(RESULT_PARAM);

	FScriptSetHelper SetHelper = FScriptSetHelper::CreateHelperFormElementProperty(InnerProperty, RESULT_PARAM);
	SetHelper.EmptyElements(Num);

	while (*Stack.Code != EX_EndSetConst)
	{
		int32 Index = SetHelper.AddDefaultValue_Invalid_NeedsRehash();
		Stack.Step(Stack.Object, SetHelper.GetElementPtr(Index));
	}
	SetHelper.Rehash();

	P_FINISH;	// EX_EndSetConst
}
IMPLEMENT_VM_FUNCTION(EX_SetConst, execSetConst);

DEFINE_FUNCTION(UObject::execMapConst)
{
	FProperty* KeyProperty = CastFieldChecked<FProperty>((FField*)Stack.ReadPropertyUnchecked());
	FProperty* ValProperty = CastFieldChecked<FProperty>((FField*)Stack.ReadPropertyUnchecked());
	int32 Num = Stack.ReadInt<int32>();
	check(RESULT_PARAM);

	FScriptMapHelper MapHelper = FScriptMapHelper::CreateHelperFormInnerProperties(KeyProperty, ValProperty, RESULT_PARAM);
	MapHelper.EmptyValues(Num);

	while (*Stack.Code != EX_EndMapConst)
	{
		int32 Index = MapHelper.AddDefaultValue_Invalid_NeedsRehash();
		Stack.Step(Stack.Object, MapHelper.GetKeyPtr(Index));
		Stack.Step(Stack.Object, MapHelper.GetValuePtr(Index));
	}
	MapHelper.Rehash();

	P_FINISH;	// EX_EndMapConst
}
IMPLEMENT_VM_FUNCTION(EX_MapConst, execMapConst);

DEFINE_FUNCTION(UObject::execBitFieldConst)
{
	FBoolProperty* BitProperty = CastFieldChecked<FBoolProperty>((FField*)Stack.ReadPropertyUnchecked());
	uint8 ByteValue = Stack.Read<uint8>();
	// we could pack the bit into the lower bits of the FProperty pointer, but this instruction is rarely used
	// and it's likely that a simple implementation will be appreciated by readers, debuggers, and even optimizers:
	checkSlow(ByteValue == 0 || ByteValue == 1); 
	BitProperty->SetPropertyValue(RESULT_PARAM, (bool)ByteValue);
}
IMPLEMENT_VM_FUNCTION(EX_BitFieldConst, execBitFieldConst);

DEFINE_FUNCTION(UObject::execIntZero)
{
	*(int32*)RESULT_PARAM = 0;
}
IMPLEMENT_VM_FUNCTION( EX_IntZero, execIntZero );

DEFINE_FUNCTION(UObject::execIntOne)
{
	*(int32*)RESULT_PARAM = 1;
}
IMPLEMENT_VM_FUNCTION( EX_IntOne, execIntOne );

DEFINE_FUNCTION(UObject::execTrue)
{
	*(bool*)RESULT_PARAM = true;
}
IMPLEMENT_VM_FUNCTION( EX_True, execTrue );

DEFINE_FUNCTION(UObject::execFalse)
{
	*(bool*)RESULT_PARAM = false;
}
IMPLEMENT_VM_FUNCTION( EX_False, execFalse );

DEFINE_FUNCTION(UObject::execNoObject)
{
	*(UObject**)RESULT_PARAM = NULL;
}
IMPLEMENT_VM_FUNCTION( EX_NoObject, execNoObject );

DEFINE_FUNCTION(UObject::execNullInterface)
{
	FScriptInterface& InterfaceValue = *(FScriptInterface*)RESULT_PARAM;
	InterfaceValue.SetObject(nullptr);
}
IMPLEMENT_VM_FUNCTION( EX_NoInterface, execNullInterface );

DEFINE_FUNCTION(UObject::execIntConstByte)
{
	*(int32*)RESULT_PARAM = *Stack.Code++;
}
IMPLEMENT_VM_FUNCTION( EX_IntConstByte, execIntConstByte );


DEFINE_FUNCTION(UObject::execDynamicCast)
{
	// Get "to cast to" class for the dynamic actor class
	UClass* ClassPtr = (UClass *)Stack.ReadObject();

	// Compile object expression.
	UObject* Castee = NULL;
	Stack.Step( Stack.Object, &Castee );
	//*(UObject**)RESULT_PARAM = (Castee && Castee->IsA(Class)) ? Castee : NULL;
	*(UObject**)RESULT_PARAM = NULL; // default value

	if (ClassPtr)
	{
		// if we were passed in a null value
		if( Castee == NULL )
		{
			if(ClassPtr->HasAnyClassFlags(CLASS_Interface) )
			{
				((FScriptInterface*)RESULT_PARAM)->SetObject(NULL);
			}
			else
			{
				*(UObject**)RESULT_PARAM = NULL;
			}
			return;
		}

		// check to see if the Castee is an implemented interface by looking up the
		// class hierarchy and seeing if any class in said hierarchy implements the interface
		if(ClassPtr->HasAnyClassFlags(CLASS_Interface) )
		{
			if ( Castee->GetClass()->ImplementsInterface(ClassPtr) )
			{
				// interface property type - convert to FScriptInterface
				((FScriptInterface*)RESULT_PARAM)->SetObject(Castee);
				((FScriptInterface*)RESULT_PARAM)->SetInterface(Castee->GetInterfaceAddress(ClassPtr));
			}
		}
		// check to see if the Castee is a castable class
		else if( Castee->IsA(ClassPtr) )
		{
			*(UObject**)RESULT_PARAM = Castee;
		}
	}
}
IMPLEMENT_VM_FUNCTION( EX_DynamicCast, execDynamicCast );

DEFINE_FUNCTION(UObject::execMetaCast)
{
	UClass* MetaClass = (UClass*)Stack.ReadObject();

	// Compile actor expression.
	UObject* Castee = nullptr;
	Stack.Step( Stack.Object, &Castee );
	UClass* CasteeClass = dynamic_cast<UClass*>(Castee);
	*(UObject**)RESULT_PARAM = (CasteeClass && CasteeClass->IsChildOf(MetaClass)) ? Castee : nullptr;
}
IMPLEMENT_VM_FUNCTION( EX_MetaCast, execMetaCast );

DEFINE_FUNCTION(UObject::execCast)
{
	int32 B = *(Stack.Code)++;
	(*GCasts[B])( Stack.Object, Stack, RESULT_PARAM );
}
IMPLEMENT_VM_FUNCTION( EX_Cast, execCast );

DEFINE_FUNCTION(UObject::execInterfaceCast)
{
	(*GCasts[CST_ObjectToInterface])(Stack.Object, Stack, RESULT_PARAM);
}
IMPLEMENT_VM_FUNCTION( EX_ObjToInterfaceCast, execInterfaceCast );

DEFINE_FUNCTION(UObject::execDoubleToFloatCast)
{
	if (Stack.StepAndCheckMostRecentProperty(Stack.Object, nullptr))
	{
		const double* Source = reinterpret_cast<const double*>(Stack.MostRecentPropertyAddress);
		float* Destination = reinterpret_cast<float*>(RESULT_PARAM);

		*Destination = static_cast<float>(*Source);
	}
	else
	{
		UE_LOGF(LogScript, Verbose, "Cast failed: recent properties were null!");
	}
}
IMPLEMENT_CAST_FUNCTION( CST_DoubleToFloat, execDoubleToFloatCast )

DEFINE_FUNCTION(UObject::execFloatToDoubleCast)
{
	if (Stack.StepAndCheckMostRecentProperty(Stack.Object, nullptr))
	{
		const float* Source = reinterpret_cast<const float*>(Stack.MostRecentPropertyAddress);
		double* Destination = reinterpret_cast<double*>(RESULT_PARAM);

		*Destination = *Source;
	}
	else
	{
		UE_LOGF(LogScript, Verbose, "Cast failed: recent properties were null!");
	}
}
IMPLEMENT_CAST_FUNCTION( CST_FloatToDouble, execFloatToDoubleCast )

DEFINE_FUNCTION(UObject::execObjectToBool)
{
	UObject* Obj=NULL;
	Stack.Step( Stack.Object, &Obj );
	*(bool*)RESULT_PARAM = Obj != NULL;
}
IMPLEMENT_CAST_FUNCTION( CST_ObjectToBool, execObjectToBool );

DEFINE_FUNCTION(UObject::execInterfaceToBool)
{
	FScriptInterface Interface;
	Stack.Step( Stack.Object, &Interface);
	*(bool*)RESULT_PARAM = (Interface.GetObject() != NULL);
}
IMPLEMENT_CAST_FUNCTION( CST_InterfaceToBool, execInterfaceToBool );

DEFINE_FUNCTION(UObject::execObjectToInterface)
{
	FScriptInterface& InterfaceValue = *(FScriptInterface*)RESULT_PARAM;

	// read the interface class off the stack
	UClass* InterfaceClass = dynamic_cast<UClass*>(Stack.ReadObject());
	checkSlow(InterfaceClass != NULL);

	// read the object off the stack
	UObject* ObjectValue = NULL;
	Stack.Step( Stack.Object, &ObjectValue );

	if ( ObjectValue && ObjectValue->GetClass()->ImplementsInterface(InterfaceClass) )
	{
		InterfaceValue.SetObject(ObjectValue);

		void* IAddress = ObjectValue->GetInterfaceAddress(InterfaceClass);
		InterfaceValue.SetInterface(IAddress);
	}
	else
	{
		InterfaceValue.SetObject(NULL);
	}
}
IMPLEMENT_CAST_FUNCTION( CST_ObjectToInterface, execObjectToInterface );

DEFINE_FUNCTION(UObject::execInterfaceToInterface)
{
	FScriptInterface& CastResult = *(FScriptInterface*)RESULT_PARAM;

	// read the interface class off the stack
	UClass* ClassToCastTo = dynamic_cast<UClass*>(Stack.ReadObject());
	checkSlow(ClassToCastTo != NULL);
	checkSlow(ClassToCastTo->HasAnyClassFlags(CLASS_Interface));

	// read the input interface-object off the stack
	FScriptInterface InterfaceInput;
	Stack.Step(Stack.Object, &InterfaceInput);

	UObject* ObjectWithInterface = InterfaceInput.GetObjectRef();
	if ((ObjectWithInterface != NULL) && ObjectWithInterface->GetClass()->ImplementsInterface(ClassToCastTo))
	{
		CastResult.SetObject(ObjectWithInterface);

		void* IAddress = ObjectWithInterface->GetInterfaceAddress(ClassToCastTo);
		CastResult.SetInterface(IAddress);
	}
	else
	{
 		CastResult.SetObject(NULL);
 	}
}
IMPLEMENT_VM_FUNCTION( EX_CrossInterfaceCast, execInterfaceToInterface );

DEFINE_FUNCTION(UObject::execInterfaceToObject)
{
	// read the interface class off the stack
	UClass* ObjClassToCastTo = dynamic_cast<UClass*>(Stack.ReadObject());
	checkSlow(ObjClassToCastTo != nullptr);

	// read the input interface-object off the stack
	FScriptInterface InterfaceInput;
	Stack.Step(Stack.Object, &InterfaceInput);

	UObject* InputObjWithInterface = InterfaceInput.GetObjectRef();
	if (InputObjWithInterface && InputObjWithInterface->IsA(ObjClassToCastTo))
	{
		*(UObject**)RESULT_PARAM = InputObjWithInterface;
	}
	else
	{
		*(UObject**)RESULT_PARAM = nullptr;
	}
}
IMPLEMENT_VM_FUNCTION( EX_InterfaceToObjCast, execInterfaceToObject );

DEFINE_FUNCTION(UObject::execAutoRtfmTransact)
{
	int32 TransactionId = Stack.ReadInt<int32>();
	CodeSkipSizeType JumpOffset = Stack.ReadCodeSkipCount();

	uint8* JumpTarget = &Stack.Node->Script[JumpOffset];

	// sometimes if this inner transaction commits, we want to
	// abort the parent transaction afterwards (logical not does this)
	bool bAbortParentOnCommit = false;

	// now wrap the next step around a transaction
	AutoRTFM::ETransactionResult Result = AutoRTFM::Transact([&]()
	{
		bool bKeepRunning = true;
		for (int i = 0; bKeepRunning; i++)
		{
			if(*Stack.Code == EX_AutoRtfmStopTransact)
			{
				Stack.Code++;
				int32 Value = Stack.ReadInt<int32>();
				EAutoRtfmStopTransactMode Mode = Stack.Read<EAutoRtfmStopTransactMode>();

				if (TransactionId == Value)
				{
					switch(Mode)
					{
					case EAutoRtfmStopTransactMode::GracefulExit:
						// gracefully terminate this transaction
						bKeepRunning = false;
						break;

					case EAutoRtfmStopTransactMode::AbortingExit:
						// abort this transaction
						AutoRTFM::AbortTransaction();
						break;

					case EAutoRtfmStopTransactMode::AbortingExitAndAbortParent:
						// abort this transaction and also abort the parent
						UE_AUTORTFM_OPEN{ bAbortParentOnCommit = true; };
						AutoRTFM::AbortTransaction();
						break;
					}
				}
			}
			else
			{
				Stack.Step(Stack.Object, RESULT_PARAM);
			}
		}
	});

	P_NATIVE_BEGIN;

	if (UNLIKELY(Result == AutoRTFM::ETransactionResult::AbortedByLanguage))
	{
		FBlueprintExceptionInfo AbortedByLanguage(
			EBlueprintExceptionType::FatalError,
			LOCTEXT("AbortedByLanguage", "AutoRTFM aborted because of unhandled constructs in the code (atomics, unhandled function calls, etc)")
		);

		FBlueprintCoreDelegates::ThrowScriptException(Context, Stack, AbortedByLanguage);

		if (AutoRTFM::IsTransactional())
		{
			AutoRTFM::CascadingAbortTransaction();
		}
	}

	if (Result != AutoRTFM::ETransactionResult::Committed)
	{
		// if this transaction didn't commit, move our code pointer to the target
		Stack.Code = JumpTarget;
	}

	if (bAbortParentOnCommit)
	{
		AutoRTFM::AbortTransaction();
	}

	P_NATIVE_END;
}
IMPLEMENT_VM_FUNCTION( EX_AutoRtfmTransact, execAutoRtfmTransact );

DEFINE_FUNCTION(UObject::execAutoRtfmStopTransact)
{
	Stack.ReadInt<int32>();
	Stack.Read<EAutoRtfmStopTransactMode>();

	// if we were in a transaction, the processing loop inside execAutoRtfmTransact
	// would handle this opcode specially. if we are handling it here then we
	// are not in a transaction, so this opcode is a no-op
}
IMPLEMENT_VM_FUNCTION( EX_AutoRtfmStopTransact, execAutoRtfmStopTransact );

DEFINE_FUNCTION(UObject::execAutoRtfmAbortIfNot)
{
	// evaluate a boolean expression
	uint8 TempValue = 0;
	Stack.Step(Stack.Object, &TempValue);
	const bool NewBoolValue = (TempValue != 0);

	// if the expression returned false, abort the current transaction
	if (!NewBoolValue)
	{
		AutoRTFM::AbortTransaction();
	}
}
IMPLEMENT_VM_FUNCTION( EX_AutoRtfmAbortIfNot, execAutoRtfmAbortIfNot );

DEFINE_FUNCTION(UObject::execAutoRtfmAbort)
{
	AutoRTFM::AbortTransaction();
}
IMPLEMENT_VM_FUNCTION(EX_AutoRtfmAbort, execAutoRtfmAbort);

#undef LOCTEXT_NAMESPACE
