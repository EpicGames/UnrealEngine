// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ScriptStackTracker.cpp: Stack Tracking within Unreal Engine.
=============================================================================*/
#include "UObject/ScriptStackTracker.h"
#include "UObject/Stack.h"


DEFINE_LOG_CATEGORY_STATIC(LogScriptStackTracker, Log, All);

/**
* Captures the current stack and updates stack tracking information.
*/
void FScriptStackTracker::CaptureStackTrace(const FFrame* StackFrame, int32 EntriesToIgnore /*=0*/)
{
	// Avoid re-rentrancy as the code uses TArray/TMap.
	if( !bAvoidCapturing && bIsEnabled )
	{
		// Scoped true/ false.
		bAvoidCapturing = true;

		// Capture callstack and create CRC.

		FString StackTrace = StackFrame->GetStackTrace();
		uint32 CRC = FCrc::MemCrc_DEPRECATED( *StackTrace, StackTrace.Len() );

		// Use index if found
		int32* IndexPtr = CRCToCallStackIndexMap.Find( CRC );
		if( IndexPtr )
		{
			// Increase stack count for existing callstack.
			CallStacks[*IndexPtr].StackCount++;
		}
		// Encountered new call stack, add to array and set index mapping.
		else
		{
			FCallStack NewCallStack;
			NewCallStack.StackCount = 1;
			NewCallStack.StackTrace = StackTrace;
			int32 Index = CallStacks.Add(NewCallStack);
			CRCToCallStackIndexMap.Add( CRC, Index );
		}

		// We're done capturing.
		bAvoidCapturing = false;
	}
}

/**
* Dumps capture stack trace summary to the passed in log.
*/
void FScriptStackTracker::DumpStackTraces( int32 StackThreshold, FOutputDevice& Ar )
{
	// Avoid distorting results while we log them.
	check( !bAvoidCapturing );
	bAvoidCapturing = true;

	// Make a copy as sorting causes index mismatch with TMap otherwise.
	TArray<FCallStack> SortedCallStacks = CallStacks;
	// Compare function, sorting callstack by stack count in descending order.
	struct FCompareStackCount
	{
		FORCEINLINE bool operator()( const FCallStack& A, const FCallStack& B ) const { return B.StackCount < A.StackCount; }
	};
	// Sort callstacks in descending order by stack count.
	SortedCallStacks.Sort( FCompareStackCount() );

	// Iterate over each callstack to get total stack count.
	uint64 TotalStackCount = 0;
	for( int32 CallStackIndex=0; CallStackIndex<SortedCallStacks.Num(); CallStackIndex++ )
	{
		const FCallStack& CallStack = SortedCallStacks[CallStackIndex];
		TotalStackCount += CallStack.StackCount;
	}

	// Calculate the number of frames we captured.
	uint64 FramesCaptured = 0;
	if( bIsEnabled )
	{
		FramesCaptured = GFrameCounter - StartFrameCounter;
	}
	else
	{
		FramesCaptured = StopFrameCounter - StartFrameCounter;
	}

	// Log quick summary as we don't log each individual so totals in CSV won't represent real totals.
	Ar.Logf(TEXT("Captured %i unique callstacks totalling %" UINT64_FMT " function calls over %" UINT64_FMT " frames, averaging %5.2f calls/frame"), 
		SortedCallStacks.Num(), TotalStackCount, FramesCaptured, (float)TotalStackCount / (float)FramesCaptured);

	// Iterate over each callstack and write out info in human readable form in CSV format
	for( int32 CallStackIndex=0; CallStackIndex<SortedCallStacks.Num(); CallStackIndex++ )
	{
		const FCallStack& CallStack = SortedCallStacks[CallStackIndex];

		// Avoid log spam by only logging above threshold.
		if( CallStack.StackCount > StackThreshold )
		{
			// First row is stack count.
			FString CallStackString = FString::FromInt((int32)CallStack.StackCount);
			CallStackString += LINE_TERMINATOR;
			CallStackString += CallStack.StackTrace;

			// Finally log with ',' prefix so "Log:" can easily be discarded as row in Excel.
			Ar.Logf(TEXT(",%s"),*CallStackString);
		}
	}

	// Done logging.
	bAvoidCapturing = false;
}

/** Resets stack tracking. */
void FScriptStackTracker::ResetTracking()
{
	check(!bAvoidCapturing);
	CRCToCallStackIndexMap.Empty();
	CallStacks.Empty();

	// Reset the markers
	StartFrameCounter = GFrameCounter;
	StopFrameCounter = GFrameCounter;
}

/** Toggles tracking. */
void FScriptStackTracker::ToggleTracking()
{
	bIsEnabled = !bIsEnabled;
	// Enabled
	if( bIsEnabled )
	{
		UE_LOGF(LogScriptStackTracker, Log, "Script stack tracking is now enabled.");
		StartFrameCounter = GFrameCounter;
	}
	// Disabled.
	else
	{
		StopFrameCounter = GFrameCounter;
		UE_LOGF(LogScriptStackTracker, Log, "Script stack tracking is now disabled.");
	}
}
