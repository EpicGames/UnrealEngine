// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/ScriptTimeLimiter.h"
#include "CoreMinimal.h"
#include "HAL/PlatformTime.h"
#include "VerseVM/VVMVerseHangDetection.h"

UE::FScriptTimeLimiter& UE::FScriptTimeLimiter::Get()
{
	thread_local FScriptTimeLimiter TimeLimiterSingleton;
	return TimeLimiterSingleton;
}

void UE::FScriptTimeLimiter::StartTimer()
{
	check(IsInGameThread());

	if (NestingDepth == 0)
	{
		StartingTime = FPlatformTime::Seconds();
		TotalOverheadTime = 0.0;

#if WITH_VERSE_BPVM
		VerseHangDetection::InstallOverheadTracking(
			/* OnOverheadBegin */ nullptr,
			/* OnOverheadEnd   */ [this](double Duration)
			{
				// Adjust overhead time based on time spent in this overhead scope.
				AddOverheadTime(Duration);
			}
		);
#endif // WITH_VERSE_BPVM
	}

	NestingDepth++;
}

void UE::FScriptTimeLimiter::StopTimer()
{
	check(IsInGameThread());

	NestingDepth--;
	check(NestingDepth >= 0);

	if (NestingDepth == 0)
	{
#if WITH_VERSE_BPVM
		VerseHangDetection::UninstallOverheadTracking();
#endif // WITH_VERSE_BPVM

		StartingTime = 0.0;
		TotalOverheadTime = 0.0;
	}
}

void UE::FScriptTimeLimiter::AddOverheadTime(double Duration)
{
	TotalOverheadTime += Duration;
}

bool UE::FScriptTimeLimiter::HasExceededTimeLimit()
{
	// If no time limit is specified, we default to the Verse hang threshold.
	return HasExceededTimeLimit(VerseHangDetection::VerseHangThreshold());
}

bool UE::FScriptTimeLimiter::HasExceededTimeLimit(double TimeLimit)
{
	// Account for overhead time by adjusting the time limit accordingly.
	const double AdjustedTimeLimit = VerseHangDetection::ShouldExcludeOverhead() ? TimeLimit + TotalOverheadTime : TimeLimit;

	// Note that `IsComputationLimitExceeded` with a StartingTime of 0.0 will always return false.
	return VerseHangDetection::IsComputationLimitExceeded(StartingTime, AdjustedTimeLimit);
}
