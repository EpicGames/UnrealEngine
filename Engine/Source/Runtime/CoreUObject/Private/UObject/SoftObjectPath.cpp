// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/SoftObjectPath.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/UnrealType.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "Misc/AsciiSet.h"
#include "Misc/PackageName.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/LinkerLoad.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/CoreRedirects.h"
#include "Misc/RedirectCollector.h"
#include "Misc/AutomationTest.h"
#include "String/Find.h"
#include "AutoRTFM.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SoftObjectPath)

namespace SoftObjectPath
{
static bool bResolveCoreRedirects = true;

#if !WITH_EDITOR
static FAutoConsoleVariableRef CVarEnablePathFixupOutsideEditor(TEXT("SoftObjectPath.EnablePathFixupOutsideEditor"),
	bResolveCoreRedirects,
	TEXT("When true (by default) we will call FixupCoreRedirects when resolving, loading, or saving soft object paths outside the editor. When false, we will revert to the legacy behavior and not do the extra fixup.")
);
#endif
}

static FTransactionallySafeRWLock PIEPackageNamesLock;

#if WITH_EDITOR
FName FSoftObjectPath::NAME_Untracked(TEXT("Untracked"));
#endif

/** Static methods for more meaningful construction sites. */
FSoftObjectPath FSoftObjectPath::ConstructFromAssetPathAndSubpath(FTopLevelAssetPath InAssetPath, TStringOverload<FWideString> InSubPathString)
{
	return FSoftObjectPath(InAssetPath, FUtf8String(InSubPathString.MoveTemp()));
}

FSoftObjectPath FSoftObjectPath::ConstructFromAssetPathAndSubpath(FTopLevelAssetPath InAssetPath, TStringOverload<FUtf8String> InSubPathString)
{
	return FSoftObjectPath(InAssetPath, InSubPathString.MoveTemp());
}

FSoftObjectPath FSoftObjectPath::ConstructFromAssetPath(FTopLevelAssetPath InAssetPath)
{
	return FSoftObjectPath(InAssetPath);
}

FSoftObjectPath FSoftObjectPath::ConstructFromStringPath(FString&& InPath)
{
	FSoftObjectPath Tmp;
	Tmp.SetPath(FStringView(InPath));
	return Tmp;
}

FSoftObjectPath FSoftObjectPath::ConstructFromStringPath(FStringView InPath)
{
	FSoftObjectPath Tmp;
	Tmp.SetPath(InPath);
	return Tmp;
}

FSoftObjectPath FSoftObjectPath::ConstructFromStringPath(FUtf8StringView InPath)
{
	FSoftObjectPath Tmp;
	Tmp.SetPath(InPath);
	return Tmp;
}

FSoftObjectPath FSoftObjectPath::ConstructFromObject(const FObjectPtr& InObject)
{
	return FSoftObjectPath(InObject);
}

FSoftObjectPath FSoftObjectPath::ConstructFromObject(const UObject* InObject)
{
	return FSoftObjectPath(InObject);
}


FString FSoftObjectPath::ToString() const
{
	// Most of the time there is no sub path so we can do a single string allocation
	if (SubPathString.IsEmpty())
	{
		return GetAssetPathString();
	}

	TStringBuilder<FName::StringBufferSize> Builder;
	Builder << AssetPath << SUBOBJECT_DELIMITER_CHAR << SubPathString;
	return Builder.ToString();
}

void FSoftObjectPath::ToString(FStringBuilderBase& Builder) const
{
	AppendString(Builder);
}

void FSoftObjectPath::ToString(FUtf8StringBuilderBase& Builder) const
{
	AppendString(Builder);
}

void FSoftObjectPath::AppendString(FStringBuilderBase& Builder) const
{
	if (AssetPath.IsNull())
	{
		return;
	}

	Builder << AssetPath;
	if (SubPathString.Len() > 0)
	{
		Builder << SUBOBJECT_DELIMITER_CHAR << SubPathString;
	}
}

void FSoftObjectPath::AppendString(FUtf8StringBuilderBase& Builder) const
{
	if (AssetPath.IsNull())
	{
		return;
	}

	Builder << AssetPath;
	if (SubPathString.Len() > 0)
	{
		Builder << SUBOBJECT_DELIMITER_CHAR_ANSI << SubPathString;
	}
}

void FSoftObjectPath::AppendString(FString& Builder) const
{
	if (AssetPath.IsNull())
	{
		return;
	}

	AssetPath.AppendString(Builder);
	if (SubPathString.Len() > 0)
	{
		Builder += SUBOBJECT_DELIMITER_CHAR;
		Builder += SubPathString;
	}
}

/** Helper function that adds info about the object currently being serialized when triggering an ensure about invalid soft object path */
static FString GetObjectBeingSerializedForSoftObjectPath()
{
	FString Result;
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	if (SerializeContext && SerializeContext->SerializedObject)
	{
		Result = FString::Printf(TEXT(" while serializing %s"), *SerializeContext->SerializedObject->GetFullName());
	}
	return Result;
}

void FSoftObjectPath::SetPath(const FTopLevelAssetPath& InAssetPath)
{
	AssetPath = InAssetPath;
	SubPathString.Empty();
}

void FSoftObjectPath::SetPath(const FTopLevelAssetPath& InAssetPath, TStringOverload<FWideString> InSubPathString)
{
	AssetPath = InAssetPath;
	SubPathString = FUtf8String(InSubPathString.MoveTemp());
}

void FSoftObjectPath::SetPath(const FTopLevelAssetPath& InAssetPath, TStringOverload<FUtf8String> InSubPathString)
{
	AssetPath = InAssetPath;
	SubPathString = InSubPathString.MoveTemp();
}

void FSoftObjectPath::SetPath(FWideStringView Path)
{
	if (Path.IsEmpty() || Path.Equals(TEXT("None"), ESearchCase::CaseSensitive))
	{
		// Empty path, just empty the pathname.
		Reset();
	}
	else 
	{
		// Possibly an ExportText path. Trim the ClassName.
		Path = FPackageName::ExportTextPathToObjectPath(Path);

		constexpr FAsciiSet Delimiters = FAsciiSet(".") + (char)SUBOBJECT_DELIMITER_CHAR;
		if (  !Path.StartsWith('/')  // Must start with a package path 
			|| Delimiters.Contains(Path[Path.Len() - 1]) // Must not end with a trailing delimiter
		)
		{
			// Not a recognized path. No ensure/logging here because many things attempt to construct paths from user input. 
			Reset();
			return;
		}

		
		// Reject paths that contain two consecutive delimiters in any position 
		for (int32 i=2; i < Path.Len(); ++i) // Start by comparing index 2 and index 1 because index 0 is known to be '/'
		{
			if (Delimiters.Contains(Path[i]) && Delimiters.Contains(Path[i-1]))
			{
				Reset();
				return;
			}
		}

		FWideStringView PackageNameView = FAsciiSet::FindPrefixWithout(Path, Delimiters);
		if (PackageNameView.Len() == Path.Len())
		{
			// No delimiter, package name only
			AssetPath = FTopLevelAssetPath(FName(PackageNameView), FName());
			SubPathString.Empty();
			return;
		}

		Path.RightChopInline(PackageNameView.Len() + 1);
		check(!Path.IsEmpty() && !Delimiters.Contains(Path[0])); // Sanitized to avoid trailing delimiter or consecutive delimiters above

		FWideStringView AssetNameView = FAsciiSet::FindPrefixWithout(Path, Delimiters);
		if (AssetNameView.Len() == Path.Len())
		{
			// No subobject path
			AssetPath = FTopLevelAssetPath(FName(PackageNameView), FName(AssetNameView));
			SubPathString.Empty();
			return;
		}

		Path.RightChopInline(AssetNameView.Len() + 1);
		check(!Path.IsEmpty() && !Delimiters.Contains(Path[0])); // Sanitized to avoid trailing delimiter or consecutive delimiters above

		// Replace delimiters in subpath string with . to normalize
		AssetPath = FTopLevelAssetPath(FName(PackageNameView), FName(AssetNameView));
		SubPathString = UE_PRIVATE_TO_UTF8_STRING(Path);
		SubPathString.ReplaceCharInline(UTF8TEXT(SUBOBJECT_DELIMITER_CHAR_ANSI), UTF8TEXT('.'));
	}
}

void FSoftObjectPath::SetPath(FAnsiStringView Path)
{
	TStringBuilder<256> Wide;
	Wide << Path;
	SetPath(Wide);
}

void FSoftObjectPath::SetPath(FUtf8StringView Path)
{
	TStringBuilder<256> Wide;
	Wide << Path;
	SetPath(Wide);
}

void FSoftObjectPath::SetPath(const UObject* InObject)
{
	if (!InObject)
	{
		Reset();
		return;
	}

	const UObject* ObjectOuter = InObject->GetOuter();

	// Fast path: InObject is a package
	if (!ObjectOuter)
	{
		AssetPath.TrySetPath(CastChecked<UPackage>(InObject)->GetFName(), NAME_None);
		SubPathString.Empty();
		return;
	}
	
	// Fast path: InObject is a top-level asset
	if (!ObjectOuter->GetOuter())
	{
		AssetPath.TrySetPath(CastChecked<UPackage>(ObjectOuter)->GetFName(), InObject->GetFName());
		SubPathString.Empty();
		return;
	}

	// Slow path: walk the outer chain and construct the soft object path parts
	TArray<FName, TInlineAllocator<64>> SubObjectPath;

	const UObject* CurrentObject = InObject;
	const UObject* CurrentObjectOuter = ObjectOuter;

	while (CurrentObjectOuter->GetOuter())
	{
		SubObjectPath.Push(CurrentObject->GetFName());
		CurrentObject = CurrentObjectOuter;
		CurrentObjectOuter = CurrentObject->GetOuter();
	}

	AssetPath.TrySetPath(CastChecked<UPackage>(CurrentObjectOuter)->GetFName(), CurrentObject->GetFName());

	TStringBuilder<1024> SubPathStringBuilder;

	while (!SubObjectPath.IsEmpty())
	{
		const FName ObjectName = SubObjectPath.Pop(EAllowShrinking::No);
		ObjectName.AppendString(SubPathStringBuilder);
		SubPathStringBuilder.AppendChar(TEXT('.'));
	}

	SubPathStringBuilder.RemoveSuffix(1);

	SubPathString = SubPathStringBuilder.ToString();
}

bool FSoftObjectPath::PreSavePath(bool* bReportSoftObjectPathRedirects)
{
#if WITH_EDITOR
	if (IsNull())
	{
		return false;
	}

	FSoftObjectPath FoundRedirection = GRedirectCollector.GetAssetPathRedirection(*this);

	if (!FoundRedirection.IsNull())
	{
		if (*this != FoundRedirection && bReportSoftObjectPathRedirects)
		{
			*bReportSoftObjectPathRedirects = true;
		}
		*this = FoundRedirection;
		return true;
	}
#endif

	if (SoftObjectPath::bResolveCoreRedirects)
	{
		if (FixupCoreRedirects())
		{
			return true;
		}
	}
	return false;
}

void FSoftObjectPath::PostLoadPath(FArchive* InArchive) const
{
#if WITH_EDITOR
	GRedirectCollector.OnSoftObjectPathLoaded(*this, InArchive);
#endif // WITH_EDITOR
}

bool FSoftObjectPath::Serialize(FArchive& Ar)
{
	// Archivers will call back into SerializePath for the various fixups
	Ar << *this;

	return true;
}

bool FSoftObjectPath::Serialize(FStructuredArchive::FSlot Slot)
{
	// Archivers will call back into SerializePath for the various fixups
	Slot << *this;

	return true;
}

void FSoftObjectPath::SerializePath(FArchive& Ar)
{
	bool bSerializeInternals = true;
#if WITH_EDITOR
	if (Ar.IsSaving() && !(Ar.IsModifyingWeakAndStrongReferences() && Ar.IsObjectReferenceCollector()))
	{
		PreSavePath(nullptr);
	}

	// Only read serialization options in editor as it is a bit slow
	FName PackageName, PropertyName;
	ESoftObjectPathCollectType CollectType = ESoftObjectPathCollectType::AlwaysCollect;
	ESoftObjectPathSerializeType SerializeType = ESoftObjectPathSerializeType::AlwaysSerialize;

	FSoftObjectPathThreadContext& ThreadContext = FSoftObjectPathThreadContext::Get();
	ThreadContext.GetSerializationOptions(PackageName, PropertyName, CollectType, SerializeType, &Ar);

	if (SerializeType == ESoftObjectPathSerializeType::NeverSerialize)
	{
		bSerializeInternals = false;
	}
	else if (SerializeType == ESoftObjectPathSerializeType::SkipSerializeIfArchiveHasSize)
	{
		bSerializeInternals = Ar.IsObjectReferenceCollector() || Ar.Tell() < 0;
	}
#endif // WITH_EDITOR

	if (bSerializeInternals)
	{
		SerializePathWithoutFixup(Ar);
	}

#if WITH_EDITOR
	if (Ar.IsLoading())
	{
		if (Ar.IsPersistent())
		{
			PostLoadPath(&Ar);

			// If we think it's going to work, we try to do the pre-save fixup now. This is important because it helps
			// with blueprint CDO save determinism with redirectors. It's important that the entire CDO hierarchy gets
			// fixed up before an instance in a map gets saved otherwise the delta serialization will save too much.
			// 
			// We need to not do this for SoftObjectPaths loaded in AssetBundle tags, because those are loaded before
			// the AssetRegistry finishes gathering all of the redirectors, so the results would be indeterministic.
			// We process those paths for redirectors instead at the end of the AssetRegistry.
			// We detect whether the SoftObjectPaths came from the AssetRegistry by checking for IsPackageType(CollectType)
			// Other non-package sources will also be skipped, but that's okay; we only need to do the PreSavePath for
			// SoftObjectPaths that are in Blueprints.
			//
			// For paths in blueprints, the handling of redirects is only needed for editor (game does not save blueprints),
			// and also even in editor we know the information is not yet ready during engine startup, so skip the PreSave
			// for !GIsEditor || GIsInitialLoad.
			if (GIsEditor && !GIsInitialLoad && UE::SoftObjectPath::IsPackageType(CollectType))
			{
				PreSavePath(nullptr);
			}
		}
		if (Ar.GetPortFlags() & PPF_DuplicateForPIE)
		{
			// Remap unique ID if necessary
			// only for fixing up cross-level references, inter-level references handled in FDuplicateDataReader
			FixupForPIE();
		}
	}
#endif // WITH_EDITOR
}

void SoftObjectPathLoadSubPathWorkaround(FArchive& Ar, FUtf8String& OutStr)
{
	int32 SaveNum = 0;
	Ar << SaveNum;

	if (SaveNum == 0)
	{
		// Empty strings are empty in any representation

		OutStr.Empty();
		return;
	}

	bool bLoadUnicodeChar = SaveNum < 0;
	if (bLoadUnicodeChar)
	{
		// If SaveNum is negative, it's either a wide string with non-ANSI elements, or it's a corrupted archive

		// If SaveNum cannot be negated due to integer overflow, Ar is corrupted.
		if (SaveNum == MIN_int32)
		{
			Ar.SetCriticalError();
			UE_LOGF(LogCore, Error, "Archive is corrupted");
			return;
		}

		SaveNum = -SaveNum;
	}

	// Protect against network packets allocating too much memory
	int64 MaxSerializeSize = Ar.GetMaxSerializeSize();
	if ((MaxSerializeSize > 0) && (SaveNum > MaxSerializeSize))
	{
		Ar.SetCriticalError();
		UE_LOG(LogCore, Error, TEXT("String is too large (Size: %i, Max: %" INT64_FMT ")"), SaveNum, MaxSerializeSize);
		return;
	}

	int32 NumSavedBytes = bLoadUnicodeChar ? SaveNum * sizeof(UCS2CHAR) : SaveNum * sizeof(ANSICHAR);

	// Load the saved bytes into an array
	TArray<uint8> SavedBytes;
	SavedBytes.Empty(NumSavedBytes);
	SavedBytes.AddUninitialized(NumSavedBytes);
	Ar.Serialize(SavedBytes.GetData(), NumSavedBytes);

	// Do byte swapping if necessary
	if (bLoadUnicodeChar && Ar.IsByteSwapping())
	{
		UCS2CHAR* SavedBytesPtr = (UCS2CHAR*)SavedBytes.GetData();
		for (int32 CharIndex = 0; CharIndex < SaveNum; ++CharIndex)
		{
			SavedBytesPtr[CharIndex] = ByteSwap(SavedBytesPtr[CharIndex]);
		}
	}

	// If the last byte is a zero, then it's a wide string or a UTF-8 path ending with NUL - assume the former
	if (bLoadUnicodeChar && ((UCS2CHAR*)SavedBytes.GetData())[SaveNum - 1] == (UCS2CHAR)0)
	{
		FString Temp = (UCS2CHAR*)SavedBytes.GetData();

		// Since Microsoft's vsnwprintf implementation raises an invalid parameter warning
		// with a character of 0xffff, scan for it and terminate the string there.
		// 0xffff isn't an actual Unicode character anyway.
		int Index = 0;
		if (Temp.FindChar(0xffff, Index))
		{
			Temp[Index] = TEXT('\0');
			Temp.TrimToNullTerminator();
		}

		OutStr = FUtf8String(Temp);
		return;
	}
	else if (!bLoadUnicodeChar && ((ANSICHAR*)SavedBytes.GetData())[SaveNum - 1] == (ANSICHAR)0)
	{
		OutStr = FUtf8String::ConstructFromPtrSize((ANSICHAR*)SavedBytes.GetData(), SavedBytes.Num());
		return;
	}

	// Assume it was saved as a UTF-8 string without a NUL terminator
	OutStr = FUtf8String::ConstructFromPtrSize((UTF8CHAR*)SavedBytes.GetData(), SavedBytes.Num());
}

void FSoftObjectPath::SerializePathWithoutFixup(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.UEVer() < VER_UE4_ADDED_SOFT_OBJECT_PATH)
	{
		FString Path;
		Ar << Path;

		if (Ar.UEVer() < VER_UE4_KEEP_ONLY_PACKAGE_NAMES_IN_STRING_ASSET_REFERENCES_MAP)
		{
			Path = FPackageName::GetNormalizedObjectPath(Path);
		}

		SetPath(MoveTemp(Path));
	}
	else if (Ar.IsLoading() && Ar.UEVer() < EUnrealEngineObjectUE5Version::FSOFTOBJECTPATH_REMOVE_ASSET_PATH_FNAMES)
	{
		FName AssetPathName;
		Ar << AssetPathName;
		AssetPath = WriteToString<FName::StringBufferSize>(AssetPathName).ToView();

		FWideString SubPathWide(SubPathString);
		Ar << SubPathWide;
		SubPathString = FUtf8String(SubPathWide);
	}
	else
	{
		Ar << AssetPath;

		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SoftObjectPathUtf8SubPaths)
		{
			if (Ar.IsLoading())
			{
				//**************//
				//* WORKAROUND *//
				//**************//
				// Some packages were saved with UTF-8 strings without a version check, so we need to replicate the combined
				// FWideString/FUtf8String serialization paths and attempt to guess which one we're loading.

				SoftObjectPathLoadSubPathWorkaround(Ar, SubPathString);
			}
			else
			{
				FWideString WideSubPathString(SubPathString);
				Ar << WideSubPathString;
				SubPathString = FUtf8String(WideSubPathString);
			}
		}
		else
		{
			Ar << SubPathString;
		}
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::SoftObjectPathTrailingNULsMaintained)
		{
			int32 NonZeroChar = SubPathString.FindLastCharByPredicate([](UTF8CHAR Ch){ return Ch != '\0'; });
			if (NonZeroChar != INDEX_NONE)
			{
				SubPathString.LeftInline(NonZeroChar + 1);
			}
		}
	}
}

bool FSoftObjectPath::operator==(FSoftObjectPath const& Other) const
{
	return AssetPath == Other.AssetPath && SubPathString == Other.SubPathString;
}

bool FSoftObjectPath::ExportTextItem(FString& ValueStr, FSoftObjectPath const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (!IsNull())
	{
		// Fixup any redirectors
		FSoftObjectPath Temp = *this;
		Temp.PreSavePath();

		const FString UndelimitedValue = (PortFlags & PPF_SimpleObjectText) ? Temp.GetAssetName() : Temp.ToString();

		if (PortFlags & PPF_Delimited)
		{
			ValueStr += TEXT("\"");
			ValueStr += UndelimitedValue.ReplaceQuotesWithEscapedQuotes();
			ValueStr += TEXT("\"");
		}
		else
		{
			ValueStr += UndelimitedValue;
		}
	}
	else
	{
		ValueStr += TEXT("None");
	}
	return true;
}

bool FSoftObjectPath::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive)
{
	TStringBuilder<256> ImportedPath;
	const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, /* out */ ImportedPath, /* dotted names */ true);
	if (!NewBuffer)
	{
		return false;
	}
	Buffer = NewBuffer;
	if (ImportedPath == TEXTVIEW("None"))
	{
		Reset();
	}
	else
	{
		if (*Buffer == TCHAR('('))
		{
			// Blueprints and other utilities may pass in () as a hardcoded value for an empty struct, so treat that like an empty string
			Buffer++;

			if (*Buffer == TCHAR(')'))
			{
				Buffer++;
				Reset();
				return true;
			}
			else
			{
				// Fall back to the default struct parsing, which will print an error message
				Buffer--;
				return false;
			}
		}

		if (*Buffer == TCHAR('\''))
		{
			// This is likely the remnants of stripping off the class part of a fully qualified object path
			// eg) /Script/Module.Class'/Game/Foo.Foo:Name With A Space.Bar' -> '/Game/Foo.Foo:Name With A Space.Bar'
			// We walk to the next ' so that we correctly handle object paths with spaces in them, which is technically 
			// not allowed by INVALID_OBJECTNAME_CHARACTERS, but happens all the time in BPs
			ImportedPath.Reset();
			{
				const int32 MaxTokenLength = FName::StringBufferSize - 1;
				NewBuffer = FPropertyHelpers::ReadTerminatedToken(++Buffer, /*Out*/ImportedPath, TEXT('\''), &MaxTokenLength);
			}
			if (!NewBuffer)
			{
				return false;
			}
			Buffer = NewBuffer;
			if (*Buffer++ != TCHAR('\''))
			{
				return false;
			}
		}

		SetPath(ImportedPath);
	}

#if WITH_EDITOR
	if (Parent && IsEditorOnlyObject(Parent))
	{
		// We're probably reading config for an editor only object, we need to mark this reference as editor only
		FSoftObjectPathSerializationScope SerializationScope(NAME_None, NAME_None, ESoftObjectPathCollectType::EditorOnlyCollect, ESoftObjectPathSerializeType::AlwaysSerialize);

		PostLoadPath(InSerializingArchive);
	}
	else
#endif
	{
		// Consider this a load, so Config string references get cooked
		PostLoadPath(InSerializingArchive);
	}

	return true;
}

/**
 * Serializes from mismatched tag.
 *
 * @template_param TypePolicy The policy should provide two things:
 *	- GetTypeName() method that returns registered name for this property type,
 *	- typedef Type, which is a C++ type to serialize if property matched type name.
 * @param Tag Property tag to match type.
 * @param Ar Archive to serialize from.
 */
template <class TypePolicy>
bool SerializeFromMismatchedTagTemplate(FString& Output, const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == TypePolicy::GetTypeName())
	{
		typename TypePolicy::Type* ObjPtr = nullptr;
		Slot << ObjPtr;
		if (ObjPtr)
		{
			Output = ObjPtr->GetPathName();
		}
		else
		{
			Output = FString();
		}
		return true;
	}
	else if (Tag.Type == NAME_NameProperty)
	{
		FName Name;
		Slot << Name;

		FNameBuilder NameBuilder(Name);
		Output = NameBuilder.ToView();
		return true;
	}
	else if (Tag.Type == NAME_StrProperty)
	{
		FString String;
		Slot << String;

		Output = String;
		return true;
	}
	return false;
}

bool FSoftObjectPath::SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	struct UObjectTypePolicy
	{
		typedef UObject Type;
		static const FName FORCEINLINE GetTypeName() { return NAME_ObjectProperty; }
	};

	FString Path = ToString();

	bool bReturn = SerializeFromMismatchedTagTemplate<UObjectTypePolicy>(Path, Tag, Slot);

	if (Slot.GetUnderlyingArchive().IsLoading())
	{
		SetPath(MoveTemp(Path));
		PostLoadPath(&Slot.GetUnderlyingArchive());
	}

	return bReturn;
}

bool FSoftObjectPath::RemapPackage(FName OldPackageName, FName NewPackageName)
{
	return AssetPath.RemapPackage(OldPackageName, NewPackageName);
}

UObject* FSoftObjectPath::TryLoad(FUObjectSerializeContext* InLoadContext, ELoadFlags LoadFlags /* = LOAD_None */) const
{
	UObject* LoadedObject = nullptr;

	if (!IsNull())
	{
		if (IsSubobject())
		{
			// For subobjects, it's not safe to call LoadObject directly, so we want to load the parent object and then resolve again
			FSoftObjectPath TopLevelPath = FSoftObjectPath::ConstructFromAssetPath(AssetPath);
			UObject* TopLevelObject = TopLevelPath.TryLoad(InLoadContext);

			// This probably loaded the top-level object, so re-resolve ourselves
			LoadedObject = ResolveObject();

			// If the the top-level object exists but we can't find the object, defer the loading to the top-level container object in case
			// it knows how to load that specific object.
			if (!LoadedObject && TopLevelObject)
			{
				TopLevelObject->ResolveSubobject(*UE_PRIVATE_TO_WIDE_STRING(SubPathString), LoadedObject, /*bLoadIfExists*/true);
			}
		}
		else
		{
			FString PathString = ToString();
#if WITH_EDITOR
			if (UE::GetPlayInEditorID() != INDEX_NONE)
			{
				// If we are in PIE and this hasn't already been fixed up, we need to fixup at resolution time. We cannot modify the path as it may be somewhere like a blueprint CDO
				FSoftObjectPath FixupObjectPath = *this;
				if (FixupObjectPath.FixupForPIE())
				{
					PathString = FixupObjectPath.ToString();
				}
			}
#endif

			LoadedObject = StaticLoadObject(UObject::StaticClass(), nullptr, *PathString, nullptr, LoadFlags, nullptr, true);

			// Look at core redirects if we didn't find the object
			if (!LoadedObject && SoftObjectPath::bResolveCoreRedirects)
			{
				FSoftObjectPath FixupObjectPath = *this;
				if (FixupObjectPath.FixupCoreRedirects())
				{
					LoadedObject = LoadObject<UObject>(nullptr, *FixupObjectPath.ToString(), {}, LoadFlags);
				}
			}

			while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(LoadedObject))
			{
				LoadedObject = Redirector->DestinationObject;
			}
		}
	}

	return LoadedObject;
}

int32 FSoftObjectPath::LoadAsync(FLoadSoftObjectPathAsyncDelegate InCompletionDelegate, FLoadAssetAsyncOptionalParams InOptionalParams) const
{
	FSoftObjectPath RequestedPath = *this;
	FSoftObjectPath PathToLoad = RequestedPath;
#if WITH_EDITOR
	if (UE::GetPlayInEditorID() != INDEX_NONE)
	{
		// @todo: This logic may need updating to handle level instances properly and we may want to handle other fixups like CoreRedirects before requesting
		PathToLoad.FixupForPIE();
	}
#endif

	if (SoftObjectPath::bResolveCoreRedirects)
	{
		PathToLoad.FixupCoreRedirects();
	}

	FLoadAssetAsyncDelegate WrapperDelegate = FLoadAssetAsyncDelegate::CreateLambda(
		[RequestedPath, PathToLoad, CompletionDelegate = MoveTemp(InCompletionDelegate)](const FTopLevelAssetPath& InAssetPath, UObject* InLoadedObject, EAsyncLoadingResult::Type InResult) mutable
		{
			// If this isn't a subobject, InLoadedObject is already correct
			if (PathToLoad.IsSubobject())
			{
				// Resolve the entire path, including the subobject
				InLoadedObject = PathToLoad.ResolveObject();
			}

			// Call delegate with original requested path
			CompletionDelegate.ExecuteIfBound(RequestedPath, InLoadedObject);
		});

	return LoadAssetAsync(PathToLoad.GetAssetPath(), MoveTemp(WrapperDelegate), MoveTemp(InOptionalParams));
}

UObject* FSoftObjectPath::ResolveObject() const
{
	// Don't try to resolve if we're saving a package because StaticFindObject can't be used here
	// and we usually don't want to force references to weak pointers while saving.
	if (IsNull() || (UE::IsSavingPackage() && !IsInAsyncLoadingThread()))
	{
		return nullptr;
	}

#if WITH_EDITOR
	if (UE::GetPlayInEditorID() != INDEX_NONE)
	{
		// If we are in PIE and this hasn't already been fixed up, we need to fixup at resolution time. We cannot modify the path as it may be somewhere like a blueprint CDO
		FSoftObjectPath FixupObjectPath = *this;
		if (FixupObjectPath.FixupForPIE())
		{
			return FixupObjectPath.ResolveObjectInternal();
		}
	}
#endif

	return ResolveObjectInternal();
}

UObject* FSoftObjectPath::ResolveObjectInternal() const
{
	UObject* FoundObject = FindObject<UObject>(AssetPath);
	if (FoundObject && !SubPathString.IsEmpty())
	{
		TStringBuilder<FName::StringBufferSize> Builder;
		Builder << SubPathString;
		FoundObject = FindObject<UObject>(FoundObject, *Builder);
	}

	if (!FoundObject && IsSubobject())
	{
		// Try to resolve through the top level object
		FSoftObjectPath TopLevelPath(AssetPath);
		UObject* TopLevelObject = TopLevelPath.ResolveObject();

		// If the the top-level object exists but we can't find the object, defer the resolving to the top-level container object in case
		// it knows how to load that specific object.
		if (TopLevelObject)
		{
			TopLevelObject->ResolveSubobject(*UE_PRIVATE_TO_WIDE_STRING(SubPathString), FoundObject, /*bLoadIfExists*/false);
		}
	}

	// Look at core redirects if we didn't find the object
	if (!FoundObject && SoftObjectPath::bResolveCoreRedirects)
	{
		FSoftObjectPath FixupObjectPath = *this;
		if (FixupObjectPath.FixupCoreRedirects())
		{
			FoundObject = FindObject<UObject>(nullptr, *FixupObjectPath.ToString());
		}
	}

	while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(FoundObject))
	{
		FoundObject = Redirector->DestinationObject;
	}

	return FoundObject;
}

FSoftObjectPath FSoftObjectPath::GetOrCreateIDForObject(FObjectPtr Object)
{
	check(Object);
	return FSoftObjectPath(Object);
}

void FSoftObjectPath::AddPIEPackageName(FName NewPIEPackageName)
{
	UE::TWriteScopeLock WriteLock(PIEPackageNamesLock);
	PIEPackageNames.Add(NewPIEPackageName);
}

void FSoftObjectPath::ClearPIEPackageNames()
{
	UE::TWriteScopeLock WriteLock(PIEPackageNamesLock);
	PIEPackageNames.Empty();
}

bool FSoftObjectPath::FixupForPIE(int32 InPIEInstance, TFunctionRef<void(int32, FSoftObjectPath&)> InPreFixupForPIECustomFunction)
{
#if WITH_EDITOR
	if (InPIEInstance != INDEX_NONE && !IsNull())
	{
		InPreFixupForPIECustomFunction(InPIEInstance, *this);

		const FString Path = ToString();

		// Determine if this reference has already been fixed up for PIE
		const FString ShortPackageOuterAndName = FPackageName::GetLongPackageAssetName(Path);
		if (!ShortPackageOuterAndName.StartsWith(PLAYWORLD_PACKAGE_PREFIX))
		{
			// Name of the ULevel subobject of UWorld, set in InitializeNewWorld
			const bool bIsChildOfLevel = SubPathString.StartsWith(TEXT("PersistentLevel."));

			FString PIEPath = FString::Printf(TEXT("%s/%s_%d_%s"), *FPackageName::GetLongPackagePath(Path), PLAYWORLD_PACKAGE_PREFIX, InPIEInstance, *ShortPackageOuterAndName);
			const FName PIEPackage = (!bIsChildOfLevel ? FName(*FPackageName::ObjectPathToPackageName(PIEPath)) : NAME_None);

			// Duplicate if this an already registered PIE package or this looks like a level subobject reference
			bool bIsPIEPackage;
			{
				UE::TReadScopeLock ReadLock(PIEPackageNamesLock);
				bIsPIEPackage = PIEPackageNames.Contains(PIEPackage);
			}
			if (bIsChildOfLevel || bIsPIEPackage)
			{
				// Need to prepend PIE prefix, as we're in PIE and this refers to an object in a PIE package
				SetPath(MoveTemp(PIEPath));

				return true;
			}
		}
	}
#endif
	return false;
}

bool FSoftObjectPath::FixupForPIE(TFunctionRef<void(int32, FSoftObjectPath&)> InPreFixupForPIECustomFunction)
{
	return FixupForPIE(UE::GetPlayInEditorID(), InPreFixupForPIECustomFunction);
}

bool FSoftObjectPath::FixupCoreRedirects()
{
	// Construct from FSoftObjectPath to avoid unnecessary string copying and possible FName creation
	FCoreRedirectObjectName OldName(*this);
	FCoreRedirectObjectName NewName;

	{
		TStringBuilder<NAME_SIZE> OldPackageNameString;
		OldName.PackageName.ToString(OldPackageNameString);

		// Always try the object redirect, this will pick up any package redirects as well
		// For things that look like native objects, try all types as we don't know which it would be
		const bool bIsNative = FPackageName::IsScriptPackage(OldPackageNameString);
		NewName = FCoreRedirects::GetRedirectedName(bIsNative ? ECoreRedirectFlags::Type_AllMask : ECoreRedirectFlags::Type_Object, OldName);
	}

	if (OldName != NewName)
	{
		// Only do the fixup if the old object isn't in memory (or was redirected to new name), this avoids false positives
		UObject* FoundOldObject = FindObjectSafe<UObject>(nullptr, *OldName.ToString());
		FString NewString = NewName.ToString();

		if (!FoundOldObject || FoundOldObject->GetPathName() == NewString)
		{
			SetPath(NewString);
			return true;
		}
	}

	return false;
}

bool FSoftClassPath::SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot)
{
	struct UClassTypePolicy
	{
		typedef UClass Type;
		// Class property shares the same tag id as Object property
		static const FName FORCEINLINE GetTypeName() { return NAME_ObjectProperty; }
	};

	FString Path = ToString();

	bool bReturn = SerializeFromMismatchedTagTemplate<UClassTypePolicy>(Path, Tag, Slot);

	if (Slot.GetUnderlyingArchive().IsLoading())
	{
		SetPath(MoveTemp(Path));
		PostLoadPath(&Slot.GetUnderlyingArchive());
	}

	return bReturn;
}

UClass* FSoftClassPath::ResolveClass() const
{
	return Cast<UClass>(ResolveObject());
}

FSoftClassPath FSoftClassPath::GetOrCreateIDForClass(const UClass *InClass)
{
	check(InClass);
	return FSoftClassPath(InClass);
}

UE_DEFINE_THREAD_SINGLETON_TLS(FSoftObjectPathThreadContext, COREUOBJECT_API)

bool FSoftObjectPathThreadContext::GetSerializationOptions(FName& OutPackageName, FName& OutPropertyName, ESoftObjectPathCollectType& OutCollectType, ESoftObjectPathSerializeType& OutSerializeType, FArchive* Archive) const
{
	FName CurrentPackageName, CurrentPropertyName;
	ESoftObjectPathCollectType CurrentCollectType = ESoftObjectPathCollectType::AlwaysCollect;
	ESoftObjectPathSerializeType CurrentSerializeType = ESoftObjectPathSerializeType::AlwaysSerialize;
	bool bFoundAnything = false;
	if (OptionStack.Num() > 0)
	{
		// Go from the top of the stack down
		for (int32 i = OptionStack.Num() - 1; i >= 0; i--)
		{
			const FSerializationOptions& Options = OptionStack[i];
			// Find first valid package/property names. They may not necessarily match
			if (Options.PackageName != NAME_None && CurrentPackageName == NAME_None)
			{
				CurrentPackageName = Options.PackageName;
			}
			if (Options.PropertyName != NAME_None && CurrentPropertyName == NAME_None)
			{
				CurrentPropertyName = Options.PropertyName;
			}

			// Restrict based on lowest/most restrictive collect type
			if (Options.CollectType < CurrentCollectType)
			{
				CurrentCollectType = Options.CollectType;
			}
			if (Options.SerializeType < CurrentSerializeType)
			{
				CurrentSerializeType = Options.SerializeType;
			}
		}

		bFoundAnything = true;
	}
	
	// Check UObject serialize context as a backup
	FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
	if (LoadContext && LoadContext->SerializedObject)
	{
		FLinkerLoad* Linker = LoadContext->SerializedObject->GetLinker();
		if (Linker)
		{
			if (CurrentPackageName == NAME_None)
			{
				CurrentPackageName = Linker->GetPackagePath().GetPackageFName();
			}
			if (Archive == nullptr)
			{
				// Use archive from linker if it wasn't passed in
				Archive = Linker;
			}
			bFoundAnything = true;
		}
	}

	// Check archive for property/editor only info, this works for any serialize if passed in
	if (Archive)
	{
		FProperty* CurrentProperty = Archive->GetSerializedProperty();
			
		if (CurrentProperty && CurrentPropertyName == NAME_None)
		{
			CurrentPropertyName = CurrentProperty->GetFName();
		}
		bool bEditorOnly = false;
#if WITH_EDITOR
		bEditorOnly = Archive->IsEditorOnlyPropertyOnTheStack();

		if (CurrentProperty && CurrentProperty->GetOwnerProperty()->HasMetaData(FSoftObjectPath::NAME_Untracked))
		{
			// Property has the Untracked metadata, so set to never collect references if it's higher than NeverCollect
			CurrentCollectType = FMath::Min(ESoftObjectPathCollectType::NeverCollect, CurrentCollectType);
		}
#endif
		// If we were always collect before and not overridden by stack options, set to editor only
		if (bEditorOnly && CurrentCollectType == ESoftObjectPathCollectType::AlwaysCollect)
		{
			CurrentCollectType = ESoftObjectPathCollectType::EditorOnlyCollect;
		}

		bFoundAnything = true;
	}

	if (bFoundAnything)
	{
		OutPackageName = CurrentPackageName;
		OutPropertyName = CurrentPropertyName;
		OutCollectType = CurrentCollectType;
		OutSerializeType = CurrentSerializeType;
		return true;
	}

	return bFoundAnything;
}

TSet<FName> FSoftObjectPath::PIEPackageNames;

void SerializeForLog(FCbWriter& Writer, const FSoftObjectPath& Value)
{
	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("SoftObjectPath"));
	Writer.AddString(ANSITEXTVIEW("$text"), WriteToUtf8String<256>(Value));
	Writer.AddString(ANSITEXTVIEW("PackageName"), WriteToUtf8String<256>(Value.GetLongPackageFName()));
	Writer.AddString(ANSITEXTVIEW("AssetName"), WriteToUtf8String<256>(Value.GetAssetFName()));
	Writer.AddString(ANSITEXTVIEW("SubPath"), Value.GetSubPathUtf8String());
	Writer.EndObject();
}

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"

std::ostream& operator<<(std::ostream& Stream, const FSoftObjectPath& Value)
{
	TStringBuilder<FName::StringBufferSize*2> Builder;
	Builder << Value;
	return Stream << Builder.ToView();
}
#endif 

TEST_CASE_NAMED(FSoftObjectPathImportTextTests,  "System::CoreUObject::SoftObjectPath::ImportText", "[ApplicationContextMask][EngineFilter]")
{
	const TCHAR* PackageName = TEXT("/Game/Environments/Sets/Arid/Materials/M_Arid");
	const TCHAR* AssetName = TEXT("M_Arid");
	const FString String = FString::Printf(TEXT("%s.%s"), PackageName, AssetName);

	const FString QuotedPath = FString::Printf(TEXT("\"%s\""), *String);
	const FString UnquotedPath = String;

	FSoftObjectPath Path(String);
	CHECK_EQUALS(TEXT("Correct package name"), Path.GetLongPackageName(), PackageName);
	CHECK_EQUALS(TEXT("Correct asset name"), Path.GetAssetName(), AssetName);
	CHECK_EQUALS(TEXT("Empty subpath"), Path.GetSubPathString(), TEXT(""));

	FSoftObjectPath ImportQuoted;
	const TCHAR* QuotedBuffer = *QuotedPath;
	CHECK_MESSAGE(TEXT("Quoted path imports successfully"), ImportQuoted.ImportTextItem(QuotedBuffer, PPF_None, nullptr, GLog->Get()));
	CHECK_EQUALS(TEXT("Quoted path imports correctly"), ImportQuoted, Path);

	FSoftObjectPath ImportUnquoted;
	const TCHAR* UnquotedBuffer = *UnquotedPath;
	CHECK_MESSAGE(TEXT("Unquoted path imports successfully"), ImportUnquoted.ImportTextItem(UnquotedBuffer, PPF_None, nullptr, GLog->Get()));
	CHECK_EQUALS(TEXT("Unquoted path imports correctly"), ImportUnquoted, Path);
}

TEST_CASE_NAMED(FSoftObjectPathTrySetPathTests, "System::CoreUObject::SoftObjectPath::TrySetPath", "[ApplicationContextMask][EngineFilter]")
{
	FSoftObjectPath Path;

	const TCHAR* PackageName = TEXT("/Game/Maps/Arena");
	const TCHAR* TopLevelPath = TEXT("/Game/Maps/Arena.Arena");
	const TCHAR* TopLevelPathWrongSeparator = TEXT("/Game/Maps/Arena:Arena");

	Path.SetPath(PackageName);
	REQUIRE_MESSAGE("Package name: Is valid", Path.IsValid());
	{
		CHECK_EQUALS("Package name: Round trips equal", Path.ToString(), PackageName);
		CHECK_EQUALS("Package name: Package name part", Path.GetLongPackageName(), PackageName);
		CHECK_EQUALS("Package name: Asset name part", Path.GetAssetName(), FString());
		CHECK_EQUALS("Package name: Subobject path part", Path.GetSubPathString(), FString());
	}

	Path.SetPath(TopLevelPath);
	REQUIRE_MESSAGE("Top level object path: Is valid", Path.IsValid());
	{
		CHECK_EQUALS("Top level object path: round trips equal", Path.ToString(), TopLevelPath);
	}

	const TCHAR* PathWithWideChars = TEXT("/Game/\u30ad\u30e3\u30e9\u30af\u30bf\u30fc/\u5c71\u672c.\u5c71\u672c");
	Path.SetPath(PathWithWideChars);
	REQUIRE_MESSAGE("Path with wide chars: Is valid", Path.IsValid());
	{
		CHECK_EQUALS("Path with wide chars: Round trips equal", Path.ToString(), PathWithWideChars);
		CHECK_EQUALS("Path with wide chars: Package name part", Path.GetLongPackageName(), TEXT("/Game/\u30ad\u30e3\u30e9\u30af\u30bf\u30fc/\u5c71\u672c"));
		CHECK_EQUALS("Path with wide chars: Asset name part", Path.GetAssetName(), TEXT("\u5c71\u672c"));
		CHECK_EQUALS("Path with wide chars: Subobject path part", Path.GetSubPathString(), FString());
	}

	Path.SetPath(TopLevelPathWrongSeparator);
	// Round tripping replaces dot with subobject separator for second separator 
	REQUIRE_MESSAGE("Top level object path with incorrect separator: is valid", Path.IsValid());
	{ 
		CHECK_EQUALS("Top level object path with incorrect separator: Round trips with normalized separator", Path.ToString(), TopLevelPath);
		CHECK_EQUALS("Top level object path with incorrect separator: Package name part", Path.GetLongPackageName(), TEXT("/Game/Maps/Arena"));
		CHECK_EQUALS("Top level object path with incorrect separator: Asset name part", Path.GetAssetName(), TEXT("Arena"));
		CHECK_EQUALS("Top level object path with incorrect separator: Subobject path part", Path.GetSubPathString(), FString());
	}

	const TCHAR* PackageNameTrailingDot = TEXT("/Game/Maps/Arena.");
	Path.SetPath(PackageNameTrailingDot);
	CHECK_FALSE_MESSAGE("Package name trailing dot: is not valid", Path.IsValid());

	const TCHAR* PackageNameTrailingSeparator = TEXT("/Game/Maps/Arena:");
	Path.SetPath(PackageNameTrailingSeparator);
	CHECK_FALSE_MESSAGE("Package name trailing separator: is not valid", Path.IsValid());

	const TCHAR* ObjectPathTrailingDot = TEXT("/Game/Maps/Arena.Arena.");
	Path.SetPath(ObjectPathTrailingDot);
	CHECK_FALSE_MESSAGE("Object path trailing dot: is not valid", Path.IsValid());

	const TCHAR* ObjectPathTrailingSeparator = TEXT("/Game/Maps/Arena.Arena:");
	Path.SetPath(ObjectPathTrailingSeparator);
	CHECK_FALSE_MESSAGE("Object path trailing separator: is not valid", Path.IsValid());

	const TCHAR* PackageNameWithoutLeadingSlash = TEXT("Game/Maps/Arena");
	Path.SetPath(PackageNameWithoutLeadingSlash);
	CHECK_FALSE_MESSAGE("Package name without leading slash: is not valid", Path.IsValid());

	const TCHAR* ObjectPathWithoutLeadingSlash = TEXT("Game/Maps/Arena.Arena");
	Path.SetPath(ObjectPathWithoutLeadingSlash);
	CHECK_FALSE_MESSAGE("Object name without leading slash: is not valid", Path.IsValid());

	const TCHAR* SubObjectPathWithSeparator = TEXT("/Game/Characters/Steve.Steve_C:Root");
	Path.SetPath(SubObjectPathWithSeparator);
	REQUIRE_MESSAGE("Subobject path with separator: is valid", Path.IsValid());
	{
		CHECK_EQUALS("Subobject path with separator: round trip", Path.ToString(), SubObjectPathWithSeparator);
		CHECK_EQUALS("Subobject path with separator: package name", Path.GetLongPackageName(), TEXT("/Game/Characters/Steve"));
		CHECK_EQUALS("Subobject path with separator: asset name", Path.GetAssetName(), TEXT("Steve_C"));
		CHECK_EQUALS("Subobject path with separator: subobject path", Path.GetSubPathString(), TEXT("Root"));
	}

	const TCHAR* SubObjectPathWithTrailingDot = TEXT("/Game/Characters/Steve.Steve_C:Root.");
	Path.SetPath(SubObjectPathWithTrailingDot);
	CHECK_FALSE_MESSAGE("Subobject path with trailing dot: is not valid", Path.IsValid());

	const TCHAR* SubObjectPathWithTrailingSeparator = TEXT("/Game/Characters/Steve.Steve_C:Root:");
	Path.SetPath(SubObjectPathWithTrailingSeparator);
	CHECK_FALSE_MESSAGE("Subobject path with trailing separator: is not valid", Path.IsValid());

	const TCHAR* PathWithoutAssetName = TEXT("/Game/Characters/Steve.:Root");
	Path.SetPath(PathWithoutAssetName );
	CHECK_FALSE_MESSAGE("Subobject path without asset name: is not valid", Path.IsValid());

	const TCHAR* SubObjectPathWithDot = TEXT("/Game/Characters/Steve.Steve_C:Root");
	Path.SetPath(SubObjectPathWithDot);
	REQUIRE_MESSAGE("Subobject path with dot: is valid", Path.IsValid());
	{
		CHECK_EQUALS("Subobject path with dot: round trips with normalized separator", Path.ToString(), SubObjectPathWithDot); // Round tripping replaces dot with subobject separator for second separator 
		CHECK_EQUALS("Subobject path with dot: package name", Path.GetLongPackageName(), TEXT("/Game/Characters/Steve"));
		CHECK_EQUALS("Subobject path with dot: asset name", Path.GetAssetName(), TEXT("Steve_C"));
		CHECK_EQUALS("Subobject path with dot: subobject path", Path.GetSubPathString(), TEXT("Root"));
	}

	const TCHAR* LongPath = TEXT("/Game/Characters/Steve.Steve_C:Root.Inner.AnotherInner.FurtherInner");
	Path.SetPath(LongPath);
	REQUIRE_MESSAGE("Long path: is valid", Path.IsValid());
	{
		CHECK_EQUALS("Long path: round trip", Path.ToString(), LongPath);
		CHECK_EQUALS("Long path: Package name part", Path.GetLongPackageName(), TEXT("/Game/Characters/Steve"));
		CHECK_EQUALS("Long path: Asset name part", Path.GetAssetName(), TEXT("Steve_C"));
		CHECK_EQUALS("Long path: Subobject path part", Path.GetSubPathString(), TEXT("Root.Inner.AnotherInner.FurtherInner"));
	}

	const TCHAR* LongPathWithSeparatorInWrongPlace = TEXT("/Game/Characters/Steve.Steve_C.Root.Inner.AnotherInner:FurtherInner");
	Path.SetPath(LongPathWithSeparatorInWrongPlace);
	REQUIRE_MESSAGE("Long path with separator in wrong place: is valid", Path.IsValid());
	{
		CHECK_EQUALS("Long path with separator in wrong place: round trip with normalized separator", Path.ToString(), LongPath);
		CHECK_EQUALS("Long path with separator in wrong place: package name", Path.GetLongPackageName(), TEXT("/Game/Characters/Steve"));
		CHECK_EQUALS("Long path with separator in wrong place: asset name", Path.GetAssetName(), TEXT("Steve_C"));
		CHECK_EQUALS("Long path with separator in wrong place: subobject path", Path.GetSubPathString(), TEXT("Root.Inner.AnotherInner.FurtherInner"));
	}

	const TCHAR* LongPathWithConsecutiveDelimiters = TEXT("/Game/Characters/Steve.Steve_C:Root.Inner.AnotherInner..FurtherInner");
	Path.SetPath(LongPathWithConsecutiveDelimiters );
	CHECK_FALSE_MESSAGE("Long path with consecutive delimiters: is not valid", Path.IsValid());
}

#if WITH_EDITOR

TEST_CASE_NAMED(FSoftObjectPathFixupForPIETests, "System::CoreUObject::SoftObjectPath::FixupForPIE", "[ApplicationContextMask][EngineFilter]")
{
	const TCHAR* TestOriginalPath = TEXT("/Game/Maps/Arena.Arena:PersistentLevel.Target");	
	const int32 PieInstanceID = 7;
	const FString ExpectedFinalPath = FString::Printf(TEXT("/Game/Maps/%s_%d_Arena.Arena:PersistentLevel.Target"), PLAYWORLD_PACKAGE_PREFIX, PieInstanceID);
	
	FSoftObjectPath SoftPath(TestOriginalPath);
	SoftPath.FixupForPIE(PieInstanceID);	
	CHECK_EQUALS(TEXT("Fixed up path should be PIE package with correct id"), SoftPath.ToString(), ExpectedFinalPath);
}

#endif // WITH_EDITOR
#endif // WITH_TESTS
