// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/SparseClassDataUtils.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"
#include "Templates/Function.h"

bool UE::Reflection::DoesSparseClassDataOverrideArchetype(const UClass* Class, const TFunctionRef<bool(FProperty*)>& Filter)
{
	const UScriptStruct* SparseClassDataStruct = Class->GetSparseClassDataStruct();
	const void* SCD = const_cast<UClass*>(Class)->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull);
	if (SparseClassDataStruct == nullptr ||
		SCD == nullptr)
	{
		return false;
	}

	if (SparseClassDataStruct != Class->GetSparseClassDataArchetypeStruct())
	{
		return true;
	}

	const void* ArchetypeSCD = Class->GetArchetypeForSparseClassData();
	if (!ArchetypeSCD)
	{
		return true;
	}

	for (TFieldIterator<FProperty> It(SparseClassDataStruct); It; ++It)
	{
		if (Filter(*It) && !It->Identical_InContainer(ArchetypeSCD, SCD))
		{
			return true;
		}
	}

	return false;
}
