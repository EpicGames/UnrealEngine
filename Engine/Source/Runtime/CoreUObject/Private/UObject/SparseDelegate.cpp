// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/SparseDelegate.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeLock.h"

FSparseDelegateStorage::FObjectListener FSparseDelegateStorage::SparseDelegateObjectListener;
FTransactionallySafeCriticalSection FSparseDelegateStorage::SparseDelegateMapCritical;
TMap<const UObjectBase*, FSparseDelegateStorage::FSparseDelegateMap> FSparseDelegateStorage::SparseDelegates;
TMap<TPair<FName,FName>, size_t> FSparseDelegateStorage::SparseDelegateObjectOffsets;

FSparseDelegateStorage::FObjectListener::~FObjectListener()
{
	// Destroy order might result in GUObjectArray or its critical section being invalid so don't disable since we're shutting down anyways
	if (!IsEngineExitRequested())
	{
		DisableListener();
	}
}

void FSparseDelegateStorage::FObjectListener::NotifyUObjectDeleted(const UObjectBase* Object, int32 Index)
{
	UE::TScopeLock SparseDelegateMapLock(FSparseDelegateStorage::SparseDelegateMapCritical);
	FSparseDelegateStorage::SparseDelegates.Remove(Object);
	if (FSparseDelegateStorage::SparseDelegates.Num() == 0)
	{
		DisableListener();
	}
}

void FSparseDelegateStorage::FObjectListener::OnUObjectArrayShutdown()
{
	UE::TScopeLock SparseDelegateMapLock(FSparseDelegateStorage::SparseDelegateMapCritical);
	FSparseDelegateStorage::SparseDelegates.Empty();
	DisableListener();
}

void FSparseDelegateStorage::FObjectListener::EnableListener()
{
	GUObjectArray.AddUObjectDeleteListener(this);
}

void FSparseDelegateStorage::FObjectListener::DisableListener()
{
	GUObjectArray.RemoveUObjectDeleteListener(this);
}

void FSparseDelegateStorage::RegisterDelegateOffset(const UObject* OwningObject, const FName DelegateName, const size_t DelegateOffsetToOwner)
{
	check(OwningObject);
	SparseDelegateObjectOffsets.Add(TPair<FName,FName>(OwningObject->GetClass()->GetFName(), DelegateName), DelegateOffsetToOwner);
}

FSparseDelegate* FSparseDelegateStorage::ResolveSparseDelegate(const UObject* OwningObject, const FName DelegateName)
{
	const UClass* OwningClass = OwningObject->GetClass();
	while (OwningClass)
	{
		if (OwningClass->HasAnyClassFlags(CLASS_Native) && !OwningClass->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
		{
			if (size_t* DelegateOffset = SparseDelegateObjectOffsets.Find(TPair<FName, FName>(OwningClass->GetFName(), DelegateName)))
			{
				return reinterpret_cast<FSparseDelegate*>((uint8*)OwningObject + *DelegateOffset);
			}
		}

		OwningClass = OwningClass->GetSuperClass();
	}
	check(false);
	return nullptr;
}

UObject* FSparseDelegateStorage::ResolveSparseOwner(const FSparseDelegate& SparseDelegate, const FName OwningClassName, const FName DelegateName)
{
	if (size_t* DelegateOffset = SparseDelegateObjectOffsets.Find(TPair<FName, FName>(OwningClassName, DelegateName)))
	{
		return reinterpret_cast<UObject*>((uint8*)&SparseDelegate - *DelegateOffset);
	}
	check(false);
	return nullptr;
}

FMulticastScriptDelegate* FSparseDelegateStorage::GetMulticastDelegate(const UObject* DelegateOwner, const FName DelegateName)
{
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

	if (FSparseDelegateMap* DelegateMap = SparseDelegates.Find(DelegateOwner))
	{
		if (TSharedPtr<FMulticastScriptDelegate>* MulticastDelegatePtr = DelegateMap->Find(DelegateName))
		{
			return MulticastDelegatePtr->Get();
		}
	}
	return nullptr;
}

TSharedPtr<FMulticastScriptDelegate> FSparseDelegateStorage::GetSharedMulticastDelegate(const UObject* DelegateOwner, const FName DelegateName)
{
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

	TSharedPtr<FMulticastScriptDelegate> Result;
	if (FSparseDelegateMap* DelegateMap = SparseDelegates.Find(DelegateOwner))
	{
		if (TSharedPtr<FMulticastScriptDelegate>* MulticastDelegatePtr = DelegateMap->Find(DelegateName))
		{
			Result = *MulticastDelegatePtr;
		}
	}
	return Result;
}

void FSparseDelegateStorage::SetMulticastDelegate(const UObject* DelegateOwner, const FName DelegateName, FMulticastScriptDelegate Delegate)
{
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

	if (SparseDelegates.Num() == 0)
	{
		SparseDelegateObjectListener.EnableListener();
	}

	FSparseDelegateMap& DelegateMap = SparseDelegates.FindOrAdd(DelegateOwner);
	TSharedPtr<FMulticastScriptDelegate>& MulticastDelegate = DelegateMap.FindOrAdd(DelegateName);

	if (!MulticastDelegate.IsValid())
	{
		MulticastDelegate = MakeShared<FMulticastScriptDelegate>();
	}

	*MulticastDelegate = MoveTemp(Delegate);
}

bool FSparseDelegateStorage::Add(const UObject* DelegateOwner, const FName DelegateName, FScriptDelegate Delegate)
{
	bool bDelegateWasBound = false;
	if (Delegate.IsBound())
	{
		UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

		if (SparseDelegates.Num() == 0)
		{
			SparseDelegateObjectListener.EnableListener();
		}

		FSparseDelegateMap& DelegateMap = SparseDelegates.FindOrAdd(DelegateOwner);
		TSharedPtr<FMulticastScriptDelegate>& MulticastDelegate = DelegateMap.FindOrAdd(DelegateName);

		if (!MulticastDelegate.IsValid())
		{
			MulticastDelegate = MakeShared<FMulticastScriptDelegate>();
		}

		MulticastDelegate->Add(MoveTemp(Delegate));
		bDelegateWasBound = true;
	}
	return bDelegateWasBound;
}

bool FSparseDelegateStorage::AddUnique(const UObject* DelegateOwner, const FName DelegateName, FScriptDelegate Delegate)
{
	bool bDelegateWasBound = false;
	if (Delegate.IsBound())
	{
		UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

		if (SparseDelegates.Num() == 0)
		{
			SparseDelegateObjectListener.EnableListener();
		}

		FSparseDelegateMap& DelegateMap = SparseDelegates.FindOrAdd(DelegateOwner);
		TSharedPtr<FMulticastScriptDelegate>& MulticastDelegate = DelegateMap.FindOrAdd(DelegateName);

		if (!MulticastDelegate.IsValid())
		{
			MulticastDelegate = MakeShared<FMulticastScriptDelegate>();
		}

		MulticastDelegate->AddUnique(MoveTemp(Delegate));
		bDelegateWasBound = true;
	}
	return bDelegateWasBound;
}

bool FSparseDelegateStorage::Contains(const UObject* DelegateOwner, const FName DelegateName, const FScriptDelegate& Delegate)
{
	bool bContainsDelegate = false;
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

	if (FSparseDelegateMap* DelegateMap = SparseDelegates.Find(DelegateOwner))
	{
		if (TSharedPtr<FMulticastScriptDelegate>* MulticastDelegatePtr = DelegateMap->Find(DelegateName))
		{
			if (FMulticastScriptDelegate* MulticastDelegate = MulticastDelegatePtr->Get())
			{
				bContainsDelegate = MulticastDelegate->Contains(Delegate);
			}
		}
	}

	return bContainsDelegate;
}

bool FSparseDelegateStorage::Contains(const UObject* DelegateOwner, const FName DelegateName, const UObject* InObject, FName InFunctionName)
{
	bool bContainsDelegate = false;
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

	if (FSparseDelegateMap* DelegateMap = SparseDelegates.Find(DelegateOwner))
	{
		if (TSharedPtr<FMulticastScriptDelegate>* MulticastDelegatePtr = DelegateMap->Find(DelegateName))
		{
			if (FMulticastScriptDelegate* MulticastDelegate = MulticastDelegatePtr->Get())
			{
				bContainsDelegate = MulticastDelegate->Contains(InObject, InFunctionName);
			}
		}
	}
	return bContainsDelegate;
}

bool FSparseDelegateStorage::Remove(const UObject* DelegateOwner, const FName DelegateName, const FScriptDelegate& Delegate)
{
	bool bSparseDelegateBound = false;
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

	if (FSparseDelegateMap* DelegateMap = SparseDelegates.Find(DelegateOwner))
	{
		if (TSharedPtr<FMulticastScriptDelegate>* MulticastDelegatePtr = DelegateMap->Find(DelegateName))
		{
			if (FMulticastScriptDelegate* MulticastDelegate = MulticastDelegatePtr->Get())
			{
				MulticastDelegate->Remove(Delegate);
				bSparseDelegateBound = MulticastDelegate->IsBound();
				if (!bSparseDelegateBound)
				{
					DelegateMap->Remove(DelegateName);
				}
			}
			else
			{
				DelegateMap->Remove(DelegateName);
			}
		}
		if (DelegateMap->Num() == 0)
		{
			SparseDelegates.Remove(DelegateOwner);
			if (SparseDelegates.Num() == 0)
			{
				SparseDelegateObjectListener.DisableListener();
			}
		}
	}

	return bSparseDelegateBound;
}

bool FSparseDelegateStorage::Remove(const UObject* DelegateOwner, const FName DelegateName, const UObject* InObject, FName InFunctionName)
{
	bool bSparseDelegateBound = false;
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

	if (FSparseDelegateMap* DelegateMap = SparseDelegates.Find(DelegateOwner))
	{
		if (TSharedPtr<FMulticastScriptDelegate>* MulticastDelegatePtr = DelegateMap->Find(DelegateName))
		{
			if (FMulticastScriptDelegate* MulticastDelegate = MulticastDelegatePtr->Get())
			{
				MulticastDelegate->Remove(InObject, InFunctionName);
				bSparseDelegateBound = MulticastDelegate->IsBound();
				if (!bSparseDelegateBound)
				{
					DelegateMap->Remove(DelegateName);
				}
			}
			else
			{
				DelegateMap->Remove(DelegateName);
			}
		}
		if (DelegateMap->Num() == 0)
		{
			SparseDelegates.Remove(DelegateOwner);
			if (SparseDelegates.Num() == 0)
			{
				SparseDelegateObjectListener.DisableListener();
			}
		}
	}

	return bSparseDelegateBound;
}

bool FSparseDelegateStorage::RemoveAll(const UObject* DelegateOwner, const FName DelegateName, const UObject* UserObject)
{
	bool bSparseDelegateBound = false;
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);

	if (FSparseDelegateMap* DelegateMap = SparseDelegates.Find(DelegateOwner))
	{
		if (TSharedPtr<FMulticastScriptDelegate>* MulticastDelegatePtr = DelegateMap->Find(DelegateName))
		{
			if (FMulticastScriptDelegate* MulticastDelegate = MulticastDelegatePtr->Get())
			{
				MulticastDelegate->RemoveAll(UserObject);
				bSparseDelegateBound = MulticastDelegate->IsBound();
				if (!bSparseDelegateBound)
				{
					DelegateMap->Remove(DelegateName);
				}
			}
			else
			{
				DelegateMap->Remove(DelegateName);
			}
		}
		if (DelegateMap->Num() == 0)
		{
			SparseDelegates.Remove(DelegateOwner);
			if (SparseDelegates.Num() == 0)
			{
				SparseDelegateObjectListener.DisableListener();
			}
		}
	}
	return bSparseDelegateBound;
}

void FSparseDelegateStorage::Clear(const UObject* DelegateOwner, const FName DelegateName)
{
	UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);
	if (FSparseDelegateMap* DelegateMap = SparseDelegates.Find(DelegateOwner))
	{
		if (TSharedPtr<FMulticastScriptDelegate>* MulticastDelegatePtr = DelegateMap->Find(DelegateName))
		{
			if (FMulticastScriptDelegate* MulticastDelegate = MulticastDelegatePtr->Get())
			{
				MulticastDelegate->Clear();
			}
			DelegateMap->Remove(DelegateName);
		}
		if (DelegateMap->Num() == 0)
		{
			SparseDelegates.Remove(DelegateOwner);
			if (SparseDelegates.Num() == 0)
			{
				SparseDelegateObjectListener.DisableListener();
			}
		}
	}
}

FAutoConsoleCommand SparseDelegateReportCommand(
	TEXT("SparseDelegateReport"),
	TEXT("Outputs a report of what sparse delegates are bound. SparseDelegateReport [name=<ObjectName>] [delegate=<DelegateName>] [class=<ClassName>] -details"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(FSparseDelegateStorage::SparseDelegateReport)
);

void FSparseDelegateStorage::SparseDelegateReport(const TArray<FString>& Args, UWorld*, FOutputDevice& Ar)
{
	UClass* ObjectType = nullptr;
	FName ObjectName = NAME_None;
	FName DelegateName = NAME_None;
	bool bSummary = true;
	bool bArgumentError = false;

	for (const FString& Arg : Args)
	{
		if (Arg == TEXT("-details"))
		{
			bSummary = false;
		}
		else if (Arg.StartsWith(TEXT("name=")))
		{
			FString ObjName;
			Arg.Split(TEXT("="), nullptr, &ObjName, ESearchCase::CaseSensitive);
			ObjectName = FName(*ObjName, EFindName::FNAME_Find);
			if (ObjectName.IsNone())
			{
				Ar.Logf(ELogVerbosity::Warning, TEXT("Invalid object name"));
				bArgumentError = true;
			}
		}
		else if (Arg.StartsWith(TEXT("delegate=")))
		{
			FString DelegateNameStr;
			Arg.Split(TEXT("="), nullptr, &DelegateNameStr, ESearchCase::CaseSensitive);
			DelegateName = FName(*DelegateNameStr, EFindName::FNAME_Find);
			if (DelegateName.IsNone())
			{
				Ar.Logf(ELogVerbosity::Warning, TEXT("Invalid delegate name"));
				bArgumentError = true;
			}
		}
		else if (Arg.StartsWith(TEXT("class=")))
		{
			FString ClassName;
			Arg.Split(TEXT("="), nullptr, &ClassName, ESearchCase::CaseSensitive);
			ObjectType = UClass::TryFindTypeSlow<UClass>(ClassName);
			if (ObjectType == nullptr)
			{
				Ar.Logf(ELogVerbosity::Warning, TEXT("No class of specified name found."));
				bArgumentError = true;
			}
		}
	}

	if (bArgumentError)
	{
		return;
	}

	TArray<FString> Details;
	uint32 BoundObjects = 0;
	uint32 BoundDelegates = 0;

	{
		UE::TScopeLock SparseDelegateMapLock(SparseDelegateMapCritical);
		for (const TPair<const UObjectBase*, FSparseDelegateMap>& ObjectAnnotation : SparseDelegates)
		{
			const UObject* Object = static_cast<const UObject*>(ObjectAnnotation.Key);
			if (ObjectName.IsNone() || Object->GetFName() == ObjectName)
			{
				if (ObjectType == nullptr || Object->IsA(ObjectType))
				{
					if (!bSummary)
					{
						Details.Add(FString::Printf(TEXT("%s"), *Object->GetPathName()));
					}
					++BoundObjects;
					if (DelegateName.IsNone())
					{
						BoundDelegates += ObjectAnnotation.Value.Num();
						if (!bSummary)
						{
							for (const TPair<FName, TSharedPtr<FMulticastScriptDelegate>>& BoundDelegate : ObjectAnnotation.Value)
							{
								Details.Add(FString::Printf(TEXT("   %s"), *BoundDelegate.Key.ToString()));
							}
						}
					}
					else if (ObjectAnnotation.Value.Contains(DelegateName))
					{
						++BoundDelegates;
					}
				}
			}
		}
	}
	FString SummaryString = "Bound Sparse Delegates";
	if (ObjectType)
	{
		SummaryString += FString::Printf(TEXT(" - Class=%s"), *ObjectType->GetName());
	}
	if (!ObjectName.IsNone())
	{
		SummaryString += FString::Printf(TEXT(" - Name=%s"), *ObjectName.ToString());
	}
	if (!DelegateName.IsNone())
	{
		SummaryString += FString::Printf(TEXT(" - Delegate=%s"), *DelegateName.ToString());
	}

	Ar.Logf(ELogVerbosity::Log, TEXT("%s"), *SummaryString);
	Ar.Logf(ELogVerbosity::Log, TEXT("Objects: %d"), BoundObjects);
	Ar.Logf(ELogVerbosity::Log, TEXT("Delegates: %d"), BoundDelegates);
	Ar.Logf(ELogVerbosity::Log, TEXT("------------------------------------------------------------------------"));
	for (const FString& Detail : Details)
	{
		Ar.Logf(ELogVerbosity::Log, TEXT("%s"), *Detail);
	}
}