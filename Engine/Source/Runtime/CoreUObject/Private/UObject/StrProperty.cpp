// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/StrProperty.h"

// Include StrProperty.cpp.inl's includes before defining the macros, in case the macros 'poison' other headers or there are re-entrant includes.
#include "UObject/StrPropertyIncludes.cpp.inl"

#define UE_STRPROPERTY_CLASS                FStrProperty
#define UE_STRPROPERTY_STRINGTYPE           FString
#define UE_STRPROPERTY_CASTCLASSFLAG        CASTCLASS_FStrProperty
#define UE_STRPROPERTY_PROPERTYPARAMSSTRUCT FStrPropertyParams
	#include "UObject/StrProperty.cpp.inl"
#undef UE_STRPROPERTY_PROPERTYPARAMSSTRUCT
#undef UE_STRPROPERTY_CASTCLASSFLAG
#undef UE_STRPROPERTY_STRINGTYPE
#undef UE_STRPROPERTY_CLASS
