// Copyright Epic Games, Inc. All Rights Reserved.

/*********************************************************************************************************************
 * NOTICE                                                                                                            *
 *                                                                                                                   *
 * This file is not intended to be included directly - it will be included by other .cpp files which have predefined *
 * some macros to be expanded within this file.  As such, it does not have a #pragma once as it is intended to be    *
 * included multiple times with different macro definitions.                                                         *
 *                                                                                                                   *
 * #includes needed to compile this file need to be specified in StringIncludes.cpp.inl file rather than here.       *
 *********************************************************************************************************************/
#define UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE

#ifndef UE_STRPROPERTY_CLASS
	#error "StrProperty.cpp.inl should only be included after defining UE_STRPROPERTY_CLASS"
#endif
#ifndef UE_STRPROPERTY_STRINGTYPE
	#error "StrProperty.cpp.inl should only be included after defining UE_STRPROPERTY_STRINGTYPE"
#endif
#ifndef UE_STRPROPERTY_CASTCLASSFLAG
	#error "StrProperty.cpp.inl should only be included after defining UE_STRPROPERTY_CASTCLASSFLAG"
#endif
#ifndef UE_STRPROPERTY_PROPERTYPARAMSSTRUCT
	#error "StrProperty.cpp.inl should only be included after defining UE_STRPROPERTY_PROPERTYPARAMSSTRUCT"
#endif

IMPLEMENT_FIELD(UE_STRPROPERTY_CLASS)

UE_STRPROPERTY_CLASS::UE_STRPROPERTY_CLASS(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

UE_STRPROPERTY_CLASS::UE_STRPROPERTY_CLASS(FFieldVariant InOwner, const UECodeGen_Private::UE_STRPROPERTY_PROPERTYPARAMSSTRUCT& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
UE_STRPROPERTY_CLASS::UE_STRPROPERTY_CLASS(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA

EConvertFromTypeResult UE_STRPROPERTY_CLASS::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	if (GetClass()->GetFName() == Tag.Type)
	{
		return EConvertFromTypeResult::UseSerializeItem;
	}

	// Convert serialized text to string.
	if (Tag.Type==NAME_TextProperty)
	{ 
		FText Text;
		Slot << Text;
		const UE_STRPROPERTY_STRINGTYPE String(FTextInspector::GetSourceString(Text) ? *FTextInspector::GetSourceString(Text) : TEXT(""));
		SetPropertyValue_InContainer(Data, String, Tag.ArrayIndex);
	}
	else if (Tag.Type==NAME_StrProperty)
	{
		FString SavedStr;
		Slot << SavedStr;
		const UE_STRPROPERTY_STRINGTYPE ConvertedString(SavedStr);
		SetPropertyValue_InContainer(Data, ConvertedString, Tag.ArrayIndex);
	}
	else if (Tag.Type==NAME_Utf8StrProperty)
	{
		FUtf8String SavedStr;
		Slot << SavedStr;
		const UE_STRPROPERTY_STRINGTYPE ConvertedString(SavedStr);
		SetPropertyValue_InContainer(Data, ConvertedString, Tag.ArrayIndex);
	}
	else if (Tag.Type==NAME_AnsiStrProperty)
	{
		FAnsiString SavedStr;
		Slot << SavedStr;
		const UE_STRPROPERTY_STRINGTYPE ConvertedString(SavedStr);
		SetPropertyValue_InContainer(Data, ConvertedString, Tag.ArrayIndex);
	}
	else
	{
		return EConvertFromTypeResult::CannotConvert;
	}

	return EConvertFromTypeResult::Converted;
}

void UE_STRPROPERTY_CLASS::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	UE_STRPROPERTY_STRINGTYPE StringValue;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		GetValue_InContainer(PropertyValueOrContainer, &StringValue);
	}
	else
	{
		StringValue = *(UE_STRPROPERTY_STRINGTYPE*)PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType);
	}

	if (!(PortFlags & PPF_Delimited))
	{
		ValueStr += StringValue;
	}
	else if ( StringValue.Len() > 0 )
	{
		ValueStr += UE_STRPROPERTY_STRINGTYPE::Printf( CHARTEXT(UE_STRPROPERTY_STRINGTYPE::FmtCharType, "\"%s\""), *StringValue.ReplaceCharWithEscapedChar() );
	}
	else
	{
		ValueStr += CHARTEXT(UE_STRPROPERTY_STRINGTYPE::ElementType, "\"\"");
	}
}
const TCHAR* UE_STRPROPERTY_CLASS::ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const
{
	UE_STRPROPERTY_STRINGTYPE ImportedText;
	if( !(PortFlags & PPF_Delimited) )
	{
		ImportedText = Buffer;

		// in order to indicate that the value was successfully imported, advance the buffer past the last character that was imported
		Buffer += FCString::Strlen(Buffer);
	}
	else
	{
		// require quoted string here
		if (*Buffer != TCHAR('"'))
		{
			ErrorText->Logf(TEXT("Missing opening '\"' in string property value: %s"), Buffer);
			return NULL;
		}
		const TCHAR* Start = Buffer;
		FString Temp;
		Buffer = FPropertyHelpers::ReadToken(Buffer, Temp);
		if (Buffer == NULL)
		{
			return NULL;
		}
		if (Buffer > Start && Buffer[-1] != TCHAR('"'))
		{
			ErrorText->Logf(TEXT("Missing terminating '\"' in string property value: %s"), Start);
			return NULL;
		}
		ImportedText = UE_STRPROPERTY_STRINGTYPE(MoveTemp(Temp));
	}
	if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
	{
		SetValue_InContainer(ContainerOrPropertyPtr, ImportedText);
	}
	else
	{
		*(UE_STRPROPERTY_STRINGTYPE*)PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType) = MoveTemp(ImportedText);
	}
	return Buffer;
}

uint32 UE_STRPROPERTY_CLASS::GetValueTypeHashInternal(const void* Src) const
{
	return GetTypeHash(*(const UE_STRPROPERTY_STRINGTYPE*)Src);
}
