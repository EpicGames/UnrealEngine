// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*******************************************************************************************************
 * NOTICE                                                                                              *
 *                                                                                                     *
 * This file is not intended to be included directly - it is only intended to contain the includes for *
 * StrPropetryIncludes.cpp.inl.                                                                        *
 *******************************************************************************************************/

#ifdef UE_STRPROPERTY_CLASS
	#error "StrPropertyIncludes.cpp.inl should not be included after defining UE_STRPROPERTY_CLASS"
#endif
#ifdef UE_STRPROPERTY_STRINGTYPE
	#error "StrPropertyIncludes.cpp.inl should not be included after defining UE_STRPROPERTY_STRINGTYPE"
#endif

#include "CoreMinimal.h"
#include "Containers/AnsiString.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyPortFlags.h"
#include "Misc/StringBuilder.h"
#include "Misc/AsciiSet.h"
