// Copyright Epic Games, Inc. All Rights Reserved.


#include "UObject/StrongObjectPtr.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Object.h"

namespace UEStrongObjectPtr_Private
{
	void ReleaseUObject(const UObject* InObject)
	{
		if (!GExitPurge)
		{
			InObject->ReleaseRef();
		}
	}
}
