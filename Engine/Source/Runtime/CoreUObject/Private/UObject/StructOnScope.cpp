// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/StructOnScope.h"
#include "UObject/Package.h"

FStructOnScope::FStructOnScope()
	: SampleStructMemory(nullptr)
	, OwnsMemory(false)
{
}

FStructOnScope::FStructOnScope(const UStruct* InScriptStruct)
	: ScriptStruct(InScriptStruct)
	, SampleStructMemory(nullptr)
	, OwnsMemory(false)
{
	Initialize();
}

FStructOnScope::FStructOnScope(const UStruct* InScriptStruct, uint8* InData)
	: ScriptStruct(InScriptStruct)
	, SampleStructMemory(InData)
	, OwnsMemory(false)
{
}

FStructOnScope::FStructOnScope(FStructOnScope&& InOther)
{
	ScriptStruct = InOther.ScriptStruct;
	SampleStructMemory = InOther.SampleStructMemory;
	OwnsMemory = InOther.OwnsMemory;

	InOther.OwnsMemory = false;
	InOther.Reset();
}

FStructOnScope::~FStructOnScope()
{
	Destroy();
}


UPackage* FStructOnScope::GetPackage() const
{
	return Package.Get();
}

void FStructOnScope::SetPackage(UPackage* InPackage)
{
	Package = InPackage;
}

void FStructOnScope::AddReferencedObjects(FReferenceCollector& Collector)
{
	if (!SampleStructMemory)
	{
		return;
	}
	
	// Unused result of cast just to confirm type
	// Must use GetEvenIfUnreachable to resolve not-yet-scanned structures during GC where all objects are initially marked unreachable.
	if (const UScriptStruct* ResolvedPtr = CastChecked<UScriptStruct>(ScriptStruct.GetEvenIfUnreachable()))
	{
		TWeakObjectPtr<const UScriptStruct>& StructPointer = reinterpret_cast<TWeakObjectPtr<const UScriptStruct>&>(ScriptStruct);
		Collector.AddReferencedObjects(StructPointer, SampleStructMemory);	
	}
}