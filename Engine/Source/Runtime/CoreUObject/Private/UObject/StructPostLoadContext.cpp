// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/StructPostLoadContext.h"

#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/UnrealType.h"

#if WITH_EDITOR
namespace UE
{
	static FStructPostLoadContext GStructPostLoadContext;
	FStructPostLoadContext& FStructPostLoadContext::Get()
	{
		return GStructPostLoadContext;
	}

	bool FStructPostLoadContext::RequestPostLoad(TNotNull<const FUObjectSerializeContext*> Context)
	{
		if (Context->SerializedObject)
		{
			FAnnotation Annotation = Annotations.GetAndRemoveAnnotation(Context->SerializedObject);
			if (Context->SerializedPropertyPath.GetSegmentCount() == 1
				&& Context->SerializedPropertyPath.GetSegment(0).Index == INDEX_NONE)
			{
				Annotation.StructPaths.Add(Context->SerializedPropertyPath);
			}
			else
			{
				Annotation.bHasInvalidPropertyPathName = true;
			}

			Annotations.AddAnnotation(Context->SerializedObject, MoveTemp(Annotation));
			return true;
		}
		return false;
	}

	void FStructPostLoadContext::OnPostLoad(TNotNull<UObjectBase*> Object)
	{
		const UE::FStructPostLoadContext::FAnnotation Annotation = Annotations.GetAndRemoveAnnotation(Object);
		if (!Annotation.IsDefault())
		{
			auto CallPostLoad = [](const FStructProperty* Property, void* PropertyValue)
			{
				UScriptStruct::ICppStructOps* TheCppStructOps = Property->Struct->GetCppStructOps();
				check(TheCppStructOps);
				TheCppStructOps->PostLoad(PropertyValue);
			};

			if (!Annotation.bHasInvalidPropertyPathName)
			{
				for (const FPropertyPathName& PathName : Annotation.StructPaths)
				{
					const FProperty* Property = Object->GetClass()->FindPropertyByName(PathName.GetSegment(0).Name);
					const FStructProperty* StructProperty = CastField<const FStructProperty>(Property);
					if (ensure(StructProperty))
					{
						UObjectBase* Container = Object;
						void* PropertyValue = StructProperty->ContainerPtrToValuePtr<void>(Container);
						CallPostLoad(StructProperty, PropertyValue);
					}
				}
			}
			else
			{
				for (TPropertyValueIterator<FStructProperty> It(Object->GetClass(), Object); It; ++It)
				{
					const FStructProperty* Property = It.Key();
					if (Property->Struct->StructFlags & STRUCT_PostLoad)
					{
						void* PropertyValue = (void*)It.Value();
						CallPostLoad(Property, PropertyValue);
					}
				}
			}
		}
	}
}// namespace UE
#endif // WITH_EDITOR