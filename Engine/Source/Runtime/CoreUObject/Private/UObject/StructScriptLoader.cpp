// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/StructScriptLoader.h"
#include "HAL/ThreadSingleton.h"
#include "Misc/CoreMisc.h"
#include "UObject/LinkerLoad.h"
#include "Serialization/ArchiveScriptReferenceCollector.h"
#include "UObject/Package.h"
#include "UObject/PropertyProxyArchive.h"

/*******************************************************************************
 * FDeferredScriptTracker
 ******************************************************************************/

/** 
 * Pairs a FStructScriptLoader with the target script container (so that the 
 * script can be properly serialized in at a later time).
 */
struct FDeferredScriptLoader : private FStructScriptLoader
{
public:
	FDeferredScriptLoader(const FStructScriptLoader& RHS, UStruct* ScriptContainer);

	/**
	 * If the target script container is still valid, then this will load it 
	 * with script bytecode from the supplied archiver (expects that the 
	 * archiver is the same one that originally attempted to load the script).  
	 * 
	 * @param  Ar    The archiver that this script was originally supposed to load with.
	 * @return True if the target was loaded with new bytecode, otherwise false.
	 */
	bool Resolve(FArchive& Ar);

private:
	/** Kept as a weak pointer in case the target has since been destroyed (since the initial deferral) */
	TWeakObjectPtr<UStruct> TargetScriptContainerPtr;
};

/** 
 * Tracks all deferred script loads (so that they can be resolved at a later 
 * time, via FStructScriptLoader::ResolveDeferredScriptLoads). Utilized to avoid
 * having to load possible cyclic dependencies during class serialization.
 */
struct FDeferredScriptTracker : TThreadSingleton<FDeferredScriptTracker>
{
public:
	FDeferredScriptTracker();

	/**
	 * Stores the target struct along with the serialization offset, script 
	 * size, etc. (so the script can be resolved at a later time).
	 * 
	 * @param  Linker					The loader responsible for serializing in the target struct's script.
	 * @param  TargetScriptContainer    The struct that the script should ultimately be serialized into
	 * @param  ScriptLoader				The script serialization helper that contains info on the script's serializtion offset (buffer size, etc.)
	 */
	void AddDeferredScriptObject(FLinkerLoad* Linker, UStruct* TargetScriptContainer, const FStructScriptLoader& ScriptLoader);

	/**
	 * Goes through every deferred script load associated with the specified 
	 * linker and attempts to resolve each one (will fail to resolve any if the 
	 * linker is still flagged with LOAD_DeferDependencyLoads).
	 * 
	 * @param  Linker    The linker that may have deferred script serialization (possibly for many functions).
	 * @return The number of script loads that were successfully resolved.
	 */
	int32 ResolveDeferredScripts(FLinkerLoad* Linker);

private:
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	/** Used to catch any deferred script loads that are added during a call to ResolveDeferredScripts() */
	FLinkerLoad* ResolvingLinker;
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	/** Keeps track of scripts (and their target containers) that need to be serialized in later */
	TMultiMap<FLinkerLoad*, FDeferredScriptLoader> DeferredScriptLoads;
};

//------------------------------------------------------------------------------
FDeferredScriptTracker::FDeferredScriptTracker()
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	: ResolvingLinker(nullptr)
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
{
}

//------------------------------------------------------------------------------
void FDeferredScriptTracker::AddDeferredScriptObject(FLinkerLoad* Linker, UStruct* TargetScriptContainer, const FStructScriptLoader& ScriptLoader)
{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	check(ResolvingLinker == nullptr);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	DeferredScriptLoads.Add(Linker, FDeferredScriptLoader(ScriptLoader, TargetScriptContainer));
}

//------------------------------------------------------------------------------
int32 FDeferredScriptTracker::ResolveDeferredScripts(FLinkerLoad* Linker)
{
	FArchive& Ar = *Linker;
	if (FStructScriptLoader::ShouldDeferScriptSerialization(Ar))
	{
		return 0;
	}

#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	TGuardValue<FLinkerLoad*> ScopedResolvingLinker(ResolvingLinker, Linker);
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

	TArray<FDeferredScriptLoader> DefferedLinkerScripts;
	DeferredScriptLoads.MultiFind(Linker, DefferedLinkerScripts);
	// remove before we resolve, because a failed call to 
	// FDeferredScriptLoader::Resolve() could insert back into this list 
	DeferredScriptLoads.Remove(Linker);

	int64 const SerializationPosToRestore = Ar.Tell();

	int32 ResolveCount = 0;
	for (FDeferredScriptLoader& DeferredScript : DefferedLinkerScripts)
	{
		if (DeferredScript.Resolve(Ar))
		{
			++ResolveCount;
		}
	}

	Ar.Seek(SerializationPosToRestore);
	return ResolveCount;
}

//------------------------------------------------------------------------------
FDeferredScriptLoader::FDeferredScriptLoader(const FStructScriptLoader& RHS, UStruct* TargetScriptContainer)
	: FStructScriptLoader(RHS)
	, TargetScriptContainerPtr(TargetScriptContainer)
{
}

//------------------------------------------------------------------------------
bool FDeferredScriptLoader::Resolve(FArchive& Ar)
{
	if (UStruct* Target = TargetScriptContainerPtr.Get())
	{
		return LoadStructWithScript(Target, Ar);
	}
	return false;
}

/*******************************************************************************
 * FStructScriptLoader
 ******************************************************************************/

//------------------------------------------------------------------------------
FStructScriptLoader::FStructScriptLoader(UStruct* TargetScriptContainer, FArchive& Ar)
	: BytecodeBufferSize(0)
	, SerializedScriptSize(0)
	, ScriptSerializationOffset(INDEX_NONE)
{
	if (!Ar.IsLoading())
	{
		return;
	}

	Ar << BytecodeBufferSize;
	Ar << SerializedScriptSize;
	
	if (SerializedScriptSize > 0)
	{
		ScriptSerializationOffset = Ar.Tell();
	}

	ClearScriptCode(TargetScriptContainer);
}

//------------------------------------------------------------------------------
bool FStructScriptLoader::IsPrimed()
{
	return (SerializedScriptSize > 0) && (ScriptSerializationOffset != INDEX_NONE);
}

//------------------------------------------------------------------------------
bool FStructScriptLoader::ShouldDeferScriptSerialization(FArchive& Ar)
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (Ar.IsLoading() && Ar.IsPersistent())
	{
		if (auto Linker = Cast<FLinkerLoad>(Ar.GetLinker()))
		{
			return ((Linker->LoadFlags & LOAD_DeferDependencyLoads) != 0);
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	return false;
}

//------------------------------------------------------------------------------
bool FStructScriptLoader::LoadStructWithScript(UStruct* DestScriptContainer, FArchive& Ar, bool bAllowDeferredSerialization)
{
	if (!Ar.IsLoading() || !IsPrimed() || GIsDuplicatingClassForReinstancing)
	{
		return false;
	}

	bool const bIsLinkerLoader = Ar.IsPersistent() && (Ar.GetLinker() != nullptr);
	int64 const ScriptEndOffset = ScriptSerializationOffset + SerializedScriptSize;

	// to help us move development forward (and not have to support ancient 
	// script code), we define a minimum script version
	bool bSkipScriptSerialization = (Ar.UEVer() < VER_MIN_SCRIPTVM_UE4) || (Ar.LicenseeUEVer() < VER_MIN_SCRIPTVM_LICENSEEUE4);
#if WITH_EDITOR
	if (!bSkipScriptSerialization)
	{
		// Always serialize script for cooked packages loaded in the editor
		if (!(GAllowCookedDataInEditorBuilds && Ar.GetLinker() && Ar.GetLinker()->LinkerRoot && Ar.GetLinker()->LinkerRoot->bIsCookedForEditor))
		{
			static const FBoolConfigValueHelper SkipByteCodeHelper(TEXT("StructSerialization"), TEXT("SkipByteCodeSerialization"));
			// in editor builds, we're going to regenerate the bytecode anyways, so it
			// is a waste of cycles to try and serialize it in
			bSkipScriptSerialization |= (bool)SkipByteCodeHelper;
		}
	}
#endif // WITH_EDITOR
	bSkipScriptSerialization &= bIsLinkerLoader; // to keep consistent with old UStruct::Serialize() functionality

	if (bSkipScriptSerialization)
	{
		int32 TrackedBufferSize = BytecodeBufferSize;
		BytecodeBufferSize = 0; // temporarily clear so that ClearScriptCode() doesn't leave Class->Script with anything allocated
		ClearScriptCode(DestScriptContainer);
		BytecodeBufferSize = TrackedBufferSize;

		// we have to at least move the archiver forward, so it is positioned 
		// where it expects to be (as if we read in the script)
		Ar.Seek(ScriptEndOffset);
		return false;
	}

	bAllowDeferredSerialization &= bIsLinkerLoader;
	if (bAllowDeferredSerialization && ShouldDeferScriptSerialization(Ar))
	{
		FLinkerLoad* Linker = CastChecked<FLinkerLoad>(Ar.GetLinker());
		FDeferredScriptTracker::Get().AddDeferredScriptObject(Linker, DestScriptContainer, *this);

		// we have to at least move the archiver forward, so it is positioned 
		// where it expects to be (as if we read in the script)
		Ar.Seek(ScriptEndOffset);
		return false;
	}

	Ar.Seek(ScriptSerializationOffset);
	if (bIsLinkerLoader)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// ScriptSHAHash is no longer supported; delete this if block when deleting it.
		auto LinkerLoad = CastChecked<FLinkerLoad>(Ar.GetLinker());

		TArray<uint8> ShaScriptBuffer;
		ShaScriptBuffer.AddUninitialized(SerializedScriptSize);

		Ar.Serialize(ShaScriptBuffer.GetData(), SerializedScriptSize);
		ensure(ScriptEndOffset == Ar.Tell());
		LinkerLoad->UpdateScriptSHAKey(ShaScriptBuffer);

		Ar.Seek(ScriptSerializationOffset);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	DestScriptContainer->Script.Empty(BytecodeBufferSize);
	DestScriptContainer->Script.AddUninitialized(BytecodeBufferSize);

	int32 BytecodeIndex = 0;
	{
		FPropertyProxyArchive PropertyAr(Ar, BytecodeIndex, DestScriptContainer);
		while (BytecodeIndex < BytecodeBufferSize)
		{
			DestScriptContainer->SerializeExpr(BytecodeIndex, PropertyAr);
		}

		if (PropertyAr.UnresolvedProperties.Num())
		{
			DestScriptContainer->SetUnresolvedScriptProperties(PropertyAr.UnresolvedProperties);
		}
		else
		{
			// Make sure there's no stale properties in the array
			DestScriptContainer->DeleteUnresolvedScriptProperties();
		}
	}
	ensure(ScriptEndOffset == Ar.Tell());
	checkf(BytecodeIndex == BytecodeBufferSize, TEXT("'%s' script expression-count mismatch; Expected: %i, Got: %i"), *DestScriptContainer->GetName(), BytecodeBufferSize, BytecodeIndex);

	if (!GUObjectArray.IsDisregardForGC(DestScriptContainer))
	{
		DestScriptContainer->ScriptAndPropertyObjectReferences.Empty();
		DestScriptContainer->CollectBytecodeReferencedObjects(MutableView(DestScriptContainer->ScriptAndPropertyObjectReferences));
	}

	// success! (we filled the target with serialized script code)
	return true;
}

//------------------------------------------------------------------------------
int32 FStructScriptLoader::ResolveDeferredScriptLoads(FLinkerLoad* Linker)
{
	return FDeferredScriptTracker::Get().ResolveDeferredScripts(Linker);
}

//------------------------------------------------------------------------------
void FStructScriptLoader::ClearScriptCode(UStruct* ScriptContainer)
{
	ScriptContainer->Script.Empty(BytecodeBufferSize);
	ScriptContainer->ScriptAndPropertyObjectReferences.Empty();
}
