// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/*******************************************************************************
 * FStructScriptLoader
 ******************************************************************************/

/**  
 * Utility class to aid in struct (UFunction) script serialization. Will defer 
 * or skip script loading (if deemed necessary).
 */ 
class FStructScriptLoader
{
public:	
	/**
	 * Caches data regarding the script's serialized form (archiver offset, 
	 * serialized size, etc.), so that given the same archiver later (in 
	 * LoadStructWithScript) it can seek and serialize the target's bytecode.
	 *
	 * NOTE: This expects that the supplied archiver is already positioned to 
	 *       read the start of the script's "header" (the script's BytecodeBufferSize).
	 * 
	 * @param  TargetScriptContainer    The target struct (will have its current script cleared).
	 * @param  Loader					The archiver that holds serialized script data (expected to be seeked to the proper place). 
	 */
	FStructScriptLoader(UStruct* TargetScriptContainer, FArchive& Loader);

	/**
	 * Checks to see if this was created from a valid archiver with script code
	 * to serialize in.
	 * 
	 * @return True if the cached serialization info is valid, otherwise false.
	 */
	bool IsPrimed();

	/**
	 * Can be used to determine if the specified archiver wants possible 
	 * dependency load points (such as bytecode) deferred (until after class 
	 * serialization).
	 * 
	 * @param  Ar    The archiver to check.
	 * @return True if the specified archiver has LOAD_DeferDependencyLoads set, otherwise false.
	 */
	static bool ShouldDeferScriptSerialization(FArchive& Ar);

	/**
	 * Attempts to load the specified target struct with bytecode contained in 
	 * the supplied archiver (expects the archiver to be the same one that this
	 * was created from).
	 *
	 * NOTE: Serialization could be skipped if: 1) this isn't properly "primed",
	 *       2) we've opted to skip bytecode serialization (for editor builds), 
	 *       or 3) the Loader wishes to have dependency link points deferred 
	 *       (unless bAllowDeferredSerialization is set false)
	 * 
	 * @param  DestScriptContainer			The target struct that you want script code loaded in to.	
	 * @param  Loader						The archiver that holds the serialized script code meant for the target.
	 * @param  bAllowDeferredSerialization	If false, then serialization will be forced through regardless of ShouldDeferScriptSerialization().
	 * @return 
	 */
	bool LoadStructWithScript(UStruct* DestScriptContainer, FArchive& Loader, bool bAllowDeferredSerialization = true);

	/**
	 * Looks for any struct scripts that were deferred as part of a 
	 * LoadStructWithScript() call, and attempts to serialize the original 
	 * targets with the deferred load.
	 * 
	 * @param  Linker    The linker/archiver that the structs were originally meant to be loaded by.
	 * @return The number of structs that were loaded with new bytecode.
	 */
	static int32 ResolveDeferredScriptLoads(FLinkerLoad* Linker);

private:
	/**
	 * Empties the specified struct's Script array, as well as its 
	 * ScriptAndPropertyObjectReferences container.
	 * 
	 * @param  ScriptContainer    The struct you want emptied.
	 */
	void ClearScriptCode(UStruct* ScriptContainer);

	/** Determines the size of the target struct's Script array */
	int32 BytecodeBufferSize;
	/** Denotes how many bytes the serialized script code is, stored in the archiver (NOTE: could be more compact than BytecodeBufferSize) */
	int32 SerializedScriptSize;
	/** Denotes where to offset the archiver to start reading in bytecode */
	int64 ScriptSerializationOffset;
}; 
