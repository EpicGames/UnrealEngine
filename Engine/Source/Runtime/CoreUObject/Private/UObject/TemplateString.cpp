// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/TemplateString.h"

#include "Algo/Count.h"
#include "Internationalization/TextFormatter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TemplateString)

bool FTemplateString::IsValid(const TArray<FString>& InValidArgs) const
{
	constexpr static TCHAR OpenBrace = '{';
	constexpr static TCHAR CloseBrace = '}';
	
	if(InValidArgs.IsEmpty())
	{
		int32 OpenBraceCount = 0;
		int32 CloseBraceCount = 0;
		for(const TCHAR& Char : Template)
		{
			if(Char == OpenBrace)
			{
				++OpenBraceCount;
			}
			else if(Char == CloseBrace)
			{
				++CloseBraceCount;
			}
		}

		// If false, mismatched braces!
		return OpenBraceCount == CloseBraceCount;
	}
	else
	{
		const TSet<FString> ValidArgSet(InValidArgs);

		TArray<FString> ArgsFound;
		const FTextFormat TextFormat(FText::FromString(Template));
		// If false, format string itself is probably malformed
		if(!TextFormat.IsValid())
		{
			return false;
		}
		
		TextFormat.GetFormatArgumentNames(ArgsFound);

		const TSet<FString> ArgsFoundSet(ArgsFound);

		// True if all found arguments exist in ValidArgs
		return ValidArgSet.Includes(ArgsFoundSet);		
	}
}
