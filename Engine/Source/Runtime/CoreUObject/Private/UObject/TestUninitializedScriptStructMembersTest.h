// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"

#include "TestUninitializedScriptStructMembersTest.generated.h"

class UObject;

// Helper struct to test if member initialization tests work properly
USTRUCT()
struct FTestUninitializedScriptStructMembersTest
{
    GENERATED_BODY()

	UPROPERTY(Transient)
	TObjectPtr<UObject> UninitializedObjectReference{NoInit};

	UPROPERTY(Transient)
	TObjectPtr<UObject> InitializedObjectReference = nullptr;

	UPROPERTY(Transient)
	float UnusedValue;
};
