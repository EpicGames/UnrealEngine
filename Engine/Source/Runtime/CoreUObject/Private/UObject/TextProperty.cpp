// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/TextProperty.h"
#include "Internationalization/ITextData.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/Package.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Internationalization/StringTableRegistry.h"
#include "Internationalization/StringTableCore.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"

IMPLEMENT_FIELD(FTextProperty)

FTextProperty::FTextProperty(FFieldVariant InOwner, const FName& InName)
	: Super(InOwner, InName)
{
}

FTextProperty::FTextProperty(FFieldVariant InOwner, const UECodeGen_Private::FTextPropertyParams& Prop)
	: Super(InOwner, (const UECodeGen_Private::FPropertyParamsBaseWithOffset&)Prop)
{
}

#if WITH_EDITORONLY_DATA
FTextProperty::FTextProperty(UField* InField)
	: Super(InField)
{
}
#endif // WITH_EDITORONLY_DATA

EConvertFromTypeResult FTextProperty::ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults)
{
	// Convert serialized string to text.
	if (Tag.Type==NAME_StrProperty)
	{
		FString Str;
		Slot << Str;

		FText Text = FText::FromString(MoveTemp(Str));
		Text.Flags |= ETextFlag::ConvertedProperty;
		
		SetPropertyValue_InContainer(Data, Text, Tag.ArrayIndex);
		return EConvertFromTypeResult::Converted;
	}

	// Convert serialized name to text.
	if (Tag.Type==NAME_NameProperty)
	{
		FName Name;
		Slot << Name;

		FText Text = FText::FromName(Name);
		Text.Flags |= ETextFlag::ConvertedProperty;
		
		SetPropertyValue_InContainer(Data, Text, Tag.ArrayIndex);
		return EConvertFromTypeResult::Converted;
	}

	return EConvertFromTypeResult::UseSerializeItem;
}

bool FTextProperty::Identical_Implementation(const FText& ValueA, const FText& ValueB, uint32 PortFlags)
{
	// We compare the display strings in editor (as we author in the native language)
	return Identical_Implementation(ValueA, ValueB, PortFlags, GIsEditor ? EIdenticalLexicalCompareMethod::DisplayString : EIdenticalLexicalCompareMethod::None);
}

bool FTextProperty::Identical_Implementation(const FText& ValueA, const FText& ValueB, uint32 PortFlags, EIdenticalLexicalCompareMethod LexicalCompareMethod)
{
	// A culture variant text is never equal to a culture invariant text
	// A transient text is never equal to a non-transient text
	// An empty text is never equal to a non-empty text
	// Text from a string table is never equal to text not from a string table
	if (ValueA.IsCultureInvariant() != ValueB.IsCultureInvariant() || ValueA.IsTransient() != ValueB.IsTransient() || ValueA.IsEmpty() != ValueB.IsEmpty() || ValueA.IsFromStringTable() != ValueB.IsFromStringTable())
	{
		return false;
	}

	// If both texts are empty (see the above check), then they must be equal
	if (ValueA.IsEmpty())
	{
		return true;
	}

	// String table entries should only be considered equal if they're using the same string table and key.
	if (ValueA.IsFromStringTable())
	{
		FName ValueAStringTableId;
		FTextKey ValueAStringTableEntryKey;
		FTextInspector::GetTableIdAndKey(ValueA, ValueAStringTableId, ValueAStringTableEntryKey);

		FName ValueBStringTableId;
		FTextKey ValueBStringTableEntryKey;
		FTextInspector::GetTableIdAndKey(ValueB, ValueBStringTableId, ValueBStringTableEntryKey);

		return ValueAStringTableId == ValueBStringTableId && ValueAStringTableEntryKey == ValueBStringTableEntryKey;
	}

	// If both texts share the same pointer, then they must be equal
	if (ValueA.IdenticalTo(ValueB))
	{
		return true;
	}

	// We compare the display strings if asked
	// We compare the display string for culture invariant and transient texts as they don't have an identity
	if (LexicalCompareMethod == EIdenticalLexicalCompareMethod::DisplayString || ValueA.IsCultureInvariant() || ValueA.IsTransient())
	{
		return FTextInspector::GetDisplayString(ValueA).Equals(FTextInspector::GetDisplayString(ValueB), ESearchCase::CaseSensitive);
	}
	
	// We compare the source strings if asked
	if (LexicalCompareMethod == EIdenticalLexicalCompareMethod::SourceString)
	{
		return FTextInspector::GetSourceString(ValueA)->Equals(*FTextInspector::GetSourceString(ValueB), ESearchCase::CaseSensitive);
	}

	// If we got this far then the texts don't share the same pointer, which means that they can't share the same identity
	return false;
}

bool FTextProperty::Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const
{
	const TCppType ValueA = GetPropertyValue(A);
	if ( B )
	{
		const TCppType ValueB = GetPropertyValue(B);
		return Identical_Implementation(ValueA, ValueB, PortFlags);
	}

	return FTextInspector::GetDisplayString(ValueA).IsEmpty();
}

void FTextProperty::SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const
{
	TCppType* TextPtr = GetPropertyValuePtr(Value);
	Slot << *TextPtr;
}

void FTextProperty::ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	// get current value:
	FText TextValue;
	if (PropertyPointerType == EPropertyPointerType::Container && HasGetter())
	{
		GetValue_InContainer(PropertyValueOrContainer, &TextValue);
	}
	else
	{
		TextValue = GetPropertyValue(PointerToValuePtr(PropertyValueOrContainer, PropertyPointerType));
	}

	// export based on flags:
	auto WriteSimpleTextToBuffer = [&ValueStr, &TextValue, PortFlags]()
	{
		if (PortFlags & PPF_Delimited)
		{
			ValueStr += TEXT("\"");
			ValueStr += TextValue.ToString();
			ValueStr += TEXT("\"");
		}
		else
		{
			ValueStr += TextValue.ToString();
		}
	};

	auto WriteComplexTextToBuffer = [&ValueStr, &TextValue, PortFlags]()
	{
		FTextStringHelper::WriteToBuffer(ValueStr, TextValue, !!(PortFlags & PPF_Delimited));
	};

	if (PortFlags & PPF_ForDiff)
	{
		if (TextValue.IsCultureInvariant() || TextValue.IsFromStringTable())
		{
			// Invariant and StringTable text values still need to export in their complex text form for diffing to avoid invariant and non-invariant text, 
			// and different StringTable values that have a different ID but the same display string from being considered identical
			WriteComplexTextToBuffer();
		}
		else
		{
			// Any other kind of text should exported as a simple string, to avoid ID changes from instancing text into other packages from being treated as significant
			WriteSimpleTextToBuffer();
		}
	}
	else if (PortFlags & PPF_PropertyWindow)
	{
		WriteSimpleTextToBuffer();
	}
	else
	{
		WriteComplexTextToBuffer();
	}
}

const TCHAR* FTextProperty::ImportText_Internal( const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText ) const
{
	FText ImportedText;

	FString TextNamespace;
	if (Parent && HasAnyPropertyFlags(CPF_Config))
	{
		const bool bPerObject = UsesPerObjectConfig(Parent);
		if (bPerObject)
		{
			FString PathNameString;
			UPackage* ParentOutermost = Parent->GetOutermost();
			if (ParentOutermost == GetTransientPackage())
			{
				PathNameString = Parent->GetName();
			}
			else
			{
				PathNameString = Parent->GetPathName(ParentOutermost);
			}
			TextNamespace = PathNameString + TEXT(" ") + Parent->GetClass()->GetName();

			Parent->OverridePerObjectConfigSection(TextNamespace);
		}
		else
		{
			const bool bGlobalConfig = HasAnyPropertyFlags(CPF_GlobalConfig);
			UClass* ConfigClass = bGlobalConfig ? GetOwnerClass() : Parent->GetClass();
			TextNamespace = ConfigClass->GetPathName();
		}
	}

	FString PackageNamespace;
#if USE_STABLE_LOCALIZATION_KEYS
	if (GIsEditor && !(PortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
	{
		PackageNamespace = TextNamespaceUtil::EnsurePackageNamespace(Parent);
	}
#endif // USE_STABLE_LOCALIZATION_KEYS

	const TCHAR* Result = FTextStringHelper::ReadFromBuffer(Buffer, ImportedText, *TextNamespace, *PackageNamespace, !!(PortFlags & PPF_Delimited));
	if (Result)
	{
		if (PropertyPointerType == EPropertyPointerType::Container && HasSetter())
		{
			SetValue_InContainer(ContainerOrPropertyPtr, ImportedText);
		}
		else
		{
			FText* TextPtr = GetPropertyValuePtr(PointerToValuePtr(ContainerOrPropertyPtr, PropertyPointerType));
			*TextPtr = ImportedText;
		}
	}
	return Result;
}
