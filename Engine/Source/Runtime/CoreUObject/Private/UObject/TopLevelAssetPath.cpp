// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/TopLevelAssetPath.h"

#include "Containers/UnrealString.h"

#include "Misc/AsciiSet.h"
#include "Misc/Optional.h"
#include "Misc/OutputDeviceNull.h"
#include "Misc/PackageName.h"
#include "Misc/RedirectCollector.h"
#include "Serialization/CompactBinaryWriter.h"
#include "UObject/CoreRedirects.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectThreadContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TopLevelAssetPath)

void FTopLevelAssetPath::AppendString(FWideStringBuilderBase& Builder) const
{
	if (!IsNull())
	{
		Builder << PackageName;
		if (!AssetName.IsNone())
		{
			Builder << '.' << AssetName;
		}
	}
}

void FTopLevelAssetPath::AppendString(FUtf8StringBuilderBase& Builder) const
{
	if (!IsNull())
	{
		Builder << PackageName;
		if (!AssetName.IsNone())
		{
			Builder << '.' << AssetName;
		}
	}
}

void FTopLevelAssetPath::AppendString(FString& Builder) const
{
	if (!IsNull())
	{
		PackageName.AppendString(Builder);
		if( !AssetName.IsNone() )
		{
			Builder += TEXT(".");
			AssetName.AppendString(Builder);
		}
	}
}

FString FTopLevelAssetPath::ToString() const
{
	TStringBuilder<256> Builder;
	AppendString(Builder);
	return FString(Builder);
}

void FTopLevelAssetPath::ToString(FString& OutString) const
{
	OutString.Reset();
	AppendString(OutString);
}

bool FTopLevelAssetPath::RemapPackage(FName OldPackageName, FName NewPackageName)
{
	if (PackageName == OldPackageName)
	{
		PackageName = NewPackageName;
		return true;
	}
	return false;
}

bool FTopLevelAssetPath::TrySetPath(FName InPackageName, FName InAssetName)
{
	PackageName = InPackageName;
	AssetName = InAssetName;
	return !PackageName.IsNone();
}

bool FTopLevelAssetPath::TrySetPath(const UObject* InObject)
{
	if (InObject == nullptr)
	{
		Reset();
		return false;
	}
	else if (!InObject->GetOuter())
	{
		check(Cast<UPackage>(InObject) != nullptr);
		PackageName = InObject->GetFName();
		AssetName = FName();
		return true;
	}
	else if (InObject->GetOuter()->GetOuter() != nullptr)
	{
		Reset();
		return false;
	}
	else
	{
		PackageName = InObject->GetOuter()->GetFName();
		AssetName = InObject->GetFName();
		return true;
	}
}

bool FTopLevelAssetPath::TrySetPath(FWideStringView Path)
{
	if (Path.IsEmpty() || Path.Equals(TEXT("None"), ESearchCase::CaseSensitive))
	{
		// Empty path, just empty the pathname.
		Reset();
		return false;
	}
	else
	{
		if (Path[0] != '/' || Path[Path.Len() - 1] == '\'')
		{
			// Possibly an ExportText path. Trim the ClassName.
			Path = FPackageName::ExportTextPathToObjectPath(Path);

			if (Path.IsEmpty() || Path[0] != '/')
			{
				ensureAlwaysMsgf(false, TEXT("Short asset name used to create FTopLevelAssetPath: \"%.*s\""), Path.Len(), Path.GetData());
				Reset();
				return false;
			}
		}

		FAsciiSet Delim1(".");
		FWideStringView PackageNameView = FAsciiSet::FindPrefixWithout(Path, Delim1);
		if (PackageNameView.IsEmpty())
		{
			Reset();
			return false;
		}

		FWideStringView AssetNameView = Path.Mid(PackageNameView.Len() + 1);
		if (AssetNameView.IsEmpty())
		{
			// Reference to a package itself. Iffy, but supported for legacy usage of FSoftObjectPath.
			PackageName = FName(PackageNameView);
			AssetName = FName();
			return true;
		}

		FAsciiSet Delim2(TEXT("." SUBOBJECT_DELIMITER_ANSI));
		if (FAsciiSet::HasAny(AssetNameView, Delim2))
		{
			// Subobject path or is malformed and contains multiple '.' delimiters.
			Reset();
			return false;
		}

		PackageName = FName(PackageNameView);
		AssetName = FName(AssetNameView);
		return true;
	}
}

bool FTopLevelAssetPath::TrySetPath(FUtf8StringView Path)
{
	TStringBuilder<FName::StringBufferSize> Wide;
	Wide << Path;
	return TrySetPath(Wide);
}

bool FTopLevelAssetPath::TrySetPath(FAnsiStringView Path)
{
	TStringBuilder<FName::StringBufferSize> Wide;
	Wide << Path;
	return TrySetPath(Wide);
}

bool FTopLevelAssetPath::ExportTextItem(FString& ValueStr, FTopLevelAssetPath const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const
{
	if (!IsNull())
	{
		if (PortFlags & PPF_Delimited)
		{
			ValueStr += TEXT("\"");
			ValueStr += ToString().ReplaceQuotesWithEscapedQuotes();
			ValueStr += TEXT("\"");
		}
		else
		{
			ValueStr += ToString();
		}
	}
	else
	{
		ValueStr += TEXT("None");
	}
	return true;
}

namespace UE::TopLevelAssetPath::Private
{
	void SkipWhitespace(const TCHAR*& Str)
	{
		while(FChar::IsWhitespace(*Str))
		{
			Str++;
		}
	}

	// Copied from FNameProperty::ImportText_Internal
	const TCHAR* ImportName(
		const TCHAR* Buffer,
		FName& OutName,
		int32 PortFlags
	) 
	{
		FName ImportedName;
		if (!(PortFlags & PPF_Delimited))
		{
			ImportedName = FName(Buffer);		

			// in order to indicate that the value was successfully imported, advance the buffer past the last character that was imported
			Buffer += FCString::Strlen(Buffer);
		}
		else
		{
			TStringBuilder<256> Token;
			Buffer = FPropertyHelpers::ReadToken(Buffer, /* out */ Token, true);
			if (!Buffer)
			{
				return nullptr;
			}

			ImportedName = FName(Token);
		}

		OutName = ImportedName;
		return Buffer;
	}

	// Stripped down version of FProperty::ImportSingleProperty that only supports loading the two FName fields we need 
	const TCHAR* ImportSingleProperty(
		const TCHAR* Str,
		TOptional<FName>& OutPackageName, 
		TOptional<FName>& OutAssetName, 
		UObject* SubobjectOuter,
		int32 PortFlags,
		FOutputDevice* Warn
	)
	{
		constexpr FAsciiSet Whitespaces(" \t");
		constexpr FAsciiSet Delimiters("=([.");
	
		static const FName NAME_PackageName("PackageName");
		static const FName NAME_AssetName("AssetName");

		// strip leading whitespace
		const TCHAR* Start = FAsciiSet::Skip(Str, Whitespaces);
		// find first delimiter
		Str = FAsciiSet::FindFirstOrEnd(Start, Delimiters);
		// check if delimiter was found...
		if (*Str)
		{
			// strip trailing whitespace
			int32 Len = UE_PTRDIFF_TO_INT32(Str - Start);
			while (Len > 0 && Whitespaces.Contains(Start[Len - 1]))
			{
				--Len;
			}

			const FName PropertyName(Len, Start);
			TOptional<FName>* Destination = nullptr;
			if (PropertyName == NAME_PackageName)
			{
				Destination = &OutPackageName;
			}
			else if (PropertyName == NAME_AssetName)
			{
				Destination = &OutAssetName;
			}
			
			// check to see if this property has already imported data
			if (Destination && Destination->IsSet())
			{
				Warn->Logf(ELogVerbosity::Warning, TEXT("redundant data: %s"), Start);
				return Str;
			}
			else if (Destination == nullptr)
			{
				UE_SUPPRESS(LogExec, Verbose, Warn->Logf(TEXT("Unknown property in TopLevelAssetPath: %s "), Start));
				return Str;
			}

			// strip whitespace before =
			SkipWhitespace(Str);
			if (*Str++ != '=')
			{
				Warn->Logf(ELogVerbosity::Warning, TEXT("Missing '=' in default properties assignment: %s"), Start );
				return Str;
			}
			// strip whitespace after =
			SkipWhitespace(Str);

			if (*Str == TCHAR('\0') || *Str == TCHAR(',') || *Str == TCHAR(')'))
			{
				// Empty value
				return Str;
			}

			const TCHAR* Result = ImportName(Str, Destination->Emplace(NAME_None), PortFlags);
			if (Result == NULL || Result == Str)
			{
				UE_SUPPRESS(LogExec, Verbose, Warn->Logf(TEXT("Unknown property in TopLevelAssetPath: %s "), Start));
			}

			// in the failure case, don't return NULL so the caller can potentially skip less and get values further in the string
			if (Result != NULL)
			{
				Str = Result;
			}
		}
		return Str;
	};

	 // Structure copied from UScriptStruct::ImportText to avoid dependency on UScriptStruct as we load asset registry data early during engine initialization 
	 // Support for this format is deprecated and this code should be removed in 5.2 or 5.3.
	static const TCHAR* ImportFormattedStruct(
		const TCHAR* InBuffer,
		TOptional<FName>& OutPackageName, 
		TOptional<FName>& OutAssetName,
		UObject* OwnerObject, 
		int32 PortFlags,
		FOutputDevice* ErrorText
	)
	{	
		// this keeps track of the number of errors we've logged, so that we can add new lines when logging more than one error
		int32 ErrorCount = 0;
		const TCHAR* Buffer = InBuffer;
		if (*Buffer++ == TCHAR('('))
		{
			// Parse all properties.
			while (*Buffer != TCHAR(')'))
			{
				// parse and import the value
				Buffer = ImportSingleProperty(Buffer, OutPackageName, OutAssetName, OwnerObject, PortFlags | PPF_Delimited, ErrorText);

				// skip any remaining text before the next property value
				SkipWhitespace(Buffer);
				int32 SubCount = 0;
				while (*Buffer && *Buffer != TCHAR('\r') && *Buffer != TCHAR('\n') &&
					(SubCount > 0 || *Buffer != TCHAR(')')) && (SubCount > 0 || *Buffer != TCHAR(',')))
				{
					SkipWhitespace(Buffer);
					if (*Buffer == TCHAR('\"'))
					{
						do
						{
							Buffer++;
						} while (*Buffer && *Buffer != TCHAR('\"') && *Buffer != TCHAR('\n') && *Buffer != TCHAR('\r'));

						if (*Buffer != TCHAR('\"'))
						{
							ErrorText->Logf(TEXT("%sImportText (TopLevelAssetPath): Bad quoted string at: %s"), 
								ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), Buffer);
							return nullptr;
						}
					}
					else if (*Buffer == TCHAR('('))
					{
						SubCount++;
					}
					else if (*Buffer == TCHAR(')'))
					{
						SubCount--;
						if (SubCount < 0)
						{
							ErrorText->Logf(TEXT("%sImportText (TopLevelAssetPath): Too many closing parenthesis in: %s"),
								ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), InBuffer);
							return nullptr;
						}
					}
					Buffer++;
				}
				if (SubCount > 0)
				{
					ErrorText->Logf(TEXT("%sImportText(TopLevelAssetPath): Not enough closing parenthesis in: %s"),
						ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), InBuffer);
					return nullptr;
				}

				// Skip comma.
				if (*Buffer == TCHAR(','))
				{
					// Skip comma.
					Buffer++;
				}
				else if (*Buffer != TCHAR(')'))
				{
					ErrorText->Logf(TEXT("%sImportText (TopLevelAssetPath): Missing closing parenthesis: %s"), 
						ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), InBuffer);
					return nullptr;
				}

				SkipWhitespace(Buffer);
			}

			// Skip trailing ')'.
			Buffer++;
		}
		else
		{
			ErrorText->Logf(TEXT("%sImportText (TopLevelAssetPath): Missing opening parenthesis: %s"),
				ErrorCount++ > 0 ? LINE_TERMINATOR : TEXT(""), InBuffer); //-V547
			return nullptr;
		}
		return Buffer;
	}
}

bool FTopLevelAssetPath::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive)
{
	const TCHAR* OriginalBuffer = Buffer;
	TStringBuilder<256> ImportedPath;

	if (const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, /* out */ ImportedPath, /* dotted names */ true))
	{
		Buffer = NewBuffer;
	}
	else
	{
		return false;
	}

	if (ImportedPath == TEXTVIEW("None"))
	{
		Reset();
		return true;
	}
	else if (*Buffer == TCHAR('('))
	{
		// Blueprints and other utilities may pass in () as a hardcoded value for an empty struct, so treat that like an empty string
		Buffer++;

		if (*Buffer == TCHAR(')'))
		{
			Buffer++;
			Reset();
			return true;
		}
		else
		{
			Buffer--;
			// Attempt to parse classic struct format in case there is some data in that format
			TOptional<FName> NewPackageName, NewAssetName;
			if (const TCHAR* NewBuffer = UE::TopLevelAssetPath::Private::ImportFormattedStruct(Buffer, NewPackageName, NewAssetName, Parent, PortFlags, ErrorText))
			{
				// Emit a warning because this format is deprecated and shouldn't be accepted forever.
				ErrorText->Logf(ELogVerbosity::Warning, TEXT("Struct format for FTopLevelAssetPath is deprecated. Imported struct: %.*s"), 
					(NewBuffer - Buffer), Buffer);
				
				Buffer = NewBuffer;
				TrySetPath(NewPackageName.Get({}), NewAssetName.Get({}));
				return true;
			}
			else
			{
				Buffer = OriginalBuffer;
				return false;
			}
		}
	}

	else if (*Buffer == TCHAR('\''))
	{
		// A ' token likely means we're looking at a path string in the form "Texture2d'/Game/UI/HUD/Actions/Barrel'" and we need to read and append the path part
		// We have to skip over the first ' as FPropertyHelpers::ReadToken doesn't read single-quoted strings correctly, but does read a path correctly
		Buffer++; // Skip the leading '
		ImportedPath.Reset();
		if (const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, /* out */ ImportedPath, /* dotted names */ true))
		{
			Buffer = NewBuffer;
		}
		else
		{
			Buffer = OriginalBuffer;
			return false;
		}
		if (*Buffer++ != TCHAR('\''))
		{
			Buffer = OriginalBuffer;
			return false;
		}
	}

	TrySetPath(ImportedPath);
	return true;
}

bool FTopLevelAssetPath::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		FName Name;
		Slot << Name;
		
		FNameBuilder NameBuilder(Name);
		TrySetPath(NameBuilder.ToView());

		return true;
	}
	
	if (Tag.Type == NAME_StrProperty)
	{
		FString String;
		Slot << String;

		TrySetPath(String);
		
		return true;
	}

	return false;
}

void SerializeForLog(FCbWriter& Writer, const FTopLevelAssetPath& Value)
{
	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("TopLevelAssetPath"));
	Writer.AddString(ANSITEXTVIEW("$text"), WriteToUtf8String<256>(Value));
	Writer.AddString(ANSITEXTVIEW("PackageName"), WriteToUtf8String<256>(Value.GetPackageName()));
	Writer.AddString(ANSITEXTVIEW("AssetName"), WriteToUtf8String<256>(Value.GetAssetName()));
	Writer.EndObject();
}

void FTopLevelAssetPath::WriteCompactBinary(FCbWriter& Writer) const
{
	Writer << WriteToUtf8String<FName::StringBufferSize>(*this).ToView();
}

bool LoadFromCompactBinary(FCbFieldView Field, FTopLevelAssetPath& OutPath)
{
	OutPath = FTopLevelAssetPath(Field.AsString());
	return !Field.HasError();
}

#if WITH_TESTS

#include "Tests/TestHarnessAdapter.h"
#include "Serialization/CompactBinaryValidation.h"

#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif 

// Combine import/export tests

TEST_CASE_NAMED(FTopLevelAssetPathTest, "System::Core::Misc::TopLevelAssetPath", "[ApplicationContextMask][EngineFilter]")
{
	FName PackageName("/Path/To/Package");
	FName AssetName("Asset");

	FString AssetPathString(WriteToString<FName::StringBufferSize>(PackageName, '.', AssetName).ToView());

	FTopLevelAssetPath EmptyPath;
	CHECK_MESSAGE(TEXT("Empty path to string is empty string"), EmptyPath.ToString().IsEmpty());

	FTopLevelAssetPath PackagePath;
	CHECK_FALSE_MESSAGE(TEXT("TrySetPath(NAME_None, NAME_None) fails"), PackagePath.TrySetPath(NAME_None, NAME_None));
	CHECK_MESSAGE(TEXT("TrySetPath(PackageName, NAME_None) succeeds"), PackagePath.TrySetPath(PackageName, NAME_None));
	CHECK_EQUALS(TEXT("PackagePath to string is PackageName"), PackagePath.ToString(), PackageName.ToString());

	FTopLevelAssetPath AssetPath;
	CHECK_MESSAGE(TEXT("TrySetPath(PackageName, AssetName) succeeds"), AssetPath.TrySetPath(PackageName, AssetName));
	CHECK_EQUALS(TEXT("AssetPath to string is PackageName.AssetName"), AssetPath.ToString(), AssetPathString);

	FTopLevelAssetPath EmptyPathFromString;
	CHECK_FALSE_MESSAGE(TEXT("TrySetPath with empty string fails"), EmptyPathFromString.TrySetPath(TEXT("")));
	CHECK_EQUALS(TEXT("Empty path to string is empty string"), EmptyPathFromString.ToString(), FString());

	FTopLevelAssetPath PackagePathFromString;
	CHECK_MESSAGE(TEXT("TrySetPath(PackageName.ToString()) succeeds"), PackagePathFromString.TrySetPath(PackageName.ToString()));
	CHECK_EQUALS(TEXT("PackagePathFromString to string is PackageName"), PackagePathFromString.ToString(), PackageName.ToString());

	FTopLevelAssetPath AssetPathFromString;
	CHECK_MESSAGE(TEXT("TrySetPath(AssetPath) succeeds"), AssetPathFromString.TrySetPath(AssetPathString));
	CHECK_EQUALS(TEXT("AssetPathFromString to string is PackageName.AssetName"), AssetPathFromString.ToString(), AssetPathString);

	FTopLevelAssetPath FailedPath;
	//CHECK_FALSE_MESSAGE(TEXT("TrySetPath with unrooted path string fails"), FailedPath.TrySetPath("UnrootedPackage/Subfolder")); // after ANY_PACKAGE removal this will assert
	CHECK_EQUALS(TEXT("Failed set to string is empty string"), FailedPath.ToString(), FString());

	FTopLevelAssetPath SubObjectPath;
	CHECK_FALSE_MESSAGE(TEXT("TrySetPath with subobject path string fails"), SubObjectPath.TrySetPath("/Path/To/Package.Asset:Subobject"));
	CHECK_EQUALS(TEXT("Failed set to string is empty string"), SubObjectPath.ToString(), FString());

	FTopLevelAssetPath MalformedPath;
	CHECK_FALSE_MESSAGE(TEXT("TrySetPath with malformed path string fails"), MalformedPath.TrySetPath("/Path/To/Package.Asset.Malformed"));
	CHECK_EQUALS(TEXT("Failed set to string is empty string"), MalformedPath.ToString(), FString());

	{ 
		// Temporary output device to swallow expected warning
		FOutputDeviceNull NullOut;
		FTopLevelAssetPath FromStructuredText;
		const TCHAR* StructuredTextBuffer = TEXT("(PackageName=\"/Path/To/Package\", AssetName=\"Asset\")");
		CHECK_MESSAGE(TEXT("FromStructuredText ImportTextItem succeeds"), FromStructuredText.ImportTextItem(StructuredTextBuffer, PPF_None, nullptr, &NullOut, nullptr));
		CHECK_EQUALS(TEXT("FromStructuredText ImportTextItem advances buffer"), *StructuredTextBuffer, '\0');
		CHECK_EQUALS(TEXT("FromStructuredText imports correct path"), FromStructuredText.ToString(), TEXT("/Path/To/Package.Asset"));
	}

	FTopLevelAssetPath FromExportTextPath;
	const TCHAR* ExportTextBuffer = TEXT("/Script/Module.ClassName'/Path/To/Package.Asset'");
	CHECK_MESSAGE(TEXT("FromExportTextPath ImportTextItem succeeds"), FromExportTextPath.ImportTextItem(ExportTextBuffer, PPF_None, nullptr, GLog->Get()));
	CHECK_EQUALS(TEXT("FromExportTextPath ImportTextItem advances buffer"), *ExportTextBuffer, '\0');
	CHECK_EQUALS(TEXT("FromExportTextPath imports correct path"), FromExportTextPath.ToString(), TEXT("/Path/To/Package.Asset"));

	FTopLevelAssetPath FromShortExportTextPath;
	const TCHAR* ShortExportTextBuffer = TEXT("ClassName'/Path/To/Package.Asset'");
	CHECK_MESSAGE(TEXT("FromShortExportTextPath ImportTextItem succeeds"), FromShortExportTextPath.ImportTextItem(ShortExportTextBuffer, PPF_None, nullptr, GLog->Get()));
	CHECK_EQUALS(TEXT("FromShortExportTextPath ImportTextItem advances buffer"), *ShortExportTextBuffer , '\0');
	CHECK_EQUALS(TEXT("FromShortExportTextPath  imports correct path"), FromShortExportTextPath.ToString(), TEXT("/Path/To/Package.Asset"));

	{
		FCbWriter EmptyPathWriter;
		EmptyPathWriter << FTopLevelAssetPath();
		FCbFieldIterator EmptyPathField = EmptyPathWriter.Save();
		ECbValidateError Result = ValidateCompactBinaryRange(EmptyPathField.GetOuterBuffer(), ECbValidateMode::All);
		CHECK_EQUALS(TEXT("FCbWriter(TopLevelAssetPath) Validate"), Result, ECbValidateError::None);
		if (Result == ECbValidateError::None)
		{
			FTopLevelAssetPath EmptyPathFromCompactBinary;
			CHECK_MESSAGE(TEXT("EmptyPathFromCompactBinary LoadFromCompactBinary succeeds"), LoadFromCompactBinary(FCbFieldView(*EmptyPathField), EmptyPathFromCompactBinary));
			CHECK_EQUALS(TEXT("EmptyPathFromCompactBinary LoadFromCompactBinary correct path"), EmptyPathFromCompactBinary.ToString(), FString());
		}
	}

	{
		FCbWriter PackagePathWriter;
		PackagePathWriter << FTopLevelAssetPath(PackagePath);
		FCbFieldIterator PackagePathField = PackagePathWriter.Save();
		ECbValidateError Result = ValidateCompactBinaryRange(PackagePathField.GetOuterBuffer(), ECbValidateMode::All);
		CHECK_EQUALS(TEXT("FCbWriter(TopLevelAssetPath) Validate"), Result, ECbValidateError::None);
		if (Result == ECbValidateError::None)
		{
			FTopLevelAssetPath PackagePathFromCompactBinary;
			CHECK_MESSAGE(TEXT("PackagePathFromCompactBinary LoadFromCompactBinary succeeds"), LoadFromCompactBinary(FCbFieldView(*PackagePathField), PackagePathFromCompactBinary));
			CHECK_EQUALS(TEXT("PackagePathFromCompactBinary LoadFromCompactBinary correct path"), PackagePathFromCompactBinary.ToString(), PackageName.ToString());
		}
	}

	{
		FCbWriter AssetPathWriter;
		AssetPathWriter << FTopLevelAssetPath(AssetPathString);
		FCbFieldIterator AssetPathField = AssetPathWriter.Save();
		ECbValidateError Result = ValidateCompactBinaryRange(AssetPathField.GetOuterBuffer(), ECbValidateMode::All);
		CHECK_EQUALS(TEXT("FCbWriter(TopLevelAssetPath) Validate"), Result, ECbValidateError::None);
		if (Result == ECbValidateError::None)
		{
			FTopLevelAssetPath AssetPathFromCompactBinary;
			CHECK_MESSAGE(TEXT("AssetPathFromCompactBinary LoadFromCompactBinary succeeds"), LoadFromCompactBinary(FCbFieldView(*AssetPathField), AssetPathFromCompactBinary));
			CHECK_EQUALS(TEXT("AssetPathFromCompactBinary LoadFromCompactBinary correct path"), AssetPathFromCompactBinary.ToString(), AssetPathString);
		}
	}

	// Test ExportText path with and without root 
	// Test starting with . 
}

#endif // WITH_TESTS
