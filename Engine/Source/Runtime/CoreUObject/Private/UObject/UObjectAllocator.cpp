// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnObjAllocator.cpp: Unreal object allocation
=============================================================================*/

#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectGlobals.h"

/** Global UObjectBase allocator							*/
COREUOBJECT_API FUObjectAllocator GUObjectAllocator;

// We cannot use the persistent allocator with merged modular builds, because every object can be removed eventually.
#if UE_MERGED_MODULES
static bool GPersistentAllocatorIsDisabled = true;
#else
static bool GPersistentAllocatorIsDisabled = false;
#endif // UE_MERGED_MODULES

void FUObjectAllocator::DisablePersistentAllocator()
{
	GPersistentAllocatorIsDisabled = true;
}

/**
 * Allocates a UObjectBase from the free store or the permanent object pool
 *
 * @param Size size of uobject to allocate
 * @param Alignment alignment of uobject to allocate
 * @param bAllowPermanent if true, allow allocation in the permanent object pool, if it fits
 * @return newly allocated UObjectBase (not really a UObjectBase yet, no constructor-like thing has been called).
 */
UE_AUTORTFM_ALWAYS_OPEN
UObjectBase* FUObjectAllocator::AllocateUObject(int32 Size, int32 Alignment, bool bAllowPermanent, int32 OffsetToObject)
{
	void* Result = nullptr;

	// We want to perform this allocation uninstrumented, so the GC can clean this up if the transaction is aborted.
	if (bAllowPermanent && !GPersistentAllocatorIsDisabled)
	{
		// This allocation might go over the reserved memory amount and default to FMemory::Malloc, so we are moving it into the AutoRTFM scope.
		Result = GetPersistentLinearAllocator().Allocate(Size, Alignment);
	}
	else
	{
		Result = FMemory::Malloc(Size, Alignment);
	}

	return (UObjectBase*)(((uint8*)Result) + OffsetToObject);
}

/**
 * Returns a UObjectBase to the free store, unless it is in the permanent object pool
 *
 * @param Object object to free
 */
void FUObjectAllocator::FreeUObject(UObjectBase *Object, int32 OffsetToObject) const
{
	check(Object);
	// Only free memory if it was allocated directly from allocator and not from permanent object pool.
	if (FPermanentObjectPoolExtents().Contains(Object) == false)
	{
		FMemory::Free(((uint8*)Object) - OffsetToObject);
	}
	// We only destroy objects residing in permanent object pool during the exit purge.
	else
	{
		check(GExitPurge);
	}
}


