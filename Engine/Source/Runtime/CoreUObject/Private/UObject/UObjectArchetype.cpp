// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectArchetype.cpp: Unreal object archetype relationship management
=============================================================================*/

#include "AutoRTFM.h"
#include "CoreMinimal.h"
#include "UObject/UObjectHash.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/UObjectAnnotation.h"
#include "Stats/StatsMisc.h"
#include "HAL/IConsoleManager.h"
#include "UObject/OverridableManager.h"
#include "UObject/UObjectArchetypeHelper.h"

#define UE_CACHE_ARCHETYPE (1 && !WITH_EDITORONLY_DATA)
#define UE_VERIFY_CACHED_ARCHETYPE 0

#if UE_CACHE_ARCHETYPE
struct FArchetypeInfo
{
	/**
	* default constructor
	* Default constructor must be the default item
	*/
	FArchetypeInfo()
		: ArchetypeIndex(INDEX_NONE)
		, SerialNumber(INDEX_NONE)
	{
	}
	/**
	* Determine if this linker pair is the default
	* @return true is this is a default pair. We only check the linker because CheckInvariants rules out bogus combinations
	*/
	FORCEINLINE bool IsDefault() const
	{
		return ArchetypeIndex == INDEX_NONE;
	}

	/**
	* Constructor
	* @param InArchetype Archetype to assign
	*/
	FArchetypeInfo(int32 InArchetypeIndex, int32 InSerialNumber)
		: ArchetypeIndex(InArchetypeIndex)
		, SerialNumber(InSerialNumber)
	{
	}

	int32 ArchetypeIndex;
	int32 SerialNumber;
};

namespace
{
FUObjectAnnotationChunked<FArchetypeInfo, true, 8192> ArchetypeAnnotation;

//CVar to specify if we should use the Achetype cache.
// default is true.
// 
bool bEnableArchetypeCache = true;
FAutoConsoleVariableRef CVarEnableArchetypeCache(
	TEXT("EnableArchetypeCache"),
	bEnableArchetypeCache,
	TEXT("If set to false, this will disable the use of the ArchetypeCache."),
	ECVF_Default
);
}
#endif // UE_CACHE_ARCHETYPE

#if WITH_EDITOR
FEditorCacheArchetypeManager& FEditorCacheArchetypeManager::Get()
{
	static FEditorCacheArchetypeManager Manager;
	return Manager;
}
#endif // WITH_EDITOR

UObject* GetArchetypeImpl(const UObject* InObject, const FObjectArchetypeHelper::IObjectArchetypePolicy* Policy);

template <bool bUseImmutableArchetype = false, bool bUseArchetypeCache = UE_CACHE_ARCHETYPE>
FORCENOINLINE UObject* FindArchetypeFromRequiredInfoImpl(const UClass* Class, const UObject* Outer, FName Name, EObjectFlags ObjectFlags, bool bUseUpToDateClass, const FObjectArchetypeHelper::IObjectArchetypePolicy* Policy)
{
	UObject* Result = nullptr;
	const bool bIsCDO = !!(ObjectFlags & RF_ClassDefaultObject);
	if (bIsCDO)
	{
		Result = bUseUpToDateClass ? Class->GetAuthoritativeClass()->GetArchetypeForCDO() : Class->GetArchetypeForCDO();
	}
	else
	{
		if (Outer && Cast<UPackage>(Outer) == nullptr) // packages cannot have subobjects
		{
			// Get a lock on the UObject hash tables for the duration of the GetArchetype operation
			FScopedUObjectHashTablesLock HashTablesLock;

			UObject* ArchetypeToSearch = nullptr;
			// Archetype cache is currently not supported with immutable archetypes (we'd need two caches - one for mutable and the other for immutable archetypes)
			if (bUseArchetypeCache)
			{
				ArchetypeToSearch = GetArchetypeImpl(Outer, Policy);
#if UE_VERIFY_CACHED_ARCHETYPE
				{
					UObject* VerifyArchetype = FindArchetypeFromRequiredInfoImpl<bUseImmutableArchetype, bUseArchetypeCache>(Outer->GetClass(), Outer->GetOuter(), Outer->GetFName(), Outer->GetFlags(), bUseUpToDateClass, Policy);
					checkf(ArchetypeToSearch == VerifyArchetype, TEXT("Cached archetype mismatch, expected: %s, cached: %s"), *GetFullNameSafe(VerifyArchetype), *GetFullNameSafe(ArchetypeToSearch));
				}
#endif // UE_VERIFY_CACHED_ARCHETYPE
			}
			else
			{
#if WITH_EDITOR
				ArchetypeToSearch = Policy ? Policy->GetArchetype(Outer) : nullptr;
				if (!ArchetypeToSearch)
				{
					ArchetypeToSearch = FEditorCacheArchetypeManager::Get().GetCachedArchetype(Outer);
				}
				if (!ArchetypeToSearch)
#endif
				{
					ArchetypeToSearch = FindArchetypeFromRequiredInfoImpl<bUseImmutableArchetype, bUseArchetypeCache>(Outer->GetClass(), Outer->GetOuter(), Outer->GetFName(), Outer->GetFlags(), bUseUpToDateClass, Policy);
				}
			}

			UObject* MyArchetype = static_cast<UObject*>(FindObjectWithOuter(ArchetypeToSearch, Class, Name));
			if (MyArchetype)
			{
				Result = MyArchetype; // found that my outers archetype had a matching component, that must be my archetype
			}
			else if (!!(ObjectFlags & RF_InheritableComponentTemplate) && Outer->IsA<UClass>())
			{
				const UClass* OuterSuperClass = static_cast<const UClass*>(Outer)->GetSuperClass();
				for (const UClass* SuperClassArchetype = bUseUpToDateClass && OuterSuperClass ? OuterSuperClass->GetAuthoritativeClass() : OuterSuperClass;
					SuperClassArchetype && SuperClassArchetype->HasAllClassFlags(CLASS_CompiledFromBlueprint);
					SuperClassArchetype = SuperClassArchetype->GetSuperClass())
				{
					Result = static_cast<UObject*>(FindObjectWithOuter(SuperClassArchetype, Class, Name));
					// We can have invalid archetypes halfway through the hierarchy, keep looking if it's pending kill or transient
					if (IsValid(Result) && !Result->HasAnyFlags(RF_Transient))
					{
						break;
					}
				}
			}
			else
			{
				Result = ArchetypeToSearch->GetClass()->FindArchetype(Class, Name);
			}

			if (!Result)
			{
				// No archetype found yet - but the ICH mangles the DefaultSceneRoot name *and* 
				// tries to cull unused templates so do a relaxed compare to find the real default 
				// scene root - see UInheritableComponentHandler::SCSDefaultSceneRootOverrideNamePrefix:
				static const FName DefaultSceneRootSCS("DefaultSceneRoot_GEN_VARIABLE");
				static const FName ICHDefaultSceneRoot("ICH-DefaultSceneRoot_GEN_VARIABLE");

				if (Name == DefaultSceneRootSCS || Name == ICHDefaultSceneRoot)
				{
					if (const UClass* OuterClass = Cast<UClass>(Outer))
					{
						if (const UClass* SuperClass = OuterClass->GetSuperClass())
						{
							Result = SuperClass->FindArchetype(Class, Name);
						}
					}
				}
			}
		}

		if (!Result)
		{
			// nothing found, I am not a CDO, so this is just the class CDO
#if UE_WITH_IMMUTABLE_CDO
			if (bUseImmutableArchetype)
			{
				Result = const_cast<UObject*>(Class->GetImmutableDefaultObject());
			}
			else
#endif
			{
				Result = bUseUpToDateClass ? Class->GetAuthoritativeClass()->GetDefaultObject() : Class->GetDefaultObject();
			}
		}
	}

	return Result;
}

void FObjectArchetypeHelper::CacheArchetypeForObject(UObject* Object, UObject* Archetype)
{
#if UE_CACHE_ARCHETYPE
#if UE_VERIFY_CACHED_ARCHETYPE
	bool bUseUpToDateClass = false;
	UObject* VerifyArchetype = FindArchetypeFromRequiredInfoImpl(Object->GetClass(), Object->GetOuter(), Object->GetFName(), Object->GetFlags(), bUseUpToDateClass);
	checkf(Archetype == VerifyArchetype, TEXT("Cached archetype mismatch, expected: %s, cached: %s"), *GetFullNameSafe(VerifyArchetype), *GetFullNameSafe(Archetype));
#endif
	int32 ArchetypeIndex = GUObjectArray.ObjectToIndex(Archetype);
	ArchetypeAnnotation.AddAnnotation(Object, FArchetypeInfo{ ArchetypeIndex, GUObjectArray.AllocateSerialNumber(ArchetypeIndex) });
#endif
}

UObject* UObject::GetArchetypeFromRequiredInfo(const UClass* Class, const UObject* Outer, FName Name, EObjectFlags ObjectFlags)
{
	bool bUseUpToDateClass = false;
	return FindArchetypeFromRequiredInfoImpl(Class, Outer, Name, ObjectFlags, bUseUpToDateClass, nullptr);
}

//DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("UObject::GetArchetype"), STAT_FArchiveRealtimeGC_GetArchetype, STATGROUP_GC);

UObject* GetArchetypeImpl(const UObject* InObject, const FObjectArchetypeHelper::IObjectArchetypePolicy* Policy)
{
	//SCOPE_SECONDS_ACCUMULATOR(STAT_FArchiveRealtimeGC_GetArchetype);

#if WITH_EDITOR
	if (Policy)
	{
		if (UObject* Archetype = Policy->GetArchetype(InObject))
		{
			return Archetype;
		}
	}

	if (UObject* CacheArchetype = FEditorCacheArchetypeManager::Get().GetCachedArchetype(InObject))
	{
		// Use the cached archetype if set
		return CacheArchetype;
	}

#endif

	bool bUseUpToDateClass = false;
#if UE_CACHE_ARCHETYPE
	if (!bEnableArchetypeCache)
	{
		return FindArchetypeFromRequiredInfoImpl(InObject->GetClass(), InObject->GetOuter(), InObject->GetFName(), InObject->GetFlags(), bUseUpToDateClass, Policy);
	}

	UObject* Archetype = nullptr;
	FArchetypeInfo Annoatation = ArchetypeAnnotation.GetAnnotation(InObject);
	int32 ArchetypeIndex = Annoatation.ArchetypeIndex;
	int32 SerialNumber = ArchetypeIndex == INDEX_NONE ? INDEX_NONE : GUObjectArray.GetSerialNumber(ArchetypeIndex);
	if ((ArchetypeIndex == INDEX_NONE) || (SerialNumber != Annoatation.SerialNumber))
	{
		Archetype = FindArchetypeFromRequiredInfoImpl(InObject->GetClass(), InObject->GetOuter(), InObject->GetFName(), InObject->GetFlags(), bUseUpToDateClass, Policy);
		// If the Outer is pending load we can't cache the archetype as it may be inacurate
		if (Archetype && !(InObject->GetOuter() && InObject->GetOuter()->HasAnyFlags(RF_NeedLoad)))
		{
			ArchetypeIndex = GUObjectArray.ObjectToIndex(Archetype);
			ArchetypeAnnotation.AddAnnotation(InObject, FArchetypeInfo{ ArchetypeIndex, GUObjectArray.AllocateSerialNumber(ArchetypeIndex) });
		}
	}
	else
	{
		FUObjectItem* ArchetypeItem = GUObjectArray.IndexToObject(ArchetypeIndex);
		check(ArchetypeItem != nullptr);
		Archetype = static_cast<UObject*>(ArchetypeItem->GetObject());
#if UE_VERIFY_CACHED_ARCHETYPE
		UObject* ExpectedArchetype = FindArchetypeFromRequiredInfoImpl(InObject->GetClass(), InObject->GetOuter(), InObject->GetFName(), InObject->GetFlags(), bUseUpToDateClass, Policy);
		if (ExpectedArchetype != Archetype)
		{
			UE_LOGF(LogClass, Fatal, "Cached archetype mismatch, expected: %ls, cached: %ls", *GetFullNameSafe(ExpectedArchetype), *GetFullNameSafe(Archetype));
		}
#endif // UE_VERIFY_CACHED_ARCHETYPE
	}
	// Note that IsValidLowLevelFast check may fail during initial load as not all classes are initialized at this point so skip it
	check(Archetype == nullptr || GIsInitialLoad || Archetype->IsValidLowLevelFast());

	return Archetype;
#else
	return FindArchetypeFromRequiredInfoImpl(InObject->GetClass(), InObject->GetOuter(), InObject->GetFName(), InObject->GetFlags(), bUseUpToDateClass, Policy);
#endif // UE_CACHE_ARCHETYPE
}

UObject* UObject::GetArchetype() const
{
	return GetArchetypeImpl(this, nullptr);
}

UObject* FObjectArchetypeHelper::GetArchetype(const UObject* InObject, const FObjectArchetypeHelper::IObjectArchetypePolicy* Policy)
{
	return GetArchetypeImpl(InObject, Policy);
}

void FObjectArchetypeHelper::CleanupCachedArchetypes()
{
#if UE_CACHE_ARCHETYPE
	ArchetypeAnnotation.RemoveAllAnnotations();
#endif
}

void FObjectArchetypeHelper::InvalidateArchetypeCache(const UObject* Object)
{
#if UE_CACHE_ARCHETYPE
	ArchetypeAnnotation.RemoveAnnotation(Object);
#endif
}

#ifndef UE_WITH_IMMUTABLEARCHETYPE_DEBUGGING
#define UE_WITH_IMMUTABLEARCHETYPE_DEBUGGING 0
#endif

const UObject* FindImmutableArchetype(const UObject* InObj)
{
	const bool bUseUpToDateClass = true;
	const UObject* ImmutableArchetype = FindArchetypeFromRequiredInfoImpl</*bUseImmutableArchetype=*/true, /*bUseArchetypeCache=*/false>(InObj->GetClass(), InObj->GetOuter(), InObj->GetFName(), InObj->GetFlags(), bUseUpToDateClass, nullptr);
#if UE_WITH_IMMUTABLEARCHETYPE_DEBUGGING
	const UObject* Archetype = InObj->GetArchetype();
	checkf(ImmutableArchetype, TEXT("Unable to find immutable archetype for %s"), *InObj->GetFullName());
	checkf(ImmutableArchetype->HasAnyFlags(RF_ArchetypeObject), TEXT("Immutable archetype is not an archetype"));
	checkf(ImmutableArchetype->GetFName() == Archetype->GetFName(), TEXT("Immutable archetype name mismatch: expected: %s, got: %s"), *Archetype->GetName(), *ImmutableArchetype->GetName());
	checkf(ImmutableArchetype->GetClass() == Archetype->GetClass(), TEXT("Immutable archetype class mismatch: expected: %s, got: %s"), *Archetype->GetClass()->GetPathName(), *ImmutableArchetype->GetClass()->GetPathName());
	checkf(ImmutableArchetype->HasAnyFlags(RF_ClassDefaultObject) == Archetype->HasAnyFlags(RF_ClassDefaultObject), TEXT("Immutable archetype flags mismatch"));
	// Immutable CDOs have different outers than the original CDOs so we can't check if their outer class matches the original CDO
	checkf(ImmutableArchetype->HasAnyFlags(RF_ClassDefaultObject) || ImmutableArchetype->GetOuter()->GetClass() == Archetype->GetOuter()->GetClass(), TEXT("Immutable archetype outer class mismatch: expected: %s, got: %s"), *Archetype->GetOuter()->GetClass()->GetPathName(), *ImmutableArchetype->GetOuter()->GetClass()->GetPathName());
#endif // UE_WITH_IMMUTABLEARCHETYPE_DEBUGGING
	return ImmutableArchetype;
}
