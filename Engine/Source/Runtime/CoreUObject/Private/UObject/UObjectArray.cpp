// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnObjArray.cpp: Unreal array of all objects
=============================================================================*/

#include "UObject/UObjectArray.h"
#include "AutoRTFM.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemStats.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "Stats/Stats.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollectionInternalFlags.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ReachabilityAnalysisState.h"
#include "UObject/RemoteObjectPrivate.h"

DEFINE_LOG_CATEGORY_STATIC(LogUObjectArray, Log, All);

DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("CreateStatId"), CreateStatId, STATGROUP_Object, EStatFlags::Verbose);

LLM_DEFINE_TAG(UObject_UObjectArray);

FUObjectClusterContainer GUObjectClusters;

#if STATS || ENABLE_STATNAMEDEVENTS_UOBJECT
void FUObjectItem::CreateStatID() const
{
	LLM_SCOPE_BYNAME(TEXT("Debug/CreateStatID"));
	SCOPE_CYCLE_COUNTER(CreateStatId);

	FString LongName;
	LongName.Reserve(255);
	TArray<UObjectBase const*, TInlineAllocator<24>> ClassChain;

	// Build class hierarchy
	UObjectBase const* Target = GetObject();
	do
	{
		ClassChain.Add(Target);
		Target = Target->GetOuter();
	} while (Target);

	// Start with class name
	if (GetObject()->GetClass())
	{
		GetObject()->GetClass()->GetFName().GetDisplayNameEntry()->AppendNameToString(LongName);
	}

	// Now process from parent -> child so we can append strings more efficiently.
	bool bFirstEntry = true;
	for (int32 i = ClassChain.Num() - 1; i >= 0; i--)
	{
		Target = ClassChain[i];
		const FNameEntry* NameEntry = Target->GetFNameForStatID().GetDisplayNameEntry();
		if (bFirstEntry)
		{
			NameEntry->AppendNameToPathString(LongName);
		}
		else
		{
			if (!LongName.IsEmpty())
			{
				LongName += TEXT(".");
			}
			NameEntry->AppendNameToString(LongName);
		}
		bFirstEntry = false;
	}

#if STATS
	StatID = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_UObjects>(LongName);
#else // ENABLE_STATNAMEDEVENTS
	const auto& ConversionData = StringCast<PROFILER_CHAR>(*LongName);
	const int32 NumStorageChars = (ConversionData.Length() + 1);	//length doesn't include null terminator

	PROFILER_CHAR* const StoragePtr = new PROFILER_CHAR[NumStorageChars];
	FMemory::Memcpy(StoragePtr, ConversionData.Get(), NumStorageChars * sizeof(PROFILER_CHAR));

	bool bExchanged = AutoRTFM::Open([&]
	{ 
		return FPlatformAtomics::InterlockedCompareExchangePointer((void**)&StatIDStringStorage, StoragePtr, nullptr) == nullptr;
	});

	if (bExchanged)
	{
		// If we abort, then StoragePtr will be freed so reset StatIDStringStorage.
		// This is abort handler is popped in the destructor.
		AutoRTFM::PushOnAbortHandler(StoragePtr, [this, StoragePtr]
		{
			FPlatformAtomics::InterlockedCompareExchangePointer((void**)&StatIDStringStorage, nullptr, StoragePtr);
		});
	}
	else
	{
		delete[] StoragePtr;
	}

	StatID = TStatId(StatIDStringStorage);
#endif
}
#endif
FUObjectArray::FUObjectArray()
: ObjFirstGCIndex(0)
, ObjLastNonGCIndex(-1)
, MaxObjectsNotConsideredByGC(0)
, OpenForDisregardForGC(true)
, PrimarySerialNumber(START_SERIAL_NUMBER)
{
	GCoreObjectArrayForDebugVisualizers = &GUObjectArray.ObjObjects;
}

void FUObjectArray::AllocateObjectPool(int32 InMaxUObjects, int32 InMaxObjectsNotConsideredByGC, bool bPreAllocateObjectArray)
{
	check(IsInGameThread());

	MaxObjectsNotConsideredByGC = InMaxObjectsNotConsideredByGC;

	// GObjFirstGCIndex is the index at which the garbage collector will start for the mark phase.
	// If disregard for GC is enabled this will be set to an invalid value so that later we
	// know if disregard for GC pool has already been closed (at least once)
	ObjFirstGCIndex = DisregardForGCEnabled() ? -1 : 0;

	// Pre-size array.
	check(ObjObjects.Num() == 0);
	UE_CLOG(InMaxUObjects <= 0, LogUObjectArray, Fatal, TEXT("Max UObject count is invalid. It must be a number that is greater than 0."));
	ObjObjects.PreAllocate(InMaxUObjects, bPreAllocateObjectArray);

	if (MaxObjectsNotConsideredByGC > 0)
	{
		ObjObjects.AddRange(MaxObjectsNotConsideredByGC);
	}
}

void FUObjectArray::OpenDisregardForGC()
{
	check(IsInGameThread());
	check(!OpenForDisregardForGC);
	OpenForDisregardForGC = true;
	UE_LOGF(LogUObjectArray, Log, "OpenDisregardForGC: %d/%d objects in disregard for GC pool", ObjLastNonGCIndex + 1, MaxObjectsNotConsideredByGC);
}

void FUObjectArray::CloseDisregardForGC()
{
#if THREADSAFE_UOBJECTS
	UE::TScopeLock ObjObjectsLock(ObjObjectsCritical);
#else
	// Disregard from GC pool is only available from the game thread, at least for now
	check(IsInGameThread());
#endif

	check(OpenForDisregardForGC);

	// Make sure all classes that have been loaded/created so far are properly initialized
	if (!IsEngineExitRequested())
	{
		ProcessNewlyLoadedUObjects();

		UClass::AssembleReferenceTokenStreams();

		if (GIsInitialLoad)
		{
			// Iterate over all objects and mark them to be part of root set.
			int32 NumAlwaysLoadedObjects = 0;
			int32 NumRootObjects = 0;
			for (FThreadSafeObjectIterator It; It; ++It)
			{
				UObject* Object = *It;
				if (Object->IsSafeForRootSet())
				{
					NumRootObjects++;
					Object->AddToRoot();
				}
				else if (Object->IsRooted())
				{
					Object->RemoveFromRoot();
				}
				NumAlwaysLoadedObjects++;
			}

			UE_LOGF(LogUObjectArray, Log, "%i objects as part of root set at end of initial load.", NumAlwaysLoadedObjects);
			if (GUObjectArray.DisregardForGCEnabled())
			{
				UE_LOGF(LogUObjectArray, Log, "%i objects are not in the root set, but can never be destroyed because they are in the DisregardForGC set.", NumAlwaysLoadedObjects - NumRootObjects);
			}
		}
	}

	// When disregard for GC pool is closed, make sure the first GC index is set after the last non-GC index.
	// We do allow here for some slack if MaxObjectsNotConsideredByGC > (ObjLastNonGCIndex + 1) so that disregard for GC pool
	// can be re-opened later.
	ObjFirstGCIndex = FMath::Max(ObjFirstGCIndex, ObjLastNonGCIndex + 1);

	UE_LOGF(LogUObjectArray, Log, "CloseDisregardForGC: %d/%d objects in disregard for GC pool", ObjLastNonGCIndex + 1, MaxObjectsNotConsideredByGC);	

	OpenForDisregardForGC = false;
	GIsInitialLoad = false;

	checkf(!DisregardForGCEnabled() || !GIsEditor, TEXT("Disregard For GC Set can't be enabled when running the editor"));
	checkf(DisregardForGCEnabled() || (ObjFirstGCIndex == 0 && ObjLastNonGCIndex == -1), TEXT("Disregard for GC Set is not properly disabled (FirstGCIndex = %d, LastNonGCIndex = %d"), ObjFirstGCIndex, ObjLastNonGCIndex);
}

void FUObjectArray::DisableDisregardForGC()
{
	// ObjLastNonGCIndex defaults to -1 and is being incremented with each UObject allocation when the disregard for GC set is open and enabled
	const int32 NumDisregardForGCObjects = ObjLastNonGCIndex + 1;

	// Reset the diregard for GC state before calling OnDisregardForGCSetDisabled() in case it changes any of the root flags
	MaxObjectsNotConsideredByGC = 0;
	GUObjectAllocator.DisablePersistentAllocator();
	ObjFirstGCIndex = 0;
	ObjLastNonGCIndex = -1;
	if (IsOpenForDisregardForGC())
	{
		CloseDisregardForGC();
	}

	if (!GExitPurge && NumDisregardForGCObjects > 0)
	{
		checkf(!IsAsyncLoading(), TEXT("Disregard for GC Set can't be safely disabled while async loading. Consider calling FlushAsyncLoading() first or using gc.MaxObjectsNotConsideredByGC=0 ini setting instead."));
		checkf(!AutoRTFM::IsTransactional(), TEXT("Disregard for GC can't be changed while inside of a transaction."));

		AutoRTFM::UnreachableIfTransactional();

		AUTORTFM_DISABLE void OnDisregardForGCSetDisabled(int32 NumObjects);
		OnDisregardForGCSetDisabled(NumDisregardForGCObjects);
	}
}

void FUObjectArray::AllocateUObjectIndex(UObjectBase* Object, EInternalObjectFlags InitialFlags, int32 AlreadyAllocatedIndex, int32 SerialNumber, FRemoteObjectId RemoteId)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectArray);
	// Clear asset scopes
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();

	int32 Index = INDEX_NONE;
	check(Object->InternalIndex == INDEX_NONE);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (!RemoteId.IsValid())
	{
		RemoteId = FRemoteObjectId::Generate(Object, *Object->GetFName().ToString(), nullptr, InitialFlags);
	}
	else
	{
		RemoteId = UE::RemoteObject::Private::FRemoteIdLocalizationHelper::GetLocalized(RemoteId);
	}
#endif

	LockInternalArray();

	if (AlreadyAllocatedIndex >= 0)
	{
		Index = AlreadyAllocatedIndex;
	}
	// Special non- garbage collectable range.
	else if (IsOpenForDisregardForGC() & GUObjectArray.DisregardForGCEnabled()) //-V792 
	{
		Index = ++ObjLastNonGCIndex;
		// Check if we're not out of bounds, unless there hasn't been any gc objects yet
		UE_CLOGF(ObjLastNonGCIndex >= MaxObjectsNotConsideredByGC && ObjFirstGCIndex >= 0, LogUObjectArray, Fatal, "Unable to add more objects to disregard for GC pool (Max: %d)", MaxObjectsNotConsideredByGC);
		// If we haven't added any GC objects yet, it's fine to keep growing the disregard pool past its initial size.
		if (ObjLastNonGCIndex >= MaxObjectsNotConsideredByGC)
		{
			Index = ObjObjects.AddSingle();
			check(Index == ObjLastNonGCIndex);
		}
		MaxObjectsNotConsideredByGC = FMath::Max(MaxObjectsNotConsideredByGC, ObjLastNonGCIndex + 1);
	}
	// Regular pool/ range.
	else
	{
		if (ObjAvailableList.Num() > 0)
		{
			Index = ObjAvailableList.Pop();
			const int32 AvailableCount = ObjAvailableList.Num();
			checkSlow(AvailableCount >= 0);
			ObjAvailableListEstimateCount = AvailableCount;
		}
		else
		{
			// Make sure ObjFirstGCIndex is valid, otherwise we didn't close the disregard for GC set
			check(ObjFirstGCIndex >= 0);
			Index = ObjObjects.AddSingle();			
		}
		check(Index >= ObjFirstGCIndex && Index > ObjLastNonGCIndex);
	}
	// Add to global table.
	FUObjectItem* ObjectItem = IndexToObject(Index);
	UE_CLOGF(ObjectItem->GetObject() != nullptr, LogUObjectArray, Fatal, "Attempting to add %ls at index %d but another object (0x%016llx) exists at that index!", *Object->GetFName().ToString(), Index, (int64)(PTRINT)ObjectItem->GetObject());
	// At this point all not-compiled-in objects are not fully constructed yet and this is the earliest we can mark them as such
	ObjectItem->FlagsAndRefCount = (int64)((uint64)EInternalObjectFlags::PendingConstruction << 32);
	// Objects in the disregad for GC pool don't need the reachable flag set because GC will never process them
	if (!IsIndexDisregardForGC(Index))
	{
		// It's safe to access FGCFlags::GetReachableFlagValue_ForGC() here because creating new objects is being performed 
		// under the same UObjectArray lock as swapping reachability flags inside of GC, see FGCFlags::SwapReachableAndMaybeUnreachable()
		ObjectItem->FlagsAndRefCount |= ((int64)UE::GC::Private::FGCFlags::GetReachableFlagValue_ForGC()) << 32;
	}
	ObjectItem->SetObject(Object);
	ObjectItem->ClusterRootIndex = 0;

	// AutoRTFM doesn't like atomics even when relaxed, so we need to differentiate the code here
#if USING_INSTRUMENTATION || USING_THREAD_SANITISER
	// This can race with weakptr trying to resolve an old object in this slot.
	// Avoid TSAN warning here since this is safe, the ObjectItem can't possibly match as its been
	// cleaned up during GC.
	FPlatformAtomics::AtomicStore_Relaxed(&ObjectItem->SerialNumber, SerialNumber);
#else
	ObjectItem->SerialNumber = SerialNumber;
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
	ObjectItem->SetRemoteId(RemoteId);
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
	Object->InternalIndex = Index;

	// This needs to happen after the InternalIndex is set because setting root flags may result in the object being added to UE::GC::Priate::GRoots array 
	if (InitialFlags != EInternalObjectFlags::None)
	{
		ObjectItem->ThisThreadAtomicallySetFlag(InitialFlags);
	}

	UnlockInternalArray();

#if THREADSAFE_UOBJECTS
	UE::TScopeLock UObjectCreateListenersLock(UObjectCreateListenersCritical);
#endif

	for (int32 ListenerIndex = 0; ListenerIndex < UObjectCreateListeners.Num(); ListenerIndex++)
	{
		UObjectCreateListeners[ListenerIndex]->NotifyUObjectCreated(Object,Index);
	}
}

/**
 * Reset the serial number from the game thread to invalidate all weak object pointers to it
 *
 * @param Object to reset
 */
void FUObjectArray::ResetSerialNumber(UObjectBase* Object)
{
	int32 Index = Object->InternalIndex;
	FUObjectItem* ObjectItem = IndexToObject(Index);
	checkSlow(ObjectItem);
	ObjectItem->SerialNumber = 0;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ObjectItem->RemoteId = FRemoteObjectId();
#endif
}

/**
 * Removes an object from delete listeners
 *
 * @param Object to remove from delete listeners
 */
void FUObjectArray::RemoveObjectFromDeleteListeners(UObjectBase* Object)
{
#if THREADSAFE_UOBJECTS
	UE::TScopeLock UObjectDeleteListenersLock(UObjectDeleteListenersCritical);
#endif
	int32 Index = Object->InternalIndex;
	check(Index >= 0);
	// Iterate in reverse order so that when one of the listeners removes itself from the array inside of NotifyUObjectDeleted we don't skip the next listener.
	for (int32 ListenerIndex = UObjectDeleteListeners.Num() - 1; ListenerIndex >= 0; --ListenerIndex)
	{
		UObjectDeleteListeners[ListenerIndex]->NotifyUObjectDeleted(Object, Index);
	}
}

/**
 * Returns a UObject index to the global uobject array
 *
 * @param Object object to free
 */
void FUObjectArray::FreeUObjectIndex(UObjectBase* Object)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectArray);

	// No need to call LockInternalArray(); here as it should already be locked by GC

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	UE::CoreUObject::Private::FreeObjectHandle(Object);
#endif


	int32 Index = Object->InternalIndex;
	FUObjectItem* ObjectItem = IndexToObject(Index);
	UE_CLOGF(ObjectItem->GetObject() != Object, LogUObjectArray, Fatal, "Removing object (0x%016llx) at index %d but the index points to a different object (0x%016llx)!", (int64)(PTRINT)Object, Index, (int64)(PTRINT)ObjectItem->GetObject());

	// This should only be happening on the game thread (GC runs only on game thread when it's freeing objects)
	// We loosen the restriction a little bit to allow overwrite of UObjects that are still in the construction loading phase,
	// this only happens for very narrow cases and should be fine as long as the UObject's in question have thread-safe
	// destructor and destruction virtuals.
	check(IsInGameThread() || (IsInAsyncLoadingThread() && ObjectItem->HasAnyFlags(EInternalObjectFlags::AsyncLoadingPhase1)));

	// Can't destroy a refcounted object
	check(ObjectItem->GetRefCount() == 0 || GExitPurge);

	// Clear root flags to remove this object's index from UE::GC::Private::GRoots array 
	if ((ObjectItem->GetFlagsInternal() & (int32)EInternalObjectFlags_RootFlags) != 0)
	{
		ObjectItem->ThisThreadAtomicallyClearedFlag(EInternalObjectFlags_RootFlags);
	}

	// Due to atomic operations, these fields are only modified in the open.
	// Mixing open and closed writes to the same memory location can cause memory corruption (SOL-6743)
	// so, reset these fields in the open.
	UE_AUTORTFM_OPEN
	{
		ObjectItem->FlagsAndRefCount = 0;
	};

	ObjectItem->SetObject(nullptr);
	ObjectItem->ClusterRootIndex = 0;
	ObjectItem->SerialNumber = 0;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	ObjectItem->RemoteId = FRemoteObjectId();
#endif
	Object->InternalIndex = INDEX_NONE;

	// You cannot safely recycle indicies in the non-GC range
	// No point in filling this list when doing exit purge. Nothing should be allocated afterwards anyway.
	if (Index > ObjLastNonGCIndex && !GExitPurge && bShouldRecycleObjectIndices)
	{
		ObjAvailableList.Add(Index);
		ObjAvailableListEstimateCount = ObjAvailableList.Num();
	}
}

/**
 * Adds a creation listener
 *
 * @param Listener listener to notify when an object is deleted
 */
void FUObjectArray::AddUObjectCreateListener(FUObjectCreateListener* Listener)
{
#if THREADSAFE_UOBJECTS
	UE::TScopeLock UObjectCreateListenersLock(UObjectCreateListenersCritical);
#endif

	check(!UObjectCreateListeners.Contains(Listener));
	UObjectCreateListeners.Add(Listener);
}

/**
 * Removes a listener for object creation
 *
 * @param Listener listener to remove
 */
void FUObjectArray::RemoveUObjectCreateListener(FUObjectCreateListener* Listener)
{
#if THREADSAFE_UOBJECTS
	UE::TScopeLock UObjectCreateListenersLock(UObjectCreateListenersCritical);
#endif
	int32 NumRemoved = UObjectCreateListeners.RemoveSingleSwap(Listener);
	check(NumRemoved==1);
}

/**
 * Checks whether object is part of permanent object pool.
 *
 * @param Listener listener to notify when an object is deleted
 */
void FUObjectArray::AddUObjectDeleteListener(FUObjectDeleteListener* Listener)
{
#if THREADSAFE_UOBJECTS
	UE::TScopeLock UObjectDeleteListenersLock(UObjectDeleteListenersCritical);
#endif
	check(!UObjectDeleteListeners.Contains(Listener));
	UObjectDeleteListeners.Add(Listener);
}

/**
 * removes a listener for object deletion
 *
 * @param Listener listener to remove
 */
void FUObjectArray::RemoveUObjectDeleteListener(FUObjectDeleteListener* Listener)
{
#if THREADSAFE_UOBJECTS
	UE::TScopeLock UObjectDeleteListenersLock(UObjectDeleteListenersCritical);
#endif
	UObjectDeleteListeners.RemoveSingleSwap(Listener);
}



/**
 * Checks if a UObject index is valid
 *
 * @param	Object object to test for validity
 * @return	true if this index is valid
 */
bool FUObjectArray::IsValid(const UObjectBase* Object) const 
{ 
	int32 Index = Object->InternalIndex;
	if( Index == INDEX_NONE )
	{
		UE_LOGF(LogUObjectArray, Warning, "Object is not in global object array" );
		return false;
	}
	if( !ObjObjects.IsValidIndex(Index))
	{
		UE_LOGF(LogUObjectArray, Warning, "Invalid object index %i", Index );
		return false;
	}
	const FUObjectItem& Slot = ObjObjects[Index];
	if( Slot.GetObject() == NULL )
	{
		UE_LOGF(LogUObjectArray, Warning, "Empty slot" );
		return false;
	}
	if( Slot.GetObject() != Object )
	{
		UE_LOGF(LogUObjectArray, Warning, "Other object in slot" );
		return false;
	}
	return true;
}

int32 FUObjectArray::AllocateSerialNumber(int32 Index)
{
	FUObjectItem* ObjectItem = IndexToObject(Index);
	checkSlow(ObjectItem);

	volatile int32 *SerialNumberPtr = &ObjectItem->SerialNumber;
	// Open around PrimarySerialNumber. If we fail/abort a transaction, we don't
	// need to undo this; we simply allow it to grow for the next use.
	// Disable the AutoRTFM sanitizer for this open as we're performing an
	// explicitly recorded write to SerialNumber which the AutoRTFM sanitizer
	// will treat as a false-positive mixed open / closed write.
	int32 SerialNumber;
	UE_AUTORTFM_OPEN_NO_SANITIZE
	{
		SerialNumber = FPlatformAtomics::AtomicRead_Relaxed(SerialNumberPtr);
		if (!SerialNumber)
		{
			SerialNumber = PrimarySerialNumber.Increment();
			UE_CLOG(SerialNumber <= START_SERIAL_NUMBER, LogUObjectArray, Fatal, TEXT("UObject serial numbers overflowed (trying to allocate serial number %d)."), SerialNumber);
			AutoRTFM::RecordOpenWrite(const_cast<int32*>(SerialNumberPtr)); // const_cast to remove volatile
			int32 ValueWas = FPlatformAtomics::InterlockedCompareExchange((int32*)SerialNumberPtr, SerialNumber, 0);
			if (ValueWas != 0)
			{
				// Someone else got it first; use their value.
				SerialNumber = ValueWas;
			}
		}
	};
	checkSlow(SerialNumber > START_SERIAL_NUMBER);
	
	return SerialNumber;
}

/**
 * Clears some internal arrays to get rid of false memory leaks
 */
void FUObjectArray::ShutdownUObjectArray()
{
	{
#if THREADSAFE_UOBJECTS
		UE::TScopeLock UObjectDeleteListenersLock(UObjectDeleteListenersCritical);
#endif
		for (int32 Index = UObjectDeleteListeners.Num() - 1; Index >= 0; --Index)
		{
			FUObjectDeleteListener* Listener = UObjectDeleteListeners[Index];
			Listener->OnUObjectArrayShutdown();
		}
		UE_CLOGF(UObjectDeleteListeners.Num(), LogUObjectArray, Fatal, "All UObject delete listeners should be unregistered when shutting down the UObject array");
	}
	{
#if THREADSAFE_UOBJECTS
		UE::TScopeLock UObjectCreateListenersLock(UObjectCreateListenersCritical);
#endif
		for (int32 Index = UObjectCreateListeners.Num() - 1; Index >= 0; --Index)
		{
			FUObjectCreateListener* Listener = UObjectCreateListeners[Index];
			Listener->OnUObjectArrayShutdown();
		}
		UE_CLOGF(UObjectCreateListeners.Num(), LogUObjectArray, Fatal, "All UObject delete listeners should be unregistered when shutting down the UObject array");
	}
}

void FUObjectArray::DumpUObjectCountsToLog() const
{
	UE_LOGF(LogUObjectArray, Display, "Dumping allocated UObject counts to log:");
	struct FClassEntry
	{
		UClass* Class = nullptr;
		int32 NumInstances = 0;
	};
	int32 NumClasses = 0;
	int32 NumUObjects = 0;
	for (int32 ObjectIndex = 0; ObjectIndex < GetObjectArrayNum(); ++ObjectIndex)
	{
		const FUObjectItem& ObjectItem = GetObjectItemArrayUnsafe()[ObjectIndex];
		UObject* Object = (UObject*)ObjectItem.GetObject();
		if (Object && Object->IsA(UClass::StaticClass()))
		{
			NumClasses++;
		}
	}

	TMap<UClass*, FClassEntry> ClassCountMap;
	ClassCountMap.Reserve(NumClasses);

	for (int32 ObjectIndex = 0; ObjectIndex < GetObjectArrayNum(); ++ObjectIndex)
	{
		const FUObjectItem& ObjectItem = GetObjectItemArrayUnsafe()[ObjectIndex];
		if (ObjectItem.GetObject())
		{
			UObject* Object = (UObject*)ObjectItem.GetObject();
			UClass* ObjectClass = Object->GetClass();
			FClassEntry& ClassEntry = ClassCountMap.FindOrAdd(ObjectClass);
			ClassEntry.Class = ObjectClass;
			ClassEntry.NumInstances++;
			NumUObjects++;
		}
	}

	TArray<FClassEntry> ClassArray;
	ClassCountMap.GenerateValueArray(ClassArray);

	ClassArray.Sort([](const FClassEntry& A, const FClassEntry& B) { return A.NumInstances > B.NumInstances; });

	const int32 MinInstanceNum = 10; // Don't print classes with fewer than the specified number of instances
	const double MaxPrintedInstancePercent = 0.95; // Finish printing when the specified percent of instances has already been printed
	int32 NumClassesSkipped = 0;
	int32 NumInstancesSkipped = 0;
	int32 NumInstancesPrinted = 0;
	double PercentOfInstancesPrinted = 0.0;

	for (const FClassEntry& ClassEntry : ClassArray)
	{		
		if (ClassEntry.NumInstances > MinInstanceNum && PercentOfInstancesPrinted <= MaxPrintedInstancePercent)
		{
			UE_LOGF(LogUObjectArray, Display, "%8d instances of %ls", ClassEntry.NumInstances, *ClassEntry.Class->GetPathName());
			NumInstancesPrinted += ClassEntry.NumInstances;
			PercentOfInstancesPrinted = (double)NumInstancesPrinted / NumUObjects;
		}
		else
		{
			NumClassesSkipped++;
			NumInstancesSkipped += ClassEntry.NumInstances;
		}
	}
	if (NumInstancesSkipped > 0)
	{
		if (PercentOfInstancesPrinted > MaxPrintedInstancePercent)
		{
			UE_LOGF(LogUObjectArray, Display, "%8d instances in the remaining %.3f%% of instances of %d classes", NumInstancesSkipped, (1.0f - PercentOfInstancesPrinted) * 100.0f, NumClassesSkipped);
		}
		else
		{
			UE_LOGF(LogUObjectArray, Display, "%8d instances of %d classes with less than %d instances per class", NumInstancesSkipped, NumClassesSkipped, MinInstanceNum);
		}
	}
	UE_LOGF(LogUObjectArray, Display, "%d total UObjects (%d classes)", NumUObjects, NumClasses);
}

static int32 GVarDumpObjectCountsToLogWhenMaxObjectLimitExceeded = 0;
static FAutoConsoleVariableRef CDumpObjectCountsToLogWhenMaxObjectLimitExceeded(
	TEXT("gc.DumpObjectCountsToLogWhenMaxObjectLimitExceeded"),
	GVarDumpObjectCountsToLogWhenMaxObjectLimitExceeded,
	TEXT("If not 0 dumps UObject counts to log when maximum object count limit has been reached."),
	ECVF_Default
);

void UE::UObjectArrayPrivate::FailMaxUObjectCountExceeded(const int32 MaxUObjects, const int32 NewUObjectCount)
{
	if (GVarDumpObjectCountsToLogWhenMaxObjectLimitExceeded)
	{
		GUObjectArray.DumpUObjectCountsToLog();
	}
	UE_LOGF(LogUObjectArray, Fatal, "Maximum number of UObjects (%d) exceeded when trying to add %d object(s), make sure you update MaxObjectsInGame/MaxObjectsInEditor/MaxObjectsInProgram in project settings.", MaxUObjects, NewUObjectCount);
}

bool verse::CanAllocateUObjects()
{
	// NOTE: This is an arbitrary limit. If we have less than ~10k `UObject`s available for allocation left
	// we're probably in a bad spot anyway. This just makes sure that there is some slack available before the
	// limit gets hit.
	// (The `FName` space requirement of 5MB, out of a maximum of 1GB, is chosen to match the ratio of 10k to
	// the default maximum of 2M `UObject`s. It is checked separately because we have observed islands running
	// out of `FName` space before hitting this `UObject` limit.)
	static constexpr int32 MinAvailableObjectCount = 10 * 1024;
	static constexpr int32 MinAvailableNameEntrySize = 5 * 1024 * 1024;
	return GUObjectArray.GetObjectArrayEstimatedAvailable() >= MinAvailableObjectCount &&
		FName::GetNameEntryMemoryEstimatedAvailable() >= MinAvailableNameEntrySize;
}
