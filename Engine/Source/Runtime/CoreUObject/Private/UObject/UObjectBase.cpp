// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectBase.cpp: Unreal UObject base class
=============================================================================*/

#include "UObject/UObjectBase.h"
#include "AutoRTFM.h"
#include "Misc/MessageDialog.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "Logging/StructuredLog.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Containers/StripedMap.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectHash.h"
#include "UObject/Class.h"
#include "UObject/DeferredRegistry.h"
#include "UObject/ObjectHandlePrivate.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectPartials.h"
#include "UObject/UObjectStats.h"
#include "UObject/UObjectPrivate.h"
#include "UObject/Package.h"
#include "UObject/ReferenceChainSearch.h"
#include "Templates/Casts.h"
#include "UObject/GCObject.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Reload.h"
#include "Misc/CommandLine.h"
#include "Interfaces/IPluginManager.h"
#include "Serialization/LoadTimeTrace.h"

#if UE_WITH_REMOTE_OBJECT_HANDLE
#include "UObject/RemoteObjectPrivate.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogUObjectBase, Log, All);
DEFINE_STAT(STAT_UObjectsStatGroupTester);

DECLARE_CYCLE_STAT(TEXT("CreateStatID"), STAT_CreateStatID, STATGROUP_StatSystem);

DEFINE_LOG_CATEGORY(LogUObjectBootstrap);

LLM_DEFINE_TAG(UObject_UObjectBase);

#if CSV_PROFILER_STATS && CSV_TRACK_UOBJECT_COUNT
namespace UObjectStats
{
	COREUOBJECT_API std::atomic<int32> GUObjectCount;
}
#endif

/** Whether uobject system is initialized.												*/
namespace Internal
{
	static bool& GetUObjectSubsystemInitialised()
	{
		static bool ObjInitialized = false;
		return ObjInitialized;
	}
};
bool UObjectInitialized()
{
	return Internal::GetUObjectSubsystemInitialised();
}

/** Objects to automatically register once the object system is ready.					*/
struct FPendingRegistrantInfo
{
	const TCHAR*	Name;
	const TCHAR*	PackageName;
	class UClass*  (*StaticClassFn)();
#if UE_WITH_REMOTE_OBJECT_HANDLE
	const FRemoteObjectId RemoteId;
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
	FPendingRegistrantInfo() = default;
	FPendingRegistrantInfo(class UClass* (*InStaticClassFn)(), const TCHAR* InName, const TCHAR* InPackageName
#if UE_WITH_REMOTE_OBJECT_HANDLE
		, const FRemoteObjectId InRemoteId
#endif
		)
		: Name(InName)
		, PackageName(InPackageName)
		, StaticClassFn(InStaticClassFn)
#if UE_WITH_REMOTE_OBJECT_HANDLE
		, RemoteId(InRemoteId)
#endif
	{}
	static TStripedMap<4, UObjectBase*, FPendingRegistrantInfo>& GetMap()
	{
		static TStripedMap<4, UObjectBase*, FPendingRegistrantInfo> PendingRegistrantInfo;
		return PendingRegistrantInfo;
	}
};


#if !UE_WITH_CONSTINIT_UOBJECT
/** Objects to automatically register once the object system is ready.					*/
struct FPendingRegistrant
{
	UObjectBase*	Object;
	FPendingRegistrant*	NextAutoRegister;
	FPendingRegistrant(UObjectBase* InObject)
	:	Object(InObject)
	,	NextAutoRegister(NULL)
	{}
};
static FPendingRegistrant* GFirstPendingRegistrant = NULL;
static FPendingRegistrant* GLastPendingRegistrant = NULL;
#endif // !UE_WITH_CONSTINIT_UOBJECT

#if USE_PER_MODULE_UOBJECT_BOOTSTRAP
static TMap<FName, TArray<FPendingRegistrant*>>& GetPerModuleBootstrapMap()
{
	static TMap<FName, TArray<FPendingRegistrant*>> PendingRegistrantInfo;
	return PendingRegistrantInfo;
}

#endif

/** Must be called before modules will unload that contains UObject classes */
static void RemoveCompiledInUObjects(TConstArrayView<FName> InModuleNames);

/** Can be called to confirm full removal of UObjects in a list of modules */
static bool CheckLiveObjectsInModules(TConstArrayView<FName> InModuleNames);

/**
 * Constructor used for bootstrapping
 * @param	InClass			possibly NULL, this gives the class of the new object, if known at this time
 * @param	InFlags			RF_Flags to assign
 */
UObjectBase::UObjectBase( EObjectFlags InFlags )
:	ObjectFlags			(InFlags)
,	InternalIndex		(INDEX_NONE)
,	ClassPrivate		(nullptr)
,	OuterPrivate		(nullptr)
{
#if CSV_PROFILER_STATS && CSV_TRACK_UOBJECT_COUNT
	UObjectStats::IncrementUObjectCount();
#endif
}

/**
 * Constructor used by StaticAllocateObject
 * @param	InClass				non NULL, this gives the class of the new object, if known at this time
 * @param	InFlags				RF_Flags to assign
 * @param	InOuter				outer for this object
 * @param	InName				name of the new object
 * @param	InObjectArchetype	archetype to assign
 */
UObjectBase::UObjectBase(UClass* InClass,
	EObjectFlags InFlags,
	EInternalObjectFlags InInternalFlags,
	UObject *InOuter,
	FName InName,
	int32 InInternalIndex,
	int32 InSerialNumber,
	FRemoteObjectId InRemoteId)
:	ObjectFlags			(InFlags)
,	InternalIndex		(INDEX_NONE)
,	ClassPrivate		(InClass)
,	OuterPrivate		(InOuter)
{
	check(ClassPrivate);
	// Add to global table.
	AddObject(InName, InInternalFlags, InInternalIndex, InSerialNumber, InRemoteId);
	
#if CSV_PROFILER_STATS && CSV_TRACK_UOBJECT_COUNT
	UObjectStats::IncrementUObjectCount();
#endif
}	


/**
 * Final destructor, removes the object from the object array, and indirectly, from any annotations
 **/
UObjectBase::~UObjectBase()
{
	// If not initialized, skip out.
	if (UObjectInitialized() && ClassPrivate.GetNoResolve())
	{
		// Validate it.
		check(IsValidLowLevelForDestruction());
		constexpr EObjectFlags DestroyedFlags = RF_BeginDestroyed | RF_FinishDestroyed;
		checkf((GetFlags() & DestroyedFlags) == DestroyedFlags, TEXT("Object %s was not properly destroyed"), *GetFName().ToString());
		checkf(InternalIndex == INDEX_NONE, TEXT("Object %s destroyed outside of GC (InternalIndex=%d, expected %d)"), *GetFName().ToString(), InternalIndex, INDEX_NONE);
	}

#if CSV_PROFILER_STATS && CSV_TRACK_UOBJECT_COUNT
	UObjectStats::DecrementUObjectCount();
#endif
}




/**
 * Convert a boot-strap registered class into a real one, add to uobject array, etc
 *
 * @param UClassStaticClass Now that it is known, fill in ClassPrivate.  This is usually UClass::StaticClass.
 */
void UObjectBase::DeferredRegister(UClass *UClassStaticClass, const TCHAR* PackageName, const TCHAR* InName
#if UE_WITH_REMOTE_OBJECT_HANDLE
	, FRemoteObjectId RemoteId
#endif
)
{
	check(UObjectInitialized());
	// Set object properties.
	UPackage* Package = CreatePackage(PackageName);
	check(Package);
	Package->SetPackageFlags(PKG_CompiledIn);
	OuterPrivate = Package;

	check(UClassStaticClass);
	check(!ClassPrivate);
	ClassPrivate = UClassStaticClass;

	// Add to the global object table.
	AddObject(FName(InName), EInternalObjectFlags::None
#if UE_WITH_REMOTE_OBJECT_HANDLE
		, /*IndernalIndex = */ -1, /*SerialNumber = */ 0, RemoteId
#endif
	);

	// At this point all compiled-in objects should have already been fully constructed so it's safe to remove the NotFullyConstructed flag
	// which was set in FUObjectArray::AllocateUObjectIndex (called from AddObject)
	FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(InternalIndex);
	ObjectItem->ClearFlags(EInternalObjectFlags::PendingConstruction);

#if UE_WITH_REMOTE_OBJECT_HANDLE
	checkf(ObjectItem->GetRemoteId().IsValid() && (!RemoteId.IsValid() || RemoteId == ObjectItem->GetRemoteId()), TEXT("Native object %s %s has an unexpected RemoteId:%s (expected:%s)"),
		*GetClass()->GetName(), *GetFName().ToString(), *ObjectItem->GetRemoteId().ToString(), *RemoteId.ToString());
#endif

	// Make sure that objects disregarded for GC are part of root set.
	check(!GUObjectArray.IsDisregardForGC(this) || GUObjectArray.IndexToObject(InternalIndex)->IsRootSet());

	UE_LOGF(LogUObjectBootstrap, Verbose, "UObjectBase::DeferredRegister %ls %ls", PackageName, InName);
}

/**
 * Add a newly created object to the name hash tables and the object array
 *
 * @param Name name to assign to this uobject
 */
UE_AUTORTFM_ALWAYS_OPEN
void UObjectBase::AddObject(FName InName, EInternalObjectFlags InSetInternalFlags, int32 InInternalIndex, int32 InSerialNumber, FRemoteObjectId InRemoteId)
{
	new (&NamePrivate) FNameAndObjectHashIndex(InName); // Overwrite potentially compiled-in string
	EInternalObjectFlags InternalFlagsToSet = InSetInternalFlags;
	if (!IsInGameThread())
	{
		InternalFlagsToSet |= EInternalObjectFlags::Async;
	}
	if (ObjectFlags & RF_MarkAsRootSet)
	{		
		InternalFlagsToSet |= EInternalObjectFlags::RootSet;
		ObjectFlags &= ~RF_MarkAsRootSet;
	}
	if (ObjectFlags & RF_MarkAsNative)
	{
		InternalFlagsToSet |= EInternalObjectFlags::Native;
		ObjectFlags &= ~RF_MarkAsNative;
	}
	GUObjectArray.AllocateUObjectIndex(this, InternalFlagsToSet, InInternalIndex, InSerialNumber, InRemoteId);
	check(InName != NAME_None && InternalIndex >= 0);
	HashObject(this);
	check(IsValidLowLevel());
}

/**
 * Just change the FName and Outer and rehash into name hash tables. For use by higher level rename functions.
 *
 * @param NewName	new name for this object
 * @param NewOuter	new outer for this object, if NULL, outer will be unchanged
 */
void UObjectBase::LowLevelRename(FName NewName,UObject *NewOuter)
{
	checkfSlow(!EnumHasAllFlags((EObjectFlags)GetFlagsInternal(), RF_NeedInitialization|RF_MarkAsNative),
		TEXT("UObjectBase::LowLevelRename called on a native object which is not yet initialized."));
	if (AutoRTFM::IsClosed())
	{
		FName OldName = GetFName();
		TObjectPtr<UObject> This = static_cast<UObject*>(this);
		TObjectPtr<UObject> OldOuter = OuterPrivate;
		TObjectPtr<UObject> NewOuterPtr = NewOuter;
		UE_AUTORTFM_ONABORT(This, NewName, OldName, OldOuter, NewOuterPtr)
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			// Prevent us from ever migrating an object since aborting is probably outside of a transactional scope (we will use old copies which we should have, see ensures below)
			UE::RemoteObject::Private::FUnsafeToMigrateScope CannotMigrateOutsideOfTransaction;

			// These can be LocalNotReady if they were migrated this transaction (and now undone). We will use their old copies for these operations to avoid migrations.
			ensureMsgf(This.GetResidence() != EResidence::Remote, TEXT("Somehow in-use Object %s migrated during a transaction"), *This.GetRemoteId().ToString());
			ensureMsgf(NewOuterPtr.GetResidence() != EResidence::Remote, TEXT("Somehow in-use NewOuter %s migrated during a transaction"), *NewOuterPtr.GetRemoteId().ToString());
			ensureMsgf(OldOuter.GetResidence() != EResidence::Remote, TEXT("Somehow in-use OldOuter %s migrated during a transaction"), *OldOuter.GetRemoteId().ToString());
#endif
			// Due to the unsafe to migrate above, we do need to check for existence
			if (This)
			{
				// If we were constructed this transaction, then we know we aren't going to persist don't bother renaming.
				if (This->HasAnyInternalFlags(EInternalObjectFlags::PendingConstruction))
				{
					return;
				}

				// Only rename if the new name / owner matches the current name / 
				// owner, as the name may have been changed due to a whole-object
				// revert.
				if (NewName == This->GetFName() && NewOuterPtr == This->OuterPrivate)
				{
					This->LowLevelRename(OldName, OldOuter);
				}
			}
		};
		UE_AUTORTFM_OPEN
		{
			LowLevelRename(NewName, NewOuter);
		};
		return;
	}

#if STATS || ENABLE_STATNAMEDEVENTS_UOBJECT
	((UObject*)this)->ResetStatID(); // reset the stat id since this thing now has a different name
#endif
	UnhashObject(this);
	check(InternalIndex >= 0);
	NamePrivate.Name = NewName;
	if (NewOuter)
	{
		OuterPrivate = NewOuter;
	}
	HashObject(this);
}

UPackage* UObjectBase::GetExternalPackage() const
{
	// if we have no outer, consider this a package, packages returns themselves as their external package
	if (UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(this) == nullptr)
	{
		return CastChecked<UPackage>((UObject*)(this));
	}
	UPackage* ExternalPackage = nullptr;
	if ((GetFlags() & RF_HasExternalPackage) != 0)
	{
		ExternalPackage = GetObjectExternalPackageThreadSafe(this);
	}
	return ExternalPackage;
}

UPackage* UObjectBase::GetExternalPackageInternal() const
{
	// if we have no outer, consider this a package, packages returns themselves as their external package
	if (UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(this) == nullptr)
	{
		return CastChecked<UPackage>((UObject*)(this));
	}
	return (GetFlags() & RF_HasExternalPackage) != 0 ? GetObjectExternalPackageInternal(this) : nullptr;
}

void UObjectBase::SetExternalPackage(UPackage* InPackage)
{
	// if we have no outer, consider this a package, packages have themselves as their external package and that shouldn't be added to the object hash
	if (UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(this) == nullptr)
	{
		// Just validate that we tried to set ourselves or nothing as our external package which is a no-op. anything else is illegal for package
		check(GetClass()->IsChildOf(UPackage::StaticClass()) && (InPackage == this || InPackage == nullptr));
		return;
	}
	if (InPackage)
	{
		HashObjectExternalPackage(this, InPackage);
	}
	else
	{
		UnhashObjectExternalPackage(this);
	}
}

#if WITH_EDITOR
void UObjectBase::SetClass(UClass* NewClass)
{
#if STATS || ENABLE_STATNAMEDEVENTS_UOBJECT
	((UObject*)this)->ResetStatID(); // reset the stat id since this thing now has a different name
#endif

	UnhashObject(this);

	UClass* OldClass = ClassPrivate;
	ClassPrivate->DestroyPersistentUberGraphFrame((UObject*)this);
	ClassPrivate = NewClass;
	ClassPrivate->CreatePersistentUberGraphFrame((UObject*)this, /*bCreateOnlyIfEmpty =*/false, /*bSkipSuperClass =*/false, OldClass);
	HashObject(this);
}
#endif

bool UObjectBase::IsValidLowLevelForDestruction() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!IsThisNotNull(this, "UObjectBase::IsValidLowLevelForDestruction"))
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		UE_LOGF(LogUObjectBase, Warning, "NULL object");
		return false;
	}
	if (!ClassPrivate.GetNoResolve())
	{
		UE_LOGF(LogUObjectBase, Warning, "Object is not registered");
		return false;
	}
	return true;
}

/**
 * Checks to see if the object appears to be valid
 * @return true if this appears to be a valid object
 */
bool UObjectBase::IsValidLowLevel() const
{
	return IsValidLowLevelForDestruction() && GUObjectArray.IsValid(this);
}

bool UObjectBase::IsValidLowLevelFast(bool bRecursive /*= true*/) const
{
	// As DEFAULT_ALIGNMENT is defined to 0 now, I changed that to the original numerical value here
	const int32 AlignmentCheck = MIN_ALIGNMENT - 1;

	// Check 'this' pointer before trying to access any of the Object's members
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if ((!IsThisNotNull(this, "UObjectBase::IsValidLowLevelFast")) || (UPTRINT)this < 0x100)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		UE_LOGF(LogUObjectBase, Error, "\'this\' pointer is invalid.");
		return false;
	}
	if ((UPTRINT)this & AlignmentCheck)
	{
		UE_LOGF(LogUObjectBase, Error, "\'this\' pointer is misaligned.");
		return false;
	}
	if (*(void**)this == nullptr)
	{
		UE_LOGF(LogUObjectBase, Error, "Virtual functions table is invalid.");
		return false;
	}

	// These should all be 0.
	UObject* Outer = OuterPrivate.GetNoResolve();
	UClass* Class = ClassPrivate.GetNoResolve();
	const UPTRINT CheckZero = (GetFlagsInternal() & ~RF_AllFlags) | ((UPTRINT)Class & AlignmentCheck) | ((UPTRINT)Outer & AlignmentCheck);
	if (!!CheckZero)
	{
		UE_LOGF(LogUObjectBase, Error, "Object flags are invalid or either Class or Outer is misaligned");
    return false;
	}
	// These should all be non-NULL (except CDO-alignment check which should be 0)
	if (Class == nullptr || !Class->CheckDefaultObjectIsValidLowLevelFast(AlignmentCheck))
	{
#if WITH_LIVE_CODING
		// When live coding is re-instancing blueprint generated classes, we have to clear out the default object so it can get 
		// GC'ed and deleted prior to live coding completing the patching process (of the destructor specifically)
		if (Class == nullptr || !Class->HasAnyClassFlags(CLASS_NewerVersionExists))
#endif
		{
			UE_LOGF(LogUObjectBase, Error, "Class pointer is invalid or CDO is invalid.");
			return false;
		}
	}
	// Avoid infinite recursion so call IsValidLowLevelFast on the class object with bRecursive = false.
	if (bRecursive && !Class->IsValidLowLevelFast(false))
	{
		UE_LOGF(LogUObjectBase, Error, "Class object failed IsValidLowLevelFast test.");
		return false;
	}
	// Lightweight versions of index checks.
	if (!GUObjectArray.IsValidIndex(this) || !GetFName().IsValidIndexFast())
	{
		UE_LOGF(LogUObjectBase, Error, "Object array index or name index is invalid.");
		return false;
	}
	return true;
}

#if USE_PER_MODULE_UOBJECT_BOOTSTRAP
static void UObjectReleaseModuleRegistrants(FName Module)
{
	TMap<FName, TArray<FPendingRegistrant*>>& PerModuleMap = GetPerModuleBootstrapMap();

	FName Package = IPluginManager::Get().PackageNameFromModuleName(Module);

	FName ScriptName = *(FString("/Script/") + Package.ToString());

	TArray<FPendingRegistrant*>* Array = PerModuleMap.Find(ScriptName);
	if (Array)
	{
		SCOPED_BOOT_TIMING("UObjectReleaseModuleRegistrants");
		for (FPendingRegistrant* PendingRegistration : *Array)
		{
			if (GLastPendingRegistrant)
			{
				GLastPendingRegistrant->NextAutoRegister = PendingRegistration;
			}
			else
			{
				check(!GFirstPendingRegistrant);
				GFirstPendingRegistrant = PendingRegistration;
			}
			GLastPendingRegistrant = PendingRegistration;
		}
		UE_LOGF(LogUObjectBootstrap, Verbose, "UObjectReleaseModuleRegistrants %d items in %ls", Array->Num(), *ScriptName.ToString());
		PerModuleMap.Remove(ScriptName);
	}
	else
	{
		UE_LOGF(LogUObjectBootstrap, Verbose, "UObjectReleaseModuleRegistrants no items in %ls", *ScriptName.ToString());
	}
}

void UObjectReleaseAllModuleRegistrants()
{
	SCOPED_BOOT_TIMING("UObjectReleaseAllModuleRegistrants");
	TMap<FName, TArray<FPendingRegistrant*>>& PerModuleMap = GetPerModuleBootstrapMap();
	for (auto& Pair : PerModuleMap)
	{
		for (FPendingRegistrant* PendingRegistration : Pair.Value)
		{
			if (GLastPendingRegistrant)
			{
				GLastPendingRegistrant->NextAutoRegister = PendingRegistration;
			}
			else
			{
				check(!GFirstPendingRegistrant);
				GFirstPendingRegistrant = PendingRegistration;
			}
			GLastPendingRegistrant = PendingRegistration;
		}
		UE_LOGF(LogUObjectBootstrap, Verbose, "UObjectReleaseAllModuleRegistrants %d items in %ls", Pair.Value.Num(), *Pair.Key.ToString());
	}
	PerModuleMap.Empty();
	ProcessNewlyLoadedUObjects();
}

static void DumpPendingUObjectModules(const TArray<FString>& Args)
{
	TMap<FName, TArray<FPendingRegistrant*>>& PerModuleMap = GetPerModuleBootstrapMap();
	for (auto& Pair : PerModuleMap)
	{
		UE_LOGF(LogUObjectBootstrap, Display, "Not yet loaded: %d items in %ls", Pair.Value.Num(), *Pair.Key.ToString());
	}
}

static FAutoConsoleCommand DumpPendingUObjectModulesCmd(
	TEXT("DumpPendingUObjectModules"),
	TEXT("When doing per-module UObject bootstrapping, show the modules that are not yet loaded."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&DumpPendingUObjectModules)
);

#endif

#if !UE_WITH_CONSTINIT_UOBJECT
void UObjectBase::Register(class UClass* (*StaticClassFn)(), const TCHAR* PackageName, const TCHAR* InName)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectBase);
	TStripedMap<4, UObjectBase*, FPendingRegistrantInfo>& PendingRegistrants = FPendingRegistrantInfo::GetMap();

	FPendingRegistrant* PendingRegistration = new FPendingRegistrant(this);
	PendingRegistrants.Add(this, FPendingRegistrantInfo(StaticClassFn, InName, PackageName
#if UE_WITH_REMOTE_OBJECT_HANDLE
		, FRemoteObjectId::Generate(this, InName, PackageName)
#endif
	));

#if USE_PER_MODULE_UOBJECT_BOOTSTRAP
	if (FName(PackageName) != FName("/Script/CoreUObject"))
	{
		TMap<FName, TArray<FPendingRegistrant*>>& PerModuleMap = GetPerModuleBootstrapMap();

		PerModuleMap.FindOrAdd(FName(PackageName)).Add(PendingRegistration);
	}
	else
#endif
	{
		if (GLastPendingRegistrant)
		{
			GLastPendingRegistrant->NextAutoRegister = PendingRegistration;
		}
		else
		{
			check(!GFirstPendingRegistrant);
			GFirstPendingRegistrant = PendingRegistration;
		}
		GLastPendingRegistrant = PendingRegistration;
	}
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

#if UE_WITH_REMOTE_OBJECT_HANDLE
FRemoteObjectId UObjectBase::GetPendingRegistrantRemoteId() const
{
	FRemoteObjectId RemoteId;
	TStripedMap<4, UObjectBase*, FPendingRegistrantInfo>& PendingRegistrants = FPendingRegistrantInfo::GetMap();
	bool bFound = PendingRegistrants.FindAndApply(this,
		[&RemoteId](const FPendingRegistrantInfo& Info)
		{
			RemoteId = Info.RemoteId;
		}
	);
	check(bFound);
	return RemoteId;
}
#endif

#if !UE_WITH_CONSTINIT_UOBJECT
/**
 * Dequeues registrants from the list of pending registrations into an array.
 * The contents of the array is preserved, and the new elements are appended.
 */
static void DequeuePendingAutoRegistrants(TArray<FPendingRegistrant>& OutPendingRegistrants)
{
	// We process registrations in the order they were enqueued, since each registrant ensures
	// its dependencies are enqueued before it enqueues itself.
	FPendingRegistrant* NextPendingRegistrant = GFirstPendingRegistrant;
	GFirstPendingRegistrant = NULL;
	GLastPendingRegistrant = NULL;
	while(NextPendingRegistrant)
	{
		FPendingRegistrant* PendingRegistrant = NextPendingRegistrant;
		OutPendingRegistrants.Add(*PendingRegistrant);
		NextPendingRegistrant = PendingRegistrant->NextAutoRegister;
		delete PendingRegistrant;
	};
}

/**
 * Process the auto register objects adding them to the UObject array
 */
void UObjectProcessRegistrants()
{
	SCOPED_BOOT_TIMING("UObjectProcessRegistrants");

	check(UObjectInitialized());
	// Make list of all objects to be registered.
	TArray<FPendingRegistrant> PendingRegistrants;
	DequeuePendingAutoRegistrants(PendingRegistrants);

	for(int32 RegistrantIndex = 0;RegistrantIndex < PendingRegistrants.Num();++RegistrantIndex)
	{
		const FPendingRegistrant& PendingRegistrant = PendingRegistrants[RegistrantIndex];

		UObjectForceRegistration(PendingRegistrant.Object, false);

		check(PendingRegistrant.Object->GetClass()); // should have been set by DeferredRegister

		// Register may have resulted in new pending registrants being enqueued, so dequeue those.
		DequeuePendingAutoRegistrants(PendingRegistrants);
	}
}

void UObjectForceRegistration(UObjectBase* Object, bool bCheckForModuleRelease)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectBase);
	TStripedMap<4, UObjectBase*, FPendingRegistrantInfo>& PendingRegistrants = FPendingRegistrantInfo::GetMap();

	const TCHAR* Name;
	const TCHAR* PackageName;
	UClass* StaticClass;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	FRemoteObjectId RemoteId;
#endif
	if (PendingRegistrants.RemoveIf(Object,
		[&](FPendingRegistrantInfo& Info)
		{
			PackageName = Info.PackageName;
			Name = Info.Name;
			StaticClass = Info.StaticClassFn();
#if UE_WITH_REMOTE_OBJECT_HANDLE
			RemoteId = Info.RemoteId;
#endif
			return true;
		}))
	{
#if USE_PER_MODULE_UOBJECT_BOOTSTRAP
		if (bCheckForModuleRelease)
		{
			UObjectReleaseModuleRegistrants(FName(PackageName));
		}
#endif
		Object->DeferredRegister(StaticClass, PackageName, Name
#if UE_WITH_REMOTE_OBJECT_HANDLE
			, RemoteId
#endif
		);
	}
}

// UScriptStruct deferred registration

void RegisterCompiledInInfo(class UScriptStruct* (*InOuterRegister)(ETypeConstructPhase), const TCHAR* InPackageName, const TCHAR* InName, FStructRegistrationInfo& InInfo, const FStructReloadVersionInfo& InVersionInfo)
{
	check(InOuterRegister);
	FStructDeferredRegistry::Get().AddRegistration(InOuterRegister, InPackageName, InName, InInfo, InVersionInfo);
	NotifyRegistrationEvent(InPackageName, InName, ENotifyRegistrationType::NRT_Struct, ENotifyRegistrationPhase::NRP_Added, (FTypeConstructFunc*)InOuterRegister, false);
}

class UScriptStruct *GetStaticStruct(class UScriptStruct *(*InRegister)(ETypeConstructPhase), UObject* StructOuter, const TCHAR* StructName)
{
	UScriptStruct *Result = (*InRegister)(ETypeConstructPhase::Inner);
	NotifyRegistrationEvent(*StructOuter->GetOutermost()->GetName(), StructName, ENotifyRegistrationType::NRT_Struct, ENotifyRegistrationPhase::NRP_Finished, nullptr, false, Result);
	return Result;
}

// UEnum deferred registration

void RegisterCompiledInInfo(class UEnum* (*InOuterRegister)(ETypeConstructPhase), const TCHAR* InPackageName, const TCHAR* InName, FEnumRegistrationInfo& InInfo, const FEnumReloadVersionInfo& InVersionInfo)
{
	check(InOuterRegister);
	FEnumDeferredRegistry::Get().AddRegistration(InOuterRegister, InPackageName, InName, InInfo, InVersionInfo);
	NotifyRegistrationEvent(InPackageName, InName, ENotifyRegistrationType::NRT_Enum, ENotifyRegistrationPhase::NRP_Added, (FTypeConstructFunc*)InOuterRegister, false);
}

class UEnum *GetStaticEnum(class UEnum *(*InRegister)(ETypeConstructPhase), UObject* EnumOuter, const TCHAR* EnumName)
{
	UEnum *Result = (*InRegister)(ETypeConstructPhase::Inner);
	NotifyRegistrationEvent(*EnumOuter->GetOutermost()->GetName(), EnumName, ENotifyRegistrationType::NRT_Enum, ENotifyRegistrationPhase::NRP_Finished, nullptr, false, Result);
	return Result;
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

FName UObjectBase::GetFNameForStatID() const
{
	return GetFName();
}

/** Removes prefix from the native class name */
FString UObjectBase::RemoveClassPrefix(const TCHAR* ClassName)
{
	static const TCHAR* DeprecatedPrefix = TEXT("DEPRECATED_");
	FString NameWithoutPrefix(ClassName);
	NameWithoutPrefix.MidInline(1, MAX_int32, EAllowShrinking::No);
	if (NameWithoutPrefix.StartsWith(DeprecatedPrefix))
	{
		NameWithoutPrefix.MidInline(FCString::Strlen(DeprecatedPrefix), MAX_int32, EAllowShrinking::No);
	}
	return NameWithoutPrefix;
}

#if !UE_WITH_CONSTINIT_UOBJECT
void RegisterCompiledInInfo(class UClass* (*InRegister)(ETypeConstructPhase), const TCHAR* InPackageName, const TCHAR* InName, FClassRegistrationInfo& InInfo, const FClassReloadVersionInfo& InVersionInfo)
{
	check(InRegister);
	FClassDeferredRegistry::AddResult result = FClassDeferredRegistry::Get().AddRegistration(InRegister, InPackageName, InName, InInfo, InVersionInfo);
#if WITH_RELOAD
	if (result == FClassDeferredRegistry::AddResult::ExistingChanged && !IsReloadActive())
	{
		// Class exists, this can only happen during hot-reload or live coding
		UE_LOGF(LogUObjectBase, Fatal, "Trying to recreate changed class '%ls' outside of hot reload and live coding!", InName);
	}
#endif
	FString NoPrefix(UObjectBase::RemoveClassPrefix(InName));
	NotifyRegistrationEvent(InPackageName, *NoPrefix, ENotifyRegistrationType::NRT_Class, ENotifyRegistrationPhase::NRP_Added, (FTypeConstructFunc*)InRegister, false);
	NotifyRegistrationEvent(InPackageName, *(FString(DEFAULT_OBJECT_PREFIX) + NoPrefix), ENotifyRegistrationType::NRT_ClassCDO, ENotifyRegistrationPhase::NRP_Added, (FTypeConstructFunc*)InRegister, false);
}

// UPackage registration

void RegisterCompiledInInfo(UPackage* (*InOuterRegister)(ETypeConstructPhase), const TCHAR* InPackageName, FPackageRegistrationInfo& InInfo, const FPackageReloadVersionInfo& InVersionInfo)
{
#if WITH_RELOAD
	check(InOuterRegister);
	FPackageDeferredRegistry::Get().AddRegistration(InOuterRegister, TEXT(""), InPackageName, InInfo, InVersionInfo);
#endif
}

// Multiple registrations
void RegisterCompiledInInfo(const TCHAR* PackageName, const FClassRegisterCompiledInInfo* ClassInfo, size_t NumClassInfo, const FStructRegisterCompiledInInfo* StructInfo, size_t NumStructInfo, const FEnumRegisterCompiledInInfo* EnumInfo, size_t NumEnumInfo, const FPartialRegisterCompiledInInfo *PartialsInfo, size_t NumPartialsInfo)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectBase);

	for (size_t Index = 0; Index < NumClassInfo; ++Index)
	{
		const FClassRegisterCompiledInInfo& Info = ClassInfo[Index];
		RegisterCompiledInInfo(Info.Register, PackageName, Info.Name, *Info.Info, Info.VersionInfo);
	}

	for (size_t Index = 0; Index < NumStructInfo; ++Index)
	{
		const FStructRegisterCompiledInInfo& Info = StructInfo[Index];
		RegisterCompiledInInfo(Info.OuterRegister, PackageName, Info.Name, *Info.Info, Info.VersionInfo);
		if (Info.CreateCppStructOps != nullptr)
		{
			UScriptStruct::DeferCppStructOps(FTopLevelAssetPath(FName(PackageName), FName(Info.Name)), (UScriptStruct::ICppStructOps*)Info.CreateCppStructOps());
		}
	}

	for (size_t Index = 0; Index < NumEnumInfo; ++Index)
	{
		const FEnumRegisterCompiledInInfo& Info = EnumInfo[Index];
		RegisterCompiledInInfo(Info.OuterRegister, PackageName, Info.Name, *Info.Info, Info.VersionInfo);
	}

	for (size_t Index = 0; Index < NumPartialsInfo; ++Index)
	{
		const FPartialRegisterCompiledInInfo& Info = PartialsInfo[Index];
		FPartialDeferredRegistry::Get().AddRegistration(Info.OuterRegister, PackageName, Info.Name, *Info.Info, Info.VersionInfo);
	}
}

/** Register all loaded classes */
void UClassRegisterAllCompiledInClasses()
{
#if WITH_RELOAD
	TArray<UClass*> AddedClasses;
#endif
	SCOPED_BOOT_TIMING("UClassRegisterAllCompiledInClasses");
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectBase);

	FClassDeferredRegistry& Registry = FClassDeferredRegistry::Get();

	Registry.ProcessChangedObjects();

	for (const FClassDeferredRegistry::FRegistrant& Registrant : Registry.GetRegistrations())
	{
		UClass* RegisteredClass = FClassDeferredRegistry::InnerRegister(Registrant);
#if WITH_RELOAD
		if (IsReloadActive() && Registrant.OldSingleton == nullptr)
		{
			AddedClasses.Add(RegisteredClass);
		}
#endif
	}

#if WITH_RELOAD
	if (AddedClasses.Num() > 0)
	{
		FCoreUObjectDelegates::ReloadAddedClassesDelegate.Broadcast(AddedClasses);
	}
#endif
}
#endif // UE_WITH_CONSTINIT_UOBJECT

#if WITH_RELOAD
/** Re-instance all existing classes that have changed during reload */
void UClassReplaceReloadClasses()
{
	for (const FClassDeferredRegistry::FRegistrant& Registrant : FClassDeferredRegistry::Get().GetRegistrations())
	{
		if (Registrant.OldSingleton == nullptr)
		{
			continue;
		}

		UClass* RegisteredClass = nullptr;
		if (Registrant.bHasChanged)
		{
			RegisteredClass = FClassDeferredRegistry::InnerRegister(Registrant);
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FCoreUObjectDelegates::RegisterClassForHotReloadReinstancingDelegate.Broadcast(Registrant.OldSingleton, RegisteredClass, Registrant.bHasChanged ? EHotReloadedClassFlags::Changed : EHotReloadedClassFlags::None);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCoreUObjectDelegates::ReinstanceHotReloadedClassesDelegate.Broadcast();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
#endif

#if !UE_WITH_CONSTINIT_UOBJECT
/**
 * Load any outstanding compiled in default properties
 */
static void UObjectLoadAllCompiledInDefaultProperties(TArray<UClass*>& OutAllNewClasses)
{
	TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("UObjectLoadAllCompiledInDefaultProperties"));

	static FName LongEnginePackageName("/Script/Engine");

	FClassDeferredRegistry& ClassRegistry = FClassDeferredRegistry::Get();

	if (ClassRegistry.HasPendingRegistrations())
	{
		SCOPED_BOOT_TIMING("UObjectLoadAllCompiledInDefaultProperties");
		TArray<UClass*> NewClasses;
		TArray<UClass*> NewClassesInCoreUObject;
		TArray<UClass*> NewClassesInEngine;
		{
			SCOPED_BOOT_TIMING("DoPendingOuterRegistrations");
			ClassRegistry.DoPendingOuterRegistrations(true, [&OutAllNewClasses, &NewClasses, &NewClassesInCoreUObject, &NewClassesInEngine](const TCHAR* PackageName, UClass& Class) -> void
				{
					UE_LOGF(LogUObjectBootstrap, Verbose, "UObjectLoadAllCompiledInDefaultProperties After Registrant %ls %ls", PackageName, *Class.GetName());

					if (Class.GetOutermost()->GetFName() == GLongCoreUObjectPackageName)
					{
						NewClassesInCoreUObject.Add(&Class);
					}
					else if (Class.GetOutermost()->GetFName() == LongEnginePackageName)
					{
						NewClassesInEngine.Add(&Class);
					}
					else
					{
						NewClasses.Add(&Class);
					}

					OutAllNewClasses.Add(&Class);
				});
		}		

		// Sort classes by depth to allow creating CDOs by parent type first. Otherwise, complex PostCDOConstruct calls 
		// might rely on their parent having been initialized already, which might not be possible if there is a cycle in 
		// the GetDefaultObject -> PostCDOConstruct loading graph.
		{
			SCOPED_BOOT_TIMING("SortClassesByDepth");
			// Memo to avoid re-calculating depth needlessly as we sort. It's important to note that since
			// two FindOrAdds will occur back to back that we reserve our memory upfront to avoid re-allocing
			// in-between FindOrAdd calls which will invalidate the first reference returned.
			TMap<const UClass*, int32> DepthMemo;
			DepthMemo.Reserve(OutAllNewClasses.Num());
			auto SortByClassDepth = [&DepthMemo](TArray<UClass*>& Classes)
				{
					Algo::StableSort(Classes, [&DepthMemo](const UClass* A, const UClass* B)
						{
							int32& ADepth = DepthMemo.FindOrAdd(A);
							if (ADepth == 0)
							{
								do
								{
									ADepth++;
									A = A->GetSuperClass();
								} while (A);
							}
							int32& BDepth = DepthMemo.FindOrAdd(B);
							if (BDepth == 0)
							{
								do
								{
									BDepth++;
									B = B->GetSuperClass();
								} while (B);
							}
							return ADepth < BDepth;
						});
				};
			SortByClassDepth(NewClassesInCoreUObject);
			SortByClassDepth(NewClassesInEngine);
			SortByClassDepth(NewClasses);
			checkf(DepthMemo.Num() <= OutAllNewClasses.Num(), TEXT("If we've added more than we reserved initially, we resized while iterating which may indicate a use-after-free problem."));
		}

		auto NotifyClassFinishedRegistrationEvents = [](TArray<UClass*>& Classes)
		{
			for (UClass* Class : Classes)
			{
				TCHAR PackageName[FName::StringBufferSize];
				TCHAR ClassName[FName::StringBufferSize];
				Class->GetOutermost()->GetFName().ToString(PackageName);
				Class->GetFName().ToString(ClassName);
				NotifyRegistrationEvent(PackageName, ClassName, ENotifyRegistrationType::NRT_Class, ENotifyRegistrationPhase::NRP_Finished, nullptr, false, Class);
			}
		};

		// notify async loader of all new classes before creating the class default objects
		{
			SCOPED_BOOT_TIMING("NotifyClassFinishedRegistrationEvents");
			NotifyClassFinishedRegistrationEvents(NewClassesInCoreUObject);
			NotifyClassFinishedRegistrationEvents(NewClassesInEngine);
			NotifyClassFinishedRegistrationEvents(NewClasses);
		}

		{
			SCOPED_BOOT_TIMING("CoreUObject Classes");
			for (UClass* Class : NewClassesInCoreUObject) // we do these first because we assume these never trigger loads
			{
				UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject Begin %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
				Class->GetDefaultObject();
#if UE_WITH_IMMUTABLE_CDO
				Class->GetImmutableDefaultObject();
#endif
				UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject End %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
			}
		}
		{
			SCOPED_BOOT_TIMING("Engine Classes");
			for (UClass* Class : NewClassesInEngine) // we do these second because we want to bring the engine up before the game
			{
				UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject Begin %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
				Class->GetDefaultObject();
#if UE_WITH_IMMUTABLE_CDO
				Class->GetImmutableDefaultObject();
#endif
				UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject End %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
			}
		}
		{
			SCOPED_BOOT_TIMING("Other Classes");
			for (UClass* Class : NewClasses)
			{
				UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject Begin %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
				Class->GetDefaultObject();
#if UE_WITH_IMMUTABLE_CDO
				Class->GetImmutableDefaultObject();
#endif
				UE_LOGF(LogUObjectBootstrap, Verbose, "GetDefaultObject End %ls %ls", *Class->GetOutermost()->GetName(), *Class->GetName());
			}
		}
		FFeedbackContext& ErrorsFC = UClass::GetDefaultPropertiesFeedbackContext();
		if (ErrorsFC.GetNumErrors() || ErrorsFC.GetNumWarnings())
		{
			TArray<FString> AllErrorsAndWarnings;
			ErrorsFC.GetErrorsAndWarningsAndEmpty(AllErrorsAndWarnings);

			FString AllInOne;
			UE_LOGF(LogUObjectBase, Warning, "-------------- Default Property warnings and errors:");
			for (const FString& ErrorOrWarning : AllErrorsAndWarnings)
			{
				UE_LOGF(LogUObjectBase, Warning, "%ls", *ErrorOrWarning);
				AllInOne += ErrorOrWarning;
				AllInOne += TEXT("\n");
			}
			FMessageDialog::Open(EAppMsgType::Ok, FText::Format( NSLOCTEXT("Core", "DefaultPropertyWarningAndErrors", "Default Property warnings and errors:\n{0}"), FText::FromString( AllInOne ) ) );
		}
	}
}

/**
 * Call StaticStruct for each struct...this sets up the internal singleton, and important works correctly with hot reload
 */
static void UObjectLoadAllCompiledInStructs()
{
	SCOPED_BOOT_TIMING("UObjectLoadAllCompiledInStructs");

	FEnumDeferredRegistry& EnumRegistry = FEnumDeferredRegistry::Get();
	FStructDeferredRegistry& StructRegistry = FStructDeferredRegistry::Get();

	{
		SCOPED_BOOT_TIMING("UObjectLoadAllCompiledInStructs -  CreatePackages (could be optimized!)");
		EnumRegistry.DoPendingPackageRegistrations();
		StructRegistry.DoPendingPackageRegistrations();
	}

	// Load Structs
	EnumRegistry.DoPendingOuterRegistrations(true);
	StructRegistry.DoPendingOuterRegistrations(true);
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

void RegisterModularObjectsProcessing()
{
	UE_CALL_ONCE([]()
	{
		FModuleManager::Get().OnProcessLoadedObjectsCallback().AddStatic(ProcessNewlyLoadedUObjects);
		FModuleManager::Get().OnModulesUnloaded().AddStatic(RemoveCompiledInUObjects);
		FModuleManager::Get().OnCheckLiveObjectsInModulesCallback().BindStatic(CheckLiveObjectsInModules);

		FCoreUObjectDelegates::GarbageCollectComplete.AddLambda([]()
		{
			FModuleManager::Get().OnObjectCleanup();
		});
	});
}

#if !UE_WITH_CONSTINIT_UOBJECT
void ProcessNewlyLoadedUObjects(FName InModuleName, bool bCanProcessNewlyLoadedObjects)
{
	SCOPED_BOOT_TIMING("ProcessNewlyLoadedUObjects");
#if USE_PER_MODULE_UOBJECT_BOOTSTRAP
	if (InModuleName != NAME_None)
	{
		UObjectReleaseModuleRegistrants(InModuleName);
	}
#endif
	if (!bCanProcessNewlyLoadedObjects)
	{
		FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.Broadcast(InModuleName, ECompiledInUObjectsRegisteredStatus::Delayed);
		return;
	}
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectBase);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("ProcessNewlyLoadedUObjects"), STAT_ProcessNewlyLoadedUObjects, STATGROUP_ObjectVerbose);

	FPackageDeferredRegistry& PackageRegistry = FPackageDeferredRegistry::Get();
	FClassDeferredRegistry& ClassRegistry = FClassDeferredRegistry::Get();
	FStructDeferredRegistry& StructRegistry = FStructDeferredRegistry::Get();
	FEnumDeferredRegistry& EnumRegistry = FEnumDeferredRegistry::Get();
	FPartialDeferredRegistry& PartialRegistry = FPartialDeferredRegistry::Get();

	PackageRegistry.ProcessChangedObjects(true);
	StructRegistry.ProcessChangedObjects();
	EnumRegistry.ProcessChangedObjects();

	UClassRegisterAllCompiledInClasses();

	bool bNewUObjects = false;
	TArray<UClass*> AllNewClasses;
	while (GFirstPendingRegistrant ||
		ClassRegistry.HasPendingRegistrations() ||
		StructRegistry.HasPendingRegistrations() ||
		EnumRegistry.HasPendingRegistrations() ||
		PartialRegistry.HasPendingRegistrations())
	{
		if (bNewUObjects)
		{
			// We are past first iteration and there are new pending registrations. We need to call this to make sure new classes are explicitly initialized
			// since handling default properties might have references to the new classes.
			// Example: A TSubclassOf<Type> in a new type where Type also is new. In this loop the default property of that TSubclassOf will collect
			// the metaclass but the metaclass might not have been initialized without this call
			UClassRegisterAllCompiledInClasses();
		}

		bNewUObjects = true;

		// Initializing partials first since they need to be linked to class before outer construction
		for (const FPartialDeferredRegistry::FRegistrant& Registrant : PartialRegistry.GetRegistrations())
		{
			UE::CoreUObject::Private::FPartialClass* Params = Registrant.RegisterFn(ETypeConstructPhase::Outer);
			Params->LinkToClass(*Params->Class(ETypeConstructPhase::Inner));
		}
		PartialRegistry.EmptyRegistrations();


		UObjectProcessRegistrants();
		UObjectLoadAllCompiledInStructs();

		FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.Broadcast(InModuleName, ECompiledInUObjectsRegisteredStatus::PreCDO);

		UObjectLoadAllCompiledInDefaultProperties(AllNewClasses);
	}

	FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate.Broadcast(InModuleName, ECompiledInUObjectsRegisteredStatus::PostCDO);

#if WITH_RELOAD
	IReload* Reload = GetActiveReloadInterface();
	if (Reload != nullptr)
	{
		UClassReplaceReloadClasses(); // Legacy
		PackageRegistry.NotifyReload(*Reload);
		EnumRegistry.NotifyReload(*Reload);
		StructRegistry.NotifyReload(*Reload);
		ClassRegistry.NotifyReload(*Reload);
		Reload->Reinstance();
	}
#endif

	PackageRegistry.EmptyRegistrations();
	EnumRegistry.EmptyRegistrations();
	StructRegistry.EmptyRegistrations();
	ClassRegistry.EmptyRegistrations();

	TStripedMap<4, UObjectBase*, FPendingRegistrantInfo>& PendingRegistrants = FPendingRegistrantInfo::GetMap();
	if (PendingRegistrants.Num() == 0 && PendingRegistrants.GetAllocatedSize() > 0)
	{
		PendingRegistrants.Shrink();
	}

	if (bNewUObjects && !GIsInitialLoad)
	{
		for (UClass* Class : AllNewClasses)
		{
			// Assemble reference token stream for garbage collection/ RTGC.
			if (!Class->HasAnyFlags(RF_ClassDefaultObject) && !Class->HasAnyClassFlags(CLASS_TokenStreamAssembled))
			{
				Class->AssembleReferenceTokenStream();
			}
		}
	}
}
#endif // !UE_WITH_CONSTINIT_UOBJECT

static TArray<UPackage*> GetScriptPackagesFromModuleNames(TConstArrayView<FName> InModuleNames, TArray<FName>* OutPackageNames = nullptr)
{
	TArray<UPackage*> Packages;

	Packages.Reserve(3 * InModuleNames.Num());
	if (OutPackageNames)
	{
		OutPackageNames->Reserve(3 * InModuleNames.Num());
	}

	for (const FName ModuleName : InModuleNames)
	{
		// Identify module script packages. The expected package name is '/Script/ModuleName'.
		const FName ModulePackageName = WriteToString<256>(TEXT("/Script/"), ModuleName).ToString();
		if (UPackage* Package = FindObjectFast<UPackage>(/* Outer */ nullptr, ModulePackageName))
		{
			Packages.Add(Package);
		}

		// Identify plugin script packages. The expected package name is '/Script/PluginName'.
		const FName PluginName = IPluginManager::Get().PackageNameFromModuleName(ModuleName);
		const FName PluginPackageName = WriteToString<256>(TEXT("/Script/"), PluginName).ToString();
		if (UPackage* Package = FindObjectFast<UPackage>(/* Outer */ nullptr, PluginPackageName))
		{
			Packages.AddUnique(Package);
		}

		// Identify Verse packages. The expected package name is '/PluginName/_Verse/VNI/ModuleName'.
		const FName VersePackageName = WriteToString<256>(TEXT("/"), PluginName, TEXT("/_Verse/VNI/"), ModuleName).ToString();
		if (UPackage* Package = FindObjectFast<UPackage>(/* Outer */ nullptr, VersePackageName))
		{
			Packages.Add(Package);
		}

		if (OutPackageNames)
		{
			OutPackageNames->Add(ModulePackageName);
			OutPackageNames->AddUnique(PluginPackageName);
			OutPackageNames->Add(VersePackageName);
		}
	}

	return Packages;
}

static void RemoveObjectsInPackages(TConstArrayView<UPackage*> Packages, TArray<FName> PackageNames)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectBase);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("RemoveObjectsInPackages"), STAT_RemoveObjectsInModules, STATGROUP_ObjectVerbose);

	TSet<const UObject*> ObjectsToRemove;
	const FPermanentObjectPoolExtents PermanentPool;

	for (UPackage* Package : Packages)
	{
		UE_LOGF(LogUObjectBase, Log, "RemoveObjectsInPackages: removing package %ls", *Package->GetName());

		ForEachObjectWithPackage(Package, [&PermanentPool, &ObjectsToRemove](UObject* InObject) mutable -> bool
		{
			if (!PermanentPool.Contains(InObject) && !GUObjectArray.IsDisregardForGC(InObject))
			{
				if (InObject->IsA<UClass>())
				{
					ForEachObjectOfClass(Cast<UClass>(InObject), [InObject, &ObjectsToRemove](UObject* InClassObject) -> bool
					{
						ObjectsToRemove.Add(InClassObject);
						return true;
					}, true);
				}

				ObjectsToRemove.Add(InObject);
			}

			return true;
		}, EGetObjectsFlags::IncludeNestedObjects);

		ObjectsToRemove.Add(Package);
	}

	for (const FName PackageName : PackageNames)
	{
		UScriptStruct::RemoveAllDeferredCppStructOps(PackageName);
	}

	UE::GC::Private::RemoveFromRootForUnload(ObjectsToRemove);
}

static bool CheckLiveObjectsInModules(TConstArrayView<FName> InModuleNames)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("CheckLiveObjectsInModules"), STAT_CheckLiveObjectsInModules, STATGROUP_ObjectVerbose);

	TArray<UObject*> ObjectsToRemoveStillLive;

	const TArray<UPackage*> Packages = GetScriptPackagesFromModuleNames(InModuleNames);
	for (UPackage* Package : Packages)
	{
		ForEachObjectWithPackage(Package, [&Package, &ObjectsToRemoveStillLive](UObject* InObject) -> bool
		{
			UE_LOGF(LogUObjectBase, Warning, "CheckLiveObjectsInModules: object %ls in %ls should have been removed (0x%0lX 0x%0lX)",
				*InObject->GetName(), *Package->GetName(), uint32{EnumToUnderlyingType(InObject->GetFlags())}, uint32{EnumToUnderlyingType(InObject->GetInternalFlags())});

			if (InObject->IsA<UClass>())
			{
				ForEachObjectOfClass(Cast<UClass>(InObject), [InObject, &Package, &ObjectsToRemoveStillLive](UObject* InClassObject) -> bool
				{
					UE_LOGF(LogUObjectBase, Warning, "CheckLiveObjectsInModules: instance %ls in %ls should have been removed (0x%0lX 0x%0lX)",
						*InClassObject->GetName(), *Package->GetName(), uint32{EnumToUnderlyingType(InClassObject->GetFlags())}, uint32{EnumToUnderlyingType(InClassObject->GetInternalFlags())});
					ObjectsToRemoveStillLive.Add(InClassObject);

					return true;
				}, /*bIncludeDerivedClasses*/true);
			}

			ObjectsToRemoveStillLive.Add(InObject);
			return true;
		}, EGetObjectsFlags::IncludeNestedObjects);
	}

	if (ObjectsToRemoveStillLive.Num() > 0)
	{
#if !UE_BUILD_SHIPPING
		FReferenceChainSearch::FindAndPrintStaleReferencesToObjects(ObjectsToRemoveStillLive, EPrintStaleReferencesOptions::Log);
		UE_LOGF(LogUObjectBase, Fatal, "CheckLiveObjectsInModules: all marked objects need to be removed after GC");
#endif // !UE_BUILD_SHIPPING
		return true;
	}
	else
	{
		UE_LOGF(LogUObjectBase, Log, "CheckLiveObjectsInModules: all objects were correctly removed");
		return false;
	}
}

static void RemoveCompiledInUObjects(TConstArrayView<FName> InModuleNames)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_UObjectBase);
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("RemoveCompiledInUObjects"), STAT_RemoveCompiledInUObjects, STATGROUP_ObjectVerbose);

	FlushAsyncLoading();

	TArray<FName> PackageNames;
	const TArray<UPackage*> Packages = GetScriptPackagesFromModuleNames(InModuleNames, &PackageNames);
	FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate.Broadcast(Packages);
	RemoveObjectsInPackages(Packages, PackageNames);
}

static int32 GVarMaxObjectsNotConsideredByGC;
static FAutoConsoleVariableRef CMaxObjectsNotConsideredByGC(
	TEXT("gc.MaxObjectsNotConsideredByGC"),
	GVarMaxObjectsNotConsideredByGC,
	TEXT("Placeholder console variable, currently not used in runtime."),
	ECVF_Default
	);

static int32 GMaxObjectsInEditor;
static FAutoConsoleVariableRef CMaxObjectsInEditor(
	TEXT("gc.MaxObjectsInEditor"),
	GMaxObjectsInEditor,
	TEXT("Placeholder console variable, currently not used in runtime."),
	ECVF_Default
	);

static int32 GMaxObjectsInGame = 0;
static FAutoConsoleVariableRef CMaxObjectsInGame(
	TEXT("gc.MaxObjectsInGame"),
	GMaxObjectsInGame,
	TEXT("Allows to override the maximum number of UObjects that can exist in cooked game\n")
	TEXT("- 0: No override, .ini value is used (default).\n")
	TEXT("- any other value overrides the runtime max number"),
	ECVF_Default
	);


/**
 * Final phase of UObject initialization. all auto register objects are added to the main data structures.
 */
void UObjectBaseInit()
{
	SCOPED_BOOT_TIMING("UObjectBaseInit");

	// Zero initialize and later on get value from .ini so it is overridable per game/ platform...
	int32 MaxObjectsNotConsideredByGC = 0;
	int32 MaxUObjects = 2 * 1024 * 1024; // Default to ~2M UObjects
	bool bPreAllocateUObjectArray = false;	

	// To properly set MaxObjectsNotConsideredByGC look for "Log: XXX objects as part of root set at end of initial load."
	// in your log file. This is being logged from LaunchEnglineLoop after objects have been added to the root set. 

	// Disregard for GC relies on seekfree loading for interaction with linkers. We also don't want to use it in the Editor, for which
	// FPlatformProperties::RequiresCookedData() will be false. Please note that GIsEditor and FApp::IsGame() are not valid at this point.
	if (FPlatformProperties::RequiresCookedData())
	{
		if (IsRunningCookOnTheFly())
		{
			GCreateGCClusters = false;
		}
		else
		{
			GConfig->GetInt(TEXT("/Script/Engine.GarbageCollectionSettings"), TEXT("gc.MaxObjectsNotConsideredByGC"), MaxObjectsNotConsideredByGC, GEngineIni);
		}

		// Maximum number of UObjects in cooked game
		GConfig->GetInt(TEXT("/Script/Engine.GarbageCollectionSettings"), TEXT("gc.MaxObjectsInGame"), MaxUObjects, GEngineIni);
		if (GMaxObjectsInGame > 0)
		{
			MaxUObjects = GMaxObjectsInGame;
		}

		// If true, the UObjectArray will pre-allocate all entries for UObject pointers
		GConfig->GetBool(TEXT("/Script/Engine.GarbageCollectionSettings"), TEXT("gc.PreAllocateUObjectArray"), bPreAllocateUObjectArray, GEngineIni);
	}
	else
	{
#if IS_PROGRAM
		// Maximum number of UObjects for programs can be low
		MaxUObjects = 100000; // Default to 100K for programs
		GConfig->GetInt(TEXT("/Script/Engine.GarbageCollectionSettings"), TEXT("gc.MaxObjectsInProgram"), MaxUObjects, GEngineIni);
#else
		// Maximum number of UObjects in the editor
		GConfig->GetInt(TEXT("/Script/Engine.GarbageCollectionSettings"), TEXT("gc.MaxObjectsInEditor"), MaxUObjects, GEngineIni);
#endif
	}

	if (MaxObjectsNotConsideredByGC == 0)
	{
		//Disable persistent UObjects pool if there are 0 objects not considered by GC
		GUObjectAllocator.DisablePersistentAllocator();
	}

	// Log what we're doing to track down what really happens as log in LaunchEngineLoop doesn't report those settings in pristine form.
	UE_LOGF(LogInit, Log, "%ls for max %d objects, including %i objects not considered by GC.", 
		bPreAllocateUObjectArray ? TEXT("Pre-allocating") : TEXT("Presizing"), MaxUObjects, MaxObjectsNotConsideredByGC);

	GUObjectArray.AllocateObjectPool(MaxUObjects, MaxObjectsNotConsideredByGC, bPreAllocateUObjectArray);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	UE::CoreUObject::Private::InitObjectHandles(GUObjectArray.GetObjectArrayCapacity());
#endif

	void InitGarbageElimination();
	InitGarbageElimination();

	void InitAsyncThread();
	InitAsyncThread();

	// Note initialized.
	Internal::GetUObjectSubsystemInitialised() = true;

	UObjectProcessRegistrants();
}

/**
 * Final phase of UObject shutdown
 */
void UObjectBaseShutdown()
{
	void ShutdownAsyncThread();
	ShutdownAsyncThread();

	GUObjectArray.ShutdownUObjectArray();
	Internal::GetUObjectSubsystemInitialised() = false;
}

/**
 * Helper function that can be used inside the debuggers watch window. E.g. "DebugFName(Class)". 
 *
 * @param	Object	Object to look up the name for 
 * @return			Associated name
 */
const TCHAR* DebugFName(UObject* Object)
{
	if ( Object )
	{
		// Hardcoded static array. This function is only used inside the debugger so it should be fine to return it.
		static TCHAR TempName[256];
		FName Name = Object->GetFName();
		FCString::Strcpy(TempName, *FName::SafeString(Name.GetDisplayIndex(), Name.GetNumber()));
		return TempName;
	}
	else
	{
		return TEXT("NULL");
	}
}

/**
 * Helper function that can be used inside the debuggers watch window. E.g. "DebugFName(Object)". 
 *
 * @param	Object	Object to look up the name for 
 * @return			Fully qualified path name
 */
const TCHAR* DebugPathName(UObject* Object)
{
	if( Object )
	{
		// Hardcoded static array. This function is only used inside the debugger so it should be fine to return it.
		static TCHAR PathName[1024];
		PathName[0] = TCHAR('\0');

		// Keep track of how many outers we have as we need to print them in inverse order.
		UObject*	TempObject = Object;
		int32			OuterCount = 0;
		while( TempObject )
		{
			TempObject = TempObject->GetOuter();
			OuterCount++;
		}

		// Iterate over each outer + self in reverse oder and append name.
		for( int32 OuterIndex=OuterCount-1; OuterIndex>=0; OuterIndex-- )
		{
			// Move to outer name.
			TempObject = Object;
			for( int32 i=0; i<OuterIndex; i++ )
			{
				TempObject = TempObject->GetOuter();
			}

			// Dot separate entries.
			if( OuterIndex != OuterCount -1 )
			{
				FCString::Strcat( PathName, TEXT(".") );
			}
			// And app end the name.
			FCString::Strcat( PathName, DebugFName( TempObject ) );
		}

		return PathName;
	}
	else
	{
		return TEXT("None");
	}
}

/**
 * Helper function that can be used inside the debuggers watch window. E.g. "DebugFName(Object)". 
 *
 * @param	Object	Object to look up the name for 
 * @return			Fully qualified path name prepended by class name
 */
const TCHAR* DebugFullName(UObject* Object)
{
	if( Object )
	{
		// Hardcoded static array. This function is only used inside the debugger so it should be fine to return it.
		static TCHAR FullName[1024];
		FullName[0] = TCHAR('\0');

		// Class Full.Path.Name
		FCString::Strcat( FullName, DebugFName(Object->GetClass()) );
		FCString::Strcat( FullName, TEXT(" "));
		FCString::Strcat( FullName, DebugPathName(Object) );

		return FullName;
	}
	else
	{
		return TEXT("None");
	}
}
