// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectClusters.cpp: Unreal UObject Cluster helper functions
=============================================================================*/

#include "UObject/UObjectClusters.h"
#include "HAL/ThreadSafeCounter.h"
#include "AutoRTFM.h"
#include "Containers/LockFreeList.h"
#include "Stats/Stats.h"
#include "HAL/IConsoleManager.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GarbageCollectionInternalFlags.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ReferenceChainSearch.h"
#include "UObject/LinkerLoad.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/Package.h"
#include "Misc/CommandLine.h"

int32 GCreateGCClusters = 1;
static FAutoConsoleVariableRef CCreateGCClusters(
	TEXT("gc.CreateGCClusters"),
	GCreateGCClusters,
	TEXT("If true, the engine will attempt to create clusters of objects for better garbage collection performance."),
	ECVF_Default
	);

int32 GAssetClustreringEnabled = 1;
static FAutoConsoleVariableRef CVarAssetClustreringEnabled(
	TEXT("gc.AssetClustreringEnabled"),
	GAssetClustreringEnabled,
	TEXT("If true, the engine will attempt to create clusters from asset files."),
	ECVF_Default
);

// By default cluster anything that asks to be clustered with its containing package.
int32 GMinGCClusterSize = 2;
static FAutoConsoleVariableRef CMinGCClusterSize(
	TEXT("gc.MinGCClusterSize"),
	GMinGCClusterSize,
	TEXT("Minimum GC cluster size"),
	ECVF_Default
);

FUObjectClusterContainer::FUObjectClusterContainer()
	: NumAllocatedClusters(0)
	, bClustersNeedDissolving(false)
{

}

int32 FUObjectClusterContainer::AllocateCluster(int32 InRootObjectIndex)
{
	int32 ClusterIndex = INDEX_NONE;
	if (FreeClusterIndices.Num())
	{
		ClusterIndex = FreeClusterIndices.Pop(EAllowShrinking::No);
	}
	else
	{
		ClusterIndex = Clusters.Add(FUObjectCluster());
	}
	FUObjectCluster& NewCluster = Clusters[ClusterIndex];
	check(NewCluster.RootIndex == INDEX_NONE);
	NewCluster.RootIndex = InRootObjectIndex;
	NumAllocatedClusters++;
	return ClusterIndex;
}

void FUObjectClusterContainer::FreeCluster(int32 InClusterIndex)
{
	FUObjectCluster& Cluster = Clusters[InClusterIndex];
	check(Cluster.RootIndex != INDEX_NONE);
	FUObjectItem* RootItem = GUObjectArray.IndexToObject(Cluster.RootIndex);
	check(RootItem->GetClusterIndex() == InClusterIndex);
	RootItem->SetOwnerIndex(0);
	RootItem->ClearFlags(EInternalObjectFlags::ClusterRoot);

	for (int32 ReferencedClusterRootIndex : Cluster.ReferencedClusters)
	{
		if (ReferencedClusterRootIndex >= 0)
		{
			FUObjectItem* ReferencedClusterRootItem = GUObjectArray.IndexToObjectUnsafeForGC(ReferencedClusterRootIndex);
			if (ReferencedClusterRootItem->GetOwnerIndex() < 0)
			{
				FUObjectCluster& ReferencedCluster = Clusters[ReferencedClusterRootItem->GetClusterIndex()];
				ReferencedCluster.ReferencedByClusters.Remove(Cluster.RootIndex);
			}
		}
	}

	Cluster.RootIndex = INDEX_NONE;
	Cluster.Objects.Reset();
	Cluster.MutableObjects.Reset();
	Cluster.ReferencedClusters.Reset();
	Cluster.ReferencedByClusters.Reset();
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Cluster.MutableCells.Reset();
#endif
	Cluster.bNeedsDissolving = false;
	FreeClusterIndices.Add(InClusterIndex);
	NumAllocatedClusters--;
	check(NumAllocatedClusters >= 0);
}

FUObjectCluster* FUObjectClusterContainer::GetObjectCluster(UObjectBaseUtility* ClusterRootOrObjectFromCluster)
{
	check(ClusterRootOrObjectFromCluster);

	const int32 OuterIndex = GUObjectArray.ObjectToIndex(ClusterRootOrObjectFromCluster);
	FUObjectItem* OuterItem = GUObjectArray.IndexToObjectUnsafeForGC(OuterIndex);
	int32 ClusterRootIndex = 0;
	if (OuterItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot))
	{
		ClusterRootIndex = OuterIndex;
	}
	else
	{
		ClusterRootIndex = OuterItem->GetOwnerIndex();
	}
	FUObjectCluster* Cluster = nullptr;
	if (ClusterRootIndex != 0)
	{
		const int32 ClusterIndex = ClusterRootIndex > 0 ? GUObjectArray.IndexToObject(ClusterRootIndex)->GetClusterIndex() : OuterItem->GetClusterIndex();
		Cluster = &GUObjectClusters[ClusterIndex];
	}
	return Cluster;
}

void FUObjectClusterContainer::DissolveCluster(UObjectBaseUtility* ClusterRootOrObjectFromCluster)
{
	FUObjectCluster* Cluster = GetObjectCluster(ClusterRootOrObjectFromCluster);
	if (Cluster)
	{
		DissolveCluster(*Cluster);
	}
}

void FUObjectClusterContainer::DissolveCluster(FUObjectCluster& Cluster)
{
	FUObjectItem* RootObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(Cluster.RootIndex);

	// Unreachable or not, we won't need this array later
	TArray<int32> ReferencedByClusters = MoveTemp(Cluster.ReferencedByClusters);

	// Unreachable clusters will be removed by GC during BeginDestroy phase (unhashing)
	if (!UE::GC::Private::FGCFlags::IsMaybeUnreachable_ForGC(RootObjectItem))
	{
#if UE_GCCLUSTER_VERBOSE_LOGGING
		UObject* ClusterRootObject = static_cast<UObject*>(RootObjectItem->Object);
		UE_LOGF(LogObj, Log, "Dissolving cluster (%d) %ls", RootObjectItem->GetClusterIndex(), *ClusterRootObject->GetFullName());
#endif // UE_GCCLUSTER_VERBOSE_LOGGING

		const int32 OldClusterIndex = RootObjectItem->GetClusterIndex();
		for (int32 ClusterObjectIndex : Cluster.Objects)
		{
			FUObjectItem* ClusterObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(ClusterObjectIndex);
			ClusterObjectItem->SetOwnerIndex(0);
		}		

		FreeCluster(OldClusterIndex);
	}

	// Recursively dissolve all clusters this cluster is directly referenced by
	for (int32 ReferencedByClusterRootIndex : ReferencedByClusters)
	{
		FUObjectItem* ReferencedByClusterRootObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(ReferencedByClusterRootIndex);
		if (ReferencedByClusterRootObjectItem->GetOwnerIndex())
		{
			DissolveCluster(Clusters[ReferencedByClusterRootObjectItem->GetClusterIndex()]);
		}
	}
}

void FUObjectClusterContainer::DissolveClusterAndMarkObjectsAsUnreachable(FUObjectItem* RootObjectItem)
{
	const int32 OldClusterIndex = RootObjectItem->GetClusterIndex();
	FUObjectCluster& Cluster = Clusters[OldClusterIndex];

	// Unreachable or not, we won't need this array later
	TArray<int32> ReferencedByClusters = MoveTemp(Cluster.ReferencedByClusters);

#if UE_GCCLUSTER_VERBOSE_LOGGING
	UObject* ClusterRootObject = static_cast<UObject*>(RootObjectItem->Object);
	UE_LOGF(LogObj, Log, "Dissolving cluster (%d) %ls", OldClusterIndex, *ClusterRootObject->GetFullName());
#endif

	for (int32 ClusterObjectIndex : Cluster.Objects)
	{
		FUObjectItem* ClusterObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(ClusterObjectIndex);
		ClusterObjectItem->SetOwnerIndex(0);
		UE::GC::Private::FGCFlags::SetMaybeUnreachable_ForGC(ClusterObjectItem);
	}

#if !UE_GCCLUSTER_VERBOSE_LOGGING
	UObject* ClusterRootObject = static_cast<UObject*>(RootObjectItem->GetObject());
#endif
	ClusterRootObject->OnClusterMarkedAsPendingKill();

	FreeCluster(OldClusterIndex);

	// Recursively dissolve all clusters this cluster is directly referenced by
	for (int32 ReferencedByClusterRootIndex : ReferencedByClusters)
	{
		FUObjectItem* ReferencedByClusterRootItem = GUObjectArray.IndexToObjectUnsafeForGC(ReferencedByClusterRootIndex);
		if (ReferencedByClusterRootItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot))
		{
			UE::GC::Private::FGCFlags::SetMaybeUnreachable_ForGC(ReferencedByClusterRootItem);
			DissolveClusterAndMarkObjectsAsUnreachable(ReferencedByClusterRootItem);
		}
	}
}

void FUObjectClusterContainer::DissolveClusters(bool bForceDissolveAllClusters /* = false */)
{
	for (FUObjectCluster& Cluster : Clusters)
	{
		if (Cluster.RootIndex >= 0 && (Cluster.bNeedsDissolving || bForceDissolveAllClusters))
		{
			DissolveCluster(Cluster);
		}
	}
	bClustersNeedDissolving = false;
}

int32 FUObjectClusterContainer::GetMinClusterSize() const
{
	return FMath::Max(1, GMinGCClusterSize);
}

#if !UE_BUILD_SHIPPING

static bool DoesClusterContainObjects(const FUObjectCluster& Cluster, const TArray<int32>& Objects)
{
	for (int32 ObjectIndex : Objects)
	{
		if (Cluster.RootIndex == ObjectIndex)
		{
			return true;
		}
		if (Cluster.Objects.Contains(ObjectIndex))
		{
			return true;
		}
		else if (Cluster.MutableObjects.Contains(ObjectIndex))
		{
			return true;
		}
	}
	return false;
}

static void ParseObjectNameArrayForClusters(TArray<int32>& OutIndexArray, const TArray<FString>& InNameArray, bool bWarn = true)
{
	for (const FString& ObjectName : InNameArray)
	{
		UObject* Res = StaticFindFirstObject(UObject::StaticClass(), *ObjectName, EFindFirstObjectOptions::NativeFirst, ELogVerbosity::Warning, TEXT("ParseObjectNameArrayForClusters"));
		if (Res)
		{
			int32 ObjectIndex = GUObjectArray.ObjectToIndex(Res);
			OutIndexArray.Add(ObjectIndex);
		}
		else
		{
			UE_CLOGF(bWarn, LogObj, Warning, "ParseObjectNameArrayForClusters can't find object \"%ls\"", *ObjectName);
		}
	}
}

void DumpClusterToLog(const FUObjectCluster& Cluster, bool bHierarchy, bool bIndexOnly)
{
#if UE_GCCLUSTER_VERBOSE_LOGGING
	static struct FVerboseClusterLoggingSettings
	{
		TArray<FString> WithObjects;
		FVerboseClusterLoggingSettings()
		{
			FString ObjectsList;
			FParse::Value(FCommandLine::Get(), TEXT("DumpClustersWithObjects="), ObjectsList);
			ObjectsList.ParseIntoArray(WithObjects, TEXT(","));
		}
		bool DoesClusterContainRequestedObjects(const FUObjectCluster& InCluster)
		{
			bool bContainsObjects = true;
			if (WithObjects.Num())
			{
				// We need to process the object name list each time we check it against a cluster because
				// the objects may get loaded in and out as we create new clusters
				TArray<int32> ObjectIndices;
				ParseObjectNameArrayForClusters(ObjectIndices, WithObjects, false);
				// If none of the objects is currently loaded and ObjectIndices is empty we will properly reject the cluster
				bContainsObjects = DoesClusterContainObjects(InCluster, ObjectIndices);
			}
			return bContainsObjects;
		}
	} VerboseClusterLoggingSettings;

	if (!VerboseClusterLoggingSettings.DoesClusterContainRequestedObjects(Cluster))
	{
		return;
	}
#endif

	FUObjectItem* RootItem = GUObjectArray.IndexToObjectUnsafeForGC(Cluster.RootIndex);
	UObject* RootObject = static_cast<UObject*>(RootItem->GetObject());
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	FString ExtraDetail = FString::Printf(TEXT(", MutableCells: %d"), Cluster.MutableCells.Num());
#else
	FString ExtraDetail;
#endif
	UE_LOGF(LogObj, Display, "%ls (Index: %d), Size: %d, ReferencedClusters: %d, MutableObjects: %d%ls, ReferencedByClusters: %d",
		*RootObject->GetFullName(), Cluster.RootIndex, Cluster.Objects.Num(), Cluster.ReferencedClusters.Num(), Cluster.MutableObjects.Num(), *ExtraDetail, Cluster.ReferencedByClusters.Num());
	if (bHierarchy)
	{
		int32 Index = 0;
		for (int32 ObjectIndex : Cluster.Objects)
		{
			if (!bIndexOnly)
			{
				FUObjectItem* ObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(ObjectIndex);
				UObject* Object = static_cast<UObject*>(ObjectItem->GetObject());
				UE_LOGF(LogObj, Display, "    [%.4d]: %ls (Index: %d)", Index++, *Object->GetFullName(), ObjectIndex);
			}
			else
			{
				UE_LOGF(LogObj, Display, "    [%.4d]: %d", Index++, ObjectIndex);
			}
		}
		UE_LOGF(LogObj, Display, "  Referenced clusters: %d", Cluster.ReferencedClusters.Num());
		for (int32 ClusterRootIndex : Cluster.ReferencedClusters)
		{
			if (ClusterRootIndex >= 0)
			{
				if (!bIndexOnly)
				{
					FUObjectItem* ClusterRootItem = GUObjectArray.IndexToObjectUnsafeForGC(ClusterRootIndex);
					UObject* ClusterRootObject = static_cast<UObject*>(ClusterRootItem->GetObject());
					UE_LOGF(LogObj, Display, "    -> %ls (Index: %d)", *ClusterRootObject->GetFullName(), ClusterRootIndex);
				}
				else
				{
					UE_LOGF(LogObj, Display, "    -> %d", ClusterRootIndex);
				}
			}
			else
			{
				UE_LOGF(LogObj, Display, "    -> nullptr");
			}
		}
		UE_LOGF(LogObj, Display, "  External (mutable) objects: %d", Cluster.MutableObjects.Num());
		for (int32 ObjectIndex : Cluster.MutableObjects)
		{
			if (ObjectIndex >= 0)
			{
				if (!bIndexOnly)
				{
					FUObjectItem* ObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(ObjectIndex);
					UObject* Object = static_cast<UObject*>(ObjectItem->GetObject());
					UE_LOGF(LogObj, Display, "    => %ls (Index: %d)", *Object->GetFullName(), ObjectIndex);
				}
				else
				{
					UE_LOGF(LogObj, Display, "    => %d", ObjectIndex);
				}
			}
			else
			{
				UE_LOGF(LogObj, Display, "    => nullptr");
			}
		}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		UE_LOGF(LogObj, Display, "  External (mutable) cells: %d", Cluster.MutableCells.Num());
		for (Verse::VCell* Cell : Cluster.MutableCells)
		{
			UE_LOGF(LogObj, Display, "    => %p", Cell);
		}
#endif
	}
}

// Dumps all clusters to log.
void ListClusters(const TArray<FString>& Args)
{
	const bool bHierarchy = Args.Contains(TEXT("Hierarchy"));
	int32 MaxInterClusterReferences = 0;
	int32 TotalInterClusterReferences = 0;
	int32 MaxClusterSize = 0;
	int32 TotalClusterObjects = 0;	

	TArray<FUObjectCluster*> AllClusters;
	for (FUObjectCluster& Cluster : GUObjectClusters.GetClustersUnsafe())
	{
		if (Cluster.RootIndex != INDEX_NONE)
		{
			AllClusters.Add(&Cluster);
		}
	}

	TArray<int32> WithObjects;
	for (const FString& Arg : Args)
	{
		if (Arg == TEXT("SortByName"))
		{
			Algo::SortBy(AllClusters, [](FUObjectCluster* A)
			{
				return GUObjectArray.IndexToObject(A->RootIndex)->GetObject()->GetFName();
			}, FNameLexicalLess());
		}
		else if (Arg == TEXT("SortByObjectCount"))
		{
			Algo::SortBy(AllClusters, [](FUObjectCluster* A)
			{
				return A->Objects.Num();
			});
		}
		else if (Arg == TEXT("SortByMutableObjectCount"))
		{
			Algo::SortBy(AllClusters, [](FUObjectCluster* A)
			{
				return A->MutableObjects.Num();
			});
		}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		else if (Arg == TEXT("SortByMutableCellCount"))
		{
			Algo::SortBy(AllClusters, [](FUObjectCluster* A)
				{
					return A->MutableCells.Num();
				});
		}
#endif
		else if (Arg == TEXT("SortByReferencedClustersCount"))
		{
			Algo::SortBy(AllClusters, [](FUObjectCluster* A)
			{
				return A->ReferencedClusters.Num();
			});
		}
		else if (Arg.StartsWith("With="))
		{
			FString ObjectsList = Arg.Mid(5);
			TArray<FString> ObjectNames;
			ObjectsList.ParseIntoArray(ObjectNames, TEXT(","));
			ParseObjectNameArrayForClusters(WithObjects, ObjectNames);
		}
	}

	int32 NumberOfClustersPrinted = 0;

	for (FUObjectCluster* Cluster : AllClusters)
	{
		check(Cluster->RootIndex != INDEX_NONE);

		MaxInterClusterReferences = FMath::Max(MaxInterClusterReferences, Cluster->ReferencedClusters.Num());
		TotalInterClusterReferences += Cluster->ReferencedClusters.Num();
		MaxClusterSize = FMath::Max(MaxClusterSize, Cluster->Objects.Num());
		TotalClusterObjects += Cluster->Objects.Num();

		bool bListCluster = true;
		if (WithObjects.Num())
		{
			bListCluster = DoesClusterContainObjects(*Cluster, WithObjects);
		}
		if (bListCluster)
		{
			DumpClusterToLog(*Cluster, bHierarchy, false);
			NumberOfClustersPrinted++;
		}
	}
	UE_LOGF(LogObj, Display, "Displayed %d clusters", NumberOfClustersPrinted);
	UE_LOGF(LogObj, Display, "Total number of clusters: %d", AllClusters.Num());
	UE_LOGF(LogObj, Display, "Maximum cluster size: %d", MaxClusterSize);
	UE_LOGF(LogObj, Display, "Average cluster size: %d", AllClusters.Num() ? (TotalClusterObjects / AllClusters.Num()) : 0);
	UE_LOGF(LogObj, Display, "Number of objects in GC clusters: %d", TotalClusterObjects);
	UE_LOGF(LogObj, Display, "Max number of gc objects: %d", GUObjectArray.GetObjectArrayNumPermanent());
	UE_LOGF(LogObj, Display, "Maximum number of custer-to-cluster references: %d", MaxInterClusterReferences);
	UE_LOGF(LogObj, Display, "Average number of custer-to-cluster references: %d", AllClusters.Num() ? (TotalInterClusterReferences / AllClusters.Num()) : 0);
}

void FindStaleClusters(const TArray<FString>& Args)
{
	// This is seriously slow.
	UE_LOGF(LogObj, Display, "Searching for stale clusters. This may take a while..."); 
	int32 NumStaleClusters = 0;
	int32 TotalNumClusters = 0;
	for (FRawObjectIterator It(true); It; ++It)
	{
		FUObjectItem* ObjectItem = *It;
		if (ObjectItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot))
		{
			TotalNumClusters++;

			UObject* ClusterRootObject = static_cast<UObject*>(ObjectItem->GetObject());
			FReferenceChainSearch SearchRefs(ClusterRootObject, EReferenceChainSearchMode::ExternalOnly);
			
			if (SearchRefs.GetReferenceChains().Num() == 0)
			{
				NumStaleClusters++;
				UE_LOGF(LogObj, Display, "Cluster %ls has no external references:", *ClusterRootObject->GetFullName());
				SearchRefs.PrintResults();
			}
		}
	}
	UE_LOGF(LogObj, Display, "Found %d clusters, including %d stale.", TotalNumClusters, NumStaleClusters);
}


void DumpRefsToCluster(FUObjectCluster* Cluster)
{
	FUObjectItem* RootItem = GUObjectArray.IndexToObject(Cluster->RootIndex);

	UE_LOGF(LogObj, Display, "Dumping references to objects in cluster %ls", *static_cast<UObject*>(RootItem->GetObject())->GetFullName());

	bool bIsReferenced = false;
	for (int32 ObjectIndex : Cluster->Objects)
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(ObjectIndex);
		FReferenceChainSearch SearchRefs(static_cast<UObject*>(ObjectItem->GetObject()), EReferenceChainSearchMode::ExternalOnly | EReferenceChainSearchMode::Shortest);
		if (SearchRefs.GetReferenceChains().Num())
		{
			bIsReferenced = true;
			SearchRefs.PrintResults(true);
		}
	}
	if (!bIsReferenced)
	{
		UE_LOGF(LogObj, Display, "Cluster %ls is not currently referenced by anything.", *static_cast<UObject*>(RootItem->GetObject())->GetFullName());
	}
}

void DumpRefsToCluster(const TArray<FString>& Args)
{
	// This is seriously slow.
	UE_LOGF(LogObj, Display, "Searching for references to clusteres. This may take a while...");

	TArray<int32> RootObjects;
	for (const FString& Arg : Args)
	{
		if (Arg.StartsWith("Root="))
		{
			FString ObjectsList = Arg.Mid(5);
			TArray<FString> ObjectNames;
			ObjectsList.ParseIntoArray(ObjectNames, TEXT(","));
			ParseObjectNameArrayForClusters(RootObjects, ObjectNames);
		}
	}

	for (int32 RootIndex : RootObjects)
	{
		FUObjectItem* RootItem = GUObjectArray.IndexToObject(RootIndex);
		if (RootItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot))
		{
			FUObjectCluster* Cluster = GUObjectClusters.GetObjectCluster(static_cast<UObject*>(RootItem->GetObject()));
			DumpRefsToCluster(Cluster);
		}
	}
}

static void SuggestClusters(const TArray<FString>& Args);


static FAutoConsoleCommand ListClustersCommand(
	TEXT("gc.ListClusters"),
	TEXT("Dumps all clusters do output log. When 'Hiearchy' argument is specified lists all objects inside clusters."),
	FConsoleCommandWithArgsDelegate::CreateStatic(ListClusters)
	);

static FAutoConsoleCommand FindStaleClustersCommand(
	TEXT("gc.FindStaleClusters"),
	TEXT("Dumps all clusters do output log that are not referenced by anything."),
	FConsoleCommandWithArgsDelegate::CreateStatic(FindStaleClusters)
	);

static FAutoConsoleCommand DumpRefsToClusterCommand(
	TEXT("gc.DumpRefsToCluster"),
	TEXT("Dumps references to all objects within a cluster. Specify the cluster name with Root=Name."),
	FConsoleCommandWithArgsDelegate::CreateStatic(DumpRefsToCluster)
);

static FAutoConsoleCommand SuggestClustersCommand(
	TEXT("gc.SuggestClusters"),
	TEXT("Searches for assets which contain many internal objects which are not clustered."),
	FConsoleCommandWithArgsDelegate::CreateStatic(SuggestClusters)
);

#endif // !UE_BUILD_SHIPPING

/**
 * Handles UObject references found by TFastReferenceCollector
 */
class FClusterReferenceProcessor : public FSimpleReferenceProcessorBase
{
	int32 ClusterRootIndex;
	FUObjectCluster& Cluster;
	UObject* Outermost;

public:

	FClusterReferenceProcessor(int32 InClusterRootIndex, FUObjectCluster& InCluster, UObject* InOutermost)
		: ClusterRootIndex(InClusterRootIndex)
		, Cluster(InCluster)
		, Outermost(InOutermost)
	{}

	static FString LoadFlagsToString(UObject* Obj)
	{
		FString Flags;
		if (Obj)
		{
			if (Obj->HasAnyFlags(RF_NeedLoad))
			{
				Flags += TEXT("RF_NeedLoad");
			}
			if (Obj->HasAnyFlags(RF_NeedPostLoad))
			{
				if (Flags.Len())
				{
					Flags += TEXT("|");
				}
				Flags += TEXT("RF_NeedPostLoad");
			}
		}
		else
		{
			Flags += TEXT("null");
		}
		return Flags;
	}

	UObject* GetClusterRoot()
	{
		return static_cast<UObject*>(GUObjectArray.IndexToObject(Cluster.RootIndex)->GetObject());
	}

	/**
	 * Adds an object to cluster (if possible)
	 *
	 * @param ObjectIndex UObject index in GUObjectArray
	 * @param ObjectItem UObject's entry in GUObjectArray
	 * @param Obj The object to add to cluster
	* @param Context Context of the reference collection
	 * @param bOuterAndClass If true, the Obj's Outer and Class will also be added to the cluster
	 */
	void AddObjectToCluster(int32 ObjectIndex, FUObjectItem* ObjectItem, UObject* Obj, FWorkerContext& Context, bool bOuterAndClass)
	{
		// If we haven't finished loading, we can't be sure we know all the references
		checkf(!Obj->HasAnyFlags(RF_NeedLoad), TEXT("%s hasn't been loaded (%s) but is being added to cluster %s"), 
			*Obj->GetFullName(), 
			*LoadFlagsToString(Obj),
			*GetClusterRoot()->GetFullName());

		check(ObjectItem->GetOwnerIndex() == 0 || ObjectItem->GetOwnerIndex() == ClusterRootIndex || ObjectIndex == ClusterRootIndex || GUObjectArray.IsDisregardForGC(Obj));
		check(Obj->CanBeInCluster());
		if (ObjectIndex != ClusterRootIndex && ObjectItem->GetOwnerIndex() == 0 && !GUObjectArray.IsDisregardForGC(Obj) && !Obj->IsRooted())
		{
			Context.ObjectsToSerialize.Add<Options>(Obj);
			check(!ObjectItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot));
			ObjectItem->SetOwnerIndex(ClusterRootIndex);
			Cluster.Objects.Add(ObjectIndex);

			if (bOuterAndClass)
			{
				UObject* ObjOuter = Obj->GetOuter();
				if (ObjOuter)
				{
					HandleTokenStreamObjectReference(Context, Obj, ObjOuter, UE::GC::EMemberlessId::Outer, EOrigin::Other, true);
				}
				if (!Obj->GetClass()->HasAllClassFlags(CLASS_Native))
				{
					UObject* ObjectClass = Obj->GetClass();
					HandleTokenStreamObjectReference(Context, Obj, ObjectClass, UE::GC::EMemberlessId::Class, EOrigin::Other, true);
					UObject* ObjectClassOuter = Obj->GetClass()->GetOuter();
					HandleTokenStreamObjectReference(Context, Obj, ObjectClassOuter, UE::GC::EMemberlessId::ClassOuter, EOrigin::Other, true);
				}
			}
		}
	}

	/**
	* Handles UObject reference from the token stream. Performance is critical here so we're FORCEINLINING this function.
	*
	* @param Context Context of the reference collection
	* @param ReferencingObject Object referencing the object to process.
	* @param Object Object being processed
	* @param MemberId Index to the token stream where the reference was found.
	* @param Origin Declares if a schema represents a blueprint generated type
	* @param bAllowReferenceElimination True if reference elimination is allowed (ignored when constructing clusters).
	*/
	FORCEINLINE void HandleTokenStreamObjectReference(FWorkerContext& Context, UObject* ReferencingObject, UObject*& Object, FMemberId MemberId, EOrigin Origin, bool bAllowReferenceElimination)
	{
		if (!Object)
		{
			return;
		}

		FUObjectItem* ObjectItem = GUObjectArray.ObjectToObjectItem(Object);
		// Add encountered object reference to list of to be serialized objects if it hasn't already been added.
		if (ObjectItem->GetOwnerIndex() == ClusterRootIndex)
		{
			return;
		}

		// If we haven't finished loading, we can't be sure we know all the references so the object will be added as mutable reference
		UE_CLOGF(Object->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad), LogObj, Log, "%ls hasn't been loaded (%ls) but is being added to cluster %ls",
			*Object->GetFullName(),
			*LoadFlagsToString(Object),
			*GetClusterRoot()->GetFullName());

		if (ObjectItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot) || ObjectItem->GetOwnerIndex() != 0)
		{					
			// Simply reference this cluster and all clusters it's referencing
			const int32 OtherClusterRootIndex = ObjectItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot) ? GUObjectArray.ObjectToIndex(Object) : ObjectItem->GetOwnerIndex();
			FUObjectItem* OtherClusterRootItem = GUObjectArray.IndexToObject(OtherClusterRootIndex);
			const int32 OtherClusterIndex = OtherClusterRootItem->GetClusterIndex();
			FUObjectCluster& OtherCluster = GUObjectClusters[OtherClusterIndex];
			Cluster.ReferencedClusters.AddUnique(OtherClusterRootIndex);
			OtherCluster.ReferencedByClusters.AddUnique(ClusterRootIndex);

			for (int32 OtherClusterReferencedCluster : OtherCluster.ReferencedClusters)
			{
				if (OtherClusterReferencedCluster != ClusterRootIndex)
				{
					Cluster.ReferencedClusters.AddUnique(OtherClusterReferencedCluster);
				}
			}
			for (int32 OtherClusterReferencedMutableObjectIndex : OtherCluster.MutableObjects)
			{
				Cluster.MutableObjects.AddUnique(OtherClusterReferencedMutableObjectIndex);
			}
		}
		else if (GUObjectArray.IsDisregardForGC(Object)) 
		{
			// We know that disregard for GC objects will never be GC'd so no reference is necessary
		}
		else if (!Object->CanBeInCluster() || Object->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad) || Object->IsRooted())
		{
			// This object may change state in ways which would invalidate the cluster, so keep it in a list of objects to fully explore when this cluster is encountered during GC		
			Cluster.MutableObjects.AddUnique(GUObjectArray.ObjectToIndex(Object));
		}
		else if (!Object->IsIn(Outermost) && Object != Outermost)
		{
			// This object is in a different package so we don't want it to be part of this cluster 
			// TODO: Consider exploring this object to find if it is part of a package containing an object which may add it to a cluster, so we can add a cluster ref instead of a mutable object ref 
			Cluster.MutableObjects.AddUnique(GUObjectArray.ObjectToIndex(Object));
		}
		else 
		{
			check(ObjectItem->GetOwnerIndex() == 0);

			// New object in the same package, add it to the cluster.
			AddObjectToCluster(GUObjectArray.ObjectToIndex(Object), ObjectItem, Object, Context, true);
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/**
	* Handles VCell reference from the token stream. Performance is critical here so we're FORCEINLINING this function.
	*
	* @param Context Context of the reference collection
	* @param ReferencingObject Object referencing the object to process.
	* @param Cell Cell being processed
	* @param MemberId Index to the token stream where the reference was found.
	* @param Origin Declares if a schema represents a blueprint generated type
	*/
	FORCEINLINE void HandleTokenStreamVerseCellReference(FWorkerContext& Context, UObject* ReferencingObject, Verse::VCell* Cell, FMemberId MemberId, EOrigin Origin)
	{
		// As with mutable objects, we assume that the cell is mutable but the pointer to the cell is not.  If the cell pointer needs to be modified,
		// then the underlying object must be mutable.
		if (Cell)
		{
			Cluster.MutableCells.AddUnique(Cell);
		}
	}
#endif
};

static void SuggestClusters(const TArray<FString>& Args)
{
	struct FStats 
	{
		int32 NumAssets = 0;
		int32 NumObjects = 0;
		int32 NumClusterRefs = 0;
		int32 NumObjectRefs = 0;
	};
	TMap<UClass*, FStats> SuggestedClasses;
	for (TObjectIterator<UPackage> It; It; ++It)
	{
		bool FoundCluster = false;

		// If this package contains a cluster, the package should have been added to it
		if (GUObjectClusters.GetObjectCluster(*It) != nullptr)
		{
			continue;
		}

		ForEachObjectWithOuter(*It, [&](UObject* Obj)
		{
			if (Obj->HasAllFlags(RF_Standalone))
			{
				FUObjectItem* RootItem = GUObjectArray.ObjectToObjectItem(Obj);
				const int32 InternalIndex = GUObjectArray.ObjectToIndex(Obj);
				const int32 ClusterIndex = GUObjectClusters.AllocateCluster(InternalIndex);
				FUObjectCluster& Cluster = GUObjectClusters[ClusterIndex];
				Cluster.Objects.Reserve(64);

				FClusterReferenceProcessor Processor(InternalIndex, Cluster, *It);
				FGCArrayStruct ArrayStruct;
				TArray<UObject*> ObjectsToProcess = {Obj};
				ArrayStruct.SetInitialObjectsUnpadded(ObjectsToProcess);
				CollectReferences(Processor, ArrayStruct);
				RootItem->SetClusterIndex(ClusterIndex);
				RootItem->SetFlags(EInternalObjectFlags::ClusterRoot);

				if (Cluster.Objects.Num() >= GUObjectClusters.GetMinClusterSize())
				{
					UE_LOGF(LogObj, Display, "Potential cluster root %ls with no cluster, %d internal objects, %d cluster refs, %d object refs.", 
						*Obj->GetPathName(),
						Cluster.Objects.Num(),
						Cluster.ReferencedClusters.Num(),
						Cluster.MutableObjects.Num()
					);
					FStats& Stats = SuggestedClasses.FindOrAdd(Obj->GetClass());
					Stats.NumAssets++;
					Stats.NumObjects += Cluster.Objects.Num();
					Stats.NumClusterRefs += Cluster.ReferencedClusters.Num();
					Stats.NumObjectRefs += Cluster.MutableObjects.Num();
				}
				for (int32 ClusterObjectIndex : Cluster.Objects)
				{
					FUObjectItem* ClusterObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(ClusterObjectIndex);
					ClusterObjectItem->SetOwnerIndex(0);
				}
				GUObjectClusters.FreeCluster(ClusterIndex);
			}
		});
	}
	SuggestedClasses.ValueSort([](const FStats& A, const FStats& B) { return A.NumObjects > B.NumObjects; });
	for (const TPair<UClass*, FStats>& Pair : SuggestedClasses)
	{
		UClass* Class = Pair.Key;
		FStats Stats = Pair.Value;
		UE_LOGF(LogObj, Display, "Suggested cluster class: %ls - %d assets - %d objects - %d cluster refs - %d object refs", 
			*Class->GetPathName(), Stats.NumAssets, Stats.NumObjects, Stats.NumClusterRefs, Stats.NumObjectRefs);
	}
}

bool CanCreateObjectClusters()
{
	return FPlatformProperties::RequiresCookedData() && !GIsInitialLoad && GCreateGCClusters && GAssetClustreringEnabled && !GUObjectArray.IsOpenForDisregardForGC();
}

/** Looks through objects loaded with a package and creates clusters from them */
void CreateClustersFromPackage(FLinkerLoad* PackageLinker, TArray<UObject*>& OutClusterObjects)
{	
	if (CanCreateObjectClusters())
	{
		check(PackageLinker);

		for (FObjectExport& Export : PackageLinker->ExportMap)
		{
			if (Export.Object && Export.Object->CanBeClusterRoot())
			{
				OutClusterObjects.Add(Export.Object);
			}
		}
	}
}

UE_AUTORTFM_ALWAYS_OPEN
void UObjectBaseUtility::AddToCluster(UObjectBaseUtility* ClusterRootOrObjectFromCluster, bool bAddAsMutableObject /* = false */)
{
	FUObjectCluster* Cluster = GUObjectClusters.GetObjectCluster(ClusterRootOrObjectFromCluster);
	if (Cluster)
	{
		const int32 ClusterRootIndex = Cluster->RootIndex;
		if (!bAddAsMutableObject)
		{
			UObject* ClusterRootObject = static_cast<UObject*>(GUObjectArray.IndexToObjectUnsafeForGC(Cluster->RootIndex)->GetObject());
			FClusterReferenceProcessor Processor(ClusterRootIndex, *Cluster, ClusterRootObject->GetOutermost());
			UObject* ThisObject = static_cast<UObject*>(this);
			UObject** ThisObjPtr = &ThisObject;
			FGCArrayStruct ArrayStruct;
			ArrayStruct.InitialNativeReferences = TConstArrayView<UObject**>( &ThisObjPtr, 1 );
			CollectReferences(Processor, ArrayStruct);

#if UE_GCCLUSTER_VERBOSE_LOGGING
			UE_LOGF(LogObj, Log, "Added %ls to cluster %ls:", *ThisObject->GetFullName(), *ClusterRootObject->GetFullName());
			DumpClusterToLog(*Cluster, true, false);
#endif
		}
		else
		{
			// Adds this object's index to the MutableObjects array keeping it sorted and unique
			const int32 ThisObjectIndex = GUObjectArray.ObjectToIndex(this);
			int32 InsertedAt = INDEX_NONE;
			for (int32 MutableObjectIndex = 0; MutableObjectIndex < Cluster->MutableObjects.Num() && InsertedAt == INDEX_NONE; ++MutableObjectIndex)
			{
				if (Cluster->MutableObjects[MutableObjectIndex] > ThisObjectIndex)
				{
					InsertedAt = Cluster->MutableObjects.Insert(ThisObjectIndex, MutableObjectIndex);
				}
				else if (Cluster->MutableObjects[MutableObjectIndex] == ThisObjectIndex)
				{
					InsertedAt = MutableObjectIndex;
				}
			}
			if (InsertedAt == INDEX_NONE)
			{
				Cluster->MutableObjects.Add(ThisObjectIndex);
			}
		}
	}
}

bool UObjectBaseUtility::CanBeInCluster() const
{
	return OuterPrivate ? OuterPrivate->CanBeInCluster() : true;
}

void UObjectBaseUtility::CreateCluster()
{
	check(GCreateGCClusters);

	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("UObjectBaseUtility::CreateCluster"), STAT_FArchiveRealtimeGC_CreateCluster, STATGROUP_GC);

	FUObjectItem* RootItem = GUObjectArray.IndexToObject(InternalIndex);
	if (RootItem->GetOwnerIndex() != 0 || RootItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot))
	{
		return;
	}

	// If we haven't finished loading, we can't be sure we know all the references
	check(!HasAnyFlags(RF_NeedLoad));

	// Create a new cluster, reserve an arbitrary amount of memory for it.
	const int32 ClusterIndex = GUObjectClusters.AllocateCluster(InternalIndex);
	FUObjectCluster& Cluster = GUObjectClusters[ClusterIndex];
	Cluster.Objects.Reserve(64);

	// Collect all objects referenced by cluster root and by all objects it's referencing
	FClusterReferenceProcessor Processor(InternalIndex, Cluster, GetOutermost());
	FGCArrayStruct ArrayStruct;
	TArray<UObject*> ObjectsToProcess = {static_cast<UObject*>(this)};
	ArrayStruct.SetInitialObjectsUnpadded(ObjectsToProcess);
	CollectReferences(Processor, ArrayStruct);

	check(RootItem->GetOwnerIndex() == 0);
	RootItem->SetClusterIndex(ClusterIndex);
	RootItem->SetFlags(EInternalObjectFlags::ClusterRoot);

	if (Cluster.Objects.Num() >= GUObjectClusters.GetMinClusterSize())
	{
		// Add new cluster to the global cluster map.
		Cluster.Objects.Sort();
		Cluster.ReferencedClusters.Sort();		
		Cluster.MutableObjects.Sort();

#if UE_GCCLUSTER_VERBOSE_LOGGING
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		FString ExtraDetail = FString::Printf(TEXT(", %d Verse cells"), Cluster.MutableCells.Num());
#else
		FString ExtraDetail;
#endif
		UE_LOGF(LogObj, Log, "Created cluster (%d) with %d objects, %d referenced clusters%ls and %d mutable objects.",
			ClusterIndex, Cluster.Objects.Num(), Cluster.ReferencedClusters.Num(), *ExtraDetail, Cluster.MutableObjects.Num());

		DumpClusterToLog(Cluster, true, false);
#endif
	}
	else
	{
#if UE_GCCLUSTER_VERBOSE_LOGGING
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		FString ExtraDetail = FString::Printf(TEXT(", %d Verse cells"), Cluster.MutableCells.Num());
#else
		FString ExtraDetail;
#endif
		UE_LOGF(LogObj, Log, "Not creating cluster (%d) with %d objects, %d referenced clusters%ls and %d mutable objects.",
			ClusterIndex, Cluster.Objects.Num(), Cluster.ReferencedClusters.Num(), *ExtraDetail, Cluster.MutableObjects.Num());

		DumpClusterToLog(Cluster, true, false);
#endif
		for (int32 ClusterObjectIndex : Cluster.Objects)
		{
			FUObjectItem* ClusterObjectItem = GUObjectArray.IndexToObjectUnsafeForGC(ClusterObjectIndex);
			ClusterObjectItem->SetOwnerIndex(0);
		}
		GUObjectClusters.FreeCluster(ClusterIndex);
		check(RootItem->GetOwnerIndex() == 0);
		check(!RootItem->HasAnyFlags(EInternalObjectFlags::ClusterRoot));
	}
}

