// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(UE_WITH_CONSTINIT_UOBJECT) || !UE_WITH_CONSTINIT_UOBJECT

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "UObject/Class.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "Modules/ModuleManager.h"

namespace UECodeGen_Private
{

// Helper methods defined in UObjectGlobals.cpp
void ConstructFProperties(UObject* Outer, const FPropertyParamsBase* const* PropertyArray, int32 NumProperties);
#if WITH_METADATA
void AddMetaData(UObject* Object, const FMetaDataPairParam* MetaDataArray, int32 NumMetaData);
#endif

/**
 * Construct but do not initialize a UClass
 *
 * @param PackageName name of the package this class will be inside
 * @param Name of the class
 * @param ReturnClass reference to pointer to result. This must be PrivateStaticClass.
 * @param RegisterNativeFunc Native function registration function pointer.
 * @param InSize Size of the class
 * @param InAlignment Alignment of the class
 * @param InClassFlags Class flags
 * @param InClassCastFlags Class cast flags
 * @param InConfigName Class config name
 * @param InClassConstructor Class constructor function pointer
 * @param InClassVTableHelperCtorCaller Class constructor function for vtable pointer
 * @param InCppClassStaticFunctions Function pointers for the class's version of Unreal's reflected static functions
 * @param InSuperClassFn Super class function pointer
 * @param InWithinClassFn Within class function pointer
 * *param InPostNewFn Invoked after the a new instance is created
 */
template<typename UClassClass, typename PostNewFn>
void ConstructUClassNoInitHelper(
	const TCHAR* PackageName,
	const TCHAR* Name,
	UClass*& ReturnClass,
	void(*RegisterNativeFunc)(),
	uint32 InSize,
	uint32 InAlignment,
	EClassFlags InClassFlags,
	EClassCastFlags InClassCastFlags,
	const TCHAR* InConfigName,
	UClass::ClassConstructorType InClassConstructor,
	UClass::ClassVTableHelperCtorCallerType InClassVTableHelperCtorCaller,
	FUObjectCppClassStaticFunctions&& InCppClassStaticFunctions,
	UClass::StaticClassFunctionType InSuperClassFn,
	UClass::StaticClassFunctionType InWithinClassFn,
	PostNewFn&& InPostNewFn
)
{
#if WITH_RELOAD
	if (IsReloadActive() && GetActiveReloadType() != EActiveReloadType::Reinstancing)
	{
		UPackage* Package = FindPackage(NULL, PackageName);
		if (Package)
		{
			ReturnClass = FindObject<UClassClass>((UObject*)Package, Name);
			if (ReturnClass)
			{
				if (ReturnClass->HotReloadPrivateStaticClass(
					InSize,
					InClassFlags,
					InClassCastFlags,
					InConfigName,
					InClassConstructor,
					InClassVTableHelperCtorCaller,
					FUObjectCppClassStaticFunctions(InCppClassStaticFunctions),
					InSuperClassFn(),
					InWithinClassFn()
				))
				{
					// Register the class's native functions.
					if (RegisterNativeFunc)
					{
						RegisterNativeFunc();
					}
				}
				return;
			}
			else
			{
				UE_LOGF(LogClass, Log, "Could not find existing class %ls in package %ls for reload, assuming new or modified class", Name, PackageName);
			}
		}
		else
		{
			UE_LOGF(LogClass, Log, "Could not find existing package %ls for reload of class %ls, assuming a new package.", PackageName, Name);
		}
	}
#endif

	UClassClass* NewClass = (UClassClass*)GUObjectAllocator.AllocateUObject(sizeof(UClassClass), alignof(UClassClass), true);
	NewClass = ::new (NewClass)
		UClassClass
		(
			EC_StaticConstructor,
			Name,
			InSize,
			InAlignment,
			InClassFlags,
			InClassCastFlags,
			InConfigName,
			EObjectFlags(RF_Public | RF_Standalone | RF_Transient | RF_MarkAsNative | RF_MarkAsRootSet),
			InClassConstructor,
			InClassVTableHelperCtorCaller,
			MoveTemp(InCppClassStaticFunctions)
		);
	check(NewClass);
	ReturnClass = NewClass; // This must be done here or recursive calls cause problems
	InPostNewFn(NewClass);

	InitializePrivateStaticClass(
		UClassClass::StaticClass,
		InSuperClassFn(),
		NewClass,
		InWithinClassFn(),
		PackageName,
		Name
	);

	// Register the class's native functions.
	if (RegisterNativeFunc)
	{
		RegisterNativeFunc();
	}
}

template<typename UEnumClass, typename EnumParams, typename PostNewFn>
// It is UB to call `OuterFunc` below because it can return a `UClass*` not `UObject*` (even though
// there is a parent-child relation there). So we suppress UBSan for this function.
UE_DISABLE_UBSAN
void ConstructUEnumHelper(UEnum*& OutEnum, const EnumParams& Params, PostNewFn&& InPostNewFn)
{
	FTypeConstructFunc* OuterFunc = Params.OuterFunc;

	UObject* Outer = OuterFunc ? OuterFunc(ETypeConstructPhase::Outer) : nullptr;

	if (OutEnum)
	{
		return;
	}

	UEnumClass* NewEnum = new (EC_InternalUseOnlyConstructor, Outer, UTF8_TO_TCHAR(Params.NameUTF8), Params.ObjectFlags) UEnumClass(FObjectInitializer());
	OutEnum = NewEnum;
	InPostNewFn(NewEnum, Params);

	TArray<TPair<FName, int64>> EnumNames;
	EnumNames.Reserve(Params.NumEnumerators);
	for (const FEnumeratorParam* Enumerator = Params.EnumeratorParams, *EnumeratorEnd = Enumerator + Params.NumEnumerators; Enumerator != EnumeratorEnd; ++Enumerator)
	{
		EnumNames.Emplace(UTF8_TO_TCHAR(Enumerator->NameUTF8), Enumerator->Value);
	}

	const UEnum::EAddMaxKeyIfMissing AddMaxKeyIfMissing = UEnum::EAddMaxKeyIfMissing::Yes;
	NewEnum->SetEnums(EnumNames, (UEnum::ECppForm)Params.CppForm, (UEnum::EUnderlyingType)Params.UnderlyingType, Params.EnumFlags, AddMaxKeyIfMissing);
	NewEnum->CppType = UTF8_TO_TCHAR(Params.CppTypeUTF8);

	if (Params.DisplayNameFunc)
	{
		NewEnum->SetEnumDisplayNameFn(Params.DisplayNameFunc);
	}

#if WITH_METADATA
	AddMetaData(NewEnum, Params.MetaDataArray, Params.NumMetaData);
#endif
}

template<typename UScriptStructClass, typename StructParams, typename PostNewFn>
// It is UB to call `OuterFunc` below because it can return a `UClass*` not `UObject*` (even though
// there is a parent-child relation there). So we suppress UBSan for this function.
UE_DISABLE_UBSAN
void ConstructUScriptStructHelper(UScriptStruct*& OutStruct, const StructParams& Params, PostNewFn&& InPostNewFn)
{
	FTypeConstructFunc* OuterFunc = Params.OuterFunc;
	UScriptStruct* (*SuperFunc)(ETypeConstructPhase) = Params.SuperFunc;
	UScriptStruct::ICppStructOps* (*StructOpsFunc)() = (UScriptStruct::ICppStructOps * (*)())Params.StructOpsFunc;

	UObject* Outer = OuterFunc ? OuterFunc(ETypeConstructPhase::Outer) : nullptr;
	UScriptStruct* Super = SuperFunc ? SuperFunc(ETypeConstructPhase::Outer) : nullptr;
	UScriptStruct::ICppStructOps* StructOps = StructOpsFunc ? StructOpsFunc() : nullptr;

	if (OutStruct)
	{
		return;
	}

	UScriptStructClass* NewStruct = new(EC_InternalUseOnlyConstructor, Outer, UTF8_TO_TCHAR(Params.NameUTF8), Params.ObjectFlags) UScriptStructClass(FObjectInitializer(), Super, StructOps, (EStructFlags)Params.StructFlags, Params.SizeOf, Params.AlignOf);
	OutStruct = NewStruct;
	InPostNewFn(NewStruct, Params);

	ConstructFProperties(NewStruct, Params.PropertyArray, Params.NumProperties);
	NewStruct->StaticLink();
#if WITH_METADATA
	AddMetaData(NewStruct, Params.MetaDataArray, Params.NumMetaData);
#endif
}

template<typename UClassClass, typename ClassParams, typename PostNewFn>
// It is UB to call `SingletonFunc` below because it can return a `UClass*` not `UObject*` (even though
// there is a parent-child relation there). So we suppress UBSan for this function.
UE_DISABLE_UBSAN
void ConstructUClassHelper(UClass*& OutClass, const ClassParams& Params, PostNewFn&& InPostNewFn)
{
	if (OutClass && (OutClass->ClassFlags & CLASS_Constructed))
	{
		return;
	}

	for (FTypeConstructFunc* const* SingletonFunc = Params.DependencySingletonFuncArray, *const* SingletonFuncEnd = SingletonFunc + Params.NumDependencySingletons; SingletonFunc != SingletonFuncEnd; ++SingletonFunc)
	{
		(*SingletonFunc)(ETypeConstructPhase::Outer);
	}

	UClass* NewClass = Params.ClassNoRegisterFunc(ETypeConstructPhase::Inner);
	OutClass = NewClass;

	if (NewClass->ClassFlags & CLASS_Constructed)
	{
		return;
	}

	InPostNewFn(Cast<UClassClass>(NewClass), Params);

	UObjectForceRegistration(NewClass);

	UClass* SuperClass = NewClass->GetSuperClass();
	if (SuperClass)
	{
		NewClass->ClassFlags |= (SuperClass->ClassFlags & CLASS_Inherit);
	}

	NewClass->ClassFlags |= (EClassFlags)(Params.ClassFlags | CLASS_Constructed);
	// Make sure the reference token stream is empty since it will be reconstructed later on
	// This should not apply to intrinsic classes since they emit native references before AssembleReferenceTokenStream is called.
	if ((NewClass->ClassFlags & CLASS_Intrinsic) != CLASS_Intrinsic)
	{
		check((NewClass->ClassFlags & CLASS_TokenStreamAssembled) != CLASS_TokenStreamAssembled);
		NewClass->ReferenceSchema.Reset();
	}
	NewClass->CreateLinkAndAddChildFunctionsToMap(Params.FunctionLinkArray, Params.NumFunctions);

	ConstructFProperties(NewClass, Params.PropertyArray, Params.NumProperties);

	UE::CoreUObject::Private::ConstructUClassPartials(*NewClass);

	if (Params.ClassConfigNameUTF8)
	{
		NewClass->ClassConfigName = FName(UTF8_TO_TCHAR(Params.ClassConfigNameUTF8));
	}

	NewClass->SetCppTypeInfoStatic(Params.CppClassInfo);

	if (int32 NumImplementedInterfaces = Params.NumImplementedInterfaces)
	{
		NewClass->Interfaces.Reserve(NumImplementedInterfaces);
		for (const FImplementedInterfaceParams* ImplementedInterface = Params.ImplementedInterfaceArray, *ImplementedInterfaceEnd = ImplementedInterface + NumImplementedInterfaces; ImplementedInterface != ImplementedInterfaceEnd; ++ImplementedInterface)
		{
			UClass* (*ClassFunc)(ETypeConstructPhase) = ImplementedInterface->ClassFunc;
			UClass* InterfaceClass = ClassFunc ? ClassFunc(ETypeConstructPhase::Outer) : nullptr;

			NewClass->Interfaces.Emplace(InterfaceClass, ImplementedInterface->Offset, ImplementedInterface->bImplementedByK2);
		}
	}

#if WITH_METADATA
	AddMetaData(NewClass, Params.MetaDataArray, Params.NumMetaData);
#endif

	NewClass->StaticLink();

	NewClass->SetSparseClassDataStruct(NewClass->GetSparseClassDataArchetypeStruct());
}

template<typename UFunctionClass, typename FunctionParams, typename NewFn>
// It is UB to call `OuterFunc` below because it can return a `UClass*` not `UObject*` (even though
// there is a parent-child relation there). So we suppress UBSan for this function.
UE_DISABLE_UBSAN
void ConstructUFunctionHelper(UFunction*& InOutFunction, const FunctionParams& Params, UFunction** InSingletonPtr, NewFn&& InNewFn)
{
	FTypeConstructFunc* OuterFunc = Params.OuterFunc;
	UFunction* (*SuperFunc)(ETypeConstructPhase) = Params.SuperFunc;

	UObject* Outer = OuterFunc ? OuterFunc(ETypeConstructPhase::Outer) : nullptr;
	UFunction* Super = SuperFunc ? SuperFunc(ETypeConstructPhase::Outer) : nullptr;

	if (InOutFunction)
	{
		return;
	}

	FName FuncName(UTF8_TO_TCHAR(Params.NameUTF8));

#if WITH_LIVE_CODING
	// When a package is patched, it might reference a function in a class.  When this happens, the existing UFunction
	// object gets reused but the UField's Next pointer gets nulled out.  This ends up terminating the function list
	// for the class.  To work around this issue, cache the next pointer and then restore it after the new instance
	// is created.  Only do this if we reuse the current instance.
	UField* PrevFunctionNextField = nullptr;
	UFunction* PrevFunction = nullptr;
	if (UObject* PrevObject = StaticFindObjectFastInternal( /*Class=*/ nullptr, Outer, FuncName, EFindObjectFlags::ExactClass))
	{
		PrevFunction = Cast<UFunction>(PrevObject);
		if (PrevFunction != nullptr)
		{
			PrevFunctionNextField = PrevFunction->Next;
		}
	}
#endif

	UFunction* NewFunction = InNewFn(Outer, Super, FuncName, Params);
	check(NewFunction);
	InOutFunction = NewFunction;

#if WITH_LIVE_CODING
	NewFunction->SingletonPtr = InSingletonPtr;
	if (NewFunction == PrevFunction)
	{
		NewFunction->Next = PrevFunctionNextField;
	}
#endif

#if WITH_METADATA
	AddMetaData(NewFunction, Params.MetaDataArray, Params.NumMetaData);
#endif
	NewFunction->RPCId = Params.RPCId;
	NewFunction->RPCResponseId = Params.RPCResponseId;

	ConstructFProperties(NewFunction, Params.PropertyArray, Params.NumProperties);

	NewFunction->Bind();
	NewFunction->StaticLink();
}

} // namespace UECodeGen_Private
#endif // !UE_WITH_CONSTINIT_UOBJECT