// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectGlobals.cpp: Unreal object global data and functions
=============================================================================*/

#include "UObject/UObjectGlobals.h"
#include "Containers/BitArray.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/AsciiSet.h"
#include "Misc/EnumRange.h"
#include "Misc/PackageAccessTrackingOps.h"
#include "Misc/Paths.h"
#include "Misc/ITransaction.h"
#include "Serialization/ArchiveProxy.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IConsoleManager.h"
#include "Misc/SlowTask.h"
#include "Misc/FeedbackContext.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/App.h"
#include "UObject/ScriptInterface.h"
#include "UObject/UObjectAllocator.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectHashPrivate.h"
#include "UObject/Object.h"
#include "UObject/ObjectVisibility.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GCScopeLock.h"
#include "UObject/Class.h"
#include "UObject/UObjectPartials.h"
#include "UObject/CoreRedirects.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/ObjectInstancingGraph.h"
#include "UObject/OverridableManager.h"
#include "UObject/PropertyStateTracking.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Package.h"
#include "Templates/Casts.h"
#include "UObject/PropertyPortFlags.h"
#include "Serialization/SerializedPropertyScope.h"
#include "UObject/UnrealType.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/PackageResourceManager.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/DuplicatedObject.h"
#include "Serialization/DuplicatedDataReader.h"
#include "Serialization/DuplicatedDataWriter.h"
#include "Serialization/LoadTimeTracePrivate.h"
#include "Misc/PackageName.h"
#include "Misc/PathViews.h"
#include "UObject/LinkerLoad.h"
#include "Blueprint/BlueprintSupport.h"
#include "Misc/SecureHash.h"
#include "Misc/TrackedActivity.h"
#include "UObject/ConstructorHelpers.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/LinkerManager.h"
#include "Misc/ExclusiveLoadPackageTimeTracker.h"
#include "ProfilingDebugging/CookStats.h"
#include "Modules/ModuleManager.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/StrProperty.h"
#include "UObject/AnsiStrProperty.h"
#include "UObject/Utf8StrProperty.h"
#include "UObject/FieldPathProperty.h"
#include "UObject/MetaData.h"
#include "UObject/UObjectConstructInternal.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/LowLevelMemStats.h"
#include "Misc/CoreDelegates.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "IO/IoDispatcher.h"
#include "ProfilingDebugging/LoadTimeTracker.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Misc/PackageAccessTracking.h"
#include "UObject/PropertyWithSetterAndGetter.h"
#include "Templates/GuardValueAccessors.h"
#include "UObject/UObjectGlobalsInternal.h"
#include "Serialization/AsyncPackageLoader.h"
#include "AutoRTFM.h"
#include "UObject/PropertyOptional.h"
#include "UObject/ObjectHandlePrivate.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/RemoteObjectSerialization.h"
#include "UObject/VerseClassProperty.h"
#include "UObject/VerseStringProperty.h"
#include "VerseVM/VVMDuplication.h"
#include "VerseVM/VVMVerseClass.h"
#if WITH_VERSE_BPVM
#include "VerseVM/VBPVMDynamicProperty.h"
#endif
#include "UObject/VerseValueProperty.h"

DEFINE_LOG_CATEGORY(LogUObjectGlobals);

int32 GAllowUnversionedContentInEditor = 0;
static FAutoConsoleVariableRef CVarAllowUnversionedContentInEditor(
	TEXT("s.AllowUnversionedContentInEditor"),
	GAllowUnversionedContentInEditor,
	TEXT("If true, allows unversioned content to be loaded by the editor."),
	ECVF_Default
);

bool GAllowParseObjectLoading = true;
static FAutoConsoleVariableRef CVarAllowParseObjectLoading(
	TEXT("s.AllowParseObjectLoading"),
	GAllowParseObjectLoading,
	TEXT("If true, allows ParseObject to load fully qualified objects if needed and requested."),
	ECVF_Default
);

#ifndef PROPERTY_INIT_STATE_TRACKING_ENABLE_DEFAULT
#define PROPERTY_INIT_STATE_TRACKING_ENABLE_DEFAULT 1
#endif
#if WITH_EDITORONLY_DATA
// We disable the activation of property initialization tracking when PROPERTY_INIT_STATE_TRACKING_ENABLE_DEFAULT is 0.
// This is a workaround for an AutoRTFM issue with the lifetime of static local variables.
bool bUseAnnotationsForPropertyInitializedStateTracking = PROPERTY_INIT_STATE_TRACKING_ENABLE_DEFAULT;
FAutoConsoleVariableRef CVarUseAnnotationBasedPropertyInitializedStateTracking(
	TEXT("AnnotationBasedPropertyInitStateTracking"),
	bUseAnnotationsForPropertyInitializedStateTracking,
	TEXT("If true, enables annotation-based tracking of UObject property initialization state.")
);
#endif

void EndLoad(FUObjectSerializeContext* LoadContext, TArray<UPackage*>* OutLoadedPackages);

COREUOBJECT_API bool GetAllowNativeComponentClassOverrides()
{
	static const bool bAllowNativeComponentClassOverrides = []()
	{
		bool bAllow;
		GConfig->GetBool(TEXT("Kismet"), TEXT("bAllowNativeComponentClassOverrides"), bAllow, GEngineIni);
		return bAllow;
	}();
	return bAllowNativeComponentClassOverrides;
}
DEFINE_STAT(STAT_InitProperties);
DEFINE_STAT(STAT_ConstructObject);
DEFINE_STAT(STAT_AllocateObject);
DEFINE_STAT(STAT_PostConstructInitializeProperties);
DEFINE_STAT(STAT_LoadConfig);
DEFINE_STAT(STAT_LoadObject);
DEFINE_STAT(STAT_FindObject);
DEFINE_STAT(STAT_FindObjectFast);
DEFINE_STAT(STAT_NameTableEntries);
DEFINE_STAT(STAT_NameTableAnsiEntries);
DEFINE_STAT(STAT_NameTableWideEntries);
DEFINE_STAT(STAT_NameTableMemorySize);
DEFINE_STAT(STAT_DestroyObject);

DECLARE_CYCLE_STAT(TEXT("InstanceSubobjects"), STAT_InstanceSubobjects, STATGROUP_Object);
DECLARE_CYCLE_STAT(TEXT("PostInitProperties"), STAT_PostInitProperties, STATGROUP_Object);
DECLARE_CYCLE_STAT(TEXT("PostReinitProperties"), STAT_PostReinitProperties, STATGROUP_Object);

CSV_DEFINE_CATEGORY(UObject, false);

LLM_DEFINE_TAG(UObject_StaticConstructObjectInternal);
LLM_DEFINE_TAG(UObject_StaticAllocateObject);

#if ENABLE_COOK_STATS
#include "ProfilingDebugging/ScopedTimers.h"

namespace LoadPackageStats
{
	static double LoadPackageTimeSec = 0.0;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		AddStat(TEXT("Package.Load"), FCookStatsManager::CreateKeyValueArray(
			TEXT("LoadPackageTimeSec"), LoadPackageTimeSec));
	});
}
#endif

/** CoreUObject delegates */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
TMulticastDelegate<void(UClass* OldClass, UClass* NewClass, EHotReloadedClassFlags Flags)> FCoreUObjectDelegates::RegisterClassForHotReloadReinstancingDelegate;
TMulticastDelegate<void()> FCoreUObjectDelegates::ReinstanceHotReloadedClassesDelegate;
FCoreUObjectDelegates::FOnPackageLoadCompleted FCoreUObjectDelegates::OnPackageLoadCompleted;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
FCoreUObjectDelegates::FReloadReinstancingCompleteDelegate FCoreUObjectDelegates::ReloadReinstancingCompleteDelegate;
FCoreUObjectDelegates::FReloadCompleteDelegate FCoreUObjectDelegates::ReloadCompleteDelegate;
FCoreUObjectDelegates::FReloadAddedClassesDelegate FCoreUObjectDelegates::ReloadAddedClassesDelegate;
FCoreUObjectDelegates::FCompiledInUObjectsRegisteredDelegate FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate;
FCoreUObjectDelegates::FCompiledInUObjectsRemovedDelegate FCoreUObjectDelegates::CompiledInUObjectsRemovedDelegate;
FCoreUObjectDelegates::FIsPackageOKToSaveDelegate FCoreUObjectDelegates::IsPackageOKToSaveDelegate;
FCoreUObjectDelegates::FOnPostInitSparseClassData FCoreUObjectDelegates::OnPostInitSparseClassData;
FCoreUObjectDelegates::FOnHotfixableAssetLoaded FCoreUObjectDelegates::OnHotfixableAssetLoaded;
FCoreUObjectDelegates::FOnPackageReloaded FCoreUObjectDelegates::OnPackageReloaded;
FCoreUObjectDelegates::FNetworkFileRequestPackageReload FCoreUObjectDelegates::NetworkFileRequestPackageReload;
#if WITH_EDITOR
FCoreUObjectDelegates::FOnPreObjectPropertyChanged FCoreUObjectDelegates::OnPreObjectPropertyChanged;
FCoreUObjectDelegates::FOnPostObjectPropertyChanged FCoreUObjectDelegates::OnPostObjectPropertyChanged;
FCoreUObjectDelegates::FOnObjectPropertyChanged FCoreUObjectDelegates::OnObjectPropertyChanged;
TSet<UObject*> FCoreUObjectDelegates::ObjectsModifiedThisFrame;
FCoreUObjectDelegates::FOnObjectModified FCoreUObjectDelegates::OnObjectModified;
FCoreUObjectDelegates::FOnObjectRenamed FCoreUObjectDelegates::OnObjectRenamed;
FCoreUObjectDelegates::FOnObjectRenamed FCoreUObjectDelegates::OnLoadedObjectRenamed;
FCoreUObjectDelegates::FOnObjectTransacted FCoreUObjectDelegates::OnObjectTransacted;
FCoreUObjectDelegates::FOnObjectsReplaced FCoreUObjectDelegates::OnObjectsReplaced;
FCoreUObjectDelegates::FOnObjectsReinstanced FCoreUObjectDelegates::OnObjectsReinstanced;
FCoreUObjectDelegates::FOnVerseDeadReferences FCoreUObjectDelegates::OnVerseDeadObjectReferences;
FCoreUObjectDelegates::FOnObjectPostCDOCompiled FCoreUObjectDelegates::OnObjectPostCDOCompiled;
FCoreUObjectDelegates::FOnAssetLoaded FCoreUObjectDelegates::OnAssetLoaded;
FCoreUObjectDelegates::FOnObjectConstructed FCoreUObjectDelegates::OnObjectConstructed;
FCoreUObjectDelegates::FOnObjectPreSave FCoreUObjectDelegates::OnObjectPreSave;
#endif // WITH_EDITOR

#if WITH_EDITOR || UE_ENABLE_ASSET_READ_LOGGER
FCoreUObjectDelegates::FOnEndLoadPackage FCoreUObjectDelegates::OnEndLoadPackage;
#endif // WITH_EDITOR || UE_ENABLE_ASSET_READ_LOGGER

// This map holds raw UClass pointers. It is the responsibility of the callers
// of Register/UnregisterHotfixableAssetClass to keep these pointers alive.
// The class pointer is reference counted in the map to allow for multiple
// Register/Unregister calls from different sources.
// Guarded by HotfixableAssetClassesLock: IsHotfixableAssetClass may be called
// from the async loading thread concurrently with Register/Unregister on the game thread.
static FRWLock HotfixableAssetClassesLock;
static TMap<UClass*, uint32> HotfixableAssetClasses;
// Number of distinct registered classes. Enables a lock-free fast path in IsHotfixableAssetClass.
static std::atomic<int32> HotfixableAssetClassCount{0};

void FCoreUObjectDelegates::RegisterHotfixableAssetClass(UClass* Class)
{
	FWriteScopeLock Lock(HotfixableAssetClassesLock);
	uint32& RefCount = HotfixableAssetClasses.FindOrAdd(Class);
	if (RefCount == 0)
	{
		HotfixableAssetClassCount.fetch_add(1, std::memory_order_release);
	}
	++RefCount;
}

void FCoreUObjectDelegates::UnregisterHotfixableAssetClass(UClass* Class)
{
	FWriteScopeLock Lock(HotfixableAssetClassesLock);
	if (uint32* RefPtr = HotfixableAssetClasses.Find(Class))
	{
		if (*RefPtr <= 1)
		{
			HotfixableAssetClasses.Remove(Class);
			HotfixableAssetClassCount.fetch_sub(1, std::memory_order_release);
		}
		else
		{
			--*RefPtr;
		}
	}
}

bool FCoreUObjectDelegates::IsHotfixableAssetClass(UClass* Class)
{
	if (HotfixableAssetClassCount.load(std::memory_order_acquire) == 0)
	{
		return false;
	}

	FReadScopeLock Lock(HotfixableAssetClassesLock);
	return HotfixableAssetClasses.Contains(Class);
}

FSimpleMulticastDelegate& FCoreUObjectDelegates::GetPreGarbageCollectDelegate()
{
	static FSimpleMulticastDelegate Delegate;
	return Delegate;
}

FSimpleMulticastDelegate& FCoreUObjectDelegates::GetGarbageCollectStartedDelegate()
{
	static FSimpleMulticastDelegate Delegate;
	return Delegate;
}

FSimpleMulticastDelegate& FCoreUObjectDelegates::GetPostGarbageCollect()
{
	static FSimpleMulticastDelegate Delegate;
	return Delegate;
}

FSimpleMulticastDelegate& FCoreUObjectDelegates::GetPostPurgeGarbageDelegate()
{
	static FSimpleMulticastDelegate Delegate;
	return Delegate;
}

#if !UE_BUILD_SHIPPING
FCoreUObjectDelegates::FOnReportGarbageReferencers& FCoreUObjectDelegates::GetGarbageCollectReportGarbageReferencers()
{
	static FCoreUObjectDelegates::FOnReportGarbageReferencers Delegate;
	return Delegate;
}
#endif

FCoreUObjectInternalDelegates::FPackageRename& FCoreUObjectInternalDelegates::GetOnLeakedPackageRenameDelegate()
{
	static FPackageRename Delegate;
	return Delegate;
}

void TrashObject(TNotNull<UObject* const> Object)
{
	// We should *never* trash an object that is in the process of being loaded.
	// TODO: This should probably degrade to a `check` longer term, but out of
	// an abundance of caution we'll start it as an `ensure`.

	const bool bIsBeingLoaded = Object->HasAnyFlags(EObjectFlags::RF_WillBeLoaded) ||
		Object->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading);

	if (ensure(!bIsBeingLoaded))
	{
		const bool bIsAPackage = Object->IsA<UPackage>();

		const FName BaseName = bIsAPackage ? NAME_TrashedPackage : NAME_None;

		const FName NewName = MakeUniqueObjectName(Object->GetOuter(), Object->GetClass(), BaseName);
		Object->Rename(*NewName.ToString(), nullptr, REN_DontCreateRedirectors | REN_NonTransactional | REN_DoNotDirty);

		if (bIsAPackage)
		{
			// Mark it transient to be sure it is not seen as relevant by various systems.
			Object->SetFlags(RF_Transient);
		}
	}
}

FSimpleMulticastDelegate FCoreUObjectDelegates::PostReachabilityAnalysis;

FSimpleMulticastDelegate FCoreUObjectDelegates::PreGarbageCollectConditionalBeginDestroy;
FSimpleMulticastDelegate FCoreUObjectDelegates::PostGarbageCollectConditionalBeginDestroy;

FSimpleMulticastDelegate FCoreUObjectDelegates::GarbageCollectComplete;

FCoreUObjectDelegates::FPreLoadMapDelegate FCoreUObjectDelegates::PreLoadMap;
FCoreUObjectDelegates::FPreLoadMapWithContextDelegate FCoreUObjectDelegates::PreLoadMapWithContext;
FCoreUObjectDelegates::FPostLoadMapDelegate FCoreUObjectDelegates::PostLoadMapWithWorld;
FSimpleMulticastDelegate FCoreUObjectDelegates::PostDemoPlay;
FCoreUObjectDelegates::FOnLoadObjectsOnTop FCoreUObjectDelegates::ShouldLoadOnTop;
FCoreUObjectDelegates::FShouldCookPackageForPlatform FCoreUObjectDelegates::ShouldCookPackageForPlatform;

FCoreUObjectDelegates::FPackageCreatedForLoad FCoreUObjectDelegates::PackageCreatedForLoad;
FCoreUObjectDelegates::FGetPrimaryAssetIdForObject FCoreUObjectDelegates::GetPrimaryAssetIdForObject;

/** Check whether we should report progress or not */
bool ShouldReportProgress()
{
	return GIsEditor && IsInGameThread() && !IsRunningCommandlet() && !IsAsyncLoading();
}

bool ShouldCreateThrottledSlowTask()
{
	return ShouldReportProgress();
}

// Anonymous namespace to not pollute global.
namespace
{
	/**
	 * Legacy static find object helper, that helps to find reflected types, that
	 * are no longer a subobjects of UCLASS defined in the same header.
	 *
	 * If the class looked for is of one of the relocated types (or theirs subclass)
	 * then it performs another search in containing package.
	 *
	 * If the class match wasn't exact (i.e. either nullptr or subclass of allowed
	 * ones) and we've found an object we're revalidating it to make sure the
	 * legacy search was valid.
	 *
	 * @param ObjectClass Class of the object to find.
	 * @param ObjectPackage Package of the object to find.
	 * @param ObjectName Name of the object to find.
	 * @param ExactClass If the class match has to be exact. I.e. ObjectClass == FoundObjects.GetClass()
	 * @param ExclusiveInternalFlags Do not return objects that have any of these internal flags
	 *
	 * @returns Found object.
	 */
	UObject* StaticFindObjectWithChangedLegacyPath(UClass* ObjectClass, UObject* ObjectPackage, FName ObjectName, EFindObjectFlags Flags, EInternalObjectFlags ExclusiveInternalFlags)
	{
		UObject* MatchingObject = nullptr;

		// This is another look-up for native enums, structs or delegate signatures, cause they're path changed
		// and old packages can have invalid ones. The path now does not have a UCLASS as an outer. All mentioned
		// types are just children of package of the file there were defined in.
		if (!FPlatformProperties::RequiresCookedData() && // Cooked platforms will have all paths resolved.
			ObjectPackage != nullptr &&
			ObjectPackage->IsA<UClass>()) // Only if outer is a class.
		{
			bool bHasDelegateSignaturePostfix = ObjectName.ToString().EndsWith(HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX);

			bool bExactPathChangedClass = ObjectClass == UEnum::StaticClass() // Enums
				|| ObjectClass == UScriptStruct::StaticClass() || ObjectClass == UStruct::StaticClass() // Structs
				|| (ObjectClass == UFunction::StaticClass() && bHasDelegateSignaturePostfix); // Delegates

			bool bSubclassOfPathChangedClass = !bExactPathChangedClass && !(Flags & EFindObjectFlags::ExactClass)
				&& (ObjectClass == nullptr // Any class
				|| UEnum::StaticClass()->IsChildOf(ObjectClass) // Enums
				|| UScriptStruct::StaticClass()->IsChildOf(ObjectClass) || UStruct::StaticClass()->IsChildOf(ObjectClass) // Structs
				|| (UFunction::StaticClass()->IsChildOf(ObjectClass) && bHasDelegateSignaturePostfix)); // Delegates

			if (!bExactPathChangedClass && !bSubclassOfPathChangedClass)
			{
				return nullptr;
			}

			MatchingObject = StaticFindObject(ObjectClass, ObjectPackage->GetOutermost(), *ObjectName.ToString(), Flags);

			if (MatchingObject)
			{
				if (MatchingObject->HasAnyInternalFlags(ExclusiveInternalFlags))
				{
					return nullptr;
				}

				if (bSubclassOfPathChangedClass)
				{
					// If the class wasn't given exactly, check if found object is of class that outers were changed.
					UClass* MatchingObjectClass = MatchingObject->GetClass();
					if (!(MatchingObjectClass == UEnum::StaticClass()	// Enums
						|| MatchingObjectClass == UScriptStruct::StaticClass() || MatchingObjectClass == UStruct::StaticClass() // Structs
						|| (MatchingObjectClass == UFunction::StaticClass() && bHasDelegateSignaturePostfix)) // Delegates
						)
					{
						return nullptr;
					}
				}
			}
		}

		return MatchingObject;
	}
}

FString LexToString(EObjectFlags Flags)
{
	if (Flags == RF_NoFlags)
	{
		return TEXT("None");
	}	
	
	static const TCHAR* Names[] = {
		TEXT("Public"),
		TEXT("Standalone"),
		TEXT("MarkAsNative"),
		TEXT("Transactional"),
		TEXT("ClassDefaultObject"),
		TEXT("ArchetypeObject"),
		TEXT("Transient"),
		TEXT("MarkAsRootSet"),
		TEXT("TagGarbageTemp"),
		TEXT("NeedInitialization"),
		TEXT("NeedLoad"),
		TEXT("KeepForCooker"),
		TEXT("NeedPostLoad"),
		TEXT("NeedPostLoadSubobjects"),
		TEXT("NewerVersionExists"),
		TEXT("BeginDestroyed"),
		TEXT("FinishDestroyed"),
		TEXT("BeingRegenerated"),
		TEXT("DefaultSubObject"),
		TEXT("WasLoaded"),
		TEXT("TextExportTransient"),
		TEXT("LoadCompleted"),
		TEXT("InheritableComponentTemplate"),
		TEXT("DuplicateTransient"),
		TEXT("StrongRefOnFrame"),
		TEXT("NonPIEDuplicateTransient"),
		TEXT("Dynamic"),
		TEXT("WillBeLoaded"),
		TEXT("HasExternalPackage"),
		TEXT("MigratingAsset"),
		TEXT("Garbage"),
		TEXT("AllocatedInSharedPage"),
	};

	TStringBuilder<1024> Builder;
	for (EObjectFlags Flag : MakeFlagsRange(Flags))
	{
		int32 Index = FMath::FloorLog2((uint32)Flag);	
		if (Builder.Len() > 0)
		{
			Builder << TEXT(" | ");
		}	
		Builder << Names[Index];
	}
	return Builder.ToString();
}

/** Object annotation used to keep track of the number suffixes  */
struct FPerClassNumberSuffixAnnotation
{
	// The annotation container uses this to trim annotations that return to
	// the default state - this never happens for this annotation type.
	FORCEINLINE bool IsDefault()
	{
		return false;
	}

	TMap<FName, int32> Suffixes;
};

/**
 * Updates the suffix to be given to the next newly-created unnamed object.
 *
 * Updating is done via a callback because a lock needs to be maintained while this happens.
 */
int32 UpdateSuffixForNextNewObject(UObject* Parent, FName BaseName, TFunctionRef<void(int32&)> IndexMutator)
{
	static FCriticalSection PerClassNumberSuffixAnnotationMutex;
	static FUObjectAnnotationDense<FPerClassNumberSuffixAnnotation, true> PerClassNumberSuffixAnnotation;

	FScopeLock Lock(&PerClassNumberSuffixAnnotationMutex);
	FPerClassNumberSuffixAnnotation& Annotation = PerClassNumberSuffixAnnotation.GetAnnotationRef(Parent);
	int32& Result = Annotation.Suffixes.FindOrAdd(BaseName);
	IndexMutator(Result);
	return Result;
}

int32 UpdateSuffixForNextNewObject(UObject* Parent, const UClass* Class, TFunctionRef<void(int32&)> IndexMutator)
{
	return UpdateSuffixForNextNewObject(Parent, *Class->GetPathName(), IndexMutator);
}

//
// Find an object, path must unqualified
//
UObject* StaticFindObjectFast(UClass* ObjectClass, UObject* ObjectPackage, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExclusiveFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	UE_CLOGF(UE::IsSavingPackage(nullptr), LogUObjectGlobals, Fatal, "Illegal call to StaticFindObjectFast() while serializing object data!");
	UE_CLOGF(IsGarbageCollectingAndLockingUObjectHashTables(), LogUObjectGlobals, Fatal, "Illegal call to StaticFindObjectFast() while garbage collecting!");

	// We don't want to return any objects that are currently being background loaded unless we're using FindObject during async loading.
	ExclusiveInternalFlags |= UE::GetAsyncLoadingInternalFlagsExclusion();
	UObject* FoundObject = StaticFindObjectFastInternal(ObjectClass, ObjectPackage, ObjectName, Flags, ExclusiveFlags, ExclusiveInternalFlags);

	if (!FoundObject)
	{
		FoundObject = StaticFindObjectWithChangedLegacyPath(ObjectClass, ObjectPackage, ObjectName, Flags, ExclusiveInternalFlags);
	}

	return FoundObject;
}

UObject* StaticFindObjectFastSafe(UClass* ObjectClass, UObject* ObjectPackage, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExclusiveFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	UObject* FoundObject = nullptr;

	if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
	{
		// We don't want to return any objects that are currently being background loaded unless we're using FindObject during async loading.
		ExclusiveInternalFlags |= UE::GetAsyncLoadingInternalFlagsExclusion();;
		FoundObject = StaticFindObjectFastInternal(ObjectClass, ObjectPackage, ObjectName, Flags, ExclusiveFlags, ExclusiveInternalFlags);
		if (!FoundObject)
		{
			FoundObject = StaticFindObjectWithChangedLegacyPath(ObjectClass, ObjectPackage, ObjectName, Flags, ExclusiveInternalFlags);
		}
	}

	return FoundObject;
}

// TODO: Should these appear in public API?
void ConstructorHelpers_StripObjectClass2(FStringBuilderBase& PathName, bool bAssertOnBadPath = false);
bool ResolveName2(UObject*& InPackage, FStringBuilderBase& InOutName, bool Create, bool Throw, uint32 LoadFlags = LOAD_None, const FLinkerInstancingContext* InstancingContext = nullptr);


#if WITH_EDITOR
static UObject* LoadObjectWhenImportingT3D(UClass* ObjectClass, FStringView OrigInName)
{
	UObject* MatchingObject = nullptr;
	static bool s_bCurrentlyLoading = false;
	if (s_bCurrentlyLoading == false)
	{
		if (OrigInName.Contains(TEXT("."), ESearchCase::CaseSensitive) &&
			!OrigInName.Contains(TEXT("'"), ESearchCase::CaseSensitive) &&
			!OrigInName.Contains(TEXT(":"), ESearchCase::CaseSensitive))
		{
			s_bCurrentlyLoading = true;
			MatchingObject = StaticLoadObject(ObjectClass, nullptr, OrigInName, nullptr, LOAD_NoWarn, nullptr);
			s_bCurrentlyLoading = false;
		}
	}
	return MatchingObject;
}

#endif // WITH_EDITOR
//
// Find an optional object.
//
UObject* StaticFindObject( UClass* ObjectClass, UObject* InObjectPackage, FStringView OrigInName, EFindObjectFlags Flags )
{
	INC_DWORD_STAT(STAT_FindObject);

	// Resolve the object and package name.
	UObject* ObjectPackage = InObjectPackage;

	UObject* MatchingObject = nullptr;
#if WITH_EDITOR
	// If the editor is running, and T3D is being imported, ensure any packages referenced are fully loaded.
	if ((GIsEditor == true) && (GIsImportingT3D == true))
	{
		MatchingObject = LoadObjectWhenImportingT3D(ObjectClass, OrigInName);
		if (MatchingObject)
		{
			return MatchingObject;
		}
	}
#endif	//#if !WITH_EDITOR

	TStringBuilder<512> InName;
	InName = OrigInName;
	
	// Don't resolve the name if we're searching in any package
	if (!ResolveName2(ObjectPackage, InName, false, false))
	{
		return nullptr;
	}
	
	FName ObjectName(InName.ToView(), FNAME_Add);
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return StaticFindObjectFast(ObjectClass, ObjectPackage, ObjectName, Flags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

//
// Find an optional object.
//
UObject* StaticFindObject(UClass* Class, FTopLevelAssetPath ObjectPath, EFindObjectFlags Flags)
{
	UObject* Result = nullptr;
	if (!ObjectPath.IsNull())
	{
		UObject* Package = StaticFindObjectFast(UPackage::StaticClass(), nullptr, ObjectPath.GetPackageName());
		if (Package)
		{
			Result = StaticFindObjectFast(Class, Package, ObjectPath.GetAssetName(), Flags);
		}
	}
	return Result;
}

UObject* StaticFindObjectSafe(UClass* Class, FTopLevelAssetPath ObjectPath, EFindObjectFlags Flags)
{
	if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
	{
		FGCScopeGuard GCGuard;
		return StaticFindObject(Class, ObjectPath, Flags);
	}
	else
	{
		return nullptr;
	}
}

//
// Find an object; can't fail.
//
UObject* StaticFindObjectChecked( UClass* ObjectClass, UObject* ObjectParent, FStringView InName, EFindObjectFlags Flags )
{
	UObject* Result = StaticFindObject( ObjectClass, ObjectParent, InName, Flags );
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if( !Result )
	{
		UE_LOGF(LogUObjectGlobals, Fatal, "Failed to find object '%ls %ls.%.*ls'", *ObjectClass->GetName(), ObjectParent ? *ObjectParent->GetName() : TEXT("None"), InName.Len(), InName.GetData());
	}
#endif
	return Result;
}

//
// Find an object; won't assert on UE::IsSavingPackage() or IsGarbageCollectingAndLockingUObjectHashTables()
//
UObject* StaticFindObjectSafe( UClass* ObjectClass, UObject* ObjectParent, FStringView InName, EFindObjectFlags Flags )
{
	if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
	{
		FGCScopeGuard GCGuard;
		return StaticFindObject( ObjectClass, ObjectParent, InName, Flags );
	}
	else
	{
		return nullptr;
	}
}

bool StaticFindAllObjectsFast(TArray<UObject*>& OutFoundObjects, UClass* ObjectClass, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExclusiveFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	UE_CLOGF(UE::IsSavingPackage(nullptr) || IsGarbageCollectingAndLockingUObjectHashTables(), LogUObjectGlobals, Fatal, "Illegal call to StaticFindAllObjectsFast() while serializing object data or garbage collecting!");

	// We don't want to return any objects that are currently being background loaded unless we're using FindObject during async loading.
	ExclusiveInternalFlags |= UE::GetAsyncLoadingInternalFlagsExclusion();
	return StaticFindAllObjectsFastInternal(OutFoundObjects, ObjectClass, ObjectName, Flags, ExclusiveFlags, ExclusiveInternalFlags);
}

bool StaticFindAllObjectsFastSafe(TArray<UObject*>& OutFoundObjects, UClass* ObjectClass, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExclusiveFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	bool bFoundObjects = false;
	if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
	{
		// We don't want to return any objects that are currently being background loaded unless we're using FindObject during async loading.
		ExclusiveInternalFlags |= UE::GetAsyncLoadingInternalFlagsExclusion();
		bFoundObjects = StaticFindAllObjectsFastInternal(OutFoundObjects, ObjectClass, ObjectName, Flags, ExclusiveFlags, ExclusiveInternalFlags);
	}
	return bFoundObjects;
}

bool StaticFindAllObjects(TArray<UObject*>& OutFoundObjects, UClass* ObjectClass, const TCHAR* OrigInName, EFindObjectFlags Flags)
{
	INC_DWORD_STAT(STAT_FindObject);

	UE_CLOGF(UE::IsSavingPackage(nullptr), LogUObjectGlobals, Fatal, "Illegal call to StaticFindAllObjects() while serializing object data!");
	UE_CLOGF(IsGarbageCollectingAndLockingUObjectHashTables(), LogUObjectGlobals, Fatal, "Illegal call to StaticFindAllObjects() while collecting garbage!");

#if WITH_EDITOR
	// If the editor is running, and T3D is being imported, ensure any packages referenced are fully loaded.
	if ((GIsEditor == true) && (GIsImportingT3D == true))
	{
		UObject* MatchingObject = LoadObjectWhenImportingT3D(ObjectClass, OrigInName);
		if (MatchingObject != nullptr)
		{
			OutFoundObjects.Add(MatchingObject);
			return true;
		}
	}
#endif	//#if !WITH_EDITOR

	// Don't resolve the name since we're searching in any package
	TStringBuilder<512> InName;
	InName = OrigInName;
	ConstructorHelpers_StripObjectClass2(InName);
	FName ObjectName(InName.ToView(), FNAME_Add);

	return StaticFindAllObjectsFast(OutFoundObjects, ObjectClass, ObjectName, Flags);
}

bool StaticFindAllObjectsSafe(TArray<UObject*>& OutFoundObjects, UClass* ObjectClass, const TCHAR* OrigInName, EFindObjectFlags Flags)
{
	bool bFoundObjects = false;
	if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
	{
		bFoundObjects = StaticFindAllObjects(OutFoundObjects, ObjectClass, OrigInName, Flags);
	}
	return bFoundObjects;
}

UObject* StaticFindFirstObject(UClass* Class, FStringView Name, EFindFirstObjectOptions Options /*= EFindFirstObjectOptions::None*/, ELogVerbosity::Type AmbiguousMessageVerbosity /*= ELogVerbosity::Warning*/, const TCHAR* InCurrentOperation /*= nullptr*/)
{
	UObject* Result = nullptr;
	FName ObjectName;
	if (UE::String::FindFirstChar(Name, TCHAR('\'')) == INDEX_NONE)
	{
		// Skip unnecessary allocations in StripObjectClass
		ObjectName = FName(Name, FNAME_Add);
	}
	else
	{
		TStringBuilder<512> InName;
		InName = Name;
		ConstructorHelpers_StripObjectClass2(InName);
		ObjectName = FName(InName.ToView(), FNAME_Add);
	}

	EFindObjectFlags FindObjectFlags = !!(Options & EFindFirstObjectOptions::ExactClass) ? EFindObjectFlags::ExactClass : EFindObjectFlags::None;
	if (AmbiguousMessageVerbosity == ELogVerbosity::NoLogging && !(Options & (EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous)))
	{
		Result = StaticFindFirstObjectFastInternal(Class, ObjectName, FindObjectFlags);
	}
	else
	{
		TArray<UObject*> FoundObjects;
		if (StaticFindAllObjectsFast(FoundObjects, Class, ObjectName, FindObjectFlags))
		{
			if (FoundObjects.Num() > 1)
			{
				if (!!(Options & EFindFirstObjectOptions::NativeFirst))
				{
					// Prioritize native class instances or native type objects
					for (UObject* FoundObject : FoundObjects)
					{
						if (FoundObject)
						{
							if (FoundObject->IsA<UField>())
							{
								// If we were looking for a 'type' (UEnum / UClass / UScriptStruct) object priotize native types
								if (FoundObject->GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn))
								{
									Result = FoundObject;
									break;
								}
							}
							else if (!Result && FoundObject->GetClass()->GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn))
							{
								Result = FoundObject;
								// Don't break yet, maybe we can find a native type (see above) which is usually what we're after anyway
							}
						}
					}
				}
				if (!Result)
				{
					Result = FoundObjects[0];
				}

				if (AmbiguousMessageVerbosity != ELogVerbosity::NoLogging || !!(Options & EFindFirstObjectOptions::EnsureIfAmbiguous))
				{
					TStringBuilder<256> Message;
					Message.Append(TEXT("StaticFindFirstObject: Ambiguous object name "));
					Message.Append(Name);
					if (InCurrentOperation)
					{
						Message.Append(TEXT(" while "));
						Message.Append(InCurrentOperation);
					}
					Message.Append(TEXT(", will return "));
					Message.Append(Result->GetPathName());
					Message.Append(TEXT(" but could also be: "));
					const int32 MaxObjectsToPrint = 1;
					int32 PrintedObjects = 0;
					for (int32 ObjectIndex = 0; ObjectIndex < FoundObjects.Num() && PrintedObjects < MaxObjectsToPrint; ++ObjectIndex)
					{
						if (FoundObjects[ObjectIndex] != Result)
						{
							if (PrintedObjects > 0)
							{
								Message.Append(TEXT(", "));
							}
							Message.Append(FoundObjects[ObjectIndex]->GetPathName());
							PrintedObjects++;
						}
					}
					if (FoundObjects.Num() > (MaxObjectsToPrint + 1)) // +1 because we also printed Result's PathName
					{
						Message.Appendf(TEXT(" or %d other object(s)"), FoundObjects.Num() - (MaxObjectsToPrint + 1));
					}
					if (AmbiguousMessageVerbosity == ELogVerbosity::Fatal)
					{
						UE_LOGF(LogUObjectGlobals, Fatal, "%ls", Message.ToString());
					}
					else if (AmbiguousMessageVerbosity != ELogVerbosity::NoLogging)
					{
						GLog->CategorizedLogf(TEXT("LogUObjectGlobals"), AmbiguousMessageVerbosity, TEXT("%s"), Message.ToString());
					}

					ensureAlwaysMsgf(!(Options & EFindFirstObjectOptions::EnsureIfAmbiguous), TEXT("%s"), Message.ToString());
				}
			}
			else
			{
				Result = FoundObjects[0];
			}
		}
	}
	return Result;
}

UObject* StaticFindFirstObjectSafe(UClass* Class, FStringView Name, EFindFirstObjectOptions Options /*= EFindFirstObjectOptions::None*/, ELogVerbosity::Type AmbiguousMessageVerbosity /*= ELogVerbosity::Warning*/, const TCHAR* InCurrentOperation /*= nullptr*/)
{
	UObject* Result = nullptr;
	if (!UE::IsSavingPackage(nullptr) && !IsGarbageCollectingAndLockingUObjectHashTables())
	{
		Result = StaticFindFirstObject(Class, Name, Options, AmbiguousMessageVerbosity, InCurrentOperation);
	}
	return Result;
}

//
// Global property setting.
//
void GlobalSetProperty( const TCHAR* Value, UClass* Class, FProperty* Property, bool bNotifyObjectOfChange )
{
	if ( Property != NULL && Class != NULL )
	{
		// Apply to existing objects of the class.
		for( FThreadSafeObjectIterator It; It; ++It )
		{	
			UObject* Object = *It;
			if( Object->IsA(Class) && IsValidChecked(Object) )
			{
				// If we're in a PIE session then only allow set commands to affect PlayInEditor objects.
				if( !GIsPlayInEditorWorld || Object->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor)  )
				{
#if WITH_EDITOR
					if( !Object->HasAnyFlags(RF_ClassDefaultObject) && bNotifyObjectOfChange )
					{
						Object->PreEditChange(Property);
					}
#endif // WITH_EDITOR
					Property->ImportText_InContainer(Value, Object, Object, 0);
#if WITH_EDITOR
					if( !Object->HasAnyFlags(RF_ClassDefaultObject) && bNotifyObjectOfChange )
					{
						FPropertyChangedEvent PropertyEvent(Property);
						Object->PostEditChangeProperty(PropertyEvent);
					}
#endif // WITH_EDITOR
				}
			}
		}

		if (FPlatformProperties::HasEditorOnlyData())
		{
			// Apply to defaults.
			UObject* DefaultObject = Class->GetDefaultObject();
			check(DefaultObject != NULL);
			DefaultObject->SaveConfig();
		}
	}
}

/*-----------------------------------------------------------------------------
	UObject Tick.
-----------------------------------------------------------------------------*/

// @warning: The streaming stats rely on this function not doing any work besides calling ProcessAsyncLoading.
// @todo: Move stats code into core?
void StaticTick( float DeltaTime, bool bUseFullTimeLimit, float AsyncLoadingTime )
{
	check(!IsLoading());

	// Spend a bit of time (pre)loading packages - currently 5 ms.
	ProcessAsyncLoading(true, bUseFullTimeLimit, AsyncLoadingTime);

	// Check natives.
	extern int32 GNativeDuplicate;
	if( GNativeDuplicate )
	{
		UE_LOGF(LogUObjectGlobals, Fatal, "Duplicate native registered: %i", GNativeDuplicate );
	}

#if STATS
	// Set name table stats.
	int32 NameTableAnsiEntries = FName::GetNumAnsiNames();
	int32 NameTableWideEntries = FName::GetNumWideNames();
	int32 NameTableEntries = NameTableAnsiEntries + NameTableWideEntries;
	int64 NameTableMemorySize = FName::GetNameTableMemorySize();
	SET_DWORD_STAT( STAT_NameTableEntries, NameTableEntries );
	SET_DWORD_STAT( STAT_NameTableAnsiEntries, NameTableAnsiEntries );
	SET_DWORD_STAT( STAT_NameTableWideEntries, NameTableWideEntries);
	SET_MEMORY_STAT( STAT_NameTableMemorySize, NameTableMemorySize );

#if 0 // can't read stats with the new stats system
	// Set async I/O bandwidth stats.
	static uint32 PreviousReadSize	= 0;
	static uint32 PrevioudReadCount	= 0;
	static float PreviousReadTime	= 0;
	float ReadTime	= GStatManager.GetStatValueFLOAT( STAT_AsyncIO_PlatformReadTime );
	uint32 ReadSize	= GStatManager.GetStatValueDWORD( STAT_AsyncIO_FulfilledReadSize );
	uint32 ReadCount	= GStatManager.GetStatValueDWORD( STAT_AsyncIO_FulfilledReadCount );

	// It is possible that the stats are update in between us reading the values so we simply defer till
	// next frame if that is the case. This also handles partial updates. An individual value might be 
	// slightly wrong but we have enough small requests to smooth it out over a few frames.
	if( (ReadTime  - PreviousReadTime ) > 0.f 
	&&	(ReadSize  - PreviousReadSize ) > 0 
	&&	(ReadCount - PrevioudReadCount) > 0 )
	{
		float Bandwidth = (ReadSize - PreviousReadSize) / (ReadTime - PreviousReadTime) / 1048576.f;
		SET_FLOAT_STAT( STAT_AsyncIO_Bandwidth, Bandwidth );
		PreviousReadTime	= ReadTime;
		PreviousReadSize	= ReadSize;
		PrevioudReadCount	= ReadCount;
	}
	else
	{
		SET_FLOAT_STAT( STAT_AsyncIO_Bandwidth, 0.f );
	}
#endif
#endif
}



/*-----------------------------------------------------------------------------
   File loading.
-----------------------------------------------------------------------------*/

//
// Safe load error-handling. Returns true if a message was emitted.
//
bool SafeLoadError( UObject* Outer, uint32 LoadFlags, const TCHAR* ErrorMessage)
{
	bool bRetVal = false;
	if( FParse::Param( FCommandLine::Get(), TEXT("TREATLOADWARNINGSASERRORS") ) == true )
	{
		UE_LOGF(LogUObjectGlobals, Error, "%ls", ErrorMessage);
		bRetVal = true;
	}
	else
	{
		// Don't warn here if either quiet or no warn are set
		if( (LoadFlags & LOAD_Quiet) == 0 && (LoadFlags & LOAD_NoWarn) == 0)
		{ 
			UE_LOGF(LogUObjectGlobals, Warning, "%ls", ErrorMessage);
			bRetVal = true;
		}
	}

	return bRetVal;
}

UPackage* FindPackage( UObject* InOuter, const TCHAR* PackageName )
{
	TStringBuilder<512> InName;
	if( PackageName )
	{
		InName = PackageName;
	}
	else
	{
		MakeUniqueObjectName( InOuter, UPackage::StaticClass() ).ToString(InName);
	}
	ResolveName2( InOuter, InName, true, false );

	UPackage* Result = NULL;
	if ( InName.ToView() != TEXT("None") )
	{
		Result = FindObject<UPackage>( InOuter, *InName );
	}
	else
	{
		UE_LOGF(LogUObjectGlobals, Fatal, "Attempted to find a package named 'None' - InName: %ls", PackageName);
	}
	return Result;
}

#if WITH_EDITOR
struct FCreatePackageDefaultFlagsMap
{
	uint32 Find(FStringView MountPoint)
	{
		FReadScopeLock ReadScopeLock(Lock);
		EPackageFlags* DefaultPackageFlags = MountPointToDefaultPackageFlags.FindByHash(GetTypeHash(MountPoint), MountPoint);

		return DefaultPackageFlags ? *DefaultPackageFlags : EPackageFlags::PKG_None;
	}

	void Add(const TMap<FString, EPackageFlags>& InMountPointToDefaultPackageFlags)
	{
		if (InMountPointToDefaultPackageFlags.IsEmpty())
		{
			return;
		}

		FWriteScopeLock WriteScopeLock(Lock);
		MountPointToDefaultPackageFlags.Append(InMountPointToDefaultPackageFlags);
	}

	void Remove(const TArrayView<FString>& InMountPoints)
	{
		if (InMountPoints.IsEmpty())
		{
			return;
		}

		FWriteScopeLock WriteScopeLock(Lock);
		for (const FString& MountPoint : InMountPoints)
		{
			MountPointToDefaultPackageFlags.Remove(MountPoint);
		}
	}
private:
	FRWLock Lock;
	TMap<FString, EPackageFlags> MountPointToDefaultPackageFlags;
};

static FCreatePackageDefaultFlagsMap GCreatePackageDefaultFlagsMap;

void SetMountPointDefaultPackageFlags(const TMap<FString, EPackageFlags>& InMountPointToDefaultPackageFlags)
{
	GCreatePackageDefaultFlagsMap.Add(InMountPointToDefaultPackageFlags);
}

void RemoveMountPointDefaultPackageFlags(const TArrayView<FString> InMountPoints)
{
	GCreatePackageDefaultFlagsMap.Remove(InMountPoints);
}
#endif //if WITH_EDITOR

UPackage* CreatePackage(const TCHAR* PackageName )
{
	TStringBuilder<512> InName;

	if( PackageName )
	{
		InName = PackageName;

		if( InName.ToView().Contains(TEXT("//")))
		{
			UE_LOGF(LogUObjectGlobals, Fatal, "Attempted to create a package with name containing double slashes. PackageName: %ls", PackageName);
		}

		if( InName.ToView().EndsWith( TEXT( "." ) ) )
		{
			FStringView InName2 = InName.ToView().Left( InName.Len() - 1 );
			UE_LOGF(LogUObjectGlobals, Log,  "Invalid Package Name entered - '%ls' renamed to '%ls'", *InName, InName2.GetData() );
			InName = InName2;
		}
	}

	if(InName.Len() == 0)
	{
		MakeUniqueObjectName( nullptr, UPackage::StaticClass() ).ToString(InName);
	}

	UObject* Outer = nullptr;
	ResolveName2(Outer, InName, true, false );


	UPackage* Result = NULL;
	if ( InName.Len() == 0 )
	{
		UE_LOGF(LogUObjectGlobals, Fatal, "%ls", TEXT("Attempted to create a package with an empty package name.") );
	}

	if ( InName.ToView() != TEXT("None") )
	{
		Result = FindObject<UPackage>( nullptr, *InName );
		if( Result == NULL )
		{
			FName NewPackageName(*InName, FNAME_Add);
			if (FPackageName::IsShortPackageName(NewPackageName))
			{
				UE_LOGF(LogUObjectGlobals, Warning, "Attempted to create a package with a short package name: %ls Outer: %ls", PackageName, Outer ? *Outer->GetFullName() : TEXT("NullOuter"));
			}
			else
			{
				Result = NewObject<UPackage>(nullptr, NewPackageName, RF_Public);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
				if (Result->GetClass()->GetFName() != NAME_None)
				{
					UE::CoreUObject::Private::MakePackedObjectRef(Result);
				}
#endif
#if WITH_EDITOR
				FStringView PackageMountPoint = FPathViews::GetMountPointNameFromPath(InName);
				uint32 DefaultPackageFlags = GCreatePackageDefaultFlagsMap.Find(PackageMountPoint);
				Result->SetPackageFlags(DefaultPackageFlags);
#endif
			}
		}
	}
	else
	{
		UE_LOGF(LogUObjectGlobals, Fatal, "%ls", TEXT("Attempted to create a package named 'None'") );
	}

	return Result;
}

FString ResolveIniObjectsReference(const FString& ObjectReference, const FString* IniFilename, bool bThrow)
{
	if (!IniFilename)
	{
		IniFilename = GetIniFilenameFromObjectsReference(ObjectReference);
	}

	if (!IniFilename)
	{
		return ObjectReference;
	}

	// Get .ini key and section.
	FString Section = ObjectReference.Mid(1 + ObjectReference.Find(TEXT(":"), ESearchCase::CaseSensitive));
	int32 i = Section.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
	FString Key;
	if (i != -1)
	{
		Key = Section.Mid(i + 1);
		Section.LeftInline(i, EAllowShrinking::No);
	}

	FString Output;

	// Look up name.
	if (!GConfig->GetString(*Section, *Key, Output, *IniFilename))
	{
		if (bThrow == true)
		{
			UE_LOGF(LogUObjectGlobals, Error, " %ls %ls ", *FString::Printf(TEXT("Can't find '%s' in configuration file section=%s key=%s"), *ObjectReference, *Section, *Key), **IniFilename);
		}
	}

	return Output;
}

const FString* GetIniFilenameFromObjectsReference(const FString& Name)
{
	// See if the name is specified in the .ini file.
	if (FCString::Strnicmp(*Name, TEXT("engine-ini:"), FCString::Strlen(TEXT("engine-ini:"))) == 0)
	{
		return &GEngineIni;
	}
	else if (FCString::Strnicmp(*Name, TEXT("game-ini:"), FCString::Strlen(TEXT("game-ini:"))) == 0)
	{
		return &GGameIni;
	}
	else if (FCString::Strnicmp(*Name, TEXT("input-ini:"), FCString::Strlen(TEXT("input-ini:"))) == 0)
	{
		return &GInputIni;
	}
	else if (FCString::Strnicmp(*Name, TEXT("editor-ini:"), FCString::Strlen(TEXT("editor-ini:"))) == 0)
	{
		return &GEditorIni;
	}

	return nullptr;
}

//
// Resolve a package and name.
//
bool ResolveName2(UObject*& InPackage, FStringBuilderBase& InOutName, bool Create, bool Throw, uint32 LoadFlags, const FLinkerInstancingContext* InstancingContext)
{
	// Strip off the object class.
	ConstructorHelpers_StripObjectClass2( InOutName );

	// if you're attempting to find an object in any package using a dotted name that isn't fully
	// qualified (such as ObjectName.SubobjectName - notice no package name there), you normally call
	// StaticFindObject and pass in ANY_PACKAGE as the value for InPackage.  When StaticFindObject calls ResolveName,
	// it passes NULL as the value for InPackage, rather than ANY_PACKAGE.  As a result, unless the first chunk of the
	// dotted name (i.e. ObjectName from the above example) is a UPackage, the object will not be found.  So here we attempt
	// to detect when this has happened - if we aren't attempting to create a package, and a UPackage with the specified
	// name couldn't be found, pass in ANY_PACKAGE as the value for InPackage to the call to FindObject<UObject>().
	bool bSubobjectPath = false;

	// Handle specified packages.
	constexpr FAsciiSet Delimiters = FAsciiSet(".") + (char)SUBOBJECT_DELIMITER_CHAR;
	while (true)
	{
		const TCHAR* DelimiterOrEnd = FAsciiSet::FindFirstOrEnd(*InOutName, Delimiters);

		if (*DelimiterOrEnd == '\0')
		{
			return true;
		}
		else if (*DelimiterOrEnd == SUBOBJECT_DELIMITER_CHAR)
		{
			bSubobjectPath = true;
			Create         = false;
		}
		
		const int32 DotIndex = static_cast<int32>(DelimiterOrEnd - *InOutName);

		TStringBuilder<FName::StringBufferSize> PartialName;
		PartialName.Append(*InOutName, DotIndex);

		bool bIsScriptPackage = false;
		if (!InPackage)
		{
			if (!bSubobjectPath)
			{
				// In case this is a short script package name, convert to long name before passing to CreatePackage/FindObject.
				FName* ScriptPackageName = FPackageName::FindScriptPackageName(*PartialName);
				if (ScriptPackageName)
				{
					ScriptPackageName->ToString(PartialName);
				}
				bIsScriptPackage = ScriptPackageName || FPackageName::IsScriptPackage(FStringView(PartialName));
			}

			// Process any package redirects before calling CreatePackage/FindObject
			{
				const FCoreRedirectObjectName NewPackageName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(NAME_None, NAME_None, *PartialName));
				NewPackageName.PackageName.ToString(PartialName);
			}
		}

		// Only long package names are allowed so don't even attempt to create one because whatever the name represents
		// it's not a valid package name anyway.
		
		if (!Create)
		{
			UObject* NewPackage = InPackage ? nullptr : FindObject<UPackage>( InPackage, *PartialName );
			if( !NewPackage )
			{
				if (InPackage)
				{
					NewPackage = FindObject<UObject>(InPackage, *PartialName);
				}
				else
				{
					NewPackage = FindFirstObject<UObject>(*PartialName, EFindFirstObjectOptions::NativeFirst, ELogVerbosity::Warning, TEXT("ResolveName"));
				}
				if( !NewPackage )
				{
					return bSubobjectPath;
				}
			}
			InPackage = NewPackage;
		}
		else if (!FPackageName::IsShortPackageName(FStringView(PartialName)))
		{
			// Try to find the package in memory first, should be faster than attempting to load or create
			InPackage = InPackage ? nullptr : StaticFindObjectFast(UPackage::StaticClass(), InPackage, *PartialName);
			
			if (!bIsScriptPackage && !InPackage)
			{
				InPackage = LoadPackage(Cast<UPackage>(InPackage), *PartialName, LoadFlags, nullptr, InstancingContext);
			}

			if (!InPackage)
			{
				InPackage = CreatePackage(*PartialName);
				if (bIsScriptPackage)
				{
					Cast<UPackage>(InPackage)->SetPackageFlags(PKG_CompiledIn);
				}
			}

			check(InPackage);
		}
		InOutName.RemoveAt(0, DotIndex + 1);
	}
}

bool ResolveName(UObject*& InPackage, FString& InOutName, bool Create, bool Throw, uint32 LoadFlags /*= LOAD_None*/, const FLinkerInstancingContext* InstancingContext)
{
	TStringBuilder<512> Builder;
	Builder.Append(InOutName);
	bool Result = ResolveName2(InPackage, Builder, Create, Throw, LoadFlags, InstancingContext);
	InOutName = Builder;
	return Result;
}

bool ParseObject( const TCHAR* Stream, const TCHAR* Match, UClass* Class, UObject*& DestRes, UObject* InParent, EParseObjectLoadingPolicy LoadingPolicy, bool* bInvalidObject )
{
	if (!GAllowParseObjectLoading)
	{
		LoadingPolicy = EParseObjectLoadingPolicy::Find;
	}

	TCHAR TempStr[1024];
	if (!FParse::Value(Stream, Match, TempStr, UE_ARRAY_COUNT(TempStr)))
	{
		// Match not found
		return false;
	}
	else if (FCString::Stricmp(TempStr, TEXT("NONE")) == 0)
	{
		// Match found, object explicit set to be None
		DestRes = nullptr;
		return true;
	}
	else
	{
		auto ResolveObjectImpl = [Class, InParent, LoadingPolicy](const TCHAR* ObjNameOrPathName)
		{
			if (FPackageName::IsValidObjectPath(ObjNameOrPathName))
			{
				// A fully qualified object path can be resolved with no parent
				return LoadingPolicy == EParseObjectLoadingPolicy::FindOrLoad
					? StaticLoadObject(Class, nullptr, ObjNameOrPathName)
					: StaticFindObject(Class, nullptr, ObjNameOrPathName);
			}
			else if (InParent)
			{
				// Try to find the object within its parent
				return StaticFindObject(Class, InParent, ObjNameOrPathName);
			}
			else
			{
				// Try to find first object matching the provided name
				return StaticFindFirstObject(Class, ObjNameOrPathName, EFindFirstObjectOptions::EnsureIfAmbiguous);
			}
		};

		UObject* Res = nullptr;
		// Look this object up.
		Res = ResolveObjectImpl(TempStr);
		if (!Res)
		{
			if (Class->IsChildOf<UClass>())
			{
				FString RedirectedObjectName = FLinkerLoad::FindNewPathNameForClass(TempStr, false);
				if (!RedirectedObjectName.IsEmpty())
				{
					Res = ResolveObjectImpl(*RedirectedObjectName);
				}
			}

			if (!Res)
			{
				// Match found, object not found
				if (bInvalidObject)
				{
					*bInvalidObject = true;
				}
				return false;
			}
		}

		// Match found, object found
		DestRes = Res;
		return true;
	}
}

UE_TRACE_EVENT_BEGIN(Cpu, LoadObject, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, AssetPath)
UE_TRACE_EVENT_END()

UObject* StaticLoadObjectInternal(UClass* ObjectClass, UObject* InOuter, FStringView InName, FStringView Filename, uint32 LoadFlags, UPackageMap* Sandbox, bool bAllowObjectReconciliation, const FLinkerInstancingContext* InstancingContext)
{
#if CPUPROFILERTRACE_ENABLED
	UE_TRACE_LOG_SCOPED_T(Cpu, LoadObject, CpuChannel)
		<< LoadObject.AssetPath(InName.GetData(), InName.Len());
#endif // CPUPROFILERTRACE_ENABLED
	SCOPED_NAMED_EVENT(StaticLoadObjectInternal, FColor::Red);

	FScopedLoadingState ScopedLoadingState = FScopedLoadingState(FString(InName));
	TStringBuilder<512> StrName;
	StrName = InName;
	UObject* Result = nullptr;
	const bool bContainsObjectName = UE::String::FindFirstChar(InName, TEXT('.')) != INDEX_NONE;

	uint32 ResolveFlags = LoadFlags;
	ResolveFlags &= LOAD_EditorOnly | LOAD_NoVerify | LOAD_DeferDependencyLoads;
	ResolveFlags |= LOAD_Quiet | LOAD_NoWarn;
	// break up the name into packages, returning the innermost name and its outer
	ResolveName2(InOuter, StrName, true, true, ResolveFlags, InstancingContext);

	if (InOuter)
	{
		// If we have a full UObject name then attempt to find the object in memory first,
		if (bAllowObjectReconciliation && (bContainsObjectName
#if WITH_EDITOR
			|| GIsImportingT3D
#endif
			))
		{
			Result = StaticFindObjectFast(ObjectClass, InOuter, *StrName);
			if (Result && Result->HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_WillBeLoaded))
			{
				// Object needs loading so load it before returning
				Result = nullptr;
			}
		}

		if (!Result)
		{
			if (!InOuter->GetOutermost()->HasAnyPackageFlags(PKG_CompiledIn))
			{
				// now that we have one asset per package, we load the entire package whenever a single object is requested
				LoadPackage(NULL, *InOuter->GetOutermost()->GetName(), LoadFlags & ~LOAD_Verify, nullptr, InstancingContext);
			}

			// now, find the object in the package
			Result = StaticFindObjectFast(ObjectClass, InOuter, *StrName);

			// If the object was not found, check for a redirector and follow it if the class matches
			if (!Result && !(LoadFlags & LOAD_NoRedirects))
			{
				UObjectRedirector* Redirector = FindObjectFast<UObjectRedirector>(InOuter, *StrName);
				if (Redirector && Redirector->DestinationObject && Redirector->DestinationObject->IsA(ObjectClass ? ObjectClass : UObject::StaticClass()))
				{
					if (UE::GC::GIsIncrementalReachabilityPending)
					{
						UE::GC::MarkAsReachable(Redirector);
						UE::GC::MarkAsReachable(Redirector->DestinationObject);
					}
					return Redirector->DestinationObject;
				}
			}
		}
	}

	if (!Result && !bContainsObjectName)
	{
		// Assume that the object we're trying to load is the main asset inside of the package 
		// which usually has the same name as the short package name.
		StrName = InName;
		StrName += TEXT(".");
		StrName += FPackageName::GetShortName(InName);
		Result = StaticLoadObjectInternal(ObjectClass, InOuter, StrName, Filename, LoadFlags, Sandbox, bAllowObjectReconciliation, InstancingContext);
	}

	if (Result && UE::GC::GIsIncrementalReachabilityPending)
	{
		UE::GC::MarkAsReachable(Result);
	}
	return Result;
}

UObject* StaticLoadObject(UClass* ObjectClass, UObject* InOuter, FStringView InName, FStringView Filename, uint32 LoadFlags, UPackageMap* Sandbox, bool bAllowObjectReconciliation, const FLinkerInstancingContext* InstancingContext)
{
	UObject* Result = StaticLoadObjectInternal(ObjectClass, InOuter, InName, Filename, LoadFlags | LOAD_NoWarn, Sandbox, bAllowObjectReconciliation, InstancingContext);
	if (!Result)
	{
		TStringBuilder<512> ObjectName;
		ObjectName = InName;
		ResolveName2(InOuter, ObjectName, true, true, LoadFlags & LOAD_EditorOnly, InstancingContext);

		bool bWarnIfMissing = (LoadFlags & (LOAD_NoWarn | LOAD_Quiet)) == 0;
		auto IsKnownMissing = [](UObject* Obj) { return Obj && FLinkerLoad::IsKnownMissingPackage(FName(*Obj->GetPathName())); };
		if (bWarnIfMissing && !IsKnownMissing(InOuter))
		{
			// we haven't created or found the object, error
			TStringBuilder<128> ObjectPath;
			if (ObjectClass)
			{
				ObjectPath << ObjectClass->GetFName() << TEXT(" ");
			}
			if (InOuter)
			{
				InOuter->GetPathName(nullptr, ObjectPath);
				ObjectPath << TEXT(".");
			}
			ObjectPath << ObjectName;

			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("ObjectPath"), FText::FromStringView(ObjectPath.ToView()));
			const FString Error = FText::Format(NSLOCTEXT("Core", "ObjectNotFound", "Failed to find object '{ObjectPath}'"), Arguments).ToString();
			SafeLoadError(InOuter, LoadFlags, *Error);

			if (InOuter && !InOuter->HasAnyFlags(RF_WasLoaded))
			{
				// Stop future repeated warnings
				FLinkerLoad::AddKnownMissingPackage(FName(*InOuter->GetPathName()));
			}
		}
	}
	return Result;
}

//
// Load a class.
//
UClass* StaticLoadClass( UClass* BaseClass, UObject* InOuter, FStringView InName, FStringView Filename, uint32 LoadFlags, UPackageMap* Sandbox )
{
	check(BaseClass);

	UClass* Class = LoadObject<UClass>( InOuter, InName, Filename, LoadFlags, Sandbox );
	if( Class && !Class->IsChildOf(BaseClass) )
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("ClassName"), FText::FromString( Class->GetFullName() ));
		Arguments.Add(TEXT("BaseClassName"), FText::FromString( BaseClass->GetFullName() ));
		const FString Error = FText::Format( NSLOCTEXT( "Core", "LoadClassMismatch", "{ClassName} is not a child class of {BaseClassName}" ), Arguments ).ToString();
		SafeLoadError(InOuter, LoadFlags, *Error);

		// return NULL class due to error
		Class = NULL;
	}
	return Class;
}

UObject* StaticLoadAsset(UClass* Class, FTopLevelAssetPath InPath, uint32 LoadFlags, const FLinkerInstancingContext* InstancingContext)
{
	// @todo: This could call StaticLoadObjectInternal directly with some refactoring

	TStringBuilder<256> ObjectNameString;
	InPath.AppendString(ObjectNameString);
	return StaticLoadObject(Class, nullptr, *ObjectNameString, nullptr, LoadFlags, nullptr, true, InstancingContext);
}

int32 LoadAssetAsync(FTopLevelAssetPath InAssetPath, FLoadAssetAsyncDelegate InCompletionDelegate, FLoadAssetAsyncOptionalParams InOptionalParams)
{
	// Asset paths should always have a valid package
	FPackagePath PackagePath = FPackagePath::FromPackageNameChecked(InAssetPath.GetPackageName());
	uint32 LoadFlags = InOptionalParams.LoadFlags;
	
	FLoadPackageAsyncOptionalParams PackageParams{
		.ProgressDelegate = InOptionalParams.ProgressDelegate,
		.PackagePriority = InOptionalParams.PackagePriority,
		.InstancingContext = InOptionalParams.InstancingContext,
		.LoadFlags = LoadFlags
	};

	PackageParams.CompletionDelegate = MakeUnique<FLoadPackageAsyncDelegate>(FLoadPackageAsyncDelegate::CreateLambda(
		[InAssetPath, LoadFlags, CompletionDelegate = MoveTemp(InCompletionDelegate)](const FName& LoadedPackageName, UPackage* LoadedPackage, EAsyncLoadingResult::Type Result) mutable
		{
			UObject* LoadedObject = nullptr;
			if (Result == EAsyncLoadingResult::Succeeded && LoadedPackage)
			{
				LoadedObject = StaticFindObjectFast(UObject::StaticClass(), LoadedPackage, InAssetPath.GetAssetName());

				// Package loaded but object was not found inside it, failure
				if (!LoadedObject)
				{
					Result = EAsyncLoadingResult::Failed;
				}
				else if (!(LoadFlags & LOAD_NoRedirects))
				{
					// Follow any object redirectors
					while (UObjectRedirector* Redirector = Cast<UObjectRedirector>(LoadedObject))
					{
						LoadedObject = Redirector->DestinationObject;
						if (UE::GC::GIsIncrementalReachabilityPending && LoadedObject)
						{
							// This is handled in StaticFindObjectFast for the redirector itself
							UE::GC::MarkAsReachable(LoadedObject);
						}
					}
				}
			}

			CompletionDelegate.ExecuteIfBound(InAssetPath, LoadedObject, Result);
		}));

	return LoadPackageAsync(PackagePath, MoveTemp(PackageParams));
}


#if WITH_EDITOR
#include "Containers/StackTracker.h"
class FDiffFileArchive : public FArchiveProxy
{
private:
	FArchive* DiffArchive;
	FArchive* InnerArchivePtr;
	bool bDisable;
	TArray<FName> DebugDataStack;
public:
	FDiffFileArchive(FArchive* InDiffArchive, FArchive* InInnerArchive) : FArchiveProxy(*InInnerArchive), DiffArchive(InDiffArchive), InnerArchivePtr(InInnerArchive), bDisable(false)
	{
	}

	~FDiffFileArchive()
	{
		if (InnerArchivePtr)
			delete InnerArchivePtr;

		if (DiffArchive)
			delete DiffArchive;
	}

	virtual void PushDebugDataString(const FName& DebugData) override
	{
		FArchiveProxy::PushDebugDataString(DebugData);
		DebugDataStack.Add(DebugData);
	}

	virtual void PopDebugDataString() override
	{
		FArchiveProxy::PopDebugDataString();
		DebugDataStack.Pop();
	}

	virtual void Serialize(void* V, int64 Length) override
	{
		int64 Pos = InnerArchive.Tell();
		InnerArchive.Serialize(V, Length);

		if (DiffArchive && !bDisable)
		{
			TArray64<uint8> Data;
			Data.AddUninitialized(Length);
			DiffArchive->Seek(Pos);
			DiffArchive->Serialize(Data.GetData(), Length);

			if (FMemory::Memcmp((const void*)Data.GetData(), V, Length) != 0)
			{
				// get the calls debug callstack and 
				FString DebugStackString;
				for (const FName& DebugData : DebugDataStack)
				{
					DebugStackString += DebugData.ToString();
					DebugStackString += TEXT("->");
				}

				UE_LOGF(LogUObjectGlobals, Warning, "Diff cooked package archive recognized a difference %lld Filename %ls", Pos, *InnerArchive.GetArchiveName());

				UE_LOGF(LogUObjectGlobals, Warning, "debug stack %ls", *DebugStackString);


				FStackTracker TempTracker(nullptr, nullptr, nullptr, true);
				TempTracker.CaptureStackTrace(1);
				TempTracker.DumpStackTraces(0, *GLog);
				TempTracker.ResetTracking();

				// only log one message per archive, from this point the entire package is probably messed up
				bDisable = true;

				static int i = 0;
				i++;
			}
		}
	}
};

#endif

// Temporary load counter for the game thread, used mostly for checking if we're still loading
// @todo: remove this in the new loader
static int32 GGameThreadLoadCounter = 0;
static int32 GGameThreadEndLoadCounter = -1;

UE_TRACE_EVENT_BEGIN(CUSTOM_LOADTIMER_LOG, LoadPackageInternal, NoSync)
	UE_TRACE_EVENT_FIELD(UE::Trace::WideString, PackageName)
UE_TRACE_EVENT_END()

UPackage* LoadPackageInternal(UPackage* InOuter, const FPackagePath& PackagePath, uint32 LoadFlags, FLinkerLoad* ImportLinker, FArchive* InReaderOverride,
	const FLinkerInstancingContext* InstancingContext, const FPackagePath* DiffPackagePath)
{
	DECLARE_SCOPE_CYCLE_COUNTER(TEXT("LoadPackageInternal"), STAT_LoadPackageInternal, STATGROUP_ObjectVerbose);

	FString TracePackageName;
#if LOADTIMEPROFILERTRACE_ENABLED
	if (UE_TRACE_CHANNELEXPR_IS_ENABLED(LoadTimeChannel))
	{
		TracePackageName = PackagePath.GetPackageNameOrFallback();
	}
#endif
	SCOPED_CUSTOM_LOADTIMER(LoadPackageInternal)
		ADD_CUSTOM_LOADTIMER_META(LoadPackageInternal, PackageName, *TracePackageName);

	if (PackagePath.IsEmpty())
	{
		UE_LOGF(LogUObjectGlobals, Warning, "Attempted to LoadPackage from empty PackagePath.");
		return nullptr;
	}

	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();

	checkf(!InOuter || !InOuter->GetOuter(), TEXT("Loading into subpackages is not implemented.")); // Subpackages are no longer supported in UE
	FName PackageName(InOuter ? InOuter->GetFName() : PackagePath.GetPackageFName());
	if (PackageName.IsNone())
	{
		UE_LOGF(LogUObjectGlobals, Warning, "Attempted to LoadPackage from non-mounted path %ls. This is not supported.", *PackagePath.GetDebugName());
		return nullptr;
	}

	// This delegate is not thread-safe and the subscribers are mostly interested by sync loads
	// that might stall the game thread anyway. So for now, do not broadcast when sync loading
	// from the loading thread.
	if (IsInGameThread() && FCoreDelegates::OnSyncLoadPackage.IsBound())
	{
		FCoreDelegates::OnSyncLoadPackage.Broadcast(PackageName.ToString());
	}

	ThreadContext.SyncLoadUsingAsyncLoaderCount++;
	EPackageFlags PackageFlags = PKG_None;
#if WITH_EDITOR
	// If we are loading a package for diffing, set the package flag
	if (LoadFlags & LOAD_ForDiff)
	{
		PackageFlags |= PKG_ForDiffing;
	}
	if ((!FApp::IsGame() || GIsEditor) && (LoadFlags & LOAD_PackageForPIE) != 0)
	{
		PackageFlags |= PKG_PlayInEditor;
	}
#endif
	FLoadPackageAsyncOptionalParams OptionalParams
	{
		.CustomPackageName = PackageName,
		.PackageFlags = PackageFlags,
		.PackagePriority = INT32_MAX,
		.InstancingContext = InstancingContext,
		.LoadFlags = LoadFlags
	};
	int32 RequestID = LoadPackageAsync(PackagePath, MoveTemp(OptionalParams));

	if (RequestID != INDEX_NONE)
	{
		UE_SCOPED_IO_ACTIVITY(*WriteToString<512>(TEXT("Sync "), PackagePath.GetDebugName()));
		FlushAsyncLoading(RequestID);
	}
	ThreadContext.SyncLoadUsingAsyncLoaderCount--;

	if (InOuter)
	{
		return InOuter;
	}

	UPackage* Result = FindObjectFast<UPackage>(nullptr, PackageName);
	if (!Result)
	{
		// Might have been redirected
		const FCoreRedirectObjectName NewPackageName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Package, FCoreRedirectObjectName(NAME_None, NAME_None, PackageName));
		if (NewPackageName.PackageName != PackageName)
		{
			Result = FindObjectFast<UPackage>(nullptr, NewPackageName.PackageName);
		}
	}

	return Result;
}

UPackage* LoadPackage(UPackage* InOuter, const TCHAR* InLongPackageNameOrFilename, uint32 LoadFlags, FArchive* InReaderOverride, const FLinkerInstancingContext* InstancingContext)
{
	FPackagePath PackagePath;
	FPackagePath* DiffPackagePathPtr = nullptr;

#if WITH_EDITOR
	FPackagePath DiffPackagePath;
	if (LoadFlags & LOAD_ForFileDiff)
	{
		FString TempFilenames = InLongPackageNameOrFilename;
		FString FileToLoad;
		FString DiffFileToLoad;
		ensure(TempFilenames.Split(TEXT(";"), &FileToLoad, &DiffFileToLoad, ESearchCase::CaseSensitive));
		PackagePath = FPackagePath::FromLocalPath(FileToLoad);
		DiffPackagePath = FPackagePath::FromLocalPath(DiffFileToLoad);
		DiffPackagePathPtr = &DiffPackagePath;
	}
	else
#endif
	if (InLongPackageNameOrFilename && InLongPackageNameOrFilename[0] != '\0')
	{
		bool bWarnIfMissing = (LoadFlags & (LOAD_NoWarn | LOAD_Quiet)) == 0;

		FString BufferName;
		// Make sure we're trying to load long package names only.
		if (FPackageName::IsShortPackageName(FStringView(InLongPackageNameOrFilename)))
		{
			BufferName = InLongPackageNameOrFilename;
			FName* ScriptPackageName = FPackageName::FindScriptPackageName(*BufferName);
			if (ScriptPackageName)
			{
				UE_LOGF(LogUObjectGlobals, Warning, "LoadPackage: %ls is a short script package name.", InLongPackageNameOrFilename);
				BufferName = ScriptPackageName->ToString();
				InLongPackageNameOrFilename = *BufferName;
			}
			else if (FPackageName::SearchForPackageOnDisk(BufferName, &BufferName))
			{
				InLongPackageNameOrFilename = *BufferName;
			}
			else
			{
				UE_CLOGF(bWarnIfMissing, LogUObjectGlobals, Warning, "LoadPackage can't find package %ls.", InLongPackageNameOrFilename);
				return nullptr;
			}
		}

		if (!FPackagePath::TryFromMountedName(InLongPackageNameOrFilename, PackagePath))
		{
			UE_CLOGF(bWarnIfMissing, LogUObjectGlobals, Warning, "LoadPackage can't find package %ls.", InLongPackageNameOrFilename);
			return nullptr;
		}
	}
	else if (InOuter)
	{
		PackagePath = FPackagePath::FromPackageNameChecked(InOuter->GetName());
	}
	else
	{
		UE_LOGF(LogUObjectGlobals, Warning, "Empty name passed to LoadPackage.");
		return nullptr;
	}
	return LoadPackage(InOuter, PackagePath, LoadFlags, InReaderOverride, InstancingContext, DiffPackagePathPtr);
}

UPackage* LoadPackage(UPackage* InOuter, const FPackagePath& PackagePath, uint32 LoadFlags, FArchive* InReaderOverride, const FLinkerInstancingContext* InstancingContext, const FPackagePath* DiffPackagePath)
{
	COOK_STAT(FScopedDurationTimer LoadTimer(LoadPackageStats::LoadPackageTimeSec));
	// Change to 1 if you want more detailed stats for loading packages, but at the cost of adding dynamic stats.
#if	STATS && 0
	static FString Package = "Package";
	const FString LongName = Package / PackagePath.GetPackageNameOrFallback();
	const TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_UObjects>( LongName );
	FScopeCycleCounter CycleCounter( StatId );
#endif // STATS

	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH_FNAME(PackagePath.GetPackageFName(), ELLMTagSet::Assets);
	LLM_SCOPE_DYNAMIC_STAT_OBJECTPATH(UPackage::StaticClass(), ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(NAME_None, UPackage::StaticClass()->GetFName(), PackagePath.GetPackageFName());
	TRACE_LOADTIME_REQUEST_GROUP_SCOPE(TEXT("SyncLoad - %s"), *PackagePath.GetDebugName());
	
	// if this is a supported asset, it should be loaded fully rather than just for diffing
	if(!ensure(!(LoadFlags & LOAD_ForDiff) || FPackageName::IsTempPackage(PackagePath.GetPackageName())))
	{
		// clear LOAD_ForDiff
		LoadFlags &= ~LOAD_ForDiff;
	}
	return LoadPackageInternal(InOuter, PackagePath, LoadFlags, /*ImportLinker =*/ nullptr, InReaderOverride, InstancingContext, DiffPackagePath);
}

/**
 * Returns whether we are currently loading a package (sync or async)
 *
 * @return true if we are loading a package, false otherwise
 */
bool IsLoading()
{
	return GGameThreadLoadCounter > 0;
}

//
// Begin loading packages.
//warning: Objects may not be destroyed between BeginLoad/EndLoad calls.
//
void BeginLoad(FUObjectSerializeContext* LoadContext, const TCHAR* DebugContext)
{
	check(LoadContext);
	if (!LoadContext->HasStartedLoading() && !IsInAsyncLoadingThread())
	{
		if (IsAsyncLoading() && (DebugContext != nullptr))
		{
			UE_LOGF(LogUObjectGlobals, Log, "BeginLoad(%ls) is flushing async loading", DebugContext);
		}

		// Make sure we're finishing up all pending async loads, and trigger texture streaming next tick if necessary.
		FlushAsyncLoading();
	}
	if (IsInGameThread() && !IsInAsyncLoadingThread())
	{
		GGameThreadLoadCounter++;
	}

	LoadContext->IncrementBeginLoadCount();
}

// Sort objects by linker name and file offset
struct FCompareUObjectByLinkerAndOffset
{
	FORCEINLINE bool operator()( const UObject& A, const UObject &B ) const
	{
		FLinker* LinkerA = A.GetLinker();
		FLinker* LinkerB = B.GetLinker();

		// Both objects have linkers.
		if( LinkerA && LinkerB )
		{
			// Identical linkers, sort by offset in file.
			if( LinkerA == LinkerB )
			{
				FObjectExport& ExportA = LinkerA->ExportMap[ A.GetLinkerIndex() ];
				FObjectExport& ExportB = LinkerB->ExportMap[ B.GetLinkerIndex() ];
				return ExportA.SerialOffset < ExportB.SerialOffset;
			}
			// Sort by pointer address.
			else
			{
				return false;
			}
		}
		// Neither objects have a linker, don't do anything.
		else if( LinkerA == LinkerB )
		{
			return false;
		}
		// Sort objects with linkers vs. objects without
		else
		{
			return (LinkerA != NULL);
		}
	}
};

//
// End loading packages.
//
void EndLoad(FUObjectSerializeContext* LoadContext, TArray<UPackage*>* OutLoadedPackages)
{
	if (OutLoadedPackages)
	{
		OutLoadedPackages->Reset();
	}
	check(LoadContext);

	if (IsInAsyncLoadingThread())
	{
		LoadContext->DecrementBeginLoadCount();
		return;
	}
	SCOPED_LOADTIMER(EndLoad);

#if WITH_EDITOR
	TOptional<FScopedSlowTask> SlowTask;
	if (ShouldCreateThrottledSlowTask())
	{
		static const FText PostLoadText = NSLOCTEXT("Core", "PerformingPostLoad", "Performing post-load...");
		SlowTask.Emplace(0.0f, PostLoadText);
	}

	int32 NumObjectsLoaded = 0, NumObjectsFound = 0;
	TSet<UObject*> AssetsLoaded;
#endif
	TSet<UObject*> HotfixableAssetsLoaded;
	TSet<UPackage*> LoadedPackages;

	while (LoadContext->DecrementBeginLoadCount() == 0 && (LoadContext->HasLoadedObjects() || LoadContext->HasPendingImportsOrForcedExports()))
	{
		// The time tracker keeps track of time spent in EndLoad.
		FExclusiveLoadPackageTimeTracker::FScopedEndLoadTracker Tracker;

		// Make sure we're not recursively calling EndLoad as e.g. loading a config file could cause
		// BeginLoad/EndLoad to be called.
		LoadContext->IncrementBeginLoadCount();

		// Temporary list of loaded objects as GObjLoaded might expand during iteration.
		TArray<UObject*> ObjLoaded;
		TSet<FLinkerLoad*> LoadedLinkers;
		while (LoadContext->HasLoadedObjects())
		{
			// Accumulate till GObjLoaded no longer increases.
			LoadContext->AppendLoadedObjectsAndEmpty(ObjLoaded);

			// Sort by Filename and Offset.
			ObjLoaded.StableSort(FCompareUObjectByLinkerAndOffset());

			UE_MULTI_SCOPED_COOK_STAT_INIT()
			// Finish loading everything.
			{
				SCOPED_LOADTIMER(PreLoadAndSerialize);
				UE_TRACK_REFERENCING_PACKAGE_DELAYED_SCOPED(AccessRefScope, PackageAccessTrackingOps::NAME_PreLoad);
				for (int32 i = 0; i < ObjLoaded.Num(); i++)
				{
					// Preload.
					UObject* Obj = ObjLoaded[i];
					if (Obj->HasAnyFlags(RF_NeedLoad))
					{
						FLinkerLoad* Linker = Obj->GetLinker();
						check(Linker);

						UPackage* Package = Linker->LinkerRoot;
						check(Package);

						UE_MULTI_SCOPED_COOK_STAT(Package->GetFName(), EPackageEventStatType::LoadPackage);
						UE_TRACK_REFERENCING_PACKAGE_DELAYED(AccessRefScope, Package);
#if WITH_EDITOR
						if (SlowTask)
						{
							// Don't report progress but gives a chance to tick slate to improve the responsiveness of the 
							// progress bar being shown. We expect slate to be ticked at regular intervals throughout the loading.
							SlowTask->TickProgress();
						}
#endif
						Linker->Preload(Obj);
					}
				}
				UE_MULTI_SCOPED_COOK_STAT_RESET();
			}

			// Start over again as new objects have been loaded that need to have "Preload" called on them before
			// we can safely PostLoad them.
			if (LoadContext->HasLoadedObjects())
			{
				continue;
			}

#if WITH_EDITOR
			if (SlowTask)
			{
				SlowTask->CompletedWork = SlowTask->TotalAmountOfWork;
				SlowTask->TotalAmountOfWork += static_cast<float>(ObjLoaded.Num());
				SlowTask->CurrentFrameScope = 0;
			}

			for (int32 i = 0; i < ObjLoaded.Num(); i++)
			{
				UObject* Obj = ObjLoaded[i];
				if (OutLoadedPackages)
				{
					LoadedPackages.Add(Obj->GetPackage());
				}
				if (GIsEditor && Obj->GetLinker())
				{
					LoadedLinkers.Add(Obj->GetLinker());
				}
			}
#endif		

			{
				SCOPED_LOADTIMER(PostLoad);
				// set this so that we can perform certain operations in which are only safe once all objects have been de-serialized.
				TGuardValue<bool> GuardIsRoutingPostLoad(FUObjectThreadContext::Get().IsRoutingPostLoad, true);
				FLinkerLoad* VisitedLinkerLoad = nullptr;
				// Postload objects.
				for (int32 i = 0; i < ObjLoaded.Num(); i++)
				{
					UObject* Obj = ObjLoaded[i];
					check(Obj);
#if WITH_EDITOR
					if (SlowTask)
					{
						static const FTextFormat FinalizingTextFormat = NSLOCTEXT("Core", "FinalizingUObject", "Finalizing load of {0}");
						SlowTask->EnterProgressFrame(1, FText::Format(FinalizingTextFormat, FText::FromString(Obj->GetName())));
					}
#endif
					
					FLinkerLoad* LinkerLoad = Obj->GetLinker();
					if (LinkerLoad && LinkerLoad != VisitedLinkerLoad)
					{
						LinkerLoad->FinishExternalReadDependencies(0.0);
						VisitedLinkerLoad = LinkerLoad;
					}
					
					Obj->ConditionalPostLoad();
				}
			}

			{
				// Additional operation performed by classes (used for non-native initialization)
				SCOPED_LOADTIMER(PostLoadInstance);
				for (UObject* Obj : ObjLoaded)
				{
					UE_MULTI_SCOPED_COOK_STAT(Obj->GetPackage()->GetFName(), EPackageEventStatType::LoadPackage);
					UClass* ObjClass = Obj->GetClass();
					check(ObjClass);
					ObjClass->PostLoadInstance(Obj);
				}
				UE_MULTI_SCOPED_COOK_STAT_RESET();
			}

			// Create clusters after all objects have been loaded
			if (FPlatformProperties::RequiresCookedData() && !GIsInitialLoad && GCreateGCClusters && GAssetClustreringEnabled && !GUObjectArray.IsOpenForDisregardForGC())
			{
				for (UObject* Obj : ObjLoaded)
				{
					check(Obj);
					if (Obj->CanBeClusterRoot())
					{
						Obj->CreateCluster();
					}
				}
			}

			// Schedule asset loaded callbacks for later
			for( int32 CurObjIndex=0; CurObjIndex<ObjLoaded.Num(); CurObjIndex++ )
			{
				UObject* Obj = ObjLoaded[CurObjIndex];
				check(Obj);
#if WITH_EDITOR
				if ( Obj->IsAsset() )
				{
					AssetsLoaded.Add(Obj);
				}
#endif	// WITH_EDITOR

				if (FCoreUObjectDelegates::IsHotfixableAssetClass(Obj->GetClass()))
				{
					// If this is a sync load during async loading. Async loading will handle this.
					if (!FUObjectThreadContext::Get().AsyncPackageLoader)
					{
						HotfixableAssetsLoaded.Add(Obj);
					}
				}
			}

			// Empty array before next iteration as we finished postloading all objects.
			ObjLoaded.Reset();
		}
		
		if ( GIsEditor && LoadedLinkers.Num() > 0 )
		{
			for (FLinkerLoad* LoadedLinker : LoadedLinkers)
			{
				check(LoadedLinker != nullptr);

				LoadedLinker->FlushCache();

				if (LoadedLinker->LinkerRoot != nullptr && !LoadedLinker->LinkerRoot->IsFullyLoaded())
				{
					bool bAllExportsCreated = true;
					for ( int32 ExportIndex = 0; ExportIndex < LoadedLinker->ExportMap.Num(); ExportIndex++ )
					{
						FObjectExport& Export = LoadedLinker->ExportMap[ExportIndex];
						if ( !Export.bForcedExport && Export.Object == nullptr )
						{
							bAllExportsCreated = false;
							break;
						}
					}

					if ( bAllExportsCreated )
					{
						LoadedLinker->LinkerRoot->MarkAsFullyLoaded();
					}
				}
			}
		}

		// Dissociate all linker import and forced export object references, since they
		// may be destroyed, causing their pointers to become invalid.
		FLinkerManager::Get().DissociateImportsAndForcedExports();

		// close any linkers' loaders that were requested to be closed once GObjBeginLoadCount goes to 0
		TArray<FLinkerLoad*> PackagesToClose;
		LoadContext->MoveDelayedLinkerClosePackages(PackagesToClose);
		for (FLinkerLoad* Linker : PackagesToClose)
		{
			if (Linker)
			{				
				if (Linker->HasLoader() && Linker->LinkerRoot)
				{
					ResetLoaders(Linker->LinkerRoot);
				}
				check(!Linker->HasLoader());				
			}
		}

		// If this is the first LoadPackage call, flush the BP queue
		if (GGameThreadLoadCounter < 2)
		{
			FBlueprintSupport::FlushReinstancingQueue();
		}
	}

	if (IsInGameThread())
	{
		GGameThreadLoadCounter--;
		check(GGameThreadLoadCounter >= 0);
	}

#if WITH_EDITOR
	// Now call asset loaded callbacks for anything that was loaded. We do this at the very end so any nested objects will load properly
	// Useful for updating UI such as ContentBrowser's loaded status.
	for (UObject* LoadedAsset : AssetsLoaded)
	{
		check(LoadedAsset);
		FCoreUObjectDelegates::OnAssetLoaded.Broadcast(LoadedAsset);
		
		if (SlowTask)
		{
			// Don't report progress but gives a chance to tick slate to improve the responsiveness of the 
			// progress bar being shown. We expect slate to be ticked at regular intervals throughout the loading.
			SlowTask->TickProgress();
		}
	}
#endif	// WITH_EDITOR

	for (UObject* HotfixableAsset : HotfixableAssetsLoaded)
	{
		check(HotfixableAsset);
		FCoreUObjectDelegates::OnHotfixableAssetLoaded.Broadcast(HotfixableAsset);
	}

	if (OutLoadedPackages)
	{
		OutLoadedPackages->Reserve(LoadedPackages.Num());
		for (UPackage* Package : LoadedPackages)
		{
			OutLoadedPackages->Add(Package);
		}
	}
}

void EndLoad(FUObjectSerializeContext* LoadContext)
{
	EndLoad(LoadContext, nullptr);
}


/*-----------------------------------------------------------------------------
	Object name functions.
-----------------------------------------------------------------------------*/

static TAtomic<int32> NameNumberUniqueIndex(MAX_int32 - 1000);

DEFINE_LOG_CATEGORY_STATIC(LogUniqueObjectName, Error, Error);

#define TRY_REUSE_NAMES UE_FNAME_OUTLINE_NUMBER

#if TRY_REUSE_NAMES
namespace NameReuse
{
	static bool GTryReuseNames = true;
	static FAutoConsoleVariableRef CVar_TryReuseNames(
		TEXT("UObject.ReuseNames"),
		GTryReuseNames,
		TEXT("Try to reuse object names to prevent growth of numbered name table.")
	);

	static int32 GNameRangeCycleCadence = 8;
	static FAutoConsoleVariableRef CVar_NameRangeCycleCadence(
		TEXT("UObject.NameRangeCycleCadence"),
		GNameRangeCycleCadence,
		TEXT("When we have created this many new names in the range-based allocator and the reuse range is exhausted, return to the start and try reusing existing names. Must be a power of 2.")
	);

	static int32 GNameRangeMaxIterations = 8;
	static FAutoConsoleVariableRef CVar_NameRangeMaxIterations(
		TEXT("UObject.NameRangeMaxIterations"),
		GNameRangeMaxIterations,
		TEXT("Max number of iterations of attempting to reuse old names before bailing and creating a new one.")
	);

	bool CanUseNameInOuter(UObject* TestParent, FName TestName)
	{
		return !DoesObjectPossiblyExist(TestParent, TestName);
	}

	// Pointers passed into this cache are converted to integers for storage and comparison.
	// We are ok with false caches misses when the address is reused for another object. 
	struct FRecentNameCache
	{
		static constexpr int32 MaxEntries = 64;
		static constexpr int32 MaxNamesPerEntry = 16;

		struct FEntry
		{
			FNameEntryId BaseName;
			TArray<TTuple<UPTRINT, FName>> Names;

			FEntry(FNameEntryId InBaseName)
				: BaseName(InBaseName)
			{
				LLM_SCOPE(ELLMTag::FName);
				Names.Reserve(MaxNamesPerEntry);
			}

			FEntry(FEntry&&) = default;
			FEntry& operator=(FEntry&&) = default;

			void Reset(FNameEntryId InBaseName)
			{
				BaseName = InBaseName;
				Names.Reset();
			}

			void Store(UPTRINT Parent, FName Name)
			{
				LLM_SCOPE(ELLMTag::FName);

				int32 Index = Names.IndexOfByPredicate([&](const TTuple<UPTRINT, FName>& Pair) { return Pair.Get<1>() == Name; });
				if (Index != INDEX_NONE)
				{
					Names.RemoveAt(Index, EAllowShrinking::No);
				}
				else
				{
					if (Names.Num() >= MaxNamesPerEntry)
					{
						Names.RemoveAt(0, EAllowShrinking::No);
					}
				}

				Names.Emplace(Parent, Name);
			}
		};

		TArray<FEntry> Entries;

		FRecentNameCache()
		{
		}

		FName Find(UObject* ForParent, FNameEntryId BaseId, FName BaseName)
		{
			FEntry* Entry = Entries.FindByPredicate([BaseId](const FEntry& Entry) { return Entry.BaseName == BaseId; });
			if (Entry) 
			{
				for (TTuple<UPTRINT, FName> Pair : Entry->Names)
				{
					if (Pair.Get<1>() != BaseName && Pair.Get<0>() != reinterpret_cast<UPTRINT>(ForParent))
					{
						if (CanUseNameInOuter(ForParent, Pair.Get<1>()))
						{
							return Pair.Get<1>();
						}
					}
				}
			}

			return FName();
		}

		void Store(UObject* Parent, FNameEntryId BaseId, FName UsedName)
		{
			LLM_SCOPE(ELLMTag::FName);

			FEntry* Entry = Entries.FindByPredicate([BaseId](const FEntry& Entry) { return Entry.BaseName == BaseId; });
			if (!Entry)
			{
				if (Entries.Num() < MaxEntries)
				{
					// Replace the oldest entry in the cache or add a new one
					Entry = &Entries.Emplace_GetRef(BaseId);
				}
				else
				{
					Entry = &Entries[0];

					UE_LOGF(LogUniqueObjectName, Log, "EVICT: %ls", *FName(Entry->BaseName, Entry->BaseName, NAME_NO_NUMBER_INTERNAL).ToString());
					Entry->Reset(BaseId);
				}

				UE_LOGF(LogUniqueObjectName, Log, "STORE: %ls", *FName(BaseId, BaseId, NAME_NO_NUMBER_INTERNAL).ToString());
			}

			Entry->Store(reinterpret_cast<UPTRINT>(Parent), UsedName);

			// Shift this entry to the end of the array as it's now the most recently used
			int32 Index = UE_PTRDIFF_TO_INT32(Entry - Entries.GetData());
			if (Index != Entries.Num() - 1)
			{
				FEntry Removed = MoveTemp(*Entry);
				Entries.RemoveAt(Index, EAllowShrinking::No);
				Entries.Add(MoveTemp(Removed));
			}
		}
	};

	static thread_local FRecentNameCache GRecentNameCache;

	struct FNameRangeCache
	{
		struct FNameRangeEntry
		{
			static const constexpr int32 FirstNewNumber = MAX_int32 - 1001;

			int32 NextNewNumber;
			int32 Iterator;

			FNameRangeEntry()
				: NextNewNumber(FirstNewNumber)
				, Iterator(FirstNewNumber - GNameRangeCycleCadence)
			{
			}

			FName AllocateName(UObject* Parent, FNameEntryId BaseId, FName BaseName)
			{
				LLM_SCOPE(ELLMTag::FName);

				checkSlow(BaseId == BaseName.GetComparisonIndex());
				int32 BaseNumber = BaseName.GetNumber();

				// Do we have any old numbers to try and reuse? 
				if (Iterator > NextNewNumber)
				{
					int32 Start = Iterator;
					int32 End = Start - GNameRangeMaxIterations;
					// Look for existing (name, number) pairs that are unused 
					for (int32 i = Start; i > NextNewNumber && i > End; --i)
					{
						if (i == BaseNumber)
						{
							continue;
						}

						FName TestName(BaseId, BaseId, i);
						if (CanUseNameInOuter(Parent, TestName))
						{
							UE_LOGF(LogUniqueObjectName, Log, "HIT: %ls %ls", *Parent->GetPathName(), *TestName.ToString());
							Iterator = i - 1;
							return TestName;
						}
					}

					if (Iterator == NextNewNumber)
					{
						// Failed to find a name to reuse, reset iterator to where we should reset
						Iterator = NextNewNumber - GNameRangeCycleCadence;
					}
				}

				// Start == Next or we fail to reuse an existing index
				for (int32 i = NextNewNumber; i > 0; --i)
				{
					if (i == BaseNumber)
					{
						continue;
					}

					FName TestName(BaseId, BaseId, i);
					if (CanUseNameInOuter(Parent, TestName))
					{
						NextNewNumber = i - 1;
						if (NextNewNumber < Iterator)
						{
							Iterator = FirstNewNumber;
						}

						UE_LOGF(LogUniqueObjectName, Log, "MISS: %ls %ls", *Parent->GetPathName(), *TestName.ToString());
						return TestName;
					}
				}

				return FName();
			}
		};

		FName Find(UObject* Parent, FNameEntryId BaseId, FName BaseName)
		{
			FName Result;

			UE_AUTORTFM_OPEN
			{
				Lock.ReadLock();
				FNameRangeEntry* Entry = Find(BaseId);

				if (Entry)
				{
					// already allocated name, we can release the shared lock and work on this object directly
					Lock.ReadUnlock();

					Result = Entry->AllocateName(Parent, BaseId, BaseName);
				}
				else
				{
					Lock.ReadUnlock();

					// The first time we request a name if we've never created one, don't bother adding to the cache just yet.
					FName TestName = FName::FindNumberedName(BaseId, FNameRangeEntry::FirstNewNumber + 1);
					if (TestName.IsNone())
					{
						Result = FName(BaseId, BaseId, FNameRangeEntry::FirstNewNumber + 1);
					}
					else
					{
						FWriteScopeLock _(Lock);
						// We didn't have a name but we may have been preempted as we acquired the write lock
						Entry = Find(BaseId);
						if (!Entry)
						{
							// we were not pre-empted, add a new entry for this name 
							Entry = Add(BaseId);
						}

						Result = Entry->AllocateName(Parent, BaseId, BaseName);
					}
				}
			};

			return Result;
		}
	private:
		FRWLock Lock;
		TMap<FNameEntryId, TUniquePtr<FNameRangeEntry>> Map;

		FNameRangeEntry* Find(FNameEntryId Id)
		{
			TUniquePtr<FNameRangeEntry>* Ptr = Map.Find(Id);
			if (Ptr) { return Ptr->Get(); }
			return nullptr;
		}

		FNameRangeEntry* Add(FNameEntryId Id)
		{
			LLM_SCOPE(ELLMTag::FName);
			return Map.Emplace(Id, MakeUnique<FNameRangeEntry>()).Get();
		}

	};

	static FNameRangeCache GNameRangeCache;

	FName MakeUniqueObjectNameReusingNumber(UObject* Parent, FName BaseName, EUniqueObjectNameOptions Options)
	{
		if (!GTryReuseNames)
		{
			return FName();
		}

		static const FName NamePackage(NAME_Package);
		if (!Parent || !!(Options & EUniqueObjectNameOptions::GloballyUnique) || BaseName == NamePackage || FPlatformProperties::HasEditorOnlyData() || !GFastPathUniqueNameGeneration)
		{
			return FName();
		}

		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		LLM_SCOPE(ELLMTag::FName);

		FName ReturnName;

		UE_AUTORTFM_OPEN
		{
			FNameEntryId BaseId = BaseName.GetDisplayIndex();
			ReturnName = GRecentNameCache.Find(Parent, BaseId, BaseName);

			if (ReturnName.IsNone())
			{
				ReturnName = GNameRangeCache.Find(Parent, BaseId, BaseName);
			}

			if (!ReturnName.IsNone())
			{
				// Store this name for reuse 
				GRecentNameCache.Store(Parent, BaseId, ReturnName);
			}
		};

		return ReturnName;
	}
}
#endif

#if WITH_EDITOR
void RecycleUniqueObjectSuffixAnnotation(UObject* Parent, UClass* ForClass)
{
	int32 MaxIndex = 0;
	FName BaseName = ForClass->GetFName();
	BaseName.SetNumber(0);

	ForEachObjectWithOuter(Parent, [&MaxIndex, BaseName](UObject* Object)
		{
			FName ObjectName = Object->GetFName();
			int32 Index = ObjectName.GetNumber();
			ObjectName.SetNumber(0);

			if (BaseName == ObjectName)
			{
				if (Index > MaxIndex)
				{
					MaxIndex = Index;
				}
			}
		}, EGetObjectsFlags::None);

	UpdateSuffixForNextNewObject(Parent, ForClass, [MaxIndex](int32& Index)
		{
			Index = MaxIndex;
		});
}

void ReleaseUniquelyNamedObject(UObject* Object)
{
	UObject* Outer = Object->GetOuter();
	UClass* Class = Object->GetClass();

	// Make the name available again for other objects
	Object->Rename(nullptr, GetTransientPackage());

	// Reset the suffix in FPerClassNumberSuffixAnnotation to the lowest we can 
	RecycleUniqueObjectSuffixAnnotation(Outer, Class);
}
#endif


FName MakeUniqueObjectName(UObject* Parent, const UClass* Class, FName InBaseName/*=NAME_None*/, EUniqueObjectNameOptions Options /*= EUniqueObjectNameOptions::None*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MakeUniqueObjectName);

	CSV_SCOPED_TIMING_STAT(UObject, MakeUniqueObjectName);
	check(Class);
	const FName BaseName = InBaseName.IsNone() ?
#if UE_WITH_REMOTE_OBJECT_HANDLE
		UE::RemoteObject::Private::GetServerBaseNameForUniqueName(Class)
#else
		Class->GetFName()
#endif
		: InBaseName;

	const FName ClassPathName = InBaseName.IsNone() ? *Class->GetPathName() : InBaseName;

	if (!!(Options & EUniqueObjectNameOptions::UniversallyUnique))
	{
		return MakeUniversallyUniqueObjectName(Parent, BaseName);
	}

#if TRY_REUSE_NAMES
	if (FName Result = NameReuse::MakeUniqueObjectNameReusingNumber(Parent, BaseName, Options); !Result.IsNone())
	{
		return Result;
	}
#endif
#undef TRY_REUSE_NAMES

	FName TestName;
	do
	{
		UObject* ExistingObject;

		do
		{
			// create the next name in the sequence for this class
			static const FName NamePackage(NAME_Package);
			if (BaseName == NamePackage)
			{
				int32 NextClassUnique = 0;
				UE_AUTORTFM_OPEN
				{
					NextClassUnique = FPlatformAtomics::InterlockedIncrement((volatile int32*)&Class->ClassUnique);
				};

				if (Parent == NULL)
				{
					//package names should default to "/Temp/Untitled" when their parent is NULL. Otherwise they are a group.
					TestName = FName(*FString::Printf(TEXT("/Temp/%s"), LexToString(NAME_Untitled)), NextClassUnique);
				}
				else
				{
					//package names should default to "Untitled"
					TestName = FName(NAME_Untitled, NextClassUnique);
				}
			}
			else
			{
				int32 NameNumber = 0;
				UE_AUTORTFM_OPEN
				{
					if (Parent && !(Options & EUniqueObjectNameOptions::GloballyUnique))
					{
						if (!FPlatformProperties::HasEditorOnlyData() && GFastPathUniqueNameGeneration)
						{
							/*   Fast Path Name Generation
							* A significant fraction of object creation time goes into verifying that the a chosen unique name is really unique.
							* The idea here is to generate unique names using very high numbers and only in situations where collisions are
							* impossible for other reasons.
							*
							* Rationale for uniqueness as used here.
							* - Consoles do not save objects in general, and certainly not animation trees. So we could never load an object that would later clash.
							* - We assume that we never load or create any object with a "name number" as large as, say, MAX_int32 / 2, other than via
							*   HACK_FastPathUniqueNameGeneration.
							* - After using one of these large "name numbers", we decrement the static UniqueIndex, this no two names generated this way, during the
							*   same run, could ever clash.
							* - We assume that we could never create anywhere near MAX_int32/2 total objects at runtime, within a single run.
							* - We require an outer for these items, thus outers must themselves be unique. Therefore items with unique names created on the fast path
							*   could never clash with anything with a different outer. For animation trees, these outers are never saved or loaded, thus clashes are
							*   impossible.
							*/
							NameNumber = --NameNumberUniqueIndex;
						}
						else
						{
							NameNumber = UpdateSuffixForNextNewObject(Parent, ClassPathName, [](int32& Index) { ++Index; });
						}
					}
					else
					{
						NameNumber = FPlatformAtomics::InterlockedIncrement((volatile int32*)&Class->ClassUnique);
					}
				};
				TestName = FName(BaseName, NameNumber);
			}

			if (!!(Options & EUniqueObjectNameOptions::GloballyUnique))
			{
				ExistingObject = StaticFindFirstObject(nullptr, *TestName.ToString());
			}
			else
			{
				ExistingObject = StaticFindObjectFastInternal(nullptr, Parent, TestName);
			}
		} while (ExistingObject);
	// InBaseName can be a name of an object from a different hierarchy (so it's still unique within given parents scope), we don't want to return the same name.
	} while (TestName == BaseName);
	return TestName;
}

FUniversallyUniqueObjectNameGenerator::FUniversallyUniqueObjectNameGenerator(const FGuid& Guid)
{
	uint64 Value = ((uint64)Guid.A << 32) | Guid.C;
	uint32 Index = 0;

	while (Value)
	{
		const uint8 Remainder = uint8(Value % 36);
		SessionIdBase36[Index++] = NibbleToTCharLower(Remainder);
		check(Index < (sizeof(SessionIdBase36) / sizeof(SessionIdBase36[0])));
		Value /= 36;
	}

	SessionIdBase36[Index] = 0;

	Seed = Guid.D;
}

uint32 FUniversallyUniqueObjectNameGenerator::GetNewValue() const
{
	for (uint32 CurValue = Seed.load(std::memory_order_relaxed);;)
	{
		const uint32 NewValue = CurValue * 1664525 + 1013904223;
		if (Seed.compare_exchange_weak(CurValue, NewValue))
		{
			return NewValue;
		}
	}
}

FName MakeUniversallyUniqueObjectName(UObject* InOuter, FName InBaseName, FUniversallyUniqueObjectNameGenerator const* Generator)
{
	// Generate a unique generation context for this session.
	static FUniversallyUniqueObjectNameGenerator const Context(FGuid::NewGuid());
	if (!Generator)
	{
		Generator = &Context;
	}

	TStringBuilder<NAME_SIZE> StringBuilder;

	// Append the provided base name if present.
	if (!InBaseName.IsNone())
	{
		StringBuilder += InBaseName.ToString();
		StringBuilder.AppendChar(TEXT('_'));
	}

	StringBuilder += Generator->SessionIdBase36;

	FName OutObjectName;
	do
	{
		// Store the random part in the lowest 31 bits of the name number below. This is to minimize the total names 
		// generated for globally unique names. The name number bit 31 is reserved to preserve the fast path names
		// generation (see GFastPathUniqueNameGeneration). We also set bit 30 as an indicator of a universally unique
		// object name.
		const int32 OutObjectIndex = (Generator->GetNewValue() & 0x3fffffff) | 0x40000000;
		OutObjectName = FName(*StringBuilder, OutObjectIndex);
	}
	while (StaticFindObjectFast(nullptr, InOuter, OutObjectName));

	return OutObjectName;
}

bool HasUniversallyUniqueObjectNameIdentifier(FName Name)
{
	const int32 Number = Name.GetNumber();
	return (Number & 0x40000000) != 0;
}

FName MakeObjectNameFromDisplayLabel(const FString& DisplayLabel, const FName CurrentObjectName)
{
	FString GeneratedName = SlugStringForValidName(DisplayLabel);

	// If the current object name (without a number) already matches our object's name, then use the existing name
	if( CurrentObjectName.GetPlainNameString() == GeneratedName )
	{
		// The object's current name is good enough!  This avoids renaming objects that don't really need to be renamed.
		return CurrentObjectName;
	}

	// If the new name is empty (for example, because it was composed entirely of invalid characters).
	// then we'll use the current name
	if( GeneratedName.IsEmpty() )
	{
		return CurrentObjectName;
	}

	const FName GeneratedFName( *GeneratedName );
	check( GeneratedFName.IsValidXName( INVALID_OBJECTNAME_CHARACTERS ) );

	return GeneratedFName;
}

/*-----------------------------------------------------------------------------
   Duplicating Objects.
-----------------------------------------------------------------------------*/

namespace UE::ObjectDuplication
{
	struct FSerializedObject
	{
		UObject* Object;
		int64 Offset;
	};

	// Helper method intended to gather up all default subobjects that have already been created and prepare them for duplication.
	void GatherDefaultSubobjectsForDuplication(UObject* SrcObject, UObject* DstObject, FDuplicatedObjectAnnotation& DuplicatedObjectAnnotation, FDuplicateDataWriter& Writer)
	{
		TArray<UObject*> SrcDefaultSubobjects;
		SrcObject->GetDefaultSubobjects(SrcDefaultSubobjects);
		
		// Iterate over all default subobjects within the source object.
		for (UObject* SrcDefaultSubobject : SrcDefaultSubobjects)
		{
			if (SrcDefaultSubobject)
			{
				// Attempt to find a default subobject with the same name within the destination object.
				UObject* DupDefaultSubobject = DstObject->GetDefaultSubobjectByName(SrcDefaultSubobject->GetFName());
				if (DupDefaultSubobject)
				{
					// Map the duplicated default subobject to the source and register it for serialization.
					DuplicatedObjectAnnotation.AddAnnotation(SrcDefaultSubobject, FDuplicatedObject(DupDefaultSubobject));
					Writer.UnserializedObjects.Add(SrcDefaultSubobject);

					// Recursively gather any nested default subobjects that have already been constructed through CreateDefaultSubobject().
					GatherDefaultSubobjectsForDuplication(SrcDefaultSubobject, DupDefaultSubobject, DuplicatedObjectAnnotation, Writer);
				}
			}
		}
	}
}

/**
 * Constructor - zero initializes all members
 */
FObjectDuplicationParameters::FObjectDuplicationParameters(UObject* InSourceObject, UObject* InDestOuter)
: SourceObject(InSourceObject)
, DestOuter(InDestOuter)
, DestName(NAME_None)
, FlagMask(RF_AllFlags & ~(RF_MarkAsRootSet|RF_MarkAsNative|RF_HasExternalPackage))
, InternalFlagMask(EInternalObjectFlags_AllFlags)
, ApplyFlags(RF_NoFlags)
, ApplyInternalFlags(EInternalObjectFlags::None)
, PortFlags(PPF_None)
, DuplicateMode(EDuplicateMode::Normal)
, bAssignExternalPackages(true)
, bSkipPostLoad(false)
, DestClass(nullptr)
, CreatedObjects(nullptr)
{
	checkSlow(SourceObject);
	checkSlow(DestOuter);
	checkSlow(SourceObject->IsValidLowLevel());
	checkSlow(DestOuter->IsValidLowLevel());
	DestClass = SourceObject->GetClass();
}

FObjectDuplicationParameters InitStaticDuplicateObjectParams(UObject const* SourceObject, UObject* DestOuter, const FName DestName, EObjectFlags FlagMask, UClass* DestClass, EDuplicateMode::Type DuplicateMode, EInternalObjectFlags InternalFlagsMask)
{
	// @todo: handle const down the callstack.  for now, let higher level code use it and just cast it off
	FObjectDuplicationParameters Parameters(const_cast<UObject*>(SourceObject), DestOuter);
	if (!DestName.IsNone())
	{
		Parameters.DestName = DestName;
	}
	else if (SourceObject->GetOuter() != DestOuter)
	{
		// try to keep the object name consistent if possible
		if (FindObjectFast<UObject>(DestOuter, SourceObject->GetFName()) == nullptr)
		{
			Parameters.DestName = SourceObject->GetFName();
		}
	}

	if (DestClass == nullptr)
	{
		Parameters.DestClass = SourceObject->GetClass();
	}
	else
	{
		Parameters.DestClass = DestClass;
	}
	// do not allow duplication of the Mark flags nor the HasExternalPackage flag
	Parameters.FlagMask = FlagMask & ~(RF_MarkAsRootSet | RF_MarkAsNative | RF_HasExternalPackage);
	Parameters.InternalFlagMask = InternalFlagsMask;
	Parameters.DuplicateMode = DuplicateMode;

	if (DuplicateMode == EDuplicateMode::PIE)
	{
		Parameters.PortFlags = PPF_DuplicateForPIE;
	}

	return Parameters;
}

UObject* StaticDuplicateObject(UObject const* SourceObject, UObject* DestOuter, const FName DestName, EObjectFlags FlagMask, UClass* DestClass, EDuplicateMode::Type DuplicateMode, EInternalObjectFlags InternalFlagsMask)
{
	FObjectDuplicationParameters Parameters = InitStaticDuplicateObjectParams(SourceObject, DestOuter, DestName, FlagMask, DestClass, DuplicateMode, InternalFlagsMask);
	return StaticDuplicateObjectEx(Parameters);
}

UObject* StaticDuplicateObjectEx( FObjectDuplicationParameters& Parameters )
{
	using namespace UE::ObjectDuplication;

	QUICK_SCOPE_CYCLE_COUNTER(STAT_StaticDuplicateObject);
	// make sure the two classes are the same size, as this hopefully will mean they are serialization
	// compatible. It's not a guarantee, but will help find errors
	checkf( (Parameters.DestClass->GetPropertiesSize() >= Parameters.SourceObject->GetClass()->GetPropertiesSize()),
		TEXT("Source and destination class sizes differ.  Source: %s (%i)   Destination: %s (%i)"),
		*Parameters.SourceObject->GetClass()->GetName(), Parameters.SourceObject->GetClass()->GetPropertiesSize(),
		*Parameters.DestClass->GetName(), Parameters.DestClass->GetPropertiesSize());
	
	UE_CLOGF(FPlatformProperties::RequiresCookedData() && Parameters.SourceObject->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading), LogUObjectGlobals, Warning, "Duplicating object '%ls' that's still being async loaded", *Parameters.SourceObject->GetFullName());
	// Make sure we're not duplicating the AsyncLoading, Async or LoaderImport internal flags, they will prevent the object from being gcd.
	Parameters.InternalFlagMask &= ~(EInternalObjectFlags::Async | EInternalObjectFlags::LoaderImport | EInternalObjectFlags_AsyncLoading 
#if UE_WITH_REMOTE_OBJECT_HANDLE
		| EInternalObjectFlags::RemoteReference
#endif
		);

	// We can't modify the loader from a transaction, so check for async loading and reset the 
	// loaders in the open.
	AutoRTFM::Open([&]
	{
		if (!IsAsyncLoading() && Parameters.SourceObject->HasAnyFlags(RF_ClassDefaultObject))
		{
			// Detach linker for the outer if it already exists, to avoid problems with PostLoad checking the Linker version
			ResetLoaders(Parameters.DestOuter);
		}
	});

	FObjectInstancingGraph InstanceGraph;

	if( !GIsDuplicatingClassForReinstancing )
	{
		// make sure we are not duplicating RF_RootSet as this flag is special
		// also make sure we are not duplicating the RF_ClassDefaultObject flag as this can only be set on the real CDO
		Parameters.FlagMask &= ~RF_ClassDefaultObject;
		Parameters.InternalFlagMask &= ~EInternalObjectFlags::RootSet;
	}

	// do not allow duplication of the Mark flags nor the HasExternalPackage flag in case the default flag mask was changed
	Parameters.FlagMask &= ~(RF_MarkAsRootSet | RF_MarkAsNative | RF_HasExternalPackage);


	// disable object and component instancing while we're duplicating objects, as we're going to instance components manually a little further below
	InstanceGraph.EnableSubobjectInstancing(false);

	// we set this flag so that the component instancing code doesn't think we're creating a new archetype, because when creating a new archetype,
	// the ObjectArchetype for instanced components is set to the ObjectArchetype of the source component, which in the case of duplication (or loading)
	// will be changing the archetype's ObjectArchetype to the wrong object (typically the CDO or something)
	InstanceGraph.SetLoadingObject(true);

	Parameters.SourceObject->PreDuplicate(Parameters);

	UObject* DupRootObject = Parameters.DuplicationSeed.FindRef(Parameters.SourceObject);
	if ( DupRootObject == nullptr )
	{
		FStaticConstructObjectParameters Params(Parameters.DestClass);
		Params.Outer = Parameters.DestOuter;
		Params.Name = Parameters.DestName;
		Params.SetFlags = Parameters.ApplyFlags | Parameters.SourceObject->GetMaskedFlags(Parameters.FlagMask);
		Params.InternalSetFlags = Parameters.ApplyInternalFlags | (Parameters.SourceObject->GetInternalFlags() & Parameters.InternalFlagMask);
		Params.bCopyTransientsFromClassDefaults = true;
		Params.InstanceGraph = &InstanceGraph;

		UObject* Archetype = Parameters.SourceObject->GetArchetype();
		Params.Template = Archetype->GetClass() == Parameters.DestClass ? Archetype : nullptr;

		DupRootObject = StaticConstructObject_Internal(Params);
	}

	FPooledLargeMemoryData ObjectData;

	FDuplicatedObjectAnnotation DuplicatedObjectAnnotation;
	Verse::FDuplicationContext DuplicatedCells;

	// if seed objects were specified, add those to the DuplicatedObjects map now
	if ( Parameters.DuplicationSeed.Num() > 0 )
	{
		for ( TMap<UObject*,UObject*>::TIterator It(Parameters.DuplicationSeed); It; ++It )
		{
			UObject* Src = It.Key();
			UObject* Dup = It.Value();
			checkSlow(Src);
			checkSlow(Dup);

			// create the DuplicateObjectInfo for this object
			DuplicatedObjectAnnotation.AddAnnotation( Src, FDuplicatedObject( Dup ) );
		}
	}

	const bool bShouldForceTaggedPropertySerialization = Parameters.SourceObject->GetClass() != Parameters.DestClass;

	// Read from the source object(s)
	FDuplicateDataWriter Writer(
		DuplicatedObjectAnnotation,				// Ref: Object annotation which stores the duplicated object for each source object
		ObjectData.Get(),						// Out: Serialized object data
		Parameters.SourceObject,				// Source object to copy
		DupRootObject,							// Destination object to copy into
		Parameters.FlagMask,					// Flags to be copied for duplicated objects
		Parameters.ApplyFlags,					// Flags to always set on duplicated objects
		Parameters.InternalFlagMask,			// Internal Flags to be copied for duplicated objects
		Parameters.ApplyInternalFlags,			// Internal Flags to always set on duplicated objects
		&InstanceGraph,							// Instancing graph
		Parameters.PortFlags,					// PortFlags	
		Parameters.bAssignExternalPackages);	// Assign duplicate external packages
	Writer.SetDuplicatedCells(DuplicatedCells);

	if (bShouldForceTaggedPropertySerialization)
	{
		Writer.SetUseUnversionedPropertySerialization(false);
	}

	TArray<FSerializedObject> SerializedObjects;

	
	if (GIsDuplicatingClassForReinstancing)
	{
		FBlueprintSupport::DuplicateAllFields(dynamic_cast<UStruct*>(Parameters.SourceObject), Writer);
	}

	// Add default subobjects to the DuplicatedObjects map so they don't get recreated during serialization.
	GatherDefaultSubobjectsForDuplication(Parameters.SourceObject, DupRootObject, DuplicatedObjectAnnotation, Writer);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::FAllocationContext Context = Verse::FAllocationContextPromise{};
#endif

	InstanceGraph.SetDestinationRoot( DupRootObject );
	while(Writer.UnserializedObjects.Num())
	{
		UObject*	Object = Writer.UnserializedObjects.Pop();
		int64 ObjectOffset = Writer.Tell();
		Object->Serialize(Writer);
		SerializedObjects.Emplace(Object, ObjectOffset);

		// After serialization we may have created default sub-objects (DSO) that we don't want to duplicate on subsequent calls.
		// e.g. Duplicating an ActorComponent that contains a DSO will have appended the ActorComponent to the UnserializedObjects but not accounted for the 
		// DSO in the annotation map. We do that here now.
		FDuplicatedObject DupObj = DuplicatedObjectAnnotation.GetAnnotation(Object);
		if (!DupObj.IsDefault())
		{
			GatherDefaultSubobjectsForDuplication(Object, DupObj.DuplicatedObject.GetEvenIfUnreachable(), DuplicatedObjectAnnotation, Writer);
		}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		DuplicatedCells.WriteUnserializedCells(Context, Writer);
#endif
	}

	TRefCountPtr<FUObjectSerializeContext> LoadContext(FUObjectThreadContext::Get().GetSerializeContext());
	FDuplicateDataReader Reader(DuplicatedObjectAnnotation, ObjectData.Get(), Parameters.PortFlags, Parameters.DestOuter);
	Reader.SetDuplicatedCells(DuplicatedCells);

	if (bShouldForceTaggedPropertySerialization)
	{
		Reader.SetUseUnversionedPropertySerialization(false);
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	DuplicatedCells.CreateSerializedCells(Context, Reader);
#endif
	for (const FSerializedObject& Entry : SerializedObjects)
	{
		UObject* SerializedObject = Entry.Object;
		Reader.Seek(Entry.Offset);

		FDuplicatedObject ObjectInfo = DuplicatedObjectAnnotation.GetAnnotation( SerializedObject );
		checkSlow( !ObjectInfo.IsDefault() );

		UObject* DuplicatedObject = ObjectInfo.DuplicatedObject.GetEvenIfUnreachable();
		check(DuplicatedObject);

		TGuardValue<UObject*> SerializedObjectGuard(LoadContext->SerializedObject, DuplicatedObject);
		if ( !SerializedObject->HasAnyFlags(RF_ClassDefaultObject) )
		{
			DuplicatedObject->Serialize(Reader);
		}
		else
		{
			// if the source object was a CDO, then transient property values were serialized by the FDuplicateDataWriter
			// and in order to read those properties out correctly, we'll need to enable defaults serialization on the
			// reader as well.
			Reader.StartSerializingDefaults();
			DuplicatedObject->Serialize(Reader);
			Reader.StopSerializingDefaults();
		}
	}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	DuplicatedCells.ReadSerializedCells(Context, Reader);
#endif

	InstanceGraph.EnableSubobjectInstancing(true);

	for (const FSerializedObject& Entry : SerializedObjects)
	{
		UObject* OrigObject = Entry.Object;

		// don't include any objects which were included in the duplication seed map in the instance graph, as the "duplicate" of these objects
		// may not necessarily be the object that is supposed to be its archetype (the caller can populate the duplication seed map with any objects they wish)
		// and the DuplicationSeed is only used for preserving inter-object references, not for object graphs in SCO and we don't want to call PostDuplicate/PostLoad
		// on them as they weren't actually duplicated
		if ( Parameters.DuplicationSeed.Find(OrigObject) == nullptr )
		{
			FDuplicatedObject DupObjectInfo = DuplicatedObjectAnnotation.GetAnnotation( OrigObject );

			if (UObject* DuplicatedObject = DupObjectInfo.DuplicatedObject.GetEvenIfUnreachable())
			{
				UObject* DupObjectArchetype = DuplicatedObject->GetArchetype();

				bool bDuplicateForPIE = (Parameters.PortFlags & PPF_DuplicateForPIE) != 0;

				// Any PIE duplicated object that has the standalone flag is a potential garbage collection issue
				ensure(!(bDuplicateForPIE && DuplicatedObject->HasAnyFlags(RF_Standalone)));

				DuplicatedObject->PostDuplicate(Parameters.DuplicateMode);
				if (!Parameters.bSkipPostLoad && !DuplicatedObject->IsTemplate())
				{
					// We skip post-loading during async loading if on the loader thread as we're going to handle it deferred on GT instead.
					if (IsInGameThread())
					{
						// Don't want to call PostLoad on class duplicated CDOs
						TGuardValue<bool> GuardIsRoutingPostLoad(FUObjectThreadContext::Get().IsRoutingPostLoad, true);
						DuplicatedObject->ConditionalPostLoad();
					}
					else
					{
						// The only other thread that we allow to go through here is ALT because we know
						// it is going to call post-load on new objects.
						check(IsInAsyncLoadingThread());
					}
				}

				DuplicatedObject->CheckDefaultSubobjects();

				// Now that subobjects are instanced, we can invoke the class initializer function on the instance. Note that this will get
				// deferred during object construction because subobject instancing was initially disabled through the instance graph above.
				DuplicatedObject->GetClass()->PostInitInstance(DuplicatedObject, &InstanceGraph);
			}
		}
	}

	// if the caller wanted to know which objects were created, do that now
	if ( Parameters.CreatedObjects != nullptr )
	{
		// note that we do not clear the map first - this is to allow callers to incrementally build a collection
		// of duplicated objects through multiple calls to StaticDuplicateObject

		// now add each pair of duplicated objects;
		// NOTE: we don't check whether the entry was added from the DuplicationSeed map, so this map
		// will contain those objects as well.
		for (const FSerializedObject& Entry : SerializedObjects)
		{
			UObject* OrigObject = Entry.Object;

			// don't include any objects which were in the DuplicationSeed map, as CreatedObjects should only contain the list
			// of objects actually created during this call to SDO
			if ( Parameters.DuplicationSeed.Find(OrigObject) == nullptr )
			{
				FDuplicatedObject DupObjectInfo = DuplicatedObjectAnnotation.GetAnnotation( OrigObject );
				if (UObject* DuplicatedObject = DupObjectInfo.DuplicatedObject.GetEvenIfUnreachable())
				{
					Parameters.CreatedObjects->Add(OrigObject, DuplicatedObject);
				}
			}
		}
	}
	
	return DupRootObject;
}

bool SaveToTransactionBuffer(UObject* Object, bool bMarkDirty)
{
	check(!Object->HasAnyInternalFlags(EInternalObjectFlags::Async | EInternalObjectFlags_AsyncLoading));
	bool bSavedToTransactionBuffer = false;

	// Script packages should not end up in the transaction buffer.
	// PIE objects should go through however. Additionally, in order
	// to save a copy of the object, we must have a transactor and the object must be transactional.
	const bool bIsTransactional = Object->HasAnyFlags(RF_Transactional);
	const bool bIsNotScriptPackage = (Object->GetOutermost()->HasAnyPackageFlags(PKG_ContainsScript) == false);

	if ( GUndo && bIsTransactional && bIsNotScriptPackage)
	{
		check(IsInGameThread());

		// Mark the package dirty, if requested
		if ( bMarkDirty )
		{
			Object->MarkPackageDirty();
		}

		// Save a copy of the object to the transactor
		GUndo->SaveObject( Object );
		bSavedToTransactionBuffer = true;
	}

	return bSavedToTransactionBuffer;
}

void SnapshotTransactionBuffer(UObject* Object)
{
	SnapshotTransactionBuffer(Object, TArrayView<const FProperty*>());
}

void SnapshotTransactionBuffer(UObject* Object, TArrayView<const FProperty*> Properties)
{
	// Script packages should not end up in the transaction buffer.
	// PIE objects should go through however. Additionally, in order
	// to save a copy of the object, we must have a transactor and the object must be transactional.
	const bool bIsTransactional = Object->HasAnyFlags(RF_Transactional);
	const bool bIsNotScriptPackage = (Object->GetOutermost()->HasAnyPackageFlags(PKG_ContainsScript) == false);

	if (GUndo && bIsTransactional && bIsNotScriptPackage)
	{
		GUndo->SnapshotObject(Object, Properties);
	}
}

// Utility function to evaluate whether we allow an abstract object to be allocated
int32 FScopedAllowAbstractClassAllocation::AllowAbstractCount = 0;
FScopedAllowAbstractClassAllocation::FScopedAllowAbstractClassAllocation()
{
	++AllowAbstractCount;
}

FScopedAllowAbstractClassAllocation::~FScopedAllowAbstractClassAllocation()
{
	--AllowAbstractCount;
}

bool FScopedAllowAbstractClassAllocation::IsDisallowedAbstractClass(const UClass* InClass, EObjectFlags InFlags)
{
	if (((InFlags& RF_ClassDefaultObject) == 0) && InClass->HasAnyClassFlags(CLASS_Abstract))
	{
		if (AllowAbstractCount == 0)
		{
			return true;
		}
	}

	return false;
}

#if WITH_EDITOR
bool StaticAllocateObjectErrorTests( const UClass* InClass, UObject* InOuter, FName InName, EObjectFlags InFlags)
{
	// Editor-only validation checks, StaticAllocateObject has the simpler checks for packaged builds
	if( !InClass )
	{
		UE_LOGF(LogUObjectGlobals, Fatal, "Empty class for object %ls", *InName.ToString() );
		return true;
	}

	if (FScopedAllowAbstractClassAllocation::IsDisallowedAbstractClass(InClass, InFlags))
	{
		const FString ErrorMsg = FString::Printf(TEXT("Class which was marked abstract was trying to be loaded in Outer %s.  It will be nulled out on save. %s %s"), *GetPathNameSafe(InOuter), *InName.ToString(), *InClass->GetName());
		// if we are trying instantiate an abstract class in the editor we'll warn the user that it will be nulled out on save
		UE_LOGF(LogUObjectGlobals, Warning, "%ls", *ErrorMsg);
		ensureMsgf(false, TEXT("%s"), *ErrorMsg);
	}

	if( InOuter == NULL )
	{
		if ( InClass != UPackage::StaticClass() )
		{
			UE_LOGF(LogUObjectGlobals, Fatal, "%ls", *FString::Printf( TEXT("Object is not packaged: %s %s"), *InClass->GetName(), *InName.ToString()) );
			return true;
		}
		else if ( InName == NAME_None )
		{
			UE_LOGF(LogUObjectGlobals, Fatal, "%ls", TEXT("Attempted to create a package named 'None'") );
			return true;
		}
	}

	// When reinstancing, allow any within violations as they were likely caused by users renaming
	// objects to different outers, hopefully with intent:
	if ( (InFlags & (RF_ClassDefaultObject|RF_ArchetypeObject)) == 0 && !GIsReinstancing )
	{
		if ( InOuter != nullptr && !InOuter->IsA(InClass->ClassWithin) )
		{
			// This is also validated in UObject::PreSave
			const FString ErrorMsg = FString::Printf(TEXT("Object %s of class %s with ClassWithin of %s was created in invalid Outer %s!"), *InName.ToString(), *InClass->GetPathName(), *InClass->ClassWithin->GetPathName(), *InOuter->GetClass()->GetPathName());
			UE_LOGF(LogUObjectGlobals, Warning, "%ls", *ErrorMsg);
			ensureMsgf(false, TEXT("%s"), *ErrorMsg);
		}
	}
	return false;
}
#endif // WITH_EDITOR

/**
* For object overwrites, the class may want to persist some info over the re-initialize
* this is only used for classes in the script compiler
**/
//@todo UE this is clunky
static thread_local FRestoreForUObjectOverwrite* ObjectRestoreAfterInitProps = nullptr;

extern const FName NAME_UniqueObjectNameForCooking("UniqueObjectNameForCooking");
static thread_local bool GOutputCookingWarnings = false;

COREUOBJECT_API bool GetOutputCookingWarnings()
{
	return GOutputCookingWarnings;
}

COREUOBJECT_API void SetOutputCookingWarnings(bool bValue)
{
	GOutputCookingWarnings = bValue;
}

// Used by StaticAllocateObject when constructing an object on top of existing object to simulate GC behavior
class FGCReconstructionGuard
{
	enum class EReconstructionLockType
	{
		None,
		GC
	} LockType = EReconstructionLockType::None;

public:

	FGCReconstructionGuard() = default;
	~FGCReconstructionGuard()
	{
		checkf(LockType == EReconstructionLockType::None, TEXT("FGCReconstructionGuard has not been unlocked"));
	}

	void Lock()
	{
		UE_AUTORTFM_OPEN
		{			
			if (!FGCCSyncObject::Get().IsGCLocked()) // GC lock doesn't support recursive locks atm
			{
				FGCCSyncObject::Get().GCLock();
				LockType = EReconstructionLockType::GC;
			}
			else
			{
				LockType = EReconstructionLockType::None;
			}
		};
		AutoRTFM::PushOnAbortHandler(this, [this]
			{
				this->Unlock();
			});
	}
	void Unlock()
	{
		UE_AUTORTFM_OPEN
		{
			if (LockType == EReconstructionLockType::GC)
			{
				FGCCSyncObject::Get().GCUnlock();
			}
			LockType = EReconstructionLockType::None;
		};
		AutoRTFM::PopOnAbortHandler(this);
	}
};

#if UE_WITH_REMOTE_OBJECT_HANDLE
namespace
{
	FORCENOINLINE void TryGetRemoteObjectOverrides(FName InName, UObject* InOuter, int32& InOutSerialNumber, FRemoteObjectId& InOutRemoteId)
	{
		using namespace UE::RemoteObject::Serialization;

		FRemoteObjectConstructionOverridesStack& OverridesStack = FRemoteObjectConstructionOverridesStack::Get();
		if (!OverridesStack.IsEmpty())
		{
			// Try to get remote subobject overrides so that if the object that's about to be constructed is a subobject
			// it gets the right RemoteId and SerialNumber at the time of its construction because its owner constructor may be reading the id and/or 
			// creating a weak pointer to the subobject (if not the owner directly maybe some code that gets called from the owner's constructor)

			if (const FRemoteObjectConstructionParams* ConstructionOverrides = OverridesStack.Find(InName, InOuter))
			{
				if (ConstructionOverrides->SerialNumber != 0)
				{
					checkf(InOutSerialNumber == 0 || InOutSerialNumber == ConstructionOverrides->SerialNumber, TEXT("Attempting to override valid SerialNumber of %s.%s (%d -> %d)"), *GetPathNameSafe(InOuter), *InName.ToString(), InOutSerialNumber, ConstructionOverrides->SerialNumber);
					InOutSerialNumber = ConstructionOverrides->SerialNumber;
				}
				if (ConstructionOverrides->RemoteId.IsValid())
				{
					checkf(!InOutRemoteId.IsValid() || InOutRemoteId == ConstructionOverrides->RemoteId, TEXT("Attempting to override valid RemoteId of %s.%s (%s -> %s)"), *GetPathNameSafe(InOuter), *InName.ToString(), *InOutRemoteId.ToString(), *ConstructionOverrides->RemoteId.ToString());
					InOutRemoteId = ConstructionOverrides->RemoteId;
				}
			}
		}
	}
}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

UE_AUTORTFM_ASSUME_SAFE
UObject* StaticAllocateObject
(
	const UClass*	InClass,
	UObject*		InOuter,
	FName			InName,
	EObjectFlags	InFlags,
	EInternalObjectFlags InternalSetFlags,
	bool bCanRecycleSubobjects,
	bool* bOutRecycledSubobject,
	UPackage* ExternalPackage,
	int32 SerialNumber,
	FRemoteObjectId RemoteId,
	FGCReconstructionGuard* GCGuard)
{
	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_StaticAllocateObject);
	
	// report detailed UObject classes allocations
#if ENABLE_LOW_LEVEL_MEM_TRACKER
	FName LLMScope_Name;
	FString LLMClassNameString;
	if (FLowLevelMemTracker::Get().IsTagSetActive(ELLMTagSet::UObjectClasses))
	{
		LLMClassNameString = FString::Printf(TEXT("UObject/%s"), *InClass->GetName());
		LLMScope_Name = *LLMClassNameString;
	}
	LLM_SCOPE_DYNAMIC(LLMScope_Name, ELLMTracker::Default, ELLMTagSet::UObjectClasses, FLLMDynamicTagConstructorStatString(LLMClassNameString));
#endif // ENABLE_LOW_LEVEL_MEM_TRACKER

	SCOPE_CYCLE_COUNTER(STAT_AllocateObject);
	checkSlow(InOuter != INVALID_OBJECT); // not legal
	check(InClass && InClass->ClassWithin && InClass->ClassConstructor);

	const bool bCreatingCDO = (InFlags & RF_ClassDefaultObject) != 0 && (InFlags & RF_ImmutableDefaultObject) == 0;
	const bool bCreatingArchetype = (InFlags & RF_ArchetypeObject) != 0;
#if WITH_EDITOR
	if (GIsEditor)
	{
		if (StaticAllocateObjectErrorTests(InClass,InOuter,InName,InFlags))
		{
			return NULL;
		}
	}
	else
#endif // WITH_EDITOR
	{
		// In the editor these are handled inside StaticAllocateObjectErrorTests and they may be temporary warnings
		checkf(!FScopedAllowAbstractClassAllocation::IsDisallowedAbstractClass(InClass, InFlags), TEXT("Unable to create new object: %s %s.%s. Creating an instance of an abstract class is not allowed!"),
			*GetNameSafe(InClass), *GetPathNameSafe(InOuter), *InName.ToString());
		check(bCreatingCDO || bCreatingArchetype || !InOuter || InOuter->IsA(InClass->ClassWithin));
		check(InOuter || (InClass == UPackage::StaticClass() && InName != NAME_None)); // only packages can not have an outer, and they must be named explicitly	
	}

	checkf(!IsGarbageCollectingAndLockingUObjectHashTables(), TEXT("Unable to create new object: %s %s.%s. Creating UObjects while Collecting Garbage is not allowed!"),
		*GetNameSafe(InClass), *GetPathNameSafe(InOuter), *InName.ToString());

	if (bCreatingCDO)
	{
		check(InClass->GetClass());
		ensureMsgf(!GIsDuplicatingClassForReinstancing || InClass->HasAnyClassFlags(CLASS_Native), TEXT("GIsDuplicatingClassForReinstancing %d InClass %s"), (int)GIsDuplicatingClassForReinstancing, *InClass->GetPathName());
		InName = InClass->GetDefaultObjectName();
		// never call PostLoad on class default objects
		InFlags &= ~(RF_NeedPostLoad|RF_NeedPostLoadSubobjects);
	}

	UObject* Obj = NULL;
	if(InName == NAME_None)
	{
		UE_AUTORTFM_OPEN
		{
#if WITH_EDITOR
			if ( GetOutputCookingWarnings() && GetTransientPackage() != InOuter->GetOutermost() )
			{
				InName = MakeUniqueObjectName(InOuter, InClass, NAME_UniqueObjectNameForCooking);
			}
			else
#endif
			{
				InName = MakeUniqueObjectName(InOuter, InClass);
			}
		};
	}
	else
	{
		// See if object already exists.
		Obj = StaticFindObjectFastInternal( /*Class=*/ NULL, InOuter, InName, EFindObjectFlags::ExactClass );

		// It is an error if we are trying to replace an object of a different class
		if (Obj && !Obj->GetClass()->IsChildOf(InClass))
		{
			const TCHAR* ErrorPrefix = TEXT("");

			if (InClass->HasAnyClassFlags(CLASS_PerObjectConfig) && InOuter != nullptr && InOuter->GetOutermost() == GetTransientPackage())
			{
				ErrorPrefix = TEXT("PerObjectConfig object using the transient package, has triggered a name conflict and will now crash.\n"
						"To avoid this, don't use the transient package for PerObjectConfig objects.\n"
						"This has the side effect, of using the full path name for config ini sections. Use 'OverridePerObjectConfigSection' to keep the short name.\n\n");
			}

			// This generally happens when calling NewObject with a specific object name and an object already exists at the same path.
			// If the classes look the same but have different paths, an old version may have been renamed due to plugin unloading or class recompiling.
			// If the object has the garbage flag set, it was marked as ready to destroy but still exists so it cannot be reallocated before garbage collection clears it.
			UE_LOGF(LogUObjectGlobals, Fatal,
				"%lsCannot replace existing object of a different class.\n"
				     "\tNew Object: %ls %ls.%ls\n"
				     "\tExisting Object: %ls %ls(0x%08x 0x%08x)",
				ErrorPrefix, *InClass->GetPathName(), InOuter ? *InOuter->GetPathName() : TEXT(""), *InName.ToString(),
				*Obj->GetFullName(nullptr, EObjectFullNameFlags::IncludeClassPackage),
				Obj->HasAnyInternalFlags(EInternalObjectFlags::Garbage) ? TEXT("(garbage) ") : TEXT(""),
				(int32)Obj->GetFlags(), (int32)Obj->GetInternalFlags());
		}
	}

	FLinkerLoad*	Linker						= nullptr;
	int32			LinkerIndex					= INDEX_NONE;
	bool			bWasConstructedOnOldObject	= false;
	// True when the object to be allocated already exists and is a subobject.
	bool bSubObject = false;
	int32 TotalSize = InClass->GetPropertiesSize();
	checkSlow(TotalSize);

	int32 OffsetToObject = -InClass->GetPropertiesStartOffset(); // Start offset for properties can be negative so we want to move object forward in allocation
	int32 OldIndex = -1;
	int32 OldSerialNumber = 0;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	FRemoteObjectId OldRemoteId;

	TryGetRemoteObjectOverrides(InName, InOuter, SerialNumber, RemoteId);
#endif

	if( Obj == nullptr )
	{	
		int32 Alignment	= FMath::Max( 4, InClass->GetMinAlignment() );
		Obj = (UObject *)GUObjectAllocator.AllocateUObject(TotalSize, Alignment, GIsInitialLoad, OffsetToObject);
	}
	else
	{
		// Replace an existing object without affecting the original's address or index.
		check(!Obj->IsUnreachable());

		check(!ObjectRestoreAfterInitProps); // otherwise recursive construction
		ObjectRestoreAfterInitProps = Obj->GetRestoreForUObjectOverwrite();

		// Remember linker, flags, index, and native class info.
		Linker		= Obj->GetLinker();
		LinkerIndex = Obj->GetLinkerIndex();
		InternalSetFlags |= (Obj->GetInternalFlags() & (EInternalObjectFlags::Native | EInternalObjectFlags::RootSet | EInternalObjectFlags::LoaderImport));

		if (bCreatingCDO || !!(InFlags & RF_ImmutableDefaultObject))
		{
			check(Obj->HasAllFlags(RF_ClassDefaultObject));
			Obj->SetFlags(InFlags);
			Obj->SetInternalFlags(InternalSetFlags);
			// never call PostLoad on class default objects
			Obj->ClearFlags(RF_NeedPostLoad|RF_NeedPostLoadSubobjects);
		}
		else if(!InOuter || !InOuter->HasAnyFlags(RF_ClassDefaultObject))
		{
#if !UE_BUILD_SHIPPING
			// Handle nested DSOs
			bool bIsOwnedByCDOOrArchetype = false;
			UObject* Iter = InOuter;
			while (Iter)
			{
				if (Iter->HasAnyFlags(RF_ClassDefaultObject|RF_ArchetypeObject))
				{
					bIsOwnedByCDOOrArchetype = true;
					break;
				}
				Iter = Iter->GetOuter();
			}

			// Should only get in here if we're NOT creating a subobject of a CDO.  CDO subobjects may still need to be serialized off of disk after being created by the constructor
			// if really necessary there was code to allow replacement of object just needing postload, but lets not go there unless we have to
			checkf(!Obj->HasAnyFlags(RF_NeedLoad|RF_NeedPostLoad|RF_ClassDefaultObject) || bIsOwnedByCDOOrArchetype,
				TEXT("Attempting to replace an object that hasn't been fully loaded: %s (Outer=%s, Flags=0x%08x)"),
				*Obj->GetFullName(),
				InOuter ? *InOuter->GetFullName() : TEXT("NULL"),
				(int32)Obj->GetFlags()
			);
#endif//UE_BUILD_SHIPPING
		}
		// Subobjects are always created in the constructor, no need to re-create them here unless their archetype != CDO or they're blueprint generated or they're remote
		if (!bCreatingCDO && (!bCanRecycleSubobjects || !Obj->IsDefaultSubobject()
#if UE_WITH_REMOTE_OBJECT_HANDLE
			|| (UE::RemoteObject::Handle::GetResidence(Obj) == EResidence::Remote)
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
			))
		{
			OldIndex = GUObjectArray.ObjectToIndex(Obj);
			OldSerialNumber = GUObjectArray.GetSerialNumber(OldIndex);
#if UE_WITH_REMOTE_OBJECT_HANDLE
			OldRemoteId = GUObjectArray.GetRemoteId(OldIndex);
#endif

			// It's not safe to access the object from other threads while it's being destroyed and reconstucted so simulate GC behavior and lock
			if (IsInGameThread())
			{
				// Take the GC lock if the object is still being async loaded to avoid any race condition, otherwise we don't care about the lock
				if (GCGuard && Obj->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
				{
					GCGuard->Lock();
				}
			}
			else if (IsInAsyncLoadingThread() && Obj->HasAnyInternalFlags(EInternalObjectFlags::AsyncLoadingPhase1))
			{
				// We loosen the restriction a little bit to allow overwrite of UObjects that are still in the construction loading phase,
				// this only happens for very narrow cases and should be fine as long as the UObject's in question have thread-safe
				// destructor and destruction virtuals.
				// No need for the GCGuard here since async loading always has one when its active.
			}
			else
			{
				ensureMsgf(false, TEXT("GC lock can only be acquired on the game thread. If you hit this ensure then an object is being reconstructed on a worker thread which is not thread-safe"));
			}

			// Destroy the object.
			SCOPE_CYCLE_COUNTER(STAT_DestroyObject);
			// Check that the object hasn't been destroyed yet.
			if(!Obj->HasAnyFlags(RF_FinishDestroyed))
			{
#if !UE_WITH_REMOTE_OBJECT_HANDLE
				if (FPlatformProperties::RequiresCookedData())
				{
					ensureAlwaysMsgf(!Obj->HasAnyFlags(RF_NeedLoad|RF_WasLoaded),
						TEXT("Replacing a loaded public object is not supported with cooked data: %s (Outer=%s, Flags=0x%08x, InternalObjectFlags=0x%08x)"),
						*Obj->GetFullName(),
						InOuter ? *InOuter->GetFullName() : TEXT("NULL"),
						(int32)Obj->GetFlags(),
						(int32)Obj->GetInternalFlags());
				}
#endif // !UE_WITH_REMOTE_OBJECT_HANDLE

				// Record the memory of the object before destruction, so aborting will return the
				// object to the previous object's state.
				UE_AUTORTFM_OPEN_NO_SANITIZE
				{
					// We disable the AutoRTFM sanitizer on this write because this spans the entire
					// object's memory including fields that may be accessed only in the open (e.g.
					// object flags which are set atomically), and mixing closed writes with open
					// writes will trigger the AutoRTFM sanitizer.
#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
					// ObjectListInternalIndex is an index into an array of UObjectHash's ClassToObjectListMap.
					// This index must be preserved, even if a reconstructed object is reverted as the
					// table is not instrumented by AutoRTFM.
					char* ObjectStart = reinterpret_cast<char*>(Obj);
					char* InternalIndexStart = reinterpret_cast<char*>(&Obj->GetObjectHashInternalIndex());
					char* InternalIndexEnd = InternalIndexStart + sizeof(std::remove_reference_t<decltype(Obj->GetObjectHashInternalIndex())>);
					char* ObjectEnd = ObjectStart + TotalSize;
					AutoRTFM::RecordOpenWrite(ObjectStart, InternalIndexStart - ObjectStart);
					AutoRTFM::RecordOpenWrite(InternalIndexEnd, ObjectEnd - InternalIndexEnd);
#else
					AutoRTFM::RecordOpenWrite(Obj, TotalSize);
#endif
				};

				// Get the name before we start the destroy, as destroy renames it
				FString OldName = Obj->GetFullName();

				// Begin the asynchronous object cleanup.
				Obj->ConditionalBeginDestroy();

				bool bPrinted = false;
				double StallStart = 0.0;
				// Wait for the object's asynchronous cleanup to finish.
				while (!Obj->IsReadyForFinishDestroy()) 
				{
					// If we're not in the editor, and aren't doing something specifically destructive like reconstructing blueprints, this is fatal
					if (!bPrinted && !GIsEditor && FApp::IsGame() && !GIsReconstructingBlueprintInstances)
					{
						StallStart = FPlatformTime::Seconds();
						bPrinted = true;
					}
					FPlatformProcess::Sleep(0);
				}
				if (bPrinted)
				{
					const double ThisTime = FPlatformTime::Seconds() - StallStart;
					UE_LOGF(LogUObjectGlobals, Warning, "Gamethread hitch waiting for resource cleanup on a UObject (%ls) overwrite took %6.2fms. Fix the higher level code so that this does not happen.", *OldName, ThisTime * 1000.0);
				}
				// Finish destroying the object.
				Obj->ConditionalFinishDestroy();
			}
			TGuardValue<bool> _(GUObjectArray.bShouldRecycleObjectIndices, false);
			GUObjectArray.LockInternalArray();
			GUObjectArray.FreeUObjectIndex(Obj);
			GUObjectArray.UnlockInternalArray();
			Obj->~UObject();
			bWasConstructedOnOldObject	= true;
		}
		else
		{
#if UE_WITH_REMOTE_OBJECT_HANDLE
			if (RemoteId.IsValid() && RemoteId != UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(Obj))
			{
				UE::CoreUObject::Private::FObjectHandleUtils::ChangeRemoteId(Obj, RemoteId);
			}
			if (SerialNumber != 0)
			{
				FUObjectItem* SubobjectItem = GUObjectArray.ObjectToObjectItem(Obj);
				checkf(SubobjectItem->SerialNumber == 0 || SerialNumber == 0 || SerialNumber == SubobjectItem->SerialNumber, TEXT("Attempting to change an existing subobject's %s serial number (%d -> %d)"), *Obj->GetPathName(), SubobjectItem->SerialNumber, SerialNumber);
				SubobjectItem->SerialNumber = SubobjectItem->SerialNumber == 0 ? SerialNumber : SubobjectItem->SerialNumber;
			}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
			bSubObject = true;
		}
	}

	// If class is transient, non-archetype objects must be transient.
	if ( !bCreatingCDO && InClass->HasAnyClassFlags(CLASS_Transient) && !bCreatingArchetype )
	{
		InFlags |= RF_Transient;
	}

	const bool bIsAsyncLoading = (IsInAsyncLoadingThread() || IsInParallelLoadingThread());
	
	if (!bSubObject)
	{
		// This needs to be set at construction to prevent the object from having a short period where it can be found
		// between its creation and the call to NotifyConstructedDuringAsyncLoading.
		if (bIsAsyncLoading)
		{
			InternalSetFlags |= EInternalObjectFlags::AsyncLoadingPhase1;
		}

#if UE_WITH_REMOTE_OBJECT_HANDLE
		// We can't have two different, non-zero serial numbers
		checkf(OldSerialNumber == 0 || SerialNumber == 0 || SerialNumber == OldSerialNumber, TEXT("Attempting to change a new object's (%s %s.%s) serial number (%d -> %d)"),
			*InClass->GetPathName(), *GetPathNameSafe(InOuter), *InName.ToString(), OldSerialNumber, SerialNumber);
		checkf(!OldRemoteId.IsValid() || !RemoteId.IsValid() || OldRemoteId == RemoteId, TEXT("Attempting to change a new object's (%s %s.%s) remote id (%s -> %s)"),
			*InClass->GetPathName(), *GetPathNameSafe(InOuter), *InName.ToString(), *OldRemoteId.ToString(), *RemoteId.ToString());
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

		UE_AUTORTFM_OPEN_NO_VALIDATION
		{
			FMemory::Memzero((uint8 *)Obj - OffsetToObject, TotalSize);
			new ((void *)Obj) UObjectBase(const_cast<UClass*>(InClass), InFlags|RF_NeedInitialization, InternalSetFlags, InOuter, InName, OldIndex, OldSerialNumber == 0 ? SerialNumber : OldSerialNumber, 
#if UE_WITH_REMOTE_OBJECT_HANDLE
				OldRemoteId.IsValid() ? OldRemoteId :
#endif
				RemoteId);
		};
		if (!bWasConstructedOnOldObject)
		{
			// If the object was freshly constructed inside a transaction, then
			// mark the AutoRTFMConstructionAborted bit if the transaction is
			// aborted. This is used to prevent aborted objects from being
			// returned from global lists, e.g. ForEachObjectOfClass().
			UE_AUTORTFM_ONABORT(Obj)
			{
				Obj->SetInternalFlags(EInternalObjectFlags::AutoRTFMConstructionAborted);
			};
		}
	}
	else
	{
		// Propagate flags to subobjects created in the native constructor.
		Obj->SetFlags(InFlags);
		Obj->SetInternalFlags(InternalSetFlags);
	}

	// if an external package was specified, assign it to the object
	if (ExternalPackage)
	{
		Obj->SetExternalPackage(ExternalPackage);
	}

	if (bWasConstructedOnOldObject)
	{
		// Reassociate the object with it's linker.
		Obj->SetLinker(Linker,LinkerIndex,false);
		if(Linker)
		{
			check(Linker->ExportMap[LinkerIndex].Object == NULL);
			Linker->ExportMap[LinkerIndex].Object = Obj;
		}
	}

	if (bIsAsyncLoading)
	{
		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		if (ThreadContext.AsyncPackageLoader)
		{
			LLM_SCOPE(ELLMTag::AsyncLoading);
			ThreadContext.AsyncPackageLoader->NotifyConstructedDuringAsyncLoading(Obj, bSubObject);
		}
	}
	else
	{
		// Sanity checks for async flags.
		// It's possible to duplicate an object on the game thread that is still being referenced 
		// by async loading code or has been created on a different thread than the main thread.
		Obj->ClearInternalFlags(EInternalObjectFlags_AsyncLoading);
		if (Obj->HasAnyInternalFlags(EInternalObjectFlags::Async) && IsInGameThread())
		{
			Obj->ClearInternalFlags(EInternalObjectFlags::Async);
		}
	}


	// Let the caller know if a subobject has just been recycled.
	if (bOutRecycledSubobject)
	{
		*bOutRecycledSubobject = bSubObject;
	}
	
	return Obj;
}

//@todo UE - move this stuff to UnObj.cpp or something

void UObject::PostReinitProperties()
{
}

void UObject::PostInitProperties()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	// Always write to FUObjectThreadContext from the open, as writing to a 
	// memory location in both the open and closed within the same transaction
	// can lead to state corruption.
	UE_AUTORTFM_OPEN
	{
		ThreadContext.PostInitPropertiesCheck.Push(this);
	};
	AutoRTFM::PushOnAbortHandler(&ThreadContext.PostInitPropertiesCheck, [this]
	{ 
		UObject* Popped = FUObjectThreadContext::Get().PostInitPropertiesCheck.Pop();
		check(this == Popped);
	});
#endif

	GetClass()->CreatePersistentUberGraphFrame(this, true);
	if (FOverridableManager* Manager = FOverridableManager::TryGet())
	{
		Manager->ClearOverrides(this);
	}
}

// Construct the UObject in the open with AutoRTFM sanitizer disabled as UObjects can be 
// reconstructed, and the re-initialization of the object can upset the AutoRTFM sanitizer.
UE_AUTORTFM_ALWAYS_OPEN_NO_SANITIZER
UObject::UObject()
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	checkf(UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(this).IsValid(), TEXT("Constructing an object %s with an invalid RemoteId. Did UObjectBase() constructor reset it?"), *GetPathName());
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	EnsureNotRetrievingVTablePtr();

	FObjectInitializer* ObjectInitializerPtr = FUObjectThreadContext::Get().TopInitializer();
	UE_CLOGF(!ObjectInitializerPtr, LogUObjectGlobals, Fatal, "%ls is not being constructed with NewObject.", *GetName());
	FObjectInitializer& ObjectInitializer = *ObjectInitializerPtr;
	UE_CLOGF(ObjectInitializer.Obj != nullptr && ObjectInitializer.Obj != this, LogUObjectGlobals, Fatal, "UObject() constructor called but it's not the object that's currently being constructed with NewObject. Maybe you are trying to construct it on the stack, which is not supported.");
	const_cast<FObjectInitializer&>(ObjectInitializer).Obj = this;
	const_cast<FObjectInitializer&>(ObjectInitializer).FinalizeSubobjectClassInitialization();
}

// Construct the UObject in the open with AutoRTFM sanitizer disabled as UObjects can be 
// reconstructed, and the re-initialization of the object can upset the AutoRTFM sanitizer.
UE_AUTORTFM_ALWAYS_OPEN_NO_SANITIZER
UObject::UObject(const FObjectInitializer& ObjectInitializer)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	checkf(UE::CoreUObject::Private::FObjectHandleUtils::GetRemoteId(this).IsValid(), TEXT("Constructing an object %s with an invalid RemoteId. Did UObjectBase() constructor reset it?"), *GetPathName());
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	EnsureNotRetrievingVTablePtr();

	UE_CLOGF(ObjectInitializer.Obj != nullptr && ObjectInitializer.Obj != this, LogUObjectGlobals, Fatal, "UObject(const FObjectInitializer&) constructor called but it's not the object that's currently being constructed with NewObject. Maybe you are trying to construct it on the stack, which is not supported.");
	const_cast<FObjectInitializer&>(ObjectInitializer).Obj = this;
	const_cast<FObjectInitializer&>(ObjectInitializer).FinalizeSubobjectClassInitialization();
}


static int32 GVerifyUObjectsAreNotFGCObjects = 1;
static FAutoConsoleVariableRef CVarVerifyUObjectsAreNotFGCObjects(
	TEXT("gc.VerifyUObjectsAreNotFGCObjects"),
	GVerifyUObjectsAreNotFGCObjects,
	TEXT("If true, the engine will throw a warning when it detects a UObject-derived class which also derives from FGCObject or any of its members is derived from FGCObject"),
	ECVF_Default
);

FObjectInitializer::FObjectInitializer()
	: Obj(nullptr)
	, ObjectArchetype(nullptr)
	, bCopyTransientsFromClassDefaults(false)
	, bShouldInitializePropsFromArchetype(false)
	, InstanceGraph(nullptr)
	, PropertyInitCallback([](){})
{
	Construct_Internal();
}	

FObjectInitializer::FObjectInitializer(UObject* InObj, const FStaticConstructObjectParameters& StaticConstructParams)
	: Obj(InObj)
	, ObjectArchetype(StaticConstructParams.Template)
	, bCopyTransientsFromClassDefaults(StaticConstructParams.bCopyTransientsFromClassDefaults)
	, bShouldInitializePropsFromArchetype(true)
	, InstanceGraph(StaticConstructParams.InstanceGraph)
	, PropertyInitCallback(StaticConstructParams.PropertyInitCallback)
{
	if (StaticConstructParams.SubobjectOverrides)
	{
		SubobjectOverrides = *StaticConstructParams.SubobjectOverrides;
	}

	Construct_Internal();
}	

FObjectInitializer::FObjectInitializer(UObject* InObj, UObject* InObjectArchetype, EObjectInitializerOptions InOptions, struct FObjectInstancingGraph* InInstanceGraph)
	: Obj(InObj)
	, ObjectArchetype(InObjectArchetype)
	  // if the SubobjectRoot NULL, then we want to copy the transients from the template, otherwise we are doing a duplicate and we want to copy the transients from the class defaults
	, bCopyTransientsFromClassDefaults(!!(InOptions & EObjectInitializerOptions::CopyTransientsFromClassDefaults))
	, bShouldInitializePropsFromArchetype(!!(InOptions & EObjectInitializerOptions::InitializeProperties))
	, InstanceGraph(InInstanceGraph)
	, PropertyInitCallback([](){})
{
	Construct_Internal();
}

void FObjectInitializer::Construct_Internal()
{
	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	// Always write to FUObjectThreadContext from the open, as writing to a 
	// memory location in both the open and closed within the same transaction
	// can lead to state corruption.
	UE_AUTORTFM_OPEN
	{
		// Mark we're in the constructor now.
		ThreadContext.IsInConstructor++;
		LastConstructedObject = ThreadContext.ConstructedObject;
		ThreadContext.ConstructedObject = Obj;
		ThreadContext.PushInitializer(this);
	};
	AutoRTFM::PushOnAbortHandler(&ThreadContext.IsInConstructor, [this]
	{ 
		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		ThreadContext.IsInConstructor--;
		check(ThreadContext.IsInConstructor >= 0);
		ThreadContext.ConstructedObject = LastConstructedObject;
	});
	AutoRTFM::PushOnAbortHandler(this, [this]
	{ 
		FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
		check(ThreadContext.TopInitializer() == this);
		ThreadContext.PopInitializer();
	});

	if (Obj)
	{
		bool bSetupObjectInitializer = GetAllowNativeComponentClassOverrides();
		if (!bSetupObjectInitializer)
		{
			// The Class of the Class may not be available during very early initialization of CoreUObject.
			UClass* ClassClass = Obj->GetClass()->GetClass();
			bSetupObjectInitializer = ClassClass && ClassClass->IsChildOf<UVerseClass>();
		}
		if (bSetupObjectInitializer)
		{
			Obj->GetClass()->SetupObjectInitializer(*this);
		}
	}

#if WITH_EDITORONLY_DATA
	if (GIsEditor && GVerifyUObjectsAreNotFGCObjects && FGCObject::GGCObjectReferencer && 
		// We can limit the test to native CDOs only
		Obj && Obj->HasAnyFlags(RF_ClassDefaultObject) && !Obj->GetClass()->HasAnyClassFlags(CLASS_CompiledFromBlueprint))
	{
		OnGCObjectCreatedHandle = FGCObject::GGCObjectReferencer->GetGCObjectAddedDelegate().AddRaw(this, &FObjectInitializer::OnGCObjectCreated);
	}
#endif // WITH_EDITORONLY_DATA
}

#if WITH_EDITORONLY_DATA
void FObjectInitializer::OnGCObjectCreated(FGCObject* InGCObject)
{
	check(Obj);
	uint8* ObjectAddress = (uint8*)Obj;
	uint8* GCObjectAddress = (uint8*)InGCObject;

	// Look for FGCObjects whose address is within the memory bounds of the object being initialized 
	if (GCObjectAddress >= ObjectAddress && GCObjectAddress < (ObjectAddress + Obj->GetClass()->GetPropertiesSize()))
	{
		// We can't report this FGCObject immediately as it's not fully constructed yet, so we're going to store it in a list for processing later
		CreatedGCObjects.Add(InGCObject);
	}
}
#endif // WITH_EDITORONLY_DATA

/**
 * Destructor for internal class to finalize UObject creation (initialize properties) after the real C++ constructor is called.
 **/
FObjectInitializer::~FObjectInitializer()
{
#if WITH_EDITORONLY_DATA
	if (OnGCObjectCreatedHandle.IsValid())
	{
		FGCObject::GGCObjectReferencer->GetGCObjectAddedDelegate().Remove(OnGCObjectCreatedHandle);
		for (FGCObject* CreatedObj : CreatedGCObjects)
		{
			// FObjectInitializer destructor runs after the UObject it initialized has had its constructors called so it's now safe to 
			// access GetReferencerName() function
			UE_LOGF(LogUObjectGlobals, Warning, "Class %ls contains an FGCObject (%ls) member or is derived from it", *Obj->GetClass()->GetPathName(), *CreatedObj->GetReferencerName());
		}
	}
#endif // WITH_EDITORONLY_DATA

	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (!bIsDeferredInitializer)
	{
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
		// Always write to FUObjectThreadContext from the open, as writing to a 
		// memory location in both the open and closed within the same transaction
		// can lead to state corruption.
		AutoRTFM::PopOnAbortHandler(&ThreadContext.IsInConstructor);
		UE_AUTORTFM_OPEN
		{
			// Let the FObjectFinders know we left the constructor.
			ThreadContext.IsInConstructor--;
			check(ThreadContext.IsInConstructor >= 0);
			ThreadContext.ConstructedObject = LastConstructedObject;
		};

		check(Obj != nullptr);

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	}
	else if (Obj == nullptr)
	{
		// the deferred initialization has already been ran, we clear Obj once 
		// PostConstructInit() has been executed
		return;
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

	// At this point the object has had its native constructor called so it's safe to be used
	Obj->ClearInternalFlags(EInternalObjectFlags::PendingConstruction);

	const bool bIsCDO = Obj->HasAnyFlags(RF_ClassDefaultObject);
	UClass* Class = Obj->GetClass();

	if ( Class != UObject::StaticClass() )
	{
		// InClass->GetClass() == NULL when InClass hasn't been fully initialized yet (during static registration)
		if ( !ObjectArchetype  && Class->GetClass() )
		{
			ObjectArchetype = Class->GetDefaultObject();
		}
	}
	else if (bIsCDO)
	{
		// for the Object CDO, make sure that we do not use an archetype
		check(ObjectArchetype == nullptr);
	}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	bool bIsPostConstructInitDeferred = false;
	if (!FBlueprintSupport::IsDeferredCDOInitializationDisabled())
	{
		if (FObjectInitializer* DeferredCopy = FDeferredObjInitializationHelper::DeferObjectInitializerIfNeeded(*this))
		{
			DeferredCopy->bIsDeferredInitializer = true;
			// make sure this wasn't mistakenly pushed into ObjectInitializers
			// (the copy constructor should have been what was invoked, 
			// which doesn't push to ObjectInitializers)
			check(FUObjectThreadContext::Get().TopInitializer() != DeferredCopy);

			bIsPostConstructInitDeferred = true;
		}
	}
	if (!bIsPostConstructInitDeferred)
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	{
		PostConstructInit();
	}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	// if we're not at the top of ObjectInitializers, then this is most 
	// likely a deferred FObjectInitializer that's a copy of one that was used 
	// in a constructor (that has already been popped)
	// We're not popping this initializer from the stack in the same place where we decrement IsInConstructor
	// because we still want to be able to access the current initializer from PostConstructInit or any of its callbacks
	if (!bIsDeferredInitializer)
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING		
	{
		// Always write to FUObjectThreadContext from the open, as writing to a 
		// memory location in both the open and closed within the same transaction
		// can lead to state corruption.
		AutoRTFM::PopOnAbortHandler(this);
		UE_AUTORTFM_OPEN
		{
			check(ThreadContext.TopInitializer() == this);
			ThreadContext.PopInitializer();
		};
	}
}

UE_AUTORTFM_ASSUME_SAFE
void FObjectInitializer::PostConstructInit()
{
	// we clear the Obj pointer at the end of this function, so if it is null 
	// then it most likely means that this is being ran for a second time
	if (Obj == nullptr)
	{
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		checkf(Obj != nullptr, TEXT("Looks like you're attempting to run FObjectInitializer::PostConstructInit() twice, and that should never happen."));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		return;
	}

	SCOPE_CYCLE_COUNTER(STAT_PostConstructInitializeProperties);
	const bool bIsCDO = Obj->HasAnyFlags(RF_ClassDefaultObject) && !Obj->HasAnyFlags(RF_ImmutableDefaultObject);
	UClass* Class = Obj->GetClass();
	UClass* SuperClass = Class->GetSuperClass();

	// This code disables the config access tracking for CDOs. It should also be disabled for Immutable CDOs. When RF_ImmutableDefaultObject is set, 
	// RF_ClassDefaultObject is also set so I only check for RF_ClassDefaultObject.
	const bool bIsAnyKindOfCDO = Obj->HasAnyFlags(RF_ClassDefaultObject);
	TOptional<UE::ConfigAccessTracking::FIgnoreScope> IgnoreConfigAccessTracking;
	if (bIsAnyKindOfCDO)
	{
		IgnoreConfigAccessTracking.Emplace();
	}

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (bIsDeferredInitializer)
	{
		const bool bIsDeferredSubObject = Obj->HasAnyFlags(RF_InheritableComponentTemplate);
		if (bIsDeferredSubObject)
		{
			// when this sub-object was created it's archetype object (the 
			// super's sub-obj) may not have been created yet (thanks cyclic 
			// dependencies). in that scenario, the component class's CDO would  
			// have been used in its place; now that we're resolving the defered 
			// sub-obj initialization we should try to update the archetype
			if (ObjectArchetype->HasAnyFlags(RF_ClassDefaultObject))
			{
				ObjectArchetype = UObject::GetArchetypeFromRequiredInfo(Class, Obj->GetOuter(), Obj->GetFName(), Obj->GetFlags());
				// NOTE: this may still be the component class's CDO (like when 
				// a component was removed from the super, without resaving the child)
			}			
		}

		UClass* ArchetypeClass = ObjectArchetype->GetClass();
		const bool bSuperHasBeenRegenerated = ArchetypeClass->HasAnyClassFlags(CLASS_NewerVersionExists);
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
		check(bIsCDO || bIsDeferredSubObject);
		check(ObjectArchetype->GetOutermost() != GetTransientPackage());
		check(!bIsCDO || (ArchetypeClass == SuperClass && !bSuperHasBeenRegenerated));
#endif // USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS

		if ( !ensureMsgf(!bSuperHasBeenRegenerated, TEXT("The archetype for %s has been regenerated, we cannot properly initialize inherited properties, as the class layout may have changed."), *Obj->GetName()) )
		{
			// attempt to complete initialization/instancing as best we can, but
			// it would not be surprising if our CDO was improperly initialized 
			// as a result...

			// iterate backwards, so we can remove elements as we go
			for (int32 SubObjIndex = ComponentInits.SubobjectInits.Num() - 1; SubObjIndex >= 0; --SubObjIndex)
			{
				FSubobjectsToInit::FSubobjectInit& SubObjInitInfo = ComponentInits.SubobjectInits[SubObjIndex];
				const FName SubObjName = SubObjInitInfo.Subobject->GetFName();

				UObject* OuterArchetype = SubObjInitInfo.Subobject->GetOuter()->GetArchetype();
				UObject* NewTemplate = OuterArchetype->GetClass()->GetDefaultSubobjectByName(SubObjName);

				if (ensure(NewTemplate != nullptr))
				{
					SubObjInitInfo.Template = NewTemplate;
				}
				else
				{
					ComponentInits.SubobjectInits.RemoveAtSwap(SubObjIndex);
				}
			}
		}
	}
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING

#if WITH_EDITORONLY_DATA
	if (bUseAnnotationsForPropertyInitializedStateTracking)
	{
		// Activate tracking of initialized property state for Verse objects.
		// The Class of the Class may not be available during very early initialization of CoreUObject.
		if (UClass* ClassClass = Class->GetClass(); ClassClass && ClassClass->IsChildOf<UVerseClass>())
		{
			AutoRTFM::ETransactionStatus Status = AutoRTFM::CloseIfTransactional([&]
			{
				UE::FInitializedPropertyValueState(Obj).ActivateTracking();
			});
			check(Status == AutoRTFM::ETransactionStatus::Executing);
		}
	}
#endif

	UE::CoreUObject::Private::ConstructPartials(*Class, Obj);

	if (bShouldInitializePropsFromArchetype)
	{
		UClass* BaseClass = (bIsCDO && !GIsDuplicatingClassForReinstancing) ? SuperClass : Class;
		if (BaseClass == NULL)
		{
			check(Class==UObject::StaticClass());
			BaseClass = Class;
		}
	
		UObject* Defaults = ObjectArchetype ? ObjectArchetype : BaseClass->GetDefaultObject(false); // we don't create the CDO here if it doesn't already exist
		InitProperties(Obj, BaseClass, Defaults, bCopyTransientsFromClassDefaults);
	}

	const bool bAllowInstancing = IsInstancingAllowed();
	bool bNeedSubobjectInstancing = InitSubobjectProperties(bAllowInstancing);

	// Restore class information if replacing native class.
	if (ObjectRestoreAfterInitProps != nullptr)
	{
		ObjectRestoreAfterInitProps->Restore();
		delete ObjectRestoreAfterInitProps;
		ObjectRestoreAfterInitProps = nullptr;
	}

	bool bNeedInstancing = false;
	// if HasAnyFlags(RF_NeedLoad), we do these steps later
#if !USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (!Obj->HasAnyFlags(RF_NeedLoad))
#else 
	// we defer this initialization in special set of cases (when Obj is a CDO 
	// and its parent hasn't been serialized yet)... in those cases, Obj (the 
	// CDO) wouldn't have had RF_NeedLoad set (not yet, because it is created 
	// from Class->GetDefualtObject() without that flag); since we've deferred
	// all this, it is likely that this flag is now present... these steps 
	// (specifically sub-object instancing) is important for us to run on the
	// CDO, so we allow all this when the bIsDeferredInitializer is true as well
	if (!Obj->HasAnyFlags(RF_NeedLoad) || bIsDeferredInitializer)
#endif // !USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	{
		if (bIsCDO || Class->HasAnyClassFlags(CLASS_PerObjectConfig))
		{
			Obj->LoadConfig(NULL, NULL, bIsCDO ? UE::LCPF_ReadParentSections : UE::LCPF_None);
		}
		if (bAllowInstancing)
		{
			// Instance subobject templates for non-cdo blueprint classes or when using non-CDO template.
			const bool bInitPropsWithArchetype = Class->GetDefaultObject(false) == NULL || Class->GetDefaultObject(false) != ObjectArchetype || Class->HasAnyClassFlags(CLASS_CompiledFromBlueprint);
			if ((!bIsCDO || bShouldInitializePropsFromArchetype) && Class->HasAnyClassFlags(CLASS_HasInstancedReference) && bInitPropsWithArchetype)
			{
				// Only blueprint generated CDOs can have their subobjects instanced.
				check(!bIsCDO || !Class->HasAnyClassFlags(CLASS_Intrinsic|CLASS_Native));

				bNeedInstancing = true;
			}
		}
	}

	// Allow custom property initialization to happen before PostInitProperties is called
	if (PropertyInitCallback)
	{
		PropertyInitCallback();
	}

	// After the call to `PropertyInitCallback` to allow the callback to modify the instancing graph
	if (bNeedInstancing || bNeedSubobjectInstancing)
	{
		InstanceSubobjects(Class, bNeedInstancing, bNeedSubobjectInstancing);
	}

	// Make sure subobjects knows that they had their properties overwritten
	for (int32 Index = 0; Index < ComponentInits.SubobjectInits.Num(); Index++)
	{
		SCOPE_CYCLE_COUNTER(STAT_PostReinitProperties);
		UObject* Subobject = ComponentInits.SubobjectInits[Index].Subobject;
		Subobject->PostReinitProperties();
	}

	for (TFunction<void()>& Callback : PropertyPostInitCallbacks)
	{
		if (Callback)
		{
			Callback();
		}
	}

	{
		SCOPE_CYCLE_COUNTER(STAT_PostInitProperties);
		Obj->PostInitProperties();
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	// Always write to FUObjectThreadContext from the open, as writing to a
	// memory location in both the open and closed within the same transaction
	// can lead to state corruption.
	AutoRTFM::PopOnAbortHandler(&ThreadContext.PostInitPropertiesCheck);
	UE_AUTORTFM_OPEN
	{
		if (ThreadContext.PostInitPropertiesCheck.IsEmpty() || ThreadContext.PostInitPropertiesCheck.Pop() != Obj)
		{
			UE_LOGF(LogUObjectGlobals, Fatal, "%ls failed to route PostInitProperties. Call Super::PostInitProperties() in %ls::PostInitProperties().", *Obj->GetClass()->GetName(), *Obj->GetClass()->GetName());
		}
	};
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	// Only invoke the class initializer if subobject instancing is enabled. Since object properties may have been default-initialized to
	// reference archetypes before instancing, archetypes could end up being mutated by compiled bytecode if executed before fixup occurs.
	if (bAllowInstancing)
	{
		Class->PostInitInstance(Obj, InstanceGraph);
	}

#if !USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	if (!Obj->HasAnyFlags(RF_NeedLoad) 
#else 
	// we defer this initialization in special set of cases (when Obj is a CDO 
	// and its parent hasn't been serialized yet)... in those cases, Obj (the 
	// CDO) wouldn't have had RF_NeedLoad set (not yet, because it is created 
	// from Class->GetDefualtObject() without that flag); since we've deferred
	// all this, it is likely that this flag is now present... we want to run 
	// all this as if the object was just created, so we check 
	// bIsDeferredInitializer as well
	if ( (!Obj->HasAnyFlags(RF_NeedLoad) || bIsDeferredInitializer)
#endif // !USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
		// if component instancing is not enabled, then we leave the components in an invalid state, which will presumably be fixed by the caller
		&& ((InstanceGraph == NULL) || InstanceGraph->IsSubobjectInstancingEnabled())) 
	{
		Obj->CheckDefaultSubobjects();
	}

	// If we want to be able to use RF_NeedInitialization from another thread
	// to know that Obj is fully constructed, then on weakly order platforms we 
	// need a fence to guarantee that the cleared flag is only visible to other threads
	// after the other initialization-related writes
	std::atomic_thread_fence(std::memory_order_release);
	// TSAN doesn't recognize fence yet, so use annotations. 
	// Any pointer would do as the fence is global, but needs to be paired with TSAN_AFTER
	// on the same pointer before consuming any object properties.
	TSAN_BEFORE(Obj); 
	Obj->ClearFlags(RF_NeedInitialization);

	// clear the object pointer so we can guard against running this function again
	Obj = nullptr;
}

bool FObjectInitializer::IsInstancingAllowed() const
{
	return (InstanceGraph == NULL) || InstanceGraph->IsSubobjectInstancingEnabled();
}

bool FObjectInitializer::InitSubobjectProperties(bool bAllowInstancing) const
{
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	bool bNeedSubobjectInstancing = bAllowInstancing && bIsDeferredInitializer;
#else 
	bool bNeedSubobjectInstancing = false;
#endif
	// initialize any subobjects, now that the constructors have run
	for (int32 Index = 0; Index < ComponentInits.SubobjectInits.Num(); Index++)
	{
		UObject* Subobject = ComponentInits.SubobjectInits[Index].Subobject;
		UObject* Template = ComponentInits.SubobjectInits[Index].Template;
		InitProperties(Subobject, Template->GetClass(), Template, false);
		if (bAllowInstancing && !Subobject->HasAnyFlags(RF_NeedLoad))
		{
			bNeedSubobjectInstancing = true;
		}
	}

	return bNeedSubobjectInstancing;
}

void FObjectInitializer::InstanceSubobjects(UClass* Class, bool bNeedInstancing, bool bNeedSubobjectInstancing) const
{
	SCOPE_CYCLE_COUNTER(STAT_InstanceSubobjects);

	FObjectInstancingGraph TempInstancingGraph;
	FObjectInstancingGraph* UseInstancingGraph = InstanceGraph ? InstanceGraph : &TempInstancingGraph;
	{
		UseInstancingGraph->AddNewObject(Obj, ObjectArchetype);
	}
	// Add any default subobjects
	for (const FSubobjectsToInit::FSubobjectInit& SubobjectInit : ComponentInits.SubobjectInits)
	{
		UseInstancingGraph->AddNewObject(SubobjectInit.Subobject, SubobjectInit.Template);
	}
	if (bNeedInstancing)
	{
		UObject* Archetype = ObjectArchetype ? ObjectArchetype : Obj->GetArchetype();
		Class->InstanceSubobjectTemplates(Obj, Archetype, Archetype ? Archetype->GetClass() : NULL, Obj, UseInstancingGraph);
	}
	if (bNeedSubobjectInstancing)
	{
		// initialize any subobjects, now that the constructors have run
		for (int32 Index = 0; Index < ComponentInits.SubobjectInits.Num(); Index++)
		{
			UObject* Subobject = ComponentInits.SubobjectInits[Index].Subobject;
			UObject* Template = ComponentInits.SubobjectInits[Index].Template;

#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
			if ( !Subobject->HasAnyFlags(RF_NeedLoad) || bIsDeferredInitializer )
#else 
			if ( !Subobject->HasAnyFlags(RF_NeedLoad) )
#endif
			{
				Subobject->GetClass()->InstanceSubobjectTemplates(Subobject, Template, Template->GetClass(), Subobject, UseInstancingGraph);
			}
		}
	}
}

UClass* FObjectInitializer::GetClass() const
{
	return Obj->GetClass();
}

// Binary initialize object properties to zero or defaults.
void FObjectInitializer::InitProperties(UObject* Obj, UClass* DefaultsClass, UObject* DefaultData, bool bCopyTransientsFromClassDefaults)
{
	SCOPE_CYCLE_COUNTER(STAT_InitProperties);

	check(DefaultsClass && Obj);

	if (FOverridableManager* Manager = FOverridableManager::TryGet())
	{
		Manager->InheritEnabledFrom(Obj, DefaultData);
	}

	UClass* Class = Obj->GetClass();

	// bool to indicate that we need to initialize any non-native properties (native ones were done when the native constructor was called by the code that created and passed in a FObjectInitializer object)
	bool bNeedInitialize = !Class->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic);

	// bool to indicate that we can use the faster PostConstructLink chain for initialization.
	bool bCanUsePostConstructLink = !bCopyTransientsFromClassDefaults && DefaultsClass == Class;

	if (Obj->HasAnyFlags(RF_NeedLoad))
	{
		bCopyTransientsFromClassDefaults = false;
	}

#if WITH_EDITORONLY_DATA
	UE::FInitializedPropertyValueState InitializedState(Obj);
	auto SetUpPropertyInitializedFlag = [InitializedState, bTrackInitializedProperties = InitializedState.IsTracking()](UObject* Defaults, FProperty* Property) mutable
	{
		if (bTrackInitializedProperties)
		{
			UE::FInitializedPropertyValueState DefaultsInitializedState(Defaults);
			const bool bTrackingDefaults = DefaultsInitializedState.IsTracking();
			for (int32 ArrayIndex = 0; ArrayIndex < Property->ArrayDim; ++ArrayIndex)
			{
				// A required property is only initialized if the copied default is initialized.
				const bool bInitialized = !Property->HasAllPropertyFlags(CPF_RequiredParm) || (bTrackingDefaults && DefaultsInitializedState.IsSet(Property, ArrayIndex));
				InitializedState.SetTo(bInitialized, Property, ArrayIndex);
			}
		}
	};
#endif // WITH_EDITORONLY_DATA

	if (!bNeedInitialize && bCanUsePostConstructLink)
	{
		// This is just a fast path for the below in the common case that we are not doing a duplicate or initializing a CDO and this is all native.
		// We only do it if the DefaultData object is NOT a CDO of the object that's being initialized. CDO data is already initialized in the
		// object's constructor.
		if (DefaultData)
		{
			if (Class->GetDefaultObject(false) != DefaultData)
			{
				for (FProperty* P = Class->PropertyLink; P; P = P->PropertyLinkNext)
				{
					bool bIsTransient = P->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient);
					if (!bIsTransient || !P->ContainsInstancedObjectProperty())
					{
						if (P->IsInContainer(DefaultsClass))
						{
							P->CopyCompleteValue_InContainer(Obj, DefaultData);
						#if WITH_EDITORONLY_DATA
							SetUpPropertyInitializedFlag(DefaultData, P);
						#endif
						}
					}
				}
			}
			else
			{
				// Copy all properties that require additional initialization (e.g. CPF_Config).
				for (FProperty* P = Class->PostConstructLink; P; P = P->PostConstructLinkNext)
				{
					bool bIsTransient = P->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient);
					if (!bIsTransient || !P->ContainsInstancedObjectProperty())
					{
						if (P->IsInContainer(DefaultsClass))
						{
							P->CopyCompleteValue_InContainer(Obj, DefaultData);
						#if WITH_EDITORONLY_DATA
							SetUpPropertyInitializedFlag(DefaultData, P);
						#endif
						}
					}
				}
			}
		}
	}
	else
	{
		// As with native classes, we must iterate through all properties (slow path) if default data is pointing at something other than the CDO.
		bCanUsePostConstructLink &= (DefaultData == Class->GetDefaultObject(false));

		UObject* ClassDefaults = bCopyTransientsFromClassDefaults ? DefaultsClass->GetDefaultObject() : NULL;	

		for (FProperty* P = bCanUsePostConstructLink ? Class->PostConstructLink : Class->PropertyLink; P; P = bCanUsePostConstructLink ? P->PostConstructLinkNext : P->PropertyLinkNext)
		{
			if (bNeedInitialize)
			{
				bNeedInitialize = InitNonNativeProperty(P, Obj);
			}

			bool bIsTransient = P->HasAnyPropertyFlags(CPF_Transient | CPF_DuplicateTransient | CPF_NonPIEDuplicateTransient);
			if (!bIsTransient || !P->ContainsInstancedObjectProperty())
			{
				if (bCopyTransientsFromClassDefaults && bIsTransient)
				{
					// This is a duplicate. The value for all transient or non-duplicatable properties should be copied
					// from the source class's defaults.
					P->CopyCompleteValue_InContainer(Obj, ClassDefaults);
				#if WITH_EDITORONLY_DATA
					SetUpPropertyInitializedFlag(ClassDefaults, P);
				#endif
				}
				else if (P->IsInContainer(DefaultsClass))
				{
					P->CopyCompleteValue_InContainer(Obj, DefaultData);
				#if WITH_EDITORONLY_DATA
					SetUpPropertyInitializedFlag(DefaultData, P);
				#endif
				}
			}
		}

		// This step is only necessary if we're not iterating the full property chain.
		if (bCanUsePostConstructLink)
		{
			// Initialize remaining property values from defaults using an explicit custom post-construction property list returned by the class object.
			Class->InitPropertiesFromCustomList((uint8*)Obj, (uint8*)DefaultData);
		}
	}
}

void FObjectInitializer::FOverrides::Add(FName InComponentName, const UClass* InComponentClass, const TArrayView<const FName>* InFullComponentPath)
{
	auto GetSubobjectPath = [InComponentName, InFullComponentPath]()
	{
		if (InFullComponentPath)
		{
			FString SubobjectPath;
			for (FName SubobjectName : *InFullComponentPath)
			{
				SubobjectPath += (SubobjectPath.IsEmpty() ? TEXT("") : TEXT("."));
				SubobjectPath += SubobjectName.ToString();
			}
			return SubobjectPath;
		}

		return InComponentName.ToString();
	};

	const int32 Index = Find(InComponentName);
	if (Index == INDEX_NONE)
	{
		FOverride& Override = Overrides.Emplace_GetRef(FOverride(InComponentName));
		Override.ComponentClass = InComponentClass;
		Override.bDoNotCreate = (InComponentClass == nullptr);
	}
	else if (InComponentClass)
	{
		if (Overrides[Index].ComponentClass)
		{
			// if a base class is asking for an override, the existing override (which we are going to use) had better be derived
			if (!IsLegalOverride(Overrides[Index].ComponentClass, InComponentClass))
			{
				UE_LOGF(LogUObjectGlobals, Error, "%ls is not a legal override for component %ls because it does not derive from %ls. Will use %ls when constructing component.",
					*Overrides[Index].ComponentClass->GetFullName(), *GetSubobjectPath(), *InComponentClass->GetFullName(), *InComponentClass->GetFullName());

				Overrides[Index].ComponentClass = InComponentClass;
			}
		}
		else
		{
			// if the existing recorded component class is null then we could either have suboverrides in which case we still want to use the class,
			// or it could be marked do not create, but since the base class may create it as non-optional we still want to record the class
			Overrides[Index].ComponentClass = InComponentClass;
		}
	}
	else
	{
		// Warn about existing overrides but the parent marking it DoNotCreate
		// Note that even if we report an error, these overrides may still get used if the component is created as non-optional
		if (Overrides[Index].ComponentClass)
		{
			UE_LOGF(LogUObjectGlobals, Error, "%ls is not a legal override for component %ls because a parent class is marking it do not create.",
				*Overrides[Index].ComponentClass->GetFullName(), *GetSubobjectPath());
		}
		if (Overrides[Index].SubOverrides)
		{
			UE_LOGF(LogUObjectGlobals, Error, "Component %ls has recorded nested subobject overrides, but won't be created because a parent class is marking it do not create.",
				*GetSubobjectPath());
		}

		Overrides[Index].bDoNotCreate = true;
	}
}

void FObjectInitializer::FOverrides::Add(FStringView InComponentPath, const UClass* InComponentClass)
{
	TArray<FName> ComponentPath;

	int32 PeriodIndex;
	while (InComponentPath.FindChar(TEXT('.'), PeriodIndex))
	{
		ComponentPath.Add(FName(PeriodIndex, InComponentPath.GetData()));
		InComponentPath.RemovePrefix(PeriodIndex+1);
	}
	ComponentPath.Add(FName(InComponentPath.Len(), InComponentPath.GetData()));

	TArrayView<const FName> PathArrayView(ComponentPath);
	Add(PathArrayView, InComponentClass, &PathArrayView);
}

void FObjectInitializer::FOverrides::Add(TArrayView<const FName> InComponentPath, const UClass* InComponentClass, const TArrayView<const FName>* InFullComponentPath)
{
	if (InComponentPath.Num() > 1)
	{
		const FName ComponentName = InComponentPath[0];
		int32 Index = Find(ComponentName);
		if (Index == INDEX_NONE)
		{
			Index = Overrides.Emplace(FOverride(ComponentName));
		}
		if (!Overrides[Index].SubOverrides)
		{
			Overrides[Index].SubOverrides = MakeUnique<FOverrides>();
		}

		Overrides[Index].SubOverrides->Add(InComponentPath.Slice(1, InComponentPath.Num()-1), InComponentClass, (InFullComponentPath ? InFullComponentPath : &InComponentPath));
	}
	else
	{
		Add(InComponentPath[0], InComponentClass, (InFullComponentPath ? InFullComponentPath : &InComponentPath));
	}
}

/**  Retrieve an override, or TClassToConstructByDefault::StaticClass or nullptr if this was removed by a derived class **/
FObjectInitializer::FOverrides::FOverrideDetails FObjectInitializer::FOverrides::Get(FName InComponentName, const UClass* ReturnType, const UClass* ClassToConstructByDefault, bool bOptional) const
{
	FOverrideDetails Result;

	const int32 Index = Find(InComponentName);
	if (Index == INDEX_NONE)
	{
		Result.Class = ClassToConstructByDefault; // no override so just do what the base class wanted
		Result.SubOverrides = nullptr;
	}
	else if (Overrides[Index].bDoNotCreate && bOptional)
	{
		Result.Class = nullptr;   // the override is of nullptr, which means "don't create this component"
		Result.SubOverrides = nullptr; // and if we're not creating this component also don't need sub-overrides
	}
	else if (Overrides[Index].ComponentClass)
	{
		if (IsLegalOverride(Overrides[Index].ComponentClass, ReturnType)) // if THE base class is asking for a T, the existing override (which we are going to use) had better be derived
		{
			Result.Class = Overrides[Index].ComponentClass; // the override is of an acceptable class, so use it

			if (Overrides[Index].bDoNotCreate)
			{
				UE_LOGF(LogUObjectGlobals, Error, "Ignored DoNotCreateDefaultSubobject for %ls as it's marked as required. Creating %ls.", *InComponentName.ToString(), *Result.Class->GetName());
			}
		}
		else
		{
			if (Overrides[Index].bDoNotCreate)
			{
				UE_LOGF(LogUObjectGlobals, Error, "Ignored DoNotCreateDefaultSubobject for %ls as it's marked as required. Creating %ls.", *InComponentName.ToString(), *ClassToConstructByDefault->GetName());
			}
			UE_LOGF(LogUObjectGlobals, Error, "%ls is not a legal override for component %ls because it does not derive from %ls. Using %ls to construct component.",
				*Overrides[Index].ComponentClass->GetFullName(), *InComponentName.ToString(), *ReturnType->GetFullName(), *ClassToConstructByDefault->GetFullName());

			Result.Class = ClassToConstructByDefault;
		}
		Result.SubOverrides = Overrides[Index].SubOverrides.Get();
	}
	else
	{
		if (Overrides[Index].bDoNotCreate)
		{
			UE_LOGF(LogUObjectGlobals, Error, "Ignored DoNotCreateDefaultSubobject for %ls as it's marked as required. Creating %ls.", *InComponentName.ToString(), *ClassToConstructByDefault->GetName());
		}

		Result.Class = ClassToConstructByDefault; // Only sub-overrides were overriden, so use the base class' desire
		Result.SubOverrides = Overrides[Index].SubOverrides.Get();
	}

	return Result;  
}
bool FObjectInitializer::FOverrides::IsLegalOverride(const UClass* DerivedComponentClass, const UClass* BaseComponentClass)
{
	if (DerivedComponentClass && BaseComponentClass && !DerivedComponentClass->IsChildOf(BaseComponentClass))
	{
		return false;
	}
	return true;
}

void FObjectInitializer::AssertIfSubobjectSetupIsNotAllowed(const FName SubobjectName) const
{
	UE_CLOGF(!bSubobjectClassInitializationAllowed, LogUObjectGlobals, Fatal,
		"%ls.%ls: Subobject class setup is only allowed in base class constructor call (in the initialization list)", Obj ? *Obj->GetFullName() : TEXT("NULL"), *SubobjectName.GetPlainNameString());
}

void FObjectInitializer::AssertIfSubobjectSetupIsNotAllowed(const FStringView SubobjectName) const
{
	UE_CLOGF(!bSubobjectClassInitializationAllowed, LogUObjectGlobals, Fatal,
		"%ls.%.*ls: Subobject class setup is only allowed in base class constructor call (in the initialization list)", Obj ? *Obj->GetFullName() : TEXT("NULL"), SubobjectName.Len(), SubobjectName.GetData());
}

void FObjectInitializer::AssertIfSubobjectSetupIsNotAllowed(TArrayView<const FName> SubobjectNames) const
{
	auto MakeSubobjectPath = [&SubobjectNames]()
	{
		FString SubobjectPath;
		for (FName SubobjectName : SubobjectNames)
		{
			SubobjectPath += (SubobjectPath.IsEmpty() ? TEXT("") : TEXT("."));
			SubobjectPath += SubobjectName.ToString();
		}
		return SubobjectPath;
	};

	UE_CLOGF(!bSubobjectClassInitializationAllowed, LogUObjectGlobals, Fatal,
		"%ls.%ls: Subobject class setup is only allowed in base class constructor call (in the initialization list)", Obj ? *Obj->GetFullName() : TEXT("NULL"), *MakeSubobjectPath());
}

#if DO_CHECK
void CheckIsClassChildOf_Internal(const UClass* Parent, const UClass* Child)
{
	// This is a function to avoid platform compilation issues
	checkf(Child, TEXT("NewObject called with a nullptr class object"));
	checkf(Child->IsChildOf(Parent), TEXT("NewObject called with invalid class, %s must be a child of %s"), *Child->GetName(), *Parent->GetName());
}
#endif

UObject* DuplicateObject_Internal(UClass* Class, const UObject* SourceObject, UObject* Outer, FName Name)
{
	if (SourceObject != nullptr)
	{
		if (Outer == nullptr || Outer == INVALID_OBJECT)
		{
			Outer = (UObject*)GetTransientOuterForRename(Class);
		}
		return StaticDuplicateObject(SourceObject,Outer,Name);
	}
	return nullptr;
}

FStaticConstructObjectParameters::FStaticConstructObjectParameters(const UClass* InClass)
	: Class(InClass)
	, Outer((UObject*)GetTransientPackage())
{
}

UObject* StaticConstructObject_Internal(const FStaticConstructObjectParameters& Params)
{
	const UClass* InClass = Params.Class;
	UObject* InOuter = Params.Outer;
	const FName& InName = Params.Name;
	EObjectFlags InFlags = Params.SetFlags;
	UObject* InTemplate = Params.Template;
	int32 SerialNumber = Params.SerialNumber;
	FRemoteObjectId RemoteId;

	LLM_SCOPE(ELLMTag::UObject);
	LLM_SCOPE_BYTAG(UObject_StaticConstructObjectInternal);

	SCOPE_CYCLE_COUNTER(STAT_ConstructObject);
	UObject* Result = NULL;

#if WITH_EDITORONLY_DATA
	// Check if we can construct the object: you can construct the object if its a package (InOuter is null) or the package the object is created in is not currently saving
	bool bCanConstruct = InOuter == nullptr || !UE::IsSavingPackage(Params.ExternalPackage ? Params.ExternalPackage : InOuter->GetPackage()); 
	UE_CLOGF(!bCanConstruct, LogUObjectGlobals, Fatal, "Illegal call to StaticConstructObject() while serializing object data! (Object will not be saved!)");
#endif

	checkf(!InTemplate || InTemplate->IsA(InClass) || (InFlags & RF_ClassDefaultObject), TEXT("StaticConstructObject %s is not an instance of class %s and it is not a CDO."), *GetFullNameSafe(InTemplate), *GetFullNameSafe(InClass)); // template must be an instance of the class we are creating, except CDOs

	// Subobjects are always created in the constructor, no need to re-create them unless their archetype != CDO or they're blueprint generated.
	// If the existing subobject is to be re-used it can't have BeginDestroy called on it so we need to pass this information to StaticAllocateObject.	
	const bool bIsNativeClass = InClass->HasAnyClassFlags(CLASS_Native | CLASS_Intrinsic);
	const bool bIsNativeFromCDO = bIsNativeClass &&
		(	
			!InTemplate || 
			(InName != NAME_None && (Params.bAssumeTemplateIsArchetype || InTemplate == UObject::GetArchetypeFromRequiredInfo(InClass, InOuter, InName, InFlags)))
			);

	const bool bCanRecycleSubobjects = bIsNativeFromCDO && (!(InFlags & RF_DefaultSubObject) || !FUObjectThreadContext::Get().IsInConstructor);

	
#if UE_WITH_REMOTE_OBJECT_HANDLE
	RemoteId = Params.RemoteId;
#endif

	FGCReconstructionGuard GCGuard;
	bool bRecycledSubobject = false;	
	Result = StaticAllocateObject(InClass, InOuter, InName, InFlags, Params.InternalSetFlags, bCanRecycleSubobjects, &bRecycledSubobject, Params.ExternalPackage, SerialNumber, RemoteId, &GCGuard);
	check(Result != nullptr);
	// Don't call the constructor on recycled subobjects, they haven't been destroyed.
	if (!bRecycledSubobject)
	{		
		STAT(FScopeCycleCounterUObject ConstructorScope(InClass->GetFName().IsNone() ? nullptr : InClass, GET_STATID(STAT_ConstructObject)));
		(*InClass->ClassConstructor)(FObjectInitializer(Result, Params));
	}
	// StaticAllocateObject might have locked GCGuard but it can only be unlocked after the object has been fully constructed so unlock it here
	GCGuard.Unlock();

	if (GIsEditor && 
		// Do not consider object creation in transaction if the object is marked as async or in being async loaded 
		!Result->HasAnyInternalFlags(EInternalObjectFlags::Async | EInternalObjectFlags_AsyncLoading) &&
		// Read GUndo only if not having Async flags set to avoid making TSAN unhappy that we're trying to read an unsynchronized global
		GUndo &&
		(InFlags & RF_Transactional) && !(InFlags & RF_NeedLoad) && 
		!InClass->IsChildOf(UField::StaticClass())
		)
	{
		// Set RF_PendingKill and update the undo buffer so an undo operation will set RF_PendingKill on the newly constructed object.
		Result->MarkAsGarbage();
		SaveToTransactionBuffer(Result, false);
		Result->ClearGarbage();
	}

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectConstructed.Broadcast(Result);
#endif
	return Result;
}

void FObjectInitializer::AssertIfInConstructor(UObject* Outer, const TCHAR* ErrorMessage)
{
	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	UE_CLOGF(ThreadContext.IsInConstructor && Outer == ThreadContext.ConstructedObject, LogUObjectGlobals, Fatal, "%ls", ErrorMessage);
}

FObjectInitializer& FObjectInitializer::Get()
{
	FUObjectThreadContext& ThreadContext = FUObjectThreadContext::Get();
	UE_CLOGF(!ThreadContext.IsInConstructor, LogUObjectGlobals, Fatal, "FObjectInitializer::Get() can only be used inside of UObject-derived class constructor.");
	return ThreadContext.TopInitializerChecked();
}

/**
 * Stores the object flags for all objects in the tracking array.
 */
void FScopedObjectFlagMarker::SaveObjectFlags()
{
	StoredObjectFlags.Empty();

	for (FThreadSafeObjectIterator It; It; ++It)
	{
		UObject* Obj = *It;
		StoredObjectFlags.Add(*It, FStoredObjectFlags(Obj->GetFlags(), Obj->GetInternalFlags()));
	}
}

/**
 * Restores the object flags for all objects from the tracking array.
 */
void FScopedObjectFlagMarker::RestoreObjectFlags()
{
	for (TMap<UObject*, FStoredObjectFlags>::TIterator It(StoredObjectFlags); It; ++It)
	{
		UObject* Object = It.Key();
		FStoredObjectFlags& PreviousObjectFlags = It.Value();

		// clear all flags, first clear the mirrored flags as we don't allow clearing them through ClearFlags
		Object->ClearGarbage(); // The currently mirrored flags are mutually exclusive and this will take care of both
		Object->ClearFlags(RF_AllFlags);
		Object->ClearInternalFlags(EInternalObjectFlags_AllFlags);

		// then reset the ones that were originally set
		if (!!(PreviousObjectFlags.InternalFlags & EInternalObjectFlags::Garbage) || !!(PreviousObjectFlags.Flags & RF_MirroredGarbage))
		{
			// Note that once an object is marked as Garbage (both in object and internal flags) it can't be marked as PendingKill and vice versa
			checkf(!!(PreviousObjectFlags.Flags & RF_MirroredGarbage), TEXT("%s had internal mirrored flag set but it was not matched in object flags"), *Object->GetFullName());
			checkf(!!(PreviousObjectFlags.InternalFlags & EInternalObjectFlags::Garbage), TEXT("%s had object mirrored flag set but it was not matched in internal flags"), *Object->GetFullName());
			Object->MarkAsGarbage();
		}
		Object->SetFlags(PreviousObjectFlags.Flags);
		Object->SetInternalFlags(PreviousObjectFlags.InternalFlags);
	}
}

void ConstructorHelpers::FailedToFind(const TCHAR* ObjectToFind)
{
	FObjectInitializer* CurrentInitializer = FUObjectThreadContext::Get().TopInitializer();
	const FString Message = FString::Printf(TEXT("CDO Constructor (%s): Failed to find %s\n"),
		(CurrentInitializer && CurrentInitializer->GetClass()) ? *CurrentInitializer->GetClass()->GetName() : TEXT("Unknown"),
		ObjectToFind);
	FPlatformMisc::LowLevelOutputDebugString(*Message);
#if !NO_LOGGING
	if (UE_LOG_ACTIVE(LogUObjectGlobals, Error))
	{
		UClass::GetDefaultPropertiesFeedbackContext().Log(LogUObjectGlobals.GetCategoryName(), ELogVerbosity::Error, *Message);
	}
#endif
}

void ConstructorHelpers::CheckFoundViaRedirect(UObject *Object, const FString& PathName, const TCHAR* ObjectToFind)
{
	UObjectRedirector* Redir = FindObject<UObjectRedirector>(nullptr, *PathName);
	if (Redir && Redir->DestinationObject == Object)
	{
		FString NewString = Object->GetFullName();
		NewString.ReplaceInline(TEXT(" "), TEXT("'"), ESearchCase::CaseSensitive);
		NewString += TEXT("'");

		FObjectInitializer* CurrentInitializer = FUObjectThreadContext::Get().TopInitializer();
		const FString Message = FString::Printf(TEXT("CDO Constructor (%s): Followed redirector (%s), change code to new path (%s)\n"),
			(CurrentInitializer && CurrentInitializer->GetClass()) ? *CurrentInitializer->GetClass()->GetName() : TEXT("Unknown"),
			ObjectToFind, *NewString);

		FPlatformMisc::LowLevelOutputDebugString(*Message);
#if !NO_LOGGING
		if (UE_LOG_ACTIVE(LogUObjectGlobals, Warning))
		{
			UClass::GetDefaultPropertiesFeedbackContext().Log(LogUObjectGlobals.GetCategoryName(), ELogVerbosity::Warning, *Message);
		}
#endif
	}
}

void ConstructorHelpers::CheckIfIsInConstructor(const TCHAR* ObjectToFind)
{
	auto& ThreadContext = FUObjectThreadContext::Get();
	UE_CLOGF(!ThreadContext.IsInConstructor, LogUObjectGlobals, Fatal, "FObjectFinders can't be used outside of constructors to find %ls", ObjectToFind);
}

void ConstructorHelpers::StripObjectClass( FString& PathName, bool bAssertOnBadPath /*= false */ )
{	
	int32 NameStartIndex = INDEX_NONE;
	PathName.FindChar( TCHAR('\''), NameStartIndex );
	if( NameStartIndex != INDEX_NONE )
	{
		int32 NameEndIndex = INDEX_NONE;
		PathName.FindLastChar( TCHAR('\''), NameEndIndex );
		if(NameEndIndex > NameStartIndex)
		{
			PathName.MidInline( NameStartIndex+1, NameEndIndex-NameStartIndex-1, EAllowShrinking::No);
		}
		else
		{
			UE_CLOGF( bAssertOnBadPath, LogUObjectGlobals, Fatal, "Bad path name: %ls, missing \' or an incorrect format", *PathName );
		}
	}
}

void ConstructorHelpers_StripObjectClass2(FStringBuilderBase& PathName, bool bAssertOnBadPath /*= false */ )
{
	int32 NameStartIndex = INDEX_NONE;
	PathName.ToView().FindChar( TCHAR('\''), NameStartIndex );
	if( NameStartIndex != INDEX_NONE )
	{
		int32 NameEndIndex = INDEX_NONE;
		PathName.ToView().FindLastChar( TCHAR('\''), NameEndIndex );
		if(NameEndIndex > NameStartIndex)
		{
			TStringBuilder<256> Temp;
			Temp.Append(PathName.GetData() + NameStartIndex + 1, NameEndIndex - NameStartIndex - 1);
			PathName.Reset();
			PathName.Append(Temp);
		}
		else
		{
			UE_CLOGF( bAssertOnBadPath, LogUObjectGlobals, Fatal, "Bad path name: %ls, missing \' or an incorrect format", *PathName );
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FReferenceCollectorArchive::FReferenceCollectorArchive(const UObject* InSerializingObject, FReferenceCollector& InCollector)
: SerializingObject(InSerializingObject)
, Collector(InCollector)
{
	ArIsObjectReferenceCollector = true;
	this->SetIsPersistent(InCollector.IsIgnoringTransient());
	ArIgnoreArchetypeRef = InCollector.IsIgnoringArchetypeRef();
}

class FPropertyTrackingReferenceCollectorArchive : public FReferenceCollectorArchive
{
public:
	using FReferenceCollectorArchive::FReferenceCollectorArchive;

	virtual FArchive& operator<<(UObject*& Object) override
	{
		if (Object)
		{
			FReferenceCollector& CurrentCollector = GetCollector();
			FProperty* OldCollectorSerializedProperty = CurrentCollector.GetSerializedProperty();
			CurrentCollector.SetSerializedProperty(GetSerializedProperty());
			FReferenceCollector::AROPrivate::AddReferencedObject(CurrentCollector, Object, GetSerializingObject(), GetSerializedProperty());
			CurrentCollector.SetSerializedProperty(OldCollectorSerializedProperty);
		}
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& Object) override
	{
		if (IsObjectHandleResolved_ForGC(Object.GetHandle()) && Object)
		{
			// NOTE: This is deliberately not triggering access tracking as that is an undesirable overhead
			//		 during garbage collect and GC reference collection is not meant to be trackable.
			UObject*& RawObjectPointer = *(UObject**)&Object.GetHandleRef();
			*this << RawObjectPointer;
		}
		return *this;
	}

};

FReferenceCollector::~FReferenceCollector() = default;

void FReferenceCollector::AddStableReference(UObject** Object)
{
	AROPrivate::AddReferencedObject(*this, *Object);
}

void FReferenceCollector::AddStableReferenceArray(TArray<UObject*>* Array)
{
	AROPrivate::AddReferencedObjects(*this, *Array); 
}

void FReferenceCollector::AddStableReferenceSet(TSet<UObject*>* Objects)
{
	AROPrivate::AddReferencedObjects(*this, *Objects);
}

void FReferenceCollector::AddStableReference(TObjectPtr<UObject>* Object)
{
	AddReferencedObject(*Object);
}

void FReferenceCollector::AddStableReferenceArray(TArray<TObjectPtr<UObject>>* Array)
{
	AddReferencedObjects(*Array); 
}

void FReferenceCollector::AddStableReferenceSet(TSet<TObjectPtr<UObject>>* Objects)
{
	AddReferencedObjects(*Objects);
}

void FReferenceCollector::AddReferencedObjects(const UScriptStruct*& ScriptStruct, void* StructMemory, const UObject* ReferencingObject /*= nullptr*/, const FProperty* ReferencingProperty /*= nullptr*/)
{
	AROPrivate::AddReferencedObjects(*this, ScriptStruct, StructMemory, ReferencingObject, ReferencingProperty);
}

void FReferenceCollector::AddReferencedObjects(TWeakObjectPtr<const UScriptStruct>& ScriptStruct, void* Instance, const UObject* ReferencingObject, const FProperty* ReferencingProperty)
{
	const UScriptStruct* Ptr = ScriptStruct.GetEvenIfUnreachable();
	AROPrivate::AddReferencedObjects(*this, Ptr, Instance, ReferencingObject, ReferencingProperty);	 
	ScriptStruct = Ptr;
}

void FReferenceCollector::AddReferencedObjects(TObjectPtr<const UScriptStruct>& ScriptStruct, void* Instance, const UObject* ReferencingObject, const FProperty* ReferencingProperty)
{
	AROPrivate::AddReferencedObjects(*this, UE::Core::Private::Unsafe::Decay(ScriptStruct), Instance, ReferencingObject, ReferencingProperty);
}

void FReferenceCollector::AddReferencedObject(FWeakObjectPtr& P, const UObject* ReferencingObject, const FProperty* ReferencingProperty)
{
	UObject* Ptr = P.GetEvenIfUnreachable();
	AROPrivate::AddReferencedObject(*this, Ptr, ReferencingObject, ReferencingProperty);
	P = Ptr;
}

void FReferenceCollector::AROPrivate::AddReferencedObjects(FReferenceCollector& Coll,
																													 const UScriptStruct*& ScriptStruct,
																													 void* StructMemory,
																													 const UObject* ReferencingObject /*= nullptr*/, const FProperty* ReferencingProperty /*= nullptr*/)
{
	check(ScriptStruct != nullptr);
	check(StructMemory != nullptr);

	AROPrivate::AddReferencedObject(Coll, ScriptStruct, ReferencingObject, ReferencingProperty);

	// If the script struct explicitly provided an implementation of AddReferencedObjects, make sure to capture its referenced objects
	if (ScriptStruct->StructFlags & STRUCT_AddStructReferencedObjects)
	{
		ScriptStruct->GetCppStructOps()->AddStructReferencedObjects()(StructMemory, Coll);
	}

	Coll.AddPropertyReferences(ScriptStruct, StructMemory, ReferencingObject);
}

void FReferenceCollector::HandleObjectReferences(FObjectPtr* InObjects, const int32 ObjectNum, const UObject* InReferencingObject, const FProperty* InReferencingProperty)
{
	for (int32 ObjectIndex = 0; ObjectIndex < ObjectNum; ++ObjectIndex)
	{
		FObjectPtr& Object = InObjects[ObjectIndex];
		if (IsObjectHandleResolved_ForGC(Object.GetHandle()))
		{
			HandleObjectReference(*reinterpret_cast<UObject**>(&Object), InReferencingObject, InReferencingProperty);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

enum class EPropertyCollectFlags : uint32
{
	None				= 0,
	SkipTransient		= 1 << 0,		
	NeedsReferencer		= 1 << 1,
	CallStructARO		= 1 << 2,
	OnlyObjectProperty	= 1 << 3,
};
ENUM_CLASS_FLAGS(EPropertyCollectFlags);

static constexpr EPropertyCollectFlags AllCollectorFlags =  EPropertyCollectFlags::SkipTransient | EPropertyCollectFlags::NeedsReferencer;
FORCEINLINE EPropertyCollectFlags GetCollectorPropertyFlags(FReferenceCollector& Collector)
{
	return (Collector.IsIgnoringTransient() ? EPropertyCollectFlags::SkipTransient : EPropertyCollectFlags::None)
		 | (Collector.NeedsPropertyReferencer() ? EPropertyCollectFlags::NeedsReferencer : EPropertyCollectFlags::None);
}

FORCEINLINE constexpr EPropertyFlags GetPropertyFlagsToSkip(EPropertyCollectFlags CollectFlags)
{
	return CPF_SkipSerialization | (EnumHasAnyFlags(CollectFlags, EPropertyCollectFlags::SkipTransient) ? CPF_Transient : CPF_None);
}


/** Core property types with weak references */
static constexpr EClassCastFlags WeakCastFlags =			CASTCLASS_FWeakObjectProperty | 
															CASTCLASS_FLazyObjectProperty |
															CASTCLASS_FSoftObjectProperty |
															CASTCLASS_FDelegateProperty |
															CASTCLASS_FMulticastDelegateProperty;

static constexpr EClassCastFlags ObjectCastFlags =			CASTCLASS_FObjectProperty;

static constexpr EClassCastFlags OtherStrongCastFlags =		CASTCLASS_FInterfaceProperty |
															CASTCLASS_FFieldPathProperty;

/** Core property types with strong references */
static constexpr EClassCastFlags StrongCastFlags =			ObjectCastFlags | OtherStrongCastFlags;

/** Core property types with neither weak nor strong references */
static constexpr EClassCastFlags UnreferencingCastFlags =	CASTCLASS_FByteProperty |
															CASTCLASS_FInt8Property |
															CASTCLASS_FIntProperty |
															CASTCLASS_FFloatProperty |
															CASTCLASS_FUInt64Property |
															CASTCLASS_FUInt32Property |
															CASTCLASS_FNameProperty |
															CASTCLASS_FStrProperty |
															CASTCLASS_FBoolProperty |
															CASTCLASS_FUInt16Property |
															CASTCLASS_FInt64Property |
															CASTCLASS_FNumericProperty |
															CASTCLASS_FTextProperty |
															CASTCLASS_FInt16Property |
															CASTCLASS_FDoubleProperty |
															CASTCLASS_FEnumProperty |
															CASTCLASS_FLargeWorldCoordinatesRealProperty;

FORCEINLINE constexpr EClassCastFlags GetCastFlagsToSkip(EPropertyCollectFlags CollectFlags)
{
	return WeakCastFlags | (EnumHasAnyFlags(CollectFlags, EPropertyCollectFlags::OnlyObjectProperty) ? OtherStrongCastFlags : CASTCLASS_None);
}

FORCEINLINE constexpr bool MayContainStrongReference(EClassCastFlags CastFlags)
{
	return !EnumHasAnyFlags(CastFlags, UnreferencingCastFlags | WeakCastFlags);
}

FORCEINLINE constexpr bool MayContainStrongReference(const FProperty& Property)
{
	return MayContainStrongReference(static_cast<EClassCastFlags>(Property.GetClass()->GetCastFlags()));
}

template<EPropertyCollectFlags CollectFlags>
void CollectPropertyReferences(FReferenceCollector& Collector, FProperty& Property, void* Instance, const UObject* Referencer);

template<EPropertyCollectFlags CollectFlags, /* UStruct or UScriptStruct */ class StructType>
static void CollectStructReferences(FReferenceCollector& Collector, const StructType* Struct, void* Instance, const UObject* Referencer)
{
	// The FProperty instance might start in the middle of a cache line	
	static constexpr uint32 ExtraPrefetchBytes = PLATFORM_CACHE_LINE_SIZE - /* min alignment */ 16;
	// Prefetch vtable, PropertyFlags and NextRef. NextRef comes last.
	static constexpr uint32 PropertyPrefetchBytes = offsetof(FProperty, NextRef) + ExtraPrefetchBytes;

	FPlatformMisc::PrefetchBlock(Struct->RefLink, PropertyPrefetchBytes);
	
	if constexpr (EnumHasAnyFlags(CollectFlags, EPropertyCollectFlags::CallStructARO) && std::is_same_v<StructType, UScriptStruct>)
	{
		if (Struct->StructFlags & STRUCT_AddStructReferencedObjects)
		{
			Struct->GetCppStructOps()->AddStructReferencedObjects()(Instance, Collector);
		}
	}

	for (FProperty* It = Struct->RefLink; It; It = It->NextRef)
	{
		FPlatformMisc::PrefetchBlock(It->NextRef, PropertyPrefetchBytes);
		CollectPropertyReferences<CollectFlags>(Collector, *It, Instance, Referencer);
	}
}

template<EPropertyCollectFlags CollectFlags>
void CollectArrayReferences(FReferenceCollector& Collector, FArrayProperty& Property, void* Instance, const UObject* Referencer)
{
	FProperty& InnerProperty = *Property.Inner;
	EClassCastFlags InnerCastFlags = static_cast<EClassCastFlags>(InnerProperty.GetClass()->GetCastFlags());
	if (MayContainStrongReference(InnerCastFlags))
	{
		bool bIsReferenceArray = EnumHasAnyFlags(InnerCastFlags, ObjectCastFlags) &
								!EnumHasAnyFlags(Property.ArrayFlags, EArrayPropertyFlags::UsesMemoryImageAllocator);
		if (bIsReferenceArray && !EnumHasAnyFlags(CollectFlags, EPropertyCollectFlags::NeedsReferencer))
		{
			Collector.AddStableReferenceArray(reinterpret_cast<TArray<TObjectPtr<UObject>>*>(Instance));
		}
		else if (FScriptArrayHelper Helper(&Property, Instance); int32 Num = Helper.Num())
		{
			if (bIsReferenceArray)
			{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_REMOTE_OBJECT_HANDLE
				if (InnerProperty.HasAnyPropertyFlags(CPF_TObjectPtr))
				{
					Collector.AddReferencedObjects(*reinterpret_cast<TArray<TObjectPtr<UObject>>*>(Instance), Referencer, &Property);
				}
				else
#endif
				{
					FReferenceCollector::AROPrivate::AddReferencedObjects(Collector, *reinterpret_cast<TArray<UObject*>*>(Instance), Referencer, &Property);
				}
			}
			else if (EnumHasAnyFlags(InnerCastFlags, CASTCLASS_FStructProperty))
			{
				for (int32 Idx = 0; Idx < Num; ++Idx)
				{
					CollectStructReferences<CollectFlags>(Collector, static_cast<FStructProperty&>(InnerProperty).Struct.Get(), Helper.GetRawPtr(Idx), Referencer);
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < Num; ++Idx)
				{
					CollectPropertyReferences<CollectFlags>(Collector, InnerProperty, Helper.GetRawPtr(Idx), Referencer);
				}
			}
			
		}
	}
}

template<EPropertyCollectFlags CollectFlags>
static void CollectMapReferences(FReferenceCollector& Collector, FMapProperty& Property, void* Instance, const UObject* Referencer)
{
	FScriptMapHelper MapHelper(&Property, Instance);

	if (MapHelper.Num() == 0)
	{
		return;
	}

	const bool bCollectKeys = MayContainStrongReference(*MapHelper.GetKeyProperty());
	if (bCollectKeys)
	{
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			CollectPropertyReferences<CollectFlags>(Collector, *MapHelper.GetKeyProperty(), MapHelper.GetPairPtr(It), Referencer);
		}
	}

	const bool bCollectValues = MayContainStrongReference(*MapHelper.GetValueProperty());
	if (bCollectValues)
	{
		for (FScriptMapHelper::FIterator It(MapHelper); It; ++It)
		{
			CollectPropertyReferences<CollectFlags>(Collector, *MapHelper.GetValueProperty(), MapHelper.GetPairPtr(It), Referencer);
		}
	}
}

template<EPropertyCollectFlags CollectFlags>
void CollectSetReferences(FReferenceCollector& Collector, FSetProperty& Property, void* Instance, const UObject* Referencer)
{
	FScriptSetHelper SetHelper(&Property, Instance);
	if (MayContainStrongReference(*SetHelper.GetElementProperty()))
	{
		for (FScriptSetHelper::FIterator It(SetHelper); It; ++It)
		{
			CollectPropertyReferences<CollectFlags>(Collector, *SetHelper.GetElementProperty(), SetHelper.GetElementPtr(It), Referencer);
		}
	}
}

template<EPropertyCollectFlags CollectFlags>
void CollectOptionalReference(FReferenceCollector& Collector, FOptionalProperty& Property, void* Instance, const UObject* Referencer)
{
	FProperty& InnerProperty = *Property.GetValueProperty();
	EClassCastFlags InnerCastFlags = static_cast<EClassCastFlags>(InnerProperty.GetClass()->GetCastFlags());
	if (MayContainStrongReference(InnerCastFlags))
	{
		if (void* ValueInstance = Property.GetValuePointerForReplaceIfSet(Instance))
		{
			CollectPropertyReferences<CollectFlags>(Collector, InnerProperty, ValueInstance, Referencer);			
		}
	}
}


// Process FObjectProperty reference
template<EPropertyCollectFlags CollectFlags>
FORCEINLINE_DEBUGGABLE void CollectObjectReference(FReferenceCollector& Collector, FProperty& Property, void* Value, const UObject* Referencer)
{
	UObject*& Reference = *reinterpret_cast<UObject**>(Value);
	if constexpr (EnumHasAnyFlags(CollectFlags, EPropertyCollectFlags::NeedsReferencer))
	{
		// Sync reference processors will inspect Reference immediately so might as well avoid virtual call
		if ((!!Reference) & IsObjectHandleResolved_ForGC(*reinterpret_cast<FObjectHandle*>(Value))) //-V792
		{
			FReferenceCollector::AROPrivate::AddReferencedObject(Collector, Reference, Referencer, &Property);
		}
	}
	else
	{
		// Allows batch reference processor to queue up Reference and prefetch before accessing it
		Collector.AddStableReference(&ObjectPtrWrap(Reference));
	}
}

// Process stack reference synchronously and return true if reference got nulled out
FORCEINLINE_DEBUGGABLE bool CollectStackReference(FReferenceCollector& Collector, FProperty& Property, UObject*& Reference, const UObject* Referencer)
{
	if (Reference)
	{
		FReferenceCollector::AROPrivate::AddReferencedObject(Collector, Reference, Referencer, &Property);
		return !Reference;
	}
	return false;
}

FORCENOINLINE static void CollectInterfaceReference(FReferenceCollector& Collector, FInterfaceProperty& Property, FScriptInterface& Interface, const UObject* Referencer)
{
	// Handle reference synchronously and update interface if reference was nulled out
	UObject*& Ref = UE::Core::Private::Unsafe::Decay(Interface.GetObjectRef());
	if (CollectStackReference(Collector, Property, Ref, Referencer))
	{
		Interface.SetInterface(nullptr);
	}
}

FORCENOINLINE static void CollectFieldPathReference(FReferenceCollector& Collector, FFieldPathProperty& Property, FFieldPath& FieldPath, const UObject* Referencer)
{
	if (FUObjectItem* FieldOwnerItem = FGCInternals::GetResolvedOwner(FieldPath))
	{
		// Handle reference synchronously and update field path if reference was nulled out
		UObject* Owner = static_cast<UObject*>(FieldOwnerItem->GetObject());
		if (CollectStackReference(Collector, Property, Owner, Referencer))
		{
			FGCInternals::ClearCachedField(FieldPath);
		}
	}
}

template<EPropertyCollectFlags CollectFlags>
void CollectPropertyReferences(FReferenceCollector& Collector, FProperty& Property, void* Instance, const UObject* Referencer)
{
	FFieldClass* Class = Property.GetClass();
	const int32 ArrayDim = Property.ArrayDim;
	EPropertyFlags PropertyFlags = Property.GetPropertyFlags();
	const EClassCastFlags CastFlags = static_cast<EClassCastFlags>(Class->GetCastFlags());
	
	if (EnumHasAnyFlags(CastFlags, GetCastFlagsToSkip(CollectFlags)) |			//-V792
		EnumHasAnyFlags(PropertyFlags, GetPropertyFlagsToSkip(CollectFlags)))
	{
		return;
	}

	int32 Idx = 0;
	do
	{
		void* Value = Property.ContainerPtrToValuePtr<void>(Instance, Idx);

		if (EnumHasAnyFlags(CastFlags, ObjectCastFlags))
		{
			CollectObjectReference<CollectFlags>(Collector, Property, Value, Referencer);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FArrayProperty))
		{
			CollectArrayReferences<CollectFlags>(Collector, static_cast<FArrayProperty&>(Property), Value, Referencer);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FStructProperty))
		{
			CollectStructReferences<CollectFlags>(Collector, static_cast<FStructProperty&>(Property).Struct.Get(), Value, Referencer);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FMapProperty))
		{	
			CollectMapReferences<CollectFlags>(Collector, static_cast<FMapProperty&>(Property), Value, Referencer);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FSetProperty))
		{	
			CollectSetReferences<CollectFlags>(Collector, static_cast<FSetProperty&>(Property), Value, Referencer);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FFieldPathProperty))
		{	
			CollectFieldPathReference(Collector, static_cast<FFieldPathProperty&>(Property), *reinterpret_cast<FFieldPath*>(Value), Referencer);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FInterfaceProperty))
		{	
			CollectInterfaceReference(Collector, static_cast<FInterfaceProperty&>(Property), *reinterpret_cast<FScriptInterface*>(Value), Referencer);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FOptionalProperty))
		{
			CollectOptionalReference<CollectFlags>(Collector, static_cast<FOptionalProperty&>(Property), Value, Referencer);
		}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FVValueProperty))
		{
			Collector.AddReferencedVerseValue(*static_cast<Verse::TWriteBarrier<Verse::VValue>*>(Value), Referencer, &Property);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FVRestValueProperty))
		{
			Collector.AddReferencedVerseValue(*static_cast<Verse::VRestValue*>(Value), Referencer, &Property);
		}
		else if (EnumHasAnyFlags(CastFlags, CASTCLASS_FVCellProperty))
		{
			Collector.AddReferencedVerseValue(*static_cast<Verse::TWriteBarrier<Verse::VCell>*>(Value), Referencer, &Property);
		}
#endif
		else
		{
			// Fallback to virtual SerializeItem dispatch inside SerializeBin
			// for certain Epic plugins that actually add new FProperty types (not recommended)
			checkf(MayContainStrongReference(CastFlags), TEXT("Missing code to collect references from %s properties (%llx). "
				"Core property types part of RefLink chain / overloading ContainsObjectReference should be handled above."),
				*Class->GetFName().ToString(), CastFlags);

			FReferenceCollectorArchive& Archive = Collector.GetVerySlowReferenceCollectorArchive();

			if constexpr (EnumHasAnyFlags(CollectFlags, EPropertyCollectFlags::NeedsReferencer))
			{
				FVerySlowReferenceCollectorArchiveScope CollectorScope(Archive, Referencer, &Property);
				Property.SerializeItem(FStructuredArchiveFromArchive(Archive).GetSlot(), Value, /* defaults */ nullptr);
			}
			else
			{
				Property.SerializeItem(FStructuredArchiveFromArchive(Archive).GetSlot(), Value, /* defaults */ nullptr);
			}			
		}
	}
	while (++Idx < ArrayDim);
}

//////////////////////////////////////////////////////////////////////////

template<EPropertyCollectFlags NonCollectorFlag, /* UStruct or UScriptStruct */ class StructType>
FORCEINLINE_DEBUGGABLE void CallCollectStructReferences(FReferenceCollector& Collector, const StructType* Struct, void* Instance, const UObject* Referencer)
{
	static_assert(!EnumHasAnyFlags(NonCollectorFlag, AllCollectorFlags));
	static_assert(AllCollectorFlags == EPropertyCollectFlags(3));

	EPropertyCollectFlags CollectorFlags = GetCollectorPropertyFlags(Collector);

	using Func = void (*)(FReferenceCollector&, const StructType*, void*, const UObject*);
	static constexpr Func Funcs[] = 
	{
		&CollectStructReferences<NonCollectorFlag | EPropertyCollectFlags(0), StructType>,
		&CollectStructReferences<NonCollectorFlag | EPropertyCollectFlags(1), StructType>,
		&CollectStructReferences<NonCollectorFlag | EPropertyCollectFlags(2), StructType>,
		&CollectStructReferences<NonCollectorFlag | EPropertyCollectFlags(3), StructType>,
	};

	uint32 Idx = static_cast<uint32>(CollectorFlags);
	check(Idx < UE_ARRAY_COUNT(Funcs));
	(*Funcs[Idx])(Collector, Struct, Instance, Referencer);
}

void FReferenceCollector::AddPropertyReferences(const UStruct* Struct, void* Instance, const UObject* ReferencingObject)
{
	CallCollectStructReferences<EPropertyCollectFlags::None>(*this, Struct, Instance, ReferencingObject);
}

void FReferenceCollector::AddPropertyReferencesWithStructARO(const UScriptStruct* Struct, void* Instance, const UObject* ReferencingObject)
{
	CallCollectStructReferences<EPropertyCollectFlags::CallStructARO>(*this, Struct, Instance, ReferencingObject);
}

void FReferenceCollector::AddPropertyReferencesWithStructARO(const UClass* Class, void* Instance, const UObject* ReferencingObject)
{
	CallCollectStructReferences<EPropertyCollectFlags::CallStructARO>(*this, Class, Instance, ReferencingObject);
}

void FReferenceCollector::AddPropertyReferencesLimitedToObjectProperties(const UStruct* Struct, void* Instance, const UObject* ReferencingObject)
{
	CallCollectStructReferences<EPropertyCollectFlags::OnlyObjectProperty>(*this, Struct, Instance, ReferencingObject);
}

void FReferenceCollector::CreateVerySlowReferenceCollectorArchive()
{
	check(DefaultReferenceCollectorArchive == nullptr);
	if (NeedsPropertyReferencer())
	{
		DefaultReferenceCollectorArchive.Reset(new FPropertyTrackingReferenceCollectorArchive(nullptr, *this));
	}
	else
	{
		DefaultReferenceCollectorArchive.Reset(new FReferenceCollectorArchive(nullptr, *this));
	}
}

FArchive& FReferenceCollectorArchive::operator<<(UObject*& Object)
{
	Collector.AddReferencedObject(ObjectPtrWrap(Object));
	return *this;
}

FArchive& FReferenceCollectorArchive::operator<<(FObjectPtr& Object)
{
	Collector.AddReferencedObject(reinterpret_cast<TObjectPtr<UObject>&>(Object));
	return *this;
}

/**
 * Archive for tagging unreachable objects in a non recursive manner.
 */
class FCollectorTagUsedNonRecursive : public FReferenceCollector
{
public:

	/**
	 * Default constructor.
	 */
	FCollectorTagUsedNonRecursive()
		:	CurrentObject(NULL)
	{
	}

	FORCEINLINE bool IsUnreachable(const UObject* Object) const
	{
		return !ReachabilityBits[GUObjectArray.ObjectToIndex(Object)];
	}

	// FReferenceCollector interface
	virtual bool IsIgnoringArchetypeRef() const override
	{
		return false;
	}
	virtual bool IsIgnoringTransient() const override
	{
		return false;
	}

	/**
	 * Performs reachability analysis. This information is later used by e.g. IncrementalPurgeGarbage or IsReferenced. The 
	 * algorithm is a simple mark and sweep where all objects are marked as unreachable. The root set passed in is 
	 * considered referenced and also objects that have any of the KeepFlags but none of the IgnoreFlags. RF_PendingKill is 
	 * implicitly part of IgnoreFlags and no object in the root set can have this flag set.
	 *
	 * @param KeepFlags		Objects with these flags will be kept regardless of being referenced or not (see line below)
	 * @param SearchFlags	If set, ignore objects with these flags for initial set, and stop recursion when found
	 * @param FoundReferences	If non-NULL, fill in with all objects that point to an object with SearchFlags set
	 * @param 
	 */
	void PerformReachabilityAnalysis( EObjectFlags KeepFlags, EInternalObjectFlags InternalKeepFlags, EObjectFlags SearchFlags = RF_NoFlags, FReferencerInformationList* FoundReferences = NULL)
	{
		ReferenceSearchFlags = SearchFlags;
		FoundReferencesList = FoundReferences;

		ReachabilityBits.Init(false, GUObjectArray.GetObjectArrayNum());

		// Iterate over all objects.
		for( FThreadSafeObjectIterator It; It; ++It )
		{
			UObject* Object	= *It;
			checkSlow(Object->IsValidLowLevel());

			// Special case handling for objects that are part of the root set.
			if( Object->IsRooted() )
			{
				SetReachable(Object);

				checkSlow( Object->IsValidLowLevel() );
				// We cannot use RF_PendingKill on objects that are part of the root set.
				checkCode( if( !IsValidChecked(Object) ) { UE_LOGF(LogUObjectGlobals, Fatal, "Object %ls is part of root set though is invalid!", *Object->GetFullName() ); } );
				// Add to list of objects to serialize.
				ObjectsToSerialize.Add( Object );
			}
			// Regular objects.
			else
			{
				// Mark objects as reachable when they have any of the passed in KeepFlags set and none of the passed in Search.
				if (!Object->HasAnyFlags(SearchFlags) &&
					((KeepFlags == RF_NoFlags && InternalKeepFlags == EInternalObjectFlags::None) || Object->HasAnyFlags(KeepFlags) || Object->HasAnyInternalFlags(InternalKeepFlags))
					)
				{
					SetReachable(Object);
					ObjectsToSerialize.Add(Object);
				}
			}
		}

		// Keep serializing objects till we reach the end of the growing array at which point
		// we are done.
		int32 CurrentIndex = 0;
		while( CurrentIndex < ObjectsToSerialize.Num() )
		{
			CurrentObject = ObjectsToSerialize[CurrentIndex++];
			CurrentReferenceInfo = NULL;

			// Serialize object.
			FindReferences( CurrentObject );
		}
	}

private:

	void FindReferences( UObject* Object )
	{
		check(Object != NULL);

		if( !Object->GetClass()->IsChildOf(UClass::StaticClass()) )
		{
			FPropertyTrackingReferenceCollectorArchive CollectorArchive( Object, *this );
			Object->SerializeScriptProperties( CollectorArchive );
		}
		Object->CallAddReferencedObjects(*this);
	}

	/**
	 * Adds passed in object to ObjectsToSerialize list and also removed RF_Unreachable
	 * which is used to signify whether an object already is in the list or not.
	 *
	 * @param	Object	object to add
	 */
	void AddToObjectList( const UObject* ReferencingObject, const FProperty* ReferencingProperty, UObject* Object )
	{
		// this message is to help track down culprits behind "Object in PIE world still referenced" errors
		if ( GIsEditor && !GIsPlayInEditorWorld && !CurrentObject->HasAnyFlags(RF_Transient) && Object->RootPackageHasAnyFlags(PKG_PlayInEditor) )
		{
			UPackage* ReferencingPackage = CurrentObject->GetOutermost();
			if (!ReferencingPackage->HasAnyPackageFlags(PKG_PlayInEditor) && !ReferencingPackage->HasAnyFlags(RF_Transient))
			{
				UE_LOGF(LogGarbage, Warning, "GC detected illegal reference to PIE object from content [possibly via %ls]:", ReferencingProperty != nullptr ? *ReferencingProperty->GetFullName() : *FString());
				UE_LOGF(LogGarbage, Warning, "      PIE object: %ls", *Object->GetFullName());
				UE_LOGF(LogGarbage, Warning, "  NON-PIE object: %ls", *CurrentObject->GetFullName());
			}
		}

		// Mark it as reachable.
		SetReachable(Object);

		// Add it to the list of objects to serialize.
		ObjectsToSerialize.Add( Object );
	}

	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		checkSlow(!InObject || InObject->IsValidLowLevel());
		if (InObject)
		{
			if (InObject->HasAnyFlags(ReferenceSearchFlags))
			{
				// Stop recursing, and add to the list of references
				if (FoundReferencesList)
				{
					if (!CurrentReferenceInfo)
					{
						CurrentReferenceInfo = &FoundReferencesList->ExternalReferences.Emplace_GetRef(CurrentObject);
					}
					if (InReferencingProperty)
					{
						CurrentReferenceInfo->ReferencingProperties.AddUnique(InReferencingProperty);
					}
					CurrentReferenceInfo->TotalReferences++;
				}
				// Mark it as reachable.
				SetReachable(InObject);
			}
			else if (IsUnreachable(InObject))
			{
				// Add encountered object reference to list of to be serialized objects if it hasn't already been added.
				AddToObjectList(InReferencingObject, InReferencingProperty, InObject);
			}
		}
	}

	FORCEINLINE void SetReachable(const UObject* Object)
	{
		ReachabilityBits[GUObjectArray.ObjectToIndex(Object)] = true;
	}

	/** Bitset containing reachability bits for each of the existing objects */
	TBitArray<> 		ReachabilityBits;
	/** Object we're currently serializing */
	UObject*			CurrentObject;
	/** Growing array of objects that require serialization */
	TArray<UObject*>	ObjectsToSerialize;
	/** Ignore any references from objects that match these flags */
	EObjectFlags		ReferenceSearchFlags;
	/** List of found references to fill in, if valid */
	FReferencerInformationList*	FoundReferencesList;
	/** Current reference info being filled out */
	FReferencerInformation *CurrentReferenceInfo;
};

bool IsReferenced(UObject*& Obj, EObjectFlags KeepFlags, EInternalObjectFlags InternalKeepFlags, bool bCheckSubObjects, FReferencerInformationList* FoundReferences)
{
	check(!Obj->IsUnreachable());

	FScopedObjectFlagMarker ObjectFlagMarker;
	bool bTempReferenceList = false;

	// Tag objects.
	for( FThreadSafeObjectIterator It; It; ++It )
	{
		UObject* Object = *It;
		Object->ClearFlags( RF_TagGarbageTemp );
	}
	// Ignore this object and possibly subobjects
	Obj->SetFlags( RF_TagGarbageTemp );

	if (FoundReferences)
	{
		// Clear old references
		FoundReferences->ExternalReferences.Empty();
		FoundReferences->InternalReferences.Empty();
	}

	if (bCheckSubObjects)
	{
		if (!FoundReferences)
		{
			// Allocate a temporary reference list
			FoundReferences = new FReferencerInformationList;
			bTempReferenceList = true;
		}
		Obj->TagSubobjects( RF_TagGarbageTemp );
	}

	FCollectorTagUsedNonRecursive ObjectReferenceTagger;
	// Exclude passed in object when peforming reachability analysis.
	ObjectReferenceTagger.PerformReachabilityAnalysis(KeepFlags, InternalKeepFlags, RF_TagGarbageTemp, FoundReferences);

	bool bIsReferenced = false;
	if (FoundReferences)
	{
		bool bReferencedByOuters = false;		
		// Move some from external to internal before returning
		for (int32 i = 0; i < FoundReferences->ExternalReferences.Num(); i++)
		{
			FReferencerInformation *OldRef = &FoundReferences->ExternalReferences[i];
			if (OldRef->Referencer == Obj)
			{
				FoundReferences->ExternalReferences.RemoveAt(i);
				i--;
			}
			else if (OldRef->Referencer->IsIn(Obj))
			{
				bReferencedByOuters = true;
				FReferencerInformation& NewRef = FoundReferences->InternalReferences.Emplace_GetRef(OldRef->Referencer, OldRef->TotalReferences, OldRef->ReferencingProperties);
				FoundReferences->ExternalReferences.RemoveAt(i);
				i--;
			}
		}
		bIsReferenced = FoundReferences->ExternalReferences.Num() > 0 || bReferencedByOuters || !ObjectReferenceTagger.IsUnreachable(Obj);
	}
	else
	{
		// Return whether the object was referenced
		bIsReferenced = !ObjectReferenceTagger.IsUnreachable(Obj);
	}
	
	if (bTempReferenceList)
	{
		// We allocated a temp list
		delete FoundReferences;
	}

	return bIsReferenced;
}


FArchive& FScriptInterface::Serialize(FArchive& Ar, UClass* InterfaceType)
{
	UObject* ObjectValue = GetObject();
	Ar << ObjectValue;
	SetObject(ObjectValue);
	if (Ar.IsLoading())
	{
		SetInterface(ObjectValue != NULL ? ObjectValue->GetInterfaceAddress(InterfaceType) : NULL);
	}
	return Ar;
}

/** A struct used as stub for deleted ones. */
UScriptStruct* GetFallbackStruct()
{
	return TBaseStructure<FFallbackStruct>::Get();
}

UObject* FObjectInitializer::CreateDefaultSubobject(UObject* Outer, FName SubobjectFName, const UClass* ReturnType, const UClass* ClassToCreateByDefault, bool bIsRequired, bool bIsTransient) const
{
	UE_CLOGF(!FUObjectThreadContext::Get().IsInConstructor, LogUObjectGlobals, Fatal, "Subobjects cannot be created outside of UObject constructors. UObject constructing subobjects cannot be created using new or placement new operator.");
	if (SubobjectFName == NAME_None)
	{
		UE_LOGF(LogUObjectGlobals, Fatal, "Illegal default subobject name: %ls", *SubobjectFName.ToString());
	}

	UObject* Result = nullptr;
	FOverrides::FOverrideDetails ComponentOverride = SubobjectOverrides.Get(SubobjectFName, ReturnType, ClassToCreateByDefault, !bIsRequired);
	const UClass* OverrideClass = ComponentOverride.Class;
	if (OverrideClass)
	{
		check(OverrideClass->IsChildOf(ReturnType));

		if (OverrideClass->HasAnyClassFlags(CLASS_Abstract))
		{
			// Attempts to create an abstract class will return null. If it is not optional or the owning class is not also abstract report a warning.
			if (!bIsRequired && !Outer->GetClass()->HasAnyClassFlags(CLASS_Abstract))
			{
				UE_LOGF(LogUObjectGlobals, Warning, "Required default subobject %ls not created as requested class %ls is abstract. Returning null.", *SubobjectFName.ToString(), *OverrideClass->GetName());
			}
		}
		else
		{
			UObject* Template = OverrideClass->GetDefaultObject(); // force the CDO to be created if it hasn't already
			EObjectFlags SubobjectFlags = Outer->GetMaskedFlags(RF_PropagateToSubObjects) | RF_DefaultSubObject;

			const bool bOwnerTemplateIsNotCDO = ObjectArchetype != nullptr && ObjectArchetype != Outer->GetClass()->GetDefaultObject(false) && !Outer->HasAnyFlags(RF_ClassDefaultObject);
#if !UE_BUILD_SHIPPING
			// Guard against constructing the same subobject multiple times.
			// We only need to check the name as ConstructObject would fail anyway if an object of the same name but different class already existed.
			if (ConstructedSubobjects.Find(SubobjectFName) != INDEX_NONE)
			{
				UE_LOGF(LogUObjectGlobals, Fatal, "Default subobject %ls %ls already exists for %ls.", *OverrideClass->GetName(), *SubobjectFName.ToString(), *Outer->GetFullName());
			}
			else
			{
				ConstructedSubobjects.Add(SubobjectFName);
			}
#endif
			FStaticConstructObjectParameters Params(OverrideClass);
			Params.Outer = Outer;
			Params.Name = SubobjectFName;
			Params.SetFlags = SubobjectFlags;
			Params.SubobjectOverrides = ComponentOverride.SubOverrides;

			// If the object creating a subobject is being created from a template, not a CDO 
			// then we need to use the subobject from that template as the new subobject's template
			if (!bIsTransient && bOwnerTemplateIsNotCDO)
			{
				UObject* MaybeTemplate = ObjectArchetype->GetDefaultSubobjectByName(SubobjectFName);
				if (MaybeTemplate && Template != MaybeTemplate && MaybeTemplate->IsA(ReturnType))
				{
					Params.Template = MaybeTemplate;
				}
			}

			Result = StaticConstructObject_Internal(Params);

			if (Params.Template)
			{
				ComponentInits.Add(Result, Params.Template);
			}
			else if (!bIsTransient && Outer->GetArchetype()->IsInBlueprint())
			{
				UObject* MaybeTemplate = Result->GetArchetype();
				if (MaybeTemplate && Template != MaybeTemplate && MaybeTemplate->IsA(ReturnType))
				{
					ComponentInits.Add(Result, MaybeTemplate);
				}
			}
			if (Outer->HasAnyFlags(RF_ClassDefaultObject) && Outer->GetClass()->GetSuperClass())
			{
#if WITH_EDITOR
				// Default subobjects on the CDO should be transactional, so that we can undo/redo changes made to those objects.
				// One current example of this is editing natively defined components in the Blueprint Editor.
				Result->SetFlags(RF_Transactional);
#endif
				Outer->GetClass()->AddDefaultSubobject(Result, ReturnType);
			}
			// Clear PendingKill flag in case we recycled a subobject of a dead object.
			// @todo: we should not be recycling subobjects unless we're currently loading from a package
			Result->ClearGarbage();
		}
	}
	return Result;
}
UObject* FObjectInitializer::CreateEditorOnlyDefaultSubobject(UObject* Outer, FName SubobjectName, const UClass* ReturnType, bool bTransient /*= false*/) const
{
#if WITH_EDITOR
	if (GIsEditor)
	{
		UObject* EditorSubobject = CreateDefaultSubobject(Outer, SubobjectName, ReturnType, ReturnType, /*bIsRequired =*/ false, bTransient);
		if (EditorSubobject)
		{
			EditorSubobject->MarkAsEditorOnlySubobject();
		}
		return EditorSubobject;
	}
#endif
	return nullptr;
}

COREUOBJECT_API UFunction* FindDelegateSignature(FName DelegateSignatureName)
{
	FString StringName = DelegateSignatureName.ToString();

	if (StringName.EndsWith(HEADER_GENERATED_DELEGATE_SIGNATURE_SUFFIX))
	{
		return FindFirstObject<UFunction>(*StringName, EFindFirstObjectOptions::NativeFirst | EFindFirstObjectOptions::EnsureIfAmbiguous);
	}

	return nullptr;
}

void UE::FAssetLog::SerializeForLog(FCbWriter& Writer) const
{
	const FAssetLog& AssetLog = *this;

	FString ObjectPath;
	FString LocalPath;

	if (AssetLog.Path)
	{
		ObjectPath = AssetLog.Path;
		FString PackageName, ObjectName, SubObjectName, Extension;
		if (FPackageName::TryConvertToMountedPath(ObjectPath, &LocalPath, &PackageName, &ObjectName, &SubObjectName, &Extension))
		{
			ObjectPath = PackageName;
			if (!ObjectName.IsEmpty())
			{
				ObjectPath += TEXT('.');
				ObjectPath += ObjectName;
			}
			if (!SubObjectName.IsEmpty())
			{
				ObjectPath += SUBOBJECT_DELIMITER;
				ObjectPath += SubObjectName;
			}

			if (!Extension.IsEmpty())
			{
				LocalPath += Extension;
			}
			else if (!FPackageName::DoesPackageExist(PackageName, &LocalPath))
			{
				LocalPath.Empty();
			}
		}
	}
	else if (AssetLog.PackagePath)
	{
		ObjectPath = AssetLog.PackagePath->GetPackageName();
		LocalPath = AssetLog.PackagePath->GetLocalFullPath();
	}
	else if (AssetLog.Object)
	{
		ObjectPath = AssetLog.Object->GetPathName();
		if (const UPackage* Package = AssetLog.Object->GetPackage())
		{
			LocalPath = Package->GetLoadedPath().GetLocalFullPath();
		}
	}

	const auto GetConfigBool = [](const TCHAR* Section, const TCHAR* Key, bool bDefault) -> bool
	{
		GConfig->GetBool(Section, Key, bDefault, GEngineIni);
		return bDefault;
	};

	FString AbsLocalPath;
	if (!LocalPath.IsEmpty())
	{
		AbsLocalPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*LocalPath);
		FPaths::MakePlatformFilename(AbsLocalPath);

		static bool bShowDiskPath = GetConfigBool(TEXT("Core.System"), TEXT("AssetLogShowsDiskPath"), true);
		if (bShowDiskPath)
		{
			static bool bShowAbsolutePath = GetConfigBool(TEXT("Core.System"), TEXT("AssetLogShowsAbsolutePath"), false);
			if (bShowAbsolutePath)
			{
				ObjectPath = AbsLocalPath;
			}
			else
			{
				ObjectPath = LocalPath;
				FPaths::MakePlatformFilename(ObjectPath);
			}
		}
	}

	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("Asset"));
	Writer.AddString(ANSITEXTVIEW("$text"), ObjectPath);
	if (!AbsLocalPath.IsEmpty())
	{
		Writer.AddString(ANSITEXTVIEW("file"), AbsLocalPath);
	}
	Writer.EndObject();
}

void UE::Core::Private::RecordAssetLog(
	const FName& CategoryName,
	const ELogVerbosity::Type Verbosity,
	const FAssetLog& AssetLog,
	const FString& Message,
	const ANSICHAR* File,
	const int32 Line)
{
	FCbWriter Writer;
	Writer.BeginObject();
	Writer.SetName(ANSITEXTVIEW("Asset"));
	SerializeForLog(Writer, AssetLog);
	Writer.AddString(ANSITEXTVIEW("Message"), Message);
	Writer.EndObject();

	FLogRecord Record;
	Record.SetUtf8Format(UTF8TEXT("[AssetLog] {Asset}: {Message}"));
	Record.SetFields(Writer.Save().AsObject());
	Record.SetFile(File);
	Record.SetLine(Line);
	Record.SetCategory(CategoryName);
	Record.SetVerbosity(Verbosity);
	Record.SetTime(FLogTime::Now());

	switch (Verbosity)
	{
	case ELogVerbosity::Error:
	case ELogVerbosity::Warning:
	case ELogVerbosity::Display:
		return GWarn->SerializeRecord(Record);
	default:
		return GLog->SerializeRecord(Record);
	}
}

/**
 * Takes a path of some sort and attempts to turn it into the asset log's canonical path.
 */
FString FAssetMsg::FormatPathForAssetLog(const TCHAR* InPath)
{
	static bool ShowDiskPathOnce = false;
	static bool ShowDiskPath = true;

	if (!ShowDiskPathOnce)
	{
		GConfig->GetBool(TEXT("Core.System"), TEXT("AssetLogShowsDiskPath"), ShowDiskPath, GEngineIni);
		ShowDiskPathOnce = true;
	}

	if (FPlatformProperties::RequiresCookedData() || !ShowDiskPath)
	{
		return FString(InPath);
	}
	
	FString AssetPath = InPath;
	FString FilePath;

	// check for /Game/Path/Package.obj and turn it into a package reference
	if (FPackageName::IsValidObjectPath(AssetPath))
	{
		AssetPath = FPackageName::ObjectPathToPackageName(AssetPath);
	}

	// Try to convert this to a file path
	if (FPackageName::DoesPackageExist(AssetPath, &FilePath) == false)
	{
		// if failed, assume we were given something that's a file path (e.g. ../../../Game/Whatever)
		FilePath = AssetPath;
	}

	// if that succeeded FilePath will be a relative path to a  file, if not just assume that's what we were given and proceed...
	if (IFileManager::Get().FileExists(*FilePath) == false)
	{
		return FString::Printf(TEXT("%s (no disk path found)"), InPath);
	}

	static bool DiskPathAbsolueOnce = false;
	static bool DiskPathAbsolue = true;

	if (!DiskPathAbsolueOnce)
	{
		GConfig->GetBool(TEXT("Core.System"), TEXT("AssetLogShowsAbsolutePath"), DiskPathAbsolue, GEngineIni);
		DiskPathAbsolueOnce = true;
	}

	if (DiskPathAbsolue)
	{
		// turn this into an absolute path for error logging
		FilePath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FilePath);
	}
	
	// turn into a native platform file
	FPaths::MakePlatformFilename(FilePath);
	return FilePath;
}

FString FAssetMsg::FormatPathForAssetLog(const FPackagePath& InPath)
{
	return FormatPathForAssetLog(*InPath.GetDebugName());
}

/**
 * Format the path of the passed in object
 */
FString FAssetMsg::FormatPathForAssetLog(const UObject* Object)
{
	return ensure(Object) ? FormatPathForAssetLog(*Object->GetPathName()) : FString();
}

FString FAssetMsg::GetAssetLogString(const TCHAR* Path, const FString& Message)
{
	return FString::Printf(TEXT(ASSET_LOG_FORMAT_STRING_ANSI "%s"), *FAssetMsg::FormatPathForAssetLog(Path), *Message);
}

FString FAssetMsg::GetAssetLogString(const FPackagePath& Path, const FString& Message)
{
	return FString::Printf(ASSET_LOG_FORMAT_STRING TEXT("%s"), *FAssetMsg::FormatPathForAssetLog(Path), *Message);
}

FString FAssetMsg::GetAssetLogString(const UObject* Object, const FString& Message)
{
	return ensure(Object) ? GetAssetLogString(*Object->GetOutermost()->GetName(), Message) : FString();
}

#if !UE_WITH_CONSTINIT_UOBJECT
namespace UECodeGen_Private
{
	template <typename PropertyType, typename PropertyParamsType>
	PropertyType* NewFProperty(FFieldVariant Outer, const FPropertyParamsBase& PropBase)
	{
		const PropertyParamsType& Prop = (const PropertyParamsType&)PropBase;
		PropertyType* NewProp = nullptr;

		if (Prop.SetterFunc || Prop.GetterFunc)
		{
			NewProp = new TPropertyWithSetterAndGetter<PropertyType>(Outer, Prop);
		}
		else
		{
			NewProp = new PropertyType(Outer, Prop);
		}

#if WITH_METADATA
		if (Prop.NumMetaData)
		{
			for (const FMetaDataPairParam* MetaDataData = Prop.MetaDataArray, *MetaDataEnd = MetaDataData + Prop.NumMetaData; MetaDataData != MetaDataEnd; ++MetaDataData)
			{
				NewProp->SetMetaData(UTF8_TO_TCHAR(MetaDataData->NameUTF8), UTF8_TO_TCHAR(MetaDataData->ValueUTF8));
			}
		}
#endif
		return NewProp;
	}

	FProperty* ConstructFProperty(FFieldVariant Outer, const FPropertyParamsBase* const*& PropertyArray, int32& NumProperties)
	{
		const FPropertyParamsBase* PropBase = *--PropertyArray;

		uint32 ReadMore = 0;

		FProperty* NewProp = nullptr;
		switch (PropBase->Flags & PropertyTypeMask)
		{
			default:
			{
				// Unsupported property type
				check(false);
			}

			case EPropertyGenFlags::Byte:
			{
				NewProp = NewFProperty<FByteProperty, FBytePropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Int8:
			{
				NewProp = NewFProperty<FInt8Property, FInt8PropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Int16:
			{
				NewProp = NewFProperty<FInt16Property, FInt16PropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Int:
			{
				NewProp = NewFProperty<FIntProperty, FIntPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Int64:
			{
				NewProp = NewFProperty<FInt64Property, FInt64PropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::UInt16:
			{
				NewProp = NewFProperty<FUInt16Property, FUInt16PropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::UInt32:
			{
				NewProp = NewFProperty<FUInt32Property, FUInt32PropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::UInt64:
			{
				NewProp = NewFProperty<FUInt64Property, FUInt64PropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Float:
			{
				NewProp = NewFProperty<FFloatProperty, FFloatPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::LargeWorldCoordinatesReal:
			case EPropertyGenFlags::Double:
			{
				NewProp = NewFProperty<FDoubleProperty, FDoublePropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Bool:
			{
				NewProp = NewFProperty<FBoolProperty, FBoolPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Object:
			{
				NewProp = NewFProperty<FObjectProperty, FObjectPropertyParams>(Outer, *PropBase);
				if (EnumHasAllFlags(PropBase->Flags, EPropertyGenFlags::ObjectPtr))
				{
					NewProp->SetPropertyFlags(CPF_TObjectPtrWrapper);
				}
			}
			break;

			case EPropertyGenFlags::WeakObject:
			{
				NewProp = NewFProperty<FWeakObjectProperty, FWeakObjectPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::LazyObject:
			{
				NewProp = NewFProperty<FLazyObjectProperty, FLazyObjectPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::SoftObject:
			{
				NewProp = NewFProperty<FSoftObjectProperty, FSoftObjectPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Class:
			{
				NewProp = NewFProperty<FClassProperty, FClassPropertyParams>(Outer, *PropBase);
				if (EnumHasAllFlags(PropBase->Flags, EPropertyGenFlags::ObjectPtr))
				{
					NewProp->SetPropertyFlags(CPF_TObjectPtrWrapper);
				}
			}
			break;

			case EPropertyGenFlags::SoftClass:
			{
				NewProp = NewFProperty<FSoftClassProperty, FSoftClassPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Interface:
			{
				NewProp = NewFProperty<FInterfaceProperty, FInterfacePropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Name:
			{
				NewProp = NewFProperty<FNameProperty, FNamePropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Str:
			{
				NewProp = NewFProperty<FStrProperty, FStrPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Array:
			{
				NewProp = NewFProperty<FArrayProperty, FArrayPropertyParams>(Outer, *PropBase);

				// Next property is the array inner
				ReadMore = 1;
			}
			break;

			case EPropertyGenFlags::Map:
			{
				NewProp = NewFProperty<FMapProperty, FMapPropertyParams>(Outer, *PropBase);

				// Next two properties are the map key and value inners
				ReadMore = 2;
			}
			break;

			case EPropertyGenFlags::Set:
			{
				NewProp = NewFProperty<FSetProperty, FSetPropertyParams>(Outer, *PropBase);

				// Next property is the set inner
				ReadMore = 1;
			}
			break;

			case EPropertyGenFlags::Struct:
			{
				NewProp = NewFProperty<FStructProperty, FStructPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Delegate:
			{
				NewProp = NewFProperty<FDelegateProperty, FDelegatePropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::InlineMulticastDelegate:
			{
				NewProp = NewFProperty<FMulticastInlineDelegateProperty, FMulticastDelegatePropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::SparseMulticastDelegate:
			{
				NewProp = NewFProperty<FMulticastSparseDelegateProperty, FMulticastDelegatePropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Text:
			{
				NewProp = NewFProperty<FTextProperty, FTextPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Enum:
			{
				NewProp = NewFProperty<FEnumProperty, FEnumPropertyParams>(Outer, *PropBase);

				// Next property is the underlying integer property
				ReadMore = 1;
			}
			break;

			case EPropertyGenFlags::FieldPath:
			{
				NewProp = NewFProperty<FFieldPathProperty, FFieldPathPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::Optional:
			{
				NewProp = NewFProperty<FOptionalProperty, FGenericPropertyParams>(Outer, *PropBase);

				// Next property is the optional inner
				ReadMore = 1;
			}
			break;

			case EPropertyGenFlags::VerseType:
			{
				NewProp = NewFProperty<FVerseClassProperty, FVerseClassPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::VerseValue:
			{
#if WITH_VERSE_BPVM
				static_assert(std::is_same_v<FVerseDynamicProperty, FVerseValueProperty>);
				// NOTE: When this code is removed, make sure to remove the NewVerseDynamicProperty method chain and
				// merge NewFProperty back into this source file from the UObjectGlobalsInternal.h.
				NewProp = NewFProperty<FVerseDynamicProperty, FVerseValuePropertyParams>(Outer, *PropBase);
#else
				static_assert(std::is_same_v<FVRestValueProperty, FVerseValueProperty>);
				NewProp = NewFProperty<FVRestValueProperty, FVerseValuePropertyParams>(Outer, *PropBase);
#endif
			}
			break;

#if !WITH_VERSE_BPVM
			case EPropertyGenFlags::VerseCell:
			{
				NewProp = NewFProperty<FVCellProperty, FVerseCellValuePropertyParams>(Outer, *PropBase);
			}
			break;
#endif

			case EPropertyGenFlags::Utf8Str:
			{
				NewProp = NewFProperty<FUtf8StrProperty, FUtf8StrPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::AnsiStr:
			{
				NewProp = NewFProperty<FAnsiStrProperty, FAnsiStrPropertyParams>(Outer, *PropBase);
			}
			break;

			case EPropertyGenFlags::VerseString:
			{
				NewProp = NewFProperty<FVerseStringProperty, FVerseStringPropertyParams>(Outer, *PropBase);
				ReadMore = 1; // Next property is the inner
			}
			break;

#if !WITH_VERSE_BPVM
			case EPropertyGenFlags::VerseLegacyValue:
			{
				NewProp = NewFProperty<FVRestValueProperty, FVerseValuePropertyParams>(Outer, *PropBase);
				ReadMore = 1; // Next property is the legacy value
			}
			break;
#endif

		}

		NewProp->ArrayDim = PropBase->ArrayDim;
		if (PropBase->RepNotifyFuncUTF8)
		{
			NewProp->RepNotifyFunc = FName(UTF8_TO_TCHAR(PropBase->RepNotifyFuncUTF8));
		}

		--NumProperties;

		for (; ReadMore; --ReadMore)
		{
			ConstructFProperty(NewProp, PropertyArray, NumProperties);
		}
		return NewProp;
	}

	void ConstructFProperties(UObject* Outer, const FPropertyParamsBase* const* PropertyArray, int32 NumProperties)
	{
		// Move pointer to the end, because we'll iterate backwards over the properties
		PropertyArray += NumProperties;
		while (NumProperties)
		{
			ConstructFProperty(Outer, PropertyArray, NumProperties);
		}
	}

#if WITH_METADATA
	void AddMetaData(UObject* Object, const FMetaDataPairParam* MetaDataArray, int32 NumMetaData)
	{
		if (NumMetaData)
		{
			FMetaData& MetaData = Object->GetPackage()->GetMetaData();
			for (const FMetaDataPairParam* MetaDataParam = MetaDataArray, *MetaDataParamEnd = MetaDataParam + NumMetaData; MetaDataParam != MetaDataParamEnd; ++MetaDataParam)
			{
				MetaData.SetValue(Object, UTF8_TO_TCHAR(MetaDataParam->NameUTF8), UTF8_TO_TCHAR(MetaDataParam->ValueUTF8));
			}
		}
	}
#endif

	void ConstructUFunction(UFunction** SingletonPtr, const FFunctionParams& Params)
	{
		ConstructUFunctionHelper<UFunction>(*SingletonPtr, Params, SingletonPtr,
			[](UObject* Outer, UFunction* Super, FName FuncName, const FFunctionParams& Params) -> UFunction*
			{
				return new (EC_InternalUseOnlyConstructor, Outer, FuncName, Params.ObjectFlags) UFunction(
					FObjectInitializer(),
					Super,
					Params.FunctionFlags,
					Params.StructureSize
				);
			});
	}

	void ConstructUDelegateFunction(UFunction** SingletonPtr, const FDelegateFunctionParams& Params)
	{
		ConstructUFunctionHelper<UFunction>(*SingletonPtr, Params, SingletonPtr, 
			[](UObject* Outer, UFunction* Super, FName FuncName, const FDelegateFunctionParams& Params) -> UFunction*
			{
				return new (EC_InternalUseOnlyConstructor, Outer, FuncName, Params.ObjectFlags) UDelegateFunction(
					FObjectInitializer(),
					Super,
					Params.FunctionFlags,
					Params.StructureSize
				);
			});
	}

	void ConstructUSparseDelegateFunction(UFunction** SingletonPtr, const FSparseDelegateFunctionParams& Params)
	{
		ConstructUFunctionHelper<UFunction>(*SingletonPtr, Params, SingletonPtr, 
			[](UObject* Outer, UFunction* Super, FName FuncName, const FSparseDelegateFunctionParams& Params) -> UFunction*
			{
				USparseDelegateFunction* NewFunction = new (EC_InternalUseOnlyConstructor, Outer, FuncName, Params.ObjectFlags) USparseDelegateFunction(
					FObjectInitializer(),
					Super,
					Params.FunctionFlags,
					Params.StructureSize
				);
				NewFunction->OwningClassName = FName(Params.OwningClassName);
				NewFunction->DelegateName = FName(Params.DelegateName);
				return NewFunction;
			});
	}


	void ConstructUEnum(UEnum*& OutEnum, const FEnumParams& Params)
	{
		ConstructUEnumHelper<UEnum>(OutEnum, Params, [](UEnum*, const FEnumParams&) {});
	}

	void ConstructUScriptStruct(UScriptStruct*& OutStruct, const FStructParams& Params)
	{
		ConstructUScriptStructHelper<UScriptStruct>(OutStruct, Params, [](UScriptStruct*, const FStructParams&) {});
	}

	void ConstructUClass(UClass*& OutClass, const FClassParams& Params)
	{
		ConstructUClassHelper<UClass>(OutClass, Params, [](UClass*, const FClassParams&) {});
	}

	// It is UB to call `SingletonFunc` below because it can return a `UClass*` not `UObject*` (even though
	// there is a parent-child relation there). So we suppress UBSan for this function.
	UE_DISABLE_UBSAN
	void ConstructUPackage(UPackage*& OutPackage, const FPackageParams& Params)
	{
		if (OutPackage)
		{
			return;
		}

		UObject* FoundPackage = StaticFindObjectFast(UPackage::StaticClass(), nullptr, FName(UTF8_TO_TCHAR(Params.NameUTF8)), EFindObjectFlags::None);

#if USE_PER_MODULE_UOBJECT_BOOTSTRAP
		if (!FoundPackage)
		{
			UE_LOGF(LogUObjectGlobals, Log, "Creating package on the fly %ls", UTF8_TO_TCHAR(Params.NameUTF8));
			ProcessNewlyLoadedUObjects(FName(UTF8_TO_TCHAR(Params.NameUTF8)), false);
			FoundPackage = CreatePackage(UTF8_TO_TCHAR(Params.NameUTF8));
		}
#endif

		checkf(FoundPackage, TEXT("Code not found for generated code (package %s)."), UTF8_TO_TCHAR(Params.NameUTF8));

		UPackage* NewPackage = CastChecked<UPackage>(FoundPackage);
		OutPackage = NewPackage;

#if WITH_METADATA
		AddMetaData(NewPackage, Params.MetaDataArray, Params.NumMetaData);
#endif

		NewPackage->SetPackageFlags(Params.PackageFlags);
#if WITH_EDITORONLY_DATA
		// Replace the PersistentGuid generated from UPackage::PostInitProperties() that changes every time.
		FGuid DeterministicGuid(Params.BodyCRC, Params.DeclarationsCRC, 0u, 0u);
		NewPackage->SetPersistentGuid(DeterministicGuid);
		// Set the initial saved hash to a value based on the CRCs; this is needed for script packages.
		FIoHash SavedHash;
		constexpr size_t SavedHashSize = sizeof(SavedHash.GetBytes()); // -V568
		FMemory::Memcpy(&SavedHash.GetBytes(), &DeterministicGuid,
			FMath::Min(SavedHashSize, sizeof(DeterministicGuid)));
		NewPackage->SetSavedHash(SavedHash);
#endif

#if WITH_RELOAD
		TArray<UFunction*> Delegates;
		Delegates.Reserve(Params.NumSingletons);
#endif
		for (FTypeConstructFunc*const* SingletonFunc = Params.SingletonFuncArray, *const* SingletonFuncEnd = SingletonFunc + Params.NumSingletons; SingletonFunc != SingletonFuncEnd; ++SingletonFunc)
		{
			UObject* Object = (*SingletonFunc)(ETypeConstructPhase::Outer);
#if WITH_RELOAD
			if (UFunction* Function = Cast<UFunction>(Object))
			{
				Delegates.Add(Function);
			}
#endif
			if (Object->GetOuter() == NewPackage)
			{
				// Notify loader of new top level noexport objects like UScriptStruct, UDelegateFunction and USparseDelegateFunction
				NotifyRegistrationEvent(NewPackage->GetFName(), Object->GetFName(), ENotifyRegistrationType::NRT_NoExportObject, ENotifyRegistrationPhase::NRP_Finished, nullptr, false, Object);
			}
		}
#if WITH_RELOAD
		NewPackage->SetReloadDelegates(MoveTemp(Delegates));
#endif
	}
} // namespace UECodeGen_Private
#endif // !UE_WITH_CONSTINIT_UOBJECT

void FReferenceCollector::AddStableReferenceSetFwd(TSet<FObjectPtr>* Objects)
{
	AddStableReferenceSet(reinterpret_cast<TSet<TObjectPtr<UObject>>*>(Objects));
}

void FReferenceCollector::AddStableReferenceArrayFwd(TArray<FObjectPtr>* Objects)
{
	AddStableReferenceArray(reinterpret_cast<TArray<TObjectPtr<UObject>>*>(Objects));
}

void FReferenceCollector::AddStableReferenceFwd(FObjectPtr* Object)
{
	AddStableReference(reinterpret_cast<TObjectPtr<UObject>*>(Object));
}

