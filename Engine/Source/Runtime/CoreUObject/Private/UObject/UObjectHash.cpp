// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectHash.cpp: Unreal object name hashes
=============================================================================*/

#include "UObject/UObjectHash.h"
#include "Containers/CompactSet.h"
#include "HAL/LowLevelMemTracker.h"
#include "Misc/AssertionMacros.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollectionGlobals.h"
#include "UObject/Package.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/ObjectHandlePrivate.h"
#include "UObject/UObjectIterator.h"
#include "UObject/ObjectVisibility.h"
#include "Templates/Casts.h"
#include "Misc/AsciiSet.h"
#include "Misc/PackageName.h"
#include "Async/ParallelFor.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemStats.h"
#include "ProfilingDebugging/MetadataTrace.h"
#include "AutoRTFM.h"

#include <atomic>

#ifndef UE_UOBJECT_HASH_USES_ARRAYS
#	define UE_UOBJECT_HASH_USES_ARRAYS 0
#endif

#if UE_UOBJECT_HASH_USES_ARRAYS
	static_assert(UE_STORE_OBJECT_LIST_INTERNAL_INDEX == 1, "UE_UOBJECT_HASH_USES_ARRAYS requires UE_STORE_OBJECT_LIST_INTERNAL_INDEX as it will cause a significant performance regression without it");
#endif

#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
	static_assert(sizeof(FName) == 4, "Internal object index optimization depends exploits 4 bytes padding after the FName");
	static_assert(sizeof(UObjectBase) == 40, "UObjectBase size has changed! This optimization relies on a 4 bytes padding after NamePrivate.Please make sure that the new binary layout it optimal");
#endif

DEFINE_LOG_CATEGORY_STATIC(LogUObjectHash, Log, All);

DECLARE_CYCLE_STAT( TEXT( "GetObjectsOfClass" ), STAT_Hash_GetObjectsOfClass, STATGROUP_UObjectHash );

#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
	DECLARE_CYCLE_STAT( TEXT( "HashObject" ), STAT_Hash_HashObject, STATGROUP_UObjectHash );
	DECLARE_CYCLE_STAT( TEXT( "UnhashObject" ), STAT_Hash_UnhashObject, STATGROUP_UObjectHash );
	DEFINE_STAT( STAT_Hash_NumObjects );
#endif // !UE_BUILD_TEST && !UE_BUILD_SHIPPING

LLM_DEFINE_TAG(UObjectHash, TEXT("UObject hashtables"));

// Global UObject array instance
alignas(PLATFORM_CACHE_LINE_SIZE) FUObjectArray GUObjectArray;

static EInternalObjectFlags GetMandatoryInternalExclusionFlags()
{
	// Common EInternalObjectFlags used in this file to filter out objects.
	static constexpr EInternalObjectFlags MandatoryInternalExclusionFlags =
		EInternalObjectFlags::Unreachable | EInternalObjectFlags::AutoRTFMConstructionAborted;

	return MandatoryInternalExclusionFlags | UE::GetAsyncLoadingInternalFlagsExclusion();
}

static FORCENOINLINE void OnHashFailure(UObjectBaseUtility* Object, const TCHAR* HashName, const TCHAR* FailureKind)
{
	UE_LOGF(LogUObjectHash, Error, "UObject %ls consistency failure (%ls). Checking for memory corruption", HashName, FailureKind);
	UE_CLOGF(!Object->IsValidLowLevelFast(false), LogUObjectHash, Fatal, "IsValidLowLevelFast failure");
	UE_CLOGF(!Object->IsValidLowLevel(), LogUObjectHash, Fatal, "IsValid failure");
	UObjectBaseUtility* Outer = Object->GetOuter();
	UE_CLOGF(Outer && !Outer->IsValidLowLevelFast(false), LogUObjectHash, Fatal, "Outer IsValidLowLevelFast failure");
	UObjectBaseUtility* Class = Object->GetClass();
	UE_CLOGF(Class && !Class->IsValidLowLevelFast(false), LogUObjectHash, Fatal, "Class IsvalidLowLevelFast failure");
	
	UE_LOGF(LogUObjectHash, Fatal, "Unidentified failure for object %ls, hash itself may be corrupted or buggy.", *Object->GetFullName());
}

/**
 * This implementation will use more space than the UE3 implementation. The goal was to make UObjects smaller to save L2 cache space. 
 * The hash is rarely used at runtime. A more space-efficient implementation is possible.
 */

template <typename T>
struct THashBucketIterator;

/*
 * Special hash bucket to conserve memory.
 * Contains a pointer to head element and an optional list of items if more than one element exists in the bucket.
 * The item list is only allocated if needed.
 */
struct FSetHashBucket
{
	typedef TCompactSet<UObjectBase*>::TIterator TIterator;

	/** This always empty set is used to get an iterator if the bucket doesn't use a TCompactSet (has only 1 element) */
	static TCompactSet<UObjectBase*> EmptyBucket;

	/*
	* Acts like an alias for either 1-2 elements or a pointer to a TCompactSet
	* If these are both null, this bucket is empty
	* If the first one is null, but the second one is non-null, then the second one is a TCompactSet pointer
	* If the first one is not null, then it is a uobject ptr, and the second ptr is either null or a second element
	*/
	void* Elements[2];

	FORCEINLINE TCompactSet<UObjectBase*>* GetItems()
	{
		if (Elements[1] && !Elements[0])
		{
			return (TCompactSet<UObjectBase*>*)Elements[1];
		}
		return nullptr;
	}

	FORCEINLINE const TCompactSet<UObjectBase*>* GetItems() const
	{
		if (Elements[1] && !Elements[0])
		{
			return (TCompactSet<UObjectBase*>*)Elements[1];
		}
		return nullptr;
	}

	/** Constructor */
	FORCEINLINE FSetHashBucket()
	{
		Elements[0] = nullptr;
		Elements[1] = nullptr;
	}

	FORCEINLINE ~FSetHashBucket()
	{
		delete GetItems();
	}

	/** Adds an Object to the bucket */
	FORCEINLINE void Add(UObjectBase* Object)
	{
		TCompactSet<UObjectBase*>* Items = GetItems();
		if (Items)
		{
			Items->Add(Object);
		}
		else if (Elements[0] && Elements[1])
		{
			Items = new TCompactSet<UObjectBase*>();
			Items->Add((UObjectBase*)Elements[0]);
			Items->Add((UObjectBase*)Elements[1]);
			Items->Add(Object);
			Elements[0] = nullptr;
			Elements[1] = Items;
		}
		else if (Elements[0])
		{
			Elements[1] = Object;
		}
		else
		{
			Elements[0] = Object;
			checkSlow(!Elements[1]);
		}
	}

	/** Removes an Object from the bucket */
	FORCEINLINE int32 Remove(UObjectBase* Object)
	{
		int32 Result = 0;
		TCompactSet<UObjectBase*>* Items = GetItems();
		if (Items)
		{
			Result = Items->Remove(Object);
			if (Items->Num() <= 2)
			{
				auto It = TCompactSet<UObjectBase*>::TIterator(*Items);
				Elements[0] = *It;
				checkSlow((bool)It);
				++It;
				Elements[1] = *It;
				delete Items;
			}
		}
		else if (Object == Elements[1])
		{
			Result = 1;
			Elements[1] = nullptr;
		}
		else if (Object == Elements[0])
		{
			Result = 1;
			Elements[0] = Elements[1];
			Elements[1] = nullptr;
		}
		return Result;
	}

	/** Checks if an Object exists in this bucket */
	FORCEINLINE bool Contains(UObjectBase* Object) const
	{
		const TCompactSet<UObjectBase*>* Items = GetItems();
		if (Items)
		{
			return Items->Contains(Object);
		}
		return Object == Elements[0] || Object == Elements[1];
	}

	/** Returns the number of Objects in this bucket */
	FORCEINLINE int32 Num() const
	{
		const TCompactSet<UObjectBase*>* Items = GetItems();
		if (Items)
		{
			return Items->Num();
		}
		return !!Elements[0] + !!Elements[1];
	}

	/** Returns the amount of memory allocated for and by Items TCompactSet */
	FORCEINLINE SIZE_T GetAllocatedSize() const
	{
		const TCompactSet<UObjectBase*>* Items = GetItems();
		if (Items)
		{
			return sizeof(*Items) + Items->GetAllocatedSize();
		}
		return 0;
	}

	void Shrink()
	{
		TCompactSet<UObjectBase*>* Items = GetItems();
		if (Items)
		{
			Items->Compact();
		}
	}

	FORCEINLINE struct THashBucketIterator<FSetHashBucket> CreateIterator();

	/** Gets an iterator for the TCompactSet in this bucket or for the EmptyBucker if Items is null */
	FORCEINLINE TCompactSet<UObjectBase*>::TIterator GetIterator()
	{
		TCompactSet<UObjectBase*>* Items = GetItems();
		return Items ? Items->CreateIterator() : EmptyBucket.CreateIterator();
	}
};

TCompactSet<UObjectBase*> FSetHashBucket::EmptyBucket;


/*
 * Special hash bucket to conserve memory.
 * Contains a pointer to head element or an optional array of items if more than one element exists in the bucket.
 */
struct FArrayHashBucket
{
	struct FObjectsArray : public TArray<UObjectBase*>
	{
		FORCEINLINE bool Remove(UObjectBase* Object)
		{
			const uint32 Index = TArray<UObjectBase*>::Find(Object);
			if (Index != INDEX_NONE)
			{
				RemoveAt(Index, 1);

				return true;
			}

			return false;
		}

		FObjectsArray()
		{
			static_assert(offsetof(FObjectsArray, ArrayNum) == sizeof(UObjectBase*));	// make sure that offset of ArrayNum and ArrayMax for TArray is at 8 bytes, to be aliased with Elements[1]
		}
	};

	typedef FObjectsArray::TIterator TIterator;

	/** This always empty array is used to get an iterator if the bucket doesn't use a TArray (has only 1 element) */
	static FObjectsArray EmptyBucket;

	/*
	 * A union that contains one element or acts as a binary alias for a TArray
	 * TArray's binary representation is { UObjectBase* ptr, uint32 ArrayNum, uint32 ArrayMax }
	 * So the first Element is always a pointer to either UObjectBase itself or to an array of UObjectBase
	 * If the second Element is non-null, then ArrayNum is initialized and the whole union is a TArray now
	 */
	union
	{
		FObjectsArray ObjectsList;
		struct
		{
			UObjectBase* Elements[2];
		};
	};

	static_assert(sizeof(ObjectsList) == sizeof(Elements));

	FORCEINLINE FObjectsArray* GetItems()
	{
		if (ObjectsList.Num())
		{
			return &ObjectsList;
		}
		return nullptr;
	}

	FORCEINLINE const FObjectsArray* GetItems() const
	{
		if (ObjectsList.Num())
		{
			return &ObjectsList;
		}
		return nullptr;
	}

	FORCEINLINE FArrayHashBucket()
		: Elements{ nullptr, nullptr }
	{
	}

	FORCEINLINE FArrayHashBucket(const FArrayHashBucket& Other)
		: Elements{ Other.Elements[0], Other.Elements[1] }
	{
	}

	FORCEINLINE ~FArrayHashBucket()
	{
		FObjectsArray* List = GetItems();
		if (List)
		{
			DestroyList(List);
		}
	}

	FORCEINLINE void DestroyList(FObjectsArray* List)
	{
		List->~FObjectsArray();
	}

	/** Adds an Object to the bucket */
	FORCEINLINE int32 Add(UObjectBase* Object)
	{
		FObjectsArray* Items = GetItems();
		if (Items)
		{
			return Items->Add(Object);
		}
		else if (Elements[0])
		{
			UObjectBase* Tmp = Elements[0];
			new (&ObjectsList) FObjectsArray();
			ObjectsList.Add(Tmp);
			return ObjectsList.Add(Object);
		}
		else
		{
			Elements[0] = Object;
			checkSlow(!Elements[1]);
			return 0;
		}
	}

	/** Removes an Object from the bucket */
	FORCEINLINE int32 Remove(UObjectBase* Object)
	{
		FObjectsArray* Items = GetItems();
		if (Items)
		{
			if (Items->Remove(Object))
			{
				if (Items->Num() == 1)
				{
					UObjectBase* Tmp = (*Items)[0];
					DestroyList(Items);
					Elements[0] = Tmp;
					Elements[1] = nullptr;
				}

				return 1;
			}
		}
		else if (Object == Elements[0])
		{
			Elements[0] = nullptr;
			return 1;
		}
		return 0;
	}

	/** Checks if an Object exists in this bucket */
	FORCEINLINE bool Contains(UObjectBase* Object) const
	{
		const FObjectsArray* Items = GetItems();
		if (Items)
		{
			return Items->Contains(Object);
		}
		return Object == Elements[0];
	}

	/** Returns the number of Objects in this bucket */
	FORCEINLINE int32 Num() const
	{
		const FObjectsArray* Items = GetItems();
		if (Items)
		{
			return Items->Num();
		}
		return !!Elements[0];
	}

	/** Returns the amount of memory allocated for and by Items TArray */
	FORCEINLINE SIZE_T GetAllocatedSize() const
	{
		const FObjectsArray* Items = GetItems();
		if (Items)
		{
			return Items->GetAllocatedSize();
		}
		return 0;
	}

	void Shrink()
	{
		FObjectsArray* Items = GetItems();
		if (Items)
		{
			Items->Shrink();
		}
	}

	FORCEINLINE struct THashBucketIterator<FArrayHashBucket> CreateIterator();

#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
	// special set of functions to support this code path
	FORCEINLINE UObjectBase*& Last()
	{
		FObjectsArray* Items = GetItems();
		if (Items)
		{
			return Items->Last();
		}

		return Elements[0];
	}

	FORCEINLINE void Pop()
	{
		FObjectsArray* Items = GetItems();
		if (Items)
		{
			Items->Pop();

			if (Items->Num() == 0)
			{
				DestroyList(Items);
				Elements[0] = nullptr;
				Elements[1] = nullptr;
			}
		}
		else
		{
			Elements[0] = nullptr;
		}
	}

	FORCEINLINE UObjectBase*& operator[](int32 Index)
	{
		FObjectsArray* Items = GetItems();
		if (Items)
		{
			return (*Items)[Index];
		}

		checkSlow(Index == 0);
		return Elements[0];
	}

#endif

	/** Gets an iterator for the TArray in this bucket or for the EmptyBucker if Items is null */
	FORCEINLINE TIterator GetIterator()
	{
		FObjectsArray* Items = GetItems();
		return Items ? Items->CreateIterator() : EmptyBucket.CreateIterator();
	}
};

typename FArrayHashBucket::FObjectsArray FArrayHashBucket::EmptyBucket;


/** Hash Bucket Iterator. Iterates over all Objects in the bucket */
template <typename T>
struct THashBucketIterator
{
	T& Bucket;
	typename T::TIterator Iterator;
	bool bItems;
	bool bReachedEndNoItems;
	bool bSecondItem;

	FORCEINLINE THashBucketIterator(T& InBucket)
		: Bucket(InBucket)
		, Iterator(InBucket.GetIterator())
		, bItems(!!InBucket.GetItems())
		, bReachedEndNoItems(!InBucket.Elements[0] && !InBucket.Elements[1])
		, bSecondItem(false)
	{
	}

	/** Advances the iterator to the next element. */
	FORCEINLINE THashBucketIterator& operator++()
	{
		if (bItems)
		{
			++Iterator;
		}
		else
		{
			bReachedEndNoItems = bSecondItem || !Bucket.Elements[1];
			bSecondItem = true;
		}
		return *this;
	}

	/** conversion to "bool" returning true if the iterator is valid. */
	FORCEINLINE explicit operator bool() const
	{
		if (bItems)
		{
			return (bool)Iterator;
		}
		else
		{
			return !bReachedEndNoItems;
		}
	}

	/** inverse of the "bool" operator */
	FORCEINLINE bool operator !() const
	{
		return !(bool)*this;
	}

	FORCEINLINE UObjectBase*& operator*()
	{
		if (bItems)
		{
			return *Iterator;
		}
		else
		{
			return (UObjectBase*&)Bucket.Elements[!!bSecondItem];
		}
	}
};

FORCEINLINE THashBucketIterator<FArrayHashBucket> FArrayHashBucket::CreateIterator()
{
	return THashBucketIterator(*this);
}

FORCEINLINE THashBucketIterator<FSetHashBucket> FSetHashBucket::CreateIterator()
{
	return THashBucketIterator(*this);
}

#if UE_UOBJECT_HASH_USES_ARRAYS
	using FHashBucketIterator = THashBucketIterator<FArrayHashBucket>;
	using FHashBucket = FArrayHashBucket;
#else
	// For example Editor builds need to use FSetHashBucket as a bucket type since we have a significantly higher number of UObjects than on a client, so perf with FArrayHashBucket will degrade
	using FHashBucketIterator = THashBucketIterator<FSetHashBucket>;
	using FHashBucket = FSetHashBucket;
#endif


/**
* Wrapper around a TMap with FHashBucket values that supports read only locks
*/
template <typename T, typename K = FHashBucket>
class TBucketMap : private TMap<T, K>
{
	typedef TMap<T, K> Super;
#if !UE_BUILD_SHIPPING
	int32 ReadOnlyLock = 0;
#endif // !UE_BUILD_SHIPPING
public:

	using Super::begin;
	using Super::end;
	using Super::GetAllocatedSize;
	using Super::Find;
	using Super::Num;
	using Super::FindChecked;

	FORCEINLINE void LockReadOnly()
	{
#if !UE_BUILD_SHIPPING
		ReadOnlyLock++;
#endif
	}
	FORCEINLINE void UnlockReadOnly()
	{
#if !UE_BUILD_SHIPPING
		ReadOnlyLock--;
		check(ReadOnlyLock >= 0);
#endif
	}

	FORCEINLINE void Compact()
	{
#if !UE_BUILD_SHIPPING
		UE_CLOGF(ReadOnlyLock != 0, LogObj, Fatal, "Trying to modify UObject map (Compact) that is currently being iterated. Please make sure you're not creating new UObjects or Garbage Collecting while iterating UObject hash tables.");
#endif
		Super::Compact();
	}

	FORCEINLINE void Add(const T& Key)
	{
#if !UE_BUILD_SHIPPING
		UE_CLOGF(ReadOnlyLock != 0, LogObj, Fatal, "Trying to modify UObject map (Add) that is currently being iterated. Please make sure you're not creating new UObjects or Garbage Collecting while iterating UObject hash tables.");
#endif
		Super::Add(Key);
	}

	FORCEINLINE void Remove(const T& Key)
	{
#if !UE_BUILD_SHIPPING
		UE_CLOGF(ReadOnlyLock != 0, LogObj, Fatal, "Trying to modify UObject map (Remove) that is currently being iterated. Please make sure you're not creating new UObjects or Garbage Collecting while iterating UObject hash tables.");
#endif
		Super::Remove(Key);
	}

	FORCEINLINE K& FindOrAdd(const T& Key)
	{
#if !UE_BUILD_SHIPPING
		UE_CLOGF(ReadOnlyLock != 0, LogObj, Fatal, "Trying to modify UObject map (FindOrAdd) that is currently being iterated. Please make sure you're not creating new UObjects or Garbage Collecting while iterating UObject hash tables.");
#endif
		return Super::FindOrAdd(Key);
	}

	FORCEINLINE K& FindOrAdd(T&& Key)
	{
#if !UE_BUILD_SHIPPING
		UE_CLOGF(ReadOnlyLock != 0, LogObj, Fatal, "Trying to modify UObject map (FindOrAdd) that is currently being iterated. Please make sure you're not creating new UObjects or Garbage Collecting while iterating UObject hash tables.");
#endif
		return Super::FindOrAdd(Key);
	}
};

/**
 * Simple TBucketMap read only lock scope
 */
template <typename T>
struct TBucketMapLock
{
	T& Map;
	explicit FORCEINLINE TBucketMapLock(T& InMap)
		: Map(InMap)
	{
		Map.LockReadOnly();
	}
	FORCEINLINE ~TBucketMapLock()
	{
		Map.UnlockReadOnly();
	}
};

class FUObjectHashTables
{
	/** Critical section that guards against concurrent adds from multiple threads */
	FTransactionallySafeCriticalSection CriticalSection;

public:

	/** Hash sets */
	TBucketMap<int32> Hash;
	TMultiMap<int32, uint32> HashOuter;

	/** Map of object to their outers, used to avoid an object iterator to find such things. **/
	TBucketMap<UObjectBase*> ObjectOuterMap;
#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
	TBucketMap<UClass*, FArrayHashBucket> ClassToObjectListMap;
#else
	TBucketMap<UClass*> ClassToObjectListMap;
#endif
	TMap<UClass*, TSet<UClass*> > ClassToChildListMap;
	std::atomic<uint64> AllClassesVersion;
	std::atomic<uint64> NativeClassesVersion;

	/** Map of package to the object their contain. */
	TBucketMap<UPackage*> PackageToObjectListMap;
	/** Map of object to their external package. */
	TMap<UObjectBase*, UPackage*> ObjectToPackageMap;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	TBucketMap<uint32> HashId;
#endif

	FUObjectHashTables()
		: AllClassesVersion(0)
		, NativeClassesVersion(0)
	{
	}

	UE_AUTORTFM_NOAUTORTFM  // This is not safe to be called from a closed transaction.
	void ShrinkMaps()
	{
		const EParallelForFlags BaseFlags = FTaskGraphInterface::IsRunning() ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread;
		double StartTime = FPlatformTime::Seconds();
		constexpr int32 NumHashTables = UE_WITH_REMOTE_OBJECT_HANDLE ? 8 : 7;
		ParallelFor(NumHashTables,
			[this](int32 Index)
		{
			switch (Index)
			{
			case 0:
				Hash.Compact();
				for (auto& Pair : Hash)
				{
					Pair.Value.Shrink();
				}
				break;
			case 1:
				HashOuter.Compact();
				break;
			case 2:
				ObjectOuterMap.Compact();
				for (auto& Pair : ObjectOuterMap)
				{
					Pair.Value.Shrink();
				}
				break;
			case 3:
				ClassToObjectListMap.Compact();
				for (auto& Pair : ClassToObjectListMap)
				{
					Pair.Value.Shrink();
				}
				break;
			case 4:
				ClassToChildListMap.Compact();
				for (auto& Pair : ClassToChildListMap)
				{
					Pair.Value.Shrink();
				}
				break;
			case 5:
				PackageToObjectListMap.Compact();
				for (auto& Pair : PackageToObjectListMap)
				{
					Pair.Value.Shrink();
				}
				break;
			case 6:
				ObjectToPackageMap.Compact();
				break;
#if UE_WITH_REMOTE_OBJECT_HANDLE
			case 7:
				HashId.Compact();
				break;
#endif
			}
		}, BaseFlags | EParallelForFlags::Unbalanced);
		UE_LOGF(LogUObjectHash, Log, "Compacting FUObjectHashTables data took %6.2fms", 1000.0f * float(FPlatformTime::Seconds() - StartTime));
	}

	/** Checks if the Hash/Object pair exists in the FName hash table */
	FORCEINLINE bool PairExistsInHash(int32 InHash, UObjectBase* Object)
	{
		bool bResult = false;
		FHashBucket* Bucket = Hash.Find(InHash);
		if (Bucket)
		{
			bResult = Bucket->Contains(Object);
		}
		return bResult;
	}
	/** Adds the Hash/Object pair to the FName hash table */
	FORCEINLINE void AddToHash(int32 InHash, UObjectBase* Object)
	{
		FHashBucket& Bucket = Hash.FindOrAdd(InHash);
		Bucket.Add(Object);
	}
	/** Removes the Hash/Object pair from the FName hash table */
	FORCEINLINE int32 RemoveFromHash(int32 InHash, UObjectBase* Object)
	{
		int32 NumRemoved = 0;
		FHashBucket* Bucket = Hash.Find(InHash);
		if (Bucket)
		{
			NumRemoved = Bucket->Remove(Object);
			if (Bucket->Num() == 0)
			{
				Hash.Remove(InHash);
			}
		}
		return NumRemoved;
	}

	FORCEINLINE void Lock()
	{
		CriticalSection.Lock();
	}

	FORCEINLINE void Unlock()
	{
		CriticalSection.Unlock();
	}

	static FUObjectHashTables& Get()
	{
		static FUObjectHashTables Singleton;
		return Singleton;
	}
};

class FHashTableLock
{
#if THREADSAFE_UOBJECTS
	FUObjectHashTables* Tables;
#endif
public:
	FORCEINLINE FHashTableLock(FUObjectHashTables& InTables)
	{
#if THREADSAFE_UOBJECTS
		if (!(IsGarbageCollectingAndLockingUObjectHashTables() && IsInGameThread()))
		{
			Tables = &InTables;
			InTables.Lock();
		}
		else
		{
			Tables = nullptr;
		}
#else
		check(IsInGameThread());
#endif
	}
	FORCEINLINE ~FHashTableLock()
	{
#if THREADSAFE_UOBJECTS
		if (Tables)
		{
			Tables->Unlock();
		}
#endif
	}
};

/**
 * Calculates the object's hash just using the object's name index
 *
 * @param ObjName the object's name to use the index of
 */
static FORCEINLINE int32 GetObjectHash(FName ObjName)
{
	return GetTypeHash(ObjName);
}

/**
 * Calculates the object's hash just using the object's name index
 * added with the outer. Yields much better spread in the hash
 * buckets, but requires knowledge of the outer, which isn't available
 * in all cases.
 *
 * @param ObjName the object's name to use the index of
 * @param Outer the object's outer pointer treated as an int32
 */
static int32 GetObjectOuterHash(FName ObjName, PTRINT Outer)
{
	return GetTypeHash(ObjName) + static_cast<int32>(Outer >> 6);
}

UObject* StaticFindObjectFastExplicitThreadSafe(FUObjectHashTables& ThreadHash, const UClass* ObjectClass, FName ObjectName, const FString& ObjectPathName, EFindObjectFlags Flags, EObjectFlags ExcludeFlags/*=0*/)
{
	const EInternalObjectFlags ExclusiveInternalFlags = GetMandatoryInternalExclusionFlags();

	// Find an object with the specified name and (optional) class, in any package; if bAnyPackage is false, only matches top-level packages
	int32 Hash = GetObjectHash(ObjectName);
	FHashTableLock HashLock(ThreadHash);
	FHashBucket* Bucket = ThreadHash.Hash.Find(Hash);
	if (Bucket)
	{
		bool bExactClass = !!(Flags & EFindObjectFlags::ExactClass);
		for (FHashBucketIterator It(*Bucket); It; ++It)
		{
			UObject* Object = (UObject*)*It;
			if
				((Object->GetFName() == ObjectName)

				/* Don't return objects that have any of the exclusive flags set */
				&& !Object->HasAnyFlags(ExcludeFlags)

				&& !Object->HasAnyInternalFlags(ExclusiveInternalFlags)

				/** If a class was specified, check that the object is of the correct class */
				&& (ObjectClass == nullptr || (bExactClass ? Object->GetClass() == ObjectClass : Object->IsA(ObjectClass)))
				)

			{
				FString ObjectPath = Object->GetPathName();
				/** Finally check the explicit path */
				if (ObjectPath == ObjectPathName)
				{
					checkf(!Object->IsUnreachable(), TEXT("%s"), *Object->GetFullName());
					return Object;
				}
			}
		}
	}

	return nullptr;
}

/**
 * Variation of StaticFindObjectFast that uses explicit path.
 *
 * @param	ObjectClass		The to be found object's class
 * @param	ObjectName		The to be found object's class
 * @param	ObjectPathName	Full path name for the object to search for
 * @param	ExactClass		Whether to require an exact match with the passed in class
 * @param	ExclusiveFlags	Ignores objects that contain any of the specified exclusive flags
 * @return	Returns a pointer to the found object or nullptr if none could be found
 */
UObject* StaticFindObjectFastExplicit( const UClass* ObjectClass, FName ObjectName, const FString& ObjectPathName, EFindObjectFlags Flags, EObjectFlags ExcludeFlags/*=0*/ )
{
	checkSlow(FPackageName::IsShortPackageName(ObjectName)); //@Package name transition, we aren't checking the name here because we know this is only used for texture

	// Find an object with the specified name and (optional) class, in any package; if bAnyPackage is false, only matches top-level packages
	const int32 Hash = GetObjectHash( ObjectName );
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	UObject* Result = StaticFindObjectFastExplicitThreadSafe( ThreadHash, ObjectClass, ObjectName, ObjectPathName, Flags, ExcludeFlags );

	return Result;
}

static bool NameEndsWith(FName Name, FName Suffix)
{
	if (Name == Suffix)
	{
		return true;
	}

	if (Name.GetNumber() != Suffix.GetNumber())
	{
		return false;
	}
		
	TCHAR PlainName[NAME_SIZE];
	TCHAR PlainSuffix[NAME_SIZE];
	uint32 NameLen = Name.GetPlainNameString(PlainName);
	uint32 SuffixLen = Suffix.GetPlainNameString(PlainSuffix);

	return NameLen >= SuffixLen && FCString::Strnicmp(PlainName + NameLen - SuffixLen, PlainSuffix, SuffixLen) == 0;
}

// Splits an object path into FNames representing an outer chain.
//
// Input path examples: "Object", "Package.Object", "Object:Subobject", "Object:Subobject.Nested", "Package.Object:Subobject", "Package.Object:Subobject.NestedSubobject"
struct FObjectSearchPath
{
	FName Inner;
	TArray<FName, TInlineAllocator<8>> Outers;

	explicit FObjectSearchPath(FName InPath)
	{
		TCHAR Buffer[NAME_SIZE];
		InPath.GetPlainNameString(Buffer);

		constexpr FAsciiSet DotColon(".:");
		const TCHAR* Begin = Buffer;
		const TCHAR* End = FAsciiSet::FindFirstOrEnd(Begin, DotColon);
		while (*End != '\0')
		{
			Outers.Add(FName(UE_PTRDIFF_TO_INT32(End - Begin), Begin));

			Begin = End + 1;
			End = FAsciiSet::FindFirstOrEnd(Begin, DotColon);
		}

		Inner = Outers.Num() == 0 ? InPath : FName(FStringView(Begin, UE_PTRDIFF_TO_INT32(End - Begin)), InPath.GetNumber());
	}

	bool MatchOuterNames(UObject* Outer) const
	{
		if (Outers.Num() == 0)
		{
			return true;
		}

		for (int32 Idx = Outers.Num() - 1; Idx > 0; --Idx)
		{
			if (!Outer || Outer->GetFName() != Outers[Idx])
			{
				return false;
			}

			Outer = Outer->GetOuter();
		}

		return Outer && NameEndsWith(Outer->GetFName(), Outers[0]);
	}
};

UObject* StaticFindObjectInPackageInternal(FUObjectHashTables& ThreadHash, const UClass* ObjectClass, const UPackage* ObjectPackage, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExcludeFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	ExclusiveInternalFlags |= GetMandatoryInternalExclusionFlags();
	UObject* Result = nullptr;
	if (FHashBucket* Inners = ThreadHash.PackageToObjectListMap.Find(ObjectPackage))
	{
		bool bExactClass = !!(Flags & EFindObjectFlags::ExactClass);
		for (FHashBucketIterator It(*Inners); It; ++It)
		{
			UObject* Object = static_cast<UObject*>(*It);
			if
				/* check that the name matches the name we're searching for */
				((Object->GetFName() == ObjectName)

					/* Don't return objects that have any of the exclusive flags set */
					&& !Object->HasAnyFlags(ExcludeFlags)

					/** Do not return ourselves (Packages currently have themselves as their package. )*/
					&& Object != ObjectPackage

					/** If a class was specified, check that the object is of the correct class */
					&& (ObjectClass == nullptr || (bExactClass ? Object->GetClass() == ObjectClass : Object->IsA(ObjectClass)))

					/** Include (or not) pending kill objects */
					&& !Object->HasAnyInternalFlags(ExclusiveInternalFlags))
			{
				checkf(!Object->IsUnreachable(), TEXT("%s"), *Object->GetFullName());
				Result = Object;
				break;
			}
		}
	}
	return Result;
}

UObject* StaticFindObjectFastInternalThreadSafe(FUObjectHashTables& ThreadHash, const UClass* ObjectClass, const UObject* ObjectOuter, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExcludeFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	ExclusiveInternalFlags |= GetMandatoryInternalExclusionFlags();

	// If they specified an outer use that during the hashing
	UObject* Result = nullptr;
	if (ObjectOuter != nullptr)
	{
		int32 Hash = GetObjectOuterHash(ObjectName, (PTRINT)ObjectOuter);
		FHashTableLock HashLock(ThreadHash);
		bool bExactClass = !!(Flags & EFindObjectFlags::ExactClass);
		for (TMultiMap<int32, uint32>::TConstKeyIterator HashIt(ThreadHash.HashOuter, Hash); HashIt; ++HashIt)
		{
			FUObjectItem* ObjectEntry = GUObjectArray.IndexToObject(HashIt.Value());
			UObject* Object = static_cast<UObject*>(ObjectEntry->GetObject());
			if
				/* check that the name matches the name we're searching for */
				((Object->GetFName() == ObjectName)

				/* Don't return objects that have any of the exclusive flags set */
				&& !Object->HasAnyFlags(ExcludeFlags)

				/* check that the object has the correct Outer */
				&& Object->GetOuter() == ObjectOuter

				/** If a class was specified, check that the object is of the correct class */
				&& (ObjectClass == nullptr || (bExactClass ? Object->GetClass() == ObjectClass : Object->IsA(ObjectClass)))
				
				/** Include (or not) pending kill objects */
				&& !ObjectEntry->HasAnyFlags(ExclusiveInternalFlags))
			{
				checkf(!ObjectEntry->IsUnreachable(), TEXT("%s"), *Object->GetFullName());
				if (Result)
				{
					UE_LOGF(LogUObjectHash, Warning, "Ambiguous search, could be %ls or %ls", *GetFullNameSafe(Result), *GetFullNameSafe(Object));
				}
				else
				{
					Result = Object;
				}
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
				break;
#endif
			}
		}

#if WITH_EDITOR
		// if the search fail and the ObjectOuter is a UPackage, lookup potential external package
		if (Result == nullptr)
		{
			if (const UPackage* ObjectPackage = Cast<UPackage>(ObjectOuter))
			{
				Result = StaticFindObjectInPackageInternal(ThreadHash, ObjectClass, ObjectPackage, ObjectName, Flags, ExcludeFlags, ExclusiveInternalFlags);
			}
		}
#endif
	}
	else
	{
		FObjectSearchPath SearchPath(ObjectName);

		const int32 Hash = GetObjectHash(SearchPath.Inner);
		FHashTableLock HashLock(ThreadHash);

		FHashBucket* Bucket = ThreadHash.Hash.Find(Hash);
		if (Bucket)
		{
			bool bExactClass = !!(Flags & EFindObjectFlags::ExactClass);
			for (FHashBucketIterator It(*Bucket); It; ++It)
			{
				UObject* Object = (UObject*)*It;

				if
					((Object->GetFName() == SearchPath.Inner)

					/* Don't return objects that have any of the exclusive flags set */
					&& !Object->HasAnyFlags(ExcludeFlags)

					/*If there is no package (no InObjectPackage specified, and InName's package is "")
					and the caller specified any_package, then accept it, regardless of its package.
					Or, if the object is a top-level package then accept it immediately.*/
					&& (!Object->GetOuter())

					/** If a class was specified, check that the object is of the correct class */
					&& (ObjectClass == nullptr || (bExactClass ? Object->GetClass() == ObjectClass : Object->IsA(ObjectClass)))

					/** Include (or not) pending kill objects */
					&& !Object->HasAnyInternalFlags(ExclusiveInternalFlags)

					/** Ensure that the partial path provided matches the object found */
					&& SearchPath.MatchOuterNames(Object->GetOuter()))
				{
					checkf(!Object->IsUnreachable(), TEXT("%s"), *Object->GetFullName());
					if (Result)
					{
						UE_LOGF(LogUObjectHash, Warning, "Ambiguous path search, could be %ls or %ls", *GetFullNameSafe(Result), *GetFullNameSafe(Object));
					}
					else
					{
						Result = Object;
					}
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
					break;
#endif
				}
			}
		}
	}
	// Not found.
	if (Result && UE::GC::GIsIncrementalReachabilityPending)
	{
		UE::GC::MarkAsReachable(Result);
	}
	return Result;
}

UObject* StaticFindObjectFastInternal(const UClass* ObjectClass, const UObject* ObjectPackage, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExcludeFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	UObject* Result = nullptr;

	// Transactionally, a static find operation has to occur in the open as it touches shared state.
	UE_AUTORTFM_OPEN
	{
		INC_DWORD_STAT(STAT_FindObjectFast);

		// If they specified an outer use that during the hashing
		FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
		Result = StaticFindObjectFastInternalThreadSafe(ThreadHash, ObjectClass, ObjectPackage, ObjectName, Flags, ExcludeFlags, ExclusiveInternalFlags);
	};

	return Result;
}

// Approximate search for finding unused object names
bool DoesObjectPossiblyExist(const UObject* InOuter, FName ObjectName)
{
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	check(InOuter != nullptr);
	int32 Hash = GetObjectOuterHash(ObjectName, (PTRINT)InOuter);
	FHashTableLock HashLock(ThreadHash);
	// We don't need to iterate the multimap here as we are happy with false positives.
	return ThreadHash.HashOuter.Contains(Hash);
}

bool StaticFindAllObjectsFastInternal(TArray<UObject*>& OutFoundObjects, const UClass* ObjectClass, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExcludeFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	INC_DWORD_STAT(STAT_FindObjectFast);

	ExclusiveInternalFlags |= GetMandatoryInternalExclusionFlags();

	FObjectSearchPath SearchPath(ObjectName);
	const int32 Hash = GetObjectHash(SearchPath.Inner);
	uint32 NumFoundObjects = 0; // Keeping track of the number of objects foundn. We allow OutFoundObjects to not be empty

	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);

	FHashBucket* Bucket = ThreadHash.Hash.Find(Hash);
	if (Bucket)
	{
		bool bExactClass = !!(Flags & EFindObjectFlags::ExactClass);
		for (FHashBucketIterator It(*Bucket); It; ++It)
		{
			UObject* Object = (UObject*)*It;

			if
				((Object->GetFName() == SearchPath.Inner)

					/* Don't return objects that have any of the exclusive flags set */
					&& !Object->HasAnyFlags(ExcludeFlags)

					/** If a class was specified, check that the object is of the correct class */
					&& (ObjectClass == nullptr || (bExactClass ? Object->GetClass() == ObjectClass : Object->IsA(ObjectClass)))

					/** Include (or not) pending kill objects */
					&& !Object->HasAnyInternalFlags(ExclusiveInternalFlags)

					/** Ensure that the partial path provided matches the object found */
					&& SearchPath.MatchOuterNames(Object->GetOuter()))
			{
				checkf(!Object->IsUnreachable(), TEXT("%s"), *Object->GetFullName());
				OutFoundObjects.Add(Object);
				NumFoundObjects++;
			}
		}
	}
	return !!NumFoundObjects;
}

UObject* StaticFindFirstObjectFastInternal(const UClass* ObjectClass, FName ObjectName, EFindObjectFlags Flags, EObjectFlags ExcludeFlags, EInternalObjectFlags ExclusiveInternalFlags)
{
	INC_DWORD_STAT(STAT_FindObjectFast);

	ExclusiveInternalFlags |= GetMandatoryInternalExclusionFlags();

	UObject* Result = nullptr;
	FObjectSearchPath SearchPath(ObjectName);
	const int32 Hash = GetObjectHash(SearchPath.Inner);
	uint32 NumFoundObjects = 0; // Keeping track of the number of objects found. We allow OutFoundObjects to not be empty

	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);

	FHashBucket* Bucket = ThreadHash.Hash.Find(Hash);
	if (Bucket)
	{
		bool bExactClass = !!(Flags & EFindObjectFlags::ExactClass);
		for (FHashBucketIterator It(*Bucket); It; ++It)
		{
			UObject* Object = (UObject*)*It;

			if
				((Object->GetFName() == SearchPath.Inner)

					/* Don't return objects that have any of the exclusive flags set */
					&& !Object->HasAnyFlags(ExcludeFlags)

					/** If a class was specified, check that the object is of the correct class */
					&& (ObjectClass == nullptr || (bExactClass ? Object->GetClass() == ObjectClass : Object->IsA(ObjectClass)))

					/** Include (or not) pending kill objects */
					&& !Object->HasAnyInternalFlags(ExclusiveInternalFlags)

					/** Ensure that the partial path provided matches the object found */
					&& SearchPath.MatchOuterNames(Object->GetOuter()))
			{
				checkf(!Object->IsUnreachable(), TEXT("%s"), *Object->GetFullName());
				Result = Object;
				break;
			}
		}
	}
	return Result;
}

UObject* StaticFindObjectFastInternalThreadSafe(FUObjectHashTables& ThreadHash, FRemoteObjectId RemoteId, EObjectFlags InExclusiveFlags, EInternalObjectFlags InExlusiveInternalFlags)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	const EInternalObjectFlags ExclusiveInternalFlags = GetMandatoryInternalExclusionFlags() | InExlusiveInternalFlags;
	RemoteId = UE::RemoteObject::Private::FRemoteIdLocalizationHelper::GetLocalized(RemoteId);
	uint32 Hash = GetTypeHash(RemoteId);

	FHashTableLock HashLock(ThreadHash);
	FHashBucket* Bucket = ThreadHash.HashId.Find(Hash);
	if (Bucket)
	{
		for (FHashBucketIterator It(*Bucket); It; ++It)
		{
			UObject* Object = (UObject*)*It;
			if ((FRemoteObjectId(Object) == RemoteId)
				/* Don't return objects that have any of the exclusive flags set */
				&& !Object->HasAnyFlags(InExclusiveFlags)
				&& !Object->HasAnyInternalFlags(ExclusiveInternalFlags)
				)
			{
				checkf(!Object->IsUnreachable(), TEXT("%s"), *Object->GetFullName());
				if (UE::GC::GIsIncrementalReachabilityPending)
				{
					UE::GC::MarkAsReachable(Object);
				}
				return Object;
			}
		}
	}
#else
	UE_LOGF(LogUObjectHash, Fatal, "StaticFindObjectFastInternal override that takes FRemoteObjectId can only be used with remote object handles enabled");
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	return nullptr;
}

UObject* StaticFindObjectFastInternal(FRemoteObjectId RemoteId, EObjectFlags InExclusiveFlags, EInternalObjectFlags InExlusiveInternalFlags)
{
	UObject* Result = nullptr;

	// Transactionally, a static find operation has to occur in the open as it touches shared state.
	UE_AUTORTFM_OPEN
	{
		INC_DWORD_STAT(STAT_FindObjectFast);

		FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
		Result = StaticFindObjectFastInternalThreadSafe(ThreadHash, RemoteId, InExclusiveFlags, InExlusiveInternalFlags);
	};

	return Result;
}


// Assumes that ThreadHash's critical is already locked
FORCEINLINE static void AddToOuterMap(FUObjectHashTables& ThreadHash, UObjectBase* Object)
{
	ensureMsgf(!AutoRTFM::IsClosed(), TEXT("ObjectOuterMap must not be modified from a closed transaction"));
	FHashBucket& Bucket = ThreadHash.ObjectOuterMap.FindOrAdd(Object->GetOuter());
	checkSlow(!Bucket.Contains(Object)); // if it already exists, something is wrong with the external code
	Bucket.Add(Object);
}

// Assumes that ThreadHash's critical is already locked
FORCEINLINE void AddToClassMap(FUObjectHashTables& ThreadHash, UObjectBase* Object)
{
	{
		check(Object->GetClass());
#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
		Object->GetObjectHashInternalIndex() = 
#endif
		ThreadHash.ClassToObjectListMap.FindOrAdd(Object->GetClass()).Add(Object);
	}

	UObjectBaseUtility* ObjectWithUtility = static_cast<UObjectBaseUtility*>(Object);
	if ( ObjectWithUtility->IsA(UClass::StaticClass()) )
	{
		UClass* Class = static_cast<UClass*>(ObjectWithUtility);
		UClass* SuperClass = Class->GetSuperClass();
		if ( SuperClass )
		{
			TSet<UClass*>& ChildList = ThreadHash.ClassToChildListMap.FindOrAdd(SuperClass);
			bool bIsAlreadyInSetPtr = false;
			ChildList.Add(Class, &bIsAlreadyInSetPtr);
			check(!bIsAlreadyInSetPtr); // if it already exists, something is wrong with the external code
		}
		ThreadHash.AllClassesVersion++;
		if (Class->IsNative())
		{
			ThreadHash.NativeClassesVersion++;
		}
	}
}

// Assumes that ThreadHash's critical is already locked
FORCEINLINE static void AddToPackageMap(FUObjectHashTables& ThreadHash, UObjectBase* Object, UPackage* Package)
{
	check(Package != nullptr);
	FHashBucket& Bucket = ThreadHash.PackageToObjectListMap.FindOrAdd(Package);
	checkSlow(!Bucket.Contains(Object)); // if it already exists, something is wrong with the external code
	Bucket.Add(Object);
}

// Assumes that ThreadHash's critical is already locked
FORCEINLINE static UPackage* AssignExternalPackageToObject(FUObjectHashTables& ThreadHash, UObjectBase* Object, UPackage* Package)
{
	UPackage*& ExternalPackageRef = ThreadHash.ObjectToPackageMap.FindOrAdd(Object);
	UPackage* OldPackage = ExternalPackageRef;
	ExternalPackageRef = Package;
	return OldPackage;
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
// Assumes that ThreadHash's critical is already locked
FORCEINLINE static void AddToRemoteIdMap(FUObjectHashTables& ThreadHash, UObjectBase* Object)
{
	FRemoteObjectId RemoteId(Object);
	FHashBucket& Bucket = ThreadHash.HashId.FindOrAdd(GetTypeHash(RemoteId));
#if DO_CHECK
	for (FHashBucketIterator It(Bucket); It; ++It)
	{
		UObject* ExistingObject = (UObject*)*It;
		checkf(ExistingObject != Object, TEXT("%s (%s) is being added to the remote id map for the second time"), *GetPathNameSafe(ExistingObject), *RemoteId.ToString());
		FRemoteObjectId ExistingId(ExistingObject);
		checkf(ExistingId != RemoteId || ExistingObject->IsUnreachable(), TEXT("Trying to add %s to the remote id map but it has the same remote id (%s) as %s"), *GetPathNameSafe((UObject*)Object), *RemoteId.ToString(), *GetPathNameSafe(ExistingObject));
	}
#endif
	Bucket.Add(Object);
}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

// Assumes that ThreadHash's critical is already locked
FORCEINLINE static void RemoveFromOuterMap(FUObjectHashTables& ThreadHash, UObjectBase* Object)
{
	ensureMsgf(!AutoRTFM::IsClosed(), TEXT("ObjectOuterMap must not be modified from a closed transaction"));
	UObject* Outer = UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Object);
	FHashBucket& Bucket = ThreadHash.ObjectOuterMap.FindOrAdd(Outer);
	int32 NumRemoved = Bucket.Remove(Object);

	if (!LIKELY(NumRemoved == 1))
	{
		OnHashFailure((UObjectBaseUtility*)Object, TEXT("OuterMap"), TEXT("remove miscount"));
	}

	if (!Bucket.Num())
	{
		ThreadHash.ObjectOuterMap.Remove(Outer);
	}
}

// Assumes that ThreadHash's critical is already locked
FORCEINLINE void RemoveFromClassMap(FUObjectHashTables& ThreadHash, UObjectBase* Object)
{
	UObjectBaseUtility* ObjectWithUtility = static_cast<UObjectBaseUtility*>(Object);

	{
		auto& ObjectList = ThreadHash.ClassToObjectListMap.FindOrAdd(Object->GetClass());
#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
		ensureMsgf(ObjectList[Object->GetObjectHashInternalIndex()] == Object,
			TEXT("Object doesn't match the one stored in the ObjectList"));
		ObjectList.Last()->GetObjectHashInternalIndex() = Object->GetObjectHashInternalIndex();
		ObjectList[Object->GetObjectHashInternalIndex()] = ObjectList.Last();
		ObjectList.Pop();
#else
		int32 NumRemoved = ObjectList.Remove(Object);

		if (!LIKELY(NumRemoved == 1))
		{
			OnHashFailure((UObjectBaseUtility*)Object, TEXT("ClassMap"), TEXT("remove miscount"));
		}
#endif

		if (!ObjectList.Num())
		{
			ThreadHash.ClassToObjectListMap.Remove(Object->GetClass());
		}
	}

	if ( ObjectWithUtility->IsA(UClass::StaticClass()) )
	{
		UClass* Class = static_cast<UClass*>(ObjectWithUtility);
		UClass* SuperClass = Class->GetSuperClass();
		if ( SuperClass )
		{
			// Remove the class from the SuperClass' child list
			TSet<UClass*>& ChildList = ThreadHash.ClassToChildListMap.FindOrAdd(SuperClass);
			int32 NumRemoved = ChildList.Remove(Class);

			if (!LIKELY(NumRemoved == 1))
			{
				OnHashFailure((UObjectBaseUtility*)Object, TEXT("ClassToChildListMap"), TEXT("remove miscount"));
			}

			if (!ChildList.Num())
			{
				ThreadHash.ClassToChildListMap.Remove(SuperClass);
			}
		}
		ThreadHash.AllClassesVersion++;
		if (Class->IsNative())
		{
			ThreadHash.NativeClassesVersion++;
		}
	}
}

// Assumes that ThreadHash's critical is already locked
FORCEINLINE static void RemoveFromPackageMap(FUObjectHashTables& ThreadHash, UObjectBase* Object, UPackage* Package)
{
	check(Package != nullptr);
	FHashBucket& Bucket = ThreadHash.PackageToObjectListMap.FindOrAdd(Package);
	int32 NumRemoved = Bucket.Remove(Object);

	if (!LIKELY(NumRemoved == 1))
	{
		OnHashFailure((UObjectBaseUtility*)Object, TEXT("PackageMap"), TEXT("remove miscount"));
	}

	if (!Bucket.Num())
	{
		ThreadHash.PackageToObjectListMap.Remove(Package);
	}
}

// Assumes that ThreadHash's critical is already locked
FORCEINLINE static UPackage* UnassignExternalPackageFromObject(FUObjectHashTables& ThreadHash, UObjectBase* Object)
{
	UPackage* OldPackage = nullptr;
	ThreadHash.ObjectToPackageMap.RemoveAndCopyValue(Object, OldPackage);
	return OldPackage;
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
// Assumes that ThreadHash's critical is already locked
FORCEINLINE static void RemoveFromRemoteIdMap(FUObjectHashTables& ThreadHash, UObjectBase* Object)
{
	const uint32 Hash = GetTypeHash(FRemoteObjectId(Object));
	FHashBucket& Bucket = ThreadHash.HashId.FindOrAdd(Hash);
	int32 NumRemoved = Bucket.Remove(Object);

	if (!LIKELY(NumRemoved == 1))
	{
		OnHashFailure((UObjectBaseUtility*)Object, TEXT("HashId"), TEXT("remove miscount"));
	}

	if (!Bucket.Num())
	{
		ThreadHash.HashId.Remove(Hash);
	}
}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

void ShrinkUObjectHashTables()
{
	LLM_SCOPE_BYTAG(UObjectHash);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();
	TRACE_CPUPROFILER_EVENT_SCOPE(ShrinkUObjectHashTables);
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);
	ThreadHash.ShrinkMaps();
}

uint64 GetRegisteredClassesVersionNumber()
{
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	return ThreadHash.AllClassesVersion;
}

uint64 GetRegisteredNativeClassesVersionNumber()
{
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	return ThreadHash.NativeClassesVersion;
}

static void ShrinkUObjectHashTablesDel(const TArray<FString>& Args)
{
	ShrinkUObjectHashTables();
}

static FAutoConsoleCommand ShrinkUObjectHashTablesCmd(
	TEXT("ShrinkUObjectHashTables"),
	TEXT("Shrinks all of the UObject hash tables."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&ShrinkUObjectHashTablesDel)
);

bool GAllowGetObjectsReturnUnreachableObjects = false;

static inline void AddMandatoryInternalExclusionFlags(EInternalObjectFlags& InOutExclusionFlags)
{
	InOutExclusionFlags |= GetMandatoryInternalExclusionFlags();
}
static inline void ConditionallyRemoveUnreachableFlag(EInternalObjectFlags& InOutExclusionFlags, EGetObjectsFlags Flags)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (EnumHasAnyFlags(Flags, EGetObjectsFlags::EvenIfUnreachable))
	{
		checkf(GAllowGetObjectsReturnUnreachableObjects, TEXT("EGetObjectsFlags::EvenIfUnreachable specified but it's not allowed in the current context"));
		InOutExclusionFlags &= ~(EInternalObjectFlags::Unreachable);
	}
#endif
}

void GetObjectsWithOuter(const class UObjectBase* Outer, TArray<UObject *>& Results, EGetObjectsFlags Flags, EObjectFlags ExclusionFlags, EInternalObjectFlags ExclusionInternalFlags)
{
	checkf(Outer != nullptr, TEXT("Getting objects with a null outer is no longer supported. If you want to get all packages you might consider using GetObjectsOfClass instead."));

#if WITH_EDITOR
	if (const UPackage* OuterPackage = Cast<UPackage>(Outer))
	{
		// uncomment ensure to more easily find place where GetObjectsWithOuter should be replaced with GetObjectsWithPackage
		// ensure(!OuterPackage);

		GetObjectsWithPackage(OuterPackage, Results, Flags, ExclusionFlags, ExclusionInternalFlags);
		return;
	}
#endif

	// We don't want to return any objects that are currently being background loaded unless we're using the object iterator during async loading.
	AddMandatoryInternalExclusionFlags(ExclusionInternalFlags);
	ConditionallyRemoveUnreachableFlag(ExclusionInternalFlags, Flags);
	const bool bIncludeNestedObjects = EnumHasAnyFlags(Flags, EGetObjectsFlags::IncludeNestedObjects);

	int32 StartNum = Results.Num();
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);
	FHashBucket* Inners = ThreadHash.ObjectOuterMap.Find(Outer);
	if (Inners)
	{
		for (FHashBucketIterator It(*Inners); It; ++It)
		{
			UObject *Object = static_cast<UObject *>(*It);
			if (!Object->HasAnyFlags(ExclusionFlags) && !Object->HasAnyInternalFlags(ExclusionInternalFlags))
			{
				if (UE::GC::GIsIncrementalReachabilityPending)
				{
					UE::GC::MarkAsReachable(Object);
				}
				Results.Add(Object);
			}
		}
		int32 MaxResults = GUObjectArray.GetObjectArrayNum();
		while (StartNum != Results.Num() && bIncludeNestedObjects)
		{
			int32 RangeStart = StartNum;
			int32 RangeEnd = Results.Num();
			StartNum = RangeEnd;
			for (int32 Index = RangeStart; Index < RangeEnd; Index++)
			{
				FHashBucket* InnerInners = ThreadHash.ObjectOuterMap.Find(Results[Index]);
				if (InnerInners)
				{
					for (FHashBucketIterator It(*InnerInners); It; ++It)
					{
						UObject *Object = static_cast<UObject *>(*It);
						if (!Object->HasAnyFlags(ExclusionFlags) && !Object->HasAnyInternalFlags(ExclusionInternalFlags))
						{
							if (UE::GC::GIsIncrementalReachabilityPending)
							{
								UE::GC::MarkAsReachable(Object);
							}
							Results.Add(Object);
						}
					}
				}
			}
			check(Results.Num() <= MaxResults); // otherwise we have a cycle in the outer chain, which should not be possible
		}
	}
}

void ForEachObjectWithOuterBreakable(const class UObjectBase* Outer, TFunctionRef<bool(UObject*)> Operation, EGetObjectsFlags Flags, EObjectFlags ExclusionFlags, EInternalObjectFlags ExclusionInternalFlags)
{
	checkf(Outer != nullptr, TEXT("Getting objects with a null outer is no longer supported. If you want to get all packages you might consider using GetObjectsOfClass instead."));
	
#if WITH_EDITOR
	if (const UPackage* OuterPackage = Cast<UPackage>(Outer))
	{
		// uncomment ensure to more easily find place where GetObjectsWithOuter should be replaced with GetObjectsWithPackage
		// ensure(!OuterPackage)

		ForEachObjectWithPackage(OuterPackage, [Operation](UObject* InObject) { Operation(InObject); return true; }, Flags, ExclusionFlags, ExclusionInternalFlags);
		return;
	}
#endif

	// We don't want to return any objects that are currently being background loaded unless we're using the object iterator during async loading.
	AddMandatoryInternalExclusionFlags(ExclusionInternalFlags);
	ConditionallyRemoveUnreachableFlag(ExclusionInternalFlags, Flags);
	const bool bIncludeNestedObjects = EnumHasAnyFlags(Flags, EGetObjectsFlags::IncludeNestedObjects);

	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);
	TArray<FHashBucket*, TInlineAllocator<1> > AllInners;

	TBucketMapLock MapLock(ThreadHash.ObjectOuterMap);
	if (FHashBucket* Inners = ThreadHash.ObjectOuterMap.Find(Outer))
	{
		AllInners.Add(Inners);
	}
	while (AllInners.Num())
	{
		FHashBucket* Inners = AllInners.Pop();
		for (FHashBucketIterator It(*Inners); It; ++It)
		{
			UObject *Object = static_cast<UObject*>(*It);
			if (!Object->HasAnyFlags(ExclusionFlags) && !Object->HasAnyInternalFlags(ExclusionInternalFlags))
			{
				if (UE::GC::GIsIncrementalReachabilityPending)
				{
					UE::GC::MarkAsReachable(Object);
				}
				if (!Operation(Object))
				{
					AllInners.Empty();
					break;
				}
			}
			if (bIncludeNestedObjects)
			{
				if (FHashBucket* ObjectInners = ThreadHash.ObjectOuterMap.Find(Object))
				{
					AllInners.Add(ObjectInners);
				}
			}
		}
	}
}

UObjectBase* FindObjectWithOuter(const class UObjectBase* Outer, const class UClass* ClassToLookFor, FName NameToLookFor)
{
	UObject* Result = nullptr;
	check( Outer );
	
	if(!NameToLookFor.IsNone())
	{
		// Don't calculate mandatory exclusion flags here as they are handled inside
		Result = StaticFindObjectFastInternal(ClassToLookFor, static_cast<const UObject*>(Outer), NameToLookFor);
	}
	else
	{
		// We don't want to return any objects that are currently being background loaded unless we're using the object iterator during async loading.
		EInternalObjectFlags ExclusionInternalFlags = GetMandatoryInternalExclusionFlags();

		FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
		FHashTableLock HashLock(ThreadHash);
		FHashBucket* Inners = ThreadHash.ObjectOuterMap.Find(Outer);
		if (Inners)
		{
			for (FHashBucketIterator It(*Inners); It; ++It)
			{
				UObject *Object = static_cast<UObject *>(*It);
				if (Object->HasAnyInternalFlags(ExclusionInternalFlags))
				{
					continue;
				}
				if (ClassToLookFor && !Object->IsA(ClassToLookFor))
				{
					continue;
				}
				Result = Object;
				break;
			}
			if (Result && UE::GC::GIsIncrementalReachabilityPending)
			{
				UE::GC::MarkAsReachable(Result);
			}
		}
	}
	return Result;
}

void GetObjectsWithPackage(const class UPackage* Package, TArray<UObject *>& Results, EGetObjectsFlags Flags, EObjectFlags ExclusionFlags, EInternalObjectFlags ExclusionInternalFlags)
{
	ForEachObjectWithPackage(Package, [&Results](UObject* Object)
	{
		Results.Add(Object);
		return true;
	}, Flags, ExclusionFlags, ExclusionInternalFlags);
}

void ForEachObjectWithPackage(const class UPackage* Package, TFunctionRef<bool(UObject*)> Operation, EGetObjectsFlags Flags, EObjectFlags ExclusionFlags, EInternalObjectFlags ExclusionInternalFlags)
{
	check(Package != nullptr);

	// We don't want to return any objects that are currently being background loaded unless we're using the object iterator during async loading.
	AddMandatoryInternalExclusionFlags(ExclusionInternalFlags);
	ConditionallyRemoveUnreachableFlag(ExclusionInternalFlags, Flags);
	const bool bIncludeNestedObjects = EnumHasAnyFlags(Flags, EGetObjectsFlags::IncludeNestedObjects);

	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);
	TArray<FHashBucket*, TInlineAllocator<1> > AllInners;
	
	TBucketMapLock PackageToObjectListMapLock(ThreadHash.PackageToObjectListMap);
	TBucketMapLock ObjectOuterMapLock(ThreadHash.ObjectOuterMap);

	// Add the object bucket that have this package as an external package
	if (FHashBucket* Inners = ThreadHash.PackageToObjectListMap.Find(Package))
	{
		AllInners.Add(Inners);
	}
	// Add the object bucket that have this package as an outer
	if (FHashBucket* ObjectInners = ThreadHash.ObjectOuterMap.Find(Package))
	{
		AllInners.Add(ObjectInners);
	}
	while (AllInners.Num())
	{
		FHashBucket* Inners = AllInners.Pop();
		for (FHashBucketIterator It(*Inners); It; ++It)
		{
			UObject *Object = static_cast<UObject*>(*It);

			UPackage* ObjectPackage = Object->GetExternalPackageInternal();
			bool bIsInPackage = ObjectPackage == Package || ObjectPackage == nullptr;
			
			if (!Object->HasAnyFlags(ExclusionFlags) && 
				!Object->HasAnyInternalFlags(ExclusionInternalFlags) &&
				bIsInPackage)
			{
				if (UE::GC::GIsIncrementalReachabilityPending)
				{
					UE::GC::MarkAsReachable(Object);
				}
				if (!Operation(Object))
				{
					AllInners.Empty(AllInners.Max());
					break;
				}
			}
			if (bIncludeNestedObjects && bIsInPackage)
			{
				if (FHashBucket* ObjectInners = ThreadHash.ObjectOuterMap.Find(Object))
				{
					AllInners.Add(ObjectInners);
				}
			}
		}
	}
}

/** Helper function that returns all the children of the specified class recursively */
template<typename ClassType, typename ArrayAllocator>
static void RecursivelyPopulateDerivedClasses(FUObjectHashTables& ThreadHash, const UClass* ParentClass, TArray<ClassType, ArrayAllocator>& OutAllDerivedClass, const EInternalObjectFlags ExclusionFlags)
{
	// Start search with the parent class at virtual index Num-1, then continue searching from index Num as things are added
	int32 SearchIndex = OutAllDerivedClass.Num() - 1;
	const UClass* SearchClass = ParentClass;

	while (1)
	{
		TSet<UClass*>* ChildSet = ThreadHash.ClassToChildListMap.Find(SearchClass);
		if (ChildSet)
		{
			OutAllDerivedClass.Reserve(OutAllDerivedClass.Num() + ChildSet->Num());
			for (UClass* ChildClass : *ChildSet)
			{
				if (ExclusionFlags == EInternalObjectFlags::None || !ChildClass->HasAnyInternalFlags(ExclusionFlags))
				{
					OutAllDerivedClass.Add(ChildClass);
				}
			}
		}

		// Now search at next index, if it has been filled in by code above
		SearchIndex++;

		if (SearchIndex < OutAllDerivedClass.Num())
		{
			SearchClass = OutAllDerivedClass[SearchIndex];
		}
		else
		{
			return;
		}
	}
}

void GetObjectsOfClass(const UClass* ClassToLookFor, TArray<UObject *>& Results, bool bIncludeDerivedClasses, EObjectFlags ExclusionFlags, EInternalObjectFlags ExclusionInternalFlags)
{
	SCOPE_CYCLE_COUNTER(STAT_Hash_GetObjectsOfClass);

	ForEachObjectOfClass(ClassToLookFor,
		[&Results](UObject* Object)
		{
			Results.Add(Object);
		}
	, bIncludeDerivedClasses, ExclusionFlags, ExclusionInternalFlags);

	check(Results.Num() <= GUObjectArray.GetObjectArrayNum()); // otherwise we have a cycle in the outer chain, which should not be possible
}

FORCEINLINE void ForEachObjectOfClasses_Implementation(FUObjectHashTables& ThreadHash, TArrayView<const UClass* const> ClassesToLookFor, TFunctionRef<void(UObject*)> Operation, EObjectFlags ExcludeFlags /*= RF_ClassDefaultObject*/, EInternalObjectFlags ExclusionInternalFlags /*= EInternalObjectFlags::None*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ForEachObjectOfClasses_Implementation);

	// We don't want to return any objects that are currently being background loaded unless we're using the object iterator during async loading.
	ExclusionInternalFlags |= GetMandatoryInternalExclusionFlags();

	TBucketMapLock ClassToObjectListMapLock(ThreadHash.ClassToObjectListMap);

	for (const UClass* SearchClass : ClassesToLookFor)
	{
		auto List = ThreadHash.ClassToObjectListMap.Find(SearchClass);
		if (List)
		{
			for (auto ObjectIt = List->CreateIterator(); ObjectIt; ++ObjectIt)
			{
				UObject* Object = static_cast<UObject*>(*ObjectIt);
				if (!Object->HasAnyFlags(ExcludeFlags) && !Object->HasAnyInternalFlags(ExclusionInternalFlags))
				{
					if (UE::GC::GIsIncrementalReachabilityPending)
					{
						UE::GC::MarkAsReachable(Object);
					}
					Operation(Object);
				}
			}
		}
	}
}

void ForEachObjectOfClass(const UClass* ClassToLookFor, TFunctionRef<void(UObject*)> Operation, bool bIncludeDerivedClasses, EObjectFlags ExclusionFlags, EInternalObjectFlags ExclusionInternalFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ForEachObjectOfClass);

	// Most classes searched for have around 10 subclasses, some have hundreds
	TArray<const UClass*, TInlineAllocator<16>> ClassesToSearch;
	ClassesToSearch.Add(ClassToLookFor);

	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);

	if (bIncludeDerivedClasses)
	{
		// RecursivelyPopulate expects the full exclusion flags, while the implementation function adds them
		RecursivelyPopulateDerivedClasses(ThreadHash, ClassToLookFor, ClassesToSearch, ExclusionInternalFlags | GetMandatoryInternalExclusionFlags());
	}

	ForEachObjectOfClasses_Implementation(ThreadHash, ClassesToSearch, Operation, ExclusionFlags, ExclusionInternalFlags);
}

void ForEachObjectOfClasses(TArrayView<const UClass* const> ClassesToLookFor, TFunctionRef<void(UObject*)> Operation, EObjectFlags ExcludeFlags /*= RF_ClassDefaultObject*/, EInternalObjectFlags ExclusionInternalFlags /*= EInternalObjectFlags::None*/)
{
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);

	ForEachObjectOfClasses_Implementation(ThreadHash, ClassesToLookFor, Operation, ExcludeFlags, ExclusionInternalFlags);
}

void GetDerivedClasses(const UClass* ClassToLookFor, TArray<UClass*>& Results, bool bRecursive)
{
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);

	const EInternalObjectFlags ExclusionFlags = GetMandatoryInternalExclusionFlags();
	if (bRecursive)
	{
		RecursivelyPopulateDerivedClasses(ThreadHash, ClassToLookFor, Results, ExclusionFlags);
	}
	else
	{
		TSet<UClass*>* DerivedClasses = ThreadHash.ClassToChildListMap.Find(ClassToLookFor);
		if (DerivedClasses)
		{
			Results.Reserve(Results.Num() + DerivedClasses->Num());
			for (UClass* DerivedClass : *DerivedClasses)
			{
				if (!DerivedClass->HasAnyInternalFlags(ExclusionFlags))
				{
					Results.Add(DerivedClass);
				}
			}
		}
	}
}

TMap<UClass*, TSet<UClass*>> GetAllDerivedClasses()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GetAllDerivedClasses);

	TMap<UClass*, TSet<UClass*>> Classes;
	bool bIsAsyncLoading = false;
	{
		FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
		FHashTableLock HashLock(ThreadHash);
		Classes = ThreadHash.ClassToChildListMap;
		bIsAsyncLoading = IsAsyncLoading();
	}

	// We don't need any filtering if the loader is currently idle
	if (bIsAsyncLoading)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(GetAllDerivedClasses_AsyncLoadingFilter);

		const EInternalObjectFlags ExclusionFlags = GetMandatoryInternalExclusionFlags();
		for (auto It = Classes.CreateIterator(); It; ++It)
		{
			if (It->Key->HasAnyInternalFlags(ExclusionFlags))
			{
				It.RemoveCurrent();
			}
			else
			{
				for (auto InnerIt = It->Value.CreateIterator(); InnerIt; ++InnerIt)
				{
					UClass* Class = *InnerIt;
					if (Class->HasAnyInternalFlags(ExclusionFlags))
					{
						InnerIt.RemoveCurrent();
					}
				}
			}
		}
	}

	return Classes;
}

bool ClassHasInstancesAsyncLoading(const UClass* ClassToLookFor)
{
	TArray<const UClass*> ClassesToSearch;
	ClassesToSearch.Add(ClassToLookFor);

	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock HashLock(ThreadHash);

	// No exclusion flags since the goal of this function is to test those flags presence
	RecursivelyPopulateDerivedClasses(ThreadHash, ClassToLookFor, ClassesToSearch, EInternalObjectFlags::None);

	for (const UClass* SearchClass : ClassesToSearch)
	{
		auto List = ThreadHash.ClassToObjectListMap.Find(SearchClass);
		if (List)
		{
			for (auto ObjectIt = List->CreateIterator(); ObjectIt; ++ObjectIt)
			{
				UObject *Object = static_cast<UObject*>(*ObjectIt);
				// If the object is async loading we'll want to indicate as such to the caller,
				// excepting two cases - garbage objects and the CDO:
				if (Object->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading) 
					// garbage objects won't require that the class be kept alive:
					&& !Object->HasAnyInternalFlags(EInternalObjectFlags::Garbage) 
					// CDO is required and owned by the class - also doesn't need to keep the class alive:
					&& !Object->HasAnyFlags(RF_ClassDefaultObject)) 
				{
					return true;
				}
			}
		}
	}

	return false;
}

UE_AUTORTFM_NOAUTORTFM  // This is not safe to be called from a closed transaction.
void HashObject(UObjectBase* Object)
{
	FName Name = Object->GetFName();
	if (!Name.IsNone())
	{
		LLM_SCOPE_BYTAG(UObjectHash);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		SCOPE_CYCLE_COUNTER(STAT_Hash_HashObject);
#endif
		int32 Hash = 0;

		FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
		FHashTableLock HashLock(ThreadHash);

		Hash = GetObjectHash(Name);
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		if (ThreadHash.PairExistsInHash(Hash, Object))
		{
			OnHashFailure((UObjectBaseUtility*)Object, TEXT("Hash"), TEXT("double add"));
		}
#endif
		ThreadHash.AddToHash(Hash, Object);

		if (PTRINT Outer = (PTRINT)Object->GetOuter())
		{
			Hash = GetObjectOuterHash(Name, Outer);
			checkSlow(!ThreadHash.HashOuter.FindPair(Hash, Object->GetUniqueID()));
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
			if (ThreadHash.HashOuter.FindPair(Hash, Object->GetUniqueID()))
			{
				OnHashFailure((UObjectBaseUtility*)Object, TEXT("HashOuter"), TEXT("double add"));
			}
#endif
			ThreadHash.HashOuter.Add(Hash, Object->GetUniqueID());

			AddToOuterMap(ThreadHash, Object);
		}

		AddToClassMap( ThreadHash, Object );

#if UE_WITH_REMOTE_OBJECT_HANDLE
		AddToRemoteIdMap(ThreadHash, Object);
#endif
	}
}

/**
 * Remove an object to the name hash tables
 *
 * @param	Object		Object to remove from the hash tables
 */
UE_AUTORTFM_NOAUTORTFM  // This is not safe to be called from a closed transaction.
void UnhashObject(UObjectBase* Object)
{
	FName Name = Object->GetFName();
	if (!Name.IsNone())
	{
		LLM_SCOPE_BYTAG(UObjectHash);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
		SCOPE_CYCLE_COUNTER(STAT_Hash_UnhashObject);
#endif
		int32 Hash = 0;
		int32 NumRemoved = 0;

		FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
		FHashTableLock LockHash(ThreadHash);

		Hash = GetObjectHash(Name);
		NumRemoved = ThreadHash.RemoveFromHash(Hash, Object);

		if (!LIKELY(NumRemoved == 1))
		{
			OnHashFailure((UObjectBaseUtility*)Object, TEXT("Hash"), TEXT("remove miscount"));
		}

		if (PTRINT Outer = (PTRINT)UE::CoreUObject::Private::FObjectHandleUtils::GetNonAccessTrackedOuterNoResolve(Object))
		{
			Hash = GetObjectOuterHash(Name, Outer);
			NumRemoved = ThreadHash.HashOuter.RemoveSingle(Hash, Object->GetUniqueID());

			if (!LIKELY(NumRemoved == 1))
			{
				OnHashFailure((UObjectBaseUtility*)Object, TEXT("HashOuter"), TEXT("remove miscount"));
			}

			RemoveFromOuterMap(ThreadHash, Object);
		}

		RemoveFromClassMap( ThreadHash, Object );

#if UE_WITH_REMOTE_OBJECT_HANDLE
		RemoveFromRemoteIdMap(ThreadHash, Object);
#endif
	}
}

void HashObjectExternalPackage(UObjectBase* Object, UPackage* Package)
{
	LLM_SCOPE_BYTAG(UObjectHash);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();
	if (Package)
	{
		FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
		FHashTableLock LockHash(ThreadHash);
		UPackage* OldPackage = AssignExternalPackageToObject(ThreadHash, Object, Package);
		if (OldPackage != Package)
		{
			if (OldPackage)
			{
				RemoveFromPackageMap(ThreadHash, Object, OldPackage);
			}
			AddToPackageMap(ThreadHash, Object, Package);
		}
		Object->AtomicallySetFlags(RF_HasExternalPackage);
	}
	else
	{
		UnhashObjectExternalPackage(Object);
	}
}

void UnhashObjectExternalPackage(class UObjectBase* Object)
{
	LLM_SCOPE_BYTAG(UObjectHash);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_CLEAR_SCOPE();
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock LockHash(ThreadHash);
	Object->AtomicallyClearFlags(RF_HasExternalPackage);
	if (UPackage* Package = UnassignExternalPackageFromObject(ThreadHash, Object))
	{
		RemoveFromPackageMap(ThreadHash, Object, Package);
	}
}

UPackage* GetObjectExternalPackageThreadSafe(const UObjectBase* Object)
{
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	FHashTableLock LockHash(ThreadHash);
	UPackage* ExternalPackage = ThreadHash.ObjectToPackageMap.FindRef(Object);
	UE_CLOGF(!ExternalPackage && ((Object->GetFlags() & RF_HasExternalPackage) != 0), LogUObjectHash, Warning, "Object %ls ExternalPackage is invalid: RF_HasExternalPackage is set, but ExternalPackage is nullptr.", *static_cast<const UObjectBaseUtility*>(Object)->GetPathName());
	return ExternalPackage;
}

UPackage* GetObjectExternalPackageInternal(const UObjectBase* Object)
{
	FUObjectHashTables& ThreadHash = FUObjectHashTables::Get();
	return ThreadHash.ObjectToPackageMap.FindRef(Object);
}

/**
 * Prevents any other threads from finding/adding UObjects (e.g. while GC is running)
*/
void LockUObjectHashTables()
{
#if THREADSAFE_UOBJECTS
	FUObjectHashTables::Get().Lock();
#else
	check(IsInGameThread());
#endif
}

/**
 * Releases UObject hash tables lock (e.g. after GC has finished running)
 */
void UnlockUObjectHashTables()
{
#if THREADSAFE_UOBJECTS
	FUObjectHashTables::Get().Unlock();
#else
	check(IsInGameThread());
#endif
}

void LogHashStatisticsInternal(TMultiMap<int32, uint32>& Hash, FOutputDevice& Ar, const bool bShowHashBucketCollisionInfo)
{
	TArray<int32> HashBuckets;
	// Get the set of keys in use, which is the number of hash buckets
	int32 SlotsInUse = Hash.GetKeys(HashBuckets);

	int32 TotalCollisions = 0;
	int32 MinCollisions = MAX_int32;
	int32 MaxCollisions = 0;
	int32 MaxBin = 0;

	// Dump how many slots are in use
	Ar.Logf(TEXT("Slots in use %d"), SlotsInUse);

	// Work through each slot and figure out how many collisions
	for (auto HashBucket : HashBuckets)
	{
		int32 Collisions = 0;

		for (TMultiMap<int32, uint32>::TConstKeyIterator HashIt(Hash, HashBucket); HashIt; ++HashIt)
		{
			// There's one collision per object in a given bucket
			Collisions++;
		}

		// Keep the global stats
		TotalCollisions += Collisions;
		if (Collisions > MaxCollisions)
		{
			MaxBin = HashBucket;
		}
		MaxCollisions = FMath::Max<int32>(Collisions, MaxCollisions);
		MinCollisions = FMath::Min<int32>(Collisions, MinCollisions);

		if (bShowHashBucketCollisionInfo)
		{
			// Now log the output
			Ar.Logf(TEXT("\tSlot %d has %d collisions"), HashBucket, Collisions);
		}
	}
	Ar.Logf(TEXT(""));

	// Dump the first 30 objects in the worst bin for inspection
	Ar.Logf(TEXT("Worst hash bucket contains:"));
	int32 Count = 0;
	for (TMultiMap<int32, uint32>::TConstKeyIterator HashIt(Hash, MaxBin); HashIt && Count < 30; ++HashIt)
	{
		UObject* Object = static_cast<UObject*>(GUObjectArray.IndexToObject(HashIt.Value())->GetObject());
		Ar.Logf(TEXT("\tObject is %s (%s)"), *Object->GetName(), *Object->GetFullName());
		Count++;
	}
	Ar.Logf(TEXT(""));

	// Now dump how efficient the hash is
	Ar.Logf(TEXT("Collision Stats: Best Case (%d), Average Case (%d), Worst Case (%d)"),
		MinCollisions,
		FMath::FloorToInt(((float)TotalCollisions / (float)SlotsInUse)),
		MaxCollisions);

	// Calculate Hashtable size
	const SIZE_T HashtableAllocatedSize = Hash.GetAllocatedSize();
	Ar.Logf(TEXT("Total memory allocated for Object Outer Hash: %" SIZE_T_FMT " bytes."), HashtableAllocatedSize);
}

void LogHashStatisticsInternal(TBucketMap<int32>& Hash, FOutputDevice& Ar, const bool bShowHashBucketCollisionInfo)
{
	TArray<int32> HashBuckets;
	// Get the set of keys in use, which is the number of hash buckets
	int32 SlotsInUse = Hash.Num();

	int32 TotalCollisions = 0;
	int32 MinCollisions = MAX_int32;
	int32 MaxCollisions = 0;
	int32 MaxBin = 0;
	int32 NumBucketsWithMoreThanOneItem = 0;

	// Dump how many slots are in use
	Ar.Logf(TEXT("Slots in use %d"), SlotsInUse);

	// Work through each slot and figure out how many collisions
	for (auto& HashPair : Hash)
	{
		int32 Collisions = HashPair.Value.Num();
		check(Collisions >= 0);
		if (Collisions > 1)
		{
			NumBucketsWithMoreThanOneItem++;
		}

		// Keep the global stats
		TotalCollisions += Collisions;
		if (Collisions > MaxCollisions)
		{
			MaxBin = HashPair.Key;
		}
		MaxCollisions = FMath::Max<int32>(Collisions, MaxCollisions);
		MinCollisions = FMath::Min<int32>(Collisions, MinCollisions);

		if (bShowHashBucketCollisionInfo)
		{
			// Now log the output
			Ar.Logf(TEXT("\tSlot %d has %d collisions"), HashPair.Key, Collisions);
		}
	}
	Ar.Logf(TEXT(""));

	// Dump the first 30 objects in the worst bin for inspection
	Ar.Logf(TEXT("Worst hash bucket contains:"));
	int32 Count = 0;
	FHashBucket& WorstBucket = Hash.FindChecked(MaxBin);
	for (FHashBucketIterator It(WorstBucket); It; ++It)
	{
		UObject* Object = (UObject*)*It;
		Ar.Logf(TEXT("\tObject is %s (%s)"), *Object->GetName(), *Object->GetFullName());
		Count++;
	}
	Ar.Logf(TEXT(""));

	// Now dump how efficient the hash is
	Ar.Logf(TEXT("Collision Stats: Best Case (%d), Average Case (%d), Worst Case (%d), Number of buckets with more than one item (%d/%d)"),
		MinCollisions,
		FMath::FloorToInt(((float)TotalCollisions / (float)SlotsInUse)),
		MaxCollisions,
		NumBucketsWithMoreThanOneItem,
		SlotsInUse);

	// Calculate Hashtable size
	SIZE_T HashtableAllocatedSize = Hash.GetAllocatedSize();
	// Calculate the size of a all Allocations inside of the buckets (TSet Items)
	for (auto& Pair : Hash)
	{
		HashtableAllocatedSize += Pair.Value.GetAllocatedSize();
	}
	Ar.Logf(TEXT("Total memory allocated for and by Object Hash: %" SIZE_T_FMT " bytes."), HashtableAllocatedSize);
}

void LogHashStatistics(FOutputDevice& Ar, const bool bShowHashBucketCollisionInfo)
{
	Ar.Logf(TEXT("Hash efficiency statistics for the Object Hash"));
	Ar.Logf(TEXT("-------------------------------------------------"));
	Ar.Logf(TEXT(""));
	FHashTableLock HashLock(FUObjectHashTables::Get());
	LogHashStatisticsInternal(FUObjectHashTables::Get().Hash, Ar, bShowHashBucketCollisionInfo);
	Ar.Logf(TEXT(""));
}

void LogHashOuterStatistics(FOutputDevice& Ar, const bool bShowHashBucketCollisionInfo)
{
	Ar.Logf(TEXT("Hash efficiency statistics for the Outer Object Hash"));
	Ar.Logf(TEXT("-------------------------------------------------"));
	Ar.Logf(TEXT(""));
	FHashTableLock HashLock(FUObjectHashTables::Get());
	LogHashStatisticsInternal(FUObjectHashTables::Get().HashOuter, Ar, bShowHashBucketCollisionInfo);
	Ar.Logf(TEXT(""));

	SIZE_T HashOuterMapSize = 0;
	for (TPair<UObjectBase*, FHashBucket>& OuterMapEntry : FUObjectHashTables::Get().ObjectOuterMap)
	{
		HashOuterMapSize += OuterMapEntry.Value.GetAllocatedSize();
	}
	Ar.Logf(TEXT("Total memory allocated for Object Outer Map: %" SIZE_T_FMT " bytes."), HashOuterMapSize);
	Ar.Logf(TEXT(""));
}

void LogHashMemoryOverheadStatistics(FOutputDevice& Ar, const EObjectMemoryOverheadOptions InOptions)
{
	Ar.Logf(TEXT("UObject Hash Tables and Maps memory overhead"));
	Ar.Logf(TEXT("-------------------------------------------------"));
	
	FUObjectHashTables& HashTables = FUObjectHashTables::Get();
	FHashTableLock HashLock(HashTables);

	const bool bShowIndividualStats = !!(InOptions & EObjectMemoryOverheadOptions::ShowIndividualStats);
	SIZE_T TotalSize = 0;
	
	{
		SIZE_T Size = HashTables.Hash.GetAllocatedSize();
		for (const TPair<int32, FHashBucket>& Pair : HashTables.Hash)
		{
			Size += Pair.Value.GetAllocatedSize();
		}
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UObject Hash: %" SIZE_T_FMT " bytes."), Size);
		}
		TotalSize += Size;
	}

	{
		const SIZE_T Size = HashTables.HashOuter.GetAllocatedSize();
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UObject Outer Hash: %" SIZE_T_FMT " bytes."), Size);
		}
		TotalSize += Size;
	}

	{
		int64 Size = HashTables.ObjectOuterMap.GetAllocatedSize();
		for (const TPair<UObjectBase*, FHashBucket>& Pair : HashTables.ObjectOuterMap)
		{
			Size += Pair.Value.GetAllocatedSize();
		}
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UObject Outer Map: %lld bytes."), Size);
		}
		TotalSize += Size;
	}

	{
		SIZE_T Size = HashTables.ClassToObjectListMap.GetAllocatedSize();
		for (const auto& Pair : HashTables.ClassToObjectListMap)
		{
			Size += Pair.Value.GetAllocatedSize();
		}
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UClass To UObject List Map: %" SIZE_T_FMT " bytes."), Size);
		}
		TotalSize += Size;
	}

	{
		SIZE_T Size = HashTables.ClassToChildListMap.GetAllocatedSize();
		for (const TPair<UClass*, TSet<UClass*> >& Pair : HashTables.ClassToChildListMap)
		{
			Size += Pair.Value.GetAllocatedSize();
		}
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UClass To Child UClass List Map: %" SIZE_T_FMT " bytes."), Size);
		}
		TotalSize += Size;
	}

	{
		SIZE_T Size = HashTables.PackageToObjectListMap.GetAllocatedSize();
		for (const TPair<UPackage*, FHashBucket>& Pair : HashTables.PackageToObjectListMap)
		{
			Size += Pair.Value.GetAllocatedSize();
		}
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UPackage To UObject List Map: %" SIZE_T_FMT " bytes."), Size);
		}
		TotalSize += Size;
	}

	{
		const SIZE_T Size = HashTables.ObjectToPackageMap.GetAllocatedSize();
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UObject To External Package Map: %" SIZE_T_FMT " bytes."), Size);
		}
		TotalSize += Size;
	}

	{
		int32 NumListeners = 0;
		const SIZE_T Size = GUObjectArray.GetDeleteListenersAllocatedSize(&NumListeners);
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UObject Delete Listeners (including annotations): %" SIZE_T_FMT " bytes. (%d listeners) "), Size, NumListeners);
		}
		TotalSize += Size;
	}

	{
		const SIZE_T Size = GUObjectArray.GetAllocatedSize();
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by UObjectArray: %" SIZE_T_FMT " bytes. (%d UObjects, %d slots) "), Size, GUObjectArray.GetObjectArrayNumMinusAvailable(), GUObjectArray.GetObjectArrayCapacity());
		}
		TotalSize += Size;
	}

	Ar.Logf(TEXT("Total memory allocated by Object hash tables and maps: %" SIZE_T_FMT " bytes(% .2f MB)."), TotalSize, (double)TotalSize / 1024.0 / 1024.0);
	
	if (!!(InOptions & EObjectMemoryOverheadOptions::IncludeReflectionData))
	{
		SIZE_T PropertiesSize = 0;
		int32 NumProperties = 0;
		SIZE_T UFieldsSize = 0;
		int32 NumUFields = 0;
		TArray<FField*> InnerFields;
		for (TObjectIterator<UField> It; It; ++It)
		{
			if (UStruct* Struct = Cast<UStruct>(*It))
			{
				for (FField* Property = Struct->ChildProperties; Property; Property = Property->Next)
				{
					NumProperties++;
					PropertiesSize += Property->GetFieldSize();
					InnerFields.Reset();
					Property->GetInnerFields(InnerFields);
					for (FField* InnerProperty : InnerFields)
					{
						NumProperties++;
						PropertiesSize += InnerProperty->GetFieldSize();
					}
				}
				UFieldsSize += Struct->Script.GetAllocatedSize();
				UFieldsSize += Struct->ScriptAndPropertyObjectReferences.GetAllocatedSize();
			}
			NumUFields++;
			UFieldsSize += It->GetClass()->GetPropertiesSize();
		}
		if (bShowIndividualStats)
		{
			Ar.Logf(TEXT("Memory used by FProperties: %" SIZE_T_FMT " bytes. (%d FProperties) "), PropertiesSize, NumProperties);
			Ar.Logf(TEXT("Memory used by UFields: %" SIZE_T_FMT " bytes. (%d UFields) "), UFieldsSize, NumUFields);
		}
		SIZE_T ReflectionDataSize = PropertiesSize + UFieldsSize;
		TotalSize += ReflectionDataSize;

		Ar.Logf(TEXT("Total memory allocated by Object reflection data: %" SIZE_T_FMT " bytes(% .2f MB)."), ReflectionDataSize, (double)ReflectionDataSize / 1024.0 / 1024.0);
		Ar.Logf(TEXT("Total memory overhead: %" SIZE_T_FMT " bytes(% .2f MB)."), TotalSize, (double)TotalSize / 1024.0 / 1024.0);
	}

	Ar.Logf(TEXT(""));
}
