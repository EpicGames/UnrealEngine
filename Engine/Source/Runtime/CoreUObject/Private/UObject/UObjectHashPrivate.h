// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectHashPrivate.h: UObjectHash functions for use only within CoreUObject
=============================================================================*/

#pragma once

#include "UObject/UObjectHash.h"

/**
 * Approximate version of StaticFindObjectFastInternal that may return a false positive - i.e. the object does not exist but the function returns true.
 * Will not return a false negative - i.e. if the object does exist, this function must return true
 *
 * @param	InOuter			The direct outer to search within. Must be non-null.
 * @param	ObjectName		The object name to search for.
 * @return	Returns a true if the object may possibly exist, false if it definitely does not exist.
 */
bool DoesObjectPossiblyExist(const UObject* InOuter, FName ObjectName);
