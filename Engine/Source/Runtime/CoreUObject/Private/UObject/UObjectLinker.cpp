// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectLinker.cpp: Unreal object linker relationship management
=============================================================================*/

#include "CoreMinimal.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/Object.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/UObjectAnnotation.h"

DEFINE_LOG_CATEGORY_STATIC(LogUObjectLinker, Log, All);

//@todo UE Console - Check that the mapping of UObjects to linkers is sparse and that we aren't spending a ton of time with these lookups.

struct FLinkerIndexPair
{
	/**
	 * default contructor
	 * Default constructor must be the default item
	 */
	FLinkerIndexPair() :
		Linker(NULL),
		LinkerIndex(INDEX_NONE)
	{
		CheckInvariants();
	}
	/**
	 * Determine if this linker pair is the default
	 * @return true is this is a default pair. We only check the linker because CheckInvariants rules out bogus combinations
	 */
	FORCEINLINE bool IsDefault()
	{
		CheckInvariants();
		return Linker == nullptr;
	}

	/**
	 * Constructor 
	 * @param InLinker linker to assign
	 * @param InLinkerIndex linker index to assign
	 */
	FLinkerIndexPair(FLinkerLoad* InLinker, int32 InLinkerIndex) :
		Linker(InLinker),
		LinkerIndex(InLinkerIndex)
	{
		CheckInvariants();
	}

	/**
	 * check() that either the linker and the index is valid or neither of them are
	 */
	FORCEINLINE void CheckInvariants()
	{
		check(!((Linker == nullptr) ^ (LinkerIndex == INDEX_NONE))); // you need either a valid linker and index or neither valid
	}

	/**
	 * Linker that contains the FObjectExport resource corresponding to
	 * this object.  NULL if this object is native only (i.e. never stored
	 * in an Unreal package), or if this object has been detached from its
	 * linker, for e.g. renaming operations, saving the package, etc.
	 */
	FLinkerLoad*			Linker; 

	/**
	 * Index into the linker's ExportMap array for the FObjectExport resource
	 * corresponding to this object.
	 */
	int32							LinkerIndex; 
};

template <> struct TIsPODType<FLinkerIndexPair> { enum { Value = false }; };


/**
 * Annotation to relate linkers, indices and uobjects
 *
 * Q: Why is this data structure not "garbage collection aware"
 * A: It does not need to be. This is GC-Safe.
 * Objects are detached from their linkers prior to destruction of either the linker or the object
 *
 * NOTE: We're currently using chunked annotations for linkers to emphasize memory
 * usage, but might want to revisit this decision on platforms that are memory limited.
 *
 * LINUX SERVER NOTE: For servers we are using Sparse to emphasize speed over memory usage
 */
#if PLATFORM_LINUX && UE_SERVER
static FUObjectAnnotationSparse<FLinkerIndexPair, false> LinkerAnnotation;
#else
static FUObjectAnnotationChunked<FLinkerIndexPair, false> LinkerAnnotation;
#endif

/** Remove all annotations on exit. This is to prevent issues with the order of static destruction of singletons. */
void CleanupLinkerAnnotations()
{
	// Remove all annotations and remove LinkerAnnotation from UObject delete listeners.
	LinkerAnnotation.RemoveAllAnnotations();
}


void UObject::SetLinker( FLinkerLoad* LinkerLoad, int32 LinkerIndex, bool bShouldDetachExisting, bool bSilenceLogs)
{
	FLinkerIndexPair Existing = LinkerAnnotation.GetAnnotation(this);
	Existing.CheckInvariants();

	if (Existing.Linker == LinkerLoad && Existing.LinkerIndex == LinkerIndex)
	{
		// Nothing to do
		return;
	}

	// Detach from existing linker.
	if( Existing.Linker && bShouldDetachExisting )
	{
		if (HasAnyFlags(RF_NeedLoad | RF_NeedPostLoad))
		{
			UE_CLOGF(!bSilenceLogs, LogUObjectLinker, Error,
				"Detaching from existing linker %ls while object %ls needs loading (%ls). Setting linker to %ls. See log for more information.",
				*Existing.Linker->GetArchiveName(),
				*GetFullName(),
				*LexToString(GetFlags()),
				LinkerLoad ? *LinkerLoad->GetDebugName() : TEXT("nullptr"));
			FDebug::DumpStackTraceToLog(ELogVerbosity::Display);
		}

		FObjectExport& ExportObject = Existing.Linker->ExportMap[Existing.LinkerIndex];
		checkf(ExportObject.Object != nullptr, TEXT("Expected ExportMap[%d].Object to not be null for this ('%s')"), Existing.LinkerIndex, *GetFName().ToString());
		checkf(ExportObject.Object == this, TEXT("Expected ExportMap[%d].Object ('%s') to equal this ('%s')"), Existing.LinkerIndex, *ExportObject.Object->GetFName().ToString(), *GetFName().ToString());

		// Detach the object but keep the object marked as invalid if it was previously. Now that it's detached
		// if we reload the package we don't want to reload invalid objects.
		bool bIsInvalid = ExportObject.bExportLoadFailed;
		ExportObject.ResetObject();
		ExportObject.bExportLoadFailed = bIsInvalid;
	}
	else
	{
		bShouldDetachExisting = false; // no previous linker, we don't need to detach anything
	}

	if (Existing.Linker == LinkerLoad)
	{
		bShouldDetachExisting = false; // no change so don't call notify
	}
	// if we have a valid annotation and are setting a new valid linker, remove the annotation first
	else if (!Existing.IsDefault() && LinkerLoad != nullptr)
	{
		LinkerAnnotation.RemoveAnnotation(this);
	}
	if (Existing.Linker != LinkerLoad || Existing.LinkerIndex != LinkerIndex)
	{
		LLM_SCOPE_BYTAG(UObject_Linker);
		LinkerAnnotation.AddAnnotation(this, FLinkerIndexPair(LinkerLoad, LinkerIndex));
	}
	if (bShouldDetachExisting)
	{
#if WITH_EDITOR
		PostLinkerChange();
#else
		UE_CLOGF(Existing.Linker && LinkerLoad, LogUObjectLinker, Fatal,
			"It is only legal to change linkers in the editor. Trying to change linker on %ls from %ls (Existing->LinkerRoot=%ls) to %ls (LinkerLoad->LinkerRoot=%ls)",
			*GetFullName(),
			*Existing.Linker->GetDebugName(),
			*GetNameSafe(Existing.Linker->LinkerRoot),
			*LinkerLoad->GetDebugName(),
			*GetNameSafe(LinkerLoad->LinkerRoot));
#endif
	}
}

/**
 * Returns the linker for this object.
 *
 * @return	a pointer to the linker for this object, or NULL if this object has no linker
 */
FLinkerLoad* UObjectBaseUtility::GetLinker() const
{
	FLinkerIndexPair Existing = LinkerAnnotation.GetAnnotation(this);
	Existing.CheckInvariants();
	return Existing.Linker;
}
/**
 * Returns this object's LinkerIndex.
 *
 * @return	the index into my linker's ExportMap for the FObjectExport
 *			corresponding to this object.
 */
int32 UObjectBaseUtility::GetLinkerIndex() const
{
	FLinkerIndexPair Existing = LinkerAnnotation.GetAnnotation(this);
	Existing.CheckInvariants();
	return Existing.LinkerIndex;
}



