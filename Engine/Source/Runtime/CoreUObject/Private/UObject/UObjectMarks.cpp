// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectMarks.cpp: Unreal save marks annotation
=============================================================================*/

#include "UObject/UObjectMarks.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectIterator.h"
#include "UObject/GarbageCollectionGlobals.h"
#include "UObject/ObjectVisibility.h"

struct FObjectMark
{
	FObjectMark() = default;

	FObjectMark(EObjectMark InMarks) :
		Marks(InMarks)
	{
	}

	bool IsDefault()
	{
		return Marks == OBJECTMARK_NOMARKS;
	}

	bool ClearMarks(FObjectMark InMarksToClear)
	{
		Marks = static_cast<EObjectMark>(Marks & ~InMarksToClear.Marks);
		return IsDefault();
	}

	EObjectMark				Marks = OBJECTMARK_NOMARKS;
};

template <> struct TIsPODType<FObjectMark> { enum { Value = true }; };


template<typename TAnnotation>
class FUObjectAnnotationSparseNoSync : public FUObjectArray::FUObjectDeleteListener
{
public:
	virtual void NotifyUObjectDeleted(const UObjectBase* Object, int32 Index) override
	{
		RemoveAnnotation(Object);
	}

	virtual void OnUObjectArrayShutdown() override
	{
		RemoveAllAnnotations();
		GUObjectArray.RemoveUObjectDeleteListener(this);
	}

	FUObjectAnnotationSparseNoSync() :
		AnnotationCacheKey(NULL)
	{
	}

	virtual ~FUObjectAnnotationSparseNoSync()
	{
		RemoveAllAnnotations();
	}

private:
	template<typename T>
	void AddAnnotationInternal(const UObjectBase* Object, T&& Annotation)
	{
		check(Object);
		AnnotationCacheKey = Object;
		AnnotationCacheValue = Forward<T>(Annotation);
		if (Annotation.IsDefault())
		{
			RemoveAnnotation(Object); // adding the default annotation is the same as removing an annotation
		}
		else
		{
			if (AnnotationMap.Num() == 0)
			{
				// we are adding the first one, so if we are auto removing or verifying removal, register now
				GUObjectArray.AddUObjectDeleteListener(this);
			}
			AnnotationMap.Add(AnnotationCacheKey, AnnotationCacheValue);
		}
	}

public:
	void AddAnnotation(const UObjectBase* Object, TAnnotation&& Annotation)
	{
		AddAnnotationInternal(Object, MoveTemp(Annotation));
	}

	void AddAnnotation(const UObjectBase* Object, const TAnnotation& Annotation)
	{
		AddAnnotationInternal(Object, Annotation);
	}

	void RemoveAnnotation(const UObjectBase* Object)
	{
		check(Object);
		AnnotationCacheKey = Object;
		AnnotationCacheValue = TAnnotation();
		const bool bHadElements = (AnnotationMap.Num() > 0);
		AnnotationMap.Remove(AnnotationCacheKey);
		if (bHadElements && AnnotationMap.Num() == 0)
		{
			// we are removing the last one, so unregister now
			GUObjectArray.RemoveUObjectDeleteListener(this);
		}
	}

	void RemoveAllAnnotations()
	{
		AnnotationCacheKey = NULL;
		AnnotationCacheValue = TAnnotation();
		const bool bHadElements = (AnnotationMap.Num() > 0);
		AnnotationMap.Empty();
		if (bHadElements)
		{
			GUObjectArray.RemoveUObjectDeleteListener(this);
		}
	}

	void RemoveFromAnnotations(const TAnnotation& Annotation)
	{
		if (!AnnotationMap.IsEmpty())
		{
			AnnotationCacheKey = nullptr;
			AnnotationCacheValue = TAnnotation();
			for (typename TMap<const UObjectBase*, TAnnotation>::TIterator It(AnnotationMap); It; ++It)
			{
				if (It.Value().ClearMarks(Annotation))
				{
					It.RemoveCurrent();
				}
			}
			if (AnnotationMap.IsEmpty())
			{
				GUObjectArray.RemoveUObjectDeleteListener(this);
			}
		}
	}

	TAnnotation GetAnnotation(const UObjectBase* Object)
	{
		check(Object);
		if (Object == AnnotationCacheKey)
			return AnnotationCacheValue;

		AnnotationCacheKey = Object;
		if (TAnnotation* Entry = AnnotationMap.Find(AnnotationCacheKey))
		{
			return AnnotationCacheValue = *Entry;
		}
		else
		{
			return AnnotationCacheValue = TAnnotation();
		}
	}

	const TMap<const UObjectBase*, TAnnotation>& GetAnnotationMap() const
	{
		return AnnotationMap;
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return AnnotationMap.GetAllocatedSize();
	}

private:
	TMap<const UObjectBase*, TAnnotation>	AnnotationMap;
	const UObjectBase*						AnnotationCacheKey;
	TAnnotation								AnnotationCacheValue;
};

class FThreadMarkAnnotation : public TThreadSingleton<FThreadMarkAnnotation>
{
	friend class TThreadSingleton<FThreadMarkAnnotation>;

	FThreadMarkAnnotation()	{}

public:
	FUObjectAnnotationSparseNoSync<FObjectMark> MarkAnnotation;
};

void MarkObject(const class UObjectBase* Object, EObjectMark Marks)
{
	FUObjectAnnotationSparseNoSync<FObjectMark>& ThreadMarkAnnotation = FThreadMarkAnnotation::Get().MarkAnnotation;
	ThreadMarkAnnotation.AddAnnotation(Object,FObjectMark(EObjectMark(ThreadMarkAnnotation.GetAnnotation(Object).Marks | Marks)));
}

void UnMarkObject(const class UObjectBase* Object, EObjectMark Marks)
{
	FUObjectAnnotationSparseNoSync<FObjectMark>& ThreadMarkAnnotation = FThreadMarkAnnotation::Get().MarkAnnotation;
	FObjectMark Annotation = ThreadMarkAnnotation.GetAnnotation(Object);
	if(Annotation.Marks & Marks)
	{
		ThreadMarkAnnotation.AddAnnotation(Object,FObjectMark(EObjectMark(Annotation.Marks & ~Marks)));
	}
}

void MarkAllObjects(EObjectMark Marks)
{
	for (FThreadSafeObjectIterator It; It; ++It)
	{
		MarkObject(*It, Marks);
	}
}

void UnMarkAllObjects(EObjectMark Marks)
{
	FUObjectAnnotationSparseNoSync<FObjectMark>& ThreadMarkAnnotation = FThreadMarkAnnotation::Get().MarkAnnotation;
	if (Marks == OBJECTMARK_ALLMARKS)
	{
		ThreadMarkAnnotation.RemoveAllAnnotations();
	}
	else
	{
		ThreadMarkAnnotation.RemoveFromAnnotations(Marks);
	}
}

bool ObjectHasAnyMarks(const class UObjectBase* Object, EObjectMark Marks)
{
	return !!(FThreadMarkAnnotation::Get().MarkAnnotation.GetAnnotation(Object).Marks & Marks);
}

bool ObjectHasAllMarks(const class UObjectBase* Object, EObjectMark Marks)
{
	return (FThreadMarkAnnotation::Get().MarkAnnotation.GetAnnotation(Object).Marks & Marks) == Marks;
}

EObjectMark ObjectGetAllMarks(const class UObjectBase* Object)
{
	return FThreadMarkAnnotation::Get().MarkAnnotation.GetAnnotation(Object).Marks;
}

void GetObjectsWithAllMarks(TArray<UObject *>& Results, EObjectMark Marks)
{
	// We don't want to return any objects that are currently being background loaded unless we're using the object iterator during async loading.
	EInternalObjectFlags ExclusionFlags = EInternalObjectFlags::Unreachable | UE::GetAsyncLoadingInternalFlagsExclusion();

	const TMap<const UObjectBase *, FObjectMark>& Map = FThreadMarkAnnotation::Get().MarkAnnotation.GetAnnotationMap();
	Results.Empty(Map.Num());
	for (TMap<const UObjectBase *, FObjectMark>::TConstIterator It(Map); It; ++It)
	{
		if ((It.Value().Marks & Marks) == Marks)
		{
			UObject* Item = (UObject*)It.Key();
			if (!Item->HasAnyInternalFlags(ExclusionFlags))
			{
				Results.Add(Item);
			}
		}
	}
}

void GetObjectsWithAnyMarks(TArray<UObject *>& Results, EObjectMark Marks)
{
	// We don't want to return any objects that are currently being background loaded unless we're using the object iterator during async loading.
	EInternalObjectFlags ExclusionFlags = EInternalObjectFlags::Unreachable | UE::GetAsyncLoadingInternalFlagsExclusion();

	const TMap<const UObjectBase *, FObjectMark>& Map = FThreadMarkAnnotation::Get().MarkAnnotation.GetAnnotationMap();
	Results.Empty(Map.Num());
	for (TMap<const UObjectBase *, FObjectMark>::TConstIterator It(Map); It; ++It)
	{
		if (It.Value().Marks & Marks)
		{
			UObject* Item = (UObject*)It.Key();
			if (!Item->HasAnyInternalFlags(ExclusionFlags))
			{
				Results.Add(Item);
			}
		}
	}
}
