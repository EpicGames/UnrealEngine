// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UObjectMigrationContext.h"

#include "Serialization/Archive.h"
#include "UObject/RemoteObjectTransfer.h"
#include "UObject/UObjectThreadContext.h"

bool FUObjectMigrationContext::IsValid() const
{
	return ObjectId.IsValid() && RemoteServerId.IsValid() && OwnerServerId.IsValid();
}

EObjectMigrationRecvType FUObjectMigrationContext::GetObjectMigrationRecvType(const UObjectBase* Object) const
{
	return GetObjectMigrationRecvType(FRemoteObjectId { Object });
}

const FUObjectMigrationContext::FMigratedObjectStatus& FUObjectMigrationContext::CacheObjectStatus(FRemoteObjectId InObjectId) const
{
	if (const FMigratedObjectStatus* CachedStatus = CachedObjectStatus.Find(InObjectId))
	{
		checkf(CachedStatus->OwningServerId == UE::RemoteObject::Handle::GetOwnerServerId(InObjectId), TEXT("Ownership changed after it was first cached (cached owner: %s, current owner: %s"),
			*CachedStatus->OwningServerId.ToString(),
			*UE::RemoteObject::Handle::GetOwnerServerId(InObjectId).ToString());
		checkf(CachedStatus->Residence == UE::RemoteObject::Handle::GetResidence(InObjectId), TEXT("Residence changed after it was first cached (was: %s, now: %s)"),
			EnumToString(CachedStatus->Residence),
			EnumToString(UE::RemoteObject::Handle::GetResidence(InObjectId)));
		return *CachedStatus;
	}

	return CachedObjectStatus.Add(InObjectId, { UE::RemoteObject::Handle::GetOwnerServerId(InObjectId), UE::RemoteObject::Handle::GetResidence(InObjectId)});
}

void FUObjectMigrationContext::CacheObjectStatus(const FRemoteObjectData& ObjectData)
{
	for (FRemoteObjectId SerializedId : ObjectData)
	{
		CacheObjectStatus(SerializedId);
	}
}

EObjectMigrationRecvType FUObjectMigrationContext::GetObjectMigrationRecvType(FRemoteObjectId InObjectId) const
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (!ensureMsgf(IsValid(), TEXT("%hs called on an invalid MigrationContext"), __func__))
	{
		return EObjectMigrationRecvType::Invalid;
	}

	const FRemoteServerId LocalServerId = FRemoteServerId::GetLocalServerId();
	const bool bAlreadyOwns = IsOwned(InObjectId);

	const bool bAssignOwnership = !bAlreadyOwns && (OwnerServerId == LocalServerId);
	const bool bReturnLoan = bAlreadyOwns && ensureMsgf(OwnerServerId == LocalServerId, TEXT("We are receiving object %s that we own, so the data we received should affirm we are the owner. (OwnerServerId: %s)"), *InObjectId.ToString(), *OwnerServerId.ToString());

	if (bAssignOwnership)
	{
		// We were pushed an object that we do not already own, so we must assume ownership of it
		return EObjectMigrationRecvType::AssignedOwnership;
	}

	if (bReturnLoan)
	{
		// We were given an object back that we already own
		return EObjectMigrationRecvType::ReturnedLoan;
	}

	// We received an object and ownership is not involved, so we are just borrowing it
	return EObjectMigrationRecvType::Borrowed;
#else
	return EObjectMigrationRecvType::Invalid;
#endif
}

EObjectMigrationSendType FUObjectMigrationContext::GetObjectMigrationSendType(const UObjectBase* Object) const
{
	return GetObjectMigrationSendType(FRemoteObjectId { Object });
}

EObjectMigrationSendType FUObjectMigrationContext::GetObjectMigrationSendType(FRemoteObjectId InObjectId) const
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (!ensureMsgf(IsValid(), TEXT("%hs passed an invalid MigrationContext"), __func__))
	{
		return EObjectMigrationSendType::Invalid;
	}

	const FRemoteServerId LocalServerId = FRemoteServerId::GetLocalServerId();
	const bool bCurrentlyOwns = CacheObjectStatus(InObjectId).OwningServerId.IsLocal();
	const bool bOwnershipTransfer = (OwnerServerId == RemoteServerId) && bCurrentlyOwns;
	const bool bReturnBorrowed = (OwnerServerId == RemoteServerId) && !bCurrentlyOwns;

	if (bOwnershipTransfer)
	{
		// We have ownership of this object and are pushing it to the destination (reassigning the ownership)
		return EObjectMigrationSendType::ReassignOwnership;
	}

	if (bReturnBorrowed)
	{
		// Send an object we don't own back to the owning server (it requested the object back).
		return EObjectMigrationSendType::ReturnBorrowed;
	}

	if (bCurrentlyOwns)
	{
		// We loan the object to the destination server but ownership does not change
		return EObjectMigrationSendType::Loan;
	}

	// We did not own the object and are sending it onward (forwarding it)
	return EObjectMigrationSendType::ReassignBorrowed;
#else
	return EObjectMigrationSendType::Invalid;
#endif
}

const TCHAR* ToString(EObjectMigrationRecvType Value)
{
	switch (Value)
	{
		case EObjectMigrationRecvType::Borrowed:
			return TEXT("Borrowed");

		case EObjectMigrationRecvType::ReturnedLoan:
			return TEXT("ReturnedLoan");

		case EObjectMigrationRecvType::AssignedOwnership:
			return TEXT("AssignedOwnership");

		default:
			return TEXT("Invalid");
	}
}

const TCHAR* ToString(EObjectMigrationSendType Value)
{
	switch (Value)
	{
		case EObjectMigrationSendType::Loan:
			return TEXT("Loan");

		case EObjectMigrationSendType::ReturnBorrowed:
			return TEXT("ReturnBorrowed");

		case EObjectMigrationSendType::ReassignBorrowed:
			return TEXT("ReassignBorrowed");

		case EObjectMigrationSendType::ReassignOwnership:
			return TEXT("ReassignOwnership");

		default:
			return TEXT("Invalid");
	}
}
