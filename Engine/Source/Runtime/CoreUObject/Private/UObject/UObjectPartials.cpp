// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UObjectPartials.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UECodeGen_Private
{
	FProperty* ConstructFProperty(FFieldVariant Outer, const FPropertyParamsBase* const*& PropertyArray, int32& NumProperties);
}

namespace UE::CoreUObject::Private
{
	void ConstructUClassPartials(UClass& Class)
	{
		if (!Class.HasAnyClassFlags(CLASS_Partial) || Class.PropertiesStartOffset < 0)
		{
			return;
		}

		UClass* SuperClass = Class.GetSuperClass();
		check(SuperClass); // No root class will have CLASS_Partial set
		ConstructUClassPartials(*SuperClass);

		FPartialClass* FirstPartial = Class.Partials;
		if (!FirstPartial)
		{
			if (SuperClass->PropertiesStartOffset < 0)
			{
				Class.MinAlignment = FMath::Max(Class.MinAlignment, SuperClass->MinAlignment); // Update alignment in case there are partials that increased alignment in base class
				Class.PropertiesStartOffset = SuperClass->PropertiesStartOffset;
				Class.SetPropertiesSize(Class.GetPropertiesSize() - SuperClass->PropertiesStartOffset);
			}
			return;
		}

		check(Class.HasAnyClassFlags(CLASS_Partial));

		int32 PropertiesStartOffset = SuperClass->PropertiesStartOffset;
		int16 MemoryMinAlignment = FMath::Max(Class.MinAlignment, SuperClass->MinAlignment);

		// Calculated needed size based on partials size and alignments
		int32 NeededSize = 0;
		for (FPartialClass* It = FirstPartial; It ; It = It->NextPartial)
		{
#if UE_WITH_CONSTINIT_UOBJECT
			if (It->FirstProperty)
			{
				NeededSize = Align(NeededSize, It->Alignment);
				NeededSize += It->Size;
			}
#else
			if (It->NumProperties)
			{
				NeededSize = Align(NeededSize, It->Alignment);
				NeededSize += It->Size;
			}
#endif
		}
		// PropertiesStartOffset must take alignment of data stored after current partials in to account
		PropertiesStartOffset = -Align(-PropertiesStartOffset + NeededSize, MemoryMinAlignment);



		// Create the properties/functions of partials

		int32 PartialOffset = PropertiesStartOffset;

		for (FPartialClass* It = FirstPartial; It ; It = It->NextPartial)
		{
			PartialOffset = Align(PartialOffset, It->Alignment);
			It->PartialOffset = PartialOffset;
			*It->PartialOffsetPtr = PartialOffset;

#if UE_WITH_CONSTINIT_UOBJECT
			if (It->FirstProperty)
			{
				FProperty* LastProperty = nullptr;
				for (FProperty* P = It->FirstProperty; P; P = (FProperty*)(P->Next))
				{
					UEProperty_Private::FProperty_DoNotUse::Unsafe_AlterOffset(*P, PartialOffset + P->GetOffset_ForInternal());
					LastProperty = P;
				}
				LastProperty->Next = Class.ChildProperties;
				Class.ChildProperties = It->FirstProperty;
				PartialOffset += It->Size;
			}
#else
			if (int32 NumProperties = It->NumProperties)
			{
				const UECodeGen_Private::FPropertyParamsBase* const* PropertyArray = It->Properties + NumProperties;
				while (PropertyArray > It->Properties)
				{
					FProperty* Property = UECodeGen_Private::ConstructFProperty(&Class, PropertyArray, NumProperties);
					UEProperty_Private::FProperty_DoNotUse::Unsafe_AlterOffset(*Property, PartialOffset + Property->GetOffset_ForInternal());
				}
				PartialOffset += It->Size;
			}
#endif

#if UE_WITH_CONSTINIT_UOBJECT
			if (It->FirstChild)
			{
				UField* LastChild = nullptr;
				for (UField* Child = It->FirstChild; Child; Child = Child->Next)
				{
					LastChild = Child;
				}
				It->EndChild = Class.Children;
				LastChild->Next = Class.Children;
				Class.Children = It->FirstChild;
			}
#else
			if (It->NumFunctions > 0)
			{
				const UE::CodeGen::FClassNativeFunction* NativeFuncs = static_cast<const UE::CodeGen::FClassNativeFunction*>(It->NativeFunctions);
				FNativeFunctionRegistrar::RegisterFunctions(&Class, MakeConstArrayView(NativeFuncs, It->NumFunctions));

				// Hook up functions 
				for (int32 FuncIdx = 0; FuncIdx < It->NumFunctions; ++FuncIdx)
				{
					UFunction* (*FuncConstructor)(ETypeConstructPhase) = It->FunctionConstructors[FuncIdx];
					check(FuncConstructor);
					UFunction* Func = FuncConstructor(ETypeConstructPhase::Outer);
					check(Func);
					Func->Next = Class.Children;
					Class.Children = Func;
					Class.AddFunctionToFunctionMap(Func, Func->GetFName());
				}
			}
#endif
		}

		check(((-PropertiesStartOffset) % Class.MinAlignment) == 0);

		Class.SetPropertiesSize(Class.GetPropertiesSize() - PropertiesStartOffset);
		Class.PropertiesStartOffset = PropertiesStartOffset;
		Class.MinAlignment = MemoryMinAlignment;
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void FPartialClass::LinkToClass(UClass& PartialClass)
	{
		// Sort in partials based on alignment first and then name
		FPartialClass* PrevPartial = nullptr;
		for (FPartialClass* It = PartialClass.Partials; It; It = It->NextPartial)
		{
			if (Alignment < It->Alignment || (Alignment == It->Alignment && FCString::Strcmp(Name, It->Name) < 0))
			{
				break;
			}
			PrevPartial = It;
		}
		if (!PrevPartial)
		{
			NextPartial = PartialClass.Partials;
			PartialClass.Partials = this;
		}
		else
		{
			NextPartial = PrevPartial->NextPartial;
			PrevPartial->NextPartial = this;
		}
	}

	////////////////////////////////////////////////////////////////////////////////////////////////////

	void ConstructPartials(const UClass& Class, UObject* Obj)
	{
		if (Class.HasAnyClassFlags(CLASS_Partial))
		{
			ConstructPartials(*Class.GetSuperClass(), Obj);
			for (FPartialClass* It = Class.Partials; It; It = It->NextPartial)
			{
				check(It->PartialOffset != MAX_int32);
				It->Constructor((uint8*)Obj + It->PartialOffset); // Call partial constructor to initialize C++ objects
			}
		}
	}

	void BeginDestroyPartials(const UClass& Class, UObject* Obj)
	{
		if (Class.HasAnyClassFlags(CLASS_Partial))
		{
			for (FPartialClass* It = Class.Partials; It; It = It->NextPartial)
			{
				if (It->BeginDestroy)
				{
					It->BeginDestroy((uint8*)Obj + It->PartialOffset);
				}
			}
			BeginDestroyPartials(*Class.GetSuperClass(), Obj);
		}
	}

	void DestructPartials(const UClass& Class, UObject* Obj)
	{
		if (Class.HasAnyClassFlags(CLASS_Partial))
		{
			for (FPartialClass* It = Class.Partials; It; It = It->NextPartial)
			{
				It->Destructor((uint8*)Obj + It->PartialOffset);
			}
			DestructPartials(*Class.GetSuperClass(), Obj);
		}
	}

	void CallPartialsGetLifetimeReplicatedProps(const UClass& Class, UObject* Obj, TArray<FLifetimeProperty>& OutLifetimeProps)
	{
		if (Class.HasAnyClassFlags(CLASS_Partial))
		{
			CallPartialsGetLifetimeReplicatedProps(*Class.GetSuperClass(), Obj, OutLifetimeProps);
			for (FPartialClass* It = Class.Partials; It; It = It->NextPartial)
			{
				if (It->BeginDestroy)
				{
					It->GetLifetimeReplicatedProps((uint8*)Obj + It->PartialOffset, OutLifetimeProps);
				}
			}
		}
	}

} // UE::CoreUObject::Private

////////////////////////////////////////////////////////////////////////////////////////////////////
