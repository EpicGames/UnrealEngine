// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogUObjectBootstrap, Display, Display);

// Processes the first stage of compiled in UObjects i.e. creating them and adding them to the global array and hash
// Does not do further initialization i.e. linking classes/structs/properties
void UObjectProcessRegistrants();
