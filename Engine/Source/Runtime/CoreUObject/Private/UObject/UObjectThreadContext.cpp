// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UObjectThreadContext.h"

#include "UObject/LinkerLoad.h"
#include "UObject/Object.h"
#include "UObject/UObjectSerializeContext.h"

#if WITH_EDITORONLY_DATA
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/PropertyStateTracking.h"
#endif

DEFINE_LOG_CATEGORY(LogUObjectThreadContext);

UE_DEFINE_THREAD_SINGLETON_TLS(FUObjectThreadContext, COREUOBJECT_API)

FUObjectThreadContext::FUObjectThreadContext()
: IsRoutingPostLoad(false)
, IsDeletingLinkers(false)
, SyncLoadUsingAsyncLoaderCount(0)
, IsInConstructor(0)
, ConstructedObject(nullptr)
, CurrentlyPostLoadedObjectByALT(nullptr)
, AsyncPackage(nullptr)
, AsyncPackageLoader(nullptr)
, SerializeContext(new FUObjectSerializeContext())
{}

FUObjectThreadContext::~FUObjectThreadContext()
{
}

FObjectInitializer& FUObjectThreadContext::ReportNull()
{
	FObjectInitializer* ObjectInitializerPtr = TopInitializer();
	UE_CLOGF(!ObjectInitializerPtr, LogUObjectThreadContext, Fatal, "Tried to get the current ObjectInitializer, but none is set. Please use NewObject to construct new UObject-derived classes.");
	return *ObjectInitializerPtr;
}

FUObjectSerializeContext::FUObjectSerializeContext()
	: RefCount(0)
	, ImportCount(0)
	, ForcedExportCount(0)
	, ObjBeginLoadCount(0)
	, SerializedObject(nullptr)
	, SerializedPackageLinker(nullptr)
	, SerializedImportIndex(0)
	, SerializedImportLinker(nullptr)
	, SerializedExportIndex(0)
	, SerializedExportLinker(nullptr)
#if WITH_EDITORONLY_DATA
	, bTrackSerializedPropertyPath(false)
	, bTrackInitializedProperties(false)
	, bTrackSerializedProperties(false)
	, bTrackUnknownProperties(false)
	, bTrackUnknownEnumNames(false)
	, bImpersonateProperties(false)
	, bIsLoadingToPropertyBagObject(false)
	, bIsLoadingToUnknownPropertyTree(false)
#endif
{
}

FUObjectSerializeContext::~FUObjectSerializeContext()
{
	checkf(!HasLoadedObjects(), TEXT("FUObjectSerializeContext is being destroyed but it still has pending loaded objects in its ObjectsLoaded list."));
}

int32 FUObjectSerializeContext::IncrementBeginLoadCount()
{
	return ++ObjBeginLoadCount;
}
int32 FUObjectSerializeContext::DecrementBeginLoadCount()
{
	check(HasStartedLoading());
	return --ObjBeginLoadCount;
}

void FUObjectSerializeContext::AddUniqueLoadedObjects(const TArray<UObject*>& InObjects)
{
	for (UObject* NewLoadedObject : InObjects)
	{
		ObjectsLoaded.AddUnique(NewLoadedObject);
	}
	
}

void FUObjectSerializeContext::AddLoadedObject(UObject* InObject)
{
	ObjectsLoaded.Add(InObject);
}

bool FUObjectSerializeContext::PRIVATE_PatchNewObjectIntoExport(UObject* OldObject, UObject* NewObject)
{
	const int32 ObjLoadedIdx = ObjectsLoaded.Find(OldObject);
	if (ObjLoadedIdx != INDEX_NONE)
	{
		ObjectsLoaded[ObjLoadedIdx] = NewObject;
		return true;
	}
	else
	{
		return false;
	}
}

namespace UE
{

FScopedObjectSerializeContext::FScopedObjectSerializeContext(UObject* InObject, FArchive& InArchive)
#if WITH_EDITORONLY_DATA
	: Archive(InArchive)
	, Object(InObject)
#endif
{
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();

	// Save the parts of the serialize context that are modified below.
	SavedSerializedObject = SerializeContext->SerializedObject;
#if WITH_EDITORONLY_DATA
	SavedSerializedObjectScriptStartOffset = SerializeContext->SerializedObjectScriptStartOffset;
	SavedSerializedObjectScriptEndOffset = SerializeContext->SerializedObjectScriptEndOffset;
	bSavedTrackSerializedPropertyPath = SerializeContext->bTrackSerializedPropertyPath;
	bSavedTrackInitializedProperties = SerializeContext->bTrackInitializedProperties;
	bSavedTrackSerializedProperties = SerializeContext->bTrackSerializedProperties;
	bSavedTrackUnknownProperties = SerializeContext->bTrackUnknownProperties;
	bSavedTrackUnknownEnumNames = SerializeContext->bTrackUnknownEnumNames;
	bSavedImpersonateProperties = SerializeContext->bImpersonateProperties;
	SavedSerializedPropertyPath = SerializeContext->SerializedPropertyPath;

	SerializeContext->SerializedObjectScriptStartOffset = -1;
	SerializeContext->SerializedObjectScriptEndOffset = -1;

	SerializeContext->SerializedObject = Object;

	const bool bIsLoading = Archive.IsLoading();
	const bool bIsIDO = IsInstanceDataObject(Object);

	// Disable if cooking because any extra data in the IDO will not be understood outside of the editor.
	const bool bSupportsIDO = !Archive.IsCooking() && IsInstanceDataObjectSupportEnabled(Object);

	// determine whether impersonation should be enabled when saving
	const bool bImpersonateOnSave = bSupportsIDO && UE::IsInstanceDataObjectImpersonationEnabledOnSave(InObject);

	// Disable if a newer version of the class exists because there is no point in creating an IDO of an obsolete type.
	bCreateInstanceDataObject = bIsLoading && !bIsIDO && bSupportsIDO && !Object->GetClass()->HasAnyClassFlags(CLASS_NewerVersionExists);

	// Enable tracking any property or enum that does not match the current schema when creating an IDO.
	SerializeContext->bTrackSerializedPropertyPath = bCreateInstanceDataObject;
	SerializeContext->bTrackUnknownProperties = bCreateInstanceDataObject;
	SerializeContext->bTrackUnknownEnumNames = bCreateInstanceDataObject;

	// Enable tracking of property value state on load if the object supports it.
	SerializeContext->bTrackInitializedProperties = bIsLoading && FInitializedPropertyValueState(Object).IsTracking();
	SerializeContext->bTrackSerializedProperties = bIsLoading && FSerializedPropertyValueState(Object).IsTracking();

	// Enable impersonation when loading to an IDO or saving an object that may have an IDO.
	SerializeContext->bImpersonateProperties = bIsLoading ? bIsIDO : bImpersonateOnSave;
#endif // WITH_EDITORONLY_DATA
}

FScopedObjectSerializeContext::~FScopedObjectSerializeContext()
{
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();

#if WITH_EDITORONLY_DATA
	if (bCreateInstanceDataObject)
	{
		CreateInstanceDataObject(Object, Archive, SerializeContext->SerializedObjectScriptStartOffset, SerializeContext->SerializedObjectScriptEndOffset);
	}
#endif

	// Restore the parts of the serialize context that the constructor modified.
	SerializeContext->SerializedObject = SavedSerializedObject;
#if WITH_EDITORONLY_DATA
	SerializeContext->SerializedObjectScriptStartOffset = SavedSerializedObjectScriptStartOffset;
	SerializeContext->SerializedObjectScriptEndOffset = SavedSerializedObjectScriptEndOffset;
	SerializeContext->bTrackSerializedPropertyPath = bSavedTrackSerializedPropertyPath;
	SerializeContext->bTrackInitializedProperties = bSavedTrackInitializedProperties;
	SerializeContext->bTrackSerializedProperties = bSavedTrackSerializedProperties;
	SerializeContext->bTrackUnknownProperties = bSavedTrackUnknownProperties;
	SerializeContext->bTrackUnknownEnumNames = bSavedTrackUnknownEnumNames;
	SerializeContext->bImpersonateProperties = bSavedImpersonateProperties;
	SerializeContext->SerializedPropertyPath = SavedSerializedPropertyPath;
#endif
}

FScopedObjectSerializeContext::FScopedObjectSerializeContext(UObject*const InObject, FArchive& InArchive, const FPropertyPathName& PathBeingSerialized)
	: FScopedObjectSerializeContext(InObject, InArchive)
{
#if WITH_EDITORONLY_DATA
	const bool bIsLoading = Archive.IsLoading();
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	SerializeContext->bTrackSerializedPropertyPath = bIsLoading;
	SerializeContext->bTrackUnknownProperties = bIsLoading;
	SerializeContext->bTrackUnknownEnumNames = bIsLoading;
	SerializeContext->SerializedPropertyPath = PathBeingSerialized;
#endif // WITH_EDITORONLY_DATA
}

} // UE
