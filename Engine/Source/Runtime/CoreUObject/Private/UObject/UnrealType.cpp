// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/UnrealType.h"
#include "UObject/PropertyOptional.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "StructUtils/InstancedStruct.h"

DEFINE_LOG_CATEGORY(LogType);

ENUM_CLASS_FLAGS(FPropertyValueIterator::EPropertyValueFlags)

#define EPropertyValueFlags_ContainerMask (EPropertyValueFlags::IsOptional | EPropertyValueFlags::IsArray | EPropertyValueFlags::IsMap | EPropertyValueFlags::IsSet | EPropertyValueFlags::IsStruct)

FPropertyValueIterator::FPropertyValueIterator(
	FFieldClass* InPropertyClass,
	const UStruct* InStruct,
	const void* InStructValue,
	EPropertyValueIteratorFlags InRecursionFlags,
	EFieldIteratorFlags::DeprecatedPropertyFlags InDeprecatedPropertyFlags)
	: PropertyClass(InPropertyClass)
	, RecursionFlags(InRecursionFlags)
	, DeprecatedPropertyFlags(InDeprecatedPropertyFlags)
	, bSkipRecursionOnce(false)
	, bMatchAll(InPropertyClass == FProperty::StaticClass())
{
	FPropertyValueStackEntry Entry(InStructValue);
	FillStructProperties(InStruct, Entry);
	if (Entry.ValueArray.Num() > 0)
	{
		PropertyIteratorStack.Emplace(MoveTemp(Entry));

		while (NextValue(InRecursionFlags));
	}
}

FPropertyValueIterator::EPropertyValueFlags FPropertyValueIterator::GetPropertyValueFlags(const FProperty* Property) const
{
	EPropertyValueFlags Flags = EPropertyValueFlags::None;
	if (RecursionFlags == EPropertyValueIteratorFlags::FullRecursion)
	{
		uint64 CastFlags = Property->GetClass()->GetCastFlags();
		Flags = EPropertyValueFlags(  !!(CastFlags & CASTCLASS_FArrayProperty)    * uint32(EPropertyValueFlags::IsArray)
									| !!(CastFlags & CASTCLASS_FMapProperty)      * uint32(EPropertyValueFlags::IsMap)
									| !!(CastFlags & CASTCLASS_FSetProperty)      * uint32(EPropertyValueFlags::IsSet)
									| !!(CastFlags & CASTCLASS_FStructProperty)   * uint32(EPropertyValueFlags::IsStruct)
									| !!(CastFlags & CASTCLASS_FOptionalProperty) * uint32(EPropertyValueFlags::IsOptional));
	}
	if (bMatchAll || Property->IsA(PropertyClass))
	{
		Flags |= EPropertyValueFlags::IsMatch;
	}
	return Flags;
}

void FPropertyValueIterator::FillStructProperties(const UStruct* Struct, FPropertyValueStackEntry& Entry)
{
	if (Struct == TBaseStructure<FInstancedStruct>::Get())
	{
		const FInstancedStruct* InstancedStruct = reinterpret_cast<const FInstancedStruct*>(Entry.Owner);
		if (InstancedStruct->GetScriptStruct() && InstancedStruct->GetMemory())
		{
			Entry.Owner = InstancedStruct->GetMemory();
			Struct = InstancedStruct->GetScriptStruct();
		}
	}

	const void* Owner = Entry.Owner;
	FPropertyValueStackEntry::FValueArrayType& ValueArray = Entry.ValueArray;

	auto AddPropertyToValueArray = [this, Owner, &ValueArray](const FProperty* Property)
	{
		EPropertyValueFlags PropertyValueFlags = GetPropertyValueFlags(Property);
		if (PropertyValueFlags != EPropertyValueFlags::None)
		{
			int32 Num = Property->ArrayDim;
			ValueArray.Reserve(ValueArray.Num() + Num);
			for (int32 StaticIndex = 0; StaticIndex < Num; ++StaticIndex)
			{
				const void* PropertyValue = Property->ContainerPtrToValuePtr<void>(Owner, StaticIndex);
				ValueArray.Emplace(BasePairType(Property, PropertyValue), PropertyValueFlags);
			}
		}
	};

	if (PropertyClass->IsChildOf(FObjectPropertyBase::StaticClass()))
	{
		const bool bIncludeDeprecated = EnumHasAnyFlags((EFieldIterationFlags)DeprecatedPropertyFlags, EFieldIterationFlags::IncludeDeprecated);

		// Note: This includes properties that contain object references (e.g. array), along with any inherited reference properties.
		for (FProperty* RefProp = Struct->RefLink; RefProp; RefProp = RefProp->NextRef)
		{
			if (!bIncludeDeprecated && RefProp->HasAnyPropertyFlags(CPF_Deprecated))
			{
				continue;
			}

			AddPropertyToValueArray(RefProp);
		}
	}
	else
	{
		for (TFieldIterator<FProperty> It(Struct, EFieldIteratorFlags::IncludeSuper, DeprecatedPropertyFlags, EFieldIteratorFlags::ExcludeInterfaces); It; ++It)
		{
			AddPropertyToValueArray(*It);
		}
	}
}

FORCEINLINE_DEBUGGABLE bool FPropertyValueIterator::NextValue(EPropertyValueIteratorFlags InRecursionFlags)
{
	check(PropertyIteratorStack.Num() > 0)
	FPropertyValueStackEntry& Entry = PropertyIteratorStack.Last();

	// If we have pending values, deal with them
	if (Entry.NextValueIndex < Entry.ValueArray.Num())
	{
		const bool bIsPropertyMatchProcessed = Entry.ValueIndex == Entry.NextValueIndex;
		Entry.ValueIndex = Entry.NextValueIndex;
		Entry.NextValueIndex = Entry.ValueIndex + 1;

		const FProperty* Property = Entry.ValueArray[Entry.ValueIndex].Key.Key;
		const void* PropertyValue = Entry.ValueArray[Entry.ValueIndex].Key.Value;
		const EPropertyValueFlags PropertyValueFlags = Entry.ValueArray[Entry.ValueIndex].Value;
		check(PropertyValueFlags != EPropertyValueFlags::None);

		// Handle matching properties
		if (!bIsPropertyMatchProcessed && EnumHasAnyFlags(PropertyValueFlags, EPropertyValueFlags::IsMatch))
		{
			if (EnumHasAnyFlags(PropertyValueFlags, EPropertyValueFlags_ContainerMask))
			{
				// this match is also a container/struct, so recurse into it next time
				Entry.NextValueIndex = Entry.ValueIndex;
			}
			return false; // Break at this matching property
		}

		// Handle container properties
		check(EnumHasAnyFlags(PropertyValueFlags, EPropertyValueFlags_ContainerMask));
		if (InRecursionFlags == EPropertyValueIteratorFlags::FullRecursion)
		{
			FPropertyValueStackEntry NewEntry(PropertyValue);
			
			if (EnumHasAnyFlags(PropertyValueFlags, EPropertyValueFlags::IsOptional))
			{
				const FOptionalProperty* OptionalProperty = CastFieldChecked<FOptionalProperty>(Property);
				const FProperty* InnerProperty = OptionalProperty->GetValueProperty();
				EPropertyValueFlags InnerFlags = GetPropertyValueFlags(InnerProperty);
				if (InnerFlags != EPropertyValueFlags::None)
				{
					if (const void* InnerValue = OptionalProperty->GetValuePointerForReadIfSet(PropertyValue))
					{
						NewEntry.ValueArray.Emplace(BasePairType(InnerProperty, InnerValue), InnerFlags);
					}
				}
			}
			else if (EnumHasAnyFlags(PropertyValueFlags, EPropertyValueFlags::IsArray))
			{
				const FArrayProperty* ArrayProperty = CastFieldChecked<FArrayProperty>(Property);
				const FProperty* InnerProperty = ArrayProperty->Inner;
				EPropertyValueFlags InnerFlags = GetPropertyValueFlags(InnerProperty);
				if (InnerFlags != EPropertyValueFlags::None)
				{
					FScriptArrayHelper Helper(ArrayProperty, PropertyValue);
					const int32 Num = Helper.Num();
					NewEntry.ValueArray.Reserve(Num);
					for (int32 DynamicIndex = 0; DynamicIndex < Num; ++DynamicIndex)
					{
						NewEntry.ValueArray.Emplace(
							BasePairType(InnerProperty, Helper.GetRawPtr(DynamicIndex)), InnerFlags);
					}
				}
			}
			else if (EnumHasAnyFlags(PropertyValueFlags, EPropertyValueFlags::IsMap))
			{
				const FMapProperty* MapProperty = CastFieldChecked<FMapProperty>(Property);
				const FProperty* KeyProperty = MapProperty->KeyProp;
				const FProperty* ValueProperty = MapProperty->ValueProp;
				EPropertyValueFlags KeyFlags = GetPropertyValueFlags(KeyProperty);
				EPropertyValueFlags ValueFlags = GetPropertyValueFlags(ValueProperty);
				if ((KeyFlags | ValueFlags) != EPropertyValueFlags::None)
				{
					FScriptMapHelper Helper(MapProperty, PropertyValue);
					for (FScriptMapHelper::FIterator It(Helper); It; ++It)
					{
						if (KeyFlags != EPropertyValueFlags::None)
						{
							NewEntry.ValueArray.Emplace(BasePairType(KeyProperty, Helper.GetKeyPtr(It)), KeyFlags);
						}
						if (ValueFlags != EPropertyValueFlags::None)
						{
							NewEntry.ValueArray.Emplace(BasePairType(ValueProperty, Helper.GetValuePtr(It)), ValueFlags);
						}
					}
				}
			}
			else if (EnumHasAnyFlags(PropertyValueFlags, EPropertyValueFlags::IsSet))
			{
				const FSetProperty* SetProperty = CastFieldChecked<FSetProperty>(Property);
				const FProperty* InnerProperty = SetProperty->ElementProp;
				EPropertyValueFlags InnerFlags = GetPropertyValueFlags(InnerProperty);
				if (InnerFlags != EPropertyValueFlags::None)
				{
					FScriptSetHelper Helper(SetProperty, PropertyValue);
					for (FScriptSetHelper::FIterator It(Helper); It; ++It)
					{
						NewEntry.ValueArray.Emplace(BasePairType(InnerProperty, Helper.GetElementPtr(It)), InnerFlags);
					}
				}
			}
			else if (EnumHasAnyFlags(PropertyValueFlags, EPropertyValueFlags::IsStruct))
			{
				const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Property);
				FillStructProperties(StructProperty->Struct, NewEntry);
			}
			if (NewEntry.ValueArray.Num() > 0)
			{
				PropertyIteratorStack.Emplace(MoveTemp(NewEntry));
				return true; // NextValue should be called again to move to the top of the stack
			}
		}
	}

	if (Entry.NextValueIndex == Entry.ValueArray.Num())
	{
		PropertyIteratorStack.Pop();
	}

	// NextValue should be called again to continue iteration
	return PropertyIteratorStack.Num() > 0;
}

void FPropertyValueIterator::IterateToNext()
{
	if (bSkipRecursionOnce)
	{
		bSkipRecursionOnce = false;
		
		if (!NextValue(EPropertyValueIteratorFlags::NoRecursion))
		{
			return;
		}
	}

	EPropertyValueIteratorFlags LocalRecursionFlags = RecursionFlags;	
	while (NextValue(LocalRecursionFlags));
}

void FPropertyValueIterator::GetPropertyChain(TArray<const FProperty*>& PropertyChain) const
{
	PropertyChain.Reserve(PropertyIteratorStack.Num());
	// Iterate over struct/container property stack, starting at the inner most property
	for (int32 StackIndex = PropertyIteratorStack.Num() - 1; StackIndex >= 0; StackIndex--)
	{
		const FPropertyValueStackEntry& Entry = PropertyIteratorStack[StackIndex];

		// Index should always be valid
		const FProperty* Property = Entry.ValueArray[Entry.ValueIndex].Key.Key;
		PropertyChain.Add(Property);
	}
}

FString FPropertyValueIterator::GetPropertyPathDebugString() const
{
	TStringBuilder<FName::StringBufferSize> PropertyPath;
	for (int32 StackIndex = 0; StackIndex < PropertyIteratorStack.Num(); StackIndex++)
	{
		const FPropertyValueStackEntry& Entry = PropertyIteratorStack[StackIndex];

		// Index should always be valid
		const BasePairType& PropertyAndValue = Entry.ValueArray[Entry.ValueIndex].Key;
		const FProperty* Property = PropertyAndValue.Key;
		const void* ValuePtr = PropertyAndValue.Value;

		PropertyPath.Append(Property->GetAuthoredName());

		int32 NextStackIndex = StackIndex + 1;

		if (NextStackIndex < PropertyIteratorStack.Num())
		{
			if (CastField<FOptionalProperty>(Property))
			{
				PropertyPath.Append(TEXT("?"));
				StackIndex++;
			}
			else if (CastField<FArrayProperty>(Property))
			{
				const FPropertyValueStackEntry& NextEntry = PropertyIteratorStack[StackIndex+1];
			
				PropertyPath.Append(TEXT("["));
				PropertyPath.Appendf(TEXT("%d"), NextEntry.ValueIndex);
				PropertyPath.Append(TEXT("]"));

				StackIndex++;
			}
			else if (CastField<FSetProperty>(Property))
			{
			
			}
			else if (CastField<FMapProperty>(Property))
			{
				const FPropertyValueStackEntry& NextEntry = PropertyIteratorStack[StackIndex+1];

				const FProperty* NextProperty = NextEntry.GetPropertyValue().Key;
				const void* NextValuePtr = NextEntry.GetPropertyValue().Value;

				FString KeyStr;
				NextProperty->ExportText_Direct(KeyStr, NextValuePtr, nullptr, nullptr, PPF_None);
				
				PropertyPath.Append(TEXT("["));
				PropertyPath.Append(KeyStr);
				PropertyPath.Append(TEXT("]"));
			}

			NextStackIndex = StackIndex + 1;

			if (NextStackIndex < PropertyIteratorStack.Num())
			{
				PropertyPath.Append(TEXT("."));
			}
		}
	}

	return PropertyPath.ToString();
}
