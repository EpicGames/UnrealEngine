// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WeakObjectPtr.cpp: Weak pointer to UObject
=============================================================================*/

#include "UObject/WeakObjectPtr.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/RemoteObject.h"
#include "UObject/RemoteObjectPrivate.h"
#include "UObject/UObjectHash.h"

DEFINE_LOG_CATEGORY_STATIC(LogWeakObjectPtr, Log, All);

/*-----------------------------------------------------------------------------------------------------------
	Base serial number management.
-------------------------------------------------------------------------------------------------------------*/

/*-----------------------------------------------------------------------------------------------------------
	FWeakObjectPtr
-------------------------------------------------------------------------------------------------------------*/

/**  
 * Copy from an object pointer
 * @param Object object to create a weak pointer to
**/
void FWeakObjectPtr::operator=(FObjectPtr ObjectPtr)
{
	if (ObjectPtr // && UObjectInitialized() we might need this at some point, but it is a speed hit we would prefer to avoid
		)
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		ObjectRemoteId = ObjectPtr.GetRemoteId();

		// if the object is local, fill in the index and serial number immediately
		if (ObjectPtr.GetResidence() == EResidence::Local)
#endif
		{
			const UObject* Object = ObjectPtr.Get();
			ObjectIndex = GUObjectArray.ObjectToIndex((UObjectBase*)Object);
			ObjectSerialNumber = GUObjectArray.AllocateSerialNumber(ObjectIndex);
			checkSlow(SerialNumbersMatch());
		}
	}
	else
	{
		Reset();
	}
}

bool FWeakObjectPtr::IsValid(bool bEvenIfGarbage, bool bThreadsafeTest) const
{
	// This is the external function, so we just pass through to the internal inlined method.
	return Internal_IsValid(bEvenIfGarbage, bThreadsafeTest);
}

bool FWeakObjectPtr::IsValid() const
{
	// Using literals here allows the optimizer to remove branches later down the chain.
	return Internal_IsValid(false, false);
}

bool FWeakObjectPtr::IsStale(bool bIncludingGarbage, bool bThreadsafeTest) const
{
	using namespace UE::Core::Private;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (IsExplicitlyNull())
	{
		return false;
	}
#else
	if (ObjectSerialNumber == 0)
	{
#if UE_WEAKOBJECTPTR_ZEROINIT_FIX
		checkSlow(ObjectIndex == InvalidWeakObjectIndex); // otherwise this is a corrupted weak pointer
#else
		checkSlow(ObjectIndex == 0 || ObjectIndex == -1); // otherwise this is a corrupted weak pointer
#endif
		return false;
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	if (ObjectIndex < 0)
	{
		return true;
	}
	FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(ObjectIndex);
	if (!ObjectItem)
	{
		return true;
	}
	if (!SerialNumbersMatch(ObjectItem))
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (bThreadsafeTest)
		{
			return true;
		}
		if (UE::RemoteObject::Private::FindRemoteObjectStub(ObjectRemoteId))
		{
			return false;
		}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
		return true;
	}
	if (bThreadsafeTest)
	{
		return false;
	}
	return GUObjectArray.IsStale(ObjectItem, bIncludingGarbage);
}

UObject* FWeakObjectPtr::Get(/*bool bEvenIfGarbage = false*/) const
{
	// Using a literal here allows the optimizer to remove branches later down the chain.
	return Internal_Get(false);
}

UObject* FWeakObjectPtr::Get(bool bEvenIfGarbage) const
{
	return Internal_Get(bEvenIfGarbage);
}

UObject* FWeakObjectPtr::GetEvenIfUnreachable() const
{
	UObject* Result = nullptr;
	if (Internal_IsValid(true, true))
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(GetObjectIndex_Private(), true);
		Result = static_cast<UObject*>(ObjectItem->GetObject());
	}
	return Result;
}

TStrongObjectPtr<UObject> FWeakObjectPtr::Pin(/*bool bEvenIfGarbage = false*/) const
{
	// Using a literal here allows the optimizer to remove branches later down the chain.
	return Internal_Pin(false);
}

TStrongObjectPtr<UObject> FWeakObjectPtr::Pin(bool bEvenIfGarbage) const
{
	return Internal_Pin(bEvenIfGarbage);
}

TStrongObjectPtr<UObject> FWeakObjectPtr::PinEvenIfUnreachable() const
{
	FGCScopeGuard GCScopeGuard;
	UObject* Result = nullptr;
	if (Internal_IsValid(true, true))
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(GetObjectIndex_Private(), true);
		Result = static_cast<UObject*>(ObjectItem->GetObject());
	}
	return TStrongObjectPtr<UObject>(Result);
}

TStrongObjectPtr<UObject> FWeakObjectPtr::Internal_Pin(bool bEvenIfGarbage) const
{
	FGCScopeGuard GCScopeGuard;
	FUObjectItem* const ObjectItem = Internal_GetObjectItem();
	return TStrongObjectPtr<UObject>(((ObjectItem != nullptr) && GUObjectArray.IsValid(ObjectItem, bEvenIfGarbage)) ? (UObject*)ObjectItem->GetObject() : nullptr);
}

class TStrongObjectPtr<UObject> FWeakObjectPtr::TryPin(bool& bOutPinValid, bool bEvenIfGarbage) const
{
	return Internal_TryPin(bOutPinValid, bEvenIfGarbage);
}

class TStrongObjectPtr<UObject> FWeakObjectPtr::TryPin(bool& bOutPinValid/*, bool bEvenIfGarbage = false*/) const
{
	return Internal_TryPin(bOutPinValid, false);
}

bool FWeakObjectPtr::TryPinEvenIfUnreachable(class TStrongObjectPtr<UObject>& OutResult) const
{
	FGCScopeTryGuard GCScopeGuard;
	if(!GCScopeGuard.LockSucceeded())
	{
		return false;
	}

	UObject* Result = nullptr;
	if (Internal_IsValid(true, true))
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(GetObjectIndex_Private(), true);
		Result = static_cast<UObject*>(ObjectItem->GetObject());
	}
	OutResult = TStrongObjectPtr<UObject>(Result);
	return true;
}

TStrongObjectPtr<UObject> FWeakObjectPtr::Internal_TryPin(bool& bOutPinValid, bool bEvenIfGarbage) const
{
	TStrongObjectPtr<UObject> Result;

	FGCScopeTryGuard GCScopeGuard;
	bOutPinValid = GCScopeGuard.LockSucceeded();
	if (bOutPinValid)
	{
		FUObjectItem* const ObjectItem = Internal_GetObjectItem();
		Result = TStrongObjectPtr<UObject>(((ObjectItem != nullptr) && GUObjectArray.IsValid(ObjectItem, bEvenIfGarbage)) ? (UObject*)ObjectItem->GetObject() : nullptr);
	}

	return Result;
}

void FWeakObjectPtr::Serialize(FArchive& Ar)
{
	FArchiveUObject::SerializeWeakObjectPtr(Ar, *this);
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
bool FWeakObjectPtr::HasSameObject(const UObject* Other) const
{
	if (Other)
	{
		// It's not uncommon that people unsubscribe from multicast delegates in native UObject destructors
		// in which case the (Other) object index is already reset to -1. Currently this results in objects not being
		// unsubscribed and delagate instances are left with a stale weak object pointer. 
		// Object index is being checked here to silently support this old behavior.
		return GUObjectArray.ObjectToIndex(Other) >= 0 && ObjectRemoteId == FRemoteObjectId(Other);
	}
	return IsExplicitlyNull();
}

FUObjectItem* FWeakObjectPtr::TryResolveRemoteObject() const
{
	using namespace UE::RemoteObject::Handle;
	using namespace UE::RemoteObject::Private;

	FRemoteObjectId RemoteId = ObjectRemoteId;

	UObject* ResolvedObject = StaticFindObjectFastInternal(RemoteId);

	if (ResolvedObject)
	{
		if (UE::RemoteObject::Handle::GetResidence(ResolvedObject) != EResidence::Local)
		{
			// Object memory is still on this server but it's marked as remote so we need to resolve it
			ResolvedObject = ResolveObject(ResolvedObject, ERemoteReferenceType::Weak);
		}
		else
		{
			TouchResidentObject(ResolvedObject);
		}
	}
	else if (FRemoteServerId::IsGlobalServerIdInitialized())
	{
		if (FRemoteObjectStub* Stub = FindRemoteObjectStub(RemoteId))
		{
			ResolvedObject = ResolveObject(Stub, ERemoteReferenceType::Weak);
		}
	}

	if (ResolvedObject)
	{
		if (IsForceReturnObjectHandles())
		{
			// In this mode the result from ResolveObject() is actually an FObjectHandle which means it can also be a pointer to FRemoteObjectStub
			FObjectHandle ResolvedHandle = *reinterpret_cast<FObjectHandle*>(&ResolvedObject);
			checkf(IsInGameThread(), TEXT("Trying to dereference %s but when ForceReturnObjectHandles is active, weak object pointers can only be resolved on the game thread"), *ResolvedHandle.GetRemoteId().ToString());
			static FUObjectItem TempObjectItem;
			FMemory::Memzero(&TempObjectItem, sizeof(FUObjectItem));
			TempObjectItem.SetRemoteId(ResolvedHandle.GetRemoteId());
			TempObjectItem.SetObject(ResolvedObject);
			return &TempObjectItem;
		}

		int32 NewIndex = GUObjectArray.ObjectToIndex(ResolvedObject);
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(NewIndex);
		checkf(ObjectItem->GetRemoteId() == RemoteId, TEXT("Remote object %s has a different id (%s) than the weak object pointer that resolved it (%s)"),
			*ResolvedObject->GetPathName(), *ObjectItem->GetRemoteId().ToString(), *RemoteId.ToString());
		if (ObjectIndex != NewIndex)
		{
			// Currently we can't just rely on object id to resolve weak object pointers and we don't want to heep hitting the resolve path
			// if a remote object is resolved and exists on this server so (sadly) we need to update the index, yuck.
			// In general ObjectIndex would not be required at all if all objects were hashed with RemoteId in UObjectHash.cpp but currently we don't.
			// ObjectIndex also serves as an optimization because it's faster than hash table lookup.
			// With RemoteId neither ObjectIndex nor SerialNumber is used for FWeakObjectPtr comparisons/hash value calculation
			FPlatformAtomics::AtomicStore(&const_cast<FWeakObjectPtr*>(this)->ObjectIndex, NewIndex);
		}
		if (ObjectSerialNumber != ObjectItem->GetSerialNumber())
		{
			// ObjectSerialNumber doesn't play an important role when remote object support is enabled because the remote id is a globally unique
			// identifier of an object but we keep it as an optimization (no need to resolve the remote object if the serial numbers match).
			// The reason why the SerialNumber may change is that we could have a stale weak object ptr we want to resolve after the thing it pointed at was GC'd
			// and later created / loaded / migrated back.
			FPlatformAtomics::AtomicStore(&const_cast<FWeakObjectPtr*>(this)->ObjectSerialNumber, ObjectItem->GetSerialNumber());
		}
		return ObjectItem;
	}

	return nullptr;
}

bool FWeakObjectPtr::CanBeResolved(UE::RemoteObject::Handle::ECanResolveObjectBehavior CanResolveObjectBehavior) const
{
	return UE::RemoteObject::Handle::CanResolveObject(ObjectRemoteId, CanResolveObjectBehavior);
}

#endif // UE_WITH_REMOTE_OBJECT_HANDLE