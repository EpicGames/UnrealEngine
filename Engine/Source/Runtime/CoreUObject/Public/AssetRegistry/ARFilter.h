// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"

#include "ARFilter.generated.h"

class FCbField;
class FCbWriter;
struct FAssetData;

/**
 * A struct to serve as a filter for Asset Registry queries.
 * Each component element is processed as an 'OR' operation while all the components are processed together as an 'AND' operation.
 * This type is mirrored in NoExportTypes.h 
 */
USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/Engine.KismetSystemLibrary.MakeARFilter", HasNativeBreak = "/Script/Engine.KismetSystemLibrary.BreakARFilter"))
struct FARFilter
{
	GENERATED_BODY()
public:
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Compilers can complain about deprecated members in compiler generated code
	FARFilter() = default;
	FARFilter(FARFilter&&) = default;
	FARFilter(const FARFilter&) = default;
	FARFilter& operator=(FARFilter&&) = default;
	FARFilter& operator=(const FARFilter&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The filter component for package names */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AssetRegistry)
	TArray<FName> PackageNames;

	/** The filter component for package paths */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AssetRegistry)
	TArray<FName> PackagePaths;

#if WITH_EDITORONLY_DATA
	/** The filter component containing specific object paths */
	UE_DEPRECATED(5.1, "Asset path FNames have been deprecated, use FSoftObjectPath instead.")
	UPROPERTY()
	TArray<FName> ObjectPaths;
#endif

	/** 
	 * The filter component containing the paths of specific assets to match. 
	 * Matches against FAssetData::ToSoftObjectPath().
	 * This is a top level asset path for most assets and a subobject path for assets such as world partition external actors.
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AssetRegistry)
	TArray<FSoftObjectPath> SoftObjectPaths;
	
	/** Deprecated. The filter component for class names. Instances of the specified classes, but not subclasses (by default), will be included. Derived classes will be included only if bRecursiveClasses is true. */
	UE_DEPRECATED(5.1, "Class names are now represented by path names. Please use ClassPaths.")
	UPROPERTY(BlueprintReadWrite, Category = AssetRegistry, meta=(DeprecatedProperty, DeprecationMessage="Short asset class names must be converted to full asset pathnames. Use ClassPaths instead."))
	TArray<FName> ClassNames;

	/** The filter component for class path names. Instances of the specified classes, but not subclasses (by default), will be included. Derived classes will be included only if bRecursiveClasses is true. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AssetRegistry)
	TArray<FTopLevelAssetPath> ClassPaths;

	/** The filter component for properties marked with the AssetRegistrySearchable flag */
	TMultiMap<FName, TOptional<FString>> TagsAndValues;

	/** Deprecated. Only if bRecursiveClasses is true, the results will exclude classes (and subclasses) in this list */
	UE_DEPRECATED(5.1, "Class names are now represented by path names. Please use RecursiveClassPathsExclusionSet.")
	UPROPERTY(BlueprintReadWrite, Category = AssetRegistry, meta=(DeprecatedProperty, DeprecationMessage="Short asset class names must be converted to full asset pathnames. Use RecursiveClassPathsExclusionSet instead."))
	TSet<FName> RecursiveClassesExclusionSet;

	/** Only if bRecursiveClasses is true, the results will exclude classes (and subclasses) in this list */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AssetRegistry)
	TSet<FTopLevelAssetPath> RecursiveClassPathsExclusionSet;

	/** If true, PackagePath components will be recursive */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AssetRegistry)
	bool bRecursivePaths = false;

	/** If true, subclasses of ClassPaths will also be included and RecursiveClassPathsExclusionSet will be excluded. */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AssetRegistry)
	bool bRecursiveClasses = false;

	/**
	 * If true, use only DiskGatheredData, do not calculate from UObjects. @see IAssetRegistry class header for
	 * bIncludeOnlyOnDiskAssets.
	 */
	UPROPERTY(EditDefaultsOnly, BlueprintReadWrite, Category = AssetRegistry)
	bool bIncludeOnlyOnDiskAssets = false;

	/** The exclusive filter component for package flags. Only assets without any of the specified flags will be returned. */
	uint32 WithoutPackageFlags = 0;

	/** The inclusive filter component for package flags. Only assets with all of the specified flags will be returned. */
	uint32 WithPackageFlags = 0;

	/** Appends the other filter to this one */
	void Append(const FARFilter& Other)
	{
		PackageNames.Append(Other.PackageNames);
		PackagePaths.Append(Other.PackagePaths);
		SoftObjectPaths.Append(Other.SoftObjectPaths);
		ClassPaths.Append(Other.ClassPaths);

		for (auto TagIt = Other.TagsAndValues.CreateConstIterator(); TagIt; ++TagIt)
		{
			TagsAndValues.Add(TagIt.Key(), TagIt.Value());
		}

		RecursiveClassPathsExclusionSet.Append(Other.RecursiveClassPathsExclusionSet);

		bRecursivePaths |= Other.bRecursivePaths;
		bRecursiveClasses |= Other.bRecursiveClasses;
		bIncludeOnlyOnDiskAssets |= Other.bIncludeOnlyOnDiskAssets;
		WithoutPackageFlags |= Other.WithoutPackageFlags;
		WithPackageFlags |= Other.WithPackageFlags;
	}

	/** Returns true if this filter has no entries */
	bool IsEmpty() const
	{
		return (PackageNames.Num() | PackagePaths.Num() | SoftObjectPaths.Num() | ClassPaths.Num() | TagsAndValues.Num() | WithoutPackageFlags | WithPackageFlags) == 0;
	}

	/** Returns true if this filter is recursive */
	bool IsRecursive() const
	{
		return bRecursivePaths || bRecursiveClasses;
	}

	/** Clears this filter of all entries */
	void Clear()
	{
		PackageNames.Empty();
		PackagePaths.Empty();
		SoftObjectPaths.Empty();
		ClassPaths.Empty();
		TagsAndValues.Empty();
		RecursiveClassPathsExclusionSet.Empty();

		bRecursivePaths = false;
		bRecursiveClasses = false;
		bIncludeOnlyOnDiskAssets = false;
		WithoutPackageFlags = 0;
		WithPackageFlags = 0;

		ensure(IsEmpty());
	}

	void PostSerialize(const FArchive& Ar);

	/**
	 * Sort the structures (e.g. PackageNames) on this filter to be deterministic for saving the structure during
	 * cooking. This is also required for correctness of operator== when comparing the order-independent arrays.
	 */
	COREUOBJECT_API void SortForSaving();
	COREUOBJECT_API bool operator==(const FARFilter& Other) const;
	COREUOBJECT_API bool operator<(const FARFilter& Other) const;

private:
	COREUOBJECT_API void Save(FCbWriter& Writer) const;
	COREUOBJECT_API bool TryLoad(const FCbFieldView& Field);

	// Hidden friends for our argument-dependent lookup functions
	friend FCbWriter& operator<<(FCbWriter& Writer, const FARFilter& Filter)
	{
		Filter.Save(Writer);
		return Writer;
	}
	friend bool LoadFromCompactBinary(const FCbFieldView& Field, FARFilter& Filter)
	{
		return Filter.TryLoad(Field);
	}
};

template<>
struct TStructOpsTypeTraits<FARFilter> : public TStructOpsTypeTraitsBase2<FARFilter>
{
	enum
	{
		WithPostSerialize = true,
	};
};

/**
 * A struct to serve as a filter for Asset Registry queries.
 * Each component element is processed as an 'OR' operation while all the components are processed together as an 'AND' operation.
 * This is a version of FARFilter optimized for querying, and can be generated from an FARFilter by calling IAssetRegistry::CompileFilter to resolve any recursion.
 */
struct FARCompiledFilter
{
	FARCompiledFilter() = default;
	FARCompiledFilter(FARCompiledFilter&&) = default;
	FARCompiledFilter(const FARCompiledFilter&) = default;
	FARCompiledFilter& operator=(FARCompiledFilter&&) = default;
	FARCompiledFilter& operator=(const FARCompiledFilter&) = default;

	/** The filter component for package names */
	TSet<FName> PackageNames;

	/** The filter component for package paths */
	TSet<FName> PackagePaths;

	/** The filter component containing specific object paths */
	TSet<FSoftObjectPath> SoftObjectPaths;

	/** The filter component for class names. Instances of the specified classes, but not subclasses (by default), will be included. Derived classes will be included only if bRecursiveClasses is true. */
	TSet<FTopLevelAssetPath> ClassPaths;

	/** The filter component for properties marked with the AssetRegistrySearchable flag */
	TMultiMap<FName, TOptional<FString>> TagsAndValues;

	/** The exclusive filter component for package flags. Only assets without any of the specified flags will be returned. */
	uint32 WithoutPackageFlags = 0;

	/** The inclusive filter component for package flags. Only assets with all of the specified flags will be returned. */
	uint32 WithPackageFlags = 0;

	/** If true, only on-disk assets will be returned. Be warned that this is rarely what you want and should only be used for performance reasons */
	bool bIncludeOnlyOnDiskAssets = false;

	/** Returns true if this filter has no entries */
	bool IsEmpty() const
	{
		return (PackageNames.Num() | PackagePaths.Num() | SoftObjectPaths.Num() | ClassPaths.Num() | TagsAndValues.Num() | WithoutPackageFlags | WithPackageFlags) == 0;
	}

	/** Clears this filter of all entries */
	void Clear()
	{
		PackageNames.Empty();
		PackagePaths.Empty();
		SoftObjectPaths.Empty();
		ClassPaths.Empty();
		TagsAndValues.Empty();

		bIncludeOnlyOnDiskAssets = false;
		WithoutPackageFlags = 0;
		WithPackageFlags = 0;

		ensure(IsEmpty());
	}
};

namespace UE::AssetRegistry
{
	COREUOBJECT_API uint32 GetTypeHash(const FARFilter& Filter);

	COREUOBJECT_API FARFilter ConvertToNonCompiledFilter(const FARCompiledFilter& CompiledFilter);
}
