// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/PrimaryAssetId.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/TopLevelAssetPath.h"

#include "AssetBundleData.generated.h"

class FOutputDevice;
class UObject;

/** A struct representing a single AssetBundle */
USTRUCT()
struct FAssetBundleEntry
{
	GENERATED_BODY()
public:
	/** Declare constructors inline so this can be a header only class */
	UE_FORCEINLINE_HINT FAssetBundleEntry() {}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_FORCEINLINE_HINT ~FAssetBundleEntry() = default;
	// Explicitly declared constructors/operators because we explicitly declared the destructor to hide deprecation warnings.
	// These can be removed along with the destructor after deprecation.
	FAssetBundleEntry(const FAssetBundleEntry&) = default;
	FAssetBundleEntry(FAssetBundleEntry&&) = default;
	FAssetBundleEntry& operator=(const FAssetBundleEntry&) = default;
	FAssetBundleEntry& operator=(FAssetBundleEntry&&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Specific name of this bundle, should be unique for a given scope */
	UPROPERTY()
	FName BundleName;

#if WITH_EDITORONLY_DATA
	/** List of string assets contained in this bundle */
	UE_DEPRECATED(5.1, "BundleAssets has been replaced with AssetPaths.")
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Asset bundles may only contain top level asset paths which are referenced through the AssetPaths property."))
	TArray<FSoftObjectPath> BundleAssets;
#endif

	/** List of references to top-level assets contained in this bundle */
	UPROPERTY()
	TArray<FTopLevelAssetPath> AssetPaths;
	
	explicit FAssetBundleEntry(FName InBundleName)
		: BundleName(InBundleName)
	{

	}

	/** Returns true if this represents a real entry */
	bool IsValid() const { return !BundleName.IsNone(); }

	/** Equality */
	bool operator==(const FAssetBundleEntry& Other) const
	{
		// Ignore deprecated fields for equality checks
		return BundleName == Other.BundleName
			&& AssetPaths == Other.AssetPaths;
	}
	bool operator!=(const FAssetBundleEntry& Other) const
	{
		return !(*this == Other);
	}

	COREUOBJECT_API bool ExportTextItem(FString& ValueStr, const FAssetBundleEntry& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	COREUOBJECT_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
};

/** A struct with a list of asset bundle entries. If one of these is inside a UObject it will get automatically exported as the asset registry tag AssetBundleData */
USTRUCT()
struct FAssetBundleData
{
	GENERATED_BODY()
public:

	/** Declare constructors inline so this can be a header only class */
	UE_FORCEINLINE_HINT FAssetBundleData() {}
	UE_FORCEINLINE_HINT ~FAssetBundleData() = default;
	FAssetBundleData(const FAssetBundleData&) = default;
	FAssetBundleData(FAssetBundleData&&) = default;
	FAssetBundleData& operator=(const FAssetBundleData&) = default;
	FAssetBundleData& operator=(FAssetBundleData&&) = default;

	/** List of bundles defined */
	UPROPERTY()
	TArray<FAssetBundleEntry> Bundles;

	/** Equality */
	bool operator==(const FAssetBundleData& Other) const
	{
		return Bundles == Other.Bundles;
	}
	bool operator!=(const FAssetBundleData& Other) const
	{
		return !(*this == Other);
	}

	/** Returns pointer to an entry with given Scope/Name */
	COREUOBJECT_API FAssetBundleEntry* FindEntry(FName SearchName);

	/** Adds or updates an entry with the given BundleName -> Path.*/
	COREUOBJECT_API void AddBundleAsset(FName BundleName, const FTopLevelAssetPath& AssetPath);

	/** Adds multiple assets at once */
	COREUOBJECT_API void AddBundleAssets(FName BundleName, const TArray<FTopLevelAssetPath>& AssetPaths);

	/** A fast set of asset bundle assets, will destroy copied in path list */
	COREUOBJECT_API void SetBundleAssets(FName BundleName, TArray<FTopLevelAssetPath>&& AssetPaths);

	/** Adds or updates an entry with the given BundleName -> Path, truncating to a top-level asset path if necessary.*/
	COREUOBJECT_API void AddBundleAssetTruncated(FName BundleName, const FSoftObjectPath& AssetPath);

	/** Adds multiple assets at once, truncating to top-level asset paths if necessary */
	COREUOBJECT_API void AddBundleAssetsTruncated(FName BundleName, const TArray<FSoftObjectPath>& AssetPaths);

	/** Replaces the contents of this bundle with the given assets, truncating to top-level asset paths if necessary. */
	COREUOBJECT_API void SetBundleAssetsTruncated(FName BundleName, const TArray<FSoftObjectPath>& AssetPaths);

	/** Resets the data to defaults */
	COREUOBJECT_API void Reset();

	/** Override Import/Export to not write out empty structs */
	COREUOBJECT_API bool ExportTextItem(FString& ValueStr, FAssetBundleData const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	COREUOBJECT_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	COREUOBJECT_API FString ToDebugString() const;
};

template<>
struct TStructOpsTypeTraits<FAssetBundleData> : public TStructOpsTypeTraitsBase2<FAssetBundleData>
{
	enum
	{
		WithExportTextItem = true,
		WithImportTextItem = true,
	};
};
