// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetBundleData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetIdentifier.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "Containers/VersePathFwd.h"
#include "HAL/PlatformMath.h"
#include "HAL/UnrealMemory.h"
#include "IO/IoChunkId.h"
#include "IO/IoHash.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/SecureHash.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/Class.h"
#include "UObject/LinkerInstancingContext.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ObjectVersion.h"
#include "UObject/Package.h"
#include "UObject/PrimaryAssetId.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "AssetData.generated.h"

struct FAssetBundleData;
struct FCustomVersion;
class FCbWriter;
class FCbFieldView;
enum class EAssetRegistryTagsCaller : uint8;

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogAssetData, Log, All);
COREUOBJECT_API extern const FName GAssetBundleDataName;

/** Version used for serializing asset registry caches, both runtime and editor */
struct FAssetRegistryVersion
{
	enum Type
	{
		PreVersioning = 0,					// From before file versioning was implemented
		HardSoftDependencies,				// The first version of the runtime asset registry to include file versioning.
		AddAssetRegistryState,				// Added FAssetRegistryState and support for piecemeal serialization
		ChangedAssetData,					// AssetData serialization format changed, versions before this are not readable
		RemovedMD5Hash,						// Removed MD5 hash from package data
		AddedHardManage,					// Added hard/soft manage references
		AddedCookedMD5Hash,					// Added MD5 hash of cooked package to package data
		AddedDependencyFlags,				// Added UE::AssetRegistry::EDependencyProperty to each dependency
		FixedTags,							// Major tag format change that replaces USE_COMPACT_ASSET_REGISTRY:
											// * Target tag INI settings cooked into tag data
											// * Instead of FString values are stored directly as one of:
											//		- Narrow / wide string
											//		- [Numberless] FName
											//		- [Numberless] export path
											//		- Localized string
											// * All value types are deduplicated
											// * All key-value maps are cooked into a single contiguous range 
											// * Switched from FName table to seek-free and more optimized FName batch loading
											// * Removed global tag storage, a tag map reference-counts one store per asset registry
											// * All configs can mix fixed and loose tag maps 
		WorkspaceDomain,					// Added Version information to AssetPackageData
		PackageImportedClasses,				// Added ImportedClasses to AssetPackageData
		PackageFileSummaryVersionChange,	// A new version number of UE5 was added to FPackageFileSummary
		ObjectResourceOptionalVersionChange,// Change to linker export/import resource serialization
		AddedChunkHashes,					// Added FIoHash for each FIoChunkId in the package to the AssetPackageData.
		ClassPaths,							// Classes are serialized as path names rather than short object names, e.g. /Script/Engine.StaticMesh
		RemoveAssetPathFNames,				// Asset bundles are serialized as FTopLevelAssetPath instead of FSoftObjectPath, deprecated FAssetData::ObjectPath	
		AddedHeader,						// Added header with bFilterEditorOnlyData flag 
		AssetPackageDataHasExtension,		// Added Extension to AssetPackageData.
		AssetPackageDataHasPackageLocation,	// Added PackageLocation to AssetPackageData.
		MarshalledTextAsUTF8String,			// Replaced 2 byte wide string with UTF8 String
		PackageSavedHash,					// Replaced FAssetPackageData::PackageGuid with PackageSavedHash
		ExternalActorToWorldIsEditorOnly,   // FPackageDependencyData::LoadDependenciesFromPackageHeader changed how it calculates PackageDependencies
		ManageDependenciesCookRule,			// CookRule property added to EDependencyProperties for EDependencyCategory::Manage
		TagSetMapSupports64BitStringNum,	// 64-bit arrays for strings
		MemoryMappedTagDataStore,			// Tag data store is now memory mapped

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	/** The GUID for this custom version number */
	COREUOBJECT_API const static FGuid GUID;

	/** Read/write the custom version to the archive, should call at the very beginning */
	static COREUOBJECT_API bool SerializeVersion(FArchive& Ar, FAssetRegistryVersion::Type& Version);

private:
	FAssetRegistryVersion() {}
};

namespace UE::AssetRegistry
{
	using FChunkArrayRegistryHandle = FSetElementId;
}

namespace UE::AssetRegistry::Private
{
	struct FAssetPathParts
	{
		FStringView OuterPath;
		FStringView InnermostName;
	};

	/** 
	* Split a full object path into the path of the innermost object's outer and the name of that object.
	* E.g. splits /Path/To/PackageName.AssetName:SubObject.Innermost into /Path/To/PackageName.AssetName:SubObject and Innermost
	*/
	COREUOBJECT_API FAssetPathParts SplitIntoOuterPathAndAssetName(FStringView InObjectPath);
	/** Concatenates an existing object path with an inner object with the correct separator ('.' or ':'). Assumes the outer path already contains correct separators. */
	COREUOBJECT_API void ConcatenateOuterPathAndObjectName(FStringBuilderBase& Builder, FName OuterPath, FName ObjectName);
}

/**
 * Should we try and resolve the class when you call GetClass on FAssetData?  If so then it might potentially load several
 * other assets used by the class (Example: A UBlueprintGeneratedClass, might link in multiple other assets).
 */
enum class EResolveClass : uint8
{
	No,
	Yes,
};

/** 
 * A struct to hold important information about an assets found by the Asset Registry
 * This struct is transient and should never be serialized
 */
USTRUCT(BlueprintType)
struct FAssetData
{
	GENERATED_BODY()
public:
	/** The prefix used for collection entries inside TagsAndValues */
	static const TCHAR* GetCollectionTagPrefix()
	{
		return TEXT("CL_");
	}

	enum class ECreationFlags
	{
		None = 0,
		SkipAssetRegistryTagsGathering = 1 << 0, // Do not perform expensive step of gathering asset registry tags at construction.
		AllowBlueprintClass            = 1 << 1, // Unless specified, the default when trying to create one for a blueprint class 
												 // will create one for the UBlueprint instead, but this can be overridden
	};

public:
#if WITH_EDITORONLY_DATA
	/** The object path for the asset in the form PackageName.ObjectName, or PackageName.ObjectName:SubObjectName. */
	UE_DEPRECATED(5.1, "FName asset paths have been deprecated. Use GetSoftObjectPath to get the path this asset will use in memory when loaded or GetObjectPathString() if you were just doing ObjectPath.ToString()")
	UPROPERTY()
	FName ObjectPath;
#endif
	/** The name of the package in which the asset is found, this is the full long package name such as /Game/Path/Package */
	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient)
	FName PackageName;
	/** The path to the package in which the asset is found, this is /Game/Path with the Package stripped off */
	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient)
	FName PackagePath;
	/** The name of the asset without the package */
	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient)
	FName AssetName;
	/** The name of the asset's class */
	UE_DEPRECATED(5.1, "Class names are now represented by path names. Please use AssetClassPath.")
	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient, meta=(DeprecatedProperty, DeprecationMessage="Short asset class name must be converted to full asset pathname. Use AssetClassPath instead."))
	FName AssetClass;
	/** The path of the asset's class, e.g. /Script/Engine.StaticMesh */
	UPROPERTY(BlueprintReadOnly, Category = AssetData, transient)
	FTopLevelAssetPath AssetClassPath;

private:
	UE::AssetRegistry::FChunkArrayRegistryHandle ChunkArrayRegistryHandle;

public:
	/** Asset package flags */
	uint32 PackageFlags = 0;

#if WITH_EDITORONLY_DATA
private:
	/**
	 * If this object is not a top level asset, this contains the path of the outer of this object.
	 * Non top-level assets may only be used in the editor.
	 * For some assets (such as external actors) this may not start with PackageName.
	 * e.g. PackageName			= /Game/__EXTERNAL_ACTORS__/Maps/MyMap/ABCDE12345
			OptionalOuterPath	= /Game/Maps/MyMap.MyMap:PersistentLevel
			AssetName			= SomeExternalActor
	*/
	FName OptionalOuterPath;
public:
#endif

	/** The map of values for properties that were marked AssetRegistrySearchable or added by GetAssetRegistryTags */
	FAssetDataTagMapSharedView TagsAndValues;
	/**
	 * The 'AssetBundles' tag key is separated from TagsAndValues and typed for performance reasons.
	 * This is likely a temporary solution that will be generalized in some other fashion. 	
	 */
	TSharedPtr<FAssetBundleData, ESPMode::ThreadSafe> TaggedAssetBundles;

	COREUOBJECT_API void SetTagsAndAssetBundles(FAssetDataTagMap&& Tags);

	// These are usually very small arrays and we can preallocate two elements for the same cost as one on 64-bit systems
	using FChunkArray = TArray<int32, TInlineAllocator<2>>;
	using FChunkArrayView = TConstArrayView<int32>;

	COREUOBJECT_API FChunkArrayView GetChunkIDs() const;
	COREUOBJECT_API void SetChunkIDs(FChunkArray&& InChunkIDs);
	COREUOBJECT_API void SetChunkIDs(const FChunkArrayView& InChunkIDs);
	COREUOBJECT_API void AddChunkID(int32 ChunkID);
	COREUOBJECT_API void ClearChunkIDs();
	COREUOBJECT_API bool HasSameChunkIDs(const FAssetData& OtherAssetData) const;

	/** Returns overhead of the chunk array registry that's used to manage chunk ID arrays. */
	static COREUOBJECT_API SIZE_T GetChunkArrayRegistryAllocatedSize();

public:	
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Compilers can complain about deprecated members in compiler generated code
	/** Default constructors. */
	FAssetData()
	{
	}
	FAssetData(FAssetData&&) = default;
	FAssetData(const FAssetData&) = default;
	FAssetData& operator=(FAssetData&&) = default;
	FAssetData& operator=(const FAssetData&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Constructor building the ObjectPath in the form of InPackageName.InAssetName. does not work for object outer-ed to a different package. */
	COREUOBJECT_API FAssetData(FName InPackageName, FName InPackagePath, FName InAssetName, FTopLevelAssetPath InAssetClassPathName, FAssetDataTagMap InTags = FAssetDataTagMap(), TArrayView<const int32> InChunkIDs = TArrayView<const int32>(), uint32 InPackageFlags = 0);
	/** Constructor with a long package name and a full object path which might not be part of the package this asset is in. */
	COREUOBJECT_API FAssetData(FStringView InLongPackageName, FStringView InObjectPath, FTopLevelAssetPath InAssetClassPathName, FAssetDataTagMap InTags = FAssetDataTagMap(), TArrayView<const int32> InChunkIDs = TArrayView<const int32>(), uint32 InPackageFlags = 0);

	/** Constructor taking a UObject. By default trying to create one for a blueprint class will create one for the UBlueprint instead, but this can be overridden */
	COREUOBJECT_API FAssetData(const UObject* InAsset, FAssetData::ECreationFlags InCreationFlags = ECreationFlags::None);
	COREUOBJECT_API FAssetData(const UObject* InAsset, FAssetData::ECreationFlags InCreationFlags, EAssetRegistryTagsCaller Caller);

	/** Constructor taking a UObject. By default trying to create one for a blueprint class will create one for the UBlueprint instead, but this can be overridden */
	inline FAssetData(const UObject* InAsset, bool bAllowBlueprintClass)
		: FAssetData(InAsset, bAllowBlueprintClass ? ECreationFlags::AllowBlueprintClass : ECreationFlags::None)
	{
	}

	/** FAssetDatas are uniquely identified by PackageName and AssetName. */
	bool operator==(const FAssetData& Other) const
	{
		return PackageName == Other.PackageName && AssetName == Other.AssetName;
	}

	bool operator!=(const FAssetData& Other) const
	{
		return PackageName != Other.PackageName || AssetName != Other.AssetName;
	}

	/** Perform a lexical greater-than operation on the PackageName and AssetName that uniquely identify two FAssetData. */
	bool operator>(const FAssetData& Other) const
	{
		if (PackageName == Other.PackageName)
		{
			return Other.AssetName.LexicalLess(AssetName);
		}
		return Other.PackageName.LexicalLess(PackageName);
	}

	/** Perform a lexical less-than operation on the PackageName and AssetName that uniquely identify two FAssetData. */
	bool operator<(const FAssetData& Other) const
	{
		if (PackageName == Other.PackageName)
		{
			return AssetName.LexicalLess(Other.AssetName);
		}
		return PackageName.LexicalLess(Other.PackageName);
	}

	/** Checks to see if this AssetData refers to an asset or is NULL */
	bool IsValid() const
	{
		return !PackageName.IsNone() && !AssetName.IsNone();
	}

	/**
	 * Returns true if this is the main asset in a package, true for maps and assets but false for secondary objects like class redirectors
	 * Every UAsset is also a TopLevelAsset.
	 * Specifically, this is just testing whether the asset name matches the package name, and it is possible for legacy content to not return
	 * true for any asset in a package.
	 */
	bool IsUAsset() const
	{
		if (!IsValid())
		{
			return false;
		}

#if WITH_EDITORONLY_DATA
		if (!OptionalOuterPath.IsNone())
		{
			return false;
		}
#endif //WITH_EDITORONLY_DATA

		TStringBuilder<FName::StringBufferSize> AssetNameStrBuilder;
		AssetName.ToString(AssetNameStrBuilder);

		TStringBuilder<FName::StringBufferSize> PackageNameStrBuilder;
		PackageName.ToString(PackageNameStrBuilder);
		return DetectIsUAssetByNames(PackageNameStrBuilder, AssetNameStrBuilder);
	}

	/** Convert to a SoftObjectPath. PackageName.AssetName for ToplevelAssets. OptionalOuterName and AssetName joined by : or . for sub-object assets */
	COREUOBJECT_API FSoftObjectPath GetSoftObjectPath() const; 
	
	// TODO: Deprecate in favor of GetSoftObjectPath
	FSoftObjectPath ToSoftObjectPath() const
	{
		return GetSoftObjectPath();
	}
	
	/** Return the object path as a string. */
	FString GetObjectPathString() const
	{
		TStringBuilder<FName::StringBufferSize> Builder;
		AppendObjectPath(Builder);
		return FString(Builder);
	}

	/**
	 * Returns true if the given UObject is the main asset in a package, true for maps and assets but false for secondary objects like class redirectors
	 * Every UAsset is also a TopLevelAsset.
	 */
	COREUOBJECT_API static bool IsUAsset(UObject* Object);

	/**
	 * Returns true iff the Asset is a TopLevelAsset (not a subobject, its outer is a UPackage).
	 * Only TopLevelAssets can be PrimaryAssets in the AssetManager.
	 * A TopLevelAsset is not necessarily the main asset in a package; see IsUAsset.
	 * Note that this is distinct from UObject::IsAsset because IsAsset can be overloaded (see e.g. AActor::IsAsset)
	 */
	COREUOBJECT_API bool IsTopLevelAsset() const;
	
	/**
	 * Returns true iff the given Object, assumed to be an Asset, is a TopLevelAsset (not a subobject, its outer is a UPackage).
	 * Only TopLevelAssets can be PrimaryAssets in the AssetManager.
	 * A TopLevelAsset is not necessarily the main asset in a package; see IsUAsset.
	 */
	COREUOBJECT_API static bool IsTopLevelAsset(UObject* Object);

	void Shrink()
	{
		TagsAndValues.Shrink();
	}

#if WITH_EDITORONLY_DATA	
	FName GetOptionalOuterPathName() const { return OptionalOuterPath; }
	void SetOptionalOuterPathName(FName InName) { OptionalOuterPath = InName; }
#endif

	/** Returns the full name for the asset in the form: Class FullPath */
	FString GetFullName() const
	{
		FString FullName;
		GetFullName(FullName);
		return FullName;
	}

	/** Populates OutFullName with the full name for the asset in the form: Class FullPath */
	void GetFullName(FString& OutFullName) const
	{
		OutFullName.Reset();
		OutFullName += AssetClassPath.ToString();
		OutFullName.AppendChar(TEXT(' '));
		AppendObjectPath(OutFullName);
	}

	/** Populates OutFullNameBuilder with the full name for the asset in the form: Class ObjectPath */
	void GetFullName(FStringBuilderBase& OutFullNameBuilder) const
	{
		OutFullNameBuilder.Reset();
		AssetClassPath.AppendString(OutFullNameBuilder);
		OutFullNameBuilder.AppendChar(TEXT(' '));
		AppendObjectPath(OutFullNameBuilder);
	}

	/** Returns the name for the asset in the form: Class'FullPath' */
	FString GetExportTextName() const
	{
		FString ExportTextName;
		GetExportTextName(ExportTextName);
		return ExportTextName;
	}

	/** Populates OutExportTextName with the name for the asset in the form: Class'FullPath' */
	void GetExportTextName(FString& OutExportTextName) const
	{
		OutExportTextName.Reset();
		OutExportTextName.Append(AssetClassPath.ToString());
		OutExportTextName.AppendChar(TEXT('\''));
		AppendObjectPath(OutExportTextName);
		OutExportTextName.AppendChar(TEXT('\''));
	}

	/** Populates OutExportTextNameBuilder with the name for the asset in the form: Class'FullPath' */
	void GetExportTextName(FStringBuilderBase& OutExportTextNameBuilder) const
	{
		OutExportTextNameBuilder.Reset();
		AssetClassPath.AppendString(OutExportTextNameBuilder);
		OutExportTextNameBuilder.AppendChar(TEXT('\''));
		AppendObjectPath(OutExportTextNameBuilder);
		OutExportTextNameBuilder.AppendChar(TEXT('\''));
	}

	/**
	 * Gets the versepath of the asset.
	 *
	 * @return The VersePath of the asset
	 */
	COREUOBJECT_API UE::Core::FVersePath GetVersePath() const;

	/** Returns true if the this asset is a redirector. */
	bool IsRedirector() const
	{
		return IsRedirectorClassName(AssetClassPath);
	}

	static bool IsRedirector(UObject* Object)
	{
		return Object && IsRedirectorClassName(Object->GetClass()->GetClassPathName());
	}

	/**
	 * Returns the class UClass if it is loaded.  If you choose to override ResolveClass, to EResolveClass::Yes
	 * it will attempt to load the class if unloaded.
	 */
	COREUOBJECT_API UClass* GetClass(EResolveClass ResolveClass = EResolveClass::No) const;
	
	/**
	 * Returns whether the Asset's class is equal to or a child class of the given class. Returns false if the Asset's class can not be loaded.
	 * If you choose to override ResolveClass, to EResolveClass::Yes it will attempt to load the class if unloaded.
	 */
	bool IsInstanceOf(const UClass* BaseClass, EResolveClass ResolveClass = EResolveClass::No) const
	{
		UClass* ClassPointer = GetClass(ResolveClass);
		return ClassPointer && ClassPointer->IsChildOf(BaseClass);
	}

	template<typename BaseClass>
	bool IsInstanceOf(EResolveClass ResolveClass = EResolveClass::No) const
	{
		return IsInstanceOf(BaseClass::StaticClass(), ResolveClass);
	}

	/** Append the object path to the given string builder. */
	void AppendObjectPath(FStringBuilderBase& Builder) const
	{
		if (!IsValid())
		{
			return;
		}

#if WITH_EDITORONLY_DATA
		if (!OptionalOuterPath.IsNone())
		{
			UE::AssetRegistry::Private::ConcatenateOuterPathAndObjectName(Builder, OptionalOuterPath, AssetName);
		}
		else
#endif
		{
			Builder << PackageName << '.' << AssetName;
		}
    }
	
	/** Append the object path to the given string. */
	void AppendObjectPath(FString& String) const
	{
		TStringBuilder<FName::StringBufferSize> Builder;
		AppendObjectPath(Builder);
		String.Append(FString(Builder));
	}
	
	/** Gets primary asset id of this data */
	COREUOBJECT_API FPrimaryAssetId GetPrimaryAssetId() const;

	/** 
	 * Returns the asset UObject if it is loaded or loads the asset if it is unloaded then returns the result
	 * 
	 * Note this can return nullptr even if it looks like the asset exists because it doesn't handle redirects.
	 * 
	 * @param bLoad (optional) loads the asset if it is unloaded.
	 * @param LoadTags (optional) allows passing specific tags to the linker when loading the asset (@see ULevel::LoadAllExternalObjectsTag for an example usage)
	 */
	UObject* FastGetAsset(bool bLoad = false, TSet<FName> LoadTags = {}) const
	{
		if (!IsValid())
		{
			// Do not try to find the object if the fields are not set
			return nullptr;
		}

		// We load PackageName rather than using GetObjectPath because external assets may be saved in a different package to their loaded path. 
		UPackage* FoundPackage = FindObjectFast<UPackage>(nullptr, PackageName);

		// We could find an existing package in memory that is still pending load in the loader.
		if (FoundPackage && !FoundPackage->IsFullyLoaded())
		{
			FoundPackage = nullptr;
		}

		if (FoundPackage == nullptr && bLoad)
		{
			FLinkerInstancingContext InstancingContext(MoveTemp(LoadTags));
			FoundPackage = LoadPackage(nullptr, *PackageName.ToString(), LOAD_None, nullptr, &InstancingContext);
		}

		if (!FoundPackage)
		{
			return nullptr;
		}

#if WITH_EDITORONLY_DATA
		if (!OptionalOuterPath.IsNone())
		{
			TStringBuilder<FName::StringBufferSize> Builder;
			AppendObjectPath(Builder);
			return FindObject<UObject>(nullptr, *Builder);
		}
		else
#endif
		{
			// If the package is loaded all objects in it must be loaded, just search for the asset in memory with its outer.
			return FindObjectFast<UObject>(FoundPackage, AssetName);
		}
	}

	/** 
	 * Returns the asset UObject if it is loaded or loads the asset if it is unloaded then returns the result 
	 * 
	 * Note this can return nullptr even if it looks like the asset exists because it doesn't handle redirects.
	 * 
	 * @param LoadTags (optional) allows passing specific tags to the linker when loading the asset (@see ULevel::LoadAllExternalObjectsTag for an example usage)
	 */
	UObject* GetAsset(TSet<FName> LoadTags = {}) const
	{

		return FastGetAsset(true /* bLoad */, MoveTemp(LoadTags));
	}

	/**
	 * Used to check whether the any of the passed flags are set in the cached asset package flags.
	 * @param	FlagsToCheck  Package flags to check for
	 * @return	true if any of the passed in flag are set, false otherwise
	 * @see UPackage::HasAnyPackageFlags
	 */
	bool HasAnyPackageFlags(uint32 FlagsToCheck) const
	{
		return (PackageFlags & FlagsToCheck) != 0;
	}

	/**
	 * Used to check whether all of the passed flags are set in the cached asset package flags.
	 * @param FlagsToCheck	Package flags to check for
	 * @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	 * @see UPackage::HasAllPackagesFlags
	 */
	bool HasAllPackageFlags(uint32 FlagsToCheck) const
	{
		return ((PackageFlags & FlagsToCheck) == FlagsToCheck);
	}

	/** Returns how package can be referenced from other plugins and mount points */
	EAssetAccessSpecifier GetAssetAccessSpecifier() const
	{
		if (PackageFlags & PKG_NotExternallyReferenceable)
		{
			return EAssetAccessSpecifier::Private;
		}
		else if (PackageFlags & PKG_AccessSpecifierEpicInternal)
		{
			return EAssetAccessSpecifier::EpicInternal;
		}

		return EAssetAccessSpecifier::Public;
	}

	/** Tries to find the package in memory if it is loaded, otherwise loads it. */
	UPackage* GetPackage() const
	{
		if (PackageName.IsNone())
		{
			return nullptr;
		}

		UPackage* Package = FindPackage(NULL, *PackageName.ToString());
		if (Package)
		{
			Package->FullyLoad();
		}
		else
		{
			Package = LoadPackage(NULL, *PackageName.ToString(), LOAD_None);
		}

		return Package;
	}

	/** Try to find the given tag  */
	bool FindTag(const FName InTagName) const
	{
		return TagsAndValues.FindTag(InTagName).IsSet();
	}

	/** Try and get the value associated with the given tag as a type converted value */
	template <typename ValueType>
	bool GetTagValue(FName Tag, ValueType& OutValue) const;

	/** Call the given function for each tag on the asset. Function signature is: void Fn(TPair<FName, FAssetTagValueRef>); */
	template<typename Func>
	void EnumerateTags(Func Fn) const
	{
		TagsAndValues.ForEach(Fn);
	}

	/** Try and get the value associated with the given tag as a type converted value, or an empty value if it doesn't exist */
	template <typename ValueType>
	ValueType GetTagValueRef(const FName Tag) const;

	/** Returns true if the asset is loaded */
	bool IsAssetLoaded() const
	{
		return FastGetAsset(false) != nullptr;
	}

	/** Prints the details of the asset to the log */
	void PrintAssetData() const
	{
		UE_LOGF(LogAssetData, Log, "    FAssetData for %ls", *GetObjectPathString());
		UE_LOGF(LogAssetData, Log, "    =============================");
		UE_LOGF(LogAssetData, Log, "        PackageName: %ls", *PackageName.ToString());
		UE_LOGF(LogAssetData, Log, "        PackagePath: %ls", *PackagePath.ToString());
		UE_LOGF(LogAssetData, Log, "        AssetName: %ls", *AssetName.ToString());
		UE_LOGF(LogAssetData, Log, "        AssetClassPath: %ls", *AssetClassPath.ToString());
		UE_LOGF(LogAssetData, Log, "        TagsAndValues: %d", TagsAndValues.Num());

		for (const auto TagValue: TagsAndValues)
		{
			UE_LOGF(LogAssetData, Log, "            %ls : %ls", *TagValue.Key.ToString(), *TagValue.Value.AsString());
		}

		const TConstArrayView<int32> CurrentChunkIDs = GetChunkIDs();
		UE_LOGF(LogAssetData, Log, "        ChunkIDs: %d", CurrentChunkIDs.Num());

		for (int32 Chunk : CurrentChunkIDs)
		{
			UE_LOGF(LogAssetData, Log, "                 %d", Chunk);
		}

		UE_LOGF(LogAssetData, Log, "        PackageFlags: %d", PackageFlags);
	}

	/** Get the first FAssetData of a particular class from an Array of FAssetData */
	static FAssetData GetFirstAssetDataOfClass(const TArray<FAssetData>& Assets, const UClass* DesiredClass)
	{
		for(int32 AssetIdx=0; AssetIdx<Assets.Num(); AssetIdx++)
		{
			const FAssetData& Data = Assets[AssetIdx];
			if (Data.IsInstanceOf(DesiredClass))
			{
				return Data;
			}
		}
		return FAssetData();
	}

	/** Convenience template for finding first asset of a class */
	template <class T>
	static T* GetFirstAsset(const TArray<FAssetData>& Assets)
	{
		UClass* DesiredClass = T::StaticClass();
		UObject* Asset = FAssetData::GetFirstAssetDataOfClass(Assets, DesiredClass).GetAsset();
		check(Asset == NULL || Asset->IsA(DesiredClass));
		return (T*)Asset;
	}

	/** 
	 * Serialize as part of the registry cache. This is not meant to be serialized as part of a package so  it does not handle versions normally
	 * To version this data change FAssetRegistryVersion
	 */
	template<class Archive>
	inline void SerializeForCache(Archive&& Ar)
	{
		SerializeForCacheWithTagsAndBundles(Ar, [](FArchive& Ar, FAssetData& Ad, FAssetRegistryVersion::Type) {
			static_cast<Archive&>(Ar).SerializeTagsAndBundles(Ad);
			});
	}
	/**
	 * Serialize as part of the registry cache using legacy paths (versioned)
	 */
	template<class Archive>
	inline void SerializeForCacheOldVersion(Archive&& Ar, FAssetRegistryVersion::Type Version = FAssetRegistryVersion::LatestVersion)
	{
		SerializeForCacheOldVersionWithTagsAndBundles(Ar, Version, [](FArchive& Ar, FAssetData& Ad, FAssetRegistryVersion::Type Version) {
			static_cast<Archive&>(Ar).SerializeTagsAndBundlesOldVersion(Ad, Version);
			});
	}

	// Note: these functions should only be used for live communication between processes running the same version of the engine.
	// There is no versioning support
	COREUOBJECT_API void NetworkWrite(FCbWriter& Writer, bool bWritePackageName) const;
	COREUOBJECT_API bool TryNetworkRead(FCbFieldView Field, bool bReadPackageName, FName InPackageName);
private:
	/**
	 * The actual implementation of SerializeForCache.
	 * Note that this function is force-inlined but defined in AssetData.cpp which is fine as functions will get inlined
	 * as long as they're defined before they are used for the first time by other functions (SerializeForCacheWithTagsAndBundles in this case)
	 */
	UE_FORCEINLINE_HINT void SerializeForCacheInternal(FArchive& Ar, FAssetRegistryVersion::Type Version, void (*SerializeTagsAndBundles)(FArchive& , FAssetData&, FAssetRegistryVersion::Type));
	COREUOBJECT_API void SerializeForCacheWithTagsAndBundles(FArchive& Ar, void (*SerializeTagsAndBundles)(FArchive&, FAssetData&, FAssetRegistryVersion::Type));
	COREUOBJECT_API void SerializeForCacheOldVersionWithTagsAndBundles(FArchive& Ar, FAssetRegistryVersion::Type Version, void (*SerializeTagsAndBundles)(FArchive&, FAssetData&, FAssetRegistryVersion::Type));

	static bool DetectIsUAssetByNames(FStringView PackageName, FStringView ObjectPathName)
	{
		FStringView PackageBaseName;
		{
			// Get everything after the last slash
			int32 IndexOfLastSlash = INDEX_NONE;
			PackageName.FindLastChar(TEXT('/'), IndexOfLastSlash);
			PackageBaseName = PackageName.Mid(IndexOfLastSlash + 1);
		}

		return PackageBaseName.Equals(ObjectPathName, ESearchCase::IgnoreCase);
	}

public:
	COREUOBJECT_API static bool IsRedirectorClassName(FTopLevelAssetPath ClassPathName);

	/** Helper function that tries to convert short class name to path name */
	COREUOBJECT_API static FTopLevelAssetPath TryConvertShortClassNameToPathName(FName InClassName, ELogVerbosity::Type FailureMessageVerbosity = ELogVerbosity::Warning);

	friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const FAssetData& AssetData)
	{
		return HashCombine(GetTypeHash(AssetData.PackageName), GetTypeHash(AssetData.AssetName));
	}
};

ENUM_CLASS_FLAGS(FAssetData::ECreationFlags);


template<>
struct TStructOpsTypeTraits<FAssetData> : public TStructOpsTypeTraitsBase2<FAssetData>
{
	enum
	{
		WithIdenticalViaEquality = true
	};
};

template <typename ValueType>
inline bool FAssetData::GetTagValue(FName Tag, ValueType& OutValue) const
{
	const FAssetDataTagMapSharedView::FFindTagResult FoundValue = TagsAndValues.FindTag(Tag);
	if (FoundValue.IsSet())
	{
		FMemory::Memzero(&OutValue, sizeof(ValueType));
		LexFromString(OutValue, *FoundValue.GetValue());
		return true;
	}
	return false;
}

template <>
inline bool FAssetData::GetTagValue<FString>(FName Tag, FString& OutValue) const
{
	const FAssetDataTagMapSharedView::FFindTagResult FoundValue = TagsAndValues.FindTag(Tag);
	if (FoundValue.IsSet())
	{
		OutValue = FoundValue.AsString();
		return true;
	}

	return false;
}

template <>
inline bool FAssetData::GetTagValue<FText>(FName Tag, FText& OutValue) const
{
	const FAssetDataTagMapSharedView::FFindTagResult FoundValue = TagsAndValues.FindTag(Tag);
	if (FoundValue.IsSet())
	{
		OutValue = FoundValue.AsText();
		return true;
	}

	return false;
}

template <>
inline bool FAssetData::GetTagValue<FName>(FName Tag, FName& OutValue) const
{
	const FAssetDataTagMapSharedView::FFindTagResult FoundValue = TagsAndValues.FindTag(Tag);
	if (FoundValue.IsSet())
	{
		OutValue = FoundValue.AsName();
		return true;
	}
	return false;
}

template <typename ValueType>
inline ValueType FAssetData::GetTagValueRef(FName Tag) const
{
	ValueType TmpValue;
	FMemory::Memzero(&TmpValue, sizeof(ValueType));
	const FAssetDataTagMapSharedView::FFindTagResult FoundValue = TagsAndValues.FindTag(Tag);
	if (FoundValue.IsSet())
	{
		LexFromString(TmpValue, *FoundValue.GetValue());
	}
	return TmpValue;
}

template <>
inline FString FAssetData::GetTagValueRef<FString>(FName Tag) const
{
	const FAssetDataTagMapSharedView::FFindTagResult FoundValue = TagsAndValues.FindTag(Tag);
	return FoundValue.IsSet() ? FoundValue.AsString() : FString();
}

template <>
inline FText FAssetData::GetTagValueRef<FText>(FName Tag) const
{
	FText TmpValue;
	GetTagValue(Tag, TmpValue);
	return TmpValue;
}

template <>
inline FName FAssetData::GetTagValueRef<FName>(FName Tag) const
{
	const FAssetDataTagMapSharedView::FFindTagResult FoundValue = TagsAndValues.FindTag(Tag);
	return FoundValue.IsSet() ? FoundValue.AsName() : FName();
}

template <>
inline FAssetRegistryExportPath FAssetData::GetTagValueRef<FAssetRegistryExportPath>(FName Tag) const
{
	const FAssetDataTagMapSharedView::FFindTagResult FoundValue = TagsAndValues.FindTag(Tag);
	return FoundValue.IsSet() ? FoundValue.AsExportPath() : FAssetRegistryExportPath();
}

namespace UE::AssetRegistry
{

/** Low-memory version of FCustomVersion; holds only Guid and integer version. */
struct FPackageCustomVersion
{
	FGuid Key;
	int32 Version = 0;

	FPackageCustomVersion() = default;
	FPackageCustomVersion(const FGuid& InKey, const int32 InVersion)
		: Key(InKey)
		, Version(InVersion)
	{
	}
	bool operator<(const FPackageCustomVersion& RHS) const
	{
		if (Key < RHS.Key) return true;
		if (RHS.Key < Key) return false;
		return Version < RHS.Version;
	}
	bool operator==(const FPackageCustomVersion& RHS) const
	{
		return Key == RHS.Key && Version == RHS.Version;
	}
	friend FArchive& operator<<(FArchive& Ar, FPackageCustomVersion& CustomVersion)
	{
		return Ar << CustomVersion.Key << CustomVersion.Version;
	}

private:
	friend FCbWriter& operator<<(FCbWriter& Writer, const FPackageCustomVersion& Value)
	{
		return Value.Write(Writer);
	}
	COREUOBJECT_API FCbWriter& Write(FCbWriter& Writer) const;
	friend bool LoadFromCompactBinary(const FCbFieldView& Field, FPackageCustomVersion& Value)
	{
		return Value.TryRead(Field);
	}
	COREUOBJECT_API bool TryRead(const FCbFieldView& Field);
};

/** A handle to a deduplicated, sorted array of FPackageCustomVersion. */
class FPackageCustomVersionsHandle
{
public:
	TConstArrayView<FPackageCustomVersion> Get() const { return Ptr; }

	COREUOBJECT_API static FPackageCustomVersionsHandle FindOrAdd(TConstArrayView<FCustomVersion> InCustomVersions);
	COREUOBJECT_API static FPackageCustomVersionsHandle FindOrAdd(TConstArrayView<FPackageCustomVersion> InCustomVersions);
	COREUOBJECT_API static FPackageCustomVersionsHandle FindOrAdd(TArray<FPackageCustomVersion>&& InCustomVersions);
	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FPackageCustomVersionsHandle& Handle);

private:
	TConstArrayView<FPackageCustomVersion> Ptr;
	friend class FPackageCustomVersionRegistry;
};

}

/** A class to hold data about a package on disk, this data is updated on save/load and is not updated when an asset changes in memory */
class FAssetPackageData
{
public:
	/** MD5 of the cooked package on disk, for tracking nondeterministic changes */
	FMD5Hash CookedHash;

	EPackageExtension Extension;

private:
	/** Bit storage for flags */
	uint32 Flags;

public:

	/** 
		The hash of all the chunks written by the package writer for this package. This is only available after cooking.
		(Note, this only includes the ExportBundleData chunk if using ZenStoreWriter, as that chunk contains both the uasset and uexp
		data, which is managed separately in LooseCookedPackageWriter.)
	*/
	TMap<FIoChunkId, FIoHash> ChunkHashes;

	/** List of classes used by exports in the package. Does not include classes in the same package. */
	TArray<FName> ImportedClasses;

	/** Total size of this asset on disk */
	int64 DiskSize;

	/** UE file version that the package was saved with */
	FPackageFileVersion FileVersionUE;

private:
#if WITH_EDITORONLY_DATA
	FIoHash PackageSavedHash;
#endif

public:
	/** Licensee file version that the package was saved with */
	int32 FileVersionLicenseeUE;

private:
	UE::AssetRegistry::FPackageCustomVersionsHandle CustomVersions;

public:


	FAssetPackageData()
		: Extension(EPackageExtension::Unspecified)
		, Flags(0)
		, DiskSize(0)
		, FileVersionLicenseeUE(-1)
	{
	}

	/**
	 * Custom versions used by the package, used to check whether we need to update the package for the current binary.
	 * The array is sorted by FPackageCustomVersion::operator<.
	 */
	TConstArrayView<UE::AssetRegistry::FPackageCustomVersion> GetCustomVersions() const
	{
		return CustomVersions.Get();
	}
	void SetCustomVersions(TConstArrayView<FCustomVersion> InCustomVersions)
	{
		CustomVersions = UE::AssetRegistry::FPackageCustomVersionsHandle::FindOrAdd(InCustomVersions);
	}
	void SetCustomVersions(TConstArrayView<UE::AssetRegistry::FPackageCustomVersion> InCustomVersions)
	{
		CustomVersions = UE::AssetRegistry::FPackageCustomVersionsHandle::FindOrAdd(InCustomVersions);
	}
	void SetCustomVersions(TArray<UE::AssetRegistry::FPackageCustomVersion>&& InCustomVersions)
	{
		CustomVersions = UE::AssetRegistry::FPackageCustomVersionsHandle::FindOrAdd(MoveTemp(InCustomVersions));
	}

	/** Whether the package was saved from a licensee executable, used to tell whether non-matching FileVersionLicenseeUE requires a resave */
	bool IsLicenseeVersion() const { return (Flags & FLAG_LICENSEE_VERSION) != 0; }
	void SetIsLicenseeVersion(bool bValue) { Flags = (Flags & ~FLAG_LICENSEE_VERSION) | (bValue ? FLAG_LICENSEE_VERSION : 0); }

	/** Whether the package contains virtualized payloads or not */
	bool HasVirtualizedPayloads() const { return (Flags & FLAG_HAS_VIRTUALIZED_PAYLOADS) != 0; }
	void SetHasVirtualizedPayloads(bool bValue) { Flags = (Flags & ~FLAG_HAS_VIRTUALIZED_PAYLOADS) | (bValue ? FLAG_HAS_VIRTUALIZED_PAYLOADS : 0); }

	/** Whether the package is readonly or not */
	bool IsReadOnly() const { return (Flags & FLAG_PACKAGE_READ_ONLY) != 0; }
	void SetIsReadOnly(bool bValue) { Flags = (Flags & ~FLAG_PACKAGE_READ_ONLY) | (bValue ? FLAG_PACKAGE_READ_ONLY : 0); }

	/**
	 * Serialize as part of the registry cache. This is not meant to be serialized as part of a package so it does not handle versions normally
	 * To version this data change FAssetRegistryVersion
	 */
	COREUOBJECT_API void SerializeForCache(FArchive& Ar);
	COREUOBJECT_API void SerializeForCacheOldVersion(FArchive& Ar, FAssetRegistryVersion::Type Version);

	/** Returns the amount of memory allocated by this container, not including sizeof(*this). */
	SIZE_T GetAllocatedSize() const
	{
		return ImportedClasses.GetAllocatedSize();
	}

	/** Hash of the package's .uasset/.umap file when it was last saved by the editor. */
#if WITH_EDITORONLY_DATA
	COREUOBJECT_API FIoHash GetPackageSavedHash() const;
	COREUOBJECT_API void SetPackageSavedHash(const FIoHash& InHash);
#endif

	// Note: these functions should only be used for live communication between processes running the same version of the engine.
	// There is no versioning support
	COREUOBJECT_API void NetworkWrite(FCbWriter& Writer) const;
	COREUOBJECT_API bool TryNetworkRead(FCbFieldView Field);

	void SetPackageLocation(FPackageName::EPackageLocationFilter PackageLocation)
	{	
		// Clear existing location flags
		Flags = Flags & ~FLAG_PACKAGE_LOCATION_MASK;

		static_assert((uint8)FPackageName::EPackageLocationFilter::MaxBit <= 2, "We are storing PackageLocation in two bits");
		check(PackageLocation <= FPackageName::EPackageLocationFilter::ValidMask);
		Flags |= uint32(PackageLocation) << FLAG_PACKAGE_LOCATION_TO_ENUM_SHIFT_SIZE;
	}

	FPackageName::EPackageLocationFilter GetPackageLocation() const
	{
		return FPackageName::EPackageLocationFilter((Flags & FLAG_PACKAGE_LOCATION_MASK) >> FLAG_PACKAGE_LOCATION_TO_ENUM_SHIFT_SIZE);
	}

private:
	UE_FORCEINLINE_HINT void SerializeForCacheInternal(FArchive& Ar, FAssetPackageData& PackageData, FAssetRegistryVersion::Type Version);

	enum : uint32
	{
		FLAG_LICENSEE_VERSION						= 1u << 0,
		FLAG_HAS_VIRTUALIZED_PAYLOADS				= 1u << 1,
		FLAG_PACKAGE_READ_ONLY						= 1u << 2,

		FLAG_PACKAGE_LOCATION_IODISPATCHER			= 1u << 30,
		FLAG_PACKAGE_LOCATION_FILESYSTEM			= 1u << 31,

		FLAG_PACKAGE_LOCATION_TO_ENUM_SHIFT_SIZE	= 30u,
		FLAG_PACKAGE_LOCATION_MASK					= uint32(FPackageName::EPackageLocationFilter::ValidMask) << FLAG_PACKAGE_LOCATION_TO_ENUM_SHIFT_SIZE
	};
};


/**
 * Helper struct for FAssetIdentifier (e.g., for the FOnViewAssetIdentifiersInReferenceViewer delegate and Reference Viewer functions).
 */
#if WITH_EDITORONLY_DATA
struct FReferenceViewerParams
{
	FReferenceViewerParams()
		// Displayed-on-graph options
		: bShowReferencers(true)
		, bShowDependencies(true)
		// Slider-based options
		, FixAndHideSearchDepthLimit(0)
		, FixAndHideSearchBreadthLimit(0)
		, bShowCollectionFilter(true)
		, bShowPluginFilter(true)
		// Checkbox options
		, bShowShowReferencesOptions(true)
		, bShowShowSearchableNames(true)
		, bShowShowCodePackages(true)
		, bShowShowFilteredPackagesOnly(true)
		, bShowCompactMode(true)
	{}

	/* Whether to display the Referencers */
	bool bShowReferencers;
	/* Whether to display the Dependencies */
	bool bShowDependencies;
	/* Whether to only display the References/Dependencies which match the text filter, if any. 
	   If the optional is not set, don't change the current reference viewer's value. */
	TOptional<bool> bShowFilteredPackagesOnly;
	/** Whether to only display the References/Dependencies which match the plugin name filter */
	TArray<FName> PluginFilter;
	/* Compact mode allows to hide the thumbnail and minimize the space taken by the nodes. Useful when there are many dependencies to inspect, to keep the UI responsive. 
	   If the optional is not set, don't change the current reference viewer's value. */
	TOptional<bool> bCompactMode;
	/**
	 * Whether to visually show to the user the option of "Search Depth Limit" or hide it and fix it to a default value:
	 * - If 0 or negative, it will show to the user the option of "Search Depth Limit".
	 * - If >0, it will hide that option and fix the Depth value to this value.
	 */
	int32 FixAndHideSearchDepthLimit;
	/** Limits Search Breadth to the specified value */
	int32 FixAndHideSearchBreadthLimit;
	/** Whether to visually show to the user the option of "Collection Filter" */
	bool bShowCollectionFilter;
	/** Whether to visually show to the user the option of "Plugin Filter" */
	bool bShowPluginFilter;
	/** Whether to visually show to the user the options of "Show Soft/Hard/Management References" */
	bool bShowShowReferencesOptions;
	/** Whether to visually show to the user the option of "Show Searchable Names" */
	bool bShowShowSearchableNames;
	/** Whether to visually show to the user the option of "Show C++ Packages" */
	bool bShowShowCodePackages;
	/** Whether to visually show to the user the option of "Show Filtered Packages Only" */
	bool bShowShowFilteredPackagesOnly;
	/** Whether to visually show to the user the option of "Compact Mode" */
	bool bShowCompactMode;
};
#endif // WITH_EDITORONLY_DATA
