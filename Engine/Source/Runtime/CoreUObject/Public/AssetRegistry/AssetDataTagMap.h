// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SortedMap.h"
#include "HAL/CriticalSection.h"
#include "Misc/StringBuilder.h"
#include "Templates/RefCounting.h"
#include "Templates/TypeCompatibleBytes.h"
#include "UObject/TopLevelAssetPath.h"

class FAssetRegistryState;
class FAssetTagValueRef;
class FAssetDataTagMapSharedView;
struct FAssetRegistrySerializationOptions;

namespace FixedTagPrivate { class FMarshalledText; }
namespace FixedTagPrivate { class FStoreBuilder; }

/**
 * Helper class for condensing strings of these types into  1 - 3 FNames
 * [class]'[package].[object]'
 * [package].[object]
 * [package]
 */
struct FAssetRegistryExportPath
{
	FAssetRegistryExportPath() = default;
	FAssetRegistryExportPath(FAssetRegistryExportPath&&) = default;
	FAssetRegistryExportPath(const FAssetRegistryExportPath&) = default;
	FAssetRegistryExportPath& operator=(FAssetRegistryExportPath&&) = default;
	FAssetRegistryExportPath& operator=(const FAssetRegistryExportPath&) = default;

	explicit FAssetRegistryExportPath(FWideStringView String);
	COREUOBJECT_API explicit FAssetRegistryExportPath(FAnsiStringView String);

	FTopLevelAssetPath ClassPath;
	FName Package;
	FName Object;

	COREUOBJECT_API FString ToString() const;
	COREUOBJECT_API FName ToName() const;
	COREUOBJECT_API void ToString(FStringBuilderBase& Out) const;
	COREUOBJECT_API FString ToPath() const;
	COREUOBJECT_API void ToPath(FStringBuilderBase& Out) const;
	FTopLevelAssetPath ToTopLevelAssetPath() const
	{
		return FTopLevelAssetPath(Package, Object);
	}
	bool IsEmpty() const { return ClassPath.IsNull() & Package.IsNone() & Object.IsNone(); } //-V792
	explicit operator bool() const { return !IsEmpty(); }

	friend bool operator==(const FAssetRegistryExportPath& A, const FAssetRegistryExportPath& B);
	friend uint32 GetTypeHash(const FAssetRegistryExportPath& Export);
};

namespace FixedTagPrivate
{
	// Compact FAssetRegistryExportPath equivalent for when all FNames are numberless
	struct FNumberlessExportPath
	{
		FNameEntryId ClassPackage;
		FNameEntryId ClassObject;
		FNameEntryId Package;
		FNameEntryId Object;

		FString ToString() const;
		FName ToName() const;
		void ToString(FStringBuilderBase& Out) const;
	};

	bool operator==(const FNumberlessExportPath& A, const FNumberlessExportPath& B);
	uint32 GetTypeHash(const FNumberlessExportPath& Export);

	enum class EValueType : uint32;

	struct FValueId
	{
		static constexpr uint32 TypeBits = 3;
		static constexpr uint32 IndexBits = 32 - TypeBits;

		EValueType 		Type : TypeBits;
		uint32 			Index : IndexBits;

		uint32 ToInt() const
		{
			return static_cast<uint32>(Type) | (Index << TypeBits);
		}

		static FValueId FromInt(uint32 Int)
		{
			return { static_cast<EValueType>((Int << IndexBits) >> IndexBits), Int >> TypeBits };
		}
	};

	struct FNumberedPair
	{
		FName Key;
		FValueId Value;
	};

	struct FNumberlessPair
	{
		FDisplayNameEntryId Key;
		FValueId Value;
	};

	// Handle to a tag value owned by a managed FStore
	struct FValueHandle
	{
		uint32 StoreIndex;
		FValueId Id;

		COREUOBJECT_API FString						AsDisplayString() const;
		COREUOBJECT_API FString						AsStorageString() const;
		COREUOBJECT_API FName						AsName() const;
		COREUOBJECT_API FAssetRegistryExportPath	AsExportPath() const;
		COREUOBJECT_API bool						AsText(FText& Out) const;
		COREUOBJECT_API bool						AsMarshalledText(FMarshalledText& Out) const;
		COREUOBJECT_API bool						Equals(FStringView Str) const;
		COREUOBJECT_API bool						Contains(const TCHAR* Str) const;
		COREUOBJECT_API int64						GetResourceSize() const;

	private:
		template <bool bForStorage>
		FString						AsString() const;
	};

	// Handle to a tag map owned by a managed FStore
	struct COREUOBJECT_API alignas(uint64) FMapHandle
	{
		static constexpr uint32 StoreIndexBits = 14;

		uint16 IsValid : 1;
		uint16 HasNumberlessKeys : 1;
		uint16 StoreIndex : StoreIndexBits; // @see FStoreManager
		uint16 Num;
		uint32 PairBegin;

		const FValueId*						FindValue(FName Key) const;

		TArrayView<const FNumberedPair>		GetNumberedView() const;
		TArrayView<const FNumberlessPair>	GetNumberlessView() const;

		// Get numbered pair at an index regardless if numberless keys are used
		FNumberedPair						At(uint32 Index) const;

		friend bool operator==(FMapHandle A, FMapHandle B);

		template<typename Func>
		void ForEachPair(Func Fn) const
		{
			if (HasNumberlessKeys != 0)
			{
				for (FNumberlessPair Pair : GetNumberlessView())
				{
					Fn(FNumberedPair{Pair.Key.ToName(NAME_NO_NUMBER_INTERNAL), Pair.Value});
				}
			}
			else
			{
				for (FNumberedPair Pair : GetNumberedView())
				{
					Fn(Pair);
				}
			}
		}
	};
} // end namespace FixedTagPrivate

namespace UE::CoreUObject::Private
{
	// Base struct for FixedTagPrivate::FStore (defined in AssetDataTagMapSerializationDetails.h), containing members needed
	// by debug visualizer for FixedTagPrivate::FMapHandle.
	struct FStoreBase
	{
		// Pairs for all unsorted maps that use this store
		TArrayView<FixedTagPrivate::FNumberedPair> Pairs;
		TArrayView<FixedTagPrivate::FNumberlessPair> NumberlessPairs;

		// Values for visualizable maps in this store
		TArrayView<uint32> AnsiStringOffsets;
		TArrayView<ANSICHAR> AnsiStrings;
		TArrayView<uint32> WideStringOffsets;
		TArrayView<WIDECHAR> WideStrings;
		TArrayView<FDisplayNameEntryId> NumberlessNames;
		TArrayView<FName> Names;
	};
} // end namespace UE::CoreUObject::Private

/**
 * Reference to a tagged value in a FAssetDataTagMapSharedView
 * 
 * Helps avoid needless FString conversions when using fixed / cooked tag values
 * that are stored as FName, FText or FAssetRegistryExportPath.
 */
class FAssetTagValueRef
{
	friend class FAssetDataTagMapSharedView;

	class FFixedTagValue
	{
		static constexpr uint64 FixedMask = uint64(1) << FPlatformMemory::KernelAddressBit;
		static_assert(FixedTagPrivate::FMapHandle::StoreIndexBits <= (FPlatformMemory::KernelAddressBit - 32), 
			"Too few bits remain for the StoreIndex. Consider using other high bits but note that ARM64 use the top byte for HWASAN & MTE.)");

		uint64 Bits;

	public:
		uint64 IsFixed() const { return Bits & FixedMask; }
		uint32 GetStoreIndex() const { return static_cast<uint32>((Bits & ~FixedMask) >> 32); }
		uint32 GetValueId() const { return static_cast<uint32>(Bits); }

		FFixedTagValue() = default;
		FFixedTagValue(uint32 StoreIndex, uint32 ValueId)
		: Bits(FixedMask | (uint64(StoreIndex) << 32) | uint64(ValueId)) 
		{}
	};

	using FStringPointer = const FString*;

	union
	{
		FStringPointer Loose;
		FFixedTagValue Fixed;
		uint64 Bits = 0;
	};

	FixedTagPrivate::FValueHandle AsFixed() const;
	const FString& AsLoose() const;

public:
	FAssetTagValueRef() = default;
	FAssetTagValueRef(const FAssetTagValueRef&) = default;
	FAssetTagValueRef(FAssetTagValueRef&&) = default;
	explicit FAssetTagValueRef(const FString* Str) : Loose(Str) {}
	FAssetTagValueRef(uint32 StoreIndex, FixedTagPrivate::FValueId ValueId) : Fixed(StoreIndex, ValueId.ToInt()) {}

	FAssetTagValueRef& operator=(const FAssetTagValueRef&) = default;
	FAssetTagValueRef& operator=(FAssetTagValueRef&&) = default;

	bool										IsSet() const { return Bits != 0; }
	bool										IsFixed() const { return Fixed.IsFixed() != 0; }

	COREUOBJECT_API FString						AsString() const;
	COREUOBJECT_API FName						AsName() const;
	COREUOBJECT_API FAssetRegistryExportPath	AsExportPath() const;
	COREUOBJECT_API FText						AsText() const;
	COREUOBJECT_API bool						TryGetAsText(FText& Out) const; // @return false if value isn't a localized string

	FString										GetValue() const { return AsString(); }
	/** Coerce the type to a Complex String capable of representing the type */
	FString										GetStorageString() const { return ToLoose(); }
	/**
	 * Measure how much memory is used by the value. Does not account for deduplication, adding the results
	 * for keys sharing a duplicated value will overreport how much memory is used.
	 */
	COREUOBJECT_API int64						GetResourceSize() const;

	COREUOBJECT_API bool						Equals(FStringView Str) const;

private:
	/** Return whether this's value is a MarshalledFText, and copy it into out parameter if so */
	bool										TryGetAsMarshalledText(FixedTagPrivate::FMarshalledText& Out) const;
	/**
	 * Copy this's value (whether loose or fixed) into the loose format.
	 * The returned loose value is in StorageFormat (e.g. complex strings) rather than display format.
	 */
	COREUOBJECT_API FString						ToLoose() const;

	friend class FixedTagPrivate::FStoreBuilder;
	friend FAssetRegistryState;

	friend inline bool operator==(FAssetTagValueRef A, FStringView B) { return  A.Equals(B); }
	friend inline bool operator!=(FAssetTagValueRef A, FStringView B) { return !A.Equals(B); }
	friend inline bool operator==(FStringView A, FAssetTagValueRef B) { return  B.Equals(A); }
	friend inline bool operator!=(FStringView A, FAssetTagValueRef B) { return !B.Equals(A); }

	// These overloads can be removed when the deprecated implicit operator FString is removed
	friend inline bool operator==(FAssetTagValueRef A, const FString& B) { return  A.Equals(B); }
	friend inline bool operator!=(FAssetTagValueRef A, const FString& B) { return !A.Equals(B); }
	friend inline bool operator==(const FString& A, FAssetTagValueRef B) { return  B.Equals(A); }
	friend inline bool operator!=(const FString& A, FAssetTagValueRef B) { return !B.Equals(A); }
};

using FAssetDataTagMapBase = TSortedMap<FName, FString, FDefaultAllocator, FNameFastLess>;

/** "Loose" FName -> FString that is optionally ref-counted and owned by a FAssetDataTagMapSharedView */
class FAssetDataTagMap : public FAssetDataTagMapBase
{
	mutable FThreadSafeCounter RefCount;
	friend class FAssetDataTagMapSharedView;

public:
	FAssetDataTagMap() = default;
	FAssetDataTagMap(const FAssetDataTagMap& O) : FAssetDataTagMapBase(O) {}
	FAssetDataTagMap(FAssetDataTagMap&& O) : FAssetDataTagMapBase(MoveTemp(O)) {}
	FAssetDataTagMap& operator=(const FAssetDataTagMap& O) { return static_cast<FAssetDataTagMap&>(this->FAssetDataTagMapBase::operator=(O)); }
	FAssetDataTagMap& operator=(FAssetDataTagMap&& O) { return static_cast<FAssetDataTagMap&>(this->FAssetDataTagMapBase::operator=(MoveTemp(O))); }
};

/** Reference-counted handle to a loose FAssetDataTagMap or a fixed / immutable cooked tag map */
class FAssetDataTagMapSharedView
{
	union
	{
		FixedTagPrivate::FMapHandle Fixed;
		FAssetDataTagMap* Loose;
		uint64 Bits = 0;
	};

	bool IsFixed() const
	{
		return Fixed.IsValid;
	}

	bool IsLoose() const
	{
		return (!Fixed.IsValid) & (Loose != nullptr);
	}

	FAssetTagValueRef FindFixedValue(FName Key) const
	{
		checkSlow(IsFixed());
		const FixedTagPrivate::FValueId* Value = Fixed.FindValue(Key);
		return Value ? FAssetTagValueRef(Fixed.StoreIndex, *Value) : FAssetTagValueRef();
	}

	static TPair<FName, FAssetTagValueRef> MakePair(FixedTagPrivate::FNumberedPair FixedPair, uint32 StoreIndex)
	{
		return MakeTuple(FixedPair.Key, FAssetTagValueRef(StoreIndex, FixedPair.Value));
	}

	static TPair<FName, FAssetTagValueRef> MakePair(const FAssetDataTagMap::ElementType& LoosePair)
	{
		return MakeTuple(LoosePair.Key, FAssetTagValueRef(&LoosePair.Value));
	}

public:
	FAssetDataTagMapSharedView() = default;
	COREUOBJECT_API FAssetDataTagMapSharedView(const FAssetDataTagMapSharedView& O);
	COREUOBJECT_API FAssetDataTagMapSharedView(FAssetDataTagMapSharedView&& O);
	COREUOBJECT_API explicit FAssetDataTagMapSharedView(FixedTagPrivate::FMapHandle InFixed);
	COREUOBJECT_API explicit FAssetDataTagMapSharedView(FAssetDataTagMap&& InLoose);

	COREUOBJECT_API FAssetDataTagMapSharedView& operator=(const FAssetDataTagMapSharedView&);
	COREUOBJECT_API FAssetDataTagMapSharedView& operator=(FAssetDataTagMapSharedView&&);

	COREUOBJECT_API ~FAssetDataTagMapSharedView();

	using FFindTagResult = FAssetTagValueRef;

	/** Find a value by key and return an option indicating if it was found, and if so, what the value is. */
	FAssetTagValueRef FindTag(FName Tag) const
	{
		if (IsFixed())
		{
			return FindFixedValue(Tag);
		}

		return Loose != nullptr ? FAssetTagValueRef(Loose->Find(Tag)) : FAssetTagValueRef();
	}

	/** Return true if this map contains a specific key value pair. Value comparisons are NOT cases sensitive.*/
	bool ContainsKeyValue(FName Tag, const FString& Value) const
	{
		return FindTag(Tag).Equals(Value);
	}

	/** Determine whether a key is present in the map */
	bool Contains(FName Key) const
	{
		return FindTag(Key).IsSet();
	}

	/** Retrieve size of map */
	int32 Num() const
	{
		if (IsFixed())
		{
			return Fixed.Num;
		}

		return Loose != nullptr ? Loose->Num() : 0;
	}

	/** Copy map contents to a loose FAssetDataTagMap */
	COREUOBJECT_API FAssetDataTagMap CopyMap() const;

	template<typename Func>
	void ForEach(Func Fn) const
	{
		if (IsFixed())
		{
			Fixed.ForEachPair([&](FixedTagPrivate::FNumberedPair Pair) { Fn(MakePair(Pair, Fixed.StoreIndex)); });
		}
		else if (Loose != nullptr)
		{
			for (const FAssetDataTagMap::ElementType& Pair : *Loose)
			{
				Fn(MakePair(Pair));
			}
		}
	}

	// Note that FAssetDataTagMap isn't sorted and that order matters
	COREUOBJECT_API friend bool operator==(const FAssetDataTagMapSharedView& A, const FAssetDataTagMap& B);
	COREUOBJECT_API friend bool operator==(const FAssetDataTagMapSharedView& A, const FAssetDataTagMapSharedView& B);

	///** Shrinks the contained map */
	COREUOBJECT_API void Shrink();

	class TConstIterator
	{
	public:
		TConstIterator(const FAssetDataTagMapSharedView& InView, uint32 InIndex)
			: View(InView)
			, Index(InIndex)
		{}

		TPair<FName, FAssetTagValueRef> operator*() const
		{
			check(View.Bits != 0);
			return View.IsFixed()	? MakePair(View.Fixed.At(Index), View.Fixed.StoreIndex)
									: MakePair((&*View.Loose->begin())[Index]);
		}

		TConstIterator& operator++()
		{
			++Index;
			return *this;
		}

		bool operator!=(TConstIterator Rhs) const { return Index != Rhs.Index; }

	protected:
		const FAssetDataTagMapSharedView& View;
		uint32 Index;
	};

	class TConstIteratorWithEnd : public TConstIterator
	{
	public:
		TConstIteratorWithEnd(const FAssetDataTagMapSharedView& InView, uint32 InBeginIndex, uint32 InEndIndex)
			: TConstIterator(InView, InBeginIndex)
			, EndIndex(InEndIndex)
		{}

		explicit operator bool() const		{ return Index != EndIndex; }
		FName Key() const					{ return operator*().Key; }
		FAssetTagValueRef Value() const		{ return operator*().Value; }

	private:
		const uint32 EndIndex;
	};

	TConstIteratorWithEnd CreateConstIterator() const
	{
		return TConstIteratorWithEnd(*this, 0, Num());
	}

	/** Range for iterator access - DO NOT USE DIRECTLY */
	TConstIterator begin() const
	{
		return TConstIterator(*this, 0);
	}

	/** Range for iterator access - DO NOT USE DIRECTLY */
	TConstIterator end() const
	{
		return TConstIterator(*this, Num());
	}

	/** Helps count deduplicated memory usage */
	class FMemoryCounter
	{
		TSet<uint32> FixedStoreIndices;
		SIZE_T LooseBytes = 0;
	public:
		COREUOBJECT_API void Include(const FAssetDataTagMapSharedView& Tags);
		SIZE_T GetLooseSize() const { return LooseBytes; }
		COREUOBJECT_API SIZE_T GetFixedSize() const;
		COREUOBJECT_API void ReportFixedStoreBreakdown() const;
	};

	friend inline bool operator==(const FAssetDataTagMap& A, const FAssetDataTagMapSharedView& B)			{ return B == A; }
	friend inline bool operator!=(const FAssetDataTagMap& A, const FAssetDataTagMapSharedView& B)			{ return !(B == A); }
	friend inline bool operator!=(const FAssetDataTagMapSharedView& A, const FAssetDataTagMap& B)			{ return !(A == B); }
	friend inline bool operator!=(const FAssetDataTagMapSharedView& A, const FAssetDataTagMapSharedView& B)	{ return !(A == B); }
};
