// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetRegistrySerializationDetails.h"
#include "Async/Async.h"
#include "AutoRTFM.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/CoreMiscDefines.h"

namespace FixedTagPrivate { class FStoreManager; }
namespace FixedTagPrivate { class FWeakStorePtr; }
namespace FixedTagPrivate { struct FStore; }
struct FAssetRegistrySerializationOptions;

namespace FixedTagPrivate
{
	// Legacy version of FAssetRegistryExportPath (before FAssetRegistryVersion::ClassPaths)
	struct FLegacyAssetRegistryExportPath
	{
		FName Class;
		FName Package;
		FName Object;
	};

	/**
	 * The AssetRegistry's representation of an FText AssetData Tag value.
	 * It can be stored and copied without being interpreted as an FText.
	 */
	class FMarshalledText
	{
	public:
		FMarshalledText() = default;
		COREUOBJECT_API explicit FMarshalledText(const FUtf8String& InString);
		COREUOBJECT_API explicit FMarshalledText(const FString& InString);
		COREUOBJECT_API explicit FMarshalledText(FUtf8String&& InString);
		COREUOBJECT_API explicit FMarshalledText(const FText& InText);
		COREUOBJECT_API explicit FMarshalledText(FText&& InText);

		COREUOBJECT_API const FUtf8String& GetAsComplexString() const;
		COREUOBJECT_API FText GetAsText() const;
		COREUOBJECT_API int32 CompareToCaseIgnored(const FMarshalledText& Other) const;
		COREUOBJECT_API int64 GetResourceSize() const;

	private:
		FUtf8String String;
	};

	/**
	 * A separate allocation used by FStore that stores the normal (aka strong) refcount alongside the weak refcount,
	 * similar to the object used in TSharedPtr. When the strong refcount goes to zero, the FStore is deleted, but
	 * this FStoreRefCount lives on to provide notification to any weak pointers, until all weak pointers
	 * are dropped as well and this FStoreRefCount is deleted.
	 * 
	 * We use this custom implementation of WeakPointers rather than the general TSharedPtr/TWeakPtr solution for two
	 * reasons:
	 *    1. Independent AddRef+Release: We have references to FStores that require refcounting that come from
	 *       16-bit FAssetDataTagMapSharedView.StoreIndex that we convert to a pointer via lookup in GStores. We need
	 *       to be able to call AddRef and Release during constructor/destructor without having to hold a 64-bit
	 *       TSharedPtr, to save memory on the hundreds of thousands of FAssetDataTagMapSharedView.
	 *    2. Implementation hiding: We want to make FWeakStorePtr a public type to an implementation-hidden forward
	 *       declare of FStore, and TWeakPtr<FStore> would require a public define of FStore.
	 */
	struct FStoreRefCount
	{
	public:
		UE_NONCOPYABLE(FStoreRefCount);
		FStoreRefCount(FStore* InOwner);

		void StrongAddRef()
		{
			UE_AUTORTFM_OPEN
			{
				StrongRefCount.fetch_add(1, std::memory_order_relaxed);
			};

			UE_AUTORTFM_ONABORT(this)
			{
				StrongRefCount.fetch_add(-1, std::memory_order_relaxed);
			};
		}

		COREUOBJECT_API void StrongRelease();
		void WeakAddRef();
		void WeakRelease();
		TRefCountPtr<const FStore> Pin();

	private:
		std::atomic<int> StrongRefCount{ 0 };
		std::atomic<int> WeakRefCount{ 0 };
		FStore* Owner = nullptr;
	};

	/// Stores a fixed set of values and all the key-values maps used for lookup
	struct FStore : public UE::CoreUObject::Private::FStoreBase
	{
		// Values for non-visualizable maps in this store
		TArrayView<FNumberlessExportPath> NumberlessExportPaths;
		TArrayView<FAssetRegistryExportPath> ExportPaths;
		TArrayView<FMarshalledText> Texts;

		const uint32 Index;

#if WITH_EDITOR
		/**
		 * File where the data is kept, if any. If nullptr, the archive is not using memory-mapped files. Editor cache
		 * files usually use memory-mapped files, but in-game cooked AssetRegistry.bin data do not.
		 */
		mutable TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> BackingFile;
#endif
		/** Allocates a pointer from the heap if a view cannot use the memory-mapped data in place. */
		void* AllocateMemoryForView(SIZE_T NumBytes, SIZE_T AlignmentRequired);

		void AddRef() const
		{
			RefCountObject->StrongAddRef();
		}

		void Release() const
		{
			RefCountObject->StrongRelease();
		}
		
		const ANSICHAR* GetAnsiString(uint32 Idx) const { return &AnsiStrings[AnsiStringOffsets[Idx]]; }
		const WIDECHAR* GetWideString(uint32 Idx) const { return &WideStrings[WideStringOffsets[Idx]]; }

	private:
		friend FStoreManager;
		friend FStoreRefCount;
		friend FWeakStorePtr;

		COREUOBJECT_API explicit FStore(uint32 InIndex);
		~FStore();

		FStoreRefCount* RefCountObject = nullptr;

		/** Guards access to the View pointers. No need for an R/O as we're only accessing the set when we're about to modify it. */
		FCriticalSection ViewPointersCS;
		/** Pointers to individual views that aren't mmapped but allocated from the heap */
		TSet<void*> ViewPointers;
	};

	struct FOptions
	{
		TSet<FName> StoreAsName;
		TSet<FName> StoreAsPath;
	};

	// Incomplete handle to a map in an unspecified FStore.
	// Used for serialization where the store index is implicit.
	struct COREUOBJECT_API alignas(uint64) FPartialMapHandle
	{
		bool bHasNumberlessKeys = false;
		uint16 Num = 0;
		uint32 PairBegin = 0;

		FMapHandle MakeFullHandle(uint32 StoreIndex) const;
		uint64 ToInt() const;
		static FPartialMapHandle FromInt(uint64 Int);
	};

	// Note: Can be changed to a single allocation and array views to improve cooker performance
	struct FStoreData
	{
		TArray<FNumberedPair> Pairs;
		TArray<FNumberlessPair> NumberlessPairs;

		TArray<uint32> AnsiStringOffsets;
		TArray<ANSICHAR> AnsiStrings;
		TArray<uint32> WideStringOffsets;
		TArray<WIDECHAR> WideStrings;
		TArray<FDisplayNameEntryId> NumberlessNames;
		TArray<FName> Names;
		TArray<FNumberlessExportPath> NumberlessExportPaths;
		TArray<FAssetRegistryExportPath> ExportPaths;
		TArray<FMarshalledText> Texts;
	};

	uint32 HashCaseSensitive(const TCHAR* Str, int32 Len);
	uint32 HashCaseSensitive(const UTF8CHAR* Str, int32 Len);
	uint32 HashCombineQuick(uint32 A, uint32 B);
	uint32 HashCombineQuick(uint32 A, uint32 B, uint32 C);

	// Helper class for saving or constructing an FStore
	class FStoreBuilder
	{
	public:
		explicit FStoreBuilder(const FOptions& InOptions) : Options(InOptions) {}
		explicit FStoreBuilder(FOptions&& InOptions) : Options(MoveTemp(InOptions)) {}

		COREUOBJECT_API FPartialMapHandle AddTagMap(const FAssetDataTagMapSharedView& Map);

		// Call once after all tag maps have been added
		COREUOBJECT_API FStoreData Finalize();

	private:

		template <typename ValueType>
		struct FCaseSensitiveFuncs : BaseKeyFuncs<ValueType, FString, /*bInAllowDuplicateKeys*/ false>
		{
			template<typename KeyType>
			static const KeyType& GetSetKey(const TPair<KeyType, ValueType>& Element)
			{
				return Element.Key;
			}

			static bool Matches(const FString& A, const FString& B)
			{
				return A.Equals(B, ESearchCase::CaseSensitive);
			}
			static bool Matches(const FUtf8String& A, const FUtf8String& B)
			{
				return A.Equals(B, ESearchCase::CaseSensitive);
			}
			static uint32 GetKeyHash(const FString& Key)
			{
				return HashCaseSensitive(GetData(Key), Key.Len());
			}
			static uint32 GetKeyHash(const FUtf8String& Key)
			{
				return HashCaseSensitive(GetData(Key), Key.Len());
			}

			static bool Matches(FNameEntryId A, FNameEntryId B)
			{
				return A == B;
			}
			static uint32 GetKeyHash(FNameEntryId Key)
			{
				return GetTypeHash(Key);
			}

			static bool Matches(FName A, FName B)
			{
				return (A.GetDisplayIndex() == B.GetDisplayIndex()) & (A.GetNumber() == B.GetNumber());
			}
			static uint32 GetKeyHash(FName Key)
			{
				return HashCombineQuick(GetTypeHash(Key.GetDisplayIndex()), Key.GetNumber());
			}

			static bool Matches(const FNumberlessExportPath& A, const FNumberlessExportPath& B)
			{
				return Matches(A.ClassPackage, B.ClassPackage) & Matches(A.ClassObject, B.ClassObject) & Matches(A.Package, B.Package) & Matches(A.Object, B.Object); //-V792
			}
			static bool Matches(const FAssetRegistryExportPath& A, const FAssetRegistryExportPath& B)
			{
				return Matches(A.ClassPath.GetPackageName(), B.ClassPath.GetPackageName()) &  Matches(A.ClassPath.GetAssetName(), B.ClassPath.GetAssetName()) & Matches(A.Package, B.Package) & Matches(A.Object, B.Object); //-V792
			}

			static uint32 GetKeyHash(const FNumberlessExportPath& Key)
			{
				return HashCombineQuick(HashCombineQuick(GetKeyHash(Key.ClassPackage), GetKeyHash(Key.ClassObject)), GetKeyHash(Key.Package), GetKeyHash(Key.Object));
			}
			static uint32 GetKeyHash(const FAssetRegistryExportPath& Key)
			{
				return HashCombineQuick(HashCombineQuick(GetKeyHash(Key.ClassPath.GetPackageName()), GetKeyHash(Key.ClassPath.GetAssetName())), GetKeyHash(Key.Package), GetKeyHash(Key.Object));
			}

			static bool Matches(const FMarshalledText& A, const FMarshalledText& B)
			{
				return Matches(A.GetAsComplexString(), B.GetAsComplexString());
			}

			static uint32 GetKeyHash(const FMarshalledText& Key)
			{
				return GetKeyHash(Key.GetAsComplexString());
			}
		};

		struct FStringIndexer
		{
			uint32 NumCharacters = 0;
			TMap<FString, uint32, FDefaultSetAllocator, FCaseSensitiveFuncs<uint32>> StringIndices;
			TArray<uint32> Offsets;

			uint32 Index(FString&& String);

			TArray<ANSICHAR> FlattenAsAnsi() const;
			TArray<WIDECHAR> FlattenAsWide() const;
		};

		const FOptions Options;
		FStringIndexer AnsiStrings;
		FStringIndexer WideStrings;
		TMap<FDisplayNameEntryId, uint32> NumberlessNameIndices;
		TMap<FName, uint32, FDefaultSetAllocator, FCaseSensitiveFuncs<uint32>> NameIndices;
		TMap<FNumberlessExportPath, uint32, FDefaultSetAllocator, FCaseSensitiveFuncs<uint32>> NumberlessExportPathIndices;
		TMap<FAssetRegistryExportPath, uint32, FDefaultSetAllocator, FCaseSensitiveFuncs<uint32>> ExportPathIndices;
		TMap<FMarshalledText, uint32, FDefaultSetAllocator, FCaseSensitiveFuncs<uint32>> TextIndices;

		TArray<FNumberedPair> NumberedPairs;
		TArray<FNumberedPair> NumberlessPairs; // Stored as numbered for convenience

		bool bFinalized = false;

		FValueId IndexValue(FName Key, FAssetTagValueRef Value);
	};

	enum class ELoadOrder { Member, TextFirst };

	COREUOBJECT_API void SaveStore(const FStoreData& Store, FArchive& Ar);
	COREUOBJECT_API TRefCountPtr<const FStore> LoadStore(FArchive& Ar, FAssetRegistryVersion::Type Version = FAssetRegistryVersion::LatestVersion, TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> InBackingFile = nullptr);
	/** Advance the offset of Ar to skip over the bytes for an FStore that starts at its current offset. */
	COREUOBJECT_API void SkipBytesStore(FArchive& Ar, FAssetRegistryVersion::Type Version = FAssetRegistryVersion::LatestVersion);

	/// Loads tag store with async creation of expensive tag values
	///
	/// Caller should:
	/// * Call ReadInitialDataAndKickLoad()
	/// * Call LoadFinalData()
	/// * Wait for future before resolving stored tag values
	class FAsyncStoreLoader
	{
	public:
		COREUOBJECT_API FAsyncStoreLoader();

		/// 1) Read initial data and kick expensive tag value creation task
		///
		/// Won't load FNames to allow concurrent name batch loading
		/// 
		/// @return handle to step 3
		COREUOBJECT_API TFuture<void> ReadInitialDataAndKickLoad(FArchive& Ar, uint32 MaxWorkerTasks, FAssetRegistryVersion::Type HeaderVersion);

		/// 2) Read remaining data, including FNames
		///
		/// @return indexed store, usable for FPartialMapHandle::MakeFullHandle()
		COREUOBJECT_API TRefCountPtr<const FStore> LoadFinalData(FArchive& Ar, FAssetRegistryVersion::Type HeaderVersion);

		/** Passes backing file for memory mapping to the store. */
		inline void SetStoreBackingFile(TSharedPtr<UE::AssetRegistry::FMemoryMappedFile> InBackingFile)
		{
#if WITH_EDITOR
			Store->BackingFile = MoveTemp(InBackingFile);
#else
			if (InBackingFile)
			{
				UE_LOGF(LogAssetData, Error, "Reading asset registry from a memory mapped file is not supported in a non-editor binary.");
			}
#endif
		}

	private:
		TRefCountPtr<FStore> Store;
		TOptional<ELoadOrder> Order;
	};

	// Initializes a managed FStore from an unfinalized builder.
	COREUOBJECT_API TRefCountPtr<const FStore> InitializeFromBuilder(FStoreBuilder& InBuilder);

} // end namespace FixedTagPrivate
