// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetData.h"
#include "Logging/TokenizedMessage.h"

/**
 * A Message Log token that links to an asset, allowing a hyperlink to navigate to an asset in the content browser or an actor in a level.
 */
class FAssetDataToken : public IMessageToken
{
	// The private token allows only members or friends to call MakeShared.
	struct FPrivateToken { explicit FPrivateToken() = default; };
public:
	/** Factory method, tokens can only be constructed as shared refs */
	COREUOBJECT_API static TSharedRef<FAssetDataToken> Create(const FAssetData& AssetData, const FText& InLabelOverride = FText());

	/** Private constructor (via FPrivateToken argument) */
	COREUOBJECT_API FAssetDataToken(FPrivateToken, const FAssetData& InAssetData, const FText& InLabelOverride);

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::AssetData;
	}

	virtual const FOnMessageTokenActivated& GetOnMessageTokenActivated() const override;
	/** End IMessageToken interface */

    const FAssetData& GetAssetData() const
    {
        return AssetData;
    }

	/** Get the delegate for default token activation */
	static FOnMessageTokenActivated& DefaultOnMessageTokenActivated()
	{
		return DefaultMessageTokenActivated;
	}

	/** Get the delegate for displaying the asset name */
	DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnGetDisplayName, const FAssetData&, bool /* Full path */);
	static FOnGetDisplayName& DefaultOnGetAssetDisplayName()
	{
		return DefaultGetAssetDisplayName;
	}

private:

	/** The asset data this token was created for */
	FAssetData AssetData;

	/** The default activation method, if any. Usually populated by another module in the engine. */
	COREUOBJECT_API static FOnMessageTokenActivated DefaultMessageTokenActivated;

	/** The default method for getting a display name for assets, if any. Usually populated by another module in the ending. */
	COREUOBJECT_API static FOnGetDisplayName DefaultGetAssetDisplayName;
};
