// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/PrimaryAssetId.h"

class FCbFieldView;
class FCbWriter;

/** A structure defining a thing that can be reference by something else in the asset registry. Represents either a package of a primary asset id */
struct FAssetIdentifier
{
	/** The name of the package that is depended on, this is always set unless PrimaryAssetType is */
	FName PackageName;
	/** The primary asset type, if valid the ObjectName is the PrimaryAssetName */
	FPrimaryAssetType PrimaryAssetType;
	/** Specific object within a package. If empty, assumed to be the default asset */
	FName ObjectName;
	/** Name of specific value being referenced, if ObjectName specifies a type such as a UStruct */
	FName ValueName;

	/** Can be implicitly constructed from just the package name */
	FAssetIdentifier(FName InPackageName, FName InObjectName = FName(), FName InValueName = FName())
		: PackageName(InPackageName), PrimaryAssetType(), ObjectName(InObjectName), ValueName(InValueName)
	{}

	/** Construct from a primary asset id */
	FAssetIdentifier(const FPrimaryAssetId& PrimaryAssetId, FName InValueName = FName())
		: PackageName(), PrimaryAssetType(PrimaryAssetId.PrimaryAssetType), ObjectName(PrimaryAssetId.PrimaryAssetName), ValueName(InValueName)
	{}

	COREUOBJECT_API FAssetIdentifier(UObject* SourceObject, FName InValueName);

	FAssetIdentifier() {}

	/** Returns primary asset id for this identifier, if valid */
	FPrimaryAssetId GetPrimaryAssetId() const
	{
		if (PrimaryAssetType.IsValid())
		{
			return FPrimaryAssetId(PrimaryAssetType, ObjectName);
		}
		return FPrimaryAssetId();
	}

	/** Returns true if this represents a package */
	bool IsPackage() const
	{
		return PackageName != NAME_None && !IsObject() && !IsValue();
	}

	/** Returns true if this represents an object, true for both package objects and PrimaryAssetId objects */
	bool IsObject() const
	{
		return ObjectName != NAME_None && !IsValue();
	}

	/** Returns true if this represents a specific value */
	bool IsValue() const
	{
		return ValueName != NAME_None;
	}

	/** Returns true if this is a valid non-null identifier */
	bool IsValid() const
	{
		return PackageName != NAME_None || GetPrimaryAssetId().IsValid();
	}

	/** Returns string version of this identifier in Package.Object::Name format */
	FString ToString() const
	{
		TStringBuilder<256> Builder;
		AppendString(Builder);
		return FString::ConstructFromPtrSize(Builder.GetData(), Builder.Len());
	}

	/** Appends to the given builder the string version of this identifier in Package.Object::Name format */
	void AppendString(FStringBuilderBase& Builder) const
	{
		if (PrimaryAssetType.IsValid())
		{
			GetPrimaryAssetId().AppendString(Builder);
		}
		else
		{
			PackageName.AppendString(Builder);
			if (ObjectName != NAME_None)
			{
				Builder.Append(TEXT("."));
				ObjectName.AppendString(Builder);
			}
		}
		if (ValueName != NAME_None)
		{
			Builder.Append(TEXT("::"));
			ValueName.AppendString(Builder);
		}
	}

	/** Converts from Package.Object::Name format */
	static FAssetIdentifier FromString(const FString& String)
	{
		// To right of :: is value
		FString PackageString;
		FString ObjectString;
		FString ValueString;

		// Try to split value out
		if (!String.Split(TEXT("::"), &PackageString, &ValueString))
		{
			PackageString = String;
		}

		// Check if it's a valid primary asset id
		FPrimaryAssetId PrimaryId = FPrimaryAssetId::FromString(PackageString);

		if (PrimaryId.IsValid())
		{
			return FAssetIdentifier(PrimaryId, *ValueString);
		}

		// Try to split on first . , if it fails PackageString will stay the same
		FString(PackageString).Split(TEXT("."), &PackageString, &ObjectString);

		return FAssetIdentifier(*PackageString, *ObjectString, *ValueString);
	}

	friend inline bool operator==(const FAssetIdentifier& A, const FAssetIdentifier& B)
	{
		return A.PackageName == B.PackageName && A.PrimaryAssetType == B.PrimaryAssetType
			&& A.ObjectName == B.ObjectName && A.ValueName == B.ValueName;
	}

	friend inline uint32 GetTypeHash(const FAssetIdentifier& Key)
	{
		uint32 Hash = 0;

		// Most of the time only packagename is set
		if (Key.ObjectName.IsNone() && Key.ValueName.IsNone())
		{
			return GetTypeHash(Key.PackageName);
		}

		Hash = HashCombine(Hash, GetTypeHash(Key.PackageName));
		Hash = HashCombine(Hash, GetTypeHash(Key.PrimaryAssetType));
		Hash = HashCombine(Hash, GetTypeHash(Key.ObjectName));
		Hash = HashCombine(Hash, GetTypeHash(Key.ValueName));
		return Hash;
	}

	/** Identifiers may be serialized as part of the registry cache, or in other contexts. If you make changes here you must also change FAssetRegistryVersion */
	friend FArchive& operator<<(FArchive& Ar, FAssetIdentifier& AssetIdentifier)
	{
		// Serialize bitfield of which elements to serialize, in general many are empty
		uint8 FieldBits = 0;

		if (Ar.IsSaving())
		{
			FieldBits |= (AssetIdentifier.PackageName != NAME_None) << 0;
			FieldBits |= (AssetIdentifier.PrimaryAssetType.IsValid()) << 1;
			FieldBits |= (AssetIdentifier.ObjectName != NAME_None) << 2;
			FieldBits |= (AssetIdentifier.ValueName != NAME_None) << 3;
		}

		Ar << FieldBits;

		if (FieldBits & (1 << 0))
		{
			Ar << AssetIdentifier.PackageName;
		}
		if (FieldBits & (1 << 1))
		{
			FName TypeName = AssetIdentifier.PrimaryAssetType.GetName();
			Ar << TypeName;

			if (Ar.IsLoading())
			{
				AssetIdentifier.PrimaryAssetType = TypeName;
			}
		}
		if (FieldBits & (1 << 2))
		{
			Ar << AssetIdentifier.ObjectName;
		}
		if (FieldBits & (1 << 3))
		{
			Ar << AssetIdentifier.ValueName;
		}

		return Ar;
	}

	bool LexicalLess(const FAssetIdentifier& Other) const
	{
		if (PrimaryAssetType != Other.PrimaryAssetType)
		{
			return PrimaryAssetType.LexicalLess(Other.PrimaryAssetType);
		}
		if (PackageName != Other.PackageName)
		{
			return PackageName.LexicalLess(Other.PackageName);
		}
		if (ObjectName != Other.ObjectName)
		{
			return ObjectName.LexicalLess(Other.ObjectName);
		}
		return ValueName.LexicalLess(Other.ValueName);
	}

	bool FastLess(const FAssetIdentifier& Other) const
	{
		if (PrimaryAssetType != Other.PrimaryAssetType)
		{
			return PrimaryAssetType.FastLess(Other.PrimaryAssetType);
		}
		if (PackageName != Other.PackageName)
		{
			return PackageName.FastLess(Other.PackageName);
		}
		if (ObjectName != Other.ObjectName)
		{
			return ObjectName.FastLess(Other.ObjectName);
		}
		return ValueName.FastLess(Other.ValueName);
	}

	COREUOBJECT_API void WriteCompactBinary(FCbWriter& Writer) const;
private:
	friend FCbWriter& operator<<(FCbWriter& Writer, const FAssetIdentifier& Identifier)
	{
		// Hidden friend function needs to be inline, but call a subfunction to hide the implementation
		Identifier.WriteCompactBinary(Writer);
		return Writer;
	}
	// Load Cannot be inline because we need to hide implementation and copy-by-value is invalid without definition
	COREUOBJECT_API friend bool LoadFromCompactBinary(FCbFieldView Field, FAssetIdentifier& Identifier);
	COREUOBJECT_API friend void SerializeForLog(FCbWriter& Writer, const FAssetIdentifier& Value);
};

