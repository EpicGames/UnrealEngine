// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/ArchiveProxy.h"
#include "Async/MappedFileHandle.h"
#include "Memory/MemoryFwd.h"
#include "Templates/SharedPointer.h"
#include "Serialization/MemoryHasher.h"

namespace UE::AssetRegistry
{
	/** Memory maps the bytes of a file on disk into memory and provides MemoryViews to those bytes until *this is destructed. */
	class FMemoryMappedFile
	{
	public:
		FMemoryMappedFile() = default;
		COREUOBJECT_API explicit FMemoryMappedFile(const TCHAR* Path);

		inline FMemoryView GetWholeFileView() const
		{
			return GetView(0, MAX_int64);
		}

		/** Provides a new View, allocating it if it doesn't exist. */
		COREUOBJECT_API FMemoryView GetView(uint64 Start, uint64 Length) const;

	private:

		/** Handle for the mapped file */
		TUniquePtr<IMappedFileHandle> Handle;

		struct FMappedSubRegion
		{
			uint64 Start;
			uint64 Length;

			TUniquePtr<IMappedFileRegion> Region;
		};

		mutable TArray<FMappedSubRegion>  Regions;
	};

	inline constexpr int64 AlignmentForMemoryMappedArchive = 16;

	/** When saving, saves extra memory to make sure the following data is aligned. When loading, seeks over that many bytes (does not load them). */
	void COREUOBJECT_API AlignPosInArchive(FArchive& Archive);

	/** Reports whether the archive's current offset is aligned, relative to offset 0. */
	inline bool IsPosInArchiveAligned(FArchive& Archive)
	{
		return IsAligned(Archive.Tell(), AlignmentForMemoryMappedArchive);
	}
}
