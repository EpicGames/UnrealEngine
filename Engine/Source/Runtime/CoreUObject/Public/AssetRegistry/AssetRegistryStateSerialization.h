// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

namespace FixedTagPrivate { struct FStore; }
namespace FixedTagPrivate { struct FStoreRefCount; }
template<typename T> class TRefCountPtr;

namespace FixedTagPrivate
{

/**
 * A weak pointer to an FStore. When the normal refcount (e.g. via TRefCountPtr) on the FStore goes to zero, the
 * FStore is deleted and calls to Pin on this weak pointer will return null. FStore is a private implementation
 * detail; this class is public only to pass the opaque type between different Load functions.
 */
class FWeakStorePtr
{
public:
	inline FWeakStorePtr() = default;
	COREUOBJECT_API FWeakStorePtr(const TRefCountPtr<const FStore>& Store);
	COREUOBJECT_API FWeakStorePtr(const FWeakStorePtr& Other);
	COREUOBJECT_API FWeakStorePtr(FWeakStorePtr&& Other);
	COREUOBJECT_API FWeakStorePtr& operator=(const FWeakStorePtr& Other);
	COREUOBJECT_API FWeakStorePtr& operator=(FWeakStorePtr&& Other);
	COREUOBJECT_API ~FWeakStorePtr();

	COREUOBJECT_API TRefCountPtr<const FStore> Pin() const;
private:
	mutable FStoreRefCount* RefCountObject = nullptr;
};

} // namespace FixedTagPrivate
