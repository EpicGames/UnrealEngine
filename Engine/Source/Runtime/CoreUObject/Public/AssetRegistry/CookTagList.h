// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"

/**
*	Used for passing key-value pairs to the asset registry during serialization during cook,
*	and is accessed off of FArchive::CookContext()
* 
*	This duplicates functionality in GetAssetRegistryTags, but it exists because some tags are
*	expensive to calculate and the calculation overlaps with other calculations done during serialize.
*
*	All tags added to this list have "Cook_" prepended to their names prior to being
*	added to the asset registry.
*
*	Cook tags are *only* added to the development asset registry, and will not
*	show up in the shipped runtime registry.
*
*	Cook tags are only generated during Cook By The Book.
*
*	Example:
* 
*	// Cook context is present for all cooks, but the tag list is only for CBTB.
*	if (Ar.CookContext() && Ar.CookContext()->CookTagList())
*	{
*		FCookTagList* CookTags = Ar.CookContext()->CookTagList();
*		CookTags->Add(Object, "TagName", FString("TagValue"));
*	}
*/
struct FCookTagList
{
	UPackage* Package;
	using FTagNameValuePair = TPair<FName, FString>;
	TMap<UObject*, TArray<FTagNameValuePair>> ObjectToTags;

	FCookTagList(UPackage* InPackage) : Package(InPackage) {}

	/**
	*	Adds a tag to the list to be added to the development asset registry
	*	for this object. Note that we only carry over the name/value, not
	*	any of the other metadata available when adding tags via GetAssetRegistryTags.
	* 
	*	Tags that start with "Diff_" are used for bulk data diff blaming - see DiffAssetBulkDataCommandlet.cpp. When
	* 	adding diff tags, be sure to add explanatory prose in that commandlet to GBuiltinDiffTagHelp.
	*/
	void Add(UObject* InObject, FName InTagName, FString&& InTagValue)
	{
		if (!Package || !InObject || !ensure(InObject->GetOutermost() == Package))
		{
			return;
		}

		if (InObject->IsAsset())
		{
			TArray<FTagNameValuePair>& ObjectTags = ObjectToTags.FindOrAdd(InObject);
			ObjectTags.Add(FTagNameValuePair(InTagName, InTagValue));
		}
	}

	void Reset()
	{
		ObjectToTags.Reset();
	}
};
