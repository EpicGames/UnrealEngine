// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Misc/Optional.h"

// Kinds of Blueprint exceptions
namespace EBlueprintExceptionType
{
	enum Type
	{
		Breakpoint,
		Tracepoint,
		WireTracepoint,
		AccessViolation,
		InfiniteLoop,
		NonFatalError,
		FatalError,
		AbortExecution,
		UserRaisedError,
	};
}

// Information about a blueprint exception
struct FBlueprintExceptionInfo
{
public:
	FBlueprintExceptionInfo(EBlueprintExceptionType::Type InEventType)
		: EventType(InEventType)
	{
	}

	FBlueprintExceptionInfo(EBlueprintExceptionType::Type InEventType, const FText& InDescription)
		: EventType(InEventType)
		, Description(InDescription)
	{
	}

	EBlueprintExceptionType::Type GetType() const
	{
		return EventType;
	}

	const FText& GetDescription() const
	{
		if (!Description.IsSet())
		{
			Description = FText();
		}
		return Description.GetValue();
	}

private:
	EBlueprintExceptionType::Type EventType;
	
	// We use TOptional<FText> here as an optimization for the common case of an exception with no text.
	// For instance, every tracepoint in the code creates a FBlueprintExceptionInfo with no description.
	// Constructing an empty FText will take a reference on the shared FText::GetEmpty() string, which
	// is especially expensive in AutoRTFM because we need to track reference-count updates in order to
	// undo them. If the caller actually inspects the empty description, we synthesize it on demand.
	mutable TOptional<FText> Description;
};
