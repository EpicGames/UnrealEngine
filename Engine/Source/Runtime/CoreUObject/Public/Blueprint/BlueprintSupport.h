// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/ThreadSingleton.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

class FObjectInitializer;
class UClass;
class UObject;
class UPackage;
class UScriptStruct;
class UStruct;
struct FUObjectSerializeContext;
class ITargetPlatform;

/**
 * List of asset registry tags used by blueprints. These are here so they can be used by both the asset registry and blueprint code
 * These need to be kept in sync with UBlueprint::GetAssetRegistryTags, and any changes there will require resaving content
 */
struct FBlueprintTags
{
	/** Full path in export form ClassType'/PackagePath/PackageName.ClassName' of generated blueprint class */
	static COREUOBJECT_API const FName GeneratedClassPath;
	/** Full path in export form ClassType'/PackagePath/PackageName.ClassName' of the immediate parent, may be a blueprint or native class */
	static COREUOBJECT_API const FName ParentClassPath;
	/** Full path in export form Class'/Script/ModuleName.ClassName' of the first found parent native class */
	static COREUOBJECT_API const FName NativeParentClassPath;
	/** Integer representing bitfield EClassFlags */
	static COREUOBJECT_API const FName ClassFlags;
	/** String representing enum EBlueprintType */
	static COREUOBJECT_API const FName BlueprintType;
	/** String with user-entered description of blueprint */
	static COREUOBJECT_API const FName BlueprintDescription;
	/** String with user-entered display name for the blueprint class (used in editor along with the description to identify the Blueprint type) */
	static COREUOBJECT_API const FName BlueprintDisplayName;
	/** String with user-entered category for the blueprint */
	static COREUOBJECT_API const FName BlueprintCategory;
	/** String set to True/False, set if this is a data only blueprint */
	static COREUOBJECT_API const FName IsDataOnly;
	/** List of implemented interfaces, must be converted to FBPInterfaceDescription */
	static COREUOBJECT_API const FName ImplementedInterfaces;
	/** Very large string used to store find in blueprint data for the editor */
	static COREUOBJECT_API const FName FindInBlueprintsData;
	/** (Deprecated) Legacy tag that was initially used to store find in blueprint data for the editor */
	static COREUOBJECT_API const FName UnversionedFindInBlueprintsData;
	/** Number of replicated properties */
	static COREUOBJECT_API const FName NumReplicatedProperties;
	/** Number of native components */
	static COREUOBJECT_API const FName NumNativeComponents;
	/** Number of blueprint components */
	static COREUOBJECT_API const FName NumBlueprintComponents;
	/** The subpath of a blueprint contained within the asset. Used to determine whether, and where a blueprint exists in a package. */
	static COREUOBJECT_API const FName BlueprintPathWithinPackage;
};

struct FBlueprintWarningDeclaration
{
	FBlueprintWarningDeclaration(FName InWarningIdentifier, FText InWarningDescription)
		: WarningIdentifier(InWarningIdentifier)
		, WarningDescription( InWarningDescription )
	{
	}

	FName WarningIdentifier;
	FText WarningDescription;
};

typedef void (*FFlushReinstancingQueueFPtr)();
typedef void (*FFlushCompilationQueueFPtr)(FUObjectSerializeContext*);
typedef void (*FClassReparentingFPtr)(const TMap<UClass*, UClass*>&);

/** 
 * This set of functions contains blueprint related UObject functionality.
 */
struct FBlueprintSupport
{
	/** 
	 * Defined in BlueprintSupport.cpp
	 * Duplicates all fields of a struct in depth-first order. It makes sure that everything contained
	 * in a class is duplicated before the struct itself, as well as all function parameters before the
	 * function itself.
	 *
	 * @param	StructToDuplicate			Instance of the struct that is about to be duplicated
	 * @param	Writer						duplicate writer instance to write the duplicated data to
	 */
	static void DuplicateAllFields(class UStruct* StructToDuplicate, class FDuplicateDataWriter& Writer);

	/** 
	 * A series of query functions that we can use to easily gate-off/disable 
	 * aspects of the deferred loading (mostly for testing purposes). 
	 */
	static bool UseDeferredDependencyLoading();
	static bool IsDeferredExportCreationDisabled();
	static bool IsDeferredCDOInitializationDisabled();

	/** Checks for any old instances and reinstances them: */
	static void FlushReinstancingQueue();
	/** Flushes all queued compilation requests */
	static void FlushCompilationQueue(FUObjectSerializeContext* InLoadContext = nullptr);
	COREUOBJECT_API static void SetFlushReinstancingQueueFPtr(FFlushReinstancingQueueFPtr Ptr);	
	COREUOBJECT_API static void SetFlushCompilationQueueFPtr(FFlushCompilationQueueFPtr Ptr);

	COREUOBJECT_API static void ReparentHierarchies(const TMap<UClass*, UClass*>& OldClassToNewClass);
	COREUOBJECT_API static void SetClassReparentingFPtr(FClassReparentingFPtr Ptr);

	/** Tells if the specified object is one of the many flavors of FLinkerPlaceholderBase that we have. */
	COREUOBJECT_API static bool IsDeferredDependencyPlaceholder(const UObject* LoadedObj);

#if WITH_EDITOR
	/** Registers any object properties in this struct with the deferred dependency system */
	COREUOBJECT_API static void RegisterDeferredDependenciesInStruct(const UStruct* Struct, void* StructData);
#endif

	/** Repair function to scan for any deferred dependency placeholders and attempt to resolve them in-place with real assets */
	COREUOBJECT_API static void RepairDeferredDependenciesInObject(UObject* Object);

	/** Not a particularly fast function. Mostly intended for validation in debug builds. */
	static bool IsInBlueprintPackage(UObject* LoadedObj);

	COREUOBJECT_API static void RegisterBlueprintWarning(const FBlueprintWarningDeclaration& Warning);
	COREUOBJECT_API static const TArray<FBlueprintWarningDeclaration>& GetBlueprintWarnings();
	COREUOBJECT_API static void UpdateWarningBehavior(const TArray<FName>& WarningIdentifiersToTreatAsError, const TArray<FName>& WarningIdentifiersToSuppress);
	COREUOBJECT_API static bool ShouldTreatWarningAsError(FName WarningIdentifier);
	COREUOBJECT_API static bool ShouldSuppressWarning(FName WarningIdentifier);

	COREUOBJECT_API static bool IsClassPlaceholder(const UClass* Class);

#if WITH_EDITOR
	/** Function that walks the object graph, ensuring that there are no references to TRASH or REINST classes: */
	COREUOBJECT_API static void ValidateNoRefsToOutOfDateClasses();

	/** Function that walks the object graph, ensuring that there are no references to SKEL classes: */
	COREUOBJECT_API static void ValidateNoExternalRefsToSkeletons();
#endif
};

/**
 * When dealing with user defined structs we don't always have a UObject container
 * this registers raw addresses for tracking. This is somewhat less safe, make
 * sure to not register addresses that may change
 */
struct FScopedPlaceholderRawContainerTracker
{
public:
	COREUOBJECT_API FScopedPlaceholderRawContainerTracker(void* InData);
	COREUOBJECT_API ~FScopedPlaceholderRawContainerTracker();

private:
	void* Data;
};

#if WITH_EDITOR
/**
 * This is a helper struct that allows us to gather all previously unloaded class dependencies of a UClass
 * The first time we create a new UClass object in FLinkerLoad::CreateExport(), we register it as a dependency
 * authority. Any subsequent UClasses that are created for the first time during the preload of that class are
 * added to the list as potential cyclic referencers.  We then step over the list at the end of the load, and
 * recompile any classes that may depend on each other a second time to ensure that that functions and properties
 * are properly resolved
 */
struct FScopedClassDependencyGather
{
public:
	COREUOBJECT_API FScopedClassDependencyGather(UClass* ClassToGather, FUObjectSerializeContext* InLoadContext);
	COREUOBJECT_API ~FScopedClassDependencyGather();

	/**
	 * Post load, some systems would like an easy list of dependencies. This will
	 * retrieve that latest BatchClassDependencies (filled with dependencies from
	 * the last loaded class).
	 * 
	 * @return The most recent array of tracked dependencies.
	 */
	static COREUOBJECT_API TArray<UClass*> const& GetCachedDependencies();

	/**
	 * Returns the current authoratitative scope on the stack if one exist.
	 */
	static COREUOBJECT_API FScopedClassDependencyGather* GetAuthoritativeScope();

	DECLARE_TS_MULTICAST_DELEGATE(FOnScopeExit);

	/**
	 * Delegate called when the scope exits;
	 */
	FOnScopeExit OnScopeExitDelegate;
private:
	/** Whether or not this dependency gather is the dependency authoritative class, and thus should process all dependencies in the destructor */
	bool bAuthoritativeClass;

	/** The current class that is gathering potential dependencies in this scope */
	static thread_local UClass* BatchAuthorityClass;

	/** List of dependencies (i.e. UClasses that have been newly instantiated) in the scope of this dependency gather */
	static thread_local TArray<UClass*> BatchClassDependencies;

	/** Contains the authoritative scope on the current thread */
	static thread_local FScopedClassDependencyGather* AuthoritativeScope;

	/** Current load context */
	FUObjectSerializeContext* LoadContext;

	FScopedClassDependencyGather();
};

#endif // WITH_EDITOR

/** 
 * A base struct for storing FObjectInitializers that were not run on 
 * Blueprint objects post-construction (presumably because the object's super/archetype
 * had not been fully serialized yet). 
 * 
 * This was designed to hold onto FObjectInitializers until a later point, when 
 * they can properly be ran (after the archetype has been serialized).
 */
struct FDeferredInitializationTrackerBase
{
public:
	virtual ~FDeferredInitializationTrackerBase() = default;

	/** 
	 * Makes a copy of the specified initializer and stores it (mapped under its 
	 * dependency), so that it can instead be executed later via ResolveArchetypeInstances().
	 * 
	 * @param  InitDependency			The object (usually the initializer's archetype) that this initializer is dependent on. The key 
	 *									you'll pass to ResolveArchetypeInstances() later to run this initializer.
	 * @param  DeferringInitializer		The initializer you want to defer.
	 *
	 * @raturn A copy of the specified initializer. This should always succeed (only returns null if InitDependency is null).
	 */
	FObjectInitializer* Add(const UObject* InitDependency, const FObjectInitializer& DeferringInitializer);

	/** 
	 * Runs all deferred initializers that were dependent on the specified archetype 
	 * (unless they're dependent on another), and runs Preload() on any objects that 
	 * had their Preload() skipped as a result.
	 * 
	 * @param  ArchetypeKey		The initializer dependency that has been fully loaded/serialized.
	 */
	void ResolveArchetypeInstances(UObject* ArchetypeKey);

	/** 
	 * Checks to see if the specified object has had its initialization deferred (meaning a super/archetype
	 * hasn't had FObjectInitializer::PostConstructInit() ran on it yet). 
	 */
	bool IsInitializationDeferred(const UObject* Object) const;

	/**
	 * Determines if the specified object needs to have its Preload() call deferred 
	 * (this is meant to be called from Preload() itself). If so, this will record the object
	 * and serialize it in later, once it's initializer dependency has been resolved 
	 * (from ResolveArchetypeInstances).
	 *
	 * This should be the case for any object that's had its initialization deferred (need 
	 * to initialize before you serialize), and any dependencies (sub-objects, etc.) waiting 
	 * on that object's initialization.
	 * 
	 * @return True if Object's load/serialization should be skipped (for now).
	 */
	virtual bool DeferPreload(UObject* Object);

protected:
	/** 
	 * Used to keep DeferPreload() from re-adding objects while we're in the midst of resolving.
	 */
	bool IsResolving(UObject* ArchetypeInstance) const;

	/** 
	 * Runs the deferred initializer for the specified archetype object if its not dependent 
	 * on other archetypes (like a sub-object that first requires the super's CDO to be constructed, 
	 * and then for its archetype to be serialized).
	 *
	 * If the initializer needs to be further deferred, this should re-register under its new dependency.
	 *
	 * @param  ResolvingObject		The dependency that has been fully loaded/serialized, that the object's initializer was logged under.
	 * @param  ArchetypeInstance	The object which has had its initializer deferred.
	 *
	 * @return True if the initializer was ran, other wise false.
	 */
	virtual bool ResolveDeferredInitialization(UObject* ResolvingObject, UObject* ArchetypeInstance);

	/** 
	 * Runs through all objects that had their Preload() skipped due to an initializer 
	 * being deferred. Runs serialization on each object.
	 */
	void PreloadDeferredDependents(UObject* ArchetypeInstance);

protected:
#if USE_DEFERRED_DEPENDENCY_CHECK_VERIFICATION_TESTS
	friend struct FDeferredObjInitializationHelper;
#endif

	/** Tracks objects that have had their initialization deferred as a result of their archetype not being fully serialized (maps archetype => list-o-instances) */
	TMultiMap<const UObject*, UObject*> ArchetypeInstanceMap;
	/** A map that lets us look up FObjectInitializers by their UObject */
	TMap<UObject*, FObjectInitializer> DeferredInitializers;
	/** Used to keep ResolveArchetypeInstances() from re-adding sub-objects via DeferPreload() */
	TArray<UObject*> ResolvingObjects;
	/** Track default objects that had their Preload() skipped, because a archetype dependency should initialize first */
	TMultiMap<UObject*, UObject*> DeferredPreloads;
};

/** 
 * Specialized FDeferredInitializationTracker for tracking deferred Blueprint CDOs specifically. 
 * (every object in DeferredInitializers should be a BP CDO).
 */
struct FDeferredCdoInitializationTracker : FDeferredInitializationTrackerBase, TThreadSingleton<FDeferredCdoInitializationTracker>
{
public:
	//~ FDeferredInitializationTrackerBase interface
	virtual bool DeferPreload(UObject* Object) override;
};

/**
* Specialized FDeferredInitializationTracker for tracking deferred Blueprint sub-objects specifically.
* (every object in DeferredInitializers should be a default sub-object or component template).
*/
struct FDeferredSubObjInitializationTracker : FDeferredInitializationTrackerBase, TThreadSingleton<FDeferredSubObjInitializationTracker>
{
protected:
	//~ FDeferredInitializationTrackerBase interface
	virtual bool ResolveDeferredInitialization(UObject* ResolvingObject, UObject* ArchetypeInstance) override;
};

/**
 * Access points for making FDeferredInitializationTracker calls. Takes care of 
 * routing calls to the right tracker (CDOs vs sub-objects), and wraps the TThreadSingleton access for each.
 */
struct FDeferredObjInitializationHelper
{
	/** 
	 * Determines if the specified initializer needs to be deferred (does it have a 
	 * archetype dependency that needs to be serialized first?). If so, the FObjectInitializer will
	 * be copied and stored with the appropriate tracker (CDO vs. sub-object, etc.).
	 *
	 * Designed to be called from the FObjectInitializer itself (before it runs initialization).
	 *
	 * @return A pointer to the initializer copy (if one was made), null if no deferral was needed.
	 */
	static FObjectInitializer* DeferObjectInitializerIfNeeded(const FObjectInitializer& DeferringInitializer);

	/** 
	 * Determines if the specified object should have its Preload() skipped. If so, this
	 * should cache the sub-object so it can be loaded later, when its dependency is resolved.
	 *
	 * Designed to be called from the Preload() itself (before it runs serializes the object).
	 *
	 * More info: Because of delta serialization, we require that a parent's CDO be
	 *            fully serialized before its children's CDOs are created. However,
	 *            due to cyclic parent/child dependencies, we have some cases where
	 *            the linker breaks that expected behavior. In those cases, we
	 *            defer the child's initialization (i.e. defer copying of parent
	 *            property values, etc.), and wait until we can guarantee that the
	 *            parent CDO has been fully loaded.
	 *
	 *            In a normal scenario, the order of property initialization is:
	 *            Creation (zeroed) -> Initialization (copied super's values) -> Serialization (overridden values loaded)
	 *            When the initialization has been deferred we have to make sure to
	 *            defer serialization here as well (don't worry, it will be invoked
	 *            again from FinalizeBlueprint()->ResolveDeferredExports())
	 *
	 *            also, if this is an inherited sub-object on a CDO, and that CDO has had
	 *            its initialization deferred (for reasons explained above), then
	 *            we shouldn't serialize in data for this quite yet... not until
	 *            its owner has had a chaTnce to initialize itself (because, as part
	 *            of CDO initialization, inherited sub-objects get filled in with
	 *            values inherited from the super)
	 *
	 * @return True if the object's Preload() should be skipped.
	 */
	static bool DeferObjectPreload(UObject* Object);

	/** 
	 * Loops through all object initializers and preloads that were skipped due 
	 * to this archetype object not being ready yet.
	 *
	 * Should be called once the object has been fully serialized in.
	 */
	static void ResolveDeferredInitsFromArchetype(UObject* Archetype);
};
