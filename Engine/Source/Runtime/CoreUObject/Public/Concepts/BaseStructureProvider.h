// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class UScriptStruct;

template <typename T>
struct TBaseStructure;

/**
 * Describes a type for which TBaseStructure<T>::Get() returning a UScriptStruct* is defined.
 */
struct CBaseStructureProvider
{
	template <typename T>
	auto Requires(UScriptStruct*& StructRef) -> decltype(
		StructRef = TBaseStructure<T>::Get()
	);
};
