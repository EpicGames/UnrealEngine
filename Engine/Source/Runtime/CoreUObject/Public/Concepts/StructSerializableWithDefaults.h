// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FArchive;
class FStructuredArchiveSlot;
class UStruct;

/** Describes a struct that can be serialized with defaults. */
struct CStructSerializableWithDefaults
{
	template <typename StructType>
	auto Requires(FArchive& Ar, StructType* Data, UStruct* DefaultsStruct, const void* Defaults)
		-> decltype(Data->Serialize(Ar, DefaultsStruct, Defaults));
};

/** Describes a struct that can be serialized with defaults. */
struct CStructStructuredSerializableWithDefaults
{
	template <typename StructType>
	auto Requires(FStructuredArchiveSlot Slot, StructType* Data, UStruct* DefaultsStruct, const void* Defaults)
		-> decltype(Data->Serialize(Slot, DefaultsStruct, Defaults));
};
