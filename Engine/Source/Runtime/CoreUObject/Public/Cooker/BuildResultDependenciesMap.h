// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"

namespace UE::Cook { class FCookDependency; }
#endif

#if WITH_EDITOR

namespace UE::Cook
{

/**
 * A container for BuildResult names and an array of CookDependencies for each BuildResult. The cooker collects
 * these from UClasses during OnCookEvent, and it collects them from UStructs during FArchive::Serialize through
 * FObjectSavePackageSerializeContext. @see namespace UE::Cook::BuildResult in CookDependency.h.
 */
struct FBuildResultDependenciesMap : public TMap<FName, TArray<UE::Cook::FCookDependency>>
{
public:
	using TMap<FName, TArray<UE::Cook::FCookDependency>>::TMap;

	COREUOBJECT_API void Add(FName Name, UE::Cook::FCookDependency CookDependency);
	COREUOBJECT_API void Append(FName Name, TArray<UE::Cook::FCookDependency> CookDependencies);
	COREUOBJECT_API void Append(const FBuildResultDependenciesMap& Other);
	COREUOBJECT_API void Append(FBuildResultDependenciesMap&& Other);
};

} // namespace UE::Cook

#endif // WITH_EDITOR
