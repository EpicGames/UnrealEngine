// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/PackageWriter.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"

class ITargetPlatform;
class UCookOnTheFlyServer;
class UPackage;
namespace UE::Cook { class FSaveCookedPackageContext; }
namespace UE::Cook { class ICookArtifactReader; }
namespace UE::Cook { class ICookInfo; }
namespace UE::Cook::Artifact { struct FUpdateOplogPackagesContext; }
namespace UE::TargetDomain { class FEditorDomainOplog; }

namespace UE::Cook
{

/**
 * To what degree is a package referenced in a cook; this information is provided to CookArtifacts about all of the
 * packages reported from the current cook and from previous incremental cooks.
 */
enum class EPackageReferenceType : uint8
{
	/**
	 * Unreferenced in the cook. This value occurs for packages referenced in a previous incremental cook that were
	 * not referenced in the current cook due to e.g. commandline options such as -cooksinglepackagenorefs.
	 */
	Unreferenced,
	/**
	 * The cook needed to read and store information about this package, but it is not staged to runtime. This value
	 * occurs for build dependencies and editoronly packages.
	 */
	Cook,
	/** Cook referenced and should be staged to runtime. */
	Stage,
};

}

namespace UE::Cook::Artifact
{

/** Argument and return value passing structure for ICookArtifact::CompareSettings. */
struct FCompareSettingsContext
{
public:
	ICookInfo& GetCookInfo() const;
	UE::Cook::ICookArtifactReader& GetArtifactReader() const;

	/** The TargetPlatform being tested; invalidation requests are allowed to differ between platforms. Non-null. */
	const ITargetPlatform* GetTargetPlatform() const;
	const FConfigFile& GetPrevious() const;
	const FConfigFile& GetCurrent() const;
	const FString& GetPreviousFileName() const;
	bool IsRequestInvalidate() const;
	bool IsRequestFullRecook() const;

	/**
	 * Request an invalidation of just this artifact. Initial value is false. OnInvalidate will be called, unless
	 * the cooker or some artifact requests OnFullRecook, in which case OnFullRecook will be called and OnInvalidate
	 * will not be called.
	 */
	void RequestInvalidate(bool bValue);
	/**
	 * Request a full recook; all artifacts invalidated and all packages recooked. Initial value is false.
	 * OnFullRecook will be called. OnInvalidate will not be called.
	 */
	void RequestFullRecook(bool bValue);

private:
	FCompareSettingsContext(ICookInfo& InCookInfo, UE::Cook::ICookArtifactReader& InArtifactReader,
		const FConfigFile& InPrevious, const FConfigFile& InCurrent, const FString& InPreviousFileName);

	ICookInfo& CookInfo;
	UE::Cook::ICookArtifactReader& ArtifactReader;
	const ITargetPlatform* TargetPlatform = nullptr;
	const FConfigFile& Previous;
	const FConfigFile& Current;
	const FString& PreviousFileName;
	bool bRequestInvalidate = false;
	bool bRequestFullRecook = false;

	friend UCookOnTheFlyServer;
};

/**
 * Information about the status of each OplogPackage that is held by FUpdateOplogPackagesContext.
 * An OplogPackage is a package cooked in the current cook session or cooked in a previous incrementalcook session
 * that is still available for future cooks that reference it.
 */
struct FOplogPackageData
{
public:
	/** Degree to which the package is referenced in the cook, @see EPackageReferenceType. */
	UE::Cook::EPackageReferenceType GetReferenceType() const;
	/**
	 * True if the package was referenced in the cook (GetReferenceType != EPackageReferenceType::Unreferenced), and
	 * was either not present in an earlier incremental cook (NewCooked) or was found to be modified and was Recooked.
	 */
	bool WasIncrementallyCooked() const;
	/**
	 * True if the package was referenced in the cook (GetReferenceType != EPackageReferenceType::Unreferenced), and
	 * was found to be cooked in a previous cook and not modified so it was not Recooked.
	 */
	bool WasIncrementallySkipped() const;

private:
	UE::Cook::EPackageReferenceType ReferenceType = EPackageReferenceType::Unreferenced;
	bool bIncrementallyCooked = false;

	friend FUpdateOplogPackagesContext;
	friend UCookOnTheFlyServer;
};

/** Argument and return value passing structure for ICookArtifact::UpdateOplogPackages. */
struct FUpdateOplogPackagesContext
{
public:
	ICookInfo& GetCookInfo() const;
	/** The TargetPlatform being reported. Non-null. */
	const ITargetPlatform* GetTargetPlatform() const;

	/** 
	 * Get the list of OplogPackages that were present in the current CookSession, and data about their cookstatus.
	 * An OplogPackage is a package cooked in the current cook session or cooked in a previous incrementalcook session
	 * that is still available for future cooks that reference it. Data includes whether it was referenced and whether
	 * it was cooked, @see FOplogPackageData.
	 */
	const TMap<FName, FOplogPackageData>& GetOplogPackages() const;

private:
	FUpdateOplogPackagesContext(ICookInfo& InCookInfo);

	TMap<FName, FOplogPackageData> OplogPackages;
	ICookInfo& CookInfo;
	const ITargetPlatform* TargetPlatform = nullptr;

	friend UCookOnTheFlyServer;
};

/**
 * Context passed to ICookArtifact::AppendPackageMetadata.
 * Allows artifacts to store data along with the package's op that is recalculated each time the package is incrementally recooked.
 */
struct FAppendPackageMetaDataContext
{
	friend class UE::TargetDomain::FEditorDomainOplog;
	friend class UE::Cook::FSaveCookedPackageContext;
public:
	const UPackage* Package = nullptr;
	/**
	 * Stores a pre-built FCbObject as the attachment for Key. Use this for conditional
	 * cases where the attachment should only be written when data is available.
	 */
	COREUOBJECT_API void SetAttachment(FName Key, FCbObject Value, EFieldStorage InFieldStorage = EFieldStorage::Attachment);

	bool HasEntries() { return !Entries.IsEmpty(); }
private:
	struct FEntry
	{
		FName Key;
		FCbObject Value;
		EFieldStorage FieldStorage;
	};
	TArray<FEntry, TInlineAllocator<4>> Entries;
};

/**
 * Context passed to ICookArtifact::StoreDataInOplog. Allows artifacts to contribute
 * global per-cook attachment data without directly accessing the package writer.
 */
struct FStoreDataInOplogContext
{
	friend UCookOnTheFlyServer;
public:
	/**
	 * Stores a pre-built FCbObject as the attachment for Key.
	 */
	COREUOBJECT_API void AppendOp(FName OpName, FCbObject Object);

private:

	struct FEntry
	{
		FName Key;
		FCbObject Value;
	};
	TArray<FEntry, TInlineAllocator<4>> Entries;
};

} // namespace UE::Cook::Artifact

namespace UE::Cook
{

/**
 * Interface used during cooking for systems that create an artifact collected from cooked packages.
 * Provides hooks to save global settings that invalidate the artifact when changed.
 * Provides hooks to clean the artifact when invalidated.
 */
class ICookArtifact : public UE::Private::FQueryableRefCountedObject
{
public:
	virtual ~ICookArtifact() = default;

	/** The name of the artifact in log messages and its Metadata/CookSettings_<ArtifactName>.txt file. */
	virtual FString GetArtifactName() const = 0;

	/**
	 * Multiprocess cooks consist of a CookDirector that owns the cook output directory and CookWorkers that cook
	 * individual packages at the CookDirector's instruction. CookArtifacts that only interact with files in the cook
	 * output directory only need to exist on the CookDirector. Therefore, by default, creation of CookArtifacts on
	 * CookWorkers is suppressed. But CookWorkers that override AppendPackageMetadata need to exist on CooKWorkers to
	 * have their AppendPackageMetadata function called for packages cooked on CookWorkers. CookWorkers that override
	 * AppendPackageMetadata should override RegisterOnCookWorker to return true.
	 */
	virtual bool RegisterOnCookWorker() const { return false; }

	/**
	 * Construct a container of config settings from the current executable, config, and global assets.
	 * If non-empty, it will be passed to CompareSettings each time the cook can cook incrementally,
	 * and will be saved to disk in the Metadata folder. If the FConfigFile is empty, it will not be saved,
	 * CompareSettings will be called with empty Previous and Current values.
	 */
	virtual FConfigFile CalculateCurrentSettings(ICookInfo& CookInfo, const ITargetPlatform* TargetPlatform)
	{
		return FConfigFile();
	}

	/**
	 * Compare the results of the previous cook's CalculateCurrentSettings with the current results, and
	 * report to the cooker if needs to invalidate the artifact constructed by the previous cook.
	 */
	virtual void CompareSettings(UE::Cook::Artifact::FCompareSettingsContext& Context)
	{
	}

	/**
	 * Called from the cooker when a full recook was not required and CompareSettings requested invalidate.
	 * This function must clean the artifact's files under Saved\Cooked\<Platform> and any other invalidated files
	 * stored elsewhere.
	 */
	virtual void OnInvalidate(const ITargetPlatform* TargetPlatform)
	{
	}

	/**
	 * Called from the cooker when a full recook was required by the cooker or by CompareSettings from any
	 * artifact. Any files under Saved\Cooked\<Platform> will be deleted by the cooker; this function is responsible
	 * for cleaning invalidated files stored elsewhere.
	 */
	virtual void OnFullRecook(const ITargetPlatform* TargetPlatform)
	{
	}

	/**
	 * Called from cooker, on CookDirector only, during CookByTheBookFinished. The cooker provides the list
	 * of Oplog packages. Oplog packages include every package cooked in the current session, but also any packages
	 * cooked in previous sessions that still have IncrementalCook data stored for them and might be referenced
	 * again in a future cook. Each package has flags for whether it was referenced and whether it was incrementally
	 * recooked. The artifact should prune any information it has about packages no longer in the list, and may
	 * need to prune stale information from packages that were incrementally recooked.
	 */
	virtual void UpdateOplogPackages(UE::Cook::Artifact::FUpdateOplogPackagesContext& Context)
	{
	}

	/**
	 * Called once per cooked package. Contribute per-package metadata by calling Context.SetAttachment. If your
	 * artifact overrides this functions, to support Multiprocess cooks, also override RegisterOnCookWorker to return
	 * true.
	 */
	virtual void AppendPackageMetadata(Artifact::FAppendPackageMetaDataContext& Context)
	{
	}

	/**
	 * Called once at the end of a cook session (CookByTheBookFinished). Contribute
	 * global cook metadata by calling Context.AppendOp
	 * 
	 * Allows additional attachments to be added to the oplog.
	 */
	virtual void StoreDataInOplog(Artifact::FStoreDataInOplogContext& Context)
	{
	}
};


} // namespace UE::Cook


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


namespace UE::Cook::Artifact
{

inline FCompareSettingsContext::FCompareSettingsContext(ICookInfo& InCookInfo,
	UE::Cook::ICookArtifactReader& InArtifactReader, const FConfigFile& InPrevious, const FConfigFile& InCurrent,
	const FString& InPreviousFileName)
	: CookInfo(InCookInfo)
	, ArtifactReader(InArtifactReader)
	, Previous(InPrevious)
	, Current(InCurrent)
	, PreviousFileName(InPreviousFileName)
{
}

inline ICookInfo& FCompareSettingsContext::GetCookInfo() const
{
	return CookInfo;
}
inline UE::Cook::ICookArtifactReader& FCompareSettingsContext::GetArtifactReader() const
{
	return ArtifactReader;
}
inline const ITargetPlatform* FCompareSettingsContext::GetTargetPlatform() const
{
	return TargetPlatform;
}
inline const FConfigFile& FCompareSettingsContext::GetPrevious() const
{
	return Previous;
}
inline const FConfigFile& FCompareSettingsContext::GetCurrent() const
{
	return Current;
}
inline const FString& FCompareSettingsContext::GetPreviousFileName() const
{
	return PreviousFileName;
}
inline bool FCompareSettingsContext::IsRequestInvalidate() const
{
	return bRequestInvalidate;
}
inline bool FCompareSettingsContext::IsRequestFullRecook() const
{
	return bRequestFullRecook;
}
inline void FCompareSettingsContext::RequestInvalidate(bool bValue)
{
	bRequestInvalidate = bValue;
}
inline void FCompareSettingsContext::RequestFullRecook(bool bValue)
{
	bRequestFullRecook = bValue;
}

inline FUpdateOplogPackagesContext::FUpdateOplogPackagesContext(ICookInfo& InCookInfo)
	: CookInfo(InCookInfo)
{
}
inline ICookInfo& FUpdateOplogPackagesContext::GetCookInfo() const
{
	return CookInfo;
}

inline const ITargetPlatform* FUpdateOplogPackagesContext::GetTargetPlatform() const
{
	return TargetPlatform;
}

inline const TMap<FName, FOplogPackageData>& FUpdateOplogPackagesContext::GetOplogPackages() const
{
	return OplogPackages;
}

inline UE::Cook::EPackageReferenceType FOplogPackageData::GetReferenceType() const
{
	return ReferenceType;
}

inline bool FOplogPackageData::WasIncrementallyCooked() const
{
	return (ReferenceType != UE::Cook::EPackageReferenceType::Unreferenced) && bIncrementallyCooked;
}

inline bool FOplogPackageData::WasIncrementallySkipped() const
{
	return (ReferenceType != UE::Cook::EPackageReferenceType::Unreferenced) && !bIncrementallyCooked;
}

} // namespace UE::Cook::Artifact

#endif // WITH_EDITOR
