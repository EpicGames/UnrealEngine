// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR || IS_PROGRAM

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"

class FArchive;

namespace UE::Cook
{

/**
 * An interface for reading files that can be provided either from disk or from ZenServer storage of incremental cook
 * artifacts. Paths passed to this interface should be the same as the on-disk location of the artifacts during a cook,
 * e.g. ICookInfo.GetCookMetadataOutputFolder(TargetPlatform).
 */
class ICookArtifactReader
{
public:
	virtual ~ICookArtifactReader() = default;

	virtual void Initialize(bool bCleanBuild) {}
	
	virtual bool FileExists(const TCHAR* Filename) = 0;
	virtual int64 FileSize(const TCHAR* Filename) = 0;
	virtual IFileHandle* OpenRead(const TCHAR* Filename) = 0;

	virtual bool IterateDirectory(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) = 0;

	// Utilities
	virtual FArchive* CreateFileReader(const TCHAR* Filename) = 0;
	virtual bool IterateDirectoryRecursively(const TCHAR* Directory, IPlatformFile::FDirectoryVisitor& Visitor) = 0;
	virtual void FindFiles(TArray<FString>& Result, const TCHAR* Filename, bool Files, bool Directories) = 0;
	virtual void FindFiles(TArray<FString>& FoundFiles, const TCHAR* Directory, const TCHAR* FileExtension = nullptr) = 0;
};

} // namespace UE::Cook

#endif // WITH_EDITOR