// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

namespace UE::CookAssetRegistryAccessTracker
{
	/** RAII class used to enable the creation of dependencies for the packages returned by an asset registry request. */
	class FAddPackageDependenciesScope
	{
	public:
		COREUOBJECT_API FAddPackageDependenciesScope();
		COREUOBJECT_API ~FAddPackageDependenciesScope();
		 
		COREUOBJECT_API static bool GetAddPackageDependenciesEnabled();

	private:
		bool bPreviousValue;
	};
}
