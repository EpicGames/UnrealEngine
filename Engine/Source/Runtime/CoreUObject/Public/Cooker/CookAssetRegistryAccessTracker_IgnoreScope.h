// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"

namespace UE::CookAssetRegistryAccessTracker
{
	/** RAII class used to enable the creation of dependencies for the packages returned by an asset registry request. */
	class FIgnoreScope
	{
	public:
		COREUOBJECT_API FIgnoreScope();
		COREUOBJECT_API ~FIgnoreScope();

		COREUOBJECT_API static bool ShouldIgnoreAccessTracker();

	private:
		bool bPreviousValue;
	};
}
