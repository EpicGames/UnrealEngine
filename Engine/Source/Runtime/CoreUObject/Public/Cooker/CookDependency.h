// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/PreprocessorHelpers.h"
#include "Hash/Blake3.h"
#include "Logging/LogVerbosity.h"
#include "Serialization/CompactBinary.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

class FCbFieldView;
class FCbWriter;
class UObject;
namespace UE::ConfigAccessTracking { enum class ELoadType : uint8; }
namespace UE::ConfigAccessTracking { struct FConfigAccessData; }
namespace UE::Cook { class FConsoleVariableData; }
namespace UE::Cook { struct FCookDependencyContext; }
struct FARFilter;
#endif

#if WITH_EDITOR

namespace UE::Cook
{
/**
 * TypeSelector enum for the FCookDependency variable type. Values are serialized into the oplog as integers,
 * so do not change them without changing oplog version.
 */
enum class ECookDependency : uint8
{
	None					= 0,
	File					= 1,
	Function				= 2,
	TransitiveBuild			= 3,
	Package					= 4,
	ConsoleVariable			= 5,
	Config					= 6,
	SettingsObject			= 7,
	NativeClass				= 8,
	AssetRegistryQuery		= 9,
	RedirectionTarget		= 10,

	Count,
};
COREUOBJECT_API const TCHAR* LexToString(ECookDependency CookDependency);

namespace ResultProjection
{
/**
 * FName passed into FCookDependency::ResultProjection or UE_COOK_RESULTPROJECTION_SCOPED to indicate
 * the entire target package can contribute to the source package's cook. This is the default projection
 * used TObjectPtr resolves when no more limited scope has been declared. It causes a transitive dependency;
 * all cook dependencies declared by the target package are also used as dependencies of the source package.
 */
extern COREUOBJECT_API FName All;
/*
 * FName passed into FCookDependency::ResultProjection or UE_COOK_RESULTPROJECTION_SCOPED to indicate the bytes
 * within the target package, and the native class used by objects referenced from it, can contribute to the
 * source package's cook. This is more limited than the transitive dependency indicated by BuildProjectionAll
 * because it does not include e.g. other packages that the target package depends on for its cook.
 * This ResultProjection is applied automatically for all hard imports saved into the source cooked package;
 * the SavePackage code applies it because generic LinkerLoad code can cause those imports to be included
 * or excluded from the package based on values of e.g. NeedsLoadForClient.
 */
extern COREUOBJECT_API FName PackageAndClass;
/*
 * FName passed into UE_COOK_RESULTPROJECTION_SCOPED to indicate that dereferences of TObjectPtr within the scope
 * should not be automatically added as dependencies of any kind. The calling code for those scopes either
 * knows that the resolves are spurious or is declaring the proper ResultProjection dependency manually.
 */
extern COREUOBJECT_API FName None;

}	

/**
 * BuildResults are groups of data collected during BuildOperations during incremental cooks. Each BuildResult is owned
 * by a BuildOperation and stored in the cook's oplog, and it has a name, a set of dependencies used to create it, and
 * an explicit or implicit data object that was created by the build operation. In Unreal, most examples of
 * BuildOperations are package BuildOperations: the load of a package, the transform of its data into the format that
 * will be used at runtime, and the save of the package.
 * 
 * The generic BuildResults are @see UE::Cook::BuildResult::NAME_Load and UE::Cook::BuildResult::NAME_Save. Other
 * BuildResults are system-specific. The declaration of system-specific build results, and the reporting of
 * dependencies for the generic and system-specific BuildResults, are done during native UClass's overrides of
 * UObject::OnCookEvent(UE::Cook::ECookEvent::PlatformCookDependencies,...).
 */
namespace BuildResult
{

/** The BuildResult that is the bytes of the cooked package. */
extern COREUOBJECT_API FName NAME_Save;

/**
 * The BuildResult that is the loaded bytes of editor package in memory. This is the default BuildResult used
 * in transitive build dependencies.
 */
extern COREUOBJECT_API FName NAME_Load;

}

/**
 * TargetDomain dependencies that can be reported from the class instances in a package. These dependencies are
 * stored in the cook oplog and are evaluated during incremental cook. If any of them changes, the package is
 * invalidated and must be recooked (loaded/saved). These dependencies do not impact whether DDC keys built
 * from the package need to be recalculated.
 */
class FCookDependency
{
public:
	/**
	 * Create a dependency on the contents of the file. Filename will be normalized. Contents are loaded via
	 * IFileManager::Get().CreateFileReader and contents are hashed for comparison.
	 */
	COREUOBJECT_API static FCookDependency File(FStringView InFileName);
	/**
	 * Create a dependency on a call to the specified function with the given arguments. Arguments should be
	 * created using FCbWriter Writer; ... <append arbitrary number of fields to Writer> ...; Writer.Save().
	 * The function should read the arguments using the corresponding FCbFieldIteratorView methods and
	 * LoadFromCompactBinary calls.
	 * 
	 * The function must be registered during editor startup for use with FCookDependency via
	 * UE_COOK_DEPENDENCY_FUNCTION(CppTokenUsedAsName, CppNameOfFunctionToCall).
	 * The name to pass to FCookDependency::Function can be retrieved via
	 * UE_COOK_DEPENDENCY_FUNCTION_CALL(CppTokenUsedAsName).
	 */
	COREUOBJECT_API static FCookDependency Function(FName InFunctionName, FCbFieldIterator&& InArgs);

	/**
	 * Create a transitive build dependency on another package. In an incremental cook if the other package was not
	 * cooked in a previous cook session, or its previous cook result was invalidated, the current package will also
	 * have its cook result invalidated.
	 */
	COREUOBJECT_API static FCookDependency TransitiveBuild(FName PackageName);
	UE_DEPRECATED(5.6, "Add a TransitiveBuild and a Runtime dependency separately.")
	COREUOBJECT_API static FCookDependency TransitiveBuildAndRuntime(FName PackageName);

	/**
	 * Create a build dependency on the contents of a package.
	 * Only the bytes of the .uasset/.umap file are considered.
	 */
	COREUOBJECT_API static FCookDependency Package(FName PackageName);

	/**
	 * Create a dependency on the value of a cvar. The cvar will be read and its value (as a string) will be hashed into the oplog data
	 * If the cvar value is changed, the packages that depend on it will be invalidated
	 */
	COREUOBJECT_API static FCookDependency ConsoleVariable(const FStringView& VariableName);
	COREUOBJECT_API static FCookDependency ConsoleVariable(const FStringView& VariableName, const ITargetPlatform* TargetPlatform, bool bFallbackToNonPlatform);

	/** Create a dependency on the value of a config variable. */
	COREUOBJECT_API static FCookDependency Config(UE::ConfigAccessTracking::FConfigAccessData AccessData);
	COREUOBJECT_API static FCookDependency Config(UE::ConfigAccessTracking::ELoadType LoadType, FName Platform,
		FName FileName, FName SectionName, FName ValueName);
	/** Create a dependency on the value of a config variable, with LoadType=ConfigSystem and Platform=NAME_None. */
	COREUOBJECT_API static FCookDependency Config(FName FileName, FName SectionName, FName ValueName);

	/**
	 * Adds a dependency on the config values and class schema of a settings object. Gives an error and ignores the object
	 * if the object is not a config-driven settings object, such as the CDO of a config UClass or a perObjectConfig
	 * object.
	 *
	 * SettingsObject dependencies are not directly persistable; all of the dependencies reported by the SettingsObject are
	 * copied onto the dependencies of the package declaring the SettingsObject dependency.
	 */
	COREUOBJECT_API static FCookDependency SettingsObject(const UObject* InObject);

	/** Adds a dependency on the class schema of a nativeclass. */
	COREUOBJECT_API static FCookDependency NativeClass(const UClass* InClass);
	COREUOBJECT_API static FCookDependency NativeClass(FStringView ClassPath);

	/** Adds a dependency on the ObjectRedirectors and CoreRedirects that affect the given PackageName. */
	COREUOBJECT_API static FCookDependency RedirectionTarget(FName PackageName);

	/** Adds a dependency on the results reported by an AssetRegistry query. */
	COREUOBJECT_API static FCookDependency AssetRegistryQuery(FARFilter Filter);

	/** Construct an empty dependency; it will never be invalidated. */
	COREUOBJECT_API FCookDependency();

	COREUOBJECT_API ~FCookDependency();
	COREUOBJECT_API FCookDependency(const FCookDependency& Other);
	COREUOBJECT_API FCookDependency(FCookDependency&& Other);
	COREUOBJECT_API FCookDependency& operator=(const FCookDependency& Other);
	COREUOBJECT_API FCookDependency& operator=(FCookDependency&& Other);
	
	/** FCookDependency is a vararg type. Return the type of this instance. */
	ECookDependency GetType() const;

	/** Return a string that describes which data is looked up by this CookDependency, within the context of its type. */
	COREUOBJECT_API FString GetDebugIdentifier() const;

	/** FileName if GetType() == File, else empty. StringView points to null or a null-terminated string. */
	FStringView GetFileName() const;

	/** FunctionName if GetType() == Function, else NAME_None. */
	FName GetFunctionName() const;
	/** FunctionArgs if GetType() == Function, else FCbFieldViewIterator(). */
	FCbFieldViewIterator GetFunctionArgs() const;

	/** PackageName if GetType() == TransitiveBuild or GetType() == Package, else NAME_None. */
	FName GetPackageName() const;
	UE_DEPRECATED(5.6, "This function is only needed for TransitiveBuildAndRuntime dependencies, which are deprecated.")
	bool IsAlsoAddRuntimeDependency() const;

	/** Returns the ConfigAccess in its struct form (File,Section,Key) if GetType() == Config, else empty. */
	COREUOBJECT_API UE::ConfigAccessTracking::FConfigAccessData GetConfigAccessData() const;
	/**
	 * Returns the full path of the config access (e.g. Platform.Filename.Section.ValueName)
	 * if GetType() == Config, else empty.
	 */
	COREUOBJECT_API FString GetConfigPath() const;

	/**
	 * Returns the SettingsObject pointer if GetType() == SettingsObject, else nullptr. Can also be null for
	 * SettingsObject that was found to be invalid.
	 */
	const UObject* GetSettingsObject() const;

	/**
	 * Returns the classpath if GetType() == NativeClass, else empty string. Can also be empty for
	 * NativeClass that was found to be invalid.
	 */
	FStringView GetClassPath() const;

	/** Returns the FARFilter if GetType() == AssetRegistryQuery, else nullptr. */
	const FARFilter* GetARFilter() const;

	/** Returns the value of the dependency. It's an array of size FCookDependency::VALUE_MAX_BYTES_SIZE */
	const uint8* GetRawValue() const;
	/** Returns the value of the dependency, interpreted as a Blake3Hash. */
	const FBlake3Hash& GetHashValue() const;

	/**
	 * Comparison operator for e.g. deterministic ordering of dependencies.
	 * Uses persistent comparison data and is somewhat expensive.
	 */
	bool operator<(const FCookDependency& Other) const;
	/** Equality operator for uniqueness testing */
	bool operator==(const FCookDependency& Other) const;
	bool operator!=(const FCookDependency& Other) const;

	/** Calculate the current hash of this CookDependency, add it into Context, and optionally return it. */
	COREUOBJECT_API void UpdateHash(FCookDependencyContext& Context, FBlake3Hash* OutCurrentValue = nullptr) const;

	COREUOBJECT_API void SetValue(const FBlake3Hash& Hash);
	void SetValue(const FIoHash& Hash);
	void SetValue(const FUtf8String& String);

	COREUOBJECT_API static FBlake3Hash ConvertToHash(const FIoHash& Hash);
	COREUOBJECT_API static FBlake3Hash ConvertToHash(const FUtf8String& String);

	static const uint8 ValueSizeInBytes = 32;

private:
	explicit FCookDependency(ECookDependency InType);
	void Construct();
	void Destruct();
	COREUOBJECT_API void Save(FCbWriter& Writer) const;
	COREUOBJECT_API bool Load(FCbFieldView Value);
	static COREUOBJECT_API bool ConfigAccessDataLessThan(const UE::ConfigAccessTracking::FConfigAccessData& A,
		const UE::ConfigAccessTracking::FConfigAccessData& B);
	static COREUOBJECT_API bool ConfigAccessDataEqual(const UE::ConfigAccessTracking::FConfigAccessData& A,
		const UE::ConfigAccessTracking::FConfigAccessData& B);
	static COREUOBJECT_API bool ARFilterLessThan(const FARFilter& A, const FARFilter& B);
	static COREUOBJECT_API bool ARFilterEqual(const FARFilter& A, const FARFilter& B);

	COREUOBJECT_API static bool ConsoleVariableDataLessThan(const FConsoleVariableData& A, const FConsoleVariableData& B);
	COREUOBJECT_API static bool ConsoleVariableDataEqual(const FConsoleVariableData& A, const FConsoleVariableData& B);

	/** Public hidden friend for operator<< into an FCbWriter. */
	friend FCbWriter& operator<<(FCbWriter& Writer, const FCookDependency& CookDependency)
	{
		CookDependency.Save(Writer);
		return Writer;
	}
	/** Public hidden friend for LoadFromCompactBinary. */
	friend bool LoadFromCompactBinary(FCbFieldView Value, FCookDependency& CookDependency)
	{
		return CookDependency.Load(Value);
	}

private:
	ECookDependency Type;
	struct FFunctionData
	{
		FName Name;
		FCbFieldIterator Args;
	};
	struct FTransitiveBuildData
	{
		FName PackageName;
		bool bAlsoAddRuntimeDependency = false;
	};
	union
	{
		FString StringData;
		FFunctionData FunctionData;
		FTransitiveBuildData TransitiveBuildData;
		FName NameData;
		const UObject* ObjectPtr;
		TUniquePtr<UE::ConfigAccessTracking::FConfigAccessData> ConfigAccessData;
		TUniquePtr<FARFilter> ARFilter;
		TUniquePtr<FConsoleVariableData> ConsoleVariableData;
	};

	/** 
	 * This is the value of the dependency stored as 32 bytes. Other types can be added to the union as long as it
	 * doesn't go above 32 bytes.
	 */
	union
	{
		uint8 RawValue[ValueSizeInBytes];
		FBlake3Hash HashedValue;
	};
};

/**
 * Type of functions used in FCookDependency to append the hash values of arbitrary data.
 * 
 * @param Args Variable-length, variable-typed input data (e.g. names of files, configuration flags)
 *             that specify which hash data. The function should read this data using FCbFieldViewIterator
 *             methods and LoadFromCompactBinary calls that correspond to the FCbWriter methods used
 *             at the callsite of FCookDependency::Function.
 * @param Context that provides calling flags and receives the hashdata. The function should call
 *        Context.Update with the data to be added to the target key (e.g. the hash of the contents
 *        of a filename that was specified in Args).
 */
using FCookDependencyFunction = void (*)(FCbFieldViewIterator Args, FCookDependencyContext& Context);

} // namespace UE::Cook

namespace UE::Cook::Dependency::Private
{

/**
 * Implementation struct of UE_COOK_DEPENDENCY_FUNCTION. Instances of this class are stored in global or
 * namespace scope and add themselves to a list during c++ pre-main static initialization. This list
 * is read later to create a map from FName to c++ function.
 */
struct FCookDependencyFunctionRegistration
{
	template<int N>
	FCookDependencyFunctionRegistration(const TCHAR(&InName)[N], FCookDependencyFunction InFunction)
		: Name(InName), Function(InFunction), Next(nullptr)
	{
		static_assert(N > 0, "Name must be provided");
		check(InName[0] != '\0');
		Construct();
	}
	COREUOBJECT_API ~FCookDependencyFunctionRegistration();
	COREUOBJECT_API void Construct();
	FName GetFName();

	FLazyName Name;
	FCookDependencyFunction Function;
	FCookDependencyFunctionRegistration* Next;
};

} // namespace UE::Cook::Dependency::Private

/**
 * UE_COOK_DEPENDENCY_FUNCTION(<CppToken> Name, void (*)(FCbFieldView Args, FCookDependencyContext& Context))
 * 
 * Registers the given function pointer to handle FCookDependency::Function(Name, Args) calls.
 * @see FCookDependencyFunction. @see FCookDependency::Function.
 * 
 * Name should be a bare cpptoken, e.g.
 * UE_COOK_DEPENDENCY_FUNCTION(MyTypeDependencies, UE::MyTypeDependencies::ImplementationFunction).
*/
#define UE_COOK_DEPENDENCY_FUNCTION(Name, Function) \
	UE::Cook::Dependency::Private::FCookDependencyFunctionRegistration \
	UE_JOIN(FCookDependencyFunctionRegistration_,Name)(TEXT(#Name), Function)
/**
 * Return the FName to use to call a function that was registered via UE_COOK_DEPENDENCY_FUNCTION(Name, Function).
 * Name should be the same bare cpptoken that was passed into UE_COOK_DEPENDENCY_FUNCTION.
 */
#define UE_COOK_DEPENDENCY_FUNCTION_CALL(Name) \
	UE_JOIN(FCookDependencyFunctionRegistration_,Name).GetFName()

#else // WITH_EDITOR

#define UE_COOK_RESULTPROJECTION_SCOPED(ProjectionName)
#define UE_COOK_DEPENDENCY_FUNCTION(Name, Function)
#define UE_COOK_DEPENDENCY_FUNCTION_CALL(Name) NAME_None

#endif // !WITH_EDITOR

#if WITH_EDITOR
namespace UE::Cook
{
inline ECookDependency FCookDependency::GetType() const
{
	return Type;
}

inline FStringView FCookDependency::GetFileName() const
{
	return Type == ECookDependency::File ? StringData : FStringView();
}

inline FName FCookDependency::GetFunctionName() const
{
	return Type == ECookDependency::Function ? FunctionData.Name : NAME_None;
}

inline FCbFieldViewIterator FCookDependency::GetFunctionArgs() const
{
	return Type == ECookDependency::Function ? FunctionData.Args : FCbFieldViewIterator();
}

inline FName FCookDependency::GetPackageName() const
{
	PRAGMA_DISABLE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
	switch (GetType())
	{
	case ECookDependency::TransitiveBuild:
		return TransitiveBuildData.PackageName;
	case ECookDependency::Package:
		return NameData;
	case ECookDependency::RedirectionTarget:
		return NameData;
	default:
		return NAME_None;
	};
	PRAGMA_RESTORE_SWITCH_UNHANDLED_ENUM_CASE_WARNINGS;
}

inline bool FCookDependency::IsAlsoAddRuntimeDependency() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	return Type == ECookDependency::TransitiveBuild ? TransitiveBuildData.bAlsoAddRuntimeDependency : false;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

inline const UObject* FCookDependency::GetSettingsObject() const
{
	return Type == ECookDependency::SettingsObject ? ObjectPtr : nullptr;
}

inline FStringView FCookDependency::GetClassPath() const
{
	return Type == ECookDependency::NativeClass ? StringData : FStringView();
}

inline const FARFilter* FCookDependency::GetARFilter() const
{
	return Type == ECookDependency::AssetRegistryQuery ? ARFilter.Get() : nullptr;
}

inline const uint8* FCookDependency::GetRawValue() const
{
	return RawValue;
}

inline const FBlake3Hash& FCookDependency::GetHashValue() const
{
	return HashedValue;
}

inline bool FCookDependency::operator<(const FCookDependency& Other) const
{
	if (static_cast<uint8>(Type) != static_cast<uint8>(Other.Type))
	{
		return static_cast<uint8>(Type) < static_cast<uint8>(Other.Type);
	}

	switch (Type)
	{
	case ECookDependency::None:
		return false;
	case ECookDependency::File:
	case ECookDependency::NativeClass:
		return StringData.Compare(Other.StringData, ESearchCase::IgnoreCase) < 0;

	case ECookDependency::ConsoleVariable:
	{
		if (ConsoleVariableData.IsValid() != Other.ConsoleVariableData.IsValid())
		{
			return !ConsoleVariableData.IsValid();
		}

		if (ConsoleVariableData.IsValid())
		{
			return ConsoleVariableDataLessThan(*ConsoleVariableData, *Other.ConsoleVariableData);
		}

		return false; // Both pointers are invalid
	}

	case ECookDependency::Function:
	{
		int32 Compare = FunctionData.Name.Compare(Other.FunctionData.Name);
		if (Compare != 0)
		{
			return Compare < 0;
		}
		FMemoryView ViewA;
		FMemoryView ViewB;
		bool bHasViewA = FunctionData.Args.TryGetRangeView(ViewA);
		bool bHasViewB = Other.FunctionData.Args.TryGetRangeView(ViewB);
		if ((!bHasViewA) | (!bHasViewB))
		{
			return bHasViewB; // If both false, return false. If only one, return true only if A is the false.
		}
		return ViewA.CompareBytes(ViewB) < 0;
	}
	case ECookDependency::TransitiveBuild:
	{
		// FName.Compare is lexical and case-insensitive, which is what we want
		int32 Compare = TransitiveBuildData.PackageName.Compare(Other.TransitiveBuildData.PackageName);
		if (Compare != 0)
		{
			return Compare < 0;
		}
		return false;
	}
	case ECookDependency::Package:
		return NameData.Compare(Other.NameData) < 0;
	case ECookDependency::Config:
		if (ConfigAccessData.IsValid() != Other.ConfigAccessData.IsValid())
		{
			return !ConfigAccessData.IsValid();
		}
		if (!ConfigAccessData.IsValid())
		{
			return false; // equal
		}
		return ConfigAccessDataLessThan(*ConfigAccessData, *Other.ConfigAccessData);
	case ECookDependency::SettingsObject:
		// SettingsObjects are not persistable, so we do not use a persistent sort key; just the object ptr.
		return ObjectPtr < Other.ObjectPtr;
	case ECookDependency::AssetRegistryQuery:
		if (ARFilter.IsValid() != Other.ARFilter.IsValid())
		{
			return !ARFilter.IsValid();
		}
		if (!ARFilter.IsValid())
		{
			return false; // equal
		}
		return ARFilterLessThan(*ARFilter, *Other.ARFilter);
	case ECookDependency::RedirectionTarget:
		return NameData.Compare(Other.NameData) < 0;
	case ECookDependency::Count:
	default:
		checkNoEntry();
		return false;
	}
}

inline bool FCookDependency::operator==(const FCookDependency& Other) const
{
	if (static_cast<uint8>(Type) != static_cast<uint8>(Other.Type))
	{
		return false;
	}

	switch (Type)
	{
	case ECookDependency::None:
		return true;
	case ECookDependency::File:
	case ECookDependency::NativeClass:
		return StringData.Compare(Other.StringData, ESearchCase::IgnoreCase) == 0;

	case ECookDependency::ConsoleVariable:
	{
		if (ConsoleVariableData.IsValid() != Other.ConsoleVariableData.IsValid())
		{
			return false;
		}

		if (!ConsoleVariableData.IsValid())
		{
			return true;
		}

		return ConsoleVariableDataEqual(*ConsoleVariableData, *Other.ConsoleVariableData);
	}

	case ECookDependency::Function:
	{
		if (FunctionData.Name.Compare(Other.FunctionData.Name) != 0)
		{
			return false;
		}
		FMemoryView ViewA;
		FMemoryView ViewB;
		bool bHasViewA = FunctionData.Args.TryGetRangeView(ViewA);
		bool bHasViewB = Other.FunctionData.Args.TryGetRangeView(ViewB);
		if (bHasViewA != bHasViewB)
		{
			return false;
		}
		return ViewA.CompareBytes(ViewB) == 0;
	}
	case ECookDependency::TransitiveBuild:
	{
		// FName.Compare is lexical and case-insensitive, which is what we want
		int32 Compare = TransitiveBuildData.PackageName.Compare(Other.TransitiveBuildData.PackageName);
		if (Compare != 0)
		{
			return false;
		}
		return true;
	}
	case ECookDependency::Package:
		return NameData.Compare(Other.NameData) == 0;
	case ECookDependency::Config:
		if (ConfigAccessData.IsValid() != Other.ConfigAccessData.IsValid())
		{
			return false;
		}
		if (!ConfigAccessData.IsValid())
		{
			return true;
		}
		return ConfigAccessDataEqual(*ConfigAccessData, *Other.ConfigAccessData);
	case ECookDependency::SettingsObject:
		// SettingsObjects are not persistable, so we do not use a persistent sort key; just the object ptr.
		return ObjectPtr == Other.ObjectPtr;
	case ECookDependency::AssetRegistryQuery:
	{
		if (ARFilter.IsValid() != Other.ARFilter.IsValid())
		{
			return false;
		}
		if (!ARFilter.IsValid())
		{
			return true;
		}

		return ARFilterEqual(*ARFilter, *Other.ARFilter);
	}
	case ECookDependency::RedirectionTarget:
		return NameData.Compare(Other.NameData) == 0;
	case ECookDependency::Count:
	default:
		checkNoEntry();
		return false;
	}
}

inline bool FCookDependency::operator!=(const FCookDependency& Other) const
{
	return !(*this == Other);
}

inline void FCookDependency::SetValue(const FIoHash& Hash)
{
	SetValue(ConvertToHash(Hash));
}

inline void FCookDependency::SetValue(const FUtf8String& String)
{
	SetValue(ConvertToHash(String));
}

} // namespace UE::Cook

namespace UE::Cook::Dependency::Private
{

inline FName FCookDependencyFunctionRegistration::GetFName()
{
	return Name.Resolve();
}

} // namespace UE::Cook::Dependency::Private

#endif
