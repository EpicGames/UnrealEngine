// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR

namespace UE::Cook
{
/** Context passed into UE_COOK_DEPENDENCY_FUNCTION functions to provide calling flags and receive hash output. */
struct FCookDependencyContext
{
public:
	/** InHasher is void* to mask the implementation details of the hashbuilder. See Update function. */
	UE_DEPRECATED(5.7, "OnInvalidated doesn't exist anymore. Use the new constructor instead.")
	COREUOBJECT_API explicit FCookDependencyContext(void* InHasher,
		TUniqueFunction<void(ELogVerbosity::Type, FString&&)>&& InOnLog,
		TUniqueFunction<void(ELogVerbosity::Type)>&& InOnInvalidated,
		FName InPackageName);

	COREUOBJECT_API explicit FCookDependencyContext(void* InHasher,
		TUniqueFunction<void(ELogVerbosity::Type, FString&&, bool)>&& InOnLog,
		FName InPackageName);

	/**
	 * Update the hashbuilder for the key being constructed (e.g. TargetDomainKey for cooked packages)
	 * with the given Data of Size bytes.
	 */
	COREUOBJECT_API void Update(const void* Data, uint64 Size);

	/**
	 * Reports that current evaluation of the function is different from all previous evaluations for a reason that
	 * cannot be reported as data passed into Update. When called while calculating the initial hash this call is
	 * ignored. When called while testing incrementally skippable the package is marked modified and recooked.
	 */
	UE_DEPRECATED(5.7, "Use function LogInvalidated instead.")
	COREUOBJECT_API void ReportInvalidated();

	/**
	 * Reports failure to compute the hash (e.g. because a file cannot be read).
	 * When called while calculating the initial hash the storage of the key fails and the package will be recooked on
	 * the next cook. When called while testing incrementally skippable the package is marked modified and recooked.
	 */
	UE_DEPRECATED(5.7, "Use function LogError instead.")
	COREUOBJECT_API void ReportError();

	/**
	 * Send a message to the cook dependency context with the given severity. The message may be suppressed or
	 * reduced in verbosity based on the calling context.
	 */
	COREUOBJECT_API void Log(ELogVerbosity::Type, FString Message, bool bInvalidated = false);

	/** Calls Log(Error, Message) and ReportError. */
	COREUOBJECT_API void LogError(FString Message);

	/** Calls Log(Display, Message) and ReportInvalidated. */
	COREUOBJECT_API void LogInvalidated(FString Message);

	/**
	 * Private implementation struct used for AddErrorHandlerScope. Should only be used via
	 * FCookDependencyContext::FErrorHandlerScope& Scope = Context.ErrorHandlerScope(<Function>);
	 */
	struct FErrorHandlerScope
	{
	public:
		COREUOBJECT_API ~FErrorHandlerScope();
	private:
		COREUOBJECT_API FErrorHandlerScope(FCookDependencyContext& InContext);
		friend FCookDependencyContext;
		FCookDependencyContext& Context;
	};

	/**
	 * Add a function that will be removed when the return value goes out of scope, to modify error strings reported
	 * inside the scope before passing them on to higher scopes or the error consumer.
	 * e.g.
	 * FCookDependencyContext::FErrorHandlerScope Scope = Context.ErrorHandlerScope([](FString&& Inner)
	 * { return FString::Printf(TEXT("OuterClass for %s: %s"), *Name, *Inner);});
	 */
	[[nodiscard]] COREUOBJECT_API FErrorHandlerScope ErrorHandlerScope(
		TUniqueFunction<FString(FString&&)>&& ErrorHandler);

	/**
	 * Get the name of the package being considered
	 */
	FName GetPackageName() const { return PackageName; }

	/** Set a new hasher and return the old one. */
	void* SetHasher(void* NewHasher);

private:
	TUniqueFunction<void(ELogVerbosity::Type, FString&&, bool)> OnLog;
	TArray<TUniqueFunction<FString(FString&&)>, TInlineAllocator<1>> ErrorHandlers;
	FName PackageName;
	void* Hasher; // Type is void* to mask the implementation detail
};
}

#endif //#if WITH_EDITOR
