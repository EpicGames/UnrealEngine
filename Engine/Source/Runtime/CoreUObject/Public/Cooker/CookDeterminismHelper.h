// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Containers/Utf8String.h"
#include "Templates/RefCounting.h"
#include "Serialization/CompactBinary.h"

class ITargetPlatform;
class UObject;

namespace UE::Cook
{

struct IDeterminismModifiedPackageContext;

/**
 * Struct to hold input and receive output of IDeterminismHelper::ConstructDiagnostics. Virtual base class so it can
 * be implemented in the UnrealEd project.
 */
struct IDeterminismConstructDiagnosticsContext
{
	virtual ~IDeterminismConstructDiagnosticsContext() = default;

	/**
	 * Reports the TargetPlatform being cooked for which the diagnostics are created. In a multi-platform cook each
	 * platform will have a separate ConstructDiagnostics call.
	 */
	virtual const ITargetPlatform* GetTargetPlatform() = 0;

	/** Output function; call this to add a diagnostic with the given value. */
	virtual void AddDiagnostic(FUtf8StringView DiagnosticName, const FCbField& Value) = 0;

	/** Whether full DDC keys are required in the diagnostic */
	virtual bool FullDDCKeysRequested() const = 0;
};

/**
 * Struct to hold input and receive output of IDeterminismHelper::OnExportModified. Virtual base class so it can
 * be implemented in the UnrealEd project.
 */
struct IDeterminismModifiedExportContext
{
	virtual ~IDeterminismModifiedExportContext() = default;

	/**
	 * True iff the export that registered the DeterminismHelper was found to have modifications in the bytes created
	 * by its Serialize function.
	 */
	virtual bool IsModified() = 0;
	/** True iff the export that registered the DeterminismHelper is the primary asset in the package. */
	virtual bool IsPrimaryAsset() = 0;
	/** Reports the TargetPlatform for which the package was found to be modified. */
	virtual const ITargetPlatform* GetTargetPlatform() = 0;
	/**
	 * Returns a container of the diagnostics written for the export in the old version of the package from the
	 * previous cook.
	 */
	virtual const TMap<FUtf8String, FCbField>& GetOldDiagnostics() = 0;
	/**
	 * Returns a container of the diagnostics written for the export in the current in-memory version of the package
	 * from the current cook.
	 */
	virtual const TMap<FUtf8String, FCbField>& GetNewDiagnostics() = 0;
	/** Return a reference to information about the package in general and all modified exports in the package. */
	virtual IDeterminismModifiedPackageContext& GetPackageContext() = 0;
	/** Create a string version of the values of the old and new diagnostics for the export. */
	virtual FString GetCompareText() = 0;

	/**
	 * Output function; add the given text to the -diffonly or -incrementalvalidate log output for the modified
	 * package.
	 */
	virtual void AppendLog(FStringView LogText) = 0;
	/** Output function; calls AppendLog(GetCompareText()). Activates only once per export if called multiple times. */
	virtual void AppendDiagnostics() = 0;
};

/** Extended data for IDeterminismModifiedExportContext. Holds data about the entire package. */
struct IDeterminismModifiedPackageContext
{
	virtual ~IDeterminismModifiedPackageContext() = default;

	/** Reports the TargetPlatform for which the package was found to be modified. */
	virtual const ITargetPlatform* GetTargetPlatform() = 0;
	/**
	 * Return a list of all the exports in the package that were found to have modifications in the bytes created
	 * by their Serialize functions.
	 */
	virtual const TSet<UObject*>& GetModifiedExports() = 0;
	/**
	 * Get the UObject that is the primary asset for the package, or nullptr in the few instances of packages that
	 * lack a primary asset.
	 */
	virtual UObject* GetPrimaryAsset() = 0;
	/**
	 * Get the IDeterminismModifiedExportContext object for the given export. Creates it if it doesn't exist;
	 * if it doesn't exist the IDeterminismModifiedExportContext will have IsModified=false and emtpy diagnostics.
	 * Invalid to call with a Export pointer that is not in the package; in that case it will log an error and
	 * return an empty IDeterminismModifiedExportContext pointing to the UPackage itself.
	 */
	virtual const IDeterminismModifiedExportContext& GetExportContext(UObject* Export) = 0;
};

/**
 * Interface implemented by UObjects that want to store diagnostic data for comparison when
 * a package is found to contain indeterminism by a -diffonly cook or is found to have a
 * FalsePositiveIncrementalSkip by a -incrementalvalidate cook.
 * 
 * DeterminismHelper objects are registered via FObjectPreSaveContext.RegisterDeterminismHelper
 * in the PreSave of a UObject.
 */
class IDeterminismHelper : public FRefCountedObject
{
public:
	/**
	 * Override this function to add diagnostics to the package containing data about
	 * the UObject that registered the DeterminismHelper.
	 */
	virtual void ConstructDiagnostics(IDeterminismConstructDiagnosticsContext& Context)
	{
	}

	/**
	 * Override this function to get a callback with the old and new values of the diagnostics
	 * when a package is found to be modified by indeterminism or FalsePositiveIncrementalSkip.
	 * The default implementation just prints out the old and new values of each diagnostic.
	 */
	virtual void OnPackageModified(IDeterminismModifiedExportContext& ExportContext)
	{
		if (ExportContext.IsModified() || ExportContext.IsPrimaryAsset())
		{
			ExportContext.AppendDiagnostics();
		}
	}
};

} // namespace UE::Cook

#endif // WITH_EDITOR
