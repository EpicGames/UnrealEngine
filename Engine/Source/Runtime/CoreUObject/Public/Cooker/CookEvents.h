// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/Array.h"
#include "Cooker/CookDependency.h"
#include "UObject/ObjectSaveContext.h"

class ITargetPlatform;
namespace UE::Cook { class ICookInfo; }
#endif // WITH_EDITOR

#if WITH_EDITOR

namespace UE::Cook
{

/**
 * The list of possible events that the cooker can call on UObjects during load/transform/save. Native UObject
 * classes can respond to these events by overriding UObject::OnCookEvent. All UObject classes should call their Super
 * class's version of the function during their call.
 */
enum class ECookEvent : uint32
{
	/**
	 * Called when saving the package to fetch the list of build and runtime dependencies required by the object that
	 * are not automaticallyd detected and need to be manually declared. For example, a dependency on an AssetRegistry
	 * query, or a runtime dependency for platforms that enabled Nanite materials on the nanite material that is
	 * declared as editoronly for non-Nanite-enabled platforms. When called for objects in packages being saved for th
	 * cook, it is calledfor each object immediately after PreSave on that object.
	 * 
	 * This event is also called on objects in packages that are transitive build dependencies of other packages but
	 * are not cooked themselves. For example, a DataTable can include another DataTable, and needs to depend on all of the
	 * dependencies of the DataTable it includes, but the DataTable it includes is not needed at runtime. When called for
	 * objects in these build dependency packages, it is called by itself, with no BeginCacheForCookedPlatformData,
	 * PreSave, or save-Serialize functions being called on the Object.
	 * 
	 * This event is also be called when NOT cooking. In that case, it provides the UObject a way to declare build
	 * dependencies that are registered with the AssetRegistry and cause propagation of AssetManager chunk assignments
	 * without causing the target to be cooked. @see UE::Cook::FCookEventContext::AddLoadBuildDependency.
	 * 
	 * UObjects that need to declare dependencies should call the Add*Dependency functions on the
	 * UE::Cook::FCookEventContext passed into UObject::OnCookEvent.
	 */
	PlatformCookDependencies,
};

/**
 * Context structure to provide information about the cook and the event being called on an object during
 * UObject::OnCookEvent, and to receive output from those events. The available input and output depend on the event
 * being called, @see UE::Cook::ECookEvent.
 */
class FCookEventContext
{
public:
	explicit FCookEventContext(FObjectSaveContextData& InData)
		: Data(InData)
	{
	}
	FCookEventContext(const UE::Cook::FCookEventContext& Other)
		: Data(Other.Data)
	{
	}

	/**
	 * Report whether this OnCookEvent was called during a cook. Some cook events (such as
	 * UE::Cook::PlatformCookDependencies) can also be called during editor saves of a package.
	 */
	bool IsCooking() const;
	/**
	 * Return the CookInfo that provides information about the overall cook. Returns null if not cooking.
	 * Can sometimes be null even if IsCooking is true. (TODO: Fix these cases, make it guaranteed not null.)
	 */
	ICookInfo* GetCookInfo() const;
	/**
	 * Return the TargetPlatform on behalf of which the event was called on the UObject. In multiprocess cooks,
	 * this will be the specific platform being saved; each platform will get its own call. Non-null if and only if
	 * IsCooking().
	 */
	const ITargetPlatform* GetTargetPlatform() const;

	/**
	 * Available during the event UE::Cook::PlatformCookDependencies, ignored in other events.
	 * 
	 * Add the given FCookDependency to the build dependencies for the package being cook-saved. Incremental cooks will
	 * invalidate the package and recook it if the CookDependency changes. Other packages that incorporate the loaded
	 * data of the package into their own cooked results will also be recooked if this build dependency changes.
	 * Calling this function during editor save (rather than cook save) has another meaning. It is ignored for
	 * FCookDependency of type other than FCookDependency::Package, but for FCookDependency::Package it identifies
	 * a reference that propagates chunk management in the AssetRegistry, but does not cause its target to be cooked.
	 */
	COREUOBJECT_API void AddLoadBuildDependency(FCookDependency CookDependency);
	/**
	 * Available during the event UE::Cook::PlatformCookDependencies, ignored in other events.
	 * 
	 * Add the given FCookDependency to the build dependencies for the package being cook-saved. Incremental cooks will
	 * invalidate the package and recook it if the CookDependency changes.
	 */
	COREUOBJECT_API void AddSaveBuildDependency(FCookDependency CookDependency);
	/**
	 * Available during the event UE::Cook::PlatformCookDependencies, ignored in other events.
	 * 
	 * Report that the given PackageName is a runtime dependency of the current package and needs to be cooked.
	 */
	COREUOBJECT_API void AddRuntimeDependency(FSoftObjectPath PackageName);
	/**
	 * Available during the event UE::Cook::PlatformCookDependencies, ignored in other events.
	 * 
	 * Serialize an object to find all packages that it references, and AddCookRuntimeDependency for each one.
	 */
	COREUOBJECT_API void HarvestCookRuntimeDependencies(UObject* HarvestReferencesFrom);

private:
	FObjectSaveContextData& Data;
};


inline bool FCookEventContext::IsCooking() const
{
	return Data.TargetPlatform != nullptr;
}

inline ICookInfo* FCookEventContext::GetCookInfo() const
{
	return Data.CookInfo;
}

inline const ITargetPlatform* FCookEventContext::GetTargetPlatform() const
{
	return Data.TargetPlatform;
}

} // namespace UE::Cook

#endif // WITH_EDITOR