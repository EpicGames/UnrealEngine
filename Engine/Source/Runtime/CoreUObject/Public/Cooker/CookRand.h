// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/StringFwd.h"
#include "HAL/CriticalSection.h"
#include "Math/RandomStream.h"
#include "Templates/RefCounting.h"
#include "Templates/UnrealTemplate.h"

struct FGuid;

namespace UE::Cook
{

/**
 * A RefCounted RandomStream and a threadlocal global accessor for accessing a currently scoped instance.
 * Intended use is for cook operations to set a scope that can be used for deterministic replacement of random or
 * other fundamentally non-deterministic calculations such as NewGuid.
 * 
 * Like RandomStream, const functions on FCookRand can have the side effect of mutating the RandomSeed to the next
 * random value.
 * 
 * An FCookRand instance is not thread-safe because call order in a threaded environment is in general
 * non-deterministic. Callers that run multithreaded operations using FCookRand should maintain their own FCookRand
 * and should pass exclusive ownership of it to each task thread in a deterministic order.
 * 
 * Example usage:
 * 
 * void HighLevelFunctionWithUserCode()
 * {
 *     int32 MySeed = ComputeMyDeterministicSeed();
 *     UE::Cook::FCookRandScope Scope(MySeed, UE::Cook::ECookRandScope::Overwrite);
 *
 *    // ... Call UserCode that uses MyAPI
 * }
 *
 * FGuid MyAPI::NewGuid()
 * {
 *     TRefCountPtr<FCookRand> Rand = FCookRand::GetThreadScope();
 *     return Rand ? Rand->RandomGuid() : FGuid::NewGuid();
 * }
 * float MyAPI::FRand()
 * {
 *    TRefCountPtr<FCookRand> Rand = FCookRand::GetThreadScope();
 *    return Rand ? Rand->RandomStream().FRand() : FMath::FRand();
 * }
 */
class FCookRand : public FRefCountedObject
{
public:
	COREUOBJECT_API explicit FCookRand(int32 InSeed);


	COREUOBJECT_API FGuid RandomGuid() const;
	const FRandomStream& RandomStream() const;

	/** Construct a seed to initialize a CookRand, from a StringView. Normalizes to UTF8 and LowerCase. */
	COREUOBJECT_API static int32 ConstructSeed(FStringView String);

	/**
	 * Get the FCookRand that was declared in a containing scope on the current thread, or null if none has been
	 * declared.
	 */
	COREUOBJECT_API static TRefCountPtr<FCookRand> GetThreadScope();


private:
	FRandomStream RandomStreamVar;
};

enum class ECookRandScope
{
	/** If an FCookRand is already in scope, use it, otherwise create a new one and use that. */
	FindOrAdd,
	/** Create a new FCookRand and put it in scope, pushing to idle any existing one. */
	Overwrite,
};

class FCookRandScope : public FNoncopyable
{
public:
	COREUOBJECT_API FCookRandScope(int32 Seed, ECookRandScope RandScope = ECookRandScope::Overwrite);
	COREUOBJECT_API FCookRandScope(TRefCountPtr<FCookRand> CookRand, ECookRandScope RandScope = ECookRandScope::FindOrAdd);
	COREUOBJECT_API ~FCookRandScope();

private:
	bool ShouldAddNewInCurrentScope(ECookRandScope RandScope);

	TRefCountPtr<FCookRand> Previous;
	bool bRestorePrevious = false;
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////

inline const FRandomStream& FCookRand::RandomStream() const
{
	return RandomStreamVar;
}

}; // namespace UE::Cook

#endif // WITH_EDITOR
