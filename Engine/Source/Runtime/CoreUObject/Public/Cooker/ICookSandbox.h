// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

namespace UE::Cook { struct FCookSandboxConvertCookedPathToPackageNameContext; }

namespace UE::Cook
{

/**
 * A wrapper around FSandboxPlatformFile that provides a similar interface but also handles cook-specific
 * functionality like REMAPPED_PLUGINS
 */
class ICookSandbox
{
public:
	virtual ~ICookSandbox()
	{
	}

	// ISandboxPlatformFile
	virtual const FString& GetSandboxDirectory() const = 0;
	virtual const FString& GetGameSandboxDirectoryName() const = 0;
	virtual FString ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename) const = 0;
	virtual FString ConvertFromSandboxPath(const TCHAR* Filename) const = 0;

	// Cooker API functions that handle the Platform mapping on top of ISandboxPlatformFile
	virtual FString GetSandboxDirectory(const FString& PlatformName) const = 0;
	virtual FString ConvertToAbsolutePathForExternalAppForWrite(const TCHAR* Filename,
		const FString& PlatformName) const = 0;
	virtual FString ConvertFromSandboxPathInPlatformRoot(const TCHAR* Filename,
		FStringView PlatformSandboxRootDir) const = 0;

	// Cooker API functions on top of ISandboxPlatformFile
	virtual FString ConvertToFullSandboxPath(const FString& FileName, bool bForWrite) const = 0;
	virtual FString ConvertToFullPlatformSandboxPath(const FString& FileName, bool bForWrite,
		const FString& PlatformName) const = 0;
	virtual FString ConvertToFullSandboxPathInPlatformRoot(const FString& FileName, bool bForWrite,
		FStringView PlatformSandboxRootdir) const = 0;

	virtual bool TryConvertUncookedFilenameToCookedRemappedPluginFilename(FStringView FileName,
		FString& OutCookedFileName, FStringView PlatformSandboxRootdir = FStringView()) const = 0;

	virtual void FillContext(FCookSandboxConvertCookedPathToPackageNameContext& Context) const = 0;
	virtual FString& ConvertCookedPathToUncookedPath(FStringView CookedPath,
		FCookSandboxConvertCookedPathToPackageNameContext& Context) const = 0;
	virtual FName ConvertCookedPathToPackageName(FStringView CookedPath,
		FCookSandboxConvertCookedPathToPackageNameContext& Context) const = 0;
	virtual FString ConvertPackageNameToCookedPath(FStringView PackageName,
		FCookSandboxConvertCookedPathToPackageNameContext& Context) const = 0;
};

struct FCookSandboxConvertCookedPathToPackageNameContext
{
	FStringView SandboxRootDir;
	FStringView UncookedRelativeRootDir;
	FStringView SandboxProjectDir;
	FStringView UncookedRelativeProjectDir;
	FString ScratchSandboxProjectDir;
	FString ScratchUncookedRelativeProjectDir;
	FString ScratchFileName;
	FString ScratchPackageName;
};

} // namespace UE::Cook

#endif // WITH_EDITOR