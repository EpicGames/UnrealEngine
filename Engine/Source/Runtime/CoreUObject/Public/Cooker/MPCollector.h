// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/Guid.h"
#include "Serialization/CompactBinary.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "UObject/CookEnums.h"
#endif

#if WITH_EDITOR

class FCbObject;
class FCbObjectView;
class FCbWriter;
class ITargetPlatform;

namespace UE::Cook { class FCookWorkerClient; }
namespace UE::Cook { class FCookWorkerServer; }

namespace UE::Cook
{

/**
 *  Identifier for a CookWorker process launched from a Director process, or for the local process.
 *  A director can have multiple CookWorkers.
 */
struct FWorkerId
{
public:
	FWorkerId() { Id = InvalidId; }
	constexpr static FWorkerId Invalid() { return FWorkerId(InvalidId); }
	constexpr static FWorkerId Local() { return FWorkerId(LocalId); }
	static FWorkerId FromRemoteIndex(uint8 Index) { check(Index < InvalidId - 1U);  return FWorkerId(Index + 1U); }
	static FWorkerId FromLocalOrRemoteIndex(uint8 Index) { check(Index < InvalidId);  return FWorkerId(Index); }
	static int32 GetMaxCookWorkerCount() { return static_cast<int32>(InvalidId - 1); }

	bool IsValid() const { return Id != InvalidId; }
	bool IsInvalid() const { return Id == InvalidId; }
	bool IsLocal() const { return Id == LocalId; }
	bool IsRemote() const { return Id != InvalidId && Id != LocalId; }
	uint8 GetRemoteIndex() const { check(IsRemote()); return Id - 1U; }
	uint8 GetMultiprocessId() const { check(IsValid()); return Id; }
	bool operator==(const FWorkerId& Other) const { return Id == Other.Id; }
	bool operator!=(const FWorkerId& Other) const { return Id != Other.Id; }
	bool operator<(const FWorkerId& Other) const { return Id < Other.Id; }
	inline friend int32 GetTypeHash(const FWorkerId& WorkerId) { return WorkerId.Id; }

	FString ToString();

private:
	constexpr explicit FWorkerId(uint8 InId) : Id(InId) {}

private:
	uint8 Id;

	constexpr static uint8 InvalidId = 255;
	constexpr static uint8 LocalId = 0;
};

class FMPCollectorClientTickContext
{
public:
	TConstArrayView<const ITargetPlatform*> GetPlatforms() const { return Platforms; }
	bool IsFlush()  const { return bFlush; }

	COREUOBJECT_API void AddMessage(FCbObject Object);

	COREUOBJECT_API uint8 PlatformToInt(const ITargetPlatform* Platform) const;
	COREUOBJECT_API const ITargetPlatform* IntToPlatform(uint8 PlatformAsInt) const;

private:
	TConstArrayView<const ITargetPlatform*> Platforms;
	TArray<FCbObject> Messages;
	bool bFlush = false;

	friend class FCookWorkerClient;
};

class FMPCollectorServerTickContext
{
public:
	enum class EServerEventType : uint8
	{
		WorkerStartup,
		Count
	};

	FMPCollectorServerTickContext(EServerEventType InEventType) : EventType(InEventType) { check(InEventType < EServerEventType::Count); }

	TConstArrayView<const ITargetPlatform*> GetPlatforms() const { return Platforms; }
	
	COREUOBJECT_API void AddMessage(FCbObject Object);

	COREUOBJECT_API uint8 PlatformToInt(const ITargetPlatform* Platform) const;
	COREUOBJECT_API const ITargetPlatform* IntToPlatform(uint8 PlatformAsInt) const;
	
	EServerEventType GetEventType() const { return EventType; }

private:
	TConstArrayView<const ITargetPlatform*> Platforms;
	TArray<FCbObject> Messages;
	EServerEventType EventType;
	bool bFlush = false;

	friend class FCookDirector;
};

class FMPCollectorClientTickPackageContext
{
public:
	struct FPlatformData
	{
		const ITargetPlatform* TargetPlatform = nullptr;
		ECookResult CookResults = ECookResult::NotAttempted;
	};
	TConstArrayView<const ITargetPlatform*> GetPlatforms() const { return Platforms; }
	TConstArrayView<FPlatformData> GetPlatformDatas() const { return PlatformDatas; }
	FName GetPackageName() const { return PackageName; }

	COREUOBJECT_API void AddMessage(FCbObject Object);
	COREUOBJECT_API void AddAsyncMessage(TFuture<FCbObject>&& ObjectFuture);
	COREUOBJECT_API void AddPlatformMessage(const ITargetPlatform* Platform, FCbObject Object);
	COREUOBJECT_API void AddAsyncPlatformMessage(const ITargetPlatform* Platform, TFuture<FCbObject>&& ObjectFuture);

	COREUOBJECT_API uint8 PlatformToInt(const ITargetPlatform* Platform) const;
	COREUOBJECT_API const ITargetPlatform* IntToPlatform(uint8 PlatformAsInt) const;

private:
	TArray<TPair<const ITargetPlatform*, FCbObject>> Messages;
	TArray<TPair<const ITargetPlatform*, TFuture<FCbObject>>> AsyncMessages;
	TConstArrayView<const ITargetPlatform*> Platforms;
	TConstArrayView<FPlatformData> PlatformDatas;
	FName PackageName;

	friend class FCookWorkerClient;
};

class FMPCollectorServerTickPackageContext
{
public:
	struct FPlatformData
	{
		const ITargetPlatform* TargetPlatform = nullptr;
		ECookResult CookResults = ECookResult::NotAttempted;
	};
	FName GetPackageName() const { return PackageName; }

	COREUOBJECT_API void AddMessage(FCbObject Object);

	COREUOBJECT_API uint8 PlatformToInt(const ITargetPlatform* Platform) const;
	COREUOBJECT_API const ITargetPlatform* IntToPlatform(uint8 PlatformAsInt) const;

private:
	TArray<FCbObject> Messages;
	TConstArrayView<const ITargetPlatform*> Platforms;
	TConstArrayView<FPlatformData> PlatformDatas;
	FName PackageName;

	friend class FCookDirector;
};

class FMPCollectorClientMessageContext
{
public:
	TConstArrayView<const ITargetPlatform*> GetPlatforms() { return Platforms; }
	// Name of the relevant package or NAME_None if not available
	FName GetPackageName() const { return PackageName; }

	COREUOBJECT_API uint8 PlatformToInt(const ITargetPlatform* Platform) const;
	COREUOBJECT_API const ITargetPlatform* IntToPlatform(uint8 PlatformAsInt) const;

private:
	TConstArrayView<const ITargetPlatform*> Platforms;
	FName PackageName;

	friend class FCookWorkerClient;
};

class FMPCollectorServerMessageContext
{
public:
	TConstArrayView<const ITargetPlatform*> GetPlatforms() { return Platforms; }

	COREUOBJECT_API uint8 PlatformToInt(const ITargetPlatform* Platform) const;
	COREUOBJECT_API const ITargetPlatform* IntToPlatform(uint8 PlatformAsInt) const;
	FWorkerId GetWorkerId() const { return WorkerId; }
	int32 GetProfileId() const { return ProfileId; }
	FCookWorkerServer* GetCookWorkerServer() { return Server; }

	bool HasPackageName() const { return !PackageName.IsNone(); }
	FName GetPackageName() const { return PackageName; }
	bool HasTargetPlatform() const { return TargetPlatform != nullptr; }
	const ITargetPlatform* GetTargetPlatform() const { return TargetPlatform; }

private:
	TConstArrayView<const ITargetPlatform*> Platforms;
	FName PackageName;
	FCookWorkerServer* Server = nullptr;
	const ITargetPlatform* TargetPlatform = nullptr;
	int32 ProfileId;
	FWorkerId WorkerId;

	friend class FCookWorkerServer;
};

/**
 * Interface used during cooking to send data collected from save/load on a remote CookWorker
 * to the Director for aggregation into files saves at the end of the cook. 
 */
class IMPCollector : public FRefCountedObject
{
public:
	virtual ~IMPCollector() = default;

	/**
	 * Return a Guid that identifies the IMPCollector uniquely from all other IMPCollectors. Implement this by
	 * return FGuid(0xA,0xB,0xC,0xD), where the values for A,B,C,D are copied from a guid generator tool.
	 */
	virtual FGuid GetMessageType() const = 0;
	/** Return a debug name for the Collector, used in warnings or errors from the cooker about it. */
	virtual const TCHAR* GetDebugName() const = 0;

	/**
	 * Called during events on the CookDirector (see Context.GetEventType()). Not called periodically, if you
	 * need a periodic call, implement FTickableCookObject. Call AddMessage which will be passed to
	 * ClientReceiveMessage on each CookWorker.
	 **/
	virtual void ServerTick(FMPCollectorServerTickContext& Context) {}
	/**
	 * Called periodically on CookWorkers (period is determined by cooker), and called once with IsFlush()=true at end
	 * of cook. Call AddMessage which will be passed to ServerReceiveMessage on the CookDirector.
	 */
	virtual void ClientTick(FMPCollectorClientTickContext& Context) {}
	/**
	 * Called on Director each time a package is assigned to a CookWorker (note this may occur several times for a
	 * single package in cases of retraction). Call AddMessage which will be passed to ClientReceiveMessage on the
	 * assigned CookWorker.
	 */
	virtual void ServerTickPackage(FMPCollectorServerTickPackageContext& Context) {}
	/**
	 * Called on CookWorker when a package is saved, or found to be unsaveable.
	 * In MultiprocessCooks, it will usually be called once with a set of the platforms saved, but it is possible it
	 * will be called once per platform if the package is found at different times to be needed on different platforms.
	 * Call AddMessage which will be passed to ServerReceiveMessage on the CookDirector.
	 */
	virtual void ClientTickPackage(FMPCollectorClientTickPackageContext& Context) {}
	/** Called on CookWorker to receive a message from ServerTick or ServerTickPackage. */
	virtual void ClientReceiveMessage(FMPCollectorClientMessageContext& Context, FCbObjectView Message) {}
	/** Called on CookDirector to receive a message from ClientTick or ClientTickPackage. */
	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message) {}
};


/** Baseclass for messages used by IMPCollectors that want to interpret their messages as c++ structs. */
class IMPCollectorMessage
{
public:
	virtual ~IMPCollectorMessage() = default;

	/** Marshall the message to a CompactBinaryObject. */
	virtual void Write(FCbWriter& Writer) const = 0;
	/** Unmarshall the message from a CompactBinaryObject. */
	virtual bool TryRead(FCbObjectView Object) = 0;
	/** Return the Guid that identifies the message to the remote connection. */
	virtual FGuid GetMessageType() const = 0;
	/** Return the debugname for diagnostics. */
	virtual const TCHAR* GetDebugName() const = 0;
};


/** A subinterface of IMPCollector that uses a ICollectorMessage subclass to serialize the message. */
template <typename MessageType>
class IMPCollectorForMessage : public IMPCollector
{
public:
	virtual void ClientReceiveMessage(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
		MessageType&& Message) {}
	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, bool bReadSuccessful,
		MessageType&& Message) {}

	virtual FGuid GetMessageType() const override
	{
		return MessageType().GetMessageType();
	}

	virtual const TCHAR* GetDebugName() const override
	{
		return MessageType().GetDebugName();
	}

	virtual void ClientReceiveMessage(FMPCollectorClientMessageContext& Context, FCbObjectView Message) override
	{
		MessageType CBMessage;
		bool bReadSuccessful = CBMessage.TryRead(Message);
		ClientReceiveMessage(Context, bReadSuccessful, MoveTemp(CBMessage));
	}

	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, FCbObjectView Message) override
	{
		MessageType CBMessage;
		bool bReadSuccessful = CBMessage.TryRead(Message);
		ServerReceiveMessage(Context, bReadSuccessful, MoveTemp(CBMessage));
	}
};

/**
 * An implementation of IMPCollector that uses an ICollectorMessage subclass to serialize the message,
 * and directs messages received on the client to the given callback.
 */
template <typename MessageType>
class TMPCollectorClientMessageCallback : public IMPCollectorForMessage<MessageType>
{
public:
	TMPCollectorClientMessageCallback(TUniqueFunction<void(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
		MessageType&& Message)>&& InCallback)
		: Callback(MoveTemp(InCallback))
	{
		check(Callback);
	}

	virtual void ClientReceiveMessage(FMPCollectorClientMessageContext& Context, bool bReadSuccessful,
		MessageType&& Message) override
	{
		Callback(Context, bReadSuccessful, MoveTemp(Message));
	}

private:
	TUniqueFunction<void(FMPCollectorClientMessageContext& Context, bool bReadSuccessful, MessageType&& Message)> Callback;
};

/**
 * An implementation of IMPCollector that uses an IMPCollectorMessage subclass to serialize the message,
 * and directs messages received on the server to the given callback.
 */
template <typename MessageType>
class TMPCollectorServerMessageCallback : public IMPCollectorForMessage<MessageType>
{
public:
	TMPCollectorServerMessageCallback(TUniqueFunction<void(FMPCollectorServerMessageContext& Context, bool bReadSuccessful,
		MessageType&& Message)>&& InCallback)
		: Callback(MoveTemp(InCallback))
	{
		check(Callback);
	}

	virtual void ServerReceiveMessage(FMPCollectorServerMessageContext& Context, bool bReadSuccessful,
		MessageType&& Message) override
	{
		Callback(Context, bReadSuccessful, MoveTemp(Message));
	}

private:
	TUniqueFunction<void(FMPCollectorServerMessageContext& Context, bool bReadSuccessful, MessageType&& Message)> Callback;
};

inline FString FWorkerId::ToString()
{
	return IsInvalid() ? TEXT("<Invalid>") : (IsLocal() ? TEXT("Local") :
		FString::Printf(TEXT("CookWorker %u"), static_cast<uint32>(GetRemoteIndex())));
}

} // namespace UE::Cook

#endif // WITH_EDITOR
