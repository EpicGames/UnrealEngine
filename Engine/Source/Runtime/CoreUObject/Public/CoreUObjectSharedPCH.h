// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreSharedPCH.h"

// From CoreUObject:
#include "Misc/NotifyHook.h"
#include "Misc/PackageName.h"
#include "Misc/WorldCompositionUtility.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "Serialization/BulkData.h"
#include "Serialization/SerializedPropertyScope.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/Class.h"
#include "UObject/CoreNative.h"
#include "UObject/CoreNet.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GCObject.h"
#include "UObject/GeneratedCppIncludes.h"
#include "UObject/Interface.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/Linker.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Object.h"
#include "UObject/ObjectKey.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/ObjectResource.h"
#include "UObject/Package.h"
#include "UObject/PackageFileSummary.h"
#include "UObject/PersistentObjectPtr.h"
#include "UObject/PropertyTag.h"
#include "UObject/Script.h"
#include "UObject/ScriptInterface.h"
#include "UObject/ScriptMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/Stack.h"
#include "UObject/StructOnScope.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectAnnotation.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/UObjectMarks.h"
#include "UObject/UObjectThreadContext.h"
#include "UObject/WeakObjectPtr.h"