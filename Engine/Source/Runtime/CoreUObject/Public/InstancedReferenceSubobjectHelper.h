// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

/** 
 * Meant to represent a specific object property that is setup to reference a 
 * instanced sub-object. Tracks the property hierarchy used to reach the 
 * property, so that we can easily retrieve instanced sub-objects from a 
 * container object.
 */
struct UE_DEPRECATED(5.6, "Deprecated; this type is no longer in use.") FInstancedPropertyPath
{
private:
	struct FPropertyLink
	{
		FPropertyLink(const FProperty* Property, int32 ArrayIndexIn, bool bIsMapValueIn)
			: PropertyPtr(Property), ArrayIndex(ArrayIndexIn), bIsMapValue(bIsMapValueIn)
		{}

		const FProperty* PropertyPtr;
		int32            ArrayIndex;
		bool             bIsMapValue;
	};

public:
	//--------------------------------------------------------------------------
	FInstancedPropertyPath(FProperty* RootProperty, int32 ArrayIndex = INDEX_NONE, bool bIsMapValue = false)
	{
		Push(RootProperty, ArrayIndex, bIsMapValue);
	}

	//--------------------------------------------------------------------------
	void Push(const FProperty* Property, int32 ArrayIndex = INDEX_NONE, bool bIsMapValue = false)
	{
		PropertyChain.Add(FPropertyLink(Property, ArrayIndex, bIsMapValue));		
	}

	//--------------------------------------------------------------------------
	void Pop()
	{
 		PropertyChain.RemoveAt(PropertyChain.Num() - 1);
	}

	//--------------------------------------------------------------------------
	const FProperty* Head() const
	{
		return PropertyChain.Last().PropertyPtr;
	}

	//--------------------------------------------------------------------------
	COREUOBJECT_API UObject* Resolve(const UObject* Container) const;

private:
	TArray<FPropertyLink> PropertyChain;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

/** 
 * Can be used as a raw sub-object pointer, but also contains a 
 * FInstancedPropertyPath to identify the property that this sub-object is 
 * referenced by. Paired together for ease of use (so API users don't have to manage a map).
 */
struct UE_DEPRECATED(5.6, "Deprecated; this type is no longer in use.") FInstancedSubObjRef
{
	FInstancedSubObjRef(UObject* SubObj, const FInstancedPropertyPath& PropertyPathIn)
		: SubObjInstance(SubObj)
		, PropertyPath(PropertyPathIn)
	{}

	//--------------------------------------------------------------------------
	operator UObject*() const
	{
		return SubObjInstance;
	}

	//--------------------------------------------------------------------------
	UObject* operator->() const
	{
		return SubObjInstance;
	}

	//--------------------------------------------------------------------------
	friend uint32 GetTypeHash(const FInstancedSubObjRef& SubObjRef)
	{
		return GetTypeHash((UObject*)SubObjRef);
	}

	UObject* SubObjInstance;
	FInstancedPropertyPath PropertyPath;
};
 
/** 
 * Contains a set of utility functions useful for searching out and identifying
 * instanced sub-objects contained within a specific outer object. (DEPRECATED)
 */
class FFindInstancedReferenceSubobjectHelper
{
public:
	template<typename T>
	UE_DEPRECATED(5.6, "Deprecated; use GetObjectsWithOuter() instead, or use TPropertyValueIterator<FObjectProperty> and check for the CPF_InstancedReference flag if you specifically need this functionality.")
	static void GetInstancedSubObjects(const UObject* Container, T& OutObjects)
	{
		const UClass* ContainerClass = Container->GetClass();
		for (FProperty* Prop = ContainerClass->RefLink; Prop; Prop = Prop->NextRef)
		{
			for (int32 ArrayIdx = 0; ArrayIdx < Prop->ArrayDim; ++ArrayIdx)
			{
				FInstancedPropertyPath RootPropertyPath(Prop, ArrayIdx);
				const uint8* ValuePtr = Prop->ContainerPtrToValuePtr<uint8>(Container, ArrayIdx);
				ForEachInstancedSubObject<const void*>(RootPropertyPath, ValuePtr, [&OutObjects](const FInstancedSubObjRef& Ref, const void*){ OutObjects.Add(Ref); });
			}
		}
	}

	template<typename T>
	UE_DEPRECATED(5.6, "Deprecated; use GetObjectsWithOuter() instead, or recursively use a TPropertyValueIterator<FObjectProperty> and check for the CPF_InstancedReference flag if you specifically need this functionality.")
	static void GetInstancedSubObjectsRecursive(const UObject* Container, T& OutObjects)
	{
		const UClass* ContainerClass = Container->GetClass();
		for (FProperty* Prop = ContainerClass->RefLink; Prop; Prop = Prop->NextRef)
		{
			for (int32 ArrayIdx = 0; ArrayIdx < Prop->ArrayDim; ++ArrayIdx)
			{
				FInstancedPropertyPath RootPropertyPath(Prop, ArrayIdx);
				const uint8* ValuePtr = Prop->ContainerPtrToValuePtr<uint8>(Container, ArrayIdx);
				ForEachInstancedSubObject<const void*>(RootPropertyPath, ValuePtr, [&OutObjects](const FInstancedSubObjRef& Ref, const void*)
				{
					if (!OutObjects.Contains(Ref))
					{
						OutObjects.Add(Ref);
						GetInstancedSubObjectsRecursive(Ref.SubObjInstance, OutObjects);
					}
				});
			}
		}
	}

	UE_DEPRECATED(5.6, "Deprecated; this method was originally implemented for internal use only.")
	static COREUOBJECT_API void Duplicate(
		UObject* OldObject, 
		UObject* NewObject, 
		TMap<UObject*, UObject*>& ReferenceReplacementMap, 
		TArray<UObject*>& DuplicatedObjects, 
		TMap<UObject*, UObject*>* OptionalMappings);

	template<typename T>
	UE_DEPRECATED(5.6, "Deprecated; use a container helper (e.g. FScriptArrayHelper) instead if you need to iterate over the values of a container property given its value address.")
	static void ForEachInstancedSubObject(FInstancedPropertyPath& PropertyPath, T ContainerAddress, TFunctionRef<void(const FInstancedSubObjRef& Ref, T PropertyValueAddress)> ObjRefFunc);
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS