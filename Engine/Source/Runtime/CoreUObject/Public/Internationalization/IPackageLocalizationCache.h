// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Interface for types that provide caching for package localization. */
class IPackageLocalizationCache
{
public:
	virtual ~IPackageLocalizationCache() = default;

	/**
	 * Invalidate any cached state for the given root source path, and add it to the queue of things to process when ConditionalUpdateCache is called.
	 * eg) when new asset registry state is loaded for a plugin that may invalidate its cached data
	 */
	virtual void InvalidateRootSourcePath(const FString& InRootPath) = 0;

	/**
	 * Update this cache, but only if it is dirty.
	 */
	virtual void ConditionalUpdateCache() = 0;

	/**
	 * Try and find the localized package name for the given source package for the active culture.
	 *
	 * @param InSourcePackageName	The name of the source package to find.
	 *
	 * @return The localized package name, or NAME_None if there is no localized package.
	 */
	virtual FName FindLocalizedPackageName(const FName InSourcePackageName) = 0;

	/**
	 * Try and find the localized package name for the given source package for the given culture.
	 *
	 * @param InSourcePackageName	The name of the source package to find.
	 * @param InCultureName			The name of the culture to find the package for.
	 *
	 * @return The localized package name, or NAME_None if there is no localized package.
	 */
	virtual FName FindLocalizedPackageNameForCulture(const FName InSourcePackageName, const FString& InCultureName) = 0;
};
