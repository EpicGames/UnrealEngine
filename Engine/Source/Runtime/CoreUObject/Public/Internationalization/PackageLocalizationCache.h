// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "Internationalization/IPackageLocalizationCache.h"
#include "Templates/SharedPointer.h"
#include "Templates/Tuple.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

class FPackageLocalizationCache;

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogPackageLocalizationCache, Log, All);

/** Package localization cache for a specific culture (may contain a chain of cultures internally based on their priority) */
class FPackageLocalizationCultureCache
{
public:
	/**
	 * Construct a new culture specific cache.
	 *
	 * @param InOwnerCache		A pointer to our owner cache. Used to perform an implementation specific search for localized packages (see FPackageLocalizationCache::FindLocalizedPackages).
	 * @param InCultureName		The name of the culture this cache is for.
	 */
	COREUOBJECT_API FPackageLocalizationCultureCache(FPackageLocalizationCache* InOwnerCache, const FString& InCultureName);

	/**
	 * Update this cache, but only if it is dirty.
	 */
	COREUOBJECT_API void ConditionalUpdateCache();

	/**
	 * Add a source path to be scanned the next time ConditionalUpdateCache is called.
	 *
	 * @param InRootPath		The root source path to add.
	 */
	COREUOBJECT_API void AddRootSourcePath(const FString& InRootPath);

	/**
	 * Remove a source path. This will remove it from the pending list, as well as immediately remove any localized package entries mapped from this source root.
	 *
	 * @param InRootPath		The root source path to remove.
	 */
	COREUOBJECT_API void RemoveRootSourcePath(const FString& InRootPath);

	/**
	 * Add a package (potentially source or localized) to this cache.
	 *
	 * @param InPackageName		The name of the package to add.
	 * @return TRUE if the supplied package is a localized asset and was added to the culture cache, FALSE otherwise
	 */
	COREUOBJECT_API bool AddPackage(const FString& InPackageName);

	/**
	 * Remove a package (potentially source or localized) from this cache.
	 *
	 * @param InPackageName		The name of the package to remove.
	 * @return TRUE if the supplied package is a localized asset and was removed from the culture cache, FALSE otherwise
	 */
	COREUOBJECT_API bool RemovePackage(const FString& InPackageName);

	/**
	 * Restore this cache to an empty state.
	 */
	COREUOBJECT_API void Empty();

	/**
	 * Try and find the localized package name for the given source package for culture we represent.
	 *
	 * @param InSourcePackageName	The name of the source package to find.
	 *
	 * @return The localized package name, or NAME_None if there is no localized package.
	 */
	COREUOBJECT_API FName FindLocalizedPackageName(const FName InSourcePackageName);

private:
	/**
	 * Update this cache, but only if it is dirty. Same as ConditionalUpdateCache, but doesn't take a lock (you must have locked in the calling code).
	 */
	void ConditionalUpdateCache_NoLock();

private:
	/** Critical section preventing concurrent access to our data. */
	mutable FCriticalSection LocalizedPackagesCS;

	/** A pointer to our owner cache. */
	FPackageLocalizationCache* OwnerCache;

	/** An array of culture names that should be scanned, sorted in priority order. */
	TArray<FString> PrioritizedCultureNames;

	/** An array of culture names and Verse identifiers that should be scanned, sorted in priority order. */
	TArray<FString> PrioritizedCultureNamesAndVerseIdentifiers;

	/** An array of source paths we should scan on the next call to ConditionalUpdateCache. */
	TArray<FString> PendingSourceRootPathsToSearch;

	/** Mapping between source paths, and prioritized localized paths. */
	TMap<FString, TArray<FString>> SourcePathsToLocalizedPaths;

	/** Mapping between source package names, and prioritized localized package names. */
	TMap<FName, TArray<FName>> SourcePackagesToLocalizedPackages;
};

/** Common implementation for the package localization cache */
class FPackageLocalizationCache : public IPackageLocalizationCache
{
	friend class FPackageLocalizationCultureCache;

public:
	COREUOBJECT_API FPackageLocalizationCache();
	COREUOBJECT_API virtual ~FPackageLocalizationCache();

	//~ IPackageLocalizationCache interface
	COREUOBJECT_API virtual void InvalidateRootSourcePath(const FString& InRootPath) override;
	COREUOBJECT_API virtual void ConditionalUpdateCache() override;
	COREUOBJECT_API virtual FName FindLocalizedPackageName(const FName InSourcePackageName) override;
	COREUOBJECT_API virtual FName FindLocalizedPackageNameForCulture(const FName InSourcePackageName, const FString& InCultureName) override;

protected:
	/**
	 * Find all of the localized packages under the given roots, and update the map with the result.
	 *
	 * @param NewSourceToLocalizedPaths Map containing a key for each of the the source root paths we're finding localized packages for, e.g. /Game
	 *                                  The value for each key is an array of the roots to search for localized packages for that source, e.g. { /Game/L10/hu, /Game/L10/fr }
	 * @param InOutSourcePackagesToLocalizedPackages The map to update. This will accumulate results from each root in order to build
	 *                          a mapping between each source package and its array of prioritized localized packages.
	 */
	virtual void FindLocalizedPackages(const TMap<FString, TArray<FString>>& NewSourceToLocalizedPaths, TMap<FName, TArray<FName>>& InOutSourcePackagesToLocalizedPackages) = 0;

	/**
	 * Find all of the packages using the given asset group class, and update the PackageNameToAssetGroup map with the result.
	 *
	 * @param InAssetGroupName	The name of the asset group packages of this type belong to.
	 * @param InAssetClassName	The name of a class used by this asset group.
	 */
	virtual void FindAssetGroupPackages(const FName InAssetGroupName, const FTopLevelAssetPath& InAssetClassName) = 0;

	/**
	 * Try and find an existing cache for the given culture name, and create an entry for one if no such cache currently exists.
	 *
	 * @param InCultureName		The name of the culture to find the cache for.
	 *
	 * @return A pointer to the cache for the given culture.
	 */
	COREUOBJECT_API TSharedPtr<FPackageLocalizationCultureCache> FindOrAddCacheForCulture_NoLock(const FString& InCultureName);

	/**
	 * Update the mapping of package names to asset groups (if required).
	 */
	COREUOBJECT_API void ConditionalUpdatePackageNameToAssetGroupCache_NoLock();

	/**
	 * Callback handler for when a new content path is mounted.
	 *
	 * @param InAssetPath		The package path that was mounted, eg) /Game
	 * @param InFilesystemPath	The file-system path for the asset path, eg) ../../../MyGame/Content
	 */
	COREUOBJECT_API void HandleContentPathMounted(const FString& InAssetPath, const FString& InFilesystemPath);

	/**
	 * Callback handler for when an existing content path is dismounted.
	 *
	 * @param InAssetPath		The package path that was dismounted, eg) /Game
	 * @param InFilesystemPath	The file-system path for the asset path, eg) ../../../MyGame/Content
	 */
	COREUOBJECT_API void HandleContentPathDismounted(const FString& InAssetPath, const FString& InFilesystemPath);

	/**
	 * Callback handler for when the active culture is changed.
	 */
	COREUOBJECT_API void HandleCultureChanged();

protected:
	/** Critical section preventing concurrent access to our data. */
	mutable FCriticalSection LocalizedCachesCS;

	/** Pointer to the culture specific cache for the current culture. */
	TSharedPtr<FPackageLocalizationCultureCache> CurrentCultureCache;

	/** Mapping between a culture name, and the culture specific cache for that culture. */
	TArray<TTuple<FString, TSharedPtr<FPackageLocalizationCultureCache>>> AllCultureCaches;

	/** Mapping between a class name, and the asset group the class belongs to (for class specific package localization). */
	TArray<TTuple<FTopLevelAssetPath, FName>> AssetClassesToAssetGroups;

	/** Mapping between a package name, and the asset group it belongs to. */
	TMap<FName, FName> PackageNameToAssetGroup;
	bool bPackageNameToAssetGroupDirty;
};
