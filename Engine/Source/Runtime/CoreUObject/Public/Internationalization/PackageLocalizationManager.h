// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Misc/CoreDelegates.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#include <atomic>

class IPackageLocalizationCache;

/** Singleton class that manages localized package data. */
class FPackageLocalizationManager
{
public:
	typedef TFunction<void(FPackageLocalizationManager&)> FLazyInitFunc;

	/**
	 * Initialize the manager from the callback set by InitializeFromLazyCallback. It is expected that this callback calls one of the InitializeFromX functions.
	 */
	COREUOBJECT_API void PerformLazyInitialization();
	
	/**
	 * Initialize the manager lazily using the given callback. It is expected that this callback calls one of the InitializeFromX functions.
	 *
	 * @param InLazyInitFunc	The function to call to initialize the manager.
	 */
	COREUOBJECT_API void InitializeFromLazyCallback(FLazyInitFunc InLazyInitFunc);

	/**
	 * Initialize the manager using the given cache. This will perform an initial scan for localized packages.
	 *
	 * @param InCache	The cache the manager should use.
	 */
	COREUOBJECT_API void InitializeFromCache(const TSharedRef<IPackageLocalizationCache>& InCache);

	/**
	 * Initialize this manager using the default cache. This will perform an initial scan for localized packages.
	 */
	COREUOBJECT_API void InitializeFromDefaultCache();

	/**
	 * Try and find the localized package name for the given source package for the active culture.
	 *
	 * @param InSourcePackageName	The name of the source package to find.
	 *
	 * @return The localized package name, or NAME_None if there is no localized package.
	 */
	COREUOBJECT_API FName FindLocalizedPackageName(const FName InSourcePackageName);

	/**
	 * Try and find the localized package name for the given source package for the given culture.
	 *
	 * @param InSourcePackageName	The name of the source package to find.
	 * @param InCultureName			The name of the culture to find the package for.
	 *
	 * @return The localized package name, or NAME_None if there is no localized package.
	 */
	COREUOBJECT_API FName FindLocalizedPackageNameForCulture(const FName InSourcePackageName, const FString& InCultureName);

	/**
	 * Invalidate any cached state for the given root source path, and add it to the queue of things to process when ConditionalUpdateCache is called.
	 * eg) when new asset registry state is loaded for a plugin that may invalidate its cached data
	 */
	COREUOBJECT_API void InvalidateRootSourcePath(const FString& InRootPath);

	/**
	 * Update this cache, but only if it is dirty.
	 */
	COREUOBJECT_API void ConditionalUpdateCache();

	/**
	 * Singleton accessor.
	 *
	 * @return The singleton instance of the localization manager.
	 */
	static COREUOBJECT_API FPackageLocalizationManager& Get();

private:
	/**
	 * Try and find the localized package name for the given source package for the given culture, but without going through the cache.
	 *
	 * @param InSourcePackageName	The name of the source package to find.
	 * @param InCultureName			The name of the culture to find the package for.
	 *
	 * @return The localized package name, or NAME_None if there is no localized package.
	 */
	FName FindLocalizedPackageNameNoCache(const FName InSourcePackageName, const FString& InCultureName) const;

	/** Function to call to lazily initialize the manager. */
	FLazyInitFunc LazyInitFunc;

	/** Pointer to our currently active cache. Only valid after Initialize has been called. */
	TSharedPtr<IPackageLocalizationCache> ActiveCache;

	/** The ActiveCache pointer is not atomic, so use a real atomic test for initialization */
	std::atomic<bool> bIsInitialized = false;

	/** To make sure initialization is thread-safe. */
	UE::FMutex InitializationMutex;
};
