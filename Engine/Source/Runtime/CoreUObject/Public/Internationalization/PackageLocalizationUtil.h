// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

/** Utility functions for dealing with localized package names */
struct FPackageLocalizationUtil
{
	/**
	 * Converts a localized version of a package path to the source version (by removing /L10N/<code> from the package path)
	 * Note: This does not check whether the source package exists
	 *
	 * @param InLocalized	Path to the localized package.
	 * @param OutSource		Path to the source package. If the conversion failed, OutSource won't change.
	 *
	 * @returns True if the conversion happened, false otherwise (it will fail it the input was not a localized variant)
	 */
	static COREUOBJECT_API bool ConvertLocalizedToSource(const FStringView& InLocalized, FString& OutSource);

	/**
	 * Convert a package path to its source version (by removing /L10N/<code> from the package path, if present)
	 * Note: This does not check whether the source package exists
	 *
	 * @param InPath		Path to a package. It can be a source asset already or a localized variant.
	 * @param OutSource		Path to the source package. It cannot fail. If no /L10N/<code> were found, it is already a Source Asset.
	 */
	static COREUOBJECT_API void ConvertToSource(const FStringView& InPath, FString& OutSource);

	/** 
	 * Converts a source version of a package path to the localized version for the given culture (by adding /L10N/<code> to the package path)
	 * Note: This does not check whether the source package exists
	 *
	 * @param InSource		Path to the source package.
	 * @param InCulture		Culture code to use.
	 * @param OutLocalized	Path to the localized package. If the conversion failed, OutLocalized won't change.
	 *
	 * @returns True if the conversion happened, false otherwise
	 */
	static COREUOBJECT_API bool ConvertSourceToLocalized(const FStringView& InSource, const FStringView& InCulture, FString& OutLocalized);

	/**
	 * Converts a source version of a package path to the regex localized version (by adding "/L10N/" + "*" to the package path)
	 * Note: This does not check whether the source package exists
	 *
	 * @param InSource		Path to the source package.
	 * @param OutLocalized	Path to the regex localized package. If the conversion failed, OutLocalized won't change.
	 *
	 * @returns True if the conversion happened, false otherwise
	 */
	static COREUOBJECT_API bool ConvertSourceToRegexLocalized(const FStringView& InSource, FString& OutLocalized);

	/**
	 * Given a package path, get the localized root package for the given culture (eg, if given "/Game/MyFolder/MyAsset" and a culture of "fr", this would return "/Game/L10N/fr")
	 *
	 * @param InPath		Package path to use.
	 * @param InCulture		Culture code to use (if any).
	 * @param OutLocalized	Localized package root.
	 *
	 * @returns True if the conversion happened, false otherwise
	 */
	static COREUOBJECT_API bool GetLocalizedRoot(const FString& InPath, const FString& InCulture, FString& OutLocalized);

	/**
	 * Extract a culture code from a localized version of a package path (by finding /L10N/<code> from the package path, if present)
	 * Note: This does not check whether the source package exists
	 *
	 * @param InLocalized	Path to the localized package.
	 * @param OutCulture	Culture code (found from /L10N/<code>)
	 *
	 * @returns True if the conversion happened, false otherwise
	 */
	static COREUOBJECT_API bool ExtractCultureFromLocalized(const FStringView& InLocalized, FString& OutCulture);

	/**
	 * Given a package path, returns all localized variants absolute paths found on disk
	 * Note: There is a similar function to check in Revision Control (if not on disk) 
	 * (see USourceControlHelpers::GetLocalizedVariantsAbsolutePaths)
	 *
	 * @param InPackageName				Package path to use.
	 * @param OutLocalizedAbsolutePaths	Absolute paths of localized variants founds
	 */
	static COREUOBJECT_API void GetLocalizedVariantsAbsolutePaths(const FStringView& InSource, TArray<FString>& OutLocalizedAbsolutePaths);
};
