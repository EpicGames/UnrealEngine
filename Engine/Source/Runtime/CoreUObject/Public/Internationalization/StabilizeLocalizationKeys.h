// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

class FProperty;
class UStruct;

namespace StabilizeLocalizationKeys
{

#if WITH_EDITOR

/**
 * Convert any text that has been initialized from a raw string to have a deterministic key based on the given key root and the property name.
 * @note This function will recurse into arrays, sets, maps, and sub-structures.
 */
COREUOBJECT_API void StabilizeLocalizationKeysForProperty(FProperty* InProp, void* InPropData, const FString& InNamespace, const FString& InKeyRoot, const bool bAppendPropertyNameToKey = true);

/**
 * Walk through the struct and convert any text that has been initialized from a raw string to have a deterministic key based on the given key root and the property name.
 * @note This function will recurse into arrays, sets, maps, and sub-structures.
 */
COREUOBJECT_API void StabilizeLocalizationKeysForStruct(UStruct* InStruct, void* InStructData, const FString& InNamespace, const FString& InKeyRoot);

#endif	// WITH_EDITOR

}
