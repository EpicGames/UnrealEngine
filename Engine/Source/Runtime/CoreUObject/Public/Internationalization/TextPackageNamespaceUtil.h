// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Internationalization/TextNamespaceUtil.h"
#include "Templates/PimplPtr.h"

class FTextProperty;
class UObject;
class UPackage;
struct FPackageFileSummary;

namespace TextNamespaceUtil
{

#if USE_STABLE_LOCALIZATION_KEYS

/**
* Given a package, all texts found in all objects within the package will be updated to actually use
* the package namespace on its text.
*/
class FUpdateTextsPackageNamespace
{
public:
	typedef TFunction<void(UObject*, FUpdateTextsPackageNamespace&)> FProcessObjectCallback;
	typedef TMap<const UClass*, FProcessObjectCallback> FProcessObjectCallbackMap;

	// Main Entry point to this class
	static void UpdatePackage(UPackage* InPackage);

	// Callback map to customize this class
	COREUOBJECT_API static FProcessObjectCallbackMap& GetTypeSpecificProcessObjectCallbacks();

	// This function can be used during a custom callback to update one FText
	COREUOBJECT_API bool UpdateSingleText(FText& InOutText);

	// This function can be used during a custom callback to use the default serialize
	COREUOBJECT_API void Serialize(UObject* InObject);

private:
	FUpdateTextsPackageNamespace() = delete;
	FUpdateTextsPackageNamespace(UPackage* InPackage);

	class FImpl;
	TPimplPtr<FImpl> PImpl;
};

/** Struct to automatically register a UpdateTextsPackageNamespace callback when it's constructed */
struct FAutoRegisterLocalizationUpdateTextsPackageNamespaceCallback
{
	UE_FORCEINLINE_HINT FAutoRegisterLocalizationUpdateTextsPackageNamespaceCallback(const UClass* InClass, const FUpdateTextsPackageNamespace::FProcessObjectCallback& InCallback)
	{
		FUpdateTextsPackageNamespace::GetTypeSpecificProcessObjectCallbacks().Add(InClass, InCallback);
	}
};

/**
 * Generate a deterministic package namespace based on the given package info.
 * @note This key will be formatted like a GUID, but the value will actually be based on deterministic hashes.
 *
 * @param InPackage						The package to generate the namespace for.
 */
COREUOBJECT_API FString GenerateDeterministicPackageNamespace(const UPackage* InPackage);

/**
 * Check if the given package uses its deterministictly generatable namespace.
 *
 * @param InPackage					The package to check the namespace for
 */
COREUOBJECT_API bool HasDeterministicPackageNamespace(const UPackage* InPackage);

/**
 * Check if the given UObject uses its deterministictly generatable namespace.
 *
 * @param InObject					The UObject to check the namespace for (from its owner package)
 */
COREUOBJECT_API bool HasDeterministicPackageNamespace(UObject* InObject);

/**
 * Check if the given package file summary uses its deterministictly generatable namespace.
 *
 * @param InPackagePath					The package to check the namespace for
 * @param InPackageFileSummary				The package current Loc ID.
 */
COREUOBJECT_API bool HasDeterministicPackageNamespace(const FString& InPackagePath, const FPackageFileSummary& InPackageFileSummary);

/**
 * Given a package, make sure it uses its deterministic package namespace if USE_STABLE_LOCALIZATION_KEYS is enabled.
 * If not or if USE_STABLE_LOCALIZATION_KEYS is disabled, reset it and fix up any text in the package.
 *
 * @param InPackage			The package to reset the namespace for.
 *
 * @return True if the reset occurred. False if the package namespace is already the deterministic one.
 */
COREUOBJECT_API bool ResetPackageNamespace(UPackage* InPackage);

/**
 * Given an object, make sure it uses its deterministic package namespace (from its owner package) if USE_STABLE_LOCALIZATION_KEYS is enabled.
 * If not or if USE_STABLE_LOCALIZATION_KEYS is disabled, reset it and fix up any text in the package.
 *
 * @param InObject			The object to clear the namespace for.
 *
 * @return True if the reset occurred. False if the package namespace is already the deterministic one..
 */
COREUOBJECT_API bool ResetPackageNamespace(UObject* InObject);

/**
 * Given a package, try and get the namespace it should use for localization.
 *
 * @param InPackage			The package to try and get the namespace for.
 *
 * @return The package namespace, or an empty string if the package has no namespace.
 */
COREUOBJECT_API FString GetPackageNamespace(const UPackage* InPackage);

/**
 * Given an object, try and get the namespace it should use for localization (from its owner package).
 *
 * @param InObject			The object to try and get the namespace for.
 *
 * @return The package namespace, or an empty string if the package has no namespace.
 */
COREUOBJECT_API FString GetPackageNamespace(const UObject* InObject);

/**
 * Given a package, try and ensure it has a namespace it should use for localization.
 *
 * @param InPackage			The package to try and get the namespace for.
 *
 * @return The package namespace, or an empty string if the package has no namespace and one could not be added.
 */
COREUOBJECT_API FString EnsurePackageNamespace(UPackage* InPackage);

/**
 * Given an object, try and ensure it has a namespace it should use for localization (from its owner package).
 *
 * @param InObject			The object to try and get the namespace for.
 *
 * @return The package namespace, or an empty string if the package has no namespace and one could not be added.
 */
COREUOBJECT_API FString EnsurePackageNamespace(UObject* InObject);

/**
 * Given a package, clear any namespace it has set for localization.
 *
 * @param InPackage			The package to clear the namespace for.
 */
COREUOBJECT_API void ClearPackageNamespace(UPackage* InPackage);

/**
 * Given an object, clear any namespace it has set for localization (from its owner package).
 *
 * @param InObject			The object to clear the namespace for.
 */
COREUOBJECT_API void ClearPackageNamespace(UObject* InObject);

/**
 * Given a package, force it to have the given namespace for localization (even if a transient package!).
 *
 * @param InPackage			The package to set the namespace for.
 * @param InNamespace		The namespace to set.
 */
COREUOBJECT_API void ForcePackageNamespace(UPackage* InPackage, const FString& InNamespace);

/**
 * Given an object, force it to have the given namespace for localization (from its owner package, even if a transient package!).
 *
 * @param InObject			The object to set the namespace for.
 * @param InNamespace		The namespace to set.
 */
COREUOBJECT_API void ForcePackageNamespace(UObject* InObject, const FString& InNamespace);

#endif // USE_STABLE_LOCALIZATION_KEYS

/**
 * Make a copy of the given text that's valid to use with the given package, optionally preserving its existing key.
 * @note Returns the result verbatim if there is no change when applying the package namespace to the text.
 *
 * @param InText						The current FText instance.
 * @param InPackage/InObject			The package (or object to get the owner package from) to get the namespace for (will call EnsurePackageNamespace).
 * @param InCopyMethod					The method that should be used to copy the FText instance.
 * @param bAlwaysApplyPackageNamespace	If true, this will always apply the package namespace to the text namespace (always treated as ETextCopyMethod::Verbatim when USE_STABLE_LOCALIZATION_KEYS is false).
 *										If false, this will only apply the package namespace if the text namespace already contains package namespace markers.
 *
 * @return A copy of the given text that's valid to use with the given package.
 */
COREUOBJECT_API FText CopyTextToPackage(const FText& InText, UPackage* InPackage, const ETextCopyMethod InCopyMethod = ETextCopyMethod::NewKey, const bool bAlwaysApplyPackageNamespace = false);
COREUOBJECT_API FText CopyTextToPackage(const FText& InText, UObject* InObject, const ETextCopyMethod InCopyMethod = ETextCopyMethod::NewKey, const bool bAlwaysApplyPackageNamespace = false);

/**
 * Generate a random text key.
 * @note This key will be a GUID.
 */
COREUOBJECT_API FString GenerateRandomTextKey();

/**
 * Generate a deterministic text key based on the given object and property info.
 * @note This key will be formatted like a GUID, but the value will actually be based on deterministic hashes.
 *
 * @param InTextOwner					The object that owns the given TextProperty.
 * @param InTextProperty				The text property to generate the key for.
 * @param bApplyPackageNamespace		If true, apply the package namespace to the generated key hash (when USE_STABLE_LOCALIZATION_KEYS is true).
 */
COREUOBJECT_API FString GenerateDeterministicTextKey(UObject* InTextOwner, const FTextProperty* InTextProperty, const bool bApplyPackageNamespace = true);
COREUOBJECT_API FString GenerateDeterministicTextKey(UObject* InTextOwner, const FName InTextPropertyName, const bool bApplyPackageNamespace = true);

enum class ETextEditAction : uint8
{
	Namespace,
	Key,
	SourceString,
};

/**
 * Called when editing a text property to determine the new ID for the text, ideally using the proposed text ID when possible (and when USE_STABLE_LOCALIZATION_KEYS is true).
 * 
 * @param InPackage						The package to query the namespace for.
 * @param InEditAction					How has the given text been edited?
 * @param InTextSource					The current source string for the text being edited. Can be empty when InEditAction is ETextEditAction::SourceString.
 * @param InProposedNamespace			The namespace we'd like to assign to the edited text.
 * @param InProposedKey					The key we'd like to assign to the edited text.
 * @param OutStableNamespace			The namespace that should be assigned to the edited text.
 * @param OutStableKey					The key that should be assigned to the edited text.
 * @param InTextKeyGenerator			Generator for the new text key. Will generate a random key by default.
 * @param bApplyPackageNamespace		If true, apply the package namespace to the generated text ID (when USE_STABLE_LOCALIZATION_KEYS is true).
 */
COREUOBJECT_API void GetTextIdForEdit(UPackage* InPackage, const ETextEditAction InEditAction, const FString& InTextSource, const FString& InProposedNamespace, const FString& InProposedKey, FString& OutStableNamespace, FString& OutStableKey, TFunctionRef<FString()> InTextKeyGenerator = &GenerateRandomTextKey, const bool bApplyPackageNamespace = true);

/**
 * Edit an attribute of the given text property, akin to what happens when editing a text property in a details panel.
 *
 * @param InTextOwner					The object that owns the given TextProperty to be edited.
 * @param InTextProperty				The text property to edit. This must be a property that exists on TextOwner.
 * @param InEditAction					How has the given text been edited?
 * @param InEditValue					The new value of the attribute that was edited.
 * @param InTextKeyGenerator			Generator for the new text key. Will generate a random key by default.
 * @param bApplyPackageNamespace		If true, apply the package namespace to the generated text ID (when USE_STABLE_LOCALIZATION_KEYS is true).
 * 
 * @return True if edit was possible, or false if not.
 */
COREUOBJECT_API bool EditTextProperty(UObject* InTextOwner, const FTextProperty* InTextProperty, const ETextEditAction InEditAction, const FString& InEditValue, TFunctionRef<FString()> InTextKeyGenerator = &GenerateRandomTextKey, const bool bApplyPackageNamespace = true);

/**
 * Edit an attribute of the given text property, akin to what happens when editing a text property in a details panel.
 *
 * @param InPackage						The package that hosts the text value.
 * @param InTextValue					The raw value of the TextProperty to be edited.
 * @param InTextProperty				The text property to edit.
 * @param InEditAction					How has the given text been edited?
 * @param InEditValue					The new value of the attribute that was edited.
 * @param InTextKeyGenerator			Generator for the new text key. Will generate a random key by default.
 * @param bApplyPackageNamespace		If true, apply the package namespace to the generated text ID (when USE_STABLE_LOCALIZATION_KEYS is true).
 *
 * @return True if edit was possible, or false if not.
 */
COREUOBJECT_API bool EditTextProperty_Direct(UPackage* InPackage, void* InTextValue, const FTextProperty* InTextProperty, const ETextEditAction InEditAction, const FString& InEditValue, TFunctionRef<FString()> InTextKeyGenerator = &GenerateRandomTextKey, const bool bApplyPackageNamespace = true);

}
