// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"

namespace UE::AssetAccessRestrictions
{
	/** Is path allowed to reference Epic Internal assets */
	extern COREUOBJECT_API TDelegate<bool(FStringView)> IsPathAllowedToReferenceEpicInternalAssets;
}