// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/AssetReadLoggerConfig.h"

#if UE_ENABLE_ASSET_READ_LOGGER

#include "CoreMinimal.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "IO/PackageId.h"
#include "Misc/PackagePath.h"
#include "Misc/ScopeLock.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

class IPakFile;

enum class EAssetContainerType : uint8
{
	Unknown = 0,
	LooseFile = 1,        // Not in any container (loose file on disk)
	Pak = 2,              // Traditional .pak file
	IoStoreOnDemand = 3,  // IAD - IoStore on demand (CDN/DLC content)
	IoStoreStreaming = 4, // IAS - IoStore streaming (level streaming)
	IoStoreLocal = 5      // Standard IoStore container (local disk)
};

COREUOBJECT_API const TCHAR* LexToString(EAssetContainerType ContainerType);

struct FAssetContainerInfo
{
	EAssetContainerType Type = EAssetContainerType::Unknown;
	int32 ChunkID = -1;
	FString Name;
	bool bIsValid = false;
};

struct FAssetContainerStats
{
	int32 Count = 0;
	int64 TotalSize = 0;
};

struct FAssetReadInfo
{
	FString AssetPath;
	FString MountPoint;  // Virtual filesystem root (e.g., /Game/, /Engine/, /BRCosmetics/)
	EAssetContainerType ContainerType = EAssetContainerType::Unknown;
	int32 ChunkID = -1;
	FString ContainerName;
	int64 SizeBytes = 0;
	double LoadTimestamp = 0.0;
	uint64 PathHash = 0;
	FString ThreadName;  // Thread from which the asset was loaded

	FAssetReadInfo() = default;

	FAssetReadInfo(const FString& InAssetPath, EAssetContainerType InContainerType, int32 InChunkID,
		const FString& InContainerName, int64 InSize, double InTime,
		const FString& InThreadName = FString());

	static FString ExtractMountPoint(const FString& AssetPath);
	const TCHAR* GetContainerTypeString() const;

	bool operator==(const FAssetReadInfo& Other) const { return PathHash == Other.PathHash; }
	friend uint32 GetTypeHash(const FAssetReadInfo& Info) { return GetTypeHash(Info.PathHash); }
	bool operator<(const FAssetReadInfo& Other) const { return AssetPath < Other.AssetPath; }
};

/**
 * Asset Read Logger - tracks unique asset reads during runtime
 *
 * Logs each unique package load to CSV for profiling and optimization analysis.
 * Tracks container type (Pak, IAD, IAS, IoStore, Loose), chunk ID, size, and
 * which thread initiated the load.
 *
 * Usage:
 *   -LogAssetReads              Output to Saved/Profiling/AssetReads_<timestamp>.csv
 *   -LogAssetReads=<path.csv>   Output to specified path
 *
 * Console command:
 *   AssetReadLogger.SweepAndFlush   Sweep for missed packages and flush CSV to disk
 *
 * CSV columns: AssetPath, MountPoint, ContainerType, ChunkID, ContainerName,
 *              SizeBytes, LoadTimestamp, Thread
 */
class FAssetReadLogger
{
public:
	COREUOBJECT_API static FAssetReadLogger& Get();

	/** Returns true if logging was enabled via command line */
	COREUOBJECT_API bool Initialize();

	COREUOBJECT_API bool IsEnabled() const;

	COREUOBJECT_API void LogAssetRead(const FString& AssetPath, EAssetContainerType ContainerType,
		int32 ChunkID, const FString& ContainerName,
		int64 SizeBytes, const FString& ThreadName = FString());

	/** Writes final CSV and unregisters hooks */
	COREUOBJECT_API void Shutdown();

	/** Sweep + flush without stopping the logger */
	COREUOBJECT_API void SweepAndFlushToFile();

	COREUOBJECT_API int64 GetUniqueAssetCount() const;
	COREUOBJECT_API int64 GetTotalReadAttempts() const;

	/** Must be called from GameThread */
	COREUOBJECT_API void SweepForMissedPackages();

	/**
	 * Computes aggregate stats from the in-memory tracked asset reads and pushes them
	 * to CsvProfiler SetMetadata under `assetreads_*`. Called automatically at the end
	 * of Shutdown() and SweepAndFlushToFile()
	 */
	COREUOBJECT_API void AppendCSVMetadataStats() const;

private:
	FAssetReadLogger();
	~FAssetReadLogger();

	FAssetReadLogger(const FAssetReadLogger&) = delete;
	FAssetReadLogger& operator=(const FAssetReadLogger&) = delete;

	/** Data extracted from a UPackage on the GameThread so processing does not retain a UObject reference. */
	struct FPackageLoadSnapshot
	{
		FString AssetPath;
		FString PackageName;
		FPackagePath LoadedPath;
		FPackageId PackageId;
		int64 PreloadedFileSize = 0;
		FString ThreadName;
	};

	void RegisterAssetLoadingHooks();
	void UnregisterAssetLoadingHooks();

	/** Bound to FCoreUObjectDelegates::OnEndLoadPackage. Snapshots each package on the GameThread and processes container detection / logging synchronously. */
	void OnEndLoadPackage(const FEndLoadPackageContext& Context);

	/** Performs container detection, size lookup, and logging for a batch of snapshots synchronously on the calling thread. */
	void ProcessSnapshots(TArray<FPackageLoadSnapshot> Snapshots);

	/**
	 * Populates a snapshot from a UPackage on the GameThread. Returns false if Package is null or its
	 * path is not trackable. ThreadName is left empty — the caller assigns it (thread attribution for
	 * delegate-driven loads, or a fixed tag like "PreInit"/"Sweep" for bulk scans).
	 */
	static bool TryCapturePackageSnapshot(UPackage* Package, FPackageLoadSnapshot& OutSnapshot);

	/** GameThread only */
	void OnSyncLoadPackageStarted(const FString& PackageName);

	static FString GetCurrentThreadName();

	FAssetContainerInfo DetectAssetContainer(const FPackageLoadSnapshot& Snapshot) const;
	FAssetContainerInfo GetIoStoreContainerInfo(FPackageId PackageId, const FString& PackageName) const;
	FAssetContainerInfo GetPakContainerInfo(const FString& PackageFileName) const;

	/** Uses container name patterns and flags to determine IAD/IAS/local */
	EAssetContainerType DetermineIoStoreType(const FString& ContainerName) const;

	/** Returns 0 if not cached */
	uint8 GetCachedContainerFlags(const FString& ContainerName) const;

	bool IsOnDemandContainer(uint8 Flags) const;

	/** e.g., "pakchunk10-WindowsClient.pak" -> 10 */
	int32 ExtractChunkIDFromPakFilename(const FString& FileName) const;

	int64 GetPackageSizeBytes(const FPackageLoadSnapshot& Snapshot) const;
	FString FormatCSVLine(const FAssetReadInfo& Asset) const;
	static FString EscapeCSVField(const FString& Field);
	void PrintContainerStatistics() const;

	/** Writes "# METADATA: {...}" JSON line with platform, engine version, build config */
	void WriteMetadataBlock();

	/** Caches package -> container mapping from pak mount events */
	void OnPakFileMounted(const IPakFile& PakFile);

	FAssetContainerInfo LookupCachedContainerInfo(const FString& PackageName) const;

	/** Logs packages loaded before Initialize() was called */
	void BackfillLoadedPackages();

	/** Thread-safe, fires on requesting thread */
	void OnAsyncLoadPackageRequested(FStringView PackageName);

	/** For coverage verification */
	int32 CountLoadedPackages() const;

	/** Filters out /Temp/, /Memory/, /Script/, transient packages */
	static bool IsTrackableAssetPath(const FString& AssetPath);

private:
	std::atomic<bool> bIsEnabled{false};
	FString OutputFilePath;
	TUniquePtr<FArchive> OutputFileArchive;
	mutable FCriticalSection OutputFileLock;
	double StartTime = 0.0;

	TSet<FAssetReadInfo> UniqueAssetReads;
	mutable FRWLock AssetReadLock;

	std::atomic<int64> TotalReadsAttempted{0};
	std::atomic<int64> UniqueReadsLogged{0};
	TMap<EAssetContainerType, FAssetContainerStats> StatsByContainerType;

	FDelegateHandle PackageLoadedDelegateHandle;
	FDelegateHandle SyncLoadPackageDelegateHandle;      // GameThread only

	// Tracks sync loads that may not trigger OnEndLoadPackage immediately
	TSet<FString> PendingSyncLoadPackages;
	mutable FCriticalSection PendingSyncLoadLock;

	// Built from pak mount events for container detection
	struct FCachedContainerInfo
	{
		FString ContainerName;
		int32 ChunkID = 0;
		EAssetContainerType Type = EAssetContainerType::Pak;
	};
	TMap<FString, FCachedContainerInfo> PackageToContainerCache;  // Key: lowercase normalized path
	mutable FRWLock PackageContainerCacheLock;

	TMap<FString, uint8> ContainerFlagsCache;  // Container name -> EIoContainerFlags (for IAD detection)
	mutable FRWLock ContainerFlagsCacheLock;

	FDelegateHandle PakFileMountedDelegateHandle;
	FDelegateHandle AsyncLoadPackageDelegateHandle;

	// Maps package name -> requesting thread (for accurate async load attribution)
	TMap<FString, FString> AsyncLoadRequestThreads;
	mutable FCriticalSection AsyncLoadRequestLock;

	FDelegateHandle EngineInitCompleteDelegateHandle;  // For deferred backfill
};

#endif // UE_ENABLE_ASSET_READ_LOGGER
