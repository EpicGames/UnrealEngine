// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#ifndef UE_ENABLE_ASSET_READ_LOGGER
// Controls whether the asset read logger is compiled in.
// Default: disabled. Define to 1 via Build.cs / UBT (target-specific) to opt in.
// Note: even when compiled in, -LogAssetReads is still required at runtime to activate logging.
// This only gates the logger itself: FCoreUObjectDelegates::OnEndLoadPackage and UPackage's
// end-loaded state remain available in their usual editor builds independently of this switch.
#define UE_ENABLE_ASSET_READ_LOGGER 0
#endif
