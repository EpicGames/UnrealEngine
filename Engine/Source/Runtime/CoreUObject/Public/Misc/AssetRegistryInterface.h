// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Set.h"
#include "CoreTypes.h"
#include "Misc/EnumClassFlags.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"
#include "UObject/TopLevelAssetPath.h"

class FLinkerSave;
class FStructuredArchiveRecord;
class ITargetPlatform;
class UObject;
class UPackage;
struct FArchiveCookContext;
struct FARFilter;
struct FAssetData;

class IAssetRegistry;
class UAssetRegistryImpl;

namespace UE::AssetRegistry
{

	/**
	 * Multiple meanings of dependency are used in the AssetRegistry; the category specifes which meaning is referred to.
	 * During queries for dependencies, the Category variable can be a bitfield combination of Category values, in which case dependencies in any of the specified categories are returned.
	 */
	// 
	enum class EDependencyCategory : uint8
	{
		// This enum is serialized by value into the runtime AssetRegistry and the AssetDataGatherer.
		// If any values change or are removed, bump FAssetRegistryVersion and add backwards compatibility in FAssetRegistryState::Serialize.

		Package = 0x01,			// The target asset of any package dependency is expected to be loadable whenever the source asset is available; see EDependencyProperty for different requirements of the loadability.
		Manage = 0x02,			// The target asset of any manage dependency is managed (e.g. given a disk layout location) either directly or indirectly by the source asset. Used by UAssetManager.
		SearchableName = 0x04,  // Targets of SearchableName dependencies are FNames Keys inside of an Asset. The Source Asset of the dependency has a value for that Key. Used to search for Assets with a given Key,Value for the custom Key.

		None = 0x0,
		All = Package | Manage | SearchableName,
	};
	ENUM_CLASS_FLAGS(EDependencyCategory);

	/**
	 * Properties that might be possessed by a dependency. Each property is specific to a EDependencyCategory value.
	 */
	enum class EDependencyProperty : uint8
	{
		None = 0,
		// This enum is serialized by value into the runtime AssetRegistry and the AssetDataGatherer.
		// If any values change or are removed, bump FAssetRegistryVersion and add backwards compatibility in FAssetRegistryState::Serialize.

		// Package Dependencies
		/**
		 * The target asset must be loaded before the source asset can finish loading.The lack of this property is
		 * known as a Soft dependency, and indicates only that the source asset expects the target asset to be
		 * loadable on demand.
		 */
		Hard = 1 << 0,
		/**
		 * The target asset is needed in the game as well as the editor. The lack of this property is known as an
		 * EditorOnly dependency.
		 */
		Game = 1 << 1,
		/**
		 * Fields on the target asset are used in the transformation of the source asset during cooking in addition
		 * to being required in the game or editor. The lack of this property indicates that the target asset is
		 * required in game or editor, but is not required during cooking.
		 */
		Build = 1 << 2,
		PackageMask = Hard | Game | Build,

		// SearchableName Dependencies (None Yet)
		SearchableNameMask = None,

		// ManageDependencies
		/**
		 * The target asset was specified explicitly as a managee by the source asset. Lack of this property is known 
		 * as an indirect dependency; the target asset is reachable by following the transitive closure of Direct
		 * Manage Dependencies and Package dependencies from the source asset.
		 */
		Direct = 1 << 3,
		/**
		 * The manager manages the target for the purposes of deciding its CookRule in addition to deciding its Chunk
		 * assignment. Lack of this property is known as a ChunkOnly dependency. Every manage dependency used for
		 * deciding the CookRule also decides the Chunk; Chunk assignment is a superset of CookRule assignment.
		 */
		CookRule = 1 << 4,
		ManageMask = Direct | CookRule,


		AllMask = PackageMask | SearchableNameMask | ManageMask,
	};
	ENUM_CLASS_FLAGS(EDependencyProperty);

	/**
	 * Flags that specify required properties (or required-not-present properties) for a dependency to be returned from a query.
	 * Values in this enum correspond to values in EDependencyProperty; each EDependencyProperty value has a positive and negative equivalent in this enum.
	 * This allows a single bitfield to indicate required-present, required-not-present, or dont-care for each property.
	 * For any category-specific values, those values apply only to dependencies in the category, and do not impose restrictions on dependencies from other categories.
	 */
	enum class EDependencyQuery : uint32
	{
		NoRequirements = 0,

		// Package Dependencies Only
		/** Return only dependencies with EDependencyProperty::Hard. */
		Hard = 1 << 0,
		/** Return only dependencies without EDependencyProperty::Hard. */
		NotHard = 1 << 1,
		Soft = NotHard,

		/** Return only dependencies with EDependencyProperty::Game. */
		Game = 1 << 2,
		/** Return only dependencies without EDependencyProperty::Game. */
		NotGame = 1 << 3,
		EditorOnly = NotGame,

		/** Return only dependencies with EDependencyProperty::Build. */
		Build = 1 << 4,
		/** Return only dependencies without EDependencyProperty::Build. */
		NotBuild = 1 << 5,

		/**
		 * Return only dependencies that cause propagation of manage dependencies, which means
		 * either Game or Build. Presence of the Propagation flag in a query causes the Game, NotGame, EditorOnly,
		 * Build, and NotBuild flags to be ignored in the query if present. Either Game or Build is sufficient to pass
		 * the query no matter which of those other flags are present. Hard vs Soft is still respected, and Soft Build
		 * or Soft Game dependencies will be skipped query if Hard is required.
		 */
		Propagation = 1 << 6,

		// Manage Dependencies Only
		/** Return only dependencies with EDependencyProperty::Direct. */
		Direct = 1 << 8,
		/** Return only dependencies without EDependencyProperty::Direct. */
		NotDirect = 1 << 9,
		Indirect = NotDirect,

		/** Return only dependencies with EDependencyProperty::CookRule. */
		CookRule = 1 << 10,
		/** Return only dependencies without EDependencyProperty::CookRule. */
		NotCookRule = 1 << 11,
		ChunkOnly = NotCookRule,

		// Masks used for manipulating EDependencyQuerys
		PackageMask = Hard | NotHard | Game | NotGame | Build | NotBuild,
		SearchableNameMask = 0x0000, // None yet
		ManageMask = Direct | NotDirect | CookRule | NotCookRule,
	};
	ENUM_CLASS_FLAGS(EDependencyQuery);

	/** Options used to read/write the DevelopmentAssetRegistry when serializing, which includes all data */
	enum class ESerializationTarget : uint8
	{
		ForGame,
		ForDevelopment
	};

	/**
	 * Return values for AssetRegistry functions
	 */
	enum class EExists
	{
		DoesNotExist,	// Does not exist on disk
		Exists,			// Exists on disk
		Unknown,		// Not known. AssetRegistry might still be indexing
	};

	enum class EEnumerateAssetsFlags : uint32
	{
		/**
		 * No flags. This implies a request to filter by some properties, because filtering is the default state that
		 * that is turned off by some flags; see e.g. AllowUnfilteredAssets.
		 **/
		None = 0,
		/** If set, only DiskGatheredData are returned, AssetDatas are not created from in-memory UObjects. */
		OnlyOnDiskAssets = (1 << 0),
		/**
		 * If set, unmounted asset paths (!FPackageName::IsValidPath) are returned.
		 * When not set (the default), they are skipped.
		 */
		AllowUnmountedPaths = (1 << 1),
		/**
		 * If set, asset paths are returned even if UE::AssetRegistry::FFiltering::ShouldSkipAsset is true for them.
		 * When not set (the default), they are skipped.
		 */
		AllowUnfilteredArAssets = (1 << 2),
		/**
		 * All allowflags are set, no discovered assets are filtered. Does not impact memory vs OnDiskAssets; in-memory
		 * assets may still be skipped even if this flag is set, depending on whether OnlyOnDiskAssets is set.
		 */
		AllowAll = AllowUnmountedPaths | AllowUnfilteredArAssets,
		/**
		 * Enumeration of the Assets is done with ParallelFor; the callback must be threadsafe. This flag is a request
		 * only, and some functions may not support it or may ignore it if not performant; caller must behave correctly
		 * if callbacks are called singlethreadedly.
		 */
		Parallel = (1 << 3),
	};
	ENUM_CLASS_FLAGS(EEnumerateAssetsFlags);
	
	enum class EEnumeratePackagesFlags : uint32
	{
		None = 0,							// No flags
		Parallel = (1 << 0),	            // Iterate with a ParallelFor
	};
	ENUM_CLASS_FLAGS(EEnumeratePackagesFlags);

	/**
	 * A struct that is equivalent to EDependencyQuery, but is more useful for performance in filtering operations.
	 * This is used by the filter implementations inside of GetDependency/GetReferencer calls; callers of those functions can instead use the more convenient values in EDependencyQuery.
	 */
	struct FDependencyQuery
	{
		/**
		 * Only Dependencies that possess all of these properties will be returned.
		 * Note that flags specific to another EDependencyCategory are ignored when querying dependencies in a given category.
		 */
		UE::AssetRegistry::EDependencyProperty Required;
		/**
		 * Only Dependencies that possess none of these properties will be returned.
		 * Note that flags specific to another EDependencyCategory are ignored when querying dependencies in a given category.
		 */
		UE::AssetRegistry::EDependencyProperty Excluded;

		/**
		 * RequiredUnions is an intersection of unions. Each element of RequiredUnions is a set of bit flags that are
		 * unioned: having any one of the bit flags causes that element of RequiredUnions to pass.
		 * After pass/fail is decided for each element, they are intersected: all must pass for the total to pass.
		 * This allows RequiredUnions to be a conjunction of disjunctions, whereas the Required field is just a conjunction
		 * of atoms.
		 */
		TArray<UE::AssetRegistry::EDependencyProperty, TInlineAllocator<1>> RequiredUnions;

		FDependencyQuery()
		{
			Required = UE::AssetRegistry::EDependencyProperty::None;
			Excluded = UE::AssetRegistry::EDependencyProperty::None;
		}

		inline FDependencyQuery(EDependencyQuery QueryFlags)
		{
			if (!EnumHasAnyFlags(QueryFlags, EDependencyQuery::Propagation))
			{
				Required = (!!(QueryFlags & EDependencyQuery::Game) ? UE::AssetRegistry::EDependencyProperty::Game : UE::AssetRegistry::EDependencyProperty::None)
					| (!!(QueryFlags & EDependencyQuery::Build) ? UE::AssetRegistry::EDependencyProperty::Build : UE::AssetRegistry::EDependencyProperty::None);
			}
			else
			{
				Required = UE::AssetRegistry::EDependencyProperty::None;
				EnumRemoveFlags(QueryFlags, EDependencyQuery::Game | EDependencyQuery::NotGame | EDependencyQuery::Build | EDependencyQuery::NotBuild);
				RequiredUnions.Add(UE::AssetRegistry::EDependencyProperty::Game | UE::AssetRegistry::EDependencyProperty::Build);
			}
			Required |= (!!(QueryFlags & EDependencyQuery::Hard) ? UE::AssetRegistry::EDependencyProperty::Hard : UE::AssetRegistry::EDependencyProperty::None)
				| (!!(QueryFlags & EDependencyQuery::Direct) ? UE::AssetRegistry::EDependencyProperty::Direct : UE::AssetRegistry::EDependencyProperty::None)
				| (!!(QueryFlags & EDependencyQuery::CookRule) ? UE::AssetRegistry::EDependencyProperty::CookRule : UE::AssetRegistry::EDependencyProperty::None);

			Excluded = (!!(QueryFlags & EDependencyQuery::NotHard) ? UE::AssetRegistry::EDependencyProperty::Hard : UE::AssetRegistry::EDependencyProperty::None)
				| (!!(QueryFlags & EDependencyQuery::NotGame) ? UE::AssetRegistry::EDependencyProperty::Game : UE::AssetRegistry::EDependencyProperty::None)
				| (!!(QueryFlags & EDependencyQuery::NotBuild) ? UE::AssetRegistry::EDependencyProperty::Build : UE::AssetRegistry::EDependencyProperty::None)
				| (!!(QueryFlags & EDependencyQuery::NotDirect) ? UE::AssetRegistry::EDependencyProperty::Direct : UE::AssetRegistry::EDependencyProperty::None)
				| (!!(QueryFlags & EDependencyQuery::NotCookRule) ? UE::AssetRegistry::EDependencyProperty::CookRule : UE::AssetRegistry::EDependencyProperty::None);
		}
	};

	struct FWritePackageDataArgs
	{
		// Required inputs, must be initialized and non-null
		FStructuredArchiveRecord* ParentRecord = nullptr;
		const UPackage* Package = nullptr;
		FLinkerSave* Linker = nullptr;
		const TSet<TObjectPtr<UObject>>* ImportsUsedInGame = nullptr;
		const TSet<FName>* SoftPackagesUsedInGame = nullptr;
		const TArray<FName>* PackageBuildDependencies = nullptr;
		bool bProceduralSave = false;

		// Optional inputs that may be null
		FArchiveCookContext* CookContext = nullptr;

		// Optional outputs that may be null
		TArray<FAssetData>* OutAssetDatas = nullptr;
	};
	/**
	 * Bitfield of flags written into a package's AssetRegistry DependencyData section to represent what kind of
	 * dependency is stored for each PackageName in ExtraPackageDependencies. Values are serialized as integers;
	 * new bits can be added as necessary, but the integer values for existing enum values may not be changed.
	 */
	enum class EExtraDependencyFlags : uint32
	{
		None					= 0,
		Build					= 0x1,
		PropagateManage			= 0x2,
	};
	ENUM_CLASS_FLAGS(EExtraDependencyFlags);

	/**
	 * Writes the data used by the AssetRegistry in each package; the format of this data is separate from the
	 * format of the data in the asset registry.
	 * The corresponding read functions are ReadPackageDataMain and ReadPackageDataDependencies; they are are declared
	 * in IAssetRegistry.h, in the AssetRegistry module, because they depend upon some structures defined in the
	 * AssetRegistry module
	 */
	COREUOBJECT_API void WritePackageData(FWritePackageDataArgs& Args);
	UE_DEPRECATED(5.5, "Use version that takes FWritePackageDataArgs");
	COREUOBJECT_API void WritePackageData(FStructuredArchiveRecord& ParentRecord, FArchiveCookContext* CookContext,
		const UPackage* Package, FLinkerSave* Linker, const TSet<TObjectPtr<UObject>>& ImportsUsedInGame,
		const TSet<FName>& SoftPackagesUsedInGame, TArray<FAssetData>* OutAssetDatas, bool bProceduralSave);
	UE_DEPRECATED(5.4, "Use version that takes FWritePackageDataArgs");
	COREUOBJECT_API void WritePackageData(FStructuredArchiveRecord& ParentRecord, bool bIsCooking, const UPackage* Package,
		FLinkerSave* Linker, const TSet<TObjectPtr<UObject>>& ImportsUsedInGame, const TSet<FName>& SoftPackagesUsedInGame,
		const ITargetPlatform* TargetPlatform, TArray<FAssetData>* OutAssetDatas);

	namespace Private
	{
		/**
		 * Storage for the singleton IAssetRegistry*
		 * TODO: this storage should be a class static variable on IAssetRegistry, but that type is defined in the AssetRegistry module, and many modules try to access the singleton (and call virtual functions on it) without linking against
		 * the AssetRegistry module, so the storage for the singleton needs to be defined in a lower-level module that all of those modules do include
		 */
		class IAssetRegistrySingleton
		{
		public:
			static IAssetRegistry* Get()
			{
				return Singleton;
			}
		private:
			static COREUOBJECT_API IAssetRegistry* Singleton;
			friend class ::UAssetRegistryImpl;
		};
	}

	class FFiltering
	{
	public:
		/** Return whether to filter out assets of the given class and flags from the editor's asset registry */
		static COREUOBJECT_API bool ShouldSkipAsset(const FTopLevelAssetPath& AssetClass, uint32 PackageFlags);

		/** Return whether to filter out the given object (assumed to be an asset) from the editor's asset registry */
		static COREUOBJECT_API bool ShouldSkipAsset(const UObject* InAsset);

		/** Call to invalidate the list of skip assets and cause their next use to recreate them on demand */
		static COREUOBJECT_API void MarkDirty();

#if WITH_ENGINE && WITH_EDITOR
		/** Copy the global skip classes set from the given external sets that were already populated. */
		static COREUOBJECT_API void SetSkipClasses(const TSet<FTopLevelAssetPath>& InSkipUncookedClasses, const TSet<FTopLevelAssetPath>& InSkipCookedClasses);

		/**
		 * Prepare the data structure needed for a call to Should skip asset. 
		 * This make the function bool ShouldSkipAsset(const FTopLevelAssetPath& AssetClass, uint32 PackageFlags) thread safe if the Game Thread is on hold during those calls.
		 */
		static COREUOBJECT_API void InitializeShouldSkipAsset();
#endif
	};

#if WITH_ENGINE && WITH_EDITOR
namespace Utils
{
	/** Return whether to filter out assets of the given class and flags based on the skip classes */
	COREUOBJECT_API bool ShouldSkipAsset(const FTopLevelAssetPath& AssetClass, uint32 PackageFlags,
		const TSet<FTopLevelAssetPath>& InSkipUncookedClasses, const TSet<FTopLevelAssetPath>& InSkipCookedClasses);
	/** Return whether to filter out the given object (assumed to be an asset) based on the skip classes */
	COREUOBJECT_API bool ShouldSkipAsset(const UObject* InAsset,
		const TSet<FTopLevelAssetPath>& InSkipUncookedClasses, const TSet<FTopLevelAssetPath>& InSkipCookedClasses);
	/** Run the calculation of which classes to skip and store results in the given sets. */
	COREUOBJECT_API void PopulateSkipClasses(TSet<FTopLevelAssetPath>& OutSkipUncookedClasses, TSet<FTopLevelAssetPath>& OutSkipCookedClasses);
}
#endif

	COREUOBJECT_API FName GetScriptPackageNameCoreUObject();
	COREUOBJECT_API FName GetScriptPackageNameEngine();
	COREUOBJECT_API FName GetScriptPackageNameBlueprintGraph();
	COREUOBJECT_API FName GetScriptPackageNameUnrealEd();
	COREUOBJECT_API FName GetClassNameObject();
	COREUOBJECT_API FName GetClassNameObjectRedirector();
	COREUOBJECT_API FName GetClassNameBlueprintCore();
	COREUOBJECT_API FName GetClassNameBlueprint();
	COREUOBJECT_API FName GetClassNameBlueprintGeneratedClass();
	COREUOBJECT_API FTopLevelAssetPath GetClassPathObject();
	COREUOBJECT_API FTopLevelAssetPath GetClassPathObjectRedirector();
	COREUOBJECT_API FTopLevelAssetPath GetClassPathBlueprintCore();
	COREUOBJECT_API FTopLevelAssetPath GetClassPathBlueprint();
	COREUOBJECT_API FTopLevelAssetPath GetClassPathBlueprintGeneratedClass();

}

// Enums used in public Engine headers
namespace EAssetSetManagerResult
{
	enum Type
	{
		DoNotSet,			// Do not set manager
		SetButDoNotRecurse,	// Set but do not recurse
		SetAndRecurse		// Set and recurse into reference
	};
}

namespace EAssetSetManagerFlags
{
	enum Type
	{
		IsDirectSet = 1,				// This attempt is a direct set instead of a recursive set
		TargetHasExistingManager = 2,	// Target already has a manager from previous run
		TargetHasDirectManager = 4,		// Target has another direct manager that will be set in this run
	};
}

/**
 * Asset Registry module interface
 */
class IAssetRegistryInterface
{
public:
	/**
	 * Tries to get a pointer to the active AssetRegistryInterface implementation. 
	 */
	static COREUOBJECT_API IAssetRegistryInterface* GetPtr();

	/**
	 * Lookup dependencies for the given package name and fill OutDependencies with direct dependencies
	 */
	virtual void GetDependencies(FName InPackageName, TArray<FName>& OutDependencies, UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::Package, const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) = 0;

	/**
	 * Tries to get the asset data for the specified object path
	 *
	 * @param ObjectPath the path of the object to be looked up
	 * @param OutAssetData out FAssetData
	 * @return Return code enum
	 */
	virtual UE::AssetRegistry::EExists TryGetAssetByObjectPath(const FSoftObjectPath& ObjectPath, struct FAssetData& OutAssetData) const = 0;

	/**
	 * Tries to get the package data for the specified package name
	 *
	 * @param PackageName name of the package
	 * @param OutAssetPackageData out FAssetPackageData
	 * @param bFailIfLockHeld If the AssetRegistry is currently being written to by another thread this
	 *	      function will return EExists::Unknown rather than waiting for safe read access.
	 * @return Return code enum
	 */
	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName, class FAssetPackageData& OutPackageData, bool bFailIfLockHeld = false) const = 0;

	/**
	 * Tries to get the package data for the specified package name. If found, OutCorrectCasePackageName
	 * will be populated with the PackageName that matches the casing used by the filesystem
	 * 
	 * @param PackageName name of the package
	 * @param OutAssetPackageData out FAssetPackageData
	 * @param OutCorrectCasePackageName out FName matching filesystem casing
	 * @param bFailIfLockHeld If the AssetRegistry is currently being written to by another thread this 
	 *	      function will return EExists::Unknown rather than waiting for safe read access.
	 * @return Return code enum
	 */
	virtual UE::AssetRegistry::EExists TryGetAssetPackageData(FName PackageName, class FAssetPackageData& OutPackageData, FName& OutCorrectCasePackageName, bool bFailIfLockHeld = false) const = 0;

	/**
	 * Enumerate asset data for all assets that match the filter.
	 * Assets returned must satisfy every filter component if there is at least one element in the component's array.
	 * Assets will satisfy a component if they match any of the elements in it.
	 *
	 * @param Filter filter to apply to the assets in the AssetRegistry
	 * @param Callback function to call for each asset data enumerated
	 * @param InEnumerateFlags flags to control enumeration and filtering.
	 *        @see EEnumerateAssetsFlags.
	 * @return False if the AssetRegistry is not available or the filter is invalid, otherwise true.
	 */
	virtual bool EnumerateAssets(const FARFilter& Filter, TFunctionRef<bool(const FAssetData&)> Callback,
		UE::AssetRegistry::EEnumerateAssetsFlags InEnumerateFlags = UE::AssetRegistry::EEnumerateAssetsFlags::None) const = 0;

	/** Register delegate for when assets are added to the registry */
	virtual bool RegisterOnAssetsAddedDelegate(const TFunction<void(TConstArrayView<FAssetData>)>& Function, FDelegateHandle& OutHandle) = 0;

	/** Removes registered delegate for when assets are added to the registry */
	virtual bool UnregisterOnAssetsAddedDelegate(const FDelegateHandle& Handle) = 0;

	/** Register delegate for when assets are removed from the registry */
	virtual bool RegisterOnAssetsRemovedDelegate(const TFunction<void(TConstArrayView<FAssetData>)>& Function, FDelegateHandle& OutHandle) = 0;

	/** Removes registered delegate for when assets are removed from the registry */
	virtual bool UnregisterOnAssetsRemovedDelegate(const FDelegateHandle& Handle) = 0;

protected:

	static COREUOBJECT_API IAssetRegistryInterface* Default;
	friend class UAssetRegistryImpl;
};
