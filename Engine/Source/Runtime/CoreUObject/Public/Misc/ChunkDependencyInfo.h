// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ChunkDependencyInfo.generated.h"

/** A single dependency, read from ini file */
USTRUCT()
struct FChunkDependency
{
	GENERATED_USTRUCT_BODY()

	FChunkDependency()
	{};

	FChunkDependency(int32 InChunkId, int32 InParentChunkId)
		: ChunkID(InChunkId)
		, ParentChunkID(InParentChunkId)
	{};

	/** The child chunk */
	UPROPERTY(EditAnywhere, Category = ChunkInfo)
	int32 ChunkID = 0;

	/** Parent chunk, anything in both Parent and Child is only placed into Parent */
	UPROPERTY(EditAnywhere, Category = ChunkInfo)
	int32 ParentChunkID = 0;

	bool operator== (const FChunkDependency& RHS) const
	{
		return ChunkID == RHS.ChunkID;
	}
};

/** In memory structure used for dependency tree */
struct FChunkDependencyTreeNode
{
	FChunkDependencyTreeNode(int32 InChunkID=0)
		: ChunkID(InChunkID)
	{}

	int32 ChunkID;

	TArray<FChunkDependencyTreeNode> ChildNodes;
};

/** This is read out of config and defines a tree of chunk dependencies */
UCLASS(config=Engine, defaultconfig, MinimalAPI)
class UChunkDependencyInfo : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Will return an existing dependency graph, or build if it necessary */
	COREUOBJECT_API const FChunkDependencyTreeNode* GetOrBuildChunkDependencyGraph(int32 HighestChunk = 0, bool bForceRebuild = false);

	/** Will create a dependency tree starting with RootTreeNode. If HighestChunk is == 0 it will only add dependencies on 0 for chunks already in the dependencies list */
	COREUOBJECT_API const FChunkDependencyTreeNode* BuildChunkDependencyGraph(int32 HighestChunk);

	/** Removes redundant chunks from a chunk list */
	COREUOBJECT_API void RemoveRedundantChunks(TArray<int32>& ChunkIDs) const;

	/** Given a set of chunk Ids will use the dependencies to find which chunk is shared among all the given chunks.
	All chunks should converge to the 0 startup chunk.
	Potentially will return a chunk that is not in the given input array */
	COREUOBJECT_API int32 FindHighestSharedChunk(const TArray<int32>& ChunkIDs) const;

	/**
	 * Returns the chunks that the in chunk depends upon through the parent rules.
	 */
	COREUOBJECT_API void GetChunkDependencies(const int32 InChunk, TSet<int32>& OutChunkDependencies) const;

	/** List of dependencies used to remove redundant chunks */
	UPROPERTY(config)
	TArray<FChunkDependency> DependencyArray;

private:
	/** Transfers data from the deprecated location [/Script/UnrealEd.ChunkDependencyInfo] and emits a warning in that case */
	virtual void PostInitProperties() override;

	/** Fills out the Dependency Tree starting with Node */
	void AddChildrenRecursive(FChunkDependencyTreeNode& Node, TArray<FChunkDependency>& DepInfo, TSet<int32> Parents);

	/** Root of tree, this will be valid after calling BuildChunkDependencyGraph */
	FChunkDependencyTreeNode RootTreeNode;

	/** Map of child chunks to all parent chunks, computed in BuildChunkDependencyGraph */
	TMap<int32, TSet<int32>> ChildToParentMap;

	/** Cached array of topologically sorted chunk. */
	TArray<int32> TopologicallySortedChunks;

	/** Cached value of HighestChunk at time of building */
	int32 CachedHighestChunk;
};
