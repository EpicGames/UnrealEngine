// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/TokenizedMessage.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "DataValidation.generated.h"

namespace UE::DataValidation
{
	extern COREUOBJECT_API const FName MessageLogName;
}

struct FAssetData;

UENUM(BlueprintType)
enum class EDataValidationUsecase : uint8
{
	/** No usecase specified */
	None = 0,

	/** Triggered on user's demand */
	Manual,

	/** A commandlet invoked the validation */
	Commandlet,

	/** Saving a package triggered the validation */
	Save,

	/** Submit dialog triggered the validation */
	PreSubmit,

	/** Triggered by blueprint or c++ */
	Script,
};

/** 
 * This class is used to interface between UObject::IsDataValid and the data validation system.
 */
class FDataValidationContext : public FNoncopyable
{
public:
	struct FIssue
	{
		FText Message;
		EMessageSeverity::Type Severity;
		TSharedPtr<FTokenizedMessage> TokenizedMessage;

		FIssue(TSharedRef<FTokenizedMessage> InTokenizedMessage)
			: Message()
			, Severity(InTokenizedMessage->GetSeverity())
			, TokenizedMessage(InTokenizedMessage)
		{}
		FIssue(const FText& InMessage, EMessageSeverity::Type InSeverity)
			: Message(InMessage)
			, Severity(InSeverity)
			, TokenizedMessage()
		{}
	};
	
	FDataValidationContext() = default;
	
	/**
	 * Constructor for use by UEditorValidatorSubsystem.
	 * @param Inusecase allows validators to skip certain validation steps depending on the validation trigger condition
	 * @param The name of the message log to use for warnings/errors/etc
	 * @param InAssociatedObjects allows validators to check additional associated objects of the asset being validated e.g. validating external actors
	 */
	FDataValidationContext(
			bool InWasAssetLoadedForValidation,
			EDataValidationUsecase InUsecase,
			TConstArrayView<FAssetData> InAssociatedObjects,
			FName InMessageLogName = UE::DataValidation::MessageLogName)
		: AssociatedExternalObjects(InAssociatedObjects)
		, ValidationUsecase(InUsecase)
		, MessageLogName(InMessageLogName)
		, bWasAssetLoadedForValidation(InWasAssetLoadedForValidation)
	{}

	EDataValidationUsecase GetValidationUsecase() const
	{
		return ValidationUsecase;
	}
	
	/**
	 * Other objects associated with the asset being validated. 
	 * e.g. if the asset being validated is a world, these may be actors within that world's PersistentLevel with external packages.
	 * May also be actors in a streamed in sublevel which are stored in external packages.
	 */
	TConstArrayView<FAssetData> GetAssociatedExternalObjects() const
	{
		return AssociatedExternalObjects;
	}
	
	// Returns whether the asset currently being validation was loaded specifically for the purposes of validation
	// Otherwise it was already loaded before validation started
	// Default valid is false so that validators can attempt to reload asset to verify on-disk state/load process 
	// if unsure.
	// If true, allows that duplicate work to be skipped in some context. 
	bool WasAssetLoadedForValidation() const
	{
		return bWasAssetLoadedForValidation;
	}

	void MarkAssetLoadedForValidation()
	{
		bWasAssetLoadedForValidation = true;
	}

	COREUOBJECT_API TSharedRef<FTokenizedMessage> AddMessage(const FAssetData& ForAsset, EMessageSeverity::Type InSeverity, FText InText = {});
	COREUOBJECT_API TSharedRef<FTokenizedMessage> AddMessage(EMessageSeverity::Type InSeverity, FText InText = {});
	COREUOBJECT_API void AddMessage(TSharedRef<FTokenizedMessage> Message);
	void AddWarning(const FText& Text) { Issues.Emplace(Text, EMessageSeverity::Warning); NumWarnings++; }
	void AddError(const FText& Text) { Issues.Emplace(Text, EMessageSeverity::Error); NumErrors++; }

	const TArray<FIssue>& GetIssues() const { return Issues; }
	uint32 GetNumWarnings() const { return NumWarnings; }
	uint32 GetNumErrors() const { return NumErrors; }

	COREUOBJECT_API void SplitIssues(TArray<FText>& Warnings, TArray<FText>& Errors, TArray<TSharedRef<FTokenizedMessage>>* Messages = nullptr) const;

	FName GetMessageLogName() const { return MessageLogName; }
	
private:
	TArray<FIssue> Issues;
	uint32 NumWarnings = 0;
	uint32 NumErrors = 0;
	
	TConstArrayView<FAssetData> AssociatedExternalObjects;
	EDataValidationUsecase ValidationUsecase = EDataValidationUsecase::None;
	FName MessageLogName;
	bool bWasAssetLoadedForValidation = false;
};