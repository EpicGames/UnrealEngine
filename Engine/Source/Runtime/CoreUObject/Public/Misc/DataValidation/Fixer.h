// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/TokenizedMessage.h"
#include "Templates/SharedPointer.h"

#define UE_API COREUOBJECT_API

namespace UE::DataValidation
{

/**
 * Provider of automatic fixes for an asset.
 *
 * Fixers are composable: it is possible to layer many fixers on top of each other to provide
 * fixes which perform extra actions on top of a base fixer.
 * These layers are provided within the DataValidation plugin. See DataValidationFixers.h.
 */
struct IFixer : TSharedFromThis<IFixer>
{
	virtual ~IFixer() = default;

	/** Returns whether the fix can be applied at the moment. This must be called before `ApplyFix`. */
	virtual EFixApplicability GetApplicability(int32 FixIndex) const = 0;
	/** Applies a fix provided by this fixer. */
	virtual FFixResult ApplyFix(int32 FixIndex) = 0;

	/**
	 * Sugar for wrapping the fixer in another fixer using postfix `Fixer->WrappedIn<FOtherFixer>(Args)`
	 * instead of `FSomeFixer::Create(Fixer, Args)`.
	 */
	template <typename FixerType, typename... ArgTypes>
	TSharedRef<FixerType> WrappedIn(ArgTypes&&... Args)
	{
		return FixerType::Create(AsShared(), Forward<ArgTypes>(Args)...);
	}

	/**
	 * Creates an FFixToken out of the fix and a given label.
	 * This should generally be preferred over the lower level FFixToken::Create.
	 */
	UE_API TSharedRef<FFixToken> CreateToken(const FText& Label);
};

}

#undef UE_API
