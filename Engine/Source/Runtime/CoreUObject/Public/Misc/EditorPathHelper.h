// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"

/**
 * Helper class that can build Editor FSoftObjectPath(s) based on EditorPathOwners in an object's outer chain.
 */
class FEditorPathHelper
{
#if WITH_EDITOR
public:
	static COREUOBJECT_API FSoftObjectPath GetEditorPath(const UObject* InObject);
	static COREUOBJECT_API FSoftObjectPath GetEditorPathFromReferencer(const UObject* InObject, const UObject* InReferencer);
	static COREUOBJECT_API FSoftObjectPath GetEditorPathFromEditorPathOwner(const UObject* InObject, const UObject* InEditorPathOwner);
	static COREUOBJECT_API UObject* GetEditorPathOwner(const UObject* InObject);
	static COREUOBJECT_API bool IsInEditorPath(const UObject* InEditorPathOwner, const UObject* InObject);
	static COREUOBJECT_API bool IsEnabled();
#endif
};