// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreFwd.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "EditorPathObjectInterface.generated.h"

UINTERFACE(BlueprintType, meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class UEditorPathObjectInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

/**
 * Interface to be implemented by Objects that can be part of an Editor Path through a Editor Path owner
 * 
 * See FEditorPathHelper::GetEditorPath. Paths returned by FEditorPathHelper::GetEditorPath variants should be resolvable through the EditorPathOwner's class override of ResolveSubObject.
 * 
 * Example implementation of this is to allow Objects outside of a Level Instance to reference objects inside a Level Instance.
 */

class IEditorPathObjectInterface
{
	GENERATED_IINTERFACE_BODY()

#if WITH_EDITOR
public:
	virtual UObject* GetEditorPathOwner() const = 0;
#endif
};