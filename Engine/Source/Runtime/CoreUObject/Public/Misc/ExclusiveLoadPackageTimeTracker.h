// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/IConsoleManager.h"
#include "HAL/Platform.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/Package.h"
#include "UObject/UnrealNames.h"

class FString;

#define WITH_LOADPACKAGE_TIME_TRACKER STATS

/**
 * A singleton to keep track of the exclusive load time of every package. Also reports inclusive load time for convenience.
 * The code that loads packages "pushes" a load on the tracker which will record the time the package started loading.
 * Later that package will be "popped" which will cause the tracker to record the time spent.
 * If another package is pushed while this is happening, it will pause the time recorded from the first package and start a new time for the second. This is recursive.
 * DumpReport() can be used to log out all the recorded values in a couple different ways to get an idea about what packages are the slowest to load.
 */
class FExclusiveLoadPackageTimeTracker
{
public:
	/** Scoped helper for LoadPackage */
	struct FScopedPackageTracker
	{
		FScopedPackageTracker(UPackage* InPackageToTrack)
			: PackageToTrack(InPackageToTrack)
		{
			FExclusiveLoadPackageTimeTracker::PushLoadPackage(PackageToTrack ? PackageToTrack->GetFName() : NAME_None);
		}
		FScopedPackageTracker(FName PackageNameToTrack)
			: PackageToTrack(nullptr)
		{
			FExclusiveLoadPackageTimeTracker::PushLoadPackage(PackageNameToTrack);
		}
		~FScopedPackageTracker()
		{
			FExclusiveLoadPackageTimeTracker::PopLoadPackage(PackageToTrack);
		}
		UPackage* PackageToTrack;
	};

	/** Scoped helper for PostLoad */
	struct FScopedPostLoadTracker
	{
		FScopedPostLoadTracker(UObject* InPostLoadObject)
		{
			if (FExclusiveLoadPackageTimeTracker::PushPostLoad(InPostLoadObject))
			{
				PostLoadObject = InPostLoadObject;
			}
			else
			{
				PostLoadObject = nullptr;
			}
		}
		~FScopedPostLoadTracker()
		{
			if (PostLoadObject)
			{
				FExclusiveLoadPackageTimeTracker::PopPostLoad(PostLoadObject);
			}
		}
		UObject* PostLoadObject;
	};

	/** Scoped helper for EndLoad */
	struct FScopedEndLoadTracker
	{
		FScopedEndLoadTracker() { FExclusiveLoadPackageTimeTracker::PushEndLoad(); }
		~FScopedEndLoadTracker() { FExclusiveLoadPackageTimeTracker::PopEndLoad(); }
	};

	/** Starts a time for the specified package name. */
	UE_FORCEINLINE_HINT static void PushLoadPackage(FName PackageName)
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		Get().InternalPushLoadPackage(PackageName);
#endif
	}

	/** Records a time and stats for the loaded package. */
	UE_FORCEINLINE_HINT static void PopLoadPackage(UPackage* LoadedPackage)
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		Get().InternalPopLoadPackage(LoadedPackage, nullptr);
#endif
	}

	/** Starts a time for the specified package name. */
	inline static bool PushPostLoad(UObject* PostLoadObject)
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		if (PostLoadObject && PostLoadObject->IsAsset())
		{
			Get().InternalPushLoadPackage(PostLoadObject->GetOutermost()->GetFName());
			return true;
		}
#endif
		return false;
	}

	/** Records a time and stats for the loaded package. Optionally provide the loaded asset if it is known, otherwise the asset in the package is detected. */
	UE_FORCEINLINE_HINT static void PopPostLoad(UObject* PostLoadObject)
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		Get().InternalPopLoadPackage(nullptr, PostLoadObject);
#endif
	}

	/** Starts a time for time spent in "EndLoad" */
	inline static void PushEndLoad()
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		FExclusiveLoadPackageTimeTracker& Instance = Get();
		Instance.InternalPushLoadPackage(Instance.EndLoadName);
#endif
	}

	/** Records some time spent in "EndLoad" */
	UE_FORCEINLINE_HINT static void PopEndLoad()
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		Get().InternalPopLoadPackage(nullptr, nullptr);
#endif
	}

	/** Displays the data gathered in various ways. */
	UE_FORCEINLINE_HINT static void DumpReport(const TArray<FString>& Args)
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		Get().InternalDumpReport(Args);
#endif
	}

	/** Resets the data. */
	UE_FORCEINLINE_HINT static void ResetReport()
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		Get().InternalResetReport();
#endif
	}

	/** Returns the load time for the specified package, excluding its dependencies */
	inline static double GetExclusiveLoadTime(FName PackageName)
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		return Get().InternalGetExclusiveLoadTime(PackageName);
#else
		return 0;
#endif
	}

	/** Returns the load time for the specified package, including its dependencies */
	inline static double GetInclusiveLoadTime(FName PackageName)
	{
#if WITH_LOADPACKAGE_TIME_TRACKER
		return Get().InternalGetInclusiveLoadTime(PackageName);
#else
		return 0;
#endif
	}


#if WITH_LOADPACKAGE_TIME_TRACKER // The rest of the class is only available if WITH_LOADPACKAGE_TIME_TRACKER

private:

	/** Struct to keep track of package load times */
	struct FLoadTime
	{
		FName TimeName;
		FName AssetClass;
		double ExclusiveTime;
		double InclusiveTime;
		double LastStartTime;
		double OriginalStartTime;

		FLoadTime()
			: ExclusiveTime(0), InclusiveTime(0), LastStartTime(0), OriginalStartTime(0)
		{}

		FLoadTime(FName InName, double InStartTime)
			: TimeName(InName), ExclusiveTime(0), InclusiveTime(0), LastStartTime(InStartTime), OriginalStartTime(InStartTime)
		{}

		FLoadTime(FName InName, FName InAssetClass, double InExclusive, double InInclusive)
			: TimeName(InName), ExclusiveTime(InExclusive), InclusiveTime(InInclusive), LastStartTime(0), OriginalStartTime(0)
		{}
	};

	inline static FExclusiveLoadPackageTimeTracker& Get()
	{
		if (TrackerInstance)
			return *TrackerInstance;
		else
			return Construct();
	}

	FExclusiveLoadPackageTimeTracker();
	COREUOBJECT_API static FExclusiveLoadPackageTimeTracker& Construct();

	/** Starts a time for the specified package name. */
	COREUOBJECT_API void InternalPushLoadPackage(FName PackageName);

	/** Records a time and stats for the loaded package. Optionally provide the loaded asset if it is known, otherwise the asset in the package is detected. */
	COREUOBJECT_API void InternalPopLoadPackage(UPackage* LoadedPackage, UObject* LoadedAsset);

	/** Displays the data gathered in various ways. */
	COREUOBJECT_API void InternalDumpReport(const TArray<FString>& Args) const;

	/** Resets the data */
	COREUOBJECT_API void InternalResetReport();

	/** Returns the load time for the specified package, excluding its dependencies */
	COREUOBJECT_API double InternalGetExclusiveLoadTime(FName PackageName) const;

	/** Returns the load time for the specified package, including its dependencies */
	COREUOBJECT_API double InternalGetInclusiveLoadTime(FName PackageName) const;

	/** Used to determine if a time is for a package and not for endload */
	bool IsPackageLoadTime(const FLoadTime& Time) const;

	/** Handler for the DumpReportCommand */
	void DumpReportCommandHandler(const TArray<FString>& Args);

	/** Handler for the ResetReportCommand */
	void ResetReportCommandHandler(const TArray<FString>& Args);
	
	/** Time stack for tracked code sections. Does not have inclusive time set. That is done in the LoadTimes map. */
	TArray<FLoadTime> TimeStack;

	/** Critical sections for thread safety */
	mutable FCriticalSection TimesCritical;

	/** A couple maps used to make the report and to report calculated times */
	TMap<FName, FLoadTime> LoadTimes;

	/** The amount of time spent in this singleton keeping track of load times. This overhead is included in the report. */
	double TrackerOverhead;

	/** Special case name for EndLoad and unknown asset types */
	const FName EndLoadName;
	const FName UnknownAssetName;

	/** Auto-registered console command to call DumpReport() */
	const FAutoConsoleCommand DumpReportCommand;

	/** Auto-registered console command to call ResetReport() */
	const FAutoConsoleCommand ResetReportCommand;

	static FExclusiveLoadPackageTimeTracker* TrackerInstance;

#endif // WITH_LOADPACKAGE_TIME_TRACKER
};
