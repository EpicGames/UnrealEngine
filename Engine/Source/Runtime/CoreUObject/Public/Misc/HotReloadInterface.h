// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/CoreNative.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Misc/CompilationResult.h"

enum class EHotReloadFlags : uint8
{
	None = 0x00,

	// Should not return until the recompile and reload has completed
	WaitForCompletion = 0x01
};
ENUM_CLASS_FLAGS(EHotReloadFlags)

enum class ERecompileModuleFlags : uint8
{
	None = 0x00,

	// Perform a reload of the module after the recompile finishes
	ReloadAfterRecompile = 0x01,

	// Report failure if UHT-generated code changes as a result of the recompile
	FailIfGeneratedCodeChanges = 0x02,

	// Even if this is not code-based project compile with game project as the target for UBT (do not use UnrealEditor target)
	ForceCodeProject = 0x04,
};
ENUM_CLASS_FLAGS(ERecompileModuleFlags)

/**
 * HotReload module interface
 */
class IHotReloadInterface : public IModuleInterface
{
public:
	/**
	 * Tries to gets a pointer to the active HotReload implementation. 
	 */
	static inline IHotReloadInterface* GetPtr()
	{
		static FName HotReload("HotReload");
		return FModuleManager::GetModulePtr<IHotReloadInterface>(HotReload);
	}

	/**
	 * Save the current state to disk before quitting.
	 */
	virtual void SaveConfig() = 0;

	/**
	 * Queries the compilation method for a given module.
	 *
	 * @param InModuleName Module to query the name of
	 * @return A string describing the method used to compile the module.
	 */
	virtual FString GetModuleCompileMethod(FName InModuleName) = 0;

	/**
	 * Recompiles a single module
	 *
	 * @param InModuleName Name of the module to compile
	 * @param Ar Output device (logging)
	 * @param Flags Recompilation flags
	 */
	virtual bool RecompileModule(const FName InModuleName, FOutputDevice &Ar, ERecompileModuleFlags Flags) = 0;

	/**
	 * Returns whether modules are currently being compiled
	 */
	virtual bool IsCurrentlyCompiling() const = 0;

	/**
	 * Request that current compile be stopped
	 */
	virtual void RequestStopCompilation() = 0;

	/**
	 * Performs hot reload from the editor of all currently loaded game modules.
	 *
	 * @param	Flags	Flags which to control the hot reload.
	 *
	 * @return	If Flags & EHotReloadFlags::WaitForCompletion was set, this will return the result of the compilation, otherwise will return ECompilationResult::Unknown
	 */
	virtual ECompilationResult::Type DoHotReloadFromEditor(EHotReloadFlags Flags) = 0;

	/**
	 * HotReload: Reloads the DLLs for given packages.
	 *
	 * @param	Package		Packages to reload.
	 * @param	Flags		Flags which control the hot reload.
	 * @param	Ar			Output device for logging compilation status
	 * 
	 * @return	If bWaitForCompletion was set to true, this will return the result of the compilation, otherwise will return ECompilationResult::Unknown
	 */
	virtual ECompilationResult::Type RebindPackages(const TArray<UPackage*>& Packages, EHotReloadFlags Flags, FOutputDevice &Ar) = 0;

	/**
	 * Gets an event delegate that is executed when compilation of a module has started.
	 *
	 * @return The event delegate.
	 */
	DECLARE_MULTICAST_DELEGATE_OneParam(FModuleCompilerStartedEvent, bool /*bIsAsyncCompile*/);
	virtual FModuleCompilerStartedEvent& OnModuleCompilerStarted() = 0;

	/**
	 * Gets an event delegate that is executed when compilation of a module has finished.
	 *
	 * The first parameter is the result of the compilation operation.
	 * The second parameter determines whether the log should be shown.
	 *
	 * @return The event delegate.
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FModuleCompilerFinishedEvent, const FString&, ECompilationResult::Type, bool);
	virtual FModuleCompilerFinishedEvent& OnModuleCompilerFinished() = 0;

	/**
	 * Checks if there's any game modules currently loaded
	 */
	virtual bool IsAnyGameModuleLoaded() = 0;
};

