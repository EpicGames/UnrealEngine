// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"

class UObject;
struct FTransactionObjectChange;

/**
 * Interface for transaction object annotations.
 *
 * Transaction object annotations are used for attaching additional user defined data to a transaction.
 * This is sometimes useful, because the transaction system only remembers changes that are serializable
 * on the UObject that a modification was performed on, but it does not see other changes that may have
 * to be remembered in order to properly restore the object internals.
 */
class ITransactionObjectAnnotation
{
public:
	virtual ~ITransactionObjectAnnotation() = default;
	virtual void AddReferencedObjects(class FReferenceCollector& Collector) = 0;
	virtual void Serialize(class FArchive& Ar) = 0;

	virtual bool SupportsAdditionalObjectChanges() const { return false; }
	virtual void ComputeAdditionalObjectChanges(const ITransactionObjectAnnotation* OriginalAnnotation, TMap<UObject*, FTransactionObjectChange>& OutAdditionalObjectChanges) {}
};
