// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "Templates/RefCounting.h"

namespace UE::PackageName { class FMountPoint; }
struct FLongPackagePathsSingleton;

namespace UE::PackageName
{

/**
 * A reference-counted mountpoint that has been registered with FPackageName. It might not be mounted; mountpoints can
 * be preregistered before mounting, or can be kept allocated after unmounting.
 * If mounted, a MountPoint provides a location on disk from which packages can be loaded (and saved in most cases),
 * and provides a two-way mapping from LongPackageName <-> LocalPath.
 * 
 * Multiple MountPoints can be mapped from a single LongPackageName to multiple LocalPaths, or from a single LocalPath
 * to multiple LongPackageNames. Only the last MountPoint mounted for a LongPackageName or LocalPath is returned from
 * queries for those paths, but all mounted MountPoints can still be used to load packages, they report
 * IsMounted == true, and they are reported in query functions that report the list of mounted MountPoints.
 * 
 * Most MountPoints have a one-segment length path in their LongPackageName, such as /Game/ or /PluginName/, but the
 * system supports paths of any length as long as they are rooted. MountPoints that are subpaths of existing
 * MountPoints will take precedence for mapping files under their path, even if their parent MountPoint is higher
 * priority because it was added later. This feature is used for example by the UnrealEd templates system in
 * FUnrealEdMisc::MountTemplateSharedPaths, which mounts some providers for feature templates in a mountpoint
 * subdirectory of /Game/ such as /Game/Weapons/.
 */
class IMountPoint : public IRefCountedObject
{
public:
	/**
	 * PackageName path that is the parent directory of packagenames in this mountpoint. Has a trailing slash.
	 * e.g.
	 *     "/Game/",
	 *     "/PluginName/",
	 *     "/Game/Weapons/"
	 * Immutable, can be read from any thread.
	 */
	virtual FStringView GetLongPackageName() const = 0;
	/**
	 * LocalPath, in Unreal standard path form (normalized and relative to Engine/Binaries/<Platform> directory), that
	 * is the parent directory of filenames in this mountpoint. Has a trailing slash.
	 * e.g.
	 *     "../../../EngineTest/Content/",
	 *     "../../../EngineTest/Plugins/PluginName/Content/",
	 *     "../../../Templates/TemplateResources/Standard/Weapons/Content/"
	 * Immutable, can be read from any thread.
	 */
	virtual FStringView GetLocalPathStandard() const = 0;
	/**
	 * LocalPath, absolute, that is the parent directory of filenames in this mountpoint. Has a trailing slash.
	 * e.g.
	 *     "d:/UnrealRoot/EngineTest/Content/",
	 *     "d:/UnrealRoot/EngineTest/Plugins/PluginName/Content/"
	 *     "d:/UnrealRoot/Templates/TemplateResources/Standard/Weapons/Content/",
	 *
	 * Immutable, can be read from any thread.
	 */
	virtual FStringView GetLocalPathAbsolute() const = 0;

	/**
	 * Whether the mountpoint was created by InsertMountPoint and can be removed by RemoveMountPoint.
	 * Immutable, can be read from any thread.
	 */
	virtual bool IsRemovable() const = 0;
	/**
	 * Whether new Packages are unable to be saved into the mountpoint.
	 * e.g. returns false for /Game/, returns true for /Script/.
	 * Immutable, can be read from any thread.
	 */
	virtual bool IsReadOnly() const = 0;
	/**
	 * Whether the MountPoint is currently mounted. IMountPoints can be registered with PackageName before they are mounted,
	 * and IMountPoints will remain allocated, and be used by PackageName rather than replaced with a new allocation when they
	 * are mounted, as long as they have a non-zero refcount.
	 * Mutable but atomic, can be read from any thread.
	 */
	virtual bool IsMounted() const = 0;

protected:
	/**
	 * A downcast function to the implementation class inside FPackageName. There should be no other implementation of
	 * IMountPoint; we log an error if this fails.
	 */
	virtual FMountPoint* AsFMountPoint() = 0;

	friend FLongPackagePathsSingleton;
};

} // namespace UE::PackageName