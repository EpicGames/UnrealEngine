// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CoreMisc.h: General-purpose file utilities.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

class FProperty;
struct FPropertyChangedEvent;

// Notification hook.
class FNotifyHook
{
public:
	virtual void NotifyPreChange( FProperty* PropertyAboutToChange ) {}
	virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged ) {}
	COREUOBJECT_API virtual void NotifyPreChange( class FEditPropertyChain* PropertyAboutToChange );
	COREUOBJECT_API virtual void NotifyPostChange( const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged );
};
