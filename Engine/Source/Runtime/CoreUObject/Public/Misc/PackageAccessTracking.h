// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Optional.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectHandleDefines.h"

class ITargetPlatform;

#define UE_WITH_PACKAGE_ACCESS_TRACKING UE_WITH_OBJECT_HANDLE_TRACKING

#if UE_WITH_PACKAGE_ACCESS_TRACKING
#define UE_TRACK_REFERENCING_PACKAGE_SCOPED(Object, OpName) PackageAccessTracking_Private::FPackageAccessRefScope ANONYMOUS_VARIABLE(PackageAccessTracker_)(Object, OpName);
#define UE_TRACK_REFERENCING_PACKAGE_DELAYED_SCOPED(TrackerName, OpName) TOptional<PackageAccessTracking_Private::FPackageAccessRefScope> TrackerName; FName TrackerName##_OpName(OpName);
#define UE_TRACK_REFERENCING_PACKAGE_DELAYED(TrackerName, Package) if (TrackerName) TrackerName->SetPackageName(Package->GetFName()); else TrackerName.Emplace(Package->GetFName(), TrackerName##_OpName);
#define UE_TRACK_REFERENCING_PLATFORM_SCOPED(TargetPlatform) PackageAccessTracking_Private::FPackageAccessRefScope ANONYMOUS_VARIABLE(PackageAccessTracker_)(TargetPlatform);
#define UE_TRACK_REFERENCING_OPNAME_SCOPED(OpName) PackageAccessTracking_Private::FPackageAccessRefScope ANONYMOUS_VARIABLE(PackageAccessTracker_)(OpName);
#define UE_TRACK_REFERENCING_PACKAGE_DECLARE_SCOPE_VARIABLE(VariableName) TOptional<PackageAccessTracking_Private::FPackageAccessRefScope> VariableName;
#define UE_TRACK_REFERENCING_PACKAGE_ACTIVATE_SCOPE_VARIABLE(VariableName, Object, OpName) check(!VariableName.IsSet()); VariableName.Emplace(Object, OpName);
#define UE_TRACK_REFERENCING_PACKAGE_DEACTIVATE_SCOPE_VARIABLE(VariableName) VariableName.Reset();
#else
#define UE_TRACK_REFERENCING_PACKAGE_SCOPED(Package, OpName)
#define UE_TRACK_REFERENCING_PACKAGE_DELAYED_SCOPED(TrackerName, OpName)
#define UE_TRACK_REFERENCING_PACKAGE_DELAYED(TrackerName, Package)
#define UE_TRACK_REFERENCING_PLATFORM_SCOPED(TargetPlatform)
#define UE_TRACK_REFERENCING_OPNAME_SCOPED(OpName)
#define UE_TRACK_REFERENCING_PACKAGE_DECLARE_SCOPE_VARIABLE(VariableName)
#define UE_TRACK_REFERENCING_PACKAGE_ACTIVATE_SCOPE_VARIABLE(VariableName, Object, OpName)
#define UE_TRACK_REFERENCING_PACKAGE_DEACTIVATE_SCOPE_VARIABLE(VariableName)
#endif //UE_WITH_PACKAGE_ACCESS_TRACKING

// Cook-specific AccessTracking Scopes
#if WITH_EDITOR && UE_WITH_PACKAGE_ACCESS_TRACKING
/**
 * Specify that within the current scope, dereferenced TObjectPtrs cause a dependency on the specified
 * ResultProjection of the dereferenced object and package, instead of the outer scope's projection or the default
 * projection of UE::Cook::ResultProjection::All. ProjectionName must be either one of the generic terms provided
 * in UE::Cook::ResultProjection or it must be a name declared as a result projection by one of the objects in
 * the target package.
 */
#define UE_COOK_RESULTPROJECTION_SCOPED(ProjectionName) \
	PackageAccessTracking_Private::FPackageAccessRefScope ANONYMOUS_VARIABLE(PackageAccessTracker_)( \
		PackageAccessTracking_Private::FPackageAccessRefScope::CookResultProjectionType, ProjectionName)

/** 
 * Disable the package tracker in the current scope. It means when the code will dereference a TObjectPtr, no
 * package dependency will be created for the currently cooked package.  
 */
#define UE_COOK_DISABLE_PACKAGE_ACCESS_TRACKING_SCOPED() UE_COOK_RESULTPROJECTION_SCOPED(UE::Cook::ResultProjection::None)

#else
#define UE_COOK_RESULTPROJECTION_SCOPED(ProjectionName)
#define UE_COOK_DISABLE_PACKAGE_ACCESS_TRACKING_SCOPED()
#endif //WITH_EDITOR && UE_WITH_PACKAGE_ACCESS_TRACKING

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

#if UE_WITH_PACKAGE_ACCESS_TRACKING
namespace UE { struct FInheritedContextExtension; class FInheritedContextExtensionScope; }

namespace PackageAccessTracking_Private
{
	struct FTrackedData
	{
		/** Standard constructor; sets variables from DirectData passed to a FPackageAccessRefScope. */
		COREUOBJECT_API FTrackedData(FName PackageNameIn, FName InOpName, FName InCookResultProjection,
			const ITargetPlatform* InTargetPlatform, const UObject* InObject);

		/** Direct-init constructor; copies all fields as-is without recomputing BuildOpName via IsBuildOpName. */
		COREUOBJECT_API FTrackedData(FName InPackageName, FName InOpName, FName InBuildOpName,
			FName InCookResultProjection, const ITargetPlatform* InTargetPlatform, const UObject* InObject);

		/** Accumulating constructor; sets new AccumulatedData by combining DirectData with Outer's AccumulatedData. */
		COREUOBJECT_API FTrackedData(FTrackedData& DirectData, FTrackedData* OuterAccumulatedData);

		FName PackageName;
		FName OpName;
		FName BuildOpName;
		FName CookResultProjection;
		const UObject* Object = nullptr;
		const ITargetPlatform* TargetPlatform = nullptr;
	};

	class FPackageAccessRefScope
	{
	public:
		COREUOBJECT_API FPackageAccessRefScope(FName InPackageName, FName InOpName);
		COREUOBJECT_API FPackageAccessRefScope(const UObject* InObject, FName InOpName);
		COREUOBJECT_API FPackageAccessRefScope(FName InOpName);
		COREUOBJECT_API FPackageAccessRefScope(const ITargetPlatform* InTargetPlatform);
		enum ECookResultProjectionType
		{
			CookResultProjectionType,
		};
		COREUOBJECT_API FPackageAccessRefScope(ECookResultProjectionType, FName InCookResultProjection);

		COREUOBJECT_API ~FPackageAccessRefScope();

		COREUOBJECT_API void SetPackageName(FName InPackageName);

		UE_FORCEINLINE_HINT FName GetPackageName() const { return DirectData.PackageName; }
		UE_FORCEINLINE_HINT FName GetOpName() const { return DirectData.OpName; }
		UE_FORCEINLINE_HINT FPackageAccessRefScope* GetOuter() const { return Outer; }
		UE_FORCEINLINE_HINT const ITargetPlatform* GetTargetPlatform() const { return DirectData.TargetPlatform; }

		static COREUOBJECT_API FPackageAccessRefScope* GetCurrentThreadScope();
		static COREUOBJECT_API FTrackedData* GetCurrentThreadAccumulatedData();
	private:
		COREUOBJECT_API FPackageAccessRefScope(FName InPackageName, FName InOpName, FName InCookResultProjection,
			const ITargetPlatform* InTargetPlatform, const UObject* InObject);

		// Tag type + constructor for restoring from inherited context
		struct FInheritedContextTag { explicit FInheritedContextTag() = default; };
		COREUOBJECT_API explicit FPackageAccessRefScope(FInheritedContextTag,
			FName InPackageName, FName InOpName, FName InBuildOpName,
			FName InCookResultProjection, const UObject* InObject,
			const ITargetPlatform* InTargetPlatform);

		// Extension factory (lambdas inside have private access)
		static UE::FInheritedContextExtension& GetInheritedContextExtension();

		FTrackedData DirectData;
		FTrackedData AccumulatedData;
		FPackageAccessRefScope* Outer = nullptr;
		// Extension scope, active only for outermost scope per thread
		TUniquePtr<UE::FInheritedContextExtensionScope> InheritedExtensionScope;
		static thread_local FPackageAccessRefScope* CurrentThreadScope;
	};
};
#endif // UE_WITH_PACKAGE_ACCESS_TRACKING

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Misc/PackageName.h"
#include "UObject/ObjectHandle.h"
#include "UObject/Package.h"
#endif
