// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/PackageAccessTracking.h"
#include "UObject/NameTypes.h"

#if UE_WITH_PACKAGE_ACCESS_TRACKING

namespace PackageAccessTrackingOps
{
	extern COREUOBJECT_API FName NAME_Load;
	extern COREUOBJECT_API FName NAME_PreLoad;
	extern COREUOBJECT_API FName NAME_PostLoad;
	extern COREUOBJECT_API FName NAME_Save;
	extern COREUOBJECT_API FName NAME_CreateDefaultObject;
	extern COREUOBJECT_API FName NAME_CookerBuildObject;

	/**
	 * A global-scope operation started as AssetRegistry tick is occurring on the current thread inside of another
	 * scope, and the references made during this operation should not be associated with the previous scope.
	 */
	extern COREUOBJECT_API FName NAME_ResetContext;

	/**
	 * A debug-only scope that is used to find global-scope operations. Any accesses made when a NoAccessExpected
	 * scope is on top of the context stack will log a warning. To resolve the warning either the NoAccessExpected
	 * scope should be narrowed, or another operation should be added (ResetContext or a repeat of the higher
	 * level scope).
	 */
	extern COREUOBJECT_API FName NAME_NoAccessExpected;
}

#endif //UE_WITH_PACKAGE_ACCESS_TRACKING
