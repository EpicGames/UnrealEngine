// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PackageName.h: Unreal package name utility functions.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Containers/VersePathFwd.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Misc/MountPoint.h"
#include "Misc/PackagePath.h"
#include "Templates/Function.h"
#include "UObject/NameTypes.h"

class FPackagePath;
class UPackage;
struct FFileStatData;
struct FGuid;
struct FTopLevelAssetPath;
namespace UE::AssetRegistry::Impl { struct FScanPathContext; }

DECLARE_LOG_CATEGORY_EXTERN(LogPackageName, Log, All);

class FPackageName
{
public:
	/** Indicates the format the input was in in functions that take flexible input that can be a LocalPath, PackageName, or ObjectPath */
	enum class EFlexNameType
	{
		Invalid,
		PackageName,
		LocalPath,
		ObjectPath,
	};
	enum class EErrorCode
	{
		PackageNameUnknown,
		PackageNameEmptyPath,
		PackageNamePathNotMounted,
		PackageNamePathIsMemoryOnly,
		PackageNameSpacesNotAllowed,
		PackageNameContainsInvalidCharacters,
		LongPackageNames_PathTooShort,
		LongPackageNames_PathWithNoStartingSlash,
		LongPackageNames_PathWithTrailingSlash,
		LongPackageNames_PathWithDoubleSlash,
	};

	enum class EConvertFlags
	{
		None = 0,
		/// When set, this will allow filenames following the wildcard pattern of `*.*.*`, rather than `*.*`.
		AllowDots = 0x1
	};

	/**
	 * Return a user-readable string for the error code returned from a FPackageName function
	 *
	 * @param InPath the path that was passed to the convert function
	 * @param ErrorCode The error code returned from the convert function
	 */
	static COREUOBJECT_API FString FormatErrorAsString(FStringView InPath, EErrorCode ErrorCode);

	/**
	 * Return a user-readable string for the error code returned from a FPackageName function
	 *
	 * @param InPath the path that was passed to the convert function
	 * @param ErrorCode The error code returned from the convert function
	 */
	static COREUOBJECT_API FText FormatErrorAsText(FStringView InPath, EErrorCode ErrorCode);

	/**
	 * Helper function for converting short to long script package name (InputCore -> /Script/InputCore)
	 *
	 * @param InShortName Short package name.
	 * @return Long package name.
	 */
	static COREUOBJECT_API FString ConvertToLongScriptPackageName(const TCHAR* InShortName);

	/** Return the LongPackageName of module's native script package. Does not check whether module is native. */
	static COREUOBJECT_API FName GetModuleScriptPackageName(FName ModuleName);
	static COREUOBJECT_API FString GetModuleScriptPackageName(FStringView ModuleName);
	/** If PackageName is a script package (/Script/<ModuleName>), return true and set OutModuleName=<ModuleName> */
	static COREUOBJECT_API bool TryConvertScriptPackageNameToModuleName(FStringView PackageName, FStringView& OutModuleName);

	/**
	 * Registers all short package names found in ini files.
	 */
	static COREUOBJECT_API void RegisterShortPackageNamesForUObjectModules();

	/**
	 * Finds long script package name associated with a short package name.
	 *
	 * @param InShortName Short script package name.
	 * @return Long script package name (/Script/Package) associated with short name or NULL.
	 */
	static COREUOBJECT_API FName* FindScriptPackageName(FName InShortName);

	/**
	 * Tries to convert the supplied relative or absolute filename to a long package name/path starting with a root like
	 * /game This works on both package names and directories, and it does not validate that it actually exists on disk.
	 *
	 * @param InFilename Filename to convert.
	 * @param OutPackageName The resulting long package name if the conversion was successful.
	 * @param OutFailureReason Description of an error if the conversion failed.
	 * @param Flags Modifies the behaviour of how filename conversions are applied.
	 * @return Returns true if the supplied filename properly maps to one of the long package roots.
	 */
	static COREUOBJECT_API bool TryConvertFilenameToLongPackageName(const FString& InFilename, FString& OutPackageName,
		FString* OutFailureReason = nullptr, const EConvertFlags Flags = EConvertFlags::None);
	static COREUOBJECT_API bool TryConvertFilenameToLongPackageName(FStringView InFilename,
		FStringBuilderBase& OutPackageName, FStringBuilderBase* OutFailureReason = nullptr,
		const EConvertFlags Flags = EConvertFlags::None);

	/** 
	 * Converts the supplied filename to long package name.
	 * Throws a fatal error if the conversion is not successful.
	 * 
	 * @param InFilename Filename to convert.
	 * @return Long package name.
	 */
	static COREUOBJECT_API FString FilenameToLongPackageName(const FString& InFilename);

	/** 
	 * Tries to convert a long package name to a file name with the supplied extension.
	 * This can be called on package paths as well, provide no extension in that case
	 *
	 * @param InLongPackageName Long Package Name
	 * @param InExtension Package extension.
	 * @return Package filename.
	 */
	static COREUOBJECT_API bool TryConvertLongPackageNameToFilename(const FString& InLongPackageName, FString& OutFilename, const FString& InExtension = TEXT(""));
	static COREUOBJECT_API bool TryConvertLongPackageNameToFilename(FStringView InLongPackageName, FString& OutFilename, FStringView InExtension = {});

	/** 
	 * Find the MountPoint for a LocalPath, LongPackageName, or ObjectPath and return its elements. Use this function instead of TryConvertFilenameToLongPackageName or
	 * TryConvertLongPackageNameToFilename if you need to handle InPaths that might be ObjectPaths.
	 * @param InPath					The LocalPath (with path,name,extension), PackageName, or ObjectPath we want to 
	 * @param OutLocalPathNoExtension	If non-null, will be set to the LocalPath with path and basename but not extension, or empty string if input was not mounted
	 * @param OutPackageName			If non-null, will be set to the LongPackageName, or empty string if input was not mounted
	 * @param OutObjectName				If non-null, will be set to the ObjectName, or empty string if the input was a LocalPath or PackageName or was not mounted
	 * @param OutSubObjectName			If non-null, will be set to the SubObjectName, or empty string if the input was a LocalPath or PackageName or was not mounted
	 * @param OutExtension				If non-null, will be set to the LocalPath's extension, or empty string if the input was not a LocalPath or was not mounted
	 * @param OutFlexNameType			If non-null, will be set to the FlexNameType of InPath.
	 * @param OutFailureReason			If non-null, will be set to the reason InPath could not be converted, or to EErrorCode::Unknown if the function was successful.
	 */
	static COREUOBJECT_API bool TryConvertToMountedPath(FStringView InPath, FString* OutLocalPathNoExtension, FString* OutPackageName, FString* OutObjectName, FString* OutSubObjectName, FString* OutExtension, EFlexNameType* OutFlexNameType = nullptr, EErrorCode* OutFailureReason = nullptr);

	/** 
	 * Converts a long package name to a file name with the supplied extension.
	 * Throws a fatal error if the conversion is not successful.
	 *
	 * @param InLongPackageName Long Package Name
	 * @param InExtension Package extension.
	 * @return Package filename.
	 */
	static COREUOBJECT_API FString LongPackageNameToFilename(const FString& InLongPackageName, const FString& InExtension = TEXT(""));

	/** 
	 * Returns the path to the specified package, excluding the short package name
	 * e.g. given /Game/Maps/MyMap returns /Game/Maps
	 *
	 * @param InLongPackageName Long Package Name.
	 * @return The path containing the specified package.
	 */
	static COREUOBJECT_API FString GetLongPackagePath(const FString& InLongPackageName);

	/** 
	 * Returns the clean asset name for the specified package, same as GetShortName
	 *
	 * @param InLongPackageName Long Package Name
	 * @return Clean asset name.
	 */
	static COREUOBJECT_API FString GetLongPackageAssetName(const FString& InLongPackageName);

	/**
	 * Split a full object path (Class /Path/To/A/Package.Object:Subobject1.Subobject2) into its constituent pieces
	 *  
	 * @param InFullObjectPath  Full object path we want to split
	 * @param OutClassName      The extracted class name (Class)
	 * @param OutPackageName    The extracted package name (/Path/To/A/Package)
	 * @param OutObjectName     The extracted object name (Object)
	 * @param OutSubObjectName  The extracted subobject name (Subobject1.Subobject2) - Note: nested subobjects are not split
	 * @param bDetectClassName  If true, the optional Class will be detected and separated based on a space.
	 *                          If false, and there is a space, the space and text before it will be included in the
	 *                          other names. Spaces in those names is invalid, but some code ignores the
	 *                          invalidity in ObjectName if it only cares about packageName.
	 */
	static COREUOBJECT_API void SplitFullObjectPath(const FString& InFullObjectPath, FString& OutClassName,
		FString& OutPackageName, FString& OutObjectName, FString& OutSubObjectName, bool bDetectClassName = true);
	static COREUOBJECT_API void SplitFullObjectPath(FStringView InFullObjectPath, FStringView& OutClassName,
		FStringView& OutPackageName, FStringView& OutObjectName, FStringView& OutSubObjectName, bool bDetectClassName=true);

	/**
	 * Split a full object path (Class /Path/To/A/Package.Object:Subobject1.Subobject2) into its constituent pieces.
	 * All subobjects are split individually and stored in an array.
	 *
	 * @param InFullObjectPath  Full object path we want to split
	 * @param OutClassName      The extracted class name (Class)
	 * @param OutPackageName    The extracted package name (/Path/To/A/Package)
	 * @param OutSubobjectNames The extracted object name (Object)
	 * @param OutSubObjectName  The extracted subobject names (Subobject1 and Subobject2)
	 * @param bDetectClassName  If true, the optional Class will be detected and separated based on a space.
	 *                          If false, and there is a space, the space and text before it will be included in the
	 *                          other names. Spaces in those names is invalid, but some code ignores the
	 *                          invalidity in ObjectName if it only cares about packageName.
	 */
	static COREUOBJECT_API void SplitFullObjectPath(const FString& InFullObjectPath, FString& OutClassName,
		FString& OutPackageName, FString& OutObjectName, TArray<FString>& OutSubobjectNames, bool bDetectClassName = true);
	static COREUOBJECT_API void SplitFullObjectPath(FStringView InFullObjectPath, FStringView& OutClassName,
		FStringView& OutPackageName, FStringView& OutObjectName, TArray<FStringView>& OutSubobjectNames, bool bDetectClassName = true);

	/** 
	 * Returns true if the path starts with a valid root (i.e. /Game/, /Engine/, etc) and contains no illegal characters.
	 *
	 * @param InLongPackageName			The package name to test
	 * @param bIncludeReadOnlyRoots		If true, will include roots that you should not save to. (/Temp/, /Script/)
	 * @param OutReason					When returning false, this will provide a description of what was wrong with the name.
	 * @return							true if a valid long package name
	 */
	static COREUOBJECT_API bool IsValidLongPackageName(FStringView InLongPackageName, bool bIncludeReadOnlyRoots = false, EErrorCode* OutReason = nullptr);
	static COREUOBJECT_API bool IsValidLongPackageName(FStringView InLongPackageName, bool bIncludeReadOnlyRoots, FText* OutReason );

	/**
	 * Report whether a given name is the proper format for a PackageName, without checking whether it is in one of the registered mount points
	 *
	 * @param InLongPackageName			The package name to test
	 * @param OutReason					When returning false, this will provide a description of what was wrong with the name.
	 * @return							true if valid text for a long package name
	 */
	static COREUOBJECT_API bool IsValidTextForLongPackageName(FStringView InLongPackageName, EErrorCode* OutReason = nullptr);
	static COREUOBJECT_API bool IsValidTextForLongPackageName(FStringView InLongPackageName, FText* OutReason);


	/**
	 * Returns true if the path starts with a valid root (i.e. /Game/, /Engine/, etc) and contains no illegal characters.
	 * This validates that the packagename is valid, and also makes sure the object after package name is also correct.
	 * This will return false if passed a path starting with Classname'
	 *
	 * @param InObjectPath				The object path to test
	 * @param OutReason					When returning false, this will provide a description of what was wrong with the name.
	 * @return							true if a valid object path
	 */
	static COREUOBJECT_API bool IsValidObjectPath(FStringView InObjectPath, FText* OutReason = nullptr);

	/**
	 * Returns true if the path starts with a valid root (i.e. /Game/, /Engine/, /Memory/, etc).
	 * 
	 * @param InObjectPath				The object path to test
	 * @return							true if a valid object path
	 */
	static COREUOBJECT_API bool IsValidPath(FStringView InPath);

	/**
	 * Checks if the string is a ShortPackageName. A ShortPackageName is the leaf name after the last
	 * slash in a LongPackageName. Handling ShortPackageNames is useful for console commands and other UI.
	 * A ShortPackageName requires a search to convert to a LongPackageName.
	 *
	 * @param PossiblyLongName Package name.
	 * @return true if the given name is a short package name (contains no slashes), false otherwise.
	 */
	static COREUOBJECT_API bool IsShortPackageName(const FString& PossiblyLongName);
	static COREUOBJECT_API bool IsShortPackageName(const FName PossiblyLongName);
	static COREUOBJECT_API bool IsShortPackageName(FStringView PossiblyLongName);

	/**
	 * Converts package name to short name.
	 *
	 * @param Package Package with name to convert.
	 * @return Short package name.
	 */
	static COREUOBJECT_API FString GetShortName(const UPackage* Package);

	/**
	 * Converts package name to short name.
	 *
	 * @param LongName Package name to convert.
	 * @return Short package name.
	 */
	static COREUOBJECT_API FString GetShortName(FStringView LongName);
	static COREUOBJECT_API FString GetShortName(const FName& LongName);
	static COREUOBJECT_API FString GetShortName(const TCHAR* LongName);

	/**
	 * Converts package name to short name.
	 *
	 * @param LongName Package name to convert.
	 * @return Short package name.
	 */
	static COREUOBJECT_API FName GetShortFName(FStringView LongName);
	static COREUOBJECT_API FName GetShortFName(const FName& LongName);
	static COREUOBJECT_API FName GetShortFName(const TCHAR* LongName);

	/**
	 * Tries to convert a file or directory in game-relative package name format to the corresponding local path
	 * Game-relative package names can be a full package path (/Game/Folder/File, /Engine/Folder/File, /PluginName/Folder/File) or
	 * a relative path (Folder/File).
	 * Full package paths must be in a mounted directory to be successfully converted.
	 *
	 * @param RelativePackagePath The path in game-relative package format (allowed to have or not have an extension).
	 * @param OutLocalPath The corresponding local-path file (with the extension or lack of extension from the input).
	 * @return Whether the conversion was successful.
	 */
	static COREUOBJECT_API bool TryConvertGameRelativePackagePathToLocalPath(FStringView RelativePackagePath, FString& OutLocalPath);

	/**
	 * This will mount a mount point if not already mounted, and will update its order to the front of the override
	 * chain (so it can overlap an existing mount point and win).
	 * If you register a mount point (even if you do so only in certain circumstances) consider also adding a handler
	 * for GetExplanationForUnavailablePackage to help debug cases where a package can't be found.
	 * This function can fail only if the input arguments are not valid text for RootPath or ContentPath.
	 *
	 * @param RootPath LongPackageName location for the root of the MountPoint,
	 *        Examples: /Engine/, /PluginName/
	 * @param ContentPath LocalPath location for the root of the MountPoint,
	 *        Examples: ../../../Engine/Content, d:\FullPathToUnreal\Engine\Plugins\PluginName
	 * 
	 * @return RefCountPtr to the mounted IMountPoint, or nullptr if failed.
	 */
	static COREUOBJECT_API TRefCountPtr<UE::PackageName::IMountPoint>
		RegisterMountPoint(const FString& RootPath, const FString& ContentPath);
	/**
	 * Mounts a mount point if not already mounted, and updates its order to front of the override chain.
	 * If you register a mount point (even if you do so only in certain circumstances) consider also adding a handler
	 * for GetExplanationForUnavailablePackage to help debug cases where a package can't be found.
	 */
	static COREUOBJECT_API void RegisterMountPoint(const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint);

	/**
	 * This will remove a previously inserted mount point. Logs a message if not found or not removable.
	 *
	 * @param RootPath LongPackageName location for the root of the MountPoint,
	 *        Examples: /Engine/, /PluginName/
	 * @param ContentPath LocalPath location for the root of the MountPoint,
	 *        Examples: ../../../Engine/Content, d:\FullPathToUnreal\Engine\Plugins\PluginName
	 */
	static COREUOBJECT_API void UnRegisterMountPoint(const FString& RootPath, const FString& ContentPath);
	/** Unmounts a MountPoint; the mountpoint remains findable and updatable until all references are released. */
	static COREUOBJECT_API void UnRegisterMountPoint(const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint);


	/**
	 * Returns whether the given RootPath is the LongPackageName of a MountPoint that has been mounted, by default or
	 * by an earlier call to RegisterMountPoint. 
	 */
	static COREUOBJECT_API bool MountPointExists(FStringView RootPath);
	static COREUOBJECT_API bool MountPointExists(const FString& RootPath);
	static COREUOBJECT_API bool MountPointExists(const TCHAR* RootPath);

	/**
	 * Looks for the first allocated MountPoint with the given RootLongPackageName. If multiple are allocated, only
	 * returns the one with the highest override priority (the one that was added last). A discovered MountPoint might
	 * or might not be mounted, call MountPoint->IsMounted() to check for mounted.
	 * 
	 * @param RootLongPackageName LongPackageName location for the root of the MountPoint,
	 *        Examples: /Engine/, /PluginName/
	 * @return RefCountPtr to the IMountPoint, or nullptr if not yet allocated.
	 */
	static COREUOBJECT_API TRefCountPtr<UE::PackageName::IMountPoint>
		FindMountPointByRootPackageName(FStringView RootLongPackageName);
	/**
	 * Looks for the first MountPoint with the given RootLocalPath. If multiple are allocated, only returns the
	 * one with the highest override priority (the one that was added last). A discovered MountPoint might or might
	 * not be mounted, call MountPoint->IsMounted() to check for mounted.
	 *
	 * @param RootLocalPath LocalPath location for the root of the MountPoint,
	 *        Examples: ../../../Engine/Content, d:\FullPathToUnreal\Engine\Plugins\PluginName
	 * @return RefCountPtr to the IMountPoint, or nullptr if not yet allocated.
	 */
	static COREUOBJECT_API TRefCountPtr<UE::PackageName::IMountPoint>
		FindMountPointByRootLocalPath(FStringView RootLocalPath);
	/**
	 * Looks for an existing allocated MountPoint with the given RootLongPackageName and RootLocalPath. These fields
	 * uniquely identify a MountPoint so at most one can exist.
	 * If the MountPoint exists, it might or might not be mounted, call MountPoint->IsMounted() to check for mounted.
	 *
	 * @param RootLongPackageName LongPackageName location for the root of the MountPoint,
	 *        Examples: /Engine/, /PluginName/
	 * @param RootLocalPath LocalPath location for the root of the MountPoint,
	 *        Examples: ../../../Engine/Content, d:\FullPathToUnreal\Engine\Plugins\PluginName
	 * @return RefCountPtr to the IMountPoint, or nullptr if not yet allocated.
	 */
	static COREUOBJECT_API TRefCountPtr<UE::PackageName::IMountPoint>
		FindMountPoint(FStringView RootLongPackageName, FStringView RootLocalPath);
	/**
	 * Looks for an existing allocated MountPoint with the given RootLongPackageName and RootLocalPath, adds it
	 * if it does not already exist. These fields uniquely identify a MountPoint so at most one can exist.
	 * When the MountPoint is returned, it might or might not be mounted, call MountPoint->IsMounted() to check for
	 * mounted.
	 * This function can only return null if the input arguments are not valid text for RootLongPackageName or
	 * RootLocalPath.
	 *
	 * @param RootLongPackageName LongPackageName location for the root of the MountPoint,
	 *        Examples: /Engine/, /PluginName/
	 * @param RootLocalPath LocalPath location for the root of the MountPoint,
	 *        Examples: ../../../Engine/Content, d:\FullPathToUnreal\Engine\Plugins\PluginName
	 * @return RefCountPtr to the IMountPoint, or nullptr if failed.
	 */
	static COREUOBJECT_API TRefCountPtr<UE::PackageName::IMountPoint>
		FindOrAddMountPoint(FStringView RootLongPackageName, FStringView RootLocalPath);

	/**
	 * Looks for the first MountPoint that is equal to or is a parent path of the given ChildLongPackageName.
	 * If multiple are allocated, only returns the one with the highest override priority (the one that was added
	 * last). A discovered MountPoint might or might not be mounted, call MountPoint->IsMounted() to check for mounted.
	 *
	 * @param ChildLongPackageName LongPackageName for a path under the MountPoint,
	 *        Examples: /Engine/, /Engine/Asset, /PluginName/, /PluginName/Asset
	 * @return RefCountPtr to the IMountPoint, or nullptr if not yet allocated.
	 */
	static COREUOBJECT_API TRefCountPtr<UE::PackageName::IMountPoint>
		FindMountPointByChildLongPackageName(FStringView ChildLongPackageName);

	/**
	 * Get the mount point for a given package path
	 * 
	 * @param InPackagePath The package path to get the mount point for
	 * @param InWithoutSlashes Optional parameters that keeps the slashes around the mount point if false
	 * @return FName corresponding to the mount point, or Empty if invalid
	 */
	static COREUOBJECT_API FName GetPackageMountPoint(FStringView InPackagePath, bool InWithoutSlashes = true);
	static COREUOBJECT_API FName GetPackageMountPoint(const FString& InPackagePath, bool InWithoutSlashes = true);
	static COREUOBJECT_API FName GetPackageMountPoint(const TCHAR* InPackagePath, bool InWithoutSlashes = true);

	/**
	 * Get the path associated with the given package root
	 * @param InPackageRoot Package root to return the path for, e.g. /Game/, /Engine/, /PluginName/
	 * @return Filesystem path associated with the provided package root
	 */
	static COREUOBJECT_API FString GetContentPathForPackageRoot(FStringView InPackageRoot);

	/**
	 * Checks if the package exists on disk.
	 * 
	 * @param LongPackageName Package name.
	 * @param OutFilename Package filename on disk.
	 * @param InAllowTextFormats Detect text format packages as well as binary (priority to text)
	 * @return true if the specified package name points to an existing package, false otherwise.
	 **/
	static COREUOBJECT_API bool DoesPackageExist(const FString& LongPackageName, FString* OutFilename = nullptr, bool InAllowTextFormats = true);

	/**
	 * Checks if the package exists on disk. PackagePath must be a mounted path, otherwise returns false
	 * 
	 * @param PackagePath Package package.
	 * @param Guid If nonnull, and the package is found on disk but does not have this PackageGuid in its FPackageFileSummary::Guid, false is returned
	 * @param bMatchCaseOnDisk If true, the OutPackagePath is modified to match the capitalization of the discovered file
	 * @param OutPackagePath If nonnull and the package exists, set to a copy of PackagePath with the HeaderExtension set to the extension that exists on disk (and if bMatchCaseOnDisk is true, capitalization changed to match). If not found, this variable is not written
	 * @return true if the specified package name points to an existing package, false otherwise.
	 **/
	static COREUOBJECT_API bool DoesPackageExist(const FPackagePath& PackagePath, bool bMatchCaseOnDisk = false, FPackagePath* OutPackagePath = nullptr);

	/**
	 * Checks if the package exists on disk. PackagePath must be a mounted path, otherwise returns false
	 * 
	 * @param PackagePath Package package.
	 * @param OutPackagePath If nonnull and the package exists, set to a copy of PackagePath with the HeaderExtension set to the extension that exists on disk. If not found, this variable is not written
	 * @return true if the specified package name points to an existing package, false otherwise.
	 **/
	static COREUOBJECT_API bool DoesPackageExist(const FPackagePath& PackagePath, FPackagePath* OutPackagePath);

	enum class EPackageLocationFilter : uint8
	{
		None = 0,
		IoDispatcher = 1,
		FileSystem = 2,

		MaxBit = FileSystem,
		ValidMask = (1 << MaxBit) - 1,

		// Any allows finding the PackageData if it exists anywhere at all, and won't need to check both. When using Any,
		// as soon as the package is found in Any location, DoesPackageExistEx will return true
		Any = 0xFF,
	};

	/**
	 * Checks if the package exists in IOStore containers, on disk outside of IOStore, both, or neither
	 *
	 * @param PackagePath Package package.
	 * @param Filter Indication of where it should look for 
	 * @param Guid If non-null, and the package is found on disk but does not have this PackageGuid in its FPackageFileSummary::Guid, false is returned
	 * @param bMatchCaseOnDisk If true, the OutPackagePath is modified to match the capitalization of the discovered file
	 * @param OutPackagePath If non-null and the package exists, set to a copy of PackagePath with the HeaderExtension set to the extension that exists on disk (and if bMatchCaseOnDisk is true, capitalization changed to match). If not found, this variable is not written
	 * @return the set of locations where the package exists (IoDispatcher or FileSystem, both or neither)
	 **/
	static COREUOBJECT_API EPackageLocationFilter DoesPackageExistEx(const FPackagePath& PackagePath, EPackageLocationFilter Filter, bool bMatchCaseOnDisk = false, FPackagePath* OutPackagePath = nullptr);

	/**
	 * Attempts to find a package given its short name on disk (very slow).
	 * 
	 * @param PackageName Package to find.
	 * @param OutLongPackageName Long package name corresponding to the found file (if any).
	 * @return true if the specified package name points to an existing package, false otherwise.
	 **/
	static COREUOBJECT_API bool SearchForPackageOnDisk(const FString& PackageName, FString* OutLongPackageName = NULL, FString* OutFilename = NULL);

	/**
	 * Tries to convert object path with short package name to object path with long package name found on disk (very slow)
	 *
	 * @param ObjectPath Path to the object.
	 * @param OutLongPackageName Converted object path.
	 *
	 * @return True if succeeded. False otherwise.
	 */
	static COREUOBJECT_API bool TryConvertShortPackagePathToLongInObjectPath(const FString& ObjectPath, FString& ConvertedObjectPath);

	/**
	 * Gets normalized object path i.e. with long package format.
	 *
	 * @param ObjectPath Path to the object.
	 *
	 * @return Normalized path (or empty path, if short object path was given and it wasn't found on the disk).
	 */
	static COREUOBJECT_API FString GetNormalizedObjectPath(const FString& ObjectPath);

	/**
	 * Gets the resolved path of a long package as determined by the delegates registered with FCoreDelegates::PackageNameResolvers.
	 * This allows systems such as localization to redirect requests for a package to a more appropriate alternative, or to 
	 * nix the request altogether.
	 *
	 * @param InSourcePackagePath	Path to the source package.
	 *
	 * @return Resolved package path, or the source package path if there is no resolution occurs.
	 */
	static COREUOBJECT_API FString GetDelegateResolvedPackagePath(const FString& InSourcePackagePath);

	/**
	 * Gets the source version of a localized long package path (it is also safe to pass non-localized paths into this function).
	 *
	 * @param InLocalizedPackagePath Path to the localized package.
	 *
	 * @return Source package path.
	 */
	static COREUOBJECT_API FString GetSourcePackagePath(const FString& InLocalizedPackagePath);

	/**
	 * Gets the localized version of a long package path for the current culture, or returns the source package if there is no suitable localized package.
	 *
	 * @param InSourcePackagePath	Path to the source package.
	 *
	 * @return Localized package path, or the source package path if there is no suitable localized package.
	 */
	static COREUOBJECT_API FString GetLocalizedPackagePath(const FString& InSourcePackagePath);

	/**
	 * Gets the localized version of a long package path for the given culture, or returns the source package if there is no suitable localized package.
	 *
	 * @param InSourcePackagePath	Path to the source package.
	 * @param InCultureName			Culture name to get the localized package for.
	 *
	 * @return Localized package path, or the source package path if there is no suitable localized package.
	 */
	static COREUOBJECT_API FString GetLocalizedPackagePath(const FString& InSourcePackagePath, const FString& InCultureName);

	/**
	 * Gets the versepath of the top level object.
	 *
	 * @return The VersePath of the top level object
	 */
	UE_INTERNAL static COREUOBJECT_API UE::Core::FVersePath GetVersePath(const FTopLevelAssetPath& AssetPath);

	/**
	 * Gets the Verse path of a long package path.  Should not be used with a long package name.
	 *
	 * @return The Verse path of the long package path
	 */
	UE_INTERNAL static COREUOBJECT_API UE::Core::FVersePath LongPackagePathToVersePath(FStringView LongPackagePath);

	/** 
	 * Returns the file extension for packages containing assets.
	 *
	 * @return	file extension for asset packages ( dot included )
	 */
	static COREUOBJECT_API const FString& GetAssetPackageExtension();
	/** 
	 * Returns the file extension for packages containing assets.
	 *
	 * @return	file extension for asset packages ( dot included )
	 */
	static COREUOBJECT_API const FString& GetMapPackageExtension();

	/**
	* Returns the file extension for packages containing text assets.
	*
	* @return	file extension for text asset packages ( dot included )
	*/
	static COREUOBJECT_API const FString& GetTextAssetPackageExtension();

	/**
	* Returns the file extension for packages containing text maps.
	*
	* @return	file extension for text map packages ( dot included )
	*/
	static COREUOBJECT_API const FString& GetTextMapPackageExtension();

	/**
	 * Returns whether the passed in extension is a valid text package
	 * extension. Extensions with and without trailing dots are supported.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is either an text asset or a text map extension, otherwise false
	 */
	static COREUOBJECT_API bool IsTextPackageExtension(const TCHAR* Ext);

	/**
	 * Returns whether the passed in extension is a text header extension
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is either an text asset or a text map extension, otherwise false
	 */
	static COREUOBJECT_API bool IsTextPackageExtension(EPackageExtension Extension);

	/**
	 * Returns whether the passed in extension is a valid text asset package
	 * extension. Extensions with and without trailing dots are supported.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is a text asset extension, otherwise false
	 */
	static COREUOBJECT_API bool IsTextAssetPackageExtension(const TCHAR* Ext);

	/**
	 * Returns whether the passed in extension is a valid text map package
	 * extension. Extensions with and without trailing dots are supported.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is a text map extension, otherwise false
	 */
	static COREUOBJECT_API bool IsTextMapPackageExtension(const TCHAR* Ext);

	/** 
	 * Returns whether the passed in extension is a valid binary package
	 * extension. Extensions with and without trailing dots are supported.
	 *
	 * @param	Extension to test. 
	 * @return	True if Ext is either a binary  asset or map extension, otherwise false
	 */
	static COREUOBJECT_API bool IsPackageExtension(const TCHAR* Ext);

	/**
	 * Returns whether the passed in extension is a valid binary package extension.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is either a binary asset or a binary map extension, otherwise false
	 */
	static COREUOBJECT_API bool IsPackageExtension(EPackageExtension Extension);

	/**
	 * Returns whether the passed in extension is a valid binary asset package
	 * extension. Extensions with and without trailing dots are supported.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is a binary asset extension, otherwise false
	 */
	static COREUOBJECT_API bool IsAssetPackageExtension(const TCHAR* Ext);

	/**
	 * Returns whether the passed in extension is a valid binary map package
	 * extension. Extensions with and without trailing dots are supported.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is a binary asset extension, otherwise false
	 */
	static COREUOBJECT_API bool IsMapPackageExtension(const TCHAR* Ext);

	/**
	 * Returns whether the passed in extension is a valid verse extension.
	 * Extensions with and without trailing dots are supported.
	 *
	 * @param	Extension to test.
	 * @return	True if Ext is a verse extension, otherwise false
	 */
	static COREUOBJECT_API bool IsVerseExtension(const TCHAR* Ext);

	/** 
	 * Returns whether the passed in filename ends with any of the known
	 * package extensions.
	 *
	 * @param	Filename to test. 
	 * @return	True if the filename ends with a package extension.
	 */
	static inline bool IsPackageFilename(FStringView Filename)
	{
		FStringView AssetPackageExtension(LexToString(EPackageExtension::Asset));
		FStringView MapPackageExtension(LexToString(EPackageExtension::Map));
		return Filename.EndsWith(AssetPackageExtension) || Filename.EndsWith(MapPackageExtension);
	}

	/** Return the text string used to mark the _Generated_ directory of packages created by CookPackageSplitters. */
	static COREUOBJECT_API const TCHAR* GetGeneratedPackageSubPath();

	/**
	 * Return whether a given packagename or file path is generated based on its name (we evaluate whether it
	 * is in a GetGeneratedPackageSubPath directory).
	 */
	static COREUOBJECT_API bool IsUnderGeneratedPackageSubPath(FStringView FileOrLongPackagePath);

	/**
	 * This will recurse over a directory structure looking for packages.
	 * 
	 * @param	OutPackages			The output array that is filled out with the discovered file paths
	 * @param	RootDir				The root of the directory structure to recurse through
	 * @return	Returns true if any packages have been found, otherwise false
	 */
	static COREUOBJECT_API bool FindPackagesInDirectory(TArray<FString>& OutPackages, const FString& RootDir);

	/**
	 * This will recurse over the given list of directory structures looking for packages.
	 *
	 * @param	OutPackages			The output array that is filled out with the discovered file paths
	 * @param	RootDirss			The roots of the directory structures to recurse through
	 * @return	Returns true if any packages have been found, otherwise false
	 */
	static COREUOBJECT_API bool FindPackagesInDirectories(TArray<FString>& OutPackages, const TArrayView<const FString>& RootDirs);

	/**
	 * This will recurse over a directory structure looking for packages.
	 * 
	 * @param	RootDirectory		The root of the directory structure to recurse through
	 * @param	Visitor				Visitor to call for each package file found (takes the package filename, and optionally the stat data for the file - returns true to continue iterating)
	 */
	typedef TFunctionRef<bool(const TCHAR*)> FPackageNameVisitor;
	typedef TFunctionRef<bool(const TCHAR*, const FFileStatData&)> FPackageNameStatVisitor;
	static COREUOBJECT_API void IteratePackagesInDirectory(const FString& RootDir, const FPackageNameVisitor& Visitor);
	static COREUOBJECT_API void IteratePackagesInDirectory(const FString& RootDir, const FPackageNameStatVisitor& Visitor);

	/** Event triggered when a MountPoint is mounted. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FMountPointEvent, const TRefCountPtr<UE::PackageName::IMountPoint>& MountPoint)
	static FMountPointEvent& OnMountPointMounted()
	{
		return OnMountPointMountedEvent;
	}
	/** Legacy event triggered when a MountPoint is mounted; passes strings instead of IMountPoint. */
	DECLARE_MULTICAST_DELEGATE_TwoParams( FOnContentPathMountedEvent, const FString& /* Asset path */, const FString& /* ContentPath */ );
	static FOnContentPathMountedEvent& OnContentPathMounted()
	{
		return OnContentPathMountedEvent;
	}

	/** Event triggered when a MountPoint is dismounted. */
	static FMountPointEvent& OnMountPointDismounted()
	{
		return OnMountPointDismountedEvent;
	}
	/** Legacy event triggered when a MountPoint is dismounted; passes strings instead of IMountPoint. */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnContentPathDismountedEvent, const FString& /* Asset path */, const FString& /* ContentPath */ );
	static FOnContentPathDismountedEvent& OnContentPathDismounted()
	{
		return OnContentPathDismountedEvent;
	}

	/**
	 * Queries all of the root content paths, like "/Game/", "/Engine/", and any dynamically added paths
	 *
	 * @param	OutRootContentPaths	[Out] List of content paths
	 * @param	bIncludeReadOnlyRoots	  Include read only root content paths such as "/Temp/"
	 * @param	bWithoutLeadingSlashes	  Strip slash at start of each path to end up with "Game/"
	 * @param	bWithoutTrailingSlashes	  Strip trailing slash at end of each path to end up with "/Game"
	 */
	static COREUOBJECT_API void QueryRootContentPaths( TArray<FString>& OutRootContentPaths, bool bIncludeReadOnlyRoots = false, bool bWithoutLeadingSlashes = false, bool bWithoutTrailingSlashes = false);
	
	/**
	 * Returns all of the local paths on disk of the root content paths, like
	 * "c:\MyProjects\ProjectA\Content", "d:\Unreal\Engine\Content", "c:\MyProjects\ProjectA\Plugins\MyPlugin\Content"
	 */
	static COREUOBJECT_API TArray<FString> QueryMountPointLocalAbsPaths();

	/** If the FLongPackagePathsSingleton is not created yet, this function will create it and thus allow mount points to be added */
	static COREUOBJECT_API void OnCoreUObjectInitialized();

	/** 
	 * Converts the supplied export text path to an object path and class name.
	 * 
	 * @param InExportTextPath The export text path for an object. Takes on the form: ClassName'ObjectPath'
	 * @param OutClassName The name of the class at the start of the path.
	 * @param OutObjectPath The path to the object.
	 * @return True if the supplied export text path could be parsed
	 */
	static COREUOBJECT_API bool ParseExportTextPath(FWideStringView InExportTextPath, FWideStringView* OutClassName, FWideStringView* OutObjectPath);
	static COREUOBJECT_API bool ParseExportTextPath(FAnsiStringView InExportTextPath, FAnsiStringView* OutClassName, FAnsiStringView* OutObjectPath);
	static COREUOBJECT_API bool ParseExportTextPath(const FString& InExportTextPath, FString* OutClassName, FString* OutObjectPath);	
	static COREUOBJECT_API bool ParseExportTextPath(const TCHAR* InExportTextPath, FStringView* OutClassName, FStringView* OutObjectPath);


	/** 
	 * Returns the path to the object referred to by the supplied export text path, excluding the class name.
	 * 
	 * @param InExportTextPath The export text path for an object. Takes on the form: ClassName'ObjectPath'
	 * @return The path to the object referred to by the supplied export path.
	 */
	static COREUOBJECT_API FWideStringView	ExportTextPathToObjectPath(FWideStringView InExportTextPath);
	static COREUOBJECT_API FAnsiStringView	ExportTextPathToObjectPath(FAnsiStringView InExportTextPath);
	static COREUOBJECT_API FString			ExportTextPathToObjectPath(const FString& InExportTextPath);
	static COREUOBJECT_API FString			ExportTextPathToObjectPath(const TCHAR* InExportTextPath);

	/** Flags for the output format of path functions. */
	enum class EPathFormatFlags : uint32
	{
		None = 0x0,
		MountPointLeadingSlash = 0x1,
		MountPointTrailingSlash = 0x2,
		MountPointSlashes = MountPointLeadingSlash | MountPointTrailingSlash,
		MountPointNoSlashes = None,
	};

	/**
	 * Convert a long package name into root, path, and leafname components.
	 * For the FStringView version, the returned FStringViews are views into the input FStringView.
	 *
	 * @param InLongPackageName Package Name, e.g. "/Game/Maps/TestMaps/MyMap".
	 *        Also supports packagenames with extension (which are not valid package names),
	 *        e.g. /Game/Maps/TestMaps/MyMap.umap.
	 * @param OutPackageRoot The package root path, eg "/Game/"
	 * @param OutPackagePath The path from the mount point to the package, eg "Maps/TestMaps/
	 * @param OutPackageLeafName The leaf name, including extension if present, eg "MyMap" or "MyMap.umap".
	 * @param Flags Flags specifying which characters are included in OutputPackageRoot, @see EPathFormatFlags.
	 * 
	 * Edge cases examples:
	 * "" -> "","",""
	 * "/" -> "", "", ""
	 * "/root" -> "", "", ""
	 * "/root/" -> "/root/", "", ""
	 * "/root/leaf" -> "/root/", "", "leaf"
	 * "/root/path/" -> "/root/", "path/", ""
	 * "/root/path/leaf" -> "/root/", "path/", "leaf"
	 */
	static COREUOBJECT_API void SplitPackageName(FStringView InPackageName, FStringView* OutPackageRoot,
		FStringView* OutPackagePath, FStringView* OutPackageLeafName,
		EPathFormatFlags Flags = EPathFormatFlags::MountPointSlashes);
	static COREUOBJECT_API void SplitPackageName(const FString& InLongPackageName, FString* OutPackageRoot,
		FString* OutPackagePath, FString* OutPackageLeafName,
		EPathFormatFlags Flags = EPathFormatFlags::MountPointSlashes);

	/**
	 * Test whether a given packagename is in a valid root, and convert it to root, path, and leafname components.
	 * If not in a valid root, all output arguments are reset to empty.
	 *
	 * @param InLongPackageName Package Name, e.g. "/Game/Maps/TestMaps/MyMap".
	 *        Also supports packagenames with extension (which are not valid package names),
	 *        e.g. /Game/Maps/TestMaps/MyMap.umap.
	 * @param OutPackageRoot The package root path, eg "/Game/"
	 * @param OutPackagePath The path from the mount point to the package, eg "Maps/TestMaps/
	 * @param OutPackageLeafName The leaf name, including extension if present, eg "MyMap" or "MyMap.umap".
	 * @param bStripRootLeadingSlash: If false, OutpackageRoot="/<RootName>/", otherwise "<RootName>/".
	 * @return Whether the path is in a valid root.
	 */
	static COREUOBJECT_API bool SplitLongPackageName(const FString& InLongPackageName, FString& OutPackageRoot,
		FString& OutPackagePath, FString& OutPackageLeafName, const bool bStripRootLeadingSlash = false);

	/**
	 * Returns the top level 'directory' in a package name. If the package is part of a
	 * plugin, the root will be the name of the associated plugin, otherwise it will be
	 * 'Engine' or 'Game'.
	 * 
	 * Note the slashes are removed by default (EPathFormatFlags::MountPointNoSlashes), @see EPathFormatFlags.
	 * 
	 * Default:
	 * "/PackageRoot/Path/Leaf" -> (return "PackageRoot"; OutRelativePath = "Path/Leaf")
	 * With EPathFormatFlags::MountPointSlashes:
	 * "/PackageRoot/Path/Leaf" -> (return "/PackageRoot/"; OutRelativePath = "Path/Leaf")
	 */
	static COREUOBJECT_API FStringView SplitPackageNameRoot(FStringView InPackageName, FStringView* OutRelativePath,
		EPathFormatFlags Flags = EPathFormatFlags::MountPointNoSlashes);
	static COREUOBJECT_API FString SplitPackageNameRoot(FName InPackageName, FString* OutRelativePath,
		EPathFormatFlags Flags = EPathFormatFlags::MountPointNoSlashes);
	
	/** 
	 * Returns the name of the package referred to by the specified object path
	 * 
	 * Examples:
	 *   "/Game/MyAsset.MyAsset:SubObject.AnotherObject" -> "/Game/MyAsset"
	 *   "/Game/MyAsset.MyAsset:SubObject"               -> "/Game/MyAsset"
	 *   "/Game/MyAsset.MyAsset"                         -> "/Game/MyAsset"
	 *   "/Game/MyAsset"                                 -> "/Game/MyAsset"
	 */
	static COREUOBJECT_API FWideStringView ObjectPathToPackageName(FWideStringView InObjectPath);
	static COREUOBJECT_API FAnsiStringView ObjectPathToPackageName(FAnsiStringView InObjectPath);
	static COREUOBJECT_API FString ObjectPathToPackageName(const FString& InObjectPath);

	/**
	 * Returns any remaining object path after trimming the package name from the specified object path
	 *
	 * Examples:
	 *   "/Game/MyAsset.MyAsset:SubObject.AnotherObject" -> "MyAsset:SubObject.AnotherObject"
	 *   "/Game/MyAsset.MyAsset:SubObject"               -> "MyAsset:SubObject"
	 *   "/Game/MyAsset.MyAsset"                         -> "MyAsset"
	 *   "/Game/MyAsset"                                 -> ""
	 */
	static COREUOBJECT_API FWideStringView ObjectPathToPathWithinPackage(FWideStringView InObjectPath);
	static COREUOBJECT_API FAnsiStringView ObjectPathToPathWithinPackage(FAnsiStringView InObjectPath);
	static COREUOBJECT_API FString ObjectPathToPathWithinPackage(const FString& InObjectPath);

	/**
	 * Returns the path name of the outer of the leaf object referred to by the specified object path
	 *
	 * Examples:
	 *   "/Game/MyAsset.MyAsset:SubObject.AnotherObject" -> "/Game/MyAsset.MyAsset:SubObject"
	 *   "/Game/MyAsset.MyAsset:SubObject"               -> "/Game/MyAsset.MyAsset"
	 *   "/Game/MyAsset.MyAsset"                         -> "/Game/MyAsset"
	 *   "/Game/MyAsset"                                 -> ""
	 */
	static COREUOBJECT_API FWideStringView ObjectPathToOuterPath(FWideStringView InObjectPath);
	static COREUOBJECT_API FAnsiStringView ObjectPathToOuterPath(FAnsiStringView InObjectPath);
	static COREUOBJECT_API FString ObjectPathToOuterPath(const FString& InObjectPath);

	/**
	 * Returns the path from (and including) the subobject referred to by the specified object path
	 *
	* Examples:
	 *   "/Game/MyAsset.MyAsset:SubObject.AnotherObject" -> "SubObject.AnotherObject"
	 *   "/Game/MyAsset.MyAsset:SubObject"               -> "SubObject"
	 *   "/Game/MyAsset.MyAsset"                         -> "MyAsset"
	 *   "/Game/MyAsset"                                 -> "/Game/MyAsset"
	 */
	static COREUOBJECT_API FWideStringView ObjectPathToSubObjectPath(FWideStringView InObjectPath);
	static COREUOBJECT_API FAnsiStringView ObjectPathToSubObjectPath(FAnsiStringView InObjectPath);
	static COREUOBJECT_API FString ObjectPathToSubObjectPath(const FString& InObjectPath);

	/** 
	 * Returns the name of the leaf object referred to by the specified object path
	 *
	 * Examples:
	 *   "/Game/MyAsset.MyAsset:SubObject.AnotherObject" -> "AnotherObject"
	 *   "/Game/MyAsset.MyAsset:SubObject"               -> "SubObject"
	 *   "/Game/MyAsset.MyAsset"                         -> "MyAsset"
	 *   "/Game/MyAsset"                                 -> "/Game/MyAsset"
	 */
	static COREUOBJECT_API FWideStringView ObjectPathToObjectName(FWideStringView InObjectPath);
	static COREUOBJECT_API FAnsiStringView ObjectPathToObjectName(FAnsiStringView InObjectPath);
	static COREUOBJECT_API FString ObjectPathToObjectName(const FString& InObjectPath);

	/**
	 * Splits an ObjectPath string into the first component and the remainder. 
	 *
	 * "/Path/To/A/Package.Object:SubObject" -> { "/Path/To/A/Package", "Object:SubObject" }
	 * "Object:SubObject" -> { "Object", "SubObject" }
	 * "Object.SubObject" -> { "Object", "SubObject" }
	 */
	static COREUOBJECT_API void ObjectPathSplitFirstName(FWideStringView Text,
		FWideStringView& OutFirst, FWideStringView& OutRemainder);
	static COREUOBJECT_API void ObjectPathSplitFirstName(FAnsiStringView Text,
		FAnsiStringView& OutFirst, FAnsiStringView& OutRemainder);

	/**
	 * Combines an ObjectPath with an ObjectName.
	 * { "/Package", "Object" } -> "/Package.Object"
	 * { "/Package.Object", "SubObject" } -> "/Package.Object:SubObject"
	 * { "/Package.Object:SubObject", "NextSubObject" } -> "/Package.Object:SubObject.NextSubObject"
	 * { "/Package", "Object.SubObject" } -> "/Package.Object:SubObject"
	 * { "/Package", "/OtherPackage.Object:SubObject" } -> "/OtherPackage.Object:SubObject"
	*/
	static COREUOBJECT_API void ObjectPathAppend(FStringBuilderBase& ObjectPath, FStringView NextName);
	/**
	 * Combines an ObjectPath with an ObjectName, the same as ObjectPathAppend but returns the result rather
	 * than modifying the input argument.
	 */
	static COREUOBJECT_API FString ObjectPathCombine(FStringView ObjectPath, FStringView NextName);

	/**
	 * Checks the package's path to see if it's a Verse package
	 */
	static COREUOBJECT_API bool IsVersePackage(FStringView InPackageName);

	/**
	 * Checks the root of the package's path to see if it is a script package
	 * @return true if the root of the path matches the script path
	 */
	static COREUOBJECT_API bool IsScriptPackage(FStringView InPackageName);

	/**
	 * Checks the root of the package's path to see if it's a memory package
	 * This should be set for packages that reside in memory and not on disk, we treat them similar to a script package
	 * @return true if the root of the patch matches the memory path
	 */
	static COREUOBJECT_API bool IsMemoryPackage(FStringView InPackageName);

	/**
	 * Checks the root of the package's path to see if it is a temp package
	 * Temp packages are sometimes saved to disk, and sometimes only exist in memory. They are never in source control
	 * @return true if the root of the patch matches the temp path
	 */
	static COREUOBJECT_API bool IsTempPackage(FStringView InPackageName);

	/**
	 * Checks the root of the package's path to see if it is the EngineTransient package in specific, OR
	 * one of the packages in the directory of EngineTransient packages.
	 */
	static COREUOBJECT_API bool IsInEngineTransientPackages(FStringView InPackageName);

	/**
	 * Checks the root of the package's path to see if it is a localized package
	 * @return true if the root of the path matches any localized root path
	 */
	static COREUOBJECT_API bool IsLocalizedPackage(FStringView InPackageName);

	/**
	 * Checks if a package name contains characters that are invalid for package names.
	 */
	static COREUOBJECT_API bool DoesPackageNameContainInvalidCharacters(FStringView InLongPackageName, FText* OutReason);
	static COREUOBJECT_API bool DoesPackageNameContainInvalidCharacters(FStringView InLongPackageName, EErrorCode* OutReason = nullptr);
	
	/**
	* Checks if a package can be found using known package extensions (header extensions only; files with the extensions of other segments are not returned).
	*
	* @param InPackageFilename Package filename without the extension.
	* @param OutFilename If the package could be found, filename with the extension.
	* @param InAllowTextFormats Detect text format packages as well as binary (priority to text)
	* @return true if the package could be found on disk.
	*/
	static COREUOBJECT_API bool FindPackageFileWithoutExtension(const FString& InPackageFilename, FString& OutFilename);

	/**
	 * Converts a long package name to the case it exists as on disk.
	 *
	 * @param LongPackageName The long package name
	 * @param Extension The extension for this package
	 * @return True if the long package name was fixed up, false otherwise
	 */
	static COREUOBJECT_API bool FixPackageNameCase(FString& LongPackageName, FStringView Extension);

	/** Override whether a package exist or not. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FDoesPackageExistOverride, FName);
	static FDoesPackageExistOverride& DoesPackageExistOverride()
	{
		return DoesPackageExistOverrideDelegate;
	}

	/**
	 * Find a mount point that contains the given LocalFilePath or PackageName, and if found, return the MountPoint and RelativePath
	 *
	 * @param InFilePathOrPackageName The path to test, either a LocalFilePath or a PackageName or an ObjectPath
	 * @param OutMountPointPackageName If the MountPoint is found, the PackageName of the MountPoint is copied into this variable, otherwise it is set to empty string
	 * @param OutMountPointFilePath If the MountPoint is found, the LocalFilePath of the MountPoint is copied into this variable, otherwise it is set to empty string
	 * @param OutRelPath If the MountPoint is found, the RelativePath from the MountPoint to InFilePathOrPackageName is copied into this variable, otherwise it is set to empty string
	 * 					 If InFilePathOrPackageName was a filepath, the extension is removed before copying it into OutRelpath
	 *					 The OutRelPath is the same for both the LocalFilePath and the PackageName
	 * @param OutFlexNameType If non-null, will be set to whether InFilePathOrPackageName was a PackageName or Filename if the MountPoint is found, otherwise it is set to EFlexNameType::Invalid
	 * @param OutFailureReason If non-null, will be set to the reason InPath could not be converted, or to EErrorCode::Unknown if the function was successful.
	 * @return True if the MountPoint was found, else false
	 */
	static COREUOBJECT_API bool TryGetMountPointForPath(FStringView InFilePathOrPackageName, FStringBuilderBase& OutMountPointPackageName, FStringBuilderBase& OutMountPointFilePath, FStringBuilderBase& OutRelPath,
		EFlexNameType* OutFlexNameType = nullptr, EErrorCode* OutFailureReason = nullptr);

	/** Delegate type for allowing higher levels systems to provide additional context for why a package is not available */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FGetExplanationForUnavailablePackageDelegate, const FName& /*Unavailable Package*/, FStringBuilderBase& /*InOutExplanation*/);

	/**
	 * Accessor for installing callbacks for use with GetExplanationForUnavailablePackage
	 * 
	 */ 
	static COREUOBJECT_API FGetExplanationForUnavailablePackageDelegate& GetExplanationForUnavailablePackageDelegate();

	/**
	 * Attempt to generate an explanation for why a particular package was not available for loading
	 * 
	 * @param UnavailablePackageName The package that was requested but could not be loaded
	 * @param InOutExplanation A string builder that will be populated with any available information about why the package was not available. The string builder will *not* be reset and will be appended to.
	 */
	 static COREUOBJECT_API void GetExplanationForUnavailablePackage(const FName& UnavailablePackageName, FStringBuilderBase& InOutExplanation);

private:

	/**
	 * Internal function used to rename filename to long package name.
	 *
	 * @param InFilename
	 * @param OutPackageName Long package name.
	 */
	static COREUOBJECT_API void InternalFilenameToLongPackageName(FStringView InFilename, FStringBuilderBase& OutPackageName);

	/**
	 * Internal helper to find a mount point that contains the given LocalFilePath or PackageName, and if found, return the MountPoint, RelativePath, and PackageExtension
	 *
	 * @param InPath The path to test, either a LocalFilePath, PackageName, or ObjectPath
	 * @param OutMountPointPackageName If the MountPoint is found, the PackageName of the MountPoint is copied into this variable, otherwise it is set to empty string
	 * @param OutMountPointFilePath If the MountPoint is found, the LocalFilePath of the MountPoint is copied into this variable, otherwise it is set to empty string
	 * @param OutRelPath If the MountPoint is found, the RelativePath from the MountPoint to the PackageName is copied into this variable, otherwise it is set to empty string
	 * @param OutObjectPath If the MountPoint is found and InPath was an ObjectPath, this variable is set to the object's relative path from the package: ObjectPath:SubObject1.SubObject2, otherwise it is set to the empty string
	 * @param OutExtension If the MountPoint is found and InPath was a LocalFilePath, this variable is set to the EPackageExtension that was in the filepath, otherwise it is set to EPackageExtension::Unspecified
	 * @param OutCustomExtension If the mount point was found and InPath was a filepath with extension and the extension was not a recognized package extension, this varialbe is set to the extension text (including .), otherwise it is set to the empty string
	 * @param OutFlexNameType If non-null, it is set to whether InFilePathOrPackageName is a PackageName or Filename if the MountPoint is found, otherwise it is set to EFlexNameType::Invalid
	 * @param OutFailureReason If non-null, it is set to the failurereason if the MountPoint is not found, otherwise it is set to EErrorCode::PackageNameUnknown
	 * @return True if the MountPoint was found, else false
	 */
	static COREUOBJECT_API bool TryConvertToMountedPathComponents(FStringView InPath, FStringBuilderBase& OutMountPointPackageName, FStringBuilderBase& OutMountPointFilePath, FStringBuilderBase& OutRelPath,
		FStringBuilderBase& OutObjectName, EPackageExtension& OutExtension, FStringBuilderBase& OutCustomExtension, EFlexNameType* OutFlexNameType = nullptr, EErrorCode* OutFailureReason = nullptr);

	/**
	 * Internal helper to create a FPackagePath given LocalFilePath or PackageName, and if found, return the MountPoint, RelativePath, and PackageExtension
	 *
	 * @param InPath The path to test, either a LocalFilePath, PackageName, or ObjectPath
	 * @param OutPackagePath FPackagePath to store the converted InPath
	 * @param OutFailureReason it is set to the failurereason if the MountPoint is not found, otherwise it is set to EErrorCode::PackageNameUnknown
	 * @return True if InPath be converted (was not malformed)
	 */
	static COREUOBJECT_API bool TryConvertToMountedPackagePath(const FString& InPath, FPackagePath& OutPackagePath, EErrorCode& OutFailureReason);

	/** Event that is triggered when a new content path is mounted */
	static COREUOBJECT_API FMountPointEvent OnMountPointMountedEvent;
	static COREUOBJECT_API FOnContentPathMountedEvent OnContentPathMountedEvent;

	/** Event that is triggered when a new content path is removed */
	static COREUOBJECT_API FMountPointEvent OnMountPointDismountedEvent;
	static COREUOBJECT_API FOnContentPathDismountedEvent OnContentPathDismountedEvent;

	/** Delegate used to check whether a package exist without using the filesystem. */
	static COREUOBJECT_API FDoesPackageExistOverride DoesPackageExistOverrideDelegate;

	friend class FPackagePath;

	// Internal helper not meant for public use. These versions of DoesPackageExist do not use the AssetRegistry; any code that might be holding an AssetRegistry lock should call these functions instead
	friend class FAssetRegistryConsoleCommands;
	friend class UAssetRegistryImpl;
	friend struct ::UE::AssetRegistry::Impl::FScanPathContext;
	static COREUOBJECT_API EPackageLocationFilter InternalDoesPackageExistEx(const FPackagePath& PackagePath, EPackageLocationFilter Filterconst, bool bMatchCaseOnDisk = false, FPackagePath* OutPackagePath = nullptr);
	static COREUOBJECT_API EPackageLocationFilter InternalDoesPackageExistEx(const FString& LongPackageName, EPackageLocationFilter Filterconst, bool bMatchCaseOnDisk = false, FPackagePath* OutPackagePath = nullptr);
};

ENUM_CLASS_FLAGS(FPackageName::EConvertFlags);
ENUM_CLASS_FLAGS(FPackageName::EPathFormatFlags);
