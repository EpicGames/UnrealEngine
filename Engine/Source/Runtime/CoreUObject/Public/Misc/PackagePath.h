// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#ifndef UE_SUPPORT_FULL_PACKAGEPATH
	#define UE_SUPPORT_FULL_PACKAGEPATH WITH_EDITOR
#endif

class FArchive;

/**
 * An EPackageSegment is a portion of the package that can be requested in cooked builds.
 * It is only used by the loading code in CoreUObject; all other uses default to EPackageSegment::Header
 * See PackageSegment.h
 */
enum class EPackageSegment : uint8;

/**
 * Enum for the extensions that a package payload can be stored under.
 * Each extension can be used by only one EPackageSegment
 * EPackageSegment::Header segment has multiple possible extensions
 * Use LexToString to convert an EPackageExtension to the extension string to append to a file's basename
 * (e.g. LexToString(EPackageExtension::Asset) -> ".uasset"
 *	 	Exceptions:
 *			Unspecified -> <emptystring>
 *  		Custom -> ".CustomExtension"; the actual string to use is stored as a separate field on an FPackagePath
 */
enum class EPackageExtension : uint8
{
	// Header Segments
	/**
	 * A PackageResourceManager will search for any of the other header extensions
	 * when receiving a PackagePath with Unspecified
	 */
	Unspecified=0,
	/** A binary-format header that does not contain a UWorld or ULevel */
	Asset,
	/** A binary-format header that contains a UWorld or ULevel */
	Map,
	/** A text-format header that does not contain a UWorld or ULevel */
	TextAsset,
	/** A text-format header that contains a UWorld or ULevel */
	TextMap,
	/**
	 * Used when the owner of an EPackageExtension has a specific extension that does not match
	 * one of the enumerated possibilies, e.g. a custom extension on a temp file
	 */
	Custom,
	/** Used by iostore to indicate that the package should be requested with no extension */
	EmptyString,
					
	// Other Segments

	Exports,
	BulkDataDefault,
	BulkDataOptional,
	BulkDataMemoryMapped,
	PayloadSidecar,
	Count,
};
inline constexpr int EPackageExtensionCount = static_cast<int>(EPackageExtension::PayloadSidecar) + 1;
COREUOBJECT_API const TCHAR* LexToString(EPackageExtension PackageExtension);


/**
 * Internal class only; not used by blueprints. This class is only used in the CoreUObject linker layer
 * and (optionally) as an argument to LoadPackage
 *
 * Specifies a path to the contents of a UPackage that can be loaded from disk (or other persistent storage)
 * through IPackageResourceManager
 * Package contents are requested from the PackageResourceManager by FPackagePath and EPackageSegment
 * FPackagePaths are stored internally in one of a few options necessary to maintain their construction
 * arguments without losing data
 * const FPackagePaths can still be mutated to be stored as another internal type
 * const FPackagePaths can still be mutated to indicate which header extension was found for them on disk
 *    (It is illegal to have multiple packages on disk with the same PackagePath but different header extensions)
 */
class FPackagePath
{
public:
	// Public API

	/** Construct an empty PackagePath */
	constexpr FPackagePath() = default;

	// Copy/Move constructors/assignment

	COREUOBJECT_API FPackagePath(const FPackagePath& Other);
	FPackagePath(FPackagePath&& Other) = default;
	COREUOBJECT_API FPackagePath& operator=(const FPackagePath& Other);
	FPackagePath& operator=(FPackagePath&& Other) = default;

	/** Free this PackagePath's data and set it to an empty PackagePath */
	COREUOBJECT_API void Empty();

	/**
	 * Construct a PackagePath from a LocalPath or PackageName string, assuming the given LocalPath or PackageName
	 * is in a directory that is mounted (aka registered with FPackageName::MountPointExists)
	 *
	 * @param InPackageNameOrHeaderFilePath A string that describes either the PackageName or LocalPath of a PackagePath
	 * @param OutPackagePath If the function succeeds, the created PackagePath is assigned to this variable.
	 *        Otherwise this variable is not written
	 * @return True if the input string was found in a mounted path and OutPackagePath has been set, false otherwise
	 */
	COREUOBJECT_API static bool TryFromMountedName(FStringView InPackageNameOrHeaderFilePath, FPackagePath& OutPackagePath);

	/**
	 * Attempt to construct a PackagePath from a LongPackageName StringView, FName, or TCHAR*
	 *
	 * Does not handle InPackageNames that are actually LocalPaths; use TryFromMountedName if you need to handle
	 * either PackageName or LocalPath
	 * Fails and returns false if and only if InPackageName is not a valid LongPackageName (/Root/Folder/File)
	 * Will be converted to a MountedPath when the LocalPath is required; if the package is not mounted at that
	 * point the LocalPath will be empty
	 *
	 * @param InPackageName The LongPackageName to test, does not have to be mounted or existing
	 * @param OutPackagePath If InPackageName is valid, the constructed PackagePath is copied into this variable,
	 *        otherwise the variable is not written
	 * @return True if InPackageName is valid, else false
	 */
	COREUOBJECT_API static bool TryFromPackageName(FStringView InPackageName, FPackagePath& OutPackagePath);
	COREUOBJECT_API static bool TryFromPackageName(FName InPackageName, FPackagePath& OutPackagePath);
	COREUOBJECT_API static bool TryFromPackageName(const TCHAR* InPackageName, FPackagePath& OutPackagePath);

	/**
	 * Construct a PackagePath from a LongPackageName StringView, FName, or TCHAR*
	 *
	 * Does not handle InPackageNames that are actually LocalPaths; use TryFromMountedName if you need to handle
	 * either PackageName or LocalPath
	 * Gives an error and returns an empty PackagePath if InPackageName is not a valid LongPackageName (/Root/Folder/File)
	 * Will be converted to a MountedPath when the LocalPath is required; if the package is not mounted at that point
	 * the LocalPath will be empty
	 *
	 * @param InPackageName A valid LongPackageName, does not have to be mounted or existing
	 * @return The constructed PackagePath
	 */
	COREUOBJECT_API static FPackagePath FromPackageNameChecked(FStringView InPackageName);
	COREUOBJECT_API static FPackagePath FromPackageNameChecked(FName InPackageName);
	COREUOBJECT_API static FPackagePath FromPackageNameChecked(const TCHAR* InPackageName);

	/**
	 * Construct a PackagePath from a known valid LongPackageName FName
	 *
	 * Will be converted to a MountedPath when the LocalPath is required; if the package is not mounted at that point
	 * the LocalPath will be empty
	 *
	 * @param InPackageName A valid LongPackageName, does not have to be mounted or existing
	 * @return The constructed PackagePath
	 */
	COREUOBJECT_API static FPackagePath FromPackageNameUnchecked(FName InPackageName);

	/**
	 * Construct a PackagePath from a LocalPath string
	 *
	 * Does not handle InFilenames that are actually PackageNames; use TryFromMountedName if you need to handle either
	 * PackageName or LocalPath
	 * Will be converted to a MountedPath when the PackageName is required; if the package is not mounted at that
	 * point the PackageName will be empty.
	 * Always succeeds; all strings are valid filenames
	 *
	 * @param InFilename The full LocalPath, D:\Folder\File.Ext, may be relative or absolute, extension is not required
	 * @return The constructed PackagePath
	 */
	COREUOBJECT_API static FPackagePath FromLocalPath(FStringView InFilename);

	/**
	 * Construct a PackagePath from the components of a MountedPath
	 *
	 * This function is less expensive than TryFromMountedName 
	 * It is invalid to call this function with arguments that do not match an existing directory that is mounted
	 * (aka registered with FPackageName::MountPointExists)
	 *
	 * @param PackageNameRoot The PackageName of the MountPoint
	 * @param FilePathRoot The FilePath of the MountPoint
	 * @param RelPath The relative path of the PackagePath from the PackageNameRoot/FilePathRoot
	 *        (relative path from MountPoint is the same in PackageNames and LocalPaths)
	 * @param InExtension The header extension to give the PackagePath. Extensions that are not header extensions are ignored
	 * @param InCustomExtension The custom string to use if InExtension is EPackageExtension::Custom
	 * @return The constructed PackagePath
	 */
	COREUOBJECT_API static FPackagePath FromMountedComponents(FStringView PackageNameRoot, FStringView FilePathRoot, FStringView RelPath,
		EPackageExtension InExtension, FStringView InCustomExtension = FStringView());

	/**
	 * Set the capitalization of a PackagePath to match the given string
	 *
	 * If the given FilePathToMatch is case-insensitively equal to the LocalPath of the source PackagePath,
	 * set the output PackagePath equal to a copy of the source but with
	 * capitalization set equal to the capitalization in FilePathToMatch
	 *
	 * @param SourcePackagePath The PackagePath to compare against FilePathToMatch
	 * @param FilePathToMatch The LocalPath to compare against SourcePackagePath's LocalPath
	 * @param OutPackagePath Set to the matching-capitalization copy of SourcePackagePath if FilePathToMatch matched
	 *        SourcePackagePath case-insensitively
	 * @return True if FilePathToMatch matched SourcePackagePath case-insensitively, else false
	 */
	COREUOBJECT_API static bool TryMatchCase(const FPackagePath& SourcePackagePath, FStringView FilePathToMatch,
		FPackagePath& OutPackagePath);

	// Comparsion operators. An FPackagePath is == another FPackagePath if they are known describe the same path;
	// the comparison is case insensitive, and if one PackagePath is a not-yet-mounted
	// PackageName and the other is a not-yet-mounted LocalPath, they are attempted to be mounted and will be equal
	// if they correspond to the same relative path under the same mount point

	COREUOBJECT_API bool operator==(const FPackagePath& Other) const;
	COREUOBJECT_API bool operator!=(const FPackagePath& Other) const;

	/**
	 * Serialization operator. It is invalid to serialize a PackagePath to persistent data; package path serialization
	 * is dependent upon transient data and may change between process invocations
	 */ 
	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FPackagePath& PackagePath);

	/**
	 * If this PackagePath is not yet internally stored as MountedPath, search the FPackageName MountPoints again for
	 * its PackageName or LocalPath, and convert it to mounted if found
	 */
	COREUOBJECT_API bool TryConvertToMounted() const;

	/**
	 * Return whether this is a PackagePath with no path information and can never identify a package on disk
	 *
	 * Note that IsEmpty == false does not imply that the package exists on disk, only that the PackagePath is valid
	 */
	COREUOBJECT_API bool IsEmpty() const;

	/**
	 * Return whether a mount point has been found for this PackagePath
	 *
	 * Will attempt to mount before returning false
	 */
	COREUOBJECT_API bool IsMountedPath() const;

	/**
	 * Return true if and only if GetPackageName returns non-empty
	 *
	 * Returning true is possible only for MountedPath and PackageOnly internal types.
	 * Will attempt to mount before returning false
	 */
	COREUOBJECT_API bool HasPackageName() const;

	/**
	 * Return true if and only if GetLocalBaseFilenameWithPath returns non-empty
	 *
	 * Returning true is possible for MountedPath and LocalOnly internal types.
	 * Will attempt to mount before returning false
	 */
	COREUOBJECT_API bool HasLocalPath() const;

	/**
	 * Return the PackageName indicated by this PackagePath if known or available, or empty string if not
	 *
	 * Returning non-empty is possible only for MountedPath and PackageOnly internal types.
	 * Will attempt to mount before returning empty
	 */
	COREUOBJECT_API FString GetPackageName() const;

	/**
	 * Append the PackageName indicated by this PackagePath if known or available, or take no action if not
	 *
	 * Appending is possible only for MountedPath and PackageOnly internal types.
	 * Will attempt to mount before returning with no action
	 */
	COREUOBJECT_API void AppendPackageName(FStringBuilderBase& Builder) const;

	/**
	 * Return as an FName the PackageName indicated by this PackagePath if known or available, or NAME_None if not
	 *
	 * Returning a valid name is possible only for MountedPath and PackageOnly internal types.
	 * Will attempt to mount before returning NAME_None
	 */
	COREUOBJECT_API FName GetPackageFName() const;

	/**
	 * Return the full LocalPath (path,basename,extension)
	 *
	 * The path is in the standard UnrealEngine form - it is as a relative path from the process binary directory
	 * Extension is set based on the given segment
	 * If the LocalPath for this PackagePath is unknown, returns the empty string
	 * Returning non-empty is possible only for MountedPath and LocalOnly internal types.
	 * Will attempt to mount before returning empty
	 */
	COREUOBJECT_API FString GetLocalFullPath() const;

	/**
	 * Append the full relative LocalPath (path,basename,extension)
	 *
	 * The path is in the standard UnrealEngine form - it is as a relative path from the process binary directory
	 * Extension is set based on the given segment
	 * If the LocalPath for this PackagePath is unknown, nothing is written to the Builder
	 * Appending is possible only for MountedPath and LocalOnly internal types.
	 * Will attempt to mount before returning with no action
	 */
	COREUOBJECT_API void AppendLocalFullPath(FStringBuilderBase& Builder) const;

	/**
	 * Return the Local (path,basename) of this PackagePath if known or available, or empty string if not
	 *
	 * Returning non-empty is possible only for MountedPath and LocalOnly internal types.
	 * Will attempt to mount before returning empty
	 */
	COREUOBJECT_API FString GetLocalBaseFilenameWithPath() const;

	/**
	 * Append the Local (path,basename) of this PackagePath if known or available, or take no action if not
	 *
	 * Appending is possible only for MountedPath and LocalOnly internal types.
	 * Will attempt to mount before returning with no action
	 */
	COREUOBJECT_API void AppendLocalBaseFilenameWithPath(FStringBuilderBase& Builder) const;

	/**
	 * Return the HeaderExtension
	 *
	 * The header extension is the extension the header segment of this PackagePath has on disk.
	 * It is illegal to have multiple files on disk with the same BaseNameWithPath but different header extensions,
	 * so the header extension is optional. If unspecified, all header extensions will be searched.
	 * If specified, it is assumed correct and searches will return false if they have a different header extension.
	 * The header extension is a performance hint and matching header extensions is not required for equality.
	 * The header extension will be mutated even on const FPackagePaths when it is not already set
	 * and functions are called that need to calculate it
	 */
	COREUOBJECT_API EPackageExtension GetHeaderExtension() const;

	/** Returns a descriptor of this PackagePath, usable for an identifier in warning and log messages. Extension is not indicated. */
	COREUOBJECT_API FString GetDebugName() const;

	/** Returns GetDebugName converted to FText */
	COREUOBJECT_API FText GetDebugNameText() const;

	/** Returns a descriptor of this PackagePath for the given segment, with extension. */
	COREUOBJECT_API FString GetDebugNameWithExtension() const;

	/**
	 * Returns the PackageName if available, otherwise the LocalPath
	 *
	 * Do not use this function as a PackageName; use it only for uniquely identifying PackagePaths in e.g. a TMap
	 */
	COREUOBJECT_API FString GetPackageNameOrFallback() const;

	/** Set the HeaderExtension to use, including a custom string if Extension is EPackageExtension::Custom */
	COREUOBJECT_API void SetHeaderExtension(EPackageExtension Extension, FStringView CustomExtension = FStringView()) const;

	/**
	 * Return the custom string if this PackagePath is using HeaderExtension=EPackageExtension::Custom,
	 * otherwise return an empty string
	 */
	COREUOBJECT_API FStringView GetCustomExtension() const;

	/**
	 * Parse the extension from a filepath and convert it to an EPackageExtension.
	 *
	 * Note this is not the same as FPaths::GetExtension because some EPackageExtensions have multiple '.'s
	 * (e.g. .m.ubulk)
	 *
	 * @param Filename The path to parse the extension from
	 * @param OutExtensionStart If non-null, will be filled with the index of the '.' at the start of the extension,
	 *        or with Filename.Len() if no extension is found
	 * @return The EPackageExtension matching the (case-insensitive) extension text in the filename.
	 *         EPackageExtension::Unspecified if the filename has no extension.
	 *         EPackageExtension::Custom if the filename's extension is not one of the
	 *         enumerated possibilities in EPackageExtension
	 */
	COREUOBJECT_API static EPackageExtension ParseExtension(FStringView Filename, int32* OutExtensionStart = nullptr);

	/**
	 * Get the string identifying optional segments
	 * @return the string modifier added prior to the file extension that identify optional segments (i.e. ".o")
	 */
	COREUOBJECT_API static const TCHAR* GetOptionalSegmentExtensionModifier();

	/**
	 * Get the folder name from which all external actors paths are created
	 * @return folder name
	 */
	COREUOBJECT_API static const TCHAR* GetExternalActorsFolderName();

	/**
	 * Get the folder name from which all external objects paths are created
	 * @return folder name
	 */
	COREUOBJECT_API static const TCHAR* GetExternalObjectsFolderName();

public:
	// Internal API used by low-level PackageResourceManager users

	/**
	 * Report the extension this PackagePath uses for the given segment
	 *
	 * @param PackageSegment The segment of the extension to look up
	 * @param OutCustomExtension If the segment is EPackageSegment::Header and this PackagePath has a custom header
	 *        extension, filled with the Custom string, otherwise set to empty. CustomExtension can become invalid
	 *        the next time a function (even a const function) or the destructor is called on this FPackagePath
	 * @return the EPackageExtension for the given segment, which is possibly EPackageExtension::Unspecified or
	 *         EPackageExtension::Custom for EPackageSegment::Header
	 */
	COREUOBJECT_API EPackageExtension GetExtension(EPackageSegment PackageSegment, FStringView& OutCustomExtension) const;

	/**
	 * Report the extension string this PackagePath uses for the given segment.
	 *
	 * @param PackageSegment The segment of the extension to look up
	 * @return the string for the given segment, which is possibly the empty string
	 *         or a custom string for EPackageSegment::Header
	 */
	COREUOBJECT_API FStringView GetExtensionString(EPackageSegment PackageSegment) const;

	/**
	 * Report the EPackageExtensions this PackagePath should look for on disk
	 *
	 * The reported array will be length 1 for every case except
	 * EPackageSegment::Header and HeaderExtension == EPackageExtension::Unspecified,
	 * in that case it will be the list of all enumerated header extensions
	 *
	 * @param PackageSegment The segment of the extension to look up
	 * @return A TArray of EPackageExtensions. If the array contains EPackageExtension::Custom, caller must call
	 *         GetCustomExtension to get the specific string.
	 */
	COREUOBJECT_API TConstArrayView<EPackageExtension> GetPossibleExtensions(EPackageSegment PackageSegment) const;

	/** Version of GetLocalFullPath that takes a PackageSegment, otherwise same behavior as parameterless version */
	COREUOBJECT_API FString GetLocalFullPath(EPackageSegment PackageSegment) const;

	/** Version of AppendLocalFullPath that takes a PackageSegment, otherwise same behavior as parameterless version */
	COREUOBJECT_API void AppendLocalFullPath(FStringBuilderBase& Builder, EPackageSegment PackageSegment) const;

	/**
	 * Version of GetDebugName that takes a PackageSegment, otherwise same behavior as parameterless version
	 * For EPackageSegment::Header, HeaderExtension is not indicated.
	 * For all other segments, segment is identified by extension
	 */
	COREUOBJECT_API FString GetDebugName(EPackageSegment PackageSegment) const;

	/** Version of GetDebugNameText that takes a PackageSegment, otherwise same behavior as parameterless version */
	COREUOBJECT_API FText GetDebugNameText(EPackageSegment PackageSegment) const;

	/** Version of GetDebugNameWithExtension that takes a PackageSegment, otherwise same behavior as parameterless version */
	COREUOBJECT_API FString GetDebugNameWithExtension(EPackageSegment PackageSegment) const;

	/** Version of FromLocalPath that also returns the PackageSegment matching the Filename's extension */
	COREUOBJECT_API static FPackagePath FromLocalPath(FStringView InFilename, EPackageSegment& OutPackageSegment);

private:
	// Accessors for the StringData stored in this PackagePath; the various strings are packed together
	// into a single allocation
	FStringView GetPackageNameRoot() const;
	FStringView GetFilePathRoot() const;
	FStringView GetPathData() const;
	// GetCustomExtension is also an accessor, but unlike the others is publically readable
	void SetStringData(FStringView PathData, FStringView PackageNameRoot, FStringView FilePathRoot,
		FStringView CustomExtension) const;
	int32 GetStringDataLen() const;

	enum class EPackageIdType : uint8
	{
		/** Does not specify a Package */
		Empty,
		/**
		 * A package in a mounted ContentDirectory; both PackageName and BaseFilenameWithPath are available; extension
		 * may be available or unspecified. PathData is RelPath.
		 */
		MountedPath,
		/**
		 * A PackageName that is not in a mounted ContentDirectory. PackageName is available,
		 * BaseNameWithPath is not available. Extension may be available or unspecified. PathData is the PackagePath.
		 */
		PackageOnlyPath,
		/**
		 * A filename on the local disk that is not in a mounted ContentDirectory. PackageName is not available,
		 * BaseNameWithPath and Extension are available. PathData is the full path including extension.
		 */
		LocalOnlyPath,
	};

#if UE_SUPPORT_FULL_PACKAGEPATH
	/**
	 * Combined storage for (EPackageIDType-specific path+filename storage) followed by (possibly empty)
	 * PackageNameRoot followed by (possibly empty) FilePathRoot followed by (possibly empty) Extension
	 * Total length is (PathDataLen + PackageNameRootLen + FilePathRootLen + ExtensionLen)
	 */
	mutable TUniquePtr<TCHAR[]> StringData = nullptr;
	mutable uint16 PathDataLen = 0;
	mutable uint16 PackageNameRootLen = 0;
	mutable uint16 FilePathRootLen = 0;
	mutable uint16 ExtensionLen = 0;
	mutable EPackageIdType IdType = EPackageIdType::Empty;
	mutable EPackageExtension HeaderExtension = EPackageExtension::Unspecified;
#else
	FName PackageName;
	mutable EPackageExtension HeaderExtension = EPackageExtension::Unspecified;
#endif //UE_SUPPORT_FULL_PACKAGEPATH
};
