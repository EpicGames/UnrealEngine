// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PackagePath.h"

/**
 * Enum for the different portions of a package than can be requested from a PackageResourceManager
 * Not all segments are available in all PackageResourceManager; header is always available but the
 * other segments are cooked-only
 */
enum class EPackageSegment : uint8
{
	/** The summary and list of exports. This segment also contains exports and inline bulk data in uncooked repositories */
	Header,
	/** The serialization of the exports */
	Exports,
	/** BulkData that is stored in a separate file */
	BulkDataDefault,
	/** BulkData that is stored in a separate file that is optional */
	BulkDataOptional,
	/** BulkData that is stored in a separate file aligned for memory mapping */
	BulkDataMemoryMapped,
	/** BulkData payloads that are stored in a separate file along side the Header in the workspace domain*/
	PayloadSidecar,
};
const TCHAR* LexToString(EPackageSegment PackageSegment);

/** Map a PackageExtension to a PackageSegment. All extensions can be used by only one segment */
COREUOBJECT_API EPackageSegment ExtensionToSegment(EPackageExtension PackageExtension);

/**
 * Map a PackageSegment to a PackageExtension. Header goes to Unspecified;
 * all other segments have only a single extension and go to that extension
 */
COREUOBJECT_API EPackageExtension SegmentToExtension(EPackageSegment PackageSegment);


