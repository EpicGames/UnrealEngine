// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	RedirectCollector:  Editor-only global object that handles resolving redirectors and handling string asset cooking rules
=============================================================================*/

#pragma once

#include "Async/UniqueLock.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Misc/ScopeLock.h"
#include "Misc/TVariant.h"
#include "Templates/Function.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UnrealNames.h"

class FArchive;

#if WITH_EDITOR

enum class ESoftObjectPathCollectType : uint8;

class FRedirectCollector
{
private:
	
	/** Helper struct for soft object path tracking */
	struct FSoftObjectPathProperty
	{
		FSoftObjectPathProperty(FSoftObjectPath InObjectPath, FName InProperty, bool bInReferencedByEditorOnlyProperty)
			: ObjectPath(MoveTemp(InObjectPath))
			, PropertyName(InProperty)
			, bReferencedByEditorOnlyProperty(bInReferencedByEditorOnlyProperty)
		{}

		 bool operator==(const FSoftObjectPathProperty& Other) const
		 {
		 	return ObjectPath == Other.ObjectPath &&
		 		PropertyName == Other.PropertyName &&
		 		bReferencedByEditorOnlyProperty == Other.bReferencedByEditorOnlyProperty;
		 }

		friend inline uint32 GetTypeHash(const FSoftObjectPathProperty& Key)
		{
			uint32 Hash = 0;
			Hash = HashCombine(Hash, GetTypeHash(Key.ObjectPath));
			Hash = HashCombine(Hash, GetTypeHash(Key.PropertyName));
			Hash = HashCombine(Hash, (uint32)Key.bReferencedByEditorOnlyProperty);
			return Hash;
		}

		const FSoftObjectPath& GetObjectPath() const
		{
			return ObjectPath;
		}

		const FName& GetPropertyName() const
		{
			return PropertyName;
		}

		bool GetReferencedByEditorOnlyProperty() const
		{
			return bReferencedByEditorOnlyProperty;
		}

	private:
		FSoftObjectPath ObjectPath;
		FName PropertyName;
		bool bReferencedByEditorOnlyProperty;
	};

public:

	/**
	 * Called from FSoftObjectPath::PostLoadPath, registers the given SoftObjectPath for later querying
	 * @param InPath The soft object path that was loaded
	 * @Param InArchive The archive that loaded this path
	 */
	COREUOBJECT_API void OnSoftObjectPathLoaded(const struct FSoftObjectPath& InPath, FArchive* InArchive);

	/**
	 * Called at the end of Package Save to record soft package references that might have been created by save transformations
	 * @param ReferencingPackage The package on which we are recording the references
	 * @param PackageNames List of of soft package references needed by the referencing package
	 * @param bEditorOnlyReferences if the PackageNames list are references made by editor only properties
	 */
	COREUOBJECT_API void CollectSavedSoftPackageReferences(FName ReferencingPackage, const TSet<FName>& PackageNames, bool bEditorOnlyReferences);

	/**
	 * Load all soft object paths to resolve them, add that to the remap table, and empty the array
	 * @param FilterPackage If set, only load references that were created by FilterPackage. If empty, resolve  all of them
	 */
	COREUOBJECT_API void ResolveAllSoftObjectPaths(FName FilterPackage = NAME_None);

	/**
	 * Returns the list of packages referenced by soft object paths loaded by FilterPackage, and remove them from the internal list
	 * @param FilterPackage Return references made by loading this package. If passed null will return all references made with no explicit package
	 * @param bGetEditorOnly If true will return references loaded by editor only objects, if false it will not
	 * @param OutReferencedPackages Return list of packages referenced by FilterPackage
	 */
	COREUOBJECT_API void ProcessSoftObjectPathPackageList(FName FilterPackage, bool bGetEditorOnly, TSet<FName>& OutReferencedPackages);

	/** Adds a new mapping for redirector path to destination path, this is called from the Asset Registry to register all redirects it knows about */
	COREUOBJECT_API void AddAssetPathRedirection(const FSoftObjectPath& OriginalPath, const FSoftObjectPath& RedirectedPath);

	/** Removes an asset path redirection, call this when deleting redirectors */
	COREUOBJECT_API void RemoveAssetPathRedirection(const FSoftObjectPath& OriginalPath);

	/** Returns a remapped asset path, if it returns null there is no relevant redirector */
	/** Returns a remapped asset path, if there is no relevant redirector, the return value reports true from IsNull() */
	COREUOBJECT_API FSoftObjectPath GetAssetPathRedirection(const FSoftObjectPath& OriginalPath) const;

	/**
	 * Do we have any references to resolve.
	 * @return true if we have references to resolve
	 */
	bool HasAnySoftObjectPathsToResolve() const
	{
		return SoftObjectPathMap.Num() > 0;
	}

	/**
	 * Removes and copies the value of the list of package dependencies of the given package that were
	 * marked as excluded by FSoftObjectPathSerializationScopes during the load of the package.
	 * This is only used on startup packages during the cook commandlet; for all other packages and
	 * modes it will find an empty list and return false.
	 * @param OutExcludedReferences Out set that is reset and then appended with any discovered values
	 * @return Whether any references were found
	 */
	COREUOBJECT_API bool RemoveAndCopySoftObjectPathExclusions(FName PackageName, TSet<FName>& OutExcludedReferences);

	/** Called from the cooker to stop the tracking of exclusions. */
	COREUOBJECT_API void OnStartupPackageLoadComplete();

	/**
	 * Data about a redirector that has been reported to the RedirectCollector, both the input Source and FirstTarget
	 * data, and the derived FinalTarget data.
	 */
	struct FRedirectionData
	{
	public:
		const FSoftObjectPath& GetSource() const;
		const FSoftObjectPath& GetFirstTarget() const;
		const FSoftObjectPath& GetFinalTarget() const;

	private:
		const FSoftObjectPath& Source;
		const FSoftObjectPath& FirstTarget;
		const FSoftObjectPath& FinalTarget;
		FRedirectionData(const FSoftObjectPath& InSource, const FSoftObjectPath& InFirstTarget,
			const FSoftObjectPath& InFinalTarget);
		friend FRedirectCollector;
	};
	/**
	 * Access to the collected list of redirects through a callback function. The function is called inside the
	 * RedirectCollector's lock and must not call any other functions on the RedirectCollector; doing so will deadlock.
	 */
	COREUOBJECT_API void EnumerateRedirectsUnderLock(TFunctionRef<void(FRedirectionData&)>) const;

	/** Returns the set of paths, if any, that are redirected TO the provided path.*/
	COREUOBJECT_API void GetAllSourcePathsForTargetPath(const FSoftObjectPath& TargetPath, TArray<FSoftObjectPath>& OutSourcePaths) const;

private:
	
	/** Handles adding forward and reverse map entries. Must be called while holding the critical section */
	void AddObjectPathRedirectionInternal(const FSoftObjectPath& Source, const FSoftObjectPath& Destination);

	/** Handles removing forward and reverse map entries. Must be called while holding the critical section */
	bool TryRemoveObjectPathRedirectionInternal(const FSoftObjectPath& Source);

	/**
	 * Searches the graph of FirstTargets starting at the input FirstTarget, to find the FinalTarget, the first one
	 * that is not itself a registered redirector. FirstTarget must not be null, see comment in function. Must be
	 * called while holding the critical section.
	 */
	const FSoftObjectPath& TraverseToFinalTarget(const FSoftObjectPath* FirstTarget) const;

	/** A map of assets referenced by soft object paths, with the key being the package with the reference */
	typedef TSet<FSoftObjectPathProperty> FSoftObjectPathPropertySet;
	typedef TMap<FName, FSoftObjectPathPropertySet> FSoftObjectPathMap;

	/** Return whether SoftObjectPathExclusions are currently being tracked, based on commandline and cook phase. */
	bool ShouldTrackPackageReferenceTypes();

	/** The discovered references that should be followed during cook */
	FSoftObjectPathMap SoftObjectPathMap;
	/** The discovered references to packages and the collect type for whether they should be followed during cook. */
	TMap<FName, TMap<FName, ESoftObjectPathCollectType>> PackageReferenceTypes;

	/** Structure to hold the target data for a redirect when the chained final target != the input first target. */
	struct FChainedRedirectionData
	{
		FSoftObjectPath FirstTarget;
		FSoftObjectPath FinalTarget;
	};
	/**
	 * Variant structure to save memory for storing redirects' target data: most redirects are non-chained, and
	 * FirstTarget == FinalTarget; a few can end up being chained and we need to store extra data for them.
	 */
	struct FSimpleOrChainedRedirect
	{
	public:
		FSimpleOrChainedRedirect();
		FSimpleOrChainedRedirect(FSoftObjectPath InSimpleTarget);
		FSimpleOrChainedRedirect(FSoftObjectPath InFirstTarget, FSoftObjectPath InFinalTarget);
		static FSimpleOrChainedRedirect ConstructSimpleOrChained(FSoftObjectPath InFirstTarget,
			FSoftObjectPath InFinalTarget);

		const FSoftObjectPath& GetFirstTarget() const;
		const FSoftObjectPath& GetFinalTarget() const;
		bool IsSimpleRedirect() const;
		bool IsChainedRedirect() const;

	private:
		TVariant<FSoftObjectPath, TUniquePtr<FChainedRedirectionData>> Data;
	};
	/** When saving, apply this remapping to all soft object paths */
	TMap<FSoftObjectPath, FSimpleOrChainedRedirect> ObjectPathRedirectionMap;

	/** A reverse lookup map for use with GetAllSourcePathsForTargetPath */
	typedef TArray<FSoftObjectPath, TInlineAllocator<1>> ObjectPathSourcesArray;
	TMap<FSoftObjectPath, TArray<FSoftObjectPath, TInlineAllocator<1>>> ObjectPathRedirectionReverseMap;

	/** For ObjectPathRedirectionMap map */
	mutable FCriticalSection CriticalSection;

	enum class ETrackingReferenceTypesState : uint8
	{
		Uninitialized,
		Disabled,
		Enabled,
	};
	ETrackingReferenceTypesState TrackingReferenceTypesState;

	friend class FRedirectCollectorReverseLookupTest;
};

// global redirect collector callback structure
COREUOBJECT_API extern FRedirectCollector GRedirectCollector;


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


inline FRedirectCollector::FSimpleOrChainedRedirect::FSimpleOrChainedRedirect()
	: Data(TInPlaceType<FSoftObjectPath>(), FSoftObjectPath())
{
}

inline FRedirectCollector::FSimpleOrChainedRedirect::FSimpleOrChainedRedirect(
	FSoftObjectPath InSimpleTarget)
	: Data(TInPlaceType<FSoftObjectPath>(), MoveTemp(InSimpleTarget))
{
}

inline FRedirectCollector::FSimpleOrChainedRedirect::FSimpleOrChainedRedirect(
	FSoftObjectPath InFirstTarget, FSoftObjectPath InFinalTarget)
	: Data(TInPlaceType<TUniquePtr<FChainedRedirectionData>>(),
		new FChainedRedirectionData{ MoveTemp(InFirstTarget), MoveTemp(InFinalTarget) })
{
}

inline FRedirectCollector::FSimpleOrChainedRedirect
FRedirectCollector::FSimpleOrChainedRedirect::ConstructSimpleOrChained(
	FSoftObjectPath InFirstTarget, FSoftObjectPath InFinalTarget)
{
	return InFirstTarget == InFinalTarget
		? FSimpleOrChainedRedirect(MoveTemp(InFirstTarget))
		: FSimpleOrChainedRedirect(MoveTemp(InFirstTarget), MoveTemp(InFinalTarget));
}

inline const FSoftObjectPath& FRedirectCollector::FSimpleOrChainedRedirect::GetFirstTarget() const
{
	return IsSimpleRedirect()
		? Data.Get<FSoftObjectPath>()
		: Data.Get<TUniquePtr<FChainedRedirectionData>>()->FirstTarget;
}

inline const FSoftObjectPath& FRedirectCollector::FSimpleOrChainedRedirect::GetFinalTarget() const
{
	return IsSimpleRedirect()
		? Data.Get<FSoftObjectPath>()
		: Data.Get<TUniquePtr<FChainedRedirectionData>>()->FinalTarget;
}

inline bool FRedirectCollector::FSimpleOrChainedRedirect::IsSimpleRedirect() const
{
	return Data.IsType<FSoftObjectPath>();
}

inline bool FRedirectCollector::FSimpleOrChainedRedirect::IsChainedRedirect() const
{
	return Data.IsType<TUniquePtr<FChainedRedirectionData>>();
}

inline const FSoftObjectPath& FRedirectCollector::FRedirectionData::GetSource() const
{
	return Source;
}

inline const FSoftObjectPath& FRedirectCollector::FRedirectionData::GetFirstTarget() const
{
	return FirstTarget;
}

inline const FSoftObjectPath& FRedirectCollector::FRedirectionData::GetFinalTarget() const
{
	return FinalTarget;
}

inline FRedirectCollector::FRedirectionData::FRedirectionData(const FSoftObjectPath& InSource,
	const FSoftObjectPath& InFirstTarget, const FSoftObjectPath& InFinalTarget)
	: Source(InSource), FirstTarget(InFirstTarget), FinalTarget(InFinalTarget)
{
}

#endif // WITH_EDITOR
