// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class UObject;

/**
 * Given a root object, this function will call the callback every time it encounters a reference in the root object
 * (including all inlined/instanced UObjects.  The returned references will be in an easily understood debugging
 * format so that they're easy to understand the context.
 *
 * For example,
 * Actions[2](GameFeatureAction_Activities)->ObjectActivitiesInfo[0].TargetReference
 *
 * Where TargetReference was one of the listed PackagesReferenced.  The point of this function is to assist in the
 * reporting of references between objects you need to show the developer.
 */
COREUOBJECT_API void FindPackageReferencesInObject(const UObject* RootObject, const TArray<FName>& PackagesReferenced, TFunctionRef<void(FName /*PackagePath*/, const FString& /*PropertyPath*/)> Callback);