// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "TargetSettingsDefinitions.generated.h"

UENUM()
enum class ETargetSettingsOfflineBVHMode
{
	Disabled UMETA(DisplayName = "Disabled", ToolTip = "Don't use offline BVHs"),
	MaximizePerformance UMETA(DisplayName = "Maximize Performance", ToolTip = "Prefer high performance at the cost of extra memory usage"),
	MinimizeMemory UMETA(DisplayName = "Minimize Memory", ToolTip = "Prefer reduced memory usage at the cost of some tracing performance"),
};

UENUM()
enum class ETargetSettingsRayTracingRuntimeMode
{
	Disabled UMETA(DisplayName = "Disabled", ToolTip = "Ray tracing is disabled."),
	Inline UMETA(DisplayName = "Inline", ToolTip = "Only inline ray tracing passes will be supported. Removes certain runtime and cook time overhead but also disables ray tracing passes that use material shaders (Lumen Hit Lighting, selected debug modes etc)."),
	Full UMETA(DisplayName = "Full", ToolTip = "Full support for ray tracing features, including ray generation, hit and miss shaders as well as inline ray queries."),
};

UENUM()
enum class ETargetSettingsShadingMode
{
	UseProjectSetting UMETA(DisplayName = "Use Project Setting", ToolTip = "Use the desktop project setting found under Engine - Rendering."),
	Deferred UMETA(DisplayName = "Deferred", ToolTip = "Use deferred shading."),
	Forward UMETA(DisplayName = "Forward", ToolTip = "Use forward shading."),
};