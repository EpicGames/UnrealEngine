// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Logging/LogVerbosity.h"
#include "Misc/OutputDevice.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "TextBuffer.generated.h"

class FArchive;

/**
 * Implements an object that buffers text.
 *
 * The text is contiguous and, if of nonzero length, is terminated by a NULL at the very last position.
 */
UCLASS(MinimalAPI)
class UTextBuffer
	: public UObject
	, public FOutputDevice
{
	GENERATED_BODY()

public:
	/** Default constructor for UObject system */
	COREUOBJECT_API UTextBuffer(const FObjectInitializer& ObjectInitializer);

	/**
	 * Creates and initializes a new text buffer.
	 *
	 * @param ObjectInitializer Initialization properties.
	 * @param InText The initial text.
	 */
	COREUOBJECT_API UTextBuffer (const FObjectInitializer& ObjectInitializer, const TCHAR* InText);

	/**
	 * Creates and initializes a new text buffer.
	 *
	 * @param InText - The initial text.
	 */
	COREUOBJECT_API UTextBuffer(const TCHAR* InText);

public:

	/**
	 * Gets the buffered text.
	 *
	 * @return The text.
	 */
	const FString& GetText () const
	{
		return Text;
	}

public:

	virtual void Serialize(FArchive& Ar) override;
	virtual void Serialize(FStructuredArchive::FRecord Record) override;
	virtual void Serialize (const TCHAR* Data, ELogVerbosity::Type Verbosity, const class FName& Category ) override;
	virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category, const double Time ) override;

private:

	int32 Pos, Top;

	/** Holds the text. */
	FString Text;
};
