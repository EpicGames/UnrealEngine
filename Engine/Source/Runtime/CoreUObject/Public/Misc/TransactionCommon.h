// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/ArchiveSerializedPropertyChain.h"
#include "Templates/TypeHash.h"
#include "Misc/TransactionObjectEvent.h"
#include "Containers/SortedMap.h"
#include "Delegates/Delegate.h"

#define UE_API COREUOBJECT_API

class FReferenceCollector;

namespace UE::Transaction
{

/**
	This type is necessary because the blueprint system is destroying and creating
	CDOs at edit time (usually on compile, but also on load), but also stores user
	entered data in the CDO. We "need"  changes to a CDO to persist across instances
	because as we undo and redo we  need to apply changes to different instances of
	the CDO - alternatively we could destroy and create the CDO as part of a transaction
	(this alternative is the reason for the bunny ears around need).

	DanO: My long term preference is for the editor to use a dynamic, mutable type
	(rather than the CDO) to store editor data. The CDO can then be re-instanced (or not)
	as runtime code requires.
*/
struct FPersistentObjectRef
{
public:
	FPersistentObjectRef() = default;
	UE_API explicit FPersistentObjectRef(UObject* InObject);

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FPersistentObjectRef& ReferencedObject)
	{
		Ar << (std::underlying_type_t<EReferenceType>&)ReferencedObject.ReferenceType;
		Ar << ReferencedObject.RootObject;
		Ar << ReferencedObject.SubObjectHierarchyIDs;
		return Ar;
	}

	friend bool operator==(const FPersistentObjectRef& LHS, const FPersistentObjectRef& RHS)
	{
		return LHS.ReferenceType == RHS.ReferenceType
			&& LHS.RootObject == RHS.RootObject
			&& (LHS.ReferenceType != EReferenceType::SubObject || LHS.SubObjectHierarchyIDs == RHS.SubObjectHierarchyIDs);
	}

	friend bool operator!=(const FPersistentObjectRef& LHS, const FPersistentObjectRef& RHS)
	{
		return !(LHS == RHS);
	}

	friend uint32 GetTypeHash(const FPersistentObjectRef& InObjRef)
	{
		return HashCombine(::GetTypeHash(InObjRef.ReferenceType), GetTypeHash(InObjRef.RootObject));
	}

	bool IsRootObjectReference() const
	{
		return ReferenceType == EReferenceType::RootObject;
	}

	bool IsSubObjectReference() const
	{
		return ReferenceType == EReferenceType::SubObject;
	}

	UE_API UObject* Get() const;
	
	UE_API void AddReferencedObjects(FReferenceCollector& Collector);

	using OnUseOuterDelegate = bool(*)(const UObject*);
	UE_API static TArray<OnUseOuterDelegate>& GetOnUseOuterDelegates();

private:
	/** This enum represents all of the different special cases we are handling with this type */
	enum class EReferenceType : uint8
	{
		Unknown,
		RootObject,
		SubObject,
	};

	/** The reference type we're handling */
	EReferenceType ReferenceType = EReferenceType::Unknown;
	/** Stores the object pointer when ReferenceType==RootObject, and the outermost pointer of the sub-object chain when when ReferenceType==SubObject */
	TObjectPtr<UObject> RootObject = nullptr;
	/** Stores the sub-object name chain when ReferenceType==SubObject */
	TArray<FName, TInlineAllocator<4>> SubObjectHierarchyIDs;

	/** Cached pointers corresponding to RootObject when ReferenceType==SubObject (@note cache needs testing on access as it may have become stale) */
	mutable TWeakObjectPtr<UObject> CachedRootObject;
	/** Cache of pointers corresponding to the items within SubObjectHierarchyIDs when ReferenceType==SubObject (@note cache needs testing on access as it may have become stale) */
	mutable TArray<TWeakObjectPtr<UObject>, TInlineAllocator<4>> CachedSubObjectHierarchy;
};

struct FSerializedTaggedData
{
public:
	static UE_API FSerializedTaggedData FromOffsetAndSize(const int64 InOffset, const int64 InSize);
	static UE_API FSerializedTaggedData FromStartAndEnd(const int64 InStart, const int64 InEnd);

	UE_API void AppendSerializedData(const int64 InOffset, const int64 InSize);
	UE_API void AppendSerializedData(const FSerializedTaggedData& InData);

	UE_API bool HasSerializedData() const;

	int64 GetStart() const
	{
		return DataOffset;
	}

	int64 GetEnd() const
	{
		return DataOffset + DataSize;
	}

	friend bool operator==(const FSerializedTaggedData& LHS, const FSerializedTaggedData& RHS)
	{
		return LHS.DataOffset == RHS.DataOffset
			&& LHS.DataSize == RHS.DataSize;
	}

	friend bool operator!=(const FSerializedTaggedData& LHS, const FSerializedTaggedData& RHS)
	{
		return !(LHS == RHS);
	}

	/** Offset to the start of the tagged data within the serialized object */
	int64 DataOffset = INDEX_NONE;

	/** Size (in bytes) of the tagged data within the serialized object */
	int64 DataSize = 0;
};

struct FSerializedObjectData
{
public:
	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	friend FArchive& operator<<(FArchive& Ar, FSerializedObjectData& SerializedData)
	{
		Ar << SerializedData.Data;
		return Ar;
	}

	friend bool operator==(const FSerializedObjectData& LHS, const FSerializedObjectData& RHS)
	{
		return LHS.Data == RHS.Data;
	}

	friend bool operator!=(const FSerializedObjectData& LHS, const FSerializedObjectData& RHS)
	{
		return !(LHS == RHS);
	}

	UE_API void Read(void* Dest, int64 Offset, int64 Num) const;
	UE_API void Write(const void* Src, int64 Offset, int64 Num);
	
	const void* GetPtr(int64 Offset) const
	{
		return &Data[Offset];
	}

	void Reset()
	{
		Data.Reset();
	}

	int64 Num() const
	{
		return Data.Num();
	}

private:
	TArray64<uint8> Data;
};

struct FSerializedObjectInfo : public FTransactionObjectId
{
public:
	FSerializedObjectInfo() = default;

	explicit FSerializedObjectInfo(const UObject* InObject)
	{
		SetObject(InObject);
	}

	void SetObject(const UObject* InObject)
	{
		FTransactionObjectId::SetObject(InObject);
		bIsPendingKill = !IsValid(InObject);
	}

	void Reset()
	{
		FTransactionObjectId::Reset();
		bIsPendingKill = false;
	}

	void Swap(FSerializedObjectInfo& Other)
	{
		FTransactionObjectId::Swap(Other);
		Exchange(bIsPendingKill, Other.bIsPendingKill);
	}

	/** The pending kill state of the object when it was serialized */
	bool bIsPendingKill = false;
};

struct FSerializedObject
{
public:
	void Reset()
	{
		SerializedData.Reset();
		ReferencedObjects.Reset();
		ReferencedNames.Reset();
	}

	void Swap(FSerializedObject& Other)
	{
		Exchange(SerializedData, Other.SerializedData);
		Exchange(ReferencedObjects, Other.ReferencedObjects);
		Exchange(ReferencedNames, Other.ReferencedNames);
	}

	/** The serialized data for the transacted object */
	FSerializedObjectData SerializedData;

	/** External objects referenced by the transacted object */
	TArray<FPersistentObjectRef> ReferencedObjects;
	
	/** Names referenced by the transacted object */
	TArray<FName> ReferencedNames;
};

struct FDiffableObject
{
public:
	void SetObject(const UObject* InObject)
	{
		ObjectInfo.SetObject(InObject);
		ObjectArchetype = FPersistentObjectRef(InObject->GetArchetype());
		ObjectFlags = InObject->GetFlags();
	}

	void Reset()
	{
		ObjectInfo.Reset();
		ObjectArchetype = FPersistentObjectRef();
		ObjectFlags = RF_NoFlags;
		SerializedData.Reset();
		SerializedTaggedData.Reset();
	}

	void Swap(FDiffableObject& Other)
	{
		ObjectInfo.Swap(Other.ObjectInfo);
		Exchange(ObjectArchetype, Other.ObjectArchetype);
		Exchange(ObjectFlags, Other.ObjectFlags);
		Exchange(SerializedData, Other.SerializedData);
		Exchange(SerializedTaggedData, Other.SerializedTaggedData);
	}

	/** Information about the object when it was serialized */
	FSerializedObjectInfo ObjectInfo;

	/** The archetype of the object when it was serialized */
	FPersistentObjectRef ObjectArchetype;

	/** The flags of the object when it was serialized */
	EObjectFlags ObjectFlags = RF_NoFlags;

	/** The serialized data for the diffable object */
	FSerializedObjectData SerializedData;

	/** Information about tagged data (mainly properties) that were serialized within this object */
	TSortedMap<FName, FSerializedTaggedData, FDefaultAllocator, FNameFastLess> SerializedTaggedData;
};

UE_API extern const FName TaggedDataKey_UnknownData;
UE_API extern const FName TaggedDataKey_ScriptData;

/** Core archive to read a transaction object from the buffer. */
class FSerializedObjectDataReader : public FArchiveUObject
{
public:
	UE_API FSerializedObjectDataReader(const FSerializedObject& InSerializedObject);

	virtual int64 Tell() override { return Offset; }
	virtual void Seek(int64 InPos) override { Offset = InPos; }
	virtual int64 TotalSize() override { return SerializedObject.SerializedData.Num(); }

protected:
	UE_API virtual void Serialize(void* SerData, int64 Num) override;

	using FArchiveUObject::operator<<;
	UE_API virtual FArchive& operator<<(class FName& N) override;
	UE_API virtual FArchive& operator<<(class UObject*& Res) override;

	const FSerializedObject& SerializedObject;
	int64 Offset = 0;
};

namespace Internal
{

/** Core archive to write a transaction object to the buffer. */
class FSerializedObjectDataWriterCommon : public FArchiveUObject
{
public:
	UE_API FSerializedObjectDataWriterCommon(FSerializedObjectData& InSerializedData);

	virtual int64 Tell() override { return Offset; }
	virtual void Seek(int64 InPos) override { checkSlow(Offset <= SerializedData.Num()); Offset = InPos; }
	virtual int64 TotalSize() override { return SerializedData.Num(); }

protected:
	UE_API virtual void Serialize(void* SerData, int64 Num) override;

	virtual void OnDataSerialized(int64 InOffset, int64 InNum) {}

	FSerializedObjectData& SerializedData;
	int64 Offset = 0;
};

} // namespace Internal

/** Core archive to write a transaction object to the buffer. */
class FSerializedObjectDataWriter : public Internal::FSerializedObjectDataWriterCommon
{
public:
	UE_API FSerializedObjectDataWriter(FSerializedObject& InSerializedObject);

protected:
	using FArchiveUObject::operator<<;
	UE_API virtual FArchive& operator<<(class FName& N) override;
	UE_API virtual FArchive& operator<<(class UObject*& Res) override;

	FSerializedObject& SerializedObject;
	int64 Offset = 0;

	TMap<UObject*, int32> ObjectMap;
	TMap<FName, int32> NameMap;
};

/** Core archive to write a diffable object to the buffer. */
class FDiffableObjectDataWriter : public Internal::FSerializedObjectDataWriterCommon
{
public:
	UE_API FDiffableObjectDataWriter(FDiffableObject& InDiffableObject, TArrayView<const FProperty*> InPropertiesToSerialize = TArrayView<const FProperty*>());

protected:
	UE_API FName GetTaggedDataKey() const;

	UE_API bool DoesObjectMatchDiffableObject(const UObject* Obj) const;

	UE_API virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;

	UE_API virtual void MarkScriptSerializationStart(const UObject* Obj) override;
	UE_API virtual void MarkScriptSerializationEnd(const UObject* Obj) override;

	UE_API virtual void OnDataSerialized(int64 InOffset, int64 InNum) override;

	using FArchiveUObject::operator<<;
	UE_API virtual FArchive& operator<<(class FName& N) override;
	UE_API virtual FArchive& operator<<(class UObject*& Res) override;

private:
	struct FCachedPropertyKey
	{
	public:
		FName SyncCache(const FArchiveSerializedPropertyChain& InPropertyChain);

	private:
		FName CachedKey;
		uint32 LastUpdateCount = 0;
	};

	struct FCachedTaggedDataEntry
	{
	public:
		FSerializedTaggedData& SyncCache(FDiffableObject& InDiffableObject, const FName InSerializedTaggedDataKey);

	private:
		FName CachedKey;
		FSerializedTaggedData* CachedEntryPtr = nullptr;
	};

	FDiffableObject& DiffableObject;
	TArrayView<const FProperty*> PropertiesToSerialize;

	bool bIsPerformingScriptSerialization = false;

	mutable bool bWasUsingTaggedDataKey_UnknownData = false;
	mutable bool bWasUsingTaggedDataKey_ScriptData = false;
	mutable int32 TaggedDataKeyIndex_UnknownData = 0;
	mutable int32 TaggedDataKeyIndex_ScriptData = 0;
	mutable FCachedPropertyKey CachedSerializedTaggedPropertyKey;
	mutable FCachedTaggedDataEntry CachedSerializedTaggedDataEntry;
};

namespace DiffUtil
{
UE_API bool IsGeneratingDiffableObject(const FArchive& Ar);

enum class EGetDiffableObjectMode : uint8
{
	/** Serialize the entire object state by calling its Serialize function */
	SerializeObject,

	/** Serialize the property state of the object by calling its SerializeScriptProperties function */
	SerializeProperties,

	/** Serialize the object via a custom serialize function */
	Custom,
};

struct FGetDiffableObjectOptions
{
	/** How should we serialize the object for diffing? */
	EGetDiffableObjectMode ObjectSerializationMode = EGetDiffableObjectMode::SerializeObject;

	/** Custom serializer for the object (must be set when ObjectSerializationMode == Custom) */
	TFunction<void(FDiffableObjectDataWriter&)> CustomSerializer;

	/** Optional list of properties to serialize on the object, or an empty array to serialize all properties */
	TArrayView<const FProperty*> PropertiesToSerialize;

	/** Should we still serialize this object if it's considered pending kill? */
	bool bSerializeEvenIfPendingKill = false;
};

/**
 * Get an object snapshot that can be diffed later.
 */
UE_API FDiffableObject GetDiffableObject(const UObject* Object, const FGetDiffableObjectOptions& Options = FGetDiffableObjectOptions());

struct FGenerateObjectDiffOptions
{
	FGenerateObjectDiffOptions()
	{
		ArchetypeOptions.bSerializeEvenIfPendingKill = true;
	}

	/** Options used when getting the diffable state of archive objects */
	FGetDiffableObjectOptions ArchetypeOptions;

	/** Optional function used to skip comparing certain properties within the diffable data */
	TFunction<bool(FName)> ShouldSkipProperty;

	/** Should we perform a "full" diff? (compares object info, and considers any missing properties to have been changed) */
	bool bFullDiff = true;

	/**
	 * Should we diff the object state even if it's considered pending kill?
	 * When false:
	 *   If the old object was pending kill and the new object isn't, then the diff will be performed against the archetype of the new object.
	 *   If the old object wasn't pending kill and the new object is, then the data diff will be skipped and only the object info diffed.
	 */
	bool bDiffDataEvenIfPendingKill = false;
};

/** Optional cache for requests to get the diffable state for a given archetype object */
class FDiffableObjectArchetypeCache
{
public:
	const FDiffableObject& GetArchetypeDiffableObject(const UObject* Archetype, const FGetDiffableObjectOptions& Options);

private:
	TMap<const UObject*, FDiffableObject> ArchetypeDiffableObjects;
};

/**
 * Generate a diff between the two object snapshots.
 */
UE_API FTransactionObjectDeltaChange GenerateObjectDiff(const FDiffableObject& OldDiffableObject, const FDiffableObject& NewDiffableObject, const FGenerateObjectDiffOptions& DiffOptions = FGenerateObjectDiffOptions(), FDiffableObjectArchetypeCache* ArchetypeCache = nullptr);
UE_API void GenerateObjectDiff(const FDiffableObject& OldDiffableObject, const FDiffableObject& NewDiffableObject, FTransactionObjectDeltaChange& OutDeltaChange, const FGenerateObjectDiffOptions& DiffOptions = FGenerateObjectDiffOptions(), FDiffableObjectArchetypeCache* ArchetypeCache = nullptr);

} // namespace DiffUtil

} // namespace UE::Transaction

#undef UE_API