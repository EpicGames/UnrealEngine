// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "Templates/SharedPointer.h"

class ITransactionObjectAnnotation;

/** Delta-change information for an object that was transacted */
struct FTransactionObjectDeltaChange
{
	FTransactionObjectDeltaChange()
		: bHasNameChange(false)
		, bHasOuterChange(false)
		, bHasExternalPackageChange(false)
		, bHasPendingKillChange(false)
		, bHasNonPropertyChanges(false)
	{
	}

	bool HasChanged() const
	{
		return bHasNameChange || bHasOuterChange || bHasExternalPackageChange || bHasPendingKillChange || bHasNonPropertyChanges || ChangedProperties.Num() > 0;
	}

	void Merge(const FTransactionObjectDeltaChange& InOther)
	{
		bHasNameChange |= InOther.bHasNameChange;
		bHasOuterChange |= InOther.bHasOuterChange;
		bHasExternalPackageChange |= InOther.bHasExternalPackageChange;
		bHasPendingKillChange |= InOther.bHasPendingKillChange;
		bHasNonPropertyChanges |= InOther.bHasNonPropertyChanges;

		for (const FName& OtherChangedPropName : InOther.ChangedProperties)
		{
			ChangedProperties.AddUnique(OtherChangedPropName);
		}
	}

	/** True if the object name has changed */
	bool bHasNameChange : 1;
	/** True of the object outer has changed */
	bool bHasOuterChange : 1;
	/** True of the object assigned package has changed */
	bool bHasExternalPackageChange : 1;
	/** True if the object "pending kill" state has changed */
	bool bHasPendingKillChange : 1;
	/** True if the object has changes other than property changes (may be caused by custom serialization) */
	bool bHasNonPropertyChanges : 1;
	/** Array of properties that have changed on the object */
	TArray<FName> ChangedProperties;
};

/**
 * ID for an object that was transacted
 */
struct FTransactionObjectId
{
public:
	FTransactionObjectId() = default;

	explicit FTransactionObjectId(const UObject* Object)
	{
		SetObject(Object);
	}

	FTransactionObjectId(const FName InObjectPackageName, const FName InObjectName, const FName InObjectPathName, const FName InObjectOuterPathName, const FName InObjectExternalPackageName, const FName InObjectClassPathName)
		: ObjectPackageName(InObjectPackageName)
		, ObjectName(InObjectName)
		, ObjectPathName(InObjectPathName)
		, ObjectOuterPathName(InObjectOuterPathName)
		, ObjectExternalPackageName(InObjectExternalPackageName)
		, ObjectClassPathName(InObjectClassPathName)
	{
	}

	COREUOBJECT_API void SetObject(const UObject* Object);

	void Reset()
	{
		ObjectPackageName = FName();
		ObjectName = FName();
		ObjectPathName = FName();
		ObjectOuterPathName = FName();
		ObjectExternalPackageName = FName();
		ObjectClassPathName = FName();
	}

	void Swap(FTransactionObjectId& Other)
	{
		Exchange(ObjectPackageName, Other.ObjectPackageName);
		Exchange(ObjectName, Other.ObjectName);
		Exchange(ObjectPathName, Other.ObjectPathName);
		Exchange(ObjectOuterPathName, Other.ObjectOuterPathName);
		Exchange(ObjectExternalPackageName, Other.ObjectExternalPackageName);
		Exchange(ObjectClassPathName, Other.ObjectClassPathName);
	}

	friend bool operator==(const FTransactionObjectId& LHS, const FTransactionObjectId& RHS)
	{
		// The other fields are subsets of this data
		return LHS.ObjectPathName == RHS.ObjectPathName
			&& LHS.ObjectExternalPackageName == RHS.ObjectExternalPackageName
			&& LHS.ObjectClassPathName == RHS.ObjectClassPathName;
	}

	friend bool operator!=(const FTransactionObjectId& LHS, const FTransactionObjectId& RHS)
	{
		return !(LHS == RHS);
	}

	friend uint32 GetTypeHash(const FTransactionObjectId& Id)
	{
		return GetTypeHash(Id.ObjectPathName);
	}

	/** The package name of the object, can be dictated either by outer chain or external package */
	FName ObjectPackageName;

	/** The name of the object */
	FName ObjectName;

	/** The path name of the object */
	FName ObjectPathName;

	/** The outer path name of the object */
	FName ObjectOuterPathName;

	/** The external package name of the object, if any */
	FName ObjectExternalPackageName;

	/** The path name of the object's class. */
	FName ObjectClassPathName;
};

/** Change information for an object that was transacted */
struct FTransactionObjectChange
{
	/** Original ID of the object that changed */
	FTransactionObjectId OriginalId;

	/** Original flags of the object that changed */
	EObjectFlags OriginalObjectFlags = RF_NoFlags;

	/** Information about how the object changed */
	FTransactionObjectDeltaChange DeltaChange;
};

/** Different things that can create an object change */
enum class ETransactionObjectChangeCreatedBy : uint8
{
	/** This change was created from an object that was explicitly tracked within a transaction record */
	TransactionRecord,
	/** This change was created from an object that was implicitly tracked within a transaction annotation */
	TransactionAnnotation,
};

/** Different kinds of actions that can trigger a transaction object event */
enum class ETransactionObjectEventType : uint8
{
	/** This event was caused by an undo/redo operation */
	UndoRedo,
	/** This event was caused by a transaction being finalized within the transaction system */
	Finalized,
	/** This event was caused by a transaction snapshot. Several of these may be generated in the case of an interactive change */
	Snapshot,
};

/**
 * Transaction object events.
 *
 * Transaction object events are used to notify objects when they are transacted in some way.
 * This mostly just means that an object has had an undo/redo applied to it, however an event is also triggered
 * when the object has been finalized as part of a transaction (allowing you to detect object changes).
 */
class FTransactionObjectEvent
{
public:
	FTransactionObjectEvent() = default;

	FTransactionObjectEvent(const FGuid& InTransactionId, const FGuid& InOperationId, const ETransactionObjectEventType InEventType, const ETransactionObjectChangeCreatedBy InObjectChangeCreatedBy, const FTransactionObjectChange& InObjectChange, const TSharedPtr<ITransactionObjectAnnotation>& InAnnotation)
		: TransactionId(InTransactionId)
		, OperationId(InOperationId)
		, EventType(InEventType)
		, ObjectChangeCreatedBy(InObjectChangeCreatedBy)
		, ObjectChange(InObjectChange)
		, Annotation(InAnnotation)
	{
		check(TransactionId.IsValid());
		check(OperationId.IsValid());
	}

	/** The unique identifier of the transaction this event belongs to */
	const FGuid& GetTransactionId() const
	{
		return TransactionId;
	}

	/** The unique identifier for the active operation on the transaction this event belongs to */
	const FGuid& GetOperationId() const
	{
		return OperationId;
	}

	/** What kind of action caused this event? */
	ETransactionObjectEventType GetEventType() const
	{
		return EventType;
	}

	/** What kind of thing created this object change? */
	ETransactionObjectChangeCreatedBy GetObjectChangeCreatedBy() const
	{
		return ObjectChangeCreatedBy;
	}

	/** Was the pending kill state of this object changed? (implies non-property changes) */
	bool HasPendingKillChange() const
	{
		return ObjectChange.DeltaChange.bHasPendingKillChange;
	}

	/** Was the name of this object changed? (implies non-property changes) */
	bool HasNameChange() const
	{
		return ObjectChange.DeltaChange.bHasNameChange;
	}

	/** Get the original ID of this object */
	const FTransactionObjectId& GetOriginalObjectId() const
	{
		return ObjectChange.OriginalId;
	}

	/** Get the original flags of this object */
	EObjectFlags GetOriginalObjectFlags() const
	{
		return ObjectChange.OriginalObjectFlags;
	}

	/** Get the original package name of this object */
	FName GetOriginalObjectPackageName() const
	{
		return ObjectChange.OriginalId.ObjectPackageName;
	}

	/** Get the original name of this object */
	FName GetOriginalObjectName() const
	{
		return ObjectChange.OriginalId.ObjectName;
	}

	/** Get the original path name of this object */
	FName GetOriginalObjectPathName() const
	{
		return ObjectChange.OriginalId.ObjectPathName;
	}

	FName GetOriginalObjectClassPathName() const
	{
		return ObjectChange.OriginalId.ObjectClassPathName;
	}

	/** Was the outer of this object changed? (implies non-property changes) */
	bool HasOuterChange() const
	{
		return ObjectChange.DeltaChange.bHasOuterChange;
	}

	/** Has the package assigned to this object changed? (implies non-property changes) */
	bool HasExternalPackageChange() const
	{
		return ObjectChange.DeltaChange.bHasExternalPackageChange;
	}

	/** Get the original outer path name of this object */
	FName GetOriginalObjectOuterPathName() const
	{
		return ObjectChange.OriginalId.ObjectOuterPathName;
	}

	/** Get the original package name of this object */
	FName GetOriginalObjectExternalPackageName() const
	{
		return ObjectChange.OriginalId.ObjectExternalPackageName;
	}

	/** Were ID (name, outer, package) or pending kill changes made to the object? */
	bool HasIdOrPendingKillChanges() const
	{
		return ObjectChange.DeltaChange.bHasNameChange || ObjectChange.DeltaChange.bHasOuterChange || ObjectChange.DeltaChange.bHasExternalPackageChange || ObjectChange.DeltaChange.bHasPendingKillChange;
	}

	/** Were any non-property changes made to the object? (name, outer, package, pending kill, or serialized non-property data) */
	bool HasNonPropertyChanges(const bool InSerializationOnly = false) const
	{
		return (!InSerializationOnly && HasIdOrPendingKillChanges()) || ObjectChange.DeltaChange.bHasNonPropertyChanges;
	}

	/** Were any property changes made to the object? */
	bool HasPropertyChanges() const
	{
		return ObjectChange.DeltaChange.ChangedProperties.Num() > 0;
	}

	/** Get the list of changed properties. */
	const TArray<FName>& GetChangedProperties() const
	{
		return ObjectChange.DeltaChange.ChangedProperties;
	}

	/** Get the annotation object associated with the object being transacted (if any). */
	TSharedPtr<ITransactionObjectAnnotation> GetAnnotation() const
	{
		return Annotation;
	}

	/** Merge this transaction event with another */
	void Merge(const FTransactionObjectEvent& InOther)
	{
		if (EventType == ETransactionObjectEventType::Snapshot)
		{
			EventType = InOther.EventType;
		}

		ObjectChange.DeltaChange.Merge(InOther.ObjectChange.DeltaChange);
	}

private:
	FGuid TransactionId;
	FGuid OperationId;
	ETransactionObjectEventType EventType = ETransactionObjectEventType::Finalized;
	ETransactionObjectChangeCreatedBy ObjectChangeCreatedBy = ETransactionObjectChangeCreatedBy::TransactionRecord;
	FTransactionObjectChange ObjectChange;
	TSharedPtr<ITransactionObjectAnnotation> Annotation;
};

/**
 * Diff for a given transaction.
 */
struct FTransactionDiff
{
	FGuid TransactionId;
	FString TransactionTitle;
	TMap<FName, TSharedPtr<FTransactionObjectEvent>> DiffMap;
};
