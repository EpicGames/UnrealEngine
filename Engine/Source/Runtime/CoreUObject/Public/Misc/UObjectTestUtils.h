// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"
#include "UObject/Class.h"
#include "UObject/ReflectedTypeAccessors.h"

namespace UE::CoreUObject
{
	/** Tests the equality of a UEnum. */
	template <typename UEnumType
		UE_REQUIRES(std::is_enum_v<UEnumType> && std::is_invocable_v<decltype(&StaticEnum<UEnumType>)> && std::negation_v<std::is_integral<UEnumType>>)>
	bool TestEqual(const FString& What, const UEnumType Actual, const UEnumType Expected, FAutomationTestBase& InTestInstance)
	{
		// Compare by value, but convert to string for display
		if (Actual != Expected)
		{
			const UEnum* EnumType = StaticEnum<UEnumType>();
			const FString EnumNameStr =
#if WITH_EDITORONLY_DATA
				EnumType->GetDisplayNameText().ToString();
#else
				EnumType->GetName();
#endif
			const FText ActualName = EnumType->GetDisplayNameTextByIndex(static_cast<int32>(Actual));
			const FText ExpectedName = EnumType->GetDisplayNameTextByIndex(static_cast<int32>(Expected));

			InTestInstance.AddError(FString::Printf(TEXT("Expected '%s' to be %s::%s, but it was %s::%s."),
					*What,
					*EnumNameStr, *ExpectedName.ToString(),
					*EnumNameStr, *ActualName.ToString()),
				1);

			return false;
		}

		return true;
	}

	/** Tests the inequality of a UEnum. */
	template <typename UEnumType
		UE_REQUIRES(std::is_enum_v<UEnumType> && std::is_invocable_v<decltype(&StaticEnum<UEnumType>)> && std::negation_v<std::is_integral<UEnumType>>)>
	bool TestNotEqual(const FString& What, const UEnumType Actual, const UEnumType Expected, FAutomationTestBase& InTestInstance)
	{
		// Compare by value, but convert to string for display
		if (Actual == Expected)
		{
			const UEnum* EnumType = StaticEnum<UEnumType>();
			const FString EnumNameStr =
#if WITH_EDITORONLY_DATA
				EnumType->GetDisplayNameText().ToString();
#else
				EnumType->GetName();
#endif
			const FText ActualName = EnumType->GetDisplayNameTextByIndex(static_cast<int32>(Actual));
			const FText ExpectedName = EnumType->GetDisplayNameTextByIndex(static_cast<int32>(Expected));

			InTestInstance.AddError(FString::Printf(TEXT("Expected '%s' to differ from %s::%s, but it was %s::%s."),
					*What,
					*EnumNameStr, *ExpectedName.ToString(),
					*EnumNameStr, *ActualName.ToString()),
				1);

			return false;
		}

		return true;
	}
}

class FAutomationTestUObjectClassBase : public FAutomationTestBase
{
public:
	FAutomationTestUObjectClassBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{ }

	virtual bool SuppressLogErrors() override { return false; }
	virtual bool SuppressLogWarnings() override { return true; }
	virtual bool ElevateLogWarningsToErrors() override { return false; }

	template <typename UEnumType
			UE_REQUIRES(std::is_enum_v<UEnumType> && std::is_invocable_v<decltype(&StaticEnum<UEnumType>)> && std::negation_v<std::is_integral<UEnumType>>)>
	bool TestEqual(const TCHAR* What, const UEnumType Actual, const UEnumType Expected, const FAutomationTestBase& InTestInstance)
	{
		return UE::CoreUObject::TestEqual(What, Actual, Expected, *this);
	}

	template <typename UEnumType
			UE_REQUIRES(std::is_enum_v<UEnumType> && std::is_invocable_v<decltype(&StaticEnum<UEnumType>)> && std::negation_v<std::is_integral<UEnumType>>)>
	bool TestNotEqual(const TCHAR* What, const UEnumType Actual, const UEnumType Expected, const FAutomationTestBase& InTestInstance)
	{
		return UE::CoreUObject::TestNotEqual(What, Actual, Expected, *this);
	}
};
