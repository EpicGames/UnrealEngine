// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Logging/TokenizedMessage.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class UObject;

/**
 * A Message Log token that links to an object, with default behavior to link to the object
 * in the content browser/scene.
 */
class FUObjectToken : public IMessageToken
{
public:
	/** Factory method, tokens can only be constructed as shared refs */
	COREUOBJECT_API static TSharedRef<FUObjectToken> Create(const UObject* InObject, const FText& InLabelOverride = FText());

	/** Begin IMessageToken interface */
	virtual EMessageToken::Type GetType() const override
	{
		return EMessageToken::Object;
	}

	virtual const FOnMessageTokenActivated& GetOnMessageTokenActivated() const override;
	/** End IMessageToken interface */

	/** Get the object referenced by this token */
	virtual const FWeakObjectPtr& GetObject() const
	{
		return ObjectBeingReferenced;
	}

	/** Get the original object Path Name as the object path name could be different when fetched later on */
	const FString& GetOriginalObjectPathName() const 
	{ 
		return OriginalObjectPathName;  
	}

	/** Get the delegate for default token activation */
	static FOnMessageTokenActivated& DefaultOnMessageTokenActivated()
	{
		return DefaultMessageTokenActivated;
	}

	/** Get the delegate for displaying the object name */
	DECLARE_DELEGATE_RetVal_TwoParams(FText, FOnGetDisplayName, const UObject*, bool);
	static FOnGetDisplayName& DefaultOnGetObjectDisplayName()
	{
		return DefaultGetObjectDisplayName;
	}

private:
	/** Private constructor */
	COREUOBJECT_API FUObjectToken( const UObject* InObject,  const FText& InLabelOverride );

	/** An object being referenced by this token, if any */
	FWeakObjectPtr ObjectBeingReferenced;

	/** The original object Path Name as the object path name could be different when fetched later on */
	FString OriginalObjectPathName;

	/** The default activation method, if any */
	COREUOBJECT_API static FOnMessageTokenActivated DefaultMessageTokenActivated;

	/** The default object name method, if any */
	COREUOBJECT_API static FOnGetDisplayName DefaultGetObjectDisplayName;
};
