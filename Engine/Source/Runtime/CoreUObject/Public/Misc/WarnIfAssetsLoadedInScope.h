// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPath.h"

class UObject;
struct FProgramCounterSymbolInfo;
struct FAssetData;

#if WITH_EDITOR

class FWarnIfAssetsLoadedInScope : public FNoncopyable
{
public:
    COREUOBJECT_API FWarnIfAssetsLoadedInScope();
	COREUOBJECT_API FWarnIfAssetsLoadedInScope(TConstArrayView<FAssetData> InSpecificAssets);

    COREUOBJECT_API ~FWarnIfAssetsLoadedInScope();
	
private:
	bool IsEnabled() const;
    void OnAssetLoaded(UObject* InObject);

private:
    FDelegateHandle AssetsLoadedHandle;
    TConstArrayView<FAssetData> SpecificAssets;

    struct FLoadEvent
    {
        FSoftObjectPath ObjectPath;
        TArray<FProgramCounterSymbolInfo> Callstack;
    };

    TArray<FLoadEvent> ObjectsLoaded;
};

#endif
