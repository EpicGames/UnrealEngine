// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WorldCompositionUtility.h: Support structures for world composition
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Math/Box.h"
#include "Math/IntPoint.h"
#include "Math/IntVector.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Serialization/StructuredArchive.h"

class FArchive;

#define WORLDTILE_LOD_PACKAGE_SUFFIX	TEXT("_LOD")
#define WORLDTILE_LOD_MAX_INDEX			4

/** World layer information for tile tagging */
class FWorldTileLayer
{
public:
	FWorldTileLayer()
		: Name(TEXT("Uncategorized"))
		, Reserved0(0)
		, Reserved1(ForceInit)
		, StreamingDistance(50000)
		, DistanceStreamingEnabled(true)
	{
	}

	bool operator==(const FWorldTileLayer& OtherLayer) const
	{
		return	(Name == OtherLayer.Name) && 
				(StreamingDistance == OtherLayer.StreamingDistance) &&
				(DistanceStreamingEnabled == OtherLayer.DistanceStreamingEnabled);

	}

	friend FArchive& operator<<(FArchive& Ar, FWorldTileLayer& D);
	friend void operator<<(FStructuredArchive::FSlot Slot, FWorldTileLayer& D);

public:
	/** Human readable name for this layer */
	FString					Name;
	int32					Reserved0;
	FIntPoint				Reserved1;
	/** Distance starting from where tiles belonging to this layer will be streamed in */
	int32					StreamingDistance;
	bool					DistanceStreamingEnabled;
};

/** 
 *  Describes LOD entry in a world tile 
 */
class FWorldTileLODInfo
{
public:
	FWorldTileLODInfo()
		: RelativeStreamingDistance(10000)
		, Reserved0(0)
		, Reserved1(0)
		, Reserved2(0)
		, Reserved3(0)
	{
	}

	friend FArchive& operator<<(FArchive& Ar, FWorldTileLayer& D);
	friend void operator<<(FStructuredArchive::FSlot Slot, FWorldTileLayer& D);

	bool operator==(const FWorldTileLODInfo& OtherInfo) const
	{
		return RelativeStreamingDistance == OtherInfo.RelativeStreamingDistance;
	}

public:
	/**  Relative to LOD0 streaming distance, absolute distance = LOD0 + StreamingDistanceDelta */
	int32	RelativeStreamingDistance;
	
	/**  Reserved for additional options */
	float	Reserved0;
	float	Reserved1;
	int32	Reserved2;
	int32	Reserved3;
};

/** 
 *  Tile information used by WorldComposition. 
 *  Defines properties necessary for tile positioning in the world. Stored with package summary 
 */
class FWorldTileInfo
{
public:
	FWorldTileInfo()
		: Position(0,0,0)
		, AbsolutePosition(0,0,0)
		, Bounds(ForceInit)
		, bHideInTileView(false)
		, ZOrder(0)
	{
	}
	
	friend FArchive& operator<<(FArchive& Ar, FWorldTileInfo& D) { D.StreamArchive(Ar); return Ar; }
	friend void operator<<(FStructuredArchive::FSlot Slot, FWorldTileInfo& D) { D.StreamStructuredArchive(Slot); }

	bool operator==(const FWorldTileInfo& OtherInfo) const
	{
		return (Position == OtherInfo.Position) &&
				(Bounds.Min.Equals(OtherInfo.Bounds.Min, 0.5f)) &&
				(Bounds.Max.Equals(OtherInfo.Bounds.Max, 0.5f)) &&
				(ParentTilePackageName == OtherInfo.ParentTilePackageName) &&
				(Layer == OtherInfo.Layer) &&
				(bHideInTileView == OtherInfo.bHideInTileView) &&
				(ZOrder == OtherInfo.ZOrder) &&
				(LODList == OtherInfo.LODList);
	}
	
	/** @return Streaming distance for a particular tile LOD */
	int32 GetStreamingDistance(int32 LODIndex) const
	{
		int32 Distance = Layer.StreamingDistance;
		if (LODList.IsValidIndex(LODIndex))	
		{
			Distance+= LODList[LODIndex].RelativeStreamingDistance;
		}
		return Distance;
	}

	/** Reads FWorldTileInfo from a specified package */
	COREUOBJECT_API static bool Read(const FString& InPackageFileName, FWorldTileInfo& OutInfo);
	
public:
	/** Tile position in the world relative to parent */
	FIntVector			Position; 
	/** Absolute tile position in the world. Calculated in runtime */
	FIntVector			AbsolutePosition; 
	/** Tile bounding box  */
	FBox				Bounds;
	/** Tile assigned layer  */
	FWorldTileLayer		Layer;
	/** Whether to hide sub-level tile in tile view*/
	bool				bHideInTileView;
	/** Parent tile package name */
	FString				ParentTilePackageName;
	/** LOD information */
	TArray<FWorldTileLODInfo> LODList;
	/** Sorting order */
	int32				ZOrder;

private:
	COREUOBJECT_API void StreamArchive(FArchive& Ar);
	COREUOBJECT_API void StreamStructuredArchive(FStructuredArchive::FSlot Slot);
};
