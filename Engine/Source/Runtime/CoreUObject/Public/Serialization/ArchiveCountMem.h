// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Serialization/ArchiveUObject.h"

/**
 * Archive for counting memory usage.
 */
class FArchiveCountMem : public FArchiveUObject
{
public:
	FArchiveCountMem( UObject* Src, bool bFilterEditorOnly = false )
		: Num(0)
		, Max(0)
	{
		ArIsCountingMemory = true;
		ArIsFilterEditorOnly = bFilterEditorOnly;
		if( Src )
		{
			Src->Serialize( *this );
		}
	}
	SIZE_T GetNum() const
	{
		return Num;
	}
	SIZE_T GetMax() const
	{
		return Max;
	}

	virtual void CountBytes( SIZE_T InNum, SIZE_T InMax ) override
	{
		Num += InNum;
		Max += InMax;
	}
	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const override { return TEXT("FArchiveCountMem"); }

protected:
	SIZE_T Num, Max;
};
