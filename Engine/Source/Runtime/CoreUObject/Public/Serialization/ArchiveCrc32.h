// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryLayout.h"
#include "Templates/EnableIf.h"
#include "Templates/Models.h"

class FArchive;
class UObject;
struct CStaticStructProvider;

/**
* Calculates a checksum from the input provided to the archive.
*/
class FArchiveCrc32 : public FArchiveUObject
{
public:
	/**
	* Default constructor.
	*/
	COREUOBJECT_API FArchiveCrc32(uint32 InCRC = 0, UObject* InRootObject = nullptr);

	/**
	 * @return The checksum computed so far.
	 */
	uint32 GetCrc() const { return CRC; }

	//~ Begin FArchive Interface
	COREUOBJECT_API virtual void Serialize(void* Data, int64 Num) override;
	COREUOBJECT_API virtual FArchive& operator<<(class FName& Name);
	COREUOBJECT_API virtual FArchive& operator<<(class UObject*& Object);
	COREUOBJECT_API virtual void Reset() override;
	virtual FString GetArchiveName() const { return TEXT("FArchiveCrc32"); }
	//~ End FArchive Interface

	/**
		* Serializes a USTRUCT value into an ArchiveCrc32.
		*
		* @param Ar The archive to serialize from or to.
		* @param Value The value to serialize.
		* NOTE: This should be deprecated and removed as soon as possible JIRA: UE-203804. It must be kept as a hidden friend operator to avoid conflict with specializations found in StructuredArchiveAdapter.
		*/
	template <typename StructType UE_REQUIRES(TModels_V<CStaticStructProvider, StructType>)>
	inline friend FArchiveCrc32& operator<<(FArchiveCrc32& Ar, const StructType& Value)
	{
		StructType* MutableValue = const_cast<StructType*>(&Value);
		StructType::StaticStruct()->SerializeItem(Ar, MutableValue, nullptr);
		return Ar;
	}

	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override

	

private:
	uint32 CRC;
	uint32 InitialCRC;
	UObject* RootObject;
};

