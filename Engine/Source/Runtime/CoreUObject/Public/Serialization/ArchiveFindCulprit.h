// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/ArchiveUObject.h"

class FArchive;
class FProperty;
class UObject;

/*----------------------------------------------------------------------------
	FArchiveFindCulprit.
----------------------------------------------------------------------------*/
/**
 * Archive for finding who references an object.
 */
class FArchiveFindCulprit : public FArchiveUObject
{
public:
	/**
	 * Constructor
	 *
	 * @param	InFind	the object that we'll be searching for references to
	 * @param	Src		the object to serialize which may contain a reference to InFind
	 * @param	InPretendSaving		if true, marks the archive as saving and persistent, so that a different serialization codepath is followed
	 */
	COREUOBJECT_API FArchiveFindCulprit( UObject* InFind, UObject* Src, bool InPretendSaving );

	int32 GetCount() const
	{
		return Count;
	}
	int32 GetCount( TArray<const FProperty*>& Properties )
	{
		Properties = Referencers;
		return Count;
	}

	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const { return TEXT("FArchiveFindCulprit"); }

protected:
	UObject*			Find;
	int32					Count;
	bool				PretendSaving;
	class TArray<const FProperty*>	Referencers;

private:
	COREUOBJECT_API FArchive& operator<<( class UObject*& Obj );
};
