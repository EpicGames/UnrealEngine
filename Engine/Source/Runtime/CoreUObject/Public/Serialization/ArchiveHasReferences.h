// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"

class FArchive;
class UObject;

class FArchiveHasReferences : private FArchiveUObject
{
public:
	COREUOBJECT_API FArchiveHasReferences(UObject* InTarget, const TSet<UObject*>& InPotentiallyReferencedObjects);

	bool HasReferences() const { return Result; }

	static COREUOBJECT_API TArray<UObject*> GetAllReferencers(const TArray<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore );
	static COREUOBJECT_API TArray<UObject*> GetAllReferencers(const TSet<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore );

private:
	COREUOBJECT_API virtual FArchive& operator<<( UObject*& Obj ) override;

	UObject* Target;
	const TSet<UObject*>& PotentiallyReferencedObjects;
	bool Result;
};
