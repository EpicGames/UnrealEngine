// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryWriter.h"

class FArchive;
class UObject;

/*----------------------------------------------------------------------------
FArchiveObjectCrc32
----------------------------------------------------------------------------*/

/**
* Calculates a checksum on an object's serialized data stream.
*/
class FArchiveObjectCrc32 : public FArchiveUObject
{
public:
	/**
	* Default constructor.
	*/
	COREUOBJECT_API FArchiveObjectCrc32();
	COREUOBJECT_API ~FArchiveObjectCrc32();

	//~ Begin FArchive Interface
	COREUOBJECT_API virtual void Serialize(void* Data, int64 Length);
	COREUOBJECT_API virtual FArchive& operator<<(class FName& Name);
	COREUOBJECT_API virtual FArchive& operator<<(class UObject*& Object);
	COREUOBJECT_API virtual FArchive& operator<<(FObjectPtr& ObjectPtr) override;
	virtual FString GetArchiveName() const { return TEXT("FArchiveObjectCrc32"); }
	//~ End FArchive Interface

	/**
	* Serialize the given object, calculate and return its checksum.
	*/
	COREUOBJECT_API uint32 Crc32(UObject* Object, uint32 CRC = 0);
	COREUOBJECT_API uint32 Crc32(UObject* Object, UObject* Root, uint32 CRC);

protected:
	/** Return if object was already serialized */
	virtual bool CustomSerialize(class UObject* Object) { return false; }

	/** Internal byte array used for serialization */
	TArray<uint8> SerializedObjectData;
	/** Internal archive used for serialization */
	FMemoryWriter MemoryWriter;
	/** Internal queue of object references awaiting serialization */
	TQueue<UObject*, EQueueMode::SingleThreaded> ObjectsToSerialize;
	/** Internal currently serialized object */
	const UObject* ObjectBeingSerialized;
	/** Internal root object */
	const UObject* RootObject;
};
