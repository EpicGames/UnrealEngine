// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/Class.h"

/*----------------------------------------------------------------------------
	FArchiveObjectPropertyMapper.
----------------------------------------------------------------------------*/

/**
 * Class for collecting references to objects, along with the properties that
 * references that object.
 */
class FArchiveObjectPropertyMapper : public FArchiveUObject
{
public:
	/**
	 * Constructor
	 *
	 * @param	InObjectArray			Array to add object references to
	 * @param	InBase					only objects with this outer will be considered, or NULL to disregard outers
	 * @param	InLimitClass			only objects of this class (and children) will be considered, or null to disregard object class
	 * @param	bInRequireDirectOuter	determines whether objects contained within 'InOuter', but that do not have an Outer
	 *									of 'InOuter' are included.  i.e. for objects that have GetOuter()->GetOuter() == InOuter.
	 *									If InOuter is NULL, this parameter is ignored.
	 * @param	bInSerializeRecursively	only applicable when LimitOuter != NULL && bRequireDirectOuter==true;
	 *									serializes each object encountered looking for subobjects of referenced
	 *									objects that have LimitOuter for their Outer (i.e. nested subobjects/components)
	 */
	FArchiveObjectPropertyMapper( TMap<FProperty*,UObject*>* InObjectGraph, UObject* InOuter=NULL, UClass* InLimitClass=NULL, bool bInRequireDirectOuter=true, bool bInSerializeRecursively=true )
	:	ObjectGraph( InObjectGraph ), LimitOuter(InOuter), LimitClass(InLimitClass), bRequireDirectOuter(bInRequireDirectOuter), bSerializeRecursively(bInSerializeRecursively)
	{
		ArIsObjectReferenceCollector = true;
		bSerializeRecursively = bInSerializeRecursively && LimitOuter != NULL;
	}
private:
	/** 
	 * UObject serialize operator implementation
	 *
	 * @param Object	reference to Object reference
	 * @return reference to instance of this class
	 */
	FArchive& operator<<( UObject*& Object )
	{
		// Avoid duplicate entries.
		if ( Object != NULL )
		{
			if ((LimitClass == NULL || Object->IsA(LimitClass)) &&
				(LimitOuter == NULL || (Object->GetOuter() == LimitOuter || (!bRequireDirectOuter && Object->IsIn(LimitOuter)))) )
			{
				ObjectGraph->Add(GetSerializedProperty(), Object);
				if ( bSerializeRecursively && !ObjectArray.Contains(Object) )
				{
					ObjectArray.Add( Object );

					// check this object for any potential object references
					Object->Serialize(*this);
				}
			}
		}

		return *this;
	}

	/** Tracks the objects which have been serialized by this archive, to prevent recursion */
	TArray<UObject*>			ObjectArray;

	/** Stored pointer to array of objects we add object references to */
	TMap<FProperty*,UObject*>*	ObjectGraph;

	/** only objects with this outer will be considered, NULL value indicates that outers are disregarded */
	UObject*			LimitOuter;

	/** only objects of this type will be considered, NULL value indicates that all classes are considered */
	UClass*				LimitClass;

	/** determines whether nested objects contained within LimitOuter are considered */
	bool				bRequireDirectOuter;

	/** determines whether we serialize objects that are encounterd by this archive */
	bool				bSerializeRecursively;
};

