// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Serialization/ArchiveUObject.h"

enum class EArchiveReferenceMarkerFlags : uint32
{
	None = 0,
	SkipCompilingAssets = 1 // Skip serialization of assets still being compiled, some data might be missing.
};

ENUM_CLASS_FLAGS(EArchiveReferenceMarkerFlags);

/**
 * This archive marks all objects referenced by the specified "root set" of objects.
 */
class FArchiveReferenceMarker : public FArchiveUObject
{
public:
	FArchiveReferenceMarker( TArray<UObject*>& SourceObjects, EArchiveReferenceMarkerFlags InFlags = EArchiveReferenceMarkerFlags::None)
	{
		ArIsObjectReferenceCollector = true;
		ArIgnoreOuterRef = true;
		SetShouldSkipCompilingAssets((InFlags & EArchiveReferenceMarkerFlags::SkipCompilingAssets) != EArchiveReferenceMarkerFlags::None);

		for ( int32 ObjectIndex = 0; ObjectIndex < SourceObjects.Num(); ObjectIndex++ )
		{
			UObject* Object = SourceObjects[ObjectIndex];
			Object->Mark(OBJECTMARK_TagImp);

			// OBJECTMARK_TagImp is used to allow serialization of objects which we would otherwise ignore.
			Object->Serialize(*this);
		}

		for ( int32 ObjectIndex = 0; ObjectIndex < SourceObjects.Num(); ObjectIndex++ )
		{
			UObject* Object = SourceObjects[ObjectIndex];
			Object->UnMark(OBJECTMARK_TagImp);
		}
	}

	/** 
	* UObject serialize operator implementation
	*
	* @param Object	reference to Object reference
	* @return reference to instance of this class
	*/
	FArchive& operator<<( UObject*& Object )
	{
		if (Object != NULL && !(Object->HasAnyMarks(OBJECTMARK_TagExp) || !IsValidChecked(Object) || Object->IsUnreachable()) )
		{
			Object->Mark(OBJECTMARK_TagExp);

			const bool bIgnoreObject = 
				// No need to call Serialize from here for any objects that were part of our root set.
				// By preventing re-entrant serialization using the OBJECTMARK_TagImp flag (instead of just marking each object in the root set with
				// OBJECTMARK_TagExp prior to calling Serialize) we can determine which objects from our root set are being referenced
				// by other objects in our root set.
				Object->HasAnyMarks(OBJECTMARK_TagImp);

			if ( bIgnoreObject == false )
			{
				Object->Serialize( *this );
			}
		}

		return *this;
	}
};
