// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Internal

#include "CoreMinimal.h"
#include "Serialization/ArchiveReplaceObjectRef.h"

/*----------------------------------------------------------------------------
	FArchiveReplaceObjectAndStructPropertyRef.
----------------------------------------------------------------------------*/
/**
 * Specialized version of FArchiveReplaceObjectRef that replaces references to FFields
 * that were owned by any of the old UStructs in the Replacement Map with their respective
 * new versions that belong to the new UStrtucts in the Replacement Map.
 */
template <class T>
class FArchiveReplaceObjectAndStructPropertyRef : public FArchiveReplaceObjectRef<T>
{
public:
	/**
	 * Initializes variables and starts the serialization search
	 *
	 * @param InSearchObject        The object to start the search on
	 * @param InReplacementMap      Map of objects to find -> objects to replace them with (null zeros them)
	 * @param Flags                 Enum specifying behavior of archive
	 */
	FArchiveReplaceObjectAndStructPropertyRef(UObject* InSearchObject, const TMap<T*, T*>& InReplacementMap, EArchiveReplaceObjectFlags Flags = EArchiveReplaceObjectFlags::None)
		: FArchiveReplaceObjectRef<T>(InSearchObject, InReplacementMap, Flags | EArchiveReplaceObjectFlags::DelayStart)
	{
		// Note: We intentionally add the 'DelayStart' flag above; otherwise the base class ctor will use the base archive type for serialization, and our overrides won't get called.
		// Here we're looking for whether or not the *caller* of this ctor has asked for a delayed start; in that case, we defer to the caller as to when our serialization will start.
		if (!(Flags & EArchiveReplaceObjectFlags::DelayStart))
		{
			FArchiveReplaceObjectRef<T>::SerializeSearchObject();
		}
	}

	/**
	 * Serializes the reference to FProperties
	 */
	virtual FArchive& operator<<(FField*& InField) override
	{
		if (InField)
		{
			// Some structs (like UFunctions in their bytecode) reference properties of another UStructs.
			// In this case we need to inspect their owner and if it's one of the objects we want to replace,
			// replace the entire property with the one matching on the struct we want to replace it with
			UStruct* OldOwnerStruct = InField->GetOwner<UStruct>();
			if (OldOwnerStruct)
			{
				T* const* ReplaceWith = (T* const*)((const TMap<UObject*, UObject*>*)&this->ReplacementMap)->Find(OldOwnerStruct);
				if (ReplaceWith)
				{
					// We want to replace the property's owner but since that would be even worse than replacing UObject's Outer
					// we need to replace the entire property instead. We need to find the new property on the object we want to replace the Owner with
					UStruct* NewOwnerStruct = *ReplaceWith ? CastChecked<UStruct>(*ReplaceWith) : nullptr;
					FField* ReplaceWithField = NewOwnerStruct ? NewOwnerStruct->FindPropertyByName(InField->GetFName()) : nullptr;
					// Do we need to verify the existence of ReplaceWithField? Theoretically it could be missing on the new version
					// of the owner struct and in this case we still don't want to keep the stale old property pointer around so it's safer to null it
					InField = ReplaceWithField;
					if (this->bTrackReplacedReferences)
					{
						this->ReplacedReferences.FindOrAdd(OldOwnerStruct).AddUnique(this->GetSerializedProperty());
					}
					this->Count++;
				}
				// A->IsIn(A) returns false, but we don't want to NULL that reference out, so extra check here.
				else if (OldOwnerStruct == this->SearchObject || OldOwnerStruct->IsIn(this->SearchObject))
				{
					bool bAlreadyAdded = false;
					this->SerializedObjects.Add(OldOwnerStruct, &bAlreadyAdded);
					if (!bAlreadyAdded)
					{
						// No recursion
						this->PendingSerializationObjects.Add(OldOwnerStruct);
					}
				}
				else if (this->bNullPrivateReferences && !OldOwnerStruct->HasAnyFlags(RF_Public))
				{
					checkf(false, TEXT("Can't null a reference to %s on property %s as it would be equivalent to nulling UObject's Outer."),
						*OldOwnerStruct->GetPathName(), *InField->GetName());
				}
			}
			else
			{
				// Just serialize the field to find any UObjects it may be referencing that we want to replace 
				InField->Serialize(*this);
			}
		}
		return *this;
	}
};


