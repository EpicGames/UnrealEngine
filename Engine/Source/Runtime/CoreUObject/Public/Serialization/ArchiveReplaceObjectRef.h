// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "VerseVM/Inline/VVMCellInline.h"

/** Base class for object replacement archives */ 
class FArchiveReplaceObjectRefBase : public FArchiveUObject
{
public:

	/**
	* Returns the number of times the object was referenced
	*/
	int64 GetCount() const { return Count; }

	/**
	* Returns a reference to the object this archive is operating on
	*/
	const UObject* GetSearchObject() const { return SearchObject; }

	/**
	* Returns a reference to the replaced references map
	*/
	COREUOBJECT_API const TMap<UObject*, TArray<FProperty*>>& GetReplacedReferences() const;

	/**
	* Returns the name of this archive.
	**/
	virtual FString GetArchiveName() const { return TEXT("ReplaceObjectRef"); }
	
protected:

	template <typename ContainerType, typename ElementToObjectType>
	bool ShouldSkipReplacementCheckForObjectPtr(FObjectPtr& Obj, const ContainerType& ReplacementContainer, const ElementToObjectType& ElementToObject)
	{
		if (Obj.IsResolved())
		{
			return false;
		}

		if (!CanIgnoreUnresolvedImports.IsSet())
		{
			bool bCanIgnoreUnresolvedImportsLocal = ReplacementContainer.Num() == 0;
			if (!bCanIgnoreUnresolvedImportsLocal)
			{
				bCanIgnoreUnresolvedImportsLocal = true; // Will be set to false if any of the criteria in the loop below are met.
				for (const auto& ReplacementElem : ReplacementContainer)
				{
					if (const UObject* ReplacementObject = ElementToObject(ReplacementElem))
					{
						if (ReplacementObject->GetOutermost() != GetTransientPackage())
						{
							bCanIgnoreUnresolvedImportsLocal = false;
							break;
						}
					}
				}
			}
			CanIgnoreUnresolvedImports = bCanIgnoreUnresolvedImportsLocal;
		}

		if (CanIgnoreUnresolvedImports.GetValue())
		{
			return true;
		}

		if (UClass* ReferenceClass = Obj.GetClass())
		{
			bool bReferenceTypeIsRelatedToReplacementType = false;
			for (const auto& ReplacementElem : ReplacementContainer)
			{
				if (const UObject* ReplacementObject = ElementToObject(ReplacementElem))
				{
					if (ReplacementObject->IsA(ReferenceClass) || Obj.IsA(ReplacementObject->GetClass()))
					{
						bReferenceTypeIsRelatedToReplacementType = true;
						break;
					}
				}
			}
			if (!bReferenceTypeIsRelatedToReplacementType)
			{
				return true;
			}
		}

		return false;
	}

	/**
	* Serializes a single object
	*/
	COREUOBJECT_API virtual void SerializeObject(UObject* ObjectToSerialize);

	/** Initial object to start the reference search from */
	UObject* SearchObject = nullptr;

	/** Object that SerializeObject was most recently called on */
	UObject* SerializingObject = nullptr;

	/** The number of times encountered */
	int32 Count = 0;

	/** List of objects that have already been serialized */
	TSet<UObject*> SerializedObjects;

	/** Object that will be serialized */
	TArray<UObject*> PendingSerializationObjects;

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/** List of VCells that have already been serialized */
	TSet<Verse::VCell*> SerializedCells;

	/** VCells that will be serialized */
	TArray<Verse::VCell*> PendingSerializationCells;
#endif

	/** Map of referencing objects to referencing properties */
	TMap<UObject*, TArray<FProperty*>> ReplacedReferences;

	/** Whether to populate the map of referencing objects to referencing properties */
	bool bTrackReplacedReferences = false;

	/**
	* Whether references to non-public objects not contained within the SearchObject
	* should be set to null
	*/
	bool bNullPrivateReferences = false;

	/**
	* Whether unresolved references to objects in other packages can be ignored when searching
	*/
	TOptional<bool> CanIgnoreUnresolvedImports;
};

enum class EArchiveReplaceObjectFlags
{
	None                      = 0,

	// References to non-public objects not contained within the SearchObject should be set to null
	NullPrivateRefs            = 1 << 0,

	// Do not replace Outer pointers on Objects.
	IgnoreOuterRef             = 1 << 1,

	// Do not replace the ObjectArchetype reference on Objects.
	IgnoreArchetypeRef         = 1 << 2,

	// Prevent the constructor from starting the process. Allows child classes to do initialization stuff in their constructor.
	DelayStart                 = 1 << 3,

	// Replace the ClassGeneratedBy reference in UClass
	IncludeClassGeneratedByRef = 1 << 4,

	// Populate the map of referencing objects to referencing properties
	TrackReplacedReferences    = 1 << 5
};
ENUM_CLASS_FLAGS(EArchiveReplaceObjectFlags)

/*----------------------------------------------------------------------------
	FArchiveReplaceObjectRef.
----------------------------------------------------------------------------*/
/**
 * Archive for replacing a reference to an object. This classes uses
 * serialization to replace all references to one object with another.
 * Note that this archive will only traverse objects with an Outer
 * that matches InSearchObject.
 *
 * NOTE: The template type must be a child of UObject or this class will not compile.
 */
template< class T >
class FArchiveReplaceObjectRef : public FArchiveReplaceObjectRefBase
{
public:
	/**
	 * Initializes variables and starts the serialization search
	 *
	 * @param InSearchObject        The object to start the search on
	 * @param ReplacementMap        Map of objects to find -> objects to replace them with (null zeros them)
	 * @param Flags                 Enum specifying behavior of archive 		
	 */
	FArchiveReplaceObjectRef(UObject* InSearchObject, const TMap<T*, T*>& InReplacementMap, EArchiveReplaceObjectFlags Flags = EArchiveReplaceObjectFlags::None)
		: ReplacementMap(InReplacementMap)
	{
		bTrackReplacedReferences = !!(Flags & EArchiveReplaceObjectFlags::TrackReplacedReferences);

		SearchObject = InSearchObject;
		Count = 0;
		bNullPrivateReferences = !!(Flags & EArchiveReplaceObjectFlags::NullPrivateRefs);

		ArIsObjectReferenceCollector = true;
		ArIsModifyingWeakAndStrongReferences = true;		// Also replace weak references too!
		ArIgnoreArchetypeRef = !!(Flags & EArchiveReplaceObjectFlags::IgnoreArchetypeRef);
		ArIgnoreOuterRef = !!(Flags & EArchiveReplaceObjectFlags::IgnoreOuterRef);
		ArIgnoreClassGeneratedByRef = !(Flags & EArchiveReplaceObjectFlags::IncludeClassGeneratedByRef);

		if (!(Flags & EArchiveReplaceObjectFlags::DelayStart))
		{
			SerializeSearchObject();
		}
	}

	/**
	 * Starts the serialization of the root object
	 */
	void SerializeSearchObject()
	{
		ReplacedReferences.Reset();

		if (SearchObject != NULL && !SerializedObjects.Find(SearchObject)
		&&	(ReplacementMap.Num() > 0 || bNullPrivateReferences))
		{
			// start the initial serialization
			SerializedObjects.Add(SearchObject);
			SerializingObject = SearchObject;
			SerializeObject(SearchObject);
			for (int32 Iter = 0; Iter < PendingSerializationObjects.Num(); Iter++)
			{
				SerializingObject = PendingSerializationObjects[Iter];
				SerializeObject(SerializingObject);
			}
			PendingSerializationObjects.Reset();
		}
	}

	/**
	 * Serializes the reference to the object
	 */
	virtual FArchive& operator<<( UObject*& Obj ) override
	{
		if (Obj != NULL)
		{
			// If these match, replace the reference
			if (T* const* ReplaceWith = (T* const*)((const TMap<UObject*, UObject*>*)&ReplacementMap)->Find(Obj))
			{
				Obj = *ReplaceWith;
				if (bTrackReplacedReferences)
				{
					ReplacedReferences.FindOrAdd(SerializingObject).AddUnique(GetSerializedProperty());
				}
				Count++;
			}
			// A->IsIn(A) returns false, but we don't want to NULL that reference out, so extra check here.
			else if ( Obj == SearchObject || Obj->IsIn(SearchObject) )
			{
#if 0
				// DEBUG: Log when we are using the A->IsIn(A) path here.
				if(Obj == SearchObject)
				{
					FString ObjName = Obj->GetPathName();
					UE_LOGF(LogSerialization, Log,  "FArchiveReplaceObjectRef: Obj == SearchObject : '%ls'", *ObjName );
				}
#endif
				bool bAlreadyAdded = false;
				SerializedObjects.Add(Obj, &bAlreadyAdded);
				if (!bAlreadyAdded)
				{
					// No recursion
					PendingSerializationObjects.Add(Obj);
				}
			}
			else if ( bNullPrivateReferences && !Obj->HasAnyFlags(RF_Public) )
			{
				Obj = NULL;
			}
		}
		return *this;
	}

	/**
	 * Serializes a resolved or unresolved object reference
	 */
	FArchive& operator<<( FObjectPtr& Obj )
	{
		if (ShouldSkipReplacementCheckForObjectPtr(Obj, ReplacementMap, [] (const TPair<T*, T*>& ReplacementPair) -> const UObject*
			{
				return ReplacementPair.Key;
			}))
		{
			return *this;
		}

		// Allow object references to go through the normal code path of resolving and running the raw pointer code path
		return FArchiveReplaceObjectRefBase::operator<<(Obj);
	}

	virtual FArchive& operator<<(FSoftObjectPath& Value) override 
	{
		if (UObject* Obj = Value.ResolveObject())
		{
			*this << Obj;
			Value = Obj;
		}
		else
		{
			FArchiveReplaceObjectRefBase::operator<<(Value);
		}
		return *this;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Cell) override
	{
		if (Cell && Cell->GetCppClassInfo()->InstancedCell)
		{
			bool bIsAlreadyInSet = false;
			SerializedCells.Add(Cell, &bIsAlreadyInSet);
			if (!bIsAlreadyInSet)
			{
				PendingSerializationCells.Add(Cell);
			}
		}
		return *this;
	}
#endif

protected:
	/** Map of objects to find references to -> object to replace references with */
	const TMap<T*,T*>& ReplacementMap;
	
};


