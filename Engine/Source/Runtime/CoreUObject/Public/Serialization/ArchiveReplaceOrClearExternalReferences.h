// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Casts.h"
#include "UObject/Package.h"
#include "Serialization/ArchiveReplaceObjectRef.h"

/*----------------------------------------------------------------------------
FArchiveReplaceOrClearExternalReferences.
----------------------------------------------------------------------------*/
/**
* Identical to FArchiveReplaceObjectRef, but for references to private objects
* in other packages we clear the reference instead of preserving it (unless it
* makes it into the replacement map)
*/
template< class T >
class FArchiveReplaceOrClearExternalReferences : public FArchiveReplaceObjectRef<T>
{
	typedef FArchiveReplaceObjectRef<T> TSuper;
public:
	FArchiveReplaceOrClearExternalReferences
		( UObject* InSearchObject
		, const TMap<T*, T*>& InReplacementMap
		, UPackage* InDestPackage
		, EArchiveReplaceObjectFlags Flags = EArchiveReplaceObjectFlags::None)
		: TSuper(InSearchObject, InReplacementMap, EArchiveReplaceObjectFlags::DelayStart | Flags)
		, DestPackage(InDestPackage)
	{
		if (!(Flags & EArchiveReplaceObjectFlags::DelayStart))
		{
			this->SerializeSearchObject();
		}
	}

	FArchive& operator<<(UObject*& Obj)
	{
		UObject* Resolved = Obj;
		TSuper::operator<<(Resolved);
		// if Resolved is a private object in another package just clear the reference:
		if (Resolved)
		{
			UPackage* ObjPackage = Resolved->GetPackage();
			if (ObjPackage)
			{
				CA_SUPPRESS(6011);
				if (ObjPackage != Obj && 
					DestPackage != ObjPackage && 
					Obj->GetOutermostObject() != TSuper::GetSearchObject()->GetOutermostObject() &&
					!Obj->HasAnyFlags(RF_Public))
				{
					Resolved = nullptr;
				}
			}
		}
		Obj = Resolved;
		return *this;
	}

protected:
	/** Package that we are loading into, references to private objects in other packages will be cleared */
	UPackage* DestPackage;
};
