// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Serialization/ArchiveSavePackageData.h"
#include "UObject/ArchiveCookContext.h"
#include "UObject/ObjectSaveContext.h"

namespace UE::Cook { class ICookInfo; }

/**
 * This is the structure that should be used by most callers of an archive (other than SavePackage which handles
 * the full complexity) that need to provide SavePackage or Cook information to the archive.
 * 
 * FArchiveSavePackageData is a minimalist struct that holds pointers to types it cannot have defined,
 * because they are defined in modules higher in the dependency graph.
 * Because of that minimalism, it needs pointers to several pieces of related data.
 * Collect all of those pieces of related data in this module that knows all of the types, and keep them
 * in a single amalgamated structure the sets the base class pointers to the appropriate internal structures.
 */
struct FArchiveSavePackageDataBuffer : public FArchiveSavePackageData
{
public:
	FArchiveSavePackageDataBuffer(const ITargetPlatform* InTargetPlatform = nullptr, UPackage* Package = nullptr,
		UE::Cook::ICookInfo* CookInfo = nullptr)
		// warning V1050 : The uninitialized class member 'ObjectSavePackageSerializeContextBuffer' is used when initializing the base class 'FArchiveSavePackageData'.
		// This warning can be ignored because the base class only records the pointer and does not dereference it
		: FArchiveSavePackageData(ObjectSavePackageSerializeContextBuffer, nullptr, nullptr) // -V1050
		, ObjectSaveContextData(FObjectSaveContextData())
		, ObjectSavePackageSerializeContextBuffer(*ObjectSaveContextData)
		, CookContextBuffer(Package, UE::Cook::ECookType::Unknown, UE::Cook::ECookingDLC::Unknown,
			InTargetPlatform, CookInfo)
	{
		SetConstructorTargetPlatform(InTargetPlatform);
	}

	FArchiveSavePackageDataBuffer(FArchiveCookContext InContext)
		: FArchiveSavePackageData(ObjectSavePackageSerializeContextBuffer, nullptr, nullptr)
		, ObjectSaveContextData(FObjectSaveContextData())
		, ObjectSavePackageSerializeContextBuffer(*ObjectSaveContextData)
		, CookContextBuffer(MoveTemp(InContext))
	{
		SetConstructorTargetPlatform(InContext.GetTargetPlatform());
	}

	FArchiveSavePackageDataBuffer(FObjectSaveContextData& InData, UPackage* InPackage = nullptr)
		: FArchiveSavePackageData(ObjectSavePackageSerializeContextBuffer, nullptr, nullptr)
		, ObjectSavePackageSerializeContextBuffer(InData)
		, CookContextBuffer(InPackage, InData.CookType, InData.CookingDLC, InData.TargetPlatform, InData.CookInfo)
	{
		SetConstructorTargetPlatform(InData.TargetPlatform);
	}

	TOptional<FObjectSaveContextData> ObjectSaveContextData;
	FObjectSavePackageSerializeContext ObjectSavePackageSerializeContextBuffer;
	FArchiveCookContext CookContextBuffer;

private:
	void SetConstructorTargetPlatform(const ITargetPlatform* InTargetPlatform)
	{
		if (InTargetPlatform)
		{
			TargetPlatform = InTargetPlatform;
			CookContext = &CookContextBuffer;
		}
	}
};
