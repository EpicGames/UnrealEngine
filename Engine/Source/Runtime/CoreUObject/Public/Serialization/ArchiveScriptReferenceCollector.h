// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "Serialization/ArchiveUObject.h"

/** Simple reference processor and collector for collecting all UObjects referenced by FProperties */
class FPropertyReferenceCollector : public FReferenceCollector
{
	/** The owner object for properties we collect references (to make sure we don't add it to the references list as it's not needed) */
	UObject* ExcludeOwner;
	/** List of unique references */
	TArray<UObject*>& UniqueReferences;
public:
	FPropertyReferenceCollector(UObject* InExcludeOwner, TArray<UObject*>& InUniqueReferences)
		: ExcludeOwner(InExcludeOwner)
		, UniqueReferences(InUniqueReferences)
	{
	}

	virtual bool IsIgnoringArchetypeRef() const override { return false; }
	virtual bool IsIgnoringTransient() const override { return false; }
	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		// Skip nulls and the owner object
		if (InObject != nullptr && InObject != ExcludeOwner)
		{
			// Don't collect objects that will never be GC'd anyway
			if (!InObject->HasAnyInternalFlags(EInternalObjectFlags::Native) && !GUObjectArray.IsDisregardForGC(InObject) && !UniqueReferences.Contains(InObject))
			{
				UniqueReferences.Add(InObject);
			}
		}
	}
};

/*******************************************************************************
 * FArchiveScriptReferenceCollector
 ******************************************************************************/

class FArchiveScriptReferenceCollector : public FArchiveUObject
{
public:
	/**
	* Constructor
	*
	* @param	InObjectArray			Array to add object references to
	*/
	explicit FArchiveScriptReferenceCollector(TArray<UObject*>& InObjectArray, UObject* InExcludeOwner = nullptr)
		: ObjectArray(InObjectArray)
		, ExcludeOwner(InExcludeOwner)
	{
		ArIsObjectReferenceCollector = true;
		this->SetIsPersistent(false);
		ArIgnoreArchetypeRef = false;
	}
protected:
	/**
	* UObject serialize operator implementation
	*
	* @param Object	reference to Object reference
	* @return reference to instance of this class
	*/
	virtual FArchive& operator<<(UObject*& Object) override
	{
		// Avoid duplicate entries.
		if (Object != nullptr && Object != ExcludeOwner && !ObjectArray.Contains(Object))
		{
			check(Object->IsValidLowLevel());
			ObjectArray.Add(Object);
		}
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& Object) override
	{
		if (Object.IsResolved())
		{
			UObject* Obj = Object.Get();
			FArchive& Ar = *this << Obj;
			return Ar;
		}
		return *this;
	}

	/**
	* FField serialize operator implementation
	*
	* @param Field	reference to field reference
	* @return reference to instance of this class
	*/
	virtual FArchive& operator<<(FField*& Field) override
	{
		if (Field != nullptr)
		{
			// It's faster to collect references via AddReferencedObjects than serialization
			FPropertyReferenceCollector Collector(ExcludeOwner, ObjectArray);
			Field->AddReferencedObjects(Collector);
		}
		return *this;
	}

	/** Stored reference to array of objects we add object references to */
	TArray<UObject*>&		ObjectArray;
	/** Script or property owner object that should be excluded from ObjectArray list */
	UObject*				ExcludeOwner;
};
