// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/ArchiveUObject.h"

class FArchive;
class FOutputDevice;
class UObject;

/*----------------------------------------------------------------------------
	FArchiveShowReferences.
----------------------------------------------------------------------------*/
/**
 * Archive for displaying all objects referenced by a particular object.
 */
class FArchiveShowReferences : public FArchiveUObject
{
	/**
	 * I/O function.  Called when an object reference is encountered.
	 *
	 * @param	Obj		a pointer to the object that was encountered
	 */
	FArchive& operator<<( UObject*& Obj );

	/** the object to display references to */
	UObject* SourceObject;

	/** ignore references to objects have the same Outer as our Target */
	UObject* SourceOuter;

	/** output device for logging results */
	FOutputDevice& OutputAr;

	/**
	 * list of Outers to ignore;  any objects encountered that have one of
	 * these objects as an Outer will also be ignored
	 */
	class TArray<UObject*>& Exclude;

	/** list of objects that have been found */
	class TArray<UObject*> Found;

	bool DidRef;

public:

	/**
	 * Constructor
	 * 
	 * @param	inOutputAr		archive to use for logging results
	 * @param	inOuter			only consider objects that do not have this object as its Outer
	 * @param	inSource		object to show references for
	 * @param	inExclude		list of objects that should be ignored if encountered while serializing SourceObject
	 */
	FArchiveShowReferences( FOutputDevice& inOutputAr, UObject* inOuter, UObject* inSource, TArray<UObject*>& InExclude );

	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const { return TEXT("FArchiveShowReferences"); }
};
