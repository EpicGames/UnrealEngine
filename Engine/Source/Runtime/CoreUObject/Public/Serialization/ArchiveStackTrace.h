// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Serialization/Archive.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/PackageWriter.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class FLinkerLoad;
class FProperty;
class FUObjectThreadContext;
class UObject;

/** Structure that holds stats from comparing two packages */
struct FArchiveDiffStats
{
	/** Size of all of the differences between two packages */
	int64 DiffSize;
	/** Number of differences between two packages */
	int64 NumDiffs;
	/** Size of the source package file (the one we compared against) */
	int64 OriginalFileTotalSize;
	/** Size of the new package file */
	int64 NewFileTotalSize;

	FArchiveDiffStats()
		: DiffSize(0)
		, NumDiffs(0)
		, OriginalFileTotalSize(0)
		, NewFileTotalSize(0)
	{}
};

/** Ignores saving the stack trace when collecting serialize offsets .*/
class FArchiveStackTraceIgnoreScope
{
	const bool bIgnore;
public:
	COREUOBJECT_API FArchiveStackTraceIgnoreScope(bool bInIgnore = true);
	COREUOBJECT_API ~FArchiveStackTraceIgnoreScope();
};

/**
 * Disables collecting both offsets and stack traces when collecting serialize callstacks.
 * Typically used when appending data from one stack tracing archive to another.
 */
class FArchiveStackTraceDisabledScope
{
public:
	COREUOBJECT_API FArchiveStackTraceDisabledScope();
	COREUOBJECT_API ~FArchiveStackTraceDisabledScope();
};

namespace UE::ArchiveStackTrace
{

struct FPackageData
{
	uint8* Data = nullptr;
	int64 Size = 0;
	int64 HeaderSize = 0;
	int64 StartOffset = 0;
};

struct FDeleteByFree
{
	void operator()(void* Ptr) const
	{
		FMemory::Free(Ptr);
	}
};

/** Helper function to load package contents into memory. Supports EDL packages. */
COREUOBJECT_API bool LoadPackageIntoMemory(const TCHAR* InFilename,
	FPackageData& OutPackageData, TUniquePtr<uint8, FDeleteByFree>& OutLoadedBytes);

COREUOBJECT_API void ForceKillPackageAndLinker(FLinkerLoad* Linker);
COREUOBJECT_API bool ShouldIgnoreDiff();
COREUOBJECT_API bool ShouldBypassDiff();

} // namespace UE::ArchiveStackTrace

struct
// Deprecated: 5.3, "FArchiveStackTrace was only used by DiffPackageWriter, and has been moved into a private helper class. Contact Epic if you need this class for another reason.")
FArchiveDiffInfo
{
	int64 Offset;
	int64 Size;
	FArchiveDiffInfo()
		: Offset(0)
		, Size(0)
	{
	}
	FArchiveDiffInfo(int64 InOffset, int64 InSize)
		: Offset(InOffset)
		, Size(InSize)
	{
	}
	bool operator==(const FArchiveDiffInfo& InOther) const
	{
		return Offset == InOther.Offset;
	}
	bool operator<(const FArchiveDiffInfo& InOther) const
	{
		return Offset < InOther.Offset;
	}
	friend FArchive& operator << (FArchive& Ar, FArchiveDiffInfo& InDiffInfo)
	{
		Ar << InDiffInfo.Offset;
		Ar << InDiffInfo.Size;
		return Ar;
	}
};

class FArchiveStackTraceReader : public FLargeMemoryReader
{
public:
	struct FSerializeData
	{
		FSerializeData()
			: Offset(0)
			, Size(0)
			, Count(0)
			, Object(nullptr)
			, PropertyName(NAME_None)
		{}
		FSerializeData(int64 InOffset, int64 InSize, UObject* InObject, FProperty* InProperty);
		int64 Offset;
		int64 Size;
		int64 Count;
		UObject* Object;
		FName PropertyName;
		FString FullPropertyName;

		bool IsContiguousSerialization(const FSerializeData& Other) const
		{
			// Return whether this and other are neighboring bits of data for the serialization of the same instance of an object\property
			return Object == Other.Object && PropertyName == Other.PropertyName &&
				(Offset == Other.Offset || (Offset + Size) == Other.Offset); // This is to merge contiguous blocks
		}
	};
private:
	TArray<FSerializeData> SerializeTrace;
	/** Cached thread context */
	FUObjectThreadContext& ThreadContext;
public:

	COREUOBJECT_API FArchiveStackTraceReader(const TCHAR* InFilename, const uint8* InData, const int64 Num);

	COREUOBJECT_API virtual void Serialize(void* OutData, int64 Num) override;
	const TArray<FSerializeData>& GetSerializeTrace() const
	{
		return SerializeTrace;
	}
	static COREUOBJECT_API FArchiveStackTraceReader* CreateFromFile(const TCHAR* InFilename);
};
