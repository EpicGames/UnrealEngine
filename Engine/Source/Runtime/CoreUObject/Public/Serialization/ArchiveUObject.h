// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/Archive.h"

struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;
struct FWeakObjectPtr;

/**
 * Base FArchive for serializing UObjects. Supports FLazyObjectPtr and FSoftObjectPtr serialization.
 */
class FArchiveUObject : public FArchive
{
public:

	static COREUOBJECT_API FArchive& SerializeLazyObjectPtr(FArchive& Ar, FLazyObjectPtr& Value);
	static COREUOBJECT_API FArchive& SerializeObjectPtr(FArchive& Ar, FObjectPtr& Value);
	static COREUOBJECT_API FArchive& SerializeSoftObjectPtr(FArchive& Ar, FSoftObjectPtr& Value);
	static COREUOBJECT_API FArchive& SerializeSoftObjectPath(FArchive& Ar, FSoftObjectPath& Value);
	static COREUOBJECT_API FArchive& SerializeWeakObjectPtr(FArchive& Ar, FWeakObjectPtr& Value);

	using FArchive::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return SerializeLazyObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FObjectPtr& Value) override { return SerializeObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return SerializeSoftObjectPtr(*this, Value); }
	virtual FArchive& operator<<(FSoftObjectPath& Value) override { return SerializeSoftObjectPath(*this, Value); }
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override { return SerializeWeakObjectPtr(*this, Value); }
	//~ End FArchive Interface
};
