// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Serialization/Archive.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/FileRegions.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectResource.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"

struct FObjectPtr;

#if WITH_TEXT_ARCHIVE_SUPPORT

class FArchiveUObjectFromStructuredArchiveImpl : public FArchiveFromStructuredArchiveImpl
{
	using Super = FArchiveFromStructuredArchiveImpl;

public:

	COREUOBJECT_API FArchiveUObjectFromStructuredArchiveImpl(FStructuredArchive::FSlot Slot);

	using FArchive::operator<<; // For visibility of the overloads we don't override

	//~ Begin FArchive Interface
	COREUOBJECT_API virtual FArchive& operator<<(FLazyObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FWeakObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FObjectPtr& Value) override;
	//~ End FArchive Interface

	COREUOBJECT_API virtual void PushFileRegionType(EFileRegionType Type) override;
	COREUOBJECT_API virtual void PopFileRegionType() override;

private:

	int64 FileRegionStart = 0;
	EFileRegionType CurrentFileRegionType = EFileRegionType::None;

	TArray<FLazyObjectPtr> LazyObjectPtrs;
	TArray<FWeakObjectPtr> WeakObjectPtrs;
	TArray<FSoftObjectPtr> SoftObjectPtrs;
	TArray<FSoftObjectPath> SoftObjectPaths;

	TMap<FLazyObjectPtr, int32> LazyObjectPtrToIndex;
	TMap<FWeakObjectPtr, int32> WeakObjectPtrToIndex;
	TMap<FSoftObjectPtr, int32> SoftObjectPtrToIndex;
	TMap<FSoftObjectPath, int32> SoftObjectPathToIndex;

	COREUOBJECT_API virtual bool Finalize(FStructuredArchive::FRecord Record) override;
};

class FArchiveUObjectFromStructuredArchive
{
public:
	explicit FArchiveUObjectFromStructuredArchive(FStructuredArchive::FSlot InSlot)
		: Impl(InSlot)
	{
	}

	      FArchive& GetArchive()       { return Impl; }
	const FArchive& GetArchive() const { return Impl; }

	void Close() { Impl.Close(); }

private:
	FArchiveUObjectFromStructuredArchiveImpl Impl;
};

#else

class FArchiveUObjectFromStructuredArchive : public FArchiveFromStructuredArchive
{
public:

	FArchiveUObjectFromStructuredArchive(FStructuredArchive::FSlot InSlot)
		: FArchiveFromStructuredArchive(InSlot)
	{

	}
};

#endif
