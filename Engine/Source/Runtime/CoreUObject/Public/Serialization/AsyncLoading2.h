// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AsyncLoading2.h: Unreal async loading #2 definitions.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "IO/PackageId.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/Archive.h"
#include "Serialization/BulkDataCookedIndex.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/MappedName.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectResource.h"
#include "UObject/ObjectVersion.h"
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/VVMUniqueString.h"
#endif

class FArchive;
class FIoDispatcher;
class IAsyncPackageLoader;
class UPackage;

class FPackageImportReference
{
public:
	FPackageImportReference(uint32 InImportedPackageIndex, uint32 InImportedPublicExportHashIndex)
		: ImportedPackageIndex(InImportedPackageIndex)
		, ImportedPublicExportHashIndex(InImportedPublicExportHashIndex)
	{
	}

	uint32 GetImportedPackageIndex() const
	{
		return ImportedPackageIndex;
	}

	uint32 GetImportedPublicExportHashIndex() const
	{
		return ImportedPublicExportHashIndex;
	}

private:
	uint32 ImportedPackageIndex;
	uint32 ImportedPublicExportHashIndex;
};

class FPackageObjectIndex
{
	static constexpr uint64 IndexBits = 62ull;
	static constexpr uint64 IndexMask = (1ull << IndexBits) - 1ull;
	static constexpr uint64 TypeMask = ~IndexMask;
	static constexpr uint64 TypeShift = IndexBits;
	static constexpr uint64 Invalid = ~0ull;

	uint64 TypeAndId = Invalid;

	enum EType
	{
		Export,
		ScriptImport,
		PackageImport,
		Null,
		TypeCount = Null,
	};
	static_assert((TypeCount - 1) <= (TypeMask >> TypeShift), "FPackageObjectIndex: Too many types for TypeMask");

	inline explicit FPackageObjectIndex(EType InType, uint64 InId) : TypeAndId((uint64(InType) << TypeShift) | InId) {}

	COREUOBJECT_API static uint64 GenerateImportHashFromObjectPath(const FStringView& ObjectPath);
	COREUOBJECT_API static uint64 GenerateImportHashFromVersePath(FUtf8StringView VersePath);

public:
	FPackageObjectIndex() = default;

	inline static FPackageObjectIndex FromExportIndex(const int32 Index)
	{
		return FPackageObjectIndex(Export, Index);
	}

	inline static FPackageObjectIndex FromScriptPath(const FStringView& ScriptObjectPath)
	{
		return FPackageObjectIndex(ScriptImport, GenerateImportHashFromObjectPath(ScriptObjectPath));
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	inline static FPackageObjectIndex FromVersePath(Verse::VUniqueString* VersePath)
	{
		return FromVersePath(VersePath->AsStringView());
	}
#endif
	inline static FPackageObjectIndex FromVersePath(FUtf8StringView VersePath)
	{
		return FPackageObjectIndex(ScriptImport, GenerateImportHashFromVersePath(VersePath));
	}

	inline static FPackageObjectIndex FromPackageImportRef(const FPackageImportReference& PackageImportRef)
	{
		uint64 Id = static_cast<uint64>(PackageImportRef.GetImportedPackageIndex()) << 32 | PackageImportRef.GetImportedPublicExportHashIndex();
		check(!(Id & TypeMask));
		return FPackageObjectIndex(PackageImport, Id);
	}

	inline bool IsNull() const
	{
		return TypeAndId == Invalid;
	}

	inline bool IsExport() const
	{
		return (TypeAndId >> TypeShift) == Export;
	}

	inline bool IsImport() const
	{
		return IsScriptImport() || IsPackageImport();
	}

	inline bool IsScriptImport() const
	{
		return (TypeAndId >> TypeShift) == ScriptImport;
	}

	inline bool IsPackageImport() const
	{
		return (TypeAndId >> TypeShift) == PackageImport;
	}

	inline uint32 ToExport() const
	{
		check(IsExport());
		return uint32(TypeAndId);
	}

	inline FPackageImportReference ToPackageImportRef() const
	{
		uint32 ImportedPackageIndex = static_cast<uint32>((TypeAndId & IndexMask) >> 32);
		uint32 ExportHash = static_cast<uint32>(TypeAndId);
		return FPackageImportReference(ImportedPackageIndex, ExportHash);
	}

	inline uint64 Value() const
	{
		return TypeAndId & IndexMask;
	}

	inline bool operator==(FPackageObjectIndex Other) const
	{
		return TypeAndId == Other.TypeAndId;
	}

	inline bool operator!=(FPackageObjectIndex Other) const
	{
		return TypeAndId != Other.TypeAndId;
	}

	friend FArchive& operator<<(FArchive& Ar, FPackageObjectIndex& Value)
	{
		Ar << Value.TypeAndId;
		return Ar;
	}

	inline friend uint32 GetTypeHash(const FPackageObjectIndex& Value)
	{
		return uint32(Value.TypeAndId);
	}
};

class FPublicExportKey
{
public:
	FPublicExportKey()
	{
	}

	bool IsNull() const
	{
		return GetExportHash() == 0;
	}

	FPackageId GetPackageId() const
	{
		return FPackageId::FromValue(uint64(PackageIdHigh) << 32 | PackageIdLow);
	}

	uint64 GetExportHash() const
	{
		return uint64(ExportHashHigh) << 32 | ExportHashLow;
	}

	inline bool operator==(const FPublicExportKey& Other) const
	{
		return GetExportHash() == Other.GetExportHash() &&
			GetPackageId() == Other.GetPackageId();
	}

	inline bool operator!=(const FPublicExportKey& Other) const
	{
		return GetExportHash() != Other.GetExportHash() ||
			GetPackageId() != Other.GetPackageId();
	}

	inline friend uint32 GetTypeHash(const FPublicExportKey& In)
	{
		return HashCombine(GetTypeHash(In.GetPackageId()), GetTypeHash(In.GetExportHash()));
	}

	static FPublicExportKey MakeKey(FPackageId PackageId, uint64 ExportHash)
	{
		check(PackageId.IsValid());
		check(ExportHash);
		uint64 PackageIdValue = PackageId.Value();
		return FPublicExportKey(uint32(PackageIdValue >> 32), uint32(PackageIdValue), uint32(ExportHash >> 32), uint32(ExportHash));
	}

	static FPublicExportKey FromPackageImport(FPackageObjectIndex ObjectIndex, const TArrayView<const FPackageId>& ImportedPackageIds, const TArrayView<const uint64>& ImportedPublicExportHashes)
	{
		check(ObjectIndex.IsPackageImport());
		FPackageImportReference PackageImportRef = ObjectIndex.ToPackageImportRef();
		FPackageId PackageId = ImportedPackageIds[PackageImportRef.GetImportedPackageIndex()];
		uint64 ExportHash = ImportedPublicExportHashes[PackageImportRef.GetImportedPublicExportHashIndex()];
		return MakeKey(PackageId, ExportHash);
	}

private:
	FPublicExportKey(uint32 InPackageIdHigh, uint32 InPackageIdLow, uint32 InExportHashHigh, uint32 InExportHashLow)
		: PackageIdHigh(InPackageIdHigh)
		, PackageIdLow(InPackageIdLow)
		, ExportHashHigh(InExportHashHigh)
		, ExportHashLow(InExportHashLow)
	{
	}

	uint32 PackageIdHigh = 0;
	uint32 PackageIdLow = 0;
	uint32 ExportHashHigh = 0;
	uint32 ExportHashLow = 0;
};

/**
 * Export filter flags.
 */
enum class EExportFilterFlags : uint8
{
	None,
	NotForClient,
	NotForServer
};
static const TCHAR* LexToString(EExportFilterFlags Flags)
{
	switch (Flags)
	{
		case EExportFilterFlags::NotForServer: return TEXT("NotForServer");
		case EExportFilterFlags::NotForClient: return TEXT("NotForClient");
		case EExportFilterFlags::None: return TEXT("None");
		default: checkNoEntry();
	}
	return TEXT("");
}

enum class EZenPackageVersion : uint32
{
	Initial,
	DataResourceTable,
	ImportedPackageNames,
	ExportDependencies,

	LatestPlusOne,
	Latest = LatestPlusOne - 1
};

struct FZenPackageVersioningInfo
{
	EZenPackageVersion ZenVersion;
	FPackageFileVersion PackageVersion;
	int32 LicenseeVersion;
	FCustomVersionContainer CustomVersions;

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FZenPackageVersioningInfo& ExportBundleEntry);
};

struct FZenPackageImportedPackageNamesContainer
{
	TArray<FName> Names;

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FZenPackageImportedPackageNamesContainer& Container);
};

/**
 * Package summary.
 */
struct FZenPackageSummary
{
	uint32 bHasVersioningInfo;
	uint32 HeaderSize;
	FMappedName Name;
	uint32 PackageFlags;
	uint32 _Unused; // Was CookedHeaderSize
	int32 ImportedPublicExportHashesOffset;
	int32 ImportMapOffset;
	int32 ExportMapOffset;
	int32 ExportBundleEntriesOffset;
	int32 DependencyBundleHeadersOffset;
	int32 DependencyBundleEntriesOffset;
	int32 ImportedPackageNamesOffset;
};

struct FZenPackageCellOffsets
{
	int32 CellImportMapOffset;
	int32 CellExportMapOffset;
};

/**
 * Export bundle entry.
 */
struct FExportBundleEntry
{
	enum EExportCommandType
	{
		ExportCommandType_Create,
		ExportCommandType_Serialize,
		ExportCommandType_Count
	};
	uint32 LocalExportIndex;
	uint32 CommandType;

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FExportBundleEntry& ExportBundleEntry);
};

struct FDependencyBundleEntry
{
	FPackageIndex LocalImportOrExportIndex;

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FDependencyBundleEntry& DependencyBundleEntry);
};

struct FDependencyBundleHeader
{
	int32 FirstEntryIndex;
	uint32 EntryCount[FExportBundleEntry::ExportCommandType_Count][FExportBundleEntry::ExportCommandType_Count];

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FDependencyBundleHeader& DependencyBundleHeader);
};

struct FScriptObjectEntry
{
	union 
	{
		FMappedName Mapped;
		FMinimalName ObjectName;
	};
	FPackageObjectIndex GlobalIndex;
	FPackageObjectIndex OuterIndex;
	FPackageObjectIndex CDOClassIndex;

	FScriptObjectEntry() {}

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FScriptObjectEntry& ScriptObjectEntry);
};
// The sizeof FMinimalName may be variable but FMappedName should always be larger, so FScriptObjectEntry has a fixed size.
static_assert(sizeof(FMappedName) >= sizeof(FMinimalName));

/**
 * Export map entry.
 */
struct FExportMapEntry
{
	uint64 CookedSerialOffset = 0; // Offset from start of exports data (HeaderSize + CookedSerialOffset gives actual offset in iobuffer)
	uint64 CookedSerialSize = 0;
	FMappedName ObjectName;
	FPackageObjectIndex OuterIndex;
	FPackageObjectIndex ClassIndex;
	FPackageObjectIndex SuperIndex;
	FPackageObjectIndex TemplateIndex;
	uint64 PublicExportHash = 0;
	EObjectFlags ObjectFlags = EObjectFlags::RF_NoFlags;
	EExportFilterFlags FilterFlags = EExportFilterFlags::None;
	uint8 Pad[3] = {};

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FExportMapEntry& ExportMapEntry);
};
struct FCellExportMapEntry
{
	uint64 CookedSerialOffset = 0;
	uint64 CookedSerialLayoutSize = 0;
	uint64 CookedSerialSize = 0;
	FMappedName CppClassInfo;
	uint64 PublicExportHash = 0;

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FCellExportMapEntry& CellExportMapEntry);
};

struct FBulkDataMapEntry
{
	int64 SerialOffset = 0;
	int64 DuplicateSerialOffset = 0;
	int64 SerialSize = 0;
	uint32 Flags = 0;
	FBulkDataCookedIndex CookedIndex;
	uint8 Pad[3] = { 0, 0, 0 };
	
	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FBulkDataMapEntry& BulkDataEntry);
};

// If we change the size of FBulkDataCookedIndex we will need to update FBulkDataMapEntry
static_assert(sizeof(FBulkDataCookedIndex) == sizeof(uint8));

// We don't want to grow the size of FBulkDataMapEntry accidently
static_assert(sizeof(FBulkDataMapEntry) <= 32, "The memory layout of FBulkDataMapEntry now exceeds 32 bytes, was this intended?");

struct FRuntimeScriptPackages
{
	TArray<UPackage*> Script;
	TArray<UPackage*> VerseVNI;
};
COREUOBJECT_API void FindAllRuntimeScriptPackages(FRuntimeScriptPackages& OutPackages);

/**
 * Creates a new instance of the AsyncPackageLoader #2.
 *
 * @param InIoDispatcher				The I/O dispatcher.
 *
 * @return The async package loader.
 */
IAsyncPackageLoader* MakeAsyncPackageLoader2(FIoDispatcher& InIoDispatcher);
