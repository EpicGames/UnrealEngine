// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FName;
class UObject;
enum class ETypeConstructPhase : uint8;
namespace Verse { struct VPackage; }

// used by the EDL at boot time to coordinate loading with what is going on with the deferred registration stuff
enum class ENotifyRegistrationType
{
	NRT_Class,
	NRT_ClassCDO,
	NRT_Struct,
	NRT_Enum,
	NRT_Package,
	NRT_NoExportObject,
};

enum class ENotifyRegistrationPhase
{
	NRP_Added,
	NRP_Started,
	NRP_Finished,
};

COREUOBJECT_API void NotifyRegistrationEvent(FName PackageName, FName Name, ENotifyRegistrationType NotifyRegistrationType, ENotifyRegistrationPhase NotifyRegistrationPhase, UObject *(*InRegister)(ETypeConstructPhase) = nullptr, bool InbDynamic = false, UObject* FinishedObject = nullptr);
void NotifyScriptVersePackage(Verse::VPackage* Package);
COREUOBJECT_API void NotifyRegistrationComplete();
