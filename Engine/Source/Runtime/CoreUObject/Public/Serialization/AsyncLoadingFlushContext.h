// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "Templates/UnrealTemplate.h"

DECLARE_LOG_CATEGORY_EXTERN(AsyncLoadingFlush, Log, All);

DECLARE_DELEGATE(FOnAsyncLoadingFlushComplete);

/**
 * Flush the async loader in a non-blocking manner.
 */
class FAsyncLoadingFlushContext final : FNoncopyable
{
public:
	COREUOBJECT_API FAsyncLoadingFlushContext(const FString & Context);
	COREUOBJECT_API FAsyncLoadingFlushContext(FString&& Context);
	COREUOBJECT_API ~FAsyncLoadingFlushContext();

	COREUOBJECT_API void Flush(const FOnAsyncLoadingFlushComplete& OnFlushComplete);

	int32 GetId() const { return Id; }

private:
	FAsyncLoadingFlushContext() = delete;

	void CleanupTickers();

	bool OnAsyncLoadingCheck(float DeltaTime);
	bool OnAsyncLoadingWarn(float DeltaTime);

	/** How often to warn while waiting for async loading to complete. */
	static constexpr float LoadingFlushWarnIntervalSeconds = 10.f;

	/** Context for logging */
	FString Context;

	/** Unique id for context. */
	int32 Id;

	/** Delegate to be fired on flush. */
	FOnAsyncLoadingFlushComplete OnFlushCompleteDelegate;

	/** Record event timing for logging. */
	double StartTime = 0;

	/** Tick delegates. */
	FTSTicker::FDelegateHandle LoadingCompleteCheckDelegateHandle;
	FTSTicker::FDelegateHandle WaitingWarnDelegateHandle;

	/** Increment ids so each instance has a unique id. */
	static COREUOBJECT_API int32 NextId;
};
