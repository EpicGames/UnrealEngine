// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/PackagePath.h"
#include "Stats/Stats.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectGlobals.h"

class FLinkerInstancingContext;
class FPackagePath;
class FThreadSafeCounter;
class UPackage;
struct FGuid;
template <typename FuncType> class TFunctionRef;
namespace Verse { struct VPackage; }

DECLARE_STATS_GROUP_VERBOSE(TEXT("Async Load"), STATGROUP_AsyncLoad, STATCAT_Advanced);
DECLARE_CYCLE_STAT(TEXT("Async Loading Time"),STAT_AsyncLoadingTime,STATGROUP_AsyncLoad);

DECLARE_STATS_GROUP(TEXT("Async Load Game Thread"), STATGROUP_AsyncLoadGameThread, STATCAT_Advanced);

DECLARE_CYCLE_STAT(TEXT("PostLoadObjects GT"), STAT_FAsyncPackage_PostLoadObjectsGameThread, STATGROUP_AsyncLoadGameThread);
DECLARE_CYCLE_STAT(TEXT("TickAsyncLoading GT"), STAT_FAsyncPackage_TickAsyncLoadingGameThread, STATGROUP_AsyncLoadGameThread);
DECLARE_CYCLE_STAT(TEXT("Flush Async Loading GT"), STAT_FAsyncPackage_FlushAsyncLoadingGameThread, STATGROUP_AsyncLoadGameThread);
DECLARE_CYCLE_STAT(TEXT("Flush All Async Loads GT"), STAT_FAsyncPackage_FlushAllAsyncLoadingGameThread, STATGROUP_AsyncLoadGameThread);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("PostLoadInstances GT"), STAT_FAsyncPackage_PostLoadInstancesGameThread, STATGROUP_AsyncLoadGameThread, EStatFlags::Verbose);
DECLARE_CYCLE_STAT_WITH_FLAGS(TEXT("CreateClusters GT"), STAT_FAsyncPackage_CreateClustersGameThread, STATGROUP_AsyncLoadGameThread, EStatFlags::Verbose);

enum class ENotifyRegistrationType;
enum class ENotifyRegistrationPhase;

extern const FName PrestreamPackageClassNameLoad;

/** Returns true if we're inside a FGCScopeGuard */
extern bool IsGarbageCollectionLocked();

UE_DEPRECATED(5.8, "This method will be removed.")
bool IsNativeCodePackage(UPackage* Package);

/** Checks if the object can have PostLoad called on the Async Loading Thread */
bool CanPostLoadOnAsyncLoadingThread(UObject* Object);

template <typename T>
UE_DEPRECATED(5.8, "This method will be removed.")
void ClearFlagsAndDissolveClustersFromLoadedObjects(T& LoadedObjects)
{
	const EObjectFlags ObjectLoadFlags = RF_NeedLoad | RF_NeedPostLoad | RF_NeedPostLoadSubobjects | RF_WasLoaded;
	for (UObject* ObjRef : LoadedObjects)
	{
		if (ObjRef)
		{
			ObjRef->AtomicallyClearFlags(ObjectLoadFlags);
			if (ObjRef->HasAnyInternalFlags(EInternalObjectFlags::ClusterRoot))
			{
				GUObjectClusters.DissolveCluster(ObjRef);
			}
		}
	}
}

class FPackageIndex;
class IAsyncPackageLoader;
class LinkerInstancingContext;

/** Structure that holds the async loading thread ini settings */
struct FAsyncLoadingThreadSettings
{
	bool bAsyncLoadingThreadEnabled;
	bool bAsyncPostLoadEnabled;
	bool bLooseFileLoadingEnabled;

	FAsyncLoadingThreadSettings();

	/** Gets the ALT settigns from ini (or command line). */
	COREUOBJECT_API static FAsyncLoadingThreadSettings& Get();
};

/**
 * Asynchronous package loader interface.
 */
class IAsyncPackageLoader
{
public:
	virtual ~IAsyncPackageLoader() = default;

	/**
	 * Initialize loading.
	 */
	virtual void InitializeLoading() = 0;

	/**
	 * Shut down loading.
	 */
	virtual void ShutdownLoading() = 0;

	virtual void StartThread() = 0;

	UE_DEPRECATED(5.8, "This method will be removed.")
	virtual bool ShouldAlwaysLoadPackageAsync(const FPackagePath& PackagePath) { return true; }

	/**
	 * Asynchronously load a package.
	 *
	 * @param	PackagePath				PackagePath to load. Must be a mounted path. The package is created if it does not already exist.
	 * @param	CustomPackageName		If not none, this is the name of the package to load into (and create if not yet existing). If none, the name is take from PackagePath.
	 * @param	InCompletionDelegate	Delegate to be invoked when the packages has finished streaming
	 * @param	InPackageFlags			Package flags used to construct loaded package in memory
	 * @param	InPIEInstanceID			Play in Editor instance ID
	 * @param	InPackagePriority		Loading priority
	 * @param   InstancingContext		Additional context to map object names to their instanced counterpart when loading an instanced package
	 * @param	LoadFlags				Flags controlling loading behavior, from the ELoadFlags enum
	 * @return Unique ID associated with this load request (the same package can be associated with multiple IDs).
	 */
	virtual int32 LoadPackage(
			const FPackagePath& PackagePath,
			FName CustomPackageName,
			FLoadPackageAsyncDelegate InCompletionDelegate,
			EPackageFlags InPackageFlags,
			int32 InPIEInstanceID,
			int32 InPackagePriority,
			const FLinkerInstancingContext* InInstancingContext,
			uint32 InLoadFlags) = 0;

	/**
	 * Asynchronously load a package.
	 *
	 * @param  PackagePath         PackagePath to load. Must be a mounted path. The package is created if it does not already exist.
	 * @param  OptionalParams      Struct containing all the parameters required to load the package.
	 * @return Unique ID associated with this load request (the same package can be associated with multiple IDs).
	 */
	virtual int32 LoadPackage(const FPackagePath& PackagePath, FLoadPackageAsyncOptionalParams OptionalParams)
	{
		checkf(false, TEXT("Not Implemented"));
		return INDEX_NONE;
	};

	/**
	 * Process all currently loading package requests.
	 *
	 * @param bUseTimeLimit			Whether to use time limit or not
	 * @param bUseFullTimeLimit	
	 * @param TimeLimit				Time limit
	 */
	virtual EAsyncPackageState::Type ProcessLoading(bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit) = 0;
	
	/**
	 * Process all loading package requests until completion predicate is satisfied.
	 *
	 * @param CompletionPredicate		Completion predicate
	 * @param TimeLimit					Time limit
	 */
	virtual EAsyncPackageState::Type ProcessLoadingUntilComplete(TFunctionRef<bool()> CompletionPredicate, double TimeLimit) = 0;

	/**
	* Cancels streaming.
	*
	* Note: Called from Game Thread.
	*/
	virtual void CancelLoading() = 0;

	/**
	* Suspends async loading thread
	*
	* Note: Called from Game Thread.
	*/
	virtual void SuspendLoading() = 0;

	/**
	* Resumes async loading thread
	*
	* Note: Called from Game Thread.
	*/
	virtual void ResumeLoading() = 0;

	/**
	 * Flush pending loading request(s).
	 *
	 * Note: Called from Game Thread.
	 */
	virtual void FlushLoading(TConstArrayView<int32> RequestIds) = 0;

	/**
	 *	Returns the number of queued packages.
	 */
	virtual int32 GetNumQueuedPackages() = 0;

	/**
	 *	Returns the number of loading packages.
	 */
	virtual int32 GetNumAsyncPackages() = 0;

	/** 
	 * [GAME THREAD] Gets the load percentage of the specified package
	 * @param PackageName Name of the package to return async load percentage for
	 * @return Percentage (0-100) of the async package load or -1 of package has not been found
	 */
	virtual float GetAsyncLoadPercentage(const FName& PackageName) = 0;

	/**
	 *	Returns whether the package loader is suspended or not.
	 */
	virtual bool IsAsyncLoadingSuspended() = 0;

	/**
	 *	Returns whether in package loader background thread or not.
	 */
	virtual bool IsInAsyncLoadThread() = 0;

	/**
	 * Returns whether loading packages with multiple threads.
	 * Note: GIsInitialLoad guards the package loader from creating background threads too early.
	 *
	 */
	virtual bool IsMultithreaded() = 0;

	/**
	 * Returns whether packages are currently being loaded on a background thread.
	 * Note: GIsInitialLoad guards the package loader from creating background threads too early.
	 *
	 */
	virtual bool IsAsyncLoadingPackages() = 0;

	virtual void NotifyConstructedDuringAsyncLoading(UObject* Object, bool bSubObject) = 0;

	virtual void NotifyUnreachableObjects(const TArrayView<FUObjectItem*>& UnreachableObjects) = 0;

	virtual void NotifyRegistrationEvent(FName PackageName, FName Name, ENotifyRegistrationType NotifyRegistrationType, ENotifyRegistrationPhase NotifyRegistrationPhase, FTypeConstructFunc* InRegister, bool InbDynamic, UObject* FinishedObject) = 0;

	/// Called for built in packages (currently only VVMIntrinsics.cpp) to allow them to be imported.
	virtual void NotifyScriptVersePackage(Verse::VPackage* Package) {}

	virtual void NotifyRegistrationComplete() = 0;

	virtual ELoaderType GetLoaderType() const = 0;

protected:
	static int32 GetNextRequestId();

private:
	static FThreadSafeCounter NextPackageRequestId;
};

// Stats for ChartCreation.cpp
extern COREUOBJECT_API double GFlushAsyncLoadingTime;
extern COREUOBJECT_API uint32 GFlushAsyncLoadingCount;
extern COREUOBJECT_API uint32 GSyncLoadCount;

extern COREUOBJECT_API void ResetAsyncLoadingStats();

// Time limit
extern COREUOBJECT_API int32 GWarnIfTimeLimitExceeded;
extern COREUOBJECT_API float GTimeLimitExceededMultiplier;
extern COREUOBJECT_API float GTimeLimitExceededMinTime;

void IsTimeLimitExceededPrint(
	double InTickStartTime,
	double CurrentTime,
	double LastTestTime,
	double InTimeLimit,
	const TCHAR* InLastTypeOfWorkPerformed = nullptr,
	UObject* InLastObjectWorkWasPerformedOn = nullptr);

