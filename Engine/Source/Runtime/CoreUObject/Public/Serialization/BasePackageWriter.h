// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/PackageWriter.h"

/** Shared Implementation for IPackageWriters that are not ICookedPackageWriters. */
class FBasePackageWriter : public IPackageWriter
{
public:
	/** Create the FLargeMemoryWriter to which the Header and Exports are written during the save. */
	COREUOBJECT_API virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerArchive(FName PackageName,
		UObject* Asset, uint16 MultiOutputIndex) override;

	/** Returns an archive to be used when serializing exports. */
	COREUOBJECT_API virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerExportsArchive(FName PackageName,
		UObject* Asset, uint16 MultiOutputIndex) override;
};

/** Shared Implementation for ICookedPackageWriters. */
class FBaseCookedPackageWriter : public ICookedPackageWriter
{
public:
	/** Create the FLargeMemoryWriter to which the Header and Exports are written during the save. */
	COREUOBJECT_API virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerArchive(FName PackageName,
		UObject* Asset, uint16 MultiOutputIndex) override;

	/** Returns an archive to be used when serializing exports. */
	COREUOBJECT_API virtual TUniquePtr<FLargeMemoryWriter> CreateLinkerExportsArchive(FName PackageName,
		UObject* Asset, uint16 MultiOutputIndex) override;
};

// Shared helper functions for PackageWriters and CookedPackageWriters
namespace UE::PackageWriter::Private
{

#if WITH_EDITOR
/**
 * Synchronously write if requested and hash if requested the data in WriteFileData to the filename in WriteFileData,
 * based on the given WriteOptions and WriteFileData's filetype flags.
 * This function is a cooker implementation detail and might be changed without deprecation in a future version.
 */
COREUOBJECT_API void HashAndWrite(const UE::PackageWriter::Private::FWriteFileData& WriteFileData,
	FMD5& AccumulatedHash, const TRefCountPtr<FPackageHashes>& PackageHashes,
	IPackageWriter::EWriteOptions WriteOptions);
#endif

}
