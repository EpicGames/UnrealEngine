// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncFileHandle.h"
#include "BulkDataBuffer.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"
#include "Containers/StringView.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/MemoryBase.h"
#include "IO/IoChunkId.h"
#include "IO/IoDispatcherPriority.h"
#include "IO/PackageId.h"
#include "Math/NumericLimits.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CompressionFlags.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/OptionalFwd.h"
#include "Misc/PackagePath.h"
#include "Misc/PackageSegment.h"
#include "Misc/SourceLocation.h"
#include "Serialization/Archive.h"
#include "Serialization/BulkDataCookedIndex.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/FileRegions.h"
#include "Templates/Function.h"
#include "Templates/IsPODType.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "Templates/PimplPtr.h"
#include "UObject/NameTypes.h"

class FIoBuffer;
class FLinkerLoad;
class FOutputDevice;
class IAsyncReadFileHandle;
class IAsyncReadRequest;
class UObject;
struct FTimespan;
struct FIoOffsetAndLength;
namespace UE { namespace Serialization { class FEditorBulkData; } }
enum class EFileRegionType : uint8;

#if WITH_EDITOR == 0 && WITH_EDITORONLY_DATA == 0
	#define USE_RUNTIME_BULKDATA 1
#else
	#define USE_RUNTIME_BULKDATA 0
#endif

// When set to 1 attempting to reload inline data will fail with the old loader in the same way that it fails in
// the new loader to keep the results consistent.
#define UE_KEEP_INLINE_RELOADING_CONSISTENT 0

class IMappedFileHandle;
class IMappedFileRegion;

/*-----------------------------------------------------------------------------
	I/O filename helper(s)
-----------------------------------------------------------------------------*/

/** A loose hash value that can be created from either a filenames or a FIoChunkId */
using FIoFilenameHash = uint32;
inline const FIoFilenameHash INVALID_IO_FILENAME_HASH = 0;
/** Helpers to create the hash from a filename. Returns IOFILENAMEHASH_NONE if and only if the filename is empty. */
COREUOBJECT_API FIoFilenameHash MakeIoFilenameHash(const FString& Filename);
/** Helpers to create the hash from a FPackagePath. Returns IOFILENAMEHASH_NONE if and only if the PackagePath is empty. */
COREUOBJECT_API FIoFilenameHash MakeIoFilenameHash(const FPackagePath& Filename);
/** Helpers to create the hash from a chunk id. Returns IOFILENAMEHASH_NONE if and only if the chunk id is invalid. */
COREUOBJECT_API FIoFilenameHash MakeIoFilenameHash(const FIoChunkId& ChunkID);

/**
 * Represents an IO request from the BulkData streaming API.
 *
 * It functions pretty much the same as IAsyncReadRequest expect that it also holds
 * the file handle as well.
 */
class IBulkDataIORequest
{
public:
	virtual ~IBulkDataIORequest() = default;

	virtual bool PollCompletion() const = 0;
	virtual bool WaitCompletion(float TimeLimitSeconds = 0.0f) = 0;

	virtual uint8* GetReadResults() = 0;
	virtual int64 GetSize() const = 0;

	virtual void Cancel() = 0;
};

/**
 * Flags serialized with the bulk data.
 */
enum EBulkDataFlags : uint32
{
	/** Empty flag set. */
	BULKDATA_None = 0,
	/** Indicates that when the payload was saved/cooked as part of a package was stored at the end of the package file rather than inline. */
	BULKDATA_PayloadAtEndOfFile = 1 << 0,
	/** If set, payload should be [un]compressed using ZLIB during serialization. */
	BULKDATA_SerializeCompressedZLIB = 1 << 1,
	/** Force usage of SerializeElement over bulk serialization. */
	BULKDATA_ForceSingleElementSerialization = 1 << 2,
	/** When set the payload will be unloaded from memory after it has been accessed via ::Lock/::Unlock. Other forms of accessing the payload are unaffected by this flag. */
	BULKDATA_SingleUse = 1 << 3,
	/** DEPRECATED */
	BULKDATA_Unused UE_DEPRECATED(5.3, "This feature is being removed") = 1 << 5,
	/** Should be set before saing/cooking to force the internal payload to be stored inline */
	BULKDATA_ForceInlinePayload = 1 << 6,
	/** @see BULKDATA_SerializeCompressedZLIB */
	BULKDATA_SerializeCompressed = (BULKDATA_SerializeCompressedZLIB),
	/** Forces the payload to be always streamed, regardless of its size. */
	BULKDATA_ForceStreamPayload UE_DEPRECATED(5.3, "This flag has had no purpose for sometime") = 1 << 7,
	/** Set when the bulkdata saved/cooked as part of a package indicates that the payload is stored in its own file (.ubulk, .uptnl or .m.ubulk depending on other flags) */
	BULKDATA_PayloadInSeparateFile = 1 << 8,
	BULKDATA_PayloadInSeperateFile UE_DEPRECATED(5.7, "This flag has been renamed to BULKDATA_PayloadInSeparateFile") = BULKDATA_PayloadInSeparateFile,
	/** DEPRECATED */
	BULKDATA_SerializeCompressedBitWindow UE_DEPRECATED(5.3, "This flag has had no purpose for sometime") = 1 << 9,
	/** Should be set before being cooked as part of a package to force the cooked payload to before stored at the end of the package file either than being inline*/
	BULKDATA_Force_NOT_InlinePayload = 1 << 10,
	/** 
	 * During cooking, this flag indicates that the the payload is optional at runtime and should be stored in an .uptnl file, unless the payload is also set to be inline in
	 * which case this flag is ignored. This flag will be preserved during cooking and and at runtime and can be used to identify where the payload should be loaded from.
	 */
	BULKDATA_OptionalPayload = 1 << 11,
	/** 
	 * During cooking this flag indicates that the payload should work with memory mapping at runtime if the target cooking platform supports it so the payload should be stored in
	 * a .m.ubulk file, unless the payload is also set to be inline in which case this flag is ignored. This flag will be preserved during cooking and and at runtime and can be
	 * used to identify where the payload should be loaded from.
	 */
	BULKDATA_MemoryMappedPayload = 1 << 12,
	/** Set during serialization to indicate if the size and offset values were serialized as int64 types rather than the default int32 */
	BULKDATA_Size64Bit = 1 << 13,
	/** During cooking this flag indicates that although the payload is NOT optional it should be stored in both the .ubulk file and the .uptnl file as duplicate non-optional data,
	 * unless the payload is also set to be inline in which case this flag is ignored. This flag will be preserved during cooking and and at runtime and can be used to identify
	 * where the payload should be loaded from.
	 */
	BULKDATA_DuplicateNonOptionalPayload = 1 << 14,
	/** DEPRECATED */
	BULKDATA_BadDataVersion UE_DEPRECATED(5.3, "This flag has had no purpose for sometime") = 1 << 15,
	/** Set during saving and indicates that the payload offset value is correct and does not need adjusting via an additional offset stored in the FLinker (older legacy behavior) */
	BULKDATA_NoOffsetFixUp = 1 << 16,
	/**
	 * INTERNAL SET ONLY - callers of bulkdata should not set this flag on the bulk data
	 * If set, payload is stored in the workspace domain version of the file.
	 */
	BULKDATA_WorkspaceDomainPayload = 1 << 17,
	/**
	 * INTERNAL SET ONLY - callers of bulkdata should not set this flag on the bulk data
	 * If true, the BulkData can be loaded from its file at any time
	 */
	BULKDATA_LazyLoadable = 1 << 18,

	/* Runtime only flags below this point! Note that they take the high bits in reverse order! */

	/** Assigned at runtime to indicate that the BulkData should be using the IoDispatcher when loading, not filepaths. */
	BULKDATA_UsesIoDispatcher = 1u << 31u,
	/** Assigned at runtime to indicate that the BulkData allocation is a memory mapped region of a file and not raw data. */
	BULKDATA_DataIsMemoryMapped = 1 << 30,
	/** DEPRECATED */
	BULKDATA_HasAsyncReadPending UE_DEPRECATED(5.5, "This flag no longer does anything")  = 1 << 29,
	/** Assigned at runtime to indicate that the BulkData object should be considered for discard even if it cannot load from disk. */
	BULKDATA_AlwaysAllowDiscard = 1 << 28,
};

COREUOBJECT_API FStringBuilderBase& LexToString(EBulkDataFlags Flags, FStringBuilderBase& Sb);
COREUOBJECT_API FString LexToString(EBulkDataFlags Flags);

/** Returns the IoChunk type that we expect to be produced from a given EBulkDataFlags */
COREUOBJECT_API EIoChunkType GetIoChunkTypeFromFlags(EBulkDataFlags Flags);

/**
 * Allows FArchive to serialize EBulkDataFlags, this will not be required once EBulkDataFlags is promoted
 * to be a enum class.
 */
inline FArchive& operator<<(FArchive& Ar, EBulkDataFlags& Flags)
{
	Ar << (uint32&)Flags;
	return Ar;
}

/**
 * Enumeration for bulk data lock status.
 */
enum EBulkDataLockStatus
{
	/** Unlocked array				*/
	LOCKSTATUS_Unlocked = 0,
	/** Locked read-only			*/
	LOCKSTATUS_ReadOnlyLock = 1,
	/** Locked read-write-realloc	*/
	LOCKSTATUS_ReadWriteLock = 2,
};

/**
 * Enumeration for bulk data lock behavior
 */
enum EBulkDataLockFlags
{
	LOCK_READ_ONLY = 1,
	LOCK_READ_WRITE = 2,
};

/**
 * Callback to use when making streaming requests
 */
typedef TFunction<void(bool bWasCancelled, IBulkDataIORequest*)> FBulkDataIORequestCallBack;

/**
 * @documentation @todo documentation
 */
struct FOwnedBulkDataPtr
{
	explicit FOwnedBulkDataPtr(void* InAllocatedData)
		: AllocatedData(InAllocatedData)
		, MappedHandle(nullptr)
		, MappedRegion(nullptr)
	{
		
	}

	FOwnedBulkDataPtr(IMappedFileHandle* Handle, IMappedFileRegion* Region)
		: AllocatedData(nullptr)
		, MappedHandle(Handle)
		, MappedRegion(Region)
	{
		
	}

	COREUOBJECT_API ~FOwnedBulkDataPtr();
	COREUOBJECT_API const void* GetPointer();

	IMappedFileHandle* GetMappedHandle()
	{
		return MappedHandle;
	}
	IMappedFileRegion* GetMappedRegion()
	{
		return MappedRegion;
	}
	bool IsDataMemoryMapped() const
	{
		return MappedHandle != nullptr;
	}

	void RelinquishOwnership()
	{
		AllocatedData = nullptr;
		MappedHandle = nullptr;
		MappedRegion = nullptr;
	}

private:
	// hidden
	FOwnedBulkDataPtr() {}


	// if allocated memory was used, this will be non-null
	void* AllocatedData;
	
	// if memory mapped IO was used, these will be non-null
	IMappedFileHandle* MappedHandle;
	IMappedFileRegion* MappedRegion;
};

namespace UE::BulkData::Private
{

/**
 * Serialized bulk meta data.
 */
struct FBulkMetaResource
{
	/** Bulk data flags. */
	EBulkDataFlags Flags = BULKDATA_None;

	/** Number of bulk data elements. */
	int64 ElementCount = -1;

	/** Size on disk, can differ from size when compressed. */
	int64 SizeOnDisk = -1;
	
	/** Offset within the bulk data chunk. */
	int64 Offset = -1;
	
	/** Non-optional duplicate bulk data flags. */
	EBulkDataFlags DuplicateFlags = BULKDATA_None;
	
	/** Duplicate bulk data size on disk. */
	int64 DuplicateSizeOnDisk = -1;

	/** Offset within the optional bulk data chunk. */
	int64 DuplicateOffset = -1;

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FBulkMetaResource& BulkMeta);
};

/**
 * Bulk meta data, size, offset, flags and lock status packed in 16 bytes.
 *
 * Uses 5 bytes for size and offset.
 * [0 - 4][5 - 9][  10	][    11	][12    -   15]([16   -  23])
 * [Size][Offset][Unused][LockFlags][BulkDataFlags]([SizeOnDisk])
 */
class FBulkMetaData
{
public:
	/** 40 bits for max bulk data size. */
	static constexpr int64 MaxSize = 0xFFffFFffFF;
	/** 39 bits for max bulk data offset and 1 bit to indicate INDEX_NONE. */
	static constexpr int64 MaxOffset = 0xFFffFFffFE;

	FBulkMetaData()
	{
		SetOffset(-1);
#if !USE_RUNTIME_BULKDATA
		SetSizeOnDisk(-1);
#endif
	}

	explicit FBulkMetaData(EBulkDataFlags Flags)
		: FBulkMetaData()
	{
		SetFlags(Flags);
	}

	inline int64 GetSize() const
	{
		return static_cast<int64>(ReadUInt40(Data));
	}

	inline void SetSize(int64 Size)
	{
		check(Size <= MaxSize);
		WriteUInt40(Data, uint64(Size));
	}

	int64 GetSizeOnDisk() const
	{
#if USE_RUNTIME_BULKDATA
		return GetSize();
#else
		return static_cast<int64>(*reinterpret_cast<const int64*>(&Data[16]));
#endif
	}

	void SetSizeOnDisk(int64 SizeOnDisk)
	{
#if !USE_RUNTIME_BULKDATA
		*reinterpret_cast<int64*>(&Data[16]) = SizeOnDisk;
#endif // !USE_RUNTIME_BULKDATA
	}

	inline int64 GetOffset() const
	{
		const int64 Offset = static_cast<int64>(ReadUInt40(Data + 5));
		return Offset > MaxOffset ? INDEX_NONE : Offset;
	}

	inline void SetOffset(int64 Offset)
	{
		check(Offset <= MaxOffset);

		if (Offset < 0)
		{
			Offset = MAX_int64;
		}

		WriteUInt40(Data + 5, uint64(Offset));
	}

	FIoOffsetAndLength GetOffsetAndLength() const;

	EBulkDataLockStatus GetLockStatus() const
	{
		return static_cast<EBulkDataLockStatus>(Data[11]);
	}

	void SetLockStatus(EBulkDataLockStatus Status)
	{
		Data[11] = uint8(Status);
	}
	
	EBulkDataFlags GetFlags() const
	{
		return static_cast<EBulkDataFlags>(*reinterpret_cast<const uint32*>(&Data[12]));
	}

	void SetFlags(EBulkDataFlags Flags)
	{
		*reinterpret_cast<uint32*>(&Data[12]) = static_cast<uint32>(Flags);
	}

	void AddFlags(EBulkDataFlags Flags)
	{
		*reinterpret_cast<uint32*>(&Data[12]) |= static_cast<uint32>(Flags);
	}
	
	void ClearFlags(EBulkDataFlags Flags)
	{
		*reinterpret_cast<uint32*>(&Data[12]) &= ~static_cast<uint32>(Flags);
	}

	inline bool HasAnyFlags(EBulkDataFlags Flags) const
	{
		return (GetFlags() & Flags) != 0;
	}

	inline bool HasAllFlags(EBulkDataFlags Flags) const
	{
		return (GetFlags() & Flags) == Flags;
	}

	/** 
	 * Serializes FBulkMetaResource from the given archive and builds the returned FBulkMetaData from it. 
	 * The offset for duplicated data will also be returned
	 * 
	 * All fields of OutMetaData will be overwritten, including lock status.
	 */
	COREUOBJECT_API static bool FromSerialized(FArchive& Ar, int64 ElementSize, FBulkMetaData& OutMetaData, int64& OutDuplicateOffset);

	/** Serializes FBulkMetaResource from the given archive and builds the returned FBulkMetaData from it. */
	static bool FromSerialized(FArchive& Ar, int64 ElementSize, FBulkMetaData& OutMetaData)
	{
		int64 DuplicateOffset = INDEX_NONE;
		return FromSerialized(Ar, ElementSize, OutMetaData, DuplicateOffset);
	}

private:
	static uint64 ReadUInt40(const uint8* Memory)
	{
		return static_cast<int64>(uint64(Memory[0]) | uint64(Memory[1]) << 8 | uint64(Memory[2]) << 16 | uint64(Memory[3]) << 24 | uint64(Memory[4]) << 32);
	}

	static void WriteUInt40(uint8* Memory, uint64 Value)
	{
		Memory[0] = uint8(Value); Memory[1] = uint8(Value >> 8); Memory[2] = uint8(Value >> 16); Memory[3] = uint8(Value >> 24); Memory[4] = uint8(Value >> 32);
	}
#if USE_RUNTIME_BULKDATA
	uint8 Data[16] = {0};
#else
	uint8 Data[24] = {0};
#endif // USE_RUNTIME_BULKDATA
};

} // namespace UE::BulkData::Private

/** Parameters when serializing bulk data. */
struct FBulkDataSerializationParams
{
	/** The owner of the bulk data. */
	UObject* Owner = nullptr;
	/** Bulk data element size. */
	int32 ElementSize = 0;
	/** The region type. */
	EFileRegionType RegionType;
	/** Flag indicating whether to try to memory map the bulk data payload or not. */
	bool bAttemptMemoryMapping = false;
	/** The group to which the cooked version of the bulkdata payload should be assigned*/
	FBulkDataCookedIndex CookedIndex;
};

/**
 * @documentation @todo documentation
 */
class FBulkData
{
	// This struct represents an optional allocation.
	struct FAllocatedPtr
	{
		FAllocatedPtr() = default;
		~FAllocatedPtr() = default;

		inline bool IsLoaded() const
		{
			return Allocation.RawData != nullptr; // Doesn't matter which allocation we test
		}

		void Free(FBulkData* Owner);

		void* ReallocateData(FBulkData* Owner, int64 SizeInBytes);

		void SetData(FBulkData* Owner, void* Buffer);
		void SetMemoryMappedData(FBulkData* Owner, IMappedFileHandle* MappedHandle, IMappedFileRegion* MappedRegion);

		void* GetAllocationForWrite(const FBulkData* Owner) const;
		const void* GetAllocationReadOnly(const FBulkData* Owner) const;

		COREUOBJECT_API FOwnedBulkDataPtr* StealFileMapping(FBulkData* Owner);
		void Swap(FBulkData* Owner, void** DstBuffer);

	private:

		union FAllocation
		{
			/** Raw memory allocations, allocated via FMemory::Malloc/Realloc */
			void* RawData;
			/** Wrapper around memory mapped allocations allocated via new */
			FOwnedBulkDataPtr* MemoryMappedData;
		};

		FAllocation Allocation{ nullptr };
	};

public:
	friend class FLinkerLoad;
	friend class FLinkerSave;
	friend class FExportArchive;
	friend class UE::Serialization::FEditorBulkData; // To allow access to AttachedAr
	friend class FBulkDataBatchRequest;

	using BulkDataRangeArray = TArray<FBulkData*, TInlineAllocator<8>>;

	static constexpr SIZE_T MaxBulkDataSize = (1ull << 40) - 1;

	/*-----------------------------------------------------------------------------
		Constructors and operators
	-----------------------------------------------------------------------------*/

	/**
	 * Constructor, initializing all member variables.
	 */
	FBulkData() = default;

	/**
	 * Copy constructor. Use the common routine to perform the copy.
	 *
	 * @param Other the source array to copy
	 */
	COREUOBJECT_API FBulkData( const FBulkData& Other );

	/**
	 * Virtual destructor, free'ing allocated memory.
	 */
	COREUOBJECT_API ~FBulkData();

	/**
	 * Copies the source array into this one after detaching from archive.
	 *
	 * @param Other the source array to copy
	 */
	COREUOBJECT_API FBulkData& operator=( const FBulkData& Other );

	/*-----------------------------------------------------------------------------
		Static functions.
	-----------------------------------------------------------------------------*/

	/**
	 * Dumps detailed information of bulk data usage.
	 *
	 * @param Log FOutputDevice to use for logging
	 */
	static COREUOBJECT_API void DumpBulkDataUsage( FOutputDevice& Log );

	/*-----------------------------------------------------------------------------
		Accessors
	-----------------------------------------------------------------------------*/

	/**
	 * Returns the size of the bulk data in bytes.
	 *
	 * @return Size of the bulk data in bytes
	 */
	COREUOBJECT_API int64 GetBulkDataSize() const;
	/**
	 * Returns the size of the bulk data on disk. This can differ from GetBulkDataSize if
	 * BULKDATA_SerializeCompressed is set.
	 *
	 * @return Size of the bulk data on disk or INDEX_NONE in case there's no association
	 */
	COREUOBJECT_API int64 GetBulkDataSizeOnDisk() const;
	/**
	 * Returns the offset into the file the bulk data is located at.
	 *
	 * @return Offset into the file or INDEX_NONE in case there is no association
	 */
	COREUOBJECT_API int64 GetBulkDataOffsetInFile() const;
	/**
	 * Returns whether the bulk data is stored compressed on disk.
	 *
	 * @return true if data is compressed on disk, false otherwise
	 */
	COREUOBJECT_API bool IsStoredCompressedOnDisk() const;

	/**
	 * Returns true if the data can be loaded from disk.
	 */
	COREUOBJECT_API bool CanLoadFromDisk() const;
	
	/**
	 * Returns true if the data references a file that currently exists and can be referenced by the file system.
	 */
	COREUOBJECT_API bool DoesExist() const;

	/**
	 * Returns flags usable to decompress the bulk data
	 * 
	 * @return NAME_None if the data was not compressed on disk, or valid format to pass to FCompression::UncompressMemory for this data
	 */
	COREUOBJECT_API FName GetDecompressionFormat() const;

	/**
	 * Returns whether the bulk data is currently loaded and resident in memory.
	 *
	 * @return true if bulk data is loaded, false otherwise
	 */
	COREUOBJECT_API bool IsBulkDataLoaded() const;


	/**
	* Returns whether this bulk data represents optional data or not
	* @return true if BULKDATA_OptionalPayload is set
	*/
	bool IsOptional() const
	{
		return (GetBulkDataFlags() & BULKDATA_OptionalPayload) != 0;
	}

	/**
	* Returns whether this bulk data is currently stored inline or not
	* @return true if BULKDATA_PayloadAtEndOfFile is not set
	*/
	bool IsInlined() const
	{
		return   (GetBulkDataFlags() & BULKDATA_PayloadAtEndOfFile) == 0;
	}

	/**
	* Returns whether this bulk data is currently stored in it's own file or not
	* @return true if BULKDATA_PayloadInSeparateFile is set
	*/
	bool IsInSeparateFile() const
	{
		return	(GetBulkDataFlags() & BULKDATA_PayloadInSeparateFile) != 0;
	}

	/**
	 * Returns whether this bulk data is stored in a PackageExternalResource rather than in
	 * a neighboring segment of its owner's PackagePath. 
	 */
	bool IsInExternalResource() const
	{
		return IsInSeparateFile() && (GetBulkDataFlags() & BULKDATA_WorkspaceDomainPayload);
	}

	/**
	* Returns whether this bulk data is accessed via the IoDispatcher or not.
	* @return false as the old BulkData API does not support it
	*/
	bool IsUsingIODispatcher() const
	{
		return (GetBulkDataFlags() & BULKDATA_UsesIoDispatcher) != 0;
	}

	/**
	 * Returns whether this bulk data is memory mapped or not.
	 */
	bool IsDataMemoryMapped() const
	{
		return (GetBulkDataFlags() & BULKDATA_DataIsMemoryMapped) != 0;
	}

	/**
	 * Returns whether to deallocate the bulk data after lock
	 */
	bool IsSingleUse() const
	{
		return (GetBulkDataFlags() & BULKDATA_SingleUse) != 0;
	}
	
	/**
	* Returns whether this bulk data represents duplicate non-optional data or not
	* @return true if BULKDATA_DuplicateNonOptionalPayload is set
	*/
	bool IsDuplicateNonOptional() const
	{
		return (GetBulkDataFlags() & BULKDATA_DuplicateNonOptionalPayload) != 0;
	}

	/**
	 * Enables the given flags without affecting any previously set flags.
	 *
	 * @param BulkDataFlagsToSet	Bulk data flags to set
	 */
	COREUOBJECT_API void SetBulkDataFlags( uint32 BulkDataFlagsToSet );

	/**
	 * Enable the given flags and disable all other flags.
	 * This can be used with ::GetBulkDataFlags() to effectively
	 * reset the flags to a previous state.
	 *
	 * @param BulkDataFlagsToSet	Bulk data flags to set
	 */
	COREUOBJECT_API void ResetBulkDataFlags(uint32 BulkDataFlagsToSet);

	/**
	* Gets the current bulk data flags.
	*
	* @return Bulk data flags currently set
	*/
	COREUOBJECT_API uint32 GetBulkDataFlags() const;

#if WITH_EDITOR
	COREUOBJECT_API void SetCookedIndex(FBulkDataCookedIndex Index);
#endif //WITH_EDITOR
	/**
	* Gets the current bulk data alignment.
	*
	* @return Bulk data alignment currently set
	*/
	COREUOBJECT_API uint32 GetBulkDataAlignment() const;

	/**
	 * Clears the passed in bulk data flags.
	 *
	 * @param BulkDataFlagsToClear	Bulk data flags to clear
	 */
	COREUOBJECT_API void ClearBulkDataFlags( uint32 BulkDataFlagsToClear );

	/** 
	 * Returns the io filename hash associated with this bulk data.
	 *
	 * @return Hash or INVALID_IO_FILENAME_HASH if invalid.
	 **/
	FIoFilenameHash GetIoFilenameHash() const { return MakeIoFilenameHash(BulkChunkId); }
	
	/** Returns a FIoChunkId for the bulkdata payload, this will be invalid if the bulkdata is not stored in the IoStore */
	COREUOBJECT_API FIoChunkId CreateChunkId() const;

	/** Returns a string representing the bulk data for debugging purposes. */
	COREUOBJECT_API FString GetDebugName() const;

	/*-----------------------------------------------------------------------------
		Data retrieval and manipulation.
	-----------------------------------------------------------------------------*/

	/**
	 * Retrieves a copy of the bulk data.
	 *
	 * @param Dest [in/out] Pointer to pointer going to hold copy, can point to NULL pointer in which case memory is allocated
	 * @param bDiscardInternalCopy Whether to discard/ free the potentially internally allocated copy of the data
	 */
	COREUOBJECT_API void GetCopy( void** Dest, bool bDiscardInternalCopy = true );

	/**
	 * Locks the bulk data and returns a pointer to it.
	 *
	 * @param	LockFlags	Flags determining lock behavior
	 */
	COREUOBJECT_API void* Lock(uint32 LockFlags, UE::FSourceLocation Location = UE::FSourceLocation::Current());

	/**
	 * Locks the bulk data and returns a read-only pointer to it.
	 * This variant can be called on a const bulkdata
	 */
	COREUOBJECT_API const void* LockReadOnly(UE::FSourceLocation Location = UE::FSourceLocation::Current()) const;

	/**
	 * Change size of locked bulk data. Only valid if locked via read-write lock.
	 *
	 * @param ElementCount	Number of elements to allocate.
	 * @param ElementSize	Size in bytes of the individual element.
	 */
	COREUOBJECT_API void* Realloc(int64 ElementCount, int64 ElementSize);

	/** 
	 * Unlocks bulk data after which point the pointer returned by Lock no longer is valid.
	 */
	COREUOBJECT_API void Unlock() const;

	/** 
	 * Checks if this bulk is locked
	 */
	bool IsLocked() const { return BulkMeta.GetLockStatus() != LOCKSTATUS_Unlocked; }
	
	/** 
	 * Checks if this bulk is unlocked
	 */
	bool IsUnlocked() const { return BulkMeta.GetLockStatus() == LOCKSTATUS_Unlocked; }

	/**
	 * Clears/removes any currently allocated data payload and resets element count to 0.
	 * 
	 * Note that once this has been called, the bulkdata object will no longer be able to reload
	 * it's payload from disk!
	 */
	COREUOBJECT_API void RemoveBulkData();

#if !USE_RUNTIME_BULKDATA
	/**
	 * Load the bulk data using a file reader. Works even when no archive is attached to the bulk data..
  	 * @return Whether the operation succeeded.
	 */
	COREUOBJECT_API bool LoadBulkDataWithFileReader();

	/**
	 * Test if it is possible to load the bulk data using a file reader, even when no archive is attached to the bulk data.
	 * @return Whether the operation is allowed.
	 */
	COREUOBJECT_API bool CanLoadBulkDataWithFileReader() const;
#endif // USE_RUNTIME_BULKDATA

	/**
	 * Forces the bulk data to be resident in memory and detaches the archive.
	 */
	COREUOBJECT_API void ForceBulkDataResident();
	
	/**
	 * Sets whether we should store the data compressed on disk.
	 *
	 * @param CompressionFormat		The format to use for compression e.g. NAME_None or NAME_Oodle.
	 * 
	 * This should likely not be used - bulk data that gets deployed should be compressed via UnrealPak so that
	 * it can leverage hardware compression as available, and editor bulk data is compressed by default and should use 
	 * FEditorBulkData::SetCompressionOptions to control that if necessary.
	 */
	COREUOBJECT_API void StoreCompressedOnDisk( FName CompressionFormat );

	/**
	 * Deallocates bulk data without detaching the archive, so that further bulk data accesses require a reload.
	 * Only supported in editor builds.
	 *
	 * @return Whether the operation succeeded.
	 */
	COREUOBJECT_API bool UnloadBulkData();

	/*-----------------------------------------------------------------------------
		Serialization.
	-----------------------------------------------------------------------------*/

	using FSerializeBulkDataElements = TFunction<void(FArchive&, void*, int64, EBulkDataFlags)>;
	
	/**
	 * Serialize function used to serialize this bulk data structure.
	 *
	 * @param Ar	Archive to serialize with
	 * @param Owner	Object owning the bulk data
	 * @param Idx	Index of bulk data item being serialized
	 * @param bAttemptFileMapping	If true, attempt to map this instead of loading it into malloc'ed memory
	 * @param FileRegionType	When cooking, a hint describing the type of data, used by some platforms to improve compression ratios
	 */
	COREUOBJECT_API void Serialize(FArchive& Ar, UObject* Owner, bool bAttemptFileMapping, int32 ElementSize, EFileRegionType FileRegionType);
	
	/**
	 * Serialize just the bulk data portion to/ from the passed in memory.
	 *
	 * @param	Ar					Archive to serialize with
	 * @param	Data				Memory to serialize either to or from
	 * @param	InBulkDataFlags		Flags describing how the data was/shouldbe serialized
	 */
	void SerializeBulkData(FArchive& Ar, void* Data, EBulkDataFlags InBulkDataFlags)
	{
		SerializeBulkData(Ar, Data, GetBulkDataSize(), InBulkDataFlags);
	}

	FOwnedBulkDataPtr* StealFileMapping()
	{
		// @todo if non-mapped bulk data, do we need to detach this, or mimic GetCopy more than we do?
		return DataAllocation.StealFileMapping(this); 
	}

	/**
	 * Get the CustomVersions used in the file containing the BulkData payload. If !IsInSeparateFile, this will be
	 * the custom versions from the archive used to serialize the FBulkData, which the caller must provide.
	 * Otherwise, the CustomVersions come from the separate file and this function will look them up.
	 *
	 * @param InlineArchive The archive that was used to load this object
	 *
	 */
	COREUOBJECT_API void GetBulkDataVersions(FArchive& InlineArchive, FPackageFileVersion& OutUEVersion, int32& OutLicenseeUEVersion,
		FCustomVersionContainer& OutCustomVersions) const;

#if WITH_EDITOR
	/**
	 * When saving BulkData, if we are overwriting the file we need to update the BulkData's (flags,offset,size) to be
	 * able to load from the new file. But SerializeBulkData modifies those values when loading, so the in-memory values after
	 * loading from disk are not the same as the values on disk.
	 * This function handles running the same steps that SerializeBulkData does, but skips the deserialization of the BulkData.
	 * */
	COREUOBJECT_API void SetFlagsFromDiskWrittenValues(EBulkDataFlags InBulkDataFlags, int64 InBulkDataOffsetInFile, int64 InBulkDataSizeOnDisk, int64 LinkerSummaryBulkDataStartOffset);
#endif

	/*-----------------------------------------------------------------------------
		Async Streaming Interface.
	-----------------------------------------------------------------------------*/

	/**
	 * Opens a new IAsyncReadFileHandle that references the file that the BulkData object represents.
	 *
	 * @return A valid handle if the file can be accessed, if it cannot then nullptr.
	 */
	COREUOBJECT_API IAsyncReadFileHandle* OpenAsyncReadHandle() const;

	/**
	 * Create an async read request for the bulk data.
	 * This version will load the entire data range that the FBulkData represents.
	 *
	 * @param Priority				Priority and flags of the request. If this includes AIOP_FLAG_PRECACHE, then memory will never be returned. The request should always be canceled and waited for, even for a precache request.
	 * @param CompleteCallback		Called from an arbitrary thread when the request is complete. Can be nullptr, if non-null, must remain valid until it is called. It will always be called.
	 * @param UserSuppliedMemory	A pointer to memory for the IO request to be written to, it is up to the caller to make sure that it is large enough. If the pointer is null then the system will allocate memory instead.
	 * @return						A request for the read. This is owned by the caller and must be deleted by the caller.
	 */
	COREUOBJECT_API IBulkDataIORequest* CreateStreamingRequest(EAsyncIOPriorityAndFlags Priority, FBulkDataIORequestCallBack* CompleteCallback, uint8* UserSuppliedMemory) const;

	/**
	 * Create an async read request for the bulk data.
	 * This version allows the user to request a subset of the data that the FBulkData represents.
	 *
	 * @param OffsetInBulkData		Offset into the bulk data to start reading from.
	 * @param BytesToRead			The number of bytes to read. If this request is AIOP_Preache, the size can be anything, even MAX_int64, otherwise the size and offset must be fully contained in the file.
	 * @param Priority				Priority and flags of the request. If this includes AIOP_FLAG_PRECACHE, then memory will never be returned. The request should always be canceled and waited for, even for a precache request.
	 * @param CompleteCallback		Called from an arbitrary thread when the request is complete. Can be nullptr, if non-null, must remain valid until it is called. It will always be called.
	 * @param UserSuppliedMemory	A pointer to memory for the IO request to be written to, it is up to the caller to make sure that it is large enough. If the pointer is null then the system will allocate memory instead.
	 * @return						A request for the read. This is owned by the caller and must be deleted by the caller.
	 */
	COREUOBJECT_API IBulkDataIORequest* CreateStreamingRequest(int64 OffsetInBulkData, int64 BytesToRead, EAsyncIOPriorityAndFlags Priority, FBulkDataIORequestCallBack* CompleteCallback, uint8* UserSuppliedMemory) const;

	/** Enable the given flags in the given accumulator variable. */
	static COREUOBJECT_API void SetBulkDataFlagsOn(EBulkDataFlags& InOutAccumulator, EBulkDataFlags FlagsToSet);
	/** Disable the given flags in the given accumulator variable. */
	static COREUOBJECT_API void ClearBulkDataFlagsOn(EBulkDataFlags& InOutAccumulator, EBulkDataFlags FlagsToClear);
	/** Returns whether all of the specified flags are set. */
	static COREUOBJECT_API bool HasFlags(EBulkDataFlags Flags, EBulkDataFlags Contains);
	/** Returns decompress method flags specified by the given bulk data flags. */
	static COREUOBJECT_API FName GetDecompressionFormat(EBulkDataFlags InFlags);

	/*-----------------------------------------------------------------------------
		Class specific virtuals.
	-----------------------------------------------------------------------------*/

protected:

	COREUOBJECT_API void SerializeBulkData(FArchive& Ar, void* Data, int64 DataSize, EBulkDataFlags InBulkDataFlags);

private:

	COREUOBJECT_API int64 SerializePayload(FArchive& Ar, EBulkDataFlags SerializationFlags, const TOptional<EFileRegionType>& RegionType);
#if WITH_EDITOR
	/**
	 * Detaches the bulk data from the passed in archive. Needs to match the archive we are currently
	 * attached to.
	 *
	 * @param Ar						Archive to detach from
	 * @param bEnsureBulkDataIsLoaded	whether to ensure that bulk data is load before detaching from archive
	 */
	COREUOBJECT_API void DetachFromArchive( FArchive* Ar, bool bEnsureBulkDataIsLoaded );
#endif // WITH_EDITOR

	/*-----------------------------------------------------------------------------
		Internal helpers
	-----------------------------------------------------------------------------*/

	/**
	 * Loads the bulk data if it is not already loaded.
	 */
	COREUOBJECT_API void MakeSureBulkDataIsLoaded();

	/**
	 * Loads the data from disk into the specified memory block. This requires us still being attached to an
	 * archive we can use for serialization.
	 *
	 * @param Dest Memory to serialize data into
	 *
	 * @return Whether the load succeeded
	 */
	COREUOBJECT_API bool TryLoadDataIntoMemory(FIoBuffer Dest);

	/** Returns if the offset needs fixing when serialized */
	COREUOBJECT_API bool NeedsOffsetFixup() const;
	
	COREUOBJECT_API bool CanDiscardInternalData() const;
	
	/** Reallocate bulk data */
	inline void* ReallocateData(int64 SizeInBytes) { return DataAllocation.ReallocateData(this, SizeInBytes); }

	/** Free bulk data */
	inline void  FreeData() { DataAllocation.Free(this); }

	/** Retrieve the internal allocation for writing */
	inline void* GetDataBufferForWrite() const { return DataAllocation.GetAllocationForWrite(this); }

	/** Retrieve the internal allocation for reading */
	inline const void* GetDataBufferReadOnly() const { return DataAllocation.GetAllocationReadOnly(this); }

	/*-----------------------------------------------------------------------------
		Member variables.
	-----------------------------------------------------------------------------*/
	
	FAllocatedPtr							DataAllocation;
	UE::BulkData::Private::FBulkMetaData	BulkMeta;
	FIoChunkId								BulkChunkId = FIoChunkId::InvalidChunkId;
	
protected:
#if WITH_EDITOR
	/** Archive associated with bulk data for serialization																*/
	FArchive*				AttachedAr = nullptr;
	/** Used to make sure the linker doesn't get garbage collected at runtime for things with attached archives			*/
	FLinkerLoad*			Linker = nullptr;
	/** The index to which the cooked version of the bulkdata payload should be assigned*/
	FBulkDataCookedIndex	CookedIndex;
#if DO_CHECK
	TOptional<UE::FSourceLocation> LockSourceLocation;
#endif
#endif // WITH_EDITOR
	   //
#if !USE_RUNTIME_BULKDATA
	/** Custom bulk data serialization hook when using FUntypedBulkData. */
	FSerializeBulkDataElements* SerializeBulkDataElements = nullptr;
#endif // !USE_RUNTIME_BULKDATA
};

/**
 * Templated bulk data.
 *
 * A bulk data array of typed elements.
 */
template<typename ElementType>
class TBulkData : public FBulkData
{
public:

	static_assert(TIsPODType<ElementType>::Value, "Bulk data is limited to POD types");

	/**
	 * Element size in bytes.
	 *
	 * @return Returns the element size in bytes.
	 */
	int32 GetElementSize() const
	{ 
		return sizeof(ElementType);
	}

	/**
	 * Returns the number of elements in this bulk data array.
	 *
	 * @return Number of elements in this bulk data array
	 */
	int64 GetElementCount() const
	{
		return GetBulkDataSize() / sizeof(ElementType);
	}

	/**
	 * Change size of locked bulk data. Only valid if locked via read-write lock.
	 *
	 * @param ElementCount	Number of elements array should be resized to
	 */
	ElementType* Realloc(int64 ElementCount)
	{
		return (ElementType*)FBulkData::Realloc(ElementCount, sizeof(ElementType));
	}

	/**
	 * Returns a copy encapsulated by a FBulkDataBuffer.
	 *
	 * @param RequestedElementCount If set to greater than 0, the returned FBulkDataBuffer will be limited to
	 * this number of elements. This will give an error if larger than the actual number of elements in the BulkData object.
	 * @param bDiscardInternalCopy If true then the BulkData object will free it's internal buffer once called.
	 *
	 * @return A FBulkDataBuffer that owns a copy of the BulkData, this might be a subset of the data depending on the value of RequestedSize.
	 */
	FBulkDataBuffer<ElementType> GetCopyAsBuffer(int64 RequestedElementCount, bool bDiscardInternalCopy)
	{ 
		const int64 MaxElementCount = GetElementCount();

		check(RequestedElementCount <= MaxElementCount);

		ElementType* Ptr = nullptr;
		GetCopy((void**)& Ptr, bDiscardInternalCopy);

		const int64 BufferSize = (RequestedElementCount > 0 ? RequestedElementCount : MaxElementCount);

		return FBulkDataBuffer<ElementType>(Ptr, BufferSize);
	}

	/**
	 * Serialize function used to serialize this bulk data structure.
	 *
	 * @param Ar	Archive to serialize with
	 * @param Owner	Object owning the bulk data
	 * @param Idx	Index of bulk data item being serialized
	 * @param bAttemptFileMapping	If true, attempt to map this instead of loading it into malloc'ed memory
	 * @param FileRegionType	When cooking, a hint describing the type of data, used by some platforms to improve compression ratios
	 */
	void Serialize(FArchive& Ar, UObject* Owner, int32 Idx=INDEX_NONE, bool bAttemptFileMapping = false, EFileRegionType FileRegionType = EFileRegionType::None)
	{
		FBulkData::Serialize(Ar, Owner, bAttemptFileMapping, GetElementSize(), FileRegionType);
	}

	/**
	 * Serialize function used to serialize this bulk data structure, applying an override of the Bulk Data Flags when saving.
	 *
	 * @param Ar					Archive to serialize with
	 * @param Owner					Object owning the bulk data
	 * @param SaveOverrideFlags		EBulkDataFlags to use when saving (restores original flags after serialization)
	 * @param bAttemptFileMapping	If true, attempt to map this instead of loading it into malloc'ed memory
	 * @param FileRegionType		When cooking, a hint describing the type of data, used by some platforms to improve compression ratios
	 */
	void SerializeWithFlags(FArchive& Ar, UObject* Owner, uint32 SaveOverrideFlags, bool bAttemptFileMapping = false, EFileRegionType FileRegionType = EFileRegionType::None)
	{
		if (Ar.IsSaving())
		{
			const uint32 OriginalBulkDataFlags = GetBulkDataFlags();
			SetBulkDataFlags(SaveOverrideFlags);		// NOTE this does an OR between existing flags and the override flags
			FBulkData::Serialize(Ar, Owner, bAttemptFileMapping, GetElementSize(), FileRegionType);
			ResetBulkDataFlags(OriginalBulkDataFlags);
		}
		else
		{
			FBulkData::Serialize(Ar, Owner, bAttemptFileMapping, GetElementSize(), FileRegionType);
		}
	}
};

using FByteBulkData = TBulkData<uint8>;
using FWordBulkData = TBulkData<uint16>;
using FIntBulkData = TBulkData<int32>;
using FFloatBulkData = TBulkData<float>;

class FFormatContainer
{
	friend class UBodySetup;

	TSortedMap<FName, FByteBulkData*, FDefaultAllocator, FNameFastLess> Formats;
public:
	~FFormatContainer()
	{
		FlushData();
	}
	bool Contains(FName Format) const
	{
		return Formats.Contains(Format);
	}
	void GetContainedFormats(TArray<FName>& OutFormats) const
	{
		Formats.GenerateKeyArray(OutFormats);
	}
	FByteBulkData& GetFormat(FName Format)
	{
		FByteBulkData* Result = Formats.FindRef(Format);
		if (!Result)
		{
			Result = new FByteBulkData;
			Formats.Add(Format, Result);
		}
		return *Result;
	}
	void FlushData()
	{
		for (const TPair<FName, FByteBulkData*>& Format : Formats)
		{
			FByteBulkData* BulkData = Format.Value;
			delete BulkData;
		}
		Formats.Empty();
	}
	
	void DeepCopy(const FFormatContainer& Other)
	{
		// Safely clean up any entries that the current FFormatContainer has before creating/adding copies from the provided container.
		FlushData();

		for (const TPair<FName, FByteBulkData*>& Format : Other.Formats)
		{
			FByteBulkData* Copy = new FByteBulkData(*Format.Value);
			Formats.Add(Format.Key, Copy);
		}
	}

	COREUOBJECT_API void Serialize(FArchive& Ar, UObject* Owner, const TArray<FName>* FormatsToSave = nullptr, bool bSingleUse = true, uint16 InAlignment = DEFAULT_ALIGNMENT, bool bInline = true, bool bMapped = false);
	COREUOBJECT_API void SerializeAttemptMappedLoad(FArchive& Ar, UObject* Owner);
};

/** Handle to a bulk data I/O request. */
class FBulkDataRequest
{
public:
	/** Bulk data request status. */
	enum class EStatus : uint32
	{
		/** The request hasn't been issued. */
		None,
		/** The request is pending. */
		Pending,
		/** The request has been completed successfully. */
		Ok,
		/** The request was cancelled. */
		Cancelled,
		/** An error occured while issuing the request. */
		Error
	};

protected:
	class IHandle
	{
	public:
		virtual ~IHandle();

		virtual void AddRef() = 0;
		virtual void Release() = 0;
		virtual uint32 GetRefCount() const = 0;

		virtual FBulkDataRequest::EStatus GetStatus() const = 0;
		virtual bool Cancel() = 0;
		virtual bool Wait(uint32 Milliseconds) = 0;
		virtual void UpdatePriority(const EAsyncIOPriorityAndFlags Priority) = 0;
	};

public:
	/** Completion callback. */
	using FCompletionCallback = TFunction<void(EStatus)>;

	/** Default bulk data I/O request priority. */
	static constexpr EAsyncIOPriorityAndFlags DefaultPriority = AIOP_BelowNormal;

	/** Constructs a new handle to bulk data request. */
	FBulkDataRequest() = default;

	/** Destructor, cancels and waits for any pending requests. */
	~FBulkDataRequest() = default;

	/** Moves ownership from an invalid or pending request. */
	FBulkDataRequest(FBulkDataRequest&&) = default;
	FBulkDataRequest& operator=(FBulkDataRequest&&) = default;
	
	/** Not copy constructable or copy assignable. */
	FBulkDataRequest(const FBulkDataRequest&) = delete;
	FBulkDataRequest& operator=(const FBulkDataRequest&) = delete;

	/** Returns current status of the request. */
	EStatus GetStatus() const { return Handle.IsValid() ? Handle->GetStatus() : EStatus::None; }

	/** Returns whether the request is associated with a pending or completed request. */
	bool IsNone() const { return FBulkDataRequest::EStatus::None == GetStatus(); }

	/** Returns whether the request is pending. */
	bool IsPending() const { return FBulkDataRequest::EStatus::Pending == GetStatus(); }

	/** Returns whether the request completed successfully. */
	bool IsOk() const { return FBulkDataRequest::EStatus::Ok == GetStatus(); }

	/** Returns whether the request has been completed. */
	bool IsCompleted() const { return GetStatus() > EStatus::Pending; }

	/** Cancel the pending request. Cancelling an invalid request will immediately return. */
	void Cancel() { if (Handle.IsValid()) { Handle->Cancel(); } }

	/** Reset the request handle to an invalid state. Will cancel and wait if the request is not completed. */
	void Reset() { Handle.SafeRelease(); }

protected:
	friend class FHandleBase;
	friend class FChunkBatchReadRequest;
	friend class FBulkDataBatchRequest;
	
	FBulkDataRequest(FBulkDataRequest::IHandle* InHandle)
		: Handle(InHandle)
	{
		check(InHandle != nullptr);
	}

	TRefCountPtr<IHandle> Handle;
};

/** Handle to a bulk data I/O batch read request. */
using FBulkDataBatchReadRequest = FBulkDataRequest;

/**
 * A batch request is a handle to one or more I/O requests.
 *
 * The batch request is kept alive by passing in handles when appending
 * read operations or by passing in a handle when issuing the batch. At least
 * one handle needs to be passed in. The last handle will block until the
 * entire batch is complete before being released.
 */
class FBulkDataBatchRequest : public FBulkDataRequest
{
public:
	class FBatchHandle;

private:
	class FBuilder
	{
	public:
		COREUOBJECT_API ~FBuilder();
		FBuilder(const FBuilder&) = delete;
		FBuilder& operator=(const FBuilder&) = delete;

	protected:
		COREUOBJECT_API explicit FBuilder(int32 MaxCount);
		COREUOBJECT_API FBulkDataBatchRequest::FBatchHandle& GetBatch();
		COREUOBJECT_API void IssueBatch(FBulkDataBatchRequest* OutRequest, FCompletionCallback&& Callback);

		int32 BatchCount = 0;
		int32 NumLoaded = 0;

	private:
		int32 BatchMax = -1;
		TRefCountPtr<FBulkDataBatchRequest::FBatchHandle> Batch;
	};

public:	
	using FBulkDataRequest::FBulkDataRequest;

	/** Blocks the calling thread until the request is completed. Waiting on an invalid request will immediately return. */
	COREUOBJECT_API void Wait();

	/**
	 * Waits the specified amount of time in milliseconds for the request to be completed.
	 * @return True if the request was completed before the wait time elapsed, otherwise false. Waiting on an invalid request returns false.
	 */
	COREUOBJECT_API bool WaitFor(uint32 Milliseconds);

	/**
	 * Waits the specified amount of time for the request to be completed.
	 * @return True if the request was completed before the wait time elapsed, otherwise false. Waiting on an invalid request returns false.
	 */
	COREUOBJECT_API bool WaitFor(const FTimespan& WaitTime);

	/**
	 * Adjusts the priority of the request(s) (if possible).
	 * @param Priority The new priority to use
	 */
	COREUOBJECT_API void UpdatePriority(const EAsyncIOPriorityAndFlags Priority);

	/** Issue one or more I/O request in a single batch. */
	class FBatchBuilder : public FBuilder
	{
	public:
		COREUOBJECT_API FBatchBuilder(int32 MaxCount);
		/** Returns whether the batch is empty. */
		COREUOBJECT_API bool IsEmpty() const;
		/** Read the entire bulk data and copy the result to the specified instance. */
		COREUOBJECT_API FBatchBuilder& Read(FBulkData& BulkData, EAsyncIOPriorityAndFlags Priority = DefaultPriority);
		/**
		 * Read the bulk data from the specified offset and size and copy the result into the destination buffer.
		 * @param BulkData		The bulk data instance.
		 * @param Offset		Offset relative to the bulk data offset.
		 * @param Size			Number of bytes to read. Use MAX_uint64 to read the entire bulk data.
		 * @param Priority		The I/O priority.
		 * @param Dst			An empty or preallocated I/O buffer.	
		 */
		FBatchBuilder& Read(const FBulkData& BulkData, uint64 Offset, uint64 Size, EAsyncIOPriorityAndFlags Priority, FIoBuffer& Dst)
		{
			return Read(BulkData, Offset, Size, Priority, Dst, nullptr);
		}
		/**
		 * Read the bulk data from the specified offset and size and copy the result into the destination buffer.
		 * @param BulkData		The bulk data instance.
		 * @param Offset		Offset relative to the bulk data.
		 * @param Size			Number of bytes to read. Use MAX_uint64 to read the entire bulk data.
		 * @param Priority		The I/O priority.
		 * @param Dst			An empty or preallocated I/O buffer.	
		 * @param OutRequest	A handle to the read request.	
		 */
		FBatchBuilder& Read(const FBulkData& BulkData, uint64 Offset, uint64 Size, EAsyncIOPriorityAndFlags Priority, FIoBuffer& Dst, FBulkDataBatchReadRequest& OutRequest)
		{
			return Read(BulkData, Offset, Size, Priority, Dst, &OutRequest);
		}

		/**
		 * Issue the batch.
		 * @param Callback		Completion callback.	
		 * @param OutRequest	A handle to the batch request.
		 */
		COREUOBJECT_API void Issue(FCompletionCallback&& Callback, FBulkDataBatchRequest& OutRequest);
		/**
		 * Issue the batch.
		 * @param OutRequest	A handle to the batch request.
		 */
		COREUOBJECT_API void Issue(FBulkDataBatchRequest& OutRequest);
		/**
		 * Issue the batch. 
		 *
		 * @note Assumes one or more handle(s) has been passed into any of the read operations.
		 */
		COREUOBJECT_API void Issue();

	private:
		COREUOBJECT_API FBatchBuilder& Read(const FBulkData& BulkData, uint64 Offset, uint64 Size, EAsyncIOPriorityAndFlags Priority, FIoBuffer& Dst, FBulkDataBatchReadRequest* OutRequest);
	};

	/** Reads one or more bulk data and copies the result into a single I/O buffer. */
	class FScatterGatherBuilder : public FBuilder
	{
	public:
		COREUOBJECT_API FScatterGatherBuilder(int32 MaxCount);
		/** Read the bulk data from the specified offset and size. */
		COREUOBJECT_API FScatterGatherBuilder& Read(const FBulkData& BulkData, uint64 Offset = 0, uint64 Size = MAX_uint64);
		/**
		 * Issue the batch.
		 * @param Dst			An empty or preallocated I/O buffer.	
		 * @param Priority		The I/O priority.
		 * @param Callback		A callback triggered when the operation is completed.
		 * @param OutRequest	A handle to the batch request.
		 *
		 * @note Assumes one or more handle(s) has been passed into any of the read operations.
		 */
		COREUOBJECT_API void Issue(FIoBuffer& Dst, EAsyncIOPriorityAndFlags Priority, FCompletionCallback&& Callback, FBulkDataBatchRequest& OutRequest);

	private:
		struct FRequest
		{
			const FBulkData* BulkData	= nullptr;
			uint64 Offset				= 0;
			uint64 Size					= MAX_uint64;
		};

		TArray<FRequest, TInlineAllocator<8>> Requests;
	};

	/** Returns a request builder that dispatches one or more I/O requests in a single batch. */
	static FBatchBuilder NewBatch(int32 MaxCount = -1) { return FBatchBuilder(MaxCount); }

	/** Returns a request builder that reads one or more bulk data into a single I/O buffer. */
	static FScatterGatherBuilder ScatterGather(int32 MaxCount = -1) { return FScatterGatherBuilder(MaxCount); }
};
