// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Used to manage a raw data buffer provided by BulkData while providing access to it
 * via a TArrayView.
 *
 * This object assumes that it owns the buffer that it has been given and will free the
 * memory when the object is destroyed.
 */
template<typename DataType>
class FBulkDataBuffer
{
public:
	using ViewType = TArrayView64<DataType>;

	/** Constructor. */
	FBulkDataBuffer() = default;

	/**
	 * Copy constructor which will create it's own memory buffer
	 * and then copy from the source object rather than share access
	 * to the same buffer.
	 *
	 * @param Other The source array to copy.
	 */
	FBulkDataBuffer(const FBulkDataBuffer& Other)
	{
		*this = Other;
	}

	/**
	 * Move constructor.
	 *
	 * @param Other FBulkDataBuffer to move from.
	 */
	FBulkDataBuffer(FBulkDataBuffer&& Other)
	{
		View = Other.View;
		Other.View = ViewType();
	}

	/** Constructor.
	* @param InBuffer Pointer to memory to take ownership of, must've have been allocated via FMemory::Malloc/Realloc.
	* @param InNumberOfElements The number of elements in the buffer.
	*/
	FBulkDataBuffer(DataType* InBuffer, int64 InNumberOfElements)
		: View(InBuffer, InNumberOfElements)
	{

	}

	/** Destructor. */
	~FBulkDataBuffer()
	{
		FreeBuffer();
	}

	/**
	 * Assignment operator which will create it's own memory buffer
	 * and then copy from the source object rather than share access
	 * to the same buffer.
	 *
	 * @param Other The source FBulkDataBuffer to assign from.
	 */
	FBulkDataBuffer& operator =(const FBulkDataBuffer& Other)
	{
		FreeBuffer();

		if (this != &Other)
		{
			const int64 BufferSize = Other.View.Num();

			DataType* BufferCopy = (DataType*)FMemory::Malloc(BufferSize);
			FMemory::Memcpy(BufferCopy, Other.View.GetData(), BufferSize);

			View = ViewType(BufferCopy, BufferSize);
		}

		return *this;
	}

	/**
	 * Move assignment operator.
	 *
	 * @param Other FBulkDataBuffer to assign and move from.
	 */
	FBulkDataBuffer& operator =(FBulkDataBuffer&& Other)
	{
		if (this != &Other)
		{
			FreeBuffer();

			View = Other.View;
			Other.View = ViewType();
		}

		return *this;
	}

	/**
	 * Frees the internal buffer and sets the internal TArrayView to an empty state
	 *
	 */
	void Empty()
	{
		FreeBuffer();

		View = ViewType();
	}

	/**
	 * Frees any existing buffer and takes ownership of the buffer provided instead.
	 *
	 * @param InBuffer Pointer to memory to take ownership of, must've have been allocated via FMemory::Malloc/Realloc.
	*  @param InNumberOfElements The number of elements in the buffer.
	 * buffers actual size if desired.
	 */
	void Reset(DataType* InBuffer, uint64 InNumberOfElements)
	{
		FreeBuffer();

		View = ViewType(InBuffer, InNumberOfElements);
	}

	/**
	* Allows access to the data buffer owned by the object in the form of a TArrayView
	* @return A valid TArrayView
	*/
	const ViewType& GetView() const 
	{ 
		return View; 
	}

private:

	void FreeBuffer()
	{
		if (View.GetData() != nullptr)
		{
			FMemory::Free(View.GetData());
		}
	}

	ViewType View;
};
