// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Misc/PathViews.h"
#include "Serialization/StructuredArchive.h"
#include "String/LexFromString.h"
#include "String/Numeric.h"

// When this is enabled the default value for FBulkDataCookedIndex, meaning that the bulkdata does not use the
// system will be 0, if it is disabled we will use the max value allowed for FBulkDataCookedIndex::Type.
// If the default is zero then we cannot use <packagename>.000.ubulk as will start indexing from .001.ubulk instead
// but it also means that the FChunkId for a payload not using the system will remain unchanged.
#define UE_DEFAULT_ZERO 1


#define UE_DISABLE_COOKEDINDEX_FOR_MEMORYMAPPED 1

#define UE_DISABLE_COOKEDINDEX_FOR_NONDUPLICATE 1

enum class EBulkDataPayloadType : uint8
{
	Inline,				// Stored inside the export data in .uexp
	AppendToExports,	// Stored after the export data in .uexp
	BulkSegment,		// Stored in .ubulk
	Optional,			// Stored in .uptnl
	MemoryMapped,		// Stored in .m.bulk
};

class FBulkDataCookedIndex
{
public:
	// It is likely that we will want to expand the number of bits that this system currently uses when addressed
	// via FIoChunkIds in the future. The following constants and aliases make it easier to track places in the
	// code base that make assumptions about this so we can safely update them all at once.
	using ValueType = uint8;
	constexpr static int32 MAX_DIGITS = 3;

	COREUOBJECT_API static const FBulkDataCookedIndex Default;

	FBulkDataCookedIndex() = default;
	explicit FBulkDataCookedIndex(ValueType InValue)
		: Value(InValue)
	{

	}

	~FBulkDataCookedIndex() = default;

	bool IsDefault() const
	{
#if UE_DEFAULT_ZERO
		return Value == 0;
#else
		return Value == TNumericLimits<ValueType>::Max();
#endif //UE_DEFAULT_ZERO
	}

	FString GetAsExtension() const
	{
		if (IsDefault())
		{
			return FString();
		}
		else
		{
			return FString::Printf(TEXT(".%03hhu"), Value);
		}
	}

	ValueType GetValue() const
	{
		return Value;
	}

	bool operator == (const FBulkDataCookedIndex& Other) const
	{
		return Value == Other.Value;
	}

	bool operator < (const FBulkDataCookedIndex& Other) const
	{
		return Value < Other.Value;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, FBulkDataCookedIndex& ChunkGroup)
	{
		Slot << ChunkGroup.Value;
	}

	friend uint32 GetTypeHash(const FBulkDataCookedIndex& ChunkGroup)
	{
		return GetTypeHash(ChunkGroup.Value);
	}

	// TODO: Unit tests
	static FBulkDataCookedIndex ParseFromPath(FStringView Path)
	{
		int32 ExtensionStartIndex = -1;
		for (int32 Index = Path.Len() - 1; Index >= 0; --Index)
		{
			if (FPathViews::IsSeparator(Path[Index]))
			{
				return FBulkDataCookedIndex();
			}
			else if (Path[Index] == '.')
			{
				if (ExtensionStartIndex != -1)
				{
					FStringView Extension = Path.SubStr(Index + 1, (ExtensionStartIndex - Index) - 1);
					if (UE::String::IsNumericOnlyDigits(Extension))
					{
						ValueType Value = 0;
						LexFromString(Value, Extension);

						return FBulkDataCookedIndex(Value);
					}
					else
					{
						return FBulkDataCookedIndex();
					}
				}
				else
				{
					ExtensionStartIndex = Index;
				}
			}
		}

		return FBulkDataCookedIndex();
	}

private:
#if UE_DEFAULT_ZERO
	ValueType Value = 0;
#else
	ValueType Value = TNumericLimits<ValueType>::Max();
#endif //UE_DEFAULT_ZERO
};

#undef UE_DEFAULT_ZERO
