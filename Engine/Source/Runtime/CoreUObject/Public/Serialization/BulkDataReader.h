// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Serialization/BufferReader.h"
#include "Serialization/BulkData.h"
#include "Containers/UnrealString.h"


/**
 * Custom archive class for reading directly from bulk data.
 */
class FBulkDataReader final : public FBufferReaderBase
{
public:
	FBulkDataReader( FByteBulkData& InBulkData, bool bIsPersistent = false )
		: FBufferReaderBase( InBulkData.Lock( LOCK_READ_ONLY ), InBulkData.GetBulkDataSize(), false, bIsPersistent )
		, BulkData( InBulkData )
	{
	}

	~FBulkDataReader() override
	{
		BulkData.Unlock();
		Close();
	}

	using FArchive::operator<<; // For visibility of the overloads we don't override

	virtual FArchive& operator<<( class FName& Name ) override
	{
		// FNames are serialized as strings in BulkData
		FString StringName;
		*this << StringName;
		Name = FName( *StringName );
		return *this;
	}

	virtual FString GetArchiveName() const { return TEXT( "FBulkDataReader" ); }

protected:
	FByteBulkData& BulkData;
};
