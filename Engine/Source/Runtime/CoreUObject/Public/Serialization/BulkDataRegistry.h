// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "UObject/NameTypes.h"

#if WITH_EDITOR

#include "Async/Future.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Set.h"
#include "Delegates/Delegate.h"
#include "HAL/CriticalSection.h"
#include "IO/IoHash.h"
#include "Misc/Optional.h"

class IBulkDataRegistry;
class UPackage;
namespace UE::Serialization { class FEditorBulkData; }
struct FEndLoadPackageContext;
struct FGuid;

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogBulkDataRegistry, Log, All);
DECLARE_DELEGATE_RetVal(IBulkDataRegistry*, FSetBulkDataRegistry);

#endif

namespace UE::BulkDataRegistry
{
	// Define ERegisterResult even if !WITH_EDITOR to make it easier to use in EditorBulkData.cpp
	enum class ERegisterResult : uint8
	{
		Success,
		AlreadyExists,
		InvalidResultCode,
		// Update LexToString when adding new values
	};
}

#if WITH_EDITOR
const TCHAR* LexToString(UE::BulkDataRegistry::ERegisterResult Value);

namespace UE::BulkDataRegistry
{
	/** Results of GetMeta call. */
	struct FMetaData
	{
		/**
		 * IoHash of the uncompressed bytes of the data that will be returned from GetData.
		 * FIoHash::Zero if and only if data was not found.
		 */
		FIoHash RawHash;
		/**
		 * Size of the uncompressed bytes of the data that will be returned from GetData.
		 * 0 if data was not found, but can be 0 for valid data as well.
		 */
		uint64 RawSize;
	};

	/** Results of GetData call. */
	struct FData
	{
		/** The discovered data. Null buffer and only if data was not found. */
		FCompressedBuffer Buffer;
	};
}

/** Registers BulkDatas so that they can be referenced by guid during builds later in the editor process. */
class IBulkDataRegistry
{
public:
	/** The BulkDataRegistry can be configured off. Return whether it is enabled. A stub is used if not enabled. */
	COREUOBJECT_API static bool IsEnabled();
	/** Get the global BulkDataRegistry; always returns a valid interface, so long as Initialize has been called. */
	COREUOBJECT_API static IBulkDataRegistry& Get();
	
	/** Register a BulkData with the registry. Its payload and metadata will be fetchable by its GetIdentifier. */
	virtual UE::BulkDataRegistry::ERegisterResult
		TryRegister(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData) = 0;
	/** Change existing registration data to have the new values (keeping old Owner), or add if it doesn't exist. */
	virtual void UpdateRegistrationData(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData) = 0;
	/** Unregister the BulkData associated with a guid, because it is being modified. */
	virtual void Unregister(const UE::Serialization::FEditorBulkData& BulkData) = 0;
	/** Report that a BulkData is leaving memory and its in-memory payload (if it had one) is no longer available. */
	virtual void OnExitMemory(const UE::Serialization::FEditorBulkData& BulkData) = 0;
	/**
	 * Notify that a legacy BulkData with a PlaceholderPayloadId has loaded its data and updated its PayloadId, and
	 * the Registry should copy the updated PayloadId into the cache if it hasn't already calculated it.
	 */
	virtual void UpdatePlaceholderPayloadId(const UE::Serialization::FEditorBulkData& BulkData) = 0;

	/** Return the metadata for the given registered BulkData; returns false if not registered. */
	virtual TFuture<UE::BulkDataRegistry::FMetaData> GetMeta(const FGuid& BulkDataId) = 0;

	/**
	 * Return the (possibly compressed) payload for the given registered BulkData.
	 * Returns an empty buffer if not registered.
	 */
	virtual TFuture<UE::BulkDataRegistry::FData> GetData(const FGuid& BulkDataId) = 0;

	/**
	 * Returns the EditorBulkData and Owner for given id.
	 * Returns true if and only if the BulkDataId is a known id. OutOwner may be null even if returning true.
	 */
	virtual bool TryGetBulkData(const FGuid& BulkDataId, UE::Serialization::FEditorBulkData* OutBulk = nullptr,
		FName* OutOwner = nullptr) = 0;

	/**
	 * Report whether the Package had BulkDatas during load that upgrade or otherwise exist in memoryonly and
	 * cannot save all its BulkDatas by reference when resaved. This function only returns the correct
	 * information until OnEndLoadPackage is called for the given package; after that it can return an arbitrary
	 * value.
	 */
	virtual uint64 GetBulkDataResaveSize(FName PackageName) = 0;

	/** Set and intialize global IBulkDataRegistry; Get fatally fails before. */
	COREUOBJECT_API static void Initialize();
	/** Shutdown and deallocate global IBulkDataRegistry; Get fatally fails afterwards. */
	COREUOBJECT_API static void Shutdown();
	/** Subscribe to set the class for the global IBulkDataRegistry. */
	COREUOBJECT_API static FSetBulkDataRegistry& GetSetBulkDataRegistryDelegate();

protected:
	virtual ~IBulkDataRegistry() = default;
};

namespace UE::BulkDataRegistry::Private
{

/** Implements behavior needed across multiple BulkDataRegistry implementations for GetBulkDataResaveSize */
class FResaveSizeTracker
{
public:
	COREUOBJECT_API FResaveSizeTracker();
	COREUOBJECT_API ~FResaveSizeTracker();

	COREUOBJECT_API void Register(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData);
	COREUOBJECT_API void UpdateRegistrationData(UPackage* Owner, const UE::Serialization::FEditorBulkData& BulkData);
	COREUOBJECT_API uint64 GetBulkDataResaveSize(FName PackageName);

private:
	void OnEndLoadPackage(const FEndLoadPackageContext& Context);
	void OnAllModuleLoadingPhasesComplete();

	FRWLock Lock;
	TMap<FName, uint64> PackageBulkResaveSize;
	TArray<FName> DeferredRemove;
	bool bPostEngineInitComplete = false;
};

}


// Temporary interface for tunneling the EditorBuildInputResolver into CoreUObject.
// In the future this will be implemented as part of the BuildAPI
namespace UE::DerivedData { class IBuildInputResolver; }

COREUOBJECT_API UE::DerivedData::IBuildInputResolver* GetGlobalBuildInputResolver();
COREUOBJECT_API void SetGlobalBuildInputResolver(UE::DerivedData::IBuildInputResolver* InResolver);


#endif // WITH_EDITOR
