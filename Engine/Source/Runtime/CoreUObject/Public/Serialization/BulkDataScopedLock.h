// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/ContainersFwd.h"
#include "Serialization/BulkData.h"

// Note that these lock structures only work with the templated versions of FBulkData, such as
// FByteBulkData or FFloatBulkData because they need to know the format of the bulkdata payload
// which the base class FBulkData cannot provide.

/** Locks the given bulkdata object for read access */
template<typename ElementType>
class TBulkDataScopedReadLock
{
public:
	UE_NONCOPYABLE(TBulkDataScopedReadLock);

	TBulkDataScopedReadLock(const TBulkData<ElementType>& InBulkData)
		: BulkData(InBulkData)
	{
		Data = reinterpret_cast<const ElementType*>(InBulkData.LockReadOnly());
	}

	~TBulkDataScopedReadLock()
	{
		BulkData.Unlock();
	}

	/** Returns the number of elements in the bulkdata payload */
	int64 Num() const
	{
		return BulkData.GetElementCount();
	}

	/** Returns the length of the bulkdata payload in bytes */
	int64 GetAllocatedSize() const
	{
		return BulkData.GetBulkDataSize();
	}

	/** Returns a raw pointer to the bulkdata payload */
	const ElementType* GetData() const
	{
		return Data;
	}

	/** Returns the bulkdata payload wrapped in a TConstArrayView64 for added safety */
	TConstArrayView64<ElementType> GetView() const
	{
		return TConstArrayView64<ElementType>(Data, Num());
	}

private:
	const TBulkData<ElementType>& BulkData;
	const ElementType* Data;
};

/**
 * Locks the given bulkdata object for read/write access.
 * Note that performing actions on the original bulkdata object while this scope
 * is active (such as calls to realloc) may cause problems. Only use this object
 * for bulkdata operations while it is active.
 */
template<typename ElementType>
class TBulkDataScopedWriteLock
{
public:
	UE_NONCOPYABLE(TBulkDataScopedWriteLock);

	TBulkDataScopedWriteLock(TBulkData<ElementType>& InBulkData)
		: BulkData(InBulkData)
	{
		Data = reinterpret_cast<ElementType*>(InBulkData.Lock(LOCK_READ_WRITE));
	}

	~TBulkDataScopedWriteLock()
	{
		BulkData.Unlock();
	}

	/** Returns the number of elements in the bulkdata payload */
	int64 Num() const
	{
		return BulkData.GetElementCount();
	}

	/** Returns the length of the bulkdata payload in bytes */
	int64 GetAllocatedSize() const
	{
		return BulkData.GetBulkDataSize();
	}

	/** Returns a raw pointer to the bulkdata payload */
	ElementType* GetData() const
	{
		return Data;
	}

	/** Returns the bulkdata payload wrapped in a TArrayView64 for added safety */
	TArrayView64<ElementType> GetView() const
	{
		return TArrayView64<ElementType>(Data, Num());
	}

private:

	TBulkData<ElementType>& BulkData;
	ElementType* Data;
};
