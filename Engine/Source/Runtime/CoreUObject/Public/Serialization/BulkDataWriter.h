// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "HAL/UnrealMemory.h"
#include "Serialization/Archive.h"
#include "Serialization/BulkData.h"
#include "Containers/UnrealString.h"


/**
 * Custom archive class for writing directly to bulk data.
 */
class FBulkDataWriter final : public FArchive
{
public:
	FBulkDataWriter( FByteBulkData& InBulkData, bool bIsPersistent = false )
		: BulkData( InBulkData )
		, WriterPos( BulkData.GetBulkDataSize() )
		, WriterSize( WriterPos )
	{
		SetIsSaving( true );
		SetIsPersistent( bIsPersistent );
		Buffer = BulkData.Lock( LOCK_READ_WRITE );
	}

	~FBulkDataWriter()
	{
		// Remove the slack from the allocated bulk data
		BulkData.Realloc( WriterSize );
		BulkData.Unlock();
	}

	virtual void Serialize( void* Data, int64 Num )
	{
		// Determine if we need to reallocate the buffer to fit the next item
		const int64 NewPos = WriterPos + Num;
		check( NewPos >= WriterPos );
		if( NewPos > BulkData.GetBulkDataSize() )
		{
			// If so, resize to the new size + 3/8
			Buffer = BulkData.Realloc( NewPos + 3 * NewPos / 8 + 16 );
		}

		FMemory::Memcpy( static_cast<unsigned char*>( Buffer ) + WriterPos, Data, Num );
		WriterPos += Num;
		WriterSize = FMath::Max( WriterSize, WriterPos );
	}

	using FArchive::operator<<; // For visibility of the overloads we don't override

	virtual FArchive& operator<<( class FName& Name ) override
	{
		// FNames are serialized as strings in BulkData
		FString StringName = Name.ToString();
		*this << StringName;
		return *this;
	}

	virtual int64 Tell() { return WriterPos; }
	virtual int64 TotalSize() { return WriterSize; }

	virtual void Seek( int64 InPos )
	{
		check( InPos >= 0 );
		check( InPos <= WriterSize );
		WriterPos = InPos;
	}

	virtual bool AtEnd()
	{
		return WriterPos >= WriterSize;
	}

	virtual FString GetArchiveName() const { return TEXT( "FBulkDataWriter" ); }

protected:
	FByteBulkData& BulkData;
	void* Buffer;
	int64 WriterPos;
	int64 WriterSize;
};
