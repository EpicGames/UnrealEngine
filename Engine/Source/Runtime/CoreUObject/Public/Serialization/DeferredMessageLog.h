// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	DeferredMessgaeLog.h: Unreal async loading log.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class Error;
class FText;
class FTokenizedMessage;

/**
 * Thread safe proxy for the FMessageLog while performing async loading.
 * Makes sure the messages does not get added to the log until async loading is 
 * finished to prevent modules from being loaded outside of game thread.
 * Also makes sure the messages are added to the message queue in a thread-safe way.
*/
class FDeferredMessageLog
{
	FName LogCategory;

	static TMap<FName, TArray<TSharedRef<FTokenizedMessage>>*> Messages;
	static FCriticalSection MessagesCritical;

	void AddMessage(TSharedRef<FTokenizedMessage>& Message);

public:
	COREUOBJECT_API FDeferredMessageLog(const FName& InLogCategory);
	
	COREUOBJECT_API TSharedRef<FTokenizedMessage> Info(const FText& Message);
	COREUOBJECT_API TSharedRef<FTokenizedMessage> Warning(const FText& Message);
	COREUOBJECT_API TSharedRef<FTokenizedMessage> Error(const FText& Message);

	static COREUOBJECT_API void Flush();
	static COREUOBJECT_API void Cleanup();
};
