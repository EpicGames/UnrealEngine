// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "CoreTypes.h"
#include "Math/NumericLimits.h"
#include "Memory/MemoryFwd.h"
#include "Memory/MemoryView.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/PimplPtr.h"

#define UE_API COREUOBJECT_API

class FArchive;
class FCompressedBuffer;
class UObject;
struct FIoHash;
template <typename FuncType> class TUniqueFunction;

namespace UE::DerivedData { struct FCacheKey; }
namespace UE::DerivedData { struct FValueId; }
namespace UE::DerivedData::Private { class FCookedData; }
namespace UE::DerivedData::Private { class FEditorData; }
namespace UE::DerivedData::Private { class FIoResponse; }

namespace UE
{

/** Flags that modify how this derived data is saved and how it may be loaded. */
enum class EDerivedDataFlags : uint32
{
	None            = 0,
	/** Stage the referenced data with required content. May combine with Optional to stage into both locations. */
	Required        = 1 << 0,
	/** Stage the referenced data with optional content. */
	Optional        = 1 << 1,
	/** Stage the referenced data with alignment padding to allow memory-mapped access on supported platforms. */
	MemoryMapped    = 1 << 2,
};

ENUM_CLASS_FLAGS(EDerivedDataFlags);

class DerivedData::Private::FCookedData
{
public:
	inline explicit operator bool() const
	{
		// Check the last byte of ChunkId, which is type of the FIoChunkId, where 0 is EIoChunkType::Invalid.
		return ChunkId[11] != 0;
	}

	bool ReferenceEquals(const FCookedData& Other) const;
	uint32 ReferenceHash() const;

	void Serialize(FArchive& Ar);

	/** The offset of this data within its chunk. */
	uint64 ChunkOffset = 0;
	/** The size of this data within its chunk. */
	uint64 ChunkSize = 0;
	/** The FIoChunkId. */
	uint8 ChunkId[12]{};

	/** The flags that modify how this derived data is saved and how it may be loaded. */
	EDerivedDataFlags Flags = EDerivedDataFlags::None;
};

/**
 * Derived Data Reference
 *
 * Provides a consistent way to reference derived data:
 * - That has the same API for the editor as for builds with cooked packages.
 * - That has the same API to load from the cache as from a raw buffer, a compressed buffer, or a cooked package.
 * - That supports asynchronous, event-driven, prioritized loading of derived data.
 * - That supports saving to a cooked package without loading from the cache, when using Zen.
 * - That replaces serialized bulk data, such as FByteBulkData, in a cooked package.
 */
class FDerivedData
{
public:
	/** A null reference. */
	static const FDerivedData Null;

	/** Constructs a null reference. */
	FDerivedData() = default;

	/** Constructs a reference from the private representation of a cooked reference. */
	inline explicit FDerivedData(const DerivedData::Private::FCookedData& CookedData);

	/** Resets the reference to null. */
	inline void Reset() { *this = FDerivedData(); }

	/** Returns true if this is a null reference. */
	inline bool IsNull() const;

	/** Returns true if this is a non-null reference. */
	inline explicit operator bool() const { return !IsNull(); }

	/** Returns true if this is a non-null reference. */
	inline bool HasData() const { return !IsNull(); }

	/** Returns true if this is a cooked reference. */
	inline bool IsCooked() const { return !!CookedData; }

	/** Returns the flags, which are mainly relevant for staged references. */
	inline EDerivedDataFlags GetFlags() const { return CookedData.Flags; }

	/** Appends the name and description of this reference to the builder. */
	UE_API friend FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FDerivedData& DerivedData);

	/** Returns true if this and the other are equivalent references. */
	UE_API bool ReferenceEquals(const FDerivedData& Other) const;

	/** Returns a hash that factors in the same members as ReferenceEquals. */
	UE_API uint32 ReferenceHash() const;

	/** Serializes this reference to or from a cooked package. */
	UE_API void Serialize(FArchive& Ar, UObject* Owner);

#if WITH_EDITORONLY_DATA
	/** References a value that is stored in a buffer. */
	UE_API FDerivedData(const FSharedString& Name, const FSharedBuffer& Data);
	UE_API FDerivedData(const FSharedString& Name, const FCompositeBuffer& Data);
	UE_API FDerivedData(const FSharedString& Name, const FCompressedBuffer& Data);

	/** References a value that was saved using ICache::PutValue. */
	UE_API FDerivedData(const FSharedString& Name, const DerivedData::FCacheKey& Key);
	/** References a value in a record that was saved using ICache::Put. */
	UE_API FDerivedData(const FSharedString& Name, const DerivedData::FCacheKey& Key, const DerivedData::FValueId& ValueId);

	/** Returns the name of the reference if available. */
	UE_API const FSharedString& GetName() const;

	/** Overwrites the existing flags. */
	UE_API void SetFlags(EDerivedDataFlags Flags);

private:
	TPimplPtr<DerivedData::Private::FEditorData, EPimplPtrMode::DeepCopy> EditorData;
#endif // WITH_EDITORONLY_DATA

private:
	DerivedData::Private::FCookedData CookedData;

	friend DerivedData::Private::FIoResponse;
};

inline const FDerivedData FDerivedData::Null;

inline FDerivedData::FDerivedData(const DerivedData::Private::FCookedData& InCookedData)
	: CookedData(InCookedData)
{
}

inline bool FDerivedData::IsNull() const
{
#if WITH_EDITORONLY_DATA
	return !EditorData && !CookedData;
#else
	return !CookedData;
#endif
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/** Priority for scheduling an operation on a Derived Data Reference. */
struct FDerivedDataIoPriority
{
	/**
	 * Interpolates between this priority and a target priority.
	 *
	 * Alpha must be in [0, 1] where 0 returns this priority and 1 returns the target priority.
	 * Blocking() is returned if either priority is blocking.
	 */
	constexpr FDerivedDataIoPriority InterpolateTo(const FDerivedDataIoPriority Target, const float Alpha) const
	{
		// Blocking input produces Blocking output.
		if (*this == Blocking() || Target == Blocking())
		{
			return Blocking();
		}
		return {Value + int32((int64(Target.Value) - int64(Value)) * double(Alpha))};
	}

	static constexpr FDerivedDataIoPriority Lowest()   { return {MIN_int32}; }
	static constexpr FDerivedDataIoPriority Low()      { return {MIN_int32 / 2}; }
	static constexpr FDerivedDataIoPriority Normal()   { return {0}; }
	static constexpr FDerivedDataIoPriority High()     { return {MAX_int32 / 2}; }
	static constexpr FDerivedDataIoPriority Highest()  { return {MAX_int32 - 1}; }
	static constexpr FDerivedDataIoPriority Blocking() { return {MAX_int32}; }

	friend constexpr bool operator==(FDerivedDataIoPriority A, FDerivedDataIoPriority B) { return A.Value == B.Value; }
	friend constexpr bool operator!=(FDerivedDataIoPriority A, FDerivedDataIoPriority B) { return A.Value != B.Value; }
	friend constexpr bool operator<(FDerivedDataIoPriority A, FDerivedDataIoPriority B) { return A.Value < B.Value; }

	/** Priority. Lower values have lower priority and 0 is equivalent to Normal(). */
	int32 Value = 0;
};

/** Status of an operation on a Derived Data Reference. */
enum class EDerivedDataIoStatus : uint8
{
	/** The operation completed successfully. */
	Ok,
	/** The operation completed unsuccessfully. */
	Error,
	/** The operation was canceled before it completed. */
	Canceled,
	/** The operation has not completed or is otherwise in an unknown state. */
	Unknown,
};

/**
 * Request on a Derived Data Reference that may be used to query the associated response.
 */
class FDerivedDataIoRequest
{
public:
	/** Resets this to a null request. */
	constexpr inline void Reset() { Index = -1; }

	/** Returns true if this is a null request. */
	constexpr inline bool IsNull() const { return Index == -1; }

	/** Returns true if this is a non-null request. */
	constexpr inline explicit operator bool() const { return !IsNull(); }

private:
	int32 Index = -1;

	friend DerivedData::Private::FIoResponse;
};

/**
 * Tracks one or more associated requests dispatched by FDerivedDataIoBatch.
 *
 * Any const member functions are thread-safe, while any non-const member functions are not.
 *
 * Asserts that requests are complete upon destruction or assignment.
 */
class FDerivedDataIoResponse
{
public:
	/** Resets this to a null response. Asserts that requests are complete. */
	inline void Reset() { Response.Reset(); }

	/** Returns true if this is a null response . */
	inline bool IsNull() const { return !Response; }

	/** Returns true if this is a non-null response. */
	inline explicit operator bool() const { return !IsNull(); }

	/** Sets the priority of the associated requests if they are not complete. */
	UE_API void SetPriority(FDerivedDataIoPriority Priority);

	/**
	 * Cancels the associated requests if they are not complete.
	 *
	 * @return true if cancellation completed synchronously, or false if further polling is needed.
	 */
	UE_API bool Cancel();

	/** Returns true if the associated requests are complete and the optional callback was invoked. */
	UE_API bool Poll() const;

	/**
	 * Returns the overall status of the requests. See EDerivedDataIoStatus.
	 *
	 * The status of the overall response is, in priority order,
	 * - Unknown if any request status is Unknown, or
	 * - Canceled if any request status is Canceled, or
	 * - Error if any request status is Error, otherwise
	 * - Ok because every request status must be Ok.
	 */
	UE_API EDerivedDataIoStatus GetOverallStatus() const;

	/** Returns the status of the request. See EDerivedDataIoStatus. */
	UE_API EDerivedDataIoStatus GetStatus(FDerivedDataIoRequest Handle) const;

	/** Returns the data for the request. Null if executing, failed, or canceled. */
	UE_API FSharedBuffer GetData(FDerivedDataIoRequest Handle) const;

	/** Returns the size for the request. Zero if not available, executing, failed, or canceled. */
	UE_API uint64 GetSize(FDerivedDataIoRequest Handle) const;

#if WITH_EDITORONLY_DATA
	/** Returns the hash of the entire derived data. Null if not available, executing, failed, or canceled. */
	UE_API const FIoHash* GetHash(FDerivedDataIoRequest Handle) const;

	/** Returns the cache key for the request. Null if not applicable, executing, failed, or canceled. */
	UE_API const DerivedData::FCacheKey* GetCacheKey(FDerivedDataIoRequest Handle) const;

	/** Returns the cache value ID for the request. Null if not applicable, executing, failed, or canceled. */
	UE_API const DerivedData::FValueId* GetCacheValueId(FDerivedDataIoRequest Handle) const;

	/** Returns the compressed data for the request. Null if not applicable, executing, failed, or canceled. */
	UE_API const FCompressedBuffer* GetCompressedData(FDerivedDataIoRequest Handle) const;
#endif // WITH_EDITORONLY_DATA

private:
	TPimplPtr<DerivedData::Private::FIoResponse> Response;

	friend DerivedData::Private::FIoResponse;
};

/** Options for operations on a Derived Data Reference. */
class FDerivedDataIoOptions
{
public:
	/** Reads the entirety of the referenced data into a buffer that is allocated on demand. */
	FDerivedDataIoOptions() = default;

	/**
	 * Reads referenced data into the target view, with an optional offset into the source data.
	 *
	 * The size of the data from the source offset must be at least the size of the target view.
	 */
	constexpr explicit FDerivedDataIoOptions(FMutableMemoryView TargetView, uint64 SourceOffset = 0)
		: Target(TargetView.GetData())
		, Size(TargetView.GetSize())
		, Offset(SourceOffset)
	{
	}

	/**
	 * Reads referenced data into a buffer that is allocated on demand, with an optional offset into the source data.
	 *
	 * The size of the data from the source offset must be at least the given source size, if not MAX_uint64.
	 */
	constexpr explicit FDerivedDataIoOptions(uint64 SourceOffset, uint64 SourceSize = MAX_uint64)
		: Size(SourceSize)
		, Offset(SourceOffset)
	{
	}

	constexpr void* GetTarget() const { return Target; }
	constexpr uint64 GetSize() const { return Size; }
	constexpr uint64 GetOffset() const { return Offset; }

private:
	/** An optional target address to read the data into. Size is required when a target is provided. */
	void* Target = nullptr;

	/** The size to read, starting from the offset. Set MAX_uint64 to read to the end. */
	uint64 Size = MAX_uint64;

	/** The offset into the data at which to start reading. */
	uint64 Offset = 0;
};

using FDerivedDataIoComplete = TUniqueFunction<void ()>;

/**
 * Batch of requests to access Derived Data References.
 *
 * Gather related requests into a batch before issuing it to reduce the overhead of accessing related data.
 */
class FDerivedDataIoBatch
{
public:
	/** Resets this to an empty batch. */
	inline void Reset() { Response.Reset(); }

	/** Returns true if this batch has no queued requests. */
	inline bool IsEmpty() const { return !Response; }

	/**
	 * Reads the derived data into memory. Not executed until Dispatch() is invoked.
	 *
	 * The data, size, and status will be available on the response.
	 *
	 * @return A request to query the response with. May be discarded by the caller if not needed.
	 */
	UE_API FDerivedDataIoRequest Read(const FDerivedData& Data, const FDerivedDataIoOptions& Options = {});

	/**
	 * Caches the derived data into local storage without loading it. Not executed until Dispatch() is invoked.
	 *
	 * The size and status will be available on the response. Status is Ok if it caches and Error if not.
	 *
	 * @return A request to query the response with. May be discarded by the caller if not needed.
	 */
	UE_API FDerivedDataIoRequest Cache(const FDerivedData& Data, const FDerivedDataIoOptions& Options = {});

	/**
	 * Caches the derived data into local storage without loading it. Not executed until Dispatch() is invoked.
	 *
	 * The size and status will be available on the response. Status is Ok if it exists and Error if not.
	 *
	 * @return A request to query the response with. May be discarded by the caller if not needed.
	 */
	UE_API FDerivedDataIoRequest Exists(const FDerivedData& Data, const FDerivedDataIoOptions& Options = {});

#if WITH_EDITORONLY_DATA
	/**
	 * Compresses the derived data. Not executed until Dispatch() is invoked.
	 *
	 * The compressed data, size, and status will be available on the response.
	 *
	 * @return A request to query the response with. May be discarded by the caller if not needed.
	 */
	UE_API FDerivedDataIoRequest Compress(const FDerivedData& Data);
#endif // WITH_EDITORONLY_DATA

	/**
	 * Dispatches the requests that have been queued to this batch and resets this batch to empty.
	 *
	 * The response must be kept alive until the batch completes, otherwise it will be automatically canceled.
	 *
	 * @param OutResponse   Response that is assigned before any requests begin executing.
	 * @param Priority      Priority of this batch relative to other batches.
	 * @param OnComplete    Completion callback invoked when the entire batch is complete or canceled.
	 */
	UE_API void Dispatch(FDerivedDataIoResponse& OutResponse);
	UE_API void Dispatch(FDerivedDataIoResponse& OutResponse, FDerivedDataIoPriority Priority);
	UE_API void Dispatch(FDerivedDataIoResponse& OutResponse, FDerivedDataIoPriority Priority, FDerivedDataIoComplete&& OnComplete);
	UE_API void Dispatch(FDerivedDataIoResponse& OutResponse, FDerivedDataIoComplete&& OnComplete);

private:
	TPimplPtr<DerivedData::Private::FIoResponse> Response;

	constexpr uint32 GetTypeHash(const FDerivedDataIoPriority& Priority)
	{
		return uint32(Priority.Value);
	}
};

} // UE

namespace UE::DerivedData::IoStore
{

inline void InitializeIoDispatcher()
{
}

inline void TearDownIoDispatcher()
{
}

} // UE::DerivedData::IoStore

#undef UE_API
