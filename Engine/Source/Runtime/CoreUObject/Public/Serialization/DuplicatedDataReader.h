// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/UObjectAnnotation.h"
#include "Serialization/DuplicatedObject.h"
#include "Serialization/LargeMemoryData.h"
#include "Templates/RefCounting.h"
#include "UObject/UObjectThreadContext.h"
#include "VerseVM/VVMDuplication.h"

/*----------------------------------------------------------------------------
	FDuplicateDataReader.
----------------------------------------------------------------------------*/

/**
 * Reads duplicated objects from a memory buffer, replacing object references to duplicated objects.
 */
class FDuplicateDataReader : public FArchiveUObject
{
private:

	FDuplicatedObjectAnnotation&			DuplicatedObjectAnnotation;
	Verse::FDuplicationContext*				DuplicatedCells;
	const FLargeMemoryData&					ObjectData;
	int64									Offset;

	//~ Begin FArchive Interface.

	virtual FArchive& operator<<(FName& N) override;
	virtual FArchive& operator<<(UObject*& Object) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Cell) override;
#endif
	virtual FArchive& operator<<(FObjectPtr& Object) override;
	virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	virtual FArchive& operator<<(FSoftObjectPath& SoftObjectPath) override;
	
	void SerializeFail();

	virtual void Serialize(void* Data,int64 Num) override
	{
		if (ObjectData.Read(Data, Offset, Num))
		{
			Offset += Num;
		}
		else
		{
			SerializeFail();
		}
	}

public:
	virtual void Seek(int64 InPos) override
	{
		Offset = InPos;
	}

	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const { return TEXT("FDuplicateDataReader"); }

	virtual int64 Tell()
	{
		return Offset;
	}
	virtual int64 TotalSize()
	{
		return ObjectData.GetSize();
	}

	/**
	 * Constructor
	 * 
	 * @param	InDuplicatedObjectAnnotation		Annotation for storing a mapping from source to duplicated object
	 * @param	InObjectData					Object data to read from
	 */
	FDuplicateDataReader(FDuplicatedObjectAnnotation& InDuplicatedObjectAnnotation, const FLargeMemoryData& InObjectData, uint32 InPortFlags, UObject* InDestOuter);
	void SetDuplicatedCells(Verse::FDuplicationContext &InDuplicatedCells);
};