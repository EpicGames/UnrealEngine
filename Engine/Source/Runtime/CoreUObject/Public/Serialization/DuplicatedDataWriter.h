// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/ArchiveUObject.h"
#include "UObject/UObjectAnnotation.h"
#include "Serialization/DuplicatedObject.h"
#include "Serialization/LargeMemoryData.h"
#include "Templates/RefCounting.h"
#include "UObject/UObjectThreadContext.h"
#include "VerseVM/VVMDuplication.h"

struct FObjectInstancingGraph;

/*----------------------------------------------------------------------------
	FDuplicateDataWriter.
----------------------------------------------------------------------------*/
/**
 * Writes duplicated objects to a memory buffer, duplicating referenced inner objects and adding the duplicates to the DuplicatedObjects map.
 */
class FDuplicateDataWriter : public FArchiveUObject
{
private:

	FDuplicatedObjectAnnotation&			DuplicatedObjectAnnotation;
	Verse::FDuplicationContext*				DuplicatedCells;
	FLargeMemoryData&						ObjectData;
	int64									Offset;
	EObjectFlags							FlagMask;
	EObjectFlags							ApplyFlags;
	EInternalObjectFlags InternalFlagMask;
	EInternalObjectFlags ApplyInternalFlags;
	bool bAssignExternalPackages;

	/**
	 * This is used to prevent object & component instancing resulting from the calls to StaticConstructObject(); instancing subobjects and components is pointless,
	 * since we do that manually and replace the current value with our manually created object anyway.
	 */
	struct FObjectInstancingGraph*			InstanceGraph;

	//~ Begin FArchive Interface.

	virtual FArchive& operator<<(FName& N) override;
	virtual FArchive& operator<<(UObject*& Object) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Cell) override;
#endif
	virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	virtual FArchive& operator<<(FField*& Field) override;
	virtual FArchive& operator<<(FObjectPtr& Object) override;

	virtual void Serialize(void* Data,int64 Num) override
	{
		if (ObjectData.Write(Data, Offset, Num))
		{
			Offset += Num;
		}
	}

	virtual void Seek(int64 InPos) override
	{
		Offset = InPos;
	}

	/**
	 * Places a new duplicate in the DuplicatedObjects map as well as the UnserializedObjects list
	 * 
	 * @param	SourceObject	the original version of the object
	 * @param	DuplicateObject	the copy of the object
	 */
	void AddDuplicate(UObject* SourceObject, UObject* DuplicateObject);

public:
	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const { return TEXT("FDuplicateDataWriter"); }

	virtual int64 Tell()
	{
		return Offset;
	}

	virtual int64 TotalSize()
	{
		return ObjectData.GetSize();
	}

	TArray<UObject*>	UnserializedObjects;

	/**
	 * Returns a pointer to the duplicate of a given object, creating the duplicate object if necessary.
	 * 
	 * @param	Object	the object to find a duplicate for
	 * @param	bCreateIfMissing Create the duplicated object if it's missing.
	 *
	 * @return	a pointer to the duplicate of the specified object
	 */
	UObject* GetDuplicatedObject(UObject* Object, bool bCreateIfMissing = true);

	/**
	 * Constructor
	 * 
	 * @param	InDuplicatedObjects		Annotation for storing a mapping from source to duplicated object
	 * @param	InObjectData			will store the serialized data
	 * @param	SourceObject			the object to copy
	 * @param	DestObject				the object to copy to
	 * @param	InFlagMask				the flags that should be copied when the object is duplicated
	 * @param	InApplyFlags			the flags that should always be set on the duplicated objects (regardless of whether they're set on the source)
	 * @param	InInstanceGraph			the instancing graph to use when creating the duplicate objects.
	 */
	FDuplicateDataWriter(
		FDuplicatedObjectAnnotation& InDuplicatedObjects,
		FLargeMemoryData& InObjectData,
		UObject* SourceObject,
		UObject* DestObject,
		EObjectFlags InFlagMask,
		EObjectFlags InApplyMask,
		EInternalObjectFlags InInternalFlagMask,
		EInternalObjectFlags InApplyInternalFlags,
		FObjectInstancingGraph* InInstanceGraph,
		uint32 InPortFlags,
		bool InAssignExternalPackages);
	void SetDuplicatedCells(Verse::FDuplicationContext& InDuplicatedCells);
};
