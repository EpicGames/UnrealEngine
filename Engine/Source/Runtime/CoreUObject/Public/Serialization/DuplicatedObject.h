// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/UObjectAnnotation.h"

/**
 * Information about a duplicated object
 * For use with a dense object annotation
 */
struct FDuplicatedObject
{
	/** The state of this object */
	bool bIsDefault;

	/** The duplicated object */
	TWeakObjectPtr<UObject> DuplicatedObject;

	FDuplicatedObject()
		: bIsDefault(true)
	{
	}

	FDuplicatedObject( UObject* InDuplicatedObject )
		: bIsDefault( !InDuplicatedObject )
		, DuplicatedObject( InDuplicatedObject != INVALID_OBJECT ? InDuplicatedObject : nullptr )
	{
	}

	/**
	 * @return true if this is the default annotation and holds no information about a duplicated object
	 */
	UE_FORCEINLINE_HINT bool IsDefault()
	{
		return bIsDefault;
	}
};

template <> struct TIsPODType<FDuplicatedObject> { enum { Value = true }; };

/** Annotation type for storing duplicated object infos */
typedef FUObjectAnnotationTemp<FDuplicatedObject> FDuplicatedObjectAnnotation;

