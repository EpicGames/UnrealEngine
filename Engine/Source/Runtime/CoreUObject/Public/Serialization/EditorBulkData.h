// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Async/RecursiveMutex.h"
#include "Async/Future.h"
#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "CoreTypes.h"
#include "HAL/Platform.h"
#include "IO/IoHash.h"
#include "Internationalization/Text.h"
#include "Memory/SharedBuffer.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/PackagePath.h"
#include "Misc/PackageSegment.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/StructuredArchive.h"

class FArchive;
class FBulkData;
class FLinkerSave;
class UObject;
struct FPackageFileVersion;

namespace UE::BulkDataRegistry { enum class ERegisterResult : uint8; }

//TODO: At some point it might be a good idea to uncomment this to make sure that FEditorBulkData is
//		never used at runtime (requires a little too much reworking of some assets for now though)
//#if WITH_EDITORONLY_DATA

namespace UE::Serialization
{

namespace Private
{
/** A wrapper around the oodle compression settings used by FEditorBulkData. */
struct FCompressionSettings
{
	COREUOBJECT_API FCompressionSettings();
	FCompressionSettings(const FCompressedBuffer& Buffer);

	[[nodiscard]] bool operator ==(const FCompressionSettings& Other) const;
	[[nodiscard]] bool operator != (const FCompressionSettings& Other) const;

	void Reset();

	void Set(ECompressedBufferCompressor InCompressor, ECompressedBufferCompressionLevel InCompressionLevel);
	void SetToDefault();
	void SetToDisabled();

	[[nodiscard]] bool IsSet() const;
	[[nodiscard]] bool IsCompressed() const;

	[[nodiscard]] ECompressedBufferCompressor GetCompressor() const;
	[[nodiscard]] ECompressedBufferCompressionLevel GetCompressionLevel();

private:

	ECompressedBufferCompressor Compressor;
	ECompressedBufferCompressionLevel CompressionLevel;

	bool bIsSet;
};
} // namespace Private

/** 
 * A set of higher level compression options that avoid the need to set the specific
 * oodle options.
 */
enum class ECompressionOptions : uint8
{
	/** Use default compression settings. */
	Default,
	/** Disable compression for the bulkdata entirely. */
	Disabled,
};

/**
 * The goal of this class is to provide an editor time version of BulkData that will work with the asset
 * virtualization system.
 *
 * Assuming that the DDC is hot, the virtualized payloads are accessed relatively infrequently, usually when the package
 * is being edited in the editor in some manner. So the payload access is designed around this. If the data is frequently 
 * accessed when running the editor then the user would not gain from having it virtualized as they would end up pulling
 * it immediately anyway.
 *
 * The biggest difference with normal bulkdata is that the access times might be significantly longer if the
 * payload is not readily available which is why the only way to access the payload is by a TFuture or a callback
 * so that the caller is forced to consider how to handle the potential stall and hopefully organize their code 
 * in such a way that the time lag is not noticeable to the user.
 *
 * The second biggest difference is that the caller will own the payload memory once it is returned to them, unlike
 * the old bulkdata class which would retain ownership. Which forces the calling code to be in control of when the 
 * memory is actually allocated and for how long. With the old bulkdata class a single access would leave that memory
 * allocated in a non-obvious way and would lead to memory bloats within the editor.
 *
 * The method ::GetGuid can be used to access a unique identifier for the payload, currently it is based on the 
 * payload itself, so that two objects with the same payload would both have the same Guid. The intent is that we 
 * would be able to share local copies of the payload between branches to reduce the cost of having multiple branches 
 * with similar data on the same machine.
 * 
 * Updating the bulkdata:
 * Unlike the older bulkdata system, EditorBulkData does not support any lock/unlock mechanism, instead the internal 
 * data is updated via a single call to ::UpdatePayload and the data is retrieved via a single call to ::GetPayload.
 * 
 * For examples see the "System.CoreUObject.Serialization.EditorBulkData.UpdatePayload" test found in
 * Engine\Source\Runtime\CoreUObject\Private\Tests\Serialization\EditorBulkDataTests.cpp.
 * 
 * Thread Safety:
 * 
 * The class should be thread safe in that you can call the public methods on multiple threads at once and expect
 * things to work. However if you were to call ::GetPayloadSize() before calling ::GetPayload() on one thread
 * it would be possible for a call to ::UpdatePayload() to occur on another thread in between those two calls, 
 * meaning that the size you got might not be the correct size as the payload returned. If you need that level
 * of thread safety it would be up to you to manage it at a higher level. 
 * It would however be safe to pass a copy of your FEditorBulkData to a background task rather than a reference.
 * The copy would either be able to load the payload from disk, pull it from virtualized storage or if the 
 * source bulkdata is holding the payload in memory it will just take a reference to that payload rather than 
 * making a copy. In this way you can be sure of the state of the FEditorBulkData on the background task as 
 * future edits to the original bulkdata will not affect it.
 */

/** The base class with no type */
class FEditorBulkData final
{
public:
	FEditorBulkData() = default;
	COREUOBJECT_API ~FEditorBulkData();

	COREUOBJECT_API FEditorBulkData(FEditorBulkData&& Other);
	COREUOBJECT_API FEditorBulkData& operator=(FEditorBulkData&& Other);

	COREUOBJECT_API FEditorBulkData(const FEditorBulkData& Other);
	COREUOBJECT_API FEditorBulkData& operator=(const FEditorBulkData& Other);

	/** 
	 * Convenience method to make it easier to convert from FBulkData to FEditorBulkData and sets the Guid 
	 *
	 * @param BulkData	The bulkdata object to create from.
	 * @param Guid		A guid associated with the bulkdata object which will be used to identify the payload.
	 *					This MUST remain the same between sessions so that the payloads key remains consistent!
	 */
	COREUOBJECT_API void CreateFromBulkData(FBulkData& BulkData, const FGuid& Guid, UObject* Owner);
	
	/** 
	 * Fix legacy content that created the Id from non-unique Guids.
	 * Run on objects created before FUE5MainStreamObjectVersion::VirtualizedBulkDataHaveUniqueGuids
	 */
	COREUOBJECT_API void CreateLegacyUniqueIdentifier(UObject* Owner);

	/** 
	 * Used to serialize the bulkdata to/from a FArchive
	 * 
	 * @param Ar	The archive to serialize the bulkdata.
	 * @param Owner	The UObject that contains the bulkdata object, if this is a nullptr then the bulkdata will
	 *				assume that it must serialize the payload immediately to memory as it will not be able to
	 *				identify it's package path.
	 * @param bAllowRegistry Legacy parameter to skip registration when loading BulkData we know we will need to
	 *				modify the identifier of. Should always be true for non-legacy serialization.
	 */
	COREUOBJECT_API void Serialize(FArchive& Ar, UObject* Owner, bool bAllowRegister=true);

	/** Reset to a truly empty state */
	COREUOBJECT_API void Reset();

	// TODO: Not sure if there is a scenario where we can reload the payload off disk but it is also
	// in memory.and so this might not really do anything anymore.

	/** Unloads the data (if possible) but leaves it in a state where the data can be reloaded */
	COREUOBJECT_API void UnloadData();

	/** 
	 * Removes the ability for the bulkdata object to load it's payload from disk (if it was doing so) 
	 * 
	 * @param Ar						The archive that the bulkdata is being detached from. This should match AttachedAr.
	 * @param bEnsurePayloadIsLoaded	If true and the bulkdata currently is storing it's payload as a package on disk then
	 *									the payload should be loaded into memory so that it can be accessed in the future.
	 *									If false then the payload is not important and does not need to be loaded.
	 */
	COREUOBJECT_API void DetachFromDisk(FArchive* Ar, bool bEnsurePayloadIsLoaded);

	/** Returns a unique identifier for the object itself. */
	COREUOBJECT_API FGuid GetIdentifier() const;

	/** Returns an unique identifier for the content of the payload. */
	COREUOBJECT_API const FIoHash& GetPayloadId() const;

	/** Returns the size of the payload in bytes. */
	int64 GetPayloadSize() const
	{ 
		return PayloadSize; 
	}

	/** Returns true if the bulkdata object specifies a valid payload greater than zero bytes in size. */
	bool HasPayloadData() const
	{ 
		return PayloadSize > 0; 
	}

	/**
	 * Returns whether the payload is loaded in memory and we have a pointer to it.
	 * If true, calling GetPayload will return immediately with the value (unless modified inbetween).
	 */
	COREUOBJECT_API bool IsPayloadLoaded() const;

	/** Returns if the payload would require loading in order to be accessed. Returns false if IsPayloadLoaded or GetPayloadSize() == 0. */
	COREUOBJECT_API bool DoesPayloadNeedLoading() const;

	/** Returns an immutable FCompressedBuffer reference to the payload data. */
	COREUOBJECT_API TFuture<FSharedBuffer> GetPayload() const;

	/**
	 * Returns an immutable FCompressedBuffer reference to the payload data.
	 *
	 * Note that depending on the internal storage formats, the payload might not actually be compressed, but that
	 * will be handled by the FCompressedBuffer interface. Call FCompressedBuffer::Decompress() to get access to
	 * the payload in FSharedBuffer format.
	 */
	COREUOBJECT_API TFuture<FCompressedBuffer> GetCompressedPayload() const;

	/**
	 * Replaces the existing payload (if any) with a new one. 
	 * It is important to consider the ownership model of the payload being passed in to the method.
	 * 
	 * To pass in a raw pointer use 'FSharedBuffer::...(Data, Size)' to create a valid FSharedBuffer.
	 * Use 'FSharedBuffer::MakeView' if you want to retain ownership on the data being passed in, and use
	 * 'FSharedBuffer::TakeOwnership' if you are okay with the bulkdata object taking over ownership of it.
	 * The bulkdata object must own its internal buffer, so if you pass in a non-owned FSharedBuffer (ie
	 * by using 'FSharedBuffer::MakeView') then a clone of the data will be created internally and assigned
	 * to the bulkdata object.
	 *
	 * @param InPayload	The payload that this bulkdata object should reference. @see FSharedBuffer
	 * @param Owner The object that owns the bulkdata, or null to not associate with a UObject.
	 */
	COREUOBJECT_API void UpdatePayload(FSharedBuffer InPayload, UObject* Owner = nullptr);

	/**
	 * Replaces the existing payload (if any) with a new one.
	 * Note that FCompressedBuffers already know the FIoHash of the raw payload and so we do not
	 * recalculate it, making this method much faster for larger payloads than the other overloads.
	 * 
	 * @param InPayload	The payload that this bulkdata object should reference. @see FCompressedBuffer
	 * @param Owner The object that owns the bulkdata, or null to not associate with a UObject.
	 */
	COREUOBJECT_API void UpdatePayload(FCompressedBuffer InPayload, UObject* Owner = nullptr);
	
	/**
	 * Utility struct used to compute the Payload ID before calling UpdatePayload
	 */
	struct FSharedBufferWithID
	{
		COREUOBJECT_API FSharedBufferWithID(FSharedBuffer InPayload);

		FSharedBufferWithID() = default;
		FSharedBufferWithID(FSharedBufferWithID&&) = default;
		FSharedBufferWithID& operator=(FSharedBufferWithID&&) = default;

		FSharedBufferWithID(const FSharedBufferWithID&) = delete;
		FSharedBufferWithID& operator=(const FSharedBufferWithID&) = delete;

		FSharedBuffer GetPayload() { return Payload; }

	private:
		friend FEditorBulkData;

		FSharedBuffer Payload;
		FIoHash PayloadId;
	};

	/**
	 * Allows the existing payload to be replaced with a new one.
	 *
	 * 
	 * To pass in a raw pointer use 'FSharedBuffer::...(Data, Size)' to create a valid FSharedBuffer.
	 * Use 'FSharedBuffer::MakeView' if you want to retain ownership on the data being passed in, and use
	 * 'FSharedBuffer::TakeOwnership' if you are okay with the bulkdata object taking over ownership of it.
	 * The bulkdata object must own its internal buffer, so if you pass in a non-owned FSharedBuffer (ie
	 * by using 'FSharedBuffer::MakeView') then a clone of the data will be created internally and assigned
	 * to the bulkdata object.
	 * 
	 * Use this override if you want compute PayloadId before updating the bulkdata
	 *
	 * @param InPayload	The payload to update the bulkdata with
	 * @param Owner The object that owns the bulkdata, or null to not associate with a UObject.
	 */
	COREUOBJECT_API void UpdatePayload(FSharedBufferWithID InPayload, UObject* Owner = nullptr);

	/** 
	 * Sets the compression options to be applied to the payload during serialization.
	 * 
	 * These settings will continue to be used until the bulkdata object is reset, a subsequent
	 * call to ::SetCompressionOptions is made or the owning package is serialized to disk.
	 * 
	 * @param Option	The high level option to use. @see UE::Serialization::ECompressionOptions
	 */ 
	COREUOBJECT_API void SetCompressionOptions(ECompressionOptions Option);

	/** 
	 * Sets the compression options to be applied to the payload during serialization.
	 * 
	 * These settings will continue to be used until the bulkdata object is reset, a subsequent
	 * call to ::SetCompressionOptions is made or the owning package is serialized to disk.
	 * 
	 * @param Compressor		The Oodle compressor to use. @see ECompressedBufferCompressor
	 * @param CompressionLevel	The Oodle compression level to use. @see ECompressedBufferCompressionLevel
	 */
	COREUOBJECT_API void SetCompressionOptions(ECompressedBufferCompressor Compressor, ECompressedBufferCompressionLevel CompressionLevel);

	/**
	 * Get the versions used in the file containing the payload.
	 *
	 * @param InlineArchive The archive that was used to load this object
	 */
	COREUOBJECT_API void GetBulkDataVersions(FArchive& InlineArchive, FPackageFileVersion& OutUEVersion, int32& OutLicenseeUEVersion,
		FCustomVersionContainer& OutCustomVersions) const;

	/**
	 * Set this BulkData into Torn-Off mode. It will no longer register with the BulkDataRegistry, even if
	 * copied from another BulkData, and it will pass on this flag to any BulkData copied/moved from it.
	 * Use Reset() to remove this state. Torn-off BulkDatas share the guid with the BulkData they copy from.
	 */
	COREUOBJECT_API void TearOff();

	/** Make a torn-off copy of this bulk data. */
	FEditorBulkData CopyTornOff() const
	{ 
		return FEditorBulkData(*this, ETornOff());
	}

	// Functions used by the BulkDataRegistry

	/** Used to serialize the bulkdata to/from a limited cache system used by the BulkDataRegistry. */
	COREUOBJECT_API void SerializeForRegistry(FArchive& Ar);
	
	/** Return true if the bulkdata has a source location that persists between editor processes (package file or virtualization). */
	COREUOBJECT_API bool CanSaveForRegistry() const;
	
	/** Return whether the BulkData has legacy payload id that needs to be updated from loaded payload before it can be used in DDC. */
	bool HasPlaceholderPayloadId() const
	{
		return EnumHasAnyFlags(Flags, EFlags::LegacyKeyWasGuidDerived);
	}
	
	/** Return whether the BulkData is an in-memory payload without a persistent source location. */
	COREUOBJECT_API bool IsMemoryOnlyPayload() const;
	
	/** Load the payload and set the correct payload id, if the bulkdata has a PlaceholderPayloadId. */
	COREUOBJECT_API void UpdatePayloadId();
	
	/** Return whether *this has the same source for the bulkdata (e.g. identical file locations if from file) as Other */
	COREUOBJECT_API bool LocationMatches(const FEditorBulkData& Other) const;

	/**
	 * Update the Owner of this BulkData in the BulkDataRegistry to include the Owner information.
	 * Has no effect if the BulkData is not valid for registration.
	 * The Owner information is lost and this function must be called again if the BulkData's payload information is modified.
	 */
	COREUOBJECT_API void UpdateRegistrationOwner(UObject* Owner);

protected:
	enum class ETornOff {};
	COREUOBJECT_API FEditorBulkData(const FEditorBulkData& Other, ETornOff);

private:
	friend struct FTocEntry;

	/** Flags used to store additional meta information about the bulk data */
	enum class EFlags : uint32
	{
		/** No flags are set */
		None						= 0,
		/** Is the data actually virtualized or not? */
		IsVirtualized				= 1 << 0,
		/** Does the package have access to a .upayload file? */
		HasPayloadSidecarFile		= 1 << 1,
		/** The bulkdata object is currently referencing a payload saved under old bulkdata formats */
		ReferencesLegacyFile		= 1 << 2,
		/** The legacy file being referenced is stored with Zlib compression format */
		LegacyFileIsCompressed		= 1 << 3,
		/** The payload should not have compression applied to it. It is assumed that the payload is already 
			in some sort of compressed format, see the compression documentation above for more details. */
		DisablePayloadCompression	= 1 << 4,
		/** The legacy file being referenced derived its key from guid and it should be replaced with a key-from-hash when saved */
		LegacyKeyWasGuidDerived		= 1 << 5,
		/** (Transient) The Guid has been registered with the BulkDataRegistry */
		HasRegistered				= 1 << 6,
		/** (Transient) The BulkData object is a copy used only to represent the id and payload; it does not communicate with the BulkDataRegistry, and will point DDC jobs toward the original BulkData */
		IsTornOff					= 1 << 7,
		/** The bulkdata object references a payload stored in a WorkspaceDomain file  */
		ReferencesWorkspaceDomain	= 1 << 8,
		/** The payload is stored in a package trailer, so the bulkdata object will have to poll the trailer to find the payload offset */
		StoredInPackageTrailer		= 1 << 9,
		/** The bulkdata object was cooked. */
		IsCooked					= 1 << 10,
		/** (Transient) The package owning the bulkdata has been detached from disk and we can no longer load from it */
		WasDetached					= 1 << 11
	};

	FRIEND_ENUM_CLASS_FLAGS(EFlags);

	/** A common grouping of EFlags */
	static constexpr EFlags TransientFlags = EFlags((uint32)EFlags::HasRegistered | (uint32)EFlags::IsTornOff | (uint32)EFlags::WasDetached);

	/** Used to control what level of error reporting we return from some methods */
	enum ErrorVerbosity
	{
		/** No errors should be logged */
		None = 0,
		/** Everything should be logged */
		All
	};

	/** Old legacy path that saved the payload to the end of the package */
	COREUOBJECT_API void SerializeToLegacyPath(FLinkerSave& LinkerSave, FCompressedBuffer PayloadToSerialize, EFlags UpdatedFlags, UObject* Owner);
	/** The new path that saves payloads to the FPackageTrailer which is then appended to the end of the package file */
	COREUOBJECT_API void SerializeToPackageTrailer(FLinkerSave& LinkerSave, FCompressedBuffer PayloadToSerialize, EFlags UpdatedFlags, UObject* Owner);

	COREUOBJECT_API void UpdatePayloadImpl(FSharedBuffer&& InPayload, const FIoHash& InPayloadID, UObject* Owner);

	COREUOBJECT_API FCompressedBuffer GetDataInternal() const;

	COREUOBJECT_API FCompressedBuffer LoadFromDisk() const;
	COREUOBJECT_API FCompressedBuffer LoadFromPackageFile() const;
	COREUOBJECT_API FCompressedBuffer LoadFromPackageTrailer() const;
	COREUOBJECT_API FCompressedBuffer LoadFromSidecarFile() const;
	COREUOBJECT_API FCompressedBuffer LoadFromSidecarFileInternal(ErrorVerbosity Verbosity) const;

	COREUOBJECT_API bool SerializeData(FArchive& Ar, FCompressedBuffer& Payload, const EFlags PayloadFlags) const;

	COREUOBJECT_API void PushData(const FPackagePath& InPackagePath);
	COREUOBJECT_API FCompressedBuffer PullData() const;

	COREUOBJECT_API bool CanUnloadData() const;
	COREUOBJECT_API bool CanLoadDataFromDisk() const;

	COREUOBJECT_API void UpdateKeyIfNeeded();
	COREUOBJECT_API void UpdateKeyIfNeeded(FCompressedBuffer InPayload) const;

	COREUOBJECT_API void RecompressForSerialization(FCompressedBuffer& InOutPayload, EFlags PayloadFlags) const;
	COREUOBJECT_API EFlags BuildFlagsForSerialization(FArchive& Ar, bool bKeepFileDataByReference) const;

	bool IsDataVirtualized() const 
	{ 
		return IsDataVirtualized(Flags);
	}

	static bool IsDataVirtualized(EFlags InFlags)
	{
		return EnumHasAnyFlags(InFlags, EFlags::IsVirtualized);
	}

	bool HasPayloadSidecarFile() const 
	{ 
		return EnumHasAnyFlags(Flags, EFlags::HasPayloadSidecarFile); 
	}

	bool IsReferencingOldBulkData() const
	{ 
		return IsReferencingOldBulkData(Flags);
	}
	static bool IsReferencingOldBulkData(EFlags InFlags)
	{
		return EnumHasAnyFlags(InFlags, EFlags::ReferencesLegacyFile);
	}

	bool IsReferencingWorkspaceDomain() const
	{
		return IsReferencingWorkspaceDomain(Flags);
	}
	static bool IsReferencingWorkspaceDomain(EFlags InFlags)
	{
		return EnumHasAnyFlags(InFlags, EFlags::ReferencesWorkspaceDomain);
	}

	bool IsReferencingByPackagePath() const
	{
		return IsReferencingByPackagePath(Flags);
	}
	static bool IsReferencingByPackagePath(EFlags InFlags)
	{
		return EnumHasAnyFlags(InFlags, EFlags::ReferencesLegacyFile | EFlags::ReferencesWorkspaceDomain);
	}

	bool IsStoredInPackageTrailer() const
	{
		return IsStoredInPackageTrailer(Flags);
	}
	static bool IsStoredInPackageTrailer(EFlags InFlags)
	{
		return EnumHasAnyFlags(InFlags, EFlags::StoredInPackageTrailer);
	}

	/** 
	 * Returns true when the bulkdata has an attachment to it's package file on disk, if the bulkdata later becomes detached
	 * then EFlag::WasDetached will be set.
	 */
	bool HasAttachedArchive() const
	{
		return AttachedAr != nullptr;
	}

	COREUOBJECT_API void Register(UObject* Owner, const TCHAR* LogCallerName, bool bAllowUpdateId);
	/** Used when we need to clear the registration for our bulkdataId because its ownership is transferring elsewhere. */
	COREUOBJECT_API void Unregister();
	/**
	 * Used when we want to keep registration of our BulkDataId tied to our Payload location, but this BulkData is exiting memory
	 * and the registry needs to know so it can drop its copy of the payload and allow us to reregister later if reloaded.
	 */
	COREUOBJECT_API void OnExitMemory();
	/** Check whether we should be registered, and register/unregister/update as necessary to match. */
	COREUOBJECT_API void UpdateRegistrationData(UObject* Owner, const TCHAR* LogCallerName, bool bAllowUpdateId);
	COREUOBJECT_API void LogRegisterError(UE::BulkDataRegistry::ERegisterResult Value, UObject* Owner, const FGuid& FailedBulkDataId,
		const TCHAR* CallerName, bool bHandledbyCreateUniqueGuid) const;

	/**
	 * Checks to make sure that the payload we are saving is what we expect it to be. If not then we need to log an error to the
	 * user, mark the archive as having an error and return.
	 * This error is not reasonable to expect, if it occurs then it means something has gone very wrong, which is why there is
	 * also an ensure to make sure that any instances of it occurring as properly recorded and can be investigated.
	 */
	COREUOBJECT_API bool TryPayloadValidationForSaving(const FCompressedBuffer& PayloadForSaving, FLinkerSave* LinkerSave) const;

	/** 
	 * Utility to return an apt error message if the payload is invalid when trying to save the bulkdata. It will try to provide the best info from the given options.
	 * 
	 * @param Linker	The FLinkerSave being used to save the bulkdata at the point of error. This can be nullptr if the bulkdata is not being saved to a package on disk.
	 * @return			The formatted error message.
	 */
	COREUOBJECT_API FText GetCorruptedPayloadErrorMsgForSave(FLinkerSave* Linker) const;

	/**
	 * A utility for validating that a package trailer builder was created correctly. If a problem is encountered we will assert
	 * to prevent a corrupted package from being saved.
	 * The main thing we check is that the payload was added to the correct list for the correct payload storage type based on
	 * the flags we have for the payload.
	 * Note that it is not expected that we will ever encounter these problems (so asserting is acceptable) and the checks could
	 * be considered a little over cautious. We should consider just removing this check in 5.2 onwards.
	 * 
	 * @param LinkerSave	The linker containing the package trailer builder
	 * @param Id			The hash of the payload we want to verify
	 * @param PayloadFlags	The flags for the payload.
	 */
	static COREUOBJECT_API void ValidatePackageTrailerBuilder(const FLinkerSave* LinkerSave, const FIoHash& Id, EFlags PayloadFlags);

	/** Returns true if we should use legacy serialization instead of the FPackageTrailer system. */
	COREUOBJECT_API bool ShouldUseLegacySerialization(const FLinkerSave* LinkerSave) const;

	/** Unique identifier for the bulkdata object itself */
	FGuid BulkDataId;

	/** Unique identifier for the contents of the payload*/
	FIoHash PayloadContentId;

	/** Pointer to the payload if it is held in memory (it has been updated but not yet saved to disk for example) */
	FSharedBuffer Payload;

	/** Length of the payload in bytes */
	int64 PayloadSize = 0;

	//---- The remaining members are used when the payload is not virtualized.
	
	/** The archive representing the file on disk containing the payload (if there is one), we keep the pointer so that the bulkdata object can be detached if needed. */
	FArchive* AttachedAr = nullptr;

	/** Offset of the payload in the file that contains it (INDEX_NONE if the payload does not come from a file)*/
	int64 OffsetInFile = INDEX_NONE;

	/** PackagePath containing the payload (this will be empty if the payload does not come from PackageResourceManager)*/
	FPackagePath PackagePath;

	/** A 32bit bitfield of flags */
	EFlags Flags = EFlags::None;

	// Use of IBulkDataRegistry requires a recursive mutex
	mutable FRecursiveMutex Mutex;

	/** 
	 * Compression settings to be applied to the payload when the package is next saved. The settings will be reset if
	 * the payload is unloaded from memory during serialization (i.e. the payload was virtualized or the package was
	 * saved to disk.&
	 */
	Private::FCompressionSettings CompressionSettings;
};

ENUM_CLASS_FLAGS(FEditorBulkData::EFlags);

/** 
 * Returns a FGuid representation of a FIoHash, to be used in existing code paths that require the id of 
 * an FEditorBulkData payload to be in FGuid form 
 */
COREUOBJECT_API FGuid IoHashToGuid(const FIoHash& Hash);

} // namespace UE::Serialization

//#endif //WITH_EDITORONLY_DATA
