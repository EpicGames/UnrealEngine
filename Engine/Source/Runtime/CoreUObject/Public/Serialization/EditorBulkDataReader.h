// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/BufferReader.h"
#include "Serialization/EditorBulkData.h"

#if WITH_EDITORONLY_DATA

namespace UE::Serialization
{

namespace Private
{

/** Wraps access to the FEditorBulkData data for FEditorBulkDataReader
so that it can be done before the FBufferReaderBase constructor is called */
class DataAccessWrapper
{
protected:
	DataAccessWrapper(FEditorBulkData& InBulkData)
		: Payload(InBulkData.GetPayload().Get())

	{

	}

	virtual ~DataAccessWrapper() = default;

	bool IsValid() const
	{
		return !Payload.IsNull();
	}

	void* GetData() const
	{
		// It's okay to remove the const qualifier here as it will only be passed
		// on to FBufferReaderBase, which will not change the data at all, but takes a non-const
		// pointer so that it can free the memory if requested, which we don't.
		return const_cast<void*>(Payload.GetData());
	}

	int64 GetDataLength() const
	{
		return Payload.GetSize();
	}

private:
	FSharedBuffer Payload;
};

} // namespace Private

class FEditorBulkDataReader : protected Private::DataAccessWrapper, public FBufferReaderBase
{
public:
	FEditorBulkDataReader(FEditorBulkData& InBulkData, bool bIsPersistent = false)
		: DataAccessWrapper(InBulkData)
		, FBufferReaderBase(GetData(), GetDataLength(), false, bIsPersistent)
	{
	}

	virtual ~FEditorBulkDataReader() = default;

	/** Returns if the FEditorBulkDataReader has a valid bulkdata payload or not */
	bool IsValid() const
	{
		return Private::DataAccessWrapper::IsValid();
	}

	using FArchive::operator<<; // For visibility of the overloads we don't override

	virtual FArchive& operator<<(class FName& Name) override
	{
		// FNames are serialized as strings in BulkData
		FString StringName;
		*this << StringName;
		Name = FName(*StringName);
		return *this;
	}

	virtual FString GetArchiveName() const
	{
		return TEXT("FEditorBulkDataReader");
	}
};

} // namespace UE::Serialization

#endif //WITH_EDITORONLY_DATA
