// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/EditorBulkData.h"

#if WITH_EDITORONLY_DATA

namespace UE::Serialization
{

class FEditorBulkDataWriter : public FArchive
{
public:
	FEditorBulkDataWriter(FEditorBulkData& InBulkData, bool bIsPersistent = false)
		: BulkData(InBulkData)
	{
		SetIsSaving(true);
		SetIsPersistent(bIsPersistent);

		FSharedBuffer Payload = InBulkData.GetPayload().Get();

		if (Payload)
		{
			const int64 CurrentDataLength = Payload.GetSize();

			// Clone the payload so that we have a local copy that we can
			// append additional data to.
			Buffer = FMemory::Malloc(CurrentDataLength, DEFAULT_ALIGNMENT);
			FMemory::Memcpy(Buffer, Payload.GetData(), CurrentDataLength);

			BufferLength = CurrentDataLength;

			// Start at the end of the existing data
			CurPos = CurrentDataLength;
			DataLength = CurrentDataLength;
		}
		else
		{
			Buffer = nullptr;
			BufferLength = 0;

			CurPos = 0;
			DataLength = 0;
		}
	}

	~FEditorBulkDataWriter()
	{
		// Remove the slack from the allocated bulk data
		Buffer = FMemory::Realloc(Buffer, DataLength, DEFAULT_ALIGNMENT);
		BulkData.UpdatePayload(FSharedBuffer::TakeOwnership(Buffer, DataLength, FMemory::Free));
	}

	/** Returns if the FEditorBulkDataWriter has a valid bulkdata payload or not */
	bool IsValid() const
	{
		return Buffer != nullptr;
	}

	virtual void Serialize(void* Data, int64 Num)
	{
		// Determine if we need to reallocate the buffer to fit the next item
		const int64 NewPos = CurPos + Num;
		checkf(NewPos >= CurPos, TEXT("Serialization has somehow gone backwards"));

		if (NewPos > BufferLength)
		{
			// If so, resize to the new size + 3/8 additional slack
			const int64 NewLength = NewPos + 3 * NewPos / 8 + 16;
			Buffer = FMemory::Realloc(Buffer, NewLength, DEFAULT_ALIGNMENT);
			BufferLength = NewLength;
		}

		FMemory::Memcpy(static_cast<unsigned char*>(Buffer) + CurPos, Data, Num);

		CurPos += Num;
		DataLength = FMath::Max(DataLength, CurPos);
	}

	using FArchive::operator<<; // For visibility of the overloads we don't override

	virtual FArchive& operator<<(class FName& Name) override
	{
		// FNames are serialized as strings in BulkData
		FString StringName = Name.ToString();
		*this << StringName;
		return *this;
	}

	virtual int64 Tell() { return CurPos; }
	virtual int64 TotalSize() { return DataLength; }

	virtual void Seek(int64 InPos)
	{
		check(InPos >= 0);
		check(InPos <= DataLength);
		CurPos = InPos;
	}

	virtual bool AtEnd()
	{
		return CurPos >= DataLength;
	}

	virtual FString GetArchiveName() const
	{
		return TEXT("FEditorBulkDataWriter");
	}

protected:
	/** The target bulkdata object */
	FEditorBulkData& BulkData;

	/** Pointer to the data buffer */
	void* Buffer;
	/** Length of the data buffer (stored as bytes) */
	int64 BufferLength;

	/** Current position in the buffer for serialization operations */
	int64 CurPos;

	/** The length of valid data in the data buffer (stored as bytes) */
	int64 DataLength;
};

} // namespace UE::Serialization

#endif //WITH_EDITORONLY_DATA
