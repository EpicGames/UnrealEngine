// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "Serialization/FindReferencersArchive.h"

/**
 * This class is used to find which objects reference any element from a list of "TargetObjects".  When invoked,
 * it will generate a mapping of each target object to an array of objects referencing that target object.
 *
 * Each key corresponds to an element of the input TargetObjects array which was referenced
 * by some other object.  The values for these keys are the objects which are referencing them.
 */
template< class T >
class TFindObjectReferencers : public TMultiMap<T*, UObject*>
{
public:

	/**
	 * Default constructor
	 *
	 * @param	TargetObjects	the list of objects to find references to
	 * @param	PackageToCheck	if specified, only objects contained in this package will be searched
	 *							for references to 
	 * @param	bIgnoreTemplates If true, do not record template objects
	 * @param	bFindAlsoWeakReferences If true, also look into weak references
	 */
	TFindObjectReferencers( TArray< T* > TargetObjects, UPackage* PackageToCheck=NULL, bool bIgnoreTemplates = true, bool bFindAlsoWeakReferences = false)
	: TMultiMap< T*, UObject* >()
	{
		TArray<UObject*> ReferencedObjects;
		TMap<UObject*, int32> ReferenceCounts;

		FFindReferencersArchive FindReferencerAr(nullptr, (TArray<UObject*>&)TargetObjects, bFindAlsoWeakReferences);

		// Loop over every object to find any reference that may exist for the target objects
		for (FThreadSafeObjectIterator It; It; ++It)
		{
			UObject* PotentialReferencer = *It;
			if ( !TargetObjects.Contains(PotentialReferencer)
			&&	(PackageToCheck == NULL || PotentialReferencer->IsIn(PackageToCheck))
			&&	(!bIgnoreTemplates || !PotentialReferencer->IsTemplate()) )
			{
				FindReferencerAr.ResetPotentialReferencer(PotentialReferencer);

				ReferenceCounts.Reset();
				if ( FindReferencerAr.GetReferenceCounts(ReferenceCounts) > 0 )
				{
					// here we don't really care about the number of references from PotentialReferencer to the target object...just that it's a referencer
					ReferencedObjects.Reset();
					ReferenceCounts.GenerateKeyArray(ReferencedObjects);
					for ( int32 RefIndex = 0; RefIndex < ReferencedObjects.Num(); RefIndex++ )
					{
						this->Add(static_cast<T*>(ReferencedObjects[RefIndex]), PotentialReferencer);
					}
				}
			}
		}
	}

private:
	/**
	 * This is a mapping of TargetObjects to the list of objects which references each one combined with
	 * the list of properties which are holding the reference to the TargetObject in that referencer.
	 *
	 * @todo - not yet implemented
	 */
//	TMap< T*, TMultiMap<UObject*, FProperty*> >		ReferenceProperties;
};
