// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/ArchiveUObject.h"

class FArchive;
class UObject;

/*----------------------------------------------------------------------------
	FFindReferencersArchive.
----------------------------------------------------------------------------*/
/**
 * Archive for mapping out the referencers of a collection of objects.
 */
class FFindReferencersArchive : public FArchiveUObject
{
public:
	/**
	 * Constructor
	 *
	 * @param	PotentialReferencer		the object to serialize which may contain references to our target objects
	 * @param	InTargetObjects			array of objects to search for references to
	 * @param	bFindAlsoWeakReferences should we also look into weak references?
	 */
	COREUOBJECT_API FFindReferencersArchive(class UObject* PotentialReferencer, TArrayView<class UObject*> InTargetObjects, bool bFindAlsoWeakReferences = false);

	/**
	 * Retrieves the number of references from PotentialReferencer to the object specified.
	 *
	 * @param	TargetObject	the object to might be referenced
	 * @param	out_ReferencingProperties
	 *							receives the list of properties which were holding references to TargetObject
	 *
	 * @return	the number of references to TargetObject which were encountered when PotentialReferencer
	 *			was serialized.
	 */
	COREUOBJECT_API int32 GetReferenceCount( class UObject* TargetObject, TArray<class FProperty*>* out_ReferencingProperties=NULL ) const;

	/**
	 * Retrieves the number of references from PotentialReferencer list of TargetObjects
	 *
	 * @param	out_ReferenceCounts		receives the number of references to each of the TargetObjects
	 *
	 * @return	the number of objects which were referenced by PotentialReferencer.
	 */
	COREUOBJECT_API int32 GetReferenceCounts( TMap<class UObject*, int32>& out_ReferenceCounts ) const;

	/** GetReferenceCounts without empyting out map */
	COREUOBJECT_API int32 AppendReferenceCounts( TMap<class UObject*, int32>& out_ReferenceCounts ) const;

	/**
	 * Retrieves the number of references from PotentialReferencer list of TargetObjects
	 *
	 * @param	out_ReferenceCounts			receives the number of references to each of the TargetObjects
	 * @param	out_ReferencingProperties	receives the map of properties holding references to each referenced object.
	 *
	 * @return	the number of objects which were referenced by PotentialReferencer.
	 */
	COREUOBJECT_API int32 GetReferenceCounts( TMap<class UObject*, int32>& out_ReferenceCounts, TMultiMap<class UObject*,class FProperty*>& out_ReferencingProperties ) const;

	/**
	 * Returns the name of the Archive.  Useful for getting the name of the package a struct or object
	 * is in when a loading error occurs.
	 *
	 * This is overridden for the specific Archive Types
	 **/
	virtual FString GetArchiveName() const { return TEXT("FFindReferencersArchive"); }

	/**
	 * Resets the reference counts.  Keeps the same target objects but sets up everything to test a new potential referencer.
	 * @param	PotentialReferencer		the object to serialize which may contain references to our target objects
	 **/
	COREUOBJECT_API void ResetPotentialReferencer(UObject* InPotentialReferencer);

protected:
	// Container specifically optimized for the operations we're doing here.
	//   - Reduce allocations while adding.
	//   - Reduce cache misses while searching.
	//   - Fast to reset its values as they're all contiguous in memory.
	//   - Reduce iteration count to initialized values only when searching for values > 0 by stopping at ValueNum().
	class FTargetObjectContainer
	{
	public:
		// Functions used to prepare the container until it is frozen

		void Reserve(int32 Num)
		{
			CheckUnfrozen();

			TargetObjects.Reserve(Num);
		}

		void AddObject(UObject* Object)
		{
			CheckUnfrozen();

			TargetObjects.Add(Object);
		}

		void Freeze()
		{
			CheckUnfrozen();

			bFrozen = true;
			Algo::Sort(TargetObjects);
			ResetRefCounts();
		}

		// Functions to use once the container has been frozen

		// This will initialize and return the refcount associated with the object if it exists
		int32* GetRefCountPtr(UObject* Object)
		{
			CheckFrozen();

			int32 ExistingIndex = Algo::BinarySearch(TargetObjects, Object);
			if (ExistingIndex == INDEX_NONE)
			{
				return nullptr;
			}

			if (ExistingIndex >= RefCounts.Num())
			{
				RefCounts.SetNumZeroed(ExistingIndex+1, EAllowShrinking::No);
			}

			return &RefCounts[ExistingIndex];
		}

		// This won't initialize the refcount associated with the object even if it exists
		const int32* TryGetRefCountPtr(UObject* Object) const
		{
			CheckFrozen();

			int32 ExistingIndex = Algo::BinarySearch(TargetObjects, Object);
			if (ExistingIndex == INDEX_NONE)
			{
				return nullptr;
			}

			return (ExistingIndex < RefCounts.Num()) ? &RefCounts[ExistingIndex] : nullptr;
		}

		void ResetRefCounts()
		{
			RefCounts.Empty(TargetObjects.Num());
		}

		int32 RefCountNum() const
		{
			return RefCounts.Num();
		}

		UObject* GetObject(int32 Index) const
		{
			return TargetObjects[Index];
		}

		// This should not be queried past RefCountNum(), otherwise you're doing useless work.
		int32 GetRefCount(int32 Index) const
		{
			return RefCounts[Index];
		}

	private:
		UE_FORCEINLINE_HINT void CheckFrozen() const
		{
			checkf(bFrozen, TEXT("Container has not been frozen and cannot be searched yet"));
		}

		UE_FORCEINLINE_HINT void CheckUnfrozen() const
		{
			checkf(!bFrozen, TEXT("Container has been frozen and cannot be modified anymore"));
		}

		bool              bFrozen = false;
		TArray<UObject*>  TargetObjects;
		TArray<int32>     RefCounts;
	};

	FTargetObjectContainer TargetObjects;

	/** a mapping of target object => the properties in PotentialReferencer that hold the reference to target object */
	TMultiMap<class UObject*,class FProperty*> ReferenceMap;

	/** The potential referencer we ignore */
	class UObject* PotentialReferencer;

private:

	/**
	 * Serializer - if Obj is one of the objects we're looking for, increments the reference count for that object
	 */
	COREUOBJECT_API FArchive& operator<<( class UObject*& Obj );
};
