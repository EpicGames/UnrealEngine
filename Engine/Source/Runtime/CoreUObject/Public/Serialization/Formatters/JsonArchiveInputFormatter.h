// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Containers/SharedString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/StructuredArchiveFormatter.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectResource.h"
#include "Dom/JsonObject.h"

class FArchive;
class FJsonObject;
class FJsonValue;
class FName;
class FPackageIndex;
class FText;
class UObject;
struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;
struct FWeakObjectPtr;

#if WITH_TEXT_ARCHIVE_SUPPORT

class FJsonArchiveInputFormatter final : public FStructuredArchiveFormatter
{
public:
	// Noncopyable
	FJsonArchiveInputFormatter(FJsonArchiveInputFormatter&&) = delete;
	FJsonArchiveInputFormatter(const FJsonArchiveInputFormatter&) = delete;
	FJsonArchiveInputFormatter& operator=(FJsonArchiveInputFormatter&&) = delete;
	FJsonArchiveInputFormatter& operator=(const FJsonArchiveInputFormatter&) = delete;

	COREUOBJECT_API explicit FJsonArchiveInputFormatter(FArchive& InInner, TFunction<UObject* (const FPackageIndex)> InResolveObject = nullptr);
	COREUOBJECT_API virtual ~FJsonArchiveInputFormatter();

	COREUOBJECT_API virtual FArchive& GetUnderlyingArchive() override;
	COREUOBJECT_API virtual FStructuredArchiveFormatter* CreateSubtreeReader() override;

	COREUOBJECT_API virtual bool HasDocumentTree() const override;

	COREUOBJECT_API virtual void EnterRecord() override;
	COREUOBJECT_API virtual void LeaveRecord() override;
	COREUOBJECT_API virtual void EnterField(FArchiveFieldName Name) override;
	COREUOBJECT_API virtual void LeaveField() override;
	COREUOBJECT_API virtual bool TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving) override;

	COREUOBJECT_API virtual void EnterArray(int32& NumElements) override;
	COREUOBJECT_API virtual void LeaveArray() override;
	COREUOBJECT_API virtual void EnterArrayElement() override;
	COREUOBJECT_API virtual void LeaveArrayElement() override;

	COREUOBJECT_API virtual void EnterStream() override;
	COREUOBJECT_API virtual void LeaveStream() override;
	COREUOBJECT_API virtual void EnterStreamElement() override;
	COREUOBJECT_API virtual void LeaveStreamElement() override;

	COREUOBJECT_API virtual void EnterMap(int32& NumElements) override;
	COREUOBJECT_API virtual void LeaveMap() override;
	COREUOBJECT_API virtual void EnterMapElement(FString& Name) override;
	COREUOBJECT_API virtual void LeaveMapElement() override;

	COREUOBJECT_API virtual void EnterAttributedValue() override;
	COREUOBJECT_API virtual void EnterAttribute(FArchiveFieldName AttributeName) override;
	COREUOBJECT_API virtual void EnterAttributedValueValue() override;
	COREUOBJECT_API virtual void LeaveAttribute() override;
	COREUOBJECT_API virtual void LeaveAttributedValue() override;
	COREUOBJECT_API virtual bool TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSavin) override;
	COREUOBJECT_API virtual bool TryEnterAttributedValueValue() override;

	COREUOBJECT_API virtual void Serialize(uint8& Value) override;
	COREUOBJECT_API virtual void Serialize(uint16& Value) override;
	COREUOBJECT_API virtual void Serialize(uint32& Value) override;
	COREUOBJECT_API virtual void Serialize(uint64& Value) override;
	COREUOBJECT_API virtual void Serialize(int8& Value) override;
	COREUOBJECT_API virtual void Serialize(int16& Value) override;
	COREUOBJECT_API virtual void Serialize(int32& Value) override;
	COREUOBJECT_API virtual void Serialize(int64& Value) override;
	COREUOBJECT_API virtual void Serialize(float& Value) override;
	COREUOBJECT_API virtual void Serialize(double& Value) override;
	COREUOBJECT_API virtual void Serialize(bool& Value) override;
	COREUOBJECT_API virtual void Serialize(UTF32CHAR& Value) override;
	COREUOBJECT_API virtual void Serialize(FString& Value) override;
	COREUOBJECT_API virtual void Serialize(FName& Value) override;
	COREUOBJECT_API virtual void Serialize(UObject*& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	COREUOBJECT_API virtual void Serialize(Verse::VCell*& Value) override;
#endif
	COREUOBJECT_API virtual void Serialize(FText& Value) override;
	COREUOBJECT_API virtual void Serialize(FWeakObjectPtr& Value) override;
	COREUOBJECT_API virtual void Serialize(FSoftObjectPtr& Value) override;
	COREUOBJECT_API virtual void Serialize(FSoftObjectPath& Value) override;
	COREUOBJECT_API virtual void Serialize(FLazyObjectPtr& Value) override;
	COREUOBJECT_API virtual void Serialize(FObjectPtr& Value) override;
	COREUOBJECT_API virtual void Serialize(TArray<uint8>& Value) override;
	COREUOBJECT_API virtual void Serialize(void* Data, uint64 DataSize) override;

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	COREUOBJECT_API void SetResolveCell(TFunction<Verse::VCell* (const FPackageIndex)>&& InResolveCell);
#endif

private:
	FArchive& Inner;

	// Used for the subtree parser
	FJsonArchiveInputFormatter(FArchive& InInner, TFunction<UObject* (const FPackageIndex)>&& InResolveObject, TSharedPtr<FJsonValue>&& RootObject);

	struct FObjectRecord
	{
		FObjectRecord(TSharedPtr<FJsonObject> InJsonObject, int64 InValueCount)
			: JsonObject(InJsonObject)
			, ValueCountOnCreation(InValueCount)
		{

		}

		TSharedPtr<FJsonObject> JsonObject;
		int64 ValueCountOnCreation;	// For debugging purposes, so we can ensure all values have been consumed
	};

	TFunction<UObject* (const FPackageIndex)> ResolveObject;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	TFunction<Verse::VCell* (const FPackageIndex)> ResolveCell;
#endif
	TArray<FObjectRecord> ObjectStack;
	TArray<TSharedPtr<FJsonValue>> ValueStack;
	TArray<TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>::TConstIterator> MapIteratorStack;
	TArray<int32> ArrayValuesRemainingStack;

	static FString EscapeFieldName(const TCHAR* Name);
	static FString UnescapeFieldName(const TCHAR* Name);

	static EArchiveValueType GetValueType(const FJsonValue &Value);
};

#endif
