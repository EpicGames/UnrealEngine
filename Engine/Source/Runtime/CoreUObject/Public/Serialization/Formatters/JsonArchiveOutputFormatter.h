// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Serialization/StructuredArchiveFormatter.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "UObject/ObjectResource.h"

class FArchive;
class FName;
class FPackageIndex;
class FText;
class UObject;
struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;
struct FWeakObjectPtr;

#if WITH_TEXT_ARCHIVE_SUPPORT

class FJsonArchiveOutputFormatter final : public FStructuredArchiveFormatter
{
public:
	// Noncopyable
	FJsonArchiveOutputFormatter(FJsonArchiveOutputFormatter&&) = delete;
	FJsonArchiveOutputFormatter(const FJsonArchiveOutputFormatter&) = delete;
	FJsonArchiveOutputFormatter& operator=(FJsonArchiveOutputFormatter&&) = delete;
	FJsonArchiveOutputFormatter& operator=(const FJsonArchiveOutputFormatter&) = delete;

	COREUOBJECT_API explicit FJsonArchiveOutputFormatter(FArchive& InInner);
	COREUOBJECT_API virtual ~FJsonArchiveOutputFormatter();

	COREUOBJECT_API virtual FArchive& GetUnderlyingArchive() override;

	COREUOBJECT_API virtual bool HasDocumentTree() const override;

	COREUOBJECT_API virtual void EnterRecord() override;
	COREUOBJECT_API virtual void LeaveRecord() override;
	COREUOBJECT_API virtual void EnterField(FArchiveFieldName Name) override;
	COREUOBJECT_API virtual void LeaveField() override;
	COREUOBJECT_API virtual bool TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving) override;

	COREUOBJECT_API virtual void EnterArray(int32& NumElements) override;
	COREUOBJECT_API virtual void LeaveArray() override;
	COREUOBJECT_API virtual void EnterArrayElement() override;
	COREUOBJECT_API virtual void LeaveArrayElement() override;

	COREUOBJECT_API virtual void EnterStream() override;
	COREUOBJECT_API virtual void LeaveStream() override;
	COREUOBJECT_API virtual void EnterStreamElement() override;
	COREUOBJECT_API virtual void LeaveStreamElement() override;

	COREUOBJECT_API virtual void EnterMap(int32& NumElements) override;
	COREUOBJECT_API virtual void LeaveMap() override;
	COREUOBJECT_API virtual void EnterMapElement(FString& Name) override;
	COREUOBJECT_API virtual void LeaveMapElement() override;

	COREUOBJECT_API virtual void EnterAttributedValue() override;
	COREUOBJECT_API virtual void EnterAttribute(FArchiveFieldName AttributeName) override;
	COREUOBJECT_API virtual void EnterAttributedValueValue() override;
	COREUOBJECT_API virtual void LeaveAttribute() override;
	COREUOBJECT_API virtual void LeaveAttributedValue() override;
	COREUOBJECT_API virtual bool TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving) override;

	COREUOBJECT_API virtual bool TryEnterAttributedValueValue() override;

	COREUOBJECT_API virtual void Serialize(uint8& Value) override;
	COREUOBJECT_API virtual void Serialize(uint16& Value) override;
	COREUOBJECT_API virtual void Serialize(uint32& Value) override;
	COREUOBJECT_API virtual void Serialize(uint64& Value) override;
	COREUOBJECT_API virtual void Serialize(int8& Value) override;
	COREUOBJECT_API virtual void Serialize(int16& Value) override;
	COREUOBJECT_API virtual void Serialize(int32& Value) override;
	COREUOBJECT_API virtual void Serialize(int64& Value) override;
	COREUOBJECT_API virtual void Serialize(float& Value) override;
	COREUOBJECT_API virtual void Serialize(double& Value) override;
	COREUOBJECT_API virtual void Serialize(bool& Value) override;
	COREUOBJECT_API virtual void Serialize(UTF32CHAR& Value) override;
	COREUOBJECT_API virtual void Serialize(FString& Value) override;
	COREUOBJECT_API virtual void Serialize(FName& Value) override;
	COREUOBJECT_API virtual void Serialize(UObject*& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	COREUOBJECT_API virtual void Serialize(Verse::VCell*& Value) override;
#endif
	COREUOBJECT_API virtual void Serialize(FText& Value) override;
	COREUOBJECT_API virtual void Serialize(FWeakObjectPtr& Value) override;
	COREUOBJECT_API virtual void Serialize(FSoftObjectPtr& Value) override;
	COREUOBJECT_API virtual void Serialize(FSoftObjectPath& Value) override;
	COREUOBJECT_API virtual void Serialize(FLazyObjectPtr& Value) override;
	COREUOBJECT_API virtual void Serialize(FObjectPtr& Value) override;
	COREUOBJECT_API virtual void Serialize(TArray<uint8>& Value) override;
	COREUOBJECT_API virtual void Serialize(void* Data, uint64 DataSize) override;

	void SetObjectIndicesMap(const TMap<TObjectPtr<UObject>, FPackageIndex>* InObjectIndicesMap)
	{
		ObjectIndicesMap = InObjectIndicesMap;
	}
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	void SetCellIndicesMap(const TMap<Verse::VCell*, FPackageIndex>* InCellIndicesMap)
	{
		CellIndicesMap = InCellIndicesMap;
	}
#endif

private:
	FArchive& Inner;

	const TMap<TObjectPtr<UObject>, FPackageIndex>* ObjectIndicesMap = nullptr;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	const TMap<Verse::VCell*, FPackageIndex>* CellIndicesMap = nullptr;
#endif

	TArray<ANSICHAR> Newline;
	bool bNeedsComma   = false;
	bool bNeedsNewline = false;

	TArray<int32> NumAttributesStack;
	TArray<int64> TextStartPosStack;

	COREUOBJECT_API void Write(ANSICHAR Character);

	COREUOBJECT_API void Write(const ANSICHAR* Text);
	COREUOBJECT_API void Write(const FString& Text);

	COREUOBJECT_API void WriteFieldName(const TCHAR* Name);
	COREUOBJECT_API void WriteValue(const FString& Value);

	COREUOBJECT_API void WriteOptionalComma();
	COREUOBJECT_API void WriteOptionalNewline();
	COREUOBJECT_API void WriteOptionalAttributedBlockOpening();
	COREUOBJECT_API void WriteOptionalAttributedBlockValue();
	COREUOBJECT_API void WriteOptionalAttributedBlockClosing();

	COREUOBJECT_API void SerializeStringInternal(const FString& String);

	COREUOBJECT_API bool IsObjectAllowed(UObject* InObject) const;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	COREUOBJECT_API bool IsCellAllowed(Verse::VCell* InCell) const;
#endif
};

#endif
