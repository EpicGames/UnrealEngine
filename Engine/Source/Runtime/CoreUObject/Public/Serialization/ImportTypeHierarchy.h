// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/NameTypes.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveSlots.h"

/*
* FImportTypeHierarchy are meta-data entries stored as a map in package files (see FPackageFileSummary).
* Their purpose is to store the full type hierarchy of a package Import entry that is a Struct-based object.
* NOTE1: This is internal-engine data that should not be used by user code. The format of the data is also subject to change.
* NOTE2: The class hierarchy stored is a snapshot of the type when the object was saved. It might not correspond to the actual 
		type hierarchy when the object is loaded.
* NOTE3: UObject::StaticClass is omitted from the SuperTypes array as it is implied.
*/
namespace UE::Serialization::Private
{

/** Stores object information for a type that is part of a FImportTypeHierarchy. */
struct FTypeResource
{
	FName TypeName;
	// Package where the type object lives.
	FName PackageName;
	// This is the Class 'kind' for the type (e.g., "Class", "VerseClass", "BlueprintGeneratedClass", "UserDefinedStruct", etc).
	FName ClassName;
	// Package where the Class 'kind' object lives.
	FName ClassPackageName;

	/** I/O functions */
	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FTypeResource& R);
	friend COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, FTypeResource& R);
};

/** Stores a snapshot of hierarchical type information for a Struct import entry. */
class FImportTypeHierarchy
{
	TArray<FTypeResource> SuperTypes;

public:

	UE_INTERNAL const TArray<FTypeResource>& GetSuperTypes() const { return SuperTypes; }
	UE_INTERNAL void AddSuperType(const FTypeResource& SuperType) { SuperTypes.Add(SuperType); }
	UE_INTERNAL void AddSuperType(FTypeResource&& SuperType) { SuperTypes.Add(SuperType); }

	/** I/O functions */
	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FImportTypeHierarchy& I);
	friend COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, FImportTypeHierarchy& I);
};

} // namespace UE::Serialization::Private
