// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Serialization/NameAsStringProxyArchive.h"

class UObject;

/**
 * Implements a proxy archive that serializes UObjects and FNames as string data.
 *
 * Expected use is:
 *    FArchive* SomeAr = CreateAnAr();
 *    FObjectAndNameAsStringProxyArchive Ar(*SomeAr);
 *    SomeObject->Serialize(Ar);
 *    FinalizeAr(SomeAr);
 * 
 * @param InInnerArchive The actual FArchive object to serialize normal data types (FStrings, INTs, etc)
 */
struct FObjectAndNameAsStringProxyArchive : public FNameAsStringProxyArchive
{
	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InInnerArchive - The inner archive to proxy.
	 * @param bInLoadIfFindFails - Indicates whether to try and load a ref'd object if we don't find it
	 */
	COREUOBJECT_API FObjectAndNameAsStringProxyArchive(FArchive& InInnerArchive, bool bInLoadIfFindFails);

	COREUOBJECT_API virtual ~FObjectAndNameAsStringProxyArchive();

	/** If we fail to find an object during loading, try and load it. */
	bool bLoadIfFindFails;
	/**
	 * If bResolveRedirectors is true, when loading, in operator<< functions that return a resolved object, 
	 * (UObject*, FWeakObjectPtr, FObjectPtr if resolved), if a UObject is a UObjectRedirector, the UObjectRedirector
	 * will be followed and the output Obj will receive the target of the redirector. A chain of redirectors will
	 * return the object at the end of the chain; null will be returned if the chain has a cycle or ends in a null
	 * target.
	 * If false, any UObjectRedirectors will be left unresolved and returned in the output Obj.
	 */
	bool bResolveRedirectors = false;

	/** If enabled, we'll look for core redirects in operator<<(UObject*, FWeakObjectPtr, FObjectPtr if resolved) */
	bool bResolveCoreRedirects = false;

	COREUOBJECT_API virtual FArchive& operator<<(UObject*& Obj) override;
	COREUOBJECT_API virtual FArchive& operator<<(FWeakObjectPtr& Obj) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FObjectPtr& Obj) override;
};

