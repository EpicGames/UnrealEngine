// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/UnrealMemory.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Serialization/MemoryArchive.h"
#include "UObject/Object.h"
#include "UObject/PropertyPortFlags.h"

class FArchive;
class FName;
struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;
struct FWeakObjectPtr;

/**
 * UObject Memory Reader Archive. Reads from InBytes, writes to Obj.
 */
class FObjectReader : public FMemoryArchive
{
public:
	FObjectReader(UObject* Obj, const TArray<uint8>& InBytes, bool bIgnoreClassRef = false, bool bIgnoreArchetypeRef = false)
		: Bytes(InBytes)
	{
		this->SetIsLoading(true);
		this->SetIsPersistent(false);
		ArIgnoreClassRef = bIgnoreClassRef;
		ArIgnoreArchetypeRef = bIgnoreArchetypeRef;

#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
		{
			SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(Obj));
		}
#endif // USE_STABLE_LOCALIZATION_KEYS

		Obj->Serialize(*this);
	}

	//~ Begin FArchive Interface

	int64 TotalSize()
	{
		return (int64)Bytes.Num();
	}

	void Serialize(void* Data, int64 Num)
	{
		if (Num && !IsError())
		{
			// Only serialize if we have the requested amount of data
			if (Offset + Num <= TotalSize())
			{
				FMemory::Memcpy(Data, &Bytes[IntCastChecked<int32>(Offset)], Num);
				Offset += Num;
			}
			else
			{
				SetError();
			}
		}
	}

	COREUOBJECT_API virtual FArchive& operator<<(FName& N) override;
	COREUOBJECT_API virtual FArchive& operator<<(UObject*& Res) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	COREUOBJECT_API virtual FArchive& operator<<(Verse::VCell*& Res) override;
#endif
	COREUOBJECT_API virtual FArchive& operator<<(FObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FWeakObjectPtr& Value) override;
	COREUOBJECT_API virtual FString GetArchiveName() const override;
	//~ End FArchive Interface


	FObjectReader(const TArray<uint8>& InBytes)
		: Bytes(InBytes)
	{
		this->SetIsLoading(true);
		this->SetIsPersistent(false);
		ArIgnoreClassRef = false;
		ArIgnoreArchetypeRef = false;
	}

protected:

	const TArray<uint8>& Bytes;
};
