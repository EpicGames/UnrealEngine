// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/TextNamespaceFwd.h"
#include "Internationalization/TextPackageNamespaceUtil.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/Object.h"
#include "UObject/PropertyPortFlags.h"

class FArchive;
class FName;
struct FLazyObjectPtr;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;
struct FWeakObjectPtr;

/**
 * UObject Memory Writer Archive.
 */
class FObjectWriter : public FMemoryWriter
{

public:
	FObjectWriter(UObject* Obj, TArray<uint8>& InBytes, bool bIgnoreClassRef = false, bool bIgnoreArchetypeRef = false, bool bDoDelta = true, uint32 AdditionalPortFlags = 0)
		: FMemoryWriter(InBytes)
	{
		ArIgnoreClassRef = bIgnoreClassRef;
		ArIgnoreArchetypeRef = bIgnoreArchetypeRef;
		ArNoDelta = !bDoDelta;
		ArPortFlags |= AdditionalPortFlags;

#if USE_STABLE_LOCALIZATION_KEYS
		if (GIsEditor && !(ArPortFlags & (PPF_DuplicateVerbatim | PPF_DuplicateForPIE)))
		{
			SetLocalizationNamespace(TextNamespaceUtil::EnsurePackageNamespace(Obj));
		}
#endif // USE_STABLE_LOCALIZATION_KEYS

		Obj->Serialize(*this);
	}

	//~ Begin FArchive Interface
	COREUOBJECT_API virtual FArchive& operator<<(FName& N) override;
	COREUOBJECT_API virtual FArchive& operator<<(UObject*& Res) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	COREUOBJECT_API virtual FArchive& operator<<(Verse::VCell*& Res) override;
#endif
	COREUOBJECT_API virtual FArchive& operator<<(FObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FLazyObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FWeakObjectPtr& Value) override;
	COREUOBJECT_API virtual FString GetArchiveName() const override;
	//~ End FArchive Interface

	FObjectWriter(TArray<uint8>& InBytes)
		: FMemoryWriter(InBytes)
	{
		ArIgnoreClassRef = false;
		ArIgnoreArchetypeRef = false;
	}
};
