// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Hash/ShaderHash.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/PackagePath.h"
#include "IO/PackageId.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FArchive;
class FCbObjectView;
class FCbWriter;
class FStructuredArchiveSlot;

/**
 * The type of loader that a package store entry needs to load with
 */
enum class EPackageLoader : uint8
{
	Unknown		= 0x0, // Indicates the package was loaded from somewhere outside of the loader
	LinkerLoad	= 0x1,
	Zen			= 0x2,
};
ENUM_CLASS_FLAGS(EPackageLoader);

/**
 * Package store entry status.
 */
enum class EPackageStoreEntryStatus
{
	None,
	Missing,
	NotInstalled,
	Pending,
	Ok,
};
const TCHAR* LexToString(EPackageStoreEntryStatus PackageLoader);

/**
 * Package store entry.
 */ 
struct FPackageStoreEntry
{
#if WITH_EDITORONLY_DATA
	// the corrected package name, which may be a different case than PackageName that's passed in to GetPackageStoreEntry
	// only applies to LinkerLoad (loose on the local filesystem) zen packages are never submitted to source control
	FName LinkerLoadCaseCorrectedPackageName;
#endif

	// for LinkerLoad entries, this will contain the extension of the package. for Zen entries, this will stay Unspecified
	EPackageExtension PackageExtension = EPackageExtension::Unspecified;

	// default to IoDispatcher, since almost all package store entries will use it
	EPackageLoader LoaderType = EPackageLoader::Zen;

	TArrayView<const FPackageId> ImportedPackageIds;
	TArrayView<const FShaderHash> ShaderMapHashes;
#if WITH_EDITOR
	TArrayView<const FPackageId> OptionalSegmentImportedPackageIds;
	bool bHasOptionalSegment = false;
	// This field is used to indicate that the package should load the optional chunk instead of the regular chunk. This is
	// needed by the StorageServerPackgeStore (ie ZenStore) when loading "AutoOptional" assets, but the FilePackageStore 
	// (ie .ucas files) does _not_ set this. This is because:
	//    * AutoOptional is used to "silently" replace the normal asset with an asset that still has editor-only data in it
	//    * When .ucas files are made, they create a .o.ucas file, which is mounted with a higher priority then the .ucas file
	//    * When FilePackageStore loads, it will automatically read the editor-only version of the asset, _as if it was the normal asset_
	//    * However, ZenStore has no ".o.ucas" file to silently read from instead of the regular file, so it must, at runtime,
	//      request the editor version of the asset (the .o.uasset chunk in the store).
	// This field is how we communicate out to the AsyncLoadingThread code that it should request the optional chunk instead of the
	// regular chunk for a given asset. It is basically making a runtime decision that is handled offline by ucas files.
	// Note that bHasOptionalSegment is always false for AutoOptional because of the "silently read optional as regular".
	// So to sum up for the two main PackageStores:
	//   ZenStore:
	//     Manual Optional
	//	     bHasOptionalSegment = true
	//	     bReplaceChunkWithOptional = false
	//	   AutoOptional
	//	     bHasOptionalSegment = false
	//	     bReplaceChunkWithOptional = true
	//	 FileStore
	//	   Manual Optional(same as Zen)
	//	     bHasOptionalSegment = true
	//	     bReplaceChunkWithOptional = false
	//	   AutoOptional (_not_ same)
	//	     bHasOptionalSegment = false
	//	     bReplaceChunkWithOptional = false
	bool bReplaceChunkWithOptional = false;
#endif
};

/**
 * Package store entry flags. These flags are persisted in the oplog as integers so
 * do not change their values.
 */
enum class EPackageStoreEntryFlags : uint32
{
	None				= 0,
	HasPackageData		= 0x00000001,
	AutoOptional		= 0x00000002,
	OptionalSegment		= 0x00000004,
	HasCookError		= 0x00000008,
	LoadUncooked		= 0x00000010, // This package must be loaded uncooked, when possibe, when loading from IoStore / ZenStore (i.e. HybridCookedEditor)
};
ENUM_CLASS_FLAGS(EPackageStoreEntryFlags);

/**
 * Package store entry resource.
 *
 * This is a non-optimized serializable version
 * of a package store entry. Used when cooking
 * and when running cook-on-the-fly.
 */
struct FPackageStoreEntryResource
{
	/** The package store entry flags. */
	EPackageStoreEntryFlags Flags = EPackageStoreEntryFlags::None;
	/** The package name. */
	FName PackageName;
	FPackageId PackageId;
	/** Imported package IDs. */
	TArray<FPackageId> ImportedPackageIds;
	/** Referenced shader map hashes. */
	TArray<FShaderHash> ShaderMapHashes;
	/** Editor data imported package IDs. */
	TArray<FPackageId> OptionalSegmentImportedPackageIds;
	/** Soft package references. */
	TArray<FPackageId> SoftPackageReferences;

	/** Returns the package ID. */
	FPackageId GetPackageId() const
	{
		return PackageId;
	}

	/** Returns whether this package was saved as auto optional */
	bool IsAutoOptional() const
	{
		return EnumHasAnyFlags(Flags, EPackageStoreEntryFlags::AutoOptional);
	}

	bool HasOptionalSegment() const
	{
		return EnumHasAnyFlags(Flags, EPackageStoreEntryFlags::OptionalSegment);
	}

	bool HasPackageData() const
	{
		return EnumHasAnyFlags(Flags, EPackageStoreEntryFlags::HasPackageData);
	}

	bool HasCookError() const
	{
		return EnumHasAnyFlags(Flags, EPackageStoreEntryFlags::HasCookError);
	}

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FPackageStoreEntryResource& PackageStoreEntry);
	
	COREUOBJECT_API friend FCbWriter& operator<<(FCbWriter& Writer, const FPackageStoreEntryResource& PackageStoreEntry);
	
	COREUOBJECT_API static FPackageStoreEntryResource FromCbObject(FCbObjectView Obj);

	/**
	 * Creates a PackageStoreEntryResource that records a packagename and will be in an op with possible metadata
	 * stored in attachments, but for which the op has no packagedata. This is used for tracking build dependencies
	 * on packages even if those packages fail to cook due to error or editoronly.
	 */
	COREUOBJECT_API static FPackageStoreEntryResource CreateEmptyPackage(FName PackageName, bool bHasCookError);
};

class FPackageStoreBackendContext
{
public:
	/* Event broadcasted when pending entries are completed and added to the package store */
	DECLARE_EVENT(FPackageStoreBackendContext, FPendingEntriesAddedEvent);
	FPendingEntriesAddedEvent PendingEntriesAdded;
};

/**
 * Package store backend interface.
 */
class IPackageStoreBackend
{
public:
	/* Destructor. */
	virtual ~IPackageStoreBackend() = default;

	/** Returns what possible loader types are supported by this PackageStore backend */
	virtual EPackageLoader GetSupportedLoaders()
	{
		return EPackageLoader::Zen;
	}

	/** Called when the backend is mounted */
	virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext> Context) = 0;

	/** Called when the loader enters a package store read scope. */
	virtual void BeginRead() = 0;

	/** Called when the loader exits a package store read scope. */
	virtual void EndRead() = 0;

	/* Returns the package store entry data with export info and imported packages for the specified package ID. */
	virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName,
		FPackageStoreEntry& OutPackageStoreEntry) = 0;

	/* Returns the redirected package ID and source package name for the specified package ID if it's being redirected. */
	virtual bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) = 0;

	/* Returns all soft referenced package IDs for the specified package ID. */
	virtual TConstArrayView<uint32> GetSoftReferences(FPackageId PackageId, TConstArrayView<FPackageId>& OutPackageIds)
	{
		return TConstArrayView<uint32>();
	}
};

using FVisitPackageStoreBackend = TFunction<bool(IPackageStoreBackend&)>;

/**
 * Stores information about available packages that can be loaded.
 */
class FPackageStore
{
public:
	COREUOBJECT_API static FPackageStore& Get();

	/* Mount a package store backend. */
	COREUOBJECT_API void Mount(TSharedRef<IPackageStoreBackend> Backend, int32 Priority = 0);

	/* Returns the package store entry data with export info and imported packages for the specified package ID. */
	COREUOBJECT_API EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry);

	/* Returns the redirected package ID and source package name for the specified package ID if it's being redirected. */
	COREUOBJECT_API bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId);

	/* Returns all soft referenced package IDs for the specified package ID. */
	COREUOBJECT_API TConstArrayView<uint32> GetSoftReferences(FPackageId PackageId, TConstArrayView<FPackageId>& OutPackageIds);

	COREUOBJECT_API FPackageStoreBackendContext::FPendingEntriesAddedEvent& OnPendingEntriesAdded();

	COREUOBJECT_API bool HasAnyBackendsMounted() const;

private:
	friend class FPackageStoreInternal;
	friend class FPackageStoreReadScope;

	FPackageStore();

	COREUOBJECT_API void IterateBackends(FVisitPackageStoreBackend&& Visit);

	TSharedRef<FPackageStoreBackendContext> BackendContext;
	
	using FBackendAndPriority = TTuple<int32, TSharedRef<IPackageStoreBackend>>;
	TArray<FBackendAndPriority> Backends;

	TSharedPtr<IPackageStoreBackend> LinkerLoadBackend;
#if WITH_EDITOR
	TSharedPtr<IPackageStoreBackend> HybridBackend;
#endif

	static thread_local int32 ThreadReadCount;
};

class FPackageStoreReadScope
{
public:
	COREUOBJECT_API FPackageStoreReadScope(FPackageStore& InPackageStore);
	COREUOBJECT_API ~FPackageStoreReadScope();

private:
	FPackageStore& PackageStore;
};


class FHybridPackageStoreBackend final : public IPackageStoreBackend
{
public:
	FHybridPackageStoreBackend(TSharedPtr<IPackageStoreBackend> InLoosePackageStore, TSharedPtr<IPackageStoreBackend> InCookedPackageStore);

	virtual EPackageLoader GetSupportedLoaders() override;
	virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext>) override;
	virtual void BeginRead() override;
	virtual void EndRead() override;
	virtual bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) override;
		virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry) override;

	/**
	 * Used by the editor to tell a package to stop loading the cooked version.
	 * This can be used to "uncook" an asset to allow for editing 
	 */
	COREUOBJECT_API static void ForceLoadPackageAsLoose(FPackageId PackageId);

private:
	TSharedPtr<IPackageStoreBackend> LoosePackageStore;
	TSharedPtr<IPackageStoreBackend> CookedPackageStore;
};
