// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Internationalization/GatherableTextData.h"
#include "Internationalization/TextKey.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"

class FProperty;
class FText;
class UClass;
class UFunction;
class UObject;
class UPackage;
class UScriptStruct;
class UStruct;
struct FGatherableTextData;

enum class EPropertyLocalizationGathererTextFlags : uint8
{
	/**
	 * Automatically detect whether text is editor-only data using the flags available on the properties.
	 */
	None = 0,

	/**
	 * Force the HasScript flag to be set, even if the object in question doesn't contain bytecode.
	 */
	ForceHasScript = 1<<0,

	/**
	 * Force text gathered from object properties to be treated as editor-only data.
	 * @note Does not apply to properties gathered from script data (see ForceEditorOnlyScriptData).
	 */
	ForceEditorOnlyProperties = 1<<1,

	/**
	 * Force text gathered from script data to be treated as editor-only data.
	 */
	ForceEditorOnlyScriptData = 1<<2,

	/**
	 * Force all gathered text to be treated as editor-only data.
	 */
	ForceEditorOnly = ForceEditorOnlyProperties | ForceEditorOnlyScriptData,

	/**
	 * Force all gathered text to be considered "default" (matching its archetype value).
	 */
	ForceIsDefaultValue = 1<<3,

	/**
	 * Don't process text gathered from object properties.
	 * @note Does not apply to properties gathered from script data (see SkipScriptData).
	 */
	SkipProperties = 1<<4,

	/**
	 * Don't process text gathered from script data.
	 */
	SkipScriptData = 1<<5,

	/**
	 * Don't process any sub-objects (either inner objects or object pointers).
	 */
	SkipSubObjects = 1<<6,
};
ENUM_CLASS_FLAGS(EPropertyLocalizationGathererTextFlags);

enum class EPropertyLocalizationGathererResultFlags : uint8
{
	/**
	 * The call resulted in no text or script data being added to the array.
	 */
	Empty = 0,

	/**
	 * The call resulted in text data being added to the array.
	 */
	HasText = 1<<0,

	/**
	 * The call resulted in script data being added to the array.
	 */
	HasScript = 1<<1,

	/**
	 * The call resulted in text with an invalid package localization ID being added to the array.
	 */
	HasTextWithInvalidPackageLocalizationID = 1<<2,
	
	/**
	* The call resulted in text initialized from a string being detected. The text is NOT added to the array.
	*/
	HasTextInitializedFromString = 1 << 3,
};
ENUM_CLASS_FLAGS(EPropertyLocalizationGathererResultFlags);

class FPropertyLocalizationDataGatherer
{
public:
	typedef TFunction<void(const UObject* /*Object*/, FPropertyLocalizationDataGatherer& /*PropertyLocalizationDataGatherer*/, const EPropertyLocalizationGathererTextFlags /*GatherTextFlags*/)> FLocalizationDataObjectGatheringCallback;
	typedef TMap<const UClass*, FLocalizationDataObjectGatheringCallback> FLocalizationDataObjectGatheringCallbackMap;

	typedef TFunction<void(const FString& /*PathToParent*/, const UScriptStruct* /*Struct*/, const void* /*StructData*/, const void* /*DefaultStructData*/, FPropertyLocalizationDataGatherer& /*PropertyLocalizationDataGatherer*/, const EPropertyLocalizationGathererTextFlags /*GatherTextFlags*/)> FLocalizationDataStructGatheringCallback;
	typedef TMap<const UScriptStruct*, FLocalizationDataStructGatheringCallback> FLocalizationDataStructGatheringCallbackMap;

	struct FGatherableFieldsForType
	{
		TArray<const FProperty*> Properties;
		TArray<const UFunction*> Functions;
		const FLocalizationDataObjectGatheringCallback* CustomObjectCallback = nullptr;
		const FLocalizationDataStructGatheringCallback* CustomStructCallback = nullptr;

		bool IsEmpty() const
		{
			return Properties.Num() == 0
				&& Functions.Num() == 0
				&& !CustomObjectCallback
				&& !CustomStructCallback;
		}
	};

	COREUOBJECT_API FPropertyLocalizationDataGatherer(TArray<FGatherableTextData>& InOutGatherableTextDataArray, const UPackage* const InPackage, EPropertyLocalizationGathererResultFlags& OutResultFlags);

	// Non-copyable
	FPropertyLocalizationDataGatherer(const FPropertyLocalizationDataGatherer&) = delete;
	FPropertyLocalizationDataGatherer& operator=(const FPropertyLocalizationDataGatherer&) = delete;

	COREUOBJECT_API void GatherLocalizationDataFromObjectWithCallbacks(const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags);
	COREUOBJECT_API void GatherLocalizationDataFromObject(const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags);
	COREUOBJECT_API void GatherLocalizationDataFromObjectFields(const FString& PathToParent, const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags);
	
	COREUOBJECT_API void GatherLocalizationDataFromStructWithCallbacks(const FString& PathToParent, const UScriptStruct* Struct, const void* StructData, const void* DefaultStructData, const EPropertyLocalizationGathererTextFlags GatherTextFlags);
	COREUOBJECT_API void GatherLocalizationDataFromStruct(const FString& PathToParent, const UScriptStruct* Struct, const void* StructData, const void* DefaultStructData, const EPropertyLocalizationGathererTextFlags GatherTextFlags);
	COREUOBJECT_API void GatherLocalizationDataFromStructFields(const FString& PathToParent, const UStruct* Struct, const void* StructData, const void* DefaultStructData, const EPropertyLocalizationGathererTextFlags GatherTextFlags);
	
	COREUOBJECT_API void GatherLocalizationDataFromChildTextProperties(const FString& PathToParent, const FProperty* const Property, const void* const ValueAddress, const void* const DefaultValueAddress, const EPropertyLocalizationGathererTextFlags GatherTextFlags);

	COREUOBJECT_API void GatherTextInstance(const FText& Text, const FString& Description, const bool bIsEditorOnly);
	COREUOBJECT_API void GatherScriptBytecode(const FString& PathToScript, const TArray<uint8>& ScriptData, const bool bIsEditorOnly);

	COREUOBJECT_API bool IsDefaultTextInstance(const FText& Text) const;
	COREUOBJECT_API void MarkDefaultTextInstance(const FText& Text);

	COREUOBJECT_API bool ShouldProcessObject(const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags) const;
	COREUOBJECT_API void MarkObjectProcessed(const UObject* Object, const EPropertyLocalizationGathererTextFlags GatherTextFlags);

	COREUOBJECT_API const FGatherableFieldsForType& GetGatherableFieldsForType(const UStruct* InType);

	static COREUOBJECT_API bool IsEditorOnlyProperty(const FProperty* Property);

	static COREUOBJECT_API bool ExtractTextIdentity(const FText& Text, FString& OutNamespace, FString& OutKey, const bool bCleanNamespace);

	static COREUOBJECT_API FLocalizationDataObjectGatheringCallbackMap& GetTypeSpecificLocalizationDataObjectGatheringCallbacks();
	static COREUOBJECT_API FLocalizationDataStructGatheringCallbackMap& GetTypeSpecificLocalizationDataStructGatheringCallbacks();

	UE_FORCEINLINE_HINT TArray<FGatherableTextData>& GetGatherableTextDataArray() const
	{
		return GatherableTextDataArray;
	}

	UE_FORCEINLINE_HINT bool IsObjectValidForGather(const UObject* Object) const
	{
		return AllObjectsInPackage.Contains(Object);
	}

	const UPackage* GetPackage() const { return Package; }

private:
	COREUOBJECT_API const FGatherableFieldsForType& CacheGatherableFieldsForType(const UStruct* InType);
	COREUOBJECT_API bool CanGatherFromInnerProperty(const FProperty* InInnerProperty);

	struct FObjectAndGatherFlags
	{
		FObjectAndGatherFlags(const UObject* InObject, const EPropertyLocalizationGathererTextFlags InGatherTextFlags)
			: Object(InObject)
			, GatherTextFlags(InGatherTextFlags)
			, KeyHash(0)
		{
			KeyHash = HashCombine(KeyHash, GetTypeHash(Object));
			KeyHash = HashCombine(KeyHash, GetTypeHash(GatherTextFlags));
		}

		inline bool operator==(const FObjectAndGatherFlags& Other) const
		{
			return Object == Other.Object 
				&& GatherTextFlags == Other.GatherTextFlags;
		}

		UE_FORCEINLINE_HINT bool operator!=(const FObjectAndGatherFlags& Other) const
		{
			return !(*this == Other);
		}

		friend inline uint32 GetTypeHash(const FObjectAndGatherFlags& Key)
		{
			return Key.KeyHash;
		}

		const UObject* Object;
		EPropertyLocalizationGathererTextFlags GatherTextFlags;
		uint32 KeyHash;
	};

	TArray<FGatherableTextData>& GatherableTextDataArray;
	const UPackage* Package;
	FString PackageNamespace;
	EPropertyLocalizationGathererResultFlags& ResultFlags;
	TMap<const UStruct*, TUniquePtr<FGatherableFieldsForType>> GatherableFieldsForTypes;
	TSet<const UObject*> AllObjectsInPackage;
	TSet<FObjectAndGatherFlags> ProcessedObjects;
	TSet<FObjectAndGatherFlags> BytecodePendingGather;
	TSet<FTextId> DefaultTextInstances;
};

/** Struct to automatically register a callback when it's constructed */
struct FAutoRegisterLocalizationDataGatheringCallback
{
	UE_FORCEINLINE_HINT FAutoRegisterLocalizationDataGatheringCallback(const UClass* InClass, const FPropertyLocalizationDataGatherer::FLocalizationDataObjectGatheringCallback& InCallback)
	{
		FPropertyLocalizationDataGatherer::GetTypeSpecificLocalizationDataObjectGatheringCallbacks().Add(InClass, InCallback);
	}

	UE_FORCEINLINE_HINT FAutoRegisterLocalizationDataGatheringCallback(const UScriptStruct* InStruct, const FPropertyLocalizationDataGatherer::FLocalizationDataStructGatheringCallback& InCallback)
	{
		FPropertyLocalizationDataGatherer::GetTypeSpecificLocalizationDataStructGatheringCallbacks().Add(InStruct, InCallback);
	}
};
