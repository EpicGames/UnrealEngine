// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	AsyncPackage.h: Unreal async loading definitions.
=============================================================================*/

#pragma once

#include "Serialization/Archive.h"

/** Helper class to set and restore serialized property on an archive */
class FSerializedPropertyScope
{
	FArchive& Ar;
	FProperty* Property;
	COREUOBJECT_API void PushProperty();
	COREUOBJECT_API void PopProperty();
public:
	FSerializedPropertyScope(FArchive& InAr, FProperty* InProperty, const FProperty* OnlyIfOldProperty = nullptr)
		: Ar(InAr)
		, Property(InProperty)
	{
		if (!OnlyIfOldProperty || Ar.GetSerializedProperty() == OnlyIfOldProperty)
		{
			PushProperty();
		}
		else
		{
			Property = nullptr;
		}
	}
	~FSerializedPropertyScope()
	{
		PopProperty();
	}
};
