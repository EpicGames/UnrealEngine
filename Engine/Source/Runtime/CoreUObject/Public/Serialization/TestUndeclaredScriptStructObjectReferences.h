// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/StructOpsTypeTraits.h"
#include "UObject/WeakObjectPtr.h"

#include "TestUndeclaredScriptStructObjectReferences.generated.h"

class FArchive;
 
// Helper struct to test if struct serializer object reference declaration tests work properly
USTRUCT()
struct FTestUndeclaredScriptStructObjectReferencesTest
{
	GENERATED_BODY()
public:
	UPROPERTY(Transient)
	TObjectPtr<UObject> StrongObjectPointer;
	UPROPERTY(Transient)
	TSoftObjectPtr<UObject> SoftObjectPointer;
	UPROPERTY(Transient)
	FSoftObjectPath SoftObjectPath;
	UPROPERTY(Transient)
	TWeakObjectPtr<UObject> WeakObjectPointer;
	
	bool Serialize(FArchive& Ar);
};

template<>
struct TStructOpsTypeTraits<FTestUndeclaredScriptStructObjectReferencesTest> : public TStructOpsTypeTraitsBase2<FTestUndeclaredScriptStructObjectReferencesTest>
{
	enum 
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::Strong | EPropertyObjectReferenceType::Weak | EPropertyObjectReferenceType::Soft;
};
