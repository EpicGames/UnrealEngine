// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"

class FArchive;
class UClass;
class UObject;
class UPackage;

/** Counts all persistent text references from within a package, using the specified comparison mode */
class FTextReferenceCollector
{
public:
	enum class EComparisonMode : uint8
	{
		/** Detect a reference if it matches the given ID (ignoring the source text) */
		MatchId,
		/** Detect a reference if it matches the given ID and source string */
		MatchSource,
		/** Detect a reference if it matches the given ID but has a different source string */
		MismatchSource,
	};

	typedef TFunction<void(UObject*, FArchive&)> FTextReferenceCollectorCallback;
	typedef TMap<const UClass*, FTextReferenceCollectorCallback> FTextReferenceCollectorCallbackMap;

	COREUOBJECT_API FTextReferenceCollector(const UPackage* const InPackage, const EComparisonMode InComparisonMode, const FString& InTextNamespace, const FString& InTextKey, const FString& InTextSource, int32& OutCount);

	static COREUOBJECT_API FTextReferenceCollectorCallbackMap& GetTypeSpecificTextReferenceCollectorCallbacks();
};

/** Struct to automatically register a callback when it's constructed */
struct FAutoRegisterTextReferenceCollectorCallback
{
	UE_FORCEINLINE_HINT FAutoRegisterTextReferenceCollectorCallback(const UClass* InClass, const FTextReferenceCollector::FTextReferenceCollectorCallback& InCallback)
	{
		FTextReferenceCollector::GetTypeSpecificTextReferenceCollectorCallbacks().Add(InClass, InCallback);
	}
};
