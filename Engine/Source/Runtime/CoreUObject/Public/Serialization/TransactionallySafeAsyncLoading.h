// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class IAsyncPackageLoader;

/**
 * Creates a new instance of the Transactionally Safe Async Package Loader.
 *
 * @param WrappedPackageLoader The underlying package loader to use when not in a transaction.
 *
 * @return The async package loader.
 */
IAsyncPackageLoader* MakeTransactionallySafeAsyncPackageLoader(IAsyncPackageLoader* WrappedPackageLoader = nullptr);
