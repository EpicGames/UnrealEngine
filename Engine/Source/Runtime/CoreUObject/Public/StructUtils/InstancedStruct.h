// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "InstancedStruct.generated.h"

#define UE_API COREUOBJECT_API 

struct FConstStructView;
template<typename BaseStructT> struct TConstStructView;
struct FConstSharedStruct;
template<typename BaseStructT> struct TConstSharedStruct;
class UUserDefinedStruct;

/**
 * FInstancedStruct works similarly as instanced UObject* property but is USTRUCTs.
 * 
 * Example:
 *
 *	UPROPERTY(EditAnywhere, Category = Foo, meta = (BaseStruct = "/Script/ModuleName.TestStructBase"))
 *	FInstancedStruct Test;
 *
 *	UPROPERTY(EditAnywhere, Category = Foo, meta = (BaseStruct = "/Script/ModuleName.TestStructBase"))
 *	TArray<FInstancedStruct> TestArray;
 */
USTRUCT(BlueprintType, meta = (DisableSplitPin, HasNativeMake = "/Script/Engine.BlueprintInstancedStructLibrary.MakeInstancedStruct"))
struct [[nodiscard]] FInstancedStruct
{
	GENERATED_BODY()

public:

	UE_API FInstancedStruct();

	UE_API explicit FInstancedStruct(const UScriptStruct* InScriptStruct);

	/**
	 * This constructor is explicit to avoid accidentally converting struct views to instanced structs (which would result in costly copy of the struct to be made).
	 * Implicit conversion could happen e.g. when comparing FInstancedStruct to FConstStructView.
	 */
	UE_API explicit FInstancedStruct(const FConstStructView InOther);

	FInstancedStruct(const FInstancedStruct& InOther)
	{
		InitializeAs(InOther.GetScriptStruct(), InOther.GetMemory());
	}

	FInstancedStruct(FInstancedStruct&& InOther)
		: FInstancedStruct(InOther.GetScriptStruct(), InOther.GetMutableMemory())
	{
		InOther.SetStructData(nullptr,nullptr);
	}

	~FInstancedStruct()
	{
		Reset();
	}

	UE_API FInstancedStruct& operator=(const FConstStructView InOther);

	FInstancedStruct& operator=(const FInstancedStruct& InOther)
	{
		if (this != &InOther)
		{
			InitializeAs(InOther.GetScriptStruct(), InOther.GetMemory());
		}
		return *this;
	}

	FInstancedStruct& operator=(FInstancedStruct&& InOther)
	{
		if (this != &InOther)
		{
			Reset();

			SetStructData(InOther.GetScriptStruct(), InOther.GetMutableMemory());
			InOther.SetStructData(nullptr,nullptr);
		}
		return *this;
	}

	/** Initializes from struct type and optional data. */
	UE_API void InitializeAs(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr);

	/** Initializes from struct type and emplace construct. */
	template<typename T, typename... TArgs>
	T& InitializeAs(TArgs&&... InArgs)
	{
		UE::StructUtils::CheckStructType<T>();

		const UScriptStruct* Struct = TBaseStructure<T>::Get();
		uint8* Memory = nullptr;

		const UScriptStruct* CurrentScriptStruct = GetScriptStruct();
		if (Struct == CurrentScriptStruct)
		{
			// Struct type already matches; return the struct memory to a destroyed state so we can placement new over it
			Memory = GetMutableMemory();
			((T*)Memory)->~T();
		}
		else
		{
			// Struct type mismatch; reset and reinitialize
			Reset();

			const int32 MinAlignment = Struct->GetMinAlignment();
			const int32 RequiredSize = Struct->GetStructureSize();
			Memory = (uint8*)FMemory::Malloc(FMath::Max(1, RequiredSize), MinAlignment);
			SetStructData(Struct, Memory);
		}

		check(Memory);
		new (Memory) T(Forward<TArgs>(InArgs)...);

		return *((T*)Memory);
	}

	/** Creates a new FInstancedStruct from templated struct type. */
	template<typename T>
	static FInstancedStruct Make()
	{
		UE::StructUtils::CheckStructType<T>();

		FInstancedStruct InstancedStruct;
		InstancedStruct.InitializeAs(TBaseStructure<T>::Get(), nullptr);
		return InstancedStruct;
	}

	/** Creates a new FInstancedStruct from templated struct. */
	template<typename T>
	static FInstancedStruct Make(const T& Struct)
	{
		UE::StructUtils::CheckStructType<T>();

		FInstancedStruct InstancedStruct;
		InstancedStruct.InitializeAs(TBaseStructure<T>::Get(), reinterpret_cast<const uint8*>(&Struct));
		return InstancedStruct;
	}

	/** Creates a new FInstancedStruct from the templated type and forward all arguments to constructor. */
	template<typename T, typename... TArgs>
	static FInstancedStruct Make(TArgs&&... InArgs)
	{
		UE::StructUtils::CheckStructType<T>();

		FInstancedStruct InstancedStruct;
		InstancedStruct.InitializeAs<T>(Forward<TArgs>(InArgs)...);
		return InstancedStruct;
	}

	/** For StructOpsTypeTraits */
	UE_API bool Serialize(FArchive& Ar, UStruct* DefaultsStruct = nullptr, const void* Defaults = nullptr);
	UE_API bool Identical(const FInstancedStruct* Other, uint32 PortFlags) const;
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
	UE_API bool ExportTextItem(FString& ValueStr, FInstancedStruct const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	UE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive = nullptr);
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
	UE_API void GetPreloadDependencies(TArray<UObject*>& OutDeps);
	UE_API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);
	UE_API bool FindInnerPropertyInstance(FName PropertyName, const FProperty*& OutProp, const void*& OutData) const;
	UE_API EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const;
	UE_API void* ResolveVisitedPathInfo(const FPropertyVisitorInfo& Info) const;

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const
	{
		return ScriptStruct;
	}

	/** Returns const pointer to struct memory. */
	const uint8* GetMemory() const
	{
		return StructMemory;
	}

	/** Reset to empty. */
	UE_API void Reset();

	/** Returns const reference to the struct, this getter assumes that all data is valid. */
	template<typename T>
	const T& Get() const
	{
		return UE::StructUtils::GetStructRef<T>(ScriptStruct, StructMemory);
	}

	/** Returns const pointer to the struct, or nullptr if cast is not valid. */
	template<typename T>
	const T* GetPtr() const
	{
		return UE::StructUtils::GetStructPtr<T>(ScriptStruct, StructMemory);
	}

	/** Returns a mutable pointer to struct memory. */
	uint8* GetMutableMemory()
	{
		return StructMemory;
	}

	/** Returns mutable reference to the struct, this getter assumes that all data is valid. */
	template<typename T>
	T& GetMutable()
	{
		uint8* Memory = GetMutableMemory();
		const UScriptStruct* Struct = GetScriptStruct();
		check(Memory != nullptr);
		check(Struct != nullptr);
		check(Struct->IsChildOf(TBaseStructure<T>::Get()));
		return *((T*)Memory);
	}

	/** Returns mutable pointer to the struct, or nullptr if cast is not valid. */
	template<typename T>
	T* GetMutablePtr()
	{
		uint8* Memory = GetMutableMemory();
		const UScriptStruct* Struct = GetScriptStruct();
		if (Memory != nullptr && Struct && Struct->IsChildOf(TBaseStructure<T>::Get()))
		{
			return ((T*)Memory);
		}
		return nullptr;
	}

	/** Returns True if the struct is valid.*/
	bool IsValid() const
	{
		return GetMemory() != nullptr && GetScriptStruct() != nullptr;
	}

	/** Comparison operators. Deep compares the struct instance when identical. */
	bool operator==(const FInstancedStruct& Other) const
	{
		return Identical(&Other, PPF_None);
	}

	bool operator!=(const FInstancedStruct& Other) const
	{
		return !Identical(&Other, PPF_None);
	}

#if WITH_EDITOR
	/** Internal method used to replace the script struct during user defined struct instantiation. */
	void ReplaceScriptStructInternal(const UScriptStruct* NewStruct);
#endif // WITH_EDITOR

	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FNetSerializeInstancedStruct, FInstancedStruct&, FArchive&, UPackageMap*);
	static UE_API FNetSerializeInstancedStruct NetSerializeScriptStructDelegate;

protected:

	FInstancedStruct(const UScriptStruct* InScriptStruct, uint8* InStructMemory)
		: ScriptStruct(InScriptStruct)
		, StructMemory(InStructMemory)
	{}
	void ResetStructData()
	{
		StructMemory = nullptr;
		ScriptStruct = nullptr;
	}
	void SetStructData(const UScriptStruct* InScriptStruct, uint8* InStructMemory)
	{
		ScriptStruct = InScriptStruct;
		StructMemory = InStructMemory;
	}

	TObjectPtr<const UScriptStruct> ScriptStruct = nullptr;
	uint8* StructMemory = nullptr;
};

template<>
struct TStructOpsTypeTraits<FInstancedStruct> : public TStructOpsTypeTraitsBase2<FInstancedStruct>
{
	enum
	{
		WithSerializer = true,
		WithIdentical = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithAddStructReferencedObjects = true,
		WithStructuredSerializeFromMismatchedTag = true,
		WithGetPreloadDependencies = true,
		WithNetSerializer = true,
		WithFindInnerPropertyInstance = true,
		WithClearOnFinishDestroy = true,
		WithVisitor = true,
		WithZeroConstructor = true,
	};
};

/**
 * TInstancedStruct is a type-safe FInstancedStruct wrapper against the given BaseStruct type.
 * @note When used as a property, this automatically defines the BaseStruct property meta-data.
 * 
 * Example:
 *
 *	UPROPERTY(EditAnywhere, Category = Foo)
 *	TInstancedStruct<FTestStructBase> Test;
 *
 *	UPROPERTY(EditAnywhere, Category = Foo)
 *	TArray<TInstancedStruct<FTestStructBase>> TestArray;
 */
template<typename BaseStructT>
struct [[nodiscard]] TInstancedStruct
{
public:
	TInstancedStruct() = default;

	/**
	 * This constructor is explicit to avoid accidentally converting struct views to instanced structs (which would result in costly copy of the struct to be made).
	 * Implicit conversion could happen e.g. when comparing TInstancedStruct to TConstStructView.
	 */
	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	explicit TInstancedStruct(const TConstStructView<T> InOther)
	{
		InitializeAsScriptStruct(InOther.GetScriptStruct(), InOther.GetMemory());
	}

	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	explicit TInstancedStruct(const UScriptStruct* InScriptStruct)
	{
		InitializeAsScriptStruct(InScriptStruct);
	}

	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	TInstancedStruct(const TInstancedStruct<T>& InOther)
		: InstancedStruct(InOther.InstancedStruct)
	{
	}

	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	TInstancedStruct(TInstancedStruct<T>&& InOther)
		: InstancedStruct(MoveTemp(InOther.InstancedStruct))
	{
	}

	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	TInstancedStruct& operator=(const TConstStructView<T> InOther)
	{
		if (TConstStructView<T>(*this) != InOther)
		{
			InstancedStruct.InitializeAs(InOther.GetScriptStruct(), InOther.GetMemory());
		}
		return *this;
	}

	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	TInstancedStruct& operator=(const TInstancedStruct<T>& InOther)
	{
		InstancedStruct = InOther.InstancedStruct;
		return *this;
	}

	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	TInstancedStruct& operator=(TInstancedStruct<T>&& InOther)
	{
		InstancedStruct = MoveTemp(InOther.InstancedStruct);
		return *this;
	}

	/** Initializes from a raw struct type and optional data. */
	void InitializeAsScriptStruct(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
	{
		checkf(InScriptStruct, TEXT("ScriptStruct cannot be nullptr"));
		checkf(InScriptStruct->IsChildOf(TBaseStructure<BaseStructT>::Get()), TEXT("ScriptStruct must be a child of BaseStruct!"));
		InstancedStruct.InitializeAs(InScriptStruct, InStructMemory);
	}

	/** Initializes from struct type and emplace construct. */
	template<typename T = BaseStructT, typename... TArgs, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	T& InitializeAs(TArgs&&... InArgs)
	{
		return InstancedStruct.InitializeAs<T>(Forward<TArgs>(InArgs)...);
	}

	/** Creates a new TInstancedStruct from templated struct type. */
	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	static TInstancedStruct Make()
	{
		TInstancedStruct This;
		This.InstancedStruct.InitializeAs(TBaseStructure<T>::Get(), nullptr);
		return This;
	}

	/** Creates a new TInstancedStruct from templated struct. */
	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	static TInstancedStruct Make(const T& Struct)
	{
		TInstancedStruct This;
		This.InstancedStruct.InitializeAs(TBaseStructure<T>::Get(), reinterpret_cast<const uint8*>(&Struct));
		return This;
	}

	/** Creates a new TInstancedStruct from the templated type and forward all arguments to constructor. */
	template<typename T = BaseStructT, typename... TArgs, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	static TInstancedStruct Make(TArgs&&... InArgs)
	{
		TInstancedStruct This;
		This.InstancedStruct.template InitializeAs<T>(Forward<TArgs>(InArgs)...);
		return This;
	}

	/**
	 * Cast directly to FInstancedStruct.
	 * Intended for compatibility with functions that require FInstancedStruct.
	 * Be sure you know what you're doing, as this will discard some type-checking safeguards!
	 */
	explicit operator const FInstancedStruct&() const &
	{
		return InstancedStruct;
	}

	/**
	 * Deleted overload to prevent accidental dangling refs to temps, such as
	 * const FInstancedStruct& BadRef = static_cast<const FInstancedStruct&>(TInstancedStruct<T>::Make());
	 */
	explicit operator const FInstancedStruct&() && = delete;

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const
	{
		return InstancedStruct.GetScriptStruct();
	}

	/** Returns const pointer to raw struct memory. */
	const uint8* GetMemory() const
	{
		return InstancedStruct.GetMemory();
	}

	/** Reset to empty. */
	void Reset()
	{
		InstancedStruct.Reset();
	}

	/** Returns const reference to the struct, this getter assumes that all data is valid. */
	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	const T& Get() const
	{
		return InstancedStruct.Get<T>();
	}

	/** Returns const pointer to the struct, or nullptr if cast is not valid. */
	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	const T* GetPtr() const
	{
		return UE::StructUtils::GetStructPtr<T, BaseStructT>(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMemory());
	}

	/** Returns a mutable pointer to raw struct memory. */
	uint8* GetMutableMemory()
	{
		return InstancedStruct.GetMutableMemory();
	}

	/** Returns mutable reference to the struct, this getter assumes that all data is valid. */
	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	T& GetMutable()
	{
		return InstancedStruct.GetMutable<T>();
	}

	/** Returns mutable pointer to the struct, or nullptr if cast is not valid. */
	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	T* GetMutablePtr()
	{
		return UE::StructUtils::GetStructPtr<T, BaseStructT>(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMutableMemory());
	}

	/** Returns a mutable reference. */
	[[nodiscard]] UE_REWRITE BaseStructT& operator*() requires (!std::is_const_v<BaseStructT>)
	{
		return GetMutablePtr();
	}

	/** Returns an immutable reference. */
	[[nodiscard]] UE_REWRITE const std::decay_t<BaseStructT>& operator*() const
	{
		return GetPtr();
	}

	/** Returns a mutable pointer. */
	[[nodiscard]] UE_REWRITE BaseStructT* operator->() requires (!std::is_const_v<BaseStructT>)
	{
		return GetMutablePtr();
	}

	/** Returns an immutable pointer. */
	[[nodiscard]] UE_REWRITE const std::decay_t<BaseStructT>* operator->() const
	{
		return GetPtr();
	}

	/** Returns True if the struct is valid.*/
	bool IsValid() const
	{
		return InstancedStruct.IsValid();
	}

	/** Comparison operators. Deep compares the struct instance when identical. */
	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	bool operator==(const TInstancedStruct<T>& Other) const
	{
		return InstancedStruct == Other.InstancedStruct;
	}

	template<typename T = BaseStructT, typename = std::enable_if_t<std::is_base_of_v<BaseStructT, std::decay_t<T>>>>
	bool operator!=(const TInstancedStruct<T>& Other) const
	{
		return InstancedStruct != Other.InstancedStruct;
	}

	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		InstancedStruct.AddStructReferencedObjects(Collector);
	}

	bool Serialize(FArchive& Ar)
	{
		return InstancedStruct.Serialize(Ar);
	}

	bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
	{
		return InstancedStruct.NetSerialize(Ar, Map, bOutSuccess);
	}

private:
	/**
	 * Note:
	 *   TInstancedStruct is a wrapper for a FInstancedStruct (rather than inheriting) so that it can provide a locked-down type-safe 
	 *   API for use in C++, without being able to accidentally take a reference to the untyped API to workaround the restrictions.
	 * 
	 *   TInstancedStruct MUST be the same size as FInstancedStruct, as the reflection layer treats a TInstancedStruct as a FInstancedStruct.
	 *   This means that any reflected APIs (like ExportText) that accept an FInstancedStruct pointer can also accept a TInstancedStruct pointer.
	 */
	FInstancedStruct InstancedStruct;

	template <typename U> friend struct TInstancedStruct;
};

#undef UE_API
