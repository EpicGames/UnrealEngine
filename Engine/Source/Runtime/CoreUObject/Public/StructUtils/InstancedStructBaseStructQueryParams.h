// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedStructBaseStructQueryParams.generated.h"

/**
 * Wrapper structure around an array of Properties to be used as a parameter of a UFunction for querying the base struct of the struct picker in UI.
 * It will be filled with the chain of properties from the owner object to the customizable property,
 * useful if the customizable InstancedStruct is nested in other structs.
 * It also have an optional array index if the struct is nested in an array.
 * TODO: Add support for struct nested in Sets and Maps.
 *
 * Example:
 * 
 * USTRUCT(BlueprintType)
 * struct FBaseStruct
 * {
 * 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Custom")
 * 	int IntProperty = 0;
 * };
 * 
 * USTRUCT(BlueprintType)
 * struct FChildStruct1 : public FBaseStruct
 * {
 * 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Custom")
 * 	float FloatProperty = 0;
 * };
 * 
 * USTRUCT(BlueprintType)
 * struct FChildStruct2 : public FBaseStruct
 * {
 * 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Custom")
 * 	FString StringProperty;
 * };
 * 
 * USTRUCT(BlueprintType)
 * struct FNestedStruct
 * {
 * 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Custom")
 * 	int StructToChoose = 0;
 * 
 * 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Custom", meta = (BaseStructFunction = "GetSettingScriptStruct"))
 * 	TInstancedStruct<FBaseStruct> MyInstancedStruct;
 * };
 * 
 * UCLASS()
 * class UCustomObject : public UObject
 * {
 * public:
 * 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Custom")
 * 	TArray<FNestedStruct> NestedStructs;
 * 
 * 	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Custom")
 * 	FNestedStruct NestedStruct2;
 * 
 * #if WITH_EDITOR
 * 	UFUNCTION()
 * 	UScriptStruct* GetSettingScriptStruct(UE::StructUtils::FInstancedStructBaseStructQueryParams Params) const
 * 	{
 * 		int32 Index = -1;
 * 		if (Params.PropertyChainWithArrayIndex.Num() == 2)
 * 		{
 * 			if (Params.PropertyChainWithArrayIndex[0].Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomObject, NestedStruct2))
 * 			{
 * 				Index = NestedStruct2.StructToChoose;
 * 			}
 * 		}
 * 		else if (Params.PropertyChainWithArrayIndex.Num() == 3)
 * 		{
 * 			if (Params.PropertyChainWithArrayIndex[1].Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCustomObject, NestedStructs))
 * 			{
 * 				const int32 ArrayIndex = Params.PropertyChainWithArrayIndex[1].ArrayIndex;
 * 				if (ensure(NestedStructs.IsValidIndex(ArrayIndex)))
 * 				{
 * 					Index = NestedStructs[ArrayIndex].StructToChoose;
 * 				}
 * 			}
 * 		}
 * 
 * 		if (Index != -1)
 * 		{
 * 			return Index == 0 ? FChildStruct1::StaticStruct() : FChildStruct2::StaticStruct();
 * 		}
 * 		else
 * 		{
 * 			return FBaseStruct::StaticStruct();
 * 		}
 * 	}
 * #endif // WITH_EDITOR
 * };
 */

namespace UE::StructUtils
{
USTRUCT(BlueprintInternalUseOnly)
struct FInstancedStructBaseStructQueryParams
{
	GENERATED_BODY()

#if WITH_EDITOR
	struct FParam
	{
		const FProperty* Property = nullptr;
		int32 ArrayIndex = INDEX_NONE;
	};
	
	TArray<FParam> PropertyChainWithArrayIndex;
#endif // WITH_EDITOR
};
}

