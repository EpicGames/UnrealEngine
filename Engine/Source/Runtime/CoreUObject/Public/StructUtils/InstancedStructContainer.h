// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/StructView.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "InstancedStructContainer.generated.h"

#define UE_API COREUOBJECT_API

struct FInstancedStruct;
class FReferenceCollector;

/**
 * Array of heterogeneous structs. Can be used as a property, supports serialization,
 * but does not have type customizations (no editing in the UI).
 *
 * If you need UI editable array of heterogeneous structs, use TArray<FInstancedStruct> instead.
 *
 * The array item values and the index to an item are stored in one contiguous block of memory.
 * The size required to specific layout of structs is larger than the sum of their sizes due to alignment,
 * and because the index to the structs is stored along with the value memory. Each item takes extra 16 bytes
 * for index. If your items are roughly same size, a TArray<TVariant<>> might be more performant.
 *
 * Adding new items is more expensive than on regular TArray<>, layout of the structs needs to be updated,
 * and initialization is done via UScriptStruct. Adding and removing items should be done in chunks if possible.
 *
 * The allocation of new items does not allocate extra space as most array implementations do.
 * Use Reserve() to reserve certain sized buffer in bytes if that is applicable to your use case.
 */

USTRUCT()
struct FInstancedStructContainer
{
	GENERATED_BODY()

private:

	/** Struct describing an item in the array. */
	struct FItem
	{
		FItem() = default;
		FItem(const UScriptStruct* InScriptStruct, const int32 InOffset) : ScriptStruct(InScriptStruct), Offset(InOffset) {}

		int32 GetStructureSize() const
		{
			return ScriptStruct != nullptr ? ScriptStruct->GetStructureSize() : 0;
		}

		int32 GetMinAlignment() const
		{
			return ScriptStruct != nullptr ? ScriptStruct->GetMinAlignment() : 1;
		}

		int32 GetEndOffset() const
		{
			return Offset + GetStructureSize();
		}

		TObjectPtr<const UScriptStruct> ScriptStruct = nullptr;
		int32 Offset = 0;
	};

public:
	
	/** How much memory is used to store info about each item. */
	static constexpr int32 OverheadPerItem = sizeof(FItem);
	
	UE_API FInstancedStructContainer();
	UE_API FInstancedStructContainer(const FInstancedStructContainer& InOther);
	UE_API FInstancedStructContainer(FInstancedStructContainer&& InOther);
	
	~FInstancedStructContainer() { Empty(); }

	UE_API FInstancedStructContainer& operator=(const FInstancedStructContainer& InOther);
	UE_API FInstancedStructContainer& operator=(FInstancedStructContainer&& InOther);
	UE_API FInstancedStructContainer& operator=(TConstArrayView<FInstancedStruct> InItems);
	UE_API FInstancedStructContainer& operator=(TConstArrayView<FStructView> InItems);
	UE_API FInstancedStructContainer& operator=(TConstArrayView<FConstStructView> InItems);

	/** Appends items to the array. */
	UE_API void Append(const FInstancedStructContainer& Other);
	UE_API void Append(TConstArrayView<FInstancedStruct> NewItemValues);
	UE_API void Append(TConstArrayView<FConstStructView> NewItemValues);

	/** Insert new items at specified location. */
	UE_API void InsertAt(const int32 InsertAtIndex, const FInstancedStructContainer& Other);
	UE_API void InsertAt(const int32 InsertAtIndex, TConstArrayView<FInstancedStruct> ValuesToInsert);
	UE_API void InsertAt(const int32 InsertAtIndex, TConstArrayView<FStructView> ValuesToInsert);
	UE_API void InsertAt(const int32 InsertAtIndex, TConstArrayView<FConstStructView> ValuesToInsert);

	/** Remove items at specific location. Does not change memory allocation. */
	UE_API void RemoveAt(const int32 RemoveAtIndex, const int32 Count);

	/** Reserves at least 'NumBytes' for internal storage. */
	UE_API void ReserveBytes(const int32 NumBytes, const int32 MinAlignment = DefaultMinAlignment);
	
	/** Sets the number of items in the array. Note: this can only shrink the array. Does not change memory allocation. */
	UE_API void SetNum(const int32 NewNum);

	/** Returns number of bytes allocated for the array  */
	int32 GetAllocatedMemory() const { return AllocatedSize; }
	
	/** Empties the array, destroys entities. Does not change memory allocation. */
	UE_API void Reset();

	/** Empties the array, destroys entities. Frees memory. */
	UE_API void Empty();

	/** @return true if the instance is correctly initialized. */
	bool IsValid() const { return NumItems > 0; }

	/** @return Number of items in the instance data. */
	int32 Num() const { return NumItems; }

	/** @return true of the index is in valid range to the array. */
	bool IsValidIndex(const int32 Index) const { return Index >= 0 && Index < NumItems; }

	/** @return view to the struct at specified index. */
	FConstStructView operator[](const int32 Index) const
	{
		check(IsValid() && IsValidIndex(Index));
		const FItem& Item = GetItem(Index);
		return FConstStructView(Item.ScriptStruct, Memory + Item.Offset);
	}

	/** @return view to the struct at specified index. */
	FStructView operator[](const int32 Index)
	{
		check(IsValid() && IsValidIndex(Index));
		const FItem& Item = GetItem(Index);
		return FStructView(Item.ScriptStruct, Memory + Item.Offset);
	}

	/**
	 * Iterators to enable range-based for loop support.
	 *
	 *	// Ranged for mutable container 
	 *	for (FStructView View : Container) {}
	 *
	 *	// Ranged for const container 
	 *	for (FConstStructView View : Container) {}
	 *
	 *	// Iterator based iteration, allows removing items.
	 *	for (FInstancedStructContainer::FIterator It = Container.CreateIterator(); It; ++It)
	 *	{
	 *		It.RemoveCurrent();
	 *	}
	 */
	template<typename T>
	struct TIterator
	{
		using StructViewType = std::conditional_t<std::is_const_v<T>, FConstStructView, FStructView>;

		/** @return struct view (or const structview) to the item. */
		StructViewType operator*()
		{
			const FItem& Item = Container.GetItem(Index);
			return StructViewType(Item.ScriptStruct, Container.Memory + Item.Offset);
		}

		TIterator& operator++()
		{
			Index++;
			return *this;
		}

		TIterator& operator--()
		{
			Index--;
			return *this;
		}

		/** @return true of two iterators point to the same item. */
		bool operator!=(const TIterator& RHS) const
		{
			return Index != RHS.Index;
		}

		/** @return true of the interator points to valid item. */
		explicit operator bool() const
		{
			return Container.IsValidIndex(Index);
		}

		/** @returns The index of the iterator.  */
		int32 GetIndex() const
		{
			return Index;
		}

		/** Removes the item pointed by the iterator and adjust the iterator. */
		void RemoveCurrent()
		{
			static_assert(!std::is_const_v<T>, "Cannot remove on const container.");
			Container.RemoveAt(Index, 1);
			Index--;
		}
		
	private:
		explicit TIterator(T& InContainer, const int32 InIndex = 0)
			: Container(InContainer)
			, Index(InIndex)
		{
		}

		T& Container;
		int32 Index = 0;

		friend FInstancedStructContainer;
	};

	using FIterator = TIterator<FInstancedStructContainer>;
	using FConstIterator = TIterator<const FInstancedStructContainer>;

	/** Creates iterator to iterate over the array. */
	FIterator CreateIterator() { return FIterator(*this, 0); }
	/** Creates const iterator to iterate over the array. */
	FConstIterator CreateConstIterator() const { return FConstIterator(*this, 0); }

	/** For ranged for, do not use directly. */
	FIterator begin() { return FIterator(*this, 0); }
	FIterator end() { return FIterator(*this, NumItems); }

	FConstIterator begin() const { return FConstIterator(*this, 0); }
	FConstIterator end() const { return FConstIterator(*this, NumItems); }

	/** Type traits */
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
	UE_API bool Identical(const FInstancedStructContainer* Other, uint32 PortFlags) const;
	UE_API bool Serialize(FArchive& Ar, UStruct* DefaultsStruct = nullptr, const void* Defaults = nullptr);
	UE_API void GetPreloadDependencies(TArray<UObject*>& OutDeps) const;
	UE_API bool ExportTextItem(FString& ValueStr, FInstancedStructContainer const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	UE_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive = nullptr);

private:

	/** Default minimum alignment for any allocation. Pointer sized to prevent trivial allocations due to alignment. */
	static constexpr int32 DefaultMinAlignment = alignof(void*);

	/** Each FItem struct is allocated at the end of the buffer (which is aligned to FItem alignment), and they are stride apart. */
	static constexpr int32 ItemStride = static_cast<int32>(Align(sizeof(FItem), alignof(FItem)));

	/** @returns offset of the item in the allocated memory.  */
	int32 GetItemOffset(const int32 Index) const
	{
		check(NumItems > 0 && Index < NumItems);
		return AllocatedSize - ((Index + 1) * ItemStride);
	}

	/** @returns item based on index.  */
	FItem& GetItem(const int32 Index) const
	{
		check(NumItems > 0 && Index < NumItems);
		const int32 Offset = AllocatedSize - ((Index + 1) * ItemStride);
		return *(FItem*)(Memory + Offset);
	}

	/** Memory holding all structs values and items. Values are stored front, and items back. */
	uint8* Memory = nullptr;

	/** Number of bytes allocated for the array. */
	int32 AllocatedSize = 0;

	/** Number of items in the array. */
	int32 NumItems = 0;

	friend FIterator;
	friend FConstIterator;
};

template<>
struct TStructOpsTypeTraits<FInstancedStructContainer> : public TStructOpsTypeTraitsBase2<FInstancedStructContainer>
{
	enum
	{
		WithSerializer = true,
		WithIdentical = true,
		WithAddStructReferencedObjects = true,
		WithGetPreloadDependencies = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithClearOnFinishDestroy = true,
		WithZeroConstructor = true,
	};
};

/** 
 * An editor-customized type that holds instanced structs.
 * It does not allow adding or removing structs in the editor.
 * Each struct is visualized as a category, with its properties below.
 * Using this type allows defining 'building blocks of data' in C++, while presenting them to the user as regular properties. * 
 */
USTRUCT()
struct FInstancedStructArray
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category="Values")
	TArray<FInstancedStruct> Values;
};

#undef UE_API
