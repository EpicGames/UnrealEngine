// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/TVariantMeta.h"
#include "StructUtils/StructView.h"
#include "Templates/ValueOrError.h"
#include "Containers/StaticArray.h"
#include "UObject/ScriptMacros.h"
#include "UObject/ObjectMacros.h"

#include "PropertyBag.generated.h"

#define UE_API COREUOBJECT_API

/** Property bag property type, loosely based on BluePrint pin types. */
UENUM(BlueprintType)
enum class EPropertyBagPropertyType : uint8
{
	None UMETA(Hidden),
	Bool,
	Byte,
	Int32,
	Int64,
	Float,
	Double,
	Name,
	String,
	Text,
	Enum,
	Struct,
	Object,
	SoftObject,
	Class,
	SoftClass,
	Int8,	// Type not fully supported at UI, will work with restrictions to type editing
	Int16,	// Type not fully supported at UI, will work with restrictions to type editing
	UInt16,	// Type not fully supported at UI, will work with restrictions to type editing
	UInt32,	// Type not fully supported at UI, will work with restrictions to type editing
	UInt64, // Type not fully supported at UI, will work with restrictions to type editing

	Count UMETA(Hidden)
};

/** Property bag property container type. */
UENUM(BlueprintType)
enum class EPropertyBagContainerType : uint8
{
	None,
	Array,
	Set,
	Map,

	Count UMETA(Hidden)
};

namespace FPropertyBagCustomVersion
{
	COREUOBJECT_API extern const FGuid GUID;
}

/** Helper to manage container types, with nested container support. */
USTRUCT()
struct FPropertyBagContainerTypes
{
	GENERATED_BODY()

	FPropertyBagContainerTypes() = default;

	explicit FPropertyBagContainerTypes(EPropertyBagContainerType ContainerType)
	{
		if (ContainerType != EPropertyBagContainerType::None)
		{
			Add(ContainerType);
		}
	}

	FPropertyBagContainerTypes(const std::initializer_list<EPropertyBagContainerType>& InTypes)
	{
		for (const EPropertyBagContainerType ContainerType : InTypes)
		{
			if (ContainerType != EPropertyBagContainerType::None)
			{
				Add(ContainerType);
			}
		}
	}

	bool Add(const EPropertyBagContainerType PropertyBagContainerType)
	{
		if (ensure(NumContainers < MaxNestedTypes))
		{
			if (PropertyBagContainerType != EPropertyBagContainerType::None)
			{
				Types[NumContainers] = PropertyBagContainerType;
				NumContainers++;

				return true;
			}
		}

		return false;
	}

	void Reset()
	{
		for (EPropertyBagContainerType& Type : Types)
		{
			Type = EPropertyBagContainerType::None;
		}
		NumContainers = 0;
	}

	bool IsEmpty() const
	{
		return NumContainers == 0;
	}

	uint32 Num() const
	{
		return NumContainers;
	}

	bool CanAdd() const
	{
		return NumContainers < MaxNestedTypes;
	}

	EPropertyBagContainerType GetFirstContainerType() const
	{
		return NumContainers > 0 ? Types[0] : EPropertyBagContainerType::None;
	}

	EPropertyBagContainerType operator[] (int32 Index) const
	{
		return ensure(Index < NumContainers) ? Types[Index] : EPropertyBagContainerType::None;
	}

	UE_API EPropertyBagContainerType PopHead();

	UE_API void Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FPropertyBagContainerTypes& ContainerTypesData)
	{
		ContainerTypesData.Serialize(Ar);
		return Ar;
	}

	UE_API bool operator == (const FPropertyBagContainerTypes& Other) const;

	UE_FORCEINLINE_HINT bool operator !=(const FPropertyBagContainerTypes& Other) const
	{
		return !(Other == *this);
	}

	friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const FPropertyBagContainerTypes& PropertyBagContainerTypes)
	{
		return GetArrayHash(PropertyBagContainerTypes.Types.GetData(), PropertyBagContainerTypes.NumContainers);
	}

	EPropertyBagContainerType* begin() { return Types.GetData(); }
	const EPropertyBagContainerType* begin() const { return Types.GetData(); }
	EPropertyBagContainerType* end()  { return Types.GetData() + NumContainers; }
	const EPropertyBagContainerType* end() const { return Types.GetData() + NumContainers; }

protected:
	static constexpr uint8 MaxNestedTypes = 2;

	TStaticArray<EPropertyBagContainerType, MaxNestedTypes> Types = TStaticArray<EPropertyBagContainerType, MaxNestedTypes>(InPlace, EPropertyBagContainerType::None);
	uint8 NumContainers = 0;
};

/** Getter and setter result code. */
UENUM()
enum class EPropertyBagResult : uint8
{
	Success,			// Operation succeeded.
	TypeMismatch,		// Tried to access mismatching type (e.g. setting a struct to bool)
	OutOfBounds,		// Tried to access an array property out of bounds.
	PropertyNotFound,	// Could not find property of specified name.
	DuplicatedValue,	// Tried to set an already existing Set Entry
};

/** Property bag alterations result code. */
UENUM()
enum class EPropertyBagAlterationResult : uint8
{
	Success,                             // Operation succeeded.
	NoOperation = Success UMETA(Hidden), // No operation was necessary to warrant a successful operation. Semantic alias.
	InternalError,                       // The operation could not be completed, due to an internal property bag error.
	PropertyNameEmpty,                   // The property name is empty.
	PropertyNameInvalidCharacters,       // The property name contains illegal characters. Consider using FInstancedPropertyBag::SanitizePropertyName.
	SourcePropertyNotFound,              // The source property or property name was not found.
	TargetPropertyNotFound,              // The target property or property name was not found.
	TargetPropertyAlreadyExists,         // The target property or property name already exists.
	PropertyKeyTypeNotHashable,          // The property has Map container type and the given key type is not hashable.
};

USTRUCT()
struct FPropertyBagPropertyDescMetaData
{
	GENERATED_BODY()
	
	FPropertyBagPropertyDescMetaData() = default;
	FPropertyBagPropertyDescMetaData(const FName InKey, const FString& InValue)
		: Key(InKey)
		, Value(InValue)
	{
	}
	
	UPROPERTY()
	FName Key;

	UPROPERTY()
	FString Value;

	UE_API void Serialize(FArchive& Ar);

	bool operator==(const FPropertyBagPropertyDescMetaData& Other) const
	{
		return Key == Other.Key && Value == Other.Value;
	}

	inline friend FArchive& operator<<(FArchive& Ar, FPropertyBagPropertyDescMetaData& PropertyDescMetaData)
	{
		PropertyDescMetaData.Serialize(Ar);
		return Ar;
	}

	friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const FPropertyBagPropertyDescMetaData& PropertyDescMetaData)
	{
		return HashCombine(GetTypeHash(PropertyDescMetaData.Key), GetTypeHash(PropertyDescMetaData.Value));
	}

	friend inline uint32 GetTypeHash(const TArrayView<const FPropertyBagPropertyDescMetaData>& MetaData)
	{
		uint32 Hash = GetTypeHash(MetaData.Num());
		for (const FPropertyBagPropertyDescMetaData& PropertyDescMetaData : MetaData)
		{
			Hash = HashCombine(Hash, GetTypeHash(PropertyDescMetaData));
		}
		return Hash;
	}

	friend UE_FORCEINLINE_HINT uint32 GetTypeHash(const TArray<FPropertyBagPropertyDescMetaData>& MetaData)
	{
		return GetTypeHash(TArrayView<const FPropertyBagPropertyDescMetaData>(MetaData.GetData(), MetaData.Num()));
	}
};

/** Describes a property in the property bag. */
USTRUCT()
struct FPropertyBagPropertyDesc
{
	GENERATED_BODY()

	static_assert(std::is_same_v<std::underlying_type_t<EPropertyFlags>, uint64>, "FPropertyBagPropertyDesc::PropertyFlag does not match EPropertyFlags type");

	FPropertyBagPropertyDesc() = default;
	UE_API FPropertyBagPropertyDesc(const FName InName, const FProperty* InSourceProperty);
	FPropertyBagPropertyDesc(const FName InName, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject = nullptr)
		: ValueTypeObject(InValueTypeObject)
		, Name(InName)
		, ValueType(InValueType)
	{
	}
	FPropertyBagPropertyDesc(const FName InName, const EPropertyBagContainerType InContainerType, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject = nullptr, EPropertyFlags InPropertyFlags = CPF_Edit, const EPropertyBagPropertyType InKeyType = EPropertyBagPropertyType::None, const UObject* InKeyTypeObject = nullptr)
		: ValueTypeObject(InValueTypeObject)
		, Name(InName)
		, ValueType(InValueType)
		, ContainerTypes(InContainerType)
		, PropertyFlags((uint64)InPropertyFlags)
		, KeyType(InKeyType)
		, KeyTypeObject(InKeyTypeObject)
	{
	}

	FPropertyBagPropertyDesc(const FName InName, const FPropertyBagContainerTypes& InNestedContainers, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject = nullptr, EPropertyFlags InPropertyFlags = CPF_Edit, const EPropertyBagPropertyType InKeyType = EPropertyBagPropertyType::None, const UObject* InKeyTypeObject = nullptr)
		: ValueTypeObject(InValueTypeObject)
		, Name(InName)
		, ValueType(InValueType)
		, ContainerTypes(InNestedContainers)
		, PropertyFlags((uint64)InPropertyFlags)
		, KeyType(InKeyType)
		, KeyTypeObject(InKeyTypeObject)
	{
	}

	/** @return true if the two descriptors have the same type. Object types are compatible if Other can be cast to this type. */
	UE_API bool CompatibleType(const FPropertyBagPropertyDesc& Other) const;

	/** @return true if the property type is numeric (bool, (u)int8, (u)int16, (u)int32, (u)int64, float, double, enum) */
	UE_API bool IsNumericType() const;

	/** @return true if the property type is unsigned (uint8, uint16, uint32, uint64) */
	UE_API bool IsUnsignedNumericType() const;

	/** @return true if the property type is integral ((u)int8, (u)int16, (u)int32, (u)int64) */
	UE_API bool IsNumericIntegralType() const;
	
	/** @return true if the property type is floating point numeric (float, double) */
	UE_API bool IsNumericFloatType() const;

	/** @return true if the property type is object or soft object */
	UE_API bool IsObjectType() const;

	/** @return true if the property type is class or soft class */
	UE_API bool IsClassType() const;

	/** @return index of the property after constructed in the bag */
	int32 GetCachedIndex() const { return CachedIndex; }

	/** @return true if the property bag's unique IDs match */
	UE_API bool operator==(const FPropertyBagPropertyDesc& OtherDesc) const;

	/**
	 * Serializes a PropertyBagPropertyDesc from or into an archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Desc The PropertyBagPropertyDesc to serialize.
	 */
	UE_API friend FArchive& operator<<(FArchive& Ar, FPropertyBagPropertyDesc& Desc);

#if WITH_EDITOR
	/** @return true if the property has a metadata specifier */
	UE_API bool HasMetaData(FName SpecifierName) const;

	/** Set the metadata specifier value. Will add the specifier if it does not exist */
	UE_API void SetMetaData(FName SpecifierName, const FString& InValue);

	/** @return retrieves the metadata specifier value */
	UE_API FString GetMetaData(FName SpecifierName) const;

	/** Remove the metadata specifier value from the property. */
	UE_API void RemoveMetadata(FName SpecifierName);
#endif // WITH_EDITOR

	/** Pointer to object that defines the Enum, Struct, or Class. */
	UPROPERTY(EditAnywhere, Category="Default")
	TObjectPtr<const UObject> ValueTypeObject = nullptr;

	/** Unique ID for this property. Used as main identifier when copying values over. */
	UPROPERTY(EditAnywhere, Category="Default")
	FGuid ID;

	/** Name for the property. */
	UPROPERTY(EditAnywhere, Category="Default")
	FName Name;

	/** Type of the value described by this property. */
	UPROPERTY(EditAnywhere, Category="Default")
	EPropertyBagPropertyType ValueType = EPropertyBagPropertyType::None;

	/** Type of the container described by this property. */
	UPROPERTY(EditAnywhere, Category="Default")
	FPropertyBagContainerTypes ContainerTypes;

	/** Flags that will get copied over to this property. uint64 since EPropertyFlags is not UEnum */
	UPROPERTY(EditAnywhere, Category = "Default")
	uint64 PropertyFlags = (uint64)CPF_Edit;

	/** Type of this property's key (for map containers only) */
	UPROPERTY(EditAnywhere, Category = "Default")
	EPropertyBagPropertyType KeyType = EPropertyBagPropertyType::None;

	/** Pointer to object that defines the Enum, Struct, or Class of this property's key (for map containers only) */
	UPROPERTY(EditAnywhere, Category = "Default")
	TObjectPtr<const UObject> KeyTypeObject = nullptr;

#if WITH_EDITORONLY_DATA
	/** Editor-only metadata for CachedProperty */
	UPROPERTY(EditAnywhere, Category="Default")
	TArray<FPropertyBagPropertyDescMetaData> MetaData;

	/** Editor-only meta class for IClassViewer */
	UPROPERTY(EditAnywhere, Category = "Default")
	TObjectPtr<class UClass> MetaClass;
#endif

	/** Cached property pointer, set in UPropertyBag::GetOrCreateFromDescs. */
	const FProperty* CachedProperty = nullptr;

private:
	friend class UPropertyBag;

	/** Index of the property in the bag, set in UPropertyBag::GetOrCreateFromDescs. */
	int32 CachedIndex = INDEX_NONE;
};

/**
 * Instanced property bag allows to create and store a bag of properties.
 *
 * When used as editable property, the UI allows properties to be added and removed, and values to be set.
 * The value is stored as a struct, the type of the value is never serialized, instead the composition of the properties
 * is saved with the instance, and the type is recreated on load. The types with same composition of properties share same type (based on hashing).
 *
 * UPROPERTY() meta tags:
 *		- FixedLayout: Property types cannot be altered, but values can be. This is useful if e.g. if the bag layout is set byt code.
 *
 * NOTE: Adding or removing properties to the instance is quite expensive as it will create new UPropertyBag, reallocate memory, and copy all values over. 
 *
 * Example usage, this allows the bag to be configured in the UI:
 *
 *		UPROPERTY(EditDefaultsOnly, Category = Common)
 *		FInstancedPropertyBag Bag;
 *
 * Changing the layout from code:
 *
 *		static const FName TemperatureName("Temperature");
 *		static const FName IsHotName("bIsHot");
 *
 *		FInstancedPropertyBag Bag;
 *
 *		// Add properties to the bag, and set their values.
 *		// Adding or removing properties is not cheap, so better do it in batches.
 *		Bag.AddProperties({
 *			{ TemperatureName, EPropertyBagPropertyType::Float },
 *			{ CountName, EPropertyBagPropertyType::Int32 }
 *		});
 *
 *		// Amend the bag with a new property.
 *		Bag.AddProperty(IsHotName, EPropertyBagPropertyType::Bool);
 *		Bag.SetValueBool(IsHotName, true);
 *
 *		// Get value and use the result
 *		if (auto Temperature = Bag.GetValueFloat(TemperatureName); Temperature.IsValid())
 *		{
 *			float Val = Temperature.GetValue();
 *		}
 */

class UPropertyBag;
class FPropertyBagArrayRef;
class FPropertyBagSetRef;
class FPropertyBagMapRef;

USTRUCT()
struct FInstancedPropertyBag
{
	GENERATED_BODY()

	FInstancedPropertyBag() = default;
	FInstancedPropertyBag(const FInstancedPropertyBag& Other) = default;
	FInstancedPropertyBag(FInstancedPropertyBag&& Other) = default;

	FInstancedPropertyBag& operator=(const FInstancedPropertyBag& InOther) = default;
	FInstancedPropertyBag& operator=(FInstancedPropertyBag&& InOther) = default;

	/** @return true if the instance contains data. */
	bool IsValid() const
	{
		return Value.IsValid();
	}
	
	/** Resets the instance to empty. */
	void Reset()
	{
		Value.Reset();
	}

	/** Initializes the instance from a bag struct. */
	UE_API void InitializeFromBagStruct(const UPropertyBag* NewBagStruct);
	
	/**
	 * Copies matching property values from another bag of potentially mismatching layout.
	 * The properties are matched between the bags based on the property ID.
	 * @param Other Reference to the bag to copy the values from
	 * @param OptionalPropertyIdsSubset Optional array of property ids subset to copy from Other
	 */
	UE_API void CopyMatchingValuesByID(const FInstancedPropertyBag& Other, TOptional<TConstArrayView<FGuid>> OptionalPropertyIdsSubset = {});

	/**
	 * Copies matching property values from another bag of potentially mismatching layout.
	 * The properties are matched between the bags based on the property name.
	 * @param Other Reference to the bag to copy the values from
	 * @param OptionalPropertyNamesSubset Optional array of property names subset to copy from Other 
	 */
	UE_API void CopyMatchingValuesByName(const FInstancedPropertyBag& Other, TOptional<TConstArrayView<FName>> OptionalPropertyNamesSubset = {});

	/** Returns number of the Properties in this Property Bag */
	UE_API int32 GetNumPropertiesInBag() const;

	/**
	 * Adds properties to the bag. If property of same name already exists, it will be replaced with the new type.
	 * Numeric property values will be converted if possible, when a property's type changes.
	 * @param Descs Descriptors of new properties to add.
	 * @param bOverwrite Overwrite the property if it already exists.
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult AddProperties(const TConstArrayView<FPropertyBagPropertyDesc> Descs, bool bOverwrite = true);
	
	/**
	 * Adds a new property to the bag. If property of same name already exists, it will be replaced with the new type.
	 * Numeric property values will be converted if possible, when a property's type changes.
	 * @param InName Name of the new property
	 * @param InValueType Type of the new property
	 * @param InValueTypeObject Type object (for struct, class, enum) of the new property
	 * @param bOverwrite Overwrite the property if it already exists.
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult AddProperty(const FName InName, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject = nullptr, bool bOverwrite = true);

	/**
	 * Adds a new container property to the bag. If property of same name already exists, it will be replaced with the new type.
	 * @param InName Name of the new property
	 * @param InContainerType Type of the new container
	 * @param InValueType Type of the new property
	 * @param InValueTypeObject Type object (for struct, class, enum) of the new property
	 * @param bOverwrite Overwrite the property if it already exists.
	 * @param InKeyType Type of the new property's key
	 * @param InKeyTypeObject Type object (for struct, class, enum) of the new property's key
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult AddContainerProperty(const FName InName, const EPropertyBagContainerType InContainerType, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject = nullptr, bool bOverwrite = true, const EPropertyBagPropertyType InKeyType = EPropertyBagPropertyType::None, const UObject* InKeyTypeObject = nullptr);

	/**
	 * Adds a new container property to the bag. If property of same name already exists, it will be replaced with the new type.
	 * @param InName Name of the new property
	 * @param InContainerTypes List of (optionally nested) containers to create
	 * @param InValueType Type of the new property
	 * @param InValueTypeObject Type object (for struct, class, enum) of the new property
	 * @param bOverwrite Overwrite the property if it already exists.
	 * @param InKeyType Type of the new property's key
	 * @param InKeyTypeObject Type object (for struct, class, enum) of the new property's key
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult AddContainerProperty(const FName InName, const FPropertyBagContainerTypes InContainerTypes, const EPropertyBagPropertyType InValueType, const UObject* InValueTypeObject, bool bOverwrite = true, const EPropertyBagPropertyType InKeyType = EPropertyBagPropertyType::None, const UObject* InKeyTypeObject = nullptr);

	/**
	 * Adds a new map property to the bag. If property of same name already exists, it will be replaced with the new type.
	 * @param InName Name of the new property
	 * @param InKeyType Type of the new property's key
	 * @param InValueType Type of the new property
	 * @param InKeyTypeObject Type object (for struct, class, enum) of the new property's key
	 * @param InValueTypeObject Type object (for struct, class, enum) of the new property
	 * @param InNestedContainerType Type of (optional) nested container to create
	 * @param bOverwrite Overwrite the property if it already exists.
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult AddMapProperty(const FName InName, const EPropertyBagPropertyType InKeyType, const EPropertyBagPropertyType InValueType, const UObject* InKeyTypeObject = nullptr, const UObject* InValueTypeObject = nullptr, const EPropertyBagContainerType InNestedContainerType = EPropertyBagContainerType::None, bool bOverwrite = true);

	/**
	 * Adds a new property to the bag. Property type duplicated from source property to. If property of same name already exists, it will be replaced with the new type.
	 * @param InName Name of the property to add
	 * @param InSourceProperty The property to add
	 * @param bOverwrite Overwrite the property if it already exists.
	 * @return The result of the alteration.

	 */
	UE_API EPropertyBagAlterationResult AddProperty(const FName InName, const FProperty* InSourceProperty, bool bOverwrite = true);

	/**
	 * Duplicates the given property in the bag if it exists.
	 * @param InName Name of the existing property
	 * @param InOutNewName Name of the new property
	 */
	UE_API EPropertyBagAlterationResult DuplicateProperty(FName InName, FName* InOutNewName = nullptr);

	/**
	 * Clears all properties, then adds the supplied properties to the bag and sets their respective values.
	 * Numeric property values will be converted if possible, when a property's type changes.
	 * @param InDescs Descriptors of new properties to add. Must be the same size as InValues.
	 * @param InValues Values of new properties to add. Must be the same size as InDescs.
	 */
	UE_API EPropertyBagResult ReplaceAllPropertiesAndValues(const TConstArrayView<FPropertyBagPropertyDesc> InDescs, const TConstArrayView<TConstArrayView<uint8>> InValues);

	/**
	 * Removes properties from the bag by name if they exist.
	 * @param PropertiesToRemove The names of the properties to remove.
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult RemovePropertiesByName(const TConstArrayView<FName> PropertiesToRemove);
	
	/**
	 * Removes a property from the bag by name if it exists.
	 * @param PropertyToRemove The name of the property to remove.
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult RemovePropertyByName(const FName PropertyToRemove);

	/**
	 * Renames a property in the bag if it exists.
	 * @param PropertyToRename The name of the property to rename
	 * @param NewName The new name of the property
	 * @return The result of the alteration
	 */
	UE_API EPropertyBagAlterationResult RenameProperty(FName PropertyToRename, FName NewName);

	/**
	 * Reorders a property either before or after another target property.
	 * @param SourcePropertyName The source property to insert elsewhere in the array.
	 * @param TargetPropertyName The target property to conduct the reorder around (before or after).
	 * @param bInsertBefore Insert the source property before the target property. If false, it will be inserted after.
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult ReorderProperty(FName SourcePropertyName, FName TargetPropertyName, bool bInsertBefore = true);

	/**
	 * Reorders a property either before or after another target property.
	 * @param SourcePropertyIndex The index of the source property to insert elsewhere in the array.
	 * @param TargetPropertyIndex The index of the target property to conduct the reorder around (before or after).
	 * @param bInsertBefore Insert the source property before the target property. If false, it will be inserted after.
	 * @return The result of the alteration.
	 */
	UE_API EPropertyBagAlterationResult ReorderProperty(int32 SourcePropertyIndex, int32 TargetPropertyIndex, bool bInsertBefore = true);

	/**
	 * Changes the type of this bag and migrates existing values.
	 * The properties are matched between the bags based on the property ID.
	 * @param NewBagStruct Pointer to the new type.
	 */
	UE_API void MigrateToNewBagStruct(const UPropertyBag* NewBagStruct);

	/**
	 * Changes the type of this bag to the InNewBagInstance, and migrates existing values over.
	 * Properties that do not exist in this bag will get values from NewBagInstance.
	 * The properties are matched between the bags based on the property ID.
	 * @param InNewBagInstance New bag composition and values used for new properties.
	 */
	UE_API void MigrateToNewBagInstance(const FInstancedPropertyBag& InNewBagInstance);

	/**
	 * Changes the type of this bag to the InNewBagInstance, and migrates existing values over if marked as overridden in the OverriddenPropertyIDs.
	 * Properties that does not exist in this bag, or are not overridden, will get values from InNewBagInstance.
	 * The properties are matched between the bags based on the property ID.
	 * @param InNewBagInstance New bag composition and values used for new properties.
	 * @param OverriddenPropertyIDs Array if property IDs which should be copied over to the new instance. 
	 */
	UE_API void MigrateToNewBagInstanceWithOverrides(const FInstancedPropertyBag& InNewBagInstance, TConstArrayView<FGuid> OverriddenPropertyIDs);

	/** @return pointer to the property bag struct. */ 
	UE_API const UPropertyBag* GetPropertyBagStruct() const;
	
	/** Returns property descriptor by specified name. */
	UE_API const FPropertyBagPropertyDesc* FindPropertyDescByID(const FGuid ID) const;
	
	/** Returns property descriptor by specified ID. */
	UE_API const FPropertyBagPropertyDesc* FindPropertyDescByName(const FName Name) const;

	/** Returns true if we own the supplied property description, false otherwise. */
	UE_API bool OwnsPropertyDesc(const FPropertyBagPropertyDesc& Desc) const;

	/** Returns true if our propertybag has the same layout & per property types as another propertybag. */
	UE_API bool HasSameLayout(const FInstancedPropertyBag& Other) const;

	/** @return const view to the struct that holds the values. NOTE: The returned value/view cannot be serialized, use this to access the struct only temporarily. */
	FConstStructView GetValue() const { return Value; };

	/** @return const view to the struct that holds the values. NOTE: The returned value/view cannot be serialized, use this to access the struct only temporarily. */
	FStructView GetMutableValue() { return Value; };

	/** Return Internal instanced struct to be able to take ownership on the internal memory of this instanced property bag. Internal value will be reset to invalid state. */
	UE_API FInstancedStruct Detach();

	/**
	 * Getters
	 * Numeric types (bool, (u)int8, (u)int16, (u)int32, (u)int64, float, double) support type conversion.
	 */
	UE_API TValueOrError<bool, EPropertyBagResult> GetValueBool(const FName Name) const;
	UE_API TValueOrError<int8, EPropertyBagResult> GetValueInt8(const FName Name) const;
	UE_API TValueOrError<uint8, EPropertyBagResult> GetValueByte(const FName Name) const;
	UE_API TValueOrError<int16, EPropertyBagResult> GetValueInt16(const FName Name) const;
	UE_API TValueOrError<uint16, EPropertyBagResult> GetValueUInt16(const FName Name) const;
	UE_API TValueOrError<int32, EPropertyBagResult> GetValueInt32(const FName Name) const;
	UE_API TValueOrError<uint32, EPropertyBagResult> GetValueUInt32(const FName Name) const;
	UE_API TValueOrError<int64, EPropertyBagResult> GetValueInt64(const FName Name) const;
	UE_API TValueOrError<uint64, EPropertyBagResult> GetValueUInt64(const FName Name) const;
	UE_API TValueOrError<float, EPropertyBagResult> GetValueFloat(const FName Name) const;
	UE_API TValueOrError<double, EPropertyBagResult> GetValueDouble(const FName Name) const;
	UE_API TValueOrError<FName, EPropertyBagResult> GetValueName(const FName Name) const;
	UE_API TValueOrError<FString, EPropertyBagResult> GetValueString(const FName Name) const;
	UE_API TValueOrError<FText, EPropertyBagResult> GetValueText(const FName Name) const;
	UE_API TValueOrError<uint8, EPropertyBagResult> GetValueEnum(const FName Name, const UEnum* RequestedEnum) const;
	UE_API TValueOrError<int64, EPropertyBagResult> GetValueEnumInt64(const FName Name, const UEnum* RequestedEnum) const;
	UE_API TValueOrError<FStructView, EPropertyBagResult> GetValueStruct(const FName Name, const UScriptStruct* RequestedStruct = nullptr) const;
	UE_API TValueOrError<UObject*, EPropertyBagResult> GetValueObject(const FName Name, const UClass* RequestedClass = nullptr) const;
	UE_API TValueOrError<UClass*, EPropertyBagResult> GetValueClass(const FName Name) const;
	UE_API TValueOrError<FSoftObjectPath, EPropertyBagResult> GetValueSoftPath(const FName Name) const;

	/** @return string-based serialized representation of the value. */
	UE_API TValueOrError<FString, EPropertyBagResult> GetValueSerializedString(const FName Name) const;

	/** @return enum value of specified type. */
	template <typename T>
	TValueOrError<T, EPropertyBagResult> GetValueEnum(const FName Name) const
	{
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		
		TValueOrError<int64, EPropertyBagResult> Result = GetValueEnumInt64(Name, StaticEnum<T>());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		return MakeValue(static_cast<T>(Result.GetValue()));
	}
	
	/** @return struct reference of specified type. */
	template <typename T>
	TValueOrError<T*, EPropertyBagResult> GetValueStruct(const FName Name) const
	{
		TValueOrError<FStructView, EPropertyBagResult> Result = GetValueStruct(Name, TBaseStructure<T>::Get());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		if (T* ValuePtr = Result.GetValue().GetPtr<T>())
		{
			return MakeValue(ValuePtr);
		}
		return MakeError(EPropertyBagResult::TypeMismatch);
	}
	
	/** @return object pointer value of specified type. */
	template <typename T>
	TValueOrError<T*, EPropertyBagResult> GetValueObject(const FName Name) const
	{
		static_assert(TIsDerivedFrom<T, UObject>::Value, "Should only call this with object types");
		
		TValueOrError<UObject*, EPropertyBagResult> Result = GetValueObject(Name, T::StaticClass());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		if (Result.GetValue() == nullptr)
		{
			return MakeValue(nullptr);
		}
		if (T* Object = Cast<T>(Result.GetValue()))
		{
			return MakeValue(Object);
		}
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

private:
	template<typename T>
	TValueOrError<T, EPropertyBagResult> GetNumericalValue(const FName Desc) const;

public:
	UE_API TValueOrError<bool, EPropertyBagResult> GetValueBool(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<int8, EPropertyBagResult> GetValueInt8(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<uint8, EPropertyBagResult> GetValueByte(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<int16, EPropertyBagResult> GetValueInt16(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<uint16, EPropertyBagResult> GetValueUInt16(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<int32, EPropertyBagResult> GetValueInt32(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<uint32, EPropertyBagResult> GetValueUInt32(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<int64, EPropertyBagResult> GetValueInt64(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<uint64, EPropertyBagResult> GetValueUInt64(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<float, EPropertyBagResult> GetValueFloat(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<double, EPropertyBagResult> GetValueDouble(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<FName, EPropertyBagResult> GetValueName(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<FString, EPropertyBagResult> GetValueString(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<FText, EPropertyBagResult> GetValueText(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<uint8, EPropertyBagResult> GetValueEnum(const FPropertyBagPropertyDesc& Desc, const UEnum* RequestedEnum) const;
	UE_API TValueOrError<int64, EPropertyBagResult> GetValueEnumInt64(const FPropertyBagPropertyDesc& Desc, const UEnum* RequestedEnum) const;
	UE_API TValueOrError<FStructView, EPropertyBagResult> GetValueStruct(const FPropertyBagPropertyDesc& Desc, const UScriptStruct* RequestedStruct = nullptr) const;
	UE_API TValueOrError<UObject*, EPropertyBagResult> GetValueObject(const FPropertyBagPropertyDesc& Desc, const UClass* RequestedClass = nullptr) const;
	UE_API TValueOrError<UClass*, EPropertyBagResult> GetValueClass(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<FSoftObjectPath, EPropertyBagResult> GetValueSoftPath(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetMutableArrayRef(const FPropertyBagPropertyDesc& Desc);
	UE_API TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> GetArrayRef(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<FPropertyBagSetRef, EPropertyBagResult> GetMutableSetRef(const FPropertyBagPropertyDesc& Desc);
	UE_API TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> GetSetRef(const FPropertyBagPropertyDesc& Desc) const;
	UE_API TValueOrError<FPropertyBagMapRef, EPropertyBagResult> GetMutableMapRef(const FPropertyBagPropertyDesc& Desc);
	UE_API TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapRef(const FPropertyBagPropertyDesc& Desc) const;

	/** @return enum value of specified type. */
	template <typename T>
	TValueOrError<T, EPropertyBagResult> GetValueEnum(const FPropertyBagPropertyDesc& Desc) const
	{
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");

		TValueOrError<int64, EPropertyBagResult> Result = GetValueEnumInt64(Desc, StaticEnum<T>());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		return MakeValue(static_cast<T>(Result.GetValue()));
	}

	/** @return struct reference of specified type. */
	template <typename T>
	TValueOrError<T*, EPropertyBagResult> GetValueStruct(const FPropertyBagPropertyDesc& Desc) const
	{
		TValueOrError<FStructView, EPropertyBagResult> Result = GetValueStruct(Desc, TBaseStructure<T>::Get());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		if (T* ValuePtr = Result.GetValue().GetPtr<T>())
		{
			return MakeValue(ValuePtr);
		}
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

	/** @return object pointer value of specified type. */
	template <typename T>
	TValueOrError<T*, EPropertyBagResult> GetValueObject(const FPropertyBagPropertyDesc& Desc) const
	{
		static_assert(TIsDerivedFrom<T, UObject>::Value, "Should only call this with object types");

		TValueOrError<UObject*, EPropertyBagResult> Result = GetValueObject(Desc, T::StaticClass());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		if (Result.GetValue() == nullptr)
		{
			return MakeValue(nullptr);
		}
		if (T* Object = Cast<T>(Result.GetValue()))
		{
			return MakeValue(Object);
		}
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

private:
	template<typename T>
	TValueOrError<T, EPropertyBagResult> GetNumericalValue(const FPropertyBagPropertyDesc& Desc) const;

public:
	/**
	 * Value Setters. A property must exist in that bag before it can be set.  
	 * Numeric types (bool, (u)int8, (u)int16, (u)int32, (u)int64, float, double) support type conversion.
	 */
	UE_API EPropertyBagResult SetValueBool(const FName Name, const bool bInValue);
	UE_API EPropertyBagResult SetValueInt8(const FName Name, const int8 InValue);
	UE_API EPropertyBagResult SetValueByte(const FName Name, const uint8 InValue);
	UE_API EPropertyBagResult SetValueInt16(const FName Name, const int16 InValue);
	UE_API EPropertyBagResult SetValueUInt16(const FName Name, const uint16 InValue);
	UE_API EPropertyBagResult SetValueInt32(const FName Name, const int32 InValue);
	UE_API EPropertyBagResult SetValueUInt32(const FName Name, const uint32 InValue);
	UE_API EPropertyBagResult SetValueInt64(const FName Name, const int64 InValue);
	UE_API EPropertyBagResult SetValueUInt64(const FName Name, const uint64 InValue);
	UE_API EPropertyBagResult SetValueFloat(const FName Name, const float InValue);
	UE_API EPropertyBagResult SetValueDouble(const FName Name, const double InValue);
	UE_API EPropertyBagResult SetValueName(const FName Name, const FName InValue);
	UE_API EPropertyBagResult SetValueString(const FName Name, const FString& InValue);
	UE_API EPropertyBagResult SetValueText(const FName Name, const FText& InValue);
	UE_API EPropertyBagResult SetValueEnum(const FName Name, const int64 InValue, const UEnum* Enum);
	UE_API EPropertyBagResult SetValueStruct(const FName Name, FConstStructView InValue);
	UE_API EPropertyBagResult SetValueObject(const FName Name, UObject* InValue);
	UE_API EPropertyBagResult SetValueClass(const FName Name, UClass* InValue);
	UE_API EPropertyBagResult SetValueSoftPath(const FName Name, const FSoftObjectPath& InValue);
	UE_API EPropertyBagResult SetValueSoftPath(const FName Name, const UObject* InValue);

	/**
	 * Sets property value from a serialized representation of the value. If the string value provided
	 * cannot be parsed by the property, the operation will fail.
	 */
	UE_API EPropertyBagResult SetValueSerializedString(const FName Name, const FString& InValue);

	/** Sets enum value specified type. */
	template <typename T>
	EPropertyBagResult SetValueEnum(const FName Name, const T InValue)
	{
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		return SetValueEnum(Name, static_cast<int64>(InValue), StaticEnum<T>());
	}

	/** Sets struct value specified type. */
	template <typename T>
	EPropertyBagResult SetValueStruct(const FName Name, const T& InValue)
	{
		return SetValueStruct(Name, FConstStructView::Make(InValue));
	}

	/** Sets object pointer value specified type. */
	template <typename T>
	EPropertyBagResult SetValueObject(const FName Name, T* InValue)
	{
		static_assert(TIsDerivedFrom<T, UObject>::Value, "Should only call this with object types");
		return SetValueObject(Name, (UObject*)InValue);
	}

	/**
	 * Sets property value from given source property and source container address
	 * A property must exist in that bag before it can be set. 
	 */
	UE_API EPropertyBagResult SetValue(const FName Name, const FProperty* InSourceProperty, const void* InSourceContainerAddress);

	UE_API EPropertyBagResult SetValueBool(const FPropertyBagPropertyDesc& Desc, const bool bInValue);
	UE_API EPropertyBagResult SetValueInt8(const FPropertyBagPropertyDesc& Desc, const int8 InValue);
	UE_API EPropertyBagResult SetValueByte(const FPropertyBagPropertyDesc& Desc, const uint8 InValue);
	UE_API EPropertyBagResult SetValueInt16(const FPropertyBagPropertyDesc& Desc, const int16 InValue);
	UE_API EPropertyBagResult SetValueUInt16(const FPropertyBagPropertyDesc& Desc, const uint16 InValue);
	UE_API EPropertyBagResult SetValueInt32(const FPropertyBagPropertyDesc& Desc, const int32 InValue);
	UE_API EPropertyBagResult SetValueUInt32(const FPropertyBagPropertyDesc& Desc, const uint32 InValue);
	UE_API EPropertyBagResult SetValueInt64(const FPropertyBagPropertyDesc& Desc, const int64 InValue);
	UE_API EPropertyBagResult SetValueUInt64(const FPropertyBagPropertyDesc& Desc, const uint64 InValue);
	UE_API EPropertyBagResult SetValueFloat(const FPropertyBagPropertyDesc& Desc, const float InValue);
	UE_API EPropertyBagResult SetValueDouble(const FPropertyBagPropertyDesc& Desc, const double InValue);
	UE_API EPropertyBagResult SetValueName(const FPropertyBagPropertyDesc& Desc, const FName InValue);
	UE_API EPropertyBagResult SetValueString(const FPropertyBagPropertyDesc& Desc, const FString& InValue);
	UE_API EPropertyBagResult SetValueText(const FPropertyBagPropertyDesc& Desc, const FText& InValue);
	UE_API EPropertyBagResult SetValueEnum(const FPropertyBagPropertyDesc& Desc, const int64 InValue, const UEnum* Enum);
	UE_API EPropertyBagResult SetValueStruct(const FPropertyBagPropertyDesc& Desc, FConstStructView InValue);
	UE_API EPropertyBagResult SetValueObject(const FPropertyBagPropertyDesc& Desc, UObject* InValue);
	UE_API EPropertyBagResult SetValueClass(const FPropertyBagPropertyDesc& Desc, UClass* InValue);
	UE_API EPropertyBagResult SetValueSoftPath(const FPropertyBagPropertyDesc& Desc, const FSoftObjectPath& InValue);
	UE_API EPropertyBagResult SetValueSoftPath(const FPropertyBagPropertyDesc& Desc, const UObject* InValue);

	/** Sets enum value specified type. */
	template <typename T>
	EPropertyBagResult SetValueEnum(const FPropertyBagPropertyDesc& Desc, const T InValue)
	{
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		return SetValueEnum(Desc, static_cast<int64>(InValue), StaticEnum<T>());
	}

	/** Sets struct value specified type. */
	template <typename T>
	EPropertyBagResult SetValueStruct(const FPropertyBagPropertyDesc& Desc, const T& InValue)
	{
		return SetValueStruct(Desc, FConstStructView::Make(InValue));
	}

	/** Sets object pointer value specified type. */
	template <typename T>
	EPropertyBagResult SetValueObject(const FPropertyBagPropertyDesc& Desc, T* InValue)
	{
		static_assert(TIsDerivedFrom<T, UObject>::Value, "Should only call this with object types");
		return SetValueObject(Desc, (UObject*)InValue);
	}

	/**
	 * Returns helper class to modify and access an array property.
	 * Note: The array reference is not valid after the layout of the referenced property bag has changed!
	 * @returns helper class to modify and access arrays
	*/
	UE_API TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetMutableArrayRef(const FName Name);

	/**
	 * Returns helper class to access an array property.
	 * Note: The array reference is not valid after the layout of the referenced property bag has changed!
	 * @returns helper class to modify and access arrays
	*/
	UE_API TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> GetArrayRef(const FName Name) const;

	/**
	 * Returns helper class to modify and access a set property.
	 * Note: The set reference is not valid after the layout of the referenced property bag has changed!
	 * @returns helper class to modify and access sets
	*/
	UE_API TValueOrError<FPropertyBagSetRef, EPropertyBagResult> GetMutableSetRef(const FName Name);
	
	/**
	 * Returns helper class to access a set property.
	 * Note: The set reference is not valid after the layout of the referenced property bag has changed!
	 * @returns helper class to modify and access sets
	*/
	UE_API TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> GetSetRef(const FName Name) const;

	/**
	 * Returns helper class to modify and access a map property.
	 * Note: The map reference is not valid after the layout of the referenced property bag has changed!
	 * @returns helper class to modify and access maps
	*/
	UE_API TValueOrError<FPropertyBagMapRef, EPropertyBagResult> GetMutableMapRef(const FName Name);

	/**
	 * Returns helper class to access a map property.
	 * Note: The map reference is not valid after the layout of the referenced property bag has changed!
	 * @returns helper class to modify and access maps
	*/
	UE_API TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetMapRef(const FName Name) const;

	UE_API bool Identical(const FInstancedPropertyBag* Other, uint32 PortFlags) const;
	UE_API bool Serialize(FArchive& Ar);
	UE_API void AddStructReferencedObjects(FReferenceCollector& Collector);
	UE_API void GetPreloadDependencies(TArray<UObject*>& OutDeps);

	/**
	 * Checks whether a provided name is a valid property bag name to create a new property bag property.
	 * Note: Some characters are allowed that are still invalid, but for workflow reasons they are acceptable and
	 * should still be sanitized to remove them when adding to the property bag, ex. spaces.
	 * @param Name The name to check for invalid characters.
	 * @return True if the property name is void of any strictly invalid characters.
	 */
	UE_API static bool IsPropertyNameValid(const FString& Name);

	/**
	 * Checks whether a provided name is a valid property bag name to create a new property bag property.
	 * Note: Some characters are allowed that are still invalid, but for workflow reasons they are acceptable and
	 * should still be sanitized to remove them when adding to the property bag, ex. spaces.
	 * @param Name The name to check for invalid characters.
	 * @return True if the property name is void of any strictly invalid characters.
	 */
	UE_API static bool IsPropertyNameValid(const FName Name);

	/**
	 * Returns a sanitized version of the provided name without invalid characters.
	 * @param Name The name to sanitize.
	 * @param ReplacementChar Will replace invalid characters.
	 * @return Potentially modified property name replacing invalid characters with the provided replacement.
	 */
	UE_API static FName SanitizePropertyName(const FString& Name, const TCHAR ReplacementChar = TEXT('_'));

	/**
	 * Returns a sanitized version of the provided name without invalid characters.
	 * @param Name The name to sanitize.
	 * @param ReplacementChar Will replace invalid characters.
	 * @return Potentially modified property name replacing invalid characters with the provided replacement.
	 */
	UE_API static FName SanitizePropertyName(FName Name, const TCHAR ReplacementChar = TEXT('_'));

protected:
	UE_API const void* GetValueAddress(const FPropertyBagPropertyDesc* Desc) const;
	UE_API void* GetMutableValueAddress(const FPropertyBagPropertyDesc* Desc);
	
	UPROPERTY(EditAnywhere, Category="", meta=(SequencerUseParentPropertyName=true))
	FInstancedStruct Value;
};

template<> struct TStructOpsTypeTraits<FInstancedPropertyBag> : public TStructOpsTypeTraitsBase2<FInstancedPropertyBag>
{
	enum
	{
		WithIdentical = true,
		WithSerializer = true,
		WithAddStructReferencedObjects = true,
		WithGetPreloadDependencies = true,
	};
};


/**
 * A reference to an array in FInstancedPropertyBag
 * Allows to modify the array via the FScriptArrayHelper API, and contains helper methods to get and set properties.
 *
 *		FInstancedPropertyBag Bag;
 *		Bag.AddProperties({
 *			{ ArrayName, EPropertyBagContainerType::Array, EPropertyBagPropertyType::Float }
 *		});
 *
 *		if (auto FloatArrayRes = Bag.GetArrayRef(ArrayName); FloatArrayRes.IsValid())
 *		{
 *			FPropertyBagArrayRef& FloatArray = FloatArrayRes.GetValue();
 *			const int32 NewIndex = FloatArray.AddValue();
 *			FloatArray.SetValueFloat(NewIndex, 123.0f);
 *		}
 * 
 * Note: The array reference is not valid after the layout of the referenced property bag has changed! 
 */
class FPropertyBagArrayRef : public FScriptArrayHelper
{
	FPropertyBagPropertyDesc ValueDesc;

	const void* GetAddress(const int32 Index) const
	{
		if (IsValidIndex(Index) == false)
		{
			return nullptr;
		}
		// Ugly, but FScriptArrayHelper does not give us other option.
		FPropertyBagArrayRef* NonConstThis = const_cast<FPropertyBagArrayRef*>(this);
		return static_cast<void*>(NonConstThis->GetRawPtr(Index));
	}

	void* GetMutableAddress(const int32 Index) const
	{
		if (IsValidIndex(Index) == false)
		{
			return nullptr;
		}
		// Ugly, but FScriptArrayHelper does not give us other option.
		FPropertyBagArrayRef* NonConstThis = const_cast<FPropertyBagArrayRef*>(this);
		return (void*)NonConstThis->GetRawPtr(Index);
	}

public:	
	inline FPropertyBagArrayRef(const FPropertyBagPropertyDesc& InDesc, const void* InArray)
		: FScriptArrayHelper(CastField<FArrayProperty>(InDesc.CachedProperty), InArray)
	{
		const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InDesc.CachedProperty);
		check(ArrayProperty);
		check(ArrayProperty->Inner);
		// Create dummy desc for the inner property. 
		ValueDesc.ValueType = InDesc.ValueType;
		ValueDesc.ValueTypeObject = InDesc.ValueTypeObject;
		ValueDesc.CachedProperty = ArrayProperty->Inner;
		ValueDesc.ContainerTypes = InDesc.ContainerTypes;
		ValueDesc.ContainerTypes.PopHead();
	}

	/**
	 * Getters
	 * Numeric types (bool, (u)int8, (u)int16, (u)int32, (u)int64, float, double) support type conversion.
	 */
	
	UE_API TValueOrError<bool, EPropertyBagResult> GetValueBool(const int32 Index) const;
	UE_API TValueOrError<int8, EPropertyBagResult> GetValueInt8(const int32 Index) const;
	UE_API TValueOrError<uint8, EPropertyBagResult> GetValueByte(const int32 Index) const;
	UE_API TValueOrError<int16, EPropertyBagResult> GetValueInt16(const int32 Index) const;
	UE_API TValueOrError<uint16, EPropertyBagResult> GetValueUInt16(const int32 Index) const;
	UE_API TValueOrError<int32, EPropertyBagResult> GetValueInt32(const int32 Index) const;
	UE_API TValueOrError<uint32, EPropertyBagResult> GetValueUInt32(const int32 Index) const;
	UE_API TValueOrError<int64, EPropertyBagResult> GetValueInt64(const int32 Index) const;
	UE_API TValueOrError<uint64, EPropertyBagResult> GetValueUInt64(const int32 Index) const;
	UE_API TValueOrError<float, EPropertyBagResult> GetValueFloat(const int32 Index) const;
	UE_API TValueOrError<double, EPropertyBagResult> GetValueDouble(const int32 Index) const;
	UE_API TValueOrError<FName, EPropertyBagResult> GetValueName(const int32 Index) const;
	UE_API TValueOrError<FString, EPropertyBagResult> GetValueString(const int32 Index) const;
	UE_API TValueOrError<FText, EPropertyBagResult> GetValueText(const int32 Index) const;
	UE_API TValueOrError<uint8, EPropertyBagResult> GetValueEnum(const int32 Index, const UEnum* RequestedEnum) const;
	UE_API TValueOrError<int64, EPropertyBagResult> GetValueEnumInt64(const int32 Index, const UEnum* RequestedEnum) const;
	UE_API TValueOrError<FStructView, EPropertyBagResult> GetValueStruct(const int32 Index, const UScriptStruct* RequestedStruct = nullptr) const;
	UE_API TValueOrError<UObject*, EPropertyBagResult> GetValueObject(const int32 Index, const UClass* RequestedClass = nullptr) const;
	UE_API TValueOrError<UClass*, EPropertyBagResult> GetValueClass(const int32 Index) const;
	UE_API TValueOrError<FSoftObjectPath, EPropertyBagResult> GetValueSoftPath(const int32 Index) const;

	/** @return enum value of specified type. */
	template <typename T>
	TValueOrError<T, EPropertyBagResult> GetValueEnum(const int32 Index) const
	{
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		
		TValueOrError<int64, EPropertyBagResult> Result = GetValueEnumInt64(Index, StaticEnum<T>());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		return MakeValue(static_cast<T>(Result.GetValue()));
	}
	
	/** @return struct reference of specified type. */
	template <typename T>
	TValueOrError<T*, EPropertyBagResult> GetValueStruct(const int32 Index) const
	{
		TValueOrError<FStructView, EPropertyBagResult> Result = GetValueStruct(Index, TBaseStructure<T>::Get());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		if (T* ValuePtr = Result.GetValue().GetPtr<T>())
		{
			return MakeValue(ValuePtr);
		}
		return MakeError(EPropertyBagResult::TypeMismatch);
	}
	
	/** @return object pointer value of specified type. */
	template <typename T>
	TValueOrError<T*, EPropertyBagResult> GetValueObject(const int32 Index) const
	{
		static_assert(TIsDerivedFrom<T, UObject>::Value, "Should only call this with object types");
		
		TValueOrError<UObject*, EPropertyBagResult> Result = GetValueObject(Index, T::StaticClass());
		if (!Result.IsValid())
		{
			return MakeError(Result.GetError());
		}
		if (Result.GetValue() == nullptr)
		{
			return MakeValue(nullptr);
		}
		if (T* Object = Cast<T>(Result.GetValue()))
		{
			return MakeValue(Object);
		}
		return MakeError(EPropertyBagResult::TypeMismatch);
	}

private:
	template<typename T>
	TValueOrError<T, EPropertyBagResult> GetNumericalValue(const int32 Index) const;

public:
	/**
     * Returns helper class to modify and access a nested array (mutable version).
     * Note: Note: The array reference is not valid after the layout of the referenced property bag has changed!
     * @returns helper class to modify and access arrays
    */
	UE_API TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetMutableNestedArrayRef(const int32 Index = 0) const;

	/**
     * Returns helper class to access a nested array (const version).
     * Note: Note: The array reference is not valid after the layout of the referenced property bag has changed!
     * @returns helper class to access arrays
    */
	UE_API TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> GetNestedArrayRef(const int32 Index = 0) const;

	/**
	 * Value Setters. A property must exist in that bag before it can be set.  
	 * Numeric types (bool, (u)int8, (u)int16, (u)int32, (u)int64, float, double) support type conversion.
	 */
	UE_API EPropertyBagResult SetValueBool(const int32 Index, const bool bInValue);
	UE_API EPropertyBagResult SetValueInt8(const int32 Index, const int8 InValue);
	UE_API EPropertyBagResult SetValueByte(const int32 Index, const uint8 InValue);
	UE_API EPropertyBagResult SetValueInt16(const int32 Index, const int16 InValue);
	UE_API EPropertyBagResult SetValueUInt16(const int32 Index, const uint16 InValue);
	UE_API EPropertyBagResult SetValueInt32(const int32 Index, const int32 InValue);
	UE_API EPropertyBagResult SetValueUInt32(const int32 Index, const uint32 InValue);
	UE_API EPropertyBagResult SetValueInt64(const int32 Index, const int64 InValue);
	UE_API EPropertyBagResult SetValueUInt64(const int32 Index, const uint64 InValue);
	UE_API EPropertyBagResult SetValueFloat(const int32 Index, const float InValue);
	UE_API EPropertyBagResult SetValueDouble(const int32 Index, const double InValue);
	UE_API EPropertyBagResult SetValueName(const int32 Index, const FName InValue);
	UE_API EPropertyBagResult SetValueString(const int32 Index, const FString& InValue);
	UE_API EPropertyBagResult SetValueText(const int32 Index, const FText& InValue);
	UE_API EPropertyBagResult SetValueEnum(const int32 Index, const int64 InValue, const UEnum* Enum);
	UE_API EPropertyBagResult SetValueStruct(const int32 Index, FConstStructView InValue);
	UE_API EPropertyBagResult SetValueObject(const int32 Index, UObject* InValue);
	UE_API EPropertyBagResult SetValueClass(const int32 Index, UClass* InValue);
	UE_API EPropertyBagResult SetValueSoftPath(const int32 Index, const FSoftObjectPath& InValue);
	UE_API EPropertyBagResult SetValueSoftPath(const int32 Index, const UObject* InValue);

	/** Sets enum value specified type. */
	template <typename T>
	EPropertyBagResult SetValueEnum(const int32 Index, const T InValue)
	{
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		return SetValueEnum(Index, static_cast<int64>(InValue), StaticEnum<T>());
	}

	/** Sets struct value specified type. */
	template <typename T>
	EPropertyBagResult SetValueStruct(const int32 Index, const T& InValue)
	{
		return SetValueStruct(Index, FConstStructView::Make(InValue));
	}

	/** Sets object pointer value specified type. */
	template <typename T>
	EPropertyBagResult SetValueObject(const int32 Index, T* InValue)
	{
		static_assert(TIsDerivedFrom<T, UObject>::Value, "Should only call this with object types");
		return SetValueObject(Index, static_cast<UObject*>(InValue));
	}
};

/**
 * A reference to a set in FInstancedPropertyBag
 * Contains helper methods to get and set properties.
 *
 *		FInstancedPropertyBag Bag;
 *		Bag.AddProperties({
 *			{ SetName, EPropertyBagContainerType::Set, EPropertyBagPropertyType::Float }
 *		});
 *
 *		if (auto FloatSetRes = Bag.GetSetRef(ArrayName); FloatSetRes.IsValid())
 *		{
 *			FPropertyBagSetRef& FloatSet = FloatSetRes.GetValue();
 *			FloatSet.AddValueFloat(123.f);
 *		}
 * 
 * Note: The set reference is not valid after the layout of the referenced property bag has changed! 
 */
class FPropertyBagSetRef : private FScriptSetHelper
{
public:	
	inline FPropertyBagSetRef(const FPropertyBagPropertyDesc& InDesc, const void* InSet)
		: FScriptSetHelper(CastField<FSetProperty>(InDesc.CachedProperty), InSet)
	{
		const FSetProperty* SetProperty = CastField<FSetProperty>(InDesc.CachedProperty);
		check(SetProperty);
		check(SetProperty->ElementProp);
		// Create dummy desc for the inner property. 
		ValueDesc.ValueType = InDesc.ValueType;
		ValueDesc.ValueTypeObject = InDesc.ValueTypeObject;
		ValueDesc.CachedProperty = SetProperty->ElementProp;
		ValueDesc.ContainerTypes = InDesc.ContainerTypes;
		ValueDesc.ContainerTypes.PopHead();
	}

	/** Add value to set. If value is already present, it will not be added */
	UE_API EPropertyBagResult AddValueBool(const bool bInValue);
	UE_API EPropertyBagResult AddValueInt8(const int8 InValue);
	UE_API EPropertyBagResult AddValueByte(const uint8 InValue);
	UE_API EPropertyBagResult AddValueInt16(const int16 InValue);
	UE_API EPropertyBagResult AddValueUInt16(const uint16 InValue);
	UE_API EPropertyBagResult AddValueInt32(const int32 InValue);
	UE_API EPropertyBagResult AddValueUInt32(const uint32 InValue);
	UE_API EPropertyBagResult AddValueInt64(const int64 InValue);
	UE_API EPropertyBagResult AddValueUInt64(const uint64 InValue);
	UE_API EPropertyBagResult AddValueFloat(const float InValue);
	UE_API EPropertyBagResult AddValueDouble(const double InValue);
	UE_API EPropertyBagResult AddValueName(const FName InValue);
	UE_API EPropertyBagResult AddValueString(const FString& InValue);
	UE_API EPropertyBagResult AddValueText(const FText& InValue);
	UE_API EPropertyBagResult AddValueEnum(const int64 InValue, const UEnum* Enum);
	UE_API EPropertyBagResult AddValueStruct(FConstStructView InValue);
	UE_API EPropertyBagResult AddValueObject(UObject* InValue);
	UE_API EPropertyBagResult AddValueClass(UClass* InValue);
	UE_API EPropertyBagResult AddValueSoftPath(const FSoftObjectPath& InValue);

	/** Adds enum value specified type. */
	template <typename T>
	EPropertyBagResult AddValueEnum(const T InValue)
	{
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		return AddValueEnum(static_cast<int64>(InValue), StaticEnum<T>());
	}

	/** Adds struct value specified type. */
	template <typename T>
	EPropertyBagResult AddValueStruct(const T& InValue)
	{
		return AddValueStruct(FConstStructView::Make(InValue));
	}

	/** Adds object pointer value specified type. */
	template <typename T>
	EPropertyBagResult AddValueObject(T* InValue)
	{
		static_assert(TIsDerivedFrom<T, UObject>::Value, "Should only call this with object types");
		return AddValueObject(static_cast<UObject*>(InValue));
	}


	/** Removes value from set if found. */
	template <typename T>
	EPropertyBagResult Remove(const T& Value)
	{
		int32 ElementIndex = FindElementIndex(&Value);
		
		if (ElementIndex == INDEX_NONE)
		{
			return EPropertyBagResult::PropertyNotFound;
		}

		RemoveAt(ElementIndex);
		return EPropertyBagResult::Success;
	}

	/** Returns a bool specifying if the element was found or not */
	template <typename T>
	TValueOrError<bool, EPropertyBagResult> Contains(const T& Value) const
	{
        if (ValueDesc.CachedProperty == nullptr)
        {
            return MakeError(EPropertyBagResult::PropertyNotFound);
        }

        return MakeValue(FindElementIndex(&Value) != INDEX_NONE);
	}

	/** Returns number of elements in set. */
	UE_FORCEINLINE_HINT int32 Num() const
	{
		return FScriptSetHelper::Num();
	}

private:

	FPropertyBagPropertyDesc ValueDesc;

	template <typename T>
	EPropertyBagResult Add(const T& Value)
	{
		if (ValueDesc.CachedProperty == nullptr)
		{
			return EPropertyBagResult::PropertyNotFound;
		}

		int32 ElementIndex = FindElementIndex(&Value);

		if (ElementIndex != INDEX_NONE)
		{
			return EPropertyBagResult::DuplicatedValue;
		}

		AddElement(&Value);
		return EPropertyBagResult::Success;
	}
};

/**
 * Dummy types used to mark up missing types when creating property bags. These are used in the UI to display error message.
 */
UENUM()
enum class EPropertyBagMissingEnum : uint8
{
	Missing,
};

USTRUCT()
struct FPropertyBagMissingStruct
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UPropertyBagMissingObject : public UObject
{
	GENERATED_BODY()
};

namespace UE::Private { struct FJsonStringifyImpl; }
/**
 * A script struct that is used to store the value of the property bag instance.
 * References to UPropertyBag cannot be serialized, instead the array of the properties
 * is serialized and new class is create on load based on the composition of the properties.
 *
 * Note: Should not be used directly.
 */
UCLASS(Transient, MinimalAPI)
class UPropertyBag : public UScriptStruct
{
public:
	GENERATED_BODY()

	/**
	 * Returns UPropertyBag struct based on the property descriptions passed in.
	 * UPropertyBag struct names will be auto-generated by prefixing 'PropertyBag_' to the hash of the descriptions.
	 * If a UPropertyBag with same name already exists, the existing object is returned.
	 * This means that a property bags which share same layout (same descriptions) will share the same UPropertyBag.
	 * If there are multiple properties that have the same name, only the first property is added.
	 * The caller is expected to ensure unique names for the property descriptions.
	 */
	static UE_API const UPropertyBag* GetOrCreateFromDescs(const TConstArrayView<FPropertyBagPropertyDesc> InPropertyDescs, const TCHAR* PrefixName = nullptr);

	/**
	 * Fix property descriptions that are not properly formed.
	 * Remove duplicated property descriptions.
	 * Remove unsupported types.
	 */
	static UE_API TArray<FPropertyBagPropertyDesc> SanitizePropertyDescs(const TArrayView<const FPropertyBagPropertyDesc> PropertyDescs);

	/** Returns property descriptions that specify this struct. */
	TConstArrayView<FPropertyBagPropertyDesc> GetPropertyDescs() const { return PropertyDescs; }

	/** @return property description based on ID. */
	UE_API const FPropertyBagPropertyDesc* FindPropertyDescByID(const FGuid ID) const;

	/** @return property description based on name. */
	UE_API const FPropertyBagPropertyDesc* FindPropertyDescByName(const FName Name) const;

	/** @return property description based on the created property name. The name can be different from the descriptor name due to name sanitization. */
	UE_API const FPropertyBagPropertyDesc* FindPropertyDescByPropertyName(const FName PropertyName) const;

	/** @return property description based on pointer to property. */
	UE_API const FPropertyBagPropertyDesc* FindPropertyDescByProperty(const FProperty* Property) const;

	/** @return property description based on index. */
	UE_API const FPropertyBagPropertyDesc* FindPropertyDescByIndex(int32 Index) const;

	/** @return true if we own the supplied property description, false otherwise. */
	UE_API bool OwnsPropertyDesc(const FPropertyBagPropertyDesc& Desc) const;

#if WITH_EDITOR
	/** @return true if any of the properties on the bag has type of the specified user defined struct. */
	UE_API bool ContainsUserDefinedStruct(const UUserDefinedStruct* UserDefinedStruct) const;
#endif // WITH_EDITOR
	
protected:

	UE_API void DecrementRefCount() const;
	UE_API void IncrementRefCount() const;
	
	UE_API virtual void InitializeStruct(void* Dest, int32 ArrayDim = 1) const override;
	UE_API virtual void DestroyStruct(void* Dest, int32 ArrayDim = 1) const override;
	UE_API virtual void FinishDestroy() override;
	UE_API virtual void Link(FArchive& Ar, bool bRelinkExistingProperties) override;

	virtual bool ArePropertyGuidsAvailable() const override
	{
		return PropertyDescs.Num() > 0;
	}

	UE_API virtual FName FindPropertyNameFromGuid(const FGuid& PropertyID) const override;
	UE_API virtual FGuid FindPropertyGuidFromName(const FName PropertyName) const override;

	UPROPERTY()
	TArray<FPropertyBagPropertyDesc> PropertyDescs;

	std::atomic<int32> RefCount = 0;
	
	// both of these types need to serialize their property bag:
	friend struct FInstancedPropertyBag;
	friend struct ::UE::Private::FJsonStringifyImpl;
};

#undef UE_API
