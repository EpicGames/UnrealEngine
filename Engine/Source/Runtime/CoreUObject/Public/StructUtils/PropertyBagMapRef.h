// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Concepts/BaseStructureProvider.h"
#include "StructUtils/PropertyBag.h"

class FPropertyBagMapRef : private FScriptMapHelper
{
public:
	FORCEINLINE FPropertyBagMapRef(const FPropertyBagPropertyDesc& InDesc, const void* InMap)
		: FScriptMapHelper(CastField<FMapProperty>(InDesc.CachedProperty), InMap)
	{
		const FMapProperty* MapProperty = CastField<FMapProperty>(InDesc.CachedProperty);
		check(MapProperty);
		check(MapProperty->KeyProp);
		check(MapProperty->ValueProp);
		// Create dummy desc for the inner properties ( key + value ). 
		KeyDesc.ValueType = InDesc.KeyType;
		KeyDesc.ValueTypeObject = InDesc.KeyTypeObject;
		KeyDesc.CachedProperty = MapProperty->KeyProp;
		KeyDesc.ContainerTypes.Reset(); // Key's can't be containers		
		ValueDesc.ValueType = InDesc.ValueType;
		ValueDesc.ValueTypeObject = InDesc.ValueTypeObject;
		ValueDesc.CachedProperty = MapProperty->ValueProp;
		ValueDesc.ContainerTypes = InDesc.ContainerTypes;
		ValueDesc.ContainerTypes.PopHead();
		if (ValueDesc.ContainerTypes.GetFirstContainerType() == EPropertyBagContainerType::Map)
		{
			ValueDesc.KeyType = InDesc.KeyType;
			ValueDesc.KeyTypeObject = InDesc.KeyTypeObject;
		}
	}

	using FScriptMapHelper::Num;

	void Empty()
	{
		FScriptMapHelper::EmptyValues();
	}

	template <typename KeyType>
	TValueOrError<bool, EPropertyBagResult> Contains(const KeyType& InKey) const
	{
		EPropertyBagResult Result = CheckType<KeyType>(KeyDesc, InKey);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}

		const int32 PairIndex = Call<KeyType, int32>([this](const void* Key)
		{
			return const_cast<FPropertyBagMapRef*>(this)->FindMapPairIndexFromHash(Key);
		}, KeyDesc, InKey);

		return MakeValue(PairIndex != INDEX_NONE);
	}

	template <typename KeyType, typename ValueType>
	TValueOrError<ValueType, EPropertyBagResult> Find(const KeyType& InKey) const
	{
		EPropertyBagResult Result = CheckType<KeyType>(KeyDesc, InKey);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}

		uint8* RawValue = Call<KeyType, uint8*>([this](const void* Key)
		{
			return const_cast<FPropertyBagMapRef*>(this)->FindValueFromHash(Key);
		}, KeyDesc, InKey);
		
		if (RawValue == nullptr)
		{
			return MakeError(EPropertyBagResult::PropertyNotFound);
		}

		ValueType Value;
		Convert(ValueDesc, RawValue, Value);

		Result = CheckType<ValueType>(ValueDesc, Value);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}

		return MakeValue(Value);
	}

	template <typename KeyType>
	EPropertyBagResult Add(const KeyType& InKey)
	{
		TValueOrError<bool, EPropertyBagResult> Result = Contains(InKey);
		if (Result.HasError())
		{
			return Result.GetError();
		}
		if (Result.GetValue())
		{
			return EPropertyBagResult::DuplicatedValue;
		}

		void* RawValue = Call<KeyType, void*>([this](const void* Key) { return FScriptMapHelper::FindOrAdd(Key); }, KeyDesc, InKey);
		check(RawValue);

		return EPropertyBagResult::Success;
	}

	template <typename KeyType, typename ValueType>
	EPropertyBagResult Add(const KeyType& InKey, const ValueType& InValue)
	{
		EPropertyBagResult Result = CheckType<KeyType>(KeyDesc, InKey);
		if (Result != EPropertyBagResult::Success)
		{
			return Result;
		}

		Result = CheckType<ValueType>(ValueDesc, InValue);
		if (Result != EPropertyBagResult::Success)
		{
			return Result;
		}

		return AddPair(KeyDesc, InKey, ValueDesc, InValue);
	}

	template <typename KeyType>
	EPropertyBagResult Remove(const KeyType& InKey)
	{
		EPropertyBagResult Result = CheckType<KeyType>(KeyDesc, InKey);
		if (Result != EPropertyBagResult::Success)
		{
			return Result;
		}

		const bool bWasRemoved = Call<KeyType, bool>([this](const void* Key) { return FScriptMapHelper::RemovePair(Key); }, KeyDesc, InKey);

		return bWasRemoved ? EPropertyBagResult::Success : EPropertyBagResult::PropertyNotFound;
	}

	template <typename KeyType>
	TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> GetMutableNestedArrayRef(const KeyType& InKey)
	{
		EPropertyBagResult Result = CheckType<KeyType>(KeyDesc, InKey);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}

		if (ValueDesc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Array)
		{
			return MakeError(EPropertyBagResult::TypeMismatch);
		}

		uint8* RawValue = Call<KeyType, uint8*>([this](const void* Key) { return FScriptMapHelper::FindValueFromHash(Key); }, KeyDesc, InKey);
		if (RawValue == nullptr)
		{
			return MakeError(EPropertyBagResult::PropertyNotFound);
		}

		return MakeValue(FPropertyBagArrayRef(ValueDesc, RawValue));
	}

	template <typename KeyType>
	TValueOrError<const FPropertyBagArrayRef, EPropertyBagResult> GetNestedArrayRef(const KeyType& InKey)
	{
		TValueOrError<FPropertyBagArrayRef, EPropertyBagResult> Result = GetMutableNestedArrayRef(InKey);
		if (Result.HasError())
		{
			return MakeError(Result.GetError());
		}

		return MakeValue(Result.GetValue());
	}

	template <typename KeyType>
	TValueOrError<FPropertyBagSetRef, EPropertyBagResult> GetMutableNestedSetRef(const KeyType& InKey)
	{
		EPropertyBagResult Result = CheckType<KeyType>(KeyDesc, InKey);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}

		if (ValueDesc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Set)
		{
			return MakeError(EPropertyBagResult::TypeMismatch);
		}

		uint8* RawValue = Call<KeyType, uint8*>([this](const void* Key) { return FScriptMapHelper::FindValueFromHash(Key); }, KeyDesc, InKey);
		if (RawValue == nullptr)
		{
			return MakeError(EPropertyBagResult::PropertyNotFound);
		}

		return MakeValue(FPropertyBagSetRef(ValueDesc, RawValue));
	}

	template <typename KeyType>
	TValueOrError<const FPropertyBagSetRef, EPropertyBagResult> GetNestedSetRef(const KeyType& InKey)
	{
		TValueOrError<FPropertyBagSetRef, EPropertyBagResult> Result = GetMutableNestedSetRef(InKey);
		if (Result.HasError())
		{
			return MakeError(Result.GetError());
		}

		return MakeValue(Result.GetValue());
	}

	template <typename KeyType>
	TValueOrError<FPropertyBagMapRef, EPropertyBagResult> GetMutableNestedMapRef(const KeyType& InKey)
	{
		EPropertyBagResult Result = CheckType<KeyType>(KeyDesc, InKey);
		if (Result != EPropertyBagResult::Success)
		{
			return MakeError(Result);
		}

		if (ValueDesc.ContainerTypes.GetFirstContainerType() != EPropertyBagContainerType::Map)
		{
			return MakeError(EPropertyBagResult::TypeMismatch);
		}

		uint8* RawValue = Call<KeyType, uint8*>([this](const void* Key) { return FScriptMapHelper::FindValueFromHash(Key); }, KeyDesc, InKey);
		if (RawValue == nullptr)
		{
			return MakeError(EPropertyBagResult::PropertyNotFound);
		}

		return MakeValue(FPropertyBagMapRef(ValueDesc, RawValue));
	}

	template <typename KeyType>
	TValueOrError<const FPropertyBagMapRef, EPropertyBagResult> GetNestedMapRef(const KeyType& InKey)
	{
		TValueOrError<FPropertyBagMapRef, EPropertyBagResult> Result = GetMutableNestedMapRef(InKey);
		if (Result.HasError())
		{
			return MakeError(Result.GetError());
		}

		return MakeValue(Result.GetValue());
	}

private:

	template <typename T, typename TEnableIf<TIsEnumClass<T>::Value>::Type* = nullptr>
	static EPropertyBagResult CheckType(FPropertyBagPropertyDesc InDesc, const T InValue)
	{
		if (!InDesc.ContainerTypes.IsEmpty())
		{
			return EPropertyBagResult::TypeMismatch;
		}

		if (InDesc.ValueType == EPropertyBagPropertyType::Enum)
		{
			const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InDesc.CachedProperty);

			if (EnumProperty && EnumProperty->GetEnum() == StaticEnum<T>())
			{
				return EPropertyBagResult::Success;
			}
		}

		return EPropertyBagResult::TypeMismatch;
	}

	template <typename T, typename TEnableIf<TModels_V<CBaseStructureProvider, T>>::Type* = nullptr>
	static EPropertyBagResult CheckType(FPropertyBagPropertyDesc InDesc, const T InValue)
	{
		if (!InDesc.ContainerTypes.IsEmpty())
		{
			return EPropertyBagResult::TypeMismatch;
		}

		if (InDesc.ValueType == EPropertyBagPropertyType::Struct)
		{
			const FStructProperty* StructProperty = CastField<FStructProperty>(InDesc.CachedProperty);

			if (StructProperty && StructProperty->Struct == TBaseStructure<T>::Get())
			{
				return EPropertyBagResult::Success;
			}
		}
		else if (TIsDerivedFrom<T, FSoftObjectPath>::Value && (InDesc.ValueType == EPropertyBagPropertyType::SoftObject || InDesc.ValueType == EPropertyBagPropertyType::SoftClass))
		{
			return EPropertyBagResult::Success;
		}

		return EPropertyBagResult::TypeMismatch;
	}

	template <typename T, typename TEnableIf<TIsDerivedFrom<std::remove_pointer_t<T>, UObject>::Value>::Type* = nullptr>
	static EPropertyBagResult CheckType(FPropertyBagPropertyDesc InDesc, const UObject* InValue)
	{
		if (!InDesc.ContainerTypes.IsEmpty())
		{
			return EPropertyBagResult::TypeMismatch;
		}

		if (InDesc.ValueType == EPropertyBagPropertyType::Object || InDesc.ValueType == EPropertyBagPropertyType::SoftObject)
		{
			if (!IsValid(InValue))
			{
				return EPropertyBagResult::Success;
			}
			else if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InDesc.CachedProperty))
			{
				const UClass* Class = InValue->GetClass();
				const UClass* PropertyClass = ObjectProperty->PropertyClass;

				if (Class && PropertyClass && Class->IsChildOf(PropertyClass))
				{
					return EPropertyBagResult::Success;
				}
			}
		}
		else if (InDesc.ValueType == EPropertyBagPropertyType::Class || InDesc.ValueType == EPropertyBagPropertyType::SoftClass)
		{
			if (!IsValid(InValue))
			{
				return EPropertyBagResult::Success;
			}

			const UClass* Class = Cast<UClass>(InValue);
			const UClass* PropertyClass = nullptr;

			if (const FClassProperty* ClassProperty = CastField<FClassProperty>(InDesc.CachedProperty))
			{
				PropertyClass = ClassProperty->MetaClass;
			}
			else if (const FSoftClassProperty* SoftClassProperty = CastField<FSoftClassProperty>(InDesc.CachedProperty))
			{
				PropertyClass = SoftClassProperty->MetaClass;
			}

			if (Class && PropertyClass && Class->IsChildOf(PropertyClass))
			{
				return EPropertyBagResult::Success;
			}
		}

		return EPropertyBagResult::TypeMismatch;
	}

	template <typename T, typename TEnableIf<TAnd<TNot<TIsEnumClass<T>>, TNot<TModels<CBaseStructureProvider, T>>, TNot<TIsDerivedFrom<std::remove_pointer_t<T>, UObject>>>::Value>::Type* = nullptr>
	static EPropertyBagResult CheckType(FPropertyBagPropertyDesc InDesc, const T InValue)
	{
		if (!InDesc.ContainerTypes.IsEmpty())
		{
			return EPropertyBagResult::TypeMismatch;
		}

		// TODO: consider adding generic traits/concepts to PropertyBag for mapping from T -> EPropertyBagPropertyType
		// This generic mechanism should consider the entirety of the PropertyBag interface and be applied consistently.
		if ((std::is_same_v<T, bool> && InDesc.ValueType == EPropertyBagPropertyType::Bool)
			|| (std::is_same_v<T, uint8> && InDesc.ValueType == EPropertyBagPropertyType::Byte)
			|| (std::is_same_v<T, int8> && InDesc.ValueType == EPropertyBagPropertyType::Int8)
			|| (std::is_same_v<T, int16> && InDesc.ValueType == EPropertyBagPropertyType::Int16)
			|| (std::is_same_v<T, int32> && InDesc.ValueType == EPropertyBagPropertyType::Int32)
			|| (std::is_same_v<T, int64> && InDesc.ValueType == EPropertyBagPropertyType::Int64)
			|| (std::is_same_v<T, uint16> && InDesc.ValueType == EPropertyBagPropertyType::UInt16)
			|| (std::is_same_v<T, uint32> && InDesc.ValueType == EPropertyBagPropertyType::UInt32)
			|| (std::is_same_v<T, uint64> && InDesc.ValueType == EPropertyBagPropertyType::UInt64)
			|| (std::is_same_v<T, float> && InDesc.ValueType == EPropertyBagPropertyType::Float)
			|| (std::is_same_v<T, double> && InDesc.ValueType == EPropertyBagPropertyType::Double)
			|| (std::is_same_v<T, FName> && InDesc.ValueType == EPropertyBagPropertyType::Name)
			|| (std::is_same_v<T, FString> && InDesc.ValueType == EPropertyBagPropertyType::String)
			|| (std::is_same_v<T, FText> && InDesc.ValueType == EPropertyBagPropertyType::Text)
			|| (std::is_same_v<T, FSoftObjectPtr> && InDesc.ValueType == EPropertyBagPropertyType::SoftObject))
		{
			return EPropertyBagResult::Success;
		}

		return EPropertyBagResult::TypeMismatch;
	}

	template <typename T, typename ReturnType>
	static ReturnType Call(TFunction<ReturnType(const void*)>&& Func, const FPropertyBagPropertyDesc& InDesc, const T& InValue)
	{
		if constexpr (TIsDerivedFrom<T, FSoftObjectPath>::Value || TIsDerivedFrom<std::remove_pointer_t<T>, UObject>::Value)
		{
			if (InDesc.ValueType == EPropertyBagPropertyType::SoftObject || InDesc.ValueType == EPropertyBagPropertyType::SoftClass)
			{
				return Call(MoveTemp(Func), InDesc, FSoftObjectPtr(InValue));
			}
		}

		return Func(&InValue);
	}

	template <typename T>
	static void Convert(const FPropertyBagPropertyDesc& InDesc, void* InValue, T& OutValue)
	{
		if constexpr (TIsDerivedFrom<std::remove_pointer_t<T>, UObject>::Value)
		{
			static_assert(std::is_pointer_v<T>);

			if (InDesc.ValueType == EPropertyBagPropertyType::SoftObject || InDesc.ValueType == EPropertyBagPropertyType::SoftClass)
			{
				const FObjectPropertyBase* ObjectProperty = CastFieldChecked<FObjectPropertyBase>(InDesc.CachedProperty);
				check(ObjectProperty->PropertyClass);

				OutValue = Cast<std::remove_pointer_t<T>>(ObjectProperty->GetObjectPropertyValue(InValue));
				return;
			}
		}
		else if constexpr (TIsDerivedFrom<T, FSoftObjectPath>::Value)
		{
			if (InDesc.ValueType == EPropertyBagPropertyType::SoftObject || InDesc.ValueType == EPropertyBagPropertyType::SoftClass)
			{
				const FSoftObjectProperty* SoftObjectProperty = CastFieldChecked<FSoftObjectProperty>(InDesc.CachedProperty);
				check(SoftObjectProperty->PropertyClass);

				if constexpr (TIsDerivedFrom<T, FSoftClassPath>::Value)
				{
					OutValue = FSoftClassPath(SoftObjectProperty->GetPropertyValue(InValue).ToSoftObjectPath().ToString());
				}
				else
				{
					OutValue = SoftObjectProperty->GetPropertyValue(InValue).ToSoftObjectPath();
				}
				return;
			}
		}

		OutValue = *reinterpret_cast<T*>(InValue);
	}

	template <typename KeyType, typename ValueType>
	EPropertyBagResult AddPair(const FPropertyBagPropertyDesc& InKeyDesc, const KeyType& InKey, const FPropertyBagPropertyDesc& InValueDesc, const ValueType& InValue)
	{
		if constexpr (
			(TIsDerivedFrom<KeyType, FSoftObjectPath>::Value || TIsDerivedFrom<std::remove_pointer_t<KeyType>, UObject>::Value) &&
			(TIsDerivedFrom<ValueType, FSoftObjectPath>::Value || TIsDerivedFrom<std::remove_pointer_t<ValueType>, UObject>::Value)
		)
		{
			if ((InKeyDesc.ValueType == EPropertyBagPropertyType::SoftObject || InKeyDesc.ValueType == EPropertyBagPropertyType::SoftClass) &&
				(InValueDesc.ValueType == EPropertyBagPropertyType::SoftObject || InValueDesc.ValueType == EPropertyBagPropertyType::SoftClass))
			{
				return AddPair(InKeyDesc, FSoftObjectPtr(InKey), InValueDesc, FSoftObjectPtr(InValue));
			}
		}
		else if constexpr (TIsDerivedFrom<KeyType, FSoftObjectPath>::Value || TIsDerivedFrom<std::remove_pointer_t<KeyType>, UObject>::Value)
		{
			if (InKeyDesc.ValueType == EPropertyBagPropertyType::SoftObject || InKeyDesc.ValueType == EPropertyBagPropertyType::SoftClass)
			{
				return AddPair(InKeyDesc, FSoftObjectPtr(InKey), InValueDesc, InValue);
			}
		}
		else if constexpr (TIsDerivedFrom<ValueType, FSoftObjectPath>::Value || TIsDerivedFrom<std::remove_pointer_t<ValueType>, UObject>::Value)
		{
			if (InValueDesc.ValueType == EPropertyBagPropertyType::SoftObject || InValueDesc.ValueType == EPropertyBagPropertyType::SoftClass)
			{
				return AddPair(InKeyDesc, InKey, InValueDesc, FSoftObjectPtr(InValue));
			}
		}

		FScriptMapHelper::AddPair(&InKey, &InValue);
		return EPropertyBagResult::Success;
	}

	FPropertyBagPropertyDesc KeyDesc;
	FPropertyBagPropertyDesc ValueDesc;

};
