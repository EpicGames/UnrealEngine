// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/StructUtilsTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include <type_traits>

#include "SharedStruct.generated.h"

#define UE_API COREUOBJECT_API

struct FInstancedStruct;
template<typename BaseStructT> struct TInstancedStruct;
template<typename BaseStructT> struct TStructView;
struct FConstStructView;
template<typename BaseStructT> struct TConstStructView;

///////////////////////////////////////////////////////////////// FStructSharedMemory /////////////////////////////////////////////////////////////////

/**
 * Holds the information and memory about a UStruct. Instances of these are shared using FConstSharedStruct and FSharedStruct.
 * 
 * The size of the allocation for this struct always includes both the size of the struct and also the size required to hold the
 * structure described by the ScriptStruct. This avoids two pointer referencing (cache misses). 
 * Look at Create() to understand more.
 * 
 * A 'const FStructSharedMemory' the memory is immutable. We restrict shallow copies of StructMemory where it's not appropriate
 * in the owning types that compose this type:
 * - FSharedStruct A; ConstSharedStruct B = A; is allowed
 * - ConstSharedStruct A; FSharedStruct B = A; is not allowed
 * 
 * This type is designed to be used in composition and should not be used outside the types that compose it.
 */
struct FStructSharedMemory
{
	~FStructSharedMemory()
	{
		ScriptStruct->DestroyStruct(GetMutableMemory());
	}

	FStructSharedMemory(const FStructSharedMemory& Other) = delete;
	FStructSharedMemory(const FStructSharedMemory&& Other) = delete;
	FStructSharedMemory& operator=(const FStructSharedMemory& Other) = delete;
	FStructSharedMemory& operator=(const FStructSharedMemory&& Other) = delete;

	struct FStructSharedMemoryDeleter
	{
		inline void operator()(FStructSharedMemory* StructSharedMemory) const
		{
			checkf(StructSharedMemory != nullptr
				, TEXT("FStructSharedMemoryDeleter is expected to be used only with a valid FStructSharedMemory. "
					"See FStructSharedMemory::Create/CreateArgs"));

			StructSharedMemory->~FStructSharedMemory();
			FMemory::Free(StructSharedMemory);
		}
	};

	static TSharedPtr<FStructSharedMemory> Create(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
	{
		checkf(InScriptStruct, TEXT("FStructSharedMemory::Create - script struct cannot be null"));

		FStructSharedMemory* const StructMemory = CreateImpl(InScriptStruct, InStructMemory);
		checkf(StructMemory != nullptr, TEXT("CreateImpl is expected to always return a valid pointer to FStructSharedMemory."));

		return MakeShareable(StructMemory, FStructSharedMemoryDeleter());
	}

	template<typename T, typename... TArgs>
	static TSharedPtr<FStructSharedMemory> CreateArgs(TArgs&&... InArgs)
	{
		UE::StructUtils::CheckStructType<T>();

		FStructSharedMemory* const StructMemory = CreateImpl(TBaseStructure<T>::Get());
		checkf(StructMemory != nullptr, TEXT("CreateImpl is expected to always return a valid pointer to FStructSharedMemory."));

		new (StructMemory->GetMutableMemory()) T(Forward<TArgs>(InArgs)...);
		return MakeShareable(StructMemory, FStructSharedMemoryDeleter());
	}

	/** Returns pointer to aligned struct memory. */
	const uint8* GetMemory() const
	{
		return Align((uint8*)StructMemory, ScriptStruct->GetMinAlignment());
	}

	/** Returns mutable pointer to aligned struct memory. */
	uint8* GetMutableMemory()
	{
		return Align((uint8*)StructMemory, ScriptStruct->GetMinAlignment());
	}

	/** Returns struct type. */
	const UScriptStruct& GetScriptStruct() const
	{
		return *ObjectPtrDecay(ScriptStruct);
	}

	TObjectPtr<const UScriptStruct>& GetScriptStructPtr() 
	{
		return ScriptStruct;
	}

private:
	explicit FStructSharedMemory(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
		: ScriptStruct(InScriptStruct)
	{
		ScriptStruct->InitializeStruct(GetMutableMemory());
		
		if (InStructMemory)
		{
			ScriptStruct->CopyScriptStruct(GetMutableMemory(), InStructMemory);
		}
	}

	static FStructSharedMemory* CreateImpl(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
	{
		// Align RequiredSize to InScriptStruct's alignment to effectively add padding in between ScriptStruct and
		// StructMemory. GetMemory will then round &StructMemory up past this 'padding' to the nearest aligned address.
		const int32 RequiredSize = static_cast<int32>(Align(sizeof(FStructSharedMemory), InScriptStruct->GetMinAlignment())) + InScriptStruct->GetStructureSize();
		// Code analysis is unable to understand correctly what we are doing here, so disabling the warning C6386: Buffer overrun while writing to...
		CA_SUPPRESS( 6386 )
		FStructSharedMemory* StructMemory = new(FMemory::Malloc(RequiredSize, InScriptStruct->GetMinAlignment())) FStructSharedMemory(InScriptStruct, InStructMemory);
		return StructMemory;
	}

	TObjectPtr<const UScriptStruct> ScriptStruct;

	/**
	 * Memory for the struct described by ScriptStruct will be allocated here using the 'Flexible array member' pattern.
	 * Access this using GetMutableMemory() / GetMemory() to account for memory alignment.
	 */ 
	uint8 StructMemory[0];
};

///////////////////////////////////////////////////////////////// FSharedStruct /////////////////////////////////////////////////////////////////
/**
 * FSharedStruct works similarly as a TSharedPtr<FInstancedStruct> but avoids the double pointer indirection.
 * (One pointer for the FInstancedStruct and one pointer for the struct memory it is wrapping).
 * Also note that because of its implementation, it is not possible for now to go from a struct reference or struct view back to a shared struct.
 *
 * This struct type is also convertible to a FStructView / FConstStructView, and like FInstancedStruct, it is the preferable way of passing it as a parameter.
 * If the calling code would like to keep a shared pointer to the struct, you may pass the FSharedStruct as a parameter, but it is recommended to pass it as
 * a "const FSharedStruct&" to limit the unnecessary recounting.
 * 
 * A 'const FSharedStruct' cannot be made to point at another instance of a struct, whilst a vanilla FSharedStruct can.
 * In either case, the shared struct memory /data is mutable.
 */
USTRUCT()
struct [[nodiscard]] FSharedStruct
{
	GENERATED_BODY();

	friend struct FConstSharedStruct;

	FSharedStruct() = default;

	~FSharedStruct()
	{
	}

	/** Copy constructors */
	FSharedStruct(const FSharedStruct& InOther) = default;
	FSharedStruct(FSharedStruct&& InOther) = default;

	/** Assignment operators */
	FSharedStruct& operator=(const FSharedStruct& InOther) = default;
	FSharedStruct& operator=(FSharedStruct&& InOther) = default;

	/** For StructOpsTypeTraits */
	UE_API bool Identical(const FSharedStruct* Other, uint32 PortFlags) const;
	UE_API void AddStructReferencedObjects(class FReferenceCollector& Collector) const;

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const
	{
		return StructMemoryPtr ? &(StructMemoryPtr.Get()->GetScriptStruct()) : nullptr;
	}

	TObjectPtr<const UScriptStruct>* GetScriptStructPtr() const
	{
		return StructMemoryPtr ? &StructMemoryPtr.Get()->GetScriptStructPtr() : nullptr;
	}

	/** Returns a mutable pointer to struct memory. */
	uint8* GetMemory() const
	{
		return StructMemoryPtr ? StructMemoryPtr.Get()->GetMutableMemory() : nullptr;
	}

	/** Reset to empty. */
	void Reset()
	{
		StructMemoryPtr.Reset();
	}

	/** Initializes from a templated struct type. This will create a new instance of the shared struct memory. */
	template<typename T>
	void InitializeAs()
	{
		UE::StructUtils::CheckStructType<T>();

		InitializeAs(TBaseStructure<T>::Get(), nullptr);
	}

	/** Initializes from other related struct types. This will create a new instance of the shared struct memory. */
	template <typename T>
	void InitializeAs(const T& Struct)
	{
		if constexpr (UE::StructUtils::TIsSharedInstancedOrViewStruct_V<T>)
		{
			InitializeAs(Struct.GetScriptStruct(), Struct.GetMemory());
		}
		else
		{
			InitializeAs(TBaseStructure<T>::Get(), reinterpret_cast<const uint8*>(&Struct));
		}
	}

	/** Initializes from a struct type and optional data. This will create a new instance of the shared struct memory. */
	void InitializeAs(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
	{
		if (InScriptStruct)
		{
			StructMemoryPtr = FStructSharedMemory::Create(InScriptStruct, InStructMemory);
		}
		else
		{
			Reset();
		}
	}

	/** Initializes from struct type and emplace args. This will create a new instance of the shared struct memory.*/
	template<typename T, typename... TArgs>
	void InitializeAs(TArgs&&... InArgs)
	{
		UE::StructUtils::CheckStructType<T>();

		StructMemoryPtr = FStructSharedMemory::CreateArgs<T>(Forward<TArgs>(InArgs)...);
	}

	/** Creates a new FSharedStruct from a templated struct type. This will create a new instance of the shared struct memory. */
	template<typename T>
	static FSharedStruct Make()
	{
		FSharedStruct SharedStruct;
		SharedStruct.InitializeAs<T>();
		return SharedStruct;
	}

	/** Creates a new FSharedStruct from templated struct instance. This will create a new instance of the shared struct memory. */
	template<typename T>
	static FSharedStruct Make(const T& Struct)
	{
		FSharedStruct SharedStruct;
		SharedStruct.InitializeAs(Struct);
		return SharedStruct;
	}

	/** Creates a new FSharedStruct from struct type and optional instance memory. This will create a new instance of the shared struct memory. */
	static FSharedStruct Make(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
	{
		FSharedStruct SharedStruct;
		SharedStruct.InitializeAs(InScriptStruct, InStructMemory);
		return SharedStruct;
	}

	/** Creates a new FSharedStruct from the templated type and forward all arguments to constructor. This will create a new instance of the shared struct memory. */
	template<typename T, typename... TArgs>
	static FSharedStruct Make(TArgs&&... InArgs)
	{
		FSharedStruct SharedStruct;
		SharedStruct.InitializeAs<T>(Forward<TArgs>(InArgs)...);
		return SharedStruct;
	}

	/** Returns reference to the struct, this getter assumes that all data is valid. */
	template<typename T>
	T& Get() const
	{
		return UE::StructUtils::GetStructRef<T>(GetScriptStruct(), GetMemory());
	}

	/** Returns pointer to the struct, or nullptr if cast is not valid. */
	template<typename T>
	T* GetPtr() const
	{
		return UE::StructUtils::GetStructPtr<T>(GetScriptStruct(), GetMemory());
	}

	/** Returns True if the struct is valid.*/
	bool IsValid() const
	{
		return GetMemory() != nullptr && GetScriptStruct() != nullptr;
	}

	/** Comparison operators. Note: it does not compare the internal structure itself*/
	template <typename OtherType>
	bool operator==(const OtherType& Other) const
	{
		return ((GetScriptStruct() == Other.GetScriptStruct()) && (GetMemory() == Other.GetMemory()));
	}

	template <typename OtherType>
	bool operator!=(const OtherType& Other) const
	{
		return !operator==(Other);
	}

	/**
	 * Determines whether Other contains the same values as `this`
	 * @return whether the values stored are equal
	 */
	template <typename OtherType>
	bool CompareStructValues(const OtherType& Other, uint32 PortFlags = 0) const
	{
		UE::StructUtils::CheckWrapperType<OtherType>();
		const UScriptStruct* ScriptStruct = GetScriptStruct();
		return (ScriptStruct == Other.GetScriptStruct())
			&& (!ScriptStruct || ScriptStruct->CompareScriptStruct(GetMemory(), Other.GetMemory(), PortFlags));
	}

protected:
	TSharedPtr<FStructSharedMemory> StructMemoryPtr;
};

/**
 * TSharedStruct is a type-safe FSharedStruct wrapper against the given BaseStruct type.
 * @note When used as a property, this automatically defines the BaseStruct property meta-data.
 * 
 * Example:
 *
 *	TSharedStruct<FTestStructBase> Test;
 *
 *	TArray<TSharedStruct<FTestStructBase>> TestArray;
 */
template<typename BaseStructT>
struct [[nodiscard]] TSharedStruct
{
	explicit TSharedStruct() = default;

	/** Copy constructors */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TSharedStruct(const TSharedStruct<T>& InOther)
		: SharedStruct(InOther.SharedStruct)
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TSharedStruct(TSharedStruct<T>&& InOther)
		: SharedStruct(MoveTemp(InOther.SharedStruct))
	{
	}

	/** Assignment operators */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TSharedStruct& operator=(const TSharedStruct<T>& InOther)
	{
		if (this != &InOther)
		{
			SharedStruct = InOther.SharedStruct;
		}
		return *this;
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TSharedStruct& operator=(TSharedStruct<T>&& InOther)
	{
		if (this != &InOther)
		{
			SharedStruct = MoveTemp(InOther.SharedStruct);
		}
		return *this;
	}

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const
	{
		return SharedStruct.GetScriptStruct();
	}

	TObjectPtr<const UScriptStruct>* GetScriptStructPtr() const
	{
		return SharedStruct.GetScriptStructPtr();
	}

	/** Returns a mutable pointer to struct memory. */
	uint8* GetMemory() const
	{
		return SharedStruct.GetMemory();
	}

	/** Reset to empty. */
	void Reset()
	{
		SharedStruct.Reset();
	}

	/** Initializes from a templated struct type. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	void Initialize()
	{
		SharedStruct.InitializeAs<T>();
	}

	/** Initializes from templated struct instance. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT>
		requires(
			std::is_base_of_v<BaseStructT, std::decay_t<T>>
			&& !(std::is_same_v<TStructView<T>, T>
				|| std::is_same_v<TConstStructView<T>, T>
				|| std::is_same_v<TSharedStruct<T>, T>
				|| std::is_same_v<TConstSharedStruct<T>, T>
				|| std::is_same_v<TInstancedStruct<T>, T>))
	void Initialize(const T& Struct)
	{
		SharedStruct.InitializeAs(TBaseStructure<T>::Get(), reinterpret_cast<const uint8*>(&Struct));
	}

	/** Initializes from struct type and emplace args. This will create a new instance of the shared struct memory.*/
	template<typename T = BaseStructT, typename... TArgs>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	void Initialize(TArgs&&... InArgs)
	{
		SharedStruct.InitializeAs(Forward<TArgs>(InArgs)...);
	}

	/** Creates a new TSharedStruct. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	static TSharedStruct Make()
	{
		TSharedStruct Result;
		Result.SharedStruct = FSharedStruct::Make<T>();
		return Result;
	}

	/** Creates a new TSharedStruct from templated struct instance. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	static TSharedStruct Make(const T& Struct)
	{
		TSharedStruct Result;
		Result.SharedStruct = FSharedStruct::Make<T>(Struct);
		return Result;
	}

	/** Creates a new TSharedStruct from the templated type and forward all arguments to constructor. This will create a new instance of the shared struct memory. */
	template< typename... TArgs>
	static TSharedStruct Make(TArgs&&... InArgs)
	{
		TSharedStruct Result;
		Result.SharedStruct = FSharedStruct::Make<BaseStructT>(Forward<TArgs>(InArgs)...);
		return Result;
	}

	/** Returns reference to the struct, this getter assumes that all data is valid. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	T& Get() const
	{
		return SharedStruct.Get<T>();
	}

	/** Returns pointer to the struct, or nullptr if cast is not valid. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	T* GetPtr() const
	{
		return SharedStruct.GetPtr<T>();
	}

	/** Returns True if the struct is valid.*/
	bool IsValid() const
	{
		return SharedStruct.IsValid();
	}

	/** Comparison operators. Note: it does not compare the internal structure itself */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator==(const TSharedStruct<T>& Other) const
	{
		return SharedStruct.operator==(Other.SharedStruct);
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator!=(const TSharedStruct<T>& Other) const
	{
		return SharedStruct.operator!=(Other.SharedStruct);
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator==(const TConstSharedStruct<T>& Other) const
	{
		return SharedStruct.operator==(Other.ConstSharedStruct);
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator!=(const TConstSharedStruct<T>& Other) const
	{
		return SharedStruct.operator!=(Other.ConstSharedStruct);
	}

private:
	/**
	 * Note:
	 *   TSharedStruct is a wrapper for a FSharedStruct (rather than inheriting) so that it can provide a locked-down type-safe 
	 *   API for use in C++, without being able to accidentally take a reference to the untyped API to work around the restrictions.
	 * 
	 *   TSharedStruct MUST be the same size as FSharedStruct, as the reflection layer treats a TSharedStruct as a FSharedStruct.
	 *   This means that any reflected APIs (like ExportText) that accept an FSharedStruct pointer can also accept a TSharedStruct pointer.
	 */
	FSharedStruct SharedStruct;

	template <typename U> friend struct TSharedStruct;
	template <typename U> friend struct TConstSharedStruct;
};

template<>
struct TStructOpsTypeTraits<FSharedStruct> : public TStructOpsTypeTraitsBase2<FSharedStruct>
{
	enum
	{
		WithIdentical = true,
		WithAddStructReferencedObjects = true,
	};
};

///////////////////////////////////////////////////////////////// FConstSharedStruct /////////////////////////////////////////////////////////////////
/**
 * FConstSharedStruct is the same as the FSharedStruct but restricts the API to return a const struct type. 
 * 
 * A 'const FConstSharedStruct' cannot be made to point at another instance of a struct, whilst a vanilla FConstSharedStruct can.
 * In either case, the struct data is immutable.
 * 
 * See FSharedStruct for more information.
 */
USTRUCT()
struct [[nodiscard]] FConstSharedStruct
{
	GENERATED_BODY();

	FConstSharedStruct() = default;

	FConstSharedStruct(const FConstSharedStruct& Other) = default;
	FConstSharedStruct(const FSharedStruct& SharedStruct)
		: StructMemoryPtr(SharedStruct.StructMemoryPtr)
	{}

	FConstSharedStruct(FConstSharedStruct&& Other) = default;
	FConstSharedStruct(FSharedStruct&& SharedStruct)
		: StructMemoryPtr(MoveTemp(SharedStruct.StructMemoryPtr))
	{}

	FConstSharedStruct& operator=(const FConstSharedStruct& Other) = default;
	FConstSharedStruct& operator=(const FSharedStruct& SharedStruct)
	{
		StructMemoryPtr = SharedStruct.StructMemoryPtr;
		return *this;
	}

	FConstSharedStruct& operator=(FConstSharedStruct&& Other) = default;
	FConstSharedStruct& operator=(FSharedStruct&& InSharedStruct)
	{
		StructMemoryPtr = MoveTemp(InSharedStruct.StructMemoryPtr);
		return *this;
	}

	/** For StructOpsTypeTraits */
	UE_API bool Identical(const FConstSharedStruct* Other, uint32 PortFlags) const;
	UE_API void AddStructReferencedObjects(class FReferenceCollector& Collector);

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const
	{
		return StructMemoryPtr ? &(StructMemoryPtr.Get()->GetScriptStruct()) : nullptr;
	}

	TObjectPtr<const UScriptStruct>* GetScriptStructPtr()
	{
		return StructMemoryPtr ?
			&const_cast<FStructSharedMemory*>(StructMemoryPtr.Get())->GetScriptStructPtr() : nullptr;
	}

	/** Returns const pointer to struct memory. */
	const uint8* GetMemory() const
	{
		return StructMemoryPtr ? StructMemoryPtr.Get()->GetMemory() : nullptr;
	}

	/** Reset to empty. */
	void Reset()
	{
		StructMemoryPtr.Reset();
	}

	/** Initializes from a templated struct type. */
	template<typename T>
	void InitializeAs()
	{
		UE::StructUtils::CheckStructType<T>();

		InitializeAs(TBaseStructure<T>::Get(), nullptr);
	}

	/** Initializes from other related struct types. This will create a new instance of the shared struct memory. */
	template <typename T>
	void InitializeAs(const T& Struct)
	{
		if constexpr (UE::StructUtils::TIsSharedInstancedOrViewStruct_V<T>)
		{
			InitializeAs(Struct.GetScriptStruct(), Struct.GetMemory());
		}
		else
		{
			InitializeAs(TBaseStructure<T>::Get(), reinterpret_cast<const uint8*>(&Struct));
		}
	}

	/** Initializes from a struct type and optional data. This will create a new instance of the shared struct memory. */
	void InitializeAs(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
	{
		if (InScriptStruct)
		{
			StructMemoryPtr = FStructSharedMemory::Create(InScriptStruct, InStructMemory);
		}
		else
		{
			Reset();
		}
	}

	/** Initializes from struct type and emplace args. This will create a new instance of the shared struct memory. */
	template<typename T, typename... TArgs>
	void InitializeAs(TArgs&&... InArgs)
	{
		UE::StructUtils::CheckStructType<T>();

		StructMemoryPtr = FStructSharedMemory::CreateArgs<T>(Forward<TArgs>(InArgs)...);
	}

	/** Creates a new FSharedStruct from a templated struct type. This will create a new instance of the shared struct memory. */
	template<typename T>
	static FConstSharedStruct Make()
	{
		FConstSharedStruct SharedStruct;
		SharedStruct.InitializeAs<T>();
		return SharedStruct;
	}

	/** Creates a new FSharedStruct from templated struct instance. This will create a new instance of the shared struct memory. */
	template<typename T>
	static FConstSharedStruct Make(const T& Struct)
	{
		FConstSharedStruct SharedStruct;
		SharedStruct.InitializeAs<T>(Struct);
		return SharedStruct;
	}

	/** Creates a new FSharedStruct from a struct type and optional data. This will create a new instance of the shared struct memory. */
	static FConstSharedStruct Make(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
	{
		FConstSharedStruct SharedStruct;
		SharedStruct.InitializeAs(InScriptStruct, InStructMemory);
		return SharedStruct;
	}

	/** Creates a new FSharedStruct from the templated type and forward all arguments to constructor. This will create a new instance of the shared struct memory. */
	template<typename T, typename... TArgs>
	static FConstSharedStruct Make(TArgs&&... InArgs)
	{
		FConstSharedStruct SharedStruct;
		SharedStruct.InitializeAs<T>(Forward<TArgs>(InArgs)...);
		return SharedStruct;
	}

	/** Returns const reference to the struct, this getter assumes that all data is valid. */
	template<typename T>
	const T& Get() const
	{
		return UE::StructUtils::GetStructRef<T>(GetScriptStruct(), GetMemory());
	}

	/** Returns const pointer to the struct, or nullptr if cast is not valid. */
	template<typename T>
	const T* GetPtr() const
	{
		return UE::StructUtils::GetStructPtr<T>(GetScriptStruct(), GetMemory());
	}

	/** Returns True if the struct is valid.*/
	bool IsValid() const
	{
		return GetMemory() != nullptr && GetScriptStruct() != nullptr;
	}

	/** Comparison operators. Note: it does not compare the internal structure itself*/
	template <typename OtherType>
	bool operator==(const OtherType& Other) const
	{
		return ((GetScriptStruct() == Other.GetScriptStruct()) && (GetMemory() == Other.GetMemory()));
	}

	template <typename OtherType>
	bool operator!=(const OtherType& Other) const
	{
		return !operator==(Other);
	}

	/** 
	 * Determines whether Other contains the same values as `this`
	 * @return whether the values stored are equal
	 */
	template <typename OtherType>
	bool CompareStructValues(const OtherType& Other, uint32 PortFlags = 0) const
	{
		UE::StructUtils::CheckWrapperType<OtherType>();
		const UScriptStruct* ScriptStruct = GetScriptStruct();
		return (ScriptStruct == Other.GetScriptStruct())
			&& (!ScriptStruct || ScriptStruct->CompareScriptStruct(GetMemory(), Other.GetMemory(), PortFlags));
	}

protected:

	TSharedPtr<const FStructSharedMemory> StructMemoryPtr;
};

/**
 * TConstSharedStruct is a type-safe FConstSharedStruct wrapper against the given BaseStruct type.
 * @note When used as a property, this automatically defines the BaseStruct property meta-data.
 * 
 * Example:
 *
 *	TConstSharedStruct<FTestStructBase> Test;
 *
 *	TArray<TConstSharedStruct<FTestStructBase>> TestArray;
 */
template<typename BaseStructT>
struct [[nodiscard]] TConstSharedStruct
{
	explicit TConstSharedStruct() = default;

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstSharedStruct(const TConstSharedStruct<T>& Other)
		: ConstSharedStruct(Other.ConstSharedStruct)
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstSharedStruct(const TSharedStruct<T>& SharedStruct)
		: ConstSharedStruct(SharedStruct.SharedStruct)
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstSharedStruct(TConstSharedStruct<T>&& Other)
		: ConstSharedStruct(MoveTemp(Other.ConstSharedStruct))
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstSharedStruct(TSharedStruct<T>&& SharedStruct)
		: ConstSharedStruct(MoveTemp(SharedStruct.SharedStruct))
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstSharedStruct& operator=(const TConstSharedStruct<T>& Other)
	{
		if (this != &Other)
		{
			ConstSharedStruct = Other.ConstSharedStruct;
		}
		return *this;
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstSharedStruct& operator=(const TSharedStruct<T>& SharedStruct)
	{
		ConstSharedStruct = SharedStruct.SharedStruct;
		return *this;
	}

	/** Returns const pointer to struct memory. */
	const uint8* GetMemory() const
	{
		return ConstSharedStruct.GetMemory();
	}

	/** Reset to empty. */
	void Reset()
	{
		ConstSharedStruct.Reset();
	}

	/** Initializes from a templated struct type. */
	void Initialize()
	{
		ConstSharedStruct.InitializeAs<BaseStructT>();
	}

	/** Initializes from templated struct instance. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT>
		requires(
			std::is_base_of_v<BaseStructT, std::decay_t<T>>
			&& !(std::is_same_v<TStructView<T>, T>
				|| std::is_same_v<TConstStructView<T>, T>
				|| std::is_same_v<TSharedStruct<T>, T>
				|| std::is_same_v<TConstSharedStruct<T>, T>
				|| std::is_same_v<TInstancedStruct<T>, T>))
	void Initialize(const T& Struct)
	{
		ConstSharedStruct.InitializeAs<T>(Struct);
	}

	/** Initializes from struct type and emplace args. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT, typename... TArgs>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	void Initialize(TArgs&&... InArgs)
	{
		ConstSharedStruct.InitializeAs<T>(Forward<TArgs>(InArgs)...);
	}

	/** Creates a new TConstSharedStruct from a templated struct type. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	static TConstSharedStruct Make()
	{
		TConstSharedStruct SharedStruct;
		SharedStruct.Initialize<BaseStructT>();
		return SharedStruct;
	}

	/** Creates a new TConstSharedStruct from templated struct instance. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	static TConstSharedStruct Make(const T& Struct)
	{
		TConstSharedStruct SharedStruct;
		SharedStruct.Initialize<BaseStructT>(Struct);
		return SharedStruct;
	}

	/** Creates a new TConstSharedStruct from the templated type and forward all arguments to constructor. This will create a new instance of the shared struct memory. */
	template<typename T = BaseStructT, typename... TArgs>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	static TConstSharedStruct Make(TArgs&&... InArgs)
	{
		TConstSharedStruct SharedStruct;
		SharedStruct.template Initialize<T>(Forward<TArgs>(InArgs)...);
		return SharedStruct;
	}

	/** Returns const reference to the struct, this getter assumes that all data is valid. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	const T& Get() const
	{
		return ConstSharedStruct.Get<T>();
	}

	/** Returns const pointer to the struct. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	const T* GetPtr() const
	{
		return ConstSharedStruct.GetPtr<T>();
	}

	/** Returns True if the struct is valid.*/
	bool IsValid() const
	{
		return ConstSharedStruct.IsValid();
	}

	/** Comparison operators. Note: it does not compare the internal structure itself */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator==(const TSharedStruct<T>& Other) const
	{
		return ConstSharedStruct.operator==(Other.SharedStruct);
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator!=(const TSharedStruct<T>& Other) const
	{
		return ConstSharedStruct.operator!=(Other.SharedStruct);
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator==(const TConstSharedStruct<T>& Other) const
	{
		return ConstSharedStruct.operator==(Other.ConstSharedStruct);
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator!=(const TConstSharedStruct<T>& Other) const
	{
		return ConstSharedStruct.operator!=(Other.ConstSharedStruct);
	}

private:
	/**
	 * Note:
	 *   TConstSharedStruct is a wrapper for a FConstSharedStruct (rather than inheriting) so that it can provide a locked-down type-safe 
	 *   API for use in C++, without being able to accidentally take a reference to the untyped API to work around the restrictions.
	 * 
	 *   TConstSharedStruct MUST be the same size as FConstSharedStruct, as the reflection layer treats a TConstSharedStruct as a FConstSharedStruct.
	 *   This means that any reflected APIs (like ExportText) that accept an FConstSharedStruct pointer can also accept a TConstSharedStruct pointer.
	 */
	FConstSharedStruct ConstSharedStruct;

	template <typename U> friend struct TSharedStruct;
	template <typename U> friend struct TConstSharedStruct;
};

template<>
struct TStructOpsTypeTraits<FConstSharedStruct> : public TStructOpsTypeTraitsBase2<FConstSharedStruct>
{
	enum
	{
		WithIdentical = true,
		WithAddStructReferencedObjects = true,
	};
};

#undef UE_API
