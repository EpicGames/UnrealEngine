// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "StructUtils.h"
#include "StructUtils/StructView.h"
#include <type_traits>

///////////////////////////////////////////////////////////////// FStructArrayView /////////////////////////////////////////////////////////////////

/** 
 *	A generic, transient view of a homogeneously-typed array of instances of a specific UScriptStruct
 *  FStructArrayView supplies mutable access to the elements of the Array.
 *  Note const FStructArrayView only what the view is pointing to can not be modified the actual struct data is still mutable.
 *  For a more indepth overview of the constness of views see FConstStructView
 */
struct FStructArrayView
{
	FStructArrayView() = default;

	template<typename T>
	explicit FStructArrayView(TArray<T>& InArray)
		: DataPtr(InArray.GetData())
		, ScriptStruct(StaticStruct<std::remove_reference_t<T>>())
		, ElementSize(sizeof(T))
		, ArrayNum(InArray.Num())
	{}

	template<typename T>
	explicit FStructArrayView(TArrayView<T> InArrayView)
		: DataPtr(InArrayView.GetData())
		, ScriptStruct(StaticStruct<std::remove_reference_t<T>>())
		, ElementSize(sizeof(T))
		, ArrayNum(InArrayView.Num())
	{}

	FStructArrayView(const UScriptStruct* InScriptStruct, void* InData, const uint32 InElementSize, const int32 InCount)
		: DataPtr(InData)
		, ScriptStruct(InScriptStruct)
		, ElementSize(InElementSize)
		, ArrayNum(InCount)
	{
		check((InData == nullptr) || ((InScriptStruct != nullptr) && (InElementSize > 0) && (InCount >= 0)));
		check((InData != nullptr) || ((InScriptStruct == nullptr) && (InElementSize == 0) && (InCount == 0)));
	}

	FStructArrayView(const UScriptStruct& InScriptStruct, void* InData, const int32 InCount)
		: DataPtr(InData)
		, ScriptStruct(&InScriptStruct)
		, ElementSize(InScriptStruct.GetStructureSize())
		, ArrayNum(InCount)
	{
		check(InCount >= 0);
		check(InData != nullptr || InCount == 0);
	}

	/**
	 * Checks array invariants: if array size is greater than or equal to zero.
	 */
	void CheckInvariants() const
	{
		checkSlow(ArrayNum >= 0);
	}

	/**
	 * Checks if index is in array range.
	 *
	 * @param Index Index to check.
	 */
	void RangeCheck(int32 Index) const
	{
		checkf((Index >= 0) & (Index < ArrayNum), TEXT("Array index out of bounds: %d from an array of size %d"), Index, ArrayNum);
	}

	/**
	 * Checks if a slice range [Index, Index+InNum) is in array range.
	 * Length is 0 is allowed on empty arrays; Index must be 0 in that case.
	 *
	 * @param Index Starting index of the slice.
	 * @param InNum Length of the slice.
	 */
	void SliceRangeCheck(const int32 Index, const int32 InNum) const
	{
		checkf(Index >= 0, TEXT("Invalid index (%d)"), Index);
		checkf(InNum >= 0, TEXT("Invalid count (%d)"), InNum);
		checkf(Index + InNum <= ArrayNum, TEXT("Range (index: %d, count: %d) lies outside the view of %d elements"), Index, InNum, ArrayNum);
	}

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of elements in the array.
	 *
	 * @param Index Index to test.
	 *
	 * @returns True if index is valid. False otherwise.
	 */
	bool IsValidIndex(const int32 Index) const
	{
		return (Index >= 0) && (Index < ArrayNum);
	}

	/**
	 * Returns true if the array is empty and contains no elements. 
	 *
	 * @returns True if array is empty.
	 * @see Num
	 */
	bool IsEmpty() const
	{
		return ArrayNum == 0;
	}

	/**
	 * Helper function for returning a pointer to the first array element.
	 *
	 * @returns Pointer to first array entry.
	 */	
	void* GetData() const
	{
		return DataPtr;
	}

	/**
	 * Helper function for returning a pointer to an element in the array.
	 *
	 * @param Index Index of element, this MUST be a valid element.
	 * @returns Pointer to array entry at index.
	 */
	void* GetDataAt(const int32 Index) const 
	{ 
		RangeCheck(Index);
		return (uint8*)DataPtr + ((SIZE_T)Index * ElementSize); 
	}

	/**
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	UE_FORCEINLINE_HINT uint32 GetTypeSize() const
	{
		return ElementSize;
	}

	/**
	 * Returns number of elements in array.
	 *
	 * @returns Number of elements in array.
	 */
	int32 Num() const { return ArrayNum; }

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const { return ScriptStruct; }

	/**
	 * Returns pointer to element at given index. The index MUST be valid. 
	 * If parametre T is invalid then nullptr will be returned.
	 *
	 * @param Index Index of element
	 * @param T Type of stuct, this must either be the type or a parent type of the struct type in the array.
	 * @returns Pointer to indexed element.
	 */
	template<typename T>
	T* GetPtrAt(const int32 Index) const
	{
		return UE::StructUtils::GetStructPtr<T>(ScriptStruct, GetDataAt(Index)); // GetDataAt() calls RangeCheck().
	}

	/**
	 * Returns reference to element at given index. Index and template param T must be must be valid.
	 *
	 * @param Index Index of element
	 * @param T Type of stuct, this must either be the type or a parent type of the struct type in the array.
	 * @returns Reference to indexed element.
	 */
	template<typename T>
	T& GetAt(const int32 Index) const
	{
		return UE::StructUtils::GetStructRef<T>(ScriptStruct, GetDataAt(Index)); // GetDataAt() calls RangeCheck().
	}

	/**
	 * Array bracket operator. Returns FStructView to element at given index. Index must be valid.
	 *
	 * @returns FStructView to indexed element.
	 */
	FStructView operator[](const int32 Index) const
	{
		return FStructView(ScriptStruct, (uint8*)GetDataAt(Index)); // GetDataAt() calls RangeCheck().
	}

	/**
	 * Returns n-th last element from the array.
	 *
	 * @param IndexFromTheEnd (Optional) Index from the end of array. This MUST index a valid element.
	 *                        Default is 0.
	 *
	 * @returns Reference to n-th last element from the array.
	 */
	template<typename T>
	T& Last(const int32 IndexFromTheEnd = 0) const
	{
		return GetAt<T>(ArrayNum - IndexFromTheEnd - 1);
	}

	/**
	 * Returns a sliced view
	 * This is similar to Mid(), but with a narrow contract, i.e. slicing outside of the range of the view is illegal.
	 *
	 * @param Index starting index of the new view
	 * @param InNum number of elements in the new view
	 * @returns Sliced view
	 *
	 * @see Mid
	 */
	[[nodiscard]] FStructArrayView Slice(const int32 Index, const int32 InNum) const
	{
		SliceRangeCheck(Index, InNum);
		return FStructArrayView(ScriptStruct, GetDataAt(Index), ElementSize, InNum);
	}

	/** Returns the left-most part of the view by taking the given number of elements from the left. */
	[[nodiscard]] FStructArrayView Left(const int32 Count) const
	{
		return FStructArrayView(ScriptStruct, DataPtr, ElementSize, FMath::Clamp(Count, 0, ArrayNum));
	}

	/** Returns the left-most part of the view by chopping the given number of elements from the right. */
	[[nodiscard]] FStructArrayView LeftChop(const int32 Count) const
	{
		return FStructArrayView(ScriptStruct, DataPtr, ElementSize, FMath::Clamp(ArrayNum - Count, 0, ArrayNum));
	}

	/** Returns the right-most part of the view by taking the given number of elements from the right. */
	[[nodiscard]] FStructArrayView Right(const int32 Count) const
	{
		const int32 OutLen = FMath::Clamp(Count, 0, ArrayNum);
		return FStructArrayView(ScriptStruct, GetDataAt(ArrayNum - OutLen), ElementSize, OutLen);
	}

	/** Returns the right-most part of the view by chopping the given number of elements from the left. */
	[[nodiscard]] FStructArrayView RightChop(const int32 Count) const
	{
		const int32 OutLen = FMath::Clamp(ArrayNum - Count, 0, ArrayNum);
		return FStructArrayView(ScriptStruct, GetDataAt(ArrayNum - OutLen), ElementSize, OutLen);
	}

	/** Returns the middle part of the view by taking up to the given number of elements from the given position. */
	[[nodiscard]] FStructArrayView Mid(int32 Index, int32 Count = TNumericLimits<int32>::Max()) const
	{
		UE::StructUtils::CalcMidIndexAndCount(ArrayNum, Index, Count);
		return FStructArrayView(ScriptStruct, GetDataAt(Index), ElementSize, Count);;
	}

	/** Modifies the view to be the given number of elements from the left. */
	void LeftInline(const int32 Count)
	{
		*this = Left(Count);
	}

	/** Modifies the view by chopping the given number of elements from the right. */
	void LeftChopInline(const int32 Count)
	{
		*this = LeftChop(Count);
	}

	/** Modifies the view to be the given number of elements from the right. */
	void RightInline(const int32 Count)
	{
		*this = Right(Count);
	}

	/** Modifies the view by chopping the given number of elements from the left. */
	void RightChopInline(const int32 Count)
	{
		*this = RightChop(Count);
	}

	/** Modifies the view to be the middle part by taking up to the given number of elements from the given position. */
	inline void MidInline(const int32 Position, const int32 Count = TNumericLimits<int32>::Max())
	{
		*this = Mid(Position, Count);
	}

	struct FIterator
	{
		FIterator(const FStructArrayView& InOwner, int32 InIndex)
			: Owner(&InOwner)
			, Index(InIndex)
		{}

		FIterator& operator++()
		{
			++Index;
			return *this;
		}

		FStructView operator*() const
		{
			return (*Owner)[Index];
		}

		inline bool operator == (const FIterator& Other) const
		{
			return Owner == Other.Owner
				&& Index == Other.Index;
		}

		UE_FORCEINLINE_HINT bool operator != (const FIterator& Other) const
		{
			return !(*this == Other);
		}

	private:
		const FStructArrayView* Owner = nullptr;
		int32 Index = INDEX_NONE;
	};

	/** Ranged iteration support. DO NOT USE DIRECTLY. */
	UE_FORCEINLINE_HINT FIterator begin() const { return FIterator(*this, 0); }
	UE_FORCEINLINE_HINT FIterator end() const { return FIterator(*this, Num()); }

	/** Swaps the elements at the specified Indicies. The indicies must be valid! */
	UE_FORCEINLINE_HINT void Swap(const int32 Index1, const int32 Index2)
	{
		FMemory::Memswap(GetDataAt(Index1), GetDataAt(Index2), ElementSize); // GetDataAt() calls RangeCheck().
	}

private:
	void* DataPtr = nullptr;
	const UScriptStruct* ScriptStruct = nullptr;
	uint32 ElementSize = 0;
	int32 ArrayNum = 0;
};

///////////////////////////////////////////////////////////////// FConstStructArrayView /////////////////////////////////////////////////////////////////

/**
 *	A generic, transient view of a homogeneously-typed array of instances of a specific UScriptStruct
 *  FConstStructArrayView supplies immutable access to the elements of the Array.
 *  For a more indepth overview of the constness of views see FConstStructView 
 */
struct FConstStructArrayView
{
	FConstStructArrayView() = default;

	template<typename T>
	explicit FConstStructArrayView(TArray<T>& InArray)
		: DataPtr(InArray.GetData())
		, ScriptStruct(StaticStruct<std::remove_reference_t<T>>())
		, ElementSize(sizeof(T))
		, ArrayNum(InArray.Num())
	{}

	template<typename T>
	explicit FConstStructArrayView(TArrayView<T> InArrayView)
		: DataPtr(InArrayView.GetData())
		, ScriptStruct(StaticStruct<std::remove_reference_t<T>>())
		, ElementSize(sizeof(T))
		, ArrayNum(InArrayView.Num())
	{}

	FConstStructArrayView(const FStructArrayView Src)
		: DataPtr(Src.GetData())
		, ScriptStruct(Src.GetScriptStruct())
		, ElementSize(Src.GetTypeSize())
		, ArrayNum(Src.Num())
	{}

	FConstStructArrayView(const UScriptStruct* InScriptStruct, const void* InData, const uint32 InElementSize, const int32 InCount)
		: DataPtr(InData)
		, ScriptStruct(InScriptStruct)
		, ElementSize(InElementSize)
		, ArrayNum(InCount)
	{
		check((InData == nullptr) || ((InScriptStruct != nullptr) && (InElementSize > 0) && (InCount >= 0)));
		check((InData != nullptr) || ((InScriptStruct == nullptr) && (InElementSize == 0) && (InCount == 0)));
	}

	FConstStructArrayView(const UScriptStruct& InScriptStruct, const void* InData, const int32 InCount)
		: DataPtr(InData)
		, ScriptStruct(&InScriptStruct)
		, ElementSize(InScriptStruct.GetStructureSize())
		, ArrayNum(InCount)
	{
		check(InCount >= 0);
		check(InData != nullptr || InCount == 0);
	}

	FConstStructArrayView& operator=(const FStructArrayView StructArrayView)
	{
		*this = FConstStructArrayView(StructArrayView);
		return *this;
	}

	/**
	 * Checks array invariants: if array size is greater than or equal to zero.
	 */
	void CheckInvariants() const
	{
		checkSlow(ArrayNum >= 0);
	}

	/**
	 * Checks if index is in array range.
	 *
	 * @param Index Index to check.
	 */
	void RangeCheck(int32 Index) const
	{
		checkf((Index >= 0) & (Index < ArrayNum), TEXT("Array index out of bounds: %d from an array of size %d"), Index, ArrayNum);
	}

	/**
	 * Checks if a slice range [Index, Index+InNum) is in array range.
	 * Length is 0 is allowed on empty arrays; Index must be 0 in that case.
	 *
	 * @param Index Starting index of the slice.
	 * @param InNum Length of the slice.
	 */
	inline void SliceRangeCheck(int32 Index, int32 InNum) const
	{
		checkf(Index >= 0, TEXT("Invalid index (%d)"), Index);
		checkf(InNum >= 0, TEXT("Invalid count (%d)"), InNum);
		checkf(Index + InNum <= ArrayNum, TEXT("Range (index: %d, count: %d) lies outside the view of %d elements"), Index, InNum, ArrayNum);
	}

	/**
	 * Tests if index is valid, i.e. greater than or equal to zero, and less than the number of elements in the array.
	 *
	 * @param Index Index to test.
	 *
	 * @returns True if index is valid. False otherwise.
	 */
	bool IsValidIndex(int32 Index) const
	{
		return (Index >= 0) && (Index < ArrayNum);
	}

	/**
	 * Returns true if the array is empty and contains no elements. 
	 *
	 * @returns True if array is empty.
	 * @see Num
	 */
	bool IsEmpty() const
	{
		return ArrayNum == 0;
	}

	/**
	 * Helper function for returning a pointer to the first array entry.
	 *
	 * @returns const Pointer to first array entry.
	 */
	const void* GetData() const
	{
		return DataPtr;
	}

	/**
	 * Helper function for returning a pointer to an element in the array.
	 *
	 * @param Index Index of element, this MUST be a valid element.
	 * @returns const Pointer to entry.
	 */
	const void* GetDataAt(const int32 Index) const
	{
		RangeCheck(Index);
		return (uint8*)DataPtr + (SIZE_T)Index * ElementSize;
	}

	/**
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	UE_FORCEINLINE_HINT uint32 GetTypeSize()
	{
		return ElementSize;
	}

	/**
	 * Returns number of elements in array.
	 *
	 * @returns Number of elements in array.
	 */
	int32 Num() const { return ArrayNum; }

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const { return ScriptStruct; }

	/**
	 * Returns pointer to element at given index. The index MUST be valid. 
	 * If parametre T is invalid then nullptr will be returned.
	 *
	 * @param Index Index of element
	 * @param T Type of stuct, this must either be the type or a parent type of the struct type in the array.
	 * @returns const pointer to indexed element.
	 */
	template<typename T>
		requires (std::is_const_v<T>)
	constexpr T* GetPtrAt(const int32 Index) const
	{
		return UE::StructUtils::GetStructPtr<T>(ScriptStruct, GetDataAt(Index)); // GetDataAt() calls RangeCheck().
	}

	/**
	 * Returns reference to element at given index. Index and template param T MUST be must be valid.
	 *
	 * @param Index Index of element
	 * @param T Type of stuct, this must either be the type or a parent type of the struct type in the array.
	 * @returns const reference to indexed element.
	 */
	template<typename T>
		requires (std::is_const_v<T>)
	constexpr T& GetAt(const int32 Index) const
	{
		return UE::StructUtils::GetStructRef<T>(ScriptStruct, GetDataAt(Index)); // GetDataAt() calls RangeCheck().
	}

	/**
	 * Array bracket operator. Returns FConstStructView to element at given index. Index must be valid.
	 *
	 * @returns FConstStructView to indexed element.
	 */
	FConstStructView operator[](int32 Index) const
	{
		return FConstStructView(ScriptStruct, (const uint8*)GetDataAt(Index)); // GetDataAt() calls RangeCheck().
	}

	/**
	 * Returns n-th last element from the array.
	 *
	 * @param IndexFromTheEnd (Optional) Index from the end of array. This MUST index a valid element.
	 *                        Default is 0.
	 *
	 * @returns const reference to n-th last element from the array.
	 */
	template<typename T>
		requires (std::is_const_v<T>)
	constexpr T& Last(int32 IndexFromTheEnd = 0) const
	{
		return GetAt<T>(ArrayNum - IndexFromTheEnd - 1);
	}

	/**
	 * Returns a sliced view
	 * This is similar to Mid(), but with a narrow contract, i.e. slicing outside of the range of the view is illegal.
	 *
	 * @param Index starting index of the new view
	 * @param InNum number of elements in the new view
	 * @returns Sliced view
	 *
	 * @see Mid
	 */
	[[nodiscard]] FConstStructArrayView Slice(const int32 Index, const int32 InNum) const
	{
		SliceRangeCheck(Index, InNum);
		return FConstStructArrayView(ScriptStruct, GetDataAt(Index), ElementSize, InNum);
	}

	/** Returns the left-most part of the view by taking the given number of elements from the left. */
	[[nodiscard]] FConstStructArrayView Left(const int32 Count) const
	{
		return FConstStructArrayView(ScriptStruct, DataPtr, ElementSize, FMath::Clamp(Count, 0, ArrayNum));
	}

	/** Returns the left-most part of the view by chopping the given number of elements from the right. */
	[[nodiscard]] FConstStructArrayView LeftChop(const int32 Count) const
	{
		return FConstStructArrayView(ScriptStruct, DataPtr, ElementSize, FMath::Clamp(ArrayNum - Count, 0, ArrayNum));
	}

	/** Returns the right-most part of the view by taking the given number of elements from the right. */
	[[nodiscard]] FConstStructArrayView Right(const int32 Count) const
	{
		const int32 OutLen = FMath::Clamp(Count, 0, ArrayNum);
		return FConstStructArrayView(ScriptStruct, GetDataAt(ArrayNum - OutLen), ElementSize, OutLen);
	}

	/** Returns the right-most part of the view by chopping the given number of elements from the left. */
	[[nodiscard]] FConstStructArrayView RightChop(const int32 Count) const
	{
		const int32 OutLen = FMath::Clamp(ArrayNum - Count, 0, ArrayNum);
		return FConstStructArrayView(ScriptStruct, GetDataAt(ArrayNum - OutLen), ElementSize, OutLen);
	}

	/** Returns the middle part of the view by taking up to the given number of elements from the given position. */
	[[nodiscard]] FConstStructArrayView Mid(int32 Index, int32 Count = TNumericLimits<int32>::Max()) const
	{
		UE::StructUtils::CalcMidIndexAndCount(ArrayNum, Index, Count);
		return FConstStructArrayView(ScriptStruct, GetDataAt(Index), ElementSize, Count);;
	}

	/** Modifies the view to be the given number of elements from the left. */
	void LeftInline(const int32 Count)
	{
		*this = Left(Count);
	}

	/** Modifies the view by chopping the given number of elements from the right. */
	void LeftChopInline(const int32 Count)
	{
		*this = LeftChop(Count);
	}

	/** Modifies the view to be the given number of elements from the right. */
	void RightInline(const int32 Count)
	{
		*this = Right(Count);
	}

	/** Modifies the view by chopping the given number of elements from the left. */
	void RightChopInline(const int32 Count)
	{
		*this = RightChop(Count);
	}

	/** Modifies the view to be the middle part by taking up to the given number of elements from the given position. */
	inline void MidInline(const int32 Position, const int32 Count = TNumericLimits<int32>::Max())
	{
		*this = Mid(Position, Count);
	}

	struct FIterator
	{
		FIterator(const FConstStructArrayView& InOwner, int32 InIndex)
			: Owner(&InOwner)
			, Index(InIndex)
		{}

		FIterator& operator++()
		{
			++Index;
			return *this;
		}

		FConstStructView operator*() const
		{
			return (*Owner)[Index];
		}

		inline bool operator == (const FIterator& Other) const
		{
			return Owner == Other.Owner
				&& Index == Other.Index;
		}

		UE_FORCEINLINE_HINT bool operator != (const FIterator& Other) const
		{
			return !(*this == Other);
		}

	private:
		const FConstStructArrayView* Owner = nullptr;
		int32 Index = INDEX_NONE;
	};

	/** Ranged iteration support. DO NOT USE DIRECTLY. */
	UE_FORCEINLINE_HINT FIterator begin() const { return FIterator(*this, 0); }
	UE_FORCEINLINE_HINT FIterator end() const { return FIterator(*this, Num()); }

private:
	const void* DataPtr = nullptr;
	const UScriptStruct* ScriptStruct = nullptr;
	uint32 ElementSize = 0;
	int32 ArrayNum = 0;
};
