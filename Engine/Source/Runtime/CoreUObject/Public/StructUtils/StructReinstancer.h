// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "UObject/ObjectKey.h"

#define UE_API COREUOBJECT_API

#if WITH_EDITOR
namespace UE::StructUtils
{
	/**
	 * Re-instance UScriptStruct with their new version.
	 * It uses the Reference Collector (AddStructReferencedObjects) to find all InstancedStruct/PropertyBag/... that use the struct.
	 * To restore only the modified property, the UObject that owns the UScriptStruct is serialized before the compilation process starts, and then serialized back once the new UScriptStruct is compiled.
	 * It utilizes tagged serialization and the default value of the archetype to obtain the new default values and restore the previously modified values.
	 * The tagged serialization only serialized the modified properties. For proper reinitialization, the owning struct must be different from the archetype. Override Identical and return false to make sure the owning struct is serialized.
	 * For UUserDefinedStruct re-instantiation, the original UScriptStruct and the new compiled UScriptStruct are the same instance. Other UScriptStruct types use different instances.
	 * 
	 * Algo:
	 * 1. For UUserDefinedStruct, create a duplicate. It will be used instead of the original UScriptStruct until it is compiled.
	 * 2. Gather all UObjects that contain a soon-to-be-dirty UScriptStruct.
	 *   1. For UUserDefinedStruct, replace the UScriptStruct with the duplicate.
	 * 3. Serialize (save) the UObjects (to gather only the modified properties).
	 * 4. Compile the UScriptStruct.
	 * 5. Serialize (load) the same UObjects. Replace the UScriptStrut with the new compiled version.
	 */
	class UE_INTERNAL FStructReinstancer final : FNoncopyable
	{
	public:
		UE_API static FStructReinstancer* GetInstance();
		/**
		 * Add a UScriptStrut to the re-instancer and its duplicated struct.
		 * The duplicated UScriptStrut is used to keep the raw memory readable while the UScriptStrut is compiled.
		 */
		UE_API void AddStruct(TNotNull<const UScriptStruct*> Original, TNotNull<const UScriptStruct*> Duplicated);

		/** Set the new compiled UScriptStrut created for the original struct. */
		UE_API void SetCompiledStruct(TNotNull<const UScriptStruct*> Original, TNotNull<const UScriptStruct*> Compiled);

		/** Collect all objects that use structs that will be re-instantiated. */
		UE_API void CollectObjects();

		/** Re-instantiate previously collected objects with the new struct. */
		UE_API void ReinstanceObjects();

		/** Serializing the UObject owners to gather the modified properties. */
		bool IsSerializingForReinstantiation() const
		{
			return bSerializingForReinstantiation;
		}

		/** Is the struct added to the FStructReinstancer. */
		bool IsReinstanting(TNotNull<const UScriptStruct*> Struct) const
		{
			return StructuresToReinstantiate.ContainsByPredicate(
				[Struct](const FStruct& Other)
				{
					return Other.Original.Get() == Struct
						|| Other.Duplicated.Get() == Struct;
				});
		}

		/**
		 * Find the duplicated UScriptStruct created to keep the raw memory readable while the struct is compiled.
		 * @note It is different for UUserDefinedStruct.
		 */
		const UScriptStruct* GetDuplicatedReinstantingStruct(TNotNull<const UScriptStruct*> Struct) const
		{
			const FStruct* Found = StructuresToReinstantiate.FindByPredicate(
				[Struct](const FStruct& Other)
				{
					return Other.Original.Get() == Struct;
				});
			return Found ? Found->Duplicated.Get() : nullptr;
		}

		/**
		 * Find the new compiled UScriptStruct created for the original struct.
		 * @note The new compiled UScriptStruct is the same as the original for UUserDefinedStruct.
		 */
		const UScriptStruct* GetCompiledReinstantingStruct(TNotNull<const UScriptStruct*> Struct) const
		{
			const FStruct* Found = StructuresToReinstantiate.FindByPredicate(
				[Struct](const FStruct& Other)
				{
					return Other.Original == Struct
						|| Other.Duplicated == Struct;
				});
			return Found ? Found->New.Get() : nullptr;
		}

		/** The current collected UObject contains at least one UScriptStruct that is being reinstantiated. */
		void MarkAsRequiresReinstantiation()
		{
			bRequiresReinstantiation = true;
		}
	private:
		struct FStruct
		{
			TObjectPtr<const UScriptStruct> Original;
			TObjectPtr<const UScriptStruct> Duplicated;
			TObjectPtr<const UScriptStruct> New;
		};
		struct FSavedDataForReinstantiation
		{
			FObjectKey Object;
			FObjectKey ClassGeneratedBy;
			TObjectKey<UStruct> Struct;
			TNotNull<void*> ObjectData;
			TObjectKey<const UStruct> SuperStruct;
			const void* SuperObjectData = nullptr;
			TArray<uint8> Buffer;
		};
		TArray<FStruct> StructuresToReinstantiate;
		TArray<FSavedDataForReinstantiation> ObjectsToReinstantiate;
		bool bRequiresReinstantiation = false;
		bool bSerializingForReinstantiation = false;
	};

	struct UE_INTERNAL FStructReinstancerScope final : FNoncopyable
	{
		UE_API explicit FStructReinstancerScope();
		UE_API ~FStructReinstancerScope();
	private:
		FStructReinstancer* OldStructReinstancer = nullptr;
	};
}

#endif // WITH_EDITOR

#undef UE_API
