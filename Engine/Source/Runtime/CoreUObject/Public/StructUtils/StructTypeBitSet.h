// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/BitArray.h"
#include "Misc/StringOutputDevice.h"
#include "StructUtils/StructUtilsTypes.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/FunctionFwd.h"

#define UE_API COREUOBJECT_API

class FArchive;

namespace FStructTypeBitSet
{

/**
 * A constant bitset container that extends TBitArray with additional utilities.
 * Used for operations such as bitwise checks and hashing
 */
struct FConstBitSetContainer : TBitArray<TInlineAllocator<8>>
{
	using FBitArrayBase = TBitArray;

	FConstBitSetContainer() = default;

	/**
	 * Checks if all bits set in the Other bit array are also set in this bitset
	 * @return True if all bits in Other are set in this bitset
	 */
	bool HasAll(const FBitArrayBase& Other) const
	{
		FConstWordIterator ThisIterator(*this);
		FConstWordIterator OtherIterator(Other);

		while (ThisIterator || OtherIterator)
		{
			const uint32 A = ThisIterator ? ThisIterator.GetWord() : 0;
			const uint32 B = OtherIterator ? OtherIterator.GetWord() : 0;
			if ((A & B) != B)
			{
				return false;
			}

			++ThisIterator;
			++OtherIterator;
		}

		return true;
	}

	/**
	 * Checks if any bits in the Other bit array are also set in this bitset
	 * @return True if any bits in Other are set in this bitset
	 */
	bool HasAny(const FBitArrayBase& Other) const
	{
		FConstWordIterator ThisIterator(*this);
		FConstWordIterator OtherIterator(Other);

		while (ThisIterator || OtherIterator)
		{
			const uint32 A = ThisIterator ? ThisIterator.GetWord() : 0;
			const uint32 B = OtherIterator ? OtherIterator.GetWord() : 0;
			if ((A & B) != 0)
			{
				return true;
			}

			++ThisIterator;
			++OtherIterator;
		}

		return false;
	}

	FConstBitSetContainer operator&(const FBitArrayBase& Other) const
	{
		const int32 MinNumBits = FMath::Min(Num(), Other.Num());
		if (MinNumBits > 0)
		{
			FBitArrayBase Result(false, MinNumBits);

			FConstWordIterator IteratorA(*this);
			FConstWordIterator IteratorB(Other);

			FWordIterator IteratorResult(Result);

			for ( ; IteratorResult; ++IteratorResult, ++IteratorA, ++IteratorB)
			{
				const uint32 NewValue = IteratorA.GetWord() & IteratorB.GetWord();
				IteratorResult.SetWord(NewValue);
			}

			return FConstBitSetContainer(MoveTemp(Result));
		}

		return {};
	}

	FConstBitSetContainer operator|(const FBitArrayBase& Other) const
	{
		const int32 MaxNumBits = FMath::Max(Num(), Other.Num());
		if (MaxNumBits > 0)
		{
			FBitArrayBase Result(false, MaxNumBits);

			FConstWordIterator IteratorA(*this);
			FConstWordIterator IteratorB(Other);

			FWordIterator IteratorResult(Result);

			for (; IteratorA && IteratorB; ++IteratorResult, ++IteratorA, ++IteratorB)
			{
				const uint32 NewValue = IteratorA.GetWord() | IteratorB.GetWord();
				IteratorResult.SetWord(NewValue);
			}
			for (; IteratorA; ++IteratorResult, ++IteratorA)
			{
				const uint32 NewValue = IteratorA.GetWord();
				IteratorResult.SetWord(NewValue);
			}
			for (; IteratorB; ++IteratorResult, ++IteratorB)
			{
				const uint32 NewValue = IteratorB.GetWord();
				IteratorResult.SetWord(NewValue);
			}

			return FConstBitSetContainer(MoveTemp(Result));
		}

		return {};
	}

	FConstBitSetContainer AndNot(const FBitArrayBase& Other) const
	{
		const int32 NumResultBits = Num();
		if (NumResultBits > 0)
		{
			FBitArrayBase Result(false, NumResultBits);

			FConstWordIterator IteratorA(*this);
			FConstWordIterator IteratorB(Other);

			FWordIterator IteratorResult(Result);

			for ( ; IteratorResult; ++IteratorResult, ++IteratorA, ++IteratorB)
			{
				const uint32 NewValue = IteratorB
					? IteratorA.GetWord() & ~IteratorB.GetWord()
					: IteratorA.GetWord();
				IteratorResult.SetWord(NewValue);
			}

			return FConstBitSetContainer(MoveTemp(Result));
		}

		return {};
	}

	/**
	 * @return whether the bitset is empty (no bits are set)
	 */
	bool IsEmpty() const
	{
		FConstWordIterator Iterator(*this);

		while (Iterator && Iterator.GetWord() == 0)
		{
			++Iterator;
		}

		return !Iterator;
	}

	/**
	 * Compares two bitsets for equality
	 * @return True if the bitsets are equal
	 */
	bool operator==(const FConstBitSetContainer& Other) const
	{
		return (IsEmpty() && Other.IsEmpty()) || FBitArrayBase::operator==(Other);
	}

	/**
	 * Computes the hash value for the given bitset
	 * @return The computed hash value
	 */
	friend uint32 GetTypeHash(const FConstBitSetContainer& Instance)
	{
		FConstWordIterator Iterator(Instance);
		uint32 Hash = 0;
		uint32 TrailingZeroHash = 0;
		while (Iterator)
		{
			const uint32 Word = Iterator.GetWord();
			if (Word)
			{
				Hash = HashCombine(TrailingZeroHash ? TrailingZeroHash : Hash, Word);
				TrailingZeroHash = 0;
			}
			else // potentially a trailing 0-word
			{
				TrailingZeroHash = HashCombine(TrailingZeroHash ? TrailingZeroHash : Hash, Word);
			}
			++Iterator;
		}
		return Hash;
	}

	/**
	 * Checks if a specific bit is set in the bitset
	 * @param Index - The index of the bit to check
	 * @return True if the bit is set, false otherwise
	 */
	bool Contains(const int32 Index) const
	{
		check(Index >= 0);
		return (Index < Num()) && (*this)[Index];
	}

	/**
	 * Counts the number of set bits in the bitset
	 * @return The number of set bits
	 */
	int32 CountStoredTypes() const
	{
		return CountSetBits();
	}

protected:
	/**
	 * Copy-constructor for creating a bitset from another bit array
	 * @param Source - The source bit array
	 */
	explicit FConstBitSetContainer(const FBitArrayBase& Source)
		: FBitArrayBase(Source)
	{
	}

	/**
	 * Move-constructor for creating a bitset from another bit array
	 * @param Source - The source bit array to move from
	 */
	explicit FConstBitSetContainer(FBitArrayBase&& Source)
		: FBitArrayBase(Forward<FBitArrayBase>(Source))
	{
	}
};

/**
 * A mutable extension of FConstBitSetContainer, adding methods for modifying bits
 */
struct FBitSetContainer : FConstBitSetContainer
{
	using FConstBitSetContainer::FConstBitSetContainer;

	FBitSetContainer() = default;

	explicit FBitSetContainer(const FBitArrayBase& Source)
		: FConstBitSetContainer(Source)
	{
	}
	explicit FBitSetContainer(FBitArrayBase&& Source)
		: FConstBitSetContainer(Forward<FBitArrayBase>(Source))
	{
	}
	FBitSetContainer(FConstBitSetContainer&& Source)
		: FConstBitSetContainer(Forward<FConstBitSetContainer>(Source))
	{
	}

	FBitSetContainer& operator=(const FBitArrayBase& Other)
	{
		static_assert(sizeof(FBitSetContainer) == sizeof(FBitArrayBase)
			, "This operator requires FBitSetContainer and FBitArrayBase sizes to match (i.e. FBitSetContainer cannot have member variables nor virtual funtions)");
		*((FBitArrayBase*)this) = Other;
		return *this;
	}


	/**
	 * Initializes the bitset to a specified size and value
	 * @param bValue - The value to set all bits to
	 * @param Count - The number of bits to initialize
	 */
	void SetAll(const bool bValue, const int32 Count)
	{
		Init(bValue, Count);
	}

	/**
	 * Removes bits set in the Other bit array from this bitset
	 * @param Other - The bit array to remove
	 */
	FBitSetContainer& operator-=(const FBitArrayBase& Other)
	{
		FWordIterator ThisIterator(*this);
		FConstWordIterator OtherIterator(Other);

		while (ThisIterator && OtherIterator)
		{
			ThisIterator.SetWord(ThisIterator.GetWord() & ~OtherIterator.GetWord());

			++ThisIterator;
			++OtherIterator;
		}
		return *this;
	}

	FBitSetContainer& operator+=(const FBitArrayBase& Other)
	{
		// make sure we have enough space for all the contents of Other
		// not that it's fine to pass in a negative number, TBitArray::Add handles that gracefully. 
		Add(false, Other.Num() - Num());
		
		FWordIterator ThisIterator(*this);
		FConstWordIterator OtherIterator(Other);

		while (OtherIterator)
		{
			ThisIterator.SetWord(ThisIterator.GetWord() | OtherIterator.GetWord());

			++ThisIterator;
			++OtherIterator;
		}
		return *this;
	}

	/**
	 * Adds a bit at the specified index
	 * @param Index - The index of the bit to add
	 */
	void AddAtIndex(const int32 Index)
	{
		check(Index >= 0);
		PadToNum(Index + 1, false);
		SetBitNoCheck(Index, true);
	}

	/**
	 * Removes a bit at the specified index
	 * @param Index - The index of the bit to remove
	 */
	void RemoveAtIndex(const int32 Index)
	{
		check(Index >= 0);
		if (Index < Num())
		{
			SetBitNoCheck(Index, false);
		}
		// else, it's already not present
	}

protected:
	/**
	 * Directly sets a bit without boundary checks for improved performance.
	 * @param Index - The index of the bit to set.
	 * @param Value - The value to set the bit to.
	 */
	void SetBitNoCheck(const int32 Index, const bool Value)
	{
		uint32& Word = GetData()[Index / NumBitsPerDWORD];
		const uint32 BitOffset = (Index % NumBitsPerDWORD);
		Word = (Word & ~(1 << BitOffset)) | (((uint32)Value) << BitOffset);
	}
};

// Ensure that FBitSetContainer does not add any new member variables compared to FConstBitSetContainer.
static_assert(sizeof(FBitSetContainer) == sizeof(FConstBitSetContainer), "FBitSetContainer as a functional extension of FConstBitSetContainer is not allowed add new member variables.");

} // namespace FStructTypeBitSet

/**
 * FStructTracker is a utility class used to track and map UStruct types to indices, which are used in bitsets.
 * It manages a mapping between UStruct instances and integer indices, allowing for efficient storage and querying
 * of types in a bitset
 *
 * The FStructTracker assigns an index to a given type the first time it encounters it
 */
struct FStructTracker
{
	using FBaseStructGetter = TFunction<UStruct*()>;
	using FTypeValidation = TFunction<bool(const UStruct*)>;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnTypeRegistered, TNotNull<const UStruct*> /*Type*/, int32 /*Index*/);

	/** 
	 * The input parameter is a function that fetches the UStruct representing the base class for all the stored structs.
	 * We're unable to get the UStruct parameter directly since FStructTracker instances are being created during
	 * module loading (via DEFINE_TYPEBITSET macro), and the ::StaticStruct call fails at that point for types defined
	 * in the same module.
	 */
	UE_API explicit FStructTracker(const FBaseStructGetter& InBaseStructGetter);
	UE_API explicit FStructTracker(const UStruct* InBaseType, const FTypeValidation& InTypeVerification = FTypeValidation());
	UE_API ~FStructTracker();

	UE_NONCOPYABLE(FStructTracker);

	/** 
	 * @return the index for the given type, or INDEX_NONE if it hasn't been registered with this struct tracker
	 */
	UE_API int32 FindStructTypeIndex(TNotNull<const UStruct*> InStructType) const;

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	int32 FindStructTypeIndex(const UStruct& InStructType) const
	{
		return FindStructTypeIndex(&InStructType);
	}

	/**
	 * Fetches the internal index representing the given UStruct instance. If it hasn't been registered yet, it will be automatically added.
	 * @return the index for the given type.
	 */
	UE_API int32 FindOrAddStructTypeIndex(TNotNull<const UStruct*> InStructType);

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	int32 FindOrAddStructTypeIndex(const UStruct& InStructType)
	{
		return FindOrAddStructTypeIndex(&InStructType);
	}

	/**
	 * Registers the given UStruct with the struct tracker.
	 * @return the index for the registered type.
	 */
	int32 Register(TNotNull<const UStruct*> InStructType)
	{
		return RegisterImplementation(InStructType, /*bCheckPrevious=*/true);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	int32 Register(const UStruct& InStructType)
	{
		return Register(&InStructType);
	}

	/**
	 * Retrieves the UStruct type associated with a given index
	 * @param StructTypeIndex - The index of the struct type
	 * @return Pointer to the UStruct if found, nullptr otherwise
	 */
	const UStruct* GetStructType(const int32 StructTypeIndex) const
	{
		return StructTypesList.IsValidIndex(StructTypeIndex) ? StructTypesList[StructTypeIndex].Get() : nullptr;
	}

	/**
	 * Retrieves the base UStruct type used for validation.
	 * @return Pointer to the base UStruct.
	 */
	UE_API const UStruct* GetBaseType() const;

	/**
	 * Gets the number of registered struct types.
	 * @return Number of struct types.
	 */
	int32 Num() const {	return StructTypeToIndexSet.Num(); }

	/**
	 * Serializes the struct types bit array using the archive provided.
	 * @param Ar - The archive to serialize with.
	 * @param StructTypesBitArray - The bit array representing struct types.
	 */
	UE_API void Serialize(FArchive& Ar, FStructTypeBitSet::FBitSetContainer& StructTypesBitArray);

	FOnTypeRegistered OnTypeRegistered;

#if WITH_STRUCTUTILS_DEBUG
	/**
	 * @return Name identifying the given struct type or NAME_None if it has never been used/seen before.
	 */
	FName DebugGetStructTypeName(const int32 StructTypeIndex) const
	{
		return DebugStructTypeNamesList.IsValidIndex(StructTypeIndex) ? DebugStructTypeNamesList[StructTypeIndex] : FName();
	}

	template<typename T>
	TConstArrayView<TWeakObjectPtr<const T>> DebugGetAllStructTypes() const 
	{ 
		return MakeArrayView(reinterpret_cast<const TWeakObjectPtr<const T>*>(StructTypesList.GetData()), StructTypesList.Num());
	}

	/**
	 * Resets all struct type mapping information. Used for debugging and testing purposes.
	 */
	void DebugResetStructTypeMappingInfo()
	{
		StructTypeToIndexSet.Reset();
		StructTypesList.Reset();
		DebugStructTypeNamesList.Reset();
	}

	[[nodiscard]] UE_API const UStruct* DebugFindTypeByPartialName(const FString& PartialName) const;

protected:
	mutable TArray<FName, TInlineAllocator<64>> DebugStructTypeNamesList;
#endif // WITH_STRUCTUTILS_DEBUG

protected:
	UE_API int32 RegisterImplementation(TNotNull<const UStruct*> InStructType, const bool bCheckPrevious);

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	int32 RegisterImplementation(const UStruct& InStructType, const bool bCheckPrevious)
	{
		return RegisterImplementation(&InStructType, bCheckPrevious);
	}

	/** Set mapping struct type hashes to their indices */
	TSet<uint32> StructTypeToIndexSet;
	/** List of weak pointers to struct types, indexed by their assigned indices */
	TArray<TWeakObjectPtr<const UStruct>, TInlineAllocator<64>> StructTypesList;
	/** Hash used during serialization to detect mismatches */
	uint32 SerializationHash = 0;
	/** Flag indicating whether this tracker is serializable */
	uint32 bIsSerializable : 1 = true;
	/** Function to retrieve the base UStruct */
	const FBaseStructGetter BaseStructGetter;
	/** Function to perform additional type verification */
	const FTypeValidation TypeVerification;

private:
	mutable const UStruct* BaseType;
};

/**
 * Base class for managing bitsets associated with specific struct types.
 * Provides a foundation for derived classes to handle operations like adding, removing, and checking for struct types.
 *
 * @param TImplementation - The derived type implementing this base class.
 * @param TBaseStruct - The base struct type for type validation.
 * @param TUStructType - Unreal's struct type, typically UScriptStruct or UClass.
 * @param TBitSetContainer - The container type for storing bitsets.
 * @param bTestInheritanceAtRuntime - Flag to enable runtime inheritance checks.
 */
template<typename TImplementation, typename TBaseStruct, typename TUStructType, typename TBitSetContainer, bool bTestInheritanceAtRuntime=WITH_STRUCTUTILS_DEBUG>
struct TTypeBitSetBase
{
	using FBaseStruct = TBaseStruct;
	using FContainer = TBitSetContainer;
	using FBitArrayBase = FStructTypeBitSet::FConstBitSetContainer::FBitArrayBase;

	/**
	 * Iterator for traversing indices of bits with a specified value (true or false).
	 */
	struct FIndexIterator
	{
		explicit FIndexIterator(const FStructTypeBitSet::FBitSetContainer& BitArray, const bool bInValueToCheck = true)
			: It(BitArray), bValueToCheck(bInValueToCheck)
		{
			if (It && It.GetValue() != bInValueToCheck)
			{
				// Will result in either setting It to the first bit with bInValueToCheck, or making bool(It) == false
				++(*this);
			}
		}

		operator bool() const
		{
			return bool(It);
		}

		/** Advances the iterator to the next bit with the specified value */
		FIndexIterator& operator++() 
		{ 
			while (++It)
			{
				if (It.GetValue() == bValueToCheck)
				{
					break;
				}
			}
			return *this; 
		}

		/** Dereferences the iterator to get the current index */
		int32 operator*() const
		{
			return It.GetIndex();
		}

	private:
		FBitArrayBase::FConstIterator It;
		const bool bValueToCheck;
	};

	/**
	 * Creates an index iterator for bits with the specified value.
	 * @param bValueToCheck - The bit value to search for (true or false).
	 * @return An iterator to traverse indices of matching bits.
	 */
	FIndexIterator GetIndexIterator(const bool bValueToCheck = true) const
	{
		return FIndexIterator(StructTypesBitArray, bValueToCheck);
	}

	/**
	 * Retrieves the derived implementation of this base class.
	 * @return Reference to the derived implementation.
	 */
	TImplementation& GetImplementation()
	{
		return *reinterpret_cast<TImplementation*>(this);
	}

	/**
	 * Retrieves the derived implementation of this base class (const version).
	 * @return Const reference to the derived implementation.
	 */
	const TImplementation& GetImplementation() const 
	{
		return *reinterpret_cast<const TImplementation*>(this);
	}

	/**
	 * Retrieves the base UStruct type used for validation.
	 * @return Pointer to the base UStruct.
	 */
	static const UStruct* GetBaseUStruct()
	{
		if constexpr (bTestInheritanceAtRuntime)
		{
			static const UStruct* Instance = UE::StructUtils::GetAsUStruct<FBaseStruct>();
			return Instance;
		}
		else
		{
			return nullptr;
		}
	}

	/**
	 * Sets all bits in the bitset to the specified value.
	 * @param bValue - The value to set for all bits (default is true).
	 */
	void SetAll(const bool bValue = true)
	{
		StructTypesBitArray.SetAll(bValue, GetImplementation().GetStructTracker().Num());
	}

	/**
	 * Adds a struct type to the bitset.
	 * @param InStructType - The struct type to add.
	 */
	void Add(TNotNull<const TUStructType*> InStructType)
	{
		if constexpr (bTestInheritanceAtRuntime)
		{
			if (UNLIKELY(!ensureMsgf(InStructType->IsChildOf(GetBaseUStruct()),
				TEXT("Registering '%s' with FStructTracker while it doesn't derive from the expected struct type %s"),
				*InStructType->GetPathName(), *GetBaseUStruct()->GetName())))
			{
				return;
			}
		}

		const int32 StructTypeIndex = GetImplementation().GetStructTracker().FindOrAddStructTypeIndex(InStructType);
		StructTypesBitArray.AddAtIndex(StructTypeIndex);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void Add(const TUStructType& InStructType)
	{
		Add(&InStructType);
	}

	/** Sets the bit at index StructTypeIndex */
	void AddAtIndex(const int32 StructTypeIndex)
	{
		StructTypesBitArray.AddAtIndex(StructTypeIndex);
	}

	/**
	 * Removes a struct type from the bitset.
	 * @param InStructType - The struct type to remove.
	 */
	void Remove(TNotNull<const TUStructType*> InStructType)
	{
		if constexpr (bTestInheritanceAtRuntime)
		{
			if (UNLIKELY(!ensureMsgf(InStructType->IsChildOf(GetBaseUStruct()),
				TEXT("Registering '%s' with FStructTracker while it doesn't derive from the expected struct type %s"),
				*InStructType->GetPathName(), *GetBaseUStruct()->GetName())))
			{
				return;
			}
		}

		const int32 StructTypeIndex = GetImplementation().GetStructTracker().FindOrAddStructTypeIndex(InStructType);
		StructTypesBitArray.RemoveAtIndex(StructTypeIndex);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void Remove(const TUStructType& InStructType)
	{
		Remove(&InStructType);
	}

	/** Clears the bit at index StructTypeIndex */
	void RemoveAtIndex(const int32 StructTypeIndex)
	{
		StructTypesBitArray.RemoveAtIndex(StructTypeIndex);
	}

	/**
	 * Resets all bits in the bitset
	 */
	void Reset()
	{
		StructTypesBitArray.Reset();
	}

	/**
	 * Checks if the bitset contains a specific struct type.
	 * @param InStructType - The struct type to check for.
	 * @return True if the struct type is in the bitset; false otherwise.
	 */
	bool Contains(TNotNull<const TUStructType*> InStructType) const
	{
		if constexpr (bTestInheritanceAtRuntime)
		{
			if (UNLIKELY(!ensureMsgf(InStructType->IsChildOf(GetBaseUStruct()),
				TEXT("Registering '%s' with FStructTracker while it doesn't derive from the expected struct type %s"),
				*InStructType->GetPathName(), *GetBaseUStruct()->GetName())))
			{
				return false;
			}
		}

		const int32 StructTypeIndex = GetImplementation().GetStructTracker().FindStructTypeIndex(InStructType);
		return StructTypeIndex != INDEX_NONE && StructTypesBitArray.Contains(StructTypeIndex);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	void Contains(const TUStructType& InStructType)
	{
		Contains(&InStructType);
	}

	/**
	 * Performs a bitwise AND operation with another bitset.
	 * @param Other - The other bitset to intersect with.
	 * @return A new bitset representing the intersection.
	 */
	UE_FORCEINLINE_HINT TImplementation operator&(const TImplementation& Other) const
	{
		return TImplementation(StructTypesBitArray & Other.StructTypesBitArray);
	}

	/**
	 * Performs a bitwise OR operation with another bitset.
	 * @param Other - The other bitset to union with.
	 * @return A new bitset representing the union.
	 */
	UE_FORCEINLINE_HINT TImplementation operator|(const TImplementation& Other) const
	{
		return TImplementation(StructTypesBitArray | Other.StructTypesBitArray);
	}

	/**
	 * Performs a bitwise AND operation with the negation of the Other bitset.
	 * @return A new bitset representing the result of the operation.
	 */
	UE_FORCEINLINE_HINT TImplementation AndNot(const TImplementation& Other) const
	{
		return TImplementation(StructTypesBitArray.AndNot(Other.StructTypesBitArray));
	}

	UE_FORCEINLINE_HINT TImplementation GetOverlap(const TImplementation& Other) const
	{
		return GetImplementation() & Other;
	}

	/**
	 * Checks if the current bitset is equivalent to another, i.e., whether both contain same "true" bits
	 * and ignoring the trailing "false" bits.
	 * @param Other - The other bitset to compare.
	 * @return True if both bitsets are equivalent; false otherwise.
	 */
	UE_FORCEINLINE_HINT bool IsEquivalent(const TImplementation& Other) const
	{
		return StructTypesBitArray.CompareSetBits(Other.StructTypesBitArray, /*bMissingBitValue=*/false);
	}

	bool IsEquivalentFiltered(const TImplementation& Other, const TImplementation& Filter) const
	{
		constexpr uint32 MissingBitsFill = 0;

		FBitArrayBase::FConstWordIterator ThisIterator(StructTypesBitArray);
		FBitArrayBase::FConstWordIterator OtherIterator(Other.StructTypesBitArray);
		FBitArrayBase::FConstWordIterator FilterIterator(Filter.StructTypesBitArray);

		ThisIterator.FillMissingBits(MissingBitsFill);
		OtherIterator.FillMissingBits(MissingBitsFill);
		FilterIterator.FillMissingBits(MissingBitsFill);

		while (FilterIterator && (ThisIterator || OtherIterator))
		{
			const uint32 A = ThisIterator  ? (ThisIterator.GetWord() & FilterIterator.GetWord()) : MissingBitsFill;
			const uint32 B = OtherIterator ? (OtherIterator.GetWord() & FilterIterator.GetWord()) : MissingBitsFill;
			if (A != B)
			{
				return false;
			}

			++ThisIterator;
			++OtherIterator;
			++FilterIterator;
		}

		return true;
	}

	UE_FORCEINLINE_HINT bool HasAll(const TImplementation& Other) const
	{
		return StructTypesBitArray.HasAll(Other.StructTypesBitArray);
	}

	UE_FORCEINLINE_HINT bool HasAny(const TImplementation& Other) const
	{
		return StructTypesBitArray.HasAny(Other.StructTypesBitArray);
	}

	UE_FORCEINLINE_HINT bool HasNone(const TImplementation& Other) const
	{
		return !StructTypesBitArray.HasAny(Other.StructTypesBitArray);
	}

	/**
	 * Checks if the bitset is empty (no bits are set).
	 * @return True if the bitset is empty.
	 */
	bool IsEmpty() const 
	{ 
		return StructTypesBitArray.IsEmpty();
	}

	/**
	 * Checks if a specific bit is set in the bitset.
	 * @param BitIndex - The index of the bit to check.
	 * @return True if the bit is set; false otherwise.
	 */
	UE_FORCEINLINE_HINT bool IsBitSet(const int32 BitIndex) const
	{
		return StructTypesBitArray.Contains(BitIndex);
	}

	/**
	 * Adds the bits from another bitset to this one (union).
	 * @param Other - The other bitset to add.
	 */
	TTypeBitSetBase& operator+=(const TImplementation& Other)
	{
		StructTypesBitArray += Other.StructTypesBitArray;
		return *this;
	}

	/**
	 * Removes the bits from another bitset from this one (difference).
	 * @param Other - The other bitset to subtract.
	 */
	UE_FORCEINLINE_HINT TTypeBitSetBase& operator-=(const TImplementation& Other)
	{
		StructTypesBitArray -= Other.StructTypesBitArray;
		return *this;
	}

	/**
	 * Adds a struct type to the bitset, returning a new bitset.
	 * @param NewElement - The struct type to add.
	 * @return A new bitset with the added struct type.
	 */
	TImplementation operator+(TNotNull<const TUStructType*> NewElement) const
	{
		TImplementation Result = GetImplementation();
		Result.Add(NewElement);
		return MoveTemp(Result);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	TImplementation operator+(const TUStructType& NewElement) const
	{
		return operator+(&NewElement);
	}

	/**
	 * Removes a struct type from the bitset, returning a new bitset.
	 * @param NewElement - The struct type to remove.
	 * @return A new bitset with the struct type removed.
	 */
	TImplementation operator-(TNotNull<const TUStructType*> NewElement) const
	{
		TImplementation Result = GetImplementation();
		Result.Remove(NewElement);
		return MoveTemp(Result);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	TImplementation operator-(const TUStructType& NewElement) const
	{
		return operator-(&NewElement);
	}

	/**
	 * Counts the number of set bits in the bitset.
	 * @return The number of set bits.
	 */
	int32 CountStoredTypes() const
	{
		return StructTypesBitArray.CountSetBits();
	}

	/**
	 * Exports types stored in the bitset to an output array.
	 * Note: This method can be slow due to the use of weak pointers in the struct tracker.
	 * @todo To be improved.
	 * @param OutTypes - The array to populate with struct types.
	 */
	template<typename TOutStructType, typename Allocator>
	void ExportTypes(TArray<const TOutStructType*, Allocator>& OutTypes) const
	{
		FBitArrayBase::FConstIterator It(StructTypesBitArray);
		while (It)
		{
			if (It.GetValue())
			{
				OutTypes.Add(Cast<TOutStructType>(GetImplementation().GetStructTracker().GetStructType(It.GetIndex())));
			}
			++It;
		}
	}

#if WITH_STRUCTUTILS_DEBUG
	/**
	 * Provides a debug string description of the bitset contents via the provided FOutputDevice.
	 */
	void DebugGetStringDesc(FOutputDevice& Ar) const
	{
		for (int32 Index = 0; Index < StructTypesBitArray.Num(); ++Index)
		{
			if (StructTypesBitArray[Index])
			{
				Ar.Logf(TEXT("%s, "), *GetImplementation().GetStructTracker().DebugGetStructTypeName(Index).ToString());
			}
		}
	}

	/**
	 * Retrieves the names of individual types stored in the bitset.
	 * @param OutFNames - The array to populate with type names.
	 */
	void DebugGetIndividualNames(TArray<FName>& OutFNames) const
	{
		for (int32 Index = 0; Index < StructTypesBitArray.Num(); ++Index)
		{
			if (StructTypesBitArray[Index])
			{
				OutFNames.Add(GetImplementation().GetStructTracker().DebugGetStructTypeName(Index));
			}
		}
	}
#endif // WITH_STRUCTUTILS_DEBUG

	/**
	 * Gets the allocated size of the bitset.
	 * @return The allocated memory size in bytes.
	 */
	SIZE_T GetAllocatedSize() const
	{
		return StructTypesBitArray.GetAllocatedSize();
	}

protected:
	/**
	 * Default constructor.
	 */
	TTypeBitSetBase() = default;

	/**
	 * Constructor accepting a container.
	 * @param InContainer - The bitset container to use.
	 */
	explicit TTypeBitSetBase(const FContainer& InContainer)
		: StructTypesBitArray(InContainer)
	{
	}

	explicit TTypeBitSetBase(FBitArrayBase&& InContainer)
		: StructTypesBitArray(MoveTemp(InContainer))
	{
	}

#if WITH_STRUCTUTILS_DEBUG
	// For unit testing purposes only
	const FBitArrayBase& DebugGetStructTypesBitArray() const { return StructTypesBitArray; }
	FBitArrayBase& DebugGetMutableStructTypesBitArray() { return StructTypesBitArray; }
#endif // WITH_STRUCTUTILS_DEBUG

	/** The bitset container storing the bits representing struct types */
	FContainer StructTypesBitArray;
};


/**
 * The TStructTypeBitSet holds information on "existence" of subtypes of a given UStruct. The information on 
 * available child-structs is gathered lazily - the internal FStructTracker assigns a given type a new index the
 * very first time the type is encountered. 
 * To create a specific instantiation of the type, you also need to provide a type that will instantiate a static 
 * FStructTracker instance. The supplied macros hide this detail.
 * 
 * To declare a bitset type for an arbitrary struct type FFooBar, add the following in your header or cpp file:
 * 
 *	DECLARE_STRUCTTYPEBITSET(FMyFooBarBitSet, FFooBar);
 * 
 * where FMyFooBarBitSet is the alias of the type you can use in your code. To have your type exposed to other modules 
 * use DECLARE_STRUCTTYPEBITSET_EXPORTED, like so:
 * 
 *	DECLARE_STRUCTTYPEBITSET_EXPORTED(MYMODULE_API, FMyFooBarBitSet, FFooBar);
 *
 * You also need to instantiate the static FStructTracker added by the DECLARE* macro. You can easily do it by placing 
 * the following in your cpp file (continuing the FFooBar example):
 *
 *	DEFINE_TYPEBITSET(FMyFooBarBitSet);
 *
 *
 * @param TBaseStruct - The base struct type for type validation.
 * @param TStructTrackerWrapper - A wrapper providing a FStructTracker instance.
 * @param TUStructType - Unreal's struct type, typically UScriptStruct or UClass (defaults to UScriptStruct).
 */
template<typename TBaseStruct, typename TStructTrackerWrapper, typename TUStructType = UScriptStruct>
struct TStructTypeBitSet : TTypeBitSetBase<TStructTypeBitSet<TBaseStruct, TStructTrackerWrapper, TUStructType>, TBaseStruct, TUStructType, FStructTypeBitSet::FBitSetContainer>
{
	using Super = TTypeBitSetBase<TStructTypeBitSet, TBaseStruct, TUStructType, FStructTypeBitSet::FBitSetContainer>;
	using FStructTrackerWrapper = TStructTrackerWrapper;
	using FBaseStruct = TBaseStruct;
	using FBitSetContainer = FStructTypeBitSet::FBitSetContainer;
	using FBitArrayBase = FStructTypeBitSet::FConstBitSetContainer::FBitArrayBase;

	friend Super;
	using Super::StructTypesBitArray;
	using Super::GetBaseUStruct;
	using Super::Add;
	using Super::Remove;
	using Super::Contains;
	using Super::operator+=;
	using Super::operator-=;
	using Super::operator+;
	using Super::operator-;
	using Super::ExportTypes;

	TStructTypeBitSet() = default;
	TStructTypeBitSet(const TStructTypeBitSet&) = default;
	TStructTypeBitSet(TStructTypeBitSet&&) = default;
	TStructTypeBitSet& operator=(const TStructTypeBitSet&) = default;
	TStructTypeBitSet& operator=(TStructTypeBitSet&&) = default;

	/**
	 * Constructor that initializes the bitset with a single struct type.
	 * @param StructType - The struct type to add.
	 */
	explicit TStructTypeBitSet(TNotNull<const TUStructType*> StructType)
	{
		Add(StructType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	explicit TStructTypeBitSet(const TUStructType& StructType)
		: TStructTypeBitSet(&StructType)
	{
	}

	/**
	 * Constructor that initializes the bitset with an initializer list of struct types.
	 * @param InitList - The initializer list of struct type pointers.
	 */
	explicit TStructTypeBitSet(std::initializer_list<const TUStructType*> InitList)

	{
		for (const TUStructType* StructType : InitList)
		{
			if (StructType)
			{
				Add(StructType);
			}
		}
	}

	/**
	 * Constructor that initializes the bitset with an array view of struct types.
	 * @param InitList - The array view of struct type pointers.
	 */
	explicit TStructTypeBitSet(TConstArrayView<const TUStructType*> InitList)
	{
		for (const TUStructType* StructType : InitList)
		{
			if (StructType)
			{
				Add(StructType);
			}
		}
	}

	/**
	 * This constructor is only available for UScriptStructs.
	 * Initializes the bitset with an array view of FInstancedStruct.
	 * @param InitList - The array view of FInstancedStruct.
	 */
	template<typename T = TBaseStruct, typename = typename TEnableIf<!TIsDerivedFrom<T, UObject>::IsDerived>::Type>
	explicit TStructTypeBitSet(TConstArrayView<FInstancedStruct> InitList)
	{
		for (const FInstancedStruct& StructInstance : InitList)
		{
			if (StructInstance.GetScriptStruct())
			{
				Add(StructInstance.GetScriptStruct());
			}
		}
	}

private:
	/** 
	 * A private constructor for creating an instance straight from TBitArrays. 
	 * Note that this constructor needs to remain private to ensure consistency of stored values with data tracked 
	 * by the TStructTrackerWrapper::StructTracker.
	 */
	TStructTypeBitSet(FBitArrayBase&& Source)
		: Super(MoveTemp(Source))
	{
	}

	TStructTypeBitSet(const FBitArrayBase& Source)
	{
		StructTypesBitArray = Source;
	}

public:
	explicit TStructTypeBitSet(const int32 BitToSet)
	{
		StructTypesBitArray.AddAtIndex(BitToSet);
	}

	/**
	 * Retrieves the FStructTracker associated with this bitset.
	 * @return Reference to the FStructTracker.
	 */
	FStructTracker& GetStructTracker()
	{
		return TStructTrackerWrapper::StructTracker;
	}

	/**
	 * Retrieves the FStructTracker associated with this bitset (const version).
	 * @return Const reference to the FStructTracker.
	 */
	const FStructTracker& GetStructTracker() const
	{
		return TStructTrackerWrapper::StructTracker;
	}

	/**
	 * Gets the index of a struct type within the tracker, adding it if not already present.
	 * @param InStructType - The struct type to get the index for.
	 * @return The index of the struct type.
	 */
	static int32 GetTypeIndex(TNotNull<const TUStructType*> InStructType)

	{
#if WITH_STRUCTUTILS_DEBUG
		ensureMsgf(InStructType->IsChildOf(GetBaseUStruct())
			, TEXT("Creating index for '%s' while it doesn't derive from the expected struct type %s")
			, *InStructType->GetPathName(), *GetBaseUStruct()->GetName());
#endif // WITH_STRUCTUTILS_DEBUG

		return TStructTrackerWrapper::StructTracker.FindOrAddStructTypeIndex(InStructType);
	}

	UE_DEPRECATED(5.8, "Use overload taking TNotNull")
	static int32 GetTypeIndex(const TUStructType& InStructType)
	{
		return GetTypeIndex(&InStructType);
	}


	/**
	 * Gets the index of a struct type within the tracker, adding it if not already present.
	 * @param T - The struct type to get the index for.
	 * @return The index of the struct type.
	 */
	template<typename T>
	static int32 GetTypeIndex()
	{
		static_assert(TIsDerivedFrom<T, TBaseStruct>::IsDerived, "Given struct type doesn't match the expected base struct type.");
		static const int32 TypeIndex = GetTypeIndex(UE::StructUtils::GetAsUStruct<T>());
		return TypeIndex;
	}
	
	/**
	 * Gets a bitset representing a single struct type.
	 * @param T - The struct type to get the bitset for.
	 * @return Reference to a static bitset representing the struct type.
	 */
	template<typename T>
	static const TStructTypeBitSet& GetTypeBitSet()
	{
		static_assert(TIsDerivedFrom<T, TBaseStruct>::IsDerived, "Given struct type doesn't match the expected base struct type.");
		static const TStructTypeBitSet TypeBitSet(GetTypeIndex<T>());
		return TypeBitSet;
	}

	/**
	 * Gets the struct type associated with a given index.
	 * @param Index - The index of the struct type.
	 * @return Pointer to the struct type if found; nullptr otherwise.
	 */
	static const TUStructType* GetTypeAtIndex(const int32 Index)
	{
		return Cast<const TUStructType>(TStructTrackerWrapper::StructTracker.GetStructType(Index));
	}

	/**
	 * Adds a struct type to the bitset.
	 * @param T - The struct type to add.
	 */
	template<typename T>
	void Add()
	{
		static_assert(TIsDerivedFrom<T, TBaseStruct>::IsDerived, "Given struct type doesn't match the expected base struct type.");
		const int32 StructTypeIndex = GetTypeIndex<T>();
		StructTypesBitArray.AddAtIndex(StructTypeIndex);
	}

	/**
	 * Removes a struct type from the bitset.
	 * @param T - The struct type to remove.
	 */
	template<typename T>
	void Remove()
	{
		static_assert(TIsDerivedFrom<T, TBaseStruct>::IsDerived, "Given struct type doesn't match the expected base struct type.");
		const int32 StructTypeIndex = GetTypeIndex<T>();
		StructTypesBitArray.RemoveAtIndex(StructTypeIndex);
	}

	/**
	 * Removes the bits from another bitset from this one (difference).
	 * @param Other - The other bitset to subtract.
	 */
	UE_FORCEINLINE_HINT void Remove(const TStructTypeBitSet& Other)
	{
		StructTypesBitArray -= Other.StructTypesBitArray;
	}

	/**
	 * Checks if the bitset contains a specific struct type.
	 * @param T - The struct type to check for.
	 * @return True if the struct type is in the bitset; false otherwise.
	 */
	template<typename T>
	bool Contains() const
	{
		static_assert(TIsDerivedFrom<T, TBaseStruct>::IsDerived, "Given struct type doesn't match the expected base struct type.");
		const int32 StructTypeIndex = GetTypeIndex<T>();
		return StructTypesBitArray.Contains(StructTypeIndex);
	}

	/**
	 * Performs a union operation with another bitset.
	 * @param Other - The other bitset to combine with.
	 * @return A new bitset representing the union.
	 */
	TStructTypeBitSet operator+(const TStructTypeBitSet& Other) const
	{
		return (*this) | Other;
	}

	/**
	 * Gets the maximum number of struct types registered in the tracker.
	 * @return The number of struct types.
	 */
	static int32 GetMaxNum() 
	{		
		return TStructTrackerWrapper::StructTracker.Num();
	}

	/**
	 * Equality operator.
	 * @param Other - The other bitset to compare
	 * @return True if the bitsets are equal; false otherwise.
	 */
	UE_FORCEINLINE_HINT bool operator==(const TStructTypeBitSet& Other) const
	{
		return Super::StructTypesBitArray == Other.StructTypesBitArray;
	}

	/**
	 * Inequality operator.
	 * @param Other - The other bitset to compare.
	 * @return True if the bitsets are not equal; false otherwise.
	 */
	UE_FORCEINLINE_HINT bool operator!=(const TStructTypeBitSet& Other) const
	{
		return !(*this == Other);
	}

	/**
	 * Performs a difference operation with another bitset.
	 * @param Other - The other bitset to subtract.
	 * @return A new bitset representing the difference.
	 */
	TStructTypeBitSet operator-(const TStructTypeBitSet& Other) const
	{
		TStructTypeBitSet Result = *this;
		Result -= Other;
		return MoveTemp(Result);
	}

	/**
	 * Lists all types used by this bitset, calling the provided callback for each one.
	 * Returning false from the callback will early-out of iterating over the types.
	 *
	 * Note that this function is slow due to the FStructTracker utilizing WeakObjectPtrs to store types.
	 * @todo To be improved.
	 * @param Callback - The callback function to invoke for each type.
	 */
	void ExportTypes(TFunctionRef<bool(const TUStructType*)> Callback) const
	{
		FBitArrayBase::FConstIterator It(Super::StructTypesBitArray);
		bool bKeepGoing = true;
		while (bKeepGoing && It)
		{
			if (It.GetValue())
			{
				bKeepGoing = Callback(GetTypeAtIndex(It.GetIndex()));
			}
			++It;
		}
	}

#if WITH_STRUCTUTILS_DEBUG
	using Super::DebugGetStringDesc;
	/**
	 * Provides a debug string description of the bitset contents.
	 * @return The debug string.
	 */
	FString DebugGetStringDesc() const
	{
		FStringOutputDevice Ar;
		DebugGetStringDesc(Ar);
		return static_cast<FString>(Ar);
	}
#else
	/**
	 * Provides a placeholder debug string when debug info is compiled out.
	 * @return The placeholder string.
	 */
	FString DebugGetStringDesc() const
	{
		return TEXT("DEBUG INFO COMPILED OUT");
	}
#endif //WITH_STRUCTUTILS_DEBUG

#if WITH_STRUCTUTILS_DEBUG
	/**
	 * Retrieves the name of a struct type by index (for debugging).
	 * @param StructTypeIndex - The index of the struct type.
	 * @return The name of the struct type.
	 */
	static FName DebugGetStructTypeName(const int32 StructTypeIndex)
	{
		return TStructTrackerWrapper::StructTracker.DebugGetStructTypeName(StructTypeIndex);
	}

	/**
	 * Retrieves all struct types (for debugging).
	 * @return An array view of weak pointers to the struct types.
	 */
	static TConstArrayView<TWeakObjectPtr<const TUStructType>> DebugGetAllStructTypes()
	{
		return TStructTrackerWrapper::StructTracker.template DebugGetAllStructTypes<TUStructType>();
	}

	/**
	 * Resets all the information gathered on the types.
	 * Calling this results in invalidating all previously created TStructTypeBitSet instances.
	 * Used only for debugging and unit/functional testing.
	 */
	static void DebugResetStructTypeMappingInfo()
	{
		TStructTrackerWrapper::StructTracker.DebugResetStructTypeMappingInfo();
	}

	[[nodiscard]] static const TUStructType* DebugFindTypeByPartialName(const FString& PartialName)
	{
		return reinterpret_cast<const TUStructType*>(TStructTrackerWrapper::StructTracker.DebugFindTypeByPartialName(PartialName));
	}
#endif // WITH_STRUCTUTILS_DEBUG

public:
	/**
	 * Hash function for the bitset.
	 * @param Instance - The bitset instance to hash.
	 * @return The hash value.
	 */
	friend uint32 GetTypeHash(const TStructTypeBitSet& Instance)
	{
		const uint32 StoredTypeHash = PointerHash(GetBaseUStruct());
		const uint32 BitArrayHash = GetTypeHash(Instance.StructTypesBitArray);
		return HashCombine(StoredTypeHash, BitArrayHash);
	}

	/**
	 * Serializes the bitset using the provided archive.
	 * @param Ar - The archive to serialize with.
	 */
	void Serialize(FArchive& Ar)
	{
		TStructTrackerWrapper::StructTracker.Serialize(Ar, Super::StructTypesBitArray);
	}

	/**
	 * Stream insertion operator for serialization.
	 * @param Ar - The archive to serialize with.
	 * @param Instance - The bitset instance to serialize.
	 * @return The archive after serialization.
	 */
	friend FArchive& operator<<(FArchive& Ar, TStructTypeBitSet& Instance)
	{
		Instance.Serialize(Ar);
		return Ar;
	}
};

/** 
 * We're declaring StructTracker this way rather than being a static member variable of TStructTypeBitSet to avoid linking
 * issues. We've run into all sorts of issues depending on compiler and whether strict mode was on, and this is the best
 * way we could come up with to solve it. Thankfully the user doesn't need to even know about this class's existence
 * as long as they are using the macros below.
 */
#define _DECLARE_TYPEBITSET_IMPL(EXPORTED_API, ContainerTypeName, BaseType, BaseUStructType) struct ContainerTypeName##StructTrackerWrapper \
	{ \
		using FBaseStructType = BaseType; \
		EXPORTED_API static FStructTracker StructTracker; \
	}; \
	using ContainerTypeName = TStructTypeBitSet<BaseType, ContainerTypeName##StructTrackerWrapper, BaseUStructType>; \
	static_assert(std::is_move_constructible_v<ContainerTypeName> && std::is_move_assignable_v<ContainerTypeName>, "TStructTypeBitSet as configured is expected to be movable")

#define DECLARE_STRUCTTYPEBITSET_EXPORTED(EXPORTED_API, ContainerTypeName, BaseStructType) _DECLARE_TYPEBITSET_IMPL(EXPORTED_API, ContainerTypeName, BaseStructType, UScriptStruct)
#define DECLARE_STRUCTTYPEBITSET(ContainerTypeName, BaseStructType) _DECLARE_TYPEBITSET_IMPL(, ContainerTypeName, BaseStructType, UScriptStruct)
#define DECLARE_CLASSTYPEBITSET_EXPORTED(EXPORTED_API, ContainerTypeName, BaseStructType) _DECLARE_TYPEBITSET_IMPL(EXPORTED_API, ContainerTypeName, BaseStructType, UClass)
#define DECLARE_CLASSTYPEBITSET(ContainerTypeName, BaseStructType) _DECLARE_TYPEBITSET_IMPL(, ContainerTypeName, BaseStructType, UClass)

#define DEFINE_TYPEBITSET(ContainerTypeName) \
	FStructTracker ContainerTypeName##StructTrackerWrapper::StructTracker([](){ return UE::StructUtils::GetAsUStruct<ContainerTypeName##StructTrackerWrapper::FBaseStructType>();});

#undef UE_API 