// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTypeTraits.h"
#include "Templates/ValueOrError.h"
#include "UObject/Class.h"

class UPropertyBagHierarchyRoot;
struct FInstancedPropertyBag;
struct FPropertyBagPropertyDesc;
class FXxHash64Builder;
struct FPropertyBagPropertyDescMetaData;

namespace UE::StructUtils
{
	template <typename T>
	void CheckStructType()
	{
		static_assert(!TIsDerivedFrom<T, struct FInstancedStruct>::IsDerived &&
					  !TIsDerivedFrom<T, struct FStructView>::IsDerived &&
					  !TIsDerivedFrom<T, struct FConstStructView>::IsDerived &&
					  !TIsDerivedFrom<T, struct FSharedStruct>::IsDerived &&
					  !TIsDerivedFrom<T, struct FConstSharedStruct>::IsDerived, "It does not make sense to create wrapper over these types.");
	}

	template <typename T>
	void CheckWrapperType()
	{
		static_assert(TIsDerivedFrom<T, struct FInstancedStruct>::IsDerived
			|| TIsDerivedFrom<T, struct FStructView>::IsDerived
			|| TIsDerivedFrom<T, struct FConstStructView>::IsDerived
			|| TIsDerivedFrom<T, struct FSharedStruct>::IsDerived
			|| TIsDerivedFrom<T, struct FConstSharedStruct>::IsDerived, "This function is meant to compare contents of wrapped structs.");
	}

	/** Returns reference to the struct, this assumes that all data is valid. */
	template<typename T>
	T& GetStructRef(const UScriptStruct* ScriptStruct, void* StructMemory)
	{
		check(StructMemory != nullptr);
		check(ScriptStruct != nullptr);
		checkf(ScriptStruct == TBaseStructure<T>::Get() || ScriptStruct->IsChildOf(TBaseStructure<T>::Get()),
			TEXT("Incompatible struct types: Cannot cast '%s' to '%s'"),
			*ScriptStruct->GetPathName(),
			*TBaseStructure<T>::Get()->GetPathName());
		return *((T*)StructMemory);
	}

	/** Returns pointer to the struct, or nullptr if cast is not valid. */
	template<typename T>
	T* GetStructPtr(const UScriptStruct* ScriptStruct, void* StructMemory)
	{
		if (StructMemory != nullptr 
			&& ScriptStruct 
			&& (ScriptStruct == TBaseStructure<T>::Get()
				|| ScriptStruct->IsChildOf(TBaseStructure<T>::Get())))
		{
			return ((T*)StructMemory);
		}
		return nullptr;
	}

	/** Returns pointer to the struct, or nullptr if cast is not valid. */
	template<typename T, typename BaseStructT>
	T* GetStructPtr(const UScriptStruct* ScriptStruct, void* StructMemory)
	{
		if constexpr (std::is_same_v<BaseStructT, std::decay_t<T>>)
		{
#if WITH_EDITOR || !UE_BUILD_SHIPPING
			if (StructMemory && ScriptStruct)
			{
				// An extra guard to verify that ScriptStruct is actually safe to cast to T
				if (ensureAlwaysMsgf(ScriptStruct == TBaseStructure<T>::Get() || ScriptStruct->IsChildOf(TBaseStructure<T>::Get()),
					TEXT("Incompatible struct types: Cannot cast '%s' to '%s'. Falling back to return nullptr, but this may crash in shipping!"), *ScriptStruct->GetPathName(), *TBaseStructure<T>::Get()->GetPathName()))
				{
					return ((T*)StructMemory);
				}
			}
			return nullptr;
#else
			return ((T*)StructMemory);
#endif
		}
		else
		{
			return GetStructPtr<T>(ScriptStruct, StructMemory);
		}
	}

	/** Returns const reference to the struct, this assumes that all data is valid. */
	template<typename T>
	const T& GetStructRef(const UScriptStruct* ScriptStruct, const void* StructMemory)
	{
		return GetStructRef<T>(ScriptStruct, const_cast<void*>(StructMemory));
	}

	/** Returns const pointer to the struct, or nullptr if cast is not valid. */
	template<typename T>
	const T* GetStructPtr(const UScriptStruct* ScriptStruct, const void* StructMemory)
	{
		return GetStructPtr<T>(ScriptStruct, const_cast<void*>(StructMemory));
	}

	/** Returns const pointer to the struct, or nullptr if cast is not valid. */
	template<typename T, typename BaseStructT>
	const T* GetStructPtr(const UScriptStruct* ScriptStruct, const void* StructMemory)
	{
		return GetStructPtr<T, BaseStructT>(ScriptStruct, const_cast<void*>(StructMemory));
	}

	/** 
	 *  Returns the middle part of an array or view by taking up to the given number of elements from the given position.
	 *  Based on TArrayView::Mid().
	 */
	inline void CalcMidIndexAndCount(int32 ArrayNum, int32& InOutIndex, int32& InOutCount)
	{
		// Clamp minimum index at the start of the range, adjusting the length down if necessary
		const int32 NegativeIndexOffset = (InOutIndex < 0) ? InOutIndex : 0;
		InOutCount += NegativeIndexOffset;
		InOutIndex -= NegativeIndexOffset;

		// Clamp maximum index at the end of the range
		InOutIndex = (InOutIndex > ArrayNum) ? ArrayNum : InOutIndex;

		// Clamp count between 0 and the distance to the end of the range
		InOutCount = FMath::Clamp(InOutCount, 0, (ArrayNum - InOutIndex));
	}
	
	COREUOBJECT_API void AppendPropertyDescs(FXxHash64Builder& HashBuilder, const FPropertyBagPropertyDesc& Desc);

	COREUOBJECT_API uint64 CalcPropertyDescArrayHash(const TConstArrayView<FPropertyBagPropertyDesc> Descs);

	/**
	 * Return the numerical value (not in container).
	 * Works with property of type: bool, int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, enum.
	 */
	COREUOBJECT_API TValueOrError<int64, void> GetPropertyValueAsInt64(TNotNull<const FProperty*> InProperty, TNotNull<const void*> Address);
	COREUOBJECT_API TValueOrError<uint64, void> GetPropertyValueAsUInt64(TNotNull<const FProperty*> InProperty, TNotNull<const void*> Address);
	COREUOBJECT_API TValueOrError<double, void> GetPropertyValueAsDouble(TNotNull<const FProperty*> InProperty, TNotNull<const void*> Address);

	/**
	 * Set the numerical value (not in container).
	 * Works with property of type: bool, int8, uint8, int16, uint16, int32, uint32, int64, uint64, float, double, enum.
	 */
	COREUOBJECT_API bool SetPropertyValueFromInt64(TNotNull<const FProperty*> InProperty, TNotNull<void*> Address, int64 Value);
	COREUOBJECT_API bool SetPropertyValueFromUInt64(TNotNull<const FProperty*> InProperty, TNotNull<void*> Address, uint64 Value);
	COREUOBJECT_API bool SetPropertyValueFromDouble(TNotNull<const FProperty*> InProperty, TNotNull<void*> Address, double Value);
}
