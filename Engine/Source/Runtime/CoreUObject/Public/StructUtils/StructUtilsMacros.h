// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#ifndef WITH_STRUCTUTILS_DEBUG
#define WITH_STRUCTUTILS_DEBUG (!(UE_BUILD_SHIPPING || UE_BUILD_SHIPPING_WITH_EDITOR || UE_BUILD_TEST) && 1)
#endif // WITH_STRUCTUTILS_DEBUG
