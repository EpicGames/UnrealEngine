// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "UObject/Class.h"
#include "StructUtils/StructUtilsMacros.h"

#define UE_API COREUOBJECT_API

struct FSharedStruct;
struct FConstSharedStruct;
struct FStructView;
struct FConstStructView;
struct FInstancedPropertyBag;
struct FInstancedStruct;
class FReferenceCollector;
class UUserDefinedStruct;

namespace UE::StructUtils
{
	/**
	 * FArchiveCrc32-based struct hashing functions using object path for UObjects.
	 * i.e., Objects with same name will produce the same hash
	 */
	extern UE_API uint32 GetStructCrc32(const UScriptStruct& ScriptStruct, const uint8* StructMemory, const uint32 CRC = 0);
	extern UE_API uint32 GetStructCrc32(const FStructView& StructView, const uint32 CRC = 0);
	extern UE_API uint32 GetStructCrc32(const FConstStructView& StructView, const uint32 CRC = 0);
	extern UE_API uint32 GetStructCrc32(const FSharedStruct& SharedView, const uint32 CRC = 0);
	extern UE_API uint32 GetStructCrc32(const FConstSharedStruct& SharedView, const uint32 CRC = 0);

	/**
	 * FArchiveCrc32-based struct hashing functions using object key for UObjects.
	 * i.e., Generated hash will be unique per object instance
	 */
	extern UE_API uint32 GetStructInstanceCrc32(const UScriptStruct& ScriptStruct, const uint8* StructMemory, const uint32 CRC = 0);
	extern UE_API uint32 GetStructInstanceCrc32(const FStructView& StructView, const uint32 CRC = 0);
	extern UE_API uint32 GetStructInstanceCrc32(const FConstStructView& StructView, const uint32 CRC = 0);
	extern UE_API uint32 GetStructInstanceCrc32(const FSharedStruct& SharedView, const uint32 CRC = 0);
	extern UE_API uint32 GetStructInstanceCrc32(const FConstSharedStruct& SharedView, const uint32 CRC = 0);

	/** 
	 * CityHash64-based struct hashing functions.
	 * Note that these are relatively slow due to using either UScriptStruct.GetStructTypeHash (if implemented) or
	 * a serialization path. 
	 */
	extern UE_API uint64 GetStructHash64(const UScriptStruct& ScriptStruct, const uint8* StructMemory);
	extern UE_API uint64 GetStructHash64(const FStructView& StructView);
	extern UE_API uint64 GetStructHash64(const FConstStructView& StructView);
	extern UE_API uint64 GetStructHash64(const FSharedStruct& SharedView);
	extern UE_API uint64 GetStructHash64(const FConstSharedStruct& SharedView);

	template <typename T>
	auto* GetAsUStruct()
	{
		if constexpr (std::is_base_of_v<UObject, T>)
		{
			return T::StaticClass();
		}
		else
		{
			return T::StaticStruct();
		}
	}

	template<typename T>
	inline constexpr bool TIsSharedInstancedOrViewStruct_V = std::is_same_v<FStructView, T>
		|| std::is_same_v<FConstStructView, T>
		|| std::is_same_v<FSharedStruct, T>
		|| std::is_same_v<FConstSharedStruct, T>
		|| std::is_same_v<FInstancedStruct, T>;

	/**
	 * Whether an FInstancedPropertyBag and an UScriptStruct have exactly the same memory layout.
	 * The struct UPropertyBag might not be generated if there are no properties in the struct.
	 * In that case, it matches if the UScriptStruct also doesn’t have properties.
	 */
	extern UE_API bool TheSameLayout(const FInstancedPropertyBag& A, const UScriptStruct* B, bool bCheckPropertiesNames = false);
}

/* Predicate useful to find a struct of a specific type in an container */
struct FStructTypeEqualOperator
{
	const UScriptStruct* TypePtr;

	FStructTypeEqualOperator(const UScriptStruct* InTypePtr)
		: TypePtr(InTypePtr)
	{
	}

	template <typename T>
		requires (UE::StructUtils::TIsSharedInstancedOrViewStruct_V<T>)
	FStructTypeEqualOperator(const T& Struct)
		: TypePtr(Struct.GetScriptStruct())
	{
	}

	template <typename T>
		requires (UE::StructUtils::TIsSharedInstancedOrViewStruct_V<T>)
	bool operator()(const T& Struct) const
	{
		return Struct.GetScriptStruct() == TypePtr;
	}
};

struct FScriptStructSortOperator
{
	template <typename T>
	bool operator()(const T& A, const T& B) const
	{
		return (A.GetStructureSize() > B.GetStructureSize())
			|| (A.GetStructureSize() == B.GetStructureSize() && B.GetFName().FastLess(A.GetFName()));
	}
};

struct FStructTypeSortOperator
{
	template <typename T>
		requires (UE::StructUtils::TIsSharedInstancedOrViewStruct_V<T>)
	bool operator()(const T& A, const T& B) const
	{
		const UScriptStruct* AScriptStruct = A.GetScriptStruct();
		const UScriptStruct* BScriptStruct = B.GetScriptStruct();
		if (!AScriptStruct)
		{
			return true;
		}
		else if (!BScriptStruct)
		{
			return false;
		}
		return FScriptStructSortOperator()(*AScriptStruct, *BScriptStruct);
	}
};

#if WITH_EDITOR
namespace UE::StructUtils::Private
{
	// Private structs used during user defined struct reinstancing.
	struct UE_DEPRECATED(5.8, "Use FStructReinstancer instead.") FStructureToReinstantiateScope
	{
		explicit FStructureToReinstantiateScope(const UUserDefinedStruct* StructureToReinstantiate) {}
	};

	struct UE_DEPRECATED(5.8, "Use FStructReinstancer instead.") FCurrentReinstantiationOuterObjectScope
	{
		explicit FCurrentReinstantiationOuterObjectScope(UObject* CurrentReinstantiateOuterObject) {}
	};

	UE_DEPRECATED(5.8, "Use FStructReinstancer instead.")
	static const UUserDefinedStruct* GetStructureToReinstantiate()
	{
		return nullptr;
	}

	UE_DEPRECATED(5.8, "Use FStructReinstancer instead.")
	static UObject* GetCurrentReinstantiationOuterObject()
	{
		return nullptr;
	}
};
#endif // WITH_EDITOR

#undef UE_API
