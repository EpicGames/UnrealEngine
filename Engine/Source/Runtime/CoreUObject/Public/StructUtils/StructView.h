// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "StructUtils/SharedStruct.h"
#include "StructUtils.h"
#include <type_traits>

///////////////////////////////////////////////////////////////// FStructView /////////////////////////////////////////////////////////////////

/**
 * FStructView is "typed" struct pointer, it contains pointer to struct plus UScriptStruct pointer.
 * FStructView does not own the memory and will not free it when out of scope.
 * It should be only used to pass struct pointer in a limited scope, or when the user controls the lifetime of the struct being stored.
 * E.g. instead of passing ref or pointer to a FInstancedStruct, you should use FConstStructView or FStructView to pass around a view to the contents.
 * FStructView is passed by value.
 * FStructView is similar to FStructOnScope, but FStructView is a view only (FStructOnScope can either own the memory or be a view)
 * const FStructView prevents the struct from pointing at a different instance of a struct. However the actual struct 
 * data can be mutated. Use FConstStructView to prevent mutation of the actual struct data.
 * See FConstStructView for examples.
 */
struct [[nodiscard]] FStructView
{
public:

	FStructView() = default;

	FStructView(const UScriptStruct* InScriptStruct, uint8* InStructMemory = nullptr)
		: ScriptStruct(InScriptStruct)
		, StructMemory(InStructMemory)
	{}

	FStructView(FInstancedStruct& InstancedStruct)
		: FStructView(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMutableMemory())
	{}

	FStructView(const FSharedStruct& SharedStruct)
		: FStructView(SharedStruct.GetScriptStruct(), SharedStruct.GetMemory())
	{}

	/** Creates a new FStructView from the templated struct. Note its not safe to make InStruct const ref as the original object may have been declared const */
	template<typename T>
	static FStructView Make(T& InStruct)
	{
		UE::StructUtils::CheckStructType<T>();
		return FStructView(TBaseStructure<T>::Get(), reinterpret_cast<uint8*>(&InStruct));
	}

	/** Returns mutable reference to the struct, this getter assumes that all data is valid. */
	template<typename T>
	T& Get() const
	{
		return UE::StructUtils::GetStructRef<T>(ScriptStruct, StructMemory);
	}

	/** Returns mutable pointer to the struct, or nullptr if cast is not valid. */
	template<typename T>
	T* GetPtr() const
	{
		return UE::StructUtils::GetStructPtr<T>(ScriptStruct, StructMemory);
	}

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const
	{
		return ScriptStruct;
	}

	/** Returns pointer to struct memory. */
	uint8* GetMemory() const
	{
		return StructMemory;
	}

	/** Reset to empty. */
	void Reset()
	{
		StructMemory = nullptr;
		ScriptStruct = nullptr;
	}

	/** Returns True if the struct is valid.*/
	bool IsValid() const
	{
		return StructMemory != nullptr && ScriptStruct != nullptr;
	}

	/** Comparison operators. Note: it does not compare the internal structure itself */
	template <typename OtherType>
	bool operator==(const OtherType& Other) const
	{
		return ((ScriptStruct == Other.GetScriptStruct()) && (StructMemory == Other.GetMemory()));
	}

	template <typename OtherType>
	bool operator!=(const OtherType& Other) const
	{
		return !operator==(Other);
	}

protected:
	const UScriptStruct* ScriptStruct = nullptr;
	uint8* StructMemory = nullptr;
};

/**
 * TStructView is a type-safe FStructView wrapper against the given BaseStruct type.
 * 
 * Example:
 *
 *	TStructView<FTestStructBase> Test;
 *
 *	TArray<TStructView<FTestStructBase>> TestArray;
 */
template<typename BaseStructT>
struct [[nodiscard]] TStructView : FStructView
{
public:

	explicit TStructView() = default;

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TStructView(T& InStruct)
		: FStructView(T::StaticStruct(), reinterpret_cast<uint8*>(&InStruct))
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	explicit TStructView(uint8* InStructMemory = nullptr)
		: FStructView(T::StaticStruct(), InStructMemory)
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	explicit TStructView(const UScriptStruct* InScriptStruct, uint8* InStructMemory = nullptr)
		: FStructView(InScriptStruct, InStructMemory)
	{
		checkf(!InScriptStruct || InScriptStruct->IsChildOf(TBaseStructure<T>::Get()), TEXT("Struct type must be a child of T!"));
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TStructView(TInstancedStruct<T>& InstancedStruct)
		: FStructView(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMutableMemory())
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TStructView(const TSharedStruct<T>& SharedStruct)
		: FStructView(SharedStruct.GetScriptStruct(), SharedStruct.GetMemory())
	{
	}

	/** Returns mutable reference to the struct, this getter assumes that all data is valid. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	T& Get() const
	{
		return UE::StructUtils::GetStructRef<T>(ScriptStruct, StructMemory);
	}

	/** Returns mutable pointer to the struct, or nullptr if cast is not valid. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	T* GetPtr() const
	{
		return UE::StructUtils::GetStructPtr<T, BaseStructT>(ScriptStruct, StructMemory);
	}

	/** Simplified accessor for members of the guaranteed minimum structure type */
	BaseStructT* operator->() const
	{
		check(IsValid());
		return GetPtr();
	}

	/** Comparison operators. Note: it does not compare the internal structure itself */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator==(const TStructView<T>& Other) const
	{
		return ((ScriptStruct == Other.GetScriptStruct()) && (StructMemory == Other.GetMemory()));
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator!=(const TStructView<T>& Other) const
	{
		return !operator==(Other);
	}
};

///////////////////////////////////////////////////////////////// FConstStructView /////////////////////////////////////////////////////////////////
/**
 * FConstStructView is "typed" struct pointer, it contains const pointer to struct plus UScriptStruct pointer.
 * FConstStructView does not own the memory and will not free it when of scope.
 * It should be only used to pass struct pointer in a limited scope, or when the user controls the lifetime of the struct being stored.
 * E.g. instead of passing ref or pointer to a FInstancedStruct, you should use FConstStructView or FStructView to pass around a view to the contents.
 * FConstStructView is passed by value.
 * FConstStructView is similar to FStructOnScope, but FConstStructView is a view only (FStructOnScope can either own the memory or be a view)
 * FConstStructView prevents mutation of the actual struct data however the struct being pointed at can be changed to point at a different instance
 * of a struct. To also prevent this use const FConstStructView.
 * e.g. FStructView A; FConstStructView B = A; // compiles 
 * e.g. FStructView A; FConstStructView B; B = A; // compiles as B can be made to point at any other StructView
 * e.g. FConstStructView A; FStructView B = A; // doesn't compile as the struct data for A is immutable (but mutable for B)
 * e.g. FStructView A; A.Foo = NewVal; // compiles as the struct data for A is mutable.
 * e.g. FConstStructView A; A.Foo = NewVal; // doesn't compile as the struct data for A is immutable.
 * e.g. FStructView A; const FStructView B; A = B; // compiles as the struct B is pointing to can't be made to point at something else but A isn't const.
 * e.g. const FStructView A; FStructView B; A = B; // doesn't compile as attempting to make const view point at something else
 */
struct [[nodiscard]] FConstStructView
{
public:

	FConstStructView() = default;

	FConstStructView(const UScriptStruct* InScriptStruct, const uint8* InStructMemory = nullptr)
		: ScriptStruct(InScriptStruct)
		, StructMemory(InStructMemory)
	{}

	FConstStructView(const FInstancedStruct& InstancedStruct)
		: FConstStructView(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMemory())
	{}

	FConstStructView(const FSharedStruct& SharedStruct)
		: FConstStructView(SharedStruct.GetScriptStruct(), SharedStruct.GetMemory())
	{}

	FConstStructView(const FConstSharedStruct& SharedStruct)
		: FConstStructView(SharedStruct.GetScriptStruct(), SharedStruct.GetMemory())
	{}

	FConstStructView(const FStructView StructView)
		: FConstStructView(StructView.GetScriptStruct(), StructView.GetMemory())

	{}

	/** Creates a new FConstStructView from the templated struct */
	template<typename T>
	static FConstStructView Make(const T& Struct)
	{
		UE::StructUtils::CheckStructType<T>();
		return FConstStructView(TBaseStructure<T>::Get(), reinterpret_cast<const uint8*>(&Struct));
	}

	/** Returns const reference to the struct, this getter assumes that all data is valid. */
	template<typename T>
	constexpr const T& Get() const
	{
		return UE::StructUtils::GetStructRef<T>(ScriptStruct, StructMemory);
	}

	/** Returns const pointer to the struct, or nullptr if cast is not valid. */
	template<typename T>
	constexpr const T* GetPtr() const
	{
		return UE::StructUtils::GetStructPtr<T>(ScriptStruct, StructMemory);
	}

	/** Returns struct type. */
	const UScriptStruct* GetScriptStruct() const
	{
		return ScriptStruct;
	}

	/** Returns const pointer to struct memory. */
	const uint8* GetMemory() const
	{
		return StructMemory;
	}

	/** Reset to empty. */
	void Reset()
	{
		StructMemory = nullptr;
		ScriptStruct = nullptr;
	}

	/** Returns True if the struct is valid.*/
	bool IsValid() const
	{
		return StructMemory != nullptr && ScriptStruct != nullptr;
	}

	/** Comparison operators. Note: it does not compare the internal structure itself */
	template <typename OtherType>
	bool operator==(const OtherType& Other) const
	{
		return ((ScriptStruct == Other.GetScriptStruct()) && (StructMemory == Other.GetMemory()));
	}

	template <typename OtherType>
	bool operator!=(const OtherType& Other) const
	{
		return !operator==(Other);
	}

	void SetStructData(const UScriptStruct* InScriptStruct, const uint8* InStructMemory)
	{
		ScriptStruct = InScriptStruct;
		StructMemory = InStructMemory;
	}

protected:
	const UScriptStruct* ScriptStruct = nullptr;
	const uint8* StructMemory = nullptr;
};

/**
 * TConstStructView is a type-safe FConstStructView wrapper against the given BaseStruct type.
 * 
 * Example:
 *
 *	TConstStructView<FTestStructBase> Test;
 *
 *	TArray<TConstStructView<FTestStructBase>> TestArray;
 */
template<typename BaseStructT>
struct [[nodiscard]] TConstStructView : FConstStructView
{
	explicit TConstStructView() = default;

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstStructView(const T& InStruct)
		: FConstStructView(T::StaticStruct(), reinterpret_cast<const uint8*>(&InStruct))
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	explicit TConstStructView(const uint8* InStructMemory = nullptr)
		: FConstStructView(T::StaticStruct(), InStructMemory)
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstStructView(const TInstancedStruct<T>& InstancedStruct)
		: FConstStructView(InstancedStruct.GetScriptStruct(), InstancedStruct.GetMemory())
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstStructView(const TSharedStruct<T>& SharedStruct)
		: FConstStructView(SharedStruct.GetScriptStruct(), SharedStruct.GetMemory())
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstStructView(const TConstSharedStruct<T>& SharedStruct)
		: FConstStructView(SharedStruct.GetScriptStruct(), SharedStruct.GetMemory())
	{
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	TConstStructView(const TStructView<T> StructView)
		: FConstStructView(StructView.GetScriptStruct(), StructView.GetMemory())
	{
	}

	/** Returns const reference to the struct, this getter assumes that all data is valid. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	constexpr const T& Get() const
	{
		return UE::StructUtils::GetStructRef<T>(ScriptStruct, StructMemory);
	}

	/** Returns const pointer to the struct, or nullptr if cast is not valid. */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	constexpr const T* GetPtr() const
	{
		return UE::StructUtils::GetStructPtr<T, BaseStructT>(ScriptStruct, StructMemory);
	}

	/** Simplified accessor for members of the guaranteed minimum structure type */
	const BaseStructT* operator->() const
	{
		check(IsValid());
		return GetPtr();
	}

	/** Comparison operators. Note: it does not compare the internal structure itself */
	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator==(const TConstStructView<T>& Other) const
	{
		return ((ScriptStruct == Other.GetScriptStruct()) && (StructMemory == Other.GetMemory()));
	}

	template<typename T = BaseStructT>
		requires (std::is_base_of_v<BaseStructT, std::decay_t<T>>)
	bool operator!=(const TConstStructView<T>& Other) const
	{
		return !operator==(Other);
	}
};
