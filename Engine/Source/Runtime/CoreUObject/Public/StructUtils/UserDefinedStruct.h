// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/StructOnScope.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/SubclassOf.h"

#include "UserDefinedStruct.generated.h"

class UUserDefinedStruct;
class UUserDefinedStructEditorData;
class UStructCookedMetaData;
struct FUserDefinedStructureCompilerInner;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnStructChanged, UUserDefinedStruct*);

UENUM()
enum EUserDefinedStructureStatus : int
{
	/** Struct is in an unknown state. */
	UDSS_UpToDate,
	/** Struct has been modified but not recompiled. */
	UDSS_Dirty,
	/** Struct tried but failed to be compiled. */
	UDSS_Error,
	/** Struct is a duplicate, the original one was changed. */
	UDSS_Duplicate,

	UDSS_MAX,
};

/** Wrapper for StructOnScope that tells it to ignore default values */
class FUserStructOnScopeIgnoreDefaults : public FStructOnScope
{
public:
	/** Constructor with no script struct, call Recreate later */
	FUserStructOnScopeIgnoreDefaults() : FStructOnScope() {}

	/** Constructor that initializes for you */
	COREUOBJECT_API FUserStructOnScopeIgnoreDefaults(const UUserDefinedStruct* InUserStruct);

	/** Initialize from existing data, will free when scope closes */
	COREUOBJECT_API FUserStructOnScopeIgnoreDefaults(const UUserDefinedStruct* InUserStruct, uint8* InData);

	/** Destroys and creates new struct */
	COREUOBJECT_API void Recreate(const UUserDefinedStruct* InUserStruct);

	COREUOBJECT_API virtual void Initialize() override;
};

namespace UE::StructUtils
{
	USTRUCT()
	struct FUserDefinedPropertyID
	{
		GENERATED_BODY()

		UPROPERTY()
		FGuid ID;
		UPROPERTY()
		FName Name;
	};
} // namespace

UCLASS(MinimalAPI)
class UUserDefinedStruct : public UScriptStruct
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITORONLY_DATA
	/** The original struct, when current struct isn't a temporary duplicate, the field should be null */
	UPROPERTY(Transient)
	TWeakObjectPtr<UUserDefinedStruct> PrimaryStruct;

	UPROPERTY()
	FString ErrorMessage;

	UPROPERTY()
	TObjectPtr<UObject> EditorData;
#endif // WITH_EDITORONLY_DATA

	/** Status of this struct, outside of the editor it is assumed to always be UpToDate */
	UPROPERTY()
	TEnumAsByte<enum EUserDefinedStructureStatus> Status;

	/** Uniquely identifies this specific user struct */
	UPROPERTY(AssetRegistrySearchable)
	FGuid Guid;

private:
	UPROPERTY()
	TArray<UE::StructUtils::FUserDefinedPropertyID> PropertyIDs;

protected:
	/** Default instance of this struct with default values filled in, used to initialize structure */
	FUserStructOnScopeIgnoreDefaults DefaultStructInstance;

	/** Bool to indicate we want to initialize a version of this struct without defaults, this is set while allocating the DefaultStructInstance itself */
	bool bIgnoreStructDefaults;

public:
#if WITH_EDITOR
	//~ UObject interface.
	COREUOBJECT_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	COREUOBJECT_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	COREUOBJECT_API virtual void PostLoad() override;
	COREUOBJECT_API virtual void PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext) override;
	COREUOBJECT_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext) override;
	//~ End of UObject interface.

	/** Creates a new guid if needed */
	COREUOBJECT_API void ValidateGuid();

	COREUOBJECT_API virtual void OnChanged();

	friend UUserDefinedStructEditorData;
	friend FUserDefinedStructureCompilerInner;
#endif	// WITH_EDITOR

	//~ UObject interface.
	COREUOBJECT_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	COREUOBJECT_API virtual void SerializeTaggedProperties(FStructuredArchive::FSlot Slot, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad = nullptr) const override;
	COREUOBJECT_API virtual FString GetAuthoredNameForField(const FField* Field) const override;
	//~ End of UObject interface.

	//~ UScriptStruct interface.
	COREUOBJECT_API virtual void InitializeStruct(void* Dest, int32 ArrayDim = 1) const override;
	COREUOBJECT_API virtual uint32 GetStructTypeHash(const void* Src) const override;
	COREUOBJECT_API virtual void RecursivelyPreload() override;
	COREUOBJECT_API virtual FGuid GetCustomGuid() const override;
	COREUOBJECT_API virtual FString GetStructCPPName(uint32 CPPExportFlags) const override;
	COREUOBJECT_API virtual FProperty* CustomFindProperty(const FName Name) const override;
	COREUOBJECT_API virtual void PrepareCppStructOps() override;
	virtual bool ArePropertyGuidsAvailable() const override
	{
		return PropertyIDs.Num() > 0;
	}
	COREUOBJECT_API virtual FName FindPropertyNameFromGuid(const FGuid& PropertyID) const override;
	COREUOBJECT_API virtual FGuid FindPropertyGuidFromName(const FName PropertyName) const override;
	//~ End of  UScriptStruct interface.

	/** Returns the raw memory of the default instance */
	COREUOBJECT_API const uint8* GetDefaultInstance() const;

	/** Specifically initialize this struct without using the default instance data */
	COREUOBJECT_API void InitializeStructIgnoreDefaults(void* Dest, int32 ArrayDim = 1) const;

	/** Computes hash */
	static COREUOBJECT_API uint32 GetUserDefinedStructTypeHash(const void* Src, const UScriptStruct* Type);

	/** returns references from default instance */
	static COREUOBJECT_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/** Inspects properties and default values, setting appropriate StructFlags */
	COREUOBJECT_API void UpdateStructFlags();

#if WITH_EDITORONLY_DATA
	UE_FORCEINLINE_HINT FOnStructChanged& OnStructChanged() { return ChangedEvent; }
public:
	FOnStructChanged ChangedEvent;

protected:
	COREUOBJECT_API virtual TSubclassOf<UStructCookedMetaData> GetCookedMetaDataClass() const;

private:
	COREUOBJECT_API UStructCookedMetaData* NewCookedMetaData();
	COREUOBJECT_API const UStructCookedMetaData* FindCookedMetaData();
	COREUOBJECT_API void PurgeCookedMetaData();

	UPROPERTY()
	TObjectPtr<UStructCookedMetaData> CachedCookedMetaDataPtr;
#endif // WITH_EDITORONLY_DATA
};
