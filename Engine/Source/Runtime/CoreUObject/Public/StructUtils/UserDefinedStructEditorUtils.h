// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR
#include "StructUtils/UserDefinedStruct.h"
#endif // WITH_EDITOR
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "UserDefinedStructEditorUtils.generated.h"

UCLASS(MinimalAPI, Abstract)
class UUserDefinedStructEditorDataBase : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	virtual void RecreateDefaultInstance(FString* OutLog = nullptr)
	{		
	}
	
	virtual void ReinitializeDefaultInstance(FString* OutLog = nullptr)
	{		
	}
	
	virtual FProperty* FindProperty(const UUserDefinedStruct* Struct, FName Name) const
	{
		return nullptr;
	}

	virtual FString GetFriendlyNameForProperty(const UUserDefinedStruct* Struct, const FProperty* Property) const
	{
		return {};
	}

	virtual FString GetTooltip() const
	{
		return {};
	}
#endif // WITH_EDITOR
};

#if WITH_EDITOR
struct FUserDefinedStructEditorUtils
{
	// NOTIFICATION
	DECLARE_DELEGATE_OneParam(FOnUserDefinedStructChanged, UUserDefinedStruct*);
	static COREUOBJECT_API FOnUserDefinedStructChanged OnUserDefinedStructChanged;

	/** called after UDS was changed by editor*/
	static COREUOBJECT_API void OnStructureChanged(UUserDefinedStruct* Struct);
	
	// VALIDATION
	enum EStructureError
	{
		Ok, 
		Recursion,
		FallbackStruct,
		NotCompiled,
		NotBlueprintType,
		NotSupportedType,
		EmptyStructure
	};

	/** Can the structure be a member variable for a BPGClass or BPGStruct */
	static COREUOBJECT_API EStructureError IsStructureValid(const UScriptStruct* Struct, const UStruct* RecursionParent = nullptr, FString* OutMsg = nullptr);
};
#endif // WITH_EDITOR
