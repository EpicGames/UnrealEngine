// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/NotNull.h"
#include "Templates/LosesQualifiersFromTo.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectHandle.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include <type_traits>

#define UE_USE_CAST_FLAGS (USTRUCT_FAST_ISCHILDOF_IMPL != USTRUCT_ISCHILDOF_STRUCTARRAY)

#ifndef UE_ENABLE_UNRELATED_CAST_WARNINGS
#define UE_ENABLE_UNRELATED_CAST_WARNINGS 1
#endif

class AActor;
class APawn;
class APlayerController;
class FField;
class FSoftClassProperty;
class UBlueprint;
class ULevel;
class UPrimitiveComponent;
class USceneComponent;
class USkeletalMeshComponent;
class USkinnedMeshComponent;
class UStaticMeshComponent;
/// @cond DOXYGEN_WARNINGS
template<class TClass> class TSubclassOf;
/// @endcond

[[noreturn]] COREUOBJECT_API void CastLogError(const TCHAR* FromType, const TCHAR* ToType);

/**
 * Metafunction which detects whether or not a class is an IInterface.  Rules:
 *
 * 1. A UObject is not an IInterface.
 * 2. A type without a UClassType typedef member is not an IInterface.
 * 3. A type whose UClassType::StaticClassFlags does not have CLASS_Interface set is not an IInterface.
 *
 * Otherwise, assume it's an IInterface.
 */
template <typename T, bool bIsAUObject_IMPL = std::is_convertible_v<T*, const volatile UObject*>>
struct TIsIInterface
{
	enum { Value = false };
};

template <typename T>
struct TIsIInterface<T, false>
{
	template <typename U> static char (&Resolve(typename U::UClassType*))[(U::UClassType::StaticClassFlags & CLASS_Interface) ? 2 : 1];
	template <typename U> static char (&Resolve(...))[1];

	enum { Value = sizeof(Resolve<T>(0)) - 1 };
};

template <typename T>
inline FString GetTypeName()
{
	if constexpr (TIsIInterface<T>::Value)
	{
		return T::UClassType::StaticClass()->GetName();
	}
	else
	{
		return T::StaticClass()->GetName();
	}
}

namespace UE::CoreUObject::Private
{
	template <typename Type>
	constexpr inline EClassCastFlags TCastFlags_V = CASTCLASS_None;

	template <typename Type>
	constexpr inline EClassCastFlags TCastFlags_V<const Type> = TCastFlags_V<Type>;
}

// Dynamically cast an object type-safely.
template <typename To, typename From>
inline TCopyQualifiersFromTo_T<From, To>* Cast(From* Src)
{
	static_assert(sizeof(From) > 0 && sizeof(To) > 0, "Attempting to cast between incomplete types");

	if (Src)
	{
		if constexpr (TIsIInterface<From>::Value)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (UObject* Obj = Src->_getUObject())
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				if constexpr (TIsIInterface<To>::Value)
				{
					return (To*)Obj->GetInterfaceAddress(To::UClassType::StaticClass());
				}
				else
				{
					if constexpr (std::is_same_v<To, UObject>)
					{
						return Obj;
					}
					else
					{
						if (Obj->IsA<To>())
						{
							return (To*)Obj;
						}
					}
				}
			}
		}
		else
		{
			static_assert(std::is_base_of_v<UObjectBase, From>, "Attempting to use Cast<> on a type that is not a UObject or an Interface");

			if constexpr (UE_USE_CAST_FLAGS && UE::CoreUObject::Private::TCastFlags_V<To> != CASTCLASS_None)
			{
				if constexpr (std::is_base_of_v<To, From>)
				{
					return (To*)Src;
				}
				else
				{
#if UE_ENABLE_UNRELATED_CAST_WARNINGS
					UE_STATIC_ASSERT_WARN((std::is_base_of_v<From, To>), "Attempting to use Cast<> on types that are not related");
#endif
					if (((const UObject*)Src)->GetClass()->HasAnyCastFlag(UE::CoreUObject::Private::TCastFlags_V<To>))
					{
						return (To*)Src;
					}
				}
			}
			else
			{
				if constexpr (TIsIInterface<To>::Value)
				{
					return (To*)((UObject*)Src)->GetInterfaceAddress(To::UClassType::StaticClass());
				}
				else if constexpr (std::is_base_of_v<To, From>)
				{
					return Src;
				}
				else
				{
#if UE_ENABLE_UNRELATED_CAST_WARNINGS
					UE_STATIC_ASSERT_WARN((std::is_base_of_v<From, To>), "Attempting to use Cast<> on types that are not related");
#endif
					if (((const UObject*)Src)->IsA<To>())
					{
						return (To*)Src;
					}
				}
			}
		}
	}

	return nullptr;
}

template <typename To, typename From>
UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* ExactCast(From* Src)
{
	return Src && (Src->GetClass() == To::StaticClass()) ? (To*)Src : nullptr;
}

#if DO_CHECK

	// Helper function to get the full name for UObjects and UInterfaces
	template <typename T>
	FString GetFullNameForCastLogError(T* InObjectOrInterface)
	{
		// A special version for FFields
		if constexpr (std::is_base_of_v<FField, T>)
		{
			return GetFullNameSafe(InObjectOrInterface);
		}
		else if constexpr (std::is_base_of_v<UObject, T>)
		{
			return InObjectOrInterface->GetFullName();;
		}
		else
		{
			return Cast<UObject>(InObjectOrInterface)->GetFullName();
		}
	}

	template <typename To, typename From>
	FUNCTION_NON_NULL_RETURN_START
		TCopyQualifiersFromTo_T<From, To>* CastChecked(From* Src)
	FUNCTION_NON_NULL_RETURN_END
	{
		static_assert(sizeof(From) > 0 && sizeof(To) > 0, "Attempting to cast between incomplete types");

		if (!Src)
		{
			CastLogError(TEXT("nullptr"), *GetTypeName<To>());
		}

		TCopyQualifiersFromTo_T<From, To>* Result = Cast<To>(Src);
		if (!Result)
		{
			CastLogError(*GetFullNameForCastLogError(Src), *GetTypeName<To>());
		}

		return Result;
	}

	template <typename To, typename From>
	TCopyQualifiersFromTo_T<From, To>* CastChecked(From* Src, ECastCheckedType::Type CheckType)
	{
		static_assert(sizeof(From) > 0 && sizeof(To) > 0, "Attempting to cast between incomplete types");

		if (Src)
		{
			TCopyQualifiersFromTo_T<From, To>* Result = Cast<To>(Src);
			if (!Result)
			{
				CastLogError(*GetFullNameForCastLogError(Src), *GetTypeName<To>());
			}

			return Result;
		}

		if (CheckType == ECastCheckedType::NullChecked)
		{
			CastLogError(TEXT("nullptr"), *GetTypeName<To>());
		}

		return nullptr;
	}

#else

	template <typename To, typename From>
	FUNCTION_NON_NULL_RETURN_START
		inline TCopyQualifiersFromTo_T<From, To>* CastChecked(From* Src)
	FUNCTION_NON_NULL_RETURN_END
	{
		static_assert(sizeof(From) > 0 && sizeof(To) > 0, "Attempting to cast between incomplete types");

		if constexpr (TIsIInterface<From>::Value)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UObject* Obj = Src->_getUObject();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if constexpr (TIsIInterface<To>::Value)
			{
				return (To*)Obj->GetInterfaceAddress(To::UClassType::StaticClass());
			}
			else
			{
				return (To*)Obj;
			}
		}
		else
		{
			static_assert(std::is_base_of_v<UObjectBase, From>, "Attempting to use Cast<> on a type that is not a UObject or an Interface");

			if constexpr (TIsIInterface<To>::Value)
			{
				return (To*)((UObject*)Src)->GetInterfaceAddress(To::UClassType::StaticClass());
			}
			else
			{
				return (To*)Src;
			}
		}
	}

	template <typename To, typename From>
	UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* CastChecked(From* Src, ECastCheckedType::Type CheckType)
	{
		if (CheckType == ECastCheckedType::NullAllowed && !Src)
		{
			return nullptr;
		}

		return CastChecked<To>(Src);
	}

#endif

// auto weak versions
template <typename To, typename From>
UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* Cast(const TWeakObjectPtr<From>& Src)
{
	return Cast<To>(Src.Get());
}
template <typename To, typename From>
UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* ExactCast(const TWeakObjectPtr<From>& Src)
{
	return ExactCast  <To>(Src.Get());
}
template <typename To, typename From>
UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* CastChecked(const TWeakObjectPtr<From>& Src)
{
	return CastChecked<To>(Src.Get());
}
template <typename To, typename From>
UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* CastChecked(const TWeakObjectPtr<From>& Src, ECastCheckedType::Type CheckType)
{
	return CastChecked<To>(Src.Get(), CheckType);
}

// object ptr versions
template <typename To, typename From>
inline TCopyQualifiersFromTo_T<From, To>* Cast(const TObjectPtr<From>& InSrc)
{
	static_assert(sizeof(To) > 0 && sizeof(From) > 0, "Attempting to cast between incomplete types");

	const FObjectPtr& Src = (const FObjectPtr&)InSrc;

	if constexpr (UE_USE_CAST_FLAGS && UE::CoreUObject::Private::TCastFlags_V<To> != CASTCLASS_None)
	{
		if (Src)
		{
			if constexpr (std::is_base_of_v<To, From>)
			{
				return (To*)Src.Get();
			}
			else
			{
	#if UE_ENABLE_UNRELATED_CAST_WARNINGS
				UE_STATIC_ASSERT_WARN((std::is_base_of_v<From, To>), "Attempting to use Cast<> on types that are not related");
	#endif
				if (Src.GetClass()->HasAnyCastFlag(UE::CoreUObject::Private::TCastFlags_V<To>))
				{
					return (To*)Src.Get();
				}
			}
		}
	}
	else if constexpr (TIsIInterface<To>::Value)
	{
		const UObject* SrcObj = UE::CoreUObject::Private::ResolveObjectHandleNoRead(Src.GetHandleRef());
		if (SrcObj)
		{
			UE::CoreUObject::Private::OnHandleRead(SrcObj);
			return (To*)Src.Get()->GetInterfaceAddress(To::UClassType::StaticClass());
		}
	}
	else if constexpr (std::is_base_of_v<To, From>)
	{
		if (Src)
		{
			return (To*)Src.Get();
		}
	}
	else
	{
#if UE_ENABLE_UNRELATED_CAST_WARNINGS
		UE_STATIC_ASSERT_WARN((std::is_base_of_v<From, To>), "Attempting to use Cast<> on types that are not related");
#endif
		if (Src && Src.IsA<To>())
		{
			return (To*)Src.Get();
		}
	}

	return nullptr;
}

template <typename To, typename From>
inline TCopyQualifiersFromTo_T<From, To>* ExactCast(const TObjectPtr<From>& Src)
{
	static_assert(sizeof(To) > 0, "Attempting to cast to an incomplete type");

	UObject* SrcObj = UE::CoreUObject::Private::ResolveObjectHandleNoRead(((const FObjectPtr&)Src).GetHandleRef());
	if (SrcObj && (SrcObj->GetClass() == To::StaticClass()))
	{
		UE::CoreUObject::Private::OnHandleRead(SrcObj);
		return (To*)SrcObj;
	}
	return nullptr;
}

template <typename To, typename From>
inline TCopyQualifiersFromTo_T<From, To>* CastChecked(const TObjectPtr<From>& Src, ECastCheckedType::Type CheckType = ECastCheckedType::NullChecked)
{
	static_assert(sizeof(From) > 0 && sizeof(To) > 0, "Attempting to cast between incomplete types");

#if DO_CHECK
	if (Src)
	{
	auto* Result = Cast<To>(Src);
	if (!Result)
	{
			CastLogError(*GetFullNameForCastLogError(Src.Get()), *GetTypeName<To>());
	}

	return Result;
}

	if (CheckType == ECastCheckedType::NullChecked)
	{
		CastLogError(TEXT("nullptr"), *GetTypeName<To>());
	}

	return nullptr;
#else
	if constexpr (UE_USE_CAST_FLAGS && UE::CoreUObject::Private::TCastFlags_V<To> != CASTCLASS_None)
	{
		return (To*)((const FObjectPtr&)Src).Get();
	}
	else if constexpr (TIsIInterface<To>::Value)
	{
		UObject* SrcObj = UE::CoreUObject::Private::ResolveObjectHandleNoRead(((const FObjectPtr&)Src).GetHandleRef());
		UE::CoreUObject::Private::OnHandleRead(SrcObj);
		return (To*)((const FObjectPtr&)Src).Get()->GetInterfaceAddress(To::UClassType::StaticClass());
	}
	else
	{
		return (To*)((const FObjectPtr&)Src).Get();
	}
#endif
}

// TSubclassOf versions
template <typename To, typename From>
UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* Cast(const TSubclassOf<From>& Src)
{
	return Cast<To>(*Src);
}
template <typename To, typename From>
UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* CastChecked(const TSubclassOf<From>& Src)
{
	return CastChecked<To>(*Src);
}
template <typename To, typename From>
UE_FORCEINLINE_HINT TCopyQualifiersFromTo_T<From, To>* CastChecked(const TSubclassOf<From>& Src, ECastCheckedType::Type CheckType)
{
	return CastChecked<To>(*Src, CheckType);
}

// TNotNull versions of the casts
#if UE_ENABLE_NOTNULL_WRAPPER
	template <typename T, typename U>
	UE_FORCEINLINE_HINT auto Cast(TNotNull<U> Ptr) -> decltype(Cast<T>((U)Ptr))
	{
		return Cast<T>((U)Ptr);
	}
	template <typename T, typename U>
	UE_FORCEINLINE_HINT auto ExactCast(TNotNull<U> Ptr) -> decltype(ExactCast<T>((U)Ptr))
	{
		return ExactCast<T>((U)Ptr);
	}
	template <typename T, typename U>
	UE_FORCEINLINE_HINT auto CastChecked(TNotNull<U> Ptr) -> decltype(CastChecked<T>((U)Ptr))
	{
		return CastChecked<T>((U)Ptr);
	}
	template <typename T, typename U>
	UE_FORCEINLINE_HINT auto CastChecked(TNotNull<U> Ptr, ECastCheckedType::Type CheckType) -> decltype(CastChecked<T>((U)Ptr, CheckType))
	{
		return CastChecked<T>((U)Ptr, CheckType);
	}
#endif

#define DECLARE_CAST_BY_FLAG(ClassName) \
	class ClassName; \
	template <> \
	constexpr inline EClassCastFlags UE::CoreUObject::Private::TCastFlags_V<ClassName> = CASTCLASS_##ClassName;

DECLARE_CAST_BY_FLAG(UField)
DECLARE_CAST_BY_FLAG(UEnum)
DECLARE_CAST_BY_FLAG(UStruct)
DECLARE_CAST_BY_FLAG(UScriptStruct)
DECLARE_CAST_BY_FLAG(UClass)
DECLARE_CAST_BY_FLAG(FProperty)
DECLARE_CAST_BY_FLAG(FObjectPropertyBase)
DECLARE_CAST_BY_FLAG(FObjectProperty)
DECLARE_CAST_BY_FLAG(FWeakObjectProperty)
DECLARE_CAST_BY_FLAG(FLazyObjectProperty)
DECLARE_CAST_BY_FLAG(FSoftObjectProperty)
DECLARE_CAST_BY_FLAG(FSoftClassProperty)
DECLARE_CAST_BY_FLAG(FBoolProperty)
DECLARE_CAST_BY_FLAG(UFunction)
DECLARE_CAST_BY_FLAG(FStructProperty)
DECLARE_CAST_BY_FLAG(FByteProperty)
DECLARE_CAST_BY_FLAG(FIntProperty)
DECLARE_CAST_BY_FLAG(FFloatProperty)
DECLARE_CAST_BY_FLAG(FDoubleProperty)
DECLARE_CAST_BY_FLAG(FClassProperty)
DECLARE_CAST_BY_FLAG(FInterfaceProperty)
DECLARE_CAST_BY_FLAG(FNameProperty)
DECLARE_CAST_BY_FLAG(FStrProperty)
DECLARE_CAST_BY_FLAG(FUtf8StrProperty)
DECLARE_CAST_BY_FLAG(FAnsiStrProperty)
DECLARE_CAST_BY_FLAG(FTextProperty)
DECLARE_CAST_BY_FLAG(FArrayProperty)
DECLARE_CAST_BY_FLAG(FDelegateProperty)
DECLARE_CAST_BY_FLAG(FMulticastDelegateProperty)
DECLARE_CAST_BY_FLAG(UPackage)
DECLARE_CAST_BY_FLAG(ULevel)
DECLARE_CAST_BY_FLAG(AActor)
DECLARE_CAST_BY_FLAG(APlayerController)
DECLARE_CAST_BY_FLAG(APawn)
DECLARE_CAST_BY_FLAG(USceneComponent)
DECLARE_CAST_BY_FLAG(UPrimitiveComponent)
DECLARE_CAST_BY_FLAG(USkinnedMeshComponent)
DECLARE_CAST_BY_FLAG(USkeletalMeshComponent)
DECLARE_CAST_BY_FLAG(UBlueprint)
DECLARE_CAST_BY_FLAG(UDelegateFunction)
DECLARE_CAST_BY_FLAG(UStaticMeshComponent)
DECLARE_CAST_BY_FLAG(FEnumProperty)
DECLARE_CAST_BY_FLAG(FNumericProperty)
DECLARE_CAST_BY_FLAG(FInt8Property)
DECLARE_CAST_BY_FLAG(FInt16Property)
DECLARE_CAST_BY_FLAG(FInt64Property)
DECLARE_CAST_BY_FLAG(FUInt16Property)
DECLARE_CAST_BY_FLAG(FUInt32Property)
DECLARE_CAST_BY_FLAG(FUInt64Property)
DECLARE_CAST_BY_FLAG(FMapProperty)
DECLARE_CAST_BY_FLAG(FSetProperty)
DECLARE_CAST_BY_FLAG(USparseDelegateFunction)
DECLARE_CAST_BY_FLAG(FMulticastInlineDelegateProperty)
DECLARE_CAST_BY_FLAG(FMulticastSparseDelegateProperty)
DECLARE_CAST_BY_FLAG(FOptionalProperty)
DECLARE_CAST_BY_FLAG(FVCellProperty)
DECLARE_CAST_BY_FLAG(FVValueProperty)
DECLARE_CAST_BY_FLAG(FVRestValueProperty)

#undef DECLARE_CAST_BY_FLAG

namespace UE::CoreUObject::Private
{
	template <typename T>
	struct TIsCastable
	{
		// It's from-castable if it's an interface or a UObject-derived type
		enum { Value = TIsIInterface<T>::Value || std::is_convertible_v<T*, const volatile UObject*> };
	};

	template <typename To, typename From>
	inline To DynamicCast(From* Arg)
	{
		using ToValueType = std::remove_pointer_t<To>;

		if constexpr (!std::is_pointer_v<To> || !TIsCastable<From>::Value || !TIsCastable<ToValueType>::Value)
		{
			return dynamic_cast<To>(Arg);
		}
		else
		{
			// Casting away const/volatile
			static_assert(!TLosesQualifiersFromTo_V<From, ToValueType>, "Conversion loses qualifiers");

			if constexpr (std::is_void_v<ToValueType>)
			{
				// When casting to void, cast to UObject instead and let it implicitly cast to void
				return Cast<UObject>(Arg);
			}
			else
			{
				return Cast<ToValueType>(Arg);
			}
		}
	}

	template <typename To, typename From>
	inline To DynamicCast(From&& Arg)
	{
		using FromValueType = std::remove_reference_t<From>;
		using ToValueType   = std::remove_reference_t<To>;

		if constexpr (!TIsCastable<FromValueType>::Value || !TIsCastable<ToValueType>::Value)
		{
			// This may fail when dynamic_casting rvalue references due to patchy compiler support
			return dynamic_cast<To>(Arg);
		}
		else
		{
			// Casting away const/volatile
			static_assert(!TLosesQualifiersFromTo_V<FromValueType, ToValueType>, "Conversion loses qualifiers");

			// T&& can only be cast to U&&
			// http://en.cppreference.com/w/cpp/language/dynamic_cast
			static_assert(std::is_lvalue_reference_v<From> || std::is_rvalue_reference_v<To>, "Cannot dynamic_cast from an rvalue to a non-rvalue reference");

			return Forward<To>(*CastChecked<ToValueType>(&Arg));
		}
	}
}

#define dynamic_cast UE::CoreUObject::Private::DynamicCast
