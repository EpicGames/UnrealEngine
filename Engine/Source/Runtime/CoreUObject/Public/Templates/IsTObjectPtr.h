// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

template <typename T>
struct TObjectPtr;

template <typename T> constexpr bool TIsTObjectPtr_V                               = false;
template <typename T> constexpr bool TIsTObjectPtr_V<               TObjectPtr<T>> = true;
template <typename T> constexpr bool TIsTObjectPtr_V<const          TObjectPtr<T>> = true;
template <typename T> constexpr bool TIsTObjectPtr_V<      volatile TObjectPtr<T>> = true;
template <typename T> constexpr bool TIsTObjectPtr_V<const volatile TObjectPtr<T>> = true;

template <typename T>
struct TIsTObjectPtr
{
	static constexpr bool Value = TIsTObjectPtr_V<T>;
};
