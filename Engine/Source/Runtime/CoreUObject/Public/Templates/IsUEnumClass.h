// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/StaticAssertCompleteType.h"

/**
 * Traits class which tests if a type is a UEnum class.
 */
template <typename T>
struct TIsUEnumClass
{
	UE_STATIC_ASSERT_COMPLETE_TYPE(T, "TIsUEnumClass instantiated with an incomplete type");
	enum { Value = false };
};
