// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "SubclassOf.h"
#include "Misc/IntrusiveUnsetOptionalState.h"

// So we can construct uninitialized TNonNullSubclassOf
enum class EDefaultConstructNonNullSubclassOf { UnsafeDoNotUse };

/**
 * Template to allow TClassType's to be passed around with type safety 
 */
template <typename T>
class TNonNullSubclassOf : public TSubclassOf<T>
{
	using Super = TSubclassOf<T>;

public:
	/** Default Constructor, defaults to null */
	[[nodiscard]] UE_FORCEINLINE_HINT TNonNullSubclassOf(EDefaultConstructNonNullSubclassOf)
		: Super(nullptr)
	{
	}

	/** Constructor that takes a UClass* or FFieldClass* if T is a UObject or a FField type respectively. */
	template <
		typename PtrType
		UE_REQUIRES(std::is_same_v<PtrType, UClass> || std::is_same_v<PtrType, FFieldClass>)>
	[[nodiscard]] inline TNonNullSubclassOf(PtrType* From)
		: Super(From)
	{
		checkf(From, TEXT("Initializing TNonNullSubclassOf with null"));
	}

	/** Copy Constructor, will only compile if types are compatible */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TNonNullSubclassOf(const TSubclassOf<U>& From)
		: Super(From)
	{
	}

	/** Assignment operator, will only compile if types are compatible */
	template
	<
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline TNonNullSubclassOf& operator=(const TSubclassOf<U>& From)
	{
		checkf(*From, TEXT("Assigning null to TNonNullSubclassOf"));
		Super::operator=(From);
		return *this;
	}
	
	/** Assignment operator from UClass, the type is checked on get not on set */
	template <typename PtrType>
	inline TNonNullSubclassOf& operator=(PtrType* From)
	{
		checkf(From, TEXT("Assigning null to TNonNullSubclassOf"));
		Super::operator=(From);
		return *this;
	}

	///////////////////////////////////////////////////////////
	// Start - intrusive TOptional<TNonNullSubclassOf> state //
	///////////////////////////////////////////////////////////
	constexpr static bool bHasIntrusiveUnsetOptionalState = true;
	using IntrusiveUnsetOptionalStateType = TNonNullSubclassOf;
	[[nodiscard]] UE_FORCEINLINE_HINT explicit TNonNullSubclassOf(FIntrusiveUnsetOptionalState)
		: Super(nullptr)
	{
	}
	
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(FIntrusiveUnsetOptionalState) const
	{
		return Super::Get() == nullptr;
	}
	/////////////////////////////////////////////////////////
	// End - intrusive TOptional<TNonNullSubclassOf> state //
	/////////////////////////////////////////////////////////
};
