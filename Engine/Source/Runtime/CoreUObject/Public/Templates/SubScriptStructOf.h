// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UnrealTypeTraits.h"
#include "Concepts/PointerConvertibleTo.h"
#include "UObject/Class.h"

#include <type_traits>

#include "SubScriptStructOf.generated.h"

class FArchive;
class FStructuredArchiveSlot;

/**
 * Allow UScriptStruct types to be passed around with type safety
 */
USTRUCT()
struct FSubScriptStructOf
{
	GENERATED_BODY()

	FSubScriptStructOf() = default;

	/** Constructor that takes a UScriptStruct*. */
	[[nodiscard]] UE_REWRITE FSubScriptStructOf(UScriptStruct* From)
		: ScriptStruct(From)
	{
	}

	/** Assign from a UScriptStruct*. */
	UE_REWRITE FSubScriptStructOf& operator=(UScriptStruct* From)
	{
		ScriptStruct = From;
		return *this;
	}

	/** Dereference back into a UScriptStruct*. */
	[[nodiscard]] UE_REWRITE UScriptStruct* operator*() const
	{
		return ScriptStruct;
	}

	/** Dereference back into a UScriptStruct*. */
	[[nodiscard]] UE_REWRITE UScriptStruct* Get() const
	{
		return **this;
	}

	/** Dereference back into a UScriptStruct*. */
	[[nodiscard]] UE_REWRITE UScriptStruct* operator->() const
	{
		return **this;
	}

	/** Implicit conversion to UScriptStruct*. */
	[[nodiscard]] UE_REWRITE operator UScriptStruct* () const
	{
		return **this;
	}

	UE_REWRITE void Serialize(FArchive& Ar)
	{
		Ar << ScriptStruct;
	}

	UE_REWRITE void Serialize(FStructuredArchiveSlot& Slot)
	{
		Slot << ScriptStruct;
	}

	[[nodiscard]] friend uint32 GetTypeHash(const FSubScriptStructOf& SubScriptStructOf)
	{
		return GetTypeHash(SubScriptStructOf.ScriptStruct);
	}

private:
	UPROPERTY()
	TObjectPtr<UScriptStruct> ScriptStruct;
};

template <typename T>
struct TSubScriptStructOf;

template <typename T>
struct TIsTSubScriptStructOf
{
	static constexpr bool Value = false;
};

template <typename T> struct TIsTSubScriptStructOf<               TSubScriptStructOf<T>> { static constexpr bool Value = true; };
template <typename T> struct TIsTSubScriptStructOf<const          TSubScriptStructOf<T>> { static constexpr bool Value = true; };
template <typename T> struct TIsTSubScriptStructOf<      volatile TSubScriptStructOf<T>> { static constexpr bool Value = true; };
template <typename T> struct TIsTSubScriptStructOf<const volatile TSubScriptStructOf<T>> { static constexpr bool Value = true; };

template <typename T>
inline constexpr bool TIsTSubScriptStructOf_V = TIsTSubScriptStructOf<T>::Value;


/**
 * Template to allow UScriptStruct types to be passed around with type safety
 */
template <typename T>
struct TSubScriptStructOf : FSubScriptStructOf
{
private:
	template <typename U>
	friend struct TSubScriptStructOf;

public:
	using ElementType = T;

	TSubScriptStructOf() = default;

	/** Constructor that takes a UScriptStruct*. */
	UE_REWRITE TSubScriptStructOf(UScriptStruct* From)
		: FSubScriptStructOf(From)
	{
	}

	/** Construct from a UScriptStruct* (or something implicitly convertible to it). */
	template <typename U>
		requires (UE::CConvertibleTo<U, UScriptStruct*> && !TIsTSubScriptStructOf_V<std::decay_t<U>>)
	UE_REWRITE TSubScriptStructOf(U&& From)
		: FSubScriptStructOf(From)
	{
	}

	/** Construct from another TSubScriptStructOf, only if types are compatible. */
	template <UE::CPointerConvertibleTo<T> U>
	UE_REWRITE TSubScriptStructOf(const TSubScriptStructOf<U>& Other)
		: FSubScriptStructOf(Other.Get())
	{
		IWYU_MARKUP_IMPLICIT_CAST(U, T);
	}

	/** Assign from another TSubScriptStructOf, only if types are compatible. */
	template <UE::CPointerConvertibleTo<T> U>
	UE_REWRITE TSubScriptStructOf& operator=(const TSubScriptStructOf<U>& From)
	{
		IWYU_MARKUP_IMPLICIT_CAST(U, T);
		FSubScriptStructOf::operator=(From.Get());
		return *this;
	}

	/** Assign from a UScriptStruct*. */
	UE_REWRITE TSubScriptStructOf& operator=(UScriptStruct* From)
	{
		FSubScriptStructOf::operator=(From);
		return *this;
	}

	/** Assign from a UScriptStruct* (or something implicitly convertible to it). */
	template <typename U>
		requires (UE::CConvertibleTo<U, UScriptStruct*> && !TIsTSubScriptStructOf_V<std::decay_t<U>>)
	UE_REWRITE TSubScriptStructOf& operator=(U&& From)
	{
		FSubScriptStructOf::operator=(From.Get());
		return *this;
	}

	/** Dereference back into a UScriptStruct*, does runtime type checking. */
	UE_REWRITE UScriptStruct* operator*() const
	{
		UScriptStruct* Local = FSubScriptStructOf::Get();
		if (!Local || !Local->IsChildOf(T::StaticStruct()))
		{
			return nullptr;
		}
		return Local;
	}

	/** Dereference back into a UScriptStruct*, does runtime type checking. */
	UE_REWRITE UScriptStruct* Get() const
	{
		return **this;
	}

	/** Dereference back into a UScriptStruct*, does runtime type checking. */
	UE_REWRITE UScriptStruct* operator->() const
	{
		return **this;
	}

	/** Implicit conversion to UScriptStruct*, does runtime type checking. */
	UE_REWRITE operator UScriptStruct* () const
	{
		return **this;
	}
};

template <typename T>
struct TCallTraits<TSubScriptStructOf<T>> : public TCallTraitsBase<TSubScriptStructOf<T>>
{
	using ConstPointerType = TSubScriptStructOf<const T>;
};
