// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Traits/IsTSubclassOf.h"
#include "UObject/Class.h"

#include <type_traits>

class FStructuredArchiveSlot;

/**
 * Template to allow UClass types to be passed around with type safety
 */
template <typename T>
class TSubclassOf
{
private:
	template <typename U>
	friend class TSubclassOf;

public:
	using ElementType = T;

	[[nodiscard]] TSubclassOf() = default;
	[[nodiscard]] TSubclassOf(TSubclassOf&&) = default;
	[[nodiscard]] TSubclassOf(const TSubclassOf&) = default;
	TSubclassOf& operator=(TSubclassOf&&) = default;
	TSubclassOf& operator=(const TSubclassOf&) = default;
	~TSubclassOf() = default;

	/** Constructor that takes a UClass*. */
	[[nodiscard]] UE_FORCEINLINE_HINT TSubclassOf(UClass* From)
		: Class(From)
	{
	}

	/** Construct from a UClass* (or something implicitly convertible to it) */
	template <
		typename U
		UE_REQUIRES(
			!TIsTSubclassOf<std::decay_t<U>>::Value &&
			std::is_convertible_v<U, UClass*>
		)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSubclassOf(U&& From)
		: Class(From)
	{
	}

	/** Construct from another TSubclassOf, only if types are compatible */
	template <
		typename OtherT
		UE_REQUIRES(std::is_convertible_v<OtherT*, T*>)
	>
	[[nodiscard]] inline TSubclassOf(const TSubclassOf<OtherT>& Other)
		: Class(Other.Class)
	{
		IWYU_MARKUP_IMPLICIT_CAST(OtherT, T);
	}

	/** Assign from another TSubclassOf, only if types are compatible */
	template <
		typename OtherT
		UE_REQUIRES(std::is_convertible_v<OtherT*, T*>)
	>
	inline TSubclassOf& operator=(const TSubclassOf<OtherT>& Other)
	{
		IWYU_MARKUP_IMPLICIT_CAST(OtherT, T);
		Class = Other.Class;
		return *this;
	}

	/** Assign from a UClass*. */
	inline TSubclassOf& operator=(UClass* From)
	{
		Class = From;
		return *this;
	}

	/** Assign from a UClass* (or something implicitly convertible to it). */
	template <
		typename U
		UE_REQUIRES(
			!TIsTSubclassOf<std::decay_t<U>>::Value &&
			std::is_convertible_v<U, UClass*>
		)
	>
	inline TSubclassOf& operator=(U&& From)
	{
		Class = From;
		return *this;
	}

	/** Dereference back into a UClass*, does runtime type checking. */
	[[nodiscard]] inline UClass* operator*() const
	{
		if (!Class || !Class->IsChildOf(T::StaticClass()))
		{
			return nullptr;
		}
		return Class;
	}

	/** Dereference back into a UClass*, does runtime type checking. */
	[[nodiscard]] UE_FORCEINLINE_HINT UClass* Get() const
	{
		return **this;
	}

	/** Dereference back into a UClass*, does runtime type checking. */
	[[nodiscard]] UE_FORCEINLINE_HINT UClass* operator->() const
	{
		return **this;
	}

	/** Implicit conversion to UClass*, does runtime type checking. */
	[[nodiscard]] UE_FORCEINLINE_HINT operator UClass*() const
	{
		return **this;
	}

	/**
	 * Get the CDO if we are referencing a valid class
	 *
	 * @return the CDO, or null if class is null
	 */
	[[nodiscard]] inline T* GetDefaultObject() const
	{
		UObject* Result = nullptr;
		if (Class)
		{
			Result = Class->GetDefaultObject();
			check(Result && Result->IsA(T::StaticClass()));
		}
		return (T*)Result;
	}

	UE_FORCEINLINE_HINT void Serialize(FArchive& Ar)
	{
		Ar << Class;
	}

	UE_FORCEINLINE_HINT void Serialize(FStructuredArchiveSlot& Slot)
	{
		Slot << Class;
	}

	[[nodiscard]] friend uint32 GetTypeHash(const TSubclassOf& SubclassOf)
	{
		return GetTypeHash(SubclassOf.Class);
	}

	[[nodiscard]] TObjectPtr<UClass>& GetGCPtr()
	{
		return Class;
	}

#if DO_CHECK
	// This is a DEVELOPMENT ONLY debugging function and should not be relied upon. Client
	// systems should never require unsafe access to the referenced UClass
	[[nodiscard]] UClass* DebugAccessRawClassPtr() const
	{
		return Class;
	}
#endif

private:
	TObjectPtr<UClass> Class = nullptr;
};

template <typename T>
struct TCallTraits<TSubclassOf<T>> : public TCallTraitsBase<TSubclassOf<T>>
{
	using ConstPointerType = TSubclassOf<const T>;
};

template <typename T>
FArchive& operator<<(FArchive& Ar, TSubclassOf<T>& SubclassOf)
{
	SubclassOf.Serialize(Ar);
	return Ar;
}

template <typename T>
void operator<<(FStructuredArchiveSlot Slot, TSubclassOf<T>& SubclassOf)
{
	SubclassOf.Serialize(Slot);
}
