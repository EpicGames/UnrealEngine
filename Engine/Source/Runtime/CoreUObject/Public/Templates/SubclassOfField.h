// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Field.h"

#include <type_traits>

template <typename T>
class TSubclassOfField;

template <typename T>
struct TIsTSubclassOfField
{
	enum { Value = false };
};

template <typename T> struct TIsTSubclassOfField<               TSubclassOfField<T>> { enum { Value = true }; };
template <typename T> struct TIsTSubclassOfField<const          TSubclassOfField<T>> { enum { Value = true }; };
template <typename T> struct TIsTSubclassOfField<      volatile TSubclassOfField<T>> { enum { Value = true }; };
template <typename T> struct TIsTSubclassOfField<const volatile TSubclassOfField<T>> { enum { Value = true }; };

/**
 * Template to allow FFieldClass types to be passed around with type safety
 */
template <typename T>
class TSubclassOfField
{
private:
	template <typename U>
	friend class TSubclassOfField;

public:
	[[nodiscard]] TSubclassOfField() = default;
	[[nodiscard]] TSubclassOfField(TSubclassOfField&&) = default;
	[[nodiscard]] TSubclassOfField(const TSubclassOfField&) = default;
	TSubclassOfField& operator=(TSubclassOfField&&) = default;
	TSubclassOfField& operator=(const TSubclassOfField&) = default;
	~TSubclassOfField() = default;

	/** Constructor that takes a FFieldClass*. */
	[[nodiscard]] UE_FORCEINLINE_HINT TSubclassOfField(FFieldClass* From)
		: Class(From)
	{
	}

	/** Construct from a FFieldClass* (or something implicitly convertible to it) */
	template <
		typename U
		UE_REQUIRES(
			!TIsTSubclassOfField<std::decay_t<U>>::Value &&
			std::is_convertible_v<U, FFieldClass*>
		)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSubclassOfField(U&& From)
		: Class(From)
	{
	}

	/** Construct from another TSubclassOfField, only if types are compatible */
	template <
		typename OtherT
		UE_REQUIRES(std::is_convertible_v<OtherT*, T*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSubclassOfField(const TSubclassOfField<OtherT>& Other)
		: Class(Other.Class)
	{
	}

	/** Assign from another TSubclassOfField, only if types are compatible */
	template <
		typename OtherT
		UE_REQUIRES(std::is_convertible_v<OtherT*, T*>)
	>
	inline TSubclassOfField& operator=(const TSubclassOfField<OtherT>& Other)
	{
		Class = Other.Class;
		return *this;
	}

	/** Assign from a FFieldClass*. */
	inline TSubclassOfField& operator=(FFieldClass* From)
	{
		Class = From;
		return *this;
	}

	/** Assign from a FFieldClass* (or something implicitly convertible to it). */
	template <
		typename U
		UE_REQUIRES(
			!TIsTSubclassOfField<std::decay_t<U>>::Value &&
			std::is_convertible_v<U, FFieldClass*>
		)
	>
	inline TSubclassOfField& operator=(U&& From)
	{
		Class = From;
		return *this;
	}

	/** Dereference back into a FFieldClass*, does runtime type checking. */
	[[nodiscard]] inline FFieldClass* operator*() const
	{
		if (!Class || !Class->IsChildOf(T::StaticClass()))
		{
			return nullptr;
		}
		return Class;
	}

	/** Dereference back into a FFieldClass*, does runtime type checking. */
	[[nodiscard]] UE_FORCEINLINE_HINT FFieldClass* Get() const
	{
		return **this;
	}

	/** Dereference back into a FFieldClass*, does runtime type checking. */
	[[nodiscard]] UE_FORCEINLINE_HINT FFieldClass* operator->() const
	{
		return **this;
	}

	/** Implicit conversion to FFieldClass*, does runtime type checking. */
	[[nodiscard]] UE_FORCEINLINE_HINT operator FFieldClass*() const
	{
		return **this;
	}

	/**
	 * Get the CDO if we are referencing a valid class
	 *
	 * @return the CDO, or null if class is null
	 */
	[[nodiscard]] inline T* GetDefaultObject() const
	{
		FField* Result = nullptr;
		if (Class)
		{
			Result = Class->GetDefaultObject();
			check(Result && Result->IsA(T::StaticClass()));
		}
		return (T*)Result;
	}

	UE_FORCEINLINE_HINT void Serialize(FArchive& Ar)
	{
		Ar << Class;
	}

	[[nodiscard]] friend uint32 GetTypeHash(const TSubclassOfField& SubclassOf)
	{
		return GetTypeHash(SubclassOf.Class);
	}

private:
	FFieldClass* Class = nullptr;
};

template <typename T>
FArchive& operator<<(FArchive& Ar, TSubclassOfField<T>& SubclassOf)
{
	SubclassOf.Serialize(Ar);
	return Ar;
}

