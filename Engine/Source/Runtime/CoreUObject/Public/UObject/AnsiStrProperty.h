// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Include StrProperty.h.inl's includes before defining the macros, in case the macros 'poison' other headers or there are re-entrant includes.
#include "UObject/StrPropertyIncludes.h.inl" // IWYU pragma: export

#include "Containers/AnsiString.h"

#define UE_STRPROPERTY_CLASS                FAnsiStrProperty
#define UE_STRPROPERTY_STRINGTYPE           FAnsiString
#define UE_STRPROPERTY_CASTCLASSFLAG        CASTCLASS_FAnsiStrProperty
#define UE_STRPROPERTY_PROPERTYPARAMSSTRUCT FAnsiStrPropertyParams
	#include "UObject/StrProperty.h.inl" // IWYU pragma: export
#undef UE_STRPROPERTY_PROPERTYPARAMSSTRUCT
#undef UE_STRPROPERTY_CASTCLASSFLAG
#undef UE_STRPROPERTY_STRINGTYPE
#undef UE_STRPROPERTY_CLASS
