// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/CookTagList.h"
#include "UObject/CookEnums.h"

namespace UE::Cook { class ICookInfo; }

/**
*	Accessor for data about the package being cooked during UObject::Serialize calls.
*/
struct FArchiveCookContext
{
private:

	/** CookTagList is only valid for cook by the book; it is not publically accessible otherwise. */
	FCookTagList CookTagList;
	const ITargetPlatform* TargetPlatform = nullptr;
	bool bCookTagListEnabled = false;
	UE::Cook::ECookType CookType = UE::Cook::ECookType::Unknown;
	UE::Cook::ECookingDLC CookingDLC = UE::Cook::ECookingDLC::Unknown;
	UE::Cook::ICookInfo* CookInfo = nullptr;

public:

	FArchiveCookContext(UPackage* InPackage, UE::Cook::ECookType InCookType, UE::Cook::ECookingDLC InCookingDLC,
		const ITargetPlatform* InTargetPlatform, UE::Cook::ICookInfo* InCookInfo);

	void Reset();

	FCookTagList* GetCookTagList();
	const ITargetPlatform* GetTargetPlatform() const;

	bool IsCookByTheBook() const;
	bool IsCookOnTheFly() const;
	bool IsCookTypeUnknown() const;
	UE::Cook::ECookType GetCookType() const;
	UE::Cook::ECookingDLC GetCookingDLC() const;
	UE::Cook::ICookInfo* GetCookInfo();
};


///////////////////////////////////////////////////////
// Inline implementations
///////////////////////////////////////////////////////


inline FArchiveCookContext::FArchiveCookContext(UPackage* InPackage, UE::Cook::ECookType InCookType,
	UE::Cook::ECookingDLC InCookingDLC, const ITargetPlatform* InTargetPlatform, UE::Cook::ICookInfo* InCookInfo)
	: CookTagList(InPackage)
	, TargetPlatform(InTargetPlatform)
	, bCookTagListEnabled(InPackage && InCookType == UE::Cook::ECookType::ByTheBook)
	, CookType(InCookType)
	, CookingDLC(InCookingDLC)
	, CookInfo(InCookInfo)
{
}

inline void FArchiveCookContext::Reset()
{
	CookTagList.Reset();
}

inline FCookTagList* FArchiveCookContext::GetCookTagList()
{
	return bCookTagListEnabled ? &CookTagList : nullptr;
}

inline const ITargetPlatform* FArchiveCookContext::GetTargetPlatform() const
{
	return TargetPlatform;
}

inline bool FArchiveCookContext::IsCookByTheBook() const
{
	return CookType == UE::Cook::ECookType::ByTheBook;
}

inline bool FArchiveCookContext::IsCookOnTheFly() const
{
	return CookType == UE::Cook::ECookType::OnTheFly;
}

inline bool FArchiveCookContext::IsCookTypeUnknown() const
{
	return CookType == UE::Cook::ECookType::Unknown;
}

inline UE::Cook::ECookType FArchiveCookContext::GetCookType() const
{
	return CookType;
}

inline UE::Cook::ECookingDLC FArchiveCookContext::GetCookingDLC() const
{
	return CookingDLC;
}

inline UE::Cook::ICookInfo* FArchiveCookContext::GetCookInfo()
{
	return CookInfo;
}
