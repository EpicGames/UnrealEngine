// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/CookEnums.h"
#include "UObject/Object.h" // UObject::FAssetRegistryTag

class ITargetPlatform;
class UObject;
struct FAssetBundleData;

namespace UE::AssetRegistry
{

/**
 * We have to store all tags as Name/String pairs, without any extra data for e.g. bIsCookTag,
 * so we use a naming convention on the Name. The FAssetRegistryTagsContextData API prepends CookTagPrefix
 * to tags specified as CookTags, and gives a warning and ignores any regular tags starting with CookTagPrefix.
 */
inline constexpr FStringView CookTagPrefix = TEXTVIEW("Cook_");

}

/** Marked up at the callsite with the category of callsite. */
enum class EAssetRegistryTagsCaller : uint8
{
	/** The AssetRegistry is calling FAssetData(UObject*) to return results for a query from an in-memory UObject. */
	AssetRegistryQuery,
	/**
	 * The AssetRegistry is calling FAssetData(UObject*) on a UObject that has just loaded and will write them into
	 * the AR's registered tags for that object.
	 */
	AssetRegistryLoad,
	/** Caller has requested a full update. */
	FullUpdate,
	/**
	 * SavePackage is calling GetAssetRegistryTags and will store the result in the package header.
	 * See other functions on the context to determine what kind of save it is.
	 */
	SavePackage,
	/**
	 * System-specific code outside of SavePackage or the AssetRegistry is constructing an FAssetData, and
	 * wants to skip any expensive tags.
	 */
	Fast,
	/** System-specific code outside of SavePackage or the AssetRegistry is constructing an FAssetData. */
	Uncategorized,
};

/**
 * Data used to provide information about the calling context and receive results from GetAssetRegistryTags.
 * The fields provided by this class are hidden from implementers of GetAssetRegistryTags so that we can change the
 * implementation of this class without needing to add deprecation handling in the GetAssetRegistryTags functions.
 */
struct FAssetRegistryTagsContextData
{
public:
	COREUOBJECT_API FAssetRegistryTagsContextData(const UObject* CallTarget,
		EAssetRegistryTagsCaller InCaller = EAssetRegistryTagsCaller::Uncategorized);

	TMap<FName, UObject::FAssetRegistryTag> Tags;
	const UObject* Object = nullptr;
	const ITargetPlatform* TargetPlatform = nullptr;
	const FAssetBundleData* BundleResult = nullptr;
	EAssetRegistryTagsCaller Caller = EAssetRegistryTagsCaller::Uncategorized;
	UE::Cook::ECookType CookType = UE::Cook::ECookType::Unknown;
	UE::Cook::ECookingDLC CookingDLC = UE::Cook::ECookingDLC::Unknown;
	bool bProceduralSave = false;
	bool bWantsBundleResult = false;
	bool bWantsCookTags = false;
	bool bFullUpdateRequested = false;
};

/**
 * Interface used by GetAssetRegistryTags to access the calling context data.
 * 
 * API Notes:
 * CookTag functions on this class add CookTags - tags that are added only when cooking and that are added to the
 * development AssetRegistry used for patches patches, DLC, and Incremental cooking, but are removed from the runtime
 * AssetRegistry passed into the game. CookTags can only be added when WantsCookTags() is true, calls in other Context
 * states are ignored. All tags added with the CookTags interface have "Cook_" prepended to their names prior to being
 * saved into the development to the asset registry.
 *
 * CookType: When GetAssetRegistryTags is called for package saves during cook, the cook type is set depending on
 * the type of cook. Cooks in Unreal can be either CookByTheBook or CookOnTheFly. If GetAssetRegistryTags was called
 * for other reasons, the CookType is reported as unknown. CookByTheBook means that the project's packaging settings
 * are read by the cooker and a list of all referenced packages are found and those packages are saved and packaged.
 * CookOnTheFly means that the game is running without packaged data, and is connecting to the cooker process to 
 * request the cooked version of packages that it demands. Only the packages requested by the runtime session are cooked.
 */
class FAssetRegistryTagsContext
{
public:
	// This is an implicit constructor to reduce the boilerplate for calling GetAssetRegistryTags
	COREUOBJECT_API FAssetRegistryTagsContext(FAssetRegistryTagsContextData& InData);
	COREUOBJECT_API FAssetRegistryTagsContext(const FAssetRegistryTagsContext& Other);

	COREUOBJECT_API EAssetRegistryTagsCaller GetCaller() const;
	COREUOBJECT_API const UObject* GetObject() const;

	/** Whether all tags should be calculated, as set by the creator of the context (e.g. SavePackage). */
	COREUOBJECT_API bool IsFullUpdate() const;
	/** Whether the call is coming from SavePackage. See functions below for determining what kind of save. */
	COREUOBJECT_API bool IsSaving() const;
	/**
	 * Return whether the package is being saved due to a procedural save. False if not saving.
	 * Any save without the possibility of user-generated edits to the package is a procedural save (Cooking, EditorDomain).
	 * This allows us to add or update tags that only need to be modified in response to new user data.
	 */
	COREUOBJECT_API bool IsProceduralSave() const;
	/** Report whether this is a save into a target-specific cooked format. False if not saving. */
	COREUOBJECT_API bool IsCooking() const;
	/** Return the targetplatform of the save, if cooking. Null if not saving or not cooking. */
	COREUOBJECT_API const ITargetPlatform* GetTargetPlatform() const;

	/**
	 * If IsCooking is true, reports whether the cook is CookByTheBook. @see FAssetRegistryTagsContext.
	 * If IsCooking is false, returns false.
	 */
	COREUOBJECT_API bool IsCookByTheBook() const;
	/**
	 * If IsCooking is true, reports whether the cook is CookByTheBook. @see FAssetRegistryTagsContext.
	 * If IsCooking is false, returns false.
	 */
	COREUOBJECT_API bool IsCookOnTheFly() const;
	/**
	 * If IsCooking is true, reports whether the cook type is unknown and could be any type of cook.
	 * If IsCooking is false, returns true.
	 */
	COREUOBJECT_API bool IsCookTypeUnknown() const;
	/**
	 * If IsCooking is true, returns the cook type, @see FAssetRegistryTagsContext.
	 * If IsCooking is false, returns ECookType::Unknown.
	 */
	COREUOBJECT_API UE::Cook::ECookType GetCookType() const;
	/**
	 * If IsCooking is true, returns whether whether the cook is for a DLC rather than base game.
	 * If IsCooking is false, returns ECookingDLC::Unknown.
	 */
	COREUOBJECT_API UE::Cook::ECookingDLC GetCookingDLC() const;

	/**
	 * Output function: Add a tag if it does not already exist. Return reference to the existing tag and optionally
	 * report whether it already existed. Issues an error and returns referenced to unused global if TagName is invalid.
	 */
	COREUOBJECT_API UObject::FAssetRegistryTag& FindOrAddTag(FName TagName, bool* bOutAlreadyExists=nullptr);
	/** Output function: Find a tag in the output if it already exists. Issues an error if TagName is invalid. */
	COREUOBJECT_API UObject::FAssetRegistryTag* FindTag(FName TagName);
	/** Output function: Report whether a tag is already in the output. Issues an error if TagName is invalid. */
	COREUOBJECT_API bool ContainsTag(FName TagName) const;
	/** Output function: move the given tag into the results, overwriting previous result if it exists. */
	COREUOBJECT_API void AddTag(UObject::FAssetRegistryTag TagResult);
	/** Removes the tag from the output. */
	COREUOBJECT_API void RemoveTag(FName TagName);
	/** Return the number of tags so far reported to the context, including CookTags. */
	COREUOBJECT_API int32 GetNumTags() const;
	/** Pass each tag so far reported to the context into the provided Visitor, including CookTags. */
	COREUOBJECT_API void EnumerateTags(TFunctionRef<void(const UObject::FAssetRegistryTag&)> Visitor) const;

	/**
	 * True if the caller wants the FAssetBundleData structure (if present) to be stored as a pointer.
	 * If false, the FAssetBundleData structure should instead be converted to text and stored as a tag.
	 */
	COREUOBJECT_API bool WantsBundleResult() const;
	COREUOBJECT_API const FAssetBundleData* GetBundleResult();
	COREUOBJECT_API void SetBundleResult(const FAssetBundleData* InBundleResult);

	/**
	 * Return whether the context is collecting cook tags. CookTag functions are noops if this returns
	 * false. CookTags are only collected for CookByTheBook.
	 */
	COREUOBJECT_API bool WantsCookTags() const;

	/** FindOrAddCookTag but for a cooktag. @see FindOrAddTag and @see FAssetRegistryTagsContext */
	COREUOBJECT_API UObject::FAssetRegistryTag& FindOrAddCookTag(FName TagName, bool* bOutAlreadyExists);
	/** FindTag but for a cooktag. @see FindTag and @see FAssetRegistryTagsContext */
	COREUOBJECT_API UObject::FAssetRegistryTag* FindCookTag(FName TagName);
	/** ContainsTag but for a cooktag. @see ContainsTag and @see FAssetRegistryTagsContext */
	COREUOBJECT_API bool ContainsCookTag(FName TagName) const;
	/** AddTag but for a cooktag. @see AddTag and @see FAssetRegistryTagsContext */
	COREUOBJECT_API void AddCookTag(UObject::FAssetRegistryTag TagResult);
	/** Removes the cooktag from the output. */
	COREUOBJECT_API void RemoveCookTag(FName TagName);

private:
	bool TryValidateTagName(FName TagName) const;
	FName TransformTagName(FName TagName, FStringView PrefixToAdd, bool bCreateNameIfDoesNotExist = true) const;
	UObject::FAssetRegistryTag& FindOrAddTagInternal(FName TagName, FStringView PrefixToAdd, bool* bOutAlreadyExists);
	UObject::FAssetRegistryTag* FindTagInternal(FName TagName, FStringView PrefixToAdd);
	bool ContainsTagInternal(FName TagName, FStringView PrefixToAdd) const;
	void AddTagInternal(UObject::FAssetRegistryTag&& TagResult, FStringView PrefixToAdd);
	void RemoveTagInternal(FName TagName, FStringView PrefixToAdd);

private:
	FAssetRegistryTagsContextData& Data;
	friend class UObject;
};