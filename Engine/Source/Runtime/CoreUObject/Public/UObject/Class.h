// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Class.h: UClass definition.
=============================================================================*/

#pragma once

#include "AutoRTFM/Defines.h"
#include "Concepts/GetTypeHashable.h"
#include "Concepts/StaticClassProvider.h"
#include "Concepts/StaticStructProvider.h"
#include "Concepts/StructSerializableWithDefaults.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Algo/MaxElement.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Math/InterpCurvePoint.h"
#include "Math/MathFwd.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/NotNull.h"
#include "Misc/Optional.h"
#include "Misc/ScopeRWLock.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/Identity.h"
#include "Templates/IsAbstract.h"
#include "Templates/IsEnum.h"
#include "Templates/IsPODType.h"
#include "Templates/IsUECoreType.h"
#include "Templates/Models.h"
#include "Templates/Tuple.h"
#include "Templates/TypeCompatibleBytes.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/CoreNative.h"
#include "UObject/Field.h"
#include "UObject/FieldPath.h"
#include "UObject/GarbageCollection.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyTag.h"
#include "UObject/PropertyVisitor.h"
#include "UObject/Script.h"
#include "UObject/StructOpsTypeTraits.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectPtrLinkedListBuilder.h"
#include "UObject/ObjectPtrLinkedListIterator.h"
#include <type_traits>
#include <atomic>

#if UE_WITH_CONSTINIT_UOBJECT
#include "Async/SharedLock.h"
#include "Async/SharedMutex.h"
#include "Async/UniqueLock.h"
#elif PLATFORM_FORCE_UE_LOCK_USAGE
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#endif

#include "Class.generated.h"

class FArchive;
class FEditPropertyChain;
class FField;
class FOutputDevice;
class FProperty;
class FReferenceCollector;
class FStructProperty;
class FStructuredArchiveRecord;
class UObject;
class UPackageMap;
class UPropertyWrapper;
struct CGetTypeHashable;
struct FArchiveFieldName;
struct FAssetData;
struct FBlake3Hash;
struct FColor;
struct FCustomPropertyListNode;
struct FDoubleInterval;
struct FDoubleRange;
struct FDoubleRangeBound;
struct FFallbackStruct;
struct FFloatInterval;
struct FFloatRange;
struct FFloatRangeBound;
struct FFrame;
struct FInt32Interval;
struct FInt32Range;
struct FInt32RangeBound;
struct FInterpCurveFloat;
struct FInterpCurveLinearColor;
struct FInterpCurveQuat;
struct FInterpCurveTwoVectors;
struct FInterpCurveVector2D;
struct FInterpCurveVector;
struct FLinearColor;
struct FNetDeltaSerializeInfo;
struct FObjectInstancingGraph;
struct FPropertyTag;
struct FRandomStream;
struct FUObjectSerializeContext;
template <typename FuncType> class TFunctionRef;
enum class EPropertyObjectReferenceType : uint32;

namespace UE::CoreUObject::Private { void ConstructUClassPartials(UClass&); }

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogClass, Log, All);
COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogScriptSerialization, Log, All);

#if UE_WITH_CONSTINIT_UOBJECT
	typedef UE::TSharedLock<UE::FSharedMutex> FUClassFuncScopeReadLock;
	typedef UE::TUniqueLock<UE::FSharedMutex> FUClassFuncScopeWriteLock;
	typedef UE::FSharedMutex FUClassFuncLock;
#elif PLATFORM_FORCE_UE_LOCK_USAGE
	typedef UE::TUniqueLock<UE::FMutex> FUClassFuncScopeReadLock;
	typedef UE::TUniqueLock<UE::FMutex> FUClassFuncScopeWriteLock;
	typedef UE::FMutex FUClassFuncLock;
#else
	typedef FReadScopeLock FUClassFuncScopeReadLock;
	typedef FWriteScopeLock FUClassFuncScopeWriteLock;
	typedef FRWLock FUClassFuncLock;
#endif

#ifndef UE_WITH_IMMUTABLE_CDO
	#define UE_WITH_IMMUTABLE_CDO (UE_WITH_REMOTE_OBJECT_HANDLE && UE_AUTORTFM)
#endif

/*-----------------------------------------------------------------------------
	FRepRecord.
-----------------------------------------------------------------------------*/

//
// Information about a property to replicate.
//
struct FRepRecord
{
	FProperty* Property;
	int32 Index;
	FRepRecord(FProperty* InProperty,int32 InIndex)
	: Property(InProperty), Index(InIndex)
	{}
};

/*-----------------------------------------------------------------------------
	UField.
-----------------------------------------------------------------------------*/

//
// Base class of reflection data objects.
//
UCLASS(MinimalAPI, Abstract, Config = Engine)
class UField : public UObject
{
	GENERATED_BODY()
public:

	typedef UField BaseFieldClass;
	typedef UClass FieldTypeClass;

	/** Next Field in the linked list */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UField>			Next;

	// Constructors.
	UField(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
	{
	}

	UField(EStaticConstructor, EObjectFlags InFlags);
	
#if UE_WITH_CONSTINIT_UOBJECT
	consteval UField(
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams
	)
		: Super(ObjectParams)
		, Next(FieldParams.Next)
	{
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

	~UField() = default;

	// UObject interface.
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void PostLoad() override;
	COREUOBJECT_API virtual bool NeedsLoadForClient() const override;
	COREUOBJECT_API virtual bool NeedsLoadForServer() const override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	COREUOBJECT_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker) override;
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC

	// UField interface.
	COREUOBJECT_API virtual void AddCppProperty(FProperty* Property);
	COREUOBJECT_API virtual void Bind();

	/** Goes up the outer chain to look for a UClass */
	COREUOBJECT_API UClass* GetOwnerClass() const;

	/** Goes up the outer chain to look for a UStruct */
	COREUOBJECT_API UStruct* GetOwnerStruct() const;

	/** 
	 * Returns a human readable string that was assigned to this field at creation. 
	 * By default this is the same as GetName() but it can be overridden if that is an internal-only name.
	 * This name is consistent in editor/cooked builds, is not localized, and is useful for data import/export.
	 */
	COREUOBJECT_API FString GetAuthoredName() const;

#if WITH_EDITORONLY_DATA
	/**
	 * Finds the localized display name or native display name as a fallback.
	 *
	 * @return The display name for this object.
	 */
	COREUOBJECT_API FText GetDisplayNameText() const;

	/**
	 * Finds the localized tooltip or native tooltip as a fallback.
	 *
	 * @param bShortTooltip Look for a shorter version of the tooltip (falls back to the long tooltip if none was specified)
	 *
	 * @return The tooltip for this object.
	 */
	COREUOBJECT_API FText GetToolTipText(bool bShortTooltip = false) const;

	/** 
	 * Formats a source comment into the form we want to show in the editor, is used by GetToolTipText and anything else that will get a native tooltip 
	 * 
	 * @param ToolTipString			String parsed out of C++ headers that is modified in place
	 * @param bRemoveExtraSections	If true, cut off the comment on first line separator or 2 empty lines in a row
	 */
	static COREUOBJECT_API void FormatNativeToolTip(FString& ToolTipString, bool bRemoveExtraSections = true);
#endif // WITH_EDITORONLY_DATA

#if WITH_METADATA
	/**
	 * Determines if the property has any metadata associated with the key
	 * 
	 * @param Key The key to lookup in the metadata
	 * @return true if there is a (possibly blank) value associated with this key
	 */
	bool HasMetaData(const TCHAR* Key) const { return FindMetaData(Key) != nullptr; }
	bool HasMetaData(const FName& Key) const { return FindMetaData(Key) != nullptr; }

	/**
	 * Find the metadata value associated with the key
	 *
	 * @param Key The key to lookup in the metadata
	 * @return The value associated with the key if exists, null otherwise
	 */
	COREUOBJECT_API const FString* FindMetaData(const TCHAR* Key) const;
	COREUOBJECT_API const FString* FindMetaData(const FName& Key) const;

	/**
	 * Find the metadata value associated with the key
	 * 
	 * @param Key The key to lookup in the metadata
	 * @return The value associated with the key
	 */
	COREUOBJECT_API const FString& GetMetaData(const TCHAR* Key) const;
	COREUOBJECT_API const FString& GetMetaData(const FName& Key) const;

	/**
	 * Find the metadata value associated with the key and localization namespace and key
	 *
	 * @param Key						The key to lookup in the metadata
	 * @param LocalizationNamespace		Namespace to lookup in the localization manager
	 * @param LocalizationKey			Key to lookup in the localization manager
	 * @return							Localized metadata if available, defaults to whatever is provided via GetMetaData
	 */
	COREUOBJECT_API FText GetMetaDataText(const TCHAR* MetaDataKey, const FTextKey LocalizationNamespace = FTextKey(), const FTextKey LocalizationKey = FTextKey()) const;
	COREUOBJECT_API FText GetMetaDataText(const FName& MetaDataKey, const FTextKey LocalizationNamespace = FTextKey(), const FTextKey LocalizationKey = FTextKey()) const;

	/**
	 * Sets the metadata value associated with the key
	 * 
	 * @param Key The key to lookup in the metadata
	 * @return The value associated with the key
	 */
	COREUOBJECT_API void SetMetaData(const TCHAR* Key, const TCHAR* InValue);
	COREUOBJECT_API void SetMetaData(const FName& Key, const TCHAR* InValue);

	/**
	 * Find the metadata value associated with the key
	 * and return bool 
	 * @param Key The key to lookup in the metadata
	 * @return return true if the value was true (case insensitive)
	 */
	bool GetBoolMetaData(const TCHAR* Key) const
	{		
		const FString& BoolString = GetMetaData(Key);
		// FString == operator does case insensitive comparison
		return (BoolString == "true");
	}
	bool GetBoolMetaData(const FName& Key) const
	{		
		const FString& BoolString = GetMetaData(Key);
		// FString == operator does case insensitive comparison
		return (BoolString == "true");
	}

	/**
	 * Find the metadata value associated with the key
	 * and return int32 
	 * @param Key The key to lookup in the metadata
	 * @return the int value stored in the metadata. 0 if not a valid integer.
	 */
	int32 GetIntMetaData(const TCHAR* Key) const
	{
		const FString& IntString = GetMetaData(Key);
		int32 Value = FCString::Atoi(*IntString);
		return Value;
	}
	int32 GetIntMetaData(const FName& Key) const
	{
		const FString& IntString = GetMetaData(Key);
		int32 Value = FCString::Atoi(*IntString);
		return Value;
	}

	/**
	 * Find the metadata value associated with the key
	 * and return float
	 * @param Key The key to lookup in the metadata
	 * @return the float value stored in the metadata. 0 if not a valid float.
	 */
	float GetFloatMetaData(const TCHAR* Key) const
	{
		const FString& FloatString = GetMetaData(Key);
		float Value = FCString::Atof(*FloatString);
		return Value;
	}
	float GetFloatMetaData(const FName& Key) const
	{
		const FString& FloatString = GetMetaData(Key);
		float Value = FCString::Atof(*FloatString);
		return Value;
	}
	
	/**
	 * Find the metadata value associated with the key
	 * and return Class
	 * @param Key The key to lookup in the metadata
	 * @return the class value stored in the metadata.
	 */
	COREUOBJECT_API UClass* GetClassMetaData(const TCHAR* Key) const;
	COREUOBJECT_API UClass* GetClassMetaData(const FName& Key) const;

	/** Clear any metadata associated with the key */
	COREUOBJECT_API void RemoveMetaData(const TCHAR* Key);
	COREUOBJECT_API void RemoveMetaData(const FName& Key);
#endif // WITH_METADATA

	COREUOBJECT_API bool HasAnyCastFlags(const uint64 InCastFlags) const;
	COREUOBJECT_API bool HasAllCastFlags(const uint64 InCastFlags) const;

#if WITH_EDITORONLY_DATA
	/**
	 * Gets the FField object associated with this Field
	 */
	COREUOBJECT_API virtual FField* GetAssociatedFField();
	/**
	 * Sets the FField object associated with this Field
	 */
	COREUOBJECT_API virtual void SetAssociatedFField(FField* InField);
#endif // WITH_EDITORONLY_DATA

	using FLinkedListBuilder = TObjectPtrLinkedListBuilder<UField>;
};

#if WITH_EDITORONLY_DATA
/**
 * Flags that are optionally tracked per property by struct and class instances.
 */
enum class EPropertyValueFlags : uint8
{
	/** Whether the property value is initialized. */
	Initialized,
	/** Whether the property value was written during serialization. */
	Serialized,
};
#endif // WITH_EDITORONLY_DATA

class FStructBaseChain
{
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
protected:
	FStructBaseChain() = default;
public:
	COREUOBJECT_API ~FStructBaseChain();
	UE_NONCOPYABLE(FStructBaseChain);
	
#if UE_WITH_CONSTINIT_UOBJECT
#if !IS_MONOLITHIC
	consteval FStructBaseChain(TArrayView<UE::CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain>> InBases)
		: CompiledInBases(InBases.GetData())
		, NumStructBasesInChainMinusOne(InBases.Num()-1)
	{
	}
#endif // !IS_MONOLITHIC

	consteval FStructBaseChain(TArrayView<const FStructBaseChain*> InBases)
		: StructBaseChainArray(InBases.GetData())
		, NumStructBasesInChainMinusOne(InBases.Num()-1)
	{
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

	COREUOBJECT_API void ReinitializeBaseChainArray();

	[[nodiscard]] inline bool IsChildOfUsingStructArray(const FStructBaseChain& Parent) const
	{
		int32 NumParentStructBasesInChainMinusOne = Parent.NumStructBasesInChainMinusOne;
		return NumParentStructBasesInChainMinusOne <= NumStructBasesInChainMinusOne && StructBaseChainArray[NumParentStructBasesInChainMinusOne] == &Parent;
	}

private:
	union
	{
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
		// Compiled in list of bases which may need runtime linking.
		UE::CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain>* CompiledInBases;
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
		// Runtime allocated or post-converstion compiled in list of bases
		const FStructBaseChain** StructBaseChainArray = nullptr;
	};
	int32 NumStructBasesInChainMinusOne = -1;

	friend class UStruct;
#endif // USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
};

namespace UE::Private
{
constexpr FStructBaseChain* AsStructBaseChain(UStruct* Struct);

template<typename... Classes>
struct TIntrinsicClassBaseChainImpl;
}

/*-----------------------------------------------------------------------------
	UStruct.
-----------------------------------------------------------------------------*/
enum class EStructStateFlags : uint16;

/** Provides more explicit control over if a property is added to PostConstruct & Destructor collections */
enum class EStructPropertyLinkFlags : uint8
{
	/** Property should not be added to any link lists */
	None = 0,

	/** If set, add the property to the post constructor link list */
	LinkPostConstruct = 1 << 0,

	/** If set, add the property to the destructor link list */
	LinkDestructor = 1 << 1,

	/** INTERNAL USE ONLY - If set, never respect the LinkDestructor flag.  This is needed for VNI native classes and can be removed once VNI is deprecated */
	NeverLinkDestructor_Internal = 1 << 2,
};
ENUM_CLASS_FLAGS(EStructPropertyLinkFlags)

/**
 * Base class for all UObject types that contain fields.
 */
#ifdef _MSC_VER
//error C4091: '__declspec(empty_bases)': ignored on left of 'UStruct' when no variable is declared
// In some configurations FStructBaseChain is empty, but in some it is not which will result in warning 4091.  empty_bases does nothing if there are no empty bases, so we can just suppress the warning.
#pragma warning(push) // 4091
#pragma warning(disable:4091)
#endif //#ifdef _MSC_VER
PLATFORM_EMPTY_BASES // FStructBaseChain may be empty
UCLASS(MinimalAPI, Config = Engine)
class UStruct : public UField
#if CPP // UHT objects to FStructBaseChain not being public because it assumes it might be a UInterface
	, private FStructBaseChain
#endif
{
	GENERATED_BODY()

	// Variables.
protected:
	// For construction of intrinsic class GC schema 
	friend struct Z_Construct_UClass_UStruct_Statics;
	// For access to struct base chain from internal code
	friend constexpr FStructBaseChain* UE::Private::AsStructBaseChain(UStruct*);
	// For access to struct base chain from possible non-internal intrinsic classes
	template<typename... Classes>
	friend struct UE::Private::TIntrinsicClassBaseChainImpl;
private:
	/** Struct this inherits from, may be null */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UStruct> SuperStruct;

public:
	/** Pointer to start of linked list of child fields */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UField> Children;
	
	/** Pointer to start of linked list of child fields */
	FField* ChildProperties;

	/** Total size of all UProperties, the allocated structure may be larger due to alignment */
	int32 PropertiesSize;
	/** Alignment of structure in memory, structure will be at least this large */
	int16 MinAlignment;

private:
	/** State flags -- Access via HasAllStructStateFlags() **/
	std::atomic<uint16> StructStateFlags = 0;

public:

	union
	{
		/** Script bytecode associated with this object */
		TArray<uint8> Script = {};
#if UE_WITH_CONSTINIT_UOBJECT && WITH_METADATA
		/** 
		 * Metadata parameters stored in this object at compile time from UHT
		 * This union member is active until full compiled-in object construction in ProcessNewlyLoadedUObjects
		 */
		TConstArrayView<UE::CodeGen::ConstInit::FMetaData> CompiledInMetaData;
#endif // UE_WITH_CONSTINIT_UOBJECT && WITH_METADATA
	};

	/** In memory only: Linked list of properties from most-derived to base */
	FProperty* PropertyLink;
	/** In memory only: Linked list of object reference properties from most-derived to base */
	FProperty* RefLink;
	/** In memory only: Linked list of properties requiring destruction. Note this does not include things that will be destroyed by the native destructor */
	FProperty* DestructorLink;
	/** In memory only: Linked list of properties requiring post constructor initialization */
	FProperty* PostConstructLink;

	/** Array of object references embedded in script code and referenced by FProperties. Mirrored for easy access by realtime garbage collection code */
	UPROPERTY(SkipSerialization)
	TArray<TObjectPtr<UObject>> ScriptAndPropertyObjectReferences;

	typedef TArray<TPair<TFieldPath<FField>, int32>> FUnresolvedScriptPropertiesArray;
	/** Contains a list of script properties that couldn't be resolved at load time */
	FUnresolvedScriptPropertiesArray* UnresolvedScriptProperties;

#if WITH_EDITORONLY_DATA
	/** List of wrapper UObjects for FProperties */
	UPROPERTY(SkipSerialization)
	TArray<TObjectPtr<UPropertyWrapper>> PropertyWrappers;
	/** Unique id incremented each time this class properties get destroyed */
	int32 FieldPathSerialNumber;

	/** Number of fields, inclusive of base fields and fixed-size array elements. Generated during Link(). */
	UE_INTERNAL int32 TotalFieldCount = 0;
#endif // WITH_EDITORONLY_DATA

	/** Cached schema for optimized unversioned and filtereditoronly property serialization, owned by this. */
	mutable std::atomic<const struct FUnversionedStructSchema*> UnversionedGameSchema = nullptr;
#if WITH_EDITORONLY_DATA
	/** Cached schema for optimized unversioned property serialization, with editor data, owned by this. */
	mutable std::atomic<const struct FUnversionedStructSchema*> UnversionedEditorSchema = nullptr;

	/** Get the Schema Hash for this struct - the hash of its property names and types. */
	COREUOBJECT_API const FBlake3Hash& GetSchemaHash(bool bSkipEditorOnly) const;

protected:
	/** True if this struct has Asset Registry searchable properties */
	bool bHasAssetRegistrySearchableProperties;
#endif // WITH_EDITORONLY_DATA

public:
	using FSuperStructIterator = TObjectPtrLinkedListIterator<UStruct, &UStruct::SuperStruct, true>;
	using FConstSuperStructIterator = TObjectPtrLinkedListIteratorConst<UStruct, &UStruct::SuperStruct, true>;

	// Constructors.
	UStruct(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
	{
	}
	COREUOBJECT_API UStruct( EStaticConstructor, int32 InSize, int32 InAlignment, EObjectFlags InFlags );
	COREUOBJECT_API explicit UStruct(UStruct* InSuperStruct, SIZE_T ParamsSize = 0, SIZE_T Alignment = 0);
	COREUOBJECT_API explicit UStruct(const FObjectInitializer& ObjectInitializer, UStruct* InSuperStruct, SIZE_T ParamsSize = 0, SIZE_T Alignment = 0 );
	COREUOBJECT_API virtual ~UStruct();
#if UE_WITH_CONSTINIT_UOBJECT
	consteval UStruct(
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FStructParams StructParams
	)
		: Super(ObjectParams, FieldParams)
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
		, FStructBaseChain(StructParams.BaseChain)
#endif
		, SuperStruct(StructParams.Super)
		, Children(StructParams.FirstChild)
		, ChildProperties(StructParams.ChildProperties)
		, PropertiesSize(StructParams.PropertiesSize)
		, MinAlignment(StructParams.MinAlignment)
#if WITH_METADATA
		, CompiledInMetaData(StructParams.MetaData)
#else
		, Script(ConstEval)
#endif
		, PropertyLink(nullptr)
		, RefLink(nullptr)
		, DestructorLink(nullptr)
		, PostConstructLink(nullptr)
		, ScriptAndPropertyObjectReferences(ConstEval)
		, UnresolvedScriptProperties(nullptr)
#if WITH_EDITORONLY_DATA
		, PropertyWrappers(ConstEval)
#endif
#if WITH_EDITORONLY_DATA
		, FieldPathSerialNumber(0)
		, bHasAssetRegistrySearchableProperties(false)
#endif
	{
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

	// UObject interface.
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	COREUOBJECT_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker) override;
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	COREUOBJECT_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	COREUOBJECT_API virtual void PostLoad() override;
	COREUOBJECT_API virtual void FinishDestroy() override;
#if !UE_WITH_CONSTINIT_UOBJECT
	COREUOBJECT_API virtual void RegisterDependencies() override;
#endif // !UE_WITH_CONSTINIT_UOBJECT
	static COREUOBJECT_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	COREUOBJECT_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;

	// UField interface.
	COREUOBJECT_API virtual void AddCppProperty(FProperty* Property) override;

	/**
	 * Returns struct path name as a package + struct FName pair
	 */
	COREUOBJECT_API FTopLevelAssetPath GetStructPathName() const;

	/** Searches property link chain for a property with the specified name */
	COREUOBJECT_API FProperty* FindPropertyByName(FName InName) const;

	 /** Internal API that searches the property link chain for a property with the specified offset */
	UE_INTERNAL COREUOBJECT_API FProperty* FindPropertyByOffset(int32 Offset) const;

	/**
	 * Creates new copies of components
	 * 
	 * @param	Data						pointer to the address of the subobject referenced by this FProperty
	 * @param	DefaultData					pointer to the address of the default value of the subbject referenced by this FProperty
	 * @param	DefaultStruct				the struct corresponding to the buffer pointed to by DefaultData
	 * @param	Owner						the object that contains the component currently located at Data
	 * @param	InstanceGraph				contains the mappings of instanced objects and components to their templates
	 */
	COREUOBJECT_API void InstanceSubobjectTemplates(TNotNull<void*> Data, const void* DefaultData, const UStruct* DefaultStruct, TNotNull<UObject*> Owner, FObjectInstancingGraph* InstanceGraph);

	/** Returns the structure used for inheritance, may be changed by child types */
	virtual UStruct* GetInheritanceSuper() const {return GetSuperStruct();}

	/** Static wrapper for Link, using a dummy archive */
	COREUOBJECT_API void StaticLink(bool bRelinkExistingProperties = false);

	/** Creates the field/property links and gets structure ready for use at runtime */
	COREUOBJECT_API virtual void Link(FArchive& Ar, bool bRelinkExistingProperties);

	/**
	 * Serializes struct properties, does not handle defaults.  See SerializeBinEx for handling defaults.
	 *
	 * @param	Ar				the archive to use for serialization
	 * @param	Data			pointer to the location of the beginning of the property data
	 *
	 * @note Binary serialization will read and write unstructured data from the archive.  As deprecated
	 *       properties are read from archives but not written, it is dangerous to call this function on
	 *       types with deprecated properties, unless the ArWantBinarySerialization flag is set on the
	 *       archive to force serialization to occur always.
	 */
	virtual void SerializeBin(FArchive& Ar, TNotNull<void*> Data) const final 
	{
		SerializeBin(FStructuredArchiveFromArchive(Ar).GetSlot(), Data);
	}

	/**
	 * Serializes struct properties, does not handle defaults.  See SerializeBinEx for handling defaults.
	 *
	 * @param	Slot			The structured archive slot we are serializing to
	 * @param	Data			pointer to the location of the beginning of the property data
	 *
	 * @note Binary serialization will read and write unstructured data from the archive.  As deprecated
	 *       properties are read from archives but not written, it is dangerous to call this function on
	 *       types with deprecated properties, unless the ArWantBinarySerialization flag is set on the
	 *       archive to force serialization to occur always.
	 */
	COREUOBJECT_API virtual void SerializeBin(FStructuredArchive::FSlot Slot, TNotNull<void*> Data) const;

	/**
	 * Serializes the class properties that reside in Data if they differ from the corresponding values in DefaultData
	 *
	 * @param	Slot			The structured archive slot we are serializing to
	 * @param	Data			pointer to the location of the beginning of the property data
	 * @param	DefaultData		pointer to the location of the beginning of the data that should be compared against
	 * @param	DefaultStruct	the struct corresponding to the block of memory located at DefaultData 
	 *
	 * @note Binary serialization will read and write unstructured data from the archive.  As deprecated
	 *       properties are read from archives but not written, it is dangerous to call this function on
	 *       types with deprecated properties, unless the ArWantBinarySerialization flag is set on the
	 *       archive to force serialization to occur always.
	 */
	COREUOBJECT_API void SerializeBinEx( FStructuredArchive::FSlot Slot, TNotNull<void*> Data, void const* DefaultData, UStruct* DefaultStruct ) const;

	/** Serializes list of properties, using property tags to handle mismatches */
	virtual void SerializeTaggedProperties(FArchive& Ar, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad = nullptr) const final 
	{
		SerializeTaggedProperties(FStructuredArchiveFromArchive(Ar).GetSlot(), Data, DefaultsStruct, Defaults, BreakRecursionIfFullyLoad);
	}

	/** Serializes list of properties, using property tags to handle mismatches */
	COREUOBJECT_API virtual void SerializeTaggedProperties(FStructuredArchive::FSlot Slot, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad = nullptr) const;

	/**
	 * Preloads all fields that belong to this struct
	 * @param Ar Archive used for loading this struct
	 */
	COREUOBJECT_API virtual void PreloadChildren(FArchive& Ar);

	/**
	 * Initialize a struct over uninitialized memory. This may be done by calling the native constructor or individually initializing properties
	 *
	 * @param	Dest		Pointer to memory to initialize
	 * @param	ArrayDim	Number of elements in the array
	 * @param	Stride		Stride of the array, If this default (0), then we will pull the size from the struct
	 */
	COREUOBJECT_API virtual void InitializeStruct(void* Dest, int32 ArrayDim = 1) const;
	
	/**
	 * Destroy a struct in memory. This may be done by calling the native destructor or individually destroying properties
	 *
	 * @param	Dest		Pointer to memory to destory
	 * @param	ArrayDim	Number of elements in the array
	 * @param	Stride		Stride of the array. If this default (0), then we will pull the size from the struct
	 */
	COREUOBJECT_API virtual void DestroyStruct(void* Dest, int32 ArrayDim = 1) const;

#if DO_CHECK
	/** Returns true if this object has been initialized to the point that its property chain data is available. */
	bool DebugIsPropertyChainReady() const;
#endif

	/** Look up a property by an alternate name if it was not found in the first search, this is overridden for user structs */
	virtual FProperty* CustomFindProperty(const FName InName) const { return nullptr; };

	/** Serialize an expression to an archive. Returns expression token */
	COREUOBJECT_API virtual EExprToken SerializeExpr(int32& iCode, FArchive& Ar);

	/**
	 * Returns the struct/ class prefix used for the C++ declaration of this struct/ class.
	 *
	 * @return Prefix character used for C++ declaration of this struct/ class.
	 */
	virtual const TCHAR* GetPrefixCPP() const { return TEXT("F"); }

	/** Total size of all UProperties, the allocated structure may be larger due to alignment */
	UE_FORCEINLINE_HINT int32 GetPropertiesSize() const
	{
		return PropertiesSize;
	}

	/** Alignment of structure in memory, structure will be at least this large */
	UE_FORCEINLINE_HINT int32 GetMinAlignment() const
	{
		return MinAlignment;
	}

	/** Returns actual allocated size of structure in memory */
	UE_FORCEINLINE_HINT int32 GetStructureSize() const
	{
		return Align(PropertiesSize,MinAlignment);
	}

	/** Modifies the property size after it's been recomputed */
	void SetPropertiesSize( int32 NewSize )
	{
		PropertiesSize = NewSize;
	}

	/** Returns true if this struct either is class T, or is a child of class T. */
	template<class T>
	bool IsChildOf() const
	{
		if constexpr (TModels_V<CStaticClassProvider, T>)
		{
			return IsChildOf(T::StaticClass());
		}
		else if constexpr (TModels_V<CStaticStructProvider, T>)
		{
			return IsChildOf(T::StaticStruct());
		}
		else
		{
			static_assert(sizeof(T) == 0, "Unsupported type - requires StaticClass or StaticStruct.");
			return false;
		}
	}

	/** Returns true if this struct either is SomeBase, or is a child of SomeBase. */
#if USTRUCT_FAST_ISCHILDOF_COMPARE_WITH_OUTERWALK || USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_OUTERWALK
	COREUOBJECT_API bool IsChildOf( const UStruct* SomeBase ) const;
#else
	bool IsChildOf(const UStruct* SomeBase) const
	{
		return (SomeBase ? IsChildOfUsingStructArray(*SomeBase) : false);
	}
#endif

	/** Returns an iterate to traverse the super chain without doing TObjectPtr access tracking, starting with this struct */
	FSuperStructIterator GetSuperStructIterator()
	{
		return FSuperStructIterator(this);
	}
	/** Returns an iterate to traverse the super chain without doing TObjectPtr access tracking, starting with this struct */
	FConstSuperStructIterator GetSuperStructIterator() const
	{
		return FConstSuperStructIterator(this);
	}

	/** Struct this inherits from, may be null */
	UStruct* GetSuperStruct() const
	{
		return SuperStruct;
	}

	/**
	 * Sets the super struct pointer and updates hash information as necessary.
	 * Note that this is not sufficient to actually reparent a struct, it simply sets a pointer.
	 */
	COREUOBJECT_API virtual void SetSuperStruct(UStruct* NewSuperStruct);

	/** Returns a human readable string for a given field, overridden for user defined structs */
	COREUOBJECT_API virtual FString GetAuthoredNameForField(const UField* Field) const;

	/** Returns a human readable string for a given field, overridden for user defined structs */
	COREUOBJECT_API virtual FString GetAuthoredNameForField(const FField* Field) const;

	/** If true, this class has been cleaned and sanitized (trashed) and should not be used */
	virtual bool IsStructTrashed() const
	{
		return false;
	}

	/** Destroys all properties owned by this struct */
	COREUOBJECT_API void DestroyChildPropertiesAndResetPropertyLinks();

#if WITH_METADATA
	/** Try and find boolean metadata with the given key. If not found on this class, work up hierarchy looking for it. */
	COREUOBJECT_API bool GetBoolMetaDataHierarchical(const FName& Key) const;

	/** Try and find string metadata with the given key. If not found on this class, work up hierarchy looking for it. */
	COREUOBJECT_API bool GetStringMetaDataHierarchical(const FName& Key, FString* OutValue = nullptr) const;

	/**
	 * Determines if the struct or any of its super structs has any metadata associated with the provided key
	 *
	 * @param Key The key to lookup in the metadata
	 * @return pointer to the UStruct that has associated metadata, nullptr if Key is not associated with any UStruct in the hierarchy
	 */
	COREUOBJECT_API const UStruct* HasMetaDataHierarchical(const FName& Key) const;
#endif // WITH_METADATA

#if WITH_EDITORONLY_DATA
	/**
	 * Try to activate tracking of this type of flag for the instance.
	 *
	 * @return Whether the instance is tracking this type of flag.
	 */
	UE_INTERNAL COREUOBJECT_API virtual bool ActivateTrackingPropertyValueFlag(EPropertyValueFlags Flags, void* Data) const;

	/** Query whether the instance is tracking this type of flag. */
	UE_INTERNAL COREUOBJECT_API virtual bool IsTrackingPropertyValueFlag(EPropertyValueFlags Flags, const void* Data) const;

	/** Query whether the instance has the flag set for the property. True if tracking is inactive. */
	UE_INTERNAL COREUOBJECT_API virtual bool HasPropertyValueFlag(EPropertyValueFlags Flags, const void* Data, const FProperty* Property, int32 ArrayIndex = 0) const;

	/** Set the value of the flag for the property on the instance. Ignored if tracking is inactive. */
	UE_INTERNAL COREUOBJECT_API virtual void SetPropertyValueFlag(EPropertyValueFlags Flags, bool bValue, void* Data, const FProperty* Property, int32 ArrayIndex = 0) const;

	/** Reset the flag to false for every property on the instance. */
	UE_INTERNAL COREUOBJECT_API virtual void ResetPropertyValueFlags(EPropertyValueFlags Flags, void* Data) const;

	/** Serialize the flag for every property on the instance. NOT FOR PERSISTENT SERIALIZATION! */
	UE_INTERNAL COREUOBJECT_API virtual void SerializePropertyValueFlags(EPropertyValueFlags Flags, void* Data, FStructuredArchiveRecord Record, FArchiveFieldName Name) const;

	/* Returns true if this struct has Asset Registry searchable properties */
	UE_FORCEINLINE_HINT bool HasAssetRegistrySearchableProperties() const
	{
		return bHasAssetRegistrySearchableProperties;
	}
#endif // WITH_EDITORONLY_DATA

	/** Sets the UnresolvedScriptProperties array */
	void SetUnresolvedScriptProperties(FUnresolvedScriptPropertiesArray& InUnresolvedProperties)
	{
		if (!UnresolvedScriptProperties)
		{
			UnresolvedScriptProperties = new FUnresolvedScriptPropertiesArray();
		}
		*UnresolvedScriptProperties = MoveTemp(InUnresolvedProperties);
	}

	/** Deletes the UnresolvedScriptProperties array */
	inline void DeleteUnresolvedScriptProperties()
	{
		if (UnresolvedScriptProperties)
		{
			delete UnresolvedScriptProperties;
			UnresolvedScriptProperties = nullptr;
		}
	}

	/**
	 * Collects UObjects referenced by bytecode
	 * @param OutReferencedObjects buffer to store the referenced objects in (not cleared by this function)
	 */
	COREUOBJECT_API void CollectBytecodeReferencedObjects(TArray<UObject*>& OutReferencedObjects);
	/**
	 * Collects UObjects referenced by properties
	 * @param OutReferencedObjects buffer to store the referenced objects in (not cleared by this function)
	 */
	COREUOBJECT_API void CollectPropertyReferencedObjects(TArray<UObject*>& OutReferencedObjects);
	/**
	 * Collects UObjects referenced by bytecode and properties for faster GC access
	 */
	COREUOBJECT_API void CollectBytecodeAndPropertyReferencedObjects();
	/**
	 * Collects UObjects referenced by bytecode and properties for this class and its child fields and their children...
	 */
	COREUOBJECT_API void CollectBytecodeAndPropertyReferencedObjectsRecursively();

	UE_DEPRECATED(5.7, "Visit is deprecated, please use Visit with context instead.")
	COREUOBJECT_API EPropertyVisitorControlFlow Visit(void* Data, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorPath& /*Path*/, const FPropertyVisitorData& /*Data*/)> InFunc) const;

	/**
	 * Visits this property and allows recursion into the inner properties
	 * This method allows callers to visit inner properties without knowing about its container type as opposed to TPropertyIterator.
	 * This visit property pattern facilitates the recursion into user defined properties and allows users to add specific visit logic on UStruct via traits. 
	 * @param Data to the property to visit
	 * @param InFunc to call on each visited property, the return value controls what is the next behavior once this property has been visited
	 * @return the new action to take one visited this property
	 */
	COREUOBJECT_API EPropertyVisitorControlFlow Visit(void* Data, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc, FPropertyVisitorContext::EScope InScope = FPropertyVisitorContext::EScope::All) const;

	/**
	 * Visits this property and allows recursion into the inner properties
	 * This method allows callers to visit inner properties without knowing about its container type as opposed to TPropertyIterator.
	 * This visit property pattern facilitates the recursion into user defined properties and allows users to add specific visit logic on UStruct via traits. 
	 * @param Context which contains the path that was computed until we reached this property and the data to the property to visit
	 * @param InFunc to call on each visited property, the return value controls what is the next behavior once this property has been visited
	 * @return the new action to take one visited this property
	 */
	COREUOBJECT_API virtual EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const;

	/**
	 * Attempt to resolve the given inner path info against this outer struct to get the inner property value.
	 * @return The inner property value, or null if the path info is incompatible/missing on this struct.
	 */
	COREUOBJECT_API virtual void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const;

	/** Returns the property name from the guid */
	virtual FName FindPropertyNameFromGuid(const FGuid& PropertyGuid) const { return NAME_None; }

	/** Find property guid */
	virtual FGuid FindPropertyGuidFromName(const FName InName) const { return FGuid(); }

	/** Returns if we have access to property guids */
	virtual bool ArePropertyGuidsAvailable() const { return false; }

protected:

	/** Serializes properties of this struct */
	COREUOBJECT_API void SerializeProperties(FArchive& Ar);

#if WITH_EDITORONLY_DATA
	COREUOBJECT_API void ConvertUFieldsToFFields();
#endif // WITH_EDITORONLY_DATA

	/** Serializes list of properties to a te, using property tags to handle mismatches */
	COREUOBJECT_API void LoadTaggedPropertiesFromText(FStructuredArchive::FSlot Slot, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad) const;

private:
#if USTRUCT_FAST_ISCHILDOF_IMPL == USTRUCT_ISCHILDOF_STRUCTARRAY
	// For UObjectBaseUtility
	friend class UObjectBaseUtility;
	using FStructBaseChain::IsChildOfUsingStructArray;
	using FStructBaseChain::ReinitializeBaseChainArray;

	friend class FStructBaseChain;
	friend class FBlueprintCompileReinstancer;
#endif

	/** This method is private to discourage extension of the StructStateFlags. Returns true if StructStateFlags contains all the TestStructStateFlags specified. */
	bool HasAllStructStateFlags(EStructStateFlags TestStructStateFlags) const;

	void SerializeVersionedTaggedProperties(FStructuredArchive::FSlot Slot, TNotNull<uint8*> Data, const UStruct* DefaultsStruct, const uint8* Defaults, const UObject* BreakRecursionIfFullyLoad) const;
};
#ifdef _MSC_VER
#pragma warning(pop) // 4091
#endif

#if UE_WITH_CONSTINIT_UOBJECT
// Private access to protected base class for UHT generated code
inline constexpr FStructBaseChain* UE::Private::AsStructBaseChain(UStruct* Struct)
{
	return Struct;
}
#endif // UE_WITH_CONSTINIT_UOBJECT

/**
 * Flags describing a struct
 * 
 * This MUST be kept in sync with EStructFlags defined in
 * Engine\Source\Programs\Shared\EpicGames.Core\UnrealEngineTypes.cs
 */
enum EStructFlags
{
	// State flags.
	STRUCT_NoFlags				= 0x00000000,	
	STRUCT_Native				= 0x00000001,

	/** If set, this struct will be compared using native code */
	STRUCT_IdenticalNative		= 0x00000002,
	
	STRUCT_HasInstancedReference= 0x00000004,

	STRUCT_NoExport				= 0x00000008,

	/** Indicates that this struct should always be serialized as a single unit */
	STRUCT_Atomic				= 0x00000010,

	/** Indicates that this struct uses binary serialization; it is unsafe to add/remove members from this struct without incrementing the package version */
	STRUCT_Immutable			= 0x00000020,

	/** If set, native code needs to be run to find referenced objects */
	STRUCT_AddStructReferencedObjects = 0x00000040,

	/** Indicates that this struct should be exportable/importable at the DLL layer.  Base structs must also be exportable for this to work. */
	STRUCT_RequiredAPI			= 0x00000200,	

	/** If set, this struct will be serialized using the CPP net serializer */
	STRUCT_NetSerializeNative	= 0x00000400,	

	/** If set, this struct will be serialized using the CPP serializer */
	STRUCT_SerializeNative		= 0x00000800,	

	/** If set, this struct will be copied using the CPP operator= */
	STRUCT_CopyNative			= 0x00001000,	

	/** If set, this struct will be copied using memcpy */
	STRUCT_IsPlainOldData		= 0x00002000,	

	/** If set, this struct has no destructor and non will be called. STRUCT_IsPlainOldData implies STRUCT_NoDestructor */
	STRUCT_NoDestructor			= 0x00004000,	

	/** If set, this struct will not be constructed because it is assumed that memory is zero before construction. */
	STRUCT_ZeroConstructor		= 0x00008000,	

	/** If set, native code will be used to export text */
	STRUCT_ExportTextItemNative	= 0x00010000,	

	/** If set, native code will be used to export text */
	STRUCT_ImportTextItemNative	= 0x00020000,	

	/** If set, this struct will have PostSerialize called on it after CPP serializer or tagged property serialization is complete */
	STRUCT_PostSerializeNative  = 0x00040000,

	/** If set, this struct will have SerializeFromMismatchedTag called on it if a mismatched tag is encountered. */
	STRUCT_SerializeFromMismatchedTag = 0x00080000,

	/** If set, this struct will be serialized using the CPP net delta serializer */
	STRUCT_NetDeltaSerializeNative = 0x00100000,

	/** If set, this struct will be have PostScriptConstruct called on it after a temporary object is constructed in a running blueprint */
	STRUCT_PostScriptConstruct     = 0x00200000,

	/** If set, this struct can share net serialization state across connections */
	STRUCT_NetSharedSerialization = 0x00400000,

	/** If set, this struct has been cleaned and sanitized (trashed) and should not be used */
	STRUCT_Trashed = 0x00800000,

	/** If set, this structure has been replaced via reinstancing */
	STRUCT_NewerVersionExists = 0x01000000,

	/** If set, this struct will have CanEditChange on it in the editor to determine if a child property can be edited */
	STRUCT_CanEditChange = 0x02000000,

	/** If set, this struct will have Visit on it to allow custom property visiting implementation */
	STRUCT_Visitor = 0x04000000,

	/** If set, this struct will have PostLoad on it in the editor. */
	STRUCT_PostLoad UE_INTERNAL = 0x08000000,

	/** If set, this struct will be move-assigned using the CPP operator=(T&&) */
	STRUCT_MoveAssignNative		= 0x10000000,

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EStructFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

/** Struct flags that are automatically inherited */
#define STRUCT_Inherit			(STRUCT_HasInstancedReference | STRUCT_Atomic)

/** Flags that are always computed, never loaded or done with code generation */
#define STRUCT_ComputedFlags	(STRUCT_NetDeltaSerializeNative | STRUCT_NetSerializeNative | STRUCT_SerializeNative | STRUCT_PostSerializeNative | STRUCT_CopyNative | STRUCT_MoveAssignNative | STRUCT_IsPlainOldData | STRUCT_NoDestructor | STRUCT_ZeroConstructor | STRUCT_IdenticalNative | STRUCT_AddStructReferencedObjects | STRUCT_ExportTextItemNative | STRUCT_ImportTextItemNative | STRUCT_SerializeFromMismatchedTag | STRUCT_PostScriptConstruct | STRUCT_NetSharedSerialization)


#if CHECK_PUREVIRTUALS
#define DISABLE_ABSTRACT_CONSTRUCT TStructOpsTypeTraits<CPPSTRUCT>::WithPureVirtual
#else
#define DISABLE_ABSTRACT_CONSTRUCT (false && TStructOpsTypeTraits<CPPSTRUCT>::WithPureVirtual)
#endif


	/**
	 * Selection of AddStructReferencedObjects check.
	 */
	template<class CPPSTRUCT>
	inline void AddStructReferencedObjectsOrNot(void* A, FReferenceCollector& Collector)
	{
		if constexpr (TStructOpsTypeTraits<CPPSTRUCT>::WithAddStructReferencedObjects)
		{
			((CPPSTRUCT*)A)->AddStructReferencedObjects(Collector);
		}
	}

namespace UE::CodeGen::ConstInit
{
	struct FScriptStructParams;
}

namespace UE::CoreUObject::Private
{
	/** Filled by implementation classes to report their capabilities */
	struct FCapabilities
	{
		EPropertyFlags ComputedPropertyFlags;
		EPropertyObjectReferenceType HasSerializerObjectReferences;
		bool HasNoopConstructor : 1;
		bool HasZeroConstructor : 1;
		bool HasDestructor : 1;
		bool HasSerializer : 1;
		bool HasStructuredSerializer : 1;
		bool HasPostSerialize : 1;
		bool HasNetSerializer : 1;
		bool HasNetSharedSerialization : 1;
		bool HasNetDeltaSerializer : 1;
		bool HasPostScriptConstruct : 1;
		bool IsPlainOldData : 1;
		bool IsUECoreType : 1;
		bool IsUECoreVariant : 1;
		bool HasCopy : 1;
		bool HasMoveAssign : 1;
		bool HasIdentical : 1;
		bool HasExportTextItem : 1;
		bool HasImportTextItem : 1;
		bool HasAddStructReferencedObjects : 1;
		bool HasSerializeFromMismatchedTag : 1;
		bool HasStructuredSerializeFromMismatchedTag : 1;
		bool HasGetTypeHash : 1;
		bool HasIntrusiveUnsetOptionalState : 1;
		bool IsAbstract : 1;
		bool HasFindInnerPropertyInstance : 1;
		bool ClearOnFinishDestroy : 1;
#if WITH_EDITOR
		bool HasCanEditChange : 1;
		/*UE_INTERNAL*/ bool HasPostLoad : 1;
#endif
		bool HasVisitor : 1;
		bool HasGetPreloadDependencies : 1;
		bool IsIntrusiveOptionalSafeForGC : 1;
	};

	template <typename CppStruct>
	constexpr FCapabilities GetCapabilities()
	{
		using TraitsType = TStructOpsTypeTraits<CppStruct>;

		constexpr FCapabilities Capabilities {
			.ComputedPropertyFlags =
				(TIsPODType<CppStruct>::Value ? CPF_IsPlainOldData : CPF_None) |
				(std::is_trivially_destructible_v<CppStruct> ? CPF_NoDestructor : CPF_None) |
				(TIsZeroConstructType<CppStruct>::Value ? CPF_ZeroConstructor : CPF_None) |
				(TModels_V<CGetTypeHashable, CppStruct> ? CPF_HasGetValueTypeHash : CPF_None),
			.HasSerializerObjectReferences				= TraitsType::WithSerializerObjectReferences,
			.HasNoopConstructor							= TraitsType::WithNoInitConstructor,
			.HasZeroConstructor							= TraitsType::WithZeroConstructor,
			.HasDestructor								= !(TraitsType::WithNoDestructor || TIsPODType<CppStruct>::Value),
			.HasSerializer								= TraitsType::WithSerializer,
			.HasStructuredSerializer					= TraitsType::WithStructuredSerializer,
			.HasPostSerialize							= TraitsType::WithPostSerialize,
			.HasNetSerializer							= TraitsType::WithNetSerializer,
			.HasNetSharedSerialization					= TraitsType::WithNetSharedSerialization,
			.HasNetDeltaSerializer						= TraitsType::WithNetDeltaSerializer,
			.HasPostScriptConstruct						= TraitsType::WithPostScriptConstruct,
			.IsPlainOldData								= TIsPODType<CppStruct>::Value,
			.IsUECoreType								= TIsUECoreType<CppStruct>::Value,
			.IsUECoreVariant							= TIsUECoreVariant<CppStruct>::Value,
			.HasCopy									= TraitsType::WithCopy,
			.HasMoveAssign								= TraitsType::WithMoveAssign,
			.HasIdentical								= TraitsType::WithIdentical || TraitsType::WithIdenticalViaEquality,
			.HasExportTextItem							= TraitsType::WithExportTextItem,
			.HasImportTextItem							= TraitsType::WithImportTextItem,
			.HasAddStructReferencedObjects				= TraitsType::WithAddStructReferencedObjects,
			.HasSerializeFromMismatchedTag				= TraitsType::WithSerializeFromMismatchedTag,
			.HasStructuredSerializeFromMismatchedTag	= TraitsType::WithStructuredSerializeFromMismatchedTag,
			.HasGetTypeHash								= TModels_V<CGetTypeHashable, CppStruct>,
			.HasIntrusiveUnsetOptionalState				= ::HasIntrusiveUnsetOptionalState<CppStruct>(),
			.IsAbstract									= TIsAbstract<CppStruct>::Value,
			.HasFindInnerPropertyInstance				= TraitsType::WithFindInnerPropertyInstance,
			.ClearOnFinishDestroy						= TraitsType::WithClearOnFinishDestroy,
#if WITH_EDITOR
			.HasCanEditChange							= TraitsType::WithCanEditChange,
			PRAGMA_DISABLE_INTERNAL_WARNINGS
			.HasPostLoad								= TraitsType::WithPostLoad,
			PRAGMA_ENABLE_INTERNAL_WARNINGS
#endif
			.HasVisitor									= TraitsType::WithVisitor,
			.HasGetPreloadDependencies					= TraitsType::WithGetPreloadDependencies,
			.IsIntrusiveOptionalSafeForGC				= TraitsType::WithIntrusiveOptionalSafeForGC,
		};
		return Capabilities;
	}

	template <typename CppStruct>
		requires (!TStructOpsTypeTraits<CppStruct>::WithZeroConstructor)
	void Construct(void* Dest)
	{
		using TraitsType = TStructOpsTypeTraits<CppStruct>;

#if CHECK_PUREVIRTUALS
		if constexpr (!TraitsType::WithPureVirtual)
#endif
		{
			if constexpr (TraitsType::WithNoInitConstructor)
			{
				::new (Dest) CppStruct(ForceInit);
			}
			else
			{
				::new (Dest) CppStruct();
			}
		}
	}

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
	template <typename CppStruct>
		requires (!TStructOpsTypeTraits<CppStruct>::WithZeroConstructor)
	void ConstructForTests(void* Dest)
	{
		using TraitsType = TStructOpsTypeTraits<CppStruct>;

#if CHECK_PUREVIRTUALS
		if constexpr (!TraitsType::WithPureVirtual)
#endif
		{
			if constexpr (TraitsType::WithNoInitConstructor)
			{
				::new (Dest) CppStruct(ForceInit);
			}
			else
			{
				// NOTE: unlike regular "Construct", no value init here! (missing "()")
				::new (Dest) CppStruct;
			}
		}
	}
#endif

	template <typename CppStruct>
		requires (!TStructOpsTypeTraits<CppStruct>::WithNoDestructor && !TIsPODType<CppStruct>::Value)
	void Destruct(void* Dest)
	{
		DestructItem((CppStruct*)Dest);
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithSerializer)
	bool SerializeArchive(FArchive& Ar, void* Data, UStruct* DefaultsStruct, const void* Defaults)
	{
		if constexpr (TModels_V<CStructSerializableWithDefaults, CppStruct>)
		{
			return ((CppStruct*)Data)->Serialize(Ar, DefaultsStruct, Defaults);
		}
		else
		{
			return ((CppStruct*)Data)->Serialize(Ar);
		}
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithStructuredSerializer)
	bool SerializeSlot(FStructuredArchive::FSlot Slot, void* Data, UStruct* DefaultsStruct, const void* Defaults)
	{
		if constexpr (TModels_V<CStructStructuredSerializableWithDefaults, CppStruct>)
		{
			return ((CppStruct*)Data)->Serialize(Slot, DefaultsStruct, Defaults);
		}
		else
		{
			return ((CppStruct*)Data)->Serialize(Slot);
		}
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithPostSerialize)
	void PostSerialize(const FArchive& Ar, void *Data)
	{
		((CppStruct*)Data)->PostSerialize(Ar);
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithNetSerializer)
	bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, void* Data)
	{
		return ((CppStruct*)Data)->NetSerialize(Ar, Map, bOutSuccess);
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithNetDeltaSerializer)
	bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParms, void* Data)
	{
		return ((CppStruct*)Data)->NetDeltaSerialize(DeltaParms);
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithPostScriptConstruct)
	void PostScriptConstruct(void* Data)
	{
		((CppStruct*)Data)->PostScriptConstruct();
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithGetPreloadDependencies)
	void GetPreloadDependencies(void* Data, TArray<UObject*>& OutDeps)
	{
		((CppStruct*)Data)->GetPreloadDependencies(OutDeps);
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithCopy)
	bool Copy(void* Dest, void const* Src, int32 ArrayDim)
	{
		static_assert((!TIsPODType<CppStruct>::Value), "You probably don't want custom copy for a POD type.");

		CppStruct* TypedDest = (CppStruct*)Dest;
		const CppStruct* TypedSrc  = (const CppStruct*)Src;

		for (; ArrayDim; --ArrayDim)
		{
			*TypedDest++ = *TypedSrc++;
		}
		return true;
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithMoveAssign)
	bool MoveAssign(void* Dest, void* Src, int32 ArrayDim)
	{
		static_assert((!TIsPODType<CppStruct>::Value), "You probably don't want custom move for a POD type.");

		CppStruct* TypedDest = (CppStruct*)Dest;
		CppStruct* TypedSrc  = (CppStruct*)Src;

		for (; ArrayDim; --ArrayDim)
		{
			*TypedDest++ = MoveTemp(*TypedSrc++);
		}
		return true;
	}

	// If you get an overload resolution error here, it's likely you've specified both WithIdentical and WithIdenticalViaEquality
	// in your TStructOpsTypeTraits, which is not supported.
	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithIdentical)
	bool Identical(const void* A, const void* B, uint32 PortFlags, bool& bOutResult)
	{
		bOutResult = ((const CppStruct*)A)->Identical((const CppStruct*)B, PortFlags);
		return true;
	}
	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithIdenticalViaEquality)
	bool Identical(const void* A, const void* B, uint32 PortFlags, bool& bOutResult)
	{
		bOutResult = (*(const CppStruct*)A == *(const CppStruct*)B);
		return true;
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithExportTextItem)
	bool ExportTextItem(FString& ValueStr, const void* PropertyValue, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope)
	{
		if (DefaultValue)
		{
			return ((const CppStruct*)PropertyValue)->ExportTextItem(ValueStr, *(const CppStruct*)DefaultValue, Parent, PortFlags, ExportRootScope);
		}
		else
		{
			TTypeCompatibleBytes<CppStruct> TmpDefaultValue;
			FMemory::Memzero(TmpDefaultValue.GetTypedPtr(), sizeof(CppStruct));
			if constexpr (!GetCapabilities<CppStruct>().HasZeroConstructor)
			{
				Construct<CppStruct>(TmpDefaultValue.GetTypedPtr());
			}

			const bool bResult = ((const CppStruct*)PropertyValue)->ExportTextItem(ValueStr, *(const CppStruct*)TmpDefaultValue.GetTypedPtr(), Parent, PortFlags, ExportRootScope);

			if constexpr (GetCapabilities<CppStruct>().HasDestructor)
			{
				Destruct<CppStruct>(TmpDefaultValue.GetTypedPtr());
			}

			return bResult;
		}
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithImportTextItem)
	bool ImportTextItem(const TCHAR*& Buffer, void* Data, int32 PortFlags, UObject* OwnerObject, FOutputDevice* ErrorText)
	{
		return ((CppStruct*)Data)->ImportTextItem(Buffer, PortFlags, OwnerObject, ErrorText);
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithFindInnerPropertyInstance)
	bool FindInnerPropertyInstance(FName PropertyName, const void* Data, const FProperty*& OutProp, const void*& OutData)
	{
		return ((CppStruct*)Data)->FindInnerPropertyInstance(PropertyName, OutProp, OutData);
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithAddStructReferencedObjects)
	void (*AddStructReferencedObjects())(void* A, FReferenceCollector& Collector)
	{
		return &AddStructReferencedObjectsOrNot<CppStruct>;
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithSerializeFromMismatchedTag)
	bool SerializeFromMismatchedTag(FPropertyTag const& Tag, FArchive& Ar, void* Data)
	{
		if constexpr (TIsUECoreType<CppStruct>::Value)
		{
			// Custom version of SerializeFromMismatchedTag for core types, which don't have access to FPropertyTag.
			FName StructName;
			if (Tag.GetType().GetName() == NAME_StructProperty && Tag.GetType().GetParameterCount() >= 1)
			{
				StructName = Tag.GetType().GetParameterName(0);
			}
			return ((CppStruct*)Data)->SerializeFromMismatchedTag(StructName, Ar);
		}
		else
		{
			return ((CppStruct*)Data)->SerializeFromMismatchedTag(Tag, Ar);
		}
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithStructuredSerializeFromMismatchedTag)
	bool StructuredSerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot, void* Data)
	{
		if constexpr (TIsUECoreType<CppStruct>::Value)
		{
			// Custom version of SerializeFromMismatchedTag for core types, which don't understand FPropertyTag.
			FName StructName;
			if (Tag.GetType().GetName() == NAME_StructProperty && Tag.GetType().GetParameterCount() >= 1)
			{
				StructName = Tag.GetType().GetParameterName(0);
			}
			return ((CppStruct*)Data)->SerializeFromMismatchedTag(StructName, Slot);
		}
		else
		{
			return ((CppStruct*)Data)->SerializeFromMismatchedTag(Tag, Slot);
		}
	}

	template <typename CppStruct>
		requires (TModels_V<CGetTypeHashable, CppStruct>)
	uint32 GetStructTypeHash(const void* Src)
	{
		return GetTypeHashHelper(*(const CppStruct*)Src);
	}

	template <typename CppStruct>
		requires (HasIntrusiveUnsetOptionalState<CppStruct>())
	void InitializeIntrusiveUnsetOptionalValue(void* Data)
	{
		new (Data) TOptional<CppStruct>();
	}

	template <typename CppStruct>
		requires (HasIntrusiveUnsetOptionalState<CppStruct>())
	bool IsIntrusiveOptionalValueSet(const void* Data)
	{
		return reinterpret_cast<const TOptional<CppStruct>*>(Data)->IsSet();
	}

	template <typename CppStruct>
		requires (HasIntrusiveUnsetOptionalState<CppStruct>())
	void ClearIntrusiveOptionalValue(void* Data)
	{
		reinterpret_cast<TOptional<CppStruct>*>(Data)->Reset();
	}

#if WITH_EDITOR
	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithCanEditChange)
	bool CanEditChange(const FEditPropertyChain& PropertyChain, const void* Data)
	{
		return ((const CppStruct*)Data)->CanEditChange(PropertyChain);
	}
	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithPostLoad)
	UE_INTERNAL void PostLoad(void* Data)
	{
		((CppStruct*)Data)->PostLoad();
	}
	#define UE_FOREACH_FAKE_VTABLE_FUNC_EDITOR_ONLY(...) __VA_ARGS__
#else
	#define UE_FOREACH_FAKE_VTABLE_FUNC_EDITOR_ONLY(...)
#endif // WITH_EDITOR

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
	#define UE_FOREACH_FAKE_VTABLE_FUNC_WITH_TESTS_ONLY(...) __VA_ARGS__
#else
	#define UE_FOREACH_FAKE_VTABLE_FUNC_WITH_TESTS_ONLY(...)
#endif

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithVisitor)
	EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& Context)> InFunc)
	{
		CppStruct* Struct = (CppStruct*)Context.Data.PropertyData;
		return Struct->Visit(Context, InFunc);
	}

	template <typename CppStruct>
		requires (!!TStructOpsTypeTraits<CppStruct>::WithVisitor)
	void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info)
	{
		return ((CppStruct*)Data)->ResolveVisitedPathInfo(Info);
	}

	#define UE_FOREACH_FAKE_VTABLE_FUNC(IndexNameFunctype) \
		IndexNameFunctype(0,	Construct,								void(void* Address)) \
		UE_FOREACH_FAKE_VTABLE_FUNC_WITH_TESTS_ONLY( \
		IndexNameFunctype(1,	ConstructForTests,						void(void* Address)) \
		) \
		IndexNameFunctype(2,	Destruct,								void(void* Address)) \
		IndexNameFunctype(3,	SerializeArchive,						bool(FArchive& Ar, void* Data, UStruct* DefaultsStruct, const void* Defaults)) \
		IndexNameFunctype(4,	SerializeSlot,							bool(FStructuredArchive::FSlot Slot, void* Data, UStruct* DefaultsStruct, const void* Defaults)) \
		IndexNameFunctype(5,	PostSerialize,							void(const FArchive& Ar, void* Data)) \
		IndexNameFunctype(6,	NetSerialize,							bool(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, void* Data)) \
		IndexNameFunctype(7,	NetDeltaSerialize,						bool(FNetDeltaSerializeInfo& DeltaParms, void* Data)) \
		IndexNameFunctype(8,	PostScriptConstruct,					void(void* Data)) \
		IndexNameFunctype(9,	GetPreloadDependencies,					void(void* Data, TArray<UObject*>& OutDeps)) \
		IndexNameFunctype(10,	Copy,									bool(void* Dest, void const* Src, int32 ArrayDim)) \
		IndexNameFunctype(11,	MoveAssign,								bool(void* Dest, void* Src, int32 ArrayDim)) \
		IndexNameFunctype(12,	Identical,								bool(const void* A, const void* B, uint32 PortFlags, bool& bOutResult)) \
		IndexNameFunctype(13,	ExportTextItem,							bool(FString& ValueStr, const void* PropertyValue, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope)) \
		IndexNameFunctype(14,	ImportTextItem,							bool(const TCHAR*& Buffer, void* Data, int32 PortFlags, UObject* OwnerObject, FOutputDevice* ErrorText)) \
		IndexNameFunctype(15,	FindInnerPropertyInstance,				bool(FName PropertyName, const void* Data, const FProperty*& OutProp, const void*& OutData)) \
		IndexNameFunctype(16,	AddStructReferencedObjects,				void(*())(void* A, FReferenceCollector& Collector)) \
		IndexNameFunctype(17,	SerializeFromMismatchedTag,				bool(FPropertyTag const& Tag, FArchive& Ar, void* Data)) \
		IndexNameFunctype(18,	StructuredSerializeFromMismatchedTag,	bool(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot, void* Data)) \
		IndexNameFunctype(19,	GetStructTypeHash,						uint32(const void* Src)) \
		IndexNameFunctype(20,	InitializeIntrusiveUnsetOptionalValue,	void(void* Data)) \
		IndexNameFunctype(21,	IsIntrusiveOptionalValueSet,			bool(const void* Data)) \
		IndexNameFunctype(22,	ClearIntrusiveOptionalValue,			void(void* Data)) \
		IndexNameFunctype(23,	Visit,									EPropertyVisitorControlFlow(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& Context)> InFunc)) \
		IndexNameFunctype(24,	ResolveVisitedPathInfo,					void*(void* Data, const FPropertyVisitorInfo& Info)) \
		UE_FOREACH_FAKE_VTABLE_FUNC_EDITOR_ONLY( \
		IndexNameFunctype(25,	CanEditChange,							bool(const FEditPropertyChain& PropertyChain, const void* Data)) \
		IndexNameFunctype(26,	PostLoad,								void(void* Data)) \
		)

	// These represent one function per optional function in the fake vtable.
	enum class EStructOpsFakeVTableFlags : uint32
	{
		None = 0,

		#define UE_VTABLE_ENUM(Index, Name, Functype) Name = 1u << Index,
		UE_FOREACH_FAKE_VTABLE_FUNC(UE_VTABLE_ENUM)
		#undef UE_VTABLE_ENUM
	};
	ENUM_CLASS_FLAGS(EStructOpsFakeVTableFlags)

	template <EStructOpsFakeVTableFlags Flag>
	struct THandlingFunctionType;
	template <typename CppStruct, EStructOpsFakeVTableFlags Flag>
	struct THandlingFunctionPtr;

	#define UE_VTABLE_FUNC(Index, Name, Functype) \
		template <> \
		struct THandlingFunctionType<EStructOpsFakeVTableFlags::Name> \
		{ \
			using Type = Functype; \
		}; \
		template <typename CppStruct> \
		struct THandlingFunctionPtr<CppStruct, EStructOpsFakeVTableFlags::Name> \
		{ \
			inline static constexpr TIdentity_T<Functype>* Value = &Name<CppStruct>; \
		};
	UE_FOREACH_FAKE_VTABLE_FUNC(UE_VTABLE_FUNC)
	#undef UE_VTABLE_FUNC

	template <typename CppStruct>
	constexpr EStructOpsFakeVTableFlags GetStructOpsFakeVTableFlags()
	{
		using TraitsType = TStructOpsTypeTraits<CppStruct>;

		EStructOpsFakeVTableFlags Result = EStructOpsFakeVTableFlags::None;

		#define UE_VTABLE_SETFLAG(Index, Name, Functype) \
			if constexpr (requires(TIdentity_T<Functype>*& PtrRef) { PtrRef = &Name<CppStruct>; }) /* assignment to RefPtr shouldn't be needed, but is a workaround for MSVC bug: https://developercommunity.microsoft.com/t/Compiler-fails-overload-resolution-when/10939568 */ \
			{ \
				Result |= EStructOpsFakeVTableFlags::Name; \
			}
		UE_FOREACH_FAKE_VTABLE_FUNC(UE_VTABLE_SETFLAG)
		#undef UE_VTABLE_SETFLAG

		return Result;
	}

	struct FStructOpsFakeVTable
	{
		// These flags will indicate which functions are present in the HandlingFunctions array.
		EStructOpsFakeVTableFlags Flags;

		/** Struct capabilities */
		FCapabilities Capabilities;

// MSVC doesn't like zero-sized arrays in the middle of structs
#ifndef _MSC_VER
		// One function pointer will exist in this array for every corresponding flag in Flags.
		// The function pointer types will be different, depending on the required signature, and
		// will need to be cast to the right type before being called.
		void (*HandlingFunctions[0])();
#endif

		// Retrieves a handling func associated with a particular flag.
		template <EStructOpsFakeVTableFlags FuncFlag>
		UE_REWRITE auto GetHandlingFunc() const
		{
			// Ensure a single flag is passed
			static_assert(EnumNumSetFlags(FuncFlag) == 1);

PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
			return (typename THandlingFunctionType<FuncFlag>::Type*)this->GetHandlingFuncImpl(FuncFlag);
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
		}

	private:
		void (*GetHandlingFuncImpl(EStructOpsFakeVTableFlags FuncFlag) const)()
		{
			// Ensure that the fake vtable actually contains an entry for the requested function
			check(!!(this->Flags & FuncFlag));

			// Mask off all higher flags
			uint32 StructFlagsInt = (uint32)this->Flags;
			StructFlagsInt &= (uint32)FuncFlag - 1;

			// The number of remaining flags will be the index of the function into the function table.
			// Hopefully this turns into a popcnt.
			uint32 Index = 0;
			while (StructFlagsInt != 0)
			{
				StructFlagsInt &= StructFlagsInt - 1;
				++Index;
			}

#ifdef _MSC_VER
			// Ensure that the end of the struct is where the start of the function pointer array in TStructOpsFakeVTable will be.
			static_assert(alignof(FStructOpsFakeVTable) == alignof(void(*)()));
			return ((void(**)())(this + 1))[Index];
#else
			return this->HandlingFunctions[Index];
#endif
		}
	};

	template <auto... Funcs>
	struct TStructOpsFakeVTable : public FStructOpsFakeVTable 
	{
		TTuple<decltype(Funcs)...> Functions;
	};

	template <>
	struct TStructOpsFakeVTable<> : public FStructOpsFakeVTable 
	{
	};

	// This represents an array of function pointers to handling functions and a header describing the capabilities
	// of the struct. Only one pointer exists per handling function. The index of the handling function is the index
	// of the set bit in the capabilities.
	//
	// Example: a struct S containing HasDestructor and HasIdentical would be laid out like this in memory:
	//   ComputedPropertyFlags
    //   HasSerializerObjectReferences
	//   ...:                         false
	//   HasDestructor:               true
	//   ...:                         false
	//   HasIdentical:                true
	//   ...:                         false
	//   void(*HandlingFunctions[2]): { &Destruct<S>, &Identical<S> }
	//
	template <typename CppStruct, auto... Funcs>
	inline constexpr const TStructOpsFakeVTable<Funcs...> TStructOpsFakeVTableInstance = 
	{
		{
			GetStructOpsFakeVTableFlags<CppStruct>(),
			GetCapabilities<CppStruct>()
		},
		MakeTuple(Funcs...)
	};
	template <typename CppStruct>
	inline constexpr const TStructOpsFakeVTable<> TStructOpsFakeVTableInstance<CppStruct> = 
	{
		{
			GetStructOpsFakeVTableFlags<CppStruct>(),
			GetCapabilities<CppStruct>()
		}
	};

	template <typename CppStruct, EStructOpsFakeVTableFlags Flags, auto... Funcs>
	struct TStructOpsFakeVTableForTypeImpl : TStructOpsFakeVTableForTypeImpl<CppStruct, EnumRemoveLowestSetFlag(Flags), Funcs..., THandlingFunctionPtr<CppStruct, EnumLowestSetFlag(Flags)>::Value>
	{
	};

	template <typename CppStruct, auto... Funcs>
	struct TStructOpsFakeVTableForTypeImpl<CppStruct, EStructOpsFakeVTableFlags::None, Funcs...>
	{
		inline static constexpr const FStructOpsFakeVTable* Value = &TStructOpsFakeVTableInstance<CppStruct, Funcs...>;
	};

	template <typename CppStruct>
	using TStructOpsFakeVTableForType = TStructOpsFakeVTableForTypeImpl<CppStruct, GetStructOpsFakeVTableFlags<CppStruct>()>;
}

/**
 * Reflection data for a standalone structure declared in a header or as a user defined struct
 */
UCLASS(MinimalAPI, Config = Engine)
class UScriptStruct : public UStruct
{
	GENERATED_BODY()
public:
	/** Interface to template to manage dynamic access to C++ struct construction and destruction **/
	struct ICppStructOps
	{
		using FCapabilities = UE::CoreUObject::Private::FCapabilities;

		const UE::CoreUObject::Private::FStructOpsFakeVTable* FakeVPtr;

		/**
		 * Constructor
		 * @param InSize: sizeof() of the structure
		 */
		constexpr ICppStructOps(int32 InSize, int32 InAlignment)
			: Size(InSize)
			, Alignment(InAlignment)
		{
		}

		/** returns struct capabilities */
		const FCapabilities& GetCapabilities() const
		{
			return FakeVPtr->Capabilities;
		}

		/** return true if this class has a no-op constructor and takes EForceInit to init **/
		bool HasNoopConstructor() const
		{
			return GetCapabilities().HasNoopConstructor;
		}

		/** return true if memset can be used instead of the constructor **/
		bool HasZeroConstructor() const
		{
			return GetCapabilities().HasZeroConstructor;
		}

		/** Call the C++ constructor **/
		void Construct(void *Dest)
		{
			FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::Construct>()(Dest);
		}

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
		/** Call the C++ constructor without value-init (new T instead of new T()) **/
		void ConstructForTests(void* Dest)
		{
			FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::ConstructForTests>()(Dest);
		}
#endif

		/** return false if this destructor can be skipped **/
		bool HasDestructor() const
		{
			return GetCapabilities().HasDestructor;
		}

		/** Call the C++ destructor **/
		void Destruct(void *Dest)
		{
			FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::Destruct>()(Dest);
		}

		/** return the sizeof() of this structure **/
		UE_FORCEINLINE_HINT int32 GetSize() const
		{
			return Size;
		}

		/** return the alignof() of this structure **/
		UE_FORCEINLINE_HINT int32 GetAlignment() const
		{
			return Alignment;
		}

		/** return true if this class can serialize **/
		bool HasSerializer() const
		{
			return GetCapabilities().HasSerializer;
		}

		/** return true if this class can serialize to a structured archive**/
		bool HasStructuredSerializer() const
		{
			return GetCapabilities().HasStructuredSerializer;
		}

		/**
		 * Serialize this structure.
		 * @return true if serialization was handled, false to fall back to standard script struct serialization.
		 */
		bool Serialize(FArchive& Ar, void* Data, UStruct* DefaultsStruct, const void* Defaults)
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::SerializeArchive>()(Ar, Data, DefaultsStruct, Defaults);
		}
		bool Serialize(FStructuredArchive::FSlot Slot, void* Data, UStruct* DefaultsStruct, const void* Defaults)
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::SerializeSlot>()(Slot, Data, DefaultsStruct, Defaults);
		}

		/** return true if this class implements a post serialize call **/
		bool HasPostSerialize() const
		{
			return GetCapabilities().HasPostSerialize;
		}

		/** Call PostSerialize on this structure */
		void PostSerialize(const FArchive& Ar, void *Data)
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::PostSerialize>()(Ar, Data);
		}

		/** return true if this struct can net serialize **/
		bool HasNetSerializer() const
		{
			return GetCapabilities().HasNetSerializer;
		}
		
		/** return true if this can share net serialization across connections */
		bool HasNetSharedSerialization() const
		{
			return GetCapabilities().HasNetSharedSerialization;
		}

		/** 
		 * Net serialize this structure 
		 * @return true if the struct was serialized, otherwise it will fall back to ordinary script struct net serialization
		 */
		bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, void *Data)
		{
			return HasNetSerializer() && FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::NetSerialize>()(Ar, Map, bOutSuccess, Data);
		}

		/** return true if this struct can net delta serialize delta (serialize a network delta from a base state) **/
		bool HasNetDeltaSerializer() const
		{
			return GetCapabilities().HasNetDeltaSerializer;
		}

		/** 
		 * Net serialize delta this structure. Serialize a network delta from a base state
		 * @return true if the struct was serialized, otherwise it will fall back to ordinary script struct net delta serialization
		 */
		bool NetDeltaSerialize(FNetDeltaSerializeInfo & DeltaParms, void *Data)
		{
			return HasNetDeltaSerializer() && FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::NetDeltaSerialize>()(DeltaParms, Data);
		}

		/** return true if this class implements a post script construct call **/
		bool HasPostScriptConstruct() const
		{
			return GetCapabilities().HasPostScriptConstruct;
		}

		/** Call PostScriptConstruct on this structure */
		void PostScriptConstruct(void *Data)
		{
			FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::PostScriptConstruct>()(Data);
		}

		/** return true if this struct has GetPreloadDependencies **/
		bool HasGetPreloadDependencies() const
		{
			return GetCapabilities().HasGetPreloadDependencies;
		}

		/** Call PreloadDependencies on this structure */
		void GetPreloadDependencies(void* Data, TArray<UObject*>& OutDeps)
		{
			if (HasGetPreloadDependencies())
			{
				FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::GetPreloadDependencies>()(Data, OutDeps);
			}
		}

		/** return true if this struct should be memcopied **/
		bool IsPlainOldData() const
		{
			return GetCapabilities().IsPlainOldData;
		}

		/** return true if this struct is one of the UE Core types (and is include in CoreMinimal.h) **/
		bool IsUECoreType() const
		{
			return GetCapabilities().IsUECoreType;
		}

		/** return true if this struct is one of the UE Core types (and is include in CoreMinimal.h) **/
		bool IsUECoreVariant() const
		{
			return GetCapabilities().IsUECoreVariant;
		}

		/** return true if this struct can copy **/
		bool HasCopy() const
		{
			return GetCapabilities().HasCopy;
		}

		/** 
		 * Copy this structure 
		 * @return true if the copy was handled, otherwise it will fall back to CopySingleValue
		 */
		bool Copy(void* Dest, void const* Src, int32 ArrayDim)
		{
			return HasCopy() && FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::Copy>()(Dest, Src, ArrayDim);
		}

		/** return true if this struct can be move-assigned **/
		bool HasMoveAssign() const
		{
			return GetCapabilities().HasMoveAssign;
		}

		/**
		 * Move-assign this structure. Source is left in a valid but unspecified (moved-from) state.
		 * @return true if the move was handled, otherwise caller should fall back to CopyScriptStruct
		 */
		bool MoveAssign(void* Dest, void* Src, int32 ArrayDim)
		{
			return HasMoveAssign() && FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::MoveAssign>()(Dest, Src, ArrayDim);
		}

		/** return true if this struct can compare **/
		bool HasIdentical() const
		{
			return GetCapabilities().HasIdentical;
		}

		/** 
		 * Compare this structure
		 * @return true if the comparison was performed. bOutResult contains the result of the comparison.
		 */
		bool Identical(const void* A, const void* B, uint32 PortFlags, bool& bOutResult)
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::Identical>()(A, B, PortFlags, bOutResult);
		}

		/** return true if this struct can export **/
		bool HasExportTextItem() const
		{
			return GetCapabilities().HasExportTextItem;
		}

		/** 
		 * export this structure 
		 * @return true if the copy was exported, otherwise it will fall back to FStructProperty::ExportTextItem
		 */
		bool ExportTextItem(FString& ValueStr, const void* PropertyValue, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope)
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::ExportTextItem>()(ValueStr, PropertyValue, DefaultValue, Parent, PortFlags, ExportRootScope);
		}

		/** return true if this struct can import **/
		bool HasImportTextItem() const
		{
			return GetCapabilities().HasImportTextItem;
		}

		/** 
		 * import this structure 
		 * @return true if the copy was imported, otherwise it will fall back to FStructProperty::ImportText
		 */
		bool ImportTextItem(const TCHAR*& Buffer, void* Data, int32 PortFlags, UObject* OwnerObject, FOutputDevice* ErrorText)
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::ImportTextItem>()(Buffer, Data, PortFlags, OwnerObject, ErrorText);
		}

		bool HasFindInnerPropertyInstance() const
		{
			return GetCapabilities().HasFindInnerPropertyInstance;
		}

		bool FindInnerPropertyInstance(FName PropertyName, const void* Data, const FProperty*& OutProp, const void*& OutData) const
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::FindInnerPropertyInstance>()(PropertyName, Data, OutProp, OutData);
		}

		/** return true if this struct has custom GC code **/
		bool HasAddStructReferencedObjects() const
		{
			return GetCapabilities().HasAddStructReferencedObjects;
		}

		/** returns true if the native serialize functions may serialize object references of the given type **/
		bool HasSerializerObjectReferences(EPropertyObjectReferenceType Type) const
		{
			return EnumHasAnyFlags(GetCapabilities().HasSerializerObjectReferences, Type);
		}

		/** 
		 * return a pointer to a function that can add referenced objects
		 * @return true if the copy was imported, otherwise it will fall back to FStructProperty::ImportText
		 */
		using PointerToAddStructReferencedObjectsType = void (*)(void* A, FReferenceCollector& Collector);
		using TPointerToAddStructReferencedObjects UE_DEPRECATED(5.7, "TPointerToAddStructReferencedObjects has been deprecated, please use PointerToAddStructReferencedObjectsType instead.") = PointerToAddStructReferencedObjectsType;

		PointerToAddStructReferencedObjectsType AddStructReferencedObjects()
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::AddStructReferencedObjects>()();
		}

		/** return true if this class wants to serialize from some other tag (usually for conversion purposes) **/
		bool HasSerializeFromMismatchedTag() const
		{
			return GetCapabilities().HasSerializeFromMismatchedTag;
		}

		bool HasStructuredSerializeFromMismatchedTag() const
		{
			return GetCapabilities().HasStructuredSerializeFromMismatchedTag;
		}

		/** 
		 * Serialize this structure, from some other tag
		 * @return true if this succeeded, false will trigger a warning and not serialize at all
		 */
		bool SerializeFromMismatchedTag(FPropertyTag const& Tag, FArchive& Ar, void *Data)
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::SerializeFromMismatchedTag>()(Tag, Ar, Data);
		}

		bool StructuredSerializeFromMismatchedTag(FPropertyTag const& Tag, FStructuredArchive::FSlot Slot, void *Data)
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::StructuredSerializeFromMismatchedTag>()(Tag, Slot, Data);
		}

		/** return true if this struct has a GetTypeHash */
		bool HasGetTypeHash() const
		{
			return GetCapabilities().HasGetTypeHash;
		}

		/** Calls GetTypeHash if enabled */
		uint32 GetStructTypeHash(const void* Src)
		{
			if (HasGetTypeHash())
			{
				return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::GetStructTypeHash>()(Src);
			}
			else
			{
				return 0;
			}
		}

		/** Returns property flag values that can be computed at compile time */
		EPropertyFlags GetComputedPropertyFlags() const
		{
			return GetCapabilities().ComputedPropertyFlags;
		}

		/** return true if this struct is abstract **/
		bool IsAbstract() const
		{
			return GetCapabilities().IsAbstract;
		}

		/** return true if this struct has a ClearOnFinishDestroy */
		bool HasClearOnFinishDestroy() const
		{
			return GetCapabilities().ClearOnFinishDestroy;
		}

		/** Return true if this type can be constructed with FIntrusiveUnsetOptionalState for TOptional */
		bool HasIntrusiveUnsetOptionalState() const 
		{
			return GetCapabilities().HasIntrusiveUnsetOptionalState;
		}

		/** Construct an unset optional value */
		void InitializeIntrusiveUnsetOptionalValue(void* Data) const
		{
			FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::InitializeIntrusiveUnsetOptionalValue>()(Data);
		}

		/** Return true if the optional value at Data is in an unset state */
		bool IsIntrusiveOptionalValueSet(const void* Data) const
		{
			return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::IsIntrusiveOptionalValueSet>()(Data);
		}

		/** Reset an optional value to its unset state */
		void ClearIntrusiveOptionalValue(void* Data) const
		{
			FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::ClearIntrusiveOptionalValue>()(Data);
		}

		/** 
		 * Used for assertions only: confirms that this type has certified that its object reference fields are safe 
		 * for the GC to visit while the struct is in its intrusive unset optional state.
		*/
		bool IsIntrusiveOptionalSafeForGC() const
		{
			return GetCapabilities().IsIntrusiveOptionalSafeForGC;
		}

#if WITH_EDITOR
		/** Returns true if this struct wants to indicate whether a property can be edited in the details panel */
		bool HasCanEditChange() const
		{
			return GetCapabilities().HasCanEditChange;
		}

		/** Returns true if this struct would allow the given property to be edited in the details panel. */
		bool CanEditChange(const FEditPropertyChain& PropertyChain, const void* Data) const
		{
			return HasCanEditChange() && FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::CanEditChange>()(PropertyChain, Data);
		}

		/** Returns true if this struct support post load. */
		UE_INTERNAL bool HasPostLoad() const
		{
			return GetCapabilities().HasPostLoad;
		}

		/** PostLoad property PostLoad signature. */
		UE_INTERNAL void PostLoad(void* Data) const
		{
			PRAGMA_DISABLE_INTERNAL_WARNINGS
			if (HasPostLoad())
			{
				FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::PostLoad>()(Data);
			}
			PRAGMA_ENABLE_INTERNAL_WARNINGS
		}
#endif

		/** Returns true if this struct wants to indicate whether it has a custom impl for visiting properties */
		bool HasVisitor() const
		{
			return GetCapabilities().HasVisitor;
		}

		/** Structs property visitor signature */
		EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& Context)> InFunc) const
		{
			if (HasVisitor())
			{
				return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::Visit>()(Context, InFunc);
			}
			else
			{
				return EPropertyVisitorControlFlow::StepOver;
			}
		}

		void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const
		{
			if (HasVisitor())
			{
				return FakeVPtr->GetHandlingFunc<UE::CoreUObject::Private::EStructOpsFakeVTableFlags::ResolveVisitedPathInfo>()(Data, Info);
			}
			else
			{
				return nullptr;
			}
		}

	private:
		/** sizeof() of the structure **/
		const int32 Size;
		/** alignof() of the structure **/
		const int32 Alignment;
	};

	/** Template to manage dynamic access to C++ struct construction and destruction **/
	template<class CPPSTRUCT>
	struct TCppStructOps final : public ICppStructOps
	{
		typedef TStructOpsTypeTraits<CPPSTRUCT> TTraits;
		constexpr TCppStructOps()
			: ICppStructOps(sizeof(CPPSTRUCT), alignof(CPPSTRUCT))
		{
			static_assert(!(TTraits::WithAddStructReferencedObjects && alignof(CPPSTRUCT) < alignof(UObject*)), "AddStructReferencedObjects() requires pointer alignment");

			FakeVPtr = UE::CoreUObject::Private::TStructOpsFakeVTableForType<CPPSTRUCT>::Value;

			// We need some guarantees that TCppStructOps can be deleted via a ICppStructOps, until we fix up uses of TCppStructOps
			static_assert(sizeof(TCppStructOps) == sizeof(ICppStructOps));
			static_assert(std::is_trivially_destructible_v<TCppStructOps>);
			static_assert(!std::is_polymorphic_v<TCppStructOps>);
		}

		static_assert(!(TTraits::WithSerializeFromMismatchedTag && TTraits::WithStructuredSerializeFromMismatchedTag), "Structs cannot have both WithSerializeFromMismatchedTag and WithStructuredSerializeFromMismatchedTag set");
	};

	/** Template for noexport classes to autoregister before main starts **/
	template<class CPPSTRUCT>
	struct TAutoCppStructOps
	{
		TAutoCppStructOps(FTopLevelAssetPath InName)
		: Name(InName)
		{
			DeferCppStructOps(Name,new TCppStructOps<CPPSTRUCT>);
		}

		~TAutoCppStructOps()
		{
			RemoveDeferredCppStructOps(Name);
		}
		FTopLevelAssetPath Name;
	};
	#define IMPLEMENT_STRUCT(BaseName) \
		UE_DEPRECATED_MACRO(5.1, "IMPLEMENT_STRUCT has been deprecated. Use UE_IMPLEMENT_STRUCT and provide struct package name as well as struct name") static UScriptStruct::TAutoCppStructOps<F##BaseName> BaseName##_Ops(FTopLevelAssetPath(TEXT("/Script/CoreUObject"), TEXT(#BaseName))); 

	#define UE_IMPLEMENT_STRUCT(PackageNameText, BaseName) \
		static UScriptStruct::TAutoCppStructOps<F##BaseName> BaseName##_Ops(FTopLevelAssetPath(TEXT(PackageNameText), TEXT(#BaseName))); 

	COREUOBJECT_API UScriptStruct( EStaticConstructor, int32 InSize, int32 InAlignment, EObjectFlags InFlags );
	COREUOBJECT_API explicit UScriptStruct(const FObjectInitializer& ObjectInitializer, UScriptStruct* InSuperStruct, ICppStructOps* InCppStructOps = nullptr, EStructFlags InStructFlags = STRUCT_NoFlags, SIZE_T ExplicitSize = 0, SIZE_T ExplicitAlignment = 0);
	COREUOBJECT_API explicit UScriptStruct(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if UE_WITH_CONSTINIT_UOBJECT
	consteval UScriptStruct(
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FStructParams StructParams,
		UE::CodeGen::ConstInit::FScriptStructParams Params
	);
#endif

public:
	EStructFlags StructFlags;

protected:
	/** true if we have performed PrepareCppStructOps **/
	bool bPrepareCppStructOpsCompleted;
	/** Holds the Cpp ctors and dtors, sizeof, etc. Is not owned by this and is not released. **/
	ICppStructOps* CppStructOps;

	/** 
	* Similar to GetStructPathName() but works with nested structs by using just the package name and struct name
	* so a struct path name /Package/Name.Object:Struct will be flattened to /Package/Name.Struct.
	* This function is used only for generating keys for DeferredCppStructOps
	*/
	COREUOBJECT_API FTopLevelAssetPath GetFlattenedStructPathName() const;

public:

	// UObject Interface
	virtual COREUOBJECT_API void Serialize(FArchive& Ar) override;
	virtual COREUOBJECT_API void Serialize(FStructuredArchive::FRecord Record) override;

	// UStruct interface.
	virtual COREUOBJECT_API void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	virtual COREUOBJECT_API void InitializeStruct(void* Dest, int32 ArrayDim = 1) const override;
	virtual COREUOBJECT_API void DestroyStruct(void* Dest, int32 ArrayDim = 1) const override;
	virtual COREUOBJECT_API bool IsStructTrashed() const override;
	// End of UStruct interface.

	/** Sets or unsets the trashed flag on this struct */
	void COREUOBJECT_API SetStructTrashed(bool bIsTrash);

	/** 
	 * Stash a CppStructOps for future use 
	 * @param Target Name of the struct 
	 * @param InCppStructOps Cpp ops for this struct
	 */
	static COREUOBJECT_API void DeferCppStructOps(FTopLevelAssetPath Target, ICppStructOps* InCppStructOps);

	/**
	 * On module unload, remove the entry from the stash
	 * @param Target Name of the struct
	 */
	static COREUOBJECT_API void RemoveDeferredCppStructOps(FTopLevelAssetPath Target);

	/**
	 * On module unload, remove all entries in a package from the stash
	 * The package name should be the full package name, such as "/Script/Engine"
	 * @param PackageName Name of the package
	 */
	static COREUOBJECT_API void RemoveAllDeferredCppStructOps(FName PackageName);

	template<class CPPSTRUCT>
	static void DeferCppStructOps(FTopLevelAssetPath Target)
	{
		if constexpr (DISABLE_ABSTRACT_CONSTRUCT)
		{
			DeferCppStructOps(Target, nullptr);
		}
		else
		{
			DeferCppStructOps(Target, new UScriptStruct::TCppStructOps<CPPSTRUCT>);
		}
	}

	/** Look for the CppStructOps and hook it up **/
	virtual COREUOBJECT_API void PrepareCppStructOps();

	/** Returns the CppStructOps that can be used to do custom operations */
	inline ICppStructOps* GetCppStructOps() const
	{
		checkf(bPrepareCppStructOpsCompleted, TEXT("GetCppStructOps: PrepareCppStructOps() has not been called for class %s"), *GetPathName());
		return CppStructOps;
	}

	/** Resets currently assigned CppStructOps, called when loading a struct */
	void ClearCppStructOps()
	{
		StructFlags = EStructFlags(StructFlags & ~STRUCT_ComputedFlags);
		bPrepareCppStructOpsCompleted = false;
		CppStructOps = nullptr;
	}

	/** 
	 * If it is native, it is assumed to have defaults because it has a constructor
	 * @return true if this struct has defaults
	 */
	UE_FORCEINLINE_HINT bool HasDefaults() const
	{
		return !!GetCppStructOps();
	}

	/**
	 * Returns whether this struct should be serialized atomically.
	 * @param	Ar	Archive the struct is going to be serialized with later on
	 */
	bool ShouldSerializeAtomically(FArchive& Ar) const
	{
		if( (StructFlags&STRUCT_Atomic) != 0)
		{
			return true;
		}
		else
		{
			return false;
		}
	}

	/** Returns true if this struct has a native serialize function */
	bool UseNativeSerialization() const
	{
		if ((StructFlags&(STRUCT_SerializeNative)) != 0)
		{
			return true;
		}
		else
		{
			return false;
		}
	}

	/** Returns true if this struct should be binary serialized for the given archive */
	COREUOBJECT_API bool UseBinarySerialization(const FArchive& Ar) const;

	/** 
	 * Serializes a specific instance of a struct 
	 *
	 * @param	Slot		The structured archive slot we are serializing to
	 * @param	Value		Pointer to memory of struct
	 * @param	Defaults	Default value for this struct, pass nullptr to not use defaults 
	 */
	COREUOBJECT_API void SerializeItem(FArchive& Ar, TNotNull<void*> Value, void const* Defaults);
	COREUOBJECT_API void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults);

	/**
	 * Export script struct to a string that can later be imported
	 *
	 * @param	ValueStr		String to write to
	 * @param	Value			Actual struct being exported
	 * @param	Defaults		Default value for this struct, pass nullptr to not use defaults 
	 * @param	OwnerObject		UObject that contains this struct
	 * @param	PortFlags		EPropertyPortFlags controlling export behavior
	 * @param	ExportRootScope	The scope to create relative paths from, if the PPF_ExportsNotFullyQualified flag is passed in.  If NULL, the package containing the object will be used instead.
	 * @param	bAllowNativeOverride If true, will try to run native version of export text on the struct
	 */
	COREUOBJECT_API void ExportText(FString& ValueStr, TNotNull<const void*> Value, const void* Defaults, UObject* OwnerObject, int32 PortFlags, UObject* ExportRootScope, bool bAllowNativeOverride = true) const;

	/**
	 * Sets value of script struct based on imported string
	 *
	 * @param	Buffer			String to read text data out of
	 * @param	Value			Struct that will be modified
	 * @param	OwnerObject		UObject that contains this struct
	 * @param	PortFlags		EPropertyPortFlags controlling import behavior
	 * @param	ErrorText		What to print import errors to
	 * @param	StructName		Name of struct, used in error display
	 * @param	bAllowNativeOverride If true, will try to run native version of export text on the struct
	 * @return Buffer after parsing has succeeded, or NULL on failure
	 */
	COREUOBJECT_API const TCHAR* ImportText(const TCHAR* Buffer, TNotNull<void*> Value, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText, const FString& StructName, bool bAllowNativeOverride = true) const;

	/**
	 * Sets value of script struct based on imported string
	 *
	 * @param	Buffer			String to read text data out of
	 * @param	Value			Struct that will be modified
	 * @param	OwnerObject		UObject that contains this struct
	 * @param	PortFlags		EPropertyPortFlags controlling import behavior
	 * @param	ErrorText		What to print import errors to
	 * @param	StructNameGetter Function to return the struct name to avoid doing work if no error message is forthcoming
	 * @param	bAllowNativeOverride If true, will try to run native version of export text on the struct
	 * @return Buffer after parsing has succeeded, or NULL on failure
	 */
	COREUOBJECT_API const TCHAR* ImportText(const TCHAR* Buffer, TNotNull<void*> Value, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText, const TFunctionRef<FString()>& StructNameGetter, bool bAllowNativeOverride = true) const;

	/**
	 * Compare two script structs
	 *
	 * @param	Dest		Pointer to memory to a struct
	 * @param	Src			Pointer to memory to the other struct
	 * @param	PortFlags	Comparison flags
	 * @return true if the structs are identical
	 */
	COREUOBJECT_API bool CompareScriptStruct(const void* A, const void* B, uint32 PortFlags) const;

	/**
	 * Copy a struct over an existing struct
	 *
	 * @param	Dest		Pointer to memory to initialize
	 * @param	Src			Pointer to memory to copy from
	 * @param	ArrayDim	Number of elements in the array
	 * @param	Stride		Stride of the array, If this default (0), then we will pull the size from the struct
	 */
	COREUOBJECT_API void CopyScriptStruct(void* Dest, void const* Src, int32 ArrayDim = 1) const;

	/**
	 * Move-assign a struct over an existing struct. Uses C++ move assignment operator (operator=(T&&))
	 * if the struct has opted in via WithMoveAssign trait. Falls back to CopyScriptStruct for types
	 * without the trait, or for POD types (where move == copy).
	 * Source is left in a valid but unspecified (moved-from) state after a successful move.
	 *
	 * @param	Dest		Pointer to initialized destination memory
	 * @param	Src			Pointer to source memory (will be moved from)
	 * @param	ArrayDim	Number of elements in the array
	 */
	COREUOBJECT_API void MoveAssignScriptStruct(void* Dest, void* Src, int32 ArrayDim = 1) const;
	
	/**
	 * Reinitialize a struct in memory. This may be done by calling the native destructor and then the constructor or individually reinitializing properties
	 *
	 * @param	Dest		Pointer to memory to reinitialize
	 * @param	ArrayDim	Number of elements in the array
	 * @param	Stride		Stride of the array, only relevant if there more than one element. If this default (0), then we will pull the size from the struct
	 */
	COREUOBJECT_API void ClearScriptStruct(void* Dest, int32 ArrayDim = 1) const;

	/**
	 * Calls GetTypeHash for native structs, otherwise computes a hash of all struct members
	 * 
	 * @param Src		Pointer to instance to hash
	 * @return hashed value of Src
	 */
	virtual COREUOBJECT_API uint32 GetStructTypeHash(const void* Src) const;

	/** Used by User Defined Structs to preload this struct and any child objects */
	virtual COREUOBJECT_API void RecursivelyPreload();

	/** Returns the custom Guid assigned to this struct for User Defined Structs, or an invalid Guid */
	virtual COREUOBJECT_API FGuid GetCustomGuid() const;
	
	/** Returns the (native, c++) name of the struct */
	virtual COREUOBJECT_API FString GetStructCPPName(uint32 CPPExportFlags = 0) const;

	/**
	 * Initializes this structure to its default values
	 * @param InStructData		The memory location to initialize
	 */
	virtual COREUOBJECT_API void InitializeDefaultValue(uint8* InStructData) const;
	
	/**
	 * Provide an FProperty and data pointer when given an FName of a property that this object owns indirectly (ie FInstancedStruct)
	 * @param PropertyName	Name of the property to retrieve
	 * @param Data			Pointer to struct to retrieve property from
	 * @param OutProp		Filled with the found property
	 * @param OutData		Filled with a pointer to the data containing the property.
	 *						Not offset to property instance yet - Call ContainerPtrToValuePtr before use.
	 * @return				whether the property instance was found
	 */
	virtual COREUOBJECT_API bool FindInnerPropertyInstance(FName PropertyName, const void* Data, const FProperty*& OutProp, const void*& OutData) const;

	/* Custom visit implementation for structs */
	using Super::Visit;
	virtual COREUOBJECT_API EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const override;
	virtual COREUOBJECT_API void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const override;
};

/*-----------------------------------------------------------------------------
	UFunction.
-----------------------------------------------------------------------------*/

//
// Reflection data for a replicated or Kismet callable function.
//
UCLASS(MinimalAPI, Within=Object, Config = Engine)
class UFunction : public UStruct
{
	GENERATED_BODY()
public:
	// Persistent variables.

	/** EFunctionFlags set defined for this function */
	EFunctionFlags FunctionFlags;

	// Variables in memory only.
	
	/** Number of parameters total */
	uint8 NumParms;
	/** Total size of parameters in memory */
	uint16 ParmsSize;
	/** Memory offset of return value property */
	uint16 ReturnValueOffset;
	/** Id of this RPC function call (must be FUNC_Net & (FUNC_NetService|FUNC_NetResponse)) */
	uint16 RPCId;
	/** Id of the corresponding response call (must be FUNC_Net & FUNC_NetService) */
	uint16 RPCResponseId;

	/** pointer to first local struct property in this UFunction that contains defaults */
	FProperty* FirstPropertyToInit;

#if UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
	/** The event graph this function calls in to (persistent) */
	UFunction* EventGraphFunction;

	/** The state offset inside of the event graph (persistent) */
	int32 EventGraphCallOffset;
#endif

#if WITH_LIVE_CODING
	/** Pointer to the cached singleton pointer to this instance */
	UFunction** SingletonPtr;
#endif

private:
	/** C++ function this is bound to */
	FNativeFuncPtr Func;

public:
	/**
	 * Returns the native func pointer.
	 *
	 * @return The native function pointer.
	 */
	UE_FORCEINLINE_HINT FNativeFuncPtr GetNativeFunc() const
	{
		return Func;
	}

	/**
	 * Sets the native func pointer.
	 *
	 * @param InFunc - The new function pointer.
	 */
	UE_FORCEINLINE_HINT void SetNativeFunc(FNativeFuncPtr InFunc)
	{
		Func = InFunc;
	}

	/**
	 * Invokes the UFunction on a UObject.
	 *
	 * @param Obj    - The object to invoke the function on.
	 * @param Stack  - The parameter stack for the function call.
	 * @param Result - The result of the function.
	 */
	COREUOBJECT_API void Invoke(UObject* Obj, FFrame& Stack, RESULT_DECL);

	// Constructors.
	UFunction(const FObjectInitializer& ObjectInitializer) 
	: Super(ObjectInitializer)
	{
	}
	COREUOBJECT_API explicit UFunction(const FObjectInitializer& ObjectInitializer, UFunction* InSuperFunction, EFunctionFlags InFunctionFlags = FUNC_None, SIZE_T ParamsSize = 0 );
	COREUOBJECT_API explicit UFunction(UFunction* InSuperFunction, EFunctionFlags InFunctionFlags = FUNC_None, SIZE_T ParamsSize = 0);

#if UE_WITH_CONSTINIT_UOBJECT
	consteval UFunction(
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FStructParams StructParams,
		UE::CodeGen::ConstInit::FFunctionParams Params)
		: Super(ObjectParams, FieldParams, StructParams)
		, FunctionFlags(Params.FunctionFlags)
		, NumParms(0) 
		, ParmsSize(0) 
		, ReturnValueOffset(0) 
		, RPCId(Params.RPCId) 
		, RPCResponseId(Params.RPCResponseId) 
		, FirstPropertyToInit(nullptr) 
#if UE_BLUEPRINT_EVENTGRAPH_FASTCALLS
		, EventGraphFunction(nullptr) 
		, EventGraphCallOffset(INDEX_NONE)
#endif
		, Func(Params.NativeFunction) 
	{
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

	/** Initializes transient members like return value offset */
	COREUOBJECT_API void InitializeDerivedMembers();

	// UObject interface.
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void PostLoad() override;

	// UField interface.
	COREUOBJECT_API virtual void Bind() override;

	// UStruct interface.
	virtual UStruct* GetInheritanceSuper() const override { return nullptr;}
	COREUOBJECT_API virtual void Link(FArchive& Ar, bool bRelinkExistingProperties) override;

	/** Returns parent function if there is one, or null */
	UFunction* GetSuperFunction() const;

	/** Returns the return value property if there is one, or null */
	COREUOBJECT_API FProperty* GetReturnProperty() const;

	/** UFunction cannot be marked Within=Class because it has derived classes that may be within other types */
	UE_FORCEINLINE_HINT UClass* GetOuterUClass() const
	{
		UObject* Outer = GetOuter();
		if (Outer && Outer->IsA<UClass>())
		{
			return reinterpret_cast<UClass*>(Outer);
		}
		return nullptr;
	}

	/** Returns the owning UClass* without branching */
	UE_FORCEINLINE_HINT UClass* GetOuterUClassUnchecked() const
	{
		// declaration order mandates reinterpret_cast:
		return reinterpret_cast<UClass*>(GetOuter());
	}

	/**
	 * Used to safely check whether the passed in flag is set.
	 *
	 * @param	FlagToCheck		Class flag to check for
	 *
	 * @return	true if the passed in flag is set, false otherwise
	 *			(including no flag passed in, unless the FlagsToCheck is CLASS_AllFlags)
	 */
	UE_FORCEINLINE_HINT bool HasAnyFunctionFlags( EFunctionFlags FlagsToCheck ) const
	{
		return (FunctionFlags&FlagsToCheck) != 0 || FlagsToCheck == FUNC_AllFlags;
	}

	/**
	 * Used to safely check whether all of the passed in flags are set.
	 *
	 * @param FlagsToCheck	Function flags to check for
	 * @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	 */
	UE_FORCEINLINE_HINT bool HasAllFunctionFlags( EFunctionFlags FlagsToCheck ) const
	{
		return ((FunctionFlags & FlagsToCheck) == FlagsToCheck);
	}

	/**
	 * Returns the flags that are ignored by default when comparing function signatures.
	 */
	inline static uint64 GetDefaultIgnoredSignatureCompatibilityFlags()
	{
		//@TODO: UCREMOVAL: CPF_ConstParm added as a hack to get blueprints compiling with a const DamageType parameter.
		const uint64 IgnoreFlags = CPF_PersistentInstance | CPF_ExportObject | CPF_InstancedReference 
			| CPF_ContainsInstancedReference | CPF_ComputedFlags | CPF_ConstParm | CPF_UObjectWrapper | CPF_TObjectPtr
			| CPF_NativeAccessSpecifiers | CPF_AdvancedDisplay | CPF_BlueprintVisible | CPF_BlueprintReadOnly;
		return IgnoreFlags;
	}

	/**
	 * Determines if two functions have an identical signature (note: currently doesn't allow
	 * matches with class parameters that differ only in how derived they are; there is no
	 * directionality to the call)
	 *
	 * @param	OtherFunction	Function to compare this function against.
	 *
	 * @return	true if function signatures are compatible.
	 */
	COREUOBJECT_API bool IsSignatureCompatibleWith(const UFunction* OtherFunction) const;

	/**
	 * Determines if two functions have an identical signature (note: currently doesn't allow
	 * matches with class parameters that differ only in how derived they are; there is no
	 * directionality to the call)
	 *
	 * @param	OtherFunction	Function to compare this function against.
	 * @param   IgnoreFlags     Custom flags to ignore when comparing parameters between the functions.
	 *
	 * @return	true if function signatures are compatible.
	 */
	COREUOBJECT_API bool IsSignatureCompatibleWith(const UFunction* OtherFunction, uint64 IgnoreFlags) const;
};

//
// Function definition used by dynamic delegate declarations
//
UCLASS(MinimalAPI)
class UDelegateFunction : public UFunction
{
	GENERATED_BODY()

public:
	explicit UDelegateFunction(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
	}
	COREUOBJECT_API explicit UDelegateFunction(const FObjectInitializer& ObjectInitializer, UFunction* InSuperFunction, EFunctionFlags InFunctionFlags = FUNC_None, SIZE_T ParamsSize = 0);
	COREUOBJECT_API explicit UDelegateFunction(UFunction* InSuperFunction, EFunctionFlags InFunctionFlags = FUNC_None, SIZE_T ParamsSize = 0);

#if UE_WITH_CONSTINIT_UOBJECT
	consteval UDelegateFunction(
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FStructParams StructParams,
		UE::CodeGen::ConstInit::FFunctionParams Params
	)
		: Super(ObjectParams, FieldParams, StructParams, Params)
	{
	}
#endif // UE_WITH_CONSTINIT_UOBJECT
};

//
// Function definition used by sparse dynamic delegate declarations
//
UCLASS(MinimalAPI)
class USparseDelegateFunction : public UDelegateFunction
{
	GENERATED_BODY()

public:
	explicit USparseDelegateFunction(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
	}
	COREUOBJECT_API explicit USparseDelegateFunction(const FObjectInitializer& ObjectInitializer, UFunction* InSuperFunction, EFunctionFlags InFunctionFlags = FUNC_None, SIZE_T ParamsSize = 0);
	COREUOBJECT_API explicit USparseDelegateFunction(UFunction* InSuperFunction, EFunctionFlags InFunctionFlags = FUNC_None, SIZE_T ParamsSize = 0);

#if UE_WITH_CONSTINIT_UOBJECT
	consteval USparseDelegateFunction(
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FStructParams StructParams,
		UE::CodeGen::ConstInit::FFunctionParams FunctionParams,
		UE::CodeGen::ConstInit::FSparseDelegateParams Params
	)
		: Super(ObjectParams, FieldParams, StructParams, FunctionParams)
		, CompiledInOwningClassName(Params.SparseOwningClassNameUTF8)
		, CompiledInDelegateName(Params.SparseDelegateNameUTF8)
	{
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;

	// CONSTINIT_UOBJECT_TODO: Size increase
	union
	{
		FName OwningClassName = {};
#if UE_WITH_CONSTINIT_UOBJECT
		/** 
		 * Stored at compile time form UHT
		 * This union member is active until full compiled-in object construction in ProcessNewlyLoadedUObjects
		 */
		const UTF8CHAR* CompiledInOwningClassName;
#endif // UE_WITH_CONSTINIT_UOBJECT
	};
	union
	{ 
		FName DelegateName = {};
#if UE_WITH_CONSTINIT_UOBJECT
		/** 
		 * Stored at compile time form UHT
		 * This union member is active until full compiled-in object construction in ProcessNewlyLoadedUObjects
		 */
		const UTF8CHAR* CompiledInDelegateName;
#endif // UE_WITH_CONSTINIT_UOBJECT
	};
};

/*-----------------------------------------------------------------------------
	UEnum.
-----------------------------------------------------------------------------*/

typedef FText(*FEnumDisplayNameFn)(int32);

/** Optional flags for the UEnum::Get*ByName() functions. */
enum class EGetByNameFlags
{
	None = 0,

	/** Outputs an warning if the enum lookup fails */
	ErrorIfNotFound		= 0x01,

	/** Does a case sensitive match */
	CaseSensitive		= 0x02,

	/** Checks the GetAuthoredNameStringByIndex value as well as normal names */
	CheckAuthoredName	= 0x04,
};

ENUM_CLASS_FLAGS(EGetByNameFlags)

//
// Reflection data for an enumeration.
//
UCLASS(MinimalAPI, Config = Engine)
class UEnum : public UField
{
	GENERATED_BODY()

public:
	COREUOBJECT_API UEnum(const FObjectInitializer& ObjectInitialzer);

	/**
	 * How this enum is declared in C++, affects the internal naming of enum values
	 * Must match UhtEnumCppForm.
	 */
	enum class ECppForm : uint8
	{
		Regular,
		Namespaced,
		EnumClass
	};
	/**
	 * How this enum is declared in C++, affects the internal underlying type of enum values
	 * Match with UhtEnumUnderlyingType.
	 */
	enum class EUnderlyingType : uint8
	{
		int8,
		int16,
		int32,
		int64,
		uint8,
		uint16,
		uint32,
		uint64,
	};

#if UE_WITH_CONSTINIT_UOBJECT
	consteval UEnum(
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FEnumParams EnumParams
	)
		: Super(ObjectParams, FieldParams)
		, CppTypeStaticUTF8(EnumParams.CppTypeStaticUTF8)
#if WITH_METADATA 
		, CompiledInMetaData(EnumParams.MetaData)
#endif
		, Names(EnumParams.EnumValues.Num(), EnumParams.StaticNamesUTF8.GetData(), EnumParams.EnumValues.GetData())
		, CppForm((ECppForm)EnumParams.CppForm)
		, UnderlyingType((EUnderlyingType)EnumParams.UnderlyingType)
		, EnumFlags(EnumParams.EnumFlags)
		, EnumDisplayNameFn(EnumParams.DisplayNameFn)
	{
		check(EnumParams.StaticNamesUTF8.Num() == EnumParams.EnumValues.Num());
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

	union 
	{ 
		/** This will be the true type of the enum as a string, e.g. "ENamespacedEnum::InnerType" or "ERegularEnum" or "EEnumClass" */
		FString CppType = {};		
#if UE_WITH_CONSTINIT_UOBJECT
		/** 
		 * Static string stored by UHT
		 * This union member is active until full compiled-in UObject construction in ProcessNewlyLoadedObjects
		 */
		const UTF8CHAR* CppTypeStaticUTF8;
#endif // UE_WITH_CONSTINIT_UOBJECT
	};

#if WITH_METADATA && UE_WITH_CONSTINIT_UOBJECT
	// Nothing to alias with here to save space, but builds with metadata enabled are less space constrained
	TConstArrayView<UE::CodeGen::ConstInit::FMetaData> CompiledInMetaData;
#endif // WITH_METADATA && UE_WITH_CONSTINIT_UOBJECT

	// Index is the internal index into the Enum array, and is not useful outside of the Enum system
	// Value is the value set in the Enum Class in C++ or Blueprint
	// Enums can be sparse, which means that not every valid Index is a proper Value, and they are not necessarily equal
	// It is not safe to cast an Index to a Enum Class, always do that with a Value instead

	/** Gets the internal index for an enum value. Returns INDEX_None if not valid */
	inline int32 GetIndexByValue(int64 InValue) const
	{
		for (int32 i = 0; i < Names.Num(); ++i)
		{
			if (Names[i].Value == InValue)
			{
				return i;
			}
		}
		return INDEX_NONE;
	}

	/** Gets enum value by index in Names array. Asserts on invalid index */
	inline int64 GetValueByIndex(int32 Index) const
	{
		check(Names.IsValidIndex(Index));
		return Names[Index].Value;
	}

	/** Gets enum name by index in Names array. Returns NAME_None if Index is not valid. */
	COREUOBJECT_API FName GetNameByIndex(int32 Index) const;

	/** Gets index of name in enum, returns INDEX_NONE and optionally errors when name is not found. This is faster than ByNameString if the FName is exact, but will fall back if needed */
	COREUOBJECT_API int32 GetIndexByName(FName InName, EGetByNameFlags Flags = EGetByNameFlags::None) const;

	/** Gets enum name by value. Returns NAME_None if value is not found. */
	COREUOBJECT_API FName GetNameByValue(int64 InValue) const;

	/** Gets enum value by name, returns INDEX_NONE and optionally errors when name is not found. This is faster than ByNameString if the FName is exact, but will fall back if needed */
	COREUOBJECT_API int64 GetValueByName(FName InName, EGetByNameFlags Flags = EGetByNameFlags::None) const;

	/** Gets enum value by name, returns INDEX_NONE and optionally errors when name is not found. This is faster than ByNameString if the FName is exact, but will fall back if needed */
	COREUOBJECT_API int64 GetValueOrBitfieldFromString(FStringView InString, EGetByNameFlags LookupFlags = EGetByNameFlags::None) const;

	/** Returns the short name at the enum index, returns empty string if invalid */
	COREUOBJECT_API FString GetNameStringByIndex(int32 InIndex) const;

	/** Gets index of name in enum, returns INDEX_NONE and optionally errors when name is not found. Handles full or short names. */
	COREUOBJECT_API int32 GetIndexByNameString(const FString& SearchString, EGetByNameFlags Flags = EGetByNameFlags::None) const;

	/** Returns the short name matching the enum Value, returns empty string if invalid */
	COREUOBJECT_API FString GetNameStringByValue(int64 InValue) const;

	/** If the enumeration is declared as UENUM(Flags), returns a string of the form A | B | C representing set bits A, B, and C. If it is not a bitfield, the result is the same as calling GetNameStringByValue*/
	COREUOBJECT_API FString GetValueOrBitfieldAsString(int64 InValue) const;

	/** If the enumeration is declared as UENUM(Flags), returns a string of the form A | B | C representing set bits A, B, and C. If it is not a bitfield, the result is the same as calling GetAuthoredNameStringByValue*/
	COREUOBJECT_API FString GetValueOrBitfieldAsAuthoredNameString(int64 InValue) const;

	/** If the enumeration is declared as UENUM(Flags), returns a string of the form A | B | C representing set bits A, B, and C. If it is not a bitfield, the result is the same as calling GetDisplayNameTextByValue*/
	COREUOBJECT_API FText GetValueOrBitfieldAsDisplayNameText(int64 InValue) const;

	/** Looks for a name with a given value and returns true and writes the name to Out if one was found */
	COREUOBJECT_API bool FindNameStringByValue(FString& Out, int64 InValue) const;

	/** Gets enum value by name, returns INDEX_NONE and optionally errors when name is not found. Handles full or short names */
	COREUOBJECT_API int64 GetValueByNameString(const FString& SearchString, EGetByNameFlags Flags = EGetByNameFlags::None) const;

	/**
	 * Finds the localized display name or native display name as a fallback.
	 * If called from a cooked build this will normally return the short name as Metadata is not available.
	 *
	 * @param InIndex Index of the enum value to get Display Name for
	 *
	 * @return The display name for this object, or an empty text if Index is invalid
	 */
	COREUOBJECT_API virtual FText GetDisplayNameTextByIndex(int32 InIndex) const;

	/** Version of GetDisplayNameTextByIndex that takes a value instead */
	COREUOBJECT_API FText GetDisplayNameTextByValue(int64 InValue) const;

	/** Looks for a display name with a given value and returns true and writes the name to Out if one was found */
	COREUOBJECT_API bool FindDisplayNameTextByValue(FText& Out, int64 InValue) const;

	/**
	 * Returns the unlocalized logical name originally assigned to the enum at creation.
	 * By default this is the same as the short name but it is overridden in child classes with different internal name storage.
	 * This name is consistent in cooked and editor builds and is useful for things like external data import/export.
	 *
	 * @param InIndex Index of the enum value to get Display Name for
	 *
	 * @return The author-specified name, or an empty string if Index is invalid
	 */
	COREUOBJECT_API virtual FString GetAuthoredNameStringByIndex(int32 InIndex) const;

	/** Version of GetAuthoredNameByIndex that takes a value instead */
	COREUOBJECT_API FString GetAuthoredNameStringByValue(int64 InValue) const;

	/** Looks for a display name with a given value and returns true and writes the unlocalized logical name to Out if one was found */
	COREUOBJECT_API bool FindAuthoredNameStringByValue(FString& Out, int64 InValue) const;

	/** Gets max value of Enum. Defaults to zero if there are no entries. */
	COREUOBJECT_API int64 GetMaxEnumValue() const;

	/** Checks if enum has entry with given value. Includes autogenerated _MAX entry. */
	COREUOBJECT_API bool IsValidEnumValue(int64 InValue) const;

	/** Checks if enum has entry with given value, or combination of flag values. If the enum is not a bitfield, this is equivalent to IsValidEnumValue. */
	COREUOBJECT_API bool IsValidEnumValueOrBitfield(int64 InValue) const;

	/** Checks if enum has entry with given name. Includes autogenerated _MAX entry. */
	COREUOBJECT_API bool IsValidEnumName(FName InName) const;

	/** Removes the Names in this enum from the primary AllEnumNames list */
	COREUOBJECT_API void RemoveNamesFromPrimaryList();

	/** Try to update an out-of-date enum index after an enum changes at runtime */
	COREUOBJECT_API virtual int64 ResolveEnumerator(FArchive& Ar, int64 EnumeratorIndex) const;

	/** Associate a function for looking up Enum display names by index, only intended for use by generated code */
	void SetEnumDisplayNameFn(FEnumDisplayNameFn InEnumDisplayNameFn)
	{
		EnumDisplayNameFn = InEnumDisplayNameFn;
	}

	/**
	 * Returns the type of enum: whether it's a regular enum, namespaced enum or C++11 enum class.
	 *
	 * @return The enum type.
	 */
	ECppForm GetCppForm() const
	{
		return CppForm;
	}

	/**
	 * Returns the underlying type of the enum.
	 * @return The underlying enum type.
	 */
	EUnderlyingType GetUnderlyingType() const
	{
		return UnderlyingType;
	}

	void SetEnumFlags(EEnumFlags FlagsToSet)
	{
		EnumFlags |= FlagsToSet;
	}

	bool HasAnyEnumFlags(EEnumFlags InFlags) const
	{
		return EnumHasAnyFlags(EnumFlags, InFlags);
	}

	/**
	 * Checks if a enum name is fully qualified name.
	 *
	 * @param InEnumName Name to check.
	 * @return true if the specified name is full enum name, false otherwise.
	 */
	static bool IsFullEnumName(const TCHAR* InEnumName)
	{
		return !!FCString::Strstr(InEnumName, TEXT("::"));
	}

	/**
	 * Generates full name including EnumName:: given enum name.
	 *
	 * @param InEnumName Enum name.
	 * @return Full enum name.
	 */
	COREUOBJECT_API virtual FString GenerateFullEnumName(const TCHAR* InEnumName) const;

	/**
	 * Searches the list of all enum value names for the specified name
	 * @param PackageName Package where the enum was defined (/Script/ModuleName or uasset package name in case of script defined enums). If an empty (None) name is specified the function will look through all enums to find a match
	 * @param TestName Fully qualified enum value name (EEnumName::ValueName)
	 * @Param Options Optional options that may limit search results or define ambiguous search result behavior
	 * @param OutFoundEnum Optional address of a variable where the resulting UEnum object should be stored
	 * @return The value the specified name represents if found, otherwise INDEX_NONE
	 */
	COREUOBJECT_API static int64 LookupEnumName(FName PackageName, FName TestName, EFindFirstObjectOptions Options = EFindFirstObjectOptions::None, UEnum** OutFoundEnum = nullptr);

	/** 
	 * Searches the list of all enum value names for the specified name
	 * @param PackageName Package where the enum was defined (/Script/ModuleName or uasset package name in case of script defined enums). If an empty (None) name is specified the function will look through all native enums to find a match
	 * @param InTestShortName Fully qualified or short enum value name (EEnumName::ValueName or ValueName)
	 * @Param Options Optional Options that may limit search results or define ambiguous search result behavior
	 * @param OutFoundEnum Optional address of a variable where the resulting UEnum object should be stored
	 * @return The value the specified name represents if found, otherwise INDEX_NONE
	 */
	COREUOBJECT_API static int64 LookupEnumNameSlow(FName PackageName, const TCHAR* InTestShortName, EFindFirstObjectOptions Options = EFindFirstObjectOptions::None, UEnum** OutFoundEnum = nullptr);

	/** parses the passed in string for a name, then searches for that name in any Enum (in any package)
	 * @param Str	pointer to string to parse; if we successfully find an enum, this pointer is advanced past the name found
	 * @return index of the value the parsed enum name matches, or INDEX_NONE if no matches
	 */
	COREUOBJECT_API static int64 ParseEnum(const TCHAR*& Str);

	/**
	 * Tests if the enum contains a MAX value
	 *
	 * @return	true if the enum contains a MAX enum, false otherwise.
	 */
	COREUOBJECT_API bool ContainsExistingMax() const;

	// Specifies whether or not a default Max item should be added.
	enum class EAddMaxKeyIfMissing : uint8
	{
		No,
		Yes,
	};
	/**
	 * Sets the array of enums.
	 *
	 * @param InNames List of enum names.
	 * @param InCppForm The form of enum.
	 * @param InUnderlyingType The underlying type of enum.
	 * @param AddMaxKeyIfMissing Should a default Max item be added.
	 * @return	true unless the MAX enum already exists and isn't the last enum.
	 */
	COREUOBJECT_API virtual bool SetEnums(TArray<TPair<FName, int64>>& InNames, ECppForm InCppForm, EUnderlyingType InUnderlyingType, EEnumFlags InFlags, EAddMaxKeyIfMissing AddMaxKeyIfMissing);

	UE_DEPRECATED(5.8, "Use the SetEnums with the underlying type.")
	COREUOBJECT_API virtual bool SetEnums(TArray<TPair<FName, int64>>& InNames, ECppForm InCppForm, EEnumFlags InFlags = EEnumFlags::None, bool bAddMaxKeyIfMissing = true) final;

#if UE_WITH_CONSTINIT_UOBJECT
	/**
	 * Creates the array of enum names & values from compiled-in data.
	 * 
	 * @param bAddMaxKeyIfMissing 
	 */
	COREUOBJECT_API void InitializeNames();
#endif // UE_WITH_CONSTINIT_UOBJECT

	/**
	 * @return	 The number of enum names.
	 */
	int32 NumEnums() const
	{
		return Names.Num();
	}

	/**
	 * Find the longest common prefix of all items in the enumeration.
	 * 
	 * @return	the longest common prefix between all items in the enum.  If a common prefix
	 *			cannot be found, returns the full name of the enum.
	 */
	COREUOBJECT_API FString GenerateEnumPrefix() const;

	/**
	 * @brief Find the longest common prefix of all enum names in the provided list.
	 *
	 * @return	the longest common prefix between all items in the enum.  If a common prefix
	 *			cannot be found, returns the full name of the enum.
	 */
	COREUOBJECT_API static FString GenerateEnumPrefix(const TArray<TPair<FName, int64>>& InNames, FName InEnumName);

#if WITH_EDITOR
	/**
	 * Finds the localized tooltip or native tooltip as a fallback.
	 *
	 * @param NameIndex Index of the enum value to get tooltip for
	 *
	 * @return The tooltip for this object.
	 */
	COREUOBJECT_API FText GetToolTipTextByIndex(int32 NameIndex) const;
#endif

#if WITH_METADATA
	/**
	 * Wrapper method for easily determining whether this enum has metadata associated with it.
	 * 
	 * @param	Key			the metadata tag to check for
	 * @param	NameIndex	if specified, will search for metadata linked to a specified value in this enum; otherwise, searches for metadata for the enum itself
	 *
	 * @return true if the specified key exists in the list of metadata for this enum, even if the value of that key is empty
	 */
	COREUOBJECT_API bool HasMetaData( const TCHAR* Key, int32 NameIndex=INDEX_NONE ) const;

	/**
	 * Return the metadata value associated with the specified key.
	 * 
	 * @param	Key			the metadata tag to find the value for
	 * @param	NameIndex	if specified, will search the metadata linked for that enum value; otherwise, searches the metadata for the enum itself
	 * @param	bAllowRemap	if true, the returned value may be remapped from a .ini if the value starts with ini: Pass false when you need the exact string, including any ini:
	 *
	 * @return	the value for the key specified, or an empty string if the key wasn't found or had no value.
	 */
	COREUOBJECT_API FString GetMetaData( const TCHAR* Key, int32 NameIndex=INDEX_NONE, bool bAllowRemap=true ) const;

	/**
	 * Set the metadata value associated with the specified key.
	 * 
	 * @param	Key			the metadata tag to find the value for
	 * @param	NameIndex	if specified, will search the metadata linked for that enum value; otherwise, searches the metadata for the enum itself
	 * @param	InValue		Value of the metadata for the key
	 *
	 */
	COREUOBJECT_API void SetMetaData( const TCHAR* Key, const TCHAR* InValue, int32 NameIndex=INDEX_NONE) const;
	
	/**
	 * Remove given key meta data
	 * 
	 * @param	Key			the metadata tag to find the value for
	 * @param	NameIndex	if specified, will search the metadata linked for that enum value; otherwise, searches the metadata for the enum itself
	 *
	 */
	COREUOBJECT_API void RemoveMetaData( const TCHAR* Key, int32 NameIndex=INDEX_NONE ) const;
#endif // WITH_METADATA
	
	/**
	 * @param EnumPath         Full enum path.
	 * @param EnumeratorValue  Enumerator VAlue.
	 *
	 * @return the string associated with the enumerator for the specified enum value for the enum specified by a path.
	 */
	template <typename T>
	inline static FString GetValueAsString( const TCHAR* EnumPath, const T EnumeratorValue)
	{
		// For the C++ enum.
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		return GetValueAsString_Internal(EnumPath, (int64)EnumeratorValue);
	}

	template <typename T>
	UE_FORCEINLINE_HINT static FString GetValueAsString( const TCHAR* EnumPath, const TEnumAsByte<T> EnumeratorValue)
	{
		return GetValueAsString_Internal(EnumPath, (int64)EnumeratorValue.GetValue());
	}

	template< class T >
	UE_FORCEINLINE_HINT static void GetValueAsString( const TCHAR* EnumPath, const T EnumeratorValue, FString& out_StringValue )
	{
		out_StringValue = GetValueAsString( EnumPath, EnumeratorValue );
	}

	template <typename T>
	inline static FString GetValueOrBitfieldAsString(const TCHAR* EnumPath, const T EnumeratorValue)
	{
		// For the C++ enum.
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		return GetValueOrBitfieldAsString_Internal(EnumPath, (int64)EnumeratorValue);
	}

	template <typename T>
	UE_FORCEINLINE_HINT static FString GetValueOrBitfieldAsString(const TCHAR* EnumPath, const TEnumAsByte<T> EnumeratorValue)
	{
		return GetValueOrBitfieldAsString_Internal(EnumPath, (int64)EnumeratorValue.GetValue());
	}

	template< class T >
	UE_FORCEINLINE_HINT static void GetValueOrBitfieldAsString(const TCHAR* EnumPath, const T EnumeratorValue, FString& out_StringValue)
	{
		out_StringValue = GetValueOrBitfieldAsString( EnumPath, EnumeratorValue );
	}
	/**
	 * @param EnumPath         Full enum path.
	 * @param EnumeratorValue  Enumerator Value.
	 *
	 * @return the localized display string associated with the specified enum value for the enum specified by a path
	 */
	template <typename T>
	inline static FText GetDisplayValueAsText( const TCHAR* EnumPath, const T EnumeratorValue )
	{
		// For the C++ enum.
		static_assert(TIsEnum<T>::Value, "Should only call this with enum types");
		return GetDisplayValueAsText_Internal(EnumPath, (int64)EnumeratorValue);
	}

	template <typename T>
	UE_FORCEINLINE_HINT static FText GetDisplayValueAsText( const TCHAR* EnumPath, const TEnumAsByte<T> EnumeratorValue)
	{
		return GetDisplayValueAsText_Internal(EnumPath, (int64)EnumeratorValue.GetValue());
	}

	template< class T >
	UE_FORCEINLINE_HINT static void GetDisplayValueAsText( const TCHAR* EnumPath, const T EnumeratorValue, FText& out_TextValue )
	{
		out_TextValue = GetDisplayValueAsText( EnumPath, EnumeratorValue);
	}

	/**
	 * @param EnumeratorValue  Enumerator Value.
	 *
	 * @return the name associated with the enumerator for the specified enum value for the enum specified by the template type.
	 */
	template<typename EnumType>
	inline static FName GetValueAsName(const EnumType EnumeratorValue)
	{
		// For the C++ enum.
		static_assert(TIsEnum<EnumType>::Value, "Should only call this with enum types");
		UEnum* EnumClass = StaticEnum<EnumType>();
		check(EnumClass != nullptr);
		return EnumClass->GetNameByValue((int64)EnumeratorValue);
	}

	template<typename EnumType>
	UE_FORCEINLINE_HINT static FName GetValueAsName(const TEnumAsByte<EnumType> EnumeratorValue)
	{
		return GetValueAsName(EnumeratorValue.GetValue());
	}

	template<typename EnumType>
	UE_FORCEINLINE_HINT static void GetValueAsName(const EnumType EnumeratorValue, FName& out_NameValue )
	{
		out_NameValue = GetValueAsName(EnumeratorValue);
	}

	/**
	 * @param EnumeratorValue  Enumerator Value.
	 *
	 * @return the string associated with the enumerator for the specified enum value for the enum specified by the template type.
	 */
	template<typename EnumType>
	inline static FString GetValueAsString(const EnumType EnumeratorValue)
	{
		// For the C++ enum.
		static_assert(TIsEnum<EnumType>::Value, "Should only call this with enum types");
		return GetValueAsName(EnumeratorValue).ToString();
	}

	template<typename EnumType>
	UE_FORCEINLINE_HINT static FString GetValueAsString(const TEnumAsByte<EnumType> EnumeratorValue)
	{
		return GetValueAsString(EnumeratorValue.GetValue());
	}

	template<typename EnumType>
	UE_FORCEINLINE_HINT static void GetValueAsString(const EnumType EnumeratorValue, FString& out_StringValue )
	{
		out_StringValue = GetValueAsString(EnumeratorValue );
	}

	/**
	 * @param EnumeratorValue  Enumerator Value.
	 *
	 * @return a string representing the combination of set bits in the bitfield or, if not a bitfield/flags enumeration, the string associated with the enumerator for the specified enum value for the enum specified by the template type.
	 */
	template<
		typename EnumType
		UE_REQUIRES(TIsEnum<EnumType>::Value)
	>
	inline static FString GetValueOrBitfieldAsString(const EnumType EnumeratorValue)
	{
		// For the C++ enum.
		static_assert(TIsEnum<EnumType>::Value, "Should only call this with enum types");
		UEnum* EnumClass = StaticEnum<EnumType>();
		check(EnumClass != nullptr);
		return EnumClass->GetValueOrBitfieldAsString((int64)EnumeratorValue);
	}

	// TEnumAsByte produces a warning if you use it with EnumClass, so this UE_REQUIRES keeps this overload
	// from being matched in that case
	template<
		typename EnumType
		UE_REQUIRES(!TIsEnumClass<EnumType>::Value)
	>
	UE_FORCEINLINE_HINT static FString GetValueOrBitfieldAsString(const TEnumAsByte<EnumType> EnumeratorValue)
	{
		return GetValueOrBitfieldAsString(EnumeratorValue.GetValue());
	}

	template<
		typename EnumType,
		typename IntegralType
		UE_REQUIRES(TIsEnum<EnumType>::Value && std::is_integral_v<IntegralType>)
	>
	inline static FString GetValueOrBitfieldAsString(const IntegralType EnumeratorValue)
	{
		// For the C++ enum.
		static_assert(TIsEnum<EnumType>::Value, "Should only call this with enum types");
		UEnum* EnumClass = StaticEnum<EnumType>();
		check(EnumClass != nullptr);
		return EnumClass->GetValueOrBitfieldAsString((int64)EnumeratorValue);
	}

	template<typename EnumType>
	UE_FORCEINLINE_HINT static void GetValueOrBitfieldAsString(const EnumType EnumeratorValue, FString& out_StringValue)
	{
		out_StringValue = GetValueOrBitfieldAsString(EnumeratorValue);
	}

	template<
		typename EnumType,
		typename IntegralType
		UE_REQUIRES(TIsEnum<EnumType>::Value && std::is_integral_v<IntegralType>)
	>
	UE_FORCEINLINE_HINT static void GetValueOrBitfieldAsString(const IntegralType EnumeratorValue, FString& out_StringValue)
	{
		out_StringValue = GetValueOrBitfieldAsString<EnumType>(EnumeratorValue);
	}

	/**
	 * @param EnumeratorValue  Enumerator Value.
	 *
	 * @return the localized display string associated with the specified enum value for the enum specified by the template type.
	 */
	template<typename EnumType>
	inline static FText GetDisplayValueAsText(const EnumType EnumeratorValue )
	{
		// For the C++ enum.
		static_assert(TIsEnum<EnumType>::Value, "Should only call this with enum types");
		UEnum* EnumClass = StaticEnum<EnumType>();
		check(EnumClass != nullptr);
		return EnumClass->GetDisplayNameTextByValue((int64)EnumeratorValue);
	}

	template<typename EnumType>
	UE_FORCEINLINE_HINT static FText GetDisplayValueAsText(const TEnumAsByte<EnumType> EnumeratorValue)
	{
		return GetDisplayValueAsText(EnumeratorValue.GetValue());
	}

	template<typename EnumType>
	UE_FORCEINLINE_HINT static void GetDisplayValueAsText(const EnumType EnumeratorValue, FText& out_TextValue )
	{
		out_TextValue = GetDisplayValueAsText(EnumeratorValue);
	}

	// UObject interface.
	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	COREUOBJECT_API virtual void BeginDestroy() override;
	// End of UObject interface.

	COREUOBJECT_API ~UEnum();

	// Helper function for UHT - calculate max enum value when UHT was not able to parse all enum values
	consteval static inline int64 CalculateMaxEnumValue(std::initializer_list<int64> Values, EEnumFlags InFlags)
	{
		check(Values.size() != 0); // UHT should only generate a call to this when there are values
		if (EnumHasAllFlags(InFlags, EEnumFlags::Flags))
		{
			int64 CombinedFlags = 0;
			for (int64 Val : Values)
			{
				if (FMath::IsPowerOfTwo(Val))
				{
					CombinedFlags |= Val;
				}
			}
			return 1 + CombinedFlags;
		}
		return 1 + *Algo::MaxElement(Values);
	}

protected:
	class FNameData
	{
		// Tag bit to show wehther a pointer is dynamically allocated, or points to a static allocation
		static inline constexpr UPTRINT TagDynamic = 0x1;
		// Mask to retrieve a readable pointer from the tagged pointer regardless of whether it is static or dynamic
		static inline constexpr UPTRINT MaskPointer = ~0x1;
		union
		{
#if UE_WITH_CONSTINIT_UOBJECT
			// Immutable array of string constants from UHT. This union member is active until full compiled-in UObject construction in ProcessNewlyLoadedUObjects
			const UTF8CHAR* const* StaticNamesUTF8;
#endif // UE_WITH_CONSTINIT_UOBJECT
			// Tagged FName* pointer, noting whether array is dynamically allocated 
			UPTRINT TaggedNames = 0;
		};
		union
		{
#if UE_WITH_CONSTINIT_UOBJECT
			// Immutable array of values from UHT. This union member may be active for the entire process lifetime.
			const int64* StaticValues;
#endif // UE_WITH_CONSTINIT_UOBJECT
			// Tagged int64* pointer noting whether array is dynamically allocated
			UPTRINT TaggedValues = 0;
		};
		// Both arrays contain same number of values
		int32 NumValues = 0;

		void DeleteNames();
		void DeleteValues();

		FName* AllocateNames(int32 InNum);
		int64* AllocateValues(int32 InNum);

	public:
		UE_NONCOPYABLE(FNameData)
		constexpr FNameData() = default;
#if UE_WITH_CONSTINIT_UOBJECT
		consteval FNameData(int32 InNum, const UTF8CHAR* const* InStaticNamesUTF8, const int64* InValues)
		: StaticNamesUTF8(InStaticNamesUTF8)
		, StaticValues(InValues)
		, NumValues(InNum)
		{
		}
#endif // UE_WITH_CONSTINIT_UOBJECT
		~FNameData();

		[[nodiscard]] inline int32 Num() const 
		{ 
			return NumValues; 
		}

		[[nodiscard]] inline bool IsValidIndex(int32 Index) const
		{
			return Index >= 0 && Index < NumValues;
		}

		[[nodiscard]] inline TPair<FName, int64> operator[](int32 Index) const
		{
			check(IsValidIndex(Index));
			return { GetNames()[Index], GetValues()[Index] };
		} 

#if UE_WITH_CONSTINIT_UOBJECT
		// Initialize from constinit data
		void InitializeFromCompiledInData();
#endif //UE_WITH_CONSTINIT_UOBJECT
		// Initialize from legacy serialization of names without values
		void Initialize(const TArray<FName>& InNames);
		// Initialize from legacy serialization of names with byte values
		void Initialize(const TArray<TPair<FName, uint8>>& InNames);
		// Initialize from names and int64 values
		void Initialize(const TArray<TPair<FName, int64>>& InNames);
		// Initialize from names and int64 values with a placeholder to generate a max element
		COREUOBJECT_API void InitializeWithPlaceholder(const TArray<TPair<FName, int64>>& InNames, int64 MaxValue);
		// If necessary, replace values with sequential values counting up from zero
		COREUOBJECT_API void MakeValuesSequentialFromZero();
		// Deallocate arrays - ConstInit objects should not delete the values as they are a static array created by UHT.
		COREUOBJECT_API void Empty();

		COREUOBJECT_API void UsePlaceholder(FName InName, int64 InValue);
		COREUOBJECT_API void DiscardPlaceholder();

		// Get a mutable view over the names
		[[nodiscard]] inline TArrayView<FName> GetMutableNames()
		{
			checkfSlow(!TaggedNames || !!(TaggedNames & TagDynamic), TEXT("This function should only be used after compiled-in enums have copied their static string names into FNames"));
			return MakeArrayView(reinterpret_cast<FName*>(TaggedNames & MaskPointer), NumValues);
		}

		// Get an immutable view over the names 
		[[nodiscard]] inline TConstArrayView<FName> GetNames() const
		{
			checkfSlow(!TaggedNames || !!(TaggedNames & TagDynamic), TEXT("This function should only be used after compiled-in enums have copied their static string names into FNames"));
			return MakeConstArrayView(reinterpret_cast<const FName*>(TaggedNames & MaskPointer), NumValues);
		}

		// Get an immutable view over the values
		[[nodiscard]] inline TConstArrayView<int64> GetValues() const
		{
			return MakeConstArrayView(reinterpret_cast<int64*>(TaggedValues & MaskPointer), NumValues);
		}

		// Read-only ranged-for support.
		struct FIteratorEnd
		{
		};
		struct FIterator
		{
			void operator++()
			{
				++Index;
			}

			bool operator==(FIteratorEnd) const
			{
				return Index >= Num;
			}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
			bool operator!=(FIteratorEnd) const
			{
				return Index < Num;
			}
#endif

			TPair<FName, int64> operator*() const
			{
				return { Names[Index], Values[Index] };
			}

			int32 Index = 0;
			int32 Num;
			const FName* Names;
			const int64* Values;
		};

		inline FIterator begin() const 
		{ 
			return FIterator{ 
				.Num = Num(),
				.Names = reinterpret_cast<FName*>(TaggedNames & MaskPointer),
				.Values = reinterpret_cast<int64*>(TaggedValues & MaskPointer) 
			};
		}
		inline FIteratorEnd end() const 
		{
			return FIteratorEnd(); 
		}
	} Names;

	/**
	 * How the enum was originally defined.
	 * The value is serialized.
	 */
	ECppForm CppForm;

	/**
	 * The underlying enum type.
	 * The value is serialized.
	 */
	EUnderlyingType UnderlyingType;

	/** Enum flags. */
	EEnumFlags EnumFlags;

	/** Package name this enum was in when its names were being added to the primary list */
	FName EnumPackage;
	
	/** pointer to function used to look up the enum's display name. Currently only assigned for UEnums generated for nativized blueprints */
	FEnumDisplayNameFn EnumDisplayNameFn;

	/** lock to be taken when accessing AllEnumNames */
	static FRWLock AllEnumNamesLock;

	/** global list of all value names used by all enums in memory, used for property text import */
	static TMap<FName, TMap<FName, UEnum*> > AllEnumNames;

	/** adds the Names in this enum to the primary AllEnumNames list */
	COREUOBJECT_API void AddNamesToPrimaryList();

	/** Gets max value of Enum. Defaults to zero if there are no entries. Static version to calculate during construction. */
	COREUOBJECT_API static int64 GetMaxEnumValue(const TArray<TPair<FName, int64>>& InNames, EEnumFlags InFlags);

private:

	/**
	 * Searches the list of all enum value names and returns a UEnum object that contains an enum value name matching the provided comparison function criteria
	 * @param PackageName Package where the enum was defined (/Script/ModuleName or uasset package name in case of script defined enums). If an empty (None) name is specified the function will look through all native enums to find a match
	 * @Param Options Options that may limit search results or define ambiguous search result behavior
	 * @param CompareNameFunction Functions used to compare the existing enum value names with the one that should be found
	 * @return UEnum object if a matching enum value name was found otherwise nullptr
	 */
	static UEnum* LookupAllEnumNamesWithOptions(FName PackageName, EFindFirstObjectOptions Options, TFunctionRef<bool(FName)> CompareNameFunction);

	inline static FString GetValueAsString_Internal( const TCHAR* EnumPath, const int64 EnumeratorValue)
	{
		UEnum* EnumClass = FindObject<UEnum>( nullptr, EnumPath );
		UE_CLOGF( !EnumClass, LogClass, Fatal, "Couldn't find enum '%ls'", EnumPath );
		return EnumClass->GetNameStringByValue(EnumeratorValue);
	}

	inline static FString GetValueOrBitfieldAsString_Internal(const TCHAR* EnumPath, const int64 EnumeratorValue)
	{
		UEnum* EnumClass = FindObject<UEnum>(nullptr, EnumPath);
		UE_CLOGF(!EnumClass, LogClass, Fatal, "Couldn't find enum '%ls'", EnumPath);
		return EnumClass->GetValueOrBitfieldAsString(EnumeratorValue);
	}

	inline static FText GetDisplayValueAsText_Internal( const TCHAR* EnumPath, const int64 EnumeratorValue )
	{
		UEnum* EnumClass = FindObject<UEnum>(nullptr, EnumPath);
		UE_CLOGF(!EnumClass, LogClass, Fatal, "Couldn't find enum '%ls'", EnumPath);
		return EnumClass->GetDisplayNameTextByValue(EnumeratorValue);
	}

	/**
	 * Renames enum values to use duplicated enum name instead of base one, e.g.:
	 * 
	 * MyEnum::MyVal
	 * MyEnum::MyEnum_MAX
	 * 
	 * becomes
	 * 
	 * MyDuplicatedEnum::MyVal
	 * MyDuplicatedEnum::MyDuplicatedEnum_MAX
	 */
	void RenameNamesAfterDuplication();

	/** Gets name of enum "this" is duplicate of. If we're not duplicating, just returns "this" name. */
	FString GetBaseEnumNameOnDuplication() const;
};

/*-----------------------------------------------------------------------------
	UClass.
-----------------------------------------------------------------------------*/

/** Base definition for C++ class type traits */
struct FCppClassTypeTraitsBase
{
	enum
	{
		IsAbstract = false
	};
};


/** Defines traits for specific C++ class types */
template<class CPPCLASS>
struct TCppClassTypeTraits : public FCppClassTypeTraitsBase
{
	enum
	{
		IsAbstract = TIsAbstract<CPPCLASS>::Value
	};
};


/** Interface for accessing attributes of the underlying C++ class, for native class types */
struct ICppClassTypeInfo
{
	/** Return true if the underlying C++ class is abstract (i.e. declares at least one pure virtual function) */
	virtual bool IsAbstract() const = 0;
};


struct FCppClassTypeInfoStatic
{
	bool bIsAbstract;
};


/** Implements the type information interface for specific C++ class types */
struct FCppClassTypeInfo : ICppClassTypeInfo
{
	constexpr explicit FCppClassTypeInfo(const FCppClassTypeInfoStatic* InInfo)
		: Info(InInfo)
	{
	}

	// Non-copyable
	FCppClassTypeInfo(const FCppClassTypeInfo&) = delete;
	FCppClassTypeInfo& operator=(const FCppClassTypeInfo&) = delete;

	// ICppClassTypeInfo implementation
	virtual bool IsAbstract() const override
	{
		return Info->bIsAbstract;
	}

private:
	const FCppClassTypeInfoStatic* Info;
};


/** information about an interface a class implements */
struct FImplementedInterface
{
	/** the interface class */
	TObjectPtr<UClass> Class;
	/** the pointer offset of the interface's vtable */
	int32 PointerOffset;
	/** whether or not this interface has been implemented via K2 */
	bool bImplementedByK2;

	FImplementedInterface()
		: Class(nullptr)
		, PointerOffset(0)
		, bImplementedByK2(false)
	{}
	explicit FImplementedInterface(UClass* InClass, int32 InOffset, bool InImplementedByK2)
		: Class(InClass)
		, PointerOffset(InOffset)
		, bImplementedByK2(InImplementedByK2)
	{}

	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FImplementedInterface& A);
};


/** A struct that maps a string name to a native function */
struct FNativeFunctionLookup
{
	FName Name;
	FNativeFuncPtr Pointer;

#if UE_WITH_CONSTINIT_UOBJECT
	explicit FNativeFunctionLookup(UE::CodeGen::FClassNativeFunction InCompiledIn)
		:	Name(InCompiledIn.NameUTF8)
		,	Pointer(InCompiledIn.Pointer)
	{}
#endif // UE_WITH_CONSTINIT_UOBJECT

	FNativeFunctionLookup(FName InName, FNativeFuncPtr InPointer)
		:	Name(InName)
		,	Pointer(InPointer)
	{}
};


namespace EIncludeSuperFlag
{
	enum Type
	{
		ExcludeSuper,
		IncludeSuper
	};
}

struct FClassFunctionLinkInfo
{
	UFunction* (*CreateFuncPtr)(ETypeConstructPhase);
	const char* FuncNameUTF8;
};


/**
 * Flag describing desired behavior of GetSparseClassData in the case where the SparseClassData has not been created.
 * The class's instance is guaranteed already created if the class has sparse data of its own, so the only differences
 * in behavior are when the class does not have sparse data of its own.
 */
enum class EGetSparseClassDataMethod : uint8
{
	/**
	 * If not yet created, store and return a new instance, whether or not the class can use its archetype's instance.
	 */
	CreateIfNull,
	/**
	 * If not yet created, return the archetype instance (if possible to use for this class). If not possible to use
	 * the archetype's instance, store and return a new instance.
	 */
	ArchetypeIfNull,
	/**
	 * This is an advanced flag and should usually be avoided. Callers should instead call with ArchetypeIfNull if they
	 * are currently allowed to create an instance, or with DeferIfNull if not. Callers that want to know whether the
	 * class has sparse data of its own beyond what the archetype has should compare
	 * GetSparseClassDataStruct() != GetSparseClassDataArchetypeStruct().
	 * 
	 * If not yet created, return null. An instance might still be created later when called with ArchetypeIfNull.
	 * This call never creates an instance.
	 */
	ReturnIfNull,
	/**
	 * If not yet created, return the archetype's instance (if possible to use for this class). If not possible to use
	 * the archetype's instance, return null. User should call later with ArchetypeIfNull when they are allowed to
	 * create an instance.
	 * This call never creates an instance.
	 */
	DeferIfNull,
};

template<class T>
AUTORTFM_INFER void InternalConstructor( const FObjectInitializer& X );

template<class T>
UObject* InternalVTableHelperCtorCaller(FVTableHelper& Helper);

/**
 * An object class.
 */
UCLASS(MinimalAPI, Within=Package, Config = Engine)
class UClass : public UStruct
{
	GENERATED_BODY()

public:
	friend class FRestoreClassInfo;
	friend class FBlueprintEditorUtils;
	friend class FBlueprintCompileReinstancer;
	friend struct FBlueprintCompilationManagerImpl;

	typedef void		(*ClassConstructorType)				(const FObjectInitializer&);
	typedef UObject*	(*ClassVTableHelperCtorCallerType)	(FVTableHelper& Helper);
	typedef UClass* (*StaticClassFunctionType)();

	ClassConstructorType ClassConstructor;
	ClassVTableHelperCtorCallerType ClassVTableHelperCtorCaller;
	FUObjectCppClassStaticFunctions CppClassStaticFunctions;

	/** Class pseudo-unique counter; used to accelerate unique instance name generation */
	mutable int32 ClassUnique;

	/** Index of the first ClassRep that belongs to this class. Anything before that was defined by / belongs to parent classes. */
	int32 FirstOwnedClassRep = 0;

	/** Used to check if the class was cooked or not */
	bool bCooked;

	/** Used to check if the class layout is currently changing and therefore is not ready for a CDO to be created */
	bool bLayoutChanging;
	
protected:
	
	/** TRUE if this class should not rely exclusively on instanced property semantics for subobject template instancing. */
	bool bNeedsDynamicSubobjectInstancing = false;

	/** TRUE if instances of this class contain non-serialized references that require construction on load via instancing. */
	bool bNeedsPostLoadSubobjectInstancing = false;

private:
	/** When using partials they are allocated in front of UObject, and then this value will be negative. Note that PropertiesSize includes the negative part too */
	int32 PropertiesStartOffset = 0;

	friend void UE::CoreUObject::Private::ConstructUClassPartials(UClass&);
public:

	/** Class flags; See EClassFlags for more information */
	EClassFlags ClassFlags;

	/** Cast flags used to accelerate dynamic_cast<T*> on objects of this type for common T */
	EClassCastFlags ClassCastFlags;

	/** The required type for the outer of instances of this class */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UClass> ClassWithin;

#if WITH_EDITORONLY_DATA
	/** This is the blueprint that caused the generation of this class, or null if it is a native compiled-in class */
	UPROPERTY(SkipSerialization)
	TObjectPtr<UObject> ClassGeneratedBy;

	/** Linked list of properties to be destroyed when this class is destroyed that couldn't be destroyed in PurgeClass **/
	FField* PropertiesPendingDestruction;

	/** Destroys properties that couldn't be destroyed in PurgeClass */
	COREUOBJECT_API void DestroyPropertiesPendingDestruction();
#endif

#if WITH_EDITOR
	/**
	 * Conditionally recompiles the class after loading, in case any dependencies were also newly loaded
	 * @param ObjLoaded	If set this is the list of objects that are currently loading, usualy GObjLoaded
	 */
	virtual void ConditionalRecompileClass(FUObjectSerializeContext* InLoadContext) {}
	virtual void FlushCompilationQueueForLevel() {}
#endif //WITH_EDITOR

	union
	{
		/** Which Name.ini file to load Config variables out of */
		FName ClassConfigName = {};
#if UE_WITH_CONSTINIT_UOBJECT
  		// CONSTINIT_UOBJECT_TODO: Size increase
		/**
		 * Name stored from UHT as a static string
		 * This union member is active until initial compiled-in UObject construction in UObjectProcessRegistrants
		 */
		const TCHAR* CompiledInClassConfigName;
#endif // UE_WITH_CONSTINIT_UOBJECT
	};

	/** List of replication records */
	TArray<FRepRecord> ClassReps;

	/** List of network relevant fields (functions) */
	UPROPERTY(SkipSerialization)
	TArray<TObjectPtr<UField>> NetFields;

#if WITH_EDITOR
	// Editor only properties
	COREUOBJECT_API void GetHideFunctions(TArray<FString>& OutHideFunctions) const;
	COREUOBJECT_API bool IsFunctionHidden(const TCHAR* InFunction) const;
	COREUOBJECT_API void GetAutoExpandCategories(TArray<FString>& OutAutoExpandCategories) const;
	COREUOBJECT_API bool IsAutoExpandCategory(const TCHAR* InCategory) const;
	COREUOBJECT_API void GetPrioritizeCategories(TArray<FString>& OutPrioritizedCategories) const;
	COREUOBJECT_API bool IsPrioritizeCategory(const TCHAR* InCategory) const;
	COREUOBJECT_API void GetAutoCollapseCategories(TArray<FString>& OutAutoCollapseCategories) const;
	COREUOBJECT_API bool IsAutoCollapseCategory(const TCHAR* InCategory) const;
	COREUOBJECT_API void GetClassGroupNames(TArray<FString>& OutClassGroupNames) const;
	COREUOBJECT_API bool IsClassGroupName(const TCHAR* InGroupName) const;
#endif
	/**
	 * Calls AddReferencedObjects static method on the specified object.
	 *
	 * @param This Object to call ARO on.
	 * @param Collector Reference collector.
	 */
	inline void CallAddReferencedObjects(UObject* This, FReferenceCollector& Collector) const
	{
		// The object must of this class type.
		check(This->IsA(this)); 
		// This should always be set to something, at the very least to UObject::ARO
		check(CppClassStaticFunctions.GetAddReferencedObjects() != nullptr);
		CppClassStaticFunctions.GetAddReferencedObjects()(This, Collector);
	}

#if WITH_EDITORONLY_DATA
	/** Calls the c++ class's DeclareCustomVersions static function (from the nearest native parent if this is not native) */
	void CallDeclareCustomVersions(FArchive& Ar) const
	{
		check(CppClassStaticFunctions.GetDeclareCustomVersions());
		CppClassStaticFunctions.GetDeclareCustomVersions()(Ar, this);
	}

	/** Calls the c++ class's AppendToClassSchema static function */
	COREUOBJECT_API void CallAppendToClassSchema(FAppendToClassSchemaContext& Context) const;

	/** Calls the c++ class's DeclareConstructClasses static function (from the nearest native parent if this is not native) */
	void CallDeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses) const
	{
		check(CppClassStaticFunctions.GetDeclareConstructClasses());
		CppClassStaticFunctions.GetDeclareConstructClasses()(OutConstructClasses, this);
	}

	/** Calls the c++ class's InjectDynamicImportsFor static function (from the nearest native parent if this is not native) */
	void CallInjectDynamicImportsFor(const class FLinkerLoad& Linker, const struct FObjectExport& Export, TArray<struct FDynamicImport>& OutDynamicImports) const
	{
		check(CppClassStaticFunctions.GetInjectDynamicImportsFor());
		CppClassStaticFunctions.GetInjectDynamicImportsFor()(Linker, Export, OutDynamicImports);
	}
#endif

	/** The class default object; used for delta serialization and object initialization */
	UE_DEPRECATED(5.6, "ClassDefaultObject will be made private in the next release. For general purpose access to the CDO consider using the global GetDefault<>() or GetMutableDefault<>() functions.")
	UPROPERTY(SkipSerialization)
	TObjectPtr<UObject> ClassDefaultObject;

protected:
#if UE_WITH_IMMUTABLE_CDO
	TObjectPtr<UObject> ImmutableDefaultObject;
#endif

	/** This is where we store the data that is only changed per class instead of per instance */
	void* SparseClassData;

	/** The struct used to store sparse class data. */
	TObjectPtr<UScriptStruct> SparseClassDataStruct;

public:
	/**
	 * Returns a pointer to the sidecar data structure, based on the EGetSparseClassDataMethod.
	 * @note It is only safe to mutate this data when using "CreateIfNull", as others may return archetype/default data; consider GetOrCreateSparseClassData for this use-case.
	 */
	COREUOBJECT_API const void* GetSparseClassData(const EGetSparseClassDataMethod GetMethod);

	/**
	 * Returns a pointer to the sidecar data structure. This function will create an instance of the data structure if one has been specified and it has not yet been created.
	 */
	void* GetOrCreateSparseClassData() { return const_cast<void*>(GetSparseClassData(EGetSparseClassDataMethod::CreateIfNull)); }

	/**
	 * Returns a pointer to the type of the sidecar data structure if one is specified.
	 */
	COREUOBJECT_API UScriptStruct* GetSparseClassDataStruct() const;

	COREUOBJECT_API void SetSparseClassDataStruct(UScriptStruct* InSparseClassDataStruct);

	/** 
	 * Clears the sparse class data struct for this and all child classes that directly reference it as a super-struct 
	 * This will rename the current sparse class data struct aside into the transient package
	 */
	COREUOBJECT_API void ClearSparseClassDataStruct(bool bInRecomplingOnLoad);

	/** Assemble reference token streams for all classes if they haven't had it assembled already */
	static COREUOBJECT_API void AssembleReferenceTokenStreams();

#if WITH_EDITOR
	void GenerateFunctionList(TArray<FName>& OutArray) const 
	{ 
		FUClassFuncScopeReadLock ScopeLock(FuncMapLock);
		FuncMap.GenerateKeyArray(OutArray); 
	}
#endif // WITH_EDITOR

protected:
	COREUOBJECT_API void* CreateSparseClassData();

	COREUOBJECT_API void CleanupSparseClassData();

private:
#if WITH_EDITOR
	/** Provides access to attributes of the underlying C++ class. Should never be unset. */
	TOptional<FCppClassTypeInfo> CppTypeInfo;
#endif

	/** Map of all functions by name contained in this class */
	TMap<FName, TObjectPtr<UFunction>> FuncMap;

	/** Scope lock to avoid the FuncMap being read and written to simultaneously on multiple threads. */
	mutable FUClassFuncLock FuncMapLock;
	
	/** Scope lock to avoid the SuperFuncMap being read and written to simultaneously on multiple threads. */
	mutable FUClassFuncLock AllFunctionsCacheLock;

	/** A cache of all functions by name that exist in this class or a parent (superclass or interface) context */
	mutable TMap<FName, UFunction*> AllFunctionsCache;

public:
	union
	{ 
		/**
		* The list of interfaces which this class implements, along with the pointer property that is located at the offset of the interface's vtable.
		* If the interface class isn't native, the property will be null.
		*/
		TArray<FImplementedInterface> Interfaces = {};

#if UE_WITH_CONSTINIT_UOBJECT
		/**
		 * Static data about compiled in interfaces from UHT
		 * This union member is active until full compiled-in UObject construction in ProcessNewlyLoadedUObjects
		 * It's mutable because pointers are fixed up here before copying to Interfaces.
		 */
		TArrayView<UE::CodeGen::ConstInit::FClassImplementedInterface> CompiledInInterfaces;
#endif // UE_WITH_CONSTINIT_UOBJECT
	};

	/** GC schema, finalized in AssembleReferenceTokenStream */
	UE::GC::FSchemaOwner ReferenceSchema;

	union
	{ 
		/** This class's native functions. */
		TArray<FNativeFunctionLookup> NativeFunctionLookupTable = {};

#if UE_WITH_CONSTINIT_UOBJECT
		/** 
		 * Pairs of name/function ptr from UHT at compile time to populate NativeFunctionLookupTable 
		 * This union member is active until initial compiled-in UObject construction in UObjectProcessRegistrants
		 */
		TConstArrayView<UE::CodeGen::FClassNativeFunction> CompiledInNativeFunctions;
#endif // UE_WITH_CONSTINIT_UOBJECT
	};

	/** Linked list of partials extending this class */
	UE::CoreUObject::Private::FPartialClass* Partials = nullptr;

public:
	// Constructors
	COREUOBJECT_API UClass(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	COREUOBJECT_API explicit UClass(const FObjectInitializer& ObjectInitializer, UClass* InSuperClass);
	COREUOBJECT_API UClass( EStaticConstructor, FName InName, uint32 InSize, uint32 InAlignment, EClassFlags InClassFlags, EClassCastFlags InClassCastFlags,
		const TCHAR* InClassConfigName, EObjectFlags InFlags, ClassConstructorType InClassConstructor,
		ClassVTableHelperCtorCallerType InClassVTableHelperCtorCaller,
		FUObjectCppClassStaticFunctions&& InCppClassStaticFunctions);
#if UE_WITH_CONSTINIT_UOBJECT
	consteval UClass(
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FStructParams StructParams,
		UE::CodeGen::ConstInit::FClassParams Params
	);
	// Special constructor for the UClass representing UClass itself
	consteval UClass(
		UE::CodeGen::ConstInit::EClass,
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FStructParams StructParams,
		UE::CodeGen::ConstInit::FClassParams Params
	);
	// Special constructor for the UClass representing UObject itself
	consteval UClass(
		UE::CodeGen::ConstInit::EObject,
		UE::CodeGen::ConstInit::FObjectParams ObjectParams,
		UE::CodeGen::ConstInit::FUFieldParams FieldParams,
		UE::CodeGen::ConstInit::FStructParams StructParams,
		UE::CodeGen::ConstInit::FClassParams Params
	);
#endif

	// Destructor
	COREUOBJECT_API ~UClass();

#if WITH_RELOAD
	/**
	 * Called when a class is reloading from a DLL...updates various information in-place.
	 * @param	InSize							sizeof the class
	 * @param	InClassFlags					Class flags for the class
	 * @param	InClassCastFlags				Cast Flags for the class
	 * @param	InConfigName					Config Name
	 * @param	InClassConstructor				Pointer to InternalConstructor<TClass>
	 * @param	TClass_Super_StaticClass		Static class of the super class
	 * @param	TClass_WithinClass_StaticClass	Static class of the WithinClass
	 */
	COREUOBJECT_API bool HotReloadPrivateStaticClass(
		uint32			InSize,
		EClassFlags		InClassFlags,
		EClassCastFlags	InClassCastFlags,
		const TCHAR*    InConfigName,
		ClassConstructorType InClassConstructor,
		ClassVTableHelperCtorCallerType InClassVTableHelperCtorCaller,
		FUObjectCppClassStaticFunctions&& InCppClassStaticFunctions,
		class UClass* TClass_Super_StaticClass,
		class UClass* TClass_WithinClass_StaticClass
		);


	/**
	* Replace a native function in the  internal native function table
	* @param	InName							name of the function
	* @param	InPointer						pointer to the function
	* @param	bAddToFunctionRemapTable		For C++ hot-reloading, UFunctions are patched in a deferred manner and this should be true
	*											For script hot-reloading, script integrations may have a many to 1 mapping of UFunction to native pointer
	*											because dispatch is shared, so the C++ remap table does not work in this case, and this should be false
	* @return	true if the function was found and replaced, false if it was not
	*/
	COREUOBJECT_API bool ReplaceNativeFunction(FName InName, FNativeFuncPtr InPointer, bool bAddToFunctionRemapTable);
#endif

	/**
	 * If there are potentially multiple versions of this class (e.g. blueprint generated classes), this function will return the authoritative version, which should be used for references
	 *
	 * @return The version of this class that references should be stored to
	 */
	COREUOBJECT_API virtual UClass* GetAuthoritativeClass();
	const UClass* GetAuthoritativeClass() const { return const_cast<UClass*>(this)->GetAuthoritativeClass(); }

	/**
	 * Add a native function to the internal native function table
	 * @param	InName							name of the function
	 * @param	InPointer						pointer to the function
	 */
	COREUOBJECT_API void AddNativeFunction(const ANSICHAR* InName, FNativeFuncPtr InPointer);

	/**
	 * Add a native function to the internal native function table, but with a unicode name. Used when generating code from blueprints, 
	 * which can have unicode identifiers for functions and properties.
	 * @param	InName							name of the function
	 * @param	InPointer						pointer to the function
	 */
	COREUOBJECT_API void AddNativeFunction(const WIDECHAR* InName, FNativeFuncPtr InPointer);

	/** Add a function to the function map */
	void AddFunctionToFunctionMap(UFunction* Function, FName FuncName)
	{
		{
			FUClassFuncScopeWriteLock ScopeLock(FuncMapLock);
			FuncMap.Add(FuncName, Function);
		}
		{
			// Remove from the function cache if it exists
			FUClassFuncScopeWriteLock ScopeLock(AllFunctionsCacheLock);
			AllFunctionsCache.Remove(FuncName);
		}
	}

#if UE_WITH_CONSTINIT_UOBJECT
	/** Add all functions from the Children linked list to the function map */
	void InitFuncMap();
#else 
	COREUOBJECT_API void CreateLinkAndAddChildFunctionsToMap(const FClassFunctionLinkInfo* Functions, uint32 NumFunctions);
#endif // UE_WITH_CONSTINIT_UOBJECT

	/** Remove a function from the function map */
	void RemoveFunctionFromFunctionMap(UFunction* Function)
	{
		{
			FUClassFuncScopeWriteLock ScopeLock(FuncMapLock);
			FuncMap.Remove(Function->GetFName());
		}
		{
			// Remove from the function cache if it exists
			FUClassFuncScopeWriteLock ScopeLock(AllFunctionsCacheLock);
			AllFunctionsCache.Remove(Function->GetFName());
		}
	}

	/** Clears the function name caches, in case things have changed */
	COREUOBJECT_API void ClearFunctionMapsCaches();

	/** Looks for a given function name */
	COREUOBJECT_API UFunction* FindFunctionByName(FName InName, EIncludeSuperFlag::Type IncludeSuper = EIncludeSuperFlag::IncludeSuper) const;

	// UObject interface.
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC 
	COREUOBJECT_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker) override;
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	COREUOBJECT_API virtual void PostLoad() override;
	COREUOBJECT_API virtual void FinishDestroy() override;
	COREUOBJECT_API virtual void DeferredRegister(UClass* UClassStaticClass, const TCHAR* PackageName, const TCHAR* InName
#if UE_WITH_REMOTE_OBJECT_HANDLE
		, FRemoteObjectId RemoteId
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
	) override;
	COREUOBJECT_API virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;
	COREUOBJECT_API virtual void TagSubobjects(EObjectFlags NewFlags) override;
	COREUOBJECT_API virtual void PostInitProperties() override;
	static COREUOBJECT_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	COREUOBJECT_API virtual FRestoreForUObjectOverwrite* GetRestoreForUObjectOverwrite() override;
	COREUOBJECT_API virtual FString GetDesc() override;
	COREUOBJECT_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	COREUOBJECT_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;

#if WITH_EDITOR
protected:
	COREUOBJECT_API virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const;
public:
	COREUOBJECT_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const override;
#endif // WITH_EDITOR

	virtual bool IsAsset() const override { return false; }	
	virtual bool IsNameStableForNetworking() const override { return true; } // For now, assume all classes have stable net names
	COREUOBJECT_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
	// End of UObject interface.

	// UField interface.
	COREUOBJECT_API virtual void Bind() override;
	COREUOBJECT_API virtual const TCHAR* GetPrefixCPP() const override;
	// End of UField interface.

	// UStruct interface.
	COREUOBJECT_API virtual void Link(FArchive& Ar, bool bRelinkExistingProperties) override;
	COREUOBJECT_API virtual void SetSuperStruct(UStruct* NewSuperStruct) override;
	COREUOBJECT_API virtual bool IsStructTrashed() const override;
	UE_INTERNAL COREUOBJECT_API virtual EStructPropertyLinkFlags GetPropertyLinkFlags(UStruct* ContainerStruct, FProperty* Property) const;
	// End of UStruct interface.

#if WITH_EDITORONLY_DATA
	UE_INTERNAL COREUOBJECT_API virtual bool ActivateTrackingPropertyValueFlag(EPropertyValueFlags Flags, void* Data) const override;
	UE_INTERNAL COREUOBJECT_API virtual bool IsTrackingPropertyValueFlag(EPropertyValueFlags Flags, const void* Data) const override;
	UE_INTERNAL COREUOBJECT_API virtual bool HasPropertyValueFlag(EPropertyValueFlags Flags, const void* Data, const FProperty* Property, int32 ArrayIndex = 0) const override;
	UE_INTERNAL COREUOBJECT_API virtual void SetPropertyValueFlag(EPropertyValueFlags Flags, bool bValue, void* Data, const FProperty* Property, int32 ArrayIndex = 0) const override;
	UE_INTERNAL COREUOBJECT_API virtual void ResetPropertyValueFlags(EPropertyValueFlags Flags, void* Data) const override;
	UE_INTERNAL COREUOBJECT_API virtual void SerializePropertyValueFlags(EPropertyValueFlags Flags, void* Data, FStructuredArchiveRecord Record, FArchiveFieldName Name) const override;
#endif // WITH_EDITORONLY_DATA

	/** Offset from UObject pointer to where properties start. This method will return a negative value if partials are present */
	UE_FORCEINLINE_HINT int32 GetPropertiesStartOffset() const
	{
		return PropertiesStartOffset;
	}

	/**
	 * Returns class path name as a package + class FName pair
	 */
	UE_FORCEINLINE_HINT FTopLevelAssetPath GetClassPathName() const
	{
		return GetStructPathName();
	}

	/**
	 * Utility function that tries to find a type (class/struct/enum) given a path name or a short name however it will throw a warning (with a callstack) if it's the latter.
	 * Useful for tracking down and fixing remaining places that use short class names.
	 * This function is slow and should not be used in performance critical situations
	 * @param TypeClass Class of the expected type. Must be a subclass of UField
	 * @param InShortNameOrPathName Class name
	 * @return Pointer to a type object if successful, nullptr otherwise
	 */
	static COREUOBJECT_API UField* TryFindTypeSlow(UClass* TypeClass, FStringView InPathNameOrShortName, EFindFirstObjectOptions InOptions = EFindFirstObjectOptions::None);

	/**
	 * Utility function that tries to find a type (class/struct/enum) given a path name or a short name however it will throw a warning (with a callstack) if it's the latter.
	 * Useful for tracking down and fixing remaining places that use short class names.
	 * This function is slow and should not be used in performance critical situations
	 * @param InShortNameOrPathName Class name
	 * @return Pointer to a type object if successful, false otherwise
	 */
	template <typename T>
	static T* TryFindTypeSlow(FStringView InShortNameOrPathName, EFindFirstObjectOptions InOptions = EFindFirstObjectOptions::None)
	{
		return (T*)TryFindTypeSlow(T::StaticClass(), InShortNameOrPathName, InOptions);
	}

	/**
	 * Utility function that tries to find a type (class/struct/enum) given a path name or a short name however it will throw a warning (with a callstack) if it's the latter.
	 * Useful for tracking down and fixing remaining places that use short class names.
	 * This function is slow and should not be used in performance critical situations
	 * This version of TryFindType will not assert when saving package or when collecting garbage
	 * @param TypeClass Class of the expected type. Must be a subclass of UField
	 * @param InShortNameOrPathName Class name
	 * @return Pointer to a type object if successful, nullptr otherwise
	 */
	static COREUOBJECT_API UField* TryFindTypeSlowSafe(UClass* TypeClass, FStringView InPathNameOrShortName, EFindFirstObjectOptions InOptions = EFindFirstObjectOptions::None);

	/**
	 * Utility function that tries to find a type (class/struct/enum) given a path name or a short name however it will throw a warning (with a callstack) if it's the latter.
	 * Useful for tracking down and fixing remaining places that use short class names.
	 * This function is slow and should not be used in performance critical situations
	 * This version of TryFindType will not assert when saving package or when collecting garbage
	 * @param InShortNameOrPathName Class name
	 * @return Pointer to a type object if successful, nullptr otherwise
	 */
	template <typename T>
	static T* TryFindTypeSlowSafe(FStringView InShortNameOrPathName, EFindFirstObjectOptions InOptions = EFindFirstObjectOptions::None)
	{
		return (T*)TryFindTypeSlowSafe(T::StaticClass(), InShortNameOrPathName, InOptions);
	}

	/**
	 * Tries to convert short class name to class path name. This will only work if the class exists in memory
	 * This function is slow and should not be used in performance critical situations
	 * @param TypeClass Expected Class of the type that needs to be converted (UEnum / UStruct / UClass)
	 * @param InShortClassName Short class name. If this parameter is already a path name then it's going to be returned as FTopLevelAssetPath
	 * @param AmbiguousMessageVerbosity Verbosity with which to log a message if class name is ambiguous 
	 * @param AmbiguousClassMessage Additional message to log when class name is ambiguous (e.g. current operation) 
	 * @return Class path name if successful. Empty FTopLevelAssetPath otherwise.
	 */
	static COREUOBJECT_API FTopLevelAssetPath TryConvertShortTypeNameToPathName(UClass* TypeClass, FStringView InShortTypeName, ELogVerbosity::Type AmbiguousMessageVerbosity = ELogVerbosity::NoLogging, const TCHAR* AmbiguousClassMessage = nullptr);

	/**
	 * Tries to convert short class name to class path name. This will only work if the class exists in memory
	 * @param InShortClassName Short class name. If this parameter is already a path name then it's going to be returned as FTopLevelAssetPath
	 * @param AmbiguousMessageVerbosity Verbosity with which to log a message if class name is ambiguous
	 * @param AmbiguousClassMessage Additional message to log when class name is ambiguous (e.g. current operation)
	 * @return Class path name if successful. Empty FTopLevelAssetPath otherwise.
	 */
	template <typename T>
	static FTopLevelAssetPath TryConvertShortTypeNameToPathName(FStringView InShortTypeName, ELogVerbosity::Type AmbiguousMessageVerbosity = ELogVerbosity::NoLogging, const TCHAR* AmbiguousClassMessage = nullptr)
	{
		return TryConvertShortTypeNameToPathName(T::StaticClass(), InShortTypeName, AmbiguousMessageVerbosity, AmbiguousClassMessage);
	}

	/**
	 * Tries to fix an export path containing short class name. Will not modify the input InOutExportPathToFix if a
	 * fixup was not necessary. Optionally modifies it if a fixup is necessary but unsuccessful.
	 * @param InOutExportPathToFix Export path (Class'/Path/To.Object')
	 * @param AmbiguousMessageVerbosity Verbosity with which to log a message if class name is ambiguous
	 * @param AmbiguousClassMessage Additional message to log when class name is ambiguous (e.g. current operation)
	 * @param bClearOnError If the fixup is necessary but unsuccessful, if bClearOnError, clear the value and return
	 *        true, otherwise leave the value unchanged and return false.
	 * @return True if the short path was successfully fixed. False if the provided export path did not contain short
	 *         class name or the short path could not be fixed.
	 */
	static COREUOBJECT_API bool TryFixShortClassNameExportPath(FString& InOutExportPathToFix,
		ELogVerbosity::Type AmbiguousMessageVerbosity = ELogVerbosity::NoLogging,
		const TCHAR* AmbiguousClassMessage = nullptr, bool bClearOnError = false);

	/**
	 * Returns the ObjectName portion of a ClassPath name: "/Path/To.Object" is converted to "Object". 
	 * Returns the full string if it is already a ShortTypeName.
	 */
	static COREUOBJECT_API FString ConvertPathNameToShortTypeName(FStringView InClassPathOrShortTypeName);

	/**
	 * Takes a FullName (from e.g. AssetData.GetFullName or UObject.GetFullName) in either ShortTypeFullName or PathFullName form
	 * ShortTypeFullName: "ClassObjectName /PackagePath/PackageShortName.ObjectName:SubObjectName"
	 * PathFullName: "/ClassPath/ClassPackage.ClassObjectName /PackagePath/PackageShortName.ObjectName:SubObjectName"
	 * Converts it to ShortTypeFullName if not already in that format and returns it
	 */
	static COREUOBJECT_API FString ConvertFullNameToShortTypeFullName(FStringView InFullName);

	/**
	 * Returns whether the given stringview is in ShortTypeName form: No directory separators "/" or object separators ".", SUBOBJECT_DELIMITER
	 * Returns true for empty string.
	 */
	static COREUOBJECT_API bool IsShortTypeName(FStringView ClassPathOrShortTypeName);

#if WITH_EDITOR
	/** Provides access to C++ type info. */
	const ICppClassTypeInfo* GetCppTypeInfo() const
	{
		return CppTypeInfo ? &CppTypeInfo.GetValue() : nullptr;
	}
#endif

	/** Sets C++ type information. Should not be NULL. */
	void SetCppTypeInfoStatic(const FCppClassTypeInfoStatic* InCppTypeInfoStatic)
	{
#if WITH_EDITOR
		check(InCppTypeInfoStatic);
		CppTypeInfo.Emplace(InCppTypeInfoStatic);
#endif
	}
	
	/**
	 * Translates the hardcoded script config names (engine, editor, input and 
	 * game) to their global pendants and otherwise uses config(myini) name to
	 * look for a game specific implementation and creates one based on the
	 * default if it doesn't exist yet.
	 *
	 * @return	name of the class specific ini file
	 */
	COREUOBJECT_API const FString GetConfigName() const;

	/** Returns parent class, the parent of a Class is always another class */
	UClass* GetSuperClass() const
	{
		return (UClass*)GetSuperStruct();
	}

	/** Feedback context for default property import **/
	static COREUOBJECT_API class FFeedbackContext& GetDefaultPropertiesFeedbackContext();

	/** Returns amount of memory used by default object */
	int32 GetDefaultsCount()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ClassDefaultObject != nullptr ? GetPropertiesSize() : 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Get the default object from the class
	 * @param	bCreateIfNeeded if true (default) then the CDO is created if it is null
	 * @return		the CDO for this class
	 */
	UObject* GetDefaultObject(bool bCreateIfNeeded = true) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (ClassDefaultObject == nullptr && bCreateIfNeeded)
		{
			InternalCreateDefaultObjectWrapper();
		}

		return ClassDefaultObject;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Get the immutable default object from the class
	 * @param bCreateIfNeeded if true (default) then the CDO is created if it is null
	 * @return the immutable CDO for this class
	 */
#if UE_WITH_IMMUTABLE_CDO
	const UObject* GetImmutableDefaultObject(bool bCreateIfNeeded = true) const
	{
		if (ImmutableDefaultObject == nullptr && bCreateIfNeeded)
		{
			InternalCreateImmutableDefaultObject();
		}
		return ImmutableDefaultObject;
	}
#endif

	/**
	 * Sets the default object of the class
	 * @param	InClassDefaultObject new class default object or null
	 */
	COREUOBJECT_API void SetDefaultObject(UObject* InClassDefaultObject);

	/**
	 * Checks if the class default object is valid (not null and its address is a properly aligned pointer)
	 * @param	AlignmentCheck address alignment to check
	 * @return	true if the CDO is valid
	 */
	inline bool CheckDefaultObjectIsValidLowLevelFast(const int32 AlignmentCheck) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ClassDefaultObject != nullptr && ((UPTRINT)ObjectPtr_Private::Friend::NoAccessTrackingGet(ClassDefaultObject) & AlignmentCheck) == 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	/**
	 * Called after PostInitProperties during object construction to allow class specific initialization of an object instance.
	 */
	virtual void PostInitInstance(UObject* InObj, FObjectInstancingGraph* InstanceGraph) {}

	/**
	 * Called during PostLoad of an object to allow class specific PostLoad operations of an object instance.
	 */
	virtual void PostLoadInstance(UObject* InObj) {}

	/**
	 * Helper method to assist with initializing object properties from an explicit list.
	 *
	 * @param	InStruct			the current scope for which the given property list applies
	 * @param	DataPtr				destination address (where to start copying values to)
	 * @param	DefaultDataPtr		source address (where to start copying the defaults data from)
	 */
	virtual void InitPropertiesFromCustomList(uint8* DataPtr, const uint8* DefaultDataPtr) {}

	/**
	 * Allows class to provide data to the object initializer that can affect how native class subobjects are created.
	 */
	virtual void SetupObjectInitializer(FObjectInitializer& ObjectInitializer) const {}

	/**
	 * Some classes may not support creating assets of their type
	 */
	virtual bool CanCreateAssetOfClass() const
	{
		return true;
	}

	/**
	 * Get the name of the CDO for the this class
	 * @return The name of the CDO
	 */
	COREUOBJECT_API FName GetDefaultObjectName() const;

	/** Returns memory used to store temporary data on an instance, used by blueprints */
	virtual uint8* GetPersistentUberGraphFrame(UObject* Obj, UFunction* FuncToCheck) const
	{
		return nullptr;
	}

	/** Creates memory to store temporary data */
	virtual void CreatePersistentUberGraphFrame(UObject* Obj, bool bCreateOnlyIfEmpty = false, bool bSkipSuperClass = false, UClass* OldClass = nullptr) const
	{
	}
	
	/** Clears memory to store temporary data */
	virtual void DestroyPersistentUberGraphFrame(UObject* Obj, bool bSkipSuperClass = false) const
	{
	}

	/**
	 * Get the default object from the class and cast to a particular type
	 * @return		the CDO for this class
	 */
	template<class T>
	T* GetDefaultObject() const
	{
		UObject *Ret = GetDefaultObject();
		check(Ret->IsA(T::StaticClass()));
		return (T*)Ret;
	}

	/** Searches for the default instanced object (often a component) by name **/
	COREUOBJECT_API UObject* GetDefaultSubobjectByName(FName ToFind);

	/** Adds a new default instance map item **/
	void AddDefaultSubobject(UObject* NewSubobject, const UClass* BaseClass)
	{
		// this component must be a derived class of the base class
		check(NewSubobject->IsA(BaseClass));
		// the outer of the component must be of my class or some superclass of me
		check(IsChildOf(NewSubobject->GetOuter()->GetClass()));
	}

	/**
	 * Gets all default instanced objects (often components).
	 *
	 * @param OutDefaultSubobjects An array to be filled with default subobjects.
	 */
	COREUOBJECT_API void GetDefaultObjectSubobjects(TArray<UObject*>& OutDefaultSubobjects);

	/**
	 * Used to safely check whether the passed in flag is set.
	 *
	 * @param	FlagsToCheck		Class flag(s) to check for
	 *
	 * @return	true if the passed in flag is set, false otherwise
	 *			(including no flag passed in, unless the FlagsToCheck is CLASS_AllFlags)
	 */
	UE_FORCEINLINE_HINT bool HasAnyClassFlags( EClassFlags FlagsToCheck ) const
	{
		return EnumHasAnyFlags(ClassFlags, FlagsToCheck) != 0;
	}

	/**
	 * Used to safely check whether all of the passed in flags are set.
	 *
	 * @param FlagsToCheck	Class flags to check for
	 * @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	 */
	UE_FORCEINLINE_HINT bool HasAllClassFlags( EClassFlags FlagsToCheck ) const
	{
		return EnumHasAllFlags(ClassFlags, FlagsToCheck);
	}

	/**
	 * Gets the class flags.
	 *
	 * @return	The class flags.
	 */
	UE_FORCEINLINE_HINT EClassFlags GetClassFlags() const
	{
		return ClassFlags;
	}

	/**
	 * Used to safely check whether the passed in flag is set.
	 *
	 * @param	FlagToCheck		the cast flag to check for (value should be one of the EClassCastFlags enums)
	 *
	 * @return	true if the passed in flag is set, false otherwise
	 *			(including no flag passed in)
	 */
	UE_FORCEINLINE_HINT bool HasAnyCastFlag(EClassCastFlags FlagToCheck) const
	{
		return (ClassCastFlags&FlagToCheck) != 0;
	}
	UE_FORCEINLINE_HINT bool HasAllCastFlags(EClassCastFlags FlagsToCheck) const
	{
		return (ClassCastFlags&FlagsToCheck) == FlagsToCheck;
	}

	COREUOBJECT_API FString GetDescription() const;

	/**
	 * Assembles the token stream for realtime garbage collection by combining the per class only
	 * token stream for each class in the class hierarchy. This is only done once and duplicate
	 * work is avoided by using an object flag.
	 * @param bForce Assemble the stream even if it has been already assembled (deletes the old one)
	 */
	COREUOBJECT_API void AssembleReferenceTokenStream(bool bForce = false);

	/** 
	 * This will return whether or not this class implements the passed in class / interface 
	 *
	 * @param SomeClass - the interface to check and see if this class implements it
	 */
	COREUOBJECT_API bool ImplementsInterface(const class UClass* SomeInterface) const;

	/** serializes the passed in object as this class's default object using the given archive slot
	 * @param Object the object to serialize as default
	 * @param Slot the structured archive slot to serialize from
	 */
	COREUOBJECT_API virtual void SerializeDefaultObject(UObject* Object, FStructuredArchive::FSlot Slot);

	/** serializes the passed in object as this class's default object using the given archive
	 * @param Object the object to serialize as default
	 * @param Ar the archive to serialize from
	 */
	virtual void SerializeDefaultObject(UObject* Object, FArchive& Ar) final
	{
		SerializeDefaultObject(Object, FStructuredArchiveFromArchive(Ar).GetSlot());
	}

	/** serializes the associated sparse class data for the passed in object using the given archive slot. This should only be called if the class has an associated sparse data structure.
	 * @param Slot the structured archive slot to serialize from
	 */
	COREUOBJECT_API void SerializeSparseClassData(FStructuredArchive::FSlot Slot);

	/** Wraps the PostLoad() call for the class default object.
	 * @param Object the default object to call PostLoad() on
	 */
	COREUOBJECT_API virtual void PostLoadDefaultObject(UObject* Object);

	/** 
	 * Purges out the properties of this class in preparation for it to be regenerated
	 * @param bRecompilingOnLoad - true if we are recompiling on load
	 *
	 * In editor, properties are not freed until DestroyPropertiesPendingDestruction is called.
	 */
	COREUOBJECT_API virtual void PurgeClass(bool bRecompilingOnLoad);

	/**
	 * Finds the common base class that parents the two classes passed in.
	 *
	 * @param InClassA		the first class to find the common base for
	 * @param InClassB		the second class to find the common base for
	 * @return				the common base class or NULL
	 */
	static COREUOBJECT_API UClass* FindCommonBase(UClass* InClassA, UClass* InClassB);

	/**
	 * Finds the common base class that parents the array of classes passed in.
	 *
	 * @param InClasses		the array of classes to find the common base for
	 * @return				the common base class or NULL
	 */
	static COREUOBJECT_API UClass* FindCommonBase(const TArray<UClass*>& InClasses);

	/**
	 * Determines if the specified function has been implemented in a Blueprint
	 *
	 * @param InFunctionName	The name of the function to test
	 * @return					True if the specified function exists and is implemented in a blueprint generated class
	 */
	COREUOBJECT_API virtual bool IsFunctionImplementedInScript(FName InFunctionName) const;

	/**
	 * Checks if the property exists on this class or a parent class.
	 * @param InProperty	The property to check if it is contained in this or a parent class.
	 * @return				True if the property exists on this or a parent class.
	 */
	COREUOBJECT_API virtual bool HasProperty(const FProperty* InProperty) const;

	/** Finds the object that is used as the parent object when serializing properties, overridden for blueprints */
	virtual UObject* FindArchetype(const UClass* ArchetypeClass, const FName ArchetypeName) const { return nullptr; }

	/** Returns archetype object for CDO */
	COREUOBJECT_API virtual UObject* GetArchetypeForCDO() const;

	/** Returns archetype for sparse class data */
	COREUOBJECT_API const void* GetArchetypeForSparseClassData(
		const EGetSparseClassDataMethod GetMethod = EGetSparseClassDataMethod::ArchetypeIfNull) const;

	/** Returns the struct used by the sparse class data archetype */
	COREUOBJECT_API UScriptStruct* GetSparseClassDataArchetypeStruct() const;

	/**
	* Returns all objects that should be preloaded before the class default object is serialized at load time. Only used by the EDL.
	*
	* @param OutDeps		All objects that should be preloaded before the class default object is serialized at load time.
	*/
	virtual void GetDefaultObjectPreloadDependencies(TArray<UObject*>& OutDeps) {}

	/**
	 * Initializes the ClassReps and NetFields arrays used by replication.
	 * This happens lazily based on the CLASS_ReplicationDataIsSetUp flag,
	 * and will generally occur in Link or PostLoad. It's possible that replicated UFunctions
	 * will load after their owning class, so UFunction::PostLoad will clear the flag on its owning class
	 * to force lazy initialization next time the data is needed.
	 * Also happens after blueprint compiliation.
	 */
	COREUOBJECT_API void SetUpRuntimeReplicationData();

	/**
	 * Helper function for determining if the given class is compatible with structured archive serialization
	 */
	static COREUOBJECT_API bool IsSafeToSerializeToStructuredArchives(UClass* InClass);

#if WITH_EDITORONLY_DATA
	/** Returns true if an InstanceDataObject be created from an instance of this type. */
	UE_INTERNAL COREUOBJECT_API virtual bool CanCreateInstanceDataObject() const;
#endif

#if WITH_EDITOR
	/**
	 * If this class was recompiled, returns the path to where we should find the new (i.e. reinstanced) class.
	 * Note: If this is not a recompiled class, this will return a NULL path and should be handled appropriately.
	 */
	COREUOBJECT_API FTopLevelAssetPath GetReinstancedClassPathName() const;
#endif

	/**
	 * Returns true if the object instancing graph should analyze default data at runtime to determine which of its object
	 * properties need to be instanced at construction time, rather than exclusively relying on reflected property semantics.
	 */
	inline bool ShouldUseDynamicSubobjectInstancing() const
	{
		return bNeedsDynamicSubobjectInstancing;
	}

	/**
	 * Returns true if instances of this class rely on instancing on load even in a cooked build to construct non-serialized references.
	 */
	inline bool RequiresPostLoadSubobjectInstancing() const
	{
		return bNeedsPostLoadSubobjectInstancing;
	}

private:
	/** 
	 * This signature intentionally hides the method declared in UObjectBaseUtility to make it private.
	 * Call IsChildOf instead; Hidden because calling IsA on a class almost always indicates an error where the caller should use IsChildOf
	 */
	bool IsA(const UClass* Parent) const
	{
		return UObject::IsA(Parent);
	}

	/**
	 * This signature intentionally hides the method declared in UObject to make it private.
	 * Call ImplementsInterface instead; Hidden because calling Implements on a class almost always indicates an error where the caller should use ImplementsInterface
	 */
	template <typename T>
	bool Implements() const
	{
		return UObject::Implements<T>();
	}

	/**
	 * This signature intentionally hides the method declared in UObject to make it private.
	 * Call FindFunctionByName instead; This method will search for a function declared in UClass instead of the class it was called on
	 */
	UFunction* FindFunction(FName InName) const
	{
		return UObject::FindFunction(InName);
	}

	/** 
	 * This signature intentionally hides the method declared in UObject to make it private.
	 * Call FindFunctionByName instead; This method will search for a function declared in UClass instead of the class it was called on
	 */
	UFunction* FindFunctionChecked(FName InName) const
	{
		return UObject::FindFunctionChecked(InName);
	}

	/**
	 * Tests if all properties tagged with Replicate were registered in GetLifetimeReplicatedProps
	 */
	COREUOBJECT_API void AssembleReferenceTokenStreamInternal(bool bForce = false);
	COREUOBJECT_API void InternalCreateDefaultObjectWrapper() const;
#if UE_WITH_IMMUTABLE_CDO
	COREUOBJECT_API void InternalCreateImmutableDefaultObject() const;
#endif
protected:
	/**
	 * Get the default object from the class, creating it if missing, if requested or under a few other circumstances
	 * @return		the CDO for this class
	 **/
	COREUOBJECT_API virtual UObject* CreateDefaultObject();

#if WITH_EDITOR
	/**
	 * Internal helper method for GetReinstancedClassPathName(). Subclasses can choose to override the default
	 * implementation to return a valid class path for recompiled class objects based on how reinstancing is handled.
	 */
	virtual FTopLevelAssetPath GetReinstancedClassPathName_Impl() const { return nullptr; }
#endif
};

#if UE_WITH_CONSTINIT_UOBJECT
extern TNoDestroy<UClass> Z_ConstInit_UClass_UObject;
extern TNoDestroy<UClass> Z_ConstInit_UClass_UClass;
consteval UClass::UClass(
	UE::CodeGen::ConstInit::EClass,
	UE::CodeGen::ConstInit::FObjectParams ObjectParams,
	UE::CodeGen::ConstInit::FUFieldParams FieldParams,
	UE::CodeGen::ConstInit::FStructParams StructParams,
	UE::CodeGen::ConstInit::FClassParams Params
)
	: UClass(ObjectParams.SetClass(&Z_ConstInit_UClass_UClass), FieldParams, StructParams, Params)
{
}
consteval UClass::UClass(
	UE::CodeGen::ConstInit::EObject,
	UE::CodeGen::ConstInit::FObjectParams ObjectParams,
	UE::CodeGen::ConstInit::FUFieldParams FieldParams,
	UE::CodeGen::ConstInit::FStructParams StructParams,
	UE::CodeGen::ConstInit::FClassParams Params
)
	: UClass(ObjectParams, FieldParams, StructParams, Params.SetWithin(&Z_ConstInit_UClass_UObject))
{
}
consteval UClass::UClass(
	UE::CodeGen::ConstInit::FObjectParams ObjectParams,
	UE::CodeGen::ConstInit::FUFieldParams FieldParams,
	UE::CodeGen::ConstInit::FStructParams StructParams,
	UE::CodeGen::ConstInit::FClassParams Params
)
	// Initialization of class for UClass passes null because of constexpr evaluation order and TNoDestroy wrapper
	: Super(ObjectParams, FieldParams, StructParams)
	, ClassConstructor(Params.ClassConstructor)
	, ClassVTableHelperCtorCaller(Params.ClassVTableHelperCtorCaller)
	, CppClassStaticFunctions(Params.CppClassStaticFunctions)
	, ClassUnique(0)
	, bCooked(false)
	, bLayoutChanging(false)
	, ClassFlags(Params.ClassFlags | CLASS_Native)
	, ClassCastFlags(Params.ClassCastFlags)
	, ClassWithin(Params.ClassWithin)
#if WITH_EDITORONLY_DATA
	, PropertiesPendingDestruction(nullptr)
#endif
	, CompiledInClassConfigName(Params.ConfigName)
	, ClassReps(ConstEval)
	, NetFields(ConstEval)
	, SparseClassData(nullptr)
	, SparseClassDataStruct(Params.SparseClassDataStruct)
#if WITH_EDITOR
	, CppTypeInfo(InPlace, Params.CppClassTypeInfo)
#endif
	, FuncMap(ConstEval)
	, FuncMapLock()
	, AllFunctionsCacheLock()
	, AllFunctionsCache(ConstEval)
	, CompiledInInterfaces(Params.Interfaces)
	, CompiledInNativeFunctions(Params.NativeFunctions)
{
	if (StructParams.Super == nullptr)
	{
		checkf(Params.NativeFunctions.Num() == 0, TEXT("UObject must have no native functions because ScriptCore will try and override it"));
	}
}
#endif // UE_WITH_CONSTINIT_UOBJECT

/**
 * Helper template to call the default constructor for a class
 */
template<class T>
void InternalConstructor( const FObjectInitializer& X )
{ 
	T::__DefaultConstructor(X);
}


/**
 * Helper template to call the vtable ctor caller for a class
 */
template<class T>
UObject* InternalVTableHelperCtorCaller(FVTableHelper& Helper)
{
	return T::__VTableCtorCaller(Helper);
}

COREUOBJECT_API void InitializePrivateStaticClass(
	class UClass* (*TClass_StaticClassFn)(),
	class UClass* TClass_Super_StaticClass,
	class UClass* TClass_PrivateStaticClass,
	class UClass* TClass_WithinClass_StaticClass,
	const TCHAR* PackageName,
	const TCHAR* Name
);

/**
 * Helper template allocate and construct a UClass
 *
 * @param PackageName name of the package this class will be inside
 * @param Name of the class
 * @param ReturnClass reference to pointer to result. This must be PrivateStaticClass.
 * @param RegisterNativeFunc Native function registration function pointer.
 * @param InSize Size of the class
 * @param InAlignment Alignment of the class
 * @param InClassFlags Class flags
 * @param InClassCastFlags Class cast flags
 * @param InConfigName Class config name
 * @param InClassConstructor Class constructor function pointer
 * @param InClassVTableHelperCtorCaller Class constructor function for vtable pointer
 * @param InCppClassStaticFunctions Function pointers for the class's version of Unreal's reflected static functions
 * @param InSuperClassFn Super class function pointer
 * @param InWithinClassFn Within class function pointer
 */
COREUOBJECT_API void GetPrivateStaticClassBody(
	const TCHAR* PackageName,
	const TCHAR* Name,
	UClass*& ReturnClass,
	void(*RegisterNativeFunc)(),
	uint32 InSize,
	uint32 InAlignment,
	EClassFlags InClassFlags,
	EClassCastFlags InClassCastFlags,
	const TCHAR* InConfigName,
	UClass::ClassConstructorType InClassConstructor,
	UClass::ClassVTableHelperCtorCallerType InClassVTableHelperCtorCaller,
	FUObjectCppClassStaticFunctions&& InCppClassStaticFunctions,
	UClass::StaticClassFunctionType InSuperClassFn,
	UClass::StaticClassFunctionType InWithinClassFn);

#if UE_WITH_CONSTINIT_UOBJECT
namespace UE::CodeGen::ConstInit
{
	struct FScriptStructParams
	{
		EStructFlags StructFlags; 
		UScriptStruct::ICppStructOps* StructOps;
	};
}

consteval UScriptStruct::UScriptStruct(
	UE::CodeGen::ConstInit::FObjectParams ObjectParams,
	UE::CodeGen::ConstInit::FUFieldParams FieldParams,
	UE::CodeGen::ConstInit::FStructParams StructParams,
	UE::CodeGen::ConstInit::FScriptStructParams Params
)
	: UStruct(ObjectParams, FieldParams, StructParams)
	, StructFlags(Params.StructFlags)
	, bPrepareCppStructOpsCompleted(false)
	, CppStructOps(Params.StructOps)
{
}
#endif // UE_WITH_CONSTINIT_UOBJECT

/*-----------------------------------------------------------------------------
	FObjectInstancingGraph.
-----------------------------------------------------------------------------*/

enum class EInstancePropertyValueFlags
{
	None                   = 0x00,

	// if set, then this property causes an instance to be created, otherwise this is just a pointer to a uobject that should be remapped if the object is instanced for some other property
	CausesInstancing       = 0x01,

	// if set, instance the reference to the subobjectroot, so far only delegates remap a self reference
	AllowSelfReference     = 0x02,

	// if set, then we do not create a new instance, but we will reassign one if there is already a mapping in the table
	DoNotCreateNewInstance = 0x04
};

ENUM_CLASS_FLAGS(EInstancePropertyValueFlags)

enum class EObjectInstancingGraphOptions
{
	None = 0x00,

	// if set, start with component instancing disabled
	DisableInstancing = 0x01,

	// if set, instance only subobject template values
	InstanceTemplatesOnly = 0x02,
};

ENUM_CLASS_FLAGS(EObjectInstancingGraphOptions)

struct FObjectInstancingGraph
{
public:

	/** 
	 * Default Constructor 
	 * @param bDisableInstancing - if true, start with component instancing disabled
	**/
	COREUOBJECT_API explicit FObjectInstancingGraph(bool bDisableInstancing = false);

	/**
	 * Constructor with options
	 * @param InOptions Additional options to modify the behavior of this graph
	**/
	COREUOBJECT_API explicit FObjectInstancingGraph(EObjectInstancingGraphOptions InOptions);

	/**
	 * Standard constructor
	 *
	 * @param	DestinationSubobjectRoot	the top-level object that is being created
	 * @param	InOptions					Additional options to modify the behavior of this graph
	 */
	COREUOBJECT_API explicit FObjectInstancingGraph( class UObject* DestinationSubobjectRoot, EObjectInstancingGraphOptions InOptions = EObjectInstancingGraphOptions::None);

	/**
	 * Returns whether this instancing graph has a valid destination root.
	 */
	bool HasDestinationRoot() const
	{
		return GetDestinationRoot() != nullptr;
	}

	/**
	 * Returns the DestinationRoot for this instancing graph.
	 */
	const UObject* GetDestinationRoot() const
	{
		return DestinationRoot;
	}

	/**
	 * Sets the DestinationRoot for this instancing graph.
	 *
	 * @param	DestinationSubobjectRoot	the top-level object that is being created
	 * @param	InSourceRoot	Archetype of DestinationSubobjectRoot
	 */
	COREUOBJECT_API void SetDestinationRoot( class UObject* DestinationSubobjectRoot, class UObject* InSourceRoot = nullptr );

	/**
	 * Finds the destination object instance corresponding to the specified source object.
	 *
	 * @param	SourceObject			the object to find the corresponding instance for
	 */
	COREUOBJECT_API class UObject* GetDestinationObject(class UObject* SourceObject);

	/**
	 * Returns the component that has SourceComponent as its archetype, instancing the component as necessary.
	 *
	 * @param	SourceComponent		the component to find the corresponding component instance for
	 * @param	CurrentValue		the component currently assigned as the value for the component property
	 *								being instanced.  Used when updating archetypes to ensure that the new instanced component
	 *								replaces the existing component instance in memory.
	 * @param	Owner				the object that owns the component property currently being instanced;  this is NOT necessarily the object
	 *								that should be the Outer for the new component.
	 * @param	Flags				reinstancing flags - see EInstancePropertyValueFlags
	 *
	 * @return	As with GetInstancedSubobject, above, but also deals with archetype creation and a few other special cases
	 */
	COREUOBJECT_API class UObject* InstancePropertyValue( class UObject* SourceComponent, TNotNull<UObject*> CurrentValue, TNotNull<UObject*> Owner, EInstancePropertyValueFlags Flags = EInstancePropertyValueFlags::None );

	/**
	 * Adds a partially built object instance to the map(s) of source objects to their instances.
	 * @param	ObjectInstance  Object that was just allocated, but has not been constructed yet
	 * @param	InArchetype     Archetype of ObjectInstance
	 */
	COREUOBJECT_API void AddNewObject(class UObject* ObjectInstance, class UObject* InArchetype = nullptr);

	/**
	 * Adds an object instance to the map of source objects to their instances.  If there is already a mapping for this object, it will be replaced
	 * and the value corresponding to ObjectInstance's archetype will now point to ObjectInstance.
	 *
	 * @param	ObjectInstance  the object that should be added as the corresopnding instance for ObjectSource
	 * @param	InArchetype     Archetype of ObjectInstance
	 */
	COREUOBJECT_API void AddNewInstance(class UObject* ObjectInstance, class UObject* InArchetype = nullptr);

	/**
	 * Adds an object instance to the map of source objects to their instances of the outer chain already instanciated as part of prepopulation.
	 * We use this to pre-create the instantiation graph when subobject are re-instantiated
	 * If there is already a mapping for this object, it will be replaced
	 * and the value corresponding to ObjectInstance's archetype will now point to ObjectInstance.
	 *
	 * @param	ObjectInstance  the object that should be added as the corresopnding instance for ObjectSource
	 * @param	InArchetype     Archetype of ObjectInstance
	 */
	COREUOBJECT_API void PrepopulateWithInstance(TNotNull<UObject*> ObjectInstance, TNotNull<UObject*> InArchetype);

	/**
	 * Removes an object instance from the prepopulated graph, if it exists
	 */
	COREUOBJECT_API void RemoveInstanceFromPrepopulate(TNotNull<UObject*> ObjectInstance);

	/**
	 * Retrieves a list of objects that have the specified Outer
	 *
	 * @param	SearchOuter		the object to retrieve object instances for
	 * @param	out_Components	receives the list of objects contained by SearchOuter
	 */
	COREUOBJECT_API void RetrieveObjectInstances( class UObject* SearchOuter, TArray<class UObject*>& out_Objects );

	/**
	 * Allows looping over instances that were created during this instancing.
	 *
	 * @param	Pred		the object to retrieve object instances for
	 */
	template <typename Predicate>
	void ForEachObjectInstance(Predicate Pred)
	{
		for (TMap<UObject*, UObject*>::TIterator It(SourceToDestinationMap); It; ++It)
		{
			UObject* InstancedObject = It.Value();
			Pred(InstancedObject);
		}
	}

	/**
	 * Enables / disables component instancing.
	 */
	void EnableSubobjectInstancing( bool bEnabled )
	{
		if (bEnabled)
		{
			InstancingOptions &= ~EObjectInstancingGraphOptions::DisableInstancing;
		}
		else
		{
			InstancingOptions |= EObjectInstancingGraphOptions::DisableInstancing;
		}
	}

	/**
	 * Returns whether component instancing is enabled
	 */
	bool IsSubobjectInstancingEnabled() const
	{
		return !(InstancingOptions & EObjectInstancingGraphOptions::DisableInstancing);
	}

	/**
	 * Sets whether DestinationRoot is currently being loaded from disk.
	 */
	void SetLoadingObject( bool bIsLoading )
	{
		bLoadingObject = bIsLoading;
	}

	/**
	 * Adds a member variable property that should not instantiate subobjects
	 */
	void AddPropertyToSubobjectExclusionList(const FProperty* Property)
	{
		SubobjectInstantiationExclusionList.Add(Property);
	}

	/**
	 * Checks if a member variable property is in subobject instantiation exclusion list
	 */
	bool IsPropertyInSubobjectExclusionList(const FProperty* Property) const
	{
		return SubobjectInstantiationExclusionList.Contains(Property);
	}

private:
	/**
	 * Returns whether DestinationRoot corresponds to an archetype object.
	 *
	 * @param	bUserGeneratedOnly	true indicates that we only care about cases where the user selected "Create [or Update] Archetype" in the editor
	 *								false causes this function to return true even if we are just loading an archetype from disk
	 */
	bool IsCreatingArchetype( bool bUserGeneratedOnly=true ) const
	{
		// if we only want cases where we are creating an archetype in response to user input, return false if we are in fact just loading the object from disk
		return bCreatingArchetype && (!bUserGeneratedOnly || !bLoadingObject);
	}

	/**
	 * Returns whether DestinationRoot is currently being loaded from disk.
	 */
	bool IsLoadingObject() const
	{
		return bLoadingObject;
	}

	/**
	 * Returns the component that has SourceComponent as its archetype, instancing the component as necessary.
	 *
	 * @param	SourceComponent		the component to find the corresponding component instance for
	 * @param	CurrentValue		the component currently assigned as the value for the component property
	 *								being instanced.  Used when updating archetypes to ensure that the new instanced component
	 *								replaces the existing component instance in memory.
	 * @param	CurrentObject		the object that owns the component property currently being instanced;  this is NOT necessarily the object
	 *								that should be the Outer for the new component.
	 * @param	Flags				reinstancing flags - see EInstancePropertyValueFlags
	 *
	 * @return	if SourceComponent is contained within SourceRoot, returns a pointer to a unique component instance corresponding to
	 *			SourceComponent if SourceComponent is allowed to be instanced in this context, or NULL if the component isn't allowed to be
	 *			instanced at this time (such as when we're a client and the component isn't loaded on clients)
	 *			if SourceComponent is not contained by SourceRoot, return INVALID_OBJECT, indicating that the that has SourceComponent as its ObjectArchetype, or NULL if SourceComponent is not contained within
	 *			SourceRoot.
	 */
	COREUOBJECT_API class UObject* GetInstancedSubobject( class UObject* SourceSubobject, class UObject* CurrentValue, class UObject* CurrentObject, EInstancePropertyValueFlags Flags );

	/**
	 * The root of the object tree that is the source used for instancing components;
	 * - when placing an instance of an actor class, this would be the actor class default object
	 * - when placing an instance of an archetype, this would be the archetype
	 * - when creating an archetype, this would be the actor instance
	 * - when duplicating an object, this would be the duplication source
	 */
	class		UObject*						SourceRoot = nullptr;

	/**
	 * The root of the object tree that is the destination used for instancing components
	 * - when placing an instance of an actor class, this would be the placed actor
	 * - when placing an instance of an archetype, this would be the placed actor
	 * - when creating an archetype, this would be the actor archetype
	 * - when updating an archetype, this would be the source archetype
	 * - when duplicating an object, this would be the copied object (destination)
	 */
	class		UObject*						DestinationRoot = nullptr;

	/** Subobject instancing options */
	EObjectInstancingGraphOptions				InstancingOptions = EObjectInstancingGraphOptions::None;

	/**
	 * Indicates whether we are currently instancing components for an archetype.  true if we are creating or updating an archetype.
	 */
	bool										bCreatingArchetype = false;

	/**
	 * true when loading object data from disk.
	 */
	bool										bLoadingObject = false;

	/**
	 * Maps the source (think archetype) to the destination (think instance)
	 */
	TMap<class UObject*,class UObject*>			SourceToDestinationMap;

	/**
	 * Reverse maps the destination back to the source for dynamic instancing
	 */
	TMap<class UObject*, class UObject*>		DestinationToSourceMap;

	/** List of member variable properties that should not instantiate subobjects */
	TSet<const FProperty*>						SubobjectInstantiationExclusionList;
};

Expose_TNameOf(FObjectInstancingGraph);

Expose_TNameOf(FObjectInstancingGraph*);

// UFunction interface.

inline UFunction* UFunction::GetSuperFunction() const
{
	UStruct* Result = GetSuperStruct();
	checkSlow(!Result || Result->IsA<UFunction>());
	return (UFunction*)Result;
}


// UObject.h

/**
 * Returns true if this object implements the interface T, false otherwise.
 */
template<class T>
inline bool UObject::Implements() const
{
	UClass const* const MyClass = GetClass();
	return MyClass && MyClass->ImplementsInterface(T::StaticClass());
}

// UObjectGlobals.h

/**
 * Gets the default object of a class.
 *
 * In most cases, class default objects should not be modified. This method therefore returns
 * an immutable pointer. If you need to modify the default object, use GetMutableDefault instead.
 *
 * @param Class - The class to get the CDO for.
 *
 * @return Class default object (CDO).
 *
 * @see GetMutableDefault
 */
template< class T > 
inline const T* GetDefault(const UClass* Class)
{
	check(Class->GetDefaultObject()->IsA(T::StaticClass()));
	return (const T*)Class->GetDefaultObject();
}

/**
 * Gets the mutable default object of a class.
 *
 * @param Class - The class to get the CDO for.
 *
 * @return Class default object (CDO).
 *
 * @see GetDefault
 */
template< class T > 
inline T* GetMutableDefault(UClass *Class)
{
	check(Class->GetDefaultObject()->IsA(T::StaticClass()));
	return (T*)Class->GetDefaultObject();
}

struct FStructUtils
{
	COREUOBJECT_API static bool ArePropertiesTheSame(const FProperty* A, const FProperty* B, bool bCheckPropertiesNames);

	/** Do structures have exactly the same memory layout */
	COREUOBJECT_API static bool TheSameLayout(const UStruct* StructA, const UStruct* StructB, bool bCheckPropertiesNames = false);

	/** Locates a named structure in the package with the given name. Not expected to fail */
	COREUOBJECT_API static UStruct* FindStructureInPackageChecked(const TCHAR* StructName, const TCHAR* PackageName);

#if !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
	/** Looks for uninitialized script struct pointers. Returns the number found */
	COREUOBJECT_API static int32 AttemptToFindUninitializedScriptStructMembers();

#if WITH_METADATA
	/** Looks for short type names within struct metadata. Returns the number found */
	COREUOBJECT_API static int32 AttemptToFindShortTypeNamesInMetaData();
#endif // WITH_METADATA
#endif // !(UE_BUILD_TEST || UE_BUILD_SHIPPING)
};

/*-----------------------------------------------------------------------------
	Mirrors of mirror structures in Object.h. These are used by generated code 
	to facilitate correct offsets and alignments for structures containing these
	odd types.
-----------------------------------------------------------------------------*/

template <typename T, bool bHasStaticStruct = TModels_V<CStaticStructProvider, T>>
struct TBaseStructureBase
{
	static UScriptStruct* Get()
	{
		return T::StaticStruct();
	}
};

template <typename T>
struct TBaseStructureBase<T, false>
{
};

template <typename T>
struct TBaseStructure : TBaseStructureBase<T>
{
};

template <typename T>
struct TBaseStructure<const T> : TBaseStructure<T>
{
};

template<> struct TBaseStructure<FIntPoint> 
{
	static COREUOBJECT_API UScriptStruct* Get(); 
};

template<> struct TBaseStructure<FIntVector> 
{ 
	static COREUOBJECT_API UScriptStruct* Get(); 
};

template<> struct TBaseStructure<FInt64Vector2>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FIntVector4>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FLinearColor>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FColor>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FRandomStream>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FGuid>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FFallbackStruct>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurveFloat>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurveVector2D>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurveVector>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurveQuat>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurveTwoVectors>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurveLinearColor>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurvePointFloat>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurvePointVector2D>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurvePointVector>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurvePointQuat>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurvePointTwoVectors>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInterpCurvePointLinearColor>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FFloatRangeBound>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FFloatRange>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FDoubleRangeBound>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FDoubleRange>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInt32RangeBound>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInt32Range>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FFloatInterval>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FDoubleInterval>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FInt32Interval>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FFrameNumber;

template<> struct TBaseStructure<FFrameNumber>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FFrameTime;

template<> struct TBaseStructure<FFrameTime>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FFrameRate;

template<> struct TBaseStructure<FFrameRate>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FSoftObjectPath;
struct FSoftClassPath;

struct FPrimaryAssetType;
struct FPrimaryAssetId;

struct FDateTime;

template<> struct TBaseStructure<FDateTime>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FPolyglotTextData;

template<> struct TBaseStructure<FPolyglotTextData>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FAssetBundleData;
struct FTopLevelAssetPath;

template<> struct TBaseStructure<FRemoteObjectId>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<>
struct TStructOpsTypeTraits<FRemoteObjectId> : public TStructOpsTypeTraitsBase2<FRemoteObjectId>
{
	enum
	{
		WithNetSerializer = true,
		WithSerializer = true,
	};
};

struct FRemoteObjectTables;
template<> struct TBaseStructure<FRemoteObjectTables>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FRemoteObjectPathName;
template<> struct TBaseStructure<FRemoteObjectPathName>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FPackedRemoteObjectPathName;
template<> struct TBaseStructure<FPackedRemoteObjectPathName>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FRemoteObjectBytes;
template<> struct TBaseStructure<FRemoteObjectBytes>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FPackedRemoteObjectHeader;
template<> struct TBaseStructure<FPackedRemoteObjectHeader>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

struct FRemoteObjectData;
template<> struct TBaseStructure<FRemoteObjectData>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<> struct TBaseStructure<FRemoteServerId>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

template<>
struct TStructOpsTypeTraits<FRemoteServerId> : public TStructOpsTypeTraitsBase2<FRemoteServerId>
{
	enum
	{
		WithNetSerializer = true,
		WithSerializer = true,
	};
};

struct FRemoteObjectReference;
template<> struct TBaseStructure<FRemoteObjectReference>
{
	static COREUOBJECT_API UScriptStruct* Get();
};

// TBaseStructure for explicit core variant types only. e.g. FVector3d returns "Vector3d" struct. 
template< class T > struct TVariantStructure
{
	static_assert(sizeof(T) == 0, "Unsupported for this type. Did you mean to use TBaseStructure?");
};

#define UE_DECLARE_CORE_VARIANT_TYPE(VARIANT, CORE)														\
template<> struct TBaseStructure<::F##CORE> { COREUOBJECT_API static UScriptStruct* Get(); };			\
template<> struct TVariantStructure<::F##VARIANT##f> { COREUOBJECT_API static UScriptStruct* Get(); };	\
template<> struct TVariantStructure<::F##VARIANT##d> { COREUOBJECT_API static UScriptStruct* Get(); };

UE_DECLARE_CORE_VARIANT_TYPE(Vector2,	Vector2D);
UE_DECLARE_CORE_VARIANT_TYPE(Vector3,	Vector);
UE_DECLARE_CORE_VARIANT_TYPE(Vector4,	Vector4);
UE_DECLARE_CORE_VARIANT_TYPE(Plane4,	Plane);
UE_DECLARE_CORE_VARIANT_TYPE(Quat4,		Quat);
UE_DECLARE_CORE_VARIANT_TYPE(Rotator3,	Rotator);
UE_DECLARE_CORE_VARIANT_TYPE(Transform3,Transform);
UE_DECLARE_CORE_VARIANT_TYPE(Matrix44,	Matrix);
UE_DECLARE_CORE_VARIANT_TYPE(Box2,		Box2D);
UE_DECLARE_CORE_VARIANT_TYPE(Ray3,		Ray);
UE_DECLARE_CORE_VARIANT_TYPE(Sphere3, Sphere);

#undef UE_DECLARE_CORE_VARIANT_TYPE

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Async/Mutex.h"
#include "Async/SharedLock.h"
#include "Async/SharedMutex.h"
#include "Async/UniqueLock.h"
#include "Math/Box2D.h"
#include "Math/Matrix.h"
#include "Math/Plane.h"
#include "Math/Quat.h"
#include "Math/RandomStream.h"
#include "Math/Rotator.h"
#include "Math/Transform.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#include "Misc/FallbackStruct.h"
#include "Templates/EnableIf.h"
#include "Templates/NoDestroy.h"
#include "Trace/Detail/Channel.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UnrealNames.h"
#endif
