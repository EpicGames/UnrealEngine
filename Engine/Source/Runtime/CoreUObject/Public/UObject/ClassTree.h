// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ClassTree.h: Class hierarchy management classes.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Containers/List.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"

enum EClassFlagMatchType
{
	/** matches if the flags contain any of the mask flags specified */
	MATCH_Any,

	/** matches only if the flags contain all of the mask flags specified */
	MATCH_All,
};

/**
 * Manages an inheritance tree.  There is one FClassTree node for each UClass.  Each node 
 * stores pointers to its parent and child nodes.  New nodes should always be added to the root node.
 * This is enforced by allowing const references to child nodes.
 */
class FClassTree
{
	UClass* Class;
	FClassTree* Parent;
	TArray<FClassTree*> Children;
	TDoubleLinkedList<UObject*> Instances;

	//@name Utility functions used for internal management of class tree
	//@{

	/**
	 * Private interface for adding a new UClass to the class tree.  Takes care of inserting the class into
	 * the correct location in the structure.  It is not required to add parent classes before adding child classes.
	 * 
	 * @param	ChildClass	the class to add to the tree
	 *
	 * @return	true if this node was succesfully added as a child node of this one
	 */
	bool AddChildClass( UClass* ChildClass )
	{
		check(ChildClass);

		// if the new class is already in the tree, don't add it again
		if ( ChildClass == Class )
			return true;

		// if the new class isn't a child of the current node, check to see if we need to insert
		// the new class into the tree at this node's location (i.e. the new class is actually the parent
		// of the current node's class)
		if ( !ChildClass->IsChildOf(Class) )
		{
			if ( Parent )
			{
				if ( ChildClass != Parent->GetClass() &&	// if our parent node's class is not the same as the new class
					Class->IsChildOf(ChildClass) )			// and the new class is a parent class of this node's class
				{
					// make the new class our parent node
					Parent->ReplaceChild(ChildClass,this);
					return true;
				}
			}

			return false;
		}

		// the new class belongs on this branch - find out if it belongs in any of our child nodes
		for ( int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++ )
		{
			if ( Children[ChildIndex]->AddChildClass(ChildClass) )
				return 1;
		}

		// none of our child nodes accepted the new class - so it'll be a child of this node
		FClassTree* NewChild = new FClassTree( ChildClass );
		AddChildNode(NewChild);
		return 1;
	}

	/**
	 * Adds a new child node to this node, sorted alphabetically by name.
	 * Ignores duplicate entries.
	 * 
	 * @param	NewChild	the node to add to the Children array
	 *
	 * @return	the index into the Children array for the new node, or the index for
	 *			the existing node is this is a duplicate
	 */
	int32 AddChildNode( FClassTree* NewChild )
	{
		check(NewChild);

		NewChild->Parent = this;
		int32 i = Children.Find(NewChild);
		if ( i == INDEX_NONE )
		{
			for ( i = 0; i < Children.Num(); i++ )
			{
				FClassTree* Child = Children[i];

				// insert this class sorted alphabetically
				if ( FCString::Stricmp(*Child->GetClass()->GetName(), *NewChild->GetClass()->GetName()) >= 0 )
				{
					break;
				}
			}

			Children.InsertUninitialized(i);
			Children[i] = NewChild;
		}

		return i;
	}

	/**
	 * Replaces an existing child with the class specified, and changes the existing
	 * child's parent node to the new child.
	 * For example, when changing NodeA -> NodeB -> NodeC to NodeA -> NodeC -> NodeB,
	 * NewChild is NodeC, and CurrentChild is NodeB
	 * 
	 * @param	NewChild		the class that should be inserted into the chain between this node and the CurrentChild node
	 * @param	CurrentChild	the existing node that should become a child of the new node
	 */
	void ReplaceChild( UClass* NewChild, FClassTree* CurrentChild )
	{
		FClassTree* NewChildNode = new FClassTree( NewChild );

		// find the index of the old child
		int32 OldIndex = Children.Find(CurrentChild);

		// remove it from our list of children
		Children.RemoveAt(OldIndex);

		// then check to see if any existing children really belong under this new child
		for ( int32 ChildIndex = Children.Num() - 1; ChildIndex >= 0; ChildIndex-- )
		{
			FClassTree* ChildNode = Children[ChildIndex];
			UClass* ChildClass = ChildNode->GetClass();
			if ( ChildClass->IsChildOf(NewChild) )
			{
				// remove this class from our list of Children
				Children.RemoveAt(ChildIndex);

				// and insert it under the new child
				NewChildNode->AddChildNode(ChildNode);
			}
		}

		// add a child node for the new class
		int32 NewIndex = AddChildNode(NewChildNode);

		// add the old node to the new node as a child
		Children[NewIndex]->AddChildNode(CurrentChild);
	}


	/**
	 * Find the node associated with the class specified
	 * 
	 * @param	SearchClass	the class to search for
	 * @param	bBruteForce	if true, searches every node for the specified class, rather than using IsChildOf() to optimize the search
	 *
	 * @return	a pointer to the node associated with the class specified, or NULL if this branch doesn't contain that class
	 */
	FClassTree* GetNode( UClass* SearchClass, bool bBruteForce )
	{
		FClassTree* Result(NULL);

		if ( SearchClass == Class )
			Result = this;

		else if ( bBruteForce || SearchClass->IsChildOf(Class) )
		{
			for ( int32 i = 0; i < Children.Num(); i++ )
			{
				Result = Children[i]->GetNode(SearchClass,bBruteForce);
				if ( Result != NULL )
					break;
			}
		}

		return Result;
	}

	/**
	 * Find the child index for the class specified
	 * 
	 * @param	SearchClass	the class to search for
	 *
	 * @return	the index into the Children array for the node associated with the class specified,
	 *			or INDEX_NONE if that class isn't in the list of children
	 */
	int32 FindChildIndex( UClass* SearchClass ) const
	{
		for ( int32 i = 0; i < Children.Num(); i++ )
		{
			if ( Children[i]->GetClass() == SearchClass )
				return i;
		}

		return INDEX_NONE;
	}

	//@}

public:

	/**
	 * Utility/convenience method for populating a class tree.
	 */
	void PopulateTree()
	{
		FClassTree* RootNode = GetRootNode();
		for ( TObjectIterator<UClass> It; It; ++It )
		{
			RootNode->AddClass(*It);
		}
	}

	/**
	 * Public interface for adding a new class to the tree.  Actual functionality implemented separately
	 * in order to support being able to call AddClass() on any node in the tree structure and have it
	 * inserted into the correct location.
	 *
	 * Can be called on any node in the tree.  Takes care of inserting the class into the correct
	 * location in the structure.  Correctly handles adding classes in any arbitrary order (i.e.
	 * parent classes do not need to be added before child classes, etc.)
	 * 
	 * @param	ChildClass	the class to add to the tree
	 *
	 * @return	buffer that will contain the post processed text
	 */
	bool AddClass( UClass* ChildClass )
	{
		// in order to ensure that all classes are placed into the correct locations in the tree,
		// only the root node can accept new classes
		FClassTree* ReceivingNode;
		for ( ReceivingNode = this; ReceivingNode->Parent != NULL; ReceivingNode = ReceivingNode->Parent );

		return ReceivingNode->AddChildClass(ChildClass);
	}

	/**
	 * Get the class associated with this node
	 * 
	 * @return	a pointer to the class associated with this node
	 */
	UClass* GetClass() const
	{
		return Class;
	}

private:
	struct FScopedRecurseDepthCounter
	{
		int32* Counter;
		FScopedRecurseDepthCounter(int32* InCounter)
			: Counter(InCounter)
		{
			(*Counter)++;
		}
		~FScopedRecurseDepthCounter()
		{
			(*Counter)--;
		}
	};
public:

	/**
	 * Retrieve the child nodes of this node
	 * 
	 * @param	ChildClasses	[out] this node's children
	 * @param	bRecurse		if false, only direct children of this node's class will be considered
	 */
	void GetChildClasses( TArray<FClassTree*>& ChildClasses, bool bRecurse=false )
	{
		static int32 RecurseDepth = 0;
		if (RecurseDepth == 0)
		{
			ChildClasses.Empty();
		}
		for ( int32 i = 0; i < Children.Num(); i++ )
		{
			ChildClasses.Add( Children[i] );
		}

		if ( bRecurse )
		{
			FScopedRecurseDepthCounter Counter(&RecurseDepth);
			for ( int32 i = 0; i < Children.Num(); i++ )
			{
				Children[i]->GetChildClasses(ChildClasses, bRecurse);
			}
		}
	}

	/**
	 * Retrieve the child nodes of this node
	 * 
	 * @param	ChildClasses	[out] this node's children
	 * @param	bRecurse		if false, only direct children of this node's class will be considered
	 */
	void GetChildClasses( TArray<const FClassTree*>& ChildClasses, bool bRecurse=false ) const
	{
		static int32 RecurseDepth = 0;
		if (RecurseDepth == 0)
		{
			ChildClasses.Empty();
		}
		for ( int32 i = 0; i < Children.Num(); i++ )
		{
			ChildClasses.Add( Children[i] );
		}

		if ( bRecurse )
		{
			FScopedRecurseDepthCounter Counter(&RecurseDepth);
			for ( int32 i = 0; i < Children.Num(); i++ )
			{
				Children[i]->GetChildClasses(ChildClasses, bRecurse);
			}
		}
	}

	/**
	 * Retrieve the child nodes of this node that match the flags specified
	 * 
	 * @param	ChildClasses	[out] array of child classes with class flags matching the flags specified
	 * @param	Mask			info on how to determine whether classes are included in the results.
	 *							Must implement bool IsValidClass(const FClassTree* Node) const.
	 * @param	bRecurse		if false, only direct children of this node's class will be considered
	 */
	template< class Comparator >
	void GetChildClasses( TArray<UClass*>& ChildClasses, const Comparator& Mask, bool bRecurse=false ) const
	{
		for ( int32 i = 0; i < Children.Num(); i++ )
		{
			FClassTree* ChildNode = Children[i];
			if ( Mask(ChildNode->GetClass()) )
			{
				ChildClasses.Add( ChildNode->GetClass() );
			}

			if ( bRecurse )
			{
				ChildNode->GetChildClasses(ChildClasses, Mask, bRecurse);
			}
		}
	}

	/**
	 * Creates a new class tree rooted at this node's class, which contains only
	 * classes which match the class flags specified.
	 * 
	 * @param	Mask	information about the types of classes to include. 
	 *					Must implement bool IsValidClass(const FClassTree* Node) const.
	 *
	 * @return	a pointer to an FClassTree that contains only classes which match the parameters
	 *			specified by MaskParams.  If the class associated with this node does not meet the
	 *			requirements of the MaskParams, the return value is a NULL pointer, and the class tree
	 *			will not contain any of the children of that class.
	 *			NOTE: the caller is responsible for deleting the FClassTree returned by this function!!
	 */
	template< class Comparator >
	FClassTree* GenerateMaskedClassTree( const Comparator& Mask ) const
	{
		FClassTree* ResultTree = NULL;
		if ( Mask.IsValidClass(this) )
		{
			ResultTree = new FClassTree(Class);
			for ( int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++ )
			{
				FClassTree* ChildNode = Children[ChildIndex]->GenerateMaskedClassTree(Mask);
				if ( ChildNode != NULL )
				{
					ChildNode->Parent = ResultTree;
					ResultTree->Children.Add(ChildNode);
				}
			}
		}

		return ResultTree;
	}

	/**
	 * Gets the root node for this class tree
	 */
	FClassTree* GetRootNode()
	{
		FClassTree* RootNode;
		for ( RootNode = this; RootNode->Parent != NULL; RootNode = RootNode->Parent );
		return RootNode;
	}

	/**
	 * Gets the root node for this class tree
	 */
	const FClassTree* GetRootNode() const
	{
		const FClassTree* RootNode;
		for ( RootNode = this; RootNode->Parent != NULL; RootNode = RootNode->Parent );
		return RootNode;
	}

	/**
	 * Find the node associated with the class specified
	 * 
	 * @param	SearchClass	the class to search for
	 *
	 * @return	a pointer to the node associated with the class specified, or NULL if this branch doesn't contain that class
	 */
	FClassTree* GetNode( UClass* SearchClass )
	{
		return GetNode(SearchClass,false);
	}

	/**
	 * Find the node associated with the class specified
	 * 
	 * @param	SearchClass	the class to search for
	 *
	 * @return	a pointer to the node associated with the class specified, or NULL if this branch doesn't contain that class
	 */
	const FClassTree* FindNode( UClass* SearchClass ) const
	{
		if ( SearchClass == Class )
			return this;

		if (!SearchClass->IsChildOf(Class))
			return NULL;

		for (auto Child : Children)
		{
			if (auto Result = Child->FindNode(SearchClass))
				return Result;
		}

		return NULL;
	}

	/**
	 * Move a class node in the hierarchy tree after a class has changed its SuperClass
	 * 
	 * @param	SearchClass			the class that has changed parents
	 * @param	InNewParentClass	if non-null force reparenting to this instead of the SuperClass
	 *
	 * @return	true if SearchClass was successfully moved to the new location
	 */
	bool ChangeParentClass( UClass* SearchClass, UClass* InNewParentClass=NULL )
	{
		// in order to ensure that all classes are placed into the correct locations in the tree,
		// only the root node can accept new classes
		if ( Parent != NULL )
		{
			FClassTree* ReceivingNode;
			for ( ReceivingNode = this; ReceivingNode->Parent != NULL; ReceivingNode = ReceivingNode->Parent );

			return ReceivingNode->ChangeParentClass(SearchClass);
		}

		check(SearchClass);

		UClass* NewParentClass = InNewParentClass ? InNewParentClass : SearchClass->GetSuperClass();

		// find the node associated with SearchClass's new SuperClass
		FClassTree* NewParentNode = GetNode(NewParentClass);
		if (!NewParentNode)
		{
			// if that class hasn't been added yet, add it now.
			if (!AddClass(NewParentClass))
				return false;

			NewParentNode = GetNode(NewParentClass);
		}
		check(NewParentNode);

		// find the node for the class that changed SuperClass
		if (FClassTree* ClassNode = GetNode(SearchClass,true))
		{
			// if the node has an existing parent, remove it from the parent's Children array
			if (ClassNode->Parent)
			{
				ClassNode->Parent->Children.RemoveSingle(ClassNode);
			}

			// move the node to the new SuperClass
			return NewParentNode->AddChildNode(ClassNode) != INDEX_NONE;
		}

		return AddClass(SearchClass);
	}


	/**
	 * Get the number of classes represented by this node, including any child nodes
	 * 
	 * @return	the number of classes represented by this node, including any child nodes
	 */
	int32 Num() const
	{
		int32 Count = 1;
		for ( int32 i = 0; i < Children.Num(); i++ )
		{
			Count += Children[i]->Num();
		}

		return Count;
	}

	/**
	 * Verify that this node is at the correct location in the class tree
	 */
	void Validate() const
	{
		if ( Parent == NULL )
		{
			// verify that we have a parent - only the UObject class is allowed to have no parents
			check(Class==UObject::StaticClass());
		}

		// if our parent is Object, verify that none of object's children could have been our parent
		else if ( Parent->GetClass() == UObject::StaticClass() )
		{
			TArray<const FClassTree*> ChildClasses;
			Parent->GetChildClasses(ChildClasses);

			for ( int32 i = 0; i < ChildClasses.Num(); i++ )
			{
				// skip ourselves
				if ( ChildClasses[i] != this )
				{
					UClass* OtherClass = ChildClasses[i]->GetClass();
					check(OtherClass);

					// verify that the other class is not a parent of this one
					check(!Class->IsChildOf(OtherClass));
				}
			}
		}
		else
		{
			// otherwise, our parent node should be the node for our class's parent class
			check(Class->GetSuperClass() == Parent->GetClass());
		}
	}

	/**
	 * Constructor
	 *
	 * @param	BaseClass	the class that should be associated with this node.
	 */
	FClassTree( UClass* BaseClass ) : Class(BaseClass), Parent(NULL)
	{
		check(Class);
	}

	/**
	 * Destructor. Frees the memory for all child nodes, recursively.
	 */
	~FClassTree()
	{
		Class = NULL;

		// clear the memory allocated by this branch
		for ( int32 i = 0; i < Children.Num(); i++ )
		{
			FClassTree* p = Children[i];
			if ( p != NULL )
			{
				delete p;
			}
		}

		Children.Empty();
	}

	void DumpClassTree( int32 IndentCount, FOutputDevice& Ar )
	{
		Ar.Logf(TEXT("%s%s"), FCString::Spc(IndentCount), *Class->GetName());
		for ( int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++ )
		{
			FClassTree* ChildNode = Children[ChildIndex];
			ChildNode->DumpClassTree(IndentCount + 2, Ar);
		}
	}

	/**
	* Get the number of child nodes of this node
	*
	* @return	the number of child nodes of this node
	*/
	UE_FORCEINLINE_HINT int32 NumChildren() const
	{
		return Children.Num();
	}

	UE_FORCEINLINE_HINT const FClassTree* GetChild(int32 Index) const
	{
		return Children[Index];
	}
};
