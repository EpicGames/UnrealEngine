// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"
#include "Containers/StringView.h"

namespace UE
{
	inline const FName NAME_IsLooseMetadata(ANSITEXTVIEW("IsLoose"));
};
