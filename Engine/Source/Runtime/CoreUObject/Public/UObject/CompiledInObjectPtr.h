// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*
 * This header contains types used to store and decode references between compiled-in objects in
 * different modules.
 * e.g. a UClass in a project module which extends Actor needs a pointer to the AActor UClass
 * instance in its UStruct::SuperStruct field. Some compilers and linkers will not allow this 
 * field to be constinit-initialized to the address of a constinit variable in another module.
 *
 * The values encoded are produced by UHT such that they correspond to UObject initialization 
 * data handled by FCompiledInObjectRegistry.
 *
 * Incomplete features:
 *	- Stability of references across live coding/hot reload recompiles (if UHT runs)
 * 	- Stability of references for installed-engine scenarios (i.e. references for engine types
 *	  must be fixed when the engine build is created so that plugins/projects can link against
 *	  the right objects at runtime)
 *
 * Non-features:
 *	- There is no ability to distinguish the encoding of these compiled in references from
 *	  other encoded pointers used by TObjectPtr such as late-resolved object handles or remote
 *	  object handles.
 *  - Therefore there is no ability to defer resolution of compiled in references until an
 *	  arbitrary later point or resolve them on demand. They can only be resolved by the
 * 	  CoreUObject module as part of initializing a module which contains UObjects.
 *
 */

#include "CoreTypes.h"
#include "Concepts/ConvertibleTo.h"
#include "Concepts/NotCVRefTo.h"
#include "UObject/ObjectMacros.h"

#include <type_traits>

#if UE_WITH_CONSTINIT_UOBJECT

namespace UE::CodeGen::ConstInit
{

// Decoded reference to a compiled in object
struct FCompiledInObjectReference
{
	uint32 ModuleId = 0;
	uint32 ObjectId = 0;

	static inline constexpr int32 NumObjectIdBits = 32;
	static inline constexpr uint32 MaskObjectId = 0xFFFF'FFFF;
	static inline constexpr uint32 MaxModuleId = 0x7FFF'FFFF;

	// To distinguish compiled-in references from pointers the low bit is always 1
	// UObjects should have a natural alignment greater than 1, allowing this.
	static inline constexpr uint32 CompiledInRefMask = 0x1;
	static inline constexpr int32 CompiledInRefShift = 1;

	static_assert(sizeof(UPTRINT) >= 8, "FCompiledInObjectReference requires pointers to be 8 bytes or more");

	/* Check whether a pointer is an encoded reference or a raw pointer */
	[[nodiscard]] static constexpr bool IsEncodedRef(UPTRINT InEncoded)
	{
		return (InEncoded & CompiledInRefMask) == CompiledInRefMask;
	}

	/* Decode an encoded pointer into this structure type */
	[[nodiscard]] static constexpr FCompiledInObjectReference FromEncoded(UPTRINT InEncoded)
	{
		InEncoded >>= CompiledInRefShift;
		uint32 ModuleId = InEncoded >> NumObjectIdBits;
		uint32 ObjectId = InEncoded & MaskObjectId;
		return { .ModuleId = ModuleId, .ObjectId = ObjectId };
	}

	/* Return this structure as an encoded pointer */
	[[nodiscard]] constexpr UPTRINT GetEncoded() const
	{
		const UPTRINT CombinedId = ((UPTRINT)ModuleId << NumObjectIdBits) | ObjectId;
		return (CombinedId << CompiledInRefShift) | CompiledInRefMask;
	}
};
	
// Encoded pointer to another compiled-in object for runtime linking
struct FCompiledInObjectPtr
{
	[[nodiscard]] constexpr FCompiledInObjectPtr() = default;
	[[nodiscard]] explicit constexpr FCompiledInObjectPtr(FCompiledInObjectReference InRef)
		: Encoded(InRef.GetEncoded())
	{
	}
	constexpr FCompiledInObjectPtr(const FCompiledInObjectPtr&) = default;
	constexpr FCompiledInObjectPtr& operator=(const FCompiledInObjectPtr&) = default;

	[[nodiscard]] constexpr bool IsUnset() const
	{
		return !FCompiledInObjectReference::IsEncodedRef(Encoded);
	}

	[[nodiscard]] constexpr bool IsEncodedRef() const
	{
		return FCompiledInObjectReference::IsEncodedRef(Encoded);
	}

	[[nodiscard]] constexpr FCompiledInObjectReference GetDecoded() const
	{
		return FCompiledInObjectReference::FromEncoded(Encoded);
	}

private:
	uint64 Encoded = 0;
};

#if IS_MONOLITHIC 
template<typename T>
using TCompiledInObjectPtr = T*;
#else // IS_MONOLITHIC

// Wrapper for a compiled-in object pointer which is not a TObjectPtr to be used in other UHT-generated
// structures
// T is not required to derive from UObject, e.g. it may be FStructBaseChain. 
// In that case if the pointer is an encoded reference, it will resolve to a UObject which can be cast to the required type.
// e.g. lists of implemented interfaces, struct base chains
template<typename T>
struct TCompiledInObjectPtr
{
	union
	{
		FCompiledInObjectPtr CompiledInPtr;
		T* Pointer = nullptr; 
	};

	constexpr TCompiledInObjectPtr() = default;
	constexpr TCompiledInObjectPtr(const TCompiledInObjectPtr&) = default;
	constexpr TCompiledInObjectPtr& operator=(const TCompiledInObjectPtr&) = default;

	// Implicit compile-time construction from object pointer
	template<UE::CNotCVRefTo<TCompiledInObjectPtr> U> requires (UE::CConvertibleTo<U, T*> && !UE::CSameAs<T, U>)
	[[nodiscard]] constexpr TCompiledInObjectPtr(U&& InPtr)
		: Pointer(InPtr)
	{
	}

	// Implicit compile-time construction from UHT encoded reference
	[[nodiscard]] constexpr TCompiledInObjectPtr(FCompiledInObjectReference InRef)
		: CompiledInPtr(InRef)
	{
	}

	// Operators to allow this or T* to be used depending on IS_MONOLITHIC
	operator T*() const
	{
		return GetPointer();
	}

	T* operator->()
	{
		return GetPointer();
	}

	const T* operator->() const
	{
		return GetPointer();
	}

	T& operator*()
	{
		return *GetPointer();
	}

	const T& operator*() const
	{
		return *GetPointer();
	}

	[[nodiscard]] T* GetPointer() const
	{
		check(!IsEncodedRef());
		return reinterpret_cast<T*>(Pointer);
	}

	[[nodiscard]] constexpr bool IsEncodedRef() const
	{
		return CompiledInPtr.IsEncodedRef();
	}

	[[nodiscard]] constexpr FCompiledInObjectReference GetDecoded() const
	{
		return CompiledInPtr.GetDecoded();
	}
	
	[[nodiscard]] constexpr bool IsUnset() const
	{
		return Pointer == nullptr;
	}

	[[nodiscard]] constexpr explicit operator bool() const 
	{
		return !IsUnset();
	}

	[[nodiscard]] constexpr bool operator==(TYPE_OF_NULLPTR)
	{
		return IsUnset();
	}
	
	[[nodiscard]] constexpr bool operator!=(TYPE_OF_NULLPTR)
	{
		return !IsUnset();
	}
};

#endif // IS_MONOLITHIC
}

#if !IS_MONOLITHIC
class FStructBaseChain;

namespace UE::Private
{
// Utility to allow both raw and encoded pointers to be wrapped in AsStructBaseChain to handle offsetting to private base class
// The dummy template parameter will cause the body of the function to be deferred until it's called, where hopefully FStructBaseChain
// is complete.
template <typename Dummy = void>
[[nodiscard]] consteval UE::CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain> AsStructBaseChain(UE::CodeGen::ConstInit::FCompiledInObjectReference Ptr)
{
	return UE::CodeGen::ConstInit::TCompiledInObjectPtr<const std::conditional_t<std::is_void_v<Dummy>, FStructBaseChain, FStructBaseChain>>(Ptr);
}
}

#endif // !IS_MONOLITHIC
#endif // UE_WITH_CONSTINIT_UOBJECT