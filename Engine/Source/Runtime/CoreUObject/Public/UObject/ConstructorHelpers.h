// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ConstructorHelpers.h: Constructor helper templates
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"
#include "UObject/GCObject.h"

namespace ConstructorHelpersInternal
{
	template<typename T>
	inline T* FindOrLoadObject( FString& PathName, uint32 LoadFlags )
	{
		// If there is no dot, add a dot and repeat the object name.
		int32 PackageDelimPos = INDEX_NONE;
		PathName.FindChar( TCHAR('.'), PackageDelimPos );
		if( PackageDelimPos == INDEX_NONE )
		{
			int32 ObjectNameStart = INDEX_NONE;
			PathName.FindLastChar( TCHAR('/'), ObjectNameStart );
			if( ObjectNameStart != INDEX_NONE )
			{
				const FString ObjectName = PathName.Mid( ObjectNameStart+1 );
				PathName += TCHAR('.');
				PathName += ObjectName;
			}
		}

		UClass* Class = T::StaticClass();
		Class->GetDefaultObject(); // force the CDO to be created if it hasn't already
		T* ObjectPtr = LoadObject<T>(NULL, *PathName, nullptr, LoadFlags);
		if (ObjectPtr)
		{
			ObjectPtr->AddToRoot();
		}
		return ObjectPtr;
	}

	template<>
	COREUOBJECT_API UPackage* FindOrLoadObject<UPackage>(FString& PathName, uint32 LoadFlags);

	inline UClass* FindOrLoadClass(FString& PathName, UClass* BaseClass)
	{
		// If there is no dot, add ".<object_name>_C"
		int32 PackageDelimPos = INDEX_NONE;
		PathName.FindChar(TCHAR('.'), PackageDelimPos);
		if (PackageDelimPos == INDEX_NONE)
		{
			int32 ObjectNameStart = INDEX_NONE;
			PathName.FindLastChar(TCHAR('/'), ObjectNameStart);
			if (ObjectNameStart != INDEX_NONE)
			{
				const FString ObjectName = PathName.Mid(ObjectNameStart + 1);
				PathName += TCHAR('.');
				PathName += ObjectName;
				PathName += TCHAR('_');
				PathName += TCHAR('C');
			}
		}
		UClass* LoadedClass = StaticLoadClass(BaseClass, NULL, *PathName);
		if (LoadedClass)
		{
			LoadedClass->AddToRoot();
		}
		return LoadedClass;
	}
}

struct ConstructorHelpers
{
public:
	template<class T>
	struct FObjectFinder : public FGCObject
	{
		TObjectPtr<T> Object;
		FObjectFinder(const TCHAR* ObjectToFind, uint32 InLoadFlags = LOAD_None)
		{
			CheckIfIsInConstructor(ObjectToFind);
			FString PathName(ObjectToFind);
			StripObjectClass(PathName,true);

			Object = ConstructorHelpersInternal::FindOrLoadObject<T>(PathName, InLoadFlags);
			ValidateObject( Object, PathName, ObjectToFind );
		}
		bool Succeeded() const
		{
			return !!Object;
		}

		virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
		{
			Collector.AddReferencedObject(Object);
		}

		virtual FString GetReferencerName() const override
		{
			return TEXT("FObjectFinder");
		}
	};

	template<class T>
	struct FObjectFinderOptional : public FGCObject
	{
	private:
		TObjectPtr<T> Object;
		const TCHAR* ObjectToFind;
		uint32 LoadFlags;
	public:
		FObjectFinderOptional(const TCHAR* InObjectToFind, uint32 InLoadFlags = LOAD_None)
			: Object(nullptr)
			, ObjectToFind(InObjectToFind)
			, LoadFlags(InLoadFlags)
		{
		}
		T* Get()
		{
			if (!Object && ObjectToFind)
			{
				CheckIfIsInConstructor(ObjectToFind);
				FString PathName(ObjectToFind);
				StripObjectClass(PathName,true);

				Object = ConstructorHelpersInternal::FindOrLoadObject<T>(PathName, LoadFlags);

				bool WarnMissing = (LoadFlags & (LOAD_Quiet | LOAD_NoWarn)) == 0;

				if (Object || WarnMissing)
				{
					ValidateObject(Object, PathName, ObjectToFind);
				}

				ObjectToFind = nullptr; // don't try to look again
			}
			return Object;
		}
		bool Succeeded()
		{
			return !!Get();
		}

		virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
		{
			Collector.AddReferencedObject(Object);
		}

		virtual FString GetReferencerName() const override
		{
			return TEXT("FObjectFinderOptional");
		}
	};

	template<class T>
	struct FClassFinder : public FGCObject
	{
		TSubclassOf<T> Class;
		FClassFinder(const TCHAR* ClassToFind)
		{
			CheckIfIsInConstructor(ClassToFind);
			FString PathName(ClassToFind);
			StripObjectClass(PathName, true);
			Class = ConstructorHelpersInternal::FindOrLoadClass(PathName, T::StaticClass());
			ValidateObject(*Class, PathName, *PathName);
		}
		bool Succeeded()
		{
			return !!*Class;
		}
		
		virtual void AddReferencedObjects( FReferenceCollector& Collector ) override
		{
			Collector.AddReferencedObject(Class.GetGCPtr());
		}

		virtual FString GetReferencerName() const override
		{
			return TEXT("FClassFinder");
		}
	};

public:
	/** If there is an object class, strips it off. */
	static COREUOBJECT_API void StripObjectClass( FString& PathName, bool bAssertOnBadPath = false );

private:
	static void ValidateObject(UObject *Object, const FString& PathName, const TCHAR* ObjectToFind)
	{
		if (!Object)
		{
			FailedToFind(ObjectToFind);
		}
#if UE_BUILD_DEBUG
		else 
		{
			CheckFoundViaRedirect(Object, PathName, ObjectToFind);
		}
#endif
	}

	static COREUOBJECT_API void FailedToFind(const TCHAR* ObjectToFind);
	static COREUOBJECT_API void CheckFoundViaRedirect(UObject *Object, const FString& PathName, const TCHAR* ObjectToFind);
	static COREUOBJECT_API void CheckIfIsInConstructor(const TCHAR* ObjectToFind);
};


