// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformMath.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Cook
{

enum class ECookType
{
	Unknown,
	OnTheFly,
	ByTheBook
};

enum class ECookingDLC
{
	Unknown,
	Yes,
	No,
};

/**
 * The cookcommandlet can run in MPCook mode, where it creates worker processes to use more parallelism and
 * reduce wall time of the cook. This enum is reported or received by some API functions on the cooker, and describes
 * whether the current process the cooker and caller are in is the MPCook director, or an MPCook worker, or a cooker
 * running as a singleprocess without MPCook.
 */
enum class EProcessType
{
	SingleProcess,
	Director,
	Worker,

	/** Used only when passed as a filter: matches either Director or Worker. */
	AllMPCook,
};

/* The Result of the Cook-Save of a package */
enum class ECookResult : uint8
{
	/* CookResults have not yet been set */
	NotAttempted,
	/* The package was saved with success. */
	Succeeded,
	/* The package was processed but SavePackage failed. */
	Failed,
	/** The package is a NeverCook package that needs to be added to cookresults for dependency tracking. */
	NeverCookPlaceholder,
	/** No information for this platform (used in CookWorker replication) */
	Invalid,
	Count,
	NumBits = FPlatformMath::ConstExprCeilLogTwo(ECookResult::Count),
};

/** Validation options used when cooking (@see ECookByTheBookOptions for the validation flags used when initializing the cooker) */
enum class ECookValidationOptions
{
	/** No flags */
	None = 0,
	/** Run asset validation (EditorValidatorSubsystem) on assets loaded during cook */
	RunAssetValidation = 1<<0,
	/** Run map validation (MapCheck) on maps loaded during cook */
	RunMapValidation = 1<<1,
	/** Consider validation errors (from RunAssetValidation or RunMapValidation) as fatal (preventing the package from being cooked) */
	ValidationErrorsAreFatal = 1<<2,
};
ENUM_CLASS_FLAGS(ECookValidationOptions);

}

