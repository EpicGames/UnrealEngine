// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "CookedMetaData.generated.h"

/**
 * Cooked meta-data for a UObject.
 */
USTRUCT()
struct FObjectCookedMetaDataStore
{
public:
	GENERATED_BODY()

	COREUOBJECT_API bool HasMetaData() const;
	COREUOBJECT_API void CacheMetaData(const UObject* SourceObject);
	COREUOBJECT_API void ApplyMetaData(UObject* TargetObject) const;

private:
	UPROPERTY()
	TMap<FName, FString> ObjectMetaData;
};

/**
 * Encapsulates the subfield metadata map key in a UScriptStruct (for UHT).
 */
USTRUCT()
struct FFieldCookedMetaDataKey
{
public:
	GENERATED_BODY()

	UPROPERTY()
	TArray<FName> FieldPath;

	FFieldCookedMetaDataKey();

	inline bool operator==(const FFieldCookedMetaDataKey& Other) const
	{
		return Other.FieldPath == FieldPath;
	}

	inline bool operator!=(const FFieldCookedMetaDataKey& Other) const
	{
		return Other.FieldPath != FieldPath;
	}

	friend uint32 GetTypeHash(const FFieldCookedMetaDataKey& Key)
	{
		return GetTypeHash(Key.FieldPath);
	}
};

/**
 * Encapsulates the subfield metadata map value in a UScriptStruct (for UHT).
 */
USTRUCT()
struct FFieldCookedMetaDataValue
{
public:
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName, FString> MetaData;
};

/**
 * Cooked meta-data for a FField, including its nested FField data.
 */
USTRUCT()
struct FFieldCookedMetaDataStore
{
public:
	GENERATED_BODY()

	COREUOBJECT_API bool HasMetaData() const;
	COREUOBJECT_API void CacheMetaData(const FField* SourceField);
	COREUOBJECT_API void ApplyMetaData(FField* TargetField) const;

protected:
	void CacheFieldMetaDataInternal(const FField* SourceField, TMap<FName, FString>& TargetFieldMetaData, FFieldCookedMetaDataKey& SubFieldMapKey);
	void ApplyFieldMetaDataInternal(FField* TargetField, const TMap<FName, FString>& SourceFieldMetaData, FFieldCookedMetaDataKey& SubFieldMapKey) const;

private:
	UPROPERTY()
	TMap<FName, FString> FieldMetaData;

	UPROPERTY()
	TMap<FFieldCookedMetaDataKey, FFieldCookedMetaDataValue> SubFieldMetaData;
};

/**
 * Cooked meta-data for a UStruct, including its nested FProperty data.
 */
USTRUCT()
struct FStructCookedMetaDataStore
{
public:
	GENERATED_BODY()

	COREUOBJECT_API bool HasMetaData() const;
	COREUOBJECT_API void CacheMetaData(const UStruct* SourceStruct);
	COREUOBJECT_API void ApplyMetaData(UStruct* TargetStruct) const;

private:
	UPROPERTY()
	FObjectCookedMetaDataStore ObjectMetaData;

	UPROPERTY()
	TMap<FName, FFieldCookedMetaDataStore> PropertiesMetaData;
};

/**
 * Cooked meta-data base class.
 */
UCLASS(Abstract)
class UCookedMetaDataBase : public UObject
{
public:
	GENERATED_BODY()
};

/**
 * Cooked meta-data for a UEnum.
 */
UCLASS(Optional, Within=Enum, MinimalAPI)
class UEnumCookedMetaData : public UCookedMetaDataBase
{
public:
	GENERATED_BODY()

	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	
	COREUOBJECT_API virtual bool HasMetaData() const;
	COREUOBJECT_API virtual void CacheMetaData(const UEnum* SourceEnum);
	COREUOBJECT_API virtual void ApplyMetaData(UEnum* TargetEnum) const;

protected:
	UPROPERTY()
	FObjectCookedMetaDataStore EnumMetaData;
};

/**
 * Cooked meta-data for a UScriptStruct, including its nested FProperty data.
 */
UCLASS(Optional, Within=ScriptStruct, MinimalAPI)
class UStructCookedMetaData : public UCookedMetaDataBase
{
public:
	GENERATED_BODY()

	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	
	COREUOBJECT_API virtual bool HasMetaData() const;
	COREUOBJECT_API virtual void CacheMetaData(const UScriptStruct* SourceStruct);
	COREUOBJECT_API virtual void ApplyMetaData(UScriptStruct* TargetStruct) const;

protected:
	UPROPERTY()
	FStructCookedMetaDataStore StructMetaData;
};

/**
 * Cooked meta-data for a UClass, including its nested FProperty and UFunction data.
 */
UCLASS(Optional, Within=Class, MinimalAPI)
class UClassCookedMetaData : public UCookedMetaDataBase
{
public:
	GENERATED_BODY()

	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	
	COREUOBJECT_API virtual bool HasMetaData() const;
	COREUOBJECT_API virtual void CacheMetaData(const UClass* SourceClass);
	COREUOBJECT_API virtual void ApplyMetaData(UClass* TargetClass) const;

protected:
	UPROPERTY()
	FStructCookedMetaDataStore ClassMetaData;

	UPROPERTY()
	TMap<FName, FStructCookedMetaDataStore> FunctionsMetaData;
};

namespace CookedMetaDataUtil
{

namespace Internal
{
COREUOBJECT_API void PrepareCookedMetaDataForPurge(UObject* CookedMetaDataPtr);
}

template <typename CookedMetaDataType>
CookedMetaDataType* NewCookedMetaData(UObject* Outer, FName Name, TSubclassOf<CookedMetaDataType> Class = CookedMetaDataType::StaticClass())
{
	return NewObject<CookedMetaDataType>(Outer, Class, Name, RF_Standalone | RF_Public);
}

template <typename CookedMetaDataType>
CookedMetaDataType* FindCookedMetaData(UObject* Outer, const TCHAR* Name)
{
	return FindObject<CookedMetaDataType>(Outer, Name);
}

template <typename CookedMetaDataType, typename CookedMetaDataPtrType>
void PurgeCookedMetaData(CookedMetaDataPtrType& CookedMetaDataPtr)
{
	Internal::PrepareCookedMetaDataForPurge(CookedMetaDataPtr);
	CookedMetaDataPtr = nullptr;
}

}
