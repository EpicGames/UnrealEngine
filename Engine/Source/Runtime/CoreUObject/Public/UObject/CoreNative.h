// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CoreNative.h: Native function lookup table.
=============================================================================*/

#pragma once

// IWYU pragma: begin_exports

#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/Script.h"

class UObject;
struct FFrame;

/** The type of a native function callable by script */
typedef void (*FNativeFuncPtr)(UObject* Context, FFrame& TheStack, RESULT_DECL);

// This class is deliberately simple (i.e. POD) to keep generated code size down.
struct FNameNativePtrPair
{
	const char* NameUTF8;
	FNativeFuncPtr Pointer;
};

namespace UE::CodeGen
{
	struct FClassNativeFunction;
}

// IWYU pragma: end_exports


/** A struct that maps a string name to a native function */
struct FNativeFunctionRegistrar
{
	FNativeFunctionRegistrar(class UClass* Class, const ANSICHAR* InName, FNativeFuncPtr InPointer)
	{
		RegisterFunction(Class, InName, InPointer);
	}
	static COREUOBJECT_API void RegisterFunction(class UClass* Class, const ANSICHAR* InName, FNativeFuncPtr InPointer);
	// overload for types generated from blueprints, which can have unicode names:
	static COREUOBJECT_API void RegisterFunction(class UClass* Class, const WIDECHAR* InName, FNativeFuncPtr InPointer);

#if !UE_WITH_CONSTINIT_UOBJECT
	static COREUOBJECT_API void RegisterFunctions(class UClass* Class, TConstArrayView<UE::CodeGen::FClassNativeFunction> InFunctions);
	static COREUOBJECT_API void RegisterFunctions(class UClass* Class, const FNameNativePtrPair* InArray, int32 NumFunctions);
#endif // !UE_WITH_CONSTINIT_UOBJECT
};
