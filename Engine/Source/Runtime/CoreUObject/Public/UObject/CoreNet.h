// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Misc/NetworkGuid.h"
#include "Serialization/BitReader.h"
#include "Serialization/BitWriter.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "Trace/Config.h"
#include "UObject/Class.h"
#include "UObject/CoreNetTypes.h"
#include "UObject/Field.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "CoreNet.generated.h"

class FArchive;
class FName;
class FNetTraceCollector;
class FGuidReferences;
class FOutBunch;
class FOutputDevice;
class FProperty;
class INetDeltaBaseState;
class UClass;
class UField;
class UFunction;
class UPackageMap;
class UStruct;
struct FObjectPtr;
struct FSoftObjectPath;
struct FSoftObjectPtr;

namespace UE::Net
{
	struct FNetResult;

	class FReplicationFragment;
	struct FReplicationStateDescriptor;
	typedef FReplicationFragment* (*CreateAndRegisterReplicationFragmentFunc)(UObject* Owner, const FReplicationStateDescriptor* Descriptor, FFragmentRegistrationContext& Context);
	
	class FNetTokenExportContext;
	class FNetTokenResolveContext;

	namespace Private
	{
		class FNetPropertyConditionManager;
	};
}


DECLARE_DELEGATE_RetVal_OneParam( bool, FNetObjectIsDynamic, const UObject*);

//
// Information about a field.
//
class FFieldNetCache
{
public:
	FFieldVariant			Field;
	int32			FieldNetIndex;
	uint32			FieldChecksum;
	mutable bool	bIncompatible;

	FFieldNetCache()
	{}
	FFieldNetCache(FFieldVariant InField, int32 InFieldNetIndex, uint32 InFieldChecksum )
		: Field(InField), FieldNetIndex(InFieldNetIndex), FieldChecksum(InFieldChecksum), bIncompatible(false)
	{}
};

//
// Information about a class, cached for network coordination.
//
class FClassNetCache
{
	friend class FClassNetCacheMgr;
public:
	COREUOBJECT_API FClassNetCache();
	COREUOBJECT_API FClassNetCache( const UClass* Class );

	int32 GetMaxIndex() const
	{
		return FieldsBase + Fields.Num();
	}

	const FFieldNetCache* GetFromField( FFieldVariant Field ) const
	{
		FFieldNetCache* Result = NULL;

		for ( const FClassNetCache* C= this; C; C = C->Super )
		{
			if ( ( Result = C->FieldMap.FindRef( Field.GetRawPointer() ) ) != NULL )
			{
				break;
			}
		}
		return Result;
	}

	const FFieldNetCache* GetFromChecksum( const uint32 Checksum ) const
	{
		FFieldNetCache* Result = NULL;

		for ( const FClassNetCache* C = this; C; C = C->Super )
		{
			if ( ( Result = C->FieldChecksumMap.FindRef( Checksum ) ) != NULL )
			{
				break;
			}
		}
		return Result;
	}

	const FFieldNetCache* GetFromIndex( const int32 Index ) const
	{
		for ( const FClassNetCache* C = this; C; C = C->Super )
		{
			if ( Index >= C->FieldsBase && Index < C->FieldsBase + C->Fields.Num() )
			{
				return &C->Fields[Index-C->FieldsBase];
			}
		}
		return NULL;
	}

	uint32 GetClassChecksum() const { return ClassChecksum; }

	const FClassNetCache* GetSuper() const { return Super; }
	const TArray< FFieldNetCache >& GetFields() const { return Fields; }

	COREUOBJECT_API void CountBytes(FArchive& Ar) const;

private:
	int32								FieldsBase;
	const FClassNetCache*				Super;
	TWeakObjectPtr< const UClass >		Class;
	uint32								ClassChecksum;
	TArray< FFieldNetCache >			Fields;
	TMap< void*, FFieldNetCache* >	FieldMap;
	TMap< uint32, FFieldNetCache* >		FieldChecksumMap;
};


class FClassNetCacheMgr
{
public:
	FClassNetCacheMgr() : bDebugChecksum( false ), DebugChecksumIndent( 0 ) { }
	~FClassNetCacheMgr() { ClearClassNetCache(); }

	/** get the cached field to index mappings for the given class */
	COREUOBJECT_API const FClassNetCache*	GetClassNetCache( UClass* Class );
	COREUOBJECT_API void					ClearClassNetCache();

	COREUOBJECT_API void				SortProperties( TArray< FProperty* >& Properties ) const;
	COREUOBJECT_API uint32				SortedStructFieldsChecksum( const UStruct* Struct, uint32 Checksum ) const;
	COREUOBJECT_API uint32				GetPropertyChecksum( const FProperty* Property, uint32 Checksum, const bool bIncludeChildren ) const;
	COREUOBJECT_API uint32				GetFunctionChecksum( const UFunction* Function, uint32 Checksum ) const;
	COREUOBJECT_API uint32				GetFieldChecksum( const UField* Field, uint32 Checksum ) const;

	bool				bDebugChecksum;
	int					DebugChecksumIndent;

	COREUOBJECT_API void CountBytes(FArchive& Ar) const;

private:
	TMap< TWeakObjectPtr< const UClass >, FClassNetCache* > ClassFieldIndices;
};


//
// Maps objects and names to and from indices for network communication.
//
UCLASS(Transient, Abstract, MinimalAPI, Config=Engine)
class UPackageMap : public UObject
{
	GENERATED_BODY()
public:

	virtual bool		WriteObject( FArchive & Ar, UObject* InOuter, FNetworkGUID NetGUID, FString ObjName ) { return false; }

	// @todo document
	virtual bool		SerializeObject( FArchive& Ar, UClass* InClass, UObject*& Obj, FNetworkGUID *OutNetGUID = NULL ) { return false; }

	// @todo document
	COREUOBJECT_API virtual bool		SerializeName( FArchive& Ar, FName& InName );

	static COREUOBJECT_API bool			StaticSerializeName( FArchive& Ar, FName& InName );

	virtual UObject*	ResolvePathAndAssignNetGUID( const FNetworkGUID& NetGUID, const FString& PathName ) { return NULL; }

	virtual bool		SerializeNewActor(FArchive & Ar, class UActorChannel * Channel, class AActor *& Actor) { return false; }

	virtual void		ReceivedNak( const int32 NakPacketId ) { }
	virtual void		ReceivedAck( const int32 AckPacketId ) { }
	virtual void		NotifyBunchCommit( const int32 OutPacketId, const FOutBunch* OutBunch ) { }

	virtual void		GetNetGUIDStats(int32& AckCount, int32& UnAckCount, int32& PendingCount) { }

	virtual void		NotifyStreamingLevelUnload( UObject* UnloadedLevel ) { }

	virtual bool		PrintExportBatch() { return false; }

	void				SetDebugContextString( const FString& Str ) { DebugContextString = Str; }
	void				ClearDebugContextString() { DebugContextString.Empty(); }

	void							ResetTrackedGuids( bool bShouldTrack ) { TrackedUnmappedNetGuids.Empty(); TrackedMappedDynamicNetGuids.Empty(); bShouldTrackUnmappedGuids = bShouldTrack; }
	const TSet< FNetworkGUID > &	GetTrackedUnmappedGuids() const { return TrackedUnmappedNetGuids; }
	const TSet< FNetworkGUID > &	GetTrackedDynamicMappedGuids() const { return TrackedMappedDynamicNetGuids; }

	virtual void AddUnmappedNetGUIDReference(FNetworkGUID UnmappedGUID) {}
	virtual void RemoveUnmappedNetGUIDReference(FNetworkGUID UnmappedGUID) {}

	// For sync load debugging with LogNetSyncLoads
	virtual void			ResetTrackedSyncLoadedGuids() {}
	virtual void			ReportSyncLoadsForProperty(const FProperty* Property, const UObject* Object) {}

	virtual void			LogDebugInfo( FOutputDevice & Ar) { }
	virtual UObject*		GetObjectFromNetGUID( const FNetworkGUID& NetGUID, const bool bIgnoreMustBeMapped ) { return NULL; }
	virtual FNetworkGUID	GetNetGUIDFromObject( const UObject* InObject) const { return FNetworkGUID(); }
	virtual bool			IsGUIDBroken( const FNetworkGUID& NetGUID, const bool bMustBeRegistered ) const { return false; }
	virtual bool			IsUsingRemoteObjectReferences() const { return false; }

	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;

	// Provides access to the context required to resolve received NetTokens.
	virtual const UE::Net::FNetTokenResolveContext* GetNetTokenResolveContext() const { return nullptr; }

	COREUOBJECT_API UPackageMap();
	COREUOBJECT_API UPackageMap(const FObjectInitializer& ObjectInitializer);
	COREUOBJECT_API virtual ~UPackageMap();

protected:
	bool					bShouldTrackUnmappedGuids;
	TSet< FNetworkGUID >	TrackedUnmappedNetGuids;
	TSet< FNetworkGUID >	TrackedMappedDynamicNetGuids;

	FString					DebugContextString;
};


/** Represents a range of PacketIDs, inclusive */
struct FPacketIdRange
{
	FPacketIdRange(int32 _First, int32 _Last) : First(_First), Last(_Last) { }
	FPacketIdRange(int32 PacketId) : First(PacketId), Last(PacketId) { }
	FPacketIdRange() : First(INDEX_NONE), Last(INDEX_NONE) { }
	int32 First;
	int32 Last;

	bool InRange(int32 PacketId) const
	{
		return (First <= PacketId && PacketId <= Last);
	}
};


/** Information for tracking retirement and retransmission of a property. */
struct FPropertyRetirement
{
	FPropertyRetirement* Next;

	TSharedPtr<class INetDeltaBaseState> DynamicState;

	FPacketIdRange OutPacketIdRange;
	uint32 FastArrayChangelistHistory;

	FPropertyRetirement() :
		 Next(nullptr)
		, DynamicState(nullptr)
		, FastArrayChangelistHistory(0)
	{}

	void CountBytes(FArchive& Ar) const;
};


/** FLifetimeProperty
 *	This class is used to track a property that is marked to be replicated for the lifetime of the actor channel.
 *  This doesn't mean the property will necessarily always be replicated, it just means:
 *	"check this property for replication for the life of the actor, and I don't want to think about it anymore"
 *  A secondary condition can also be used to skip replication based on the condition results
 */
class FLifetimeProperty
{
public:

	/** The FProperty's RepIndex this struct is representing */
	uint16 RepIndex = 0;

	/** The replication condition assigned to this property. Default is 'None' which means no conditions are needed. */
	ELifetimeCondition Condition = COND_None;

	/** The RepNotify behavior of the property. Only useful if CPF_RepNotify is set on the property.  */
	ELifetimeRepNotifyCondition RepNotifyCondition = REPNOTIFY_OnChanged;

	/** Is this property using push model suggestions to help the RepSystem tell when it's really dirty. */
	bool bIsPushBased = false;

	/** Iris only: optional functor that will be used  to create and register a ReplicationFragment for the property */
	UE::Net::CreateAndRegisterReplicationFragmentFunc CreateAndRegisterReplicationFragmentFunction = nullptr;

	FLifetimeProperty() = default;

	FLifetimeProperty(int32 InRepIndex)
		: RepIndex((uint16)InRepIndex)
	{
		check(InRepIndex <= 65535);
	}

	FLifetimeProperty(int32 InRepIndex, ELifetimeCondition InCondition, ELifetimeRepNotifyCondition InRepNotifyCondition=REPNOTIFY_OnChanged, bool bInIsPushBased=false)
		: RepIndex((uint16)InRepIndex)
		, Condition(InCondition)
		, RepNotifyCondition(InRepNotifyCondition)
		, bIsPushBased(bInIsPushBased)
	{
		check(InRepIndex <= 65535);
	}

	inline bool operator==(const FLifetimeProperty& Other) const
	{
		if (RepIndex == Other.RepIndex)
		{
			// Can't have different conditions if the RepIndex matches, doesn't make sense
			check(Condition == Other.Condition);
			check(RepNotifyCondition == Other.RepNotifyCondition);
			check(bIsPushBased == Other.bIsPushBased);
			check(CreateAndRegisterReplicationFragmentFunction == Other.CreateAndRegisterReplicationFragmentFunction);
			return true;
		}

		return false;
	}
};

template <> struct TIsZeroConstructType<FLifetimeProperty> { enum { Value = true }; };

GENERATE_MEMBER_FUNCTION_CHECK(GetLifetimeReplicatedProps, void, const, TArray<FLifetimeProperty>&)

/**
 * Used to pass in temporary pointers to NetBitWriter which we absolutely need to avoid copying by acciden.
 * Used by NetTraceCollector and NetTokenExport scope
 */
template<typename T>
class TNetDoNotCopyPtr
{
public:
	TNetDoNotCopyPtr() : Ptr(nullptr) {}
	TNetDoNotCopyPtr(const TNetDoNotCopyPtr&) : Ptr(nullptr) {}
    TNetDoNotCopyPtr(TNetDoNotCopyPtr&&) { Ptr = nullptr; }
	TNetDoNotCopyPtr& operator=(const TNetDoNotCopyPtr& Other) { Ptr = nullptr; return *this; }
    TNetDoNotCopyPtr& operator=(TNetDoNotCopyPtr&&) { Ptr = nullptr; return *this; }
	void Set(T* InPtr) { Ptr = InPtr; }
	T* Get() const { return Ptr; }
private:
	T* Ptr;
};

#if UE_TRACE_ENABLED
using FNetTraceCollectorDoNotCopyWrapper = TNetDoNotCopyPtr<FNetTraceCollector>;
#endif

/**
 * FNetBitWriter
 *	A bit writer that serializes FNames and UObject* through
 *	a network packagemap.
 */
class FNetBitWriter : public FBitWriter
{
public:
	COREUOBJECT_API FNetBitWriter(UPackageMap* InPackageMap, int64 InMaxBits);
	COREUOBJECT_API FNetBitWriter(int64 InMaxBits);
	COREUOBJECT_API FNetBitWriter();

	class UPackageMap * PackageMap;

#if UE_TRACE_ENABLED
	FNetTraceCollectorDoNotCopyWrapper TraceCollector;
#endif

	COREUOBJECT_API virtual FArchive& operator<<(FName& Name) override;
	COREUOBJECT_API virtual FArchive& operator<<(UObject*& Object) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(struct FWeakObjectPtr& Value) override;

	COREUOBJECT_API virtual void CountMemory(FArchive& Ar) const override;

	/* Context allowing exports of NetTokens from NetSerialize(), the context will not be copied when copying NetBitWriter */
	TNetDoNotCopyPtr<UE::Net::FNetTokenExportContext> NetTokenExportContext;
};

/**
 * FNetBitReader
 *	A bit reader that serializes FNames and UObject* through
 *	a network packagemap.
 */
class FNetBitReader : public FBitReader
{
public:
	UPackageMap*													PackageMap		= nullptr;

	/** Stores additional error information. Avoid using to modify control flow, where bunches with errors may be copied/cached/queued. */
	TPimplPtr<UE::Net::FNetResult, EPimplPtrMode::DeepCopy>	ExtendedError;


public:
	COREUOBJECT_API FNetBitReader(UPackageMap* InPackageMap=nullptr, const uint8* Src=nullptr, int64 CountBits=0);
	COREUOBJECT_API FNetBitReader(const FNetBitReader&);
    COREUOBJECT_API FNetBitReader& operator=(const FNetBitReader&);
    COREUOBJECT_API FNetBitReader(FNetBitReader&&);
    COREUOBJECT_API FNetBitReader& operator=(FNetBitReader&&);
	COREUOBJECT_API virtual ~FNetBitReader();

	COREUOBJECT_API virtual FArchive& operator<<(FName& Name) override;
	COREUOBJECT_API virtual FArchive& operator<<(UObject*& Object) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPath& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FSoftObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(FObjectPtr& Value) override;
	COREUOBJECT_API virtual FArchive& operator<<(struct FWeakObjectPtr& Value) override;

	COREUOBJECT_API virtual void CountMemory(FArchive& Ar) const override;
};

/** Helper methods to allow us to instrument different type of BitStreams */
inline uint32 GetBitStreamPositionForNetTrace(const FBitWriter& Stream) { return (uint32)Stream.GetNumBits(); }
inline uint32 GetBitStreamPositionForNetTrace(const FBitReader& Stream) { return (uint32)Stream.GetPosBits(); }

#if UE_TRACE_ENABLED
inline FNetTraceCollector* GetTraceCollector(const FNetBitWriter& BitWriter) { return BitWriter.TraceCollector.Get(); }
inline void SetTraceCollector(FNetBitWriter& BitWriter, FNetTraceCollector* Collector) { BitWriter.TraceCollector.Set(Collector); }
#else
inline FNetTraceCollector* GetTraceCollector(const FNetBitWriter& BitWriter) { return nullptr; }
inline void SetTraceCollector(FNetBitWriter& BitWriter, FNetTraceCollector* Collector) {}
#endif

bool inline NetworkGuidSetsAreSame( const TSet< FNetworkGUID >& A, const TSet< FNetworkGUID >& B )
{
	if ( A.Num() != B.Num() )
	{
		return false;
	}

	for ( const FNetworkGUID& CompareGuid : A )
	{
		if ( !B.Contains( CompareGuid ) )
		{
			return false;
		}
	}

	return true;
}

/**
 * INetDeltaBaseState
 *	An abstract interface for the base state used in net delta serialization. See notes in NetSerialization.h
 */
class INetDeltaBaseState : public TSharedFromThis<INetDeltaBaseState>
{
public:
	INetDeltaBaseState()
		: LastAckedHistory(0)
		, ChangelistHistory(0)
	{

	}

	virtual ~INetDeltaBaseState() = default;

	virtual bool IsStateEqual(INetDeltaBaseState* Otherstate) = 0;

	/**
	 * Used when tracking memory to gather the total size of a given instance.
	 * This should include the dynamically allocated data, as well as the classes size.
	 */
	virtual void CountBytes(FArchive& Ar) const {}

	uint32 GetLastAckedHistory() const { return LastAckedHistory; }
	void SetLastAckedHistory(uint32 InAckedHistory) { LastAckedHistory = InAckedHistory; }

	uint32 GetChangelistHistory() const { return ChangelistHistory; }
	void SetChangelistHistory(uint32 InChangelistHistory) { ChangelistHistory = InChangelistHistory; }

private:
	uint32 LastAckedHistory;
	uint32 ChangelistHistory;
};

struct FNetDeltaSerializeInfo;

/**
 * An interface for handling serialization of Structs for networking.
 *
 * See notes in NetSerialization.h
 */
class INetSerializeCB
{
protected:

	using FGuidReferencesMap = TMap<int32, FGuidReferences>;

public:

	INetSerializeCB() { }

	virtual ~INetSerializeCB() = default;

	/**
	 * Serializes an entire struct to / from the given archive.
	 * It is up to callers to manage Guid References created during reads.
	 *
	 * @param Params		NetDeltaSerialization Params to use.
	 *						Object must be valid.
	 *						Data must be valid.
	 *						Connection must be valid.
	 *						Map must be valid.
	 *						Struct must point to the UScriptStruct of Data.
	 *						Either Reader or Writer (but not both) must be valid.
	 *						bOutHasMoreUnmapped will be used to return whether or not we have we have unmapped guids.
	 *						Only used when reading.
	 */
	virtual void NetSerializeStruct(FNetDeltaSerializeInfo& Params) = 0;

	/**
	 * Gathers any guid references for a FastArraySerializer.
	 * @see GuidReferences.h for more info.
	 */
	virtual void GatherGuidReferencesForFastArray(struct FFastArrayDeltaSerializeParams& Params) = 0;

	/**
	 * Moves a previously mapped guid to an unmapped state for a FastArraySerializer.
	 * @see GuidReferences.h for more info.
	 *
	 * @return True if the guid was found and unmapped.
	 */
	virtual bool MoveGuidToUnmappedForFastArray(struct FFastArrayDeltaSerializeParams& Params) = 0;

	/**
	 * Updates any unmapped guid references for a FastArraySerializer.
	 * @see GuidReferences.h for more info.
	 */
	virtual void UpdateUnmappedGuidsForFastArray(struct FFastArrayDeltaSerializeParams& Params) = 0;

	/**
	 * Similar to NetSerializeStruct, except serializes an entire FastArraySerializer at once
	 * instead of element by element.
	 */
	virtual bool NetDeltaSerializeForFastArray(struct FFastArrayDeltaSerializeParams& Params) = 0;
};


class IRepChangedPropertyTracker
{
public:
	UE_DEPRECATED(5.5, "Will be removed or renamed and used to pass paramters.")
	IRepChangedPropertyTracker() { }
	virtual ~IRepChangedPropertyTracker() = default;
};

class FCustomPropertyConditionState
{
public:
	FCustomPropertyConditionState() = delete;
	FCustomPropertyConditionState(int32 NumProperties)
	: CurrentState(true /* initial value of all bits */, NumProperties)
	, DynamicConditionChangeCounter(0)
	{
	}

	void SetActiveState(const uint16 RepIndex, const bool bIsActive)
	{
		CurrentState[RepIndex] = bIsActive;
	}

	bool GetActiveState(const uint16 RepIndex) const
	{
		return CurrentState[RepIndex];
	}

	void SetDynamicCondition(const uint16 RepIndex, const ELifetimeCondition Condition)
	{
		static_assert(static_cast<__underlying_type(ELifetimeCondition)>(ELifetimeCondition::COND_Max - 1) <= int16(32767), "Unable to use int16 for ELifetimeCondition values");

		++DynamicConditionChangeCounter;
		DynamicConditions.Emplace(RepIndex, static_cast<int16>(Condition));
	}

	ELifetimeCondition GetDynamicCondition(const uint16 RepIndex) const
	{
		if (const int16* Condition = DynamicConditions.Find(RepIndex))
		{
			return static_cast<const ELifetimeCondition>(*Condition);
		}

		return COND_Dynamic;
	}


	int32 GetNumProperties() const
	{
		return CurrentState.Num();
	}

	uint32 GetDynamicConditionChangeCounter() const
	{
		return DynamicConditionChangeCounter;
	}

	void CountBytes(FArchive& Ar) const
	{
		CurrentState.CountBytes(Ar);
	}

private:
	TBitArray<> CurrentState;
	// Storing int16 instead of int-sized ELifetimeCondition to save some memory.
	TMap<uint16, int16> DynamicConditions;
	uint32 DynamicConditionChangeCounter;
};

/**
 * FNetDeltaSerializeInfo
 *  This is the parameter structure for delta serialization. It is kind of a dumping ground for anything custom implementations may need.
 */
struct FNetDeltaSerializeInfo
{
	/** Used when writing */
	FBitWriter* Writer = nullptr;

	/** Used when reading */
	FBitReader* Reader = nullptr;

	/** SharedPtr to new base state created by NetDeltaSerialize. Used when writing.*/
	TSharedPtr<INetDeltaBaseState>* NewState = nullptr;

	/** Pointer to the previous base state. Used when writing. */
	INetDeltaBaseState* OldState = nullptr;

	/** PackageMap that can be used to serialize objects and track Guid References. Used primarily when reading. */
	class UPackageMap* Map = nullptr;

	/** Connection that we're currently serializing data for. */
	class UNetConnection* Connection = nullptr;

	/** Pointer to the struct that we're serializing.*/
	void* Data = nullptr;

	/** Type of struct that we're serializing. */
	class UStruct* Struct = nullptr;

	/** Pointer to a NetSerializeCB implementation that can be used when serializing. */
	INetSerializeCB* NetSerializeCB = nullptr;

	/** If true, we are updating unmapped objects */
	bool bUpdateUnmappedObjects = false;

	/** If true, then we successfully mapped some unmapped objects. */
	bool bOutSomeObjectsWereMapped = false;

	/** Whether or not PreNetReceive has been called on the owning object. */
	bool bCalledPreNetReceive = false;

	/** Whether or not there are still some outstanding unmapped objects referenced by the struct. */
	bool bOutHasMoreUnmapped = false;

	/** Whether or not we changed Guid / Object references. Used when reading. */
	bool bGuidListsChanged = false;

	/** Whether or not we're sending / writing data from the client. */
	bool bIsWritingOnClient = false;

	//~ TODO: This feels hacky, and a better alternative might be something like connection specific
	//~ capabilities.

	/** Whether we are currently initializing base from defaults in which case we should not modify the source **/
	bool bIsInitializingBaseFromDefault = false;

	/** Whether or not we support FFastArraySerializer::FastArrayDeltaSerialize_DeltaSerializeStructs */
	bool bSupportsFastArrayDeltaStructSerialization = false;
	
#if UE_SUPPORT_FOR_ACTOR_TICK_DISABLE
	/** Whether or not FFastArraySerializer will call user provided client-side callbacks (e.g. PreReplicatedRemove()). */
	bool bCallFastArrayClientCallbacks = true;
#endif

	/**
	 * Whether or the connection is completely reliable.
	 * We cache this off separate from UNetConnection so we can limit usage.
	 */
	bool bInternalAck = false;

	/** The object that owns the struct we're serializing, may be an archetype. */
	UObject* Object = nullptr;

	/** Used by SendCustomDeltaProperty to distinguish between the source object (archetype) and the replicating object. */
	UObject* CustomDeltaObject = nullptr;

	/**
	 * When non-null, this indicates that we're gathering Guid References.
	 * Any Guids the struct is referencing should be added.
	 * This may contain gathered Guids from other structs, so do not clear this set.
	 */
	TSet<FNetworkGUID>* GatherGuidReferences = nullptr;

	/**
	 * When we're gathering guid references, ny memory used to track Guids can be added to this.
	 * This may be tracking Guid memory from other structs, so do not reset this.
	 * Note, this is not guaranteed to be valid when GatherGuidReferences is.
	 */
	int32* TrackedGuidMemoryBytes = nullptr;

	/** When non-null, this indicates the given Guid has become unmapped and any references to it should be updated. */
	const FNetworkGUID* MoveGuidToUnmapped = nullptr;

	uint16 CustomDeltaIndex = uint16(INDEX_NONE);

	// Debugging variables
	FString DebugName;
};

struct FEncryptionData
{
	/** Encryption key */
	TArray<uint8> Key;
	/** Encryption fingerprint */
	TArray<uint8> Fingerprint;
	/** Encryption identifier */
	FString Identifier;
};

/**
 * Checksum macros for verifying archives stay in sync
 */
COREUOBJECT_API void SerializeChecksum(FArchive &Ar, uint32 x, bool ErrorOK);

#define NET_ENABLE_CHECKSUMS 0


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST) && NET_ENABLE_CHECKSUMS

#define NET_CHECKSUM_OR_END(Ser) \
{ \
	SerializeChecksum(Ser,0xE282FA84, true); \
}

#define NET_CHECKSUM(Ser) \
{ \
	SerializeChecksum(Ser,0xE282FA84, false); \
}

#define NET_CHECKSUM_CUSTOM(Ser, x) \
{ \
	SerializeChecksum(Ser,x, false); \
}

// There are cases where a checksum failure is expected, but we still need to eat the next word (just dont without erroring)
#define NET_CHECKSUM_IGNORE(Ser) \
{ \
	uint32 Magic = 0; \
	Ser << Magic; \
}

#else

// No ops in shipping builds
#define NET_CHECKSUM(Ser)
#define NET_CHECKSUM_IGNORE(Ser)
#define NET_CHECKSUM_CUSTOM(Ser, x)
#define NET_CHECKSUM_OR_END(ser)


#endif

/**
* Values used for initializing UNetConnection and LanBeacon
*/
enum { MAX_PACKET_SIZE = 1024 }; // MTU for the connection
enum { LAN_BEACON_MAX_PACKET_SIZE = 1024 }; // MTU for the connection

/**
 * Functions to assist in detecting errors during RPC calls
 */
COREUOBJECT_API void			RPC_ResetLastFailedReason();
COREUOBJECT_API void			RPC_ValidateFailed( const TCHAR* Reason );
COREUOBJECT_API const TCHAR *	RPC_GetLastFailedReason();
