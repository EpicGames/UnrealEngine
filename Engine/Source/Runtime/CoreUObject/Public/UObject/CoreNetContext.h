// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

#define UE_API COREUOBJECT_API

class UFunction;

namespace UE::Net::Private
{
	// Indicates whether a the current scope of a ProcessEvent is call is due to
	// receiving or sending a "Remote"-specified RPC, or not processing a "Remote" RPC at all.
	enum class ERemoteFunctionMode
	{
		None,
		Receiving,
		Sending
	};

	// Global state required by the replication system, that also needs to be
	// accessible to generated code.
	class FCoreNetContext
	{
	public:
		static COREUOBJECT_API FCoreNetContext* Get();

		// Returns the "Remote" function mode currently at the top of the stack.
		// Used internally to determine whether a "Remote"-specified function
		// should be run locally or sent over the network as an RPC instead.
		COREUOBJECT_API ERemoteFunctionMode GetCurrentRemoteFunctionMode();

	private:
		friend class FScopedRemoteRPCMode;

		// Since Remote-specified functions don't run locally and can't recurse in the same callstack,
		// the stack count shouldn't grow beyond 2.
		static constexpr uint32 MaxRemoteStackSize = 2;
		TArray<ERemoteFunctionMode, TFixedAllocator<MaxRemoteStackSize>> RemoteFunctionStack;
	};

	// Used internally to indicate whether a ProcessEvent call within the scope is due
	// to sending or receiving a "Remote"-specified UFUNCTION as an RPC.
	class FScopedRemoteRPCMode
	{
	public:
		UE_API explicit FScopedRemoteRPCMode(const UFunction* Function, ERemoteFunctionMode Mode);
		UE_API ~FScopedRemoteRPCMode();

	private:
		bool bAddToStack = false;
	};
}

#undef UE_API
