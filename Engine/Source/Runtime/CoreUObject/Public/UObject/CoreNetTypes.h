// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "CoreNetTypes.generated.h"

/** Whether or not caching of actor/subobject names to the stack should be enabled, for async demo recording crashdumps */
#ifndef UE_NET_REPACTOR_NAME_DEBUG
	#define UE_NET_REPACTOR_NAME_DEBUG 0
#endif


/** Secondary condition to check before considering the replication of a lifetime property. */
UENUM(BlueprintType)
enum ELifetimeCondition : int
{
	COND_None = 0							UMETA(DisplayName = "None"),							// This property has no condition, and will send anytime it changes
	COND_InitialOnly = 1					UMETA(DisplayName = "Initial Only"),					// This property will only attempt to send on the initial bunch
	COND_OwnerOnly = 2						UMETA(DisplayName = "Owner Only"),						// This property will only send to the actor's owner
	COND_SkipOwner = 3						UMETA(DisplayName = "Skip Owner"),						// This property send to every connection EXCEPT the owner
	COND_SimulatedOnly = 4					UMETA(DisplayName = "Simulated Only"),					// This property will only send to simulated actors
	COND_AutonomousOnly = 5					UMETA(DisplayName = "Autonomous Only"),					// This property will only send to autonomous actors
	COND_SimulatedOrPhysics = 6				UMETA(DisplayName = "Simulated Or Physics"),			// This property will send to simulated OR bRepPhysics actors
	COND_InitialOrOwner = 7					UMETA(DisplayName = "Initial Or Owner"),				// This property will send on the initial packet, or to the actors owner
	COND_Custom = 8							UMETA(DisplayName = "Custom"),							// This property has no particular condition, but wants the ability to toggle on/off via SetCustomIsActiveOverride
	COND_ReplayOrOwner = 9					UMETA(DisplayName = "Replay Or Owner"),					// This property will only send to the replay connection, or to the actors owner
	COND_ReplayOnly = 10					UMETA(DisplayName = "Replay Only"),						// This property will only send to the replay connection
	COND_SimulatedOnlyNoReplay = 11			UMETA(DisplayName = "Simulated Only No Replay"),		// This property will send to actors only, but not to replay connections
	COND_SimulatedOrPhysicsNoReplay = 12	UMETA(DisplayName = "Simulated Or Physics No Replay"),	// This property will send to simulated Or bRepPhysics actors, but not to replay connections
	COND_SkipReplay = 13					UMETA(DisplayName = "Skip Replay"),						// This property will not send to the replay connection
	COND_Dynamic = 14						UMETA(Hidden),											// This property wants to override the condition at runtime. Defaults to always replicate until you override it to a new condition.
	COND_Never = 15							UMETA(Hidden),											// This property will never be replicated
	COND_NetGroup = 16						UMETA(Hidden),											// This subobject will replicate to connections that are part of the same group the subobject is registered to. Not usable on properties.
	COND_Max = 17							UMETA(Hidden)
};


enum ELifetimeRepNotifyCondition
{
	REPNOTIFY_OnChanged = 0,		// Only call the property's RepNotify function if it changes from the local value
	REPNOTIFY_Always = 1,		// Always Call the property's RepNotify function when it is received from the server
};

enum class EChannelCloseReason : uint8
{
	Destroyed,
	Dormancy,
	LevelUnloaded,
	Relevancy,
	TearOff,
	Migrated,
	/* reserved */
	MAX	= 15		// this value is used for serialization, modifying it may require a network version change
};

COREUOBJECT_API const TCHAR* LexToString(const EChannelCloseReason Value);
