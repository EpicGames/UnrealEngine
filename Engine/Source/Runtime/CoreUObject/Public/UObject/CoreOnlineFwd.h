// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_UNSUPPORTED - Should not include this

static_assert(false, "UObject/CoreOnlineFwd.h has moved to Online/CoreOnlineFwd.h in the CoreOnline module. Please update #includes and module dependencies.");