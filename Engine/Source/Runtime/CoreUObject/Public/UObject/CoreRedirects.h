// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	CoreRedirects.h: Object/Class/Field redirects read from ini files or registered at startup
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

// TODO: Internal headers should not be included
#include "Runtime/CoreUObject/Internal/UObject/CoreRedirects/PM-k.h"

class FBlake3;
class IPakFile;
class UClass;
struct FSoftObjectPath;
struct FTopLevelAssetPath;
struct FCoreRedirectsContext;
struct FScopeCoreRedirectsReadLockedContext;
struct FScopeCoreRedirectsWriteLockedContext;
namespace UE::CoreRedirects::Private { struct FCoreRedirectObjectUtf8Name; }

DECLARE_LOG_CATEGORY_EXTERN(LogCoreRedirects, Log, All);

/** 
 * Flags describing the type and properties of this redirect
 */
enum class ECoreRedirectFlags : uint32
{
	None = 0,

	// Core type of the Thing being redirected, multiple can be set.  A Query will only find Redirects that have at least one of the same Type bits set.
	Type_Object =				0x00000001, // UObject
	Type_Class =				0x00000002, // UClass
	Type_Struct =				0x00000004, // UStruct
	Type_Enum =					0x00000008, // UEnum
	Type_Function =				0x00000010, // UFunction
	Type_Property =				0x00000020, // FProperty
	Type_Package =				0x00000040, // UPackage
	Type_Asset =				0x00000080, // Redirects derived from UObjectRedirectors. Implicitly included with other search types
	Type_AllMask =				0x0000FFFF, // Bit mask of all possible Types

	// Category flags.  A Query will only match Redirects that have the same value for every category bit.
	Category_InstanceOnly =		0x00010000, // Only redirect instances of this type, not the type itself
	Category_Removed =			0x00020000, // This type was explicitly removed, new name isn't valid
	Category_AllMask =			0x00FF0000, // Bit mask of all possible Categories

	// Option flags.  Does not behave as a bit-match between Queries and Redirects.  Each one specifies a custom rule for how FCoreRedirects handles the Redirect.
	Option_MatchPrefix =		0x01000000, // Does a prefix string match
	Option_MatchSuffix =		0x02000000, // Does a suffix string match
	Option_MatchSubstring =		Option_MatchPrefix | Option_MatchSuffix, // Does a slow substring match
	Option_MatchWildcardMask =	Option_MatchSubstring, // Bit mask of all possible wildcards

	Option_MissingLoad =		0x04000000, // An automatically-created redirect that was created in response to a missing Thing during load. Redirect will be removed if and when the Thing is loaded.
	Option_Iterate =			0x08000000, // Indicates that the result of applying this redirect should be iteratively redirected.
	Option_AllMask =			0xFF000000, // Bit mask of all possible Options
};
ENUM_CLASS_FLAGS(ECoreRedirectFlags);

enum class ECoreRedirectMatchFlags
{
	None = 0,
	/** The passed-in CoreRedirectObjectName has null fields in Package, Outer, or Name, and should still be allowed to match
	 against redirectors that were created with a full Package.[Outer:]Name. */
	AllowPartialMatch = (1 << 0),
	/** Used for Type_Asset redirects to ensure package redirects only match package queries and 
	 *  full path redirects only match full path queries
	 */
	DisallowPartialLHSMatch = (1<<1)
};
ENUM_CLASS_FLAGS(ECoreRedirectMatchFlags);

/**
 * An object path extracted into component names for matching. TODO merge with FSoftObjectPath?
 */
struct FCoreRedirectObjectName
{
	/** Raw name of object */
	FName ObjectName;

	/** String of outer chain, may be empty */
	FName OuterName;

	/** Package this was in before, may be extracted out of OldName */
	FName PackageName;

	/** Default to invalid names */
	FCoreRedirectObjectName() = default;

	/** Construct from FNames that are already expanded */
	FCoreRedirectObjectName(FName InObjectName, FName InOuterName, FName InPackageName)
		: ObjectName(InObjectName), OuterName(InOuterName), PackageName(InPackageName)
	{

	}

	COREUOBJECT_API FCoreRedirectObjectName(const FTopLevelAssetPath& TopLevelAssetPath);

	COREUOBJECT_API FCoreRedirectObjectName(const FSoftObjectPath& SoftObjectPath);

	COREUOBJECT_API FCoreRedirectObjectName(const FString& InString);

	/** Construct from object in memory */
	COREUOBJECT_API FCoreRedirectObjectName(const class UObject* Object);

	/** Creates FString version */
	COREUOBJECT_API FString ToString() const;

	/** Sets back to invalid state */
	COREUOBJECT_API void Reset();

	/** Checks for exact equality */
	bool operator==(const FCoreRedirectObjectName& Other) const
	{
		return ObjectName == Other.ObjectName && OuterName == Other.OuterName && PackageName == Other.PackageName;
	}

	bool operator!=(const FCoreRedirectObjectName& Other) const
	{
		return !(*this == Other);
	}

	/** Compares the two names lexically, returning -,0,+ */
	COREUOBJECT_API int Compare(const FCoreRedirectObjectName& Other) const;

	/** Flags for the Matches function. These flags overlap but are lower-level than ECoreRedirectMatchFlags. */
	enum class EMatchFlags
	{
		None = 0,
		/** Do not match if LHS (aka *this) has null fields that RHS (aka Other) does not. Default is to match. */
		DisallowPartialLHSMatch = (1 << 0),
		/** Match even if RHS (aka Other) has null fields that LHS (aka *this) does not. Default is to NOT match. */
		AllowPartialRHSMatch = (1 << 1),
		/**
		 * LHS fields (aka *this) are searchstrings; RHS (aka Other) fields are searched for that substring.
		 * Without this flag a Match returns true if and only if the complete string matches: LHS == RHS.
		 * With this flag a Match returns true if and only if RHS.Contains(LHS).
		 * This flag makes the match more expensive and should be avoided when possible.
		 * This flag forces partial LHS matches, and will ignore the DisallowPartialLHSMatch flag when matching.
		 */
		CheckSubString = (1 << 2),
		/**
		 * LHS fields (aka *this) are searchstrings; RHS (aka Other) fields are searched for that prefix.
		 * Without this flag a Match returns true if and only if the complete string matches: LHS == RHS.
		 * With this flag a Match returns true if and only if RHS.StartsWith(LHS).
		 * This flag makes the match more expensive and should be avoided when possible.
		 * This flag forces partial LHS matches, and will ignore the DisallowPartialLHSMatch flag when matching.
		 */
		CheckPrefix = (1 << 3),
		/**
		 * LHS fields (aka *this) are searchstrings; RHS (aka Other) fields are searched for that suffix.
		 * Without this flag a Match returns true if and only if the complete string matches: LHS == RHS.
		 * With this flag a Match returns true if and only if RHS.EndsWith(LHS).
		 * This flag makes the match more expensive and should be avoided when possible.
		 * This flag forces partial LHS matches, and will ignore the DisallowPartialLHSMatch flag when matching.
		 */
		CheckSuffix = (1 << 4),
	};
	/** Returns true if the passed in name matches requirements. */
	COREUOBJECT_API bool Matches(const FCoreRedirectObjectName& Other, EMatchFlags MatchFlags = EMatchFlags::None) const;

	/** Returns integer of degree of match. 0 if doesn't match at all, higher integer for better matches */
	COREUOBJECT_API int32 MatchScore(const FCoreRedirectObjectName& Other, ECoreRedirectFlags RedirectFlags, ECoreRedirectMatchFlags MatchFlags) const;

	/** Fills in any empty fields on this with the corresponding fields from Other. */
	COREUOBJECT_API void UnionFieldsInline(const FCoreRedirectObjectName& Other);

	/** Returns the name used as the key into the acceleration map */
	FName GetSearchKey(ECoreRedirectFlags Type) const;

	/** Returns true if this refers to an actual object */
	bool IsValid() const
	{
		return ObjectName != NAME_None || PackageName != NAME_None;
	}

	friend uint32 GetTypeHash(const FCoreRedirectObjectName& RedirectName)
	{
		return HashCombine(GetTypeHash(RedirectName.ObjectName), HashCombine(GetTypeHash(RedirectName.OuterName), GetTypeHash(RedirectName.PackageName)));
	}

	/** Returns true if all names have valid characters */
	COREUOBJECT_API bool HasValidCharacters(ECoreRedirectFlags Type) const;

	/** Update Hasher with all fields from this */
	COREUOBJECT_API void AppendHash(FBlake3& Hasher) const;

	/** Expand OldName/NewName as needed */
	static COREUOBJECT_API bool ExpandNames(const FStringView FullString, FName& OutName, FName& OutOuter, FName& OutPackage);

	/** Turn it back into an FString */
	static COREUOBJECT_API FString CombineNames(FName NewName, FName NewOuter, FName NewPackage);

	/** Given parent FCoreRedirectObjectName and FName of a child under it, return child's FCoreRedirectObjectName. */
	static COREUOBJECT_API FCoreRedirectObjectName
		AppendObjectName(const FCoreRedirectObjectName& Parent, FName ObjectName);
	/**
	 * Given a child FCoreRedirectObjectName, return its parent's FCoreRedirectObjectName.
	 * If the input has no parent (empty or is package with no outer), returns an empty FCoreRedirectObjectName.
	 */
	static COREUOBJECT_API FCoreRedirectObjectName GetParent(const FCoreRedirectObjectName& Child);
};
ENUM_CLASS_FLAGS(FCoreRedirectObjectName::EMatchFlags);

/** 
 * A single redirection from an old name to a new name, parsed out of an ini file
 */
struct FCoreRedirect
{
	/** Flags of this redirect */
	ECoreRedirectFlags RedirectFlags;

	/** Name of object to look for */
	FCoreRedirectObjectName OldName;

	/** Name to replace with */
	FCoreRedirectObjectName NewName;

	/** Change the class of this object when doing a redirect */
	FCoreRedirectObjectName OverrideClassName;

	/** Map of value changes, from old value to new value */
	TMap<FString, FString> ValueChanges;

	/** Construct from name strings, which may get parsed out */
	FCoreRedirect(ECoreRedirectFlags InRedirectFlags, FString InOldName, FString InNewName)
		: RedirectFlags(InRedirectFlags), OldName(InOldName), NewName(InNewName)
	{
		NormalizeNewName();
	}
	
	/** Construct parsed out object names */
	FCoreRedirect(ECoreRedirectFlags InRedirectFlags, const FCoreRedirectObjectName& InOldName, const FCoreRedirectObjectName& InNewName)
		: RedirectFlags(InRedirectFlags), OldName(InOldName), NewName(InNewName)
	{
		NormalizeNewName();
	}

	/** Normalizes NewName with data from OldName */
	COREUOBJECT_API void NormalizeNewName();

	/** Parses a char buffer into the ValueChanges map */
	COREUOBJECT_API const TCHAR* ParseValueChanges(const TCHAR* Buffer);

	/** Returns true if the passed in name and flags match requirements */
	COREUOBJECT_API bool Matches(ECoreRedirectFlags InFlags, const FCoreRedirectObjectName& InName,
		ECoreRedirectMatchFlags MatchFlags = ECoreRedirectMatchFlags::None) const;
	/** Returns true if the passed in name matches requirements */
	COREUOBJECT_API bool Matches(const FCoreRedirectObjectName& InName,
		ECoreRedirectMatchFlags MatchFlags = ECoreRedirectMatchFlags::None) const;

	/** Returns true if this has value redirects */
	COREUOBJECT_API bool HasValueChanges() const;

	/** Returns true if this is a substring match */
	COREUOBJECT_API bool IsSubstringMatch() const;

	/** Convert to new names based on mapping */
	COREUOBJECT_API FCoreRedirectObjectName RedirectName(const FCoreRedirectObjectName& OldObjectName) const;

	/** See if search criteria is identical */
	COREUOBJECT_API bool IdenticalMatchRules(const FCoreRedirect& Other) const;

	/** Returns the name used as the key into the acceleration map */
	FName GetSearchKey() const
	{
		return OldName.GetSearchKey(RedirectFlags);
	}

	/** Update Hasher with all fields from this */
	COREUOBJECT_API void AppendHash(FBlake3& Hasher) const;
	/** Returns -,0,+ based on a full lexical-fnames compare of all fields on the two CoreRedirects. */
	COREUOBJECT_API int Compare(const FCoreRedirect& Other) const;

	COREUOBJECT_API int32 GetTypeHash() const;

	friend int32 GetTypeHash(const FCoreRedirect& CoreRedirect)
	{
		return CoreRedirect.GetTypeHash();
	}

	bool operator==(const FCoreRedirect& Other) const
	{
		return Compare(Other) == 0;
	}

	bool operator<(const FCoreRedirect& Other) const
	{
		return Compare(Other) < 0;
	}

private:
	friend struct FCoreRedirects;
	friend class FRedirectionSummary;

	/* Returns the updated name after redirection. If bIsKnownToMatch is true, OldObjectName must have 
	been validated previously to be acceptable for redirection */
	FCoreRedirectObjectName RedirectName(const FCoreRedirectObjectName& OldObjectName, bool bIsKnownToMatch) const;

	/** Returns true if this is a Wildcard match (substring, prefix or suffix) */
	bool IsWildcardMatch() const
	{
		return EnumHasAnyFlags(RedirectFlags, ECoreRedirectFlags::Option_MatchWildcardMask);
	}

	/** Returns true if this is a prefix match */
	bool IsPrefixMatch() const
	{
		return EnumHasAllFlags(RedirectFlags, ECoreRedirectFlags::Option_MatchPrefix);
	}

	/** Returns true if this is a prefix match */
	bool IsSuffixMatch() const
	{
		return EnumHasAllFlags(RedirectFlags, ECoreRedirectFlags::Option_MatchSuffix);
	}
};

/**
 * A container for all of the registered core-level redirects 
 */
struct FCoreRedirects
{
	/**
	 * Run initialization steps that are needed before any data can be stored in FCoreRedirects.
	 * Reads can occur before this, but no redirects will exist and redirect queries will all return empty.
	 */
	static COREUOBJECT_API void Initialize();

	/** Returns a redirected version of the object name. If there are no valid redirects, it will return the original name */
	static COREUOBJECT_API FCoreRedirectObjectName GetRedirectedName(ECoreRedirectFlags Type,
		const FCoreRedirectObjectName& OldObjectName, ECoreRedirectMatchFlags MatchFlags = ECoreRedirectMatchFlags::None);

	/** Returns map of String->String value redirects for the object name, or nullptr if none found */
	static COREUOBJECT_API const TMap<FString, FString>* GetValueRedirects(ECoreRedirectFlags Type,
		const FCoreRedirectObjectName& OldObjectName, ECoreRedirectMatchFlags MatchFlags = ECoreRedirectMatchFlags::None);

	/** Performs both a name redirect and gets a value redirect struct if it exists. Returns true if either redirect found */
	static COREUOBJECT_API bool RedirectNameAndValues(ECoreRedirectFlags Type, const FCoreRedirectObjectName& OldObjectName,
		FCoreRedirectObjectName& NewObjectName, const FCoreRedirect** FoundValueRedirect,
		ECoreRedirectMatchFlags MatchFlags = ECoreRedirectMatchFlags::None);

	/** Returns true if this name has been registered as explicitly missing */
	static COREUOBJECT_API bool IsKnownMissing(ECoreRedirectFlags Type, const FCoreRedirectObjectName& ObjectName);

	/**
	  * Adds the given combination of (Type, ObjectName, Channel) as a missing name; IsKnownMissing queries will now find it
	  *
	  * @param Type Combination of the ECoreRedirectFlags::Type_* flags specifying the type of the object now known to be missing
	  * @param ObjectName The name of the object now known to be missing
	  * @param Channel may be Option_MissingLoad or Option_None; used to distinguish between detected-at-runtime and specified-by-ini
	  */
	static COREUOBJECT_API bool AddKnownMissing(ECoreRedirectFlags Type, const FCoreRedirectObjectName& ObjectName, ECoreRedirectFlags Channel = ECoreRedirectFlags::Option_MissingLoad);

	/**
	  * Removes the given combination of (Type, ObjectName, Channel) as a missing name
	  *
	  * @param Type Combination of the ECoreRedirectFlags::Type_* flags specifying the type of the object that has just been loaded.
	  * @param ObjectName The name of the object that has just been loaded.
	  * @param Channel may be Option_MissingLoad or Option_None; used to distinguish between detected-at-runtime and specified-by-ini
	  */
	static COREUOBJECT_API bool RemoveKnownMissing(ECoreRedirectFlags Type, const FCoreRedirectObjectName& ObjectName, ECoreRedirectFlags Channel = ECoreRedirectFlags::Option_MissingLoad);

	static COREUOBJECT_API void ClearKnownMissing(ECoreRedirectFlags Type, ECoreRedirectFlags Channel = ECoreRedirectFlags::Option_MissingLoad);

	/** Returns list of names it may have been before */
	static COREUOBJECT_API bool FindPreviousNames(ECoreRedirectFlags Type, const FCoreRedirectObjectName& NewObjectName, TArray<FCoreRedirectObjectName>& PreviousNames);

	/** Returns list of all core redirects that match requirements */
	static COREUOBJECT_API bool GetMatchingRedirects(ECoreRedirectFlags Type, const FCoreRedirectObjectName& OldObjectName,
		TArray<const FCoreRedirect*>& FoundRedirects, ECoreRedirectMatchFlags MatchFlags = ECoreRedirectMatchFlags::None);

	/** Parse all redirects out of a given ini file */
	static COREUOBJECT_API bool ReadRedirectsFromIni(const FString& IniName);

	/** Adds an array of redirects to global list */
	static COREUOBJECT_API bool AddRedirectList(TArrayView<const FCoreRedirect> Redirects, const FString& SourceString);

	/** Removes an array of redirects from global list */
	static COREUOBJECT_API bool RemoveRedirectList(TArrayView<const FCoreRedirect> Redirects, const FString& SourceString);

	/** Returns true if this has ever been initialized */
	static COREUOBJECT_API bool IsInitialized();

	/** Returns true if this is in debug mode that slows loading and adds additional warnings */
	static COREUOBJECT_API bool IsInDebugMode();

	/** Validate a named list of redirects */
	static COREUOBJECT_API void ValidateRedirectList(TArrayView<const FCoreRedirect> Redirects, const FString& SourceString);

	/** Validates all known redirects and warn if they seem to point to missing things or violate other constraints */
	static COREUOBJECT_API void ValidateAllRedirects();

	/** Validates asset redirects and warns if chains are detected. Chains should be resolved before adding asset redirects. */
	static COREUOBJECT_API bool ValidateAssetRedirects();

	/** Gets map from config key -> Flags. It may only be accessed once it becomes constant data after the system is initialized */
	static COREUOBJECT_API const TMap<FName, ECoreRedirectFlags>& GetConfigKeyMap();

	/** Goes from the containing package and name of the type to the type flag */
	static COREUOBJECT_API ECoreRedirectFlags GetFlagsForTypeName(FName PackageName, FName TypeName);

	/** Goes from UClass Type to the type flag */
	static COREUOBJECT_API ECoreRedirectFlags GetFlagsForTypeClass(UClass *TypeClass);

#if WITH_EDITOR
	/**
	 * For each package in PackageNames, compute the hash of all redirects that affect the package, either
	 * redirecting from or to the package. It doesn't include the GlobalRedirects and doesn't include the object redirects.
	 * Used in Incremental cooking to invalidate the cooked version of packages when CoreRedirects change.
	 */
	static COREUOBJECT_API void GetHashOfRedirectsAffectingPackages(const TConstArrayView<FName>& PackageNames, TArray<FBlake3Hash>& Hashes);

	/**
	 * Append the hash of all redirects that can affect multiple packages, or for which the affected packages are unknown.
	 * Used in Incremental cooking to invalidate the cooked version of packages when CoreRedirects change.
	 */
	static COREUOBJECT_API void AppendHashOfGlobalRedirects(FBlake3& Hasher);

	/** Calculate the hash of the object redirectors affecting a package, if any. */
	static COREUOBJECT_API void GetObjectRedirectHashAffectingPackage(const FName& PackageName, FBlake3& Hasher);

	/** Add the given Source->Path redirector to the summary used for GetObjectRedirectHashAffectingPackage. */
	static COREUOBJECT_API void RecordAddedObjectRedirector(const FSoftObjectPath& Source, const FSoftObjectPath& Dest);

	/** Remove the given Source->Path redirector to the summary used for GetObjectRedirectHashAffectingPackage. */
	static COREUOBJECT_API void RecordRemovedObjectRedirector(const FSoftObjectPath& Source, const FSoftObjectPath& Dest);

	/** Thread-safe delegate broadcasted when a global core redirect is added to FRedirectionSummary. It can be called from any thread. */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FGlobalCoreRedirectAddedEvent, const FCoreRedirect& CoreRedirect);
	
	/** 
	 * Return the delegate called when a global core redirect is added to FRedirectionSummary.
	 * Can return nullptr if the current thread context does not use a FRedirectionSummary.
	 */
	static COREUOBJECT_API FGlobalCoreRedirectAddedEvent* GetOnGlobalCoreRedirectAdded();

#endif

	/** Runs set of redirector tests, returns false on failure */
	static COREUOBJECT_API bool RunTests();

	/** Adds a collection of redirects as Type_Asset. These allow FCoreRedirects to support the functions
	 *  of UObjectRedirector. Any duplicate sources are logged and discarded (only the first redirect from a path is used)
	 *  Package redirects corresponding to the soft object paths are implicitly created.
	 */
	static COREUOBJECT_API void AddAssetRedirects(const TMap<FSoftObjectPath, FSoftObjectPath>& InRedirects);

	/** Clears all redirects added via AddAssetRedirects */
	static COREUOBJECT_API void RemoveAllAssetRedirects();
	 
private:

	/** Static only class, never constructed */
	COREUOBJECT_API FCoreRedirects();

	/** Internal implementation for AddRedirectList that requires a write lock to already have been acquired */
	static COREUOBJECT_API bool AddRedirectListUnderWriteLock(TArrayView<const FCoreRedirect> Redirects, 
		const FString& SourceString, FScopeCoreRedirectsWriteLockedContext& LockedContext);

	/** Internal implementation for AddSingleRedirect that requires a write lock to already have been acquired */
	static COREUOBJECT_API bool AddSingleRedirectUnderWriteLock(const FCoreRedirect& NewRedirect, 
		const FString& SourceString, FScopeCoreRedirectsWriteLockedContext& LockedContext);

	/** Internal implementation for RemoveSingleRedirect that requires a write lock to already have been acquired */
	static COREUOBJECT_API bool RemoveSingleRedirectUnderWriteLock(const FCoreRedirect& OldRedirect, 
		const FString& SourceString, FScopeCoreRedirectsWriteLockedContext& LockedContext);

	/** Add native redirects, called before ini is parsed for the first time */
	static COREUOBJECT_API void RegisterNativeRedirectsUnderWriteLock(FScopeCoreRedirectsWriteLockedContext& LockedContext);

	/** Internal implementation for GetMatchingRedirects that requires a read lock to already have been acquired */
	static COREUOBJECT_API bool GetMatchingRedirectsUnderReadLock(ECoreRedirectFlags Type, 
		const FCoreRedirectObjectName& OldObjectName, TArray<const FCoreRedirect*>& FoundRedirects, ECoreRedirectMatchFlags MatchFlags, 
		FScopeCoreRedirectsReadLockedContext& LockedContext);

	/** Internal implementation for RedirectNameAndValues that requires a read lock to already have been acquired */
	static COREUOBJECT_API bool RedirectNameAndValuesUnderReadLock(ECoreRedirectFlags Type, const FCoreRedirectObjectName& OldObjectName,
		FCoreRedirectObjectName& NewObjectName, const FCoreRedirect** FoundValueRedirect,
		ECoreRedirectMatchFlags MatchFlags, FScopeCoreRedirectsReadLockedContext& LockedContext);

	/** Internal implementation for ValidateAssetRedirects that requires a read lock to already have been acquired */
	static COREUOBJECT_API bool ValidateAssetRedirectsUnderReadLock(FScopeCoreRedirectsReadLockedContext& LockedContext);

	/** Container for managing Wildcard redirects (substrings, prefixes, suffixes) */
	struct FWildcardData
	{
		void Add(const FCoreRedirect& Redirect);

		void Rebuild();
		bool Matches(ECoreRedirectFlags InFlags, const FCoreRedirectObjectName& InName, ECoreRedirectMatchFlags InMatchFlags, TArray<const FCoreRedirect*>& OutFoundRedirects) const;

		TArray<FCoreRedirect> Substrings;
		TArray<FCoreRedirect> Prefixes;
		TArray<FCoreRedirect> Suffixes;
	private:
		/** This function may return false positives, but will not return false negatives */
		bool MatchSubstringApproximate(const UE::CoreRedirects::Private::FCoreRedirectObjectUtf8Name& RedirectName) const;
		void AddPredictionWords(const FCoreRedirect& Redirect);

		FPredictMatch8 PredictMatch;
	};

	/** There is one of these for each registered set of redirect flags */
	struct FRedirectNameMap
	{
		/** Map from name of thing being mapped to full list. List must be filtered further */
		TMap<FName, TArray<FCoreRedirect>> RedirectMap;
		/** Used to manage wildcard data and accelerate wildcard queries */
		TSharedPtr<FWildcardData> Wildcards;
	};

	/** Map from name of thing being mapped to full list. List must be filtered further */
	struct FRedirectTypeMap
	{
	public:
		FRedirectTypeMap() = default;
		FRedirectTypeMap(const FRedirectTypeMap& Other);
		FRedirectTypeMap& operator=(const FRedirectTypeMap& Other);
		FRedirectNameMap& FindOrAdd(ECoreRedirectFlags Key);
		FRedirectNameMap* Find(ECoreRedirectFlags Key);
		void Empty();

		TArray<TPair<ECoreRedirectFlags, FRedirectNameMap>>::RangedForIteratorType begin() { return FastIterable.begin(); }
		TArray<TPair<ECoreRedirectFlags, FRedirectNameMap>>::RangedForIteratorType end() { return FastIterable.end(); }

		FRedirectTypeMap(const FRedirectTypeMap&& Other) = delete;
		FRedirectTypeMap& operator=(const FRedirectTypeMap&& Other) = delete;
	private:
		TMap<ECoreRedirectFlags, FRedirectNameMap*> Map;
		TArray<TPair<ECoreRedirectFlags, FRedirectNameMap>> FastIterable;
	};

	friend struct FCoreRedirectsContext;
};


