// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "DynamicDelegateTestObject.h"

#include "Concepts/DynamicDelegate.h"
#include "Concepts/DynamicMulticastDelegate.h"
#include "Containers/AnsiString.h"
#include "Tests/TestHarnessAdapter.h"

// Set by the name of the function to say which one was called.
FAnsiString GCalledFunctionName;

int16 GInt16Arg = 5;
int16 GInt16Out = 97;

ETestUnscopedEnum::Type GUnscopedEnumArg = ETestUnscopedEnum::E2;
ETestUnscopedEnum::Type GUnscopedEnumOut = ETestUnscopedEnum::E1;

FString GStrArg = "Hello";
FString GStrOut = "Goodbye";

const UDynamicDelegateTestObject* GPtrArg = nullptr;
const UDynamicDelegateTestObject* GPtrOut = nullptr;

FName GNameArg = "Package";
FName GNameOut = "Object";

bool bGBoolArg = true;
bool bGBoolOut = false;

TWeakObjectPtr<UDynamicDelegateTestObject> GWeakObjPtrArg = nullptr;
TWeakObjectPtr<UDynamicDelegateTestObject> GWeakObjPtrOut = nullptr;

int32   GIntRet = 99;
FString GStrRet = "World";

ETestScopedEnum GScopedEnumArg = ETestScopedEnum::Value2;
ETestScopedEnum GScopedEnumOut = ETestScopedEnum::Value1;

float GFloatArg = 3.14f;
float GFloatOut = 9.81f;

double GDoubleArg = 2.718281828;
double GDoubleOut = 1.41421356;

uint8 GByteArg = 42;
uint8 GByteOut = 255;

int64 GInt64Arg = 123456789012345LL;
int64 GInt64Out = 987654321098765LL;

FText GTextArg = FText::FromString(TEXT("TestText"));
FText GTextOut = FText::FromString(TEXT("OutputText"));

FTestPayloadStruct GStructArg = { 1, 2, 3 };
FTestPayloadStruct GStructOut = { 10, 20, 30 };

FVector GVectorArg = FVector(1.0, 2.0, 3.0);
FVector GVectorOut = FVector(4.0, 5.0, 6.0);

TArray<int32> GIntArrayArg = { 10, 20, 30 };
TArray<int32> GIntArrayOut = { 40, 50, 60 };

TSet<int32> GIntSetArg = { 1, 2, 3 };
TSet<int32> GIntSetOut = { 4, 5, 6 };

TOptional<int32> GOptionalIntArg = 42;
TOptional<int32> GOptionalIntOut = 99;

UDynamicDelegateTestObject::UDynamicDelegateTestObject()
{
}

void UDynamicDelegateTestObject::Func0()
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "Func0";
}

void UDynamicDelegateTestObject::Func1(int16 Int)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "Func1";

	CHECK(Int == GInt16Arg);
}

void UDynamicDelegateTestObject::Func2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "Func2";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
}

void UDynamicDelegateTestObject::Func3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "Func3";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
}

void UDynamicDelegateTestObject::Func4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "Func4";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
	CHECK(Ptr          == GPtrArg);
}

void UDynamicDelegateTestObject::Func5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "Func5";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
	CHECK(Ptr          == GPtrArg);
	CHECK(Name         == GNameArg);
}

void UDynamicDelegateTestObject::Func6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "Func6";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
	CHECK(Ptr          == GPtrArg);
	CHECK(Name         == GNameArg);
	CHECK(bBool        == bGBoolArg);
}

void UDynamicDelegateTestObject::Func7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakObjPtr)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "Func7";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
	CHECK(Ptr          == GPtrArg);
	CHECK(Name         == GNameArg);
	CHECK(bBool        == bGBoolArg);
	CHECK(WeakObjPtr   == GWeakObjPtrArg);
}

void UDynamicDelegateTestObject::ConstFunc0() const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFunc0";
}

void UDynamicDelegateTestObject::ConstFunc1(int16 Int) const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFunc1";

	CHECK(Int == GInt16Arg);
}

void UDynamicDelegateTestObject::ConstFunc2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum) const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFunc2";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
}

void UDynamicDelegateTestObject::ConstFunc3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str) const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFunc3";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
}

void UDynamicDelegateTestObject::ConstFunc4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr) const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFunc4";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
	CHECK(Ptr          == GPtrArg);
}

void UDynamicDelegateTestObject::ConstFunc5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name) const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFunc5";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
	CHECK(Ptr          == GPtrArg);
	CHECK(Name         == GNameArg);
}

void UDynamicDelegateTestObject::ConstFunc6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool) const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFunc6";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
	CHECK(Ptr          == GPtrArg);
	CHECK(Name         == GNameArg);
	CHECK(bBool        == bGBoolArg);
}

void UDynamicDelegateTestObject::ConstFunc7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakObjPtr) const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFunc7";

	CHECK(Int          == GInt16Arg);
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	CHECK(Str          == GStrArg);
	CHECK(Ptr          == GPtrArg);
	CHECK(Name         == GNameArg);
	CHECK(bBool        == bGBoolArg);
	CHECK(WeakObjPtr   == GWeakObjPtrArg);
}

int UDynamicDelegateTestObject::FuncRetInt0()
{
	Func0();
	GCalledFunctionName = "FuncRetInt0";
	return GIntRet;
}

int UDynamicDelegateTestObject::FuncRetInt1(int16 Int)
{
	Func1(Int);
	GCalledFunctionName = "FuncRetInt1";
	return GIntRet;
}

int UDynamicDelegateTestObject::FuncRetInt2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum)
{
	Func2(Int, UnscopedEnum);
	GCalledFunctionName = "FuncRetInt2";
	return GIntRet;
}

int UDynamicDelegateTestObject::FuncRetInt3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str)
{
	Func3(Int, UnscopedEnum, Str);
	GCalledFunctionName = "FuncRetInt3";
	return GIntRet;
}

int UDynamicDelegateTestObject::FuncRetInt4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr)
{
	Func4(Int, UnscopedEnum, Str, Ptr);
	GCalledFunctionName = "FuncRetInt4";
	return GIntRet;
}

int UDynamicDelegateTestObject::FuncRetInt5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name)
{
	Func5(Int, UnscopedEnum, Str, Ptr, Name);
	GCalledFunctionName = "FuncRetInt5";
	return GIntRet;
}

int UDynamicDelegateTestObject::FuncRetInt6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool)
{
	Func6(Int, UnscopedEnum, Str, Ptr, Name, bBool);
	GCalledFunctionName = "FuncRetInt6";
	return GIntRet;
}

int UDynamicDelegateTestObject::FuncRetInt7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakObjPtr)
{
	Func7(Int, UnscopedEnum, Str, Ptr, Name, bBool, WeakObjPtr);
	GCalledFunctionName = "FuncRetInt7";
	return GIntRet;
}

FString UDynamicDelegateTestObject::FuncRetStr0()
{
	Func0();
	GCalledFunctionName = "FuncRetStr0";
	return GStrRet;
}

FString UDynamicDelegateTestObject::FuncRetStr1(int16 Int)
{
	Func1(Int);
	GCalledFunctionName = "FuncRetStr1";
	return GStrRet;
}

FString UDynamicDelegateTestObject::FuncRetStr2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum)
{
	Func2(Int, UnscopedEnum);
	GCalledFunctionName = "FuncRetStr2";
	return GStrRet;
}

FString UDynamicDelegateTestObject::FuncRetStr3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str)
{
	Func3(Int, UnscopedEnum, Str);
	GCalledFunctionName = "FuncRetStr3";
	return GStrRet;
}

FString UDynamicDelegateTestObject::FuncRetStr4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr)
{
	Func4(Int, UnscopedEnum, Str, Ptr);
	GCalledFunctionName = "FuncRetStr4";
	return GStrRet;
}

FString UDynamicDelegateTestObject::FuncRetStr5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name)
{
	Func5(Int, UnscopedEnum, Str, Ptr, Name);
	GCalledFunctionName = "FuncRetStr5";
	return GStrRet;
}

FString UDynamicDelegateTestObject::FuncRetStr6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool)
{
	Func6(Int, UnscopedEnum, Str, Ptr, Name, bBool);
	GCalledFunctionName = "FuncRetStr6";
	return GStrRet;
}

FString UDynamicDelegateTestObject::FuncRetStr7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakObjPtr)
{
	Func7(Int, UnscopedEnum, Str, Ptr, Name, bBool, WeakObjPtr);
	GCalledFunctionName = "FuncRetStr7";
	return GStrRet;
}

int UDynamicDelegateTestObject::ConstFuncRetInt0() const
{
	ConstFunc0();
	GCalledFunctionName = "ConstFuncRetInt0";
	return GIntRet;
}

int UDynamicDelegateTestObject::ConstFuncRetInt1(int16 Int) const
{
	ConstFunc1(Int);
	GCalledFunctionName = "ConstFuncRetInt1";
	return GIntRet;
}

int UDynamicDelegateTestObject::ConstFuncRetInt2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum) const
{
	ConstFunc2(Int, UnscopedEnum);
	GCalledFunctionName = "ConstFuncRetInt2";
	return GIntRet;
}

int UDynamicDelegateTestObject::ConstFuncRetInt3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str) const
{
	ConstFunc3(Int, UnscopedEnum, Str);
	GCalledFunctionName = "ConstFuncRetInt3";
	return GIntRet;
}

int UDynamicDelegateTestObject::ConstFuncRetInt4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr) const
{
	ConstFunc4(Int, UnscopedEnum, Str, Ptr);
	GCalledFunctionName = "ConstFuncRetInt4";
	return GIntRet;
}

int UDynamicDelegateTestObject::ConstFuncRetInt5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name) const
{
	ConstFunc5(Int, UnscopedEnum, Str, Ptr, Name);
	GCalledFunctionName = "ConstFuncRetInt5";
	return GIntRet;
}

int UDynamicDelegateTestObject::ConstFuncRetInt6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool) const
{
	ConstFunc6(Int, UnscopedEnum, Str, Ptr, Name, bBool);
	GCalledFunctionName = "ConstFuncRetInt6";
	return GIntRet;
}

int UDynamicDelegateTestObject::ConstFuncRetInt7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakObjPtr) const
{
	ConstFunc7(Int, UnscopedEnum, Str, Ptr, Name, bBool, WeakObjPtr);
	GCalledFunctionName = "ConstFuncRetInt7";
	return GIntRet;
}

FString UDynamicDelegateTestObject::ConstFuncRetStr0() const
{
	ConstFunc0();
	GCalledFunctionName = "ConstFuncRetStr0";
	return GStrRet;
}

FString UDynamicDelegateTestObject::ConstFuncRetStr1(int16 Int) const
{
	ConstFunc1(Int);
	GCalledFunctionName = "ConstFuncRetStr1";
	return GStrRet;
}

FString UDynamicDelegateTestObject::ConstFuncRetStr2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum) const
{
	ConstFunc2(Int, UnscopedEnum);
	GCalledFunctionName = "ConstFuncRetStr2";
	return GStrRet;
}

FString UDynamicDelegateTestObject::ConstFuncRetStr3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str) const
{
	ConstFunc3(Int, UnscopedEnum, Str);
	GCalledFunctionName = "ConstFuncRetStr3";
	return GStrRet;
}

FString UDynamicDelegateTestObject::ConstFuncRetStr4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr) const
{
	ConstFunc4(Int, UnscopedEnum, Str, Ptr);
	GCalledFunctionName = "ConstFuncRetStr4";
	return GStrRet;
}

FString UDynamicDelegateTestObject::ConstFuncRetStr5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name) const
{
	ConstFunc5(Int, UnscopedEnum, Str, Ptr, Name);
	GCalledFunctionName = "ConstFuncRetStr5";
	return GStrRet;
}

FString UDynamicDelegateTestObject::ConstFuncRetStr6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool) const
{
	ConstFunc6(Int, UnscopedEnum, Str, Ptr, Name, bBool);
	GCalledFunctionName = "ConstFuncRetStr6";
	return GStrRet;
}

FString UDynamicDelegateTestObject::ConstFuncRetStr7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakObjPtr) const
{
	ConstFunc7(Int, UnscopedEnum, Str, Ptr, Name, bBool, WeakObjPtr);
	GCalledFunctionName = "ConstFuncRetStr7";
	return GStrRet;
}

void UDynamicDelegateTestObject::FuncOut1(int16& Int)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOut1";
	CHECK(Int == GInt16Arg);
	Int = GInt16Out;
}

void UDynamicDelegateTestObject::FuncOut2(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum)
{
	FuncOut1(Int);
	GCalledFunctionName = "FuncOut2";
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	UnscopedEnum = GUnscopedEnumOut;
}

void UDynamicDelegateTestObject::FuncOut3(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str)
{
	FuncOut2(Int, UnscopedEnum);
	GCalledFunctionName = "FuncOut3";
	CHECK(Str == GStrArg);
	Str = GStrOut;
}

void UDynamicDelegateTestObject::FuncOut4(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name)
{
	FuncOut3(Int, UnscopedEnum, Str);
	GCalledFunctionName = "FuncOut4";
	CHECK(Name == GNameArg);
	Name = GNameOut;
}

void UDynamicDelegateTestObject::FuncOut5(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name, bool& bBool)
{
	FuncOut4(Int, UnscopedEnum, Str, Name);
	GCalledFunctionName = "FuncOut5";
	CHECK(bBool == bGBoolArg);
	bBool = bGBoolOut;
}

void UDynamicDelegateTestObject::FuncOut6(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name, bool& bBool, TWeakObjectPtr<UDynamicDelegateTestObject>& WeakObjPtr)
{
	FuncOut5(Int, UnscopedEnum, Str, Name, bBool);
	GCalledFunctionName = "FuncOut6";
	CHECK(WeakObjPtr == GWeakObjPtrArg);
	WeakObjPtr = GWeakObjPtrOut;
}

void UDynamicDelegateTestObject::ConstFuncOut1(int16& Int) const
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "ConstFuncOut1";
	CHECK(Int == GInt16Arg);
	Int = GInt16Out;
}

void UDynamicDelegateTestObject::ConstFuncOut2(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum) const
{
	ConstFuncOut1(Int);
	GCalledFunctionName = "ConstFuncOut2";
	CHECK(UnscopedEnum == GUnscopedEnumArg);
	UnscopedEnum = GUnscopedEnumOut;
}

void UDynamicDelegateTestObject::ConstFuncOut3(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str) const
{
	ConstFuncOut2(Int, UnscopedEnum);
	GCalledFunctionName = "ConstFuncOut3";
	CHECK(Str == GStrArg);
	Str = GStrOut;
}

void UDynamicDelegateTestObject::ConstFuncOut4(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name) const
{
	ConstFuncOut3(Int, UnscopedEnum, Str);
	GCalledFunctionName = "ConstFuncOut4";
	CHECK(Name == GNameArg);
	Name = GNameOut;
}

void UDynamicDelegateTestObject::ConstFuncOut5(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name, bool& bBool) const
{
	ConstFuncOut4(Int, UnscopedEnum, Str, Name);
	GCalledFunctionName = "ConstFuncOut5";
	CHECK(bBool == bGBoolArg);
	bBool = bGBoolOut;
}

void UDynamicDelegateTestObject::ConstFuncOut6(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name, bool& bBool, TWeakObjectPtr<UDynamicDelegateTestObject>& WeakObjPtr) const
{
	ConstFuncOut5(Int, UnscopedEnum, Str, Name, bBool);
	GCalledFunctionName = "ConstFuncOut6";
	CHECK(WeakObjPtr == GWeakObjPtrArg);
	WeakObjPtr = GWeakObjPtrOut;
}

void UDynamicDelegateTestObject::FuncFloat(float Float)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncFloat";
	CHECK(Float == GFloatArg);
}

void UDynamicDelegateTestObject::FuncDouble(double Double)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncDouble";
	CHECK(Double == GDoubleArg);
}

void UDynamicDelegateTestObject::FuncByte(uint8 Byte)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncByte";
	CHECK(Byte == GByteArg);
}

void UDynamicDelegateTestObject::FuncInt64(int64 Int64)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncInt64";
	CHECK(Int64 == GInt64Arg);
}

void UDynamicDelegateTestObject::FuncText(FText Text)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncText";
	CHECK(Text.EqualTo(GTextArg));
}

void UDynamicDelegateTestObject::FuncStruct(FTestPayloadStruct Struct)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncStruct";
	CHECK(Struct == GStructArg);
}

void UDynamicDelegateTestObject::FuncVector(FVector Vector)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncVector";
	CHECK(Vector == GVectorArg);
}

void UDynamicDelegateTestObject::FuncIntArray(TArray<int32> IntArray)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncIntArray";
	CHECK(IntArray == GIntArrayArg);
}

void UDynamicDelegateTestObject::FuncScopedEnum(ETestScopedEnum Enum)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncScopedEnum";
	CHECK(Enum == GScopedEnumArg);
}

void UDynamicDelegateTestObject::FuncIntSet(TSet<int32> IntSet)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncIntSet";
	CHECK(IntSet.Num() == GIntSetArg.Num());
	for (int32 Val : GIntSetArg)
	{
		CHECK(IntSet.Contains(Val));
	}
}

void UDynamicDelegateTestObject::FuncOptionalInt(TOptional<int32> OptionalInt)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOptionalInt";
	CHECK(OptionalInt.IsSet() == GOptionalIntArg.IsSet());
	if (OptionalInt.IsSet())
	{
		CHECK(OptionalInt.GetValue() == GOptionalIntArg.GetValue());
	}
}

void UDynamicDelegateTestObject::FuncOutFloat(float& Float)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutFloat";
	CHECK(Float == GFloatArg);
	Float = GFloatOut;
}

void UDynamicDelegateTestObject::FuncOutDouble(double& Double)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutDouble";
	CHECK(Double == GDoubleArg);
	Double = GDoubleOut;
}

void UDynamicDelegateTestObject::FuncOutByte(uint8& Byte)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutByte";
	CHECK(Byte == GByteArg);
	Byte = GByteOut;
}

void UDynamicDelegateTestObject::FuncOutInt64(int64& Int64)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutInt64";
	CHECK(Int64 == GInt64Arg);
	Int64 = GInt64Out;
}

void UDynamicDelegateTestObject::FuncOutText(FText& Text)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutText";
	CHECK(Text.EqualTo(GTextArg));
	Text = GTextOut;
}

void UDynamicDelegateTestObject::FuncOutStruct(FTestPayloadStruct& Struct)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutStruct";
	CHECK(Struct == GStructArg);
	Struct = GStructOut;
}

void UDynamicDelegateTestObject::FuncOutVector(FVector& Vector)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutVector";
	CHECK(Vector == GVectorArg);
	Vector = GVectorOut;
}

void UDynamicDelegateTestObject::FuncOutScopedEnum(ETestScopedEnum& Enum)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutScopedEnum";
	CHECK(Enum == GScopedEnumArg);
	Enum = GScopedEnumOut;
}

void UDynamicDelegateTestObject::FuncOutIntArray(TArray<int32>& IntArray)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutIntArray";
	CHECK(IntArray == GIntArrayArg);
	IntArray = GIntArrayOut;
}

void UDynamicDelegateTestObject::FuncOutIntSet(TSet<int32>& IntSet)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutIntSet";
	CHECK(IntSet.Num() == GIntSetArg.Num());
	for (int32 Val : GIntSetArg)
	{
		CHECK(IntSet.Contains(Val));
	}
	IntSet = GIntSetOut;
}

void UDynamicDelegateTestObject::FuncOutOptionalInt(TOptional<int32>& OptionalInt)
{
	CHECK(GCalledFunctionName.IsEmpty());
	GCalledFunctionName = "FuncOutOptionalInt";
	CHECK(OptionalInt.IsSet() == GOptionalIntArg.IsSet());
	if (OptionalInt.IsSet())
	{
		CHECK(OptionalInt.GetValue() == GOptionalIntArg.GetValue());
	}
	OptionalInt = GOptionalIntOut;
}

// Helper function to wrap the syntactic comparison differences of different property types.
template <typename LhsType, typename RhsType>
static bool TestValuesEqual(const LhsType& Lhs, const RhsType& Rhs)
{
	if constexpr (!std::is_same_v<LhsType, RhsType>)
	{
		return Lhs == Rhs;
	}
	else if constexpr (std::is_same_v<LhsType, FText>)
	{
		return Lhs.EqualTo(Rhs);
	}
	else if constexpr (TIsTSet_V<LhsType>)
	{
		if (Lhs.Num() != Rhs.Num())
		{
			return false;
		}
		for (const typename LhsType::ElementType& Val : Lhs)
		{
			if (!Rhs.Contains(Val))
			{
				return false;
			}
		}
		return true;
	}
	else
	{
		return Lhs == Rhs;
	}
}

// Forwards an unscoped enum
template <typename T>
UE_REWRITE decltype(auto) MoveToOutArg(T&& Ref)
{
	using DecayedT = std::decay_t<T>;

	if constexpr (UE::CUnscopedEnum<DecayedT>)
	{
		return TEnumAsByte<DecayedT>(Ref);
	}
	else
	{
		return Ref;
	}
}

#define DEFINE_ARG_VARIABLES_1(Arg1Val)                                                       auto Arg1 = MoveToOutArg(Arg1Val);
#define DEFINE_ARG_VARIABLES_2(Arg1Val, Arg2Val)                                              auto Arg1 = MoveToOutArg(Arg1Val); auto Arg2 = MoveToOutArg(Arg2Val);
#define DEFINE_ARG_VARIABLES_3(Arg1Val, Arg2Val, Arg3Val)                                     auto Arg1 = MoveToOutArg(Arg1Val); auto Arg2 = MoveToOutArg(Arg2Val); auto Arg3 = MoveToOutArg(Arg3Val);
#define DEFINE_ARG_VARIABLES_4(Arg1Val, Arg2Val, Arg3Val, Arg4Val)                            auto Arg1 = MoveToOutArg(Arg1Val); auto Arg2 = MoveToOutArg(Arg2Val); auto Arg3 = MoveToOutArg(Arg3Val); auto Arg4 = MoveToOutArg(Arg4Val);
#define DEFINE_ARG_VARIABLES_5(Arg1Val, Arg2Val, Arg3Val, Arg4Val, Arg5Val)                   auto Arg1 = MoveToOutArg(Arg1Val); auto Arg2 = MoveToOutArg(Arg2Val); auto Arg3 = MoveToOutArg(Arg3Val); auto Arg4 = MoveToOutArg(Arg4Val); auto Arg5 = MoveToOutArg(Arg5Val);
#define DEFINE_ARG_VARIABLES_6(Arg1Val, Arg2Val, Arg3Val, Arg4Val, Arg5Val, Arg6Val)          auto Arg1 = MoveToOutArg(Arg1Val); auto Arg2 = MoveToOutArg(Arg2Val); auto Arg3 = MoveToOutArg(Arg3Val); auto Arg4 = MoveToOutArg(Arg4Val); auto Arg5 = MoveToOutArg(Arg5Val); auto Arg6 = MoveToOutArg(Arg6Val);
#define DEFINE_ARG_VARIABLES_7(Arg1Val, Arg2Val, Arg3Val, Arg4Val, Arg5Val, Arg6Val, Arg7Val) auto Arg1 = MoveToOutArg(Arg1Val); auto Arg2 = MoveToOutArg(Arg2Val); auto Arg3 = MoveToOutArg(Arg3Val); auto Arg4 = MoveToOutArg(Arg4Val); auto Arg5 = MoveToOutArg(Arg5Val); auto Arg6 = MoveToOutArg(Arg6Val); auto Arg7 = MoveToOutArg(Arg7Val);

#define EXPAND_ARG_VARIABLES_1( ...) (Arg1)
#define EXPAND_ARG_VARIABLES_2( ...) (Arg1, Arg2)
#define EXPAND_ARG_VARIABLES_3( ...) (Arg1, Arg2, Arg3)
#define EXPAND_ARG_VARIABLES_4( ...) (Arg1, Arg2, Arg3, Arg4)
#define EXPAND_ARG_VARIABLES_5( ...) (Arg1, Arg2, Arg3, Arg4, Arg5)
#define EXPAND_ARG_VARIABLES_6( ...) (Arg1, Arg2, Arg3, Arg4, Arg5, Arg6)
#define EXPAND_ARG_VARIABLES_7( ...) (Arg1, Arg2, Arg3, Arg4, Arg5, Arg6, Arg7)

#define CHECK_ARG_VARIABLES_1(Arg1Val)                                                       CHECK(TestValuesEqual(Arg1, Arg1Val));
#define CHECK_ARG_VARIABLES_2(Arg1Val, Arg2Val)                                              CHECK(TestValuesEqual(Arg1, Arg1Val)); CHECK(TestValuesEqual(Arg2, Arg2Val));
#define CHECK_ARG_VARIABLES_3(Arg1Val, Arg2Val, Arg3Val)                                     CHECK(TestValuesEqual(Arg1, Arg1Val)); CHECK(TestValuesEqual(Arg2, Arg2Val)); CHECK(TestValuesEqual(Arg3, Arg3Val));
#define CHECK_ARG_VARIABLES_4(Arg1Val, Arg2Val, Arg3Val, Arg4Val)                            CHECK(TestValuesEqual(Arg1, Arg1Val)); CHECK(TestValuesEqual(Arg2, Arg2Val)); CHECK(TestValuesEqual(Arg3, Arg3Val)); CHECK(TestValuesEqual(Arg4, Arg4Val));
#define CHECK_ARG_VARIABLES_5(Arg1Val, Arg2Val, Arg3Val, Arg4Val, Arg5Val)                   CHECK(TestValuesEqual(Arg1, Arg1Val)); CHECK(TestValuesEqual(Arg2, Arg2Val)); CHECK(TestValuesEqual(Arg3, Arg3Val)); CHECK(TestValuesEqual(Arg4, Arg4Val)); CHECK(TestValuesEqual(Arg5, Arg5Val));
#define CHECK_ARG_VARIABLES_6(Arg1Val, Arg2Val, Arg3Val, Arg4Val, Arg5Val, Arg6Val)          CHECK(TestValuesEqual(Arg1, Arg1Val)); CHECK(TestValuesEqual(Arg2, Arg2Val)); CHECK(TestValuesEqual(Arg3, Arg3Val)); CHECK(TestValuesEqual(Arg4, Arg4Val)); CHECK(TestValuesEqual(Arg5, Arg5Val)); CHECK(TestValuesEqual(Arg6, Arg6Val));
#define CHECK_ARG_VARIABLES_7(Arg1Val, Arg2Val, Arg3Val, Arg4Val, Arg5Val, Arg6Val, Arg7Val) CHECK(TestValuesEqual(Arg1, Arg1Val)); CHECK(TestValuesEqual(Arg2, Arg2Val)); CHECK(TestValuesEqual(Arg3, Arg3Val)); CHECK(TestValuesEqual(Arg4, Arg4Val)); CHECK(TestValuesEqual(Arg5, Arg5Val)); CHECK(TestValuesEqual(Arg6, Arg6Val)); CHECK(TestValuesEqual(Arg7, Arg7Val));

#define DEFINE_ARG_VARIABLES(...) UE_APPEND_VA_ARG_COUNT(DEFINE_ARG_VARIABLES_, __VA_ARGS__)(__VA_ARGS__)
#define EXPAND_ARG_VARIABLES(...) UE_APPEND_VA_ARG_COUNT(EXPAND_ARG_VARIABLES_, __VA_ARGS__)(ObjectPtr, FuncPtr, __VA_ARGS__)
#define CHECK_ARG_VARIABLES(...)  UE_APPEND_VA_ARG_COUNT(CHECK_ARG_VARIABLES_,  __VA_ARGS__)(__VA_ARGS__)

// CallArgs is a parentheses-wrapped set of arguments to invoke the delegate, and the ellipsis is the payload arguments
#define BIND_CALL_CHECK_DELEGATE(DelegateType, bMulticast, FuncPtr, CallArgs, ...) \
	{ \
		GCalledFunctionName.Reset(); \
		DelegateType Delegate; \
		Delegate.UE_IF(bMulticast, AddDynamic, BindDynamic)(this, FuncPtr, ##__VA_ARGS__); \
		Delegate.UE_IF(bMulticast, Broadcast, Execute) CallArgs; \
		CHECK(GCalledFunctionName == UE::Core::Private::TrimmedMemberFunctionName<#FuncPtr>.Str); \
	}

// CallArgs is a parentheses-wrapped set of arguments to invoke the delegate, and the ellipsis is the payload arguments
#define BIND_CALL_CHECK_DELEGATE_RETVAL(ExpectedRetVal, DelegateType, bMulticast, FuncPtr, CallArgs, ...) \
	{ \
		GCalledFunctionName.Reset(); \
		DelegateType Delegate; \
		Delegate.UE_IF(bMulticast, AddDynamic, BindDynamic)(this, FuncPtr, ##__VA_ARGS__); \
		auto RetVal = Delegate.UE_IF(bMulticast, Broadcast, Execute) CallArgs; \
		CHECK(RetVal == ExpectedRetVal); \
		CHECK(GCalledFunctionName == UE::Core::Private::TrimmedMemberFunctionName<#FuncPtr>.Str); \
	}

// CallArgs is a parentheses-wrapped set of arguments to invoke the delegate, OutVals is the set of values to test being returned from the function.  There are no payload arguments, as out payload parameters are captured by value and thus not supported.
#define BIND_CALL_CHECK_DELEGATE_OUT(DelegateType, bMulticast, FuncPtr, CallArgs, OutVals) \
	{ \
		GCalledFunctionName.Reset(); \
		DelegateType Delegate; \
		DEFINE_ARG_VARIABLES CallArgs \
		Delegate.UE_IF(bMulticast, AddDynamic, BindDynamic)(this, FuncPtr); \
		Delegate.UE_IF(bMulticast, Broadcast, Execute) EXPAND_ARG_VARIABLES CallArgs; \
		CHECK_ARG_VARIABLES OutVals \
		CHECK(GCalledFunctionName == UE::Core::Private::TrimmedMemberFunctionName<#FuncPtr>.Str); \
	}

void UDynamicDelegateTestObject::RunTests()
{
	UDynamicDelegateTestObject* TestOut = NewObject<UDynamicDelegateTestObject>();

	// Update ptr args, because they can't be given a reasonable default as a global
	TGuardValue<const UDynamicDelegateTestObject*>          PtrArgGuard(GPtrArg, this);
	TGuardValue<TWeakObjectPtr<UDynamicDelegateTestObject>> WeakPtrArgGuard(GWeakObjPtrArg, this);
	TGuardValue<const UDynamicDelegateTestObject*>          OutPtrArgGuard(GPtrOut, TestOut);
	TGuardValue<TWeakObjectPtr<UDynamicDelegateTestObject>> WeakPtrOutGuard(GWeakObjPtrOut, TestOut);

	// 0 param function
	{
		// 0 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::Func0, ());
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::Func0, ());
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::Func0, ());
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc0, ());
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::ConstFunc0, ());
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::ConstFunc0, ());

		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::FuncRetInt0, ());
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::ConstFuncRetInt0, ());
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::FuncRetStr0, ());
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::ConstFuncRetStr0, ());
	}

	// 1 param function
	{
		// 0 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::Func1,      (GInt16Arg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::Func1,      (GInt16Arg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::Func1,      (GInt16Arg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc1, (GInt16Arg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::ConstFunc1, (GInt16Arg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::ConstFunc1, (GInt16Arg));

		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::FuncRetInt1,      (GInt16Arg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::ConstFuncRetInt1, (GInt16Arg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::FuncRetStr1,      (GInt16Arg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::ConstFuncRetStr1, (GInt16Arg));

#if defined(UE_USE_DYNAMIC_DELEGATE_PAYLOADS) && UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		// 1 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::Func1,      (), GInt16Arg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::Func1,      (), GInt16Arg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::Func1,      (), GInt16Arg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc1, (), GInt16Arg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::ConstFunc1, (), GInt16Arg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::ConstFunc1, (), GInt16Arg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::FuncRetInt1,      (), GInt16Arg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::ConstFuncRetInt1, (), GInt16Arg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::FuncRetStr1,      (), GInt16Arg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::ConstFuncRetStr1, (), GInt16Arg);
#endif
#endif
	}

	// 2 param function
	{
		// 0 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::Func2,      (GInt16Arg, GUnscopedEnumArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::Func2,      (GInt16Arg, GUnscopedEnumArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::Func2,      (GInt16Arg, GUnscopedEnumArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc2, (GInt16Arg, GUnscopedEnumArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::ConstFunc2, (GInt16Arg, GUnscopedEnumArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::ConstFunc2, (GInt16Arg, GUnscopedEnumArg));

		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::FuncRetInt2,      (GInt16Arg, GUnscopedEnumArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::ConstFuncRetInt2, (GInt16Arg, GUnscopedEnumArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::FuncRetStr2,      (GInt16Arg, GUnscopedEnumArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::ConstFuncRetStr2, (GInt16Arg, GUnscopedEnumArg));

#if defined(UE_USE_DYNAMIC_DELEGATE_PAYLOADS) && UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		// 1 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::Func2,      (GInt16Arg), GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::Func2,      (GInt16Arg), GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::Func2,      (GInt16Arg), GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc2, (GInt16Arg), GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::ConstFunc2, (GInt16Arg), GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::ConstFunc2, (GInt16Arg), GUnscopedEnumArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::FuncRetInt2,      (GInt16Arg), GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::ConstFuncRetInt2, (GInt16Arg), GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::FuncRetStr2,      (GInt16Arg), GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::ConstFuncRetStr2, (GInt16Arg), GUnscopedEnumArg);
#endif

		// 2 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::Func2,      (), GInt16Arg, GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::Func2,      (), GInt16Arg, GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::Func2,      (), GInt16Arg, GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc2, (), GInt16Arg, GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::ConstFunc2, (), GInt16Arg, GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::ConstFunc2, (), GInt16Arg, GUnscopedEnumArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::FuncRetInt2,      (), GInt16Arg, GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::ConstFuncRetInt2, (), GInt16Arg, GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::FuncRetStr2,      (), GInt16Arg, GUnscopedEnumArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::ConstFuncRetStr2, (), GInt16Arg, GUnscopedEnumArg);
#endif
#endif
	}

	// 3 param function
	{
		// 0 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::Func3,      (GInt16Arg, GUnscopedEnumArg, GStrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::Func3,      (GInt16Arg, GUnscopedEnumArg, GStrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::Func3,      (GInt16Arg, GUnscopedEnumArg, GStrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg, GUnscopedEnumArg, GStrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg, GUnscopedEnumArg, GStrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg, GUnscopedEnumArg, GStrArg));

		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::FuncRetInt3,      (GInt16Arg, GUnscopedEnumArg, GStrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::ConstFuncRetInt3, (GInt16Arg, GUnscopedEnumArg, GStrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::FuncRetStr3,      (GInt16Arg, GUnscopedEnumArg, GStrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::ConstFuncRetStr3, (GInt16Arg, GUnscopedEnumArg, GStrArg));

#if defined(UE_USE_DYNAMIC_DELEGATE_PAYLOADS) && UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		// 1 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::Func3,      (GInt16Arg, GUnscopedEnumArg), GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::Func3,      (GInt16Arg, GUnscopedEnumArg), GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::Func3,      (GInt16Arg, GUnscopedEnumArg), GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg, GUnscopedEnumArg), GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg, GUnscopedEnumArg), GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg, GUnscopedEnumArg), GStrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::FuncRetInt3,      (GInt16Arg, GUnscopedEnumArg), GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::ConstFuncRetInt3, (GInt16Arg, GUnscopedEnumArg), GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::FuncRetStr3,      (GInt16Arg, GUnscopedEnumArg), GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::ConstFuncRetStr3, (GInt16Arg, GUnscopedEnumArg), GStrArg);
#endif

		// 2 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::Func3,      (GInt16Arg), GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::Func3,      (GInt16Arg), GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::Func3,      (GInt16Arg), GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg), GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg), GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::ConstFunc3, (GInt16Arg), GUnscopedEnumArg, GStrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::FuncRetInt3,      (GInt16Arg), GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::ConstFuncRetInt3, (GInt16Arg), GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::FuncRetStr3,      (GInt16Arg), GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::ConstFuncRetStr3, (GInt16Arg), GUnscopedEnumArg, GStrArg);
#endif

		// 3 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::Func3,      (), GInt16Arg, GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::Func3,      (), GInt16Arg, GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::Func3,      (), GInt16Arg, GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc3, (), GInt16Arg, GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::ConstFunc3, (), GInt16Arg, GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::ConstFunc3, (), GInt16Arg, GUnscopedEnumArg, GStrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::FuncRetInt3,      (), GInt16Arg, GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::ConstFuncRetInt3, (), GInt16Arg, GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::FuncRetStr3,      (), GInt16Arg, GUnscopedEnumArg, GStrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::ConstFuncRetStr3, (), GInt16Arg, GUnscopedEnumArg, GStrArg);
#endif
#endif
	}

	// 4 param function
	{
		// 0 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4::FDelegate, 0, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate4,                     0, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4,            1, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate4,                     0, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4,            1, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));

		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt4, 0, &UDynamicDelegateTestObject::FuncRetInt4,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt4, 0, &UDynamicDelegateTestObject::ConstFuncRetInt4, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr4, 0, &UDynamicDelegateTestObject::FuncRetStr4,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr4, 0, &UDynamicDelegateTestObject::ConstFuncRetStr4, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg));

#if defined(UE_USE_DYNAMIC_DELEGATE_PAYLOADS) && UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		// 1 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::FuncRetInt4,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::ConstFuncRetInt4, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::FuncRetStr4,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::ConstFuncRetStr4, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg);
#endif

		// 2 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::Func4,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::FuncRetInt4,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::ConstFuncRetInt4, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::FuncRetStr4,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::ConstFuncRetStr4, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg);
#endif

		// 3 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::Func4,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::Func4,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::Func4,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::ConstFunc4, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::FuncRetInt4,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::ConstFuncRetInt4, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::FuncRetStr4,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::ConstFuncRetStr4, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg);
#endif

		// 4 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::Func4,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::Func4,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::Func4,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc4, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::ConstFunc4, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::ConstFunc4, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::FuncRetInt4,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::ConstFuncRetInt4, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::FuncRetStr4,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::ConstFuncRetStr4, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg);
#endif
#endif
	}

	// 5 param function
	{
		// 0 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5::FDelegate, 0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate5,                     0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5,            1, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate5,                     0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5,            1, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));

		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt5, 0, &UDynamicDelegateTestObject::FuncRetInt5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt5, 0, &UDynamicDelegateTestObject::ConstFuncRetInt5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr5, 0, &UDynamicDelegateTestObject::FuncRetStr5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr5, 0, &UDynamicDelegateTestObject::ConstFuncRetStr5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg));

#if defined(UE_USE_DYNAMIC_DELEGATE_PAYLOADS) && UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		// 1 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4::FDelegate, 0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate4,                     0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4,            1, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate4,                     0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4,            1, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt4, 0, &UDynamicDelegateTestObject::FuncRetInt5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt4, 0, &UDynamicDelegateTestObject::ConstFuncRetInt5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr4, 0, &UDynamicDelegateTestObject::FuncRetStr5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr4, 0, &UDynamicDelegateTestObject::ConstFuncRetStr5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg);
#endif

		// 2 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::FuncRetInt5,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::ConstFuncRetInt5, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::FuncRetStr5,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::ConstFuncRetStr5, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg);
#endif

		// 3 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::Func5,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::FuncRetInt5,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::ConstFuncRetInt5, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::FuncRetStr5,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::ConstFuncRetStr5, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg);
#endif

		// 4 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::Func5,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::Func5,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::ConstFunc5, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::FuncRetInt5,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::ConstFuncRetInt5, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::FuncRetStr5,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::ConstFuncRetStr5, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
#endif

		// 5 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::Func5,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::Func5,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::Func5,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc5, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::ConstFunc5, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::ConstFunc5, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::FuncRetInt5,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::ConstFuncRetInt5, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::FuncRetStr5,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::ConstFuncRetStr5, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg);
#endif
#endif
	}

	// 6 param function
	{
		// 0 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate6::FDelegate, 0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate6,                     0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate6,            1, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate6::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate6,                     0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate6,            1, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));

		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt6, 0, &UDynamicDelegateTestObject::FuncRetInt6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt6, 0, &UDynamicDelegateTestObject::ConstFuncRetInt6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr6, 0, &UDynamicDelegateTestObject::FuncRetStr6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr6, 0, &UDynamicDelegateTestObject::ConstFuncRetStr6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg));

#if defined(UE_USE_DYNAMIC_DELEGATE_PAYLOADS) && UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		// 1 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5::FDelegate, 0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate5,                     0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5,            1, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate5,                     0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5,            1, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt5, 0, &UDynamicDelegateTestObject::FuncRetInt6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt5, 0, &UDynamicDelegateTestObject::ConstFuncRetInt6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr5, 0, &UDynamicDelegateTestObject::FuncRetStr6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr5, 0, &UDynamicDelegateTestObject::ConstFuncRetStr6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg);
#endif

		// 2 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4::FDelegate, 0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate4,                     0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4,            1, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate4,                     0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4,            1, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt4, 0, &UDynamicDelegateTestObject::FuncRetInt6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt4, 0, &UDynamicDelegateTestObject::ConstFuncRetInt6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr4, 0, &UDynamicDelegateTestObject::FuncRetStr6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr4, 0, &UDynamicDelegateTestObject::ConstFuncRetStr6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg);
#endif

		// 3 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::FuncRetInt6,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::ConstFuncRetInt6, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::FuncRetStr6,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::ConstFuncRetStr6, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg);
#endif

		// 4 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::Func6,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::FuncRetInt6,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::ConstFuncRetInt6, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::FuncRetStr6,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::ConstFuncRetStr6, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg);
#endif

		// 5 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::Func6,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::Func6,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::ConstFunc6, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::FuncRetInt6,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::ConstFuncRetInt6, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::FuncRetStr6,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::ConstFuncRetStr6, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
#endif

		// 6 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::Func6,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::Func6,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::Func6,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc6, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::ConstFunc6, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::ConstFunc6, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::FuncRetInt6,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::ConstFuncRetInt6, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::FuncRetStr6,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::ConstFuncRetStr6, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg);
#endif
#endif
	}

	// 7 param function
	{
		// 0 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate7::FDelegate, 0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate7,                     0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate7,            1, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate7::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate7,                     0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate7,            1, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));

		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt7, 0, &UDynamicDelegateTestObject::FuncRetInt7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt7, 0, &UDynamicDelegateTestObject::ConstFuncRetInt7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr7, 0, &UDynamicDelegateTestObject::FuncRetStr7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr7, 0, &UDynamicDelegateTestObject::ConstFuncRetStr7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg));

#if defined(UE_USE_DYNAMIC_DELEGATE_PAYLOADS) && UE_USE_DYNAMIC_DELEGATE_PAYLOADS
		// 1 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate6::FDelegate, 0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate6,                     0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate6,            1, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate6::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate6,                     0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate6,            1, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt6, 0, &UDynamicDelegateTestObject::FuncRetInt7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt6, 0, &UDynamicDelegateTestObject::ConstFuncRetInt7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr6, 0, &UDynamicDelegateTestObject::FuncRetStr7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr6, 0, &UDynamicDelegateTestObject::ConstFuncRetStr7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg), GWeakObjPtrArg);
#endif

		// 2 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5::FDelegate, 0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate5,                     0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5,            1, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate5,                     0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate5,            1, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt5, 0, &UDynamicDelegateTestObject::FuncRetInt7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt5, 0, &UDynamicDelegateTestObject::ConstFuncRetInt7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr5, 0, &UDynamicDelegateTestObject::FuncRetStr7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr5, 0, &UDynamicDelegateTestObject::ConstFuncRetStr7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg), bGBoolArg, GWeakObjPtrArg);
#endif

		// 3 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4::FDelegate, 0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate4,                     0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4,            1, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate4,                     0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate4,            1, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt4, 0, &UDynamicDelegateTestObject::FuncRetInt7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt4, 0, &UDynamicDelegateTestObject::ConstFuncRetInt7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr4, 0, &UDynamicDelegateTestObject::FuncRetStr7,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr4, 0, &UDynamicDelegateTestObject::ConstFuncRetStr7, (GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg), GNameArg, bGBoolArg, GWeakObjPtrArg);
#endif

		// 4 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate3,                     0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate3,            1, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::FuncRetInt7,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt3, 0, &UDynamicDelegateTestObject::ConstFuncRetInt7, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::FuncRetStr7,      (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr3, 0, &UDynamicDelegateTestObject::ConstFuncRetStr7, (GInt16Arg, GUnscopedEnumArg, GStrArg), GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
#endif

		// 5 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::Func7,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate2,                     0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate2,            1, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::FuncRetInt7,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt2, 0, &UDynamicDelegateTestObject::ConstFuncRetInt7, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::FuncRetStr7,      (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr2, 0, &UDynamicDelegateTestObject::ConstFuncRetStr7, (GInt16Arg, GUnscopedEnumArg), GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
#endif

		// 6 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::Func7,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::Func7,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate1,                     0, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate1,            1, &UDynamicDelegateTestObject::ConstFunc7, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::FuncRetInt7,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt1, 0, &UDynamicDelegateTestObject::ConstFuncRetInt7, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::FuncRetStr7,      (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr1, 0, &UDynamicDelegateTestObject::ConstFuncRetStr7, (GInt16Arg), GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
#endif

		// 7 arg payload
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::Func7,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::Func7,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::Func7,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0::FDelegate, 0, &UDynamicDelegateTestObject::ConstFunc7, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0,                     0, &UDynamicDelegateTestObject::ConstFunc7, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0,            1, &UDynamicDelegateTestObject::ConstFunc7, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
#if UE_USE_DYNAMIC_DELEGATE_PAYLOADS_RETVAL
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::FuncRetInt7,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GIntRet, FTestDynamicDelegateRetInt0, 0, &UDynamicDelegateTestObject::ConstFuncRetInt7, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::FuncRetStr7,      (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
		BIND_CALL_CHECK_DELEGATE_RETVAL(GStrRet, FTestDynamicDelegateRetStr0, 0, &UDynamicDelegateTestObject::ConstFuncRetStr7, (), GInt16Arg, GUnscopedEnumArg, GStrArg, GPtrArg, GNameArg, bGBoolArg, GWeakObjPtrArg);
#endif
#endif
	}

	// 1 out param function
	{
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut1::FDelegate, 0, &UDynamicDelegateTestObject::FuncOut1,      (GInt16Arg), (GInt16Out));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut1,                     0, &UDynamicDelegateTestObject::FuncOut1,      (GInt16Arg), (GInt16Out));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut1,            1, &UDynamicDelegateTestObject::FuncOut1,      (GInt16Arg), (GInt16Out));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut1::FDelegate, 0, &UDynamicDelegateTestObject::ConstFuncOut1, (GInt16Arg), (GInt16Out));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut1,                     0, &UDynamicDelegateTestObject::ConstFuncOut1, (GInt16Arg), (GInt16Out));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut1,            1, &UDynamicDelegateTestObject::ConstFuncOut1, (GInt16Arg), (GInt16Out));
	}

	// 2 out param function
	{
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut2::FDelegate, 0, &UDynamicDelegateTestObject::FuncOut2,      (GInt16Arg, GUnscopedEnumArg), (GInt16Out, GUnscopedEnumOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut2,                     0, &UDynamicDelegateTestObject::FuncOut2,      (GInt16Arg, GUnscopedEnumArg), (GInt16Out, GUnscopedEnumOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut2,            1, &UDynamicDelegateTestObject::FuncOut2,      (GInt16Arg, GUnscopedEnumArg), (GInt16Out, GUnscopedEnumOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut2::FDelegate, 0, &UDynamicDelegateTestObject::ConstFuncOut2, (GInt16Arg, GUnscopedEnumArg), (GInt16Out, GUnscopedEnumOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut2,                     0, &UDynamicDelegateTestObject::ConstFuncOut2, (GInt16Arg, GUnscopedEnumArg), (GInt16Out, GUnscopedEnumOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut2,            1, &UDynamicDelegateTestObject::ConstFuncOut2, (GInt16Arg, GUnscopedEnumArg), (GInt16Out, GUnscopedEnumOut));
	}

	// 3 out param function
	{
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut3::FDelegate, 0, &UDynamicDelegateTestObject::FuncOut3,      (GInt16Arg, GUnscopedEnumArg, GStrArg), (GInt16Out, GUnscopedEnumOut, GStrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut3,                     0, &UDynamicDelegateTestObject::FuncOut3,      (GInt16Arg, GUnscopedEnumArg, GStrArg), (GInt16Out, GUnscopedEnumOut, GStrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut3,            1, &UDynamicDelegateTestObject::FuncOut3,      (GInt16Arg, GUnscopedEnumArg, GStrArg), (GInt16Out, GUnscopedEnumOut, GStrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut3::FDelegate, 0, &UDynamicDelegateTestObject::ConstFuncOut3, (GInt16Arg, GUnscopedEnumArg, GStrArg), (GInt16Out, GUnscopedEnumOut, GStrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut3,                     0, &UDynamicDelegateTestObject::ConstFuncOut3, (GInt16Arg, GUnscopedEnumArg, GStrArg), (GInt16Out, GUnscopedEnumOut, GStrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut3,            1, &UDynamicDelegateTestObject::ConstFuncOut3, (GInt16Arg, GUnscopedEnumArg, GStrArg), (GInt16Out, GUnscopedEnumOut, GStrOut));
	}

	// 4 out param function
	{
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut4::FDelegate, 0, &UDynamicDelegateTestObject::FuncOut4,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut4,                     0, &UDynamicDelegateTestObject::FuncOut4,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut4,            1, &UDynamicDelegateTestObject::FuncOut4,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut4::FDelegate, 0, &UDynamicDelegateTestObject::ConstFuncOut4, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut4,                     0, &UDynamicDelegateTestObject::ConstFuncOut4, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut4,            1, &UDynamicDelegateTestObject::ConstFuncOut4, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut));
	}

	// 5 out param function
	{
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut5::FDelegate, 0, &UDynamicDelegateTestObject::FuncOut5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut5,                     0, &UDynamicDelegateTestObject::FuncOut5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut5,            1, &UDynamicDelegateTestObject::FuncOut5,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut5::FDelegate, 0, &UDynamicDelegateTestObject::ConstFuncOut5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut5,                     0, &UDynamicDelegateTestObject::ConstFuncOut5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut5,            1, &UDynamicDelegateTestObject::ConstFuncOut5, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut));
	}

	// 6 out param function
	{
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut6::FDelegate, 0, &UDynamicDelegateTestObject::FuncOut6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg, GWeakObjPtrArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut, GWeakObjPtrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut6,                     0, &UDynamicDelegateTestObject::FuncOut6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg, GWeakObjPtrArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut, GWeakObjPtrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut6,            1, &UDynamicDelegateTestObject::FuncOut6,      (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg, GWeakObjPtrArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut, GWeakObjPtrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut6::FDelegate, 0, &UDynamicDelegateTestObject::ConstFuncOut6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg, GWeakObjPtrArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut, GWeakObjPtrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOut6,                     0, &UDynamicDelegateTestObject::ConstFuncOut6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg, GWeakObjPtrArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut, GWeakObjPtrOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOut6,            1, &UDynamicDelegateTestObject::ConstFuncOut6, (GInt16Arg, GUnscopedEnumArg, GStrArg, GNameArg, bGBoolArg, GWeakObjPtrArg), (GInt16Out, GUnscopedEnumOut, GStrOut, GNameOut, bGBoolOut, GWeakObjPtrOut));
	}

#if defined(UE_USE_DYNAMIC_DELEGATE_PAYLOADS) && UE_USE_DYNAMIC_DELEGATE_PAYLOADS
	// Additional property type payload tests — unicast
	{
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncFloat,       (), GFloatArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncDouble,      (), GDoubleArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncByte,        (), GByteArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncInt64,       (), GInt64Arg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncText,        (), GTextArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncStruct,      (), GStructArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncVector,      (), GVectorArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncIntArray,    (), GIntArrayArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncScopedEnum,  (), GScopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncIntSet,      (), GIntSetArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicDelegate0, 0, &UDynamicDelegateTestObject::FuncOptionalInt, (), GOptionalIntArg);
	}

	// Additional property type payload tests — multicast
	{
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncFloat,       (), GFloatArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncDouble,      (), GDoubleArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncByte,        (), GByteArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncInt64,       (), GInt64Arg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncText,        (), GTextArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncStruct,      (), GStructArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncVector,      (), GVectorArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncIntArray,    (), GIntArrayArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncScopedEnum,  (), GScopedEnumArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncIntSet,      (), GIntSetArg);
		BIND_CALL_CHECK_DELEGATE(FTestDynamicMulticastDelegate0, 1, &UDynamicDelegateTestObject::FuncOptionalInt, (), GOptionalIntArg);
	}

	// Additional property type out-parameter tests — unicast
	{
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutFloat,       0, &UDynamicDelegateTestObject::FuncOutFloat,       (GFloatArg),       (GFloatOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutDouble,      0, &UDynamicDelegateTestObject::FuncOutDouble,      (GDoubleArg),      (GDoubleOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutByte,        0, &UDynamicDelegateTestObject::FuncOutByte,        (GByteArg),        (GByteOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutInt64,       0, &UDynamicDelegateTestObject::FuncOutInt64,       (GInt64Arg),       (GInt64Out));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutText,        0, &UDynamicDelegateTestObject::FuncOutText,        (GTextArg),        (GTextOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutStruct,      0, &UDynamicDelegateTestObject::FuncOutStruct,      (GStructArg),      (GStructOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutVector,      0, &UDynamicDelegateTestObject::FuncOutVector,      (GVectorArg),      (GVectorOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutScopedEnum,  0, &UDynamicDelegateTestObject::FuncOutScopedEnum,  (GScopedEnumArg),  (GScopedEnumOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutIntArray,    0, &UDynamicDelegateTestObject::FuncOutIntArray,    (GIntArrayArg),    (GIntArrayOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutIntSet,      0, &UDynamicDelegateTestObject::FuncOutIntSet,      (GIntSetArg),      (GIntSetOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicDelegateOutOptionalInt, 0, &UDynamicDelegateTestObject::FuncOutOptionalInt, (GOptionalIntArg), (GOptionalIntOut));
	}

	// Additional property type out-parameter tests — multicast
	{
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutFloat,       1, &UDynamicDelegateTestObject::FuncOutFloat,       (GFloatArg),       (GFloatOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutDouble,      1, &UDynamicDelegateTestObject::FuncOutDouble,      (GDoubleArg),      (GDoubleOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutByte,        1, &UDynamicDelegateTestObject::FuncOutByte,        (GByteArg),        (GByteOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutInt64,       1, &UDynamicDelegateTestObject::FuncOutInt64,       (GInt64Arg),       (GInt64Out));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutText,        1, &UDynamicDelegateTestObject::FuncOutText,        (GTextArg),        (GTextOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutStruct,      1, &UDynamicDelegateTestObject::FuncOutStruct,      (GStructArg),      (GStructOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutVector,      1, &UDynamicDelegateTestObject::FuncOutVector,      (GVectorArg),      (GVectorOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutScopedEnum,  1, &UDynamicDelegateTestObject::FuncOutScopedEnum,  (GScopedEnumArg),  (GScopedEnumOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutIntArray,    1, &UDynamicDelegateTestObject::FuncOutIntArray,    (GIntArrayArg),    (GIntArrayOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutIntSet,      1, &UDynamicDelegateTestObject::FuncOutIntSet,      (GIntSetArg),      (GIntSetOut));
		BIND_CALL_CHECK_DELEGATE_OUT(FTestDynamicMulticastDelegateOutOptionalInt, 1, &UDynamicDelegateTestObject::FuncOutOptionalInt, (GOptionalIntArg), (GOptionalIntOut));
	}
#endif
}

TEST_CASE_NAMED(DynamicDelegateTest, "UE::CoreUObject::DynamicDelegateTest", "[ApplicationContextMask][SmokeFilter]")
{
	UDynamicDelegateTestObject* TestObj = NewObject<UDynamicDelegateTestObject>();
	TestObj->RunTests();
}

static_assert(UE::CDynamicDelegate<FTestDynamicDelegate0>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegate1>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegate2>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegate3>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegate4>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegate5>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegate6>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetInt0>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetInt1>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetInt2>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetInt3>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetInt4>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetInt5>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetInt6>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetStr0>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetStr1>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetStr2>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetStr3>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetStr4>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetStr5>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateRetStr6>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateOut1>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateOut2>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateOut3>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateOut4>);
static_assert(UE::CDynamicDelegate<FTestDynamicDelegateOut5>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegate0>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegate1>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegate2>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegate3>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegate4>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegate5>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegate6>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetInt0>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetInt1>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetInt2>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetInt3>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetInt4>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetInt5>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetInt6>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetStr0>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetStr1>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetStr2>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetStr3>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetStr4>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetStr5>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateRetStr6>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateOut1>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateOut2>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateOut3>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateOut4>);
static_assert(!UE::CDynamicMulticastDelegate<FTestDynamicDelegateOut5>);

static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegate0>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegate1>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegate2>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegate3>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegate4>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegate5>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegate6>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegateOut1>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegateOut2>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegateOut3>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegateOut4>);
static_assert(UE::CDynamicMulticastDelegate<FTestDynamicMulticastDelegateOut5>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegate0>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegate1>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegate2>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegate3>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegate4>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegate5>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegate6>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegateOut1>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegateOut2>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegateOut3>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegateOut4>);
static_assert(!UE::CDynamicDelegate<FTestDynamicMulticastDelegateOut5>);

#undef CHECK_ARG_VARIABLES
#undef EXPAND_ARG_VARIABLES
#undef DEFINE_ARG_VARIABLES

#undef CHECK_ARG_VARIABLES_6
#undef CHECK_ARG_VARIABLES_5
#undef CHECK_ARG_VARIABLES_4
#undef CHECK_ARG_VARIABLES_3
#undef CHECK_ARG_VARIABLES_2
#undef CHECK_ARG_VARIABLES_1

#undef EXPAND_ARG_VARIABLES_6
#undef EXPAND_ARG_VARIABLES_5
#undef EXPAND_ARG_VARIABLES_4
#undef EXPAND_ARG_VARIABLES_3
#undef EXPAND_ARG_VARIABLES_2
#undef EXPAND_ARG_VARIABLES_1

#undef DEFINE_ARG_VARIABLES_6
#undef DEFINE_ARG_VARIABLES_5
#undef DEFINE_ARG_VARIABLES_4
#undef DEFINE_ARG_VARIABLES_3
#undef DEFINE_ARG_VARIABLES_2
#undef DEFINE_ARG_VARIABLES_1

#undef BIND_CALL_CHECK_DELEGATE_OUT
#undef BIND_CALL_CHECK_DELEGATE_RETVAL
#undef BIND_CALL_CHECK_DELEGATE

#endif // #if WITH_TESTS