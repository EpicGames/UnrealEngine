// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TESTS

#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"

#include "DynamicDelegateTestObject.generated.h"

USTRUCT()
struct FTestPayloadStruct
{
	GENERATED_BODY()

	UPROPERTY()
	int32 X = 0;

	UPROPERTY()
	int32 Y = 0;

	UPROPERTY()
	int32 Z = 0;

	bool operator==(const FTestPayloadStruct&) const = default;
};

UENUM()
enum class ETestScopedEnum : uint8
{
	Value1,
	Value2
};

UENUM()
namespace ETestUnscopedEnum
{
	enum Type : int
	{
		E1,
		E2
	};
}

DECLARE_DYNAMIC_DELEGATE            (FTestDynamicDelegate0);
DECLARE_DYNAMIC_DELEGATE_OneParam   (FTestDynamicDelegate1, int16, Int);
DECLARE_DYNAMIC_DELEGATE_TwoParams  (FTestDynamicDelegate2, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FTestDynamicDelegate3, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str);
DECLARE_DYNAMIC_DELEGATE_FourParams (FTestDynamicDelegate4, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr);
DECLARE_DYNAMIC_DELEGATE_FiveParams (FTestDynamicDelegate5, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name);
DECLARE_DYNAMIC_DELEGATE_SixParams  (FTestDynamicDelegate6, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name, bool, bBool);
DECLARE_DYNAMIC_DELEGATE_SevenParams(FTestDynamicDelegate7, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name, bool, bBool, TWeakObjectPtr<UDynamicDelegateTestObject>, WeakPtr);

DECLARE_DYNAMIC_MULTICAST_DELEGATE            (FTestDynamicMulticastDelegate0);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam   (FTestDynamicMulticastDelegate1, int16, Int);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams  (FTestDynamicMulticastDelegate2, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FTestDynamicMulticastDelegate3, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams (FTestDynamicMulticastDelegate4, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams (FTestDynamicMulticastDelegate5, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_SixParams  (FTestDynamicMulticastDelegate6, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name, bool, bBool);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_SevenParams(FTestDynamicMulticastDelegate7, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name, bool, bBool, TWeakObjectPtr<UDynamicDelegateTestObject>, WeakPtr);

DECLARE_DYNAMIC_DELEGATE_RetVal            (int, FTestDynamicDelegateRetInt0);
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam   (int, FTestDynamicDelegateRetInt1, int16, Int);
DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams  (int, FTestDynamicDelegateRetInt2, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum);
DECLARE_DYNAMIC_DELEGATE_RetVal_ThreeParams(int, FTestDynamicDelegateRetInt3, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str);
DECLARE_DYNAMIC_DELEGATE_RetVal_FourParams (int, FTestDynamicDelegateRetInt4, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr);
DECLARE_DYNAMIC_DELEGATE_RetVal_FiveParams (int, FTestDynamicDelegateRetInt5, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name);
DECLARE_DYNAMIC_DELEGATE_RetVal_SixParams  (int, FTestDynamicDelegateRetInt6, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name, bool, bBool);
DECLARE_DYNAMIC_DELEGATE_RetVal_SevenParams(int, FTestDynamicDelegateRetInt7, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name, bool, bBool, TWeakObjectPtr<UDynamicDelegateTestObject>, WeakPtr);

DECLARE_DYNAMIC_DELEGATE_RetVal            (FString, FTestDynamicDelegateRetStr0);
DECLARE_DYNAMIC_DELEGATE_RetVal_OneParam   (FString, FTestDynamicDelegateRetStr1, int16, Int);
DECLARE_DYNAMIC_DELEGATE_RetVal_TwoParams  (FString, FTestDynamicDelegateRetStr2, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum);
DECLARE_DYNAMIC_DELEGATE_RetVal_ThreeParams(FString, FTestDynamicDelegateRetStr3, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str);
DECLARE_DYNAMIC_DELEGATE_RetVal_FourParams (FString, FTestDynamicDelegateRetStr4, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr);
DECLARE_DYNAMIC_DELEGATE_RetVal_FiveParams (FString, FTestDynamicDelegateRetStr5, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name);
DECLARE_DYNAMIC_DELEGATE_RetVal_SixParams  (FString, FTestDynamicDelegateRetStr6, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name, bool, bBool);
DECLARE_DYNAMIC_DELEGATE_RetVal_SevenParams(FString, FTestDynamicDelegateRetStr7, int16, Int, ETestUnscopedEnum::Type, UnscopedEnum, FString, Str, const UDynamicDelegateTestObject*, Ptr, FName, Name, bool, bBool, TWeakObjectPtr<UDynamicDelegateTestObject>, WeakPtr);

DECLARE_DYNAMIC_DELEGATE_OneParam   (FTestDynamicDelegateOut1, int16&, Int);
DECLARE_DYNAMIC_DELEGATE_TwoParams  (FTestDynamicDelegateOut2, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum);
DECLARE_DYNAMIC_DELEGATE_ThreeParams(FTestDynamicDelegateOut3, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum, FString&, Str);
DECLARE_DYNAMIC_DELEGATE_FourParams (FTestDynamicDelegateOut4, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum, FString&, Str, FName&, Name);
DECLARE_DYNAMIC_DELEGATE_FiveParams (FTestDynamicDelegateOut5, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum, FString&, Str, FName&, Name, bool&, bBool);
DECLARE_DYNAMIC_DELEGATE_SixParams  (FTestDynamicDelegateOut6, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum, FString&, Str, FName&, Name, bool&, bBool, TWeakObjectPtr<UDynamicDelegateTestObject>&, WeakPtr);

// Pointer out params aren't currently supported
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam   (FTestDynamicMulticastDelegateOut1, int16&, Int);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams  (FTestDynamicMulticastDelegateOut2, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FTestDynamicMulticastDelegateOut3, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum, FString&, Str);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FourParams (FTestDynamicMulticastDelegateOut4, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum, FString&, Str, FName&, Name);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_FiveParams (FTestDynamicMulticastDelegateOut5, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum, FString&, Str, FName&, Name, bool&, bBool);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_SixParams  (FTestDynamicMulticastDelegateOut6, int16&, Int, TEnumAsByte<ETestUnscopedEnum::Type>&, UnscopedEnum, FString&, Str, FName&, Name, bool&, bBool, TWeakObjectPtr<UDynamicDelegateTestObject>&, WeakPtr);

// Delegates for testing additional property types in payload serialization.
// Only single arguments supported at the moment- we need to add UHT support for proper variadic dynamic delegates to get full combination testsing, like:
// UDELEGATE()
// using FTestDynamicDelegateRetStr6 = TDynamicDelegate<void(int Int, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakPtr)>;
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateFloat,        float,              Float);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateDouble,       double,             Double);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateByte,         uint8,              Byte);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateInt64,        int64,              Int64);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateText,         FText,              Text);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateStruct,       FTestPayloadStruct, Struct);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateVector,       FVector,            Vector);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateIntArray,     TArray<int32>,      IntArray);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateScopedEnum,   ETestScopedEnum,    Enum);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateIntSet,       TSet<int32>,        IntSet);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOptionalInt,  TOptional<int32>,   OptionalInt);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateFloat,       float,              Float);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateDouble,      double,             Double);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateByte,        uint8,              Byte);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateInt64,       int64,              Int64);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateText,        FText,              Text);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateStruct,      FTestPayloadStruct, Struct);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateVector,      FVector,            Vector);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateIntArray,    TArray<int32>,      IntArray);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateScopedEnum,  ETestScopedEnum,    Enum);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateIntSet,      TSet<int32>,        IntSet);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOptionalInt, TOptional<int32>,   OptionalInt);

// Out-parameter delegates for additional property types
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutFloat,       float&,              Float);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutDouble,      double&,             Double);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutByte,        uint8&,              Byte);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutInt64,       int64&,              Int64);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutText,        FText&,              Text);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutStruct,      FTestPayloadStruct&, Struct);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutVector,      FVector&,            Vector);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutScopedEnum,  ETestScopedEnum&,    Enum);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutIntArray,    TArray<int32>&,      IntArray);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutIntSet,      TSet<int32>&,        IntSet);
DECLARE_DYNAMIC_DELEGATE_OneParam(FTestDynamicDelegateOutOptionalInt, TOptional<int32>&,   OptionalInt);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutFloat,       float&,              Float);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutDouble,      double&,             Double);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutByte,        uint8&,              Byte);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutInt64,       int64&,              Int64);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutText,        FText&,              Text);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutStruct,      FTestPayloadStruct&, Struct);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutVector,      FVector&,            Vector);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutScopedEnum,  ETestScopedEnum&,    Enum);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutIntArray,    TArray<int32>&,      IntArray);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutIntSet,      TSet<int32>&,        IntSet);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FTestDynamicMulticastDelegateOutOptionalInt, TOptional<int32>&,   OptionalInt);

UCLASS()
class UDynamicDelegateTestObject : public UObject
{
	GENERATED_BODY()

public:
	UDynamicDelegateTestObject();

	UFUNCTION() void Func0();
	UFUNCTION() void Func1(int16 Int);
	UFUNCTION() void Func2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum);
	UFUNCTION() void Func3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str);
	UFUNCTION() void Func4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr);
	UFUNCTION() void Func5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name);
	UFUNCTION() void Func6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool);
	UFUNCTION() void Func7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakPtr);

	UFUNCTION() void ConstFunc0() const;
	UFUNCTION() void ConstFunc1(int16 Int) const;
	UFUNCTION() void ConstFunc2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum) const;
	UFUNCTION() void ConstFunc3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str) const;
	UFUNCTION() void ConstFunc4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr) const;
	UFUNCTION() void ConstFunc5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name) const;
	UFUNCTION() void ConstFunc6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool) const;
	UFUNCTION() void ConstFunc7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakPtr) const;

	UFUNCTION() int FuncRetInt0();
	UFUNCTION() int FuncRetInt1(int16 Int);
	UFUNCTION() int FuncRetInt2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum);
	UFUNCTION() int FuncRetInt3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str);
	UFUNCTION() int FuncRetInt4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr);
	UFUNCTION() int FuncRetInt5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name);
	UFUNCTION() int FuncRetInt6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool);
	UFUNCTION() int FuncRetInt7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakPtr);

	UFUNCTION() FString FuncRetStr0();
	UFUNCTION() FString FuncRetStr1(int16 Int);
	UFUNCTION() FString FuncRetStr2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum);
	UFUNCTION() FString FuncRetStr3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str);
	UFUNCTION() FString FuncRetStr4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr);
	UFUNCTION() FString FuncRetStr5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name);
	UFUNCTION() FString FuncRetStr6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool);
	UFUNCTION() FString FuncRetStr7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakPtr);

	UFUNCTION() int ConstFuncRetInt0() const;
	UFUNCTION() int ConstFuncRetInt1(int16 Int) const;
	UFUNCTION() int ConstFuncRetInt2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum) const;
	UFUNCTION() int ConstFuncRetInt3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str) const;
	UFUNCTION() int ConstFuncRetInt4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr) const;
	UFUNCTION() int ConstFuncRetInt5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name) const;
	UFUNCTION() int ConstFuncRetInt6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool) const;
	UFUNCTION() int ConstFuncRetInt7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakPtr) const;

	UFUNCTION() FString ConstFuncRetStr0() const;
	UFUNCTION() FString ConstFuncRetStr1(int16 Int) const;
	UFUNCTION() FString ConstFuncRetStr2(int16 Int, ETestUnscopedEnum::Type UnscopedEnum) const;
	UFUNCTION() FString ConstFuncRetStr3(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str) const;
	UFUNCTION() FString ConstFuncRetStr4(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr) const;
	UFUNCTION() FString ConstFuncRetStr5(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name) const;
	UFUNCTION() FString ConstFuncRetStr6(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool) const;
	UFUNCTION() FString ConstFuncRetStr7(int16 Int, ETestUnscopedEnum::Type UnscopedEnum, FString Str, const UDynamicDelegateTestObject* Ptr, FName Name, bool bBool, TWeakObjectPtr<UDynamicDelegateTestObject> WeakPtr) const;

	// Pointer out params aren't currently supported
	UFUNCTION() void FuncOut1(int16& Int);
	UFUNCTION() void FuncOut2(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum);
	UFUNCTION() void FuncOut3(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str);
	UFUNCTION() void FuncOut4(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name);
	UFUNCTION() void FuncOut5(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name, bool& bBool);
	UFUNCTION() void FuncOut6(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name, bool& bBool, TWeakObjectPtr<UDynamicDelegateTestObject>& WeakPtr);

	// Pointer out params aren't currently supported
	UFUNCTION() void ConstFuncOut1(int16& Int) const;
	UFUNCTION() void ConstFuncOut2(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum) const;
	UFUNCTION() void ConstFuncOut3(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str) const;
	UFUNCTION() void ConstFuncOut4(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name) const;
	UFUNCTION() void ConstFuncOut5(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name, bool& bBool) const;
	UFUNCTION() void ConstFuncOut6(int16& Int, TEnumAsByte<ETestUnscopedEnum::Type>& UnscopedEnum, FString& Str, FName& Name, bool& bBool, TWeakObjectPtr<UDynamicDelegateTestObject>& WeakPtr) const;

	// Functions for testing additional property types in payload serialization
	UFUNCTION() void FuncFloat(float Float);
	UFUNCTION() void FuncDouble(double Double);
	UFUNCTION() void FuncByte(uint8 Byte);
	UFUNCTION() void FuncInt64(int64 Int64);
	UFUNCTION() void FuncText(FText Text);
	UFUNCTION() void FuncStruct(FTestPayloadStruct Struct);
	UFUNCTION() void FuncVector(FVector Vector);
	UFUNCTION() void FuncIntArray(TArray<int32> IntArray);
	UFUNCTION() void FuncScopedEnum(ETestScopedEnum Enum);
	UFUNCTION() void FuncIntSet(TSet<int32> IntSet);
	UFUNCTION() void FuncOptionalInt(TOptional<int32> OptionalInt);

	// Out-parameter functions for additional property types
	UFUNCTION() void FuncOutFloat(float& Float);
	UFUNCTION() void FuncOutDouble(double& Double);
	UFUNCTION() void FuncOutByte(uint8& Byte);
	UFUNCTION() void FuncOutInt64(int64& Int64);
	UFUNCTION() void FuncOutText(FText& Text);
	UFUNCTION() void FuncOutStruct(FTestPayloadStruct& Struct);
	UFUNCTION() void FuncOutVector(FVector& Vector);
	UFUNCTION() void FuncOutScopedEnum(ETestScopedEnum& Enum);
	UFUNCTION() void FuncOutIntArray(TArray<int32>& IntArray);
	UFUNCTION() void FuncOutIntSet(TSet<int32>& IntSet);
	UFUNCTION() void FuncOutOptionalInt(TOptional<int32>& OptionalInt);

	void RunTests();
};

#endif // WITH_TESTS

// Forward declaration still needed for .gen.cpp when building without tests
class UDynamicDelegateTestObject;
