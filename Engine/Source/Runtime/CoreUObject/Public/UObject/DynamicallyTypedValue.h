// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "AutoRTFM.h"

#if WITH_VERSE_VM
#include "VerseVM/VVMContext.h"
#include "VerseVM/VVMValue.h"
#endif

class FReferenceCollector;

namespace UE
{
	namespace Verse
	{
		struct FRuntimeType;
	}

	// Provides methods to interact with values of a specific type.
	struct FDynamicallyTypedValueType
	{
		enum class EContainsReferences : bool
		{
			DoesNot,
			Maybe,
		};

		constexpr FDynamicallyTypedValueType(SIZE_T InNumBytes, uint8 InMinAlignmentLogTwo, EContainsReferences InContainsReferences)
			: NumBytes(InNumBytes)
			, MinAlignmentLogTwo(InMinAlignmentLogTwo)
			, ContainsReferences(InContainsReferences)
		{
		}

		// Marks the type itself as reachable.
		virtual void MarkReachable(FReferenceCollector& Collector) = 0;

		// Marks a value of the type as reachable.
		virtual void MarkValueReachable(void* Data, FReferenceCollector& Collector) const = 0;

		virtual void InitializeValue(void* Data) const = 0;
		virtual void InitializeValueFrom(void* DestData, const void* SourceData, const Verse::FRuntimeType& SourceType) const = 0;
		virtual void InitializeValueFromCopy(void* DestData, const void* SourceData) const = 0;
		virtual void DestroyValue(void* Data) const = 0;

#if WITH_VERSE_VM
		virtual ::Verse::VValue ToVValue(::Verse::FAllocationContext Context, const void* Data) const = 0;
#endif

		virtual void SerializeValue(FStructuredArchive::FSlot Slot, void* Data, const void* DefaultData) const = 0;

		virtual uint32 GetValueHash(const void* Data) const = 0;
		virtual bool AreIdentical(const void* DataA, const void* DataB) const = 0;

		SIZE_T GetNumBytes() const { return NumBytes; }
		uint8 GetMinAlignmentLogTwo() const { return MinAlignmentLogTwo; }
		uint32 GetMinAlignment() const { return 1u << MinAlignmentLogTwo; }
		EContainsReferences GetContainsReferences() const { return ContainsReferences; }

		virtual void ForEachSubobject(const void* Data, TFunctionRef<void(const UObject*)> ObjectFunc) const = 0;

	private:
		const SIZE_T NumBytes;
		const uint8 MinAlignmentLogTwo;
		const EContainsReferences ContainsReferences;
	};

	// A value stored in some uninterpreted memory and a pointer to a type that contains methods to interpret it.
	struct FDynamicallyTypedValue
	{
		static COREUOBJECT_API FDynamicallyTypedValueType& NullType();

		FDynamicallyTypedValue()
		{
			// We need `this` to be in a special state such that if we've created this within a transaction
			// the GC will be able to destroy the value. To do that it needs a very specific null state,
			// which is what `InitializeToNull` gives us. So run this in the open, but disable any memory
			// validation on the value as its fine if we've written to this in the open, then write to it in
			// the closed. Those closed writes will be undone, reverting the value back to the specific null
			// state, which the GC can handle.
			UE_AUTORTFM_OPEN_NO_VALIDATION { InitializeToNull(); };
		}

		// Use a delegated constructor so that the value is nulled correctly before initialized from the copy.
		FDynamicallyTypedValue(const FDynamicallyTypedValue& Copyee) : FDynamicallyTypedValue() { InitializeFromCopy(Copyee); }

		// Use a delegated constructor so that the value is nulled correctly before initialized from the move.
		FDynamicallyTypedValue(FDynamicallyTypedValue&& Movee) : FDynamicallyTypedValue() { InitializeFromMove(MoveTemp(Movee)); }

		~FDynamicallyTypedValue() { Deinit(); }

		FDynamicallyTypedValue& operator=(const FDynamicallyTypedValue& Copyee)
		{
			if (this != &Copyee)
			{
				Deinit();
				InitializeFromCopy(Copyee);
			}
			return *this;
		}
		FDynamicallyTypedValue& operator=(FDynamicallyTypedValue&& Movee)
		{
			if (this != &Movee)
			{
				Deinit();
				InitializeFromMove(MoveTemp(Movee));
			}
			return *this;
		}

		// Returns a pointer to the value's data.
		const void* GetDataPointer() const { return IsInline() ? &InlineData : HeapData; }
		void* GetDataPointer() { return IsInline() ? &InlineData : HeapData; }

		// Returns the value's type.
		FDynamicallyTypedValueType& GetType() const
		{
			return Type ? *Type : NullType();
		}

		// Sets the value to the null state.
		void SetToNull()
		{
			Deinit();
			InitializeToNull();
		}

		// Sets the value to the initial value of a type.
		void InitializeAsType(FDynamicallyTypedValueType& NewType)
		{
			check(&NewType != nullptr);
			Deinit();
			Type = &NewType;
			AllocateData();
			Type->InitializeValue(GetDataPointer());
			MarkTypeReachableIfIncrementalReachabilityPending();			
		}

#if WITH_VERSE_VM
		::Verse::VValue ToVValue(::Verse::FAllocationContext Context) const
		{
			return Type->ToVValue(Context, GetDataPointer());
		}
#endif

		// Returns hash of the underlying FDynamicallyTypedValue's value. Added to allow for FDynamicallyTypedValue to be used as TMap keys.
		[[nodiscard]] friend uint32 GetTypeHash(const FDynamicallyTypedValue& DynamicallyTypedValue)
		{
			return DynamicallyTypedValue.GetType().GetValueHash(DynamicallyTypedValue.GetDataPointer());
		}

		// Traverses the data based on the type of value that is stored and invokes ObjectFunc for each UObject that is found.
		void ForEachSubobject(TFunctionRef<void(const UObject*)> ObjectFunc) const
		{
			Type->ForEachSubobject(GetDataPointer(), ObjectFunc);
		}

	private:
		void MarkTypeReachableIfIncrementalReachabilityPending()
		{
			struct FTypeReferenceCollector final : public FReferenceCollector
			{
				bool IsIgnoringArchetypeRef() const override { return false; }
				bool IsIgnoringTransient() const override { return false; }
				void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
				{
					if (InObject)
					{
						UE::GC::MarkAsReachable(InObject);
					}
				}
			};
			
			if (UE::GC::GIsIncrementalReachabilityPending && Type)
			{
				// nb: this is done to simulate a write barrier for this type, which
				//     enables it to behave properly with incremental gc.
				FTypeReferenceCollector Collector;
				Type->MarkReachable(Collector);
			}
		}
		
		FDynamicallyTypedValueType* Type;

		// Store pointer-sized or smaller values inline, heap allocate all others.
		union
		{
			UPTRINT InlineData;
			void* HeapData;
		};

		// Initialize this value from the primordial state to the null state.
		void InitializeToNull()
		{
			Type = &NullType();
			HeapData = nullptr;
			MarkTypeReachableIfIncrementalReachabilityPending();			
		}
		// Deinitializes this value back to the primordial state.
		void Deinit()
		{
			if (!Type)
			{
				return;
			}
			Type->DestroyValue(GetDataPointer());
			FreeData();
			Type = nullptr;
		}
		// Copies the data from another value to this one, which is assumed to be in the primordial state.
		void InitializeFromCopy(const FDynamicallyTypedValue& Copyee)
		{
			Type = Copyee.Type;
			AllocateData();
			Type->InitializeValueFromCopy(GetDataPointer(), Copyee.GetDataPointer());
			MarkTypeReachableIfIncrementalReachabilityPending();			
		}
		// Moves the data from another value to this one, which is assumed to be in the primordial state.
		// The source value is set to the null state.
		void InitializeFromMove(FDynamicallyTypedValue&& Movee)
		{
			// Simply copy the type and data from the source value.
			// This assumes that the data is trivially relocatable.
			Type = Movee.Type;
			InlineData = Movee.InlineData;
			MarkTypeReachableIfIncrementalReachabilityPending();			

			// Reset the source value to null.
			Movee.InitializeToNull();
		}

		// Whether the value's data is stored in InlineData or in the memory pointed to by HeapData.
		bool IsInline() const
		{
			return Type->GetNumBytes() <= sizeof(UPTRINT)
				&& Type->GetMinAlignmentLogTwo() <= UE_FORCE_CONSTEVAL(FMath::ConstExprCeilLogTwo(alignof(UPTRINT)));
		}

		// Allocates heap memory for the value if it uses it.
		void AllocateData()
		{
			if (IsInline())
			{
				// Ensure that the data is zeroed in the inline case to avoid spurious static analysis
				// errors about passing a reference to uninitialized data to InitializeValueFromCopy.
				InlineData = 0;
			}
			else
			{
				HeapData = FMemory::Malloc(Type->GetNumBytes(), Type->GetMinAlignment());
			}
		}

		// Frees heap memory for the value if it uses it.
		void FreeData()
		{
			if (!IsInline())
			{
				FMemory::Free(HeapData);
				HeapData = nullptr;
			}
			else if (::AutoRTFM::IsClosed())
			{
				// Assign to InlineData if we're in a closed transaction.
				// This is done to ensure that the value of InlineData is
				// recorded in the transaction. This is important as some
				// code paths destruct then re-construct the 
				// FDynamicallyTypedValue, and the constructor calls
				// InitializeToNull() in the open (without recording the initial
				// values). This can result in the FDynamicallyTypedValue being
				// nulled without a write record to restore the original value.
				InlineData = 0;
			}
		}
	};
}

Expose_TNameOf(UE::FDynamicallyTypedValue)
