// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"
#include "UObjectGlobals.h"

class FArchive;
class FBlake3;
class FNumericProperty;
class FOutputDevice;
class FReferenceCollector;
class UEnum;
class UField;
class UObject;
class UPackageMap;
class UStruct;
namespace UECodeGen_Private { struct FEnumPropertyParams; }
struct FPropertyTag;

class FEnumProperty : public FProperty
{
	DECLARE_FIELD_API(FEnumProperty, FProperty, CASTCLASS_FEnumProperty, COREUOBJECT_API)

public:
	COREUOBJECT_API FEnumProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FEnumProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FEnumProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FEnumProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param Prop Pointer to the compiled in structure describing the property
	 **/
	COREUOBJECT_API FEnumProperty(FFieldVariant InOwner, const UECodeGen_Private::FEnumPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	consteval FEnumProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		FNumericProperty* InUnderlyingProperty,
		int32 InUnderlyingSize,
		UE::CodeGen::ConstInit::TObjectRefParam<UEnum> InEnum
	)
		: Super(InBaseParams.SetElementSize(InUnderlyingSize))
		, UnderlyingProp(InUnderlyingProperty)
		, Enum(InEnum)
	{
	}
#endif


#if WITH_EDITORONLY_DATA
	COREUOBJECT_API explicit FEnumProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA
	COREUOBJECT_API virtual ~FEnumProperty();

	// UObject interface
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;
	COREUOBJECT_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	COREUOBJECT_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	COREUOBJECT_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	COREUOBJECT_API virtual void PostDuplicate(const FField& InField) override;
	COREUOBJECT_API virtual FField* GetInnerFieldByName(const FName& InName) override;
	COREUOBJECT_API virtual void GetInnerFields(TArray<FField*>& OutFields) override;

	// UField interface
	COREUOBJECT_API virtual void AddCppProperty(FProperty* Property) override;
	// End of UField interface

	// FProperty interface
	COREUOBJECT_API virtual FString GetCPPMacroType( FString& ExtendedTypeText ) const  override;
	COREUOBJECT_API virtual FString GetCPPType( FString* ExtendedTypeText, uint32 CPPExportFlags ) const override;
	COREUOBJECT_API virtual void LinkInternal(FArchive& Ar) override;
	COREUOBJECT_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	COREUOBJECT_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	COREUOBJECT_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8> * MetaData = nullptr) const override;
	COREUOBJECT_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	COREUOBJECT_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	COREUOBJECT_API virtual int32 GetMinAlignment() const override;
	COREUOBJECT_API virtual bool SameType(const FProperty* Other) const override;
	COREUOBJECT_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
#if WITH_EDITORONLY_DATA
	COREUOBJECT_API virtual void AppendSchemaHash(FBlake3& Builder, bool bSkipEditorOnly) const override;
#endif
	COREUOBJECT_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr) override;
	COREUOBJECT_API virtual void SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const override;
	COREUOBJECT_API virtual bool CanSerializeFromTypeName(UE::FPropertyTypeName Type) const override;
	// End of FProperty interface

	/**
	 * Set the UEnum of this property.
	 * @note May only be called once to lazily initialize the property when using the default constructor.
	 */
	inline void SetEnum(UEnum* InEnum)
	{
		checkf(!Enum, TEXT("FEnumProperty enum may only be set once"));
		Enum = InEnum;
	}

	/**
	 * Set the UEnum of this property.
	 * @note Internal! Used for creation of enums that use impersonation.
	 */
	UE_INTERNAL UE_FORCEINLINE_HINT void SetEnumForImpersonation(UEnum* InEnum)
	{
		Enum = InEnum;
	}

	/**
	 * Returns a pointer to the UEnum of this property.
	 */
	UE_FORCEINLINE_HINT UEnum* GetEnum() const
	{
		return Enum;
	}

	/**
	 * Returns the numeric property which represents the integral type of the enum.
	 */
	UE_FORCEINLINE_HINT FNumericProperty* GetUnderlyingProperty() const
	{
		return UnderlyingProp;
	}

	// Returns the number of bits required by NetSerializeItem to encode this enum, based on the maximum value
	COREUOBJECT_API uint64 GetMaxNetSerializeBits() const;

	virtual bool HasIntrusiveUnsetOptionalState() const override
	{
		return false;
	}

private:
	COREUOBJECT_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;

	FNumericProperty* UnderlyingProp; // The property which represents the underlying type of the enum
	TObjectPtr<UEnum> Enum; // The enum represented by this property
};
