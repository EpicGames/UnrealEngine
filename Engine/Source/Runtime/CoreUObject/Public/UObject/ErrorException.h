// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.2, "ErrorException.h has been deprecated.")
