// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/Defines.h"
#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GarbageCollectionSchema.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "Async/TaskGraphInterfaces.h"
#include "UObject/UnrealType.h"
#include "Misc/ScopeLock.h"
#include "HAL/PlatformProcess.h"
#include "UObject/FieldPath.h"
#include "Async/ParallelFor.h"
#include "UObject/UObjectArray.h"
#include "UObject/DynamicallyTypedValue.h"
#include "UObject/GCObject.h"

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
#include "VerseVM/Inline/VVMCellInline.h"
#include "VerseVM/Inline/VVMNativeStructInline.h"
#include "VerseVM/Inline/VVMValueInline.h"
#include "VerseVM/VVMValue.h"
#include "VerseVM/VVMVerseStruct.h"
#endif

#if ENABLE_GC_HISTORY
#include "UObject/ReferenceToken.h"
#endif

/*=============================================================================
	FastReferenceCollector.h: Unreal realtime garbage collection helpers
=============================================================================*/

namespace UE::GC
{
class FWorkCoordinator;
struct FWorkerContext;
}

enum class EGCOptions : uint32
{
	None = 0,
	Parallel = 1 << 0,					// Use all task workers to collect references, must be started on main thread
	AutogenerateSchemas = 1 << 1,		// Assemble schemas for new UClasses
	EliminateGarbage  = 1 << 2,			// Internal flag used by reachability analysis
	IncrementalReachability = 1 << 3	// Run Reachability Analysis incrementally
};
ENUM_CLASS_FLAGS(EGCOptions);

inline constexpr bool IsParallel(EGCOptions Options) { return !!(Options & EGCOptions::Parallel); }

inline constexpr bool IsEliminatingGarbage(EGCOptions Options) { return !!(Options & EGCOptions::EliminateGarbage); }

/** Helper to give GC internals friend access to certain core classes */
struct FGCInternals
{
	FORCEINLINE static FUObjectItem* GetResolvedOwner(FFieldPath& Path) { return Path.GetResolvedOwnerItemInternal(); }
	FORCEINLINE static void ClearCachedField(FFieldPath& Path) { Path.ClearCachedFieldInternal(); }
};

/** Interface to allow external systems to trace additional object references, used for bridging GCs */
class FGarbageCollectionTracer
{
public:
	virtual ~FGarbageCollectionTracer() = default;
	virtual void PerformReachabilityAnalysisOnObjects(UE::GC::FWorkerContext* Context, EGCOptions Options) = 0;
};

//////////////////////////////////////////////////////////////////////////

namespace UE::GC
{

#if !WITH_VERSE_BPVM || defined(__INTELLISENSE__)
// Avoid name lookup into UE::Verse, sometimes pulled into the TU before this header.
namespace Verse = ::Verse;
#endif

struct FStructArrayBlock;

static constexpr uint32 ObjectLookahead = 16;

// Prefetches ClassPrivate, OuterPrivate, class schema and schema data while iterating over an object array
//
// Tuned on a Gen5 console using an internal game replay and an in-game GC pass
class FPrefetchingObjectIterator
{
public:
	// Objects must be padded with PadObjects
	explicit FPrefetchingObjectIterator(TConstArrayView<UObject*> Objects)
	: It(Objects.begin())
	, End(Objects.end())
	, PrefetchedSchema(Objects.Num() ? &It[1]->GetClass()->ReferenceSchema.Get() : nullptr)
	{}

	FORCEINLINE_DEBUGGABLE void Advance()
	{	
		FPlatformMisc::Prefetch(PrefetchedSchema->GetWords());
		PrefetchedSchema = &It[2]->GetClass()->ReferenceSchema.Get();

		UObjectBase::PrefetchOuter(It[6]);
		FPlatformMisc::Prefetch(It[6]->GetClass(), offsetof(UClass, ReferenceSchema));
		UObjectBase::PrefetchClass(It[ObjectLookahead]);

		++It;
	}
	
	bool HasMore() const { return It != End; }
	UObject* GetCurrentObject() { return *It; }

private:
	UObject*const* It;
	UObject*const* End;
	const FSchemaView* PrefetchedSchema;
};

// Pad object array for FPrefetchingObjectIterator use
COREUOBJECT_API void PadObjectArray(TArray<UObject*>& Objects);

//////////////////////////////////////////////////////////////////////////

/** Fixed block of reachable objects waiting to be processed */
struct FWorkBlock
{
	static constexpr uint32 ObjectCapacity = 512 - /* Previous */ 1 - ObjectLookahead;

	FWorkBlock* Previous;
	UObject* Objects[ObjectCapacity + ObjectLookahead];

	TArrayView<UObject*> GetObjects() { return MakeArrayView(Objects, ObjectCapacity); }
	TArrayView<UObject*> GetPadding() { return MakeArrayView(Objects + ObjectCapacity, ObjectLookahead); }
};

class FWorkstealingQueue;

/** Reachable objects waiting to be processed. Type-erases parallel/serial queue. */
class FWorkBlockifier
{
public:
	UE_NONCOPYABLE(FWorkBlockifier);
	FWorkBlockifier() = default;
	COREUOBJECT_API ~FWorkBlockifier();
	
	void Init() { AllocateWipBlock(); }
	void SetAsyncQueue(FWorkstealingQueue& Queue) { AsyncQueue = &Queue; }
	void ResetAsyncQueue();

	template<EGCOptions Options>
	FORCEINLINE_DEBUGGABLE void Add(UObject* Object)
	{
		*WipIt = Object;
		if (++WipIt == Wip->GetPadding().GetData())
		{
			if constexpr (IsParallel(Options))
			{
				PushFullBlockAsync();
			}
			else
			{
				PushFullBlockSync();
			}
		}
	}

	FORCEINLINE_DEBUGGABLE FWorkBlock* PopPartialBlock(int32& OutNum)
	{
		if (int32 Num = PartialNum())
		{
			OutNum = Num;
			return PopWipBlock();
		}

		return nullptr;
	}

	template<EGCOptions Options>
	FORCEINLINE FWorkBlock* PopFullBlock()
	{
		return IsParallel(Options) ? PopFullBlockAsync() : PopFullBlockSync();
	}

	FORCEINLINE FWorkBlock* StealFullBlock() const
	{
		return StealAsyncBlock();
	}

	COREUOBJECT_API void FreeOwningBlock(UObject*const* BlockObjects);

	FORCEINLINE bool IsUnused() const
	{
		return PartialNum() == 0 && SyncQueue == nullptr;
	}

	void SetWorkerIndex(int32 Idx) { WorkerIndex = Idx; }
	int32 GetWorkerIndex() const { return WorkerIndex; }

	bool HasWork() const
	{
		return PartialNum() != 0;
	}

private:
	UObject** WipIt; // Wip->Objects cursor
	FWorkBlock* Wip;
	union
	{
		FWorkBlock* SyncQueue = nullptr;
		FWorkstealingQueue* AsyncQueue;
	};
	int32 WorkerIndex = INDEX_NONE;
	
	void AllocateWipBlock();
	COREUOBJECT_API void PushFullBlockSync();
	COREUOBJECT_API void PushFullBlockAsync();
	COREUOBJECT_API FWorkBlock* PopFullBlockSync();
	COREUOBJECT_API FWorkBlock* PopFullBlockAsync();
	COREUOBJECT_API FWorkBlock* PopWipBlock();
	COREUOBJECT_API FWorkBlock* StealAsyncBlock() const;

	FORCEINLINE int32 PartialNum() const 
	{
		return static_cast<int32>(WipIt - Wip->Objects);
	}
};

//////////////////////////////////////////////////////////////////////////

struct FSlowARO
{
	COREUOBJECT_API static void CallSync(uint32 SlowAROIndex, UObject* Object, FReferenceCollector& Collector);
	COREUOBJECT_API static bool TryQueueCall(uint32 SlowAROIndex, UObject* Object, FWorkerContext& Context);
	COREUOBJECT_API static void ProcessUnbalancedCalls(FWorkerContext& Context, FReferenceCollector& Collector);
	// @return if any calls were made
	COREUOBJECT_API static bool ProcessAllCalls(FWorkerContext& Context, FReferenceCollector& Collector);
};

//////////////////////////////////////////////////////////////////////////

struct FProcessorStats
{
#if UE_BUILD_SHIPPING
	static constexpr uint32 NumObjects = 0;
	static constexpr uint32 NumReferences = 0;
	static constexpr uint32 NumVerseCells = 0;
	static constexpr bool bFoundGarbageRef = false;
	FORCEINLINE constexpr void AddObjects(uint32) {}
	FORCEINLINE constexpr void AddReferences(uint32) {}
	FORCEINLINE constexpr void AddVerseCells(uint32) {}
	FORCEINLINE constexpr void TrackPotentialGarbageReference(bool) {}
#else
	uint32 NumObjects = 0;
	uint32 NumReferences = 0;
	uint32 NumVerseCells = 0;
	bool bFoundGarbageRef = false;
	FORCEINLINE void AddObjects(uint32 Num) { NumObjects += Num; }
	FORCEINLINE void AddReferences(uint32 Num) { NumReferences += Num; }
	FORCEINLINE void AddVerseCells(uint32 Num) { NumVerseCells += Num; }
	FORCEINLINE void TrackPotentialGarbageReference(bool bDetectedGarbage) { bFoundGarbageRef |= bDetectedGarbage; }
#endif

	void AddStats(FProcessorStats Stats)
	{
		AddObjects(Stats.NumObjects);
		AddReferences(Stats.NumReferences);
		AddVerseCells(Stats.NumVerseCells);
		TrackPotentialGarbageReference(Stats.bFoundGarbageRef);
	}
};

struct FStructArray
{
	FSchemaView Schema{ NoInit };
	uint8* Data;
	int32 Num;
	uint32 Stride;
};

struct FSuspendedStructBatch
{
	FStructArrayBlock* Wip = nullptr;
	FStructArray* WipIt = nullptr;

	FORCEINLINE bool ContainsBatchData() const
	{
		return !!Wip;
	}
};

struct FWeakReferenceInfo
{
	UObject* ReferencedObject = nullptr;
	UObject** Reference = nullptr;
	UObject* ReferenceOwner = nullptr;
};

/** Maintains a stack of schemas currently processed by reachability analysis for debugging referencing property names */
struct FDebugSchemaStackNode
{
#if !UE_BUILD_SHIPPING
	FMemberId Member;
	FSchemaView Schema;
	FDebugSchemaStackNode* Prev;
	
	FDebugSchemaStackNode()
		: Member(0)
		, Prev(nullptr)
	{
	}
	FDebugSchemaStackNode(FSchemaView InSchema, FDebugSchemaStackNode* PrevNode)
		: Member(0)
		, Schema(InSchema)
		, Prev(PrevNode)
	{
	}
#endif // !UE_BUILD_SHIPPING

	FORCEINLINE void SetMemberId(FMemberId MemberId)
	{
#if !UE_BUILD_SHIPPING
		Member = MemberId;
#endif
	}

	COREUOBJECT_API FString ToString() const;
};

/** Thread-local context containing initial objects and references to collect */
struct alignas(PLATFORM_CACHE_LINE_SIZE) FWorkerContext
{
private:
	template <typename ProcessorType, typename CollectorType>
	friend class TFastReferenceCollector;
	friend class FSlowAROManager;
	
	// This is set by GC when processing references from the current referencing object
	UObject* ReferencingObject = nullptr;
	TConstArrayView<UObject*> InitialObjects;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	TConstArrayView<Verse::VNativeStruct*> InitialNativeStructs;
#endif
public:
	UE_NONCOPYABLE(FWorkerContext);
	COREUOBJECT_API FWorkerContext();
	COREUOBJECT_API ~FWorkerContext();

	FWorkBlockifier ObjectsToSerialize;
	TConstArrayView<UObject**> InitialNativeReferences;
	FWorkCoordinator* Coordinator = nullptr;
	TArray<FWeakReferenceInfo> WeakReferences;
	FProcessorStats Stats;

#if !UE_BUILD_SHIPPING
	TArray<FGarbageReferenceInfo> GarbageReferences;
#endif
#if ENABLE_GC_HISTORY
	TMap<FReferenceToken, TArray<FGCDirectReference>*> History;
#endif

	FSuspendedStructBatch IncrementalStructs;
	bool bIsSuspended = false;
	bool bDidWork = false;

	FDebugSchemaStackNode* SchemaStack = nullptr;

	FORCEINLINE UObject* GetReferencingObject()	{ return ReferencingObject;	}

	TConstArrayView<UObject*> GetInitialObjects() { return InitialObjects; }
	void ResetInitialObjects() { InitialObjects = {}; }

	/** @param Objects must outlive this context. It's data is padded by repeating the last object to allow prefetching past the end. */
	void SetInitialObjectsUnpadded(TArray<UObject*>& Objects)
	{
		PadObjectArray(Objects);
		SetInitialObjectsPrepadded(Objects);
	}

	/** @param PaddedObjects must already be padded to allow reading valid objects past the end */
	void SetInitialObjectsPrepadded(TConstArrayView<UObject*> PaddedObjects)
	{
		check(PaddedObjects.IsEmpty() || PaddedObjects.GetData()[PaddedObjects.Num() + ObjectLookahead - 1]->IsValidLowLevel() );
		InitialObjects = PaddedObjects;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	TConstArrayView<Verse::VNativeStruct*> GetInitialNativeStructs() { return InitialNativeStructs; }	

	void ResetInitialNativeStructs() { InitialNativeStructs = {}; }

	void SetInitialNativeStructs(TConstArrayView<Verse::VNativeStruct*> NativeStructs)
	{
		InitialNativeStructs = NativeStructs;
	}
#endif

	/** Returns the size of memory allocated by internal arrays */
	int64 GetAllocatedSize() const
	{
		return WeakReferences.GetAllocatedSize() + sizeof(FWorkBlock);
	}
	
	FORCEINLINE int32 GetWorkerIndex() const { return ObjectsToSerialize.GetWorkerIndex(); }
	void AllocateWorkerIndex();
	void FreeWorkerIndex();
};

//////////////////////////////////////////////////////////////////////////

struct FDebugSchemaStackScope
{
#if !UE_BUILD_SHIPPING
	FWorkerContext& Context;
	FDebugSchemaStackNode Node;
#endif

	FDebugSchemaStackScope(FWorkerContext& InContext, FSchemaView Schema)
#if !UE_BUILD_SHIPPING
		: Context(InContext)
		, Node(Schema, InContext.SchemaStack)
#endif
	{
#if !UE_BUILD_SHIPPING
		InContext.SchemaStack = &Node;
#endif
	}
	~FDebugSchemaStackScope()
	{
#if !UE_BUILD_SHIPPING
		Context.SchemaStack = Node.Prev;
#endif
	}
};

struct FDebugSchemaStackNoOpScope
{
	FDebugSchemaStackNoOpScope(FWorkerContext& InContext, FSchemaView Schema)
	{
	}
};

//////////////////////////////////////////////////////////////////////////

namespace Private {

struct FMemberUnpacked
{
	FMemberUnpacked(FMemberPacked In) 
	: Type(static_cast<EMemberType>(In.Type))
	, WordOffset(In.WordOffset)
	{
		check(Type < EMemberType::Count);
	}
	EMemberType Type;
	uint32 WordOffset;
};

struct FMemberWordUnpacked
{
	FMemberWordUnpacked(const FMemberPacked In[4]) : Members{In[0], In[1], In[2], In[3]} {}
	FMemberUnpacked Members[4];
};

struct FStridedReferenceArray
{
	FScriptArray* Array;
	FStridedLayout Layout;
};

struct FStridedReferenceView
{
	UObject** Data;
	int32 Num;
	uint32 Stride;
};

struct FStridedReferenceIterator
{
	UObject** It;
	uint32 Stride;

	UObject*& operator*() { return *It; }
	FStridedReferenceIterator& operator++() { It += Stride; return *this;}
	bool operator!=(FStridedReferenceIterator Rhs) const { return It != Rhs.It; }
};
	
FORCEINLINE	FStridedReferenceIterator begin(FStridedReferenceView View) { return { View.Data, View.Stride }; }
FORCEINLINE	FStridedReferenceIterator end(FStridedReferenceView View) { return { View.Data + View.Stride * View.Num, View.Stride }; }
FORCEINLINE int32 GetNum(FStridedReferenceView View)  { return View.Num; }
FORCEINLINE FStridedReferenceView ToView(FStridedReferenceArray In) 
{ 
	return { reinterpret_cast<UObject**>(In.Array->GetData()) + In.Layout.WordOffset,  In.Array->Num(), In.Layout.WordStride };
}

FORCEINLINE uint8* GetSetData(FScriptSet& Set)
{
	return reinterpret_cast<uint8*>(Set.GetData(0, FScriptSetLayout{})); 
}

FORCEINLINE const uint8* GetSetData(const FScriptSet& Set)
{
	return reinterpret_cast<const uint8*>(Set.GetData(0, FScriptSetLayout{})); 
}

template<class DispatcherType>
FORCENOINLINE void VisitNestedStructMembers(DispatcherType& Dispatcher, FSchemaView Schema, uint8* Instance);

template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void VisitStructs(DispatcherType& Dispatcher, FSchemaView StructSchema, uint8* It, const int32 Num)
{
	check(!StructSchema.IsEmpty());
	if constexpr (DispatcherType::bBatching)
	{
		Dispatcher.QueueStructArray(StructSchema, It, Num);
	}
	else
	{
		uint32 Stride = StructSchema.GetStructStride();
		for (uint8* End = It + Num*Stride; It != End; It += Stride)
		{
			VisitNestedStructMembers(Dispatcher, StructSchema, It);
		}
	}
}

template<class DispatcherType, class ArrayType>
FORCEINLINE_DEBUGGABLE void VisitStructArray(DispatcherType& Dispatcher, FSchemaView StructSchema, ArrayType& Array)
{
	typename DispatcherType::SchemaStackScopeType SchemaStack(Dispatcher.Context, StructSchema);
	VisitStructs(Dispatcher, StructSchema, (uint8*)Array.GetData(), Array.Num());
}

template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void VisitStructSet(DispatcherType& Dispatcher, FSchemaView StructSchema, FScriptSet& Set)
{
	check(!StructSchema.IsEmpty());
	if constexpr (DispatcherType::bBatching)
	{
		Dispatcher.QueueSparseStructArray(StructSchema, Set);
	}
	else if (int32 Num = Set.Num())
	{
		uint8* It = GetSetData(Set);
		const uint32 Stride = StructSchema.GetStructStride();
		for (int32 Idx = 0, MaxIdx = Set.GetMaxIndex(); Idx < MaxIdx; ++Idx, It += Stride)
		{
			if (Set.IsValidIndex(Idx))
			{
				typename DispatcherType::SchemaStackScopeType SchemaStack(Dispatcher.Context, StructSchema);
				VisitNestedStructMembers(Dispatcher, StructSchema, It);
			}
		}
	}
}

template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void VisitFieldPath(DispatcherType& Dispatcher, FFieldPath& FieldPath, EOrigin Origin, uint32 MemberIdx)
{
	if (FUObjectItem* FieldOwnerItem = FGCInternals::GetResolvedOwner(FieldPath))
	{
		UObject* OwnerObject = static_cast<UObject*>(FieldOwnerItem->GetObject());
		UObject* PreviousOwner = OwnerObject;
		Dispatcher.HandleReferenceDirectly(Dispatcher.Context.GetReferencingObject(), OwnerObject, FMemberId(MemberIdx), Origin, true);
								
		// Handle reference elimination (PendingKill owner)
		if (PreviousOwner && !OwnerObject)
		{
			FGCInternals::ClearCachedField(FieldPath);
		}
	}
}
template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void VisitFieldPathArray(DispatcherType& Dispatcher, TArray<FFieldPath>& FieldPaths, EOrigin Origin, uint32 MemberIdx)
{
	for (FFieldPath& FieldPath : FieldPaths)
	{
		VisitFieldPath(Dispatcher, FieldPath, Origin, MemberIdx);
	}
}

template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void VisitOptional(DispatcherType& Dispatcher, FSchemaView StructSchema, uint8* Instance)
{
	check(!StructSchema.IsEmpty());
	uint32 ValueSize = StructSchema.GetStructStride();
	bool bIsSet = *(bool*)(Instance + ValueSize);
	typename DispatcherType::SchemaStackScopeType SchemaStack(Dispatcher.Context, StructSchema);
	VisitStructs(Dispatcher, StructSchema, Instance, bIsSet);
}

template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void VisitDynamicallyTypedValue(DispatcherType& Dispatcher, UE::FDynamicallyTypedValue& Value)
{
	Value.GetType().MarkReachable(Dispatcher.Collector);
	if (Value.GetType().GetContainsReferences() != UE::FDynamicallyTypedValueType::EContainsReferences::DoesNot)
	{
		Value.GetType().MarkValueReachable(Value.GetDataPointer(), Dispatcher.Collector);
	}
}

#if WITH_EDITORONLY_DATA
COREUOBJECT_API bool& CalledSuperARO();
#endif

template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void CallARO(DispatcherType& Dispatcher, UObject* Instance, FMemberWord Word)
{
	if (Instance->HasAnyInternalFlags(EInternalObjectFlags::AutoRTFMConstructionAborted))
	{
		// Object was constructed in a transaction that was aborted.
		// The object is in a pre-construction state and won't have any valid subreferences.
		return;
	}

#if WITH_EDITORONLY_DATA
	bool& bCalledSuperARO = CalledSuperARO();
	TGuardValue<bool> CalledSuperAROScope(bCalledSuperARO, false);
	Word.ObjectARO(Instance, Dispatcher.Collector);
	if (!bCalledSuperARO)
	{
		UE_LOGF(LogGarbage, Warning, "Class %ls or a super class did not call Super::AddReferencedObjects", *Instance->GetClass()->GetName());
	}
#else
	Word.ObjectARO(Instance, Dispatcher.Collector);
#endif
}

template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void CallARO(DispatcherType& Dispatcher, uint8* Instance, FMemberWord Word)
{
	Word.StructARO(Instance, Dispatcher.Collector);
}

template<class DispatcherType>
FORCEINLINE_DEBUGGABLE void CallSlowARO(DispatcherType& Dispatcher, uint32 SlowAROIdx, UObject* Instance, uint32 MemberIdx)
{
	if constexpr (DispatcherType::bBatching && DispatcherType::bParallel)
	{
		if (!FSlowARO::TryQueueCall(SlowAROIdx, Instance, Dispatcher.Context))
		{
			FSlowARO::CallSync(SlowAROIdx, Instance, Dispatcher.Collector);
		}
	}
	else
	{
		FSlowARO::CallSync(SlowAROIdx, Instance, Dispatcher.Collector);
	}
}

FORCENOINLINE static void LogIllegalTypeFatal(EMemberType Type, uint32 Idx, UObject* Instance)
{
	UE_LOGF(LogGarbage, Fatal, "Illegal GC object member type %d at %d, class:%ls object:%ls", int(Type), Idx, Instance ? *GetNameSafe(Instance->GetClass()) : TEXT("Unknown"), *GetPathNameSafe(Instance));
}

FORCENOINLINE static void LogIllegalTypeFatal(EMemberType Type, uint32 Idx, uint8*)
{
	UE_LOGF(LogGarbage, Fatal, "Illegal GC struct member type %d at %d", int(Type), Idx);
}

template<class DispatcherType>
FORCEINLINE void CallSlowARO(DispatcherType&, uint32 SlowAROIdx, uint8* Instance, uint32 MemberIdx)
{
	LogIllegalTypeFatal(EMemberType::SlowARO, MemberIdx, Instance);
}

template <typename ObjectType>
FORCEINLINE EOrigin GetSchemaOrigin(FSchemaView Schema, ObjectType* Instance)
{
	return Schema.GetOrigin();
}
#if UE_WITH_REMOTE_OBJECT_HANDLE
template <>
FORCEINLINE EOrigin GetSchemaOrigin(FSchemaView Schema, UObject* Instance)
{
	FUObjectItem* Item = GUObjectArray.ObjectToObjectItem(Instance);
	// Consider borrowed objects to always be non-BP to prevent garbage elimination from mutating these objects
	return !Item->HasAnyFlags(EInternalObjectFlags::Borrowed) ? Schema.GetOrigin() : EOrigin::Other;
}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

template<class DispatcherType, typename ObjectType>
AUTORTFM_INFER FORCEINLINE_DEBUGGABLE void VisitMembers(DispatcherType& Dispatcher, FSchemaView Schema, ObjectType* Instance, int32 OffsetToPropertiesStart)
{
	check(!Schema.IsEmpty());

	const EOrigin Origin = GetSchemaOrigin(Schema, Instance);
	uint8* InstanceBegin = (uint8*)Instance + OffsetToPropertiesStart;
	uint64* InstanceCursor = (uint64*)InstanceBegin;	// Advanced via Jump to reach far members
	uint32 DebugIdx = 0;
	for (const FMemberWord* WordIt = Schema.GetWords(); true; ++WordIt)
	{
		const FMemberWordUnpacked Quad(WordIt->Members);
		for (FMemberUnpacked Member : Quad.Members)
		{
			uint8* MemberPtr = (uint8*)(InstanceCursor + Member.WordOffset);
			Dispatcher.SetDebugSchemaStackMemberId(FMemberId(DebugIdx));

			switch (Member.Type)
			{
			case EMemberType::Reference:				Dispatcher.HandleKillableReference(*(UObject**)MemberPtr, FMemberId(DebugIdx), Origin);
			break;
			case EMemberType::ReferenceArray:			Dispatcher.HandleKillableArray(*(TArray<UObject*>*)MemberPtr, FMemberId(DebugIdx), Origin);
			break;
			case EMemberType::StridedArray:				Dispatcher.HandleKillableArray(FStridedReferenceArray{(FScriptArray*)MemberPtr, (++WordIt)->StridedLayout}, FMemberId(DebugIdx), Origin);
			break;	
			case EMemberType::FreezableReferenceArray:	Dispatcher.HandleKillableReferences(*(TArray<UObject*, FMemoryImageAllocator>*)MemberPtr, FMemberId(DebugIdx), Origin);
			break;
			case EMemberType::StructArray:				VisitStructArray(			Dispatcher, FSchemaView((++WordIt)->InnerSchema, Origin), *(FScriptArray*)MemberPtr);
			break;
			case EMemberType::StructSet:				VisitStructSet(				Dispatcher, FSchemaView((++WordIt)->InnerSchema, Origin), *(FScriptSet*)MemberPtr);
			break;
			case EMemberType::FreezableStructArray:		VisitStructArray(			Dispatcher, FSchemaView((++WordIt)->InnerSchema, Origin), *(FFreezableScriptArray*)MemberPtr);
			break;
			case EMemberType::Optional:					VisitOptional(				Dispatcher, FSchemaView((++WordIt)->InnerSchema, Origin), MemberPtr);
			break;
			case EMemberType::FieldPath:				VisitFieldPath(				Dispatcher, *(FFieldPath*)MemberPtr, Origin, DebugIdx);
			break;
			case EMemberType::FieldPathArray:			VisitFieldPathArray(		Dispatcher, *(TArray<FFieldPath>*)MemberPtr, Origin, DebugIdx);
			break;
			case EMemberType::DynamicallyTypedValue:	VisitDynamicallyTypedValue(	Dispatcher, *(UE::FDynamicallyTypedValue*)MemberPtr);
			break;
			case EMemberType::Jump:						InstanceCursor += (Member.WordOffset + 1) * FMemberPacked::OffsetRange;
			break;
			case EMemberType::MemberARO:				CallARO(Dispatcher, MemberPtr, *++WordIt);
			break; // Struct member ARO isn't an implicit stop
			case EMemberType::ARO:						CallARO(Dispatcher, Instance, *++WordIt);
			return; // Instance ARO is an implicit stop
			case EMemberType::SlowARO:					CallSlowARO(Dispatcher, /* slow ARO index */ Member.WordOffset, Instance, DebugIdx);
			return; // ARO is an implicit stop
			case EMemberType::Stop:
			return; // Stop schema without ARO call
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
			case EMemberType::VerseValue:				Dispatcher.HandleVerseValue(*(Verse::VValue*)MemberPtr, FMemberId(DebugIdx), Origin);
			break;
			case EMemberType::VerseValueArray:			Dispatcher.HandleVerseValueArray(*(TArray<Verse::VValue>*)MemberPtr, FMemberId(DebugIdx), Origin);
			break;
#endif
			case EMemberType::Count:
			default:									LogIllegalTypeFatal(Member.Type, DebugIdx, Instance);
			return;
			}

			DebugIdx += UE_GC_DEBUGNAMES;
		} // for quad members
	} // for schema member words
}

template<class DispatcherType>
void VisitNestedStructMembers(DispatcherType& Dispatcher, FSchemaView Schema, uint8* Instance)
{
	static_assert(!DispatcherType::bBatching);
	VisitMembers(Dispatcher, Schema, Instance, 0);
}

} // namespace Private

//////////////////////////////////////////////////////////////////////////

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
// Some helper templates to detect if the ProcessorType supports HasHandleTokenStreamVerseCellReference
template <typename T, typename = void>
struct HasHandleTokenStreamVerseCellReference : std::false_type {};

template <typename T>
using HandleTokenStreamVerseCellReference_t = decltype(std::declval<T>().HandleTokenStreamVerseCellReference(std::declval<FWorkerContext&>(), std::declval<UObject*>(), std::declval<Verse::VCell*>(), std::declval<FMemberId>(), std::declval<EOrigin>()));

template <typename T>
struct HasHandleTokenStreamVerseCellReference <T, std::void_t<HandleTokenStreamVerseCellReference_t<T>>> : std::true_type {};
#endif

/** Forwards references directly to ProcessorType::HandleTokenStreamObjectReference(), unlike TBatchDispatcher */
template<class ProcessorType>
struct AUTORTFM_INFER TDirectDispatcher
{
	static constexpr bool bBatching = false;
	static constexpr bool bParallel = IsParallel(ProcessorType::Options);

	typedef FDebugSchemaStackScope SchemaStackScopeType;
	ProcessorType& Processor;
	FWorkerContext& Context;
	FReferenceCollector& Collector;

	FORCEINLINE void HandleReferenceDirectly(UObject* ReferencingObject, UObject*& Object, FMemberId MemberId, EOrigin Origin, bool bAllowReferenceElimination) const
	{
		if (IsObjectHandleResolved_ForGC(*reinterpret_cast<FObjectHandle*>(&Object)))
		{
			Processor.HandleTokenStreamObjectReference(Context, ReferencingObject, Object, MemberId, Origin, bAllowReferenceElimination);
		}
		Context.Stats.AddReferences(1);
	}
	
	FORCEINLINE void HandleKillableReference(UObject*& Object, FMemberId MemberId, EOrigin Origin) const
	{
		HandleReferenceDirectly(Context.GetReferencingObject(), Object, MemberId, Origin, true);
	}

	FORCEINLINE void HandleImmutableReference(UObject* Object, FMemberId MemberId, EOrigin Origin) const
	{
		HandleReferenceDirectly(Context.GetReferencingObject(), Object, MemberId, Origin, false);
	}
	
	template<class ArrayType>
	FORCEINLINE void HandleKillableReferences(ArrayType&& Objects , FMemberId MemberId, EOrigin Origin) const
	{
		for (UObject*& Object : Objects)
		{
			HandleReferenceDirectly(Context.GetReferencingObject(), Object, MemberId, Origin, true);
		}
	}

	FORCEINLINE void HandleKillableArray(TArray<UObject*>& Array, FMemberId MemberId, EOrigin Origin) const
	{
		HandleKillableReferences(Array, MemberId, Origin);
	}

	FORCEINLINE void HandleKillableArray(Private::FStridedReferenceArray Array, FMemberId MemberId, EOrigin Origin) const
	{
		HandleKillableReferences(ToView(Array), MemberId, Origin);
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	FORCEINLINE_DEBUGGABLE void HandleVerseCellDirectly(UObject* ReferencingObject, Verse::VCell* Cell, FMemberId MemberId, EOrigin Origin) const
	{
		if constexpr (HasHandleTokenStreamVerseCellReference<ProcessorType>::value)
		{
			Processor.HandleTokenStreamVerseCellReference(Context, ReferencingObject, Cell, MemberId, Origin);
		}
		Context.Stats.AddVerseCells(1);
	}

	FORCEINLINE_DEBUGGABLE void HandleVerseValueDirectly(UObject* ReferencingObject, Verse::VValue Value, FMemberId MemberId, EOrigin Origin) const
	{
		if (Verse::VCell* Cell = Value.ExtractCell())
		{
			HandleVerseCellDirectly(ReferencingObject, Cell, MemberId, Origin);
		}
		else if (UObject* Object = Value.ExtractUObject())
		{
			HandleImmutableReference(Object, MemberId, Origin);
		}
	}

	FORCEINLINE_DEBUGGABLE void HandleVerseValue(Verse::VValue Value, FMemberId MemberId, EOrigin Origin)
	{
		HandleVerseValueDirectly(Context.GetReferencingObject(), Value, MemberId, Origin);
	}

	FORCEINLINE void HandleVerseValueArray(TArrayView<Verse::VValue> Values, FMemberId MemberId, EOrigin Origin)
	{
		for (Verse::VValue Value : Values)
		{
			HandleVerseValueDirectly(Context.GetReferencingObject(), Value, MemberId, Origin);
		}
	}
#endif

	void Suspend()
	{
	}

	void SetDebugSchemaStackMemberId(FMemberId Member)
	{
		Context.SchemaStack->SetMemberId(Member);
	}
};

// Default implementation is to create new direct dispatcher
template<class CollectorType, class ProcessorType>
TDirectDispatcher<ProcessorType> GetDispatcher(CollectorType& Collector, ProcessorType& Processor, FWorkerContext& Context)
{
	return { Processor, Context, Collector };
}

template<class CollectorType, class ProcessorType, class = void >
struct TGetDispatcherType
{
	using RetType = decltype(GetDispatcher(*(CollectorType*)nullptr, *(ProcessorType*)nullptr, *(FWorkerContext*)nullptr));
	using Type = typename std::remove_reference_t<RetType>;
};

//////////////////////////////////////////////////////////////////////////

enum class ELoot { Nothing, Block, ARO, Context };
AUTORTFM_DISABLE COREUOBJECT_API ELoot StealWork(FWorkerContext& Context, FReferenceCollector& Collector, FWorkBlock*& OutBlock, EGCOptions Options);

COREUOBJECT_API void SuspendWork(FWorkerContext& Context);

/** Allocates contexts and coordinator, kicks worker tasks that also call ProcessSync. Processor is type-erased to void* to avoid templated code. */
AUTORTFM_DISABLE COREUOBJECT_API void ProcessAsync(void (*ProcessSync)(void*, FWorkerContext&), void* Processor, FWorkerContext& InitialContext);

//////////////////////////////////////////////////////////////////////////

/**
 * Helper class that looks for UObject references by traversing UClass token stream and calls AddReferencedObjects.
 * Provides a generic way of processing references that is used by Unreal Engine garbage collection.
 * Can be used for fast (does not use serialization) reference collection purposes.
 * 
 * IT IS CRITICAL THIS CLASS DOES NOT CHANGE WITHOUT CONSIDERING PERFORMANCE IMPACT OF SAID CHANGES
 *
 * @see FSimpleReferenceProcessorBase and TDefaultCollector for documentation on required APIs
 */
template <typename ProcessorType, typename CollectorType>
class TFastReferenceCollector : public FGCInternals
{
public:
	TFastReferenceCollector(ProcessorType& InProcessor) : Processor(InProcessor) {}

	void ProcessObjectArray(FWorkerContext& Context)
	{
		Context.bDidWork = true;
		Context.bIsSuspended = false;
		static_assert(!EnumHasAllFlags(Options, EGCOptions::Parallel | EGCOptions::AutogenerateSchemas), "Can't assemble token streams in parallel");
		
		CollectorType Collector(Processor, Context);

		// Either TDirectDispatcher living on the stack or TBatchDispatcher reference owned by Collector
		decltype(GetDispatcher(Collector, Processor, Context)) Dispatcher = GetDispatcher(Collector, Processor, Context);

StoleContext:
		// Process initial references first
		Context.ReferencingObject = FGCObject::GGCObjectReferencer;
		for (UObject** InitialReference : Context.InitialNativeReferences)
		{
			Dispatcher.HandleKillableReference(*InitialReference, EMemberlessId::InitialReference, EOrigin::Other);
		}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
		// Just process native structs once
		ProcessNativeStructs(Dispatcher, Context.InitialNativeStructs);
#endif

		TConstArrayView<UObject*> CurrentObjects = Context.InitialObjects;
		while (true)
		{
			Context.Stats.AddObjects(CurrentObjects.Num());
			ProcessObjects(Dispatcher, CurrentObjects);

			// Free finished work block
			if (CurrentObjects.GetData() != Context.InitialObjects.GetData())
			{
				Context.ObjectsToSerialize.FreeOwningBlock(CurrentObjects.GetData());
			}

			if (Processor.IsTimeLimitExceeded())
			{
				FlushWork(Dispatcher);
				Dispatcher.Suspend();
				SuspendWork(Context);
				return;
			}

			int32 BlockSize = FWorkBlock::ObjectCapacity;
			FWorkBlockifier& RemainingObjects = Context.ObjectsToSerialize;
			FWorkBlock* Block = RemainingObjects.PopFullBlock<Options>();
			if (!Block)
			{
				if constexpr (bIsParallel)
				{
					FSlowARO::ProcessUnbalancedCalls(Context, Collector);
				}

StoleARO:
				FlushWork(Dispatcher);

				if (	 Block = RemainingObjects.PopFullBlock<Options>(); Block);
				else if (Block = RemainingObjects.PopPartialBlock(/* out if successful */ BlockSize); Block);
				else if (bIsParallel) // if constexpr yields MSVC unreferenced label warning
				{
					switch (StealWork(/* in-out */ Context, Collector, /* out */ Block, Options))
					{
						case ELoot::Nothing:	break;				// Done, stop working
						case ELoot::Block:		break;				// Stole full block, process it
						case ELoot::ARO:		goto StoleARO;		// Stole and made ARO calls that feed into Dispatcher queues and RemainingObjects
						case ELoot::Context:	goto StoleContext;	// Stole initial references and initial objects worker that hasn't started working
					}
				}

				if (!Block)
				{
					break;
				}
			}

			CurrentObjects = MakeArrayView(Block->Objects, BlockSize);
		} // while (true)
		
		Processor.LogDetailedStatsSummary();
	}

private:
	using DispatcherType = typename TGetDispatcherType<CollectorType, ProcessorType>::Type;
	static constexpr EGCOptions Options = ProcessorType::Options;
	static constexpr bool bIsParallel = IsParallel(Options);
	
	ProcessorType& Processor;

	FORCEINLINE_DEBUGGABLE void ProcessObjects(DispatcherType& Dispatcher, TConstArrayView<UObject*> CurrentObjects)
	{
		for (FPrefetchingObjectIterator It(CurrentObjects); It.HasMore(); It.Advance())
		{
			UObject* CurrentObject = It.GetCurrentObject();
			UClass* Class = CurrentObject->GetClass();
			UObject* Outer = CurrentObject->GetOuter();

			if (!!(Options & EGCOptions::AutogenerateSchemas) && !Class->HasAnyClassFlags(CLASS_TokenStreamAssembled))
			{			
				Class->AssembleReferenceTokenStream();
			}

			checkSlow(Class->HasAllClassFlags(CLASS_TokenStreamAssembled));

			FSchemaView Schema = Class->ReferenceSchema.Get();
			Dispatcher.Context.ReferencingObject = CurrentObject;

			// Emit base references
			Dispatcher.HandleImmutableReference(Class, EMemberlessId::Class, EOrigin::Other);
			Dispatcher.HandleImmutableReference(Outer, EMemberlessId::Outer, EOrigin::Other);

			UObject* Package = CurrentObject->GetExternalPackageInternal();
			Package = Package != CurrentObject ? Package : nullptr;
			Dispatcher.HandleImmutableReference(Package, EMemberlessId::ExternalPackage, EOrigin::Other);

			if (!Schema.IsEmpty())
			{
				int32 OffsetToProperty = Class->GetPropertiesStartOffset();
				typename DispatcherType::SchemaStackScopeType SchemaStack(Dispatcher.Context, Schema);
				Processor.BeginTimingObject(CurrentObject);
				Private::VisitMembers(Dispatcher, Schema, CurrentObject, OffsetToProperty);
				Processor.UpdateDetailedStats(CurrentObject);
			}
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	FORCEINLINE_DEBUGGABLE void ProcessNativeStructs(DispatcherType& Dispatcher, TConstArrayView<Verse::VNativeStruct*> CurrentNativeStructs)
	{
		// Not doing any prefetching here, as expected number of native structs is currently small
		for (Verse::VNativeStruct* NativeStruct : CurrentNativeStructs)
		{
			Verse::VEmergentType* EmergentType = NativeStruct->GetEmergentType();
			UVerseStruct* VerseStruct = CastChecked<UVerseStruct>(Verse::VNativeStruct::GetUScriptStruct(*EmergentType));
			FSchemaView Schema = VerseStruct->ReferenceSchema.Get();
			check(!Schema.IsEmpty()); // Native structs with empty schemas should not be in the list to begin with
			typename DispatcherType::SchemaStackScopeType SchemaStack(Dispatcher.Context, Schema);
			Private::VisitMembers(Dispatcher, Schema, (uint8*)NativeStruct->GetStruct(), 0);
		}
	}
#endif

	// Some helper templates to detect if the DispatcherType supports FlushWork
	template <typename T, typename = void>
	struct HasFlushWork : std::false_type {};

	template <typename T>
	struct HasFlushWork <T, std::void_t<decltype(std::declval<T>().FlushWork())>> : std::true_type {};

	FORCEINLINE_DEBUGGABLE void FlushWork(DispatcherType& Dispatcher)
	{
		if constexpr (DispatcherType::bBatching)
		{
			if (Dispatcher.FlushToStructBlocks())
			{
				ProcessStructs(Dispatcher);
			}

			Dispatcher.FlushQueuedReferences();
		}

		if constexpr (HasFlushWork<DispatcherType>::value)
		{
			Dispatcher.FlushWork();
		}
	}

	FORCENOINLINE void ProcessStructs(DispatcherType& Dispatcher);
};

//////////////////////////////////////////////////////////////////////////

/** Default reference collector for CollectReferences() */
template <typename ProcessorType>
class AUTORTFM_INFER TDefaultCollector : public FReferenceCollector
{
protected:
	ProcessorType& Processor;
	UE::GC::FWorkerContext& Context;

public:
	TDefaultCollector(ProcessorType& InProcessor, UE::GC::FWorkerContext& InContext)
	: Processor(InProcessor)
	, Context(InContext)
	{}

	virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
	{
		if (!ReferencingObject)
		{
			ReferencingObject = Context.GetReferencingObject();
		}
		Processor.HandleTokenStreamObjectReference(Context, const_cast<UObject*>(ReferencingObject), Object, EMemberlessId::Collector, EOrigin::Other, false);
	}
	virtual void HandleObjectReferences(UObject** InObjects, const int32 ObjectNum, const UObject* ReferencingObject, const FProperty* InReferencingProperty) override
	{
		if (!ReferencingObject)
		{
			ReferencingObject = Context.GetReferencingObject();
		}
		for (int32 ObjectIndex = 0; ObjectIndex < ObjectNum; ++ObjectIndex)
		{
			UObject*& Object = InObjects[ObjectIndex];
			Processor.HandleTokenStreamObjectReference(Context, const_cast<UObject*>(ReferencingObject), Object, EMemberlessId::Collector, EOrigin::Other, false);
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual void HandleVCellReference(Verse::VCell* Cell, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
	{
		if (!ReferencingObject)
		{
			ReferencingObject = Context.GetReferencingObject();
		}
		if constexpr (HasHandleTokenStreamVerseCellReference<ProcessorType>::value)
		{
			Processor.HandleTokenStreamVerseCellReference(Context, const_cast<UObject*>(ReferencingObject), Cell, EMemberlessId::Collector, EOrigin::Other);
		}
	}
#endif

	virtual bool IsIgnoringArchetypeRef() const override { return false;}
	virtual bool IsIgnoringTransient() const override {	return false; }
};

inline constexpr EGCOptions DefaultOptions = EGCOptions::AutogenerateSchemas;

} // namespace UE::GC

/** Simple single-threaded reference processor base class for use with CollectReferences() */
class FSimpleReferenceProcessorBase
{
public:
	using FMemberId				= UE::GC::FMemberId;
	using EOrigin				= UE::GC::EOrigin;
	using FPropertyStack		= UE::GC::FPropertyStack;
	using FWorkerContext		= UE::GC::FWorkerContext;

	static constexpr EGCOptions Options = UE::GC::DefaultOptions;

	// These functions are implemented in the GC collectors to generate detail stats on objects considered by GC.
	// They are generally not needed for other reference collection tasks.
	void BeginTimingObject(UObject* CurrentObject) {}
	void UpdateDetailedStats(UObject* CurrentObject) {}
	void LogDetailedStatsSummary() {}

	FORCEINLINE bool IsTimeLimitExceeded() const
	{
		return false;
	}

	// Implement this in your derived class, don't make this virtual as it will affect performance!
	//FORCEINLINE void HandleTokenStreamObjectReference(FWorkerContext& Context, UObject* ReferencingObject, UObject*& Object, FMemberId MemberId, EOrigin Origin, bool bAllowReferenceElimination)

	// Implement this in your derived class to add VCell support, don't make this virtual as it will affect performance!
	//FORCEINLINE void HandleTokenStreamVerseCellReference(FWorkerContext& Context, UObject* ReferencingObject, Verse::VCell* Cell, FMemberId MemberId, EOrigin Origin)
};


template<class CollectorType, class ProcessorType>
AUTORTFM_INFER FORCEINLINE static void CollectReferences(ProcessorType& Processor, UE::GC::FWorkerContext& Context)
{
	using namespace UE::GC;
	using FastReferenceCollector = TFastReferenceCollector<ProcessorType, CollectorType>;
	
	if (IsParallel(ProcessorType::Options) && !UE::GC::GIsIncrementalReachabilityPending)
	{
		ProcessAsync([](void* P, FWorkerContext& C) { FastReferenceCollector(*reinterpret_cast<ProcessorType*>(P)).ProcessObjectArray(C); }, &Processor, Context);
	}
	else
	{
		FastReferenceCollector(Processor).ProcessObjectArray(Context);
	}
}

template<class ProcessorType>
AUTORTFM_INFER FORCEINLINE static void CollectReferences(ProcessorType& Processor, UE::GC::FWorkerContext& Context)
{
	CollectReferences<UE::GC::TDefaultCollector<ProcessorType>, ProcessorType>(Processor, Context);
}

// Get number of workers to use when calling CollectReferences in parallel
COREUOBJECT_API int32 GetNumCollectReferenceWorkers();

// Temporary aliases to old types

using FTokenInfo = UE::GC::FMemberInfo;
using EGCTokenType = UE::GC::EOrigin;
using FGCArrayStruct = UE::GC::FWorkerContext;
namespace UE::GC { using FTokenId = FMemberId; }
namespace UE::GC { using ETokenlessId = EMemberlessId; }
