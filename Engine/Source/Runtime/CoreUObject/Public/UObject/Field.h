// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
Field.h: Declares FField property system fundamentals
=============================================================================*/

#pragma once

#include "Concepts/DerivedFrom.h"
#include "Containers/Array.h"
#include "Containers/LinkedListBuilder.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformMath.h"
#include "HAL/PreprocessorHelpers.h"
#include "HAL/UnrealMemory.h"
#include "Internationalization/Text.h"
#include "Math/RandomStream.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Templates/EnableIf.h"
#include "Templates/IsAbstract.h"
#include "Templates/IsEnum.h"
#include "Templates/NoDestroy.h"
#include "Templates/Requires.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/CoreNative.h"
#include "UObject/GarbageCollection.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PersistentObjectPtr.h"
#include "UObject/Script.h"
#include "UObject/SparseDelegate.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"

#include <type_traits>

class FField;
class FFieldVariant;
class FProperty;
class FReferenceCollector;
class UClass;
class UField;
class UPackage;
class UStruct;

/**
  * Object representing a type of an FField struct. 
  * Mimics a subset of UObject reflection functions.
  */
class FFieldClass
{
public:
	using FConstructFunction = FField*(const FFieldVariant&, const FName&);
private:
	UE_NONCOPYABLE(FFieldClass);

	/** Name of this field class */
	FName Name;
	/** Class flags */
	EClassFlags ClassFlags;
	/** Unique Id of this field class (for casting) */
	uint64 Id;
	/** Cast flags used for casting to other classes */
	EClassCastFlags CastFlags;
	/** Super of this class */
	FFieldClass* SuperClass;	
	/** Default instance of this class */
	FField* DefaultObject;
	/** Pointer to a function that can construct an instance of this class */
	FConstructFunction* ConstructFn;
	/** Counter for generating runtime unique names */
	std::atomic<int32> UniqueNameIndexCounter = 0;

	/** Creates a default object instance of this class */
	COREUOBJECT_API FField* ConstructDefaultObject();

public:

	/** Gets the list of all field classes in existance */
	static COREUOBJECT_API TArray<FFieldClass*>& GetAllFieldClasses();
	/** Gets a mapping of all field class names to the actuall class objects */
	static COREUOBJECT_API TMap<FName, FFieldClass*>& GetNameToFieldClassMap();

	COREUOBJECT_API explicit FFieldClass(const TCHAR* InCPPName, uint64 InId, EClassCastFlags InCastFlags, FFieldClass* InSuperClass, FConstructFunction* InConstructFn);
	COREUOBJECT_API virtual ~FFieldClass();

#if UE_WITH_CONSTINIT_UOBJECT
	consteval explicit FFieldClass(
		EConstEval,
		uint64 InId,
		EClassCastFlags InCastFlags,
		FFieldClass* InSuperClass,
		FConstructFunction* InConstructFn
	)
		: Name()
		, ClassFlags()
		, Id(InId)
		, CastFlags(InCastFlags)
		, SuperClass(InSuperClass)
		, DefaultObject()
		, ConstructFn(InConstructFn)
	{
	}
#endif // UE_WITH_CONSTINIT_UOBJECT 

#if UE_WITH_CONSTINIT_UOBJECT
	bool IsFullyConstructed() const
	{
		return Name != FName();
	}
#endif

	inline FString GetName() const
	{
#if UE_WITH_CONSTINIT_UOBJECT
		checkSlow(IsFullyConstructed());
#endif
		return Name.ToString();
	}
	inline FName GetFName() const
	{
#if UE_WITH_CONSTINIT_UOBJECT
		checkSlow(IsFullyConstructed());
#endif
		return Name;
	}
	inline uint64 GetId() const
	{
		return Id;
	}
	inline uint64 GetCastFlags() const
	{
		return CastFlags;
	}
	inline bool HasAnyCastFlags(const uint64 InCastFlags) const
	{
		return !!(CastFlags & InCastFlags);
	}
	inline bool HasAllCastFlags(const uint64 InCastFlags) const
	{
		return (CastFlags & InCastFlags) == InCastFlags;
	}
	inline bool IsChildOf(const FFieldClass* InClass) const
	{
		const uint64 OtherClassId = InClass->GetId();
		return OtherClassId ? !!(CastFlags & OtherClassId) : IsChildOf_Walk(InClass);
	}
	COREUOBJECT_API FString GetDescription() const;
	COREUOBJECT_API FText GetDisplayNameText() const;
	FField* Construct(const FFieldVariant& InOwner, const FName& InName) const
	{
		return ConstructFn(InOwner, InName);
	}
	UE_DEPRECATED(5.8, "FFieldClass::Construct with InFlags is deprecated, remove that parameter.")
	FField* Construct(const FFieldVariant& InOwner, const FName& InName, EObjectFlags InFlags) const { return Construct(InOwner, InName); }

	FFieldClass* GetSuperClass() const
	{
		return SuperClass;
	}

	FField* GetDefaultObject()
	{
		if (!DefaultObject)
		{
			DefaultObject = ConstructDefaultObject();
			check(DefaultObject);
		}
		return DefaultObject;
	}

	bool HasAnyClassFlags(EClassFlags FlagsToCheck) const
	{
		return EnumHasAnyFlags(ClassFlags, FlagsToCheck) != 0;
	}

	int32 GetNextUniqueNameIndex()
	{
		return ++UniqueNameIndexCounter;
	}

	friend FArchive& operator << (FArchive& Ar, FFieldClass& InField)
	{
		check(false);
		return Ar;
	}
	COREUOBJECT_API friend FArchive& operator << (FArchive& Ar, FFieldClass*& InOutFieldClass);

protected:
#if UE_WITH_CONSTINIT_UOBJECT
	virtual FFieldClass* GetSuperStaticClass() = 0;
#endif // UE_WITH_CONSTINIT_UOBJECT

private:
	bool IsChildOf_Walk(const FFieldClass* InBaseClass) const
	{
		for (const FFieldClass* TempField = this; TempField; TempField = TempField->GetSuperClass())
		{
			if (TempField == InBaseClass)
			{
				return true;
			}
		}
		return false;
	}

#if UE_WITH_CONSTINIT_UOBJECT
	friend struct FRegisterFieldClass;
#endif // UE_WITH_CONSTINIT_UOBJECT
};

/** 
 * Helper subclass to initialize an FFieldClass singleton for a particular subtype of FField 
 * e.g. helps with linking FFieldClass to its super for modular constinit-uobject builds.
 * Some of this code could be removed if UHT generated some of the code for FField/FFieldClass
 */
#if UE_WITH_CONSTINIT_UOBJECT
template<typename FieldType /* Type derived from FField */>
class TFieldClass : public FFieldClass
{
public: 
	consteval TFieldClass()
		: FFieldClass(ConstEval, 
			FieldType::StaticClassCastFlagsPrivate(),
			FieldType::StaticClassCastFlags(),
#if IS_MONOLITHIC
			// FField::Super is FField, in that case put null for the super for termination condition
			std::is_same_v<FieldType, typename FieldType::Super> ? nullptr : &FieldType::Super::ConstInitClass,
#else
			nullptr,
#endif // IS_MONOLITHIC
			&FieldType::Construct
		)
	{
		// Only FField should have itself as Super
		static_assert(std::is_same_v<FieldType, FField> || !std::is_same_v<FieldType, typename FieldType::Super>);
	}

protected:
	FFieldClass* GetSuperStaticClass() override final
	{
		if constexpr (std::is_same_v<typename FieldType::Super, typename FieldType::ThisClass>)
		{
			return nullptr;
		}
		else
		{
			return &FieldType::Super::ConstInitClass;
		}
	}
};
#endif // UE_WITH_CONSTINIT_UOBJECT

#if UE_WITH_CONSTINIT_UOBJECT
struct FRegisterFieldClass
{
	COREUOBJECT_API FRegisterFieldClass(FFieldClass* InFieldClass, const TCHAR* InCppName);
};
#endif // UE_WITH_CONSTINIT_UOBJECT

#if !CHECK_PUREVIRTUALS
	#define DECLARE_FIELD_NEW_IMPLEMENTATION(TClass) \
		ThisClass* Mem = (ThisClass*)FMemory::Malloc(InSize); \
		new (Mem) TClass(EC_InternalUseOnlyConstructor, TClass::StaticClass()); \
		return Mem; 
#else
	#define DECLARE_FIELD_NEW_IMPLEMENTATION(TClass) \
			ThisClass* Mem = (ThisClass*)FMemory::Malloc(InSize); \
			return Mem; 
#endif

#if UE_WITH_CONSTINIT_UOBJECT
#define UE_DECLARE_GET_FIELD_CLASS_PRIVATE(TRequiredAPI) 				TRequiredAPI virtual FFieldClass* GetFieldClassPrivate() override;
#define UE_DEFINE_GET_FIELD_CLASS_PRIVATE(TClass) 						FFieldClass* TClass::GetFieldClassPrivate() { return &TClass::ConstInitClass; }
#define UE_DECLARE_FIELD_CLASS_STATIC_CONSTINIT(TClass, TRequiredAPI) 	TRequiredAPI static constinit TNoDestroy<TFieldClass<TClass>> ConstInitClass;
#define UE_DEFINE_FIELD_CLASS_STATIC_CONSTINIT(TClass)					constinit TNoDestroy<TFieldClass<TClass>> TClass::ConstInitClass{NoDestroyConstEval};
#else
#define UE_DECLARE_GET_FIELD_CLASS_PRIVATE(TRequiredAPI) 
#define UE_DEFINE_GET_FIELD_CLASS_PRIVATE(TClass) 
#define UE_DECLARE_FIELD_CLASS_STATIC_CONSTINIT(TClass, TRequiredAPI)
#define UE_DEFINE_FIELD_CLASS_STATIC_CONSTINIT(...)
#endif

#define DECLARE_FIELD(TClass, TSuperClass, TStaticFlags) \
	DECLARE_FIELD_API(TClass, TSuperClass, TStaticFlags, NO_API)

#define DECLARE_FIELD_API(TClass, TSuperClass, TStaticFlags, TRequiredAPI) \
private: \
	TClass& operator=(TClass&&) = delete;   \
	TClass& operator=(const TClass&) = delete;   \
	UE_DECLARE_GET_FIELD_CLASS_PRIVATE(TRequiredAPI) \
public: \
	using Super = UE_REMOVE_OPTIONAL_PARENS(TSuperClass);\
	using ThisClass = TClass;\
	TClass(EInternal InInernal, FFieldClass* InClass) \
		: Super(EC_InternalUseOnlyConstructor, InClass) \
	{ \
	} \
	UE_NODEBUG static TRequiredAPI FFieldClass* StaticClass(); \
	UE_DECLARE_FIELD_CLASS_STATIC_CONSTINIT(TClass, TRequiredAPI) \
	static TRequiredAPI FField* Construct(const FFieldVariant& InOwner, const FName& InName); \
	UE_REWRITE static constexpr EClassCastFlags StaticClassCastFlagsPrivate() \
	{ \
		return TStaticFlags; \
	} \
	UE_REWRITE static constexpr EClassCastFlags StaticClassCastFlags() \
	{ \
		return TStaticFlags | Super::StaticClassCastFlags(); \
	} \
	inline void* operator new(const size_t InSize, void* InMem) \
	{ \
		return InMem; \
	} \
	inline void* operator new(const size_t InSize) \
	{ \
		DECLARE_FIELD_NEW_IMPLEMENTATION(TClass) \
	} \
	inline void operator delete(void* InMem) noexcept \
	{ \
		FMemory::Free(InMem); \
	} \
	friend FArchive &operator<<( FArchive& Ar, ThisClass*& Res ) \
	{ \
		return Ar << (FField*&)Res; \
	} \
	friend void operator<<(FStructuredArchive::FSlot InSlot, ThisClass*& Res) \
	{ \
		InSlot << (FField*&)Res; \
	} \
	virtual SIZE_T GetFieldSize() const override \
	{ \
		return sizeof(TClass); \
	}

#if !CHECK_PUREVIRTUALS
	#define IMPLEMENT_FIELD_CONSTRUCT_IMPLEMENTATION(TClass) \
		FField* Instance = new TClass(InOwner, InName); \
		return Instance; 
#else
	#define IMPLEMENT_FIELD_CONSTRUCT_IMPLEMENTATION(TClass) \
		return nullptr;
#endif

#if UE_WITH_CONSTINIT_UOBJECT
#define IMPLEMENT_FIELD(TClass)                                                                           \
	FField* TClass::Construct(const FFieldVariant& InOwner, const FName& InName)    \
	{                                                                                                     \
		IMPLEMENT_FIELD_CONSTRUCT_IMPLEMENTATION(TClass)                                                  \
	}                                                                                                     \
	UE_DEFINE_GET_FIELD_CLASS_PRIVATE(TClass) 														      \
	UE_DEFINE_FIELD_CLASS_STATIC_CONSTINIT(TClass)                                                        \
	static FRegisterFieldClass Register_FieldClass_##TClass(                                              \
		&TClass::ConstInitClass, TEXT(UE_STRINGIZE(TClass)));                                             \
	FFieldClass* TClass::StaticClass()                                                                    \
	{                                                                                                     \
		checkf(ConstInitClass->IsFullyConstructed(),                                                      \
			UE_STRINGIZE(TClass) TEXT(": FieldClass not fully constructed before ::StaticClass called")); \
		return &ConstInitClass;                                                                           \
	}
#else // UE_WITH_CONSTINIT_UOBJECT
#define IMPLEMENT_FIELD(TClass) \
FField* TClass::Construct(const FFieldVariant& InOwner, const FName& InName) \
{ \
	IMPLEMENT_FIELD_CONSTRUCT_IMPLEMENTATION(TClass) \
} \
FFieldClass* TClass::StaticClass() \
{ \
	static FFieldClass StaticFieldClass(TEXT(UE_STRINGIZE(TClass)), TClass::StaticClassCastFlagsPrivate(), TClass::StaticClassCastFlags(), TClass::Super::StaticClass(), &TClass::Construct); \
	return &StaticFieldClass; \
} 
#endif // UE_WITH_CONSTINIT_UOBJECT

class FField;
class FLinkerLoad;
class FProperty;
class UObject;

/**
 * Special container that can hold either UObject or FField.
 * Exposes common interface of FFields and UObjects for easier transition from UProperties to FProperties.
 * DO NOT ABUSE. IDEALLY THIS SHOULD ONLY BE FFIELD INTERNAL STRUCTURE FOR HOLDING A POINTER TO THE OWNER OF AN FFIELD.
 */
class FFieldVariant
{
	union FFieldObjectUnion
	{
		FField* Field;
		UObject* Object;
		UPTRINT Unknown;	
	} Container;

	static constexpr uintptr_t UObjectMask = 0x1;

	constexpr void ConditionallyMarkAsReachable()
	{
		UE_IF_NOT_CONSTEVAL
		{
			if (IsUObject() && ToUObjectUnsafe() && UE::GC::GIsIncrementalReachabilityPending)
			{
				UE::GC::MarkAsReachable(ToUObjectUnsafe());
			}
		}
	}
	
public:

	constexpr FFieldVariant()
	{
		Container.Field = nullptr;
	}

	FFieldVariant(const FField* InField)
	{
		Container.Field = const_cast<FField*>(InField);
		check(!IsUObject());
	}

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FFieldVariant(EConstEval, UObject* InObject)
	{
		// Object is not tagged here - when the engine starts up, pointers will be tagged during UObject bootstrap 
		Container.Object = InObject;
	}

	explicit consteval FFieldVariant(EConstEval, const FField* InField)
	{
		Container.Field = const_cast<FField*>(InField);
	}
#endif 

	template <
		typename T
		UE_REQUIRES(std::is_convertible_v<T, const UObject*>)
	>
	FFieldVariant(T&& InObject)
	{
		Container.Object = const_cast<UObject*>(ImplicitConv<const UObject*>(InObject));
		Container.Object = (UObject*)((uintptr_t)Container.Object | UObjectMask);
		ConditionallyMarkAsReachable();
	}

	constexpr FFieldVariant(TYPE_OF_NULLPTR)
		: FFieldVariant()
	{
	}

	constexpr FFieldVariant(const FFieldVariant& Other)
		: Container(Other.Container)
	{
		ConditionallyMarkAsReachable();
	}
	
	constexpr FFieldVariant& operator=(const FFieldVariant& Other)
	{
		Container = Other.Container;
		ConditionallyMarkAsReachable();
		return *this;
	}
	
	constexpr FFieldVariant(FFieldVariant&& Other)
		: Container(Other.Container)
	{
		ConditionallyMarkAsReachable();
	}
	
	constexpr FFieldVariant& operator=(FFieldVariant&& Other)
	{
		Container = Other.Container;
		ConditionallyMarkAsReachable();
		return *this;
	}

	inline bool IsUObject() const
	{
		return (uintptr_t)Container.Object & UObjectMask;
	}
	inline bool IsValid() const
	{
		return !!ToUObjectUnsafe();
	}
	COREUOBJECT_API bool IsValidLowLevel() const;
	inline operator bool() const
	{
		return IsValid();
	}
	COREUOBJECT_API bool IsA(const UClass* InClass) const;
	COREUOBJECT_API bool IsA(const FFieldClass* InClass) const;
	template <typename T>
	bool IsA() const
	{
		static_assert(sizeof(T) > 0, "T must not be an incomplete type");
		return IsA(T::StaticClass());
	}

	template <typename T>
	T* Get() const
	{
		static_assert(sizeof(T) > 0, "T must not be an incomplete type");
		if (IsA(T::StaticClass()))
		{
			if constexpr (std::is_base_of_v<UObject, T>)
			{
				return static_cast<T*>(ToUObjectUnsafe());
			}
			else
			{
				return static_cast<T*>(ToFieldUnsafe());
			}
		}
		return nullptr;
	}

	UObject* ToUObject() const
	{
		if (IsUObject())
		{
			return ToUObjectUnsafe();
		}
		else
		{
			return nullptr;
		}
	}
	FField* ToField() const
	{
		if (!IsUObject())
		{
			return ToFieldUnsafe();
		}
		else
		{
			return nullptr;
		}
	}
	/** FOR INTERNAL USE ONLY: Function that returns the owner as FField without checking if it's actually an FField */
	UE_FORCEINLINE_HINT FField* ToFieldUnsafe() const
	{
		return Container.Field;
	}
	/** FOR INTERNAL USE ONLY: Function that returns the owner as UObject without checking if it's actually a UObject */
	UE_FORCEINLINE_HINT UObject* ToUObjectUnsafe() const
	{
		return (UObject*)((uintptr_t)Container.Object & ~UObjectMask);
	}

	/* Return the raw value of the union without masking any tag bits away regardless of what type it is */
	void* GetRawPointer() const
	{
		return (void*)Container.Unknown;
	}
	COREUOBJECT_API FFieldVariant GetOwnerVariant() const;
	COREUOBJECT_API UClass* GetOwnerClass() const;
	COREUOBJECT_API FString GetFullName() const;
	COREUOBJECT_API FString GetPathName() const;
	COREUOBJECT_API FString GetName() const;
	COREUOBJECT_API FString GetClassName() const;
	COREUOBJECT_API FName GetFName() const;
	COREUOBJECT_API bool IsNative() const;
	COREUOBJECT_API UPackage* GetOutermost() const;

	bool operator==(const FFieldVariant& Other) const
	{
		return Container.Field == Other.Container.Field;
	}
	bool operator!=(const FFieldVariant& Other) const
	{
		return Container.Field != Other.Container.Field;
	}

#if WITH_METADATA
	COREUOBJECT_API bool HasMetaData(const FName& Key) const;
#endif // WITH_METADATA

	/** Support comparison functions that make this usable as a KeyValue for a TSet<> */
	[[nodiscard]] friend uint32 GetTypeHash(const FFieldVariant& InFieldVariant)
	{
		return GetTypeHash(InFieldVariant.GetRawPointer());
	}
};

/**
 * Base class of reflection data objects.
 * NOTE: Take care with inline initializers in members of FField as it executes two-stage construction
 * via operator new and the EInternal constructor.
 */
class FField
{
	UE_NONCOPYABLE(FField);

	union
	{
		/** Pointer to the class object representing the type of this FField */
		FFieldClass* ClassPrivate;
#if UE_WITH_CONSTINIT_UOBJECT
		// Share storage for compiled-in name with ClassPrivate which can be re-acquired through the object's vtable at runtime
		// When there is a 4-byte solution for compiled-in FNames, this will be removed.
		const UTF8CHAR* NameTempUTF8;
#endif //UE_WITH_CONSTINIT_UOBJECT
	};

#if UE_WITH_CONSTINIT_UOBJECT
	virtual FFieldClass* GetFieldClassPrivate() PURE_VIRTUAL(FField::GetFieldClassPrivate, return nullptr;)
#endif

public:
	typedef FField Super;
	typedef FField ThisClass;
	typedef FField BaseFieldClass;	
	typedef FFieldClass FieldTypeClass;

	static COREUOBJECT_API FFieldClass* StaticClass();
#if UE_WITH_CONSTINIT_UOBJECT
	static constinit TNoDestroy<TFieldClass<FField>> ConstInitClass;
#endif // UE_WITH_CONSTINIT_UOBJECT

	virtual SIZE_T GetFieldSize() const
	{
		return sizeof(FField);
	}

	inline static constexpr EClassCastFlags StaticClassCastFlagsPrivate()
	{
		return CASTCLASS_UField;
	}
	inline static constexpr EClassCastFlags StaticClassCastFlags()
	{
		return CASTCLASS_UField;
	}

	/** Owner of this field */
	FFieldVariant Owner;

	/** Next Field in the linked list within the same owning struct - note that the list does not continue to parent structs */
	FField* Next;

	/** Name of this field */
	FName NamePrivate;

#if !UE_STRIP_DEPRECATED_PROPERTIES
	/** Object flags */
	UE_DEPRECATED(5.8, "FField::FlagsPrivate is deprecated, consider using FProperty::PropertyFlags instead.")
	EObjectFlags FlagsPrivate = {};
#endif

	// Constructors.
	COREUOBJECT_API FField(EInternal InInernal, FFieldClass* InClass);
	COREUOBJECT_API FField(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FField constructor with InObjectFlags is deprecated, remove that parameter.")
	FField(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FField(InOwner, InName) {}
#if WITH_EDITORONLY_DATA
	COREUOBJECT_API explicit FField(UField* InField);
#endif // WITH_EDITORONLY_DATA
	COREUOBJECT_API virtual ~FField();

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FField(EConstEval, UE::CodeGen::ConstInit::FFieldOwnerParam InOwner, FField* InNext, const UTF8CHAR* InNameUTF8)
	: NameTempUTF8(InNameUTF8)
	, Owner(InOwner.bIsObject ? FFieldVariant(ConstEval, InOwner.Value.Object) : FFieldVariant(ConstEval, InOwner.Value.Field))
	, Next(InNext)
	, NamePrivate()
#if WITH_METADATA
	, MetaDataMap(nullptr)
#endif
	{
	}
#endif

	// Begin UObject interface: the following functions mimic UObject interface for easier transition from UProperties to FProperties
	COREUOBJECT_API virtual void Serialize(FArchive& Ar);
#if !UE_STRIP_DEPRECATED_PROPERTIES
	UE_DEPRECATED(5.8, "FField::PostLoad is deprecated. It is never called.")
	virtual void PostLoad() final {};
	UE_DEPRECATED(5.8, "FField::BeginDestroy is deprecated. It is never called.")
	virtual void BeginDestroy() final {};
#endif
	COREUOBJECT_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps);
	COREUOBJECT_API virtual void AddReferencedObjects(FReferenceCollector& Collector);
	COREUOBJECT_API bool IsRooted() const;
	COREUOBJECT_API bool IsNative() const;
	COREUOBJECT_API bool IsValidLowLevel() const;
	COREUOBJECT_API bool IsIn(const UObject* InOwner) const;
	COREUOBJECT_API bool IsIn(const FField* InOwner) const;
	COREUOBJECT_API FLinkerLoad* GetLinker() const;	
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	COREUOBJECT_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End UObject interface

	// Begin UField interface.
	COREUOBJECT_API virtual void AddCppProperty(FProperty* Property);
#if !UE_STRIP_DEPRECATED_PROPERTIES
	UE_DEPRECATED(5.8, "FField::Bind is deprecated. It is never called.")
	virtual void Bind() final {};
#endif
	// End UField interface

	/** Constructs a new field given its class */
	static COREUOBJECT_API FField* Construct(const FFieldVariant& InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FField::Construct with InFlags is deprecated, remove that parameter.")
	static FField* Construct(const FFieldVariant& InOwner, const FName& InName, EObjectFlags InFlags) { return Construct(InOwner, InName); };
	/** Constructs a new field given the name of its class */
	static COREUOBJECT_API FField* Construct(const FName& FieldTypeName, const FFieldVariant& InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FField::Construct with InFlags is deprecated, remove that parameter.")
	static FField* Construct(const FName& FieldTypeName, const FFieldVariant& InOwner, const FName& InName, EObjectFlags InFlags) {	return Construct(FieldTypeName, InOwner, InName); };
	/** Tries to construct a new field given the name of its class. Returns null if the type does not exist. */
	static COREUOBJECT_API FField* TryConstruct(const FName& FieldTypeName, const FFieldVariant& InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FField::TryConstruct with InFlags is deprecated, remove that parameter.")
	static FField* TryConstruct(const FName& FieldTypeName, const FFieldVariant& InOwner, const FName& InName, EObjectFlags InFlags) { return TryConstruct(FieldTypeName, InOwner, InName);	};

	/** Fixups after duplicating a Field */
	COREUOBJECT_API virtual void PostDuplicate(const FField& InField);

#if UE_WITH_CONSTINIT_UOBJECT
	// For a const-initialized property, initialize the FName from a cached string and tag the owner pointer
	void InitializeConstInitField(FField* InOwner);
	void InitializeConstInitField(UObject* InOwner);
#endif

protected:
	UE_DEPRECATED(5.8, "FField::SetFlagsTo is deprecated, for FProperties consider using PropertyFlags instead.")
	void SetFlagsTo(EObjectFlags NewFlags) {}
public:
	UE_DEPRECATED(5.8, "FField::GetFlags is deprecated, for FProperties consider using PropertyFlags instead.")
	EObjectFlags GetFlags() const { return RF_NoFlags; }
	UE_DEPRECATED(5.8, "FField::SetFlags is deprecated, for FProperties consider using PropertyFlags instead.")
	void SetFlags(EObjectFlags NewFlags) {}
	UE_DEPRECATED(5.8, "FField::ClearFlags is deprecated, for FProperties consider using PropertyFlags instead.")
	void ClearFlags(EObjectFlags NewFlags) {}
	UE_DEPRECATED(5.8, "FField::HasAnyFlags is deprecated, for FProperties consider using PropertyFlags instead.")
	bool HasAnyFlags(EObjectFlags FlagsToCheck) const { return false; }
	UE_DEPRECATED(5.8, "FField::HasAllFlags is deprecated, for FProperties consider using PropertyFlags instead.")
	bool HasAllFlags(EObjectFlags FlagsToCheck) const { return false; }

	inline FFieldClass* GetClass() const
	{
		return ClassPrivate;
	}
	inline uint64 GetCastFlags() const
	{
		return GetClass()->GetCastFlags();
	}

	inline bool IsA(const FFieldClass* FieldType) const
	{
		check(FieldType);
		return GetClass()->IsChildOf(FieldType);
	}

	template<typename T>
	bool IsA() const
	{
		if constexpr (!!(T::StaticClassCastFlagsPrivate()))
		{
			return !!(GetCastFlags() & T::StaticClassCastFlagsPrivate());
		}
		else
		{
			return GetClass()->IsChildOf(T::StaticClass());
		}
	}

	void AppendName(FString& ResultString) const
	{
		GetFName().AppendString(ResultString);
	}

	/** Gets the owner container for this field */
	FFieldVariant GetOwnerVariant() const
	{
		return Owner;
	}

	/** Goes up the outer chain to look for a UObject. This function is used in GC so for performance reasons it has to be inlined */
	inline UObject* GetOwnerUObject() const
	{
		FFieldVariant TempOuter = Owner;
		while (!TempOuter.IsUObject() && TempOuter.IsValid())
		{
			// It's ok to use the 'Unsafe' variant of ToField here since we just checked IsUObject above
			TempOuter = TempOuter.ToFieldUnsafe()->Owner;
		}
		return TempOuter.ToUObject();
	}

	/** Internal function for quickly getting the owner of this object as UObject. FOR INTERNAL USE ONLY */
	UE_FORCEINLINE_HINT UObject* InternalGetOwnerAsUObjectUnsafe() const
	{
		return Owner.ToUObjectUnsafe();
	}

	/** Goes up the outer chain to look for a UClass */
	COREUOBJECT_API UClass* GetOwnerClass() const;

	/** Goes up the outer chain to look for a UStruct */
	COREUOBJECT_API UStruct* GetOwnerStruct() const;

	/** Goes up the outer chain to look for a UField */
	COREUOBJECT_API UField* GetOwnerUField() const;

	/** Goes up the outer chain to look for the outermost package */
	COREUOBJECT_API UPackage* GetOutermost() const;

	/** Goes up the outer chain to look for the outer of the specified type */
	COREUOBJECT_API UObject* GetTypedOwner(UClass* Target) const;

	/** Goes up the outer chain to look for the outer of the specified type */
	COREUOBJECT_API FField* GetTypedOwner(FFieldClass* Target) const;

	template <typename T>
	T* GetOwner() const
	{
		static_assert(sizeof(T) > 0, "T must not be an incomplete type");
		return Owner.Get<T>();
	}

	template <typename T>
	FUNCTION_NON_NULL_RETURN_START
	T* GetOwnerChecked() const
	FUNCTION_NON_NULL_RETURN_END
	{
		static_assert(sizeof(T) > 0, "T must not be an incomplete type");
		T* Result = Owner.Get<T>();
		check(Result);
		return Result;
	}

	template <typename T>
	T* GetTypedOwner() const
	{
		static_assert(sizeof(T) > 0, "T must not be an incomplete type");
		return static_cast<T*>(GetTypedOwner(T::StaticClass()));
	}

	inline FName GetFName() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (IsThisNotNull(this, "FField::GetFName"))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			return NamePrivate;
		}
		else
		{
			return NAME_None;
		}
	}

	inline FString GetName() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (IsThisNotNull(this, "FField::GetName"))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			return NamePrivate.ToString();
		}
		else
		{
			return TEXT("None");
		}
	}

	inline void GetName(FString& OutName) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (IsThisNotNull(this, "FField::GetName"))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			return NamePrivate.ToString(OutName);
		}
		else
		{
			OutName = TEXT("None");
		}
	}

	COREUOBJECT_API void Rename(const FName& NewName);

	COREUOBJECT_API FString GetPathName(const UObject* StopOuter = nullptr) const;
	COREUOBJECT_API void GetPathName(const UObject* StopOuter, FStringBuilderBase& ResultString) const;
	COREUOBJECT_API FString GetFullName() const;

	/**
	 * Returns a human readable string that was assigned to this field at creation.
	 * By default this is the same as GetName() but it can be overridden if that is an internal-only name.
	 * This name is consistent in editor/cooked builds, is not localized, and is useful for data import/export.
	 */
	COREUOBJECT_API FString GetAuthoredName() const;

	/** Returns an inner field by name if the field has any */
	virtual FField* GetInnerFieldByName(const FName& InName)
	{
		return nullptr;
	}

	/** Fills the provided array with all inner fields this field owns (recursively) */
	virtual void GetInnerFields(TArray<FField*>& OutFields)
	{
	}

#if WITH_METADATA
private:
	/** Editor-only meta data map */
	TMap<FName, FString>* MetaDataMap;

public:
	/**
	* Determines if the property has any metadata associated with the key
	*
	* @param Key The key to lookup in the metadata
	* @return true if there is a (possibly blank) value associated with this key
	*/
	bool HasMetaData(const TCHAR* Key) const { return FindMetaData(Key) != nullptr; }
	bool HasMetaData(const FName& Key) const { return FindMetaData(Key) != nullptr; }

	/**
	* Find the metadata value associated with the key
	*
	* @param Key The key to lookup in the metadata
	* @return The value associated with the key if it exists, null otherwise
	*/
	COREUOBJECT_API const FString* FindMetaData(const TCHAR* Key) const;
	COREUOBJECT_API const FString* FindMetaData(const FName& Key) const;

	/**
	* Find the metadata value associated with the key
	*
	* @param Key The key to lookup in the metadata
	* @return The value associated with the key
	*/
	COREUOBJECT_API const FString& GetMetaData(const TCHAR* Key) const;
	COREUOBJECT_API const FString& GetMetaData(const FName& Key) const;

	/**
	* Find the metadata value associated with the key and localization namespace and key
	*
	* @param Key						The key to lookup in the metadata
	* @param LocalizationNamespace		Namespace to lookup in the localization manager
	* @param LocalizationKey			Key to lookup in the localization manager
	* @return							Localized metadata if available, defaults to whatever is provided via GetMetaData
	*/
	COREUOBJECT_API const FText GetMetaDataText(const TCHAR* MetaDataKey, const FTextKey LocalizationNamespace = FTextKey(), const FTextKey LocalizationKey = FTextKey()) const;
	COREUOBJECT_API const FText GetMetaDataText(const FName& MetaDataKey, const FTextKey LocalizationNamespace = FTextKey(), const FTextKey LocalizationKey = FTextKey()) const;

	/**
	* Sets the metadata value associated with the key
	*
	* @param Key The key to lookup in the metadata
	* @return The value associated with the key
	*/
	COREUOBJECT_API void SetMetaData(const TCHAR* Key, const TCHAR* InValue);
	COREUOBJECT_API void SetMetaData(const FName& Key, const TCHAR* InValue);

	COREUOBJECT_API void SetMetaData(const TCHAR* Key, FString&& InValue);
	COREUOBJECT_API void SetMetaData(const FName& Key, FString&& InValue);

	/**
	* Find the metadata value associated with the key
	* and return bool
	* @param Key The key to lookup in the metadata
	* @return return true if the value was true (case insensitive)
	*/
	bool GetBoolMetaData(const TCHAR* Key) const
	{
		const FString& BoolString = GetMetaData(Key);
		// FString == operator does case insensitive comparison
		return (BoolString == "true");
	}
	bool GetBoolMetaData(const FName& Key) const
	{
		const FString& BoolString = GetMetaData(Key);
		// FString == operator does case insensitive comparison
		return (BoolString == "true");
	}

	/**
	* Find the metadata value associated with the key
	* and return int32
	* @param Key The key to lookup in the metadata
	* @return the int value stored in the metadata.
	*/
	int32 GetIntMetaData(const TCHAR* Key) const
	{
		const FString& INTString = GetMetaData(Key);
		int32 Value = FCString::Atoi(*INTString);
		return Value;
	}
	int32 GetIntMetaData(const FName& Key) const
	{
		const FString& INTString = GetMetaData(Key);
		int32 Value = FCString::Atoi(*INTString);
		return Value;
	}

	/**
	* Find the metadata value associated with the key
	* and return float
	* @param Key The key to lookup in the metadata
	* @return the float value stored in the metadata.
	*/
	float GetFloatMetaData(const TCHAR* Key) const
	{
		const FString& FLOATString = GetMetaData(Key);
		// FString == operator does case insensitive comparison
		float Value = FCString::Atof(*FLOATString);
		return Value;
	}
	float GetFloatMetaData(const FName& Key) const
	{
		const FString& FLOATString = GetMetaData(Key);
		// FString == operator does case insensitive comparison
		float Value = FCString::Atof(*FLOATString);
		return Value;
	}

	/**
	* Find the metadata value associated with the key
	* and return double
	* @param Key The key to lookup in the metadata
	* @return the float value stored in the metadata.
	*/
	double GetDoubleMetaData(const TCHAR* Key) const
	{
		const FString& DOUBLEString = GetMetaData(Key);
		// FString == operator does case insensitive comparison
		double Value = FCString::Atod(*DOUBLEString);
		return Value;
	}
	double GetDoubleMetaData(const FName& Key) const
	{
		const FString& DOUBLEString = GetMetaData(Key);
		// FString == operator does case insensitive comparison
		double Value = FCString::Atod(*DOUBLEString);
		return Value;
	}

	/**
	* Find the metadata value associated with the key
	* and return Class
	* @param Key The key to lookup in the metadata
	* @return the class value stored in the metadata.
	*/
	COREUOBJECT_API UClass* GetClassMetaData(const TCHAR* Key) const;
	COREUOBJECT_API UClass* GetClassMetaData(const FName& Key) const;

	/** Clear any metadata associated with the key */
	COREUOBJECT_API void RemoveMetaData(const TCHAR* Key);
	COREUOBJECT_API void RemoveMetaData(const FName& Key);

	/** Gets all metadata associated with this field */
	COREUOBJECT_API const TMap<FName, FString>* GetMetaDataMap() const;

	/** Append the given metadata to this field */
	COREUOBJECT_API void AppendMetaData(const TMap<FName, FString>& MetaDataMapToAppend);

	/** Copies all metadata from Source Field to Dest Field */
	static COREUOBJECT_API void CopyMetaData(const FField* InSourceField, FField* InDestField);
#endif // WITH_METADATA

#if WITH_EDITORONLY_DATA
public:
	/**
	* Walks up the chain of packages until it reaches the top level, which it ignores.
	*
	* @param	bStartWithOuter		whether to include this object's name in the returned string
	* @return	string containing the path name for this object, minus the outermost-package's name
	*/
	COREUOBJECT_API FString GetFullGroupName(bool bStartWithOuter) const;

	/**
	* Finds the localized display name or native display name as a fallback.
	*
	* @return The display name for this object.
	*/
	COREUOBJECT_API FText GetDisplayNameText() const;

	/**
	* Finds the localized tooltip or native tooltip as a fallback.
	*
	* @param bShortTooltip Look for a shorter version of the tooltip (falls back to the long tooltip if none was specified)
	*
	* @return The tooltip for this object.
	*/
	COREUOBJECT_API FText GetToolTipText(bool bShortTooltip = false) const;

	/** Creates a new FField from existing UField */
	static COREUOBJECT_API FField* CreateFromUField(UField* InField);
	
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnConvertCustomUFieldToFField, FFieldClass*, UField*, FField*&);
	/** Gets a delegate to convert custom UField types to FFields */
	static COREUOBJECT_API FOnConvertCustomUFieldToFField& GetConvertCustomUFieldToFFieldDelegate();
#endif // WITH_EDITORONLY_DATA

public:
	/** Duplicates an FField */
	static COREUOBJECT_API FField* Duplicate(const FField* InField, FFieldVariant DestOwner, const FName DestName = NAME_None);
	UE_DEPRECATED(5.8, "FField::Duplicate with FlagMask and InternalFlagsMask is deprecated, remove those parameters.")
	static FField* Duplicate(const FField* InField, FFieldVariant DestOwner, const FName DestName, EObjectFlags FlagMask, EInternalObjectFlags InternalFlagsMask = EInternalObjectFlags_AllFlags)
	{ 
		return Duplicate(InField, DestOwner, DestName);
	}

	/** Generates a name for a Field of a given type. Each generated name is unique in the current runtime */
	static COREUOBJECT_API FName GenerateFFieldName(FFieldVariant InOwner, FFieldClass* InClass);

	using FLinkedListBuilder = TLinkedListBuilder<FField>;
};

// Support for casting between different FFIeld types

template<typename FieldType>
UE_FORCEINLINE_HINT FieldType* CastField(FField* Src)
{
	return Src && Src->IsA<FieldType>() ? static_cast<FieldType*>(Src) : nullptr;
}

template<typename FieldType>
UE_FORCEINLINE_HINT const FieldType* CastField(const FField* Src)
{
	return Src && Src->IsA<FieldType>() ? static_cast<const FieldType*>(Src) : nullptr;
}

template<typename FieldType>
UE_FORCEINLINE_HINT FieldType* ExactCastField(FField* Src)
{
	return (Src && (Src->GetClass() == FieldType::StaticClass())) ? static_cast<FieldType*>(Src) : nullptr;
}

template<typename FieldType>
UE_FORCEINLINE_HINT FieldType* ExactCastField(const FField* Src)
{
	return (Src && (Src->GetClass() == FieldType::StaticClass())) ? static_cast<const FieldType*>(Src) : nullptr;
}

template<typename FieldType>
FUNCTION_NON_NULL_RETURN_START
inline FieldType* CastFieldChecked(FField* Src)
FUNCTION_NON_NULL_RETURN_END
{
#if !DO_CHECK
	return (FieldType*)Src;
#else
	FieldType* CastResult = Src && Src->IsA<FieldType>() ? static_cast<FieldType*>(Src) : nullptr;
	checkf(CastResult, TEXT("CastFieldChecked failed with 0x%016llx"), (int64)(PTRINT)Src);
	return CastResult;
#endif // !DO_CHECK
}

template<typename FieldType>
FUNCTION_NON_NULL_RETURN_START
inline const FieldType* CastFieldChecked(const FField* Src)
FUNCTION_NON_NULL_RETURN_END
{
#if !DO_CHECK
	return (const FieldType*)Src;
#else
	const FieldType* CastResult = Src && Src->IsA<FieldType>() ? static_cast<const FieldType*>(Src) : nullptr;
	checkf(CastResult, TEXT("CastFieldChecked failed with 0x%016llx"), (int64)(PTRINT)Src);
	return CastResult;
#endif // !DO_CHECK
}

template<typename FieldType>
inline FieldType* CastFieldCheckedNullAllowed(FField* Src)
{
#if !DO_CHECK
	return (FieldType*)Src;
#else
	FieldType* CastResult = Src && Src->IsA<FieldType>() ? static_cast<FieldType*>(Src) : nullptr;
	checkf(CastResult || !Src, TEXT("CastFieldCheckedNullAllowed failed with 0x%016llx"), (int64)(PTRINT)Src);
	return CastResult;
#endif // !DO_CHECK
}

template<typename FieldType>
inline const FieldType* CastFieldCheckedNullAllowed(const FField* Src)
{
#if !DO_CHECK
	return (const FieldType*)Src;
#else
	const FieldType* CastResult = Src && Src->IsA<FieldType>() ? static_cast<const FieldType*>(Src) : nullptr;
	checkf(CastResult || !Src, TEXT("CastFieldCheckedNullAllowed failed with 0x%016llx"), (int64)(PTRINT)Src);
	return CastResult;
#endif // !DO_CHECK
}

/**
 * Helper function for serializing FField to an archive. This function fully serializes the field and its properties.
 */
template <typename FieldType>
inline void SerializeSingleField(FArchive& Ar, FieldType*& Field, FFieldVariant Owner)
{
	if (Ar.IsLoading())
	{
		FName PropertyTypeName;
		Ar << PropertyTypeName;
		if (PropertyTypeName != NAME_None)
		{
			Field = CastField<FieldType>(FField::Construct(PropertyTypeName, Owner, NAME_None));
			check(Field);
			Field->Serialize(Ar);
		}
		else
		{
			Field = nullptr;
		}
	}
	else
	{
		FName PropertyTypeName = Field ? Field->GetClass()->GetFName() : NAME_None;		
		Ar << PropertyTypeName;
		if (Field)
		{
			Field->Serialize(Ar);
		}
	}
}

/**
 * Gets the name of the provided field. If the field pointer is null, the result is "none"
 */
inline FName GetFNameSafe(const FField* InField)
{
	if (InField)
	{
		return InField->GetFName();
	}
	else
	{
		return NAME_None;
	}
}
/**
 * Gets the name of the provided field. If the field pointer is null, the result is "none"
 */
inline FString GetNameSafe(const FField* InField)
{
	if (InField)
	{
		return InField->GetName();
	}
	else
	{
		return TEXT("none");
	}
}
/**
 * Gets the full name of the provided field. If the field pointer is null, the result is "none"
 */
COREUOBJECT_API FString GetFullNameSafe(const FField* InField);
/**
 * Gets the path name of the provided field. If the field pointer is null, the result is "none"
 */
COREUOBJECT_API FString GetPathNameSafe(const FField* InField, const UObject* StopOuter = nullptr);
/** 
 * Finds a field given a path to the field (Package.Class[:Subobject:...]:FieldName)
 */
COREUOBJECT_API FField* FindFPropertyByPath(const TCHAR* InFieldPath);
/**
 * Templated version of FindFieldByPath
 */
template <typename FieldType>
inline FieldType* FindFProperty(const TCHAR* InFieldPath)
{
	FField* FoundField = FindFPropertyByPath(InFieldPath);
	return CastField<FieldType>(FoundField);
}
