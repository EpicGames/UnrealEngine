// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
FieldIterator.h: FField iterators.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Field.h"
#include "UObject/UnrealType.h"
#include "UObject/UObjectIterator.h"

/**
 * Helper function for getting the inner fields of a field that works with both FFields and UFields
 */
template <class FieldType>
void GetInnerFieldsFromField(FieldType* Owner, TArray<FieldType*>& OutFields)
{
	check(false);
}

template <>
inline void GetInnerFieldsFromField(FField* Owner, TArray<FField*>& OutFields)
{
	Owner->GetInnerFields(OutFields);
}

template <>
inline void GetInnerFieldsFromField(UField* Owner, TArray<UField*>& OutFields)
{
}

//
// For iterating through all fields in all structs including inner FProperties of top level FProperties.
//
template <class T>
class TAllFieldsIterator
{
private:
	/** Iterator for iterating over all UStructs */
	TObjectIterator<UStruct> StructIterator;
	/** 
	 *  Iterator for iterating over all child UFields or FFields 
	 *  Note that we're going to be iterating over all fields (BaseFieldClass is either FField or UField), not just the ones that match the template argument because we also want
	 *  to be able to iterate inner fields (for example FArrayProperty::Inner, FSetProperty::ElementProp etc)
	 */
	TFieldIterator<typename T::BaseFieldClass> FieldIterator;
	/** List containing the currently iterated field as well as all fields it owns and all fields the fields it owns own etc.. */
	TArray<typename T::BaseFieldClass*> CurrentFields;
	/** Currently iterated field index in CurrentFields array */
	int32 CurrentFieldIndex = -1;

public:
	TAllFieldsIterator(EObjectFlags AdditionalExclusionFlags = RF_ClassDefaultObject, EInternalObjectFlags InternalExclusionFlags = EInternalObjectFlags::None)
		: StructIterator(AdditionalExclusionFlags, /*bIncludeDerivedClasses =*/ true, InternalExclusionFlags)
		, FieldIterator(nullptr)
	{
		// Currently 3 would be enough (the current field + its inners which is 2 max for FMapProperty) but we keep one extra as slack
		// We never free this array memory inside of TAllFieldsIterator except when TAllFieldsIterator gets destroyed for performance reasons so it may only grow.
		// In the future we may want to support TArrays of TArrays/TMaps (nested containers) and in such case it may grow beyond 4 but that's ok
		CurrentFields.Reserve(4);
		InitFieldIterator();
	}

	/** conversion to "bool" returning true if the iterator is valid. */
	UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return (bool)FieldIterator || (bool)StructIterator;
	}
	/** inverse of the "bool" operator */
	UE_FORCEINLINE_HINT bool operator !() const
	{
		return !(bool)*this;
	}

	inline bool operator==(const TAllFieldsIterator<T>& Rhs) const 
	{ 
		return *FieldIterator == *Rhs.FieldIterator && CurrentFieldIndex == Rhs.CurrentFieldIndex; 
	}
	inline bool operator!=(const TAllFieldsIterator<T>& Rhs) const
	{ 
		return *FieldIterator != *Rhs.FieldIterator || CurrentFieldIndex != Rhs.CurrentFieldIndex;
	}

	inline void operator++()
	{
		IterateToNextField();
		ConditionallyIterateToNextStruct();
	}
	inline T* operator*()
	{
		if (CurrentFieldIndex >= 0)
		{
			return CastFieldChecked<T>(CurrentFields[CurrentFieldIndex]);
		}
		return nullptr;
	}
	inline T* operator->()
	{
		if (CurrentFieldIndex >= 0)
		{
			return CastFieldChecked<T>(CurrentFields[CurrentFieldIndex]);
		}
		return nullptr;
	}
protected:

	/** Initializes CurrentFields array with the currently iterated field as well as the fields it owns */
	inline void InitCurrentFields()
	{
		CurrentFieldIndex = -1;
		CurrentFields.Reset();
		typename T::BaseFieldClass* CurrentField = *FieldIterator;
		CurrentFields.Add(CurrentField);
		GetInnerFieldsFromField<typename T::BaseFieldClass>(CurrentField, CurrentFields);
	}

	/** Advances to the next field of the specified template type */
	inline void IterateToNextField()
	{
		while (FieldIterator)
		{
			for (++CurrentFieldIndex; CurrentFieldIndex < CurrentFields.Num(); ++CurrentFieldIndex)
			{
				if (CurrentFields[CurrentFieldIndex]->template IsA<T>())
				{
					break;
				}
			}

			if (CurrentFieldIndex == CurrentFields.Num())
			{
				++FieldIterator;
				if (FieldIterator)
				{
					InitCurrentFields();
				}
				else
				{
					CurrentFieldIndex = -1;
				}
			}
			else
			{
				break;
			}
		}
	}

	/** Initializes the field iterator for the current struct */
	inline void InitFieldIterator()
	{
		while (StructIterator)
		{
			FieldIterator.~TFieldIterator<typename T::BaseFieldClass>();
			new (&FieldIterator) TFieldIterator<typename T::BaseFieldClass>(*StructIterator, EFieldIteratorFlags::ExcludeSuper, EFieldIteratorFlags::IncludeDeprecated, EFieldIteratorFlags::IncludeInterfaces);
			if (!FieldIterator)
			{
				// This struct has no fields, check the next one
				++StructIterator;
				CurrentFieldIndex = -1;
			}
			else
			{
				InitCurrentFields();
				IterateToNextField();

				if (!FieldIterator)
				{
					// If the field iterator is invalid after IterateToNextField() call then no fields of the speficied template type were found
					++StructIterator;
				}
				else
				{
					break;
				}
			}
		}
	}
	inline void ConditionallyIterateToNextStruct()
	{
		if (!FieldIterator)
		{
			// We finished iterating over all fields of the current struct so move to the next struct
			++StructIterator;
			InitFieldIterator();
		}
	}
};