// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	FieldPath.h: A pointer to a field and the path to it from its outermost owner.
=============================================================================*/

#pragma once

#include "Concepts/PointerEqualityComparableWith.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/AssertionMacros.h"
#include "Misc/UEOps.h"
#include "Templates/Requires.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectArray.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include <type_traits>

class FArchive;
class FLinkerLoad;
class UField;
class UStruct;
struct FUObjectItem;
template <typename T> struct TIsPODType;
template <typename T> struct TIsWeakPointerType;
template <typename T> struct TIsZeroConstructType;

struct FFieldPath
{
	friend struct FGCInternals;

	// TWeakFieldPtr needs access to ClearCachedField
	template<class T>
	friend struct TWeakFieldPtr;

	// FFieldPathProperty needs access to ConvertFromFullPath
	friend class FFieldPathProperty;

protected:

	/* Determines the behavior when resolving stored path */
	enum EPathResolveType
	{
		UseStructIfOuterNotFound = 0,
		UseStructAlways = 1
	};

	/** Untracked pointer to the resolved property */
	mutable FField* ResolvedField = nullptr;
#if WITH_EDITORONLY_DATA
	/** In editor builds, store the original class of the resolved property in case it changes after recompiling BPs */
	mutable FFieldClass* InitialFieldClass = nullptr;
	/** In editor builds, fields may get deleted even though their owner struct remains */
	mutable int32 FieldPathSerialNumber = 0;
#endif
	/** The cached owner of this field. Even though implemented as a weak pointer, GC will keep a strong reference to it if exposed through UPROPERTY macro */
	mutable TWeakObjectPtr<UStruct> ResolvedOwner;

	/** Path to the FField object from the innermost FField to the outermost UObject (UPackage) */
	TArray<FName> Path;

	inline bool NeedsResolving() const
	{
		if (ResolvedField)
		{
#if WITH_EDITORONLY_DATA
			UStruct* Owner = ResolvedOwner.Get();
			// In uncooked builds we also need to check if the serial number on the owner struct is identical
			// It will change if the struct has been recompiled or its properties have been destroyed
			if (Owner && IsFieldPathSerialNumberIdentical(Owner))
			{
				return false;
			}
#else
			// The assumption is that if we already resolved a field and its owner is still valid, there's no need to resolve again
			return !ResolvedOwner.IsValid();
#endif // WITH_EDITORONLY_DATA
		}
		return true;
	}

	/** Clears the cached value so that the next time Get() is called, it will be resolved again */
	inline void ClearCachedField() const
	{
		ResolvedField = nullptr;
#if WITH_EDITORONLY_DATA
		InitialFieldClass = nullptr;
		FieldPathSerialNumber = 0;
#endif // WITH_EDITORONLY_DATA
	}

private:

#if WITH_EDITORONLY_DATA
	/** Used to check if the serial number on the provided struct is identical to the one stored in this FFieldPath */
	COREUOBJECT_API bool IsFieldPathSerialNumberIdentical(UStruct* InStruct) const;
	/** Gets the serial number stored on the provided struct */
	COREUOBJECT_API int32 GetFieldPathSerialNumber(UStruct* InStruct) const;
#endif

	/** FOR INTERNAL USE ONLY: gets the pointer to the resolved field without trying to resolve it */
	UE_FORCEINLINE_HINT FUObjectItem* GetResolvedOwnerItemInternal()
	{
		return ResolvedOwner.Internal_GetObjectItem();
	}
	inline void ClearCachedFieldInternal()
	{
		ResolvedField = nullptr;
		ResolvedOwner.Reset();
	}

	/**
	 * Tries to resolve the field owner
	 * @param InCurrentStruct Struct that's trying to resolve this field path
	 * @param InResolveType Type of the resolve operation
	 * @return Resolved owner struct
	 */
	COREUOBJECT_API UStruct* TryToResolveOwnerFromStruct(UStruct* InCurrentStruct = nullptr, EPathResolveType InResolveType = FFieldPath::UseStructIfOuterNotFound) const;
	

	/**
	 * Tries to resolve the field owner
	 * @param InLinker the current linker load serializing this field path
	 * @return Resolved owner struct
	 */
	COREUOBJECT_API UStruct* TryToResolveOwnerFromLinker(FLinkerLoad* InLinker) const;


	/**
	 * Tries to convert the full path stored in this FFieldPath to the new format (Owner reference + path to the field)
	 * @param InLinker the current linker load serializing this field path
	 * @return Resulved owner struct
	 */
	COREUOBJECT_API UStruct* ConvertFromFullPath(FLinkerLoad* InLinker);

public:

	FFieldPath() = default;

	FFieldPath(FField* InField)
	{
		Generate(InField);
	}

#if WITH_EDITORONLY_DATA
	COREUOBJECT_API FFieldPath(UField* InField, const FName& InPropertyTypeName);
#endif

	/** Generates path from the passed in field pointer */
	COREUOBJECT_API void Generate(FField* InField);

	/** Generates path from the passed in field pointer */
	COREUOBJECT_API void Generate(const TCHAR* InFieldPathString);

#if WITH_EDITORONLY_DATA
	COREUOBJECT_API void GenerateFromUField(UField* InField);
#endif

	/**
	 * Tries to resolve the path without caching the resolved pointer 
	 * @param InCurrentStruct Struct that's trying to resolve this field path
	 * @param OutOwnerIndex ObjectIndex of the Owner UObject
	 * @return Resolved field or null
	 */
	COREUOBJECT_API FField* TryToResolvePath(UStruct* InCurrentStruct, EPathResolveType InResolveType = FFieldPath::UseStructIfOuterNotFound) const;

	/**
	 * Tries to resolve the path and caches the result
	 * @param ExpectedClass Expected class of the resolved field
	 * @param InCurrentStruct Struct that's trying to resolve this field path	 
	 */
	inline void ResolveField(FFieldClass* ExpectedClass = FField::StaticClass(), UStruct* InCurrentStruct = nullptr, EPathResolveType InResolveType = FFieldPath::UseStructIfOuterNotFound) const
	{
		FField* FoundField = TryToResolvePath(InCurrentStruct, InResolveType);
		if (FoundField && FoundField->IsA(ExpectedClass) 
#if WITH_EDITORONLY_DATA
			&& (!InitialFieldClass || FoundField->IsA(InitialFieldClass))
#endif // WITH_EDITORONLY_DATA
			)
		{
			ResolvedField = FoundField;
#if WITH_EDITORONLY_DATA
			if (!InitialFieldClass)
			{
				InitialFieldClass = FoundField->GetClass();
			}
			UStruct* Owner = ResolvedOwner.Get();
			check(Owner);
			FieldPathSerialNumber = GetFieldPathSerialNumber(Owner);
#endif // WITH_EDITORONLY_DATA
		}
		else if (ResolvedField)
		{
			// In case this field has been previously resolved, clear the owner as well as it's impossible the original field
			// will ever come back (it's most likely been deleted) and we don't want to resolve to a newly created one even if its name and class match
			ResolvedOwner.Reset();
			ResolvedField = nullptr;
		}
	}

	/**
	 * Gets the field represented by this FFieldPath
	 * @param ExpectedType Expected type of the resolved field
	 * @param InCurrentStruct Struct that's trying to resolve this field path
	 * @return Field represented by this FFieldPath or null if it couldn't be resolved
	 */
	inline FField* GetTyped(FFieldClass* ExpectedType, UStruct* InCurrentStruct = nullptr) const
	{
		if (NeedsResolving() && Path.Num())
		{
			ResolveField(ExpectedType, InCurrentStruct, FFieldPath::UseStructIfOuterNotFound);
		}
		return ResolvedField;
	}

	/** 
	 * Returns true if the field path is empty (does not test if the owner is valid) 
	 * This is usually used to verify if the reason behind this field being unresolved is because the owner is missing or the property couldn't be found.
	 **/
	inline bool IsPathToFieldEmpty() const
	{
		return !Path.Num();
	}

	/**
	* Slightly different than !IsValid(), returns true if this used to point to a FField, but doesn't any more and has not been assigned or reset in the mean time.
	* @return true if this used to point at a real object but no longer does.
	**/
	inline bool IsStale() const
	{
		return ResolvedField && (!ResolvedOwner.IsValid()
#if WITH_EDITORONLY_DATA
			|| !IsFieldPathSerialNumberIdentical(ResolvedOwner.Get())
#endif // WITH_EDITORONLY_DATA
			);
	}

	/**
	* Reset the weak pointer back to the NULL state
	*/
	inline void Reset()
	{
		ClearCachedField();
		ResolvedOwner.Reset();
		Path.Empty();
	}

	UE_FORCEINLINE_HINT bool UEOpEquals(const FFieldPath& InOther) const
	{
		return ResolvedOwner == InOther.ResolvedOwner && Path == InOther.Path;
	}

	COREUOBJECT_API FString ToString() const;

	COREUOBJECT_API friend FArchive& operator<<(FArchive& Ar, FFieldPath& InOutPropertyPath);

	/** Hash function. */
	[[nodiscard]] inline friend uint32 GetTypeHash(const FFieldPath& InPropertyPath)
	{
		uint32 HashValue = 0;
		for (const FName& PathSegment : InPropertyPath.Path)
		{
			HashValue = HashCombine(HashValue, GetTypeHash(PathSegment));
		}
		return HashValue;
	}
};

template<class PropertyType>
struct TFieldPath : public FFieldPath
{
private:

	// These exists only to disambiguate the two constructors below
	enum EDummy1 { Dummy1 };

public:
	TFieldPath()
	{}
	inline TFieldPath(const TFieldPath& Other)
	{
		//  First refresh the serial number from the other path
		Other.Get();
		// Now that the Other path is refreshed, we can copy from it
		FFieldPath::operator=(Other);
	}
	inline TFieldPath& operator=(const TFieldPath& Other)
	{
		//  First refresh the serial number from the other path
		Other.Get();
		// Now that the Other path is refreshed, we can copy from it
		FFieldPath::operator=(Other);
		return *this;
	}

	/**
	* Construct from a null pointer
	**/
	UE_FORCEINLINE_HINT TFieldPath(TYPE_OF_NULLPTR)
		: FFieldPath()
	{
	}

	/**
	* Construct from a string
	**/
	inline TFieldPath(const TCHAR* InPath)
		: FFieldPath()
	{
		Generate(InPath);
	}

#if WITH_EDITORONLY_DATA
	TFieldPath(UField* InField)
		: FFieldPath(InField, PropertyType::StaticClass()->GetFName())
	{
		// This static assert is in here rather than in the body of the class because we want
		// to be able to define TFieldPath<UUndefinedClass>.
		static_assert(std::is_convertible_v<PropertyType*, const volatile FField*>, "TFieldPath can only be constructed with FField types");
	}
#endif

	/**
	* Construct from an object pointer
	* @param Object object to create a weak pointer to
	**/
	template <
		typename OtherPropertyType
		UE_REQUIRES(std::is_convertible_v<OtherPropertyType*, PropertyType*>)
	>
	inline TFieldPath(OtherPropertyType* InProperty, EDummy1 = Dummy1)
		: FFieldPath((FField*)CastField<PropertyType>(InProperty))
	{
		// This static assert is in here rather than in the body of the class because we want
		// to be able to define TFieldPath<UUndefinedClass>.
		static_assert(std::is_convertible_v<PropertyType*, const volatile FField*>, "TFieldPath can only be constructed with FField types");
	}

	/**
	* Construct from another weak pointer of another type, intended for derived-to-base conversions
	* @param Other weak pointer to copy from
	**/
	template <
		typename OtherPropertyType
		UE_REQUIRES(std::is_convertible_v<OtherPropertyType*, PropertyType*>)
	>
	inline TFieldPath(const TFieldPath<OtherPropertyType>& Other)
		: FFieldPath(Other)
	{
		// This static assert is in here rather than in the body of the class because we want
		// to be able to define TFieldPath<UUndefinedClass>.
		static_assert(std::is_convertible_v<PropertyType*, const volatile FField*>, "TFieldPath can only be constructed with FField types");
	}

	/**
	* Copy from an object pointer
	* @param Object object to create a weak pointer to
	**/
	template <
		typename OtherPropertyType
		UE_REQUIRES(std::is_convertible_v<OtherPropertyType*, PropertyType*>)
	>
	inline void operator=(OtherPropertyType* InProperty)
	{
		// This (FField*) cast is effectively a const_cast, as we've already validated the convertibility in the
		// constraint, and that PropertyType is an FField type in the constructors.
		ResolvedField = (FField*)InProperty; 
		Generate(ResolvedField);
	}

	/**
	* Assign from another weak pointer, intended for derived-to-base conversions
	* @param Other weak pointer to copy from
	**/
	template <
		typename OtherPropertyType
		UE_REQUIRES(std::is_convertible_v<OtherPropertyType*, PropertyType*>)
	>
	inline void operator=(const TFieldPath<OtherPropertyType>& Other)
	{
		// First make sure the Other path has the serial number up to date, otherwise we'll keep having to
		// reevealuate this path because it gets the serial number copied from the Other path
		Other.Get();
		// Now that the Other path is refreshed, we can copy from it
		FFieldPath::operator=(Other);
	}

	/**
	 * Gets the field represented by this TFieldPath
	 * @param InCurrentStruct Struct that's trying to resolve this field path
	 * @return Field represented by this FFieldPath or null if it couldn't be resolved
	 */
	UE_FORCEINLINE_HINT PropertyType* Get(UStruct* InCurrentStruct = nullptr) const
	{
		return (PropertyType*)GetTyped(PropertyType::StaticClass(), InCurrentStruct);
	}

	inline PropertyType* ResolveWithRenamedStructPackage(UStruct* InCurrentStruct)
	{
		ClearCachedField();
		ResolveField(PropertyType::StaticClass(), InCurrentStruct, FFieldPath::UseStructAlways);
		return static_cast<PropertyType*>(ResolvedField);
	}

	/**
	* Dereference the weak pointer
	**/
	UE_FORCEINLINE_HINT PropertyType* operator*() const
	{
		return Get();
	}

	/**
	* Dereference the weak pointer
	**/
	UE_FORCEINLINE_HINT PropertyType* operator->() const
	{
		return Get();
	}

	template <UE::CPointerEqualityComparableWith<PropertyType> OtherPropertyType>
	bool UEOpEquals(const TFieldPath<OtherPropertyType>& Other) const
	{
		return FFieldPath::UEOpEquals(Other);
	}

	template <UE::CPointerEqualityComparableWith<PropertyType> T>
	bool UEOpEquals(const T* Other) const
	{
		return Get() == Other;
	}

	bool UEOpEquals(TYPE_OF_NULLPTR) const
	{
		return !Get();
	}
};

// Helper function which deduces the type of the initializer
template <typename PropertyType>
UE_FORCEINLINE_HINT TFieldPath<PropertyType> MakePropertyPath(PropertyType* Ptr)
{
	return TFieldPath<PropertyType>(Ptr);
}


template<class T> struct TIsPODType<TFieldPath<T> > { enum { Value = true }; };
template<class T> struct TIsZeroConstructType<TFieldPath<T> > { enum { Value = true }; };
template<class T> struct TIsWeakPointerType<TFieldPath<T> > { enum { Value = true }; };


/**
* MapKeyFuncs for TFieldPath which allow the key to become stale without invalidating the map.
*/
template <typename KeyType, typename ValueType, bool bInAllowDuplicateKeys = false>
struct TPropertyPathMapKeyFuncs : public TDefaultMapKeyFuncs<KeyType, ValueType, bInAllowDuplicateKeys>
{
	typedef typename TDefaultMapKeyFuncs<KeyType, ValueType, bInAllowDuplicateKeys>::KeyInitType KeyInitType;

	static UE_FORCEINLINE_HINT bool Matches(KeyInitType A, KeyInitType B)
	{
		return A == B;
	}

	static UE_FORCEINLINE_HINT uint32 GetKeyHash(KeyInitType Key)
	{
		return GetTypeHash(Key);
	}
};
