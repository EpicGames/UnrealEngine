// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/Field.h"
#include "UObject/FieldPath.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

class FArchive;
class FOutputDevice;
class UClass;
class UField;
class UObject;
class UStruct;
namespace UECodeGen_Private { struct FFieldPathPropertyParams; }
struct FPropertyTag;

class FFieldPathProperty : public TProperty<FFieldPath, FProperty>
{
	DECLARE_FIELD_API(FFieldPathProperty, (TProperty<FFieldPath, FProperty>), CASTCLASS_FFieldPathProperty, COREUOBJECT_API)

public:

	typedef Super::TTypeFundamentals TTypeFundamentals;
	typedef TTypeFundamentals::TCppType TCppType;

	COREUOBJECT_API FFieldPathProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FFieldPathProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FFieldPathProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FFieldPathProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	COREUOBJECT_API FFieldPathProperty(FFieldVariant InOwner, const UECodeGen_Private::FFieldPathPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	consteval FFieldPathProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, FFieldClass* InPropertyClass)
		: Super(InBaseParams)
		, PropertyClass(InPropertyClass)
	{
	}
#if !IS_MONOLITHIC
	consteval FFieldPathProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams, const UTF8CHAR* InPropertyClass)
		: Super(InBaseParams)
		, PropertyClassName(InPropertyClass)
	{
	}
#endif // !IS_MONOLITHIC
#endif // UE_WITH_CONSTINIT_UOBJECT

#if WITH_EDITORONLY_DATA
	COREUOBJECT_API explicit FFieldPathProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	union
	{
		FFieldClass* PropertyClass = nullptr;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
		// Until LinkCompiledInPointerFields holds the name of the FFieldClass that should be assigned to PropertyClass.
		// TODO: This could be linked via an encoded pointer like objects pointers, but UHT does not parse FFieldClass.
		// Also note that this pointer cannot be distinguished from PropertyClass unlike other fields handled by LinkCompiledInPointerFields
		const UTF8CHAR* PropertyClassName;
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	};

#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	COREUOBJECT_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC

	// UHT interface
	COREUOBJECT_API virtual FString GetCPPMacroType(FString& ExtendedTypeText) const  override;
	COREUOBJECT_API virtual FString GetCPPType(FString* ExtendedTypeText = nullptr, uint32 CPPExportFlags = 0) const override;
	// End of UHT interface

	// FProperty interface
	COREUOBJECT_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	COREUOBJECT_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags ) const override;
	COREUOBJECT_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;
	COREUOBJECT_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	COREUOBJECT_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	COREUOBJECT_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	COREUOBJECT_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	COREUOBJECT_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
	COREUOBJECT_API virtual bool SupportsNetSharedSerialization() const override;
	// End of FProperty interface

	static COREUOBJECT_API FString RedirectFieldPathName(const FString& InPathName);
};
