// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EnumClassFlags.h"

enum class EFindObjectFlags
{
	None,

	/** Whether to require an exact match with the passed in class */
	ExactClass
};
ENUM_CLASS_FLAGS(EFindObjectFlags)

#define UE_EXACTCLASS_BOOL_DEPRECATED(FunctionName) UE_DEPRECATED(5.7, FunctionName " with a boolean ExactClass has been deprecated - please use the EFindObjectFlags enum instead")

enum class EGetObjectsFlags
{
	None = 0,

	/** If specified, then objects whose outers directly or indirectly have Outer as an outer are included, these are the nested objects */
	IncludeNestedObjects = 1 << 0,

	/** Whether to include Unreachable objects (unsafe, only allowed inside of the garbage collector) */
	EvenIfUnreachable = 1 << 1
};
ENUM_CLASS_FLAGS(EGetObjectsFlags)

#define UE_INCLUDENESTEDOBJECTS_BOOL_DEPRECATED(FunctionName) UE_DEPRECATED(5.8, FunctionName " with a boolean bIncludeNestedObjects has been deprecated - please use the EGetObjectsFlags enum instead")
