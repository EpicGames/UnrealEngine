// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GCObject.h: Abstract base class to allow non-UObject objects reference
				UObject instances with proper handling of them by the
				Garbage Collector.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "Concepts/DerivedFrom.h"
#include "Traits/IsTriviallyRelocatable.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "GCObject.generated.h"

class FGCObject;

/**
 * This nested class is used to provide a UObject interface between non
 * UObject classes and the UObject system. It handles forwarding all
 * calls of AddReferencedObjects() to objects/ classes that register with it.
 */
UCLASS(Transient, MinimalAPI)
class UGCObjectReferencer : public UObject
{
	struct FImpl;
	TUniquePtr<FImpl> Impl;

	/** Current FGCObject* that references are being added from  */
	FGCObject* CurrentlySerializingObject = nullptr;

	friend struct FReplaceReferenceHelper;

public:
	GENERATED_BODY()

	COREUOBJECT_API UGCObjectReferencer(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	COREUOBJECT_API UGCObjectReferencer(FVTableHelper& Helper);
	COREUOBJECT_API ~UGCObjectReferencer();

	/**
	 * Adds an object to the referencer list
	 *
	 * @param Object The object to add to the list
	 */
	COREUOBJECT_API void AddObject(FGCObject* Object);

	/**
	 * Removes an object from the referencer list
	 *
	 * @param Object The object to remove from the list
	 */
	COREUOBJECT_API void RemoveObject(FGCObject* Object);

	/**
	 * Get the name of the first FGCObject that owns this object.
	 *
	 * @param Object The object that we're looking for.
	 * @param OutName the name of the FGCObject that reports this object.
	 * @param bOnlyIfAddingReferenced Only try to find the name if we are currently inside AddReferencedObjects
	 * @return true if the object was found.
	 */
	COREUOBJECT_API bool GetReferencerName(UObject* Object, FString& OutName, bool bOnlyIfAddingReferenced = false) const;

	/**
	 * Forwards this call to all registered objects so they can reference
	 * any UObjects they depend upon
	 *
	 * @param InThis This UGCObjectReferencer object.
	 * @param Collector The collector of referenced objects.
	 */
	COREUOBJECT_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	void AddInitialReferences(TArray<UObject**>& Out);
	/**
	 * Destroy function that gets called before the object is freed. This might
	 * be as late as from the destructor.
	 */
	virtual void FinishDestroy() override;

	/**
	 * Returns the currently serializing object
	 */
	FGCObject* GetCurrentlySerializingObject() const
	{
		return CurrentlySerializingObject;
	}

#if WITH_EDITORONLY_DATA
	/** Called when a new FGCObject is added to this referencer */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FGCObjectAddedDelegate, FGCObject*);

private:

	/** Called when a new FGCObject is added to this referencer */
	FGCObjectAddedDelegate OnGCObjectAdded;

public:

	/** Returns a delegate called when a new FGCObject is added to this referencer */
	FGCObjectAddedDelegate& GetGCObjectAddedDelegate()
	{
		return OnGCObjectAdded;
	}
#endif // WITH_EDITORONLY_DATA
};


/**
 * This class provides common registration for garbage collection for
 * non-UObject classes. It is an abstract base class requiring you to implement
 * the AddReferencedObjects() method.
 *
 * Note: FGCObject instances are not trivially relocatable, and are therefore not
 * suited for being stored in Unreal container types by value.
 * For example, this won't work:
 * class FMyClass : public FGCObject {};
 * struct FSomeStruct {
 *     TArray<FMyClass> Instances; // Storing anything here might crash GC at 
 *                                 // reachability analysis
 * };
 */
class FGCObject
{
public:
	/**
	 * The static object referencer object that is shared across all
	 * garbage collectible non-UObject objects.
	 */
	static COREUOBJECT_API UGCObjectReferencer* GGCObjectReferencer;

	/** Initializes the global object referencer and adds it to the root set. */
	static COREUOBJECT_API void StaticInit();

	/**
	 * Tells the global object that forwards AddReferencedObjects calls on to objects
	 * that a new object is requiring AddReferencedObjects call.
	 */
	FGCObject()
	{
		RegisterGCObject();
	}

	FGCObject(const FGCObject& Other)
	{
		RegisterGCObject();
	}

	FGCObject(FGCObject&& Other)
	{
		RegisterGCObject();
	}

	enum class EFlags : uint32
	{
		None = 0,

		/** Manually call RegisterGCObject() later to avoid collecting references from empty / late-initialized FGCObjects */
		RegisterLater = 1 << 0,

		/**
		 * Declare that AddReferencedObjects *only* calls FReferenceCollector::AddStableReference*() functions
		 * instead of the older FReferenceCollector::AddReferenceObject*() and only adds native references.
		 * 
		 * Allows gathering initial references before reachability analysis starts.
		 */
		AddStableNativeReferencesOnly = 1 << 1,
	};
	explicit FGCObject(EFlags Flags);

	virtual ~FGCObject()
	{
		UnregisterGCObject();
	}

	FGCObject& operator=(const FGCObject&) {return *this;}
	FGCObject& operator=(FGCObject&&) {return *this;}

	/** Register with GC, only needed if constructed with EFlags::RegisterLater or after unregistering */
	COREUOBJECT_API void RegisterGCObject();

	/** Unregister ahead of destruction. Safe to call multiple times. */
	COREUOBJECT_API void UnregisterGCObject();

	/**
	 * Pure virtual that must be overloaded by the inheriting class. Use this
	 * method to serialize any UObjects contained that you wish to keep around.
	 *
	 * @param Collector The collector of referenced objects.
	 */
	virtual void AddReferencedObjects( FReferenceCollector& Collector ) = 0;

	/** Overload this method to report a name for your referencer */
	virtual FString GetReferencerName() const = 0;

	/** Overload this method to report how the specified object is referenced, if necessary */
	virtual bool GetReferencerPropertyName(UObject* Object, FString& OutPropertyName) const
	{
		return false;
	}

private:
	friend UGCObjectReferencer;
	static COREUOBJECT_API const TCHAR* UnknownGCObjectName;

	const bool bCanMakeInitialReferences = false;
	bool bReferenceAdded = false;	
};

ENUM_CLASS_FLAGS(FGCObject::EFlags);

inline FGCObject::FGCObject(EFlags Flags)
: bCanMakeInitialReferences(EnumHasAllFlags(Flags, EFlags::AddStableNativeReferencesOnly))
{
	if (!EnumHasAnyFlags(Flags, EFlags::RegisterLater))
	{
		RegisterGCObject();
	}
}

// Types derived from FGCObject are not trivially relocatable, so flag their use in containers and other contexts.
template <UE::CDerivedFrom<FGCObject> T>
struct TIsTriviallyRelocatable<T>
{
	static inline constexpr bool Value = false;
};
