// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GarbageCollectionHistory.h: Unreal realtime garbage collection history
=============================================================================*/

#pragma once

#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeHash.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollection.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectArray.h"

/**
 * Structure containing information about a UObject participating in Garbage Collection.
 * It's purpose is to avoid holding onto direct references to UObjects which may have already been Garbage Collected.
 * FGCObjectInfo interface mimics that of UObject.
 **/
class FGCObjectInfo
{
public:

	FGCObjectInfo() = default;
	explicit FGCObjectInfo(const UObject* Obj)
		: Name(Obj->GetFName())
		, Flags(Obj->GetFlags())
		, InternalFlags(Obj->GetInternalFlags())
		, ClusterRootIndex(GUObjectArray.ObjectToObjectItem(Obj)->GetOwnerIndex())
		, bDisregardForGC(GUObjectArray.IsDisregardForGC(Obj))
		, RefCount(GUObjectArray.ObjectToObjectItem(Obj)->GetRefCount())
	{

	}

private:

	/** Name of the object */
	FName Name;
	/** Pointer to class info */
	FGCObjectInfo* Class = nullptr;
	/** Pointer to Outer info */
	FGCObjectInfo* Outer = nullptr;
	/** Captured Object flags */
	EObjectFlags Flags = RF_NoFlags;
	/** Captured Internal flags */
	EInternalObjectFlags InternalFlags = EInternalObjectFlags::None;
	/** Object's FGCObjectItem cluster root index */
	int32 ClusterRootIndex = -1;
	/** True if the object was inside of the disregard for GC set */
	bool bDisregardForGC = false;
	/** Current ref-count of the UObject. */
	int32 RefCount = 0;

public:

	/**
	 * Tries to find an existing object matching the stored Path.
	 * @returns Pointer to an object this info struct represents if the object is still alive, nullptr otherwise
	 */
	COREUOBJECT_API UObject* TryResolveObject();

	FName GetName() const
	{
		return Name;
	}

	FGCObjectInfo* GetClass() const
	{
		return Class;
	}

	FGCObjectInfo* GetOuter() const
	{
		return Outer;
	}

	inline bool IsIn(const FGCObjectInfo* MaybeOuter) const
	{		
		for (const FGCObjectInfo* Obj = this; Obj; Obj = Obj->Outer)
		{
			if (Obj == MaybeOuter)
			{
				return true;
			}
		}
		return false;
	}

	bool HasAnyFlags(EObjectFlags InFlags) const
	{
		return !!(Flags & InFlags);
	}

	bool HasAnyInternalFlags(EInternalObjectFlags InFlags) const
	{
		return !!(InternalFlags & InFlags);
	}

	bool IsRooted() const
	{
		return HasAnyInternalFlags(EInternalObjectFlags::RootSet);
	}

	bool IsNative() const
	{
		return HasAnyInternalFlags(EInternalObjectFlags::Native);
	}

	bool IsGarbage() const
	{
		return HasAnyInternalFlags(EInternalObjectFlags::Garbage);
	}

	int32 GetOwnerIndex() const
	{
		return ClusterRootIndex;
	}

	FString GetClassName() const
	{
		check(Class);
		return Class->Name.ToString();
	}

	int32 GetRefCount() const
	{
		return RefCount;
	}

	bool IsValid() const
	{
		return !HasAnyInternalFlags(EInternalObjectFlags::Garbage);
	}

	bool IsDisregardForGC() const
	{
		return bDisregardForGC;
	}

	COREUOBJECT_API void GetPathName(FStringBuilderBase& ResultString) const;
	COREUOBJECT_API FString GetPathName() const;
	COREUOBJECT_API FString GetFullName() const;

	friend uint32 GetTypeHash(const FGCObjectInfo& Info)
	{
		return HashCombine(GetTypeHash(Info.Outer), GetTypeHash(Info.Name));
	}

	/** Helper function for adding info about an UObject into UObject to FGCObjectInfo map */
	static COREUOBJECT_API FGCObjectInfo* FindOrAddInfoHelper(const UObject* InObject, TMap<const UObject*, FGCObjectInfo*>& InOutObjectToInfoMap);
};
