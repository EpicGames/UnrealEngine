// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/GCObject.h"

/**
 * Specific implementation of FGCObject that prevents a single UObject-based pointer from being GC'd while this guard is in scope.
 * @note This is the lean version of TStrongObjectPtr which uses an inline FGCObject so *can't* safely be used with containers that treat types as trivially relocatable.
 */
template <typename ObjectType>
class TGCObjectScopeGuard : public FGCObject
{
	static_assert(TPointerIsConvertibleFromTo<ObjectType, const UObjectBase>::Value, "TGCObjectScopeGuard: ObjectType must be pointers to a type derived from UObject");

public:
	explicit TGCObjectScopeGuard(ObjectType* InObject)
		: Object(InObject)
	{
	}

	virtual ~TGCObjectScopeGuard() = default;

	/** Non-copyable */
	TGCObjectScopeGuard(const TGCObjectScopeGuard&) = delete;
	TGCObjectScopeGuard& operator=(const TGCObjectScopeGuard&) = delete;

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(Object);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("TGCObjectScopeGuard");
	}

	UE_FORCEINLINE_HINT ObjectType* Get() const
	{
		return Object;
	}

private:
	TObjectPtr<ObjectType> Object;
};

using FGCObjectScopeGuard = TGCObjectScopeGuard<const UObject>;

/**
 * Specific implementation of FGCObject that prevents an array of UObject-based pointers from being GC'd while this guard is in scope.
 */
template <typename ObjectType>
class TGCObjectsScopeGuard : public FGCObject
{
	static_assert(TPointerIsConvertibleFromTo<ObjectType, const UObjectBase>::Value, "TGCObjectsScopeGuard: ObjectType must be pointers to a type derived from UObject");

public:
	explicit TGCObjectsScopeGuard(const TArray<ObjectType*>& InObjects)
		: Objects(InObjects)
	{
	}

	virtual ~TGCObjectsScopeGuard() = default;

	/** Non-copyable */
	TGCObjectsScopeGuard(const TGCObjectsScopeGuard&) = delete;
	TGCObjectsScopeGuard& operator=(const TGCObjectsScopeGuard&) = delete;

	//~ FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObjects(Objects);
	}

	virtual FString GetReferencerName() const override
	{
		return TEXT("TGCObjectsScopeGuard");
	}

	UE_FORCEINLINE_HINT const TArray<ObjectType*>& Get() const
	{
		return Objects;
	}

private:
	TArray<TObjectPtr<ObjectType>> Objects;
};
