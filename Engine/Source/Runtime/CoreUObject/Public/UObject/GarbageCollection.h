// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GarbageCollection.h: Unreal realtime garbage collection helpers
=============================================================================*/

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Stats/Stats.h"
#include "UObject/ReferenceToken.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

class UObject;

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
namespace Verse
{
struct VNativeStruct;
}
#endif

/** Context sensitive keep flags for garbage collection */
#define GARBAGE_COLLECTION_KEEPFLAGS	(GIsEditor ? RF_Standalone : RF_NoFlags)

/** UObject pointer checks are disabled by default in shipping and test builds as they add roughly 20% overhead to GC times */
#ifndef ENABLE_GC_OBJECT_CHECKS
	#define ENABLE_GC_OBJECT_CHECKS (!(UE_BUILD_TEST || UE_BUILD_SHIPPING) || 0)
#endif

#define ENABLE_GC_HISTORY (!UE_BUILD_SHIPPING)

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogGarbage, Warning, All);
DECLARE_STATS_GROUP(TEXT("Garbage Collection"), STATGROUP_GC, STATCAT_Advanced);

/**
 * Do extra checks on GC'd function references to catch uninitialized pointers?
 * These checks are possibly producing false positives now that our memory use is going over 128Gb = 2^39.
 */
#define DO_POINTER_CHECKS_ON_GC 0 


namespace UE::GC
{

enum class EAROFlags
{
	None			= 0,
	Unbalanced		= 1 << 0,		// Some instances are very slow but most are fast. GC can flush these more frequently.
	ExtraSlow		= 2 << 0,		// All instances are slow. GC can work-steal these at finer batch granularity.
};
ENUM_CLASS_FLAGS(EAROFlags);

// Reference collection batches up slow AddReferencedObjects calls
COREUOBJECT_API void RegisterSlowImplementation(void (*AddReferencedObjects)(UObject*, FReferenceCollector&), EAROFlags Flags = EAROFlags::None);

class FSchemaView;

/** Type-erasing owner of a ref-counted FSchemaView */
class FSchemaOwner
{
public:
	FSchemaOwner() = default;
	FSchemaOwner(const FSchemaOwner&) = delete;
	FSchemaOwner(FSchemaOwner&& In) : SchemaView(In.SchemaView) { In.SchemaView = 0; }
	COREUOBJECT_API explicit FSchemaOwner(FSchemaView In);
	~FSchemaOwner() { Reset(); }
	FSchemaOwner& operator=(const FSchemaOwner&) = delete;
	FSchemaOwner& operator=(FSchemaOwner&& In)
	{
		Reset();
		Swap(In.SchemaView, SchemaView);
		return *this;
	}
	
	COREUOBJECT_API void Set(FSchemaView In);
	const FSchemaView& Get() const { return reinterpret_cast<const FSchemaView&>(SchemaView); }
	COREUOBJECT_API void Reset();
private:
	uint64 SchemaView = 0;
};

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
/**
* Returns true if the "FrankenGC" is enabled.
*/
COREUOBJECT_API bool IsFrankenGCEnabled();

/**
* Enable/Disable merger of UE and Verse GC
*
* @parm bEnable If true, enable franken GC
*/
AUTORTFM_DISABLE COREUOBJECT_API void EnableFrankenGCMode(bool bEnable);

/**
* Test to see if franken GC needs to run
*/
AUTORTFM_DISABLE COREUOBJECT_API bool ShouldFrankenGCRun();

/** True during the whole time that Franken GC is collecting from both Verse and UE */
extern COREUOBJECT_API bool GIsFrankenGCCollecting;

/**
* Report a Verse native struct that needs to be examined further
*/
COREUOBJECT_API void MarkNativeStructAsReachable(::Verse::VNativeStruct* NativeStruct);
#endif

} // namespace UE::GC

/** Prevent GC from running in the current scope */
class FGCScopeGuard
{
public:
	COREUOBJECT_API FGCScopeGuard();
	COREUOBJECT_API ~FGCScopeGuard();
};

class FGCScopeTryGuard
{
public:
	COREUOBJECT_API FGCScopeTryGuard();
	COREUOBJECT_API ~FGCScopeTryGuard();

	bool LockSucceeded() const { return bLockSuccessful;}

private:
	bool bLockSuccessful = false;
};

class FGCObject;

/** Information about references to objects marked as Garbage that's gather by the Garbage Collector */
struct FGarbageReferenceInfo
{
	/** Object marked as garbage */
	UObject* GarbageObject;
	/** Referencing object info */
	union FReferencerUnion
	{
		/** Referencing UObject */
		const UObject* Object;
		/** Referencing FGCObject */
		FGCObject* GCObject;
	} Referencer;
	/** True if the referencing object is a UObject. If false the referencing object is an FGCObject */
	bool bReferencerUObject;
	/** Referencing property name */
	FName PropertyName;

	FGarbageReferenceInfo(const UObject* InReferencingObject, UObject* InGarbageObject, FName InPropertyName)
		: GarbageObject(InGarbageObject)
		, bReferencerUObject(true)
		, PropertyName(InPropertyName)
	{
		Referencer.Object = InReferencingObject;
	}
	FGarbageReferenceInfo(FGCObject* InReferencingObject, UObject* InGarbageObject)
		: GarbageObject(InGarbageObject)
		, bReferencerUObject(false)
	{
		Referencer.GCObject = InReferencingObject;
	}

	/** Returns a formatted string with referencing object info */
	FString GetReferencingObjectInfo() const;
};

struct FGCDirectReference
{
	explicit FGCDirectReference(FReferenceToken InReference, FName Name = NAME_None) : ReferencerName(Name), Reference(InReference) {}
	explicit FGCDirectReference(const UObject* Obj, FName Name = NAME_None) : ReferencerName(Name), Reference(Obj) {}
	explicit FGCDirectReference(const Verse::VCell* Cell, FName Name = NAME_None) : ReferencerName(Name), Reference(Cell) {}
	/** Property or FGCObject name referencing this object */
	FName ReferencerName;
	FReferenceToken Reference;
};

/** True if Garbage Collection is running. Use IsGarbageCollecting() functio n instead of using this variable directly */
extern COREUOBJECT_API TSAN_ATOMIC(bool) GIsGarbageCollecting;

/**
 * Gets the last time that the GC was run.
 *
 * @return	Returns the FPlatformTime::Seconds() for the last garbage collection, 0 if GC has never run.
 */
COREUOBJECT_API double GetLastGCTime();

/**
 * Gets the duration of the last GC run.
 *
 * @return	Returns the last GC duration, -1 if GC has never run.
 */
COREUOBJECT_API double GetLastGCDuration();

/**
* Whether we are inside garbage collection
*/
UE_FORCEINLINE_HINT bool IsGarbageCollecting()
{
	return GIsGarbageCollecting;
}

/**
* Whether garbage collection is locking the global uobject hash tables
*/
COREUOBJECT_API bool IsGarbageCollectingAndLockingUObjectHashTables();

namespace UE::GC::Private
{
	COREUOBJECT_API void SetSkipDestroy(bool bSkip);
	COREUOBJECT_API void RemoveFromRootForUnload(TSet<const UObject*> ObjectsToRemove);
}
