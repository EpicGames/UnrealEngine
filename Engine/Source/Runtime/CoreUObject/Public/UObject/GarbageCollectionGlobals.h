// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GarbageCollectionGlobals.h: Garbage Collection Global State Vars
=============================================================================*/

#pragma once

#include "UObject/ObjectMacros.h"

namespace UE::GC
{
	/** true if incremental reachability analysis is in progress (global for faster access in low level structs and functions otherwise use IsIncrementalReachabilityAnalisysPending()) */
	extern COREUOBJECT_API TSAN_ATOMIC(bool) GIsIncrementalReachabilityPending;
}
