// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GarbageCollectionHistory.h: Unreal realtime garbage collection history
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "UObject/FastReferenceCollector.h"
#include "UObject/GCObjectInfo.h"
#include "UObject/GarbageCollection.h"
#include "UObject/NameTypes.h"
#include "UObject/ReferenceToken.h"

class FGCObjectInfo;
class UObject;

#if ENABLE_GC_HISTORY

/**
 * Structure that holds all direct references traversed in a GC run as well as FGCObjectInfo structs created for all participating objects
 **/
struct FGCSnapshot
{
	/** FGCObjectInfo structs generated for each of the objects encountered during GC */
	TMap<const UObject*, FGCObjectInfo*> ObjectToInfoMap;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/** FGCVerseCellInfo structs generated for each of the cells encountered during GC */
	TMap<const Verse::VCell*, FGCVerseCellInfo*> VerseCellToInfoMap;
#endif
	/** List of direct references for all objects */
	TMap<FReferenceToken, TArray<FGCDirectReference>*> DirectReferences;

	/** Returns the number of bytes allocated by a single snapshot */
	int64 GetAllocatedSize() const;
};

/**
 * Garbage Collector History. Holds snapshots of a number of previous GC runs.
 **/
class FGCHistory
{
	/** Captured snapshots of previous GC runs. This is essentially a preallocated ring buffer. */
	TArray<FGCSnapshot> Snapshots;
	/** Index of the last recorded snapshot in the Snapshots array */
	int32 MostRecentSnapshotIndex = -1;
	/** Object representing GC barrier */
	UObject* GCBarrier = nullptr;

	COREUOBJECT_API ~FGCHistory();

	COREUOBJECT_API void Cleanup(FGCSnapshot& InShapshot);	
	COREUOBJECT_API void MergeArrayStructHistory(TMap<FReferenceToken, TArray<FGCDirectReference>*>& History, FGCSnapshot& Snapshot);
	FReferenceToken GetInfoReferenceToken(FReferenceToken InToken, FGCSnapshot& Snapshot);

public:

	/** Gets the FGCHistory singleton. */
	static COREUOBJECT_API FGCHistory& Get();

	/** Copies information generated by Garbage Collector into the next snapshot struct. */
	COREUOBJECT_API void Update(TConstArrayView<TUniquePtr<UE::GC::FWorkerContext>> Contexts);

	/** Frees memory allocated by GC history. */
	COREUOBJECT_API void Cleanup();

	/** 
	 * Sets how many GC runs the history can record. 
	 * @param HistorySize The number of GC runs the history can record. If 0 GC history will be disabled.
	 */
	COREUOBJECT_API void SetHistorySize(int32 HistorySize);

	UE_FORCEINLINE_HINT int32 GetHistorySize() const
	{
		return Snapshots.Num();
	}

	UE_FORCEINLINE_HINT bool IsActive() const
	{
		return GetHistorySize() > 0;
	}

	/**
	 * Gets a snapshot of a previous GC run.
	 * @param HistoryLevel Index of a previous run: 0 - the last one, 1 - the one before and up to (GetHistorySize() - 1). Negative values are also accepted.
	 * @returns Pointer to a prevous GC run snapshot or null if there's not been that many recorded GC runs yet
	 */
	COREUOBJECT_API FGCSnapshot* GetSnapshot(int32 HistoryLevel);

	/**
	 * Returns the snapshot of the last GC run. See GetSnapshot(int32)
	 **/
	FGCSnapshot* GetLastSnapshot()
	{
		return GetSnapshot(0);
	}

	/** Returns the number of bytes allocated by GC history */
	COREUOBJECT_API int64 GetAllocatedSize() const;

	/** Returns an object representing GC Barrier */
	UE_FORCEINLINE_HINT UObject* GetBarrierObject() const
	{
		return GCBarrier;
	}
};

#endif // ENABLE_GC_HISTORY