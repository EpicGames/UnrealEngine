// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Misc/Optional.h"
#include "UObject/GarbageCollection.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/VerseTypesFwd.h"

#if (WITH_VERSE_VM && DO_GUARD_SLOW) || defined(__INTELLISENSE__)
#include "VerseVM/VVMRestValue.h"
#endif

#ifndef UE_GC_DEBUGNAMES
#define UE_GC_DEBUGNAMES (!UE_BUILD_SHIPPING)
#endif

namespace UE::GC {

/** Describes member variable types, AddReferencedObject calls and stop/jump instructions used by VisitMembers */
enum class EMemberType : uint8
{
	Stop,								// Null terminator
	Jump,								// Move base pointer forward to reach members at large offsets
	Reference,							// Member - Scalar reference, e.g. MyObject* and TObjectPtr<MyObject>
	ReferenceArray,						// Member - Array of references, e.g. TArray<MyObject*> and TArray<TObjectPtr<MyObject>>
	StructArray,						// Array of structs
	StridedArray,						// Array of structs with single reference per struct (~half of struct instances)
	StructSet,							// TMap/TSet of structs
	FieldPath,							// Field path strong reference to owner
	FieldPathArray,					 	// Array of field paths
	FreezableReferenceArray,			// Freezable array of references
	FreezableStructArray,				// Freezable array of structs
	Optional,							// TOptional
	DynamicallyTypedValue,				// FDynamicallyTypedValue
	ARO,								// Call Add[Struct]ReferencedObjects() on current object / struct
	SlowARO,							// Call or queue AddReferencedObjects() on current object
	MemberARO,							// Call AddStructReferencedObjects() on a struct member in current object / struct
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	VerseValue,							// Member - Verse value
	VerseValueArray,					// Member - Verse value array
#endif
	Count
};

COREUOBJECT_API FName ToName(EMemberType Type);

/** Declares if a schema represents a blueprint generated type */
enum class EOrigin : uint8
{
	Other,		// Schema reflects non-blueprint type, such as native C++ classes and structs.
	Blueprint	// Schema reflects blueprint type. References to garbage-marked objects are cleared regardless of WithPendingKill to preserve old BP semantics.
};

/////////////////////////////// FPropertyStack ///////////////////////////

/** Helps generate debug property paths for GC schemas and to resolve them back to FProperties */
class FPropertyStack
{
public:
	UE_NONCOPYABLE(FPropertyStack);
	FPropertyStack() = default;
	~FPropertyStack() { check(Props.IsEmpty()); }

	/** Get string representing property stack, e.g. "Member.StructMember.InnerStructMember" */
	COREUOBJECT_API FString GetPropertyPath() const;

	/** 
	 * Converts a property path constructed with GetPropertyPath() to an array of properties (from the outermost to the innermost) *
	 * @param ObjectClass Class that defines the outermost property
	 * @param InPropertyPath Property path
	 * @param OutProperties resulting array
	 * @returns true if the conversion was successful, false otherwise
	 */
	COREUOBJECT_API static bool ConvertPathToProperties(UClass* ObjectClass, FName InPropertyPath, TArray<FProperty*>& OutProperties);
	COREUOBJECT_API static bool ConvertPathToProperties(UClass* ObjectClass, FStringView InPropertyPath, TArray<FProperty*>& OutProperties);
private:
	friend class FPropertyStackScope;
	TArray<const FProperty*> Props;
};

class FPropertyStackScope
{
	FPropertyStack& Stack;
public:
	UE_NONCOPYABLE(FPropertyStackScope);
	FPropertyStackScope(FPropertyStack& InStack, FProperty* Property)
	: Stack(InStack)
	{		
		check(Property);
		Stack.Props.Push(Property);
	}

	~FPropertyStackScope()
	{		
		Stack.Props.Pop();
	}
};

/////////////////////////////// FSchemaView //////////////////////////////

struct FSchemaHeader
{
	uint32 StructStride; // sizeof(T), required to iterate over struct array
	std::atomic<int32> RefCount;
};

union FMemberWord;

/** Describes all strong GC references in a class or struct */
class FSchemaView
{
	static constexpr uint64 OriginBit = 1;
	uint64 Handle;

public:
	FSchemaView() : Handle(0) {}
	FSchemaView(ENoInit) {} 
	FSchemaView(FSchemaView View, EOrigin Origin) : FSchemaView(View.GetWords(), Origin) {}
	explicit FSchemaView(const FMemberWord* Data, EOrigin Origin = EOrigin::Other)
	: Handle(reinterpret_cast<uint64>(Data) | static_cast<uint64>(Origin))
	{
		static_assert(sizeof(Handle) >= sizeof(Data)); //-V568
	} 
	
		
	const FMemberWord* GetWords() const				{ return reinterpret_cast<FMemberWord*>(Handle & ~OriginBit); }
	EOrigin GetOrigin() const						{ return static_cast<EOrigin>(Handle & OriginBit); }
	bool IsEmpty() const							{ return GetWords() == nullptr;}
	void SetOrigin(EOrigin Origin)					{ Handle = (Handle & ~OriginBit) | static_cast<uint64>(Origin); }

	/// @pre !IsEmpty()
	uint32 GetStructStride() const					{ return reinterpret_cast<const FSchemaHeader*>(GetWords())[-1].StructStride; }
	FSchemaHeader& GetHeader();
	FSchemaHeader* TryGetHeader();
};

///////////////////////////// Debug helpers //////////////////////////////

/* Debug id for references that lack a schema member declaration */
enum class EMemberlessId
{
	Collector = 1,
	Class,
	Outer,
	ExternalPackage,
	ClassOuter,
	InitialReference,
	Max = InitialReference
};

/** Debug identifier for a schema index or a memberless reference */
class FMemberId
{
public:
	UE_FORCEINLINE_HINT /* implicit */ FMemberId(EMemberlessId Memberless) : Index(0), MemberlessId((uint32)Memberless) {}
	UE_FORCEINLINE_HINT explicit FMemberId(uint32 Idx) : Index(Idx), MemberlessId(0) {}
	
	bool IsMemberless() const { return MemberlessId != 0; }
	uint32 GetIndex() const { check(!IsMemberless()); return Index; }
	int32 AsPrintableIndex() const { return IsMemberless() ? -int32(MemberlessId) : Index; }
	EMemberlessId AsMemberless() const { check(IsMemberless()); return (EMemberlessId)MemberlessId;}

	friend bool operator==(FMemberId A, FMemberId B) { return reinterpret_cast<uint32&>(A) == reinterpret_cast<uint32&>(B); }
	friend bool operator!=(FMemberId A, FMemberId B) { return reinterpret_cast<uint32&>(A) != reinterpret_cast<uint32&>(B); }

private:
	static constexpr uint32 MemberlessIdBits = 3;
	static constexpr uint32 IndexBits = 32 - MemberlessIdBits;

	uint32 Index : IndexBits;
	uint32 MemberlessId : MemberlessIdBits;

	void StaticAssert();
};

struct FMemberInfo
{
	int32 Offset;
	FName Name;
};

COREUOBJECT_API FMemberInfo GetMemberDebugInfo(FSchemaView Schema, FMemberId Id);

#if !UE_BUILD_SHIPPING
uint32 CountSchemas(uint32& OutNumWords);
#endif

//////////////////// Built FSchemaView representation ////////////////////

struct FMemberPacked
{
	static constexpr uint32 TypeBits = 5;
	static constexpr uint32 OffsetBits = 16 - TypeBits;
	static constexpr uint32 OffsetRange = 1u << FMemberPacked::OffsetBits;

	uint16 Type : TypeBits;
	uint16 WordOffset : OffsetBits;
};

using ObjectAROFn = void (*)(UObject*, FReferenceCollector&);
using StructAROFn = void (*)(void*, FReferenceCollector&);

struct alignas(4) FStridedLayout
{
	uint16 WordOffset;
	uint16 WordStride;
};

union FMemberWord
{
	FMemberPacked Members[4];
	FSchemaView InnerSchema{NoInit};
	ObjectAROFn ObjectARO;
	StructAROFn StructARO;
	FStridedLayout StridedLayout;
};

inline FMemberWord ToWord(FSchemaView In)
{
	FMemberWord Out;
	Out.InnerSchema = In;
	return Out;
}

inline FMemberWord ToWord(StructAROFn In)
{
	FMemberWord Out;
	Out.StructARO = In;
	return Out;
}

//////////////////// Schema declaration and building //////////////////////////

struct FMemberDeclaration
{
    FName DebugName;
	uint32 Offset = 0;
    EMemberType Type = EMemberType::Stop;
	FMemberWord ExtraWord = {{}};
};

inline FName ToName(const char* Name) { return FName(Name); }
inline FName ToName(FPropertyStack& Stack) { return FName(Stack.GetPropertyPath()); }

// Inlined so compiler can drop Name literals when compiling w/o UE_GC_DEBUGNAMES 
template<typename NameType>
FMemberDeclaration DeclareMember(NameType&& Name, uint32 Offset, EMemberType Type)
{
	FName DebugName = UE_GC_DEBUGNAMES ? ToName(Forward<NameType>(Name)) : ToName(Type);
	return { DebugName, Offset, Type };
}

template<typename NameType, typename ExtraWordType>
FMemberDeclaration DeclareMember(NameType&& Name, uint32 Offset, EMemberType Type, ExtraWordType Extra)
{
	FName DebugName = UE_GC_DEBUGNAMES ? ToName(Forward<NameType>(Name)) : ToName(Type);
	return { DebugName, Offset, Type, ToWord(Extra) };
}

class FSchemaBuilder
{
public:
	COREUOBJECT_API explicit FSchemaBuilder(uint32 InStride, std::initializer_list<FMemberDeclaration> InMembers = {});
	COREUOBJECT_API ~FSchemaBuilder();
	
	COREUOBJECT_API void Add(FMemberDeclaration Member);

	// Append all but the last Stop/ARO member
	COREUOBJECT_API void Append(FSchemaView SuperSchema, int32 OffsetForProperties = 0);

	// Emit Stop or ARO and build a view that builder holds reference to. Returns same view if called multiple times.
	COREUOBJECT_API FSchemaView Build(ObjectAROFn ARO = nullptr);

	int32 NumMembers() const { return Members.Num(); }

private:
	TArray<FMemberDeclaration, TInlineAllocator<16>> Members;
	const uint32 StructStride;
	TOptional<FSchemaOwner> BuiltSchema;
};


//////////// Type-safe wrappers for declaring native schemas //////////////

template<class T>
struct TMemberDeclaration : FMemberDeclaration
{
	template<typename... Ts>
	TMemberDeclaration(Ts&&... Args) : FMemberDeclaration(DeclareMember(Forward<Ts>(Args)...)) {}
};

// Type-safe wrapper for declaring native schemas
template<class T>
class TSchemaBuilder : public FSchemaBuilder
{
public:
	TSchemaBuilder(std::initializer_list<TMemberDeclaration<T>> InMembers = {}) : FSchemaBuilder(sizeof(T), reinterpret_cast<std::initializer_list<FMemberDeclaration>&&>(InMembers)) {}
	void Add(TMemberDeclaration<T> Member) { FSchemaBuilder::Add(Member); }
};

///////////////////////// UE_GC_MEMBER + helpers ///////////////////////
namespace Private
{

template<class T, class R>
TMemberDeclaration<T> MakeMember(const char* Name, uint32 Offset, R T::*)
{
	static_assert(std::is_convertible_v<R, UObject*>);
    return TMemberDeclaration<T>(Name, Offset, EMemberType::Reference);
}

template<class T, class R>
TMemberDeclaration<T> MakeMember(const char* Name, uint32 Offset, TArray<R> T::*)
{
	static_assert(std::is_convertible_v<R, UObject*>);
    return TMemberDeclaration<T>(Name, Offset, EMemberType::ReferenceArray);
}

template<class T, class O>
TMemberDeclaration<T> MakeMember(const char* Name, uint32 Offset, TObjectPtr<O> T::*)
{
    return TMemberDeclaration<T>(Name, Offset, EMemberType::Reference);
}

template<class T, class O>
TMemberDeclaration<T> MakeMember(const char* Name, uint32 Offset, TArray<TObjectPtr<O>> T::*)
{
    return TMemberDeclaration<T>(Name, Offset, EMemberType::ReferenceArray);
}

template<class T, class S>
TMemberDeclaration<T> MakeNestedMember(const char* Name, uint32 Offset, TArray<S> T::*, TSchemaBuilder<S>& InnerSchema)
{
    return TMemberDeclaration<T>(Name, Offset, EMemberType::StructArray, InnerSchema.Build());
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
// Note: VValue, VRestValue and TWriteBarrier<U> share the same EMemberType because they are bit-compatible
// When a VValue points to a cell, it is bit-identical to the raw pointer so we can bitcast from VCell* to VValue
// Additionally, a VRestValue is bit-identical to a TWriteBarrier<VValue>/TWriteBarrier<VCell>
#if DO_GUARD_SLOW
static_assert(sizeof(::Verse::VValue) == sizeof(::Verse::VCell*), "");
static_assert(sizeof(::Verse::VValue) == sizeof(::Verse::VRestValue), "");
static_assert(sizeof(::Verse::VValue) == sizeof(::Verse::TWriteBarrier<::Verse::VValue>), "");
static_assert(sizeof(::Verse::VValue) == sizeof(::Verse::TWriteBarrier<::Verse::VCell>), "");
#endif

template<class T>
TMemberDeclaration<T> MakeMember(const char* Name, uint32 Offset, ::Verse::VRestValue T::*)
{
	return TMemberDeclaration<T>(Name, Offset, EMemberType::VerseValue);
}

template<class T>
TMemberDeclaration<T> MakeMember(const char* Name, uint32 Offset, TArray<::Verse::VRestValue> T::*)
{
	return TMemberDeclaration<T>(Name, Offset, EMemberType::VerseValueArray);
}

template<class T, typename U>
TMemberDeclaration<T> MakeMember(const char* Name, uint32 Offset, ::Verse::TWriteBarrier<U> T::*)
{
	return TMemberDeclaration<T>(Name, Offset, EMemberType::VerseValue);
}

template<class T, typename U>
TMemberDeclaration<T> MakeMember(const char* Name, uint32 Offset, TArray<::Verse::TWriteBarrier<U>> T::*)
{
	return TMemberDeclaration<T>(Name, Offset, EMemberType::VerseValueArray);
}
#endif

} // namespace Private

// Helpers to overload/dispatch UE_GC_MEMBER with 2 or 3 parameters
#define _UE_EXPAND(x) x // MSVC macro evaluation workaround
#define _UE_GC_MEMBER2(Type, Member)				UE::GC::Private::MakeMember(#Member, offsetof(Type, Member), &Type::Member)
#define _UE_GC_MEMBER3(Type, Member, InnerSchema)	UE::GC::Private::MakeNestedMember(#Member, offsetof(Type, Member), &Type::Member, InnerSchema)
#define _UE_GC_MEMBER(_1,_2,_3,CHOSEN_OVERLOAD,...) CHOSEN_OVERLOAD // _UE_GC_MEMBER2 or _UE_GC_MEMBER3

#define UE_GC_MEMBER(...) _UE_EXPAND(_UE_EXPAND(_UE_GC_MEMBER(__VA_ARGS__, _UE_GC_MEMBER3, _UE_GC_MEMBER2)) (__VA_ARGS__))

//////////////////////////////////////////////////////////////////////////

COREUOBJECT_API void DeclareIntrinsicSchema(UClass* Class, FSchemaView IntrinsicSchema);

template<class T>
void DeclareIntrinsicMembers(UClass* Class, std::initializer_list<TMemberDeclaration<T>> Members)
{
	DeclareIntrinsicSchema(Class, TSchemaBuilder<T>(Members).Build());
}

FSchemaView GetIntrinsicSchema(UClass* Class);

//////////////////////////////////////////////////////////////////////////
} // namespace UE::GC