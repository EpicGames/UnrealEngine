// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"

#if WITH_EDITOR
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "UObject/CookEnums.h"
#include "UObject/NameTypes.h"
#include "Templates/Tuple.h"
#endif

#if WITH_EDITOR

enum class EDataValidationResult : uint8;
class FDataValidationContext;
class ITargetPlatform;
class IPlugin;
class UPackage;

namespace UE::Cook { class ICookArtifact; }
namespace UE::Cook { class IMPCollector; }
#endif

/**
 * A scope around loads when cooking that indicates whether the loaded package is needed in game or not.
 * The default is Unexpected. Declare an FCookLoadScope to set the value.
 * 
 * If the package is marked as EditorOnly, that only suppresses the reference from the LoadPackage call.
 * The package can still be pulled into the cook by UsedInGame references from the AssetRegistry or by being
 * stored as an import in a cooked package.
 *
 * Packages that are declared in the AssetRegistry as an editoronly reference by the package that loads them
 * are implicitly marked as ECookLoadType::EditorOnly for that load, and do not need an explicitly declared
 * loadtype to be marked as EditorOnly.
 *
 * LoadTypes also apply to Startup packages, or packages loaded by systems without the load being owned by
 * a referencer package. EditorOnly still suppresses these packages (for that single load) from being added
 * to the cook, UsedInGame still forces them to be added. For Startup packages, marking the package as
 * EditorOnly both removes its auto-inclusion in the cook, and also removes the special chunk rule for startup
 * packages from it, if it ends up being pulled into the cook by another referencer. Startup packages are
 * automatically added to Chunk0 when the project is using multiple pak files (aka streaming chunks);
 * ECookLoadType::EditorOnly removes that chunk rule for the given package.
 *
 * This enum is declared in non-WITH_EDITOR builds to remove the need for #if WITH_EDITOR boilerplate, but
 * it is ignored; all of the functions/classes that use it are noops if !WITH_EDITOR.
 */
enum class ECookLoadType : uint8
{
	Unspecified,
	EditorOnly,
	UsedInGame,

	Unexpected UE_DEPRECATED(5.6, "Use ECookLoadType::Unspecified") = Unspecified,
};

#if WITH_EDITOR

namespace UE::Cook
{

/**
 * List the keywords that should be enum values of the EInstigator enum
 * The list is declared as a macro instead of ordinary C keywords in the enum declaration
 * so that we can reduce duplication in the functions that specify the string name and other properties for each
 * enum value; see the uses in ICookInfo.cpp.
 */
#define EINSTIGATOR_VALUES(CallbackMacro) \
	/** CallbackMacro(CPPToken Name, bool bAllowUnparameterized) */ \
	CallbackMacro(InvalidCategory, true) \
	CallbackMacro(NotYetRequested, true) \
	CallbackMacro(Unspecified, false) \
	CallbackMacro(StartupPackage, true) \
	CallbackMacro(StartupPackageCookLoadScope, true) \
	CallbackMacro(AlwaysCookMap, true) \
	CallbackMacro(IniMapSection, false) \
	CallbackMacro(IniAllMaps, true) \
	CallbackMacro(CommandLinePackage, true) \
	CallbackMacro(CommandLineDirectory, true) \
	CallbackMacro(DirectoryToAlwaysCook, false) \
	CallbackMacro(FullDepotSearch, true) \
	CallbackMacro(GameDefaultObject, false) \
	CallbackMacro(InputSettingsIni, true) \
	CallbackMacro(StartupSoftObjectPath, true) \
	CallbackMacro(PackagingSettingsMapToCook, true) \
	CallbackMacro(ModifyCookDelegate, true) \
	CallbackMacro(AssetManagerModifyCook, true) \
	CallbackMacro(AssetManagerModifyDLCCook, true) \
	CallbackMacro(TargetPlatformExtraPackagesToCook, true) \
	CallbackMacro(ConsoleCommand, true) \
	CallbackMacro(CookOnTheFly, true) \
	CallbackMacro(LegacyIterativeCook, true) \
	CallbackMacro(PreviousAssetRegistry, true) \
	CallbackMacro(RequestPackageFunction, true) \
	CallbackMacro(Dependency, false) \
	CallbackMacro(HardDependency, false) \
	CallbackMacro(HardEditorOnlyDependency, false) \
	CallbackMacro(SoftDependency, false) \
	CallbackMacro(Unsolicited, false) \
	CallbackMacro(EditorOnlyLoad, false) \
	CallbackMacro(SaveTimeHardDependency, false) \
	CallbackMacro(SaveTimeSoftDependency, false) \
	CallbackMacro(ForceExplorableSaveTimeSoftDependency, false) \
	CallbackMacro(GeneratedPackage, false) \
	CallbackMacro(BuildDependency, false) \


/** The different ways a package can be discovered by the cooker. */
enum class EInstigator : uint8
{
#define EINSTIGATOR_VALUE_CALLBACK(Name, bAllowUnparameterized) Name,
	EINSTIGATOR_VALUES(EINSTIGATOR_VALUE_CALLBACK)
#undef EINSTIGATOR_VALUE_CALLBACK
	Count,
};
COREUOBJECT_API const TCHAR* LexToString(EInstigator Value);

/** Category and referencer for how a package was discovered by the cooker. */
struct FInstigator
{
	FName Referencer;
	EInstigator Category;

	FInstigator() : Category(EInstigator::InvalidCategory)
	{
	}
	FInstigator(EInstigator InCategory, FName InReferencer = NAME_None) : Referencer(InReferencer), Category(InCategory)
	{
	}

	COREUOBJECT_API FString ToString() const;
};

/**
 * Values for whether a package should be cooked, used in ModifyCook callbacks by systems that want to
 * mark packages cooked/not-cooked independently of the usual asset-driven and config-driven cook specifications.
 */
enum class EPackageCookRule
{
	/* No action specified, the rule will be ignored. */
	None,
	/** The package will be cooked (unless specified as NeverCook or not cooked for the platform elsewhere). */
	AddToCook,
	/** The package will be not be cooked. */
	NeverCook,

	// IgnoreStartupPackage is not yet implemented, coming in a future version. For now, only FCookLoadScope
	// provides this functionality.
	/**
	 * Only has an effect if the project configuration specifies that startup packages (packages already loaded before
	 * the cook starts) are automatically cooked, and the package is a startup package. This turns off that automatic
	 * inclusion for the given package, and the package will not be cooked unless it is referenced from another source
	 * of requested packages or from another package that is cooked.
	 *
	 * Note: This will also modify the chunking of the package if it is otherwise referenced and therefore cooked.
	 * Startup packages are added into chunk0, and marking the package as IgnoreStartupPackage will remove that chunk
	 * rule for the package and it will be chunked only according to the AssetManager's decisions on chunking.
	 *
	 * Note: Currently the decision that startup packages are automatically cooked is hardcoded and there is no way
	 * for a project to turn it off. We expect this to change in the future and eventually Ignored will become
	 * the default for startup packages.
	 * 
	 * Note: This specification can also be indicated, without the need to subscribe to ModifyCook, by wrapping the
	 * LoadPackage or CreatePackage call of the given package in FCookLoadScope Scope(ECookLoadType::EditorOnly).
	 */
	// IgnoreStartupPackage,
};

/**
 * Specification of whether a package should be cooked, for use in ModifyCook callbacks by systems that want to
 * mark packages cooked/not-cooked independently of the usual asset-driven and config-driven cook specifications.
 */
struct FPackageCookRule
{
	/** Name of the package to specify cooked/notcooked/ignored. */
	FName PackageName;

	/** Name of the subscriber system, for use by cook users debugging why a package was cooked. */
	FName InstigatorName;

	// TargetPlatform is not yet implemented, coming in a future version. For now, TargetPlatform is ignored and
	// all rules apply to all platforms.
	/**
	 * For which platforms the behavior change should apply; this is necessary for multiplatform cooks,
	 * if different platforms can have different values. nullptr indicates it applies to every platform.
	 * To set different non-defualt cookrules for multiple platforms, use multiple FPackageCookRules.
	 */
	//	const ITargetPlatform* TargetPlatform = nullptr;

	/** The desired cook behavior for the package, @see UE::Cook::EPackageCookRule. */
	EPackageCookRule CookRule = EPackageCookRule::None;
};

/** Engine interface for information provided by UCookOnTheFlyServer in cook callbacks. */
class ICookInfo
{
public:
	/**
	 * Return the instigator that first caused the package to be requested by the cook.
	 * Returns category EInstigator::NotYetRequested if package is not yet known to the cook.
	 */
	virtual FInstigator GetInstigator(FName PackageName) const = 0;
	/**
	 * Return the chain of instigators that caused the package to be requested by the cook.
	 * First element is the direct instigator of the package, last is the root instigator that started the chain.
	 */
	virtual TArray<FInstigator> GetInstigatorChain(FName PackageName) const = 0;

	/** The type (e.g. CookByTheBook) of the running cook. This function will not return ECookType::Unknown. */
	virtual UE::Cook::ECookType GetCookType() const = 0;
	/** Whether DLC is being cooked (e.g. via "-dlcname=<PluginName>"). This function will not return ECookingDLC::Unknown. */
	virtual UE::Cook::ECookingDLC GetCookingDLC() const = 0;
	/** When DLC is being cooked (@see GetCookingDLC), this will return the name of the DLC plugin. */
	virtual FString GetDLCName() const = 0;
	/**
	 * The release version of the basegame the current DLC is based on, when cooking DLC or when making a patch.
	 * Empty string when not cooking DLC or making a patch.
	 */
	virtual FString GetBasedOnReleaseVersion() const = 0;
	/** The release version being created when making a patch. */
	virtual FString GetCreateReleaseVersion() const = 0;

	/** The role the current process plays in its MPCook session, or EProcessType::SingleProcess if it is running standalone. */
	virtual UE::Cook::EProcessType GetProcessType() const = 0;
	/** Get the validation options used by the running cook, if any. */
	virtual UE::Cook::ECookValidationOptions GetCookValidationOptions() const = 0;
	/**
	 * Returns true if the cooker is cooking after a previous cook session and is cooking only the changed files.
	 * Returns false if the cooker is doing a recook of all packages discovered in the session.
	 * Returns false if not yet initialized, but it will be initialized whenever a session is in progress (GetSessionPlatforms
	 * is non-empty).
	 * When IsIncremental is true, systems that write artifacts to the cook output should load/update/resave their
	 * artifacts.
	 */
	virtual bool IsIncremental() const = 0;
	/**
	 * Returns the list of platforms that will be/are being/have been cooked for the current cook session. Returns
	 * empty array when outside of a cooksession, including in the case that GetCookType() == ECookType::OnTheFly and
	 * no platforms have been requested yet. During cook by the book, this list will not change throughout the cook,
	 * during CookOnTheFly it can be added to or removed from when platforms are requested or go idle and are dropped.
	 */
	virtual TArray<const ITargetPlatform*> GetSessionPlatforms() const = 0;
	/**
	 * Returns the output folder being used by the cooker for the given platform in the given session.
	 * Returns empty string if not in a session or the given platform is not in GetSessionPlatforms().
	 * Returns the path in FPaths::MakeStandardFilename format ("../../../<ProjectName>/Saved/Cooked/<PlatformName>")
	 */
	virtual FString GetCookOutputFolder(const ITargetPlatform* TargetPlatform) const = 0;
	/**
	 * Returns the path within the GetCookOutputFolder where the Metadata folder for the cook is written. For basegame
	 * cooks this is GetCookOutputFolder()/<ProjectName>/Metadata, but for DLC cooks it is in the relative path to the
	 * DLC Folder: GetCookOutputFolder()/<RelPathToDLC>/Metadata.
	 */
	virtual FString GetCookMetadataOutputFolder(const ITargetPlatform* TargetPlatform) const = 0;

	/**
	 * Return the set of plugins that the cooker has determined are enabled on the current platform. This is different
	 * than IPluginManager::GetEnabledPlugins. The IPluginManager version returns the plugins enabled for the editor
	 * process, this GetEnabledPlugins returns the plugins that will be enabled on the TargetPlatform.
	 * 
	 * Returns nullptr if the given Platform is not a session platform in the current cook.
	 */
	virtual const TSet<IPlugin*>* GetEnabledPlugins(const ITargetPlatform* TargetPlatform) const = 0;

	/**
	 * MPCook: register in the current process a collector that replicates system-specific and package-specific
	 * information between CookWorkers and the CookDirector. Registration will be skipped if the current cook is
	 * singleprocess, or if the provided ProcessType does not match the current processtype. If registration is
	 * skipped, the Collector will be referenced but then immediately released, which will delete the Collector if
	 * not referenced by the caller.
	 * 
	 * @param ProcessType: The given collector will only be registered on the given process types, allowing you
	 *                     to register a different class on Workers than on the Director if desired. 
	 */
	virtual void RegisterCollector(IMPCollector* Collector,
		UE::Cook::EProcessType ProcessType=UE::Cook::EProcessType::AllMPCook) = 0;
	/**
	 * MPCook: Unregister in the current process a collector that was registered via RegisterCollector. Silently
	 * returns if the collector is not registered. References to the Collector will be released, which will
	 * delete the Collector if not referenced by the caller.
	 */
	virtual void UnregisterCollector(IMPCollector* Collector) = 0;

	/**
	 * Call in SPCook or on the cookdirector to register an Artifact that saves non-package files in
	 * the cook's output directory for each TargetPlatform. On CookWorkers this is a noop and the Artifact will be
	 * deleted if not referenced by the caller. Registered artifacts receives calls that allow them to handle
	 * invalidation of the artifact during incremental cooks.
	 */
	virtual void RegisterArtifact(ICookArtifact* Artifact) = 0;
	/**
	 * Unregister Artifact that was registered via RegisterArtifact. Silently returns if the artifact is not
	 * registered. References to the Artifact will be released, which will delete the Artifact if not referenced by the
	 * caller.
	 */
	virtual void UnregisterArtifact(ICookArtifact* Artifact) = 0;

	/**
	 * Gets the current cultures that are being cooked.
	 */
	virtual void GetCulturesToCook(TArray<FString>& OutCulturesToCook) const = 0;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FCookInfoEvent, ICookInfo&);
DECLARE_DELEGATE_RetVal_TwoParams(EDataValidationResult, FValidateSourcePackage, UPackage* /*Package*/,
	FDataValidationContext& /*ValidationContext*/);
DECLARE_MULTICAST_DELEGATE_TwoParams(FPackageBlockedEvent, const UObject* /* Object */, FStringBuilderBase& OutDebugInfo)
DECLARE_MULTICAST_DELEGATE_TwoParams(FCookInfoModifyCookDelegate, ICookInfo& /* CookInfo */,
	TArray<FPackageCookRule>& /* InOutPackageCookRules */);
DECLARE_MULTICAST_DELEGATE_ThreeParams(FCookUpdateDisplayEvent, ICookInfo& /* CookInfo */, int32 /* CookedPackagesCount */, int32 /* CookPendingCount */)
DECLARE_MULTICAST_DELEGATE_FourParams(FCookSaveIdleEvent, ICookInfo& /* CookInfo */, int32 /* NumPackagesInSaveQueue */, int32 /* NumPendingCookedPlatformData */, bool /* bExpectedDueToSlowBuildOperations */)
DECLARE_MULTICAST_DELEGATE_TwoParams(FCookLoadIdleEvent, ICookInfo& /* CookInfo */, int32 /* NumPackagesInLoadQueue */)

/** UE::Cook::FDelegates: callbacks for cook events. */
struct FDelegates
{
public:
	/**
	 * Called as early as possible after parameters for the current cook session are loaded to allow modifying cook
	 * startup behavior, such as registering extra artifacts or collectors.
	 */
	static COREUOBJECT_API FCookInfoEvent ConfigureCookSession;
	/** Called after a cook session has been initialized and is about to start ticking and loading/saving packages. */
	static COREUOBJECT_API FCookInfoEvent CookStarted;
	/** Called at the end of a cook session, after writing all cook artifacts. */
	static COREUOBJECT_API FCookInfoEvent CookFinished;
	/** Called after the cooker has loaded a requested package, before starting to save the package. */
	static COREUOBJECT_API FValidateSourcePackage ValidateSourcePackage;
	/** Called when the given package has been blocked from saving for beyond the configured time threshold; can be registered to append additional debug info in this case. */
	static COREUOBJECT_API FPackageBlockedEvent PackageBlocked;
	/**
	 * Called during CookByTheBook to specify packages that should be cooked, not cooked even if referenced
	 * (aka nevercooked), or have their reference behavior changed in other ways.
	 */
	static COREUOBJECT_API FCookInfoModifyCookDelegate ModifyCook;
	/** Called during UpdateDisplay to report cooked/pending packages and controlled by the same cook.display.* CVars. for frequency */
	static COREUOBJECT_API FCookUpdateDisplayEvent CookUpdateDisplay;
	/** Called during SetSaveBusy when a potential soft lock is detected. */
	static COREUOBJECT_API FCookSaveIdleEvent CookSaveIdle;
	/** Called during SetLoadBusy when a potential soft lock is detected. */
	static COREUOBJECT_API FCookLoadIdleEvent CookLoadIdle;
};


/**
 * Return the relative path under the cook output MetaData folder to the ReferencedSet file.
 * The ReferencedSet file is a text file list of package names, one per line, that were referenced from the cook
 * session.
 * It does not include previously cooked files in an incremental cook that are no longer referenced.
 * It does not include packages from the base game for a DLC cook.
 */
COREUOBJECT_API const TCHAR* GetReferencedSetFilename();
/** Return the name of the op used to store the ReferencedSet in the zenserver oplog. */
COREUOBJECT_API const TCHAR* GetReferencedSetOpName();

/**
 * Initialize some globals that are used to track engine activity relevant to the cook before the cooker
 * is created. This should only be called when setting IsRunningCookCommandlet=true.
 */
COREUOBJECT_API void InitializeCookGlobals();

} // namespace UE::Cook

/** Set the ECookLoadType value in the current scope.*/
struct FCookLoadScope
{
	COREUOBJECT_API explicit FCookLoadScope(ECookLoadType ScopeType);
	COREUOBJECT_API ~FCookLoadScope();

	COREUOBJECT_API static ECookLoadType GetCurrentValue();
	COREUOBJECT_API static void GetCookerStartupPackages(TMap<FName, UE::Cook::FInstigator>& OutStartupPackages);

private:
	static void SetCookerStartupComplete(TArray<TPair<FName, ECookLoadType>>& OutStartupPackageLoadTypes);
	ECookLoadType PreviousScope;
};

#else // WITH_EDITOR

/** Defined in !WITH_EDITOR to remove the need for #ifdef boilerplate, but it is a noop. */
struct FCookLoadScope
{
	UE_FORCEINLINE_HINT explicit FCookLoadScope(ECookLoadType ScopeType)
	{
	}

	UE_FORCEINLINE_HINT ~FCookLoadScope() = default;
};

// Provide a forward declare but no definition for ICookInfo, to remove the need to have WITH_EDITOR around some types that include a pointer to it
namespace UE::Cook { class ICookInfo; }

#endif // WITH_EDITOR
