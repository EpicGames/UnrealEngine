// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectGlobals.h"

#include "Interface.generated.h"

/**
 * Base class for all interfaces
 *
 */
UINTERFACE(MinimalAPI, meta=(IsBlueprintBase="true", CannotImplementInterfaceInBlueprint))
class UInterface : public UObject
{
	GENERATED_BODY()
public:
};

class IInterface
{
	GENERATED_BODY()
};
