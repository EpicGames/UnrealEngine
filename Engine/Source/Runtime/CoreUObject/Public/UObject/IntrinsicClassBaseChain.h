// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(UE_WITH_CONSTINIT_UOBJECT) && UE_WITH_CONSTINIT_UOBJECT

#include "UObject/Class.h"
#include "UObject/CompiledInObjectPtr.h"

namespace UE::Private
{
    template<typename IntrinsicClass>
    constexpr bool DeferFalse = false;

    template<typename IntrinsicClass>
    consteval UClass* GetConstInitClass()
    {
        static_assert(DeferFalse<IntrinsicClass>, "UE::Private::GetConstInitClass must be specialized for intrinsic classes");
        return nullptr;
    }

	template<typename... Classes>
	struct TIntrinsicClassBaseChainImpl
	{
		using BaseChainPtr = CodeGen::ConstInit::TCompiledInObjectPtr<const FStructBaseChain>;
		static constexpr SIZE_T Num = sizeof...(Classes);
		static inline constinit BaseChainPtr Bases[] = { BaseChainPtr{UE::Private::AsStructBaseChain((GetConstInitClass<Classes>()))}... };

		static consteval TArrayView<BaseChainPtr> ToArrayView()
		{
			return MakeArrayView(Bases, Num);
		}
	};

	template<typename ClassType, typename... Ts>
	consteval auto DetermineIntrinsicClassBaseChain() 
	{ 
		if constexpr (std::is_same_v<ClassType, typename ClassType::Super>) 
		{
			return TIntrinsicClassBaseChainImpl<ClassType, Ts...>{};
		} 
		else
		{
			return DetermineIntrinsicClassBaseChain<typename ClassType::Super, ClassType, Ts...>();
		}
	}
}

// Template to build a static array of base class pointers for an intrinsic class
template<typename T>
using TIntrinsicClassBaseChain = decltype(UE::Private::DetermineIntrinsicClassBaseChain<T>());

#endif // UE_WITH_CONSTINIT_UOBJECT
