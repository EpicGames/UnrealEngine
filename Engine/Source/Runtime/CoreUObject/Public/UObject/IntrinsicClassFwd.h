// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

// TODO: These need to be emitted by UHT if we are going to support intrinsic classes outside of CoreUObject in modular constinit-uobject builds 
#if UE_WITH_CONSTINIT_UOBJECT

#include "Templates/NoDestroy.h"
#include "UObject/Class.h"
#include "UObject/IntrinsicClassBaseChain.h"

#define UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(ClassName, RequiredAPI) \
    class ClassName; \
    extern constinit TNoDestroy<UClass> Z_ConstInit_UClass_##ClassName; \
    namespace UE::Private \
    { \
        template<> \
        consteval UClass* GetConstInitClass<ClassName>() \
        { \
            return &Z_ConstInit_UClass_##ClassName; \
        } \
    }

// Parent classes for intrinsic classes also need the accessor defined 
// CONSTINIT_UOBJECT_TODO: Should this be generated by UHT?
UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UObject, COREUOBJECT_API)
UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UPackage, COREUOBJECT_API)
UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UField, COREUOBJECT_API)
UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UStruct, COREUOBJECT_API)
UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UClass, COREUOBJECT_API)
UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UScriptStruct, COREUOBJECT_API)

#else

#define UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(ClassName, RequiredAPI)

#endif // UE_WITH_CONSTINIT_UOBJECT