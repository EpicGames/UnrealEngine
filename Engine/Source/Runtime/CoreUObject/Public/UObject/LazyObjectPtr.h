// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	LazyObjectPtr.h: Lazy, guid-based weak pointer to a UObject, mostly useful for actors
=============================================================================*/

#pragma once

#include "Concepts/PointerEqualityComparableWith.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "HAL/Platform.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/Guid.h"
#include "Misc/UEOps.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/Casts.h"
#include "Templates/Requires.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/PersistentObjectPtr.h"

#include <type_traits>

class UObject;

template <typename T> struct TIsPODType;
template <typename T> struct TIsWeakPointerType;

// Max of 10 PIE instance + 1 Play As Client PIE instance
#define MAX_PIE_INSTANCES 11

/**
 * Wrapper structure for a GUID that uniquely identifies registered UObjects.
 * The actual GUID is stored in an object annotation that is updated when a new reference is made.
 */
struct FUniqueObjectGuid
{
	FUniqueObjectGuid()
	{}

	FUniqueObjectGuid(const FGuid& InGuid)
		: Guid(InGuid)
	{}

	/** Reset the guid pointer back to the invalid state */
	UE_FORCEINLINE_HINT void Reset()
	{
		Guid.Invalidate();
	}

	/** Construct from an existing object */
	COREUOBJECT_API explicit FUniqueObjectGuid(const UObject* InObject);

	/** Converts into a string */
	COREUOBJECT_API FString ToString() const;

	/** Converts from a string */
	COREUOBJECT_API void FromString(const FString& From);

	/** Fixes up this UniqueObjectID to add or remove the PIE prefix depending on what is currently active */
	COREUOBJECT_API FUniqueObjectGuid FixupForPIE(int32 PlayInEditorID = UE::GetPlayInEditorID()) const;

	/**
	 * Attempts to find a currently loaded object that matches this object ID
	 *
	 * @return Found UObject, or nullptr if not currently loaded
	 */
	COREUOBJECT_API UObject* ResolveObject() const;

	/** Test if this can ever point to a live UObject */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return Guid.IsValid();
	}

	UE_FORCEINLINE_HINT bool operator==(const FUniqueObjectGuid& Other) const
	{
		return Guid == Other.Guid;
	}

	UE_FORCEINLINE_HINT bool operator!=(const FUniqueObjectGuid& Other) const
	{
		return Guid != Other.Guid;
	}

	/** Returns true is this is the default value */
	UE_FORCEINLINE_HINT bool IsDefault() const
	{
		// A default GUID is 0,0,0,0 and this is "invalid"
		return !IsValid(); 
	}

	UE_FORCEINLINE_HINT friend uint32 GetTypeHash(const FUniqueObjectGuid& ObjectGuid)
	{
		return GetTypeHash(ObjectGuid.Guid);
	}
	/** Returns wrapped Guid */
	UE_FORCEINLINE_HINT const FGuid& GetGuid() const
	{
		return Guid;
	}

	friend FArchive& operator<<(FArchive& Ar,FUniqueObjectGuid& ObjectGuid)
	{
		Ar << ObjectGuid.Guid;
		return Ar;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, FUniqueObjectGuid& ObjectGuid)
	{
		Slot << ObjectGuid.Guid;
	}

	static COREUOBJECT_API FUniqueObjectGuid GetOrCreateIDForObject(FObjectPtr Object);
	static UE_FORCEINLINE_HINT FUniqueObjectGuid GetOrCreateIDForObject(const UObject* Object)
	{
		return GetOrCreateIDForObject(FObjectPtr(const_cast<UObject*>(Object)));
	}
	template <typename T>
	static UE_FORCEINLINE_HINT FUniqueObjectGuid GetOrCreateIDForObject(TObjectPtr<T> Object)
	{
		return GetOrCreateIDForObject(FObjectPtr(Object));
	}

private:
	/** Guid representing the object, should be unique */
	FGuid Guid;
};

template<> struct TIsPODType<FUniqueObjectGuid> { enum { Value = true }; };

/**
 * FLazyObjectPtr is a type of weak pointer to a UObject that uses a GUID created at save time.
 * Objects will only have consistent GUIDs if they are referenced by a lazy pointer and then saved.
 * It will change back and forth between being valid or pending as the referenced object loads or unloads.
 * It has no impact on if the object is garbage collected or not.
 * It can't be directly used across a network.
 *
 * NOTE: Because this only stores a GUID, it does not know how to load the destination object and does not work with Play In Editor.
 * This will be deprecated in a future engine version and new features should use FSoftObjectPtr instead.
 */
struct FLazyObjectPtr : public TPersistentObjectPtr<FUniqueObjectGuid>
{
private:
	using Super = TPersistentObjectPtr<FUniqueObjectGuid>;

public:
	/** Default constructor, sets to null */
	UE_FORCEINLINE_HINT FLazyObjectPtr()
	{
	}

	/** Construct from object already in memory */
	explicit UE_FORCEINLINE_HINT FLazyObjectPtr(FObjectPtr Object)
	{
		(*this)=Object;
	}
	explicit UE_FORCEINLINE_HINT FLazyObjectPtr(const UObject* Object)
		: FLazyObjectPtr(FObjectPtr(const_cast<UObject*>(Object)))
	{
	}
	template <typename T>
	explicit UE_FORCEINLINE_HINT FLazyObjectPtr(TObjectPtr<T> Object)
		: FLazyObjectPtr(FObjectPtr(Object))
	{
		// This needs to be a template instead of TObjectPtr<const UObject> because C++ does derived-to-base
		// pointer conversions ('standard conversion sequences') in more cases than TSmartPtr<Derived>-to-TSmartPtr<Base>
		// conversions ('user-defined conversions'), meaning it doesn't auto-convert in many real use cases.
		//
		// https://en.cppreference.com/w/cpp/language/implicit_conversion
	}

	/** Copy from an object already in memory */
	UE_FORCEINLINE_HINT void operator=(FObjectPtr Object)
	{
		Super::operator=(Object);
	}
	UE_FORCEINLINE_HINT void operator=(const UObject* Object)
	{
		*this = FObjectPtr(const_cast<UObject*>(Object));
	}
	template <typename T>
	UE_FORCEINLINE_HINT void operator=(TObjectPtr<T> Object)
	{
		// This needs to be a template instead of TObjectPtr<const UObject> because C++ does derived-to-base
		// pointer conversions ('standard conversion sequences') in more cases than TSmartPtr<Derived>-to-TSmartPtr<Base>
		// conversions ('user-defined conversions'), meaning it doesn't auto-convert in many real use cases.
		//
		// https://en.cppreference.com/w/cpp/language/implicit_conversion

		*this = FObjectPtr(Object);
	}

	/** Copy from a unique object identifier */
	UE_FORCEINLINE_HINT void operator=(const FUniqueObjectGuid& InObjectID)
	{
		Super::operator=(InObjectID);
	}

	/** Fixes up this FLazyObjectPtr to target the right UID as set in PIEGuidMap, this only works for directly serialized pointers */
	UE_FORCEINLINE_HINT void FixupForPIE(int32 PIEInstance)
	{
		*this = GetUniqueID().FixupForPIE(PIEInstance);
	}
	
	/** Called by UObject::Serialize so that we can save / load the Guid possibly associated with an object */
	COREUOBJECT_API static void PossiblySerializeObjectGuid(UObject* Object, FStructuredArchive::FRecord Record);

	/** Called when entering PIE to prepare it for PIE-specific fixups */
	COREUOBJECT_API static void ResetPIEFixups();
};

template <> struct TIsPODType<FLazyObjectPtr> { enum { Value = TIsPODType<TPersistentObjectPtr<FUniqueObjectGuid> >::Value }; };
template <> struct TIsWeakPointerType<FLazyObjectPtr> { enum { Value = TIsWeakPointerType<TPersistentObjectPtr<FUniqueObjectGuid> >::Value }; };

/**
 * TLazyObjectPtr is the templatized version of the generic FLazyObjectPtr.
 * NOTE: This will be deprecated in a future engine version and new features should use TSoftObjectPtr instead.
 */
template<class T=UObject>
struct TLazyObjectPtr : private FLazyObjectPtr
{
public:
	using ElementType = T;
	
	TLazyObjectPtr() = default;

	TLazyObjectPtr(TLazyObjectPtr<T>&&) = default;
	TLazyObjectPtr(const TLazyObjectPtr<T>&) = default;
	TLazyObjectPtr<T>& operator=(TLazyObjectPtr<T>&&) = default;
	TLazyObjectPtr<T>& operator=(const TLazyObjectPtr<T>&) = default;

	/** Construct from another lazy pointer with implicit upcasting allowed */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	UE_FORCEINLINE_HINT TLazyObjectPtr(const TLazyObjectPtr<U>& Other) :
		FLazyObjectPtr((const FLazyObjectPtr&)Other)
	{
	}
	
	/** Assign from another lazy pointer with implicit upcasting allowed */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	UE_FORCEINLINE_HINT TLazyObjectPtr<T>& operator=(const TLazyObjectPtr<U>& Other)
	{
		FLazyObjectPtr::operator=((const FLazyObjectPtr&)Other);
		return *this;
	}

	/** Construct from an object pointer */
	UE_FORCEINLINE_HINT TLazyObjectPtr(T* Object)
	{
		FLazyObjectPtr::operator=(Object);
	}

	/** Reset the lazy pointer back to the null state */
	UE_FORCEINLINE_HINT void Reset()
	{
		FLazyObjectPtr::Reset();
	}

	/** Copy from an object pointer */
	UE_FORCEINLINE_HINT void operator=(T* Object)
	{
		FLazyObjectPtr::operator=(Object);
	}

	/**
	 * Copy from a unique object identifier
	 * WARNING: this doesn't check the type of the object is correct,
	 * because the object corresponding to this ID may not even be loaded!
	 *
	 * @param ObjectID Object identifier to create a lazy pointer to
	 */
	UE_FORCEINLINE_HINT void operator=(const FUniqueObjectGuid& InObjectID)
	{
		FLazyObjectPtr::operator=(InObjectID);
	}

	/**
	 * Gets the unique object identifier associated with this lazy pointer. Valid even if pointer is not currently valid
	 *
	 * @return Unique ID for this object, or an invalid FUniqueObjectID if this pointer isn't set to anything
	 */
	UE_FORCEINLINE_HINT const FUniqueObjectGuid& GetUniqueID() const
	{
		return FLazyObjectPtr::GetUniqueID();
	}

	/**
	 * Dereference the lazy pointer.
	 *
	 * @return nullptr if this object is gone or the lazy pointer was null, otherwise a valid UObject pointer
	 */
	UE_FORCEINLINE_HINT T* Get() const
	{
		// there are cases where a TLazyObjectPtr can get an object of the wrong type assigned to it which are difficult to avoid
		// e.g. operator=(const FUniqueObjectGuid& ObjectID)
		// "WARNING: this doesn't check the type of the object is correct..."
		return dynamic_cast<T*>(FLazyObjectPtr::Get());
	}

	/** Dereference the lazy pointer */
	UE_FORCEINLINE_HINT T& operator*() const
	{
		return *Get();
	}

	/** Dereference the lazy pointer */
	UE_FORCEINLINE_HINT T* operator->() const
	{
		return Get();
	}

	/** Test if this points to a live UObject */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return FLazyObjectPtr::IsValid();
	}

	/**
	 * Slightly different than !IsValid(), returns true if this used to point to a UObject, but doesn't any more and has not been assigned or reset in the mean time.
	 *
	 * @return true if this used to point at a real object but no longer does.
	 */
	UE_FORCEINLINE_HINT bool IsStale() const
	{
		return FLazyObjectPtr::IsStale();
	}

	/**
	 * Test if this does not point to a live UObject, but may in the future
	 *
	 * @return true if this does not point to a real object, but could possibly
	 */
	UE_FORCEINLINE_HINT bool IsPending() const
	{
		return FLazyObjectPtr::IsPending();
	}

	/**
	 * Test if this can never point to a live UObject
	 *
	 * @return true if this is explicitly pointing to no object
	 */
	UE_FORCEINLINE_HINT bool IsNull() const
	{
		return FLazyObjectPtr::IsNull();
	}

	/** Dereference lazy pointer to see if it points somewhere valid */
	UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return IsValid();
	}

	/** Hash function. */
	UE_FORCEINLINE_HINT uint32 GetLazyObjecPtrTypeHash() const
	{
		return GetTypeHash(static_cast<const FLazyObjectPtr&>(*this));
	}

	UE_FORCEINLINE_HINT void SerializePtr(FArchive& Ar)
	{
		Ar << static_cast<FLazyObjectPtr&>(*this);
	}

	template <UE::CPointerEqualityComparableWith<T> U>
	UE_FORCEINLINE_HINT bool UEOpEquals(const TLazyObjectPtr<U>& Rhs) const
	{
		return (const FLazyObjectPtr&)*this == (const FLazyObjectPtr&)Rhs;
	}
	template <UE::CPointerEqualityComparableWith<T> U>
	UE_FORCEINLINE_HINT bool UEOpEquals(const U* Rhs) const
	{
		return Get() == Rhs;
	}
	UE_FORCEINLINE_HINT bool UEOpEquals(TYPE_OF_NULLPTR) const
	{
		return !IsValid();
	}
};

/** Hash function. */
template<typename T>
UE_FORCEINLINE_HINT uint32 GetTypeHash(const TLazyObjectPtr<T>& LazyObjectPtr)
{
	return LazyObjectPtr.GetLazyObjecPtrTypeHash();
}

template<typename T>
FArchive& operator<<(FArchive& Ar, TLazyObjectPtr<T>& LazyObjectPtr)
{
	LazyObjectPtr.SerializePtr(Ar);
	return Ar;
}

template<class T> struct TIsPODType<TLazyObjectPtr<T> > { enum { Value = TIsPODType<FLazyObjectPtr>::Value }; };
template<class T> struct TIsWeakPointerType<TLazyObjectPtr<T> > { enum { Value = TIsWeakPointerType<FLazyObjectPtr>::Value }; };
