// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"

/**
 * Structure to hold information about an external packages objects used in cross-level references
 */
struct FLevelGuids
{
	/** Name of the external level */
	FName LevelName;

	/** Array of Guids possible in the other level (can be emptied out if all references are resolved after level load) */
	TArray<FGuid> Guids;
};
