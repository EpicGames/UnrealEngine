// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/GatherableTextData.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Serialization/ImportTypeHierarchy.h"
#include "Templates/RefCounting.h"
#include "UObject/LinkerInstancingContext.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectResource.h"
#include "UObject/PackageFileSummary.h"

class FArchive;
class FLinkerInstancingContext;
class FLinkerLoad;
class FPackagePath;
class FReferenceCollector;
class UObject;
class UPackage;
class UPackageMap;
struct FGatherableTextData;
struct FGuid;
struct FPackageSaveInfo;
struct FUObjectSerializeContext;
template <typename FuncType> class TFunctionRef;

DECLARE_LOG_CATEGORY_EXTERN(LogLinker, Log, All);

/**
 * Information about a compressed chunk in a file.
 */
struct FCompressedChunk
{
	/** Default constructor, zero initializing all members. */
	FCompressedChunk();

	/** Original offset in uncompressed file.	*/
	int32		UncompressedOffset;
	/** Uncompressed size in bytes.				*/
	int32		UncompressedSize;
	/** Offset in compressed file.				*/
	int32		CompressedOffset;
	/** Compressed size in bytes.				*/
	int32		CompressedSize;

	/** I/O functions */
	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FCompressedChunk& Chunk);
	friend COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, FCompressedChunk& Chunk);
};


class FLinkerTables
{
public:
	/** The list of FObjectImports found in the package */
	TArray<FObjectImport> ImportMap;
	/** The list of FObjectExports found in the package */
	TArray<FObjectExport> ExportMap;
	/** The list of FCellImports found in the package */
	TArray<FCellImport> CellImportMap;
	/** The list of FCellExports found in the package */
	TArray<FCellExport> CellExportMap;
	/** List of dependency lists for each export */
	TArray<TArray<FPackageIndex> > DependsMap;
	/** List of packages that are soft referenced by this package */
	TArray<FName> SoftPackageReferenceList;
	/** List of Searchable Names, by object containing them. Not in MultiMap to allow sorting, and sizes are usually small enough where TArray makes sense */
	TMap<FPackageIndex, TArray<FName> > SearchableNamesMap;

	/**
	 * Check that this Index is non-null and return an import or export
	 * @param	Index	Package index to get
	 * @return	the resource corresponding to this index
	 */
	inline FObjectResource& ImpExp(FPackageIndex Index)
	{
		check(!Index.IsNull());
		if (Index.IsImport())
		{
			return Imp(Index);
		}
		else
		{
			return Exp(Index);
		}
	}
	/**
	 * Check that this Index is non-null and return an import or export
	 * @param	Index	Package index to get
	 * @return	the resource corresponding to this index
	 */
	inline FObjectResource const& ImpExp(FPackageIndex Index) const
	{
		check(!Index.IsNull());
		if (Index.IsImport())
		{
			return Imp(Index);
		}
		else
		{
			return Exp(Index);
		}
	}	
	/**
	 * Return an import or export for this index
	 * @param	Index	Package index to get
	 * @return	the resource corresponding to this index, or NULL if the package index is null
	 */
	inline FObjectResource* ImpExpPtr(FPackageIndex Index)
	{
		if (Index.IsImport())
		{
			return ImpPtr(Index);
		}
		else
		{
			return ExpPtr(Index);
		}
	}

	/**
	 * Check that this Index is non-null and an import and return an import
	 * @param	Index	Package index to get, must be an import
	 * @return	the import corresponding to this index
	 */
	inline FObjectImport& Imp(FPackageIndex Index)
	{
		check(Index.IsImport() && ImportMap.IsValidIndex(Index.ToImport()));
		return ImportMap[Index.ToImport()];
	}
	inline FObjectImport const& Imp(FPackageIndex Index) const
	{
		check(Index.IsImport() && ImportMap.IsValidIndex(Index.ToImport()));
		return ImportMap[Index.ToImport()];
	}
	/**
	 * Return an import for this index
	 * @param	Index	Package index to get
	 * @return	the import corresponding to this index, or NULL if the package index is null or an export
	 */
	inline FObjectImport* ImpPtr(FPackageIndex Index)
	{
		if (Index.IsImport())
		{
			check(ImportMap.IsValidIndex(Index.ToImport()));
			return &ImportMap[Index.ToImport()];
		}
		return nullptr;
	}

	/**
	 * Check that this Index is non-null and an export and return an import
	 * @param	Index	Package index to get, must be an export
	 * @return	the export corresponding to this index
	 */
	inline FObjectExport& Exp(FPackageIndex Index)
	{
		check(Index.IsExport() && ExportMap.IsValidIndex(Index.ToExport()));
		return ExportMap[Index.ToExport()];
	}
	inline FObjectExport const& Exp(FPackageIndex Index) const
	{
		check(Index.IsExport() && ExportMap.IsValidIndex(Index.ToExport()));
		return ExportMap[Index.ToExport()];
	}
	/**
	 * Return an export for this index
	 * @param	Index	Package index to get
	 * @return	the export corresponding to this index, or NULL if the package index is null or an import
	 */
	inline FObjectExport* ExpPtr(FPackageIndex Index)
	{
		if (Index.IsExport())
		{
			check(ExportMap.IsValidIndex(Index.ToExport()));
			return &ExportMap[Index.ToExport()];
		}
		return nullptr;
	}

	/** Serializes the searchable name map */
	COREUOBJECT_API void SerializeSearchableNamesMap(FArchive &Ar);
	COREUOBJECT_API void SerializeSearchableNamesMap(FStructuredArchive::FSlot Slot);

	/**
	 * Return the path name of the UObject represented by the specified import.
	 * (can be used with StaticFindObject)
	 *
	 * @param	ImportIndex	index into the ImportMap for the resource to get the name for
	 *
	 * @return	the path name of the UObject represented by the resource at ImportIndex
	 */
	COREUOBJECT_API FString GetImportPathName(int32 ImportIndex);
	/**
	 * Return the path name of the UObject represented by the specified import.
	 * (can be used with StaticFindObject)
	 *
	 * @param	PackageIndex	package index for the resource to get the name for
	 *
	 * @return	the path name of the UObject represented by the resource at PackageIndex, or the empty string if this isn't an import
	 */
	FString GetImportPathName(FPackageIndex PackageIndex)
	{
		if (PackageIndex.IsImport())
		{
			return GetImportPathName(PackageIndex.ToImport());
		}
		return FString();
	}
	/**
	 * Return the path name of the UObject represented by the specified export.
	 * (can be used with StaticFindObject)
	 *
	 * @param	RootPackagePath			Name of the root package for this export
	 * @param	ExportIndex				index into the ExportMap for the resource to get the name for	 
	 * @param	bResolveForcedExports	if true, the package name part of the return value will be the export's original package,
	 *									not the name of the package it's currently contained within.
	 *
	 * @return	the path name of the UObject represented by the resource at ExportIndex
	 */
	COREUOBJECT_API FString GetExportPathName(const FString& RootPackagePath, int32 ExportIndex, bool bResolveForcedExports = false);

	/**
	 * Return the full name of the UObject represented by the specified import.
	 *
	 * @param	ImportIndex	index into the ImportMap for the resource to get the name for
	 *
	 * @return	the full name of the UObject represented by the resource at ImportIndex
	 */

	COREUOBJECT_API FString GetImportFullName(int32 ImportIndex);
	/**
	 * Return the full name of the UObject represented by the specified package index
	 *
	 * @param	PackageIndex	package index for the resource to get the name for
	 *
	 * @return	the full name of the UObject represented by the resource at PackageIndex
	 */
	FString GetImportFullName(FPackageIndex PackageIndex)
	{
		if (PackageIndex.IsImport())
		{
			return GetImportFullName(PackageIndex.ToImport());
		}
		return FString();
	}

	/**
	 * Return the full name of the UObject represented by the specified export.
	 *
	 * @param	RootPackagePath			Name of the root package for this export
	 * @param	ExportIndex				index into the ExportMap for the resource to get the name for
	 * @param	bResolveForcedExports	if true, the package name part of the return value will be the export's original package,
	 *									not the name of the package it's currently contained within.
	 *
	 * @return	the full name of the UObject represented by the resource at ExportIndex
	 */
	COREUOBJECT_API FString GetExportFullName(const FString& RootPackagePath, int32 ExportIndex, bool bResolveForcedExports = false);

	/**
	 * Return the outermost resource package index of the resource pointed by LinkerIndex
	 * @param LinkerIndex the resource to find the outermost of
	 * @return the package index of the outermost
	 */
	FPackageIndex ResourceGetOutermost(FPackageIndex LinkerIndex) const;

	/**
	 * Return if a resource has the specified other resource in its outer chain
	 * @param LinkerIndex the package index of the resource to verify
	 * @param OuterIndex the package index of the possible outer for the resource
	 * @return true if the resource pointed by LinkerIndex is in the resource pointed by OuterIndex
	 */
	bool ResourceIsIn(FPackageIndex LinkerIndex, FPackageIndex OuterIndex) const;

	/**
	 * Return if two resources share the same outermost
	 * @param LinkerIndexLHS the package index of the first resource
	 * @param LinkerIndexRHS the package index of the second resource
	 * @return true if they share an outer
	 */
	bool DoResourcesShareOutermost(FPackageIndex LinkerIndexLHS, FPackageIndex LinkerIndexRHS) const;

	/**
	 * Return if the import is in any export
	 * @param ImportIndex the import to verify
	 * @return true if an export is in the import outer chain
	 */
	bool ImportIsInAnyExport(int32 ImportIndex) const;

	/**
	 * Return if any export is in the import
	 * @param ImportIndex the import to verify
	 * @return true if the import is in any export outer chain
	 */
	bool AnyExportIsInImport(int32 ImportIndex) const;

	/**
	 * Return if any export share an outer with the import
	 * @param ImportIndex the import to verify
	 * @reutrn true if any export share the same outer has the import
	 */
	bool AnyExportShareOuterWithImport(int32 ImportIndex) const;

	/** 
	 * Gets the class name for the specified index in the export map.
	 * @param ExportIdx			Export index
	 * @return Class name of the export at specified index
	 */
	COREUOBJECT_API FName GetExportClassName(int32 ExportIdx) const;
	/** 
	 * Gets the class name for the specified index in the export map.
	 * @param PackageIndex		PackageIndex that represents the export index
	 * @return Class name of the export at specified PackageIndex if the PackageIndex represents an export. Otherwise NAME_None
	 */
	FName GetExportClassName(FPackageIndex PackageIndex)
	{
		if (PackageIndex.IsExport())
		{
			return GetExportClassName(PackageIndex.ToExport());
		}
		return FName();
	}
	/** 
	 * Gets the class name for the specified index in the import map.
	 * @param ImportIdx			Import index
	 * @return Class name of the import at specified index
	 */
	FName GetImportClassName(int32 ImportIdx)
	{
		return ImportMap[ImportIdx].ClassName;
	}
	/** 
	 * Gets the class name for the specified index in the import map.
	 * @param ImportIdx			Import PackageIndex
	 * @return Class name of the import at specified PackageIndex if the PackageIndex represents an import. Otherwise NAME_None
	 */
	FName GetImportClassName(FPackageIndex PackageIndex)
	{
		if (PackageIndex.IsImport())
		{
			return GetImportClassName(PackageIndex.ToImport());
		}
		return FName();
	}
	/** 
	 * Gets the class name for the specified PackageIndex.
	 * @param RootPackagePath	Name of the root package these linker tables represent
	 * @param PackageIndex		PackageIndex for the entry in the linker tables (import or export)
	 * @return Class name of the export at specified PackageIndex if the PackageIndex is non-null. Otherwise NAME_None
	 */
	FName GetClassName(FPackageIndex PackageIndex)
	{
		if (PackageIndex.IsImport())
		{
			return GetImportClassName(PackageIndex);
		}
		else if (PackageIndex.IsExport())
		{
			return GetExportClassName(PackageIndex);
		}
		return FName();
	}

	/** Returns the amount of memory allocated by this container, not including sizeof(*this). */
	COREUOBJECT_API SIZE_T GetAllocatedSize() const;
};

/*----------------------------------------------------------------------------
	FLinker.
----------------------------------------------------------------------------*/
namespace ELinkerType
{
	enum Type
	{
		None,
		Load,
		Save
	};
}

/**
 * Manages the data associated with an Unreal package.  Acts as the bridge between
 * the file on disk and the UPackage object in memory for all Unreal package types.
 */
class FLinker : public FLinkerTables
{
private:	

	ELinkerType::Type LinkerType;

public:

	/** The top-level UPackage object for the package associated with this linker */
	UPackage*				LinkerRoot;

	/** Table of contents for this package's file */
	FPackageFileSummary		Summary;

	/** Names used by objects contained within this package */
	TArray<FNameEntryId>	NameMap;

	/** List of SoftObjectPath contained in this package,  */
	TArray<FSoftObjectPath> SoftObjectPathList;

	/** Gatherable text data contained within this package */
	TArray<FGatherableTextData> GatherableTextDataMap;
	
	/** Raw/bulk data references */
	TArray<FObjectDataResource> DataResourceMap;

	/** Map of hierarchical type information for FObjectImport Struct entries in the package */
	TMap<FPackageIndex, UE::Serialization::Private::FImportTypeHierarchy> ImportTypeHierarchies;

	/** If true, filter out exports that are for clients but not servers */
	bool					FilterClientButNotServer;

	/** If true, filter out exports that are for servers but not clients */
	bool					FilterServerButNotClient;

	UE_DEPRECATED(5.6, "No longer supported. Contact Epic if you need this functionality.")
	class FSHA1*			ScriptSHA = nullptr;

	/** Constructor. */
	FLinker(ELinkerType::Type InType, UPackage* InRoot);

	virtual ~FLinker();

	UE_FORCEINLINE_HINT ELinkerType::Type GetType() const
	{
		return LinkerType;
	}

	/** Returns a descriptor of the PackagePath this Linker is reading from or writing to, usable for an identifier in warning and log messages */
	virtual FString GetDebugName() const;

	/**
	 * I/O function
	*
	 * @param	Ar	the archive to read/write into
	 */
	void Serialize(FArchive& Ar);
	
	/**
	 * Return the path name of the UObject represented by the specified export.
	 * (can be used with StaticFindObject)
	 *
	 * @param	ExportIndex				index into the ExportMap for the resource to get the name for
	 * @param	FakeRoot				Optional name to replace use as the root package of this object instead of the linker
	 * @param	bResolveForcedExports	if true, the package name part of the return value will be the export's original package,
	 *									not the name of the package it's currently contained within.
	 *
	 * @return	the path name of the UObject represented by the resource at ExportIndex
	 */
	COREUOBJECT_API FString GetExportPathName(int32 ExportIndex, const TCHAR* FakeRoot = nullptr, bool bResolveForcedExports = false);

	/**
	 * Return the path name of the UObject represented by the specified export.
	 * (can be used with StaticFindObject)
	 *
	 * @param	PackageIndex			package index for the resource to get the name for
	 * @param	FakeRoot				Optional name to replace use as the root package of this object instead of the linker
	 * @param	bResolveForcedExports	if true, the package name part of the return value will be the export's original package,
	 *									not the name of the package it's currently contained within.
	 *
	 * @return	the path name of the UObject represented by the resource at PackageIndex, or the empty string if this isn't an export
	 */
	FString GetExportPathName(FPackageIndex PackageIndex, const TCHAR* FakeRoot = nullptr, bool bResolveForcedExports = false)
	{
		if (PackageIndex.IsExport())
		{
			return GetExportPathName(PackageIndex.ToExport(), FakeRoot, bResolveForcedExports);
		}
		return FString();
	}

	/**
	 * Return the path name of the UObject represented by the specified import.
	 * (can be used with StaticFindObject)
	 *
	 * @param	PackageIndex	package index
	 *
	 * @return	the path name of the UObject represented by the resource at PackageIndex, or the empty string if this is null
	 */
	FString GetPathName(FPackageIndex PackageIndex)
	{
		if (PackageIndex.IsImport())
		{
			return GetImportPathName(PackageIndex);
		}
		else if (PackageIndex.IsExport())
		{
			return GetExportPathName(PackageIndex);
		}
		return FString();
	}

	/**
	 * Return the full name of the UObject represented by the specified export.
	 *
	 * @param	ExportIndex				index into the ExportMap for the resource to get the name for
	 * @param	FakeRoot				Optional name to replace use as the root package of this object instead of the linker
	 * @param	bResolveForcedExports	if true, the package name part of the return value will be the export's original package,
	 *									not the name of the package it's currently contained within.
	 *
	 * @return	the full name of the UObject represented by the resource at ExportIndex
	 */
	COREUOBJECT_API FString GetExportFullName(int32 ExportIndex, const TCHAR* FakeRoot = nullptr, bool bResolveForcedExports = false);

	/**
	 * Return the full name of the UObject represented by the specified package index
	 *
	 * @param	PackageIndex			package index for the resource to get the name for
	 * @param	FakeRoot				Optional name to replace use as the root package of this object instead of the linker
	 * @param	bResolveForcedExports	if true, the package name part of the return value will be the export's original package,
	 *									not the name of the package it's currently contained within.
	 *
	 * @return	the full name of the UObject represented by the resource at PackageIndex
	 */
	FString GetExportFullName(FPackageIndex PackageIndex, const TCHAR* FakeRoot = nullptr, bool bResolveForcedExports = false)
	{
		if (PackageIndex.IsExport())
		{
			return GetExportFullName(PackageIndex.ToExport(), FakeRoot, bResolveForcedExports);
		}
		return FString();
	}

	/**
	 * Return the full name of the UObject represented by the specified export.
	 *
	 * @param	PackageIndex	package index
	 *
	 * @return	the path name of the UObject represented by the resource at PackageIndex, or the empty string if this is null
	 */
	FString GetFullImpExpName(FPackageIndex PackageIndex)
	{
		if (PackageIndex.IsImport())
		{
			return GetImportFullName(PackageIndex);
		}
		else if (PackageIndex.IsExport())
		{
			return GetExportFullName(PackageIndex);
		}
		return FString();
	}

	UE_DEPRECATED(5.6, "No longer supported. Contact Epic if you need this functionality.")
	void StartScriptSHAGeneration();
	UE_DEPRECATED(5.6, "No longer supported. Contact Epic if you need this functionality.")
	void UpdateScriptSHAKey(const TArray<uint8>& ScriptCode);
	UE_DEPRECATED(5.6, "No longer supported. Contact Epic if you need this functionality.")
	void GetScriptSHAKey(uint8* OutKey);

	/**
	 * Test and object against the load flag filters
	 *
	 * @return	true if the object should be filtered and not loaded
	 */
	bool FilterExport(const FObjectExport& Export)
	{
		if (Export.bExportLoadFailed || Export.bWasFiltered)
		{
			return true;
		}
#if WITH_EDITOR
		if (!Export.bNotAlwaysLoadedForEditorGame) // Always load, even if is editor only
		{
			return false;
		}
#endif
		if (FilterClientButNotServer && Export.bNotForServer) // "we are a dedicated server"
		{
			return true;
		}
		if (FilterServerButNotClient && Export.bNotForClient) // "we are a client only"
		{
			return true;
		}
		if (Export.ThisIndex.IsNull()) // Export is invalid and shouldn't be processed.
		{
			return true;
		}
		return false;
	}

};



template<typename T> 
UE_FORCEINLINE_HINT T* Cast(FLinker* Src)
{
	return Src && T::StaticType() == Src->GetType() ? (T*)Src : nullptr;
}

template<typename T>
inline T* CastChecked(FLinker* Src)
{
	T* LinkerCastResult = Src && T::StaticType() == Src->GetType() ? (T*)Src : nullptr;
	check(LinkerCastResult);
	return LinkerCastResult;
}

/*-----------------------------------------------------------------------------
	Lazy loading.
-----------------------------------------------------------------------------*/

/**
 * Flags serialized with the lazy loader.
 */
typedef uint32 ELazyLoaderFlags;

/**
 * Empty flag set.
 */
#define LLF_None					0x00000000
/**
 * If set, payload is [going to be] stored in separate file		
 */
#define	LLF_PayloadInSeparateFile	0x00000001
/**
 * If set, payload should be [un]compressed during serialization. Only bulk data that doesn't require 
 * any special serialization or endian conversion can be compressed! The code will simply serialize a 
 * block from disk and use the byte order agnostic Serialize( Data, Length ) function to fill the memory.
 */
#define	LLF_SerializeCompressed		0x00000002
/**
 * Mask of all flags
 */
#define	LLF_AllFlags				0xFFFFFFFF

/*-----------------------------------------------------------------------------
	Global functions
-----------------------------------------------------------------------------*/

/** 
 * Gets a linker for a package and returns it without loading any objects.
 * @param InOuter Package if known, can be null
 * @param PackagePath Package resource to load, must not be empty
 * @param LoadFlags Flags to pass to the new linker
 * @param Sandbox Additional sandbox for loading
 * @param CompatibleGuid Net GUID
 * @param InReaderOverride Optional archive to use for reading package data
 * @param InOutLoadContext Optional load context. If the package linker is already associated with a context that's currently loading objects that context will be returned in this param
 * @param ImportLinker Optional import linker that triggered the linker creation. it will pass in its instancing context if a linker is created, take precedence over the optional InstancingContext
 * @param InstancingContext Optional instancing context to pass in if a linker is created
 * @return Pointer to the loaded linker or null if the file didn't exist
 */
COREUOBJECT_API FLinkerLoad* GetPackageLinker(UPackage* InOuter, const FPackagePath& PackagePath, uint32 LoadFlags, UPackageMap* Sandbox, FArchive* InReaderOverride = nullptr, FUObjectSerializeContext** InOutLoadContext = nullptr, FLinkerLoad* ImportLinker = nullptr, const FLinkerInstancingContext* InstancingContext = nullptr);

COREUOBJECT_API FString GetPrestreamPackageLinkerName(const TCHAR* InLongPackageName, bool bSkipIfExists = true);

/**
 * Reset the linker exports associated with the package
 * @note, this might flush async loading if the linker is owned by the loading thread
 */
COREUOBJECT_API void ResetLinkerExports(UPackage* InPackage);

/**
 * Remove references to the linker for the given package and delete the linker.
 * Can be called after the package has finished loading.
 * Flushes async loading.
 */
COREUOBJECT_API void ResetLoaders(UObject* InOuter);
COREUOBJECT_API void ResetLoaders(TConstArrayView<UObject*> InOuters);


/**
 *  Conditionally flush async loading for a specific package if there's any pending async requests
 * 
 * @param InPackage		The package to flush for
 */
COREUOBJECT_API void ConditionalFlushAsyncLoadingForSave(UPackage* InPackage);

/**
 *
 * Reset the loader for the given package if it is using the given filename, so we can write to the file
 *
 * @param	Package			The package we are saving
 * @param	Filename		The filename we are saving too
 */
COREUOBJECT_API void ResetLoadersForSave(UPackage* Package, const TCHAR* Filename);

/**
 *
 * Reset the loaders for the given packages if they are using the given filenames, so we can write to the files
 *
 * @param	InPackage			The package we are saving along with their filename
 */
COREUOBJECT_API void ResetLoadersForSave(TConstArrayView<FPackageSaveInfo> InPackages);

/** Deletes all linkers that have finished loading */
COREUOBJECT_API void DeleteLoaders();

/** Queues linker for deletion */
COREUOBJECT_API void DeleteLoader(FLinkerLoad* Loader);

/*
 * Ensure all data that can be loaded from the linker (thumbnails, bulk data) is loaded, in preparation for saving out the given package
 *
 * @param Package	The the package for which the linker should be fully loaded
 */
COREUOBJECT_API void EnsureLoadingComplete(UPackage* Package);
