// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"

class FLinker;
class FName;
class FOutputDevice;
struct FLinkerExportObject;

class
UE_DEPRECATED(5.6, "Functionality is no longer needed, now that legacy Save is removed. Contact Epic if you need this functionality.")
FLinkerDiff
{
public:
	static COREUOBJECT_API FLinkerDiff CompareLinkers(FLinker* LHSLinker, FLinker* RHSLinker);

	COREUOBJECT_API bool HasDiffs() const;

	COREUOBJECT_API void PrintDiff(FOutputDevice& Ar);

private:
	void Generate(FLinker* LHSLinker, FLinker* RHSLinker);

	void GenerateSummaryDiff(FLinker* LHSLinker, FLinker* RHSLinker);
	void GenerateNameMapDiff(FLinker* LHSLinker, FLinker* RHSLinker);
	void GenerateGatherableTextDataMapDiff(FLinker* LHSLinker, FLinker* RHSLinker);
	void GenerateImportMapDiff(FLinker* LHSLinker, FLinker* RHSLinker);
	void GenerateExportAndDependsMapDiff(FLinker* LHSLinker, FLinker* RHSLinker);
	void GenerateSofPackageReferenceDiff(FLinker* LHSLinker, FLinker* RHSLinker);
	void GenerateSearchableNameMapDiff(FLinker* LHSLinker, FLinker* RHSLinker);

	void GenerateExportDiff(FLinker* LHSLinker, const FLinkerExportObject& LHSExport, const FLinkerExportObject& RHSExport);
	void GenerateDependsArrayDiff(FLinker* LHSLinker, int32 LHSIndex, FLinker* RHSLinker, int32 RHSIndex);
	void GenerateSearchableNameArrayDiff(const TArray<FName>& LHSNameArray, const TArray<FName>& RHSNameArray);

	// Array of diffs
	FString PackageName;
	TArray<FString> SummaryDiffs;
	TArray<FString> NameMapDiffs;
	TArray<FString> GatherableTextDataDiffs;
	TArray<FString> ImportDiffs;
	TArray<FString> ExportDiffs;
	TArray<FString> SoftPackageReferenceDiffs;
	TArray<FString> SearchableNameDiffs;
};