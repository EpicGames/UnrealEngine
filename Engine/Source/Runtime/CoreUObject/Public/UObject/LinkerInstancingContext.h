// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPath.h"
#include "IO/PackageId.h"

/**
 * Helper class to map between an original package and an instance of it (including world partition cells).
 */
class FLinkerInstancedPackageMap
{
public:
	enum class EInstanceMappingDirection : uint8
	{
		OriginalToInstanced,
		InstancedToOriginal,
	};

	FLinkerInstancedPackageMap() :
		FLinkerInstancedPackageMap(EInstanceMappingDirection::OriginalToInstanced) {}

	explicit FLinkerInstancedPackageMap(EInstanceMappingDirection MappingDirection)
		: InstanceMappingDirection(MappingDirection)
		, bIsInstanced(false)
#if WITH_EDITOR
		, bEnableNonEditorPath(false)
#else
		, bEnableNonEditorPath(true)
#endif
	{
	}

	bool IsInstanced() const
	{
		return bIsInstanced;
	}

	/** Remap the package name from the import table to its instanced counterpart, otherwise return the name unmodified. */
	FName RemapPackage(const FName& PackageName, bool* bWasFoundPtr = nullptr) const
	{
		if (const FName* RemappedName = InstancedPackageMapping.Find(FPackageId::FromName(PackageName)))
		{
			if (bWasFoundPtr)
			{
				*bWasFoundPtr = true;
			}
			return *RemappedName;
		}
		if (bWasFoundPtr)
		{
			*bWasFoundPtr = false;
		}
		return PackageName;
	}

	FPackageId RemapPackageId(const FPackageId& PackageId, bool* bWasFoundPtr = nullptr) const
	{
		if (const FName* RemappedName = InstancedPackageMapping.Find(PackageId))
		{
			if (bWasFoundPtr)
			{
				*bWasFoundPtr = true;
			}
			return FPackageId::FromName(*RemappedName);
		}
		if (bWasFoundPtr)
		{
			*bWasFoundPtr = false;
		}
		return PackageId;
	}

	/** Add a mapping from a package name to a new package name. There should be no separators (. or :) in these strings. */
	COREUOBJECT_API void AddPackageMapping(FName Original, FName Instanced);

	COREUOBJECT_API void BuildPackageMapping(FName Original, FName Instanced, const bool bBuildWorldPartitionCellMapping = true);

	COREUOBJECT_API bool FixupSoftObjectPath(FSoftObjectPath& InOutSoftObjectPath) const;

	bool ValidateAgainst(FLinkerInstancedPackageMap& OtherPackageMap) const;

private:
	friend class FLinkerInstancingContext;
	friend class FLinkerInstancingContextTests;
	void EnableAutomationTest() { bEnableNonEditorPath = true; }

	/**
	 * Map between the original package name and its instance counterpart.
	 * Key=Original and Value=Instanced when InstanceMappingDirection==OriginalToInstanced
	 * Key=Instanced and Value=Original when InstanceMappingDirection==InstancedToOriginal
	 */
	TMap<FPackageId, FName> InstancedPackageMapping;

	/**
	 * In which direction has this mapping been built?
	 */
	EInstanceMappingDirection InstanceMappingDirection;

	/** Data needed to re-map world partition cells */
	FString GeneratedPackagesFolder;
	FString InstancedPackagePrefix;
	FString InstancedPackageSuffix;

	/** Whether InstancedPackageMapping contains remapping data other that none */
	bool bIsInstanced;

	/** Allows tests to run non editor path from editor build. */
	bool bEnableNonEditorPath;
};

/**
 * Helper class to remap package imports during loading.
 * This is usually when objects in a package are outer-ed to object in another package or vice versa.
 * Instancing such a package without a instance remapping would resolve imports to the original package which is not desirable in an instancing context (i.e. loading a level instance)
 * This is because an instanced package has a different name than the package file name on disk, this class is used in the linker to remaps reference to the package name as stored in import tables on disk to the corresponding instanced package or packages we are loading.
 */
class FLinkerInstancingContext
{
public:
	enum class EContextSharingMode
	{
		Copy,
		Link,
		Share
	};

	COREUOBJECT_API FLinkerInstancingContext();

	/** Construct an FLinkerInstancingContext from another and share it's data */
	COREUOBJECT_API FLinkerInstancingContext(const FLinkerInstancingContext&);

	/** Construct an FLinkerInstancingContext from another and specifying the sharing mode: 
	* 
	*	Copy:  This will copy the ptr to the shared immutable data from the Source context, and duplicate the shared data of the Source context 
	* 
	*	Link: * If the Source context has no shared immutable data it'll construct this context by using the Source context shared data as the 
	*		     immutable data for this context and create a new share data object to use for mutations specific to this context. 
	* 
	*		   * If the Source context has shared immutable data we copy the ptr to share it as our shared immutable data , and we copy the 
	*			 the shared data ptr to share it. This means the context has been linked more than 1 time. 
	* 
	*	Share: We copy both the shared data and immutable share data ptrs. 
	* 
	*	The Link context is intended to be used internally in the loader/linker to ensure that mappings that occur during loading have no side
	*	effects on further loads done with the same FLinkerInstancingContext thus allowing them to be resued. 
	* 
	*/	
	COREUOBJECT_API FLinkerInstancingContext(const FLinkerInstancingContext& Source, EContextSharingMode Mode);

	COREUOBJECT_API explicit FLinkerInstancingContext(TSet<FName> InTags);
	COREUOBJECT_API explicit FLinkerInstancingContext(bool bInSoftObjectPathRemappingEnabled);

	COREUOBJECT_API static FLinkerInstancingContext DuplicateContext(const FLinkerInstancingContext& InLinkerInstancingContext);

	COREUOBJECT_API bool IsInstanced() const;

	/** Remap the package name from the import table to its instanced counterpart, otherwise return the name unmodified. */
	COREUOBJECT_API FName RemapPackage(const FName& PackageName) const;
	COREUOBJECT_API FPackageId RemapPackageId(const FPackageId& PackageId) const;

	/**
	 * Remap the top level asset part of the path name to its instanced counterpart, otherwise return the name unmodified. 
	 * i.e. remaps /Path/To/Package.AssetName:Inner to /NewPath/To/NewPackage.NewAssetName:Inner 
	 */
	COREUOBJECT_API FSoftObjectPath RemapPath(const FSoftObjectPath& Path) const;

	/** Add a mapping from a package name to a new package name. There should be no separators (. or :) in these strings. */
	COREUOBJECT_API void AddPackageMapping(FName Original, FName Instanced);

	/** Add a mapping function from a package name to a new package name. This function should be thread-safe, as it can be invoked from ALT. */
	COREUOBJECT_API void AddPackageMappingFunc(TFunction<FName(FName)> InInstancedPackageMapFunc);
	
	/** Add a mapping from a top level asset path (/Path/To/Package.AssetName) to another. */
	COREUOBJECT_API void AddPathMapping(FSoftObjectPath Original, FSoftObjectPath Instanced);

	COREUOBJECT_API void AddTag(FName NewTag);
	COREUOBJECT_API void AppendTags(const TSet<FName>& NewTags);
	COREUOBJECT_API bool HasTag(FName Tag) const;
	COREUOBJECT_API void SetSoftObjectPathRemappingEnabled(bool bInSoftObjectPathRemappingEnabled);
	COREUOBJECT_API bool GetSoftObjectPathRemappingEnabled() const;

	/** Return the instanced package name for a given instanced outer package and an object package name */
	static FString GetInstancedPackageName(const FString& InOuterPackageName, const FString& InPackageName)
	{
		return FString::Printf(TEXT("%s_InstanceOf_%s"), *InOuterPackageName, *InPackageName);
	}

	static FName GetInstancedPackageName(FName InOuterPackageName, FName InPackageName)
	{
		return *GetInstancedPackageName(InOuterPackageName.ToString(), InPackageName.ToString());
	}

	COREUOBJECT_API void FixupSoftObjectPath(FSoftObjectPath& InOutSoftObjectPath) const;

private:
	void EnableAutomationTest();
	void BuildPackageMapping(FName Original, FName Instanced);
	bool FindPackageMapping(FName Original, FName& Instanced) const;

	friend class FLinkerLoad;
	friend struct FAsyncPackage2;
	friend class FLinkerInstancingContextTests;
	class FSharedLinkerInstancingContextData;

	TSharedPtr<FSharedLinkerInstancingContextData> SharedData;
	TSharedPtr<FSharedLinkerInstancingContextData> ImmutableData;
};
