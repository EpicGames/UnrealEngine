// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Async/RecursiveMutex.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformMath.h"
#include "HAL/ThreadSafeCounter.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Misc/PackagePath.h"
#include "Serialization/Archive.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/AsyncLoadingEvents.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveFwd.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Templates/UniquePtr.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/Linker.h"
#include "UObject/LinkerInstancingContext.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectHandle.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectResource.h"
#include "UObject/PackageResourceManager.h"
#include "UObject/PersistentObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealNames.h"

class FBulkData;
class FLinkerLoad;
class FLinkerPlaceholderBase;
class FPackageIndex;
class FStructuredArchiveChildReader;
class IPakFile;
class UClass;
class ULinkerPlaceholderExportObject;
class UObject;
class UPackage;
class UStruct;
namespace UE::Serialization{ class FEditorBulkData; }
namespace UE::BulkData::Private{ class FBulkMetaData; }
namespace UE{ class FPackageTrailer; }
struct FObjectPtr;
struct FOpenPackageResult;
struct FScopedSlowTask;
struct FUObjectSerializeContext;
template <typename FuncType> class TFunction;

LLM_DECLARE_TAG(UObject_Linker);

/*----------------------------------------------------------------------------
	FLinkerLoad.
----------------------------------------------------------------------------*/

/**
 * Helper struct to keep track of all objects needed by an export (recursive dependency caching)
 */
struct FDependencyRef
{
	/** The Linker the export lives in */
	FLinkerLoad* Linker;

	/** Index into Linker's ExportMap for this object */
	int32 ExportIndex;

	/**
	 * Comparison operator
	 */
	bool operator==(const FDependencyRef& Other) const
	{
		return Linker == Other.Linker && ExportIndex == Other.ExportIndex;
	}

	/**
	 * Type hash implementation. Export indices are usually less than 100k, so are linker indices.
	 *
	 * @param	Ref		Reference to hash
	 * @return	hash value
	 */
	friend uint32 GetTypeHash( const FDependencyRef& Ref  );
};

/** Helper struct to keep track of the first time CreateImport() is called in the current callstack. */
struct FScopedCreateImportCounter
{
	/**
	*	Constructor. Called upon CreateImport() entry.
	*	@param Linker	- Current Linker
	*	@param Index	- Index of the current Import
	*/
	FScopedCreateImportCounter(FLinkerLoad* Linker, int32 Index);

	/** Destructor. Called upon CreateImport() exit. */
	~FScopedCreateImportCounter();

	/** Current load context object */
	FUObjectSerializeContext* LoadContext;
	/** Previously stored linker */
	FLinkerLoad* PreviousLinker;
	/** Previously stored index */
	int32 PreviousIndex;
};

/**
 * Handles loading Unreal package files, including reading UObject data from disk.
 */
class FAsyncArchive;

class FLinkerLoad 
#if !WITH_EDITOR && !WITH_LOW_LEVEL_TESTS
	final 
#endif
	: public FLinker, public FArchiveUObject
{
	// Friends.
	friend class UObject;
	friend class UPackageMap;
	friend struct FAsyncPackage;
	friend struct FAsyncPackage2;
	friend class FAsyncLoadingThread2;
	friend struct FResolvingExportTracker;
protected:
	/** Linker loading status. */
	enum ELinkerStatus
	{
		/** Error occured when loading. */
		LINKER_Failed = 0,
		/** Operation completed successfully. */
		LINKER_Loaded = 1,
		/** Operation took more time than allowed. */
		LINKER_TimedOut = 2
	};

	/** Verify result. */
	enum EVerifyResult
	{
		/** Error occured when verifying import (can be fatal). */
		VERIFY_Failed = 0,
		/** Verify completed successfully. */
		VERIFY_Success = 1,
		/** Verify completed successfully and followed a redirector. */
		VERIFY_Redirected = 2
	};

	// Variables.
public:

	UE_FORCEINLINE_HINT static ELinkerType::Type StaticType()
	{
		return ELinkerType::Load;
	}

	virtual ~FLinkerLoad();

	/** Flags determining loading behavior.																					*/
	uint32					LoadFlags;
	/** Indicates whether the imports for this loader have been verified													*/
	bool					bHaveImportsBeenVerified;

	UObject*				TemplateForGetArchetypeFromLoader;
	bool					bForceSimpleIndexToObject;
	bool					bLockoutLegacyOperations;

	// Used to protect LinkerLoad from multithreaded access
	UE::FRecursiveMutex     Mutex;

	/** True if Loader is FAsyncArchive  */
	bool					bIsAsyncLoader;
	UE_FORCEINLINE_HINT FAsyncArchive* GetAsyncLoader()
	{
		return bIsAsyncLoader ? (FAsyncArchive*)Loader : nullptr;
	}

	virtual FString GetDebugName() const override
	{
		return GetPackagePath().GetDebugName();
	}

	/** Get the PackagePath being loaded. For linkers created from LoadPackage this will be a mounted and extension-specified path, but it may be unmounted and unspecified for other linkers */
	const FPackagePath& GetPackagePath() const
	{
		return PackagePath;
	}

	UE_FORCEINLINE_HINT const FLinkerInstancingContext& GetInstancingContext() const
	{
		return InstancingContext;
	}

	const UE::FPackageTrailer* GetPackageTrailer() const
	{
		return PackageTrailer.Get();
	}

#if WITH_EDITOR
	bool IsPackageRelocated() const
	{
		return bIsPackageRelocated;
	}
#endif

private:

	/** True if the linker is currently deleting loader */
	bool					bIsDestroyingLoader;
#if WITH_EDITOR
	/** Tracks if DetachLoader has been called or not */
	bool					bDetachedLoader;
#endif // WITH_EDITOR

	/** Structured archive interface. Wraps underlying loader to provide contextual metadata to the values being written
	 *  which ultimately allows text based serialization of the data
	 */
	FStructuredArchive* StructuredArchive;
	FArchiveFormatterType* StructuredArchiveFormatter;
	TOptional<FStructuredArchive::FRecord> StructuredArchiveRootRecord;
	TArray<FStructuredArchiveChildReader*> ExportReaders;
	/** The packagepath being loaded */
	FPackagePath		PackagePath;

	/** Set the packagepath being loaded */
	void SetPackagePath(const FPackagePath& PackagePath);

	/** The archive that actually reads the raw data from disk.																*/
	FArchive*				Loader;

	/** The linker instancing context. */
	FLinkerInstancingContext InstancingContext;

	/** The trailer for the package */
	TUniquePtr<UE::FPackageTrailer> PackageTrailer;

	/** Set of imports that require additional verification at creation time. */
	TSet<int32> ImportsToVerifyOnCreate;

	// Helper function to access the InstancingContext IsInstanced, 
	bool IsContextInstanced() const;

	// Helper function to query if we should do any SoftObjectPath fixup
	bool IsSoftObjectRemappingEnabled() const;

	/** Remaps SoftObjectPaths using InstancingContextRemap. */
	void FixupSoftObjectPathForInstancedPackage(FSoftObjectPath& InOutSoftObjectPath);

protected:

	void SetLoader(FArchive* InLoader, bool bInLoaderNeedsEngineVersionChecks);
	FArchive* GetLoader() const { return Loader; }

public:

	/** Access the underlying archive. Note that if this is a text archive, the loader will point to a generic text file, and not a binary archive as
	  * in the past. General access to the underlying file format is discouraged and should only be done after checking that the linker package is what
	  * you expect it to be (i.e. !IsTextFormat())
	  */
	FArchive * GetLoader_Unsafe() const
	{
		return Loader;
	}

	bool HasLoader() const
	{
		return Loader != nullptr;
	}

	void DestroyLoader();

	/**
	 * Detaches all bulkdata currently attached to the FLinkerLoad followed by destroying the internal loader.
	 * This is a fairly dangerous method to call as it will leave the FLinkerLoad in a state where using it as
	 * a FArchive will cause an assert/crash. It is intended as a short/medium term fix to an internal engine
	 * issue and will be deprecated as soon as possible. 
	 */
	COREUOBJECT_API void DetachLoader();

	UE_FORCEINLINE_HINT bool IsDestroyingLoader() const
	{
		return bIsDestroyingLoader;
	}

	/** The async package associated with this linker */
	void* AsyncRoot;

#if WITH_EDITOR
	/** Used when accessing BulkDataLoaders/EditorBulkDataLoaders to ensure thread safety */
	UE::FMutex BulkDataMutex;

	/** Bulk data that use the FLinkerLoad to track the state of the file on disk */
	TSet<FBulkData*> BulkDataLoaders;
	TSet<UE::Serialization::FEditorBulkData*> EditorBulkDataLoaders;
#endif // WITH_EDITOR

	/** Hash table for exports.																								*/
	static constexpr int32 ExportHashCount = 256;
	TUniquePtr<int32[]> ExportHash;

	UE_FORCEINLINE_HINT static int32 GetHashBucket(FName Object)
	{
		return GetTypeHash(Object.GetComparisonIndex()) & (ExportHashCount - 1);
	}

	/**
	* List of imports and exports that must be serialized before other exports...all packed together, see FirstExportDependency
	*/
	TArray<FPackageIndex> PreloadDependencies;

	/**
	*  List of external read dependencies that must be finished to load this package
	*/
	TArray<FExternalReadCallback> ExternalReadDependencies;

	/** 
	 * Utility functions to query the object name redirects list for previous names for a class
	 * @param CurrentClassPath The current name of the class, with a full path
	 * @param bIsInstance If true, we're an instance, so check instance only maps as well
	 * @return Names without path of all classes that were redirected to this name. Empty if none found.
	 */
	COREUOBJECT_API static TArray<FName> FindPreviousNamesForClass(const FString& CurrentClassPath, bool bIsInstance);

	/**
	 * Utility functions to query the object name redirects list for previous names for a class
	 * @param CurrentClassPath The current name of the class, with a full path
	 * @param bIsInstance If true, we're an instance, so check instance only maps as well
	 * @param bIncludeShortNames If true, also include short names without a package
	 * @return Full object paths of all classes that were redirected to this name. Empty if none found.
	 */
	COREUOBJECT_API static TArray<FString> FindPreviousPathNamesForClass(const FString& CurrentClassPath, bool bIsInstance, bool bIncludeShortNames = false);

	/** 
	 * Utility functions to query the object name redirects list for the current name for a class
	 * @param OldClassName An old class name, without path
	 * @return Current name of the class. It will be None if no redirect found
	 */
	COREUOBJECT_API static FName FindNewNameForClass(FName OldClassName, bool bIsInstance);

	/**
	 * Utility functions to query the object name redirects list for the current name for a class
	 * @param OldClassNameOrPathName An old class name or pathname
	 * @return Current full path of the class. It will be empty if no redirect found
	 */
	COREUOBJECT_API static FString FindNewPathNameForClass(const FString& OldClassNameOrPathName, bool bIsInstance);

	/** 
	* Utility functions to query the enum name redirects list for the current name for an enum
	* @param OldEnumName An old enum name, without path
	* @return Current full path of the enum. It will be None if no redirect found
	*/
	COREUOBJECT_API static FName FindNewNameForEnum(FName OldEnumName);

	/** 
	* Utility functions to query the struct name redirects list for the current name for a struct
	* @param OldStructName An old struct name, without path
	* @return Current full path of the struct. It will be None if no redirect found
	*/
	COREUOBJECT_API static FName FindNewNameForStruct(FName OldStructName);

	/**
	 * Utility functions to check the list of known missing packages and silence any warnings
	 * that may have occurred on load.
	 * @return true if the provided package is in the KnownMissingPackage list
	 */
	COREUOBJECT_API static bool IsKnownMissingPackage(FName PackageName);

	/**
	 * Register that a package is now known missing and that it should silence future warnings/issues
	 */
	COREUOBJECT_API static void AddKnownMissingPackage(FName PackageName);

	/**
	 * Register that a package is no longer known missing and that it should be searched for again in the future
	 * @return true if the provided package was removed from the KnownMissingPackage list
	 */
	COREUOBJECT_API static bool RemoveKnownMissingPackage(FName PackageName);

	/**
	 * Determines if imports can be lazily loaded.  This relies on compile-time enabling of UE_WITH_OBJECT_HANDLE_LATE_RESOLVE from ObjectHandle.h as well as other factors.
	 * @return true if imports can be lazily loaded
	 */
	#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	COREUOBJECT_API static bool IsImportLazyLoadEnabled();
	#else
	inline static bool IsImportLazyLoadEnabled() { return false; }
	#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

	/**
	 * 
	 */
	COREUOBJECT_API static void OnNewFileAdded(const FString& Filename);

	COREUOBJECT_API static void OnPakFileMounted(const IPakFile& PakFile);

	/** 
	 * Checks if the linker has any objects in the export table that require loading.
	 */
	COREUOBJECT_API bool HasAnyObjectsPendingLoad() const;

	/** 
	 * Add a new redirect from old game name to new game name for ImportMap 
	 */
	COREUOBJECT_API static void AddGameNameRedirect(const FName OldName, const FName NewName);

	virtual bool SerializeBulkData(FBulkData& BulkData, const FBulkDataSerializationParams& Params) override;

private:

	void SerializeBulkMeta(UE::BulkData::Private::FBulkMetaData& Meta, FBulkDataCookedIndex& CookedIndex, int64& DuplicateSerialOffset, int32 ElementSize);

	// Variables used during async linker creation.

	/** Current index into soft object path list, used by async linker creation for spreading out serializing soft object path entries.	*/
	int32						SoftObjectPathListIndex;
	/** Current index into gatherable text data map, used by async linker creation for spreading out serializing text entries.	*/
	int32						GatherableTextDataMapIndex;
	/** Current index into import map, used by async linker creation for spreading out serializing importmap entries.			*/
	int32						ImportMapIndex;
	/** Current index into export map, used by async linker creation for spreading out serializing exportmap entries.			*/
	int32						ExportMapIndex;
#if WITH_METADATA
	/** Current index into meta data, used by async linker creation for spreading out serializing meta data entries.			*/
	int32						MetaDataMapIndex;
	int32						NumObjectMetaDataMap;
	int32						NumRootMetaDataMap;
#endif
	/** Current index into depends map, used by async linker creation for spreading out serializing dependsmap entries.			*/
	int32						DependsMapIndex;
	/** Current index into export hash map, used by async linker creation for spreading out hashing exports.					*/
	int32						ExportHashIndex;

	/** Whether we already matched up existing exports.																		*/
	std::atomic<bool> bHasFoundExistingExports;
	/** Whether we are already fully initialized.																			*/
	std::atomic<bool> bHasFinishedInitialization;

	/** Whether we already serialized the package file summary.																*/
	bool					bHasSerializedPackageFileSummary:1;
	/** Whether we already serialized the package trailer.																	*/
	bool					bHasSerializedPackageTrailer : 1;
	/** Whether we have already constructed the exports readers																*/
	bool					bHasConstructedExportsReaders:1;
	/** Whether we already serialized preload dependencies.																	*/
	bool					bHasSerializedPreloadDependencies:1;
	/** Whether we already fixed up import map.																				*/
	bool					bHasFixedUpImportMap:1;
	/** Whether we already populated the instancing context.																*/
	bool					bHasPopulatedInstancingContext:1;
	/** Whether we already relocated references.																			*/
	bool					bHasRelocatedReferences:1;
	/** Whether we already applied the instancing context to the soft object list.											*/
	bool					bHasAppliedInstancingContext : 1;
	/** Used for ActiveClassRedirects functionality */
	bool					bFixupExportMapDone:1;
	/** Whether we are gathering dependencies, can be used to streamline VerifyImports, etc									*/
	bool					bIsGatheringDependencies:1;
	/** Whether time limit is/ has been exceeded in current/ last tick.														*/
	bool					bTimeLimitExceeded:1;
	/** Whether to use a time limit for async linker creation.																*/
	bool					bUseTimeLimit:1;
	/** Whether to use the full time limit, even if we're blocked on I/O													*/
	bool					bUseFullTimeLimit:1;
	/** Whether the loader needs version and correctness checks (see OpenReadPackage)										*/
	bool					bLoaderNeedsEngineVersionChecks : 1;
	/** Whether we already serialized import type hierarchies.																*/
	bool					bHasSerializedImportTypeHierarchies : 1;
	/** Whether we already added dynamic imports.																			*/
	bool					bHasAddedDynamicImports : 1;

#if WITH_EDITOR
	/** Check to avoid multiple export duplicate fixups in case we don't save asset. */
	bool bExportsDuplicatesFixed : 1;

	/** Cache if the package is relocated or not */
	bool bIsPackageRelocated : 1;

	/** TRUE when the loader is actively serializing an object's script properties data.									*/
	bool bIsSerializingScriptProperties : 1;
#endif // WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/** Index of the 1st dynamic import that was added (all dynamic imports are consecutive at the end of the import table  */
	int32					DynamicImportsIndex;
#endif //#if WITH_EDITORONLY_DATA

	/** Call count of IsTimeLimitExceeded.																					*/
	int32					IsTimeLimitExceededCallCount;
	/** Current time limit to use if bUseTimeLimit is true.																	*/
	float					TimeLimit;
	/** Time at begin of Tick function. Used for time limit determination.													*/
	double					TickStartTime;
	/** Id of the thread that created this linker. This is to guard against using this linker on other threads than the one it was created on **/
	int32					OwnerThread;

	/**
	 * Helper struct to keep track of background file reads
	 */
	struct FPackagePrecacheInfo
	{
		/** Synchronization object used to wait for completion of async read. Pointer so it can be copied around, etc */
		FThreadSafeCounter* SynchronizationObject;

		/** Memory that contains the package data read off disk */
		void* PackageData;

		/** Size of the buffer pointed to by PackageData */
		int64 PackageDataSize;

		/**
		 * Basic constructor
		 */
		FPackagePrecacheInfo()
		: SynchronizationObject(NULL)
		, PackageData(NULL)
		, PackageDataSize(0)
		{
		}
		/**
		 * Destructor that will free the sync object
		 */
		~FPackagePrecacheInfo()
		{
			delete SynchronizationObject;
		}
	};

private:

	static FName NAME_LoadErrors;

	/** Makes sure the deprecated active redirects inis have been read */
	static bool bActiveRedirectsMapInitialized;

#if WITH_EDITOR
	/** Feedback scope that is created to house the slow task of an asynchronous linker load. Raw ptr so we don't pull in TUniquePtr for everything. */
	FScopedSlowTask* LoadProgressScope;

	/** Test whether we should report progress or not */
	UE_FORCEINLINE_HINT bool ShouldReportProgress() const
	{
		return !IsAsyncLoading() && (LoadFlags & (LOAD_Quiet | LOAD_Async)) == 0;
	}

	/** Test whether we should report progress or not based on how recently we last created a progress task */
	bool ShouldCreateThrottledSlowTask() const;

	virtual void PushDebugDataString(const FName& DebugData) override
	{
		FArchiveUObject::PushDebugDataString(DebugData);
		if ( Loader )
			Loader->PushDebugDataString(DebugData);
	}

	virtual void PopDebugDataString() override
	{
		FArchiveUObject::PopDebugDataString();
		if ( Loader )
			Loader->PopDebugDataString();
	}

#endif 

public:

	/**
	 * Add redirects to FLinkerLoad static map
	 */
	static void CreateActiveRedirectsMap(const FString& GEngineIniName);

	/**
	 * Test whether the given package index is a valid import or export in this package
	 */
	bool IsValidPackageIndex(FPackageIndex InIndex);

	/**
	 * Locates package index for a UPackage import
	 */
	COREUOBJECT_API bool FindImportPackage(FName PackageName, FPackageIndex& PackageIdx);

	/** 
	 * Locates package index for a given name in an outer.
	 */
	COREUOBJECT_API bool FindImport(FPackageIndex OuterIndex, FName ObjectName, FPackageIndex& OutObjectIndex);

	/**
	 * Finds an import given the full object path as a string.
	 * 
	 * Note: since imports are stored in an array, it needs to linearly search for _each_ import in the object path.
	 */
	COREUOBJECT_API bool FindImport(FStringView FullObjectPath, FPackageIndex& OutObjectIndex);

	/**
	 * Locates the class adjusted index and its package adjusted index for a given class name in the import map
	 */
	COREUOBJECT_API bool FindImportClassAndPackage(FName ClassName, FPackageIndex& ClassIdx, FPackageIndex& PackageIdx);
	
	/**
	 * Attempts to find the index for the given class object in the import list and adds it + its package if it does not exist
	 */
	COREUOBJECT_API bool CreateImportClassAndPackage(FName ClassName, FName PackageName, FPackageIndex& ClassIdx, FPackageIndex& PackageIdx);

	/**
	 * Allows object instances to be converted to other classes upon loading a package
	 */
	COREUOBJECT_API ELinkerStatus FixupExportMap();

	/**
	 * Returns whether linker has finished (potentially) async initialization.
	 *
	 * @return true if initialized, false if pending.
	 */
	UE_FORCEINLINE_HINT bool HasFinishedInitialization() const
	{
        return bHasFinishedInitialization;
	}

	/** Returns ID of the thread that created this linker */
	UE_FORCEINLINE_HINT int32 GetOwnerThreadId() const
	{
		return OwnerThread;
	}

	/**
	 * If this archive is a FLinkerLoad or FLinkerSave, returns a pointer to the FLinker portion.
	 */
	virtual FLinker* GetLinker() override { return this; }

	/** Flush Loader Cache */
	virtual void FlushCache() override;

	/**
	 * Creates and returns a FLinkerLoad object.
	 *
	 * @param	Parent				Parent object to load into, can be NULL (most likely case)
	 * @param	PackagePath			Path of the package on disk to load
	 * @param	LoadFlags			Load flags determining behavior
	 * @param	InLoader			Loader archive override
	 * @param	InstancingContext	Context to remap package name when loading a package on disk into a package with a different name
	 *
	 * @return	new FLinkerLoad object for Parent/ Filename
	 */
	COREUOBJECT_API static FLinkerLoad* CreateLinker(FUObjectSerializeContext* LoadContext, UPackage* Parent, const FPackagePath& PackagePath, uint32 LoadFlags, FArchive* InLoader = nullptr, const FLinkerInstancingContext* InstancingContext = nullptr);

	void Verify();

	COREUOBJECT_API FName GetExportClassPackage( int32 i );
	virtual FString GetArchiveName() const override;

#if WITH_EDITORONLY_DATA
	/**
	 * Recursively gathers the dependencies of a given export (the recursive chain of imports
	 * and their imports, and so on)

	 * @param ExportIndex Index into the linker's ExportMap that we are checking dependencies
	 * @param Dependencies Set of all dependencies needed
	 * @param bSkipLoadedObjects Whether to skip already loaded objects when gathering dependencies
	 */
	COREUOBJECT_API void GatherExportDependencies(int32 ExportIndex, TSet<FDependencyRef>& Dependencies, bool bSkipLoadedObjects=true);

	/**
	 * Recursively gathers the dependencies of a given import (the recursive chain of imports
	 * and their imports, and so on)

	 * @param ImportIndex Index into the linker's ImportMap that we are checking dependencies
	 * @param Dependencies Set of all dependencies needed
	 * @param bSkipLoadedObjects Whether to skip already loaded objects when gathering dependencies
	 */
	COREUOBJECT_API void GatherImportDependencies(int32 ImportIndex, TSet<FDependencyRef>& Dependencies, bool bSkipLoadedObjects=true);
#endif

	/**
	 * A wrapper around VerifyImportInner. If the VerifyImportInner (previously VerifyImport) fails, this function
	 * will look for a UObjectRedirector that will point to the real location of the object. You will see this if
	 * an object was renamed to a different package or group, but something that was referencing the object was not
	 * not currently open. (Rename fixes up references of all loaded objects, but naturally not for ones that aren't
	 * loaded).
	 *
	 * @param	ImportIndex	The index into this package's ImportMap to verify
	 * @return Verify import result
	 */
	EVerifyResult VerifyImport(int32 ImportIndex);
	
	/**
	 * Loads all objects in package.
	 *
	 * @param bForcePreload	Whether to explicitly call Preload (serialize) right away instead of being
	 *						called from EndLoad()
	 */
	COREUOBJECT_API void LoadAllObjects(bool bForcePreload);

	/**
	 * Returns the ObjectName associated with the resource indicated.
	 * 
	 * @param	ResourceIndex	location of the object resource
	 *
	 * @return	ObjectName for the FObjectResource at ResourceIndex, or NAME_None if not found
	 */
	FName ResolveResourceName( FPackageIndex ResourceIndex );

	/**
	 * Returns the Object associated with the resource indicated.
	 *
	 * @param	ResourceIndex	location of the object resource
	 *
	 * @return	The UObject at ResourceIndex, or nullptr if not found
	 */
	UE_INTERNAL COREUOBJECT_API UObject* ResolveResource(FPackageIndex ResourceIndex);
	
	UE_DEPRECATED(5.6, "This function was used only for the now-deleted Conform functionality and will be removed. Contact Epic if you need this function.")
	int32 FindExportIndex( FName ClassName, FName ClassPackage, FName ObjectName, FPackageIndex ExportOuterIndex );

	/**
	 * Serialize the object data for the specified object from the unreal package file.  Loads any
	 * additional resources required for the object to be in a valid state to receive the loaded
	 * data, such as the object's Outer, Class, or ObjectArchetype.
	 *
	 * When this function exits, Object is guaranteed to contain the data stored that was stored on disk.
	 *
	 * @param	Object	The object to load data for.  If the data for this object isn't stored in this
	 *					FLinkerLoad, routes the call to the appropriate linker.  Data serialization is 
	 *					skipped if the object has already been loaded (as indicated by the RF_NeedLoad flag
	 *					not set for the object), so safe to call on objects that have already been loaded.
	 *					Note that this function assumes that Object has already been initialized against
	 *					its template object.
	 *					If Object is a UClass and the class default object has already been created, calls
	 *					Preload for the class default object as well.
	 */
	COREUOBJECT_API void Preload( UObject* Object ) override;

	/**
	 * Before loading a persistent object from disk, this function can be used to discover
	 * the object in memory. This could happen in the editor when you save a package (which
	 * destroys the linker) and then play PIE, which would cause the Linker to be
	 * recreated. However, the objects are still in memory, so there is no need to reload
	 * them.
	 *
	 * @param ExportIndex	The index of the export to hunt down
	 * @return The object that was found, or null if it wasn't found
	 */
	UObject* FindExistingExport(int32 ExportIndex);

	/**
	 * @param ImportIndex	The index of the import to hunt down
	 * @return The object that was found, or null if it wasn't found
	 */
	UObject* FindExistingImport(int32 ImportIndex);

	/**
	 * Builds a string containing the full path for a resource in the export table.
	 *
	 * @param OutPathName		[out] Will contain the full path for the resource
	 * @param ResourceIndex		Index of a resource in the export table
	 */
	void COREUOBJECT_API BuildPathName( FString& OutPathName, FPackageIndex ExportIndex ) const;

	/**
	 * Checks if the specified export should be loaded or not.
	 * Performs similar checks as CreateExport().
	 *
	 * @param ExportIndex	Index of the export to check
	 * @return				true of the export should be loaded
	 */
	COREUOBJECT_API bool WillTextureBeLoaded( UClass* Class, int32 ExportIndex );

	/**
	 * Called when an object begins serializing property data using script serialization.
	 */
	virtual void MarkScriptSerializationStart( const UObject* Obj ) override;

	/**
	 * Called when an object stops serializing property data using script serialization.
	 */
	virtual void MarkScriptSerializationEnd( const UObject* Obj ) override;

	virtual UObject* GetArchetypeFromLoader(const UObject* Obj) override;

	/**
	 * Looks for an existing linker for the given package, without trying to make one if it doesn't exist
	 */
	COREUOBJECT_API static FLinkerLoad* FindExistingLinkerForPackage(const UPackage* Package);

	/**
	 * Replaces OldObject's entry in its linker with NewObject, so that all subsequent loads of OldObject will return NewObject.
	 * This is used to update instanced components that were serialized out, but regenerated during compile-on-load
	 *
	 * OldObject will be consigned to oblivion, and NewObject will take its place.
	 *
	 * WARNING!!!	This function is potentially very dangerous!  It should only be used at very specific times, and in very specific cases.
	 *				If you're unsure, DON'T TRY TO USE IT!!!
	 */
	COREUOBJECT_API static void PRIVATE_PatchNewObjectIntoExport(UObject* OldObject, UObject* NewObject,
		FUObjectSerializeContext* InLoadContext = nullptr, bool bHideGarbageObjects=true);
	COREUOBJECT_API void PRIVATE_PatchNewObjectIntoExport(int32 OldExportIndex, UObject* NewObject,
		FUObjectSerializeContext* InLoadContext = nullptr, bool bHideGarbageObjects = true);

	/**
	 * Wraps a call to the package linker's ResolveAllImports().
	 * 
	 * @param  Package    The package whose imports you want all loaded.
	 * 
	 * WARNING!!!	This function shouldn't be used carelessly, and serves as a
	 *				hacky entrypoint to FLinkerLoad's privates. It should only 
	 *				be used at very specific times, and in very specific cases.
	 *				If you're unsure, DON'T TRY TO USE IT!!!
	 */
	COREUOBJECT_API static void PRIVATE_ForceLoadAllDependencies(UPackage* Package);

	/**
	 * Invalidates the future loading of a specific object, so that subsequent loads will fail
	 * This is used to invalidate sub objects of a replaced object that may no longer be valid
	 */
	COREUOBJECT_API static void InvalidateExport(UObject* OldObject, bool bHideGarbageObjects = true);

#if WITH_EDITOR
	COREUOBJECT_API static bool GetPreloadingEnabled();
	COREUOBJECT_API static void SetPreloadingEnabled(bool bEnabled);
	COREUOBJECT_API static bool TryGetPreloadedLoader(const FPackagePath& InPackagePath, FOpenPackageResult& OutResult);
private:
		static bool bPreloadingEnabled;
public:
#endif

	/** 
	 * Adds external read dependency 
	 *
	 * @return true if dependency has been added
	 */
	virtual bool AttachExternalReadDependency(FExternalReadCallback& ReadCallback) override;

	/**
	 * Finalizes external dependencies till time limit is exceeded
	 *
	 * @return true if all dependencies are finished, false otherwise
	 */
	bool FinishExternalReadDependencies(double TimeLimit);

private:
#if WITH_EDITOR
	/**
	 * Returns a bool indicating that the class the object (specified by ExportIndex) was saved with matches the class
	 * that it was created with. Most commonly this returns false when native user (non-coreuobject) code has required
	 * a specific class of a specific type
	 */
	 bool DoesSavedClassMatchActualClass(int32 ExportIndex) const;
	
	/** 
	 * Helper that returns the current object (import or export) specified by package index. Does not attempt to resolve the object.
	 * See related function: IndexToObject. 
	 */
	const UObject* GetCurrentObjectAtIndex(FPackageIndex ObjectIndex) const;

	/**
	 * Nulls duplicated exports and fixes indexes in ExportMap to point to original objects instead of duplicates.
	 */
	void FixupDuplicateExports();

	/**
	 * Replaces all instances of OldIndex in ExportMap with NewIndex.
	 */
	void ReplaceExportIndexes(const FPackageIndex& OldIndex, const FPackageIndex& NewIndex);

#endif // WITH_EDITOR
	/**
	 * Helper for recursion in PRIVATE_ForceLoadAllDependencies; searches the Export table to find
	 * all children of the given ExportIndex. Does not search recursively - does not find children of children.
	 */
	void FindDirectChildExportsFromExportTable(int32 ExportIndex, TArray<int32>& OutChildExports);
	/**
	 * Recursive helper for InvalidateExport. Invalidates an export even if it does not have a UObject* in memory,
	 * and recursively invalidates child exports.
	 */
	void InvalidateExportIndex(int32 ExportIndex, bool bHideGarbageObjects);

	/**
	 * Validate if the current linker may reference a certain package
	 * Checks if the reference to a certain package from this linker is legal if that package isn't externally referenceable (i.e private outiside its plugin)
	 * @param InPackage the package to validate
	 * @returns true if the package can be referenced from this linker
	 */
	bool IsPackageReferenceAllowed(UPackage* InPackage);

	UObject* CreateExport( int32 Index );

	/**
	 * Creates export and preload if requested.
	 *
	 * @param Index Index of the export in export map.
	 * @param bForcePreload Whether to explicitly call Preload (serialize)
	 *        right away instead of being called from EndLoad().
	 *
	 * @return Created object.
	 */
	UObject* CreateExportAndPreload(int32 ExportIndex, bool bForcePreload = false);

	/**
	 * Utility function for easily retrieving the specified export's UClass.
	 * 
	 * @param  ExportIndex    Index of the export you want a class for.
	 * @return The class that the specified export's ClassIndex references.
	 */
	UClass* GetExportLoadClass(int32 ExportIndex);

#if WITH_EDITOR
	/**
	 * Utility function for callers to try and create a placeholder class
	 * object in the specified import slot if it represents a class object.
	 * This allows for objects to still be serialized even if missing its
	 * class type on load (the data will be serialized into a property bag).
	 * 
	 * Note: If the import is already non-NULL, this will not create a
	 * placeholder class (and the function will return NULL as a result).
	 *
	 * @param  ImportIndex    Index of the import that may be a missing class.
	 * @return A reference to the placeholder class (if created), otherwise NULL.
	 */
	UClass* TryCreatePlaceholderClassImport(int32 ImportIndex);

#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	// Universal placeholders force the loader to make placeholders for all objects
	// inferring what it can about the objects within the package and creating
	// placeholder imports to serve only the interests of the objects within the
	// package. To understand these functions simply read CreateUniversalPlaceholders
	void CreateUniversalPlaceholders();
	void TryCreateImportedPackage(FObjectImport& Import);
	void CreatePlaceholderImport(FObjectImport& Import);
	void TryCreatePlaceholderClassExport(const FObjectExport& Export);
	void TryCreateImportedClassForExport(const FObjectExport& Export);
	void TryCreateImportedClassesForImports();
	UClass* ForceCreatePlaceholderClassImport(FObjectImport& Import, UClass* SuperClassOverride = nullptr);
#endif // UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
#if UE_INTERNAL_LINKER_BOUNDS
	// FArchive bounds implementation - needed for high placeholder environments but
	// likely useful everywhere that can afford them:
	COREUOBJECT_API virtual void SetBoundsImpl(FArchiveBounds NewBounds) override;
	COREUOBJECT_API virtual FArchiveBounds GetBoundsImpl() const override;
	FArchiveBounds CurrentBounds;
#endif // UE_INTERNAL_LINKER_BOUNDS

	/**
	 * Utility function to create a placeholder class for the specified export.
	 * This will be called if the export could not resolve its LoadClass import.
	 * In that case, its data might still be serializable (into a property bag).
	 *
	 * @param  ExportIndex    Index of the export that's missing its type.
	 * @return The placeholder type that will be used to load the export's data.
	 */
	UClass* TryCreatePlaceholderClassForExport(int32 ExportIndex);
#endif

#if WITH_METADATA
	/** 
	 * Looks for and loads meta data object from export map.
	 *
	 * @param bForcePreload Whether to explicitly call Preload (serialize)
	 *        right away instead of being called from EndLoad().
	 * 
	 * @return If found returns index of meta data object in the export map,
	 *         INDEX_NONE otherwise.
	 */
	int32 LoadMetaDataFromExportMap(bool bForcePreload);
#endif // WITH_METADATA

	UObject* CreateImport( int32 Index );

	/**
	 * Determines if the specified import belongs to a native "compiled in"
	 * package (as opposed to an asset-file package). Recursive if the
	 * specified import is not a package itself.
	 * 
	 * @param  ImportIndex    An index into the ImportMap, defining the import you wish to check.
	 * @return True if the specified import comes from (or is) a "compiled in" package, otherwise false (it is an asset import).
	 */
	bool IsImportNative(const int32 ImportIndex) const;

	/**
	 * Attempts to lookup and return the corresponding FLinkerLoad object for 
	 * the specified import WITHOUT invoking  a load, or continuing to load 
	 * the import package (will only return one if it has already been 
	 * created... could still be in the process of loading).
	 * 
	 * @param  ImportIndex    Specifies the import that you would like a linker for.
	 * @return The imports associated linker (null if it hasn't been created yet).
	 */
	FLinkerLoad* FindExistingLinkerForImport(int32 ImportIndex) const;

	UObject* IndexToObject( FPackageIndex Index );

	void DetachExport( int32 i );

#if WITH_EDITOR
	/**
	 * Attaches/ associates the passed in bulk data object with the linker.
	 *
	 * @param	Owner		UObject owning the bulk data
	 * @param	BulkData	Bulk data object to associate
	 */
	virtual void AttachBulkData(UObject* Owner, FBulkData* BulkData) override;
	virtual void AttachBulkData(UE::Serialization::FEditorBulkData* BulkData) override;
	/**
	 * Detaches the passed in bulk data object from the linker.
	 *
	 * @param	BulkData	Bulk data object to detach
	 * @param	bEnsureBulkDataIsLoaded	Whether to ensure that the bulk data is loaded before detaching
	 */
	virtual void DetachBulkData(FBulkData* BulkData, bool bEnsureBulkDataIsLoaded) override;
	virtual void DetachBulkData(UE::Serialization::FEditorBulkData* BulkData, bool bEnsureBulkDataIsLoaded) override;
	/**
	 * Detaches all attached bulk  data objects.
	 *
	 * @param	bEnsureBulkDataIsLoaded	Whether to ensure that the bulk data is loaded before detaching
	 */
#endif
	void DetachAllBulkData(bool bEnsureBulkDataIsLoaded);
public:

	/**
	 * Detaches linker from bulk data.
	 */
	COREUOBJECT_API void LoadAndDetachAllBulkData();

	/**
	 * Detaches linker from bulk data and exports and removes itself from array of loaders.
	 */
	COREUOBJECT_API void Detach();

	/**
	 * Only detaches the linker from its exports and the reset their cached state in the linker without touching bulkdata or underlying loader
	 */
	COREUOBJECT_API void DetachExports();

private:

	// FArchive interface.
	/**
	 * Hint the archive that the region starting at passed in offset and spanning the passed in size
	 * is going to be read soon and should be precached.
	 *
	 * The function returns whether the precache operation has completed or not which is an important
	 * hint for code knowing that it deals with potential async I/O. The archive is free to either not
	 * implement this function or only partially precache so it is required that given sufficient time
	 * the function will return true. Archives not based on async I/O should always return true.
	 *
	 * This function will not change the current archive position.
	 *
	 * @param	PrecacheOffset	Offset at which to begin precaching.
	 * @param	PrecacheSize	Number of bytes to precache
	 * @return	false if precache operation is still pending, true otherwise
	 */
	inline virtual bool Precache(int64 PrecacheOffset, int64 PrecacheSize) override
	{
#if WITH_EDITOR
		checkf(!bDetachedLoader, TEXT("Attempting to call ::Precache on a FLinkerLoad that has previously called ::DetachLoader"));
#endif // WITH_EDITOR

		return Loader->Precache(PrecacheOffset, PrecacheSize);
	}

	inline virtual void Seek(int64 InPos) override
	{
#if WITH_EDITOR
		checkf(!bDetachedLoader, TEXT("Attempting to call ::Seek on a FLinkerLoad that has previously called ::DetachLoader"));
#endif // WITH_EDITOR

		Loader->Seek(InPos);
	}

	inline virtual int64 Tell() override
	{
#if WITH_EDITOR
		checkf(!bDetachedLoader, TEXT("Attempting to call ::Tell on a FLinkerLoad that has previously called ::DetachLoader"));
#endif // WITH_EDITOR

		return Loader->Tell();
	}

	inline virtual int64 TotalSize() override
	{
#if WITH_EDITOR
		checkf(!bDetachedLoader, TEXT("Attempting to call ::TotalSize on a FLinkerLoad that has previously called ::DetachLoader"));
#endif // WITH_EDITOR

		return Loader->TotalSize();
	}

	// this fixes the warning : 'ULinkerLoad::Serialize' hides overloaded virtual function
	using FLinker::Serialize;
	inline virtual void Serialize(void* V, int64 Length) override
	{
#if WITH_EDITOR
		checkf(!bDetachedLoader, TEXT("Attempting to call ::Serialize on a FLinkerLoad that has previously called ::DetachLoader"));
#endif // WITH_EDITOR

#if WITH_EDITOR
		// This handles the case where we're only serializing to a property bag and we don't immediately know where the script
		// property data starts/ends in the object's data stream. Allows for loading from packages saved with an older version.
		
#if UE_INTERNAL_LINKER_BOUNDS
		// UE_INTERNAL_UNIVERSAL_PLACEHOLDERS has more bounds available:
		const bool bOutOfBounds = CurrentBounds == FArchiveBounds() ? 
			false :
			Tell() + Length > CurrentBounds.BoundsBasis + CurrentBounds.BoundsSize;
#endif // UE_INTERNAL_LINKER_BOUNDS
		if (UNLIKELY(IsLoadingToPropertyBagObject() && !bIsSerializingScriptProperties))
		{
			Loader->Seek(Tell() + Length);
		}
#if UE_INTERNAL_LINKER_BOUNDS
		else if (UNLIKELY(bOutOfBounds))
		{
			Loader->Seek(Tell() + Length);
			if(bOutOfBounds)
			{
				SetCriticalError(); // report misread to FPropertyTag so we can skip bad data
			}
		}
#endif // UE_INTERNAL_LINKER_BOUNDS
		else
#endif // WITH_EDITOR
		{
			Loader->Serialize(V, Length);
		}
	}
	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override
	virtual FArchive& operator<<(UObject*& Object) override;
	inline virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override
	{
		FArchive& Ar = *this;
		FUniqueObjectGuid ID;
		Ar << ID;
		LazyObjectPtr = ID;
		return Ar;
	}

	virtual FArchive& operator<<(FObjectPtr& ObjectPtr) override;
	virtual FArchive& operator<<(FSoftObjectPath& Value) override;

	void BadSoftObjectPathError(int32 SoftObjIndex);
	void BadNameIndexError(int32 NameIndex);

	inline virtual FArchive& operator<<(FName& Name) override
	{
		FArchive& Ar = *this;
		int32 NameIndex;
		Ar << NameIndex;
		int32 Number = 0;
		Ar << Number;

		if (NameMap.IsValidIndex(NameIndex))
		{
			// if the name wasn't loaded (because it wasn't valid in this context)
			FNameEntryId MappedName = NameMap[NameIndex];

			// simply create the name from the NameMap's name and the serialized instance number
			Name = FName::CreateFromDisplayId(MappedName, Number);
		}
		else
		{
			Name = FName();
			BadNameIndexError(NameIndex);
			SetCriticalError();
		}

		return *this;
	}

	virtual void SetSerializedProperty(FProperty* InProperty) override
	{
		FArchiveUObject::SetSerializedProperty(InProperty);
		Loader->SetSerializedProperty(InProperty);
	}
	virtual void SetSerializedPropertyChain(const FArchiveSerializedPropertyChain* InSerializedPropertyChain,
		class FProperty* InSerializedPropertyOverride = nullptr) override
	{
		FArchiveUObject::SetSerializedPropertyChain(InSerializedPropertyChain, InSerializedPropertyOverride);
		Loader->SetSerializedPropertyChain(InSerializedPropertyChain, InSerializedPropertyOverride);
	}
	virtual void PushSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty) override
	{
		FArchive::PushSerializedProperty(InProperty, bIsEditorOnlyProperty);
		Loader->PushSerializedProperty(InProperty, bIsEditorOnlyProperty);
	}
	virtual void PopSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty) override
	{
		FArchive::PopSerializedProperty(InProperty, bIsEditorOnlyProperty);
		Loader->PopSerializedProperty(InProperty, bIsEditorOnlyProperty);
	}
#if WITH_EDITORONLY_DATA
	virtual bool IsEditorOnlyPropertyOnTheStack() const override
	{
		return Loader->IsEditorOnlyPropertyOnTheStack();
	}
#endif

	virtual void SetDebugSerializationFlags(uint32 InCustomFlags) override
	{
		FArchiveUObject::SetDebugSerializationFlags(InCustomFlags);
		Loader->SetDebugSerializationFlags(InCustomFlags);
	}


	// FLinkerLoad functions
	/**
	 * Safely verify that an import in the ImportMap points to a good object. This decides whether or not
	 * a failure to load the object redirector in the wrapper is a fatal error or not (return value)
	 *
	 * @param	i				The index into this packages ImportMap to verify
	 * @param	WarningSuffix	[out] additional information about the load failure that should be appended to
	 *							the name of the object in the load failure dialog.
	 *
	 * @return true if the wrapper should crash if it can't find a good object redirector to load
	 */
	bool VerifyImportInner(const int32 ImportIndex, FString& WarningSuffix);

	//
	// FLinkerLoad creation helpers BEGIN
	//

	/**
	 * Creates a FLinkerLoad object for async creation. Tick has to be called manually till it returns
	 * true in which case the returned linker object has finished the async creation process.
	 *
	 * @param	Parent				Parent object to load into, can be NULL (most likely case)
	 * @param	PackagePath			Path of the package data to load
	 * @param	LoadFlags			Load flags determining behavior
	 * @param	InstancingContext	Context to remap package name when loading a package on disk into a package with a different name
	 *
	 * @return	new FLinkerLoad object for Parent/ Filename
	 */
	COREUOBJECT_API static FLinkerLoad* CreateLinkerAsync(FUObjectSerializeContext* LoadContext, UPackage* Parent, const FPackagePath& PackagePath, uint32 LoadFlags, const FLinkerInstancingContext* InstancingContext
		, TFunction<void()>&& InSummaryReadyCallback
	);

protected: // Daniel L: Made this protected so I can override the constructor and create a custom loader to load the header of the linker in the DiffFilesCommandlet
	/**
	 * Ticks an in-flight linker and spends InTimeLimit seconds on creation. This is a soft time limit used
	 * if bInUseTimeLimit is true.
	 *
	 * @param	InTimeLimit		Soft time limit to use if bInUseTimeLimit is true
	 * @param	bInUseTimeLimit	Whether to use a (soft) timelimit
	 * @param	bInUseFullTimeLimit	Whether to use the entire time limit, even if blocked on I/O
	 * 
	 * @return	true if linker has finished creation, false if it is still in flight
	 */
	ELinkerStatus Tick( float InTimeLimit, bool bInUseTimeLimit, bool bInUseFullTimeLimit, TMap<TPair<FName, FPackageIndex>, FPackageIndex>* ObjectNameWithOuterToExportMap);

	/**
	 * Private constructor, passing arguments through from CreateLinker.
	 *
	 * @param	Parent				Parent object to load into, can be NULL (most likely case)
	 * @param	PackagePath			Path of the package data to load
	 * @param	LoadFlags			Load flags determining behavior
	 * @param	InstancingContext	The instancing context for remapping imports if needed.
	 */
	FLinkerLoad(UPackage* InParent, const FPackagePath& PackagePath, uint32 InLoadFlags, FLinkerInstancingContext InstancingContext = FLinkerInstancingContext());
private:
	/**
	 * Returns whether the time limit allotted has been exceeded, if enabled.
	 *
	 * @param CurrentTask	description of current task performed for logging spilling over time limit
	 * @param Granularity	Granularity on which to check timing, useful in cases where FPlatformTime::Seconds is slow (e.g. PC)
	 *
	 * @return true if time limit has been exceeded (and is enabled), false otherwise (including if time limit is disabled)
	 */
	COREUOBJECT_API bool IsTimeLimitExceeded( const TCHAR* CurrentTask, int32 Granularity = 1 );
protected: // Daniel L: Made this protected so I can override the constructor and create a custom loader to load the header of the linker in the DiffFilesCommandlet
	/**
	 * Creates loader used to serialize content.
	 */
	ELinkerStatus CreateLoader(
		TFunction<void()>&& InSummaryReadyCallback
	);
private:
	ELinkerStatus ProcessPackageSummary(TMap<TPair<FName, FPackageIndex>, FPackageIndex>* ObjectNameWithOuterToExportMap);

	/**
	 * Start the process of serializing the package file summary if needed
	 */
	ELinkerStatus SerializePackageFileSummary();

	/**
	 * Does the actual serialization of  the package file summary.
	 */
	ELinkerStatus SerializePackageFileSummaryInternal();

	/**
	 * Updates the linker, loader and root package with data from the package file summary.
	 * */
	ELinkerStatus UpdateFromPackageFileSummary();
	
	/** 
	 * Serializes the header for the package trailer, allowing us to identify the payloads of the package 
	 */
	ELinkerStatus SerializePackageTrailer();

	/**
	 * Serializes the name map.
	 */
	ELinkerStatus SerializeNameMap();

	/**
	 * Serializes the soft object map.
	 */
	ELinkerStatus SerializeSoftObjectPathList();

	/**
	 * Serializes the import map.
	 */
	ELinkerStatus SerializeImportMap();

	/**
	 * Fixes up the import map, performing remapping for backward compatibility and such.
	 */
	ELinkerStatus FixupImportMap();

	/**
	 * Generate remapping for the instancing context if this is an instanced package.
	 */
	ELinkerStatus PopulateInstancingContext();

	ELinkerStatus RelocateReferences();

	ELinkerStatus ApplyInstancingContext();

	/**
	 * Serializes the export map.
	 */
	ELinkerStatus SerializeExportMap();

#if WITH_TEXT_ARCHIVE_SUPPORT
	/**
	 * Create the export readers.
	 */
	ELinkerStatus ConstructExportsReaders();
#endif

#if WITH_METADATA
	/**
	 * Serializes the depends map.
	 */
	ELinkerStatus SerializeMetaData();
#endif

	/**
	 * Serializes the depends map.
	 */
	ELinkerStatus SerializeDependsMap();

	/**
	 * Serializes the preload dependencies.
	 */
	ELinkerStatus SerializePreloadDependencies();
	
	/**
	 * Serializes the data resource map.
	 */
	ELinkerStatus SerializeDataResourceMap();

#if WITH_EDITORONLY_DATA
	/**
	 * Serializes the import type hierarchy map.
	 */
	FLinkerLoad::ELinkerStatus SerializeImportTypeHierarchies();

	/**
	 * Handle class callbacks for each export.
	 */
	ELinkerStatus AddDynamicImports();
#endif

	/** Sets the basic linker archive info */
	void ResetStatusInfo();
public:
	/**
	 * Serializes the gatherable text data container.
	 */
	COREUOBJECT_API ELinkerStatus SerializeGatherableTextDataMap(bool bForceEnableForCommandlet = false);

	/**
	 * Serializes thumbnails
	 */
	COREUOBJECT_API ELinkerStatus SerializeThumbnails( bool bForceEnableForCommandlet=false );

	/** Inform the archive that blueprint finalization is pending. */
	virtual void ForceBlueprintFinalization() override;

	/**
	* Query method to help handle recursive behavior. When this returns true,
	* this linker is in the middle of, or is about to call FinalizeBlueprint()
	* (for a blueprint class somewhere in the current callstack). Needed when
	* we get to finalizing a sub-class before we've finished finalizing its
	* super (so we know we need to finish finalizing the super first).
	*
	* @return True if FinalizeBlueprint() is currently being ran (or about to be ran) for an export (Blueprint) class.
	*/
	bool IsBlueprintFinalizationPending() const;

	/**
	 * Gives external code the ability to create FLinkerPlaceholderBase objects
	 * in place of loads that may violate the LOAD_DeferDependencyLoads state.
	 * This will only produce a placeholder if LOAD_DeferDependencyLoads is set
	 * for this linker.
	 *
	 * NOTE: For now, this will only produce UClass placeholders, as that is the 
	 *       only type we've identified needing.
	 * 
	 * @param  Property		 The property for which you want to defer loading the value.
	 * @param  ObjectType    The expected type of the object you want to defer loading of.
	 * @param  ObjectPath    The full object/package path for the expected object.
	 * @return A FLinkerPlaceholderBase UObject that can be used in place of the import dependency.
	 */
	UObject* RequestPlaceholderValue(const FProperty* Property, const UClass* ObjectType, const TCHAR* ObjectPath);

private:
	/**
	 * Regenerates/Refreshes a blueprint class
	 *
	 * @param	LoadClass		Instance of the class currently being loaded and which is the parent for the blueprint
	 * @param	ExportObject	Current object bein exported
	 * @return	Returns true if regeneration was successful, otherwise false
	 */
	bool RegenerateBlueprintClass(UClass* LoadClass, UObject* ExportObject);

	/**
	 * Determines if the specified import should be deferred. If so, it will 
	 * instantiate a placeholder object in its place.
	 * 
	 * @param  ImportIndex    An index into this linker's ImportMap, specifying which import to check.
	 * @return True if the specified import was deferred, other wise false (it is ok to load it).
	 */
	bool DeferPotentialCircularImport(const int32 ImportIndex);
	
#if WITH_EDITOR
	/**
	* Determines if the Object Import error should be suppressed
	*
	* @param  ImportIndex    Internal index into this linker's ImportMap, references the import to check for suppression.
	* @return True if the import error should be suppressed
	*/
	bool IsSuppressableBlueprintImportError(int32 ImportIndex) const;

	/** @return True when the loader is serializing to a property bag placeholder object. */
	bool IsLoadingToPropertyBagObject() const;
#endif // WITH_EDITOR

	/**
	 * Stubs in a ULinkerPlaceholderExportObject for the specified export (if 
	 * one is required, meaning: the export's LoadClass is not fully formed). 
	 * This should rarely happen, but has been seen in cyclic Blueprint 
	 * scenarios involving Blueprinted components.
	 * 
	 * @param  ExportIndex    Identifies the export you want deferred.
	 * @param Outer			The outer of the export to potentially defer
	 * @return True if the export has been deferred (and should not be loaded).
	 */
	bool DeferExportCreation(const int32 ExportIndex, UObject* Outer);

	/**
	 * Iterates through this linker's ExportMap, looking for the corresponding
	 * class-default-object for the specified class (assumes that the supplied 
	 * class is an export itself, making this a Blueprint package).
	 * 
	 * @param  LoadClass    The Blueprint class that this linker is in charge of loading (also belonging to its ExportMap).
	 * @return An index into this linker's ExportMap array (INDEX_NONE if the CDO wasn't found).
	 */
	int32 FindCDOExportIndex(UClass* LoadClass);

	/**
	 * Combs the ImportMap for any imports that were deferred, and then creates 
	 * them (via CreateImport).
	 * 
	 * @param  LoadStruct    The (Blueprint) class or struct that you want resolved (so that it no longer contains dependency placeholders).
	 */
	void ResolveDeferredDependencies(UStruct* LoadStruct);

	/**
	 * Loads the import that the Placeholder was initially stubbed in for (NOTE:
	 * this could cause recursive behavior), and then replaces all known 
	 * placeholder references with the proper class.
	 * 
	 * @param  Placeholder		A ULinkerPlaceholderClass that was substituted in place of a deferred dependency.
	 * @param  ReferencingClass	The (Blueprint) class that was loading, while we deferred dependencies (now referencing the placeholder).
	 * @param  ObjectPath		Optional param that denotes the full object/package path for the object the placeholder is supposed to represent 
	 *                          (used when the passed placeholder is not tied to an import in the linker's ImportMap).
	 * @return The number of placeholder references replaced (could be none, if this was recursively resolved).
	 */
	int32 ResolveDependencyPlaceholder(class FLinkerPlaceholderBase* Placeholder, UClass* ReferencingClass = nullptr, const FName ObjectPath = NAME_None);

	/**
	 * Query method to help catch recursive behavior. When this returns true, a 
	 * dependency placeholder is in the middle of being resolved by 
	 * ResolveDependencyPlaceholder(). Used so a nested call would know to 
	 * complete that placeholder before continuing.
	 * 
	 * @return True if ResolveDependencyPlaceholder() is being ran on a placeholder that has yet to be resolved. 
	 */
	bool HasUnresolvedDependencies() const;

	/**
	 * Iterates through the ImportMap and calls CreateImport() for every entry, 
	 * creating/loading each import as we go. This also makes sure that class 
	 * imports have had ResolveDeferredDependencies() completely executed for 
	 * them (even those already running through it earlier in the callstack).
	 */
	void ResolveAllImports();

	/**
	 * Takes the supplied serialized class and serializes in its CDO, then 
	 * regenerates both.
	 * 
	 * @param  LoadClass    The loaded blueprint class (assumes that it has been fully loaded/serialized).
	 */
	void FinalizeBlueprint(UClass* LoadClass);

	/**
	 * Combs the ExportMap for any stubbed in ULinkerPlaceholderExportObjects,
	 * and finalizes the real export's class before actually creating it
	 * (exports are deferred when their class isn't fully formed at the time
	 * CreateExport() is called). Also, this function ensures that deferred CDO  
	 * serialization is executed (expects its class to be fully resolved at this
	 * point).
	 *
	 * @param  LoadClass    A fully loaded/serialized class that may have property references to placeholder export objects (in need of fix-up).
	 */
	void ResolveDeferredExports(UClass* LoadClass);

	/** Called when a class is loaded and its now safe to load deferred instances */
	void ResolvePlaceholder(ULinkerPlaceholderExportObject* Placeholder);

	/** Helper function to recursively resolve placeholders that were waiting for their outer */
	void ResolvedDeferredSubobjects(ULinkerPlaceholderExportObject* OwningPlaceholder);

	/**
	 * Sometimes we have to instantiate an export object that is of an imported 
	 * type, and sometimes in those scenarios (thanks to cyclic dependencies) 
	 * the type class could be a Blueprint type that is half resolved. To avoid
	 * having to re-instance objects on load, we have to ensure that the class
	 * is fully regenerated before we spawn any instances of it. That's where 
	 * this function comes in. It will make sure that the specified class is 
	 * fully loaded, finalized, and regenerated.
	 *
	 * NOTE: be wary, if called in the wrong place, then this could introduce 
	 *       nasty infinite recursion!
	 * 
	 * @param  ImportClass    The class you want to make sure is fully regenerated.
	 * @return True if the class could be regenerated (false if it didn't have its linker set).
	 */
	bool ForceRegenerateClass(UClass* ImportClass);

	/**
	 * Checks to see if an export (or one up its outer chain) is currently 
	 * in the middle of having its class dependency force-regenerated. This 
	 * function is meant to help avoid unnecessary recursion, as 
	 * ForceRegenerateClass() does nothing itself to stave off infinite 
	 * recursion.
	 * 
	 * @param  ExportIndex    Identifies the export you're about to call CreateExport() on.
	 * @return True if the specified export's class (or one up its outer chain) is currently being force-regenerated.
	 */
	bool IsExportBeingResolved(int32 ExportIndex);

	void ResetDeferredLoadingState();

	bool HasPerformedFullExportResolvePass();

	/** Finds import, tries to fall back to dynamic class if the object could not be found */
	UObject* FindImport(UClass* ImportClass, UObject* ImportOuter, const TCHAR* Name);
	/** Finds import, tries to fall back to dynamic class if the object could not be found */
	static UObject* FindImportFast(UClass* ImportClass, UObject* ImportOuter, FName Name, bool bFindObjectbyName = false);

#if	USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	/** 
	 * For deferring dependency loads, we block CDO serialization until the 
	 * class if complete. If we attempt to serialize the CDO while that is 
	 * happening, we instead defer it and record the export's index here (so we 
	 * can return to it later).
	 */
	bool bForceBlueprintFinalization;

	/** 
	 * Index of the CDO that should be used for blueprint finalization, may be INDEX_NONE
	 * in the case of some legacy content.
	 */
	int32 DeferredCDOIndex;

	/** 
	 * Used to track dependency placeholders currently being resolved inside of 
	 * ResolveDependencyPlaceholder()... utilized for nested reentrant behavior, 
	 * to make sure this placeholder is completely resolved before continuing on 
	 * to the next.
	 */
	TArray<class FLinkerPlaceholderBase*> ResolvingPlaceholderStack;

	/** 
	 * Internal list to track imports that were deferred, but don't belong to 
	 * the ImportMap (thinks ones loaded through config files via FProperty::ImportText).
	 */
	TMap<FName, FLinkerPlaceholderBase*> ImportPlaceholders;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING


	/** 
	0 *00 0C0r0e0ates the export hash.
	 */
	ELinkerStatus CreateExportHash();

	/**
	 * Finds existing exports in memory and matches them up with this linker. This is required for PIE to work correctly
	 * and also for script compilation as saving a package will reset its linker and loading will reload/ replace existing
	 * objects without a linker.
	 */
	ELinkerStatus FindExistingExports();

	/**
	 * Finalizes linker creation, adding linker to loaders array and potentially verifying imports.
	 */
	ELinkerStatus FinalizeCreation(TMap<TPair<FName, FPackageIndex>, FPackageIndex>* ObjectNameWithOuterToExportMap);

	//
	// FLinkerLoad creation helpers END
	//

private:

#if WITH_TEXT_ARCHIVE_SUPPORT
	// Function to get a slot for a given export
	FStructuredArchiveSlot GetExportSlot(FPackageIndex InExportIndex);
#endif
};

COREUOBJECT_API bool IsEnforcePackageCompatibleVersionCheck();
