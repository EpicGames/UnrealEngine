// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "IO/IoHash.h"
#include "Serialization/Archive.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/BulkDataCookedIndex.h"
#include "Serialization/FileRegionArchive.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "Templates/UniquePtr.h"
#include "UObject/Linker.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectResource.h"
#include "UObject/PackageTrailer.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectThreadContext.h"

class FBulkData;
class FObjectPostSaveContext;
class FOutputDevice;
class FPackagePath;
class UObject;
class UPackage;
class FSaveContext;
namespace UE { class FDerivedData; }
namespace UE { class FPackageTrailerBuilder; }
struct FLazyObjectPtr;
struct FUObjectSerializeContext;

/*----------------------------------------------------------------------------
	FLinkerSave.
----------------------------------------------------------------------------*/

/**
 * Handles saving Unreal package files.
 */
class FLinkerSave : public FLinker, public FArchiveUObject
{
public:

	UE_FORCEINLINE_HINT static ELinkerType::Type StaticType()
	{
		return ELinkerType::Save;
	}

	virtual ~FLinkerSave();

	// Variables.
	/** The archive that actually writes the data to disk. */
	FArchive* Saver = nullptr;

	FPackageIndex CurrentlySavingExport;
	UObject* CurrentlySavingExportObject = nullptr;
	TArray<FPackageIndex> DepListForErrorChecking;

	/** Index array - location of the resource for a UObject is stored in the ObjectIndices array using the UObject's Index */
	TMap<TObjectPtr<UObject>, FPackageIndex> ObjectIndicesMap;
	TMap<Verse::VCell*, FPackageIndex> CellIndicesMap;

	/** List of Searchable Names, by object containing them. This gets turned into package indices later */
	TMap<const UObject *, TArray<FName> > SearchableNamesObjectMap;

	/* Map from FName to the index of the name in the name array written into the package header. */
	TMap<FNameEntryId, int32> NameIndices;

	/* Map from FSoftObjectPath to the index of the path in the soft object path array written into the package header. */
	TMap<FSoftObjectPath, int32> SoftObjectPathIndices;

	/** Flag that indicate if we are currently serializing the package header soft object paths. Used to disable mapping while serializing the header itself. */
	bool bIsWritingHeaderSoftObjectPaths = false;

	TArray<FFileRegion> FileRegions;

	/**
	 * Callback for arbitrary serializers to append data to the end of the ExportsArchive.
	 * Some PackageWriters used by SavePackage will write this data to a separate archive.
	 * 
	 * @param ExportsArchive The archive containing the UObjects and structs, this is always this LinkerSave.
	 * @param DataArchive The archive to which the data should be written. Might be this LinkerSave, or might be a separate archive.
	 * @param DataStartOffset The offset to the beginning of the range in DataArchive that should be stored in the UObject or struct's
	 *        export data. Reading at DataStartOffset from the FArchive passed into Serialize during a load will return the data that the
	 *		  callback wrote to DataArchive.
	 */
	using AdditionalDataCallback = TUniqueFunction<void(FLinkerSave& ExportsArchive, FArchive& DataArchive, int64 DataStartOffset)>;
	/** 
	 * Array of callbacks that will be invoked when it is possible to serialize out data 
	 * to the end of the output file.
	 */
	TArray<AdditionalDataCallback> AdditionalDataToAppend;

	/**
	 * Set to true when the package is being saved due to a procedural save.
	 * Any save without the the possibility of user-generated edits to the package is a procedural save (Cooking, EditorDomain).
	 * This allows us to execute transforms that only need to be executed in response to new user data.
	 */
	bool bProceduralSave = false;

	/**
	 * Set to true when the LoadedPath of the package being saved is being updated.
	 * This allows us to update the in-memory package when it is saved in editor to match its new save file.
	 * This is used e.g. to decide whether to update the in-memory file offsets for BulkData.
	 */
	bool bUpdatingLoadedPath = false;

	/** When set to true, payloads that are currently virtualized should be downloaded and stored locally with the package */
	bool bRehydratePayloads = false;
	
	struct FSidecarStorageInfo
	{
		FIoHash Identifier;
		FCompressedBuffer Payload;
	};

	/** Used by FEditorBulkData to add payloads to be added to the payload sidecar file (currently an experimental feature) */
	TArray<FSidecarStorageInfo> SidecarDataToAppend;
	
	/** Gathers all payloads while save the package, so that they can be stored in a single data structure @see FPackageTrailer */
	TUniquePtr<UE::FPackageTrailerBuilder> PackageTrailerBuilder;

	/** 
	 * Array of callbacks that will be invoked when the package has successfully saved to disk.
	 * The callbacks will not be invoked if the package fails to save for some reason.
	 * Unlike subscribing to the UPackage::PackageSavedEvent this callback allows custom data
	 * via lambda capture. 
	 * @param PackagePath The path of the package
	 */
	TArray<TUniqueFunction<void(const FPackagePath& PackagePath, FObjectPostSaveContext ObjectSaveContext)>> PostSaveCallbacks;

	/** A mapping of package name to generated script SHA keys */
	UE_DEPRECATED(5.6, "No longer supported, contact Epic if you need this functionality.")
	COREUOBJECT_API static TMap<FString, TArray<uint8> > PackagesToScriptSHAMap;

	/** Constructor for file writer */
	UE_DEPRECATED(5.6, "Use constructor that takes only Parent and then call AssignFileSaver.")
	FLinkerSave(UPackage* InParent, const TCHAR* InFilename, bool bForceByteSwapping, bool bInSaveUnversioned = false );
	/** Constructor for memory writer */
	UE_DEPRECATED(5.6, "Use constructor that takes only Parent and then call AssignMemorySaver.")
	FLinkerSave(UPackage* InParent, bool bForceByteSwapping, bool bInSaveUnversioned = false );
	/** Constructor for custom savers. The linker assumes ownership of the custom saver. */
	UE_DEPRECATED(5.6, "Use constructor that takes only Parent and then call AssignSaver.")
	FLinkerSave(UPackage* InParent, FArchive *InSaver, bool bForceByteSwapping, bool bInSaveUnversioned = false);

	/**
	 * Partially construct the FLinkerSave. No FArchive functions may be called on *this until AssignSaver has been
	 * called with a non-null FArchive.
	 */
	explicit FLinkerSave(UPackage* InParent);

	/**
	 * Assign the Saver that will receive the binary data for the package and write it to disk/memory/unspecified.
	 * Must be called only once, after construction and before any calls to FArchive functions on this.
	 * Takes ownership of the given archive, and deletes it in CloseAndDestroySaver or destructor.
	 * 
	 * @param bForceByteSwapping Passed to FArchive::IsByteSwapping
	 * @param bInSaveUnversioned Passed to FPackageFileSummary::bUnversioned, which if true removes native versioning
	 *        and custom versions from the saved package. Used for cooked packages, which exactly match native
	 *        runtime version.
	 */
	void AssignSaver(FArchive* InSaver, bool bForceByteSwapping, bool bInSaveUnversioned = false);
	/** Construct a FMemoryWriter archive and call AssignSaver with it. @see AssignSaver. */
	void AssignMemorySaver(bool bForceByteSwapping, bool bInSaveUnversioned = false);
	/**
	 * Construct a file saver from IPlatformFile and call AssignSaver with it. @see AssignSaver.
	 * If the file write fails, the Saver will be set to nullptr and it is a fatal error to call FArchive functions on
	 * *this. Caller must handle returnvalue == false and if so destroy *this without calling FArchive functions.
	 * 
	 * @return Whether the creation of the Saver archive succeeded.
	 */
	bool TryAssignFileSaver(FStringView InFilename, bool bForceByteSwapping, bool bInSaveUnversioned = false);

	/**
	 * Assign the temporary save context with the linker
	 */
	void AssignSaveContext(FSaveContext* InContext)
	{
		SaveContext = InContext;
	}

	/**
	 * Query the temporary SaveContext for the saveable state of an object, if any.
	 * Returns false if no save context is available
	 */
	UE_INTERNAL COREUOBJECT_API bool IsUnsaveable(TObjectPtr<UObject> Object) const;

	/** Returns the appropriate name index for the source name, or INDEX_NONE if not found in NameIndices */
	int32 MapName( FNameEntryId Name) const;

	/** Returns the appropriate soft object path index for the source soft object path, or INDEX_NONE if not found. */
	int32 MapSoftObjectPath(const FSoftObjectPath& SoftObjectPath) const;

	/** Returns the appropriate package index for the source object, or default value if not found in ObjectIndicesMap */
	FPackageIndex MapObject(TObjectPtr<const UObject> Object, bool bValidateExcluded = true) const;

	/**
	 * Called when an object begins serializing property data using script serialization.
	 */
	virtual void MarkScriptSerializationStart(const UObject* Obj) override;

	/**
	 * Called when an object stops serializing property data using script serialization.
	 */
	virtual void MarkScriptSerializationEnd(const UObject* Obj) override;

	// FArchive interface.
	using FArchiveUObject::operator<<; // For visibility of the overloads we don't override
	virtual FArchive& operator<<(FName& InName) override;
	virtual FArchive& operator<<(UObject*& Obj) override;
	virtual FArchive& operator<<(FObjectPtr& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Cell) override;
#endif
	virtual FArchive& operator<<(FSoftObjectPath& SoftObjectPath) override;
	virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override;
	virtual bool ShouldSkipProperty(const FProperty* InProperty) const override;
	virtual void UsingCustomVersion(const struct FGuid& Guid) override;
	virtual void SetUseUnversionedPropertySerialization(bool bInUseUnversioned) override;
	virtual void SetDebugSerializationFlags(uint32 InCustomFlags) override;
	virtual void SetFilterEditorOnly(bool bInFilterEditorOnly) override;

	/** Sets the map of overrided properties for each export that should be treated as transient, and nulled out when serializing */
	void SetTransientPropertyOverrides(const TMap<UObject*, TSet<FProperty*>>& InTransientPropertyOverrides)
	{
		TransientPropertyOverrides = &InTransientPropertyOverrides;
	}

	/** Set target platform memory map alignment. A negative value disables memory mapped bulk data. */
	void SetMemoryMapAlignment(int64 InAlignment)
	{
		MemoryMappingAlignment = InAlignment;
	}

	/** Sets whether file regions will be written or not. */
	void SetFileRegionsEnabled(bool bEnabled)
	{
		bFileRegionsEnabled = bEnabled;
	}

	/** Sets whether a separate file regions should be declared for each bulkdata */
	void SetDeclareRegionForEachAdditionalFile(bool bValue)
	{
		bDeclareRegionForEachAdditionalFile = bValue;
	}

	/** Sets whether saving bulk data by reference, i.e. leaving the bulk data payload in the original .uasset file when using EditorDomain. */
	void SetSaveBulkDataByReference(bool bValue)
	{
		bSaveBulkDataByReference = bValue;
	}
	
	/** Sets whether bulk data will be stored in the Linker archive (.uasset) or in sepearate files (.ubulk, .m.ubulk, .opt.ubulk) .*/ 
	void SetSaveBulkDataToSeparateFiles(bool bValue)
	{
		bSaveBulkDataToSeparateFiles = bValue;
	}

#if WITH_EDITOR
	// proxy for debugdata
	virtual void PushDebugDataString(const FName& DebugData) override { Saver->PushDebugDataString(DebugData); }
	virtual void PopDebugDataString() override { Saver->PopDebugDataString(); }
#endif

	virtual FString GetArchiveName() const override;

	/**
	 * If this archive is a FLinkerLoad or FLinkerSave, returns a pointer to the FLinker portion.
	 */
	virtual FLinker* GetLinker() { return this; }

	void Seek( int64 InPos );
	int64 Tell();
	// this fixes the warning : 'FLinkerSave::Serialize' hides overloaded virtual function
	using FLinker::Serialize;
	void Serialize( void* V, int64 Length );

	/** Invoke all of the callbacks in PostSaveCallbacks and then empty it. */
	void OnPostSave(const FPackagePath& PackagePath, FObjectPostSaveContext ObjectSaveContext);
	
	/** Triggered after bulk data payloads has been serialized to disk/package writer.*/
	void OnPostSaveBulkData();

	virtual void SetSerializedProperty(FProperty* InProperty) override;
	virtual void SetSerializedPropertyChain(const FArchiveSerializedPropertyChain* InSerializedPropertyChain,
		class FProperty* InSerializedPropertyOverride = nullptr) override;
	virtual void PushSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty) override;
	virtual void PopSerializedProperty(class FProperty* InProperty, const bool bIsEditorOnlyProperty) override;
#if WITH_EDITORONLY_DATA
	virtual bool IsEditorOnlyPropertyOnTheStack() const override;
#endif

	// FLinker interface
	virtual FString GetDebugName() const override;

	// FLinkerSave interface
	/**
	 * Closes and deletes the Saver (file, memory or custom writer) which will close any associated file handle.
	 * Returns false if the owned saver contains errors after closing it, true otherwise.
	 */
	bool CloseAndDestroySaver();

	/** Get the filename being saved to */
	const FString& GetFilename() const;

	/* Set the output device used to log errors, if any. */
	void SetOutputDevice(FOutputDevice* InOutputDevice)
	{
		LogOutput = InOutputDevice;
	}

	/* Returns an output Device that can be used to log info, warnings and errors etc. */
	FOutputDevice* GetOutputDevice() const 
	{
		return LogOutput;
	}

#if WITH_EDITORONLY_DATA
	/**
	 * Adds the derived data to the package. This is only supported when saving a cooked package.
	 *
	 * @return A reference that can be used to load the derived data from the cooked package.
	 */
	UE::FDerivedData AddDerivedData(const UE::FDerivedData& Data);
#endif // WITH_EDITORONLY_DATA

	virtual bool SerializeBulkData(FBulkData& BulkData, const FBulkDataSerializationParams& Params) override;

	void ForEachBulkDataCookedIndex(TUniqueFunction<void(FBulkDataCookedIndex, FFileRegionMemoryWriter&)>&& Func, EBulkDataPayloadType Type) const;

	FFileRegionMemoryWriter& GetBulkDataArchive(FBulkDataCookedIndex CookedIndex);
	FFileRegionMemoryWriter& GetOptionalBulkDataArchive(FBulkDataCookedIndex CookedIndex);
	FFileRegionMemoryWriter& GetMemoryMappedBulkDataArchive(FBulkDataCookedIndex CookedIndex);

	bool HasCookedIndexBulkData() const;

protected:
	/** Set the filename being saved to */
	void SetFilename(FStringView InFilename);

	const TMap<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>>& GetArchives(EBulkDataPayloadType Type) const;

	void SerializeObjectPointer(const FObjectPtr& Value);

	void AssignSaverInternal(FArchive* InSaver, bool bForceByteSwapping, bool bInSaveUnversioned);

private:
	/** temporary save context used to generate this linker. */
	FSaveContext* SaveContext = nullptr;
	/** Optional log output to bubble errors back up. */
	FOutputDevice* LogOutput = nullptr;

#if WITH_EDITORONLY_DATA
	/** The index of the last derived data chunk added to the package. */
	int32 LastDerivedDataIndex = -1;
#endif

#if WITH_EDITOR
	/** Map from bulk data object to resource index. */
	TMap<FBulkData*, int32> SerializedBulkData;
#endif //WITH_EDITOR

	/** Default bulk data archive. */
	TMap<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>> BulkDataAr; // Can't use TUniqueObj as FFileRegionMemoryWriter isn't movable
	/** Optional bulk data archive. */
	TMap<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>> OptionalBulkDataAr;
	/** Memory mapped bulk data archive. */
	TMap<FBulkDataCookedIndex, TUniquePtr<FFileRegionMemoryWriter>> MemoryMappedBulkDataAr;

	/** The name of the file for this package */
	FString Filename;

	const TMap<UObject*, TSet<FProperty*>>* TransientPropertyOverrides = nullptr;
	/** Alignment for memory mapped data .*/ 
	int64 MemoryMappingAlignment = -1;
	/** Whether file regions are enabled. */
	bool bFileRegionsEnabled = false;
	/** Whether a separate file region should be declared for each bulkdata. */
	bool bDeclareRegionForEachAdditionalFile = false;
	/** Whether saving bulk data by reference. */
	bool bSaveBulkDataByReference = false;
	/** Whether saving bulk data to separate files. */
	bool bSaveBulkDataToSeparateFiles = false;
};
