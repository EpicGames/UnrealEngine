// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "MetaData.generated.h"

class FArchive;
class UPackage;

/*-----------------------------------------------------------------------------
	UMetaData.
-----------------------------------------------------------------------------*/

/**
 * An object that holds a map of key/value pairs. This is now deprecated in favor
 * of FMetaData which will be always present and owned by UPackage. Deprecation
 * happens in UDEPRECATED_MetaData::Serialize.
 */
UCLASS(MinimalAPI, Deprecated, Config = Engine)
class UDEPRECATED_MetaData : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Mapping between an object, and its key->value meta-data pairs. 
	 */
	TMap< FWeakObjectPtr, TMap<FName, FString> > ObjectMetaDataMap;

	/**
	 * Root-level (not associated with a particular object) key->value meta-data pairs.
	 * Meta-data associated with the package itself should be stored here.
	 */
	TMap< FName, FString > RootMetaDataMap;

public:
	// UObject interface
	COREUOBJECT_API virtual void Serialize(FArchive& Ar) override;
	virtual bool NeedsLoadForEditorGame() const override { return true; }
	virtual bool IsAsset() const override { return false; }
	// End of UObject interface

private:
	static COREUOBJECT_API void InitializeRedirectMap();

private:
	// Redirect map from deprecated keys to current key names
	static COREUOBJECT_API TMap<FName, FName> KeyRedirectMap;
	static COREUOBJECT_API UE::FMutex KeyRedirectMapLock;
};

#if WITH_METADATA
class FMetaData
{
public:
	/**
	 * Mapping between an object, and its key->value meta-data pairs. 
	 */
	TMap< FSoftObjectPath, TMap<FName, FString> > ObjectMetaDataMap;

	/**
	 * Root-level (not associated with a particular object) key->value meta-data pairs.
	 * Meta-data associated with the package itself should be stored here.
	 */
	TMap< FName, FString > RootMetaDataMap;

	/** Remap all object keys when renaming the owner package. */
	void RemapObjectKeys(FName OldPackageName, FName NewPackageName);

public:
	[[nodiscard]] FMetaData() = default;
	[[nodiscard]] explicit consteval FMetaData(EConstEval)
	: ObjectMetaDataMap(ConstEval)
	, RootMetaDataMap(ConstEval)
	{
	}

	// MetaData utility functions

	/**
	 * Return the value for the given key in the given property
	 * @param Object the object to lookup the metadata for
	 * @param Key The key to lookup
	 * @return The value if found, otherwise an empty string
	 */
	COREUOBJECT_API const FString& GetValue(const UObject* Object, const TCHAR* Key);
	
	/**
	 * Return the value for the given key in the given property
	 * @param Object the object to lookup the metadata for
	 * @param Key The key to lookup
	 * @return The value if found, otherwise an empty string
	 */
	COREUOBJECT_API const FString& GetValue(const UObject* Object, FName Key);

	/**
	 * Return whether or not the Key is in the meta data
	 * @param Object the object to lookup the metadata for
	 * @param Key The key to query for existence
	 * @return true if found
	 */
	bool HasValue(const UObject* Object, const TCHAR* Key)
	{
		return FindValue(Object, Key) != nullptr;
	}

	/**
	 * Return whether or not the Key is in the meta data
	 * @param Object the object to lookup the metadata for
	 * @param Key The key to query for existence
	 * @return true if found
	 */
	bool HasValue(const UObject* Object, FName Key)
	{
		return FindValue(Object, Key) != nullptr;
	}

	/**
	 * Returns the value for a the given key if it exists, null otherwise
	 * @param Object the object to lookup the metadata for
	 * @param Key The key to query for existence
	 * @return true if found
	 */
	COREUOBJECT_API const FString* FindValue(const UObject* Object, const TCHAR* Key);

	/**
	 * Returns the value for a the given key if it exists, null otherwise
	 * @param Object the object to lookup the metadata for
	 * @param Key The key to query for existence
	 * @return true if found
	 */
	COREUOBJECT_API const FString* FindValue(const UObject* Object, FName Key);

	/**
	 * Is there any metadata for this property?
	 * @param Object the object to lookup the metadata for
	 * @return TrUE if the object has any metadata at all
	 */
	COREUOBJECT_API bool HasObjectValues(const UObject* Object);

	/**
	 * Set the key/value pair in the Property's metadata
	 * @param Object the object to set the metadata for
	 * @Values The metadata key/value pairs
	 */
	COREUOBJECT_API void SetObjectValues(const UObject* Object, const TMap<FName, FString>& Values);

	/**
	 * Set the key/value pair in the Property's metadata
	 * @param Object the object to set the metadata for
	 * @Values The metadata key/value pairs
	 */
	COREUOBJECT_API void SetObjectValues(const UObject* Object, TMap<FName, FString>&& Values);

	/**
	 * Set the key/value pair in the Object's metadata
	 * @param Object the object to set the metadata for
	 * @param Key A key to set the data for
	 * @param Value The value to set for the key
	 */
	COREUOBJECT_API void SetValue(const UObject* Object, const TCHAR* Key, const TCHAR* Value);

	/**
	 * Set the key/value pair in the Property's metadata
	 * @param Object the object to set the metadata for
	 * @param Key A key to set the data for
	 * @param Value The value to set for the key
	 * @Values The metadata key/value pairs
	 */
	COREUOBJECT_API void SetValue(const UObject* Object, FName Key, const TCHAR* Value);

	/** 
	 *	Remove any entry with the supplied Key form the Property's metadata 
	 *	@param Object the object to clear the metadata for
	 *	@param Key A key to clear the data for
	 */
	COREUOBJECT_API void RemoveValue(const UObject* Object, const TCHAR* Key);

	/** 
	 *	Remove any entry with the supplied Key form the Property's metadata 
	 *	@param Object the object to clear the metadata for
	 *	@param Key A key to clear the data for
	 */
	COREUOBJECT_API void RemoveValue(const UObject* Object, FName Key);

	/** Find the name/value map for metadata for a specific object */
	static COREUOBJECT_API TMap<FName, FString>* GetMapForObject(const UObject* Object);

	/** Copy all metadata from the source object to the destination object. This will add to any existing metadata entries for SourceObject. */
	static COREUOBJECT_API void CopyMetadata(UObject* SourceObject, UObject* DestObject);

	/**
	 * Removes any metadata entries that are to objects not inside the same package as this UMetaData object.
	 */
	COREUOBJECT_API void RemoveMetaDataOutsidePackage(UPackage* MetaDataPackage);

	friend FArchive& operator<<(FArchive& Ar, FMetaData& MetaData)
	{
		return Ar << MetaData.ObjectMetaDataMap << MetaData.RootMetaDataMap;
	}

	// Returns the remapped key name, or NAME_None was not remapped.
	static COREUOBJECT_API FName GetRemappedKeyName(FName OldKey);

private:
	static COREUOBJECT_API void InitializeRedirectMap();

	// Redirect map from deprecated keys to current key names
	static COREUOBJECT_API TMap<FName, FName> KeyRedirectMap;
	static COREUOBJECT_API UE::FMutex KeyRedirectMapLock;
};

struct FMetaDataUtilities
{
private:
	/** Console command for dumping all metadata */
	static class FAutoConsoleCommand DumpAllConsoleCommand;

public:
	/** Find all UMetadata and print its contents to the log */
	COREUOBJECT_API static void DumpAllMetaData();

	/** Output contents of this metadata object to the log */
	COREUOBJECT_API static void DumpMetaData(UPackage* Package);

private:
	friend class UObject;

	/** Helper class to backup and move the metadata for a given UObject (and optionally its children). */
	class FMoveMetadataHelperContext
	{
	public:
		/**
		 * Backs up the metadata for the UObject (and optionally its children).
		 *
		 * @param	SourceObject		The main UObject to move metadata for.
		 * @param	bSearchChildren		When true all the metadata for classes 
		 */
		FMoveMetadataHelperContext(UObject *SourceObject, bool bSearchChildren);

		/**
		 * Patches up the new metadata on destruction.
		 */
		~FMoveMetadataHelperContext();
	private:

		/** Keep the old package around so we can pull in the metadata without actually duplicating it. */
		UPackage* OldPackage;

		/** Cache a pointer to the object so we can do the search on the old metadata. */
		UObject* OldObject;

		/** Cache the old path so we can compare to the metadata key */
		FSoftObjectPath OldObjectPath;

		/** When true, search children as well. */
		bool bShouldSearchChildren;
	};

private:
	FMetaDataUtilities() {}
};

#endif //WITH_METADATA
