// Copyright Epic Games, Inc. All Rights Reserved.

// Reflection mirrors of C++ structs defined in Core or CoreUObject, those modules are not parsed by the Unreal Header Tool.
// The documentation comments here are only for use in the editor tooltips, and is ignored for the API docs.
// More complete documentation will be found in the files that have the full class definition, listed below.

#pragma once

// Help intellisense to avoid interpreting this file's declaration of FVector etc as it assumes !CPP by default
#ifndef CPP
#define CPP 1
#endif

#if CPP

// Include the real definitions of the noexport classes below to allow the generated cpp file to compile.

#include "PixelFormat.h"

#include "Misc/FallbackStruct.h"
#include "Misc/Guid.h"
#include "Misc/DateTime.h"
#include "Misc/Timespan.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/FrameNumber.h"
#include "Misc/Timecode.h"

#include "UObject/TopLevelAssetPath.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/PropertyAccessUtil.h"
#include "Serialization/TestUndeclaredScriptStructObjectReferences.h"

#include "Math/InterpCurvePoint.h"
#include "Math/UnitConversion.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/Vector2D.h"
#include "Math/TwoVectors.h"
#include "Math/Plane.h"
#include "Math/Rotator.h"
#include "Math/Quat.h"
#include "Math/IntPoint.h"
#include "Math/IntVector.h"
#include "Math/Color.h"
#include "Math/Box.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/OrientedBox.h"
#include "Math/Matrix.h"
#include "Math/ScalarRegister.h"
#include "Math/RandomStream.h"
#include "Math/RangeBound.h"
#include "Math/Interval.h"
#include "Math/Sphere.h"

#include "Internationalization/PolyglotTextData.h"

#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetBundleData.h"
#include "AssetRegistry/AssetData.h"

#endif

#include "NoExportTypes.generated.h"

#if !CPP      //noexport class

/// @cond DOXYGEN_IGNORE

/**
 * Determines case sensitivity options for string comparisons. 
 * @note Mirrored from Engine\Source\Runtime\Core\Public\Containers\UnrealString.h
 */
UENUM()
namespace ESearchCase
{
	enum Type : int
	{
		CaseSensitive,
		IgnoreCase,
	};
}

/**
 * Determines search direction for string operations.
 * @note Mirrored from Engine\Source\Runtime\Core\Public\Containers\UnrealString.h
 */
UENUM()
namespace ESearchDir
{
	enum Type : int
	{
		FromStart,
		FromEnd,
	};
}

/**
 * Enum that defines how the log times are to be displayed.
 * @note Mirrored from Engine\Source\Runtime\Core\Public\Misc\OutputDevice.h
 */
UENUM()
namespace ELogTimes
{
	enum Type : int
	{
		/** Do not display log timestamps. */
		None UMETA(DisplayName = "None"),

		/** Display log timestamps in UTC. */
		UTC UMETA(DisplayName = "UTC"),

		/** Display log timestamps in seconds elapsed since GStartTime. */
		SinceGStartTime UMETA(DisplayName = "Time since application start"),

		/** Display log timestamps in local time. */
		Local UMETA(DisplayName = "Local time"),
	};
}

/** Generic axis enum (mirrored for native use in Engine/Source/Runtime/Core/Public/Axis.h). */
UENUM(BlueprintType, meta=(ScriptName="AxisType"))
namespace EAxis
{
	enum Type : int
	{
		None,
		X,
		Y,
		Z
	};
}

/** Generic axis list enum (mirrored for native use in Engine/Source/Runtime/Core/Public/Axis.h). */
UENUM()
namespace EAxisList
{
	enum Type : int
	{
		None = 0,
		X = 1 << 0,
		Y = 1 << 1,
		Z = 1 << 2,

		Screen = 1 << 3,
		XY = X | Y,
		XZ = X | Z,
		YZ = Y | Z,
		XYZ = X | Y | Z,
		All = XYZ | Screen,

		/** alias over Axis YZ since it isn't used when the z-rotation widget is being used */
		ZRotation = YZ,

		/** alias over Screen since it isn't used when the 2d translate rotate widget is being used */
		Rotate2D = Screen,

		Left = 1 << 4,
		Up = 1 << 5,
		Forward = 1 << 6,

		LU = Left | Up,
		LF = Left | Forward,
		UF = Up | Forward,
		LeftUpForward = Left | Up | Forward,
	};
}

/** Describes shape of an interpolation curve (mirrored from Engine/Source/Runtime/Core/Public/Math/InterpCurvePoint.h). */
UENUM()
enum EInterpCurveMode : int
{
	/** A straight line between two keypoint values. */
	CIM_Linear UMETA(DisplayName="Linear"),
	
	/** A cubic-hermite curve between two keypoints, using Arrive/Leave tangents. These tangents will be automatically
		updated when points are moved, etc.  Tangents are unclamped and will plateau at curve start and end points. */
	CIM_CurveAuto UMETA(DisplayName="Curve Auto"),
	
	/** The out value is held constant until the next key, then will jump to that value. */
	CIM_Constant UMETA(DisplayName="Constant"),
	
	/** A smooth curve just like CIM_Curve, but tangents are not automatically updated so you can have manual control over them (eg. in Curve Editor). */
	CIM_CurveUser UMETA(DisplayName="Curve User"),
	
	/** A curve like CIM_Curve, but the arrive and leave tangents are not forced to be the same, so you can create a 'corner' at this key. */
	CIM_CurveBreak UMETA(DisplayName="Curve Break"),
	
	/** A cubic-hermite curve between two keypoints, using Arrive/Leave tangents. These tangents will be automatically
	    updated when points are moved, etc.  Tangents are clamped and will plateau at curve start and end points. */
	CIM_CurveAutoClamped UMETA(DisplayName="Curve Auto Clamped"),
};

/**
 * Describes the format of a each pixel in a graphics buffer.
 * Mirrored from Engine/Source/Runtime/Core/Public/PixelFormat.h
 * @warning: When you update this, you must add an entry to GPixelFormats(see RenderUtils.cpp)
 * @warning: When you update this, you must add an entries to PixelFormat.h, usually just copy the generated section on the header into EPixelFormat
 * @warning: The *Tools DLLs will also need to be recompiled if the ordering is changed, but should not need code changes.
 */
UENUM()
enum EPixelFormat : uint8
{
	PF_Unknown,
	PF_A32B32G32R32F,
	/** UNORM (0..1), corresponds to FColor.  Unpacks as rgba in the shader. */
	PF_B8G8R8A8,
	/** UNORM red (0..1) */
	PF_G8,
	PF_G16,
	PF_DXT1,
	PF_DXT3,
	PF_DXT5,
	PF_UYVY,
	/** Same as PF_FloatR11G11B10 */
	PF_FloatRGB,
	/** RGBA 16 bit signed FP format.  Use FFloat16Color on the CPU. */
	PF_FloatRGBA,
	/** A depth+stencil format with platform-specific implementation, for use with render targets. */
	PF_DepthStencil,
	/** A depth format with platform-specific implementation, for use with render targets. */
	PF_ShadowDepth,
	PF_R32_FLOAT,
	PF_G16R16,
	PF_G16R16F,
	PF_G16R16F_FILTER,
	PF_G32R32F,
	PF_A2B10G10R10,
	PF_A16B16G16R16,
	PF_D24,
	PF_R16F,
	PF_R16F_FILTER,
	PF_BC5,
	/** SNORM red, green (-1..1). Not supported on all RHI e.g. Metal */
	PF_V8U8,
	PF_A1,
	/** A low precision floating point format, unsigned.  Use FFloat3Packed on the CPU. */
	PF_FloatR11G11B10,
	PF_A8,
	PF_R32_UINT,
	PF_R32_SINT,
	PF_PVRTC2,
	PF_PVRTC4,
	PF_R16_UINT,
	PF_R16_SINT,
	PF_R16G16B16A16_UINT,
	PF_R16G16B16A16_SINT,
	PF_R5G6B5_UNORM,
	PF_R8G8B8A8,
	/** Only used for legacy loading; do NOT use! */
	PF_A8R8G8B8,
	/** High precision single channel block compressed, equivalent to a single channel BC5, 8 bytes per 4x4 block. */
	PF_BC4,
	/** UNORM red, green (0..1). */
	PF_R8G8,
	/** ATITC format. */
	PF_ATC_RGB,
	/** ATITC format. */
	PF_ATC_RGBA_E,
	/** ATITC format. */
	PF_ATC_RGBA_I,
	/** Used for creating SRVs to alias a DepthStencil buffer to read Stencil.  Don't use for creating textures. */
	PF_X24_G8,
	PF_ETC1,
	PF_ETC2_RGB,
	PF_ETC2_RGBA,
	PF_R32G32B32A32_UINT,
	PF_R16G16_UINT,
	/** 8.00 bpp */
	PF_ASTC_4x4,
	/** 3.56 bpp */
	PF_ASTC_6x6,
	/** 2.00 bpp */
	PF_ASTC_8x8,
	/** 1.28 bpp */
	PF_ASTC_10x10,
	/** 0.89 bpp */
	PF_ASTC_12x12,
	PF_BC6H,
	PF_BC7,
	PF_R8_UINT,
	PF_L8,
	PF_XGXR8,
	PF_R8G8B8A8_UINT,
	/** SNORM (-1..1), corresponds to FFixedRGBASigned8. */
	PF_R8G8B8A8_SNORM,
	PF_R16G16B16A16_UNORM,
	PF_R16G16B16A16_SNORM,
	PF_PLATFORM_HDR_0,
	PF_PLATFORM_HDR_1,
	PF_PLATFORM_HDR_2,
	PF_NV12,
	PF_R32G32_UINT,
	PF_ETC2_R11_EAC,
	PF_ETC2_RG11_EAC,
	PF_R8,
	PF_B5G5R5A1_UNORM,
	PF_ASTC_4x4_HDR,	
	PF_ASTC_6x6_HDR,	
	PF_ASTC_8x8_HDR,	
	PF_ASTC_10x10_HDR,	
	PF_ASTC_12x12_HDR,
	PF_G16R16_SNORM,
	PF_R8G8_UINT,
	PF_R32G32B32_UINT,
	PF_R32G32B32_SINT,
	PF_R32G32B32F,
	PF_R8_SINT,
	PF_R64_UINT,
	PF_R9G9B9EXP5,
	PF_P010,
	PF_ASTC_4x4_NORM_RG,
	PF_ASTC_6x6_NORM_RG,
	PF_ASTC_8x8_NORM_RG,
	PF_ASTC_10x10_NORM_RG,
	PF_ASTC_12x12_NORM_RG,
	PF_R16G16_SINT,
	PF_R8G8B8,
	PF_MAX,
};

/** Mouse cursor types (mirrored from Engine/Source/Runtime/ApplicationCore/Public/GenericPlatform/ICursor.h) */
UENUM()
namespace EMouseCursor
{
	enum Type : int
	{
		/** Causes no mouse cursor to be visible. */
		None,

		/** Default cursor (arrow). */
		Default,

		/** Text edit beam. */
		TextEditBeam,

		/** Resize horizontal. */
		ResizeLeftRight,

		/** Resize vertical. */
		ResizeUpDown,

		/** Resize diagonal. */
		ResizeSouthEast,

		/** Resize other diagonal. */
		ResizeSouthWest,

		/** MoveItem. */
		CardinalCross,

		/** Target Cross. */
		Crosshairs,

		/** Hand cursor. */
		Hand,

		/** Grab Hand cursor. */
		GrabHand,

		/** Grab Hand cursor closed. */
		GrabHandClosed,

		/** a circle with a diagonal line through it. */
		SlashedCircle,

		/** Eye-dropper cursor for picking colors. */
		EyeDropper,

		/** Custom cursor shape for platforms that support setting a native cursor shape. Same as specifying None if not set. */
		Custom,
	};
}

/** A set of numerical unit types supported by the engine. Mirrored from Engine/Source/Runtime/Core/Public/Math/UnitConversion.h */
UENUM(BlueprintType)
enum class EUnit : uint8
{
	/** Scalar distance/length unit. */
	Micrometers, Millimeters, Centimeters, Meters, Kilometers,
	Inches, Feet, Yards, Miles,
	Lightyears,

	/** Angular unit. */
	Degrees, Radians,

	/** Speed unit. */
	CentimetersPerSecond, MetersPerSecond, KilometersPerHour, MilesPerHour,

	/** Angular speed unit. */
	DegreesPerSecond, RadiansPerSecond,

	/** Acceleration unit. */
	CentimetersPerSecondSquared, MetersPerSecondSquared,

	/** Temperature unit. */
	Celsius, Farenheit, Kelvin,

	/** Mass unit. */
	Micrograms, Milligrams, Grams, Kilograms, MetricTons,
	Ounces, Pounds, Stones,

	/** Density unit. */
	GramsPerCubicCentimeter, GramsPerCubicMeter, KilogramsPerCubicCentimeter, KilogramsPerCubicMeter,

	/** Force unit. */
	Newtons, PoundsForce, KilogramsForce, KilogramCentimetersPerSecondSquared,

	/** Torque unit. */
	NewtonMeters, KilogramCentimetersSquaredPerSecondSquared,

	/** Impulse unit. */
	NewtonSeconds, KilogramCentimeters, KilogramMeters,

	/** Frequency unit. */
	Hertz, Kilohertz, Megahertz, Gigahertz, RevolutionsPerMinute,

	/** Data Size unit. */
	Bytes, Kilobytes, Megabytes, Gigabytes, Terabytes,

	/** Luminous flux unit. */
	Lumens,
	
	/** Luminous intensity unit. */
	Candela,
	
	/** Illuminance unit. */
	Lux,
	
	/** Luminance unit. */
	CandelaPerMeter2,
	
	/** Exposure value unit. */
	ExposureValue,

	/** Time unit. */
	Nanoseconds, Microseconds, Milliseconds, Seconds, Minutes, Hours, Days, Months, Years,

	/** Pixel density unit. */
	PixelsPerInch,

	/** Percentage. */
	Percentage,

	/** Arbitrary multiplier. */
	Multiplier,

	/** Stress unit. */
	Pascals, KiloPascals, MegaPascals, GigaPascals,

	/** Symbolic entry, not specifiable on meta data. */
	Unspecified
};

/**
 * Enumerates supported message dialog category types.
 * @note Mirrored from Engine/Source/Runtime/Core/Public/GenericPlatform/GenericPlatformMisc.h
 */
UENUM(BlueprintType)
enum class EAppMsgCategory : uint8
{
	Warning,
	Error,
	Success,
	Info,
};

/**
* Enum denoting message dialog return types.
* @note Mirrored from Engine/Source/Runtime/Core/Public/GenericPlatform/GenericPlatformMisc.h
*/
UENUM(BlueprintType)
namespace EAppReturnType
{
	enum Type : int
	{
		No,
		Yes,
		YesAll,
		NoAll,
		Cancel,
		Ok,
		Retry,
		Continue,
	};
}

/**
* Enum denoting message dialog button choices. Used in combination with EAppReturnType.
* @note Mirrored from Engine/Source/Runtime/Core/Public/GenericPlatform/GenericPlatformMisc.h
*/
UENUM(BlueprintType)
namespace EAppMsgType
{
	/**
	 * Enumerates supported message dialog button types.
	 */
	enum Type : int
	{
		Ok,
		YesNo,
		OkCancel,
		YesNoCancel,
		CancelRetryContinue,
		YesNoYesAllNoAll,
		YesNoYesAllNoAllCancel,
		YesNoYesAll,
	};
}

/**
 * A struct used as stub for deleted ones. 
 * Mirrored from Engine/Source/Runtime/Core/Public/Misc/FallbackStruct.h
 */
USTRUCT(noexport, IsAlwaysAccessible, HasDefaults)
struct FFallbackStruct
{
};

/** A globally unique identifier (mirrored from Engine/Source/Runtime/Core/Public/Misc/Guid.h) */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults)
struct FGuid
{
	UPROPERTY(EditAnywhere, SaveGame, Category=Guid)
	int32 A;

	UPROPERTY(EditAnywhere, SaveGame, Category=Guid)
	int32 B;

	UPROPERTY(EditAnywhere, SaveGame, Category=Guid)
	int32 C;

	UPROPERTY(EditAnywhere, SaveGame, Category=Guid)
	int32 D;
};

/**
 * A point or direction FVector in 3d space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector.h
 */
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta = (ScriptDefaultMake, ScriptDefaultBreak, HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector"))
struct FVector3f
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vector, SaveGame)
	float X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vector, SaveGame)
	float Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vector, SaveGame)
	float Z;
};

/**
 * A point or direction FVector in 3d space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FVector3d
{
	UPROPERTY(EditAnywhere, Category = Vector, SaveGame)
	double X;

	UPROPERTY(EditAnywhere, Category = Vector, SaveGame)
	double Y;

	UPROPERTY(EditAnywhere, Category = Vector, SaveGame)
	double Z;
};

/**
 * A point or direction FVector in 3d space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta = (HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector"))
struct FVector
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vector, SaveGame)
	FLargeWorldCoordinatesReal X;		//~ Alias for float/double depending on LWC status. Note: Will be refactored to double before UE5 ships.

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vector, SaveGame)
	FLargeWorldCoordinatesReal Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Vector, SaveGame)
	FLargeWorldCoordinatesReal Z;
};


/**
* A 4-D homogeneous vector.
* @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector4.h
*/
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta = (ScriptDefaultMake, ScriptDefaultBreak, HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector4", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector4"))
struct FVector4f
{
	UPROPERTY(EditAnywhere, Category = Vector4, SaveGame)
	float X;

	UPROPERTY(EditAnywhere, Category = Vector4, SaveGame)
	float Y;

	UPROPERTY(EditAnywhere, Category = Vector4, SaveGame)
	float Z;

	UPROPERTY(EditAnywhere, Category = Vector4, SaveGame)
	float W;
};

/**
* A 4-D homogeneous vector.
* @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector4.h
*/
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FVector4d
{
	UPROPERTY(EditAnywhere, Category = Vector4, SaveGame)
	double X;

	UPROPERTY(EditAnywhere, Category = Vector4, SaveGame)
	double Y;

	UPROPERTY(EditAnywhere, Category = Vector4, SaveGame)
	double Z;

	UPROPERTY(EditAnywhere, Category = Vector4, SaveGame)
	double W;
};


/**
* A 4-D homogeneous vector.
* @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector4.h
*/
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta = (ScriptDefaultMake, ScriptDefaultBreak, HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector4", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector4"))
struct FVector4
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Vector4, SaveGame)
	FLargeWorldCoordinatesReal X;		//~ Alias for float/double depending on LWC status. Note: Will be refactored to double before UE5 ships.

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Vector4, SaveGame)
	FLargeWorldCoordinatesReal Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Vector4, SaveGame)
	FLargeWorldCoordinatesReal Z;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Vector4, SaveGame)
	FLargeWorldCoordinatesReal W;
};


/**
* A vector in 2-D space composed of components (X, Y) with floating point precision.
* @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector2D.h
*/
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta = (ScriptDefaultMake, ScriptDefaultBreak, HasNativeMake="/Script/Engine.KismetMathLibrary.MakeVector2D", HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeVector2D", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakVector2D"))
struct FVector2f
{
	UPROPERTY(EditAnywhere, Category=Vector2D, SaveGame)
	float X;

	UPROPERTY(EditAnywhere, Category=Vector2D, SaveGame)
	float Y;
};

/**
* A vector in 2-D space composed of components (X, Y) with floating point precision.
* @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector2D.h
*/
// LWC_TODO: CRITICAL! Name collision in UHT with FVector2D due to case insensitive FNames!
// USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
// struct FVector2d
// {
// 	UPROPERTY(EditAnywhere, Category=Vector2D, SaveGame)
// 	double X;
//
// 	UPROPERTY(EditAnywhere, Category=Vector2D, SaveGame)
// 	double Y;
// };

/**
 * A vector in 2-D space composed of components (X, Y) with floating point precision.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Vector2D.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta=(HasNativeMake="/Script/Engine.KismetMathLibrary.MakeVector2D", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakVector2D"))
struct FVector2D
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Vector2D, SaveGame)
	FLargeWorldCoordinatesReal X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Vector2D, SaveGame)
	FLargeWorldCoordinatesReal Y;
};

/** A pair of 3D vectors (mirrored from Engine/Source/Runtime/Core/Public/Math/TwoVectors.h). */
USTRUCT(immutable, BlueprintType, noexport, IsAlwaysAccessible, HasDefaults)
struct FTwoVectors
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TwoVectors, SaveGame)
	FVector v1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=TwoVectors, SaveGame)
	FVector v2;
};

/**
 * A plane definition in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Plane.h
 */
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FPlane4f : public FVector3f
{
	UPROPERTY(EditAnywhere, Category=Plane, SaveGame)
	float W;
};

/**
 * A plane definition in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Plane.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FPlane4d : public FVector3d
{
	UPROPERTY(EditAnywhere, Category = Plane, SaveGame)
	double W;
};

/**
 * A plane definition in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Plane.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FPlane : public FVector
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Plane, SaveGame)
	FLargeWorldCoordinatesReal W;
};



/**
 * 3D Ray represented by Origin and (normalized) Direction.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Ray.h
 * @note FRay3f is not currently exposed as a Blueprint type
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FRay3f
{
	UPROPERTY(EditAnywhere, Category = Ray, SaveGame)
	FVector3f Origin;

	UPROPERTY(EditAnywhere, Category = Ray, SaveGame)
	FVector3f Direction;
};

/**
 * 3D Ray represented by Origin and (normalized) Direction.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Ray.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FRay3d
{
	UPROPERTY(EditAnywhere, Category = Ray, SaveGame)
	FVector3d Origin;

	UPROPERTY(EditAnywhere, Category = Ray, SaveGame)
	FVector3d Direction;
};

/**
 * 3D Ray represented by Origin and (normalized) Direction.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Ray.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FRay
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Ray, SaveGame)
	FVector Origin;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Ray, SaveGame)
	FVector Direction;
};



/**
 * An orthogonal rotation in 3d space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Rotator.h
 */
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta = (ScriptDefaultMake, ScriptDefaultBreak, HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeRotator", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakRotator"))
struct FRotator3f
{
	/** Pitch (degrees) around Y axis */
	UPROPERTY(EditAnywhere, Category=Rotator, SaveGame, meta=(DisplayName="Y"))
	float Pitch;

	/** Yaw (degrees) around Z axis */
	UPROPERTY(EditAnywhere, Category=Rotator, SaveGame, meta=(DisplayName="Z"))
	float Yaw;

	/** Roll (degrees) around X axis */
	UPROPERTY(EditAnywhere, Category=Rotator, SaveGame, meta=(DisplayName="X"))
	float Roll;
};

/**
 * An orthogonal rotation in 3d space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Rotator.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FRotator3d
{
	/** Pitch (degrees) around Y axis */
	UPROPERTY(EditAnywhere, Category=Rotator, SaveGame, meta=(DisplayName="Y"))
	double Pitch;

	/** Yaw (degrees) around Z axis */
	UPROPERTY(EditAnywhere, Category=Rotator, SaveGame, meta=(DisplayName="Z"))
	double Yaw;

	/** Roll (degrees) around X axis */
	UPROPERTY(EditAnywhere, Category=Rotator, SaveGame, meta=(DisplayName="X"))
	double Roll;
};

/**
 * An orthogonal rotation in 3d space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Rotator.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta=(HasNativeMake="/Script/Engine.KismetMathLibrary.MakeRotator", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakRotator"))
struct FRotator
{
	/** Pitch (degrees) around Y axis */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotator, SaveGame, meta=(DisplayName="Y"))
	FLargeWorldCoordinatesReal Pitch;

	/** Yaw (degrees) around Z axis */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotator, SaveGame, meta=(DisplayName="Z"))
	FLargeWorldCoordinatesReal Yaw;

	/** Roll (degrees) around X axis */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Rotator, SaveGame, meta=(DisplayName="X"))
	FLargeWorldCoordinatesReal Roll;
};



/**
 * 3D Sphere represented by Center and Radius.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Sphere.h
 * @note FSphere3f is not currently exposed as a Blueprint type
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FSphere3f
{
	UPROPERTY(EditAnywhere, Category = Sphere, SaveGame)
	FVector3f Center;

	UPROPERTY(EditAnywhere, Category = Sphere, SaveGame, meta = (DisplayName = "Radius"))
	float W;
};
/**
 * 3D Sphere represented by Center and Radius.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Sphere.h
 * @note FSphere3d is not currently exposed as a Blueprint type
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FSphere3d
{
	UPROPERTY(EditAnywhere, Category = Sphere, SaveGame)
	FVector3d Center;

	UPROPERTY(EditAnywhere, Category = Sphere, SaveGame, meta = (DisplayName = "Radius"))
	double W;
};
/**
 * 3D Sphere represented by Center and Radius.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Sphere.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FSphere
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sphere, SaveGame)
	FVector Center;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Sphere, SaveGame, meta = (DisplayName = "Radius"))
	FLargeWorldCoordinatesReal W;
};



/**
 * Quaternion.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Quat.h
 */
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta = (ScriptDefaultMake, ScriptDefaultBreak, HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeQuat", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakQuat"))
struct FQuat4f
{
	UPROPERTY(EditAnywhere, Category=Quat, SaveGame)
	float X;

	UPROPERTY(EditAnywhere, Category=Quat, SaveGame)
	float Y;

	UPROPERTY(EditAnywhere, Category=Quat, SaveGame)
	float Z;

	UPROPERTY(EditAnywhere, Category=Quat, SaveGame)
	float W;

};


/**
 * Quaternion.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Quat.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FQuat4d
{
	UPROPERTY(EditAnywhere, Category = Quat, SaveGame)
	double X;

	UPROPERTY(EditAnywhere, Category = Quat, SaveGame)
	double Y;

	UPROPERTY(EditAnywhere, Category = Quat, SaveGame)
	double Z;

	UPROPERTY(EditAnywhere, Category = Quat, SaveGame)
	double W;

};


/**
 * Quaternion.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Quat.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta=(HasNativeMake ="/Script/Engine.KismetMathLibrary.MakeQuat", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakQuat"))
struct FQuat
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Quat, SaveGame)
	FLargeWorldCoordinatesReal X;		//~ Alias for float/double depending on LWC status. Note: Will be refactored to double before UE5 ships.

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Quat, SaveGame)
	FLargeWorldCoordinatesReal Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Quat, SaveGame)
	FLargeWorldCoordinatesReal Z;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Quat, SaveGame)
	FLargeWorldCoordinatesReal W;
};


/**
 * A packed normal.
 * @note The full C++ class is located here: Engine\Source\Runtime\RenderCore\Public\PackedNormal.h
 */
USTRUCT(immutable, noexport)
struct FPackedNormal
{
	UPROPERTY(EditAnywhere, Category=PackedNormal, SaveGame)
	uint8 X;

	UPROPERTY(EditAnywhere, Category=PackedNormal, SaveGame)
	uint8 Y;

	UPROPERTY(EditAnywhere, Category=PackedNormal, SaveGame)
	uint8 Z;

	UPROPERTY(EditAnywhere, Category=PackedNormal, SaveGame)
	uint8 W;

};

/**
 * A packed basis vector.
 * @note The full C++ class is located here: Engine\Source\Runtime\RenderCore\Public\PackedNormal.h
 */
USTRUCT(immutable, noexport)
struct FPackedRGB10A2N
{
	UPROPERTY(EditAnywhere, Category = PackedBasis, SaveGame)
	int32 Packed;
};

/**
 * A packed vector.
 * @note The full C++ class is located here: Engine\Source\Runtime\RenderCore\Public\PackedNormal.h
 */
USTRUCT(immutable, noexport)
struct FPackedRGBA16N
{
	UPROPERTY(EditAnywhere, Category = PackedNormal, SaveGame)
	int32 XY;

	UPROPERTY(EditAnywhere, Category = PackedNormal, SaveGame)
	int32 ZW;
};

/**
 * Screen coordinates.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntPoint.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FIntPoint
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=IntPoint, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=IntPoint, SaveGame)
	int32 Y;
};

/**
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntPoint.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt32Point
{
	UPROPERTY(EditAnywhere, Category = IntPoint, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, Category = IntPoint, SaveGame)
	int32 Y;
};

/**
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntPoint.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt64Point
{
	UPROPERTY(EditAnywhere, Category=IntPoint, SaveGame)
	int64 X;

	UPROPERTY(EditAnywhere, Category=IntPoint, SaveGame)
	int64 Y;
};

/**
 * Screen coordinates.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntPoint.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUintPoint
{
	UPROPERTY(EditAnywhere, Category = IntPoint, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, Category = IntPoint, SaveGame)
	int32 Y;
};

/**
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntPoint.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint32Point
{
	UPROPERTY(EditAnywhere, Category = IntPoint, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, Category = IntPoint, SaveGame)
	int32 Y;
};

/**
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntPoint.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint64Point
{
	UPROPERTY(EditAnywhere, Category = IntPoint, SaveGame)
	int64 X;

	UPROPERTY(EditAnywhere, Category = IntPoint, SaveGame)
	int64 Y;
};

/**
 * An integer rectangle in 2D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntRect.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FIntRect
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=IntRect, SaveGame)
	FIntPoint Min;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=IntRect, SaveGame)
	FIntPoint Max;
};

/**
 * An integer rectangle in 2D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntRect.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt32Rect
{
	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FInt32Point Min;

	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FInt32Point Max;	
};

/**
 * An integer rectangle in 2D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntRect.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt64Rect
{
	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FInt64Point Min;

	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FInt64Point Max;
};

/**
 * An integer rectangle in 2D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntRect.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUintRect
{
	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FUintPoint Min;

	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FUintPoint Max;
};

/**
 * An integer rectangle in 2D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntRect.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint32Rect
{
	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FUint32Point Min;

	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FUint32Point Max;
};

/**
 * An integer rectangle in 2D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntRect.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint64Rect
{
	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FUint64Point Min;

	UPROPERTY(EditAnywhere, Category=IntRect, SaveGame)
	FUint64Point Max;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt32Vector2
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 Y;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt64Vector2
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 Y;
};

/**
 * An integer vector in 4D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FIntVector2
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = IntVector, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = IntVector, SaveGame)
	int32 Y;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint32Vector2
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 Y;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint64Vector2
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 Y;
};

/**
 * An integer vector in 4D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUintVector2
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 Y;
};


/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt32Vector
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 Y;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 Z;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt64Vector
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 Y;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 Z;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FIntVector
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=IntVector, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=IntVector, SaveGame)
	int32 Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=IntVector, SaveGame)
	int32 Z;
};


/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint32Vector
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 Y;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 Z;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint64Vector
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 Y;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 Z;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUintVector
{
	UPROPERTY(EditAnywhere, Category=IntVector, SaveGame)
	uint32 X;

	UPROPERTY(EditAnywhere, Category=IntVector, SaveGame)
	uint32 Y;

	UPROPERTY(EditAnywhere, Category=IntVector, SaveGame)
	uint32 Z;
};


/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt32Vector4
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 Y;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 Z;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int32 W;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FInt64Vector4
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 Y;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 Z;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	int64 W;
};

/**
 * An integer vector in 4D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FIntVector4
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = IntVector4, SaveGame)
	int32 X;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = IntVector4, SaveGame)
	int32 Y;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = IntVector4, SaveGame)
	int32 Z;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = IntVector4, SaveGame)
	int32 W;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint32Vector4
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 Y;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 Z;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint32 W;
};

/**
 * An integer vector in 3D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUint64Vector4
{
	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 X;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 Y;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 Z;

	UPROPERTY(EditAnywhere, Category = IntVector, SaveGame)
	uint64 W;
};

/**
 * An integer vector in 4D space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\IntVector.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FUintVector4
{
	UPROPERTY(EditAnywhere, Category = IntVector4, SaveGame)
	uint32 X;

	UPROPERTY(EditAnywhere, Category = IntVector4, SaveGame)
	uint32 Y;

	UPROPERTY(EditAnywhere, Category = IntVector4, SaveGame)
	uint32 Z;

	UPROPERTY(EditAnywhere, Category = IntVector4, SaveGame)
	uint32 W;
};


/**
 * Stores a color with 8 bits of precision per channel. (BGRA).
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Color.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor)
struct FColor
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Color, SaveGame, meta=(ClampMin="0", ClampMax="255"))
	uint8 B;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Color, SaveGame, meta=(ClampMin="0", ClampMax="255"))
	uint8 G;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Color, SaveGame, meta=(ClampMin="0", ClampMax="255"))
	uint8 R;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Color, SaveGame, meta=(ClampMin="0", ClampMax="255"))
	uint8 A;

};

/**
 * A linear, 32-bit/component floating point RGBA color.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Color.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor)
struct FLinearColor
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LinearColor, SaveGame)
	float R;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LinearColor, SaveGame)
	float G;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LinearColor, SaveGame)
	float B;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=LinearColor, SaveGame)
	float A;

};

/**
 * A point or direction FVector in 3d space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Box.h
 */
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FBox3f
{
	UPROPERTY(EditAnywhere, Category = Box, SaveGame, meta=(EditCondition="IsValid"))
	FVector3f Min;

	UPROPERTY(EditAnywhere, Category = Box, SaveGame, meta=(EditCondition="IsValid"))
	FVector3f Max;

	UPROPERTY(EditAnywhere, Category = Box, SaveGame, meta=(ScriptName="IsValid"))
	bool IsValid;
};


/**
 * A point or direction FVector in 3d space.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Box.h
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FBox3d
{
	UPROPERTY(EditAnywhere, Category = Box, SaveGame, meta=(EditCondition="IsValid"))
	FVector3d Min;

	UPROPERTY(EditAnywhere, Category = Box, SaveGame, meta=(EditCondition="IsValid"))
	FVector3d Max;

	UPROPERTY(EditAnywhere, Category = Box, SaveGame, meta=(ScriptName="IsValid"))
	bool IsValid;
};

/**
 * A bounding box.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Box.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta=(HasNativeMake="/Script/Engine.KismetMathLibrary.MakeBox"))
struct FBox
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Box, SaveGame, meta=(EditCondition="IsValid"))
	FVector Min;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Box, SaveGame, meta=(EditCondition="IsValid"))
	FVector Max;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Box, SaveGame, meta=(ScriptName="IsValid"))
	bool IsValid;
};

/**
 * A rectangular 2D Box.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Box2D.h
 */
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FBox2f
{
	UPROPERTY(EditAnywhere, Category=Box2D, SaveGame, meta=(EditCondition="bIsValid"))
	FVector2f Min;

	UPROPERTY(EditAnywhere, Category=Box2D, SaveGame, meta=(EditCondition="bIsValid"))
	FVector2f Max;

	UPROPERTY(EditAnywhere, Category=Box2D, SaveGame, meta=(ScriptName="bIsValid"))
	bool bIsValid;
};

/**
* A rectangular 2D Box.
* @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Box2D.h
*/
// LWC_TODO: CRITICAL! Name collision in UHT with FBox2D due to case insensitive FNames!
// USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
// struct FBox2d
// {
// 	UPROPERTY(EditAnywhere, Category=Box2D, SaveGame, meta=(EditCondition="bIsValid"))
// 	FVector2d Min;
//
// 	UPROPERTY(EditAnywhere, Category=Box2D, SaveGame, meta=(EditCondition="bIsValid"))
// 	FVector2d Max;
//
// 	UPROPERTY(EditAnywhere, Category=Box2D, SaveGame, meta=(ScriptName="bIsValid"))
// 	bool bIsValid;
// };

/**
* A rectangular 2D Box.
* @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Box2D.h
*/
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta=(HasNativeMake="/Script/Engine.KismetMathLibrary.MakeBox2D"))
struct FBox2D
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Box2D, SaveGame, meta=(EditCondition="bIsValid"))
	FVector2D Min;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Box2D, SaveGame, meta=(EditCondition="bIsValid"))
	FVector2D Max;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Box2D, SaveGame, meta=(ScriptName="bIsValid"))
	bool bIsValid;
};

/**
 * A bounding box and bounding sphere with the same origin.
 * @note The full C++ class is located here : Engine\Source\Runtime\Core\Public\Math\BoxSphereBounds.h
 */
USTRUCT(noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FBoxSphereBounds3f
{
	/** Holds the origin of the bounding box and sphere. */
	UPROPERTY(EditAnywhere, Category = BoxSphereBounds, SaveGame)
	FVector3f Origin;

	/** Holds the extent of the bounding box, which is half the size of the box in 3D space */
	UPROPERTY(EditAnywhere, Category = BoxSphereBounds, SaveGame)
	FVector3f BoxExtent;

	/** Holds the radius of the bounding sphere. */
	UPROPERTY(EditAnywhere, Category = BoxSphereBounds, SaveGame)
	float SphereRadius;
};

/**
 * A bounding box and bounding sphere with the same origin.
 * @note The full C++ class is located here : Engine\Source\Runtime\Core\Public\Math\BoxSphereBounds.h
 */
USTRUCT(noexport, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FBoxSphereBounds3d
{
	/** Holds the origin of the bounding box and sphere. */
	UPROPERTY(EditAnywhere, Category = BoxSphereBounds, SaveGame)
	FVector3d Origin;

	/** Holds the extent of the bounding box, which is half the size of the box in 3D space */
	UPROPERTY(EditAnywhere, Category = BoxSphereBounds, SaveGame)
	FVector3d BoxExtent;

	/** Holds the radius of the bounding sphere. */
	UPROPERTY(EditAnywhere, Category = BoxSphereBounds, SaveGame)
	double SphereRadius;
};

/**
 * A bounding box and bounding sphere with the same origin.
 * @note The full C++ class is located here : Engine\Source\Runtime\Core\Public\Math\BoxSphereBounds.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, IsCoreType, meta = (HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeBoxSphereBounds", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakBoxSphereBounds"))
struct FBoxSphereBounds
{
	/** Holds the origin of the bounding box and sphere. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BoxSphereBounds, SaveGame)
	FVector Origin;

	/** Holds the extent of the bounding box, which is half the size of the box in 3D space */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BoxSphereBounds, SaveGame)
	FVector BoxExtent;

	/** Holds the radius of the bounding sphere. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=BoxSphereBounds, SaveGame)
	FLargeWorldCoordinatesReal SphereRadius;
};

/**
 * Structure for arbitrarily oriented boxes (i.e. not necessarily axis-aligned).
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\OrientedBox.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, meta = (HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeOrientedBox"))
struct FOrientedBox
{
	/** Holds the center of the box. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=OrientedBox, SaveGame)
	FVector Center;

	/** Holds the x-axis vector of the box. Must be a unit vector. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=OrientedBox, SaveGame)
	FVector AxisX;
	
	/** Holds the y-axis vector of the box. Must be a unit vector. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=OrientedBox, SaveGame)
	FVector AxisY;
	
	/** Holds the z-axis vector of the box. Must be a unit vector. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=OrientedBox, SaveGame)
	FVector AxisZ;

	/** Holds the extent of the box along its x-axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=OrientedBox, SaveGame)
	FLargeWorldCoordinatesReal ExtentX;
	
	/** Holds the extent of the box along its y-axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=OrientedBox, SaveGame)
	FLargeWorldCoordinatesReal ExtentY;

	/** Holds the extent of the box along its z-axis. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=OrientedBox, SaveGame)
	FLargeWorldCoordinatesReal ExtentZ;
};

/**
 * A 4x4 matrix.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Matrix.h
 */
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FMatrix44f
{
	UPROPERTY(EditAnywhere, Category=Matrix, SaveGame)
	FPlane4f XPlane;

	UPROPERTY(EditAnywhere, Category=Matrix, SaveGame)
	FPlane4f YPlane;

	UPROPERTY(EditAnywhere, Category=Matrix, SaveGame)
	FPlane4f ZPlane;

	UPROPERTY(EditAnywhere, Category=Matrix, SaveGame)
	FPlane4f WPlane;

};

/**
 * A 4x4 matrix.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Matrix.h
 */
USTRUCT(immutable, noexport, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FMatrix44d
{
	UPROPERTY(EditAnywhere, Category = Matrix, SaveGame)
	FPlane4d XPlane;

	UPROPERTY(EditAnywhere, Category = Matrix, SaveGame)
	FPlane4d YPlane;

	UPROPERTY(EditAnywhere, Category = Matrix, SaveGame)
	FPlane4d ZPlane;

	UPROPERTY(EditAnywhere, Category = Matrix, SaveGame)
	FPlane4d WPlane;

};

/**
 * A 4x4 matrix.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\Matrix.h
 */
USTRUCT(immutable, noexport, BlueprintType, HasDefaults, HasNoOpConstructor, IsCoreType)
struct FMatrix
{
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Matrix, SaveGame)
	FPlane XPlane;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Matrix, SaveGame)
	FPlane YPlane;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Matrix, SaveGame)
	FPlane ZPlane;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Matrix, SaveGame)
	FPlane WPlane;

};

/** 
 * Describes one specific point on an interpolation curve.
 * @note This is a mirror of TInterpCurvePoint<float>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurvePoint.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor)
struct FInterpCurvePointFloat
{
	/** Float input value that corresponds to this key (eg. time). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointFloat)
	float InVal;

	/** Float output value type when input is equal to InVal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointFloat)
	float OutVal;

	/** Tangent of curve arriving at this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointFloat)
	float ArriveTangent;

	/** Tangent of curve leaving this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointFloat)
	float LeaveTangent;

	/** Interpolation mode between this point and the next one. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointFloat)
	TEnumAsByte<enum EInterpCurveMode> InterpMode;

};

/** 
 * Describes an entire curve that is used to compute a float output value from a float input.
 * @note This is a mirror of TInterpCurve<float>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurve.h
 */
USTRUCT(noexport, BlueprintType)
struct FInterpCurveFloat
{
	/** Holds the collection of interpolation points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveFloat)
	TArray<FInterpCurvePointFloat> Points;

	/** Specify whether the curve is looped or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveFloat)
	bool bIsLooped;

	/** Specify the offset from the last point's input key corresponding to the loop point */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveFloat)
	float LoopKeyOffset;
};

/** 
 * Describes one specific point on an interpolation curve.
 * @note This is a mirror of TInterpCurvePoint<FVector2D>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurvePoint.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor)
struct FInterpCurvePointVector2D
{
	/** Float input value that corresponds to this key (eg. time). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector2D)
	float InVal;

	/** 2D vector output value of when input is equal to InVal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector2D)
	FVector2D OutVal;

	/** Tangent of curve arriving at this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector2D)
	FVector2D ArriveTangent;

	/** Tangent of curve leaving this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector2D)
	FVector2D LeaveTangent;

	/** Interpolation mode between this point and the next one. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector2D)
	TEnumAsByte<enum EInterpCurveMode> InterpMode;
};

/** 
 * Describes an entire curve that is used to compute a 2D vector output value from a float input.
 * @note This is a mirror of TInterpCurve<FVector2D>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurve.h
 */
USTRUCT(noexport, BlueprintType)
struct FInterpCurveVector2D
{
	/** Holds the collection of interpolation points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveVector2D)
	TArray<FInterpCurvePointVector2D> Points;

	/** Specify whether the curve is looped or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveVector2D)
	bool bIsLooped;

	/** Specify the offset from the last point's input key corresponding to the loop point */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveVector2D)
	float LoopKeyOffset;
};

/** 
 * Describes one specific point on an interpolation curve.
 * @note This is a mirror of TInterpCurvePoint<FVector>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurvePoint.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor)
struct FInterpCurvePointVector
{
	/** Float input value that corresponds to this key (eg. time). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector)
	float InVal;

	/** 3D vector output value of when input is equal to InVal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector)
	FVector OutVal;

	/** Tangent of curve arriving at this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector)
	FVector ArriveTangent;

	/** Tangent of curve leaving this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector)
	FVector LeaveTangent;

	/** Interpolation mode between this point and the next one. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointVector)
	TEnumAsByte<enum EInterpCurveMode> InterpMode;
};

/** 
 * Describes an entire curve that is used to compute a 3D vector output value from a float input.
 * @note This is a mirror of TInterpCurve<FVector>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurve.h
 */
USTRUCT(noexport, BlueprintType)
struct FInterpCurveVector
{
	/** Holds the collection of interpolation points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveVector)
	TArray<FInterpCurvePointVector> Points;

	/** Specify whether the curve is looped or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveVector)
	bool bIsLooped;

	/** Specify the offset from the last point's input key corresponding to the loop point */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveVector)
	float LoopKeyOffset;
};

/**
 * Describes one specific point on an interpolation curve.
 * @note This is a mirror of TInterpCurvePoint<FQuat>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurvePoint.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor)
struct FInterpCurvePointQuat
{
	/** Float input value that corresponds to this key (eg. time). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointQuat)
	float InVal;

	/** Quaternion output value of when input is equal to InVal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointQuat)
	FQuat OutVal;

	/** Tangent of curve arriving at this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointQuat)
	FQuat ArriveTangent;

	/** Tangent of curve leaving this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointQuat)
	FQuat LeaveTangent;

	/** Interpolation mode between this point and the next one. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointQuat)
	TEnumAsByte<enum EInterpCurveMode> InterpMode;
};

/**
 * Describes an entire curve that is used to compute a quaternion output value from a float input.
 * @note This is a mirror of TInterpCurve<FQuat>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurve.h
 */
USTRUCT(noexport, BlueprintType)
struct FInterpCurveQuat
{
	/** Holds the collection of interpolation points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveQuat)
	TArray<FInterpCurvePointQuat> Points;

	/** Specify whether the curve is looped or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveQuat)
	bool bIsLooped;

	/** Specify the offset from the last point's input key corresponding to the loop point */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveQuat)
	float LoopKeyOffset;
};

/**
 * Describes one specific point on an interpolation curve.
 * @note This is a mirror of TInterpCurvePoint<FTwoVectors>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurvePoint.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor)
struct FInterpCurvePointTwoVectors
{
	/** Float input value that corresponds to this key (eg. time). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointTwoVectors)
	float InVal;

	/** Two 3D vectors output value of when input is equal to InVal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointTwoVectors)
	FTwoVectors OutVal;

	/** Tangent of curve arriving at this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointTwoVectors)
	FTwoVectors ArriveTangent;

	/** Tangent of curve leaving this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointTwoVectors)
	FTwoVectors LeaveTangent;

	/** Interpolation mode between this point and the next one. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointTwoVectors)
	TEnumAsByte<enum EInterpCurveMode> InterpMode;
};

/**
 * Describes an entire curve that is used to compute two 3D vector values from a float input.
 * @note This is a mirror of TInterpCurve<FTwoVectors>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurve.h
 */
USTRUCT(noexport, BlueprintType)
struct FInterpCurveTwoVectors
{
	/** Holds the collection of interpolation points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveTwoVectors)
	TArray<FInterpCurvePointTwoVectors> Points;

	/** Specify whether the curve is looped or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveTwoVectors)
	bool bIsLooped;

	/** Specify the offset from the last point's input key corresponding to the loop point */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveTwoVectors)
	float LoopKeyOffset;
};

/**
 * Describes one specific point on an interpolation curve.
 * @note This is a mirror of TInterpCurvePoint<FLinearColor>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurvePoint.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor)
struct FInterpCurvePointLinearColor
{
	/** Float input value that corresponds to this key (eg. time). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointLinearColor)
	float InVal;

	/** Color output value of when input is equal to InVal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointLinearColor)
	FLinearColor OutVal;

	/** Tangent of curve arriving at this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointLinearColor)
	FLinearColor ArriveTangent;

	/** Tangent of curve leaving this point. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointLinearColor)
	FLinearColor LeaveTangent;

	/** Interpolation mode between this point and the next one. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurvePointLinearColor)
	TEnumAsByte<enum EInterpCurveMode> InterpMode;
};

/**
 * Describes an entire curve that is used to compute a color output value from a float input.
 * @note This is a mirror of TInterpCurve<FLinearColor>, defined in Engine/Source/Runtime/Core/Public/Math/InterpCurve.h
 */
USTRUCT(noexport, BlueprintType)
struct FInterpCurveLinearColor
{
	/** Holds the collection of interpolation points. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveLinearColor)
	TArray<FInterpCurvePointLinearColor> Points;

	/** Specify whether the curve is looped or not */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveLinearColor)
	bool bIsLooped;

	/** Specify the offset from the last point's input key corresponding to the loop point */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=InterpCurveLinearColor)
	float LoopKeyOffset;
};

/**
 * Transform composed of Quat/Translation/Scale.
 * @note This is implemented in either Engine/Source/Runtime/Core/Public/Math/TransformVectorized.h or TransformNonVectorized.h depending on the platform.
 */
USTRUCT(immutable, noexport, BlueprintType, BlueprintInternalUseOnly, IsAlwaysAccessible, HasDefaults, IsCoreType, meta = (ScriptDefaultMake, ScriptDefaultBreak, HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeTransform", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakTransform"))
struct FTransform3f
{
	/** Rotation of this transformation, as a quaternion. */
	UPROPERTY(EditAnywhere, Category = Transform, SaveGame)
	FQuat4f Rotation;

	/** Translation of this transformation, as a vector. */
	UPROPERTY(EditAnywhere, Category = Transform, SaveGame)
	FVector3f Translation;

	/** 3D scale (always applied in local space) as a vector. */
	UPROPERTY(EditAnywhere, Category = Transform, SaveGame)
	FVector3f Scale3D;
};

/**
 * Transform composed of Quat/Translation/Scale.
 * @note This is implemented in either Engine/Source/Runtime/Core/Public/Math/TransformVectorized.h or TransformNonVectorized.h depending on the platform.
 */
USTRUCT(immutable, noexport, IsAlwaysAccessible, HasDefaults, IsCoreType)
struct FTransform3d
{
	/** Rotation of this transformation, as a quaternion. */
	UPROPERTY(EditAnywhere, Category = Transform, SaveGame)
	FQuat4d Rotation;

	/** Translation of this transformation, as a vector. */
	UPROPERTY(EditAnywhere, Category = Transform, SaveGame)
	FVector3d Translation;

	/** 3D scale (always applied in local space) as a vector. */
	UPROPERTY(EditAnywhere, Category = Transform, SaveGame)
	FVector3d Scale3D;
};

/**
 * Transform composed of Quat/Translation/Scale.
 * @note This is implemented in either Engine/Source/Runtime/Core/Public/Math/TransformVectorized.h or TransformNonVectorized.h depending on the platform.
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, IsCoreType, meta=(HasNativeMake="/Script/Engine.KismetMathLibrary.MakeTransform", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakTransform"))
struct FTransform
{
	/** Rotation of this transformation, as a quaternion. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Transform, SaveGame)
	FQuat Rotation;

	/** Translation of this transformation, as a vector. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Transform, SaveGame)
	FVector Translation;

	/** 3D scale (always applied in local space) as a vector. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Transform, SaveGame)
	FVector Scale3D;
};

/**
 * Thread-safe random number generator that can be manually seeded.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Math\RandomStream.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, HasNoOpConstructor, meta = (HasNativeMake = "/Script/Engine.KismetMathLibrary.MakeRandomStream", HasNativeBreak = "/Script/Engine.KismetMathLibrary.BreakRandomStream"))
struct FRandomStream
{
public:
	/** Holds the initial seed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=RandomStream, SaveGame)
	int32 InitialSeed;
	
	/** Holds the current seed. */
	UPROPERTY()
	int32 Seed;
};

/** 
 * A value representing a specific point date and time over a wide range of years.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Misc\DateTime.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, meta=(HasNativeMake="/Script/Engine.KismetMathLibrary.MakeDateTime", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakDateTime"))
struct FDateTime
{
	UPROPERTY()
	int64 Ticks;
};

/** 
 * A frame number value, representing discrete frames since the start of timing.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Misc\FrameNumber.h
 */
USTRUCT(noexport, BlueprintType, IsAlwaysAccessible, HasDefaults)
struct FFrameNumber
{
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=FrameNumber)
	int32 Value;
};

/** 
 * A frame rate represented as a fraction comprising 2 integers: a numerator (number of frames), and a denominator (per second).
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Misc\FrameRate.h
 */
USTRUCT(noexport, BlueprintType, meta=(HasNativeMake="/Script/Engine.KismetMathLibrary.MakeFrameRate", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakFrameRate"))
struct FFrameRate
{
	/** The numerator of the framerate represented as a number of frames per second (e.g. 60 for 60 fps) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=FrameRate)
	int32 Numerator;

	/** The denominator of the framerate represented as a number of frames per second (e.g. 1 for 60 fps) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category=FrameRate)
	int32 Denominator;
};

/** 
 * Represents a time by a context-free frame number, plus a sub frame value in the range [0:1). 
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Misc\FrameTime.h
 * @note The 'SubFrame' field is private to match its C++ class declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FFrameTime
{
	/** Count of frames from start of timing */
	UPROPERTY(BlueprintReadWrite, Category=FrameTime)
	FFrameNumber FrameNumber;
	
private:
	/** Time within a frame, always between >= 0 and < 1 */
	UPROPERTY(BlueprintReadWrite, Category=FrameTime, meta=(AllowPrivateAccess="true"))
	float SubFrame;
};

/** 
 * A frame time qualified by a frame rate context.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Misc\QualifiedFrameTime.h
 */
USTRUCT(noexport, BlueprintType, meta=(ScriptName="QualifiedTime", HasNativeMake="/Script/Engine.KismetMathLibrary.MakeQualifiedFrameTime", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakQualifiedFrameTime"))
struct FQualifiedFrameTime
{
	/** The frame time */
	UPROPERTY(BlueprintReadWrite, Category=QualifiedFrameTime)
	FFrameTime Time;

	/** The rate that this frame time is in */
	UPROPERTY(BlueprintReadWrite, Category=QualifiedFrameTime)
	FFrameRate Rate;
};

/** 
 * A timecode that stores time in HH:MM:SS format with the remainder of time represented by an integer frame count. 
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Misc\TimeCode.h
 */
USTRUCT(noexport, BlueprintType)
struct FTimecode
{
	UPROPERTY(BlueprintReadWrite, Category=Timecode)
	int32 Hours;

	UPROPERTY(BlueprintReadWrite, Category=Timecode)
	int32 Minutes;

	UPROPERTY(BlueprintReadWrite, Category=Timecode)
	int32 Seconds;

	UPROPERTY(BlueprintReadWrite, Category=Timecode)
	int32 Frames;

	UPROPERTY(BlueprintReadWrite, Category=Timecode)
	float Subframe;

	/** If true, this Timecode represents a Drop Frame timecode used to account for fractional frame rates in NTSC play rates. */
	UPROPERTY(BlueprintReadWrite, Category= Timecode)
	bool bDropFrameFormat;
};

/** 
 * A time span value, which is the difference between two dates and times.
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Misc\Timespan.h
 */
USTRUCT(immutable, noexport, BlueprintType, IsAlwaysAccessible, HasDefaults, meta=(HasNativeMake="/Script/Engine.KismetMathLibrary.MakeTimespan", HasNativeBreak="/Script/Engine.KismetMathLibrary.BreakTimespan"))
struct FTimespan
{
	UPROPERTY()
	int64 Ticks;
};

/** 
 * Represents a time by a context-free musical bar/beat. 
 * @note The full C++ class is located here: Engine\Source\Runtime\Core\Public\Misc\MusicalTime.h
 */
USTRUCT(noexport, BlueprintType)
struct FMusicalTime
{
	/** 0-based Bar Index */
	UPROPERTY()
	int32 Bar;
	
	/** 0-based Tick Index */
	UPROPERTY()
	int32 TickInBar;

	UPROPERTY()
	int32 TicksPerBar;

	UPROPERTY()
	int32 TicksPerBeat;
};

/** Enumerates the valid types of range bounds (mirrored from Engine/Source/Runtime/Core/Public/Math/RangeBound.h) */
UENUM(BlueprintType)
namespace ERangeBoundTypes
{
	enum Type : int
	{
		/**
		* The range excludes the bound.
		*/
		Exclusive,

		/**
		* The range includes the bound.
		*/
		Inclusive,

		/**
		* The bound is open.
		*/
		Open
	};
}

/**
 * Defines a single bound for a range of values.
 * @note This is a mirror of TRangeBound<float>, defined in Engine/Source/Runtime/Core/Public/Math/RangeBound.h
 * @note Fields are private to match the C++ declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FFloatRangeBound
{
private:
	/** Holds the type of the bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	TEnumAsByte<ERangeBoundTypes::Type> Type;

	/** Holds the bound's value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	float Value;
};

/**
 * A contiguous set of floats described by lower and upper bound values.
 * @note This is a mirror of TRange<float>, defined in Engine/Source/Runtime/Core/Public/Math/Range.h
 * @note Fields are private to match the C++ declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FFloatRange
{
private:
	/** Holds the range's lower bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FFloatRangeBound LowerBound;

	/** Holds the range's upper bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FFloatRangeBound UpperBound;
};


/**
 * Defines a single bound for a range of values.
 * @note This is a mirror of TRangeBound<double>, defined in Engine/Source/Runtime/Core/Public/Math/RangeBound.h
 * @note Fields are private to match the C++ declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FDoubleRangeBound
{
private:
	/** Holds the type of the bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	TEnumAsByte<ERangeBoundTypes::Type> Type;

	/** Holds the bound's value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	double Value;
};

/**
 * A contiguous set of doubles described by lower and upper bound values.
 * @note This is a mirror of TRange<double>, defined in Engine/Source/Runtime/Core/Public/Math/Range.h
 * @note Fields are private to match the C++ declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FDoubleRange
{
private:
	/** Holds the range's lower bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FDoubleRangeBound LowerBound;

	/** Holds the range's upper bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FDoubleRangeBound UpperBound;
};

/**
 * Defines a single bound for a range of values.
 * @note This is a mirror of TRangeBound<int32>, defined in Engine/Source/Runtime/Core/Public/Math/RangeBound.h
 * @note Fields are private to match the C++ declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FInt32RangeBound
{
private:
	/** Holds the type of the bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	TEnumAsByte<ERangeBoundTypes::Type> Type;

	/** Holds the bound's value. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	int32 Value;
};

/**
 * A contiguous set of floats described by lower and upper bound values.
 * @note This is a mirror of TRange<int32>, defined in Engine/Source/Runtime/Core/Public/Math/Range.h
 * @note Fields are private to match the C++ declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FInt32Range
{
private:
	/** Holds the range's lower bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FInt32RangeBound LowerBound;

	/** Holds the range's upper bound. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FInt32RangeBound UpperBound;
};

/**
 * Defines a single bound for a range of frame numbers.
 * @note This is a mirror of TRangeBound<FFrameNumber>, defined in Engine/Source/Runtime/Core/Public/Math/RangeBound.h
 * @note Fields are private to match the C++ declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FFrameNumberRangeBound
{
private:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	TEnumAsByte<ERangeBoundTypes::Type> Type;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FFrameNumber Value;
};

/**
 * A contiguous set of frame numbers described by lower and upper bound values.
 * @note This is a mirror of TRange<FFrameNumber>, defined in Engine/Source/Runtime/Core/Public/Math/Range.h
 * @note Fields are private to match the C++ declaration in the header above.
 */
USTRUCT(noexport, BlueprintType)
struct FFrameNumberRange
{
private:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FFrameNumberRangeBound LowerBound;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Range, meta=(AllowPrivateAccess="true"))
	FFrameNumberRangeBound UpperBound;
};

/**
 * An interval of floats, defined by inclusive min and max values
 * @note This is a mirror of TInterval<float>, defined in Engine/Source/Runtime/Core/Public/Math/Interval.h
 */
USTRUCT(noexport, BlueprintType)
struct FFloatInterval
{
	/** Values must be >= Min */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Interval)
	float Min;

	/** Values must be <= Max */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Interval)
	float Max;
};

/**
 * An interval of doubles, defined by inclusive min and max values
 * @note This is a mirror of TInterval<double>, defined in Interval.h
 */
USTRUCT(noexport, BlueprintType)
struct FDoubleInterval
{
	/** Values must be >= Min */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Interval)
	double Min;

	/** Values must be <= Max */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Interval)
	double Max;
};

/**
 * An interval of integers, defined by inclusive min and max values
 * @note This is a mirror of TInterval<int32>, defined in Engine/Source/Runtime/Core/Public/Math/Interval.h
 */
USTRUCT(noexport, BlueprintType)
struct FInt32Interval
{
	/** Values must be >= Min */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Interval)
	int32 Min;

	/** Values must be <= Max */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Interval)
	int32 Max;
};

/** Categories of localized text (mirrored in Engine/Source/Runtime/Core/Public/Internationalization/LocalizedTextSourceTypes.h */
UENUM(BlueprintType)
enum class ELocalizedTextSourceCategory : uint8
{
	Game,
	Engine,
	Editor,
};

/**
 * Polyglot data that may be registered to the text localization manager at runtime.
 * @note This struct is mirrored in Engine/Source/Runtime/Core/Public/Internationalization/Internationalization.h
 */
USTRUCT(noexport, BlueprintType)
struct FPolyglotTextData
{
	/**
	 * The category of this polyglot data.
	 * @note This affects when and how the data is loaded into the text localization manager.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PolyglotData)
	ELocalizedTextSourceCategory Category;

	/**
	 * The native culture of this polyglot data.
	 * @note This may be empty, and if empty, will be inferred from the native culture of the text category.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PolyglotData)
	FString NativeCulture;

	/**
	 * The namespace of the text created from this polyglot data.
	 * @note This may be empty.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PolyglotData)
	FString Namespace;

	/**
	 * The key of the text created from this polyglot data.
	 * @note This must not be empty.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PolyglotData)
	FString Key;

	/**
	 * The native string for this polyglot data.
	 * @note This must not be empty (it should be the same as the originally authored text you are trying to replace).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PolyglotData)
	FString NativeString;

	/**
	 * Mapping between a culture code and its localized string.
	 * @note The native culture may also have a translation in this map.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PolyglotData)
	TMap<FString, FString> LocalizedStrings;

	/**
	 * True if this polyglot data is a minimal patch, and that missing translations should be
	 * ignored (falling back to any LocRes data) rather than falling back to the native string.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=PolyglotData)
	bool bIsMinimalPatch;

	/**
	 * Transient cached text instance from registering this polyglot data with the text localization manager.
	 */
	UPROPERTY(Transient)
	FText CachedText;
};

/** Report level of automation events (mirrored in Engine/Source/Runtime/Core/Public/Misc/AutomationEvent.h). */
UENUM()
enum class EAutomationEventType : uint8
{
	Info,
	Warning,
	Error
};

/** Event emitted by automation system (mirrored in Engine/Source/Runtime/Core/Public/Misc/AutomationEvent.h). */
USTRUCT(noexport)
struct FAutomationEvent
{
	UPROPERTY()
	EAutomationEventType Type;

	UPROPERTY()
	FString Message;

	UPROPERTY()
	FString Context;

	UPROPERTY()
	FGuid Artifact;
};

/** Information about the execution of an automation task (mirrored in Engine/Source/Runtime/Core/Public/Misc/AutomationEvent.h). */
USTRUCT(noexport)
struct FAutomationExecutionEntry
{
	UPROPERTY()
	FAutomationEvent Event;

	UPROPERTY()
	FString Filename;

	UPROPERTY()
	int32 LineNumber;

	UPROPERTY()
	FDateTime Timestamp;
};


/**
 * Represents a single input device such as a gamepad, keyboard, or mouse.
 *
 * Has a globally unique identifier.
 * 
 * Opaque struct for the FInputDeviceId struct defined in Engine/Source/Runtime/Core/Public/Misc/CoreMiscDefines.h
 */
USTRUCT(noexport, BlueprintType)
struct FInputDeviceId
{
	GENERATED_BODY()
private:
	
	UPROPERTY(VisibleAnywhere, Category = "PlatformInputDevice")
	int32 InternalId = -1;
};

/**
 * Handle that defines a local user on this platform.
 * This used to be just a typedef int32 that was used interchangeably as ControllerId and LocalUserIndex.
 * Moving forward these will be allocated by the platform application layer.
 *
 * Opaque struct for the FPlatformUserId struct defined in Engine/Source/Runtime/Core/Public/Misc/CoreMiscDefines.h
 */
USTRUCT(noexport, BlueprintType)
struct FPlatformUserId
{
	GENERATED_BODY()
private:
	
	UPROPERTY(VisibleAnywhere, Category = "PlatformInputDevice")
	int32 InternalId = -1;
};

/**
 * Represents the connection status of a given FInputDeviceId
 * @note Mirrored from Engine/Source/Runtime/Core/Public/Misc/CoreMiscDefines.h
 */
UENUM(BlueprintType)
enum class EInputDeviceConnectionState : uint8
{
	/** This is not a valid input device */
	Invalid,

	/** It is not known if this device is connected or not */
	Unknown,

	/** Device is definitely connected */
	Disconnected,

	/** Definitely connected and powered on */
	Connected
};

/**
 * The input device mapping policy controls how Human Interface Devices (HID's)
 * are "mapped" or "assigned" to local players in your game. Depending on this
 * policy, the creation of FPlatformUserId's and FInputDeviceId's can change.
 *
 * This policy is only in affect on platforms which do NOT have a managed user login
 * or some kind of dedicated input mapping OS functionality (consoles). 
 *
 * @note Mirrored from Engine/Source/Runtime/ApplicationCore/Public/GenericPlatform/InputDeviceMappingPolicy.h
 */
UENUM()
enum class EInputDeviceMappingPolicy : int32
{
	/**
	 * Invalid mapping policy. Used to validate that the one in the settings is correct.
	 */
	Invalid = -1 UMETA(Hidden),
	
	/**
	 * Use the current build target's built in platform login functionality.
	 * This is often times a prompt which is displayed by the target system
	 * to assign an input device to a specific player.
	 */
	UseManagedPlatformLogin = 0,
	
	/**
	 * Maps the keyboard/mouse and the first connected controller to the primary platform user.
	 * Any subsequently connected gamepads will be mapped to the next platform user without a
	 * valid input device already assigned to them.
	 *
	 * An example of this policy when you have 3 gamepads and a keyboard/mouse connected would be something like this;
	 *
	 * -- Platform User 0 (Primary)
	 * ---> Keyboard and mouse
	 * ---> Gamepad 0
	 * |
	 * -- Platform User 1
	 * ---> Gamepad 1
	 * |
	 * -- PlatformUser 2
	 * ---> Gamepad 2
	 */
	PrimaryUserSharesKeyboardAndFirstGamepad = 1,

	/**
	 * If this policy is set, then every input device that is connected will be mapped
	 * to a unique platform user, meaning that it is treated as a separate local player.
	 * This is what you want if you want plugging in a new controller to mean that you have a
	 * new local player in your game.
	 *
	 * This only defines the default behavior upon connection of a new device, you can still remap
	 * these input devices to new platform users later in the application lifecycle.
	 */
	CreateUniquePlatformUserForEachDevice = 2,

	/**
	 * If this policy is set, then every connected input device will be mapped to the
	 * primary platform user and new platform user Id's will not be created from input
	 * devices. 
	 */
	MapAllDevicesToPrimaryUser = 3
};

/**
 * Represents input device triggers that are available
 *
 * NOTE: Make sure to keep this type in sync with the reflected version in Engine/Source/Runtime/ApplicationCore/Public/GenericPlatform/IInputInterface.h!
 */
UENUM(BlueprintType)
enum class EInputDeviceTriggerMask : uint8
{
	None		= 0x00,
	Left		= 0x01,
	Right		= 0x02,
	All			= Left | Right
};

/**
 * Represents input device analog sticks that are available
 *
 * NOTE: Make sure to keep this type in sync with the reflected version in Engine/Source/Runtime/ApplicationCore/Public/GenericPlatform/IInputInterface.h!
 */
UENUM(BlueprintType)
enum class EInputDeviceAnalogStickMask : uint8
{
	None = 0x00,
	Left = 0x01,
	Right = 0x02
};

/**
 * Data about an input device's current state
 * @note Mirrored from Engine/Source/Runtime/ApplicationCore/Public/GenericPlatform/InputDeviceMappingPolicy.h
 */
USTRUCT(noexport, BlueprintType)
struct FPlatformInputDeviceState
{
	GENERATED_BODY()

	/** The platform user that this input device belongs to */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "PlatformInputDevice")
	FPlatformUserId OwningPlatformUser = PLATFORMUSERID_NONE;

	/** The connection state of this input device */
	UPROPERTY(BlueprintReadOnly, VisibleAnywhere, Category = "PlatformInputDevice")
	EInputDeviceConnectionState ConnectionState = EInputDeviceConnectionState::Invalid;
};

/** 
 * Enum used by DataValidation plugin to see if an asset has been validated for correctness 
 * @note Mirrored from Engine/Source/Runtime/CoreUObject/Public/UObject/InputDeviceMappingPolicy.h
 */
UENUM(BlueprintType)
enum class EDataValidationResult : uint8
{
	/** Asset has failed validation */
	Invalid,
	/** Asset has passed validation */
	Valid,
	/** Asset has not yet been validated */
	NotValidated
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectTypes.h
 */
USTRUCT(noexport, BlueprintType)
struct FRemoteObjectId
{
	GENERATED_BODY()

	UPROPERTY()
	uint64 Id;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectPathName.h
 */
USTRUCT(noexport, BlueprintType)
struct FPackedRemoteObjectPathName
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<uint16> RemoteIds;

	UPROPERTY()
	TArray<uint16> Names;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectTransfer.h
 */
USTRUCT(noexport, BlueprintType)
struct FRemoteObjectBytes
{
	UPROPERTY()
	TArray<uint8> Bytes;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectPathName.h
 */
USTRUCT(noexport, BlueprintType)
struct FRemoteObjectTables
{
	UPROPERTY()
	TArray<FName> Names;

	UPROPERTY()
	TArray<FRemoteObjectId> RemoteIds;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectPathName.h
 */
USTRUCT(noexport)
struct FRemoteObjectPathName : public FRemoteObjectTables
{
	GENERATED_BODY()
};

/**
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectTransfer.h
 */
USTRUCT(noexport)
struct FPackedRemoteObjectHeader
{
	GENERATED_BODY()

	UPROPERTY()
	uint16 NameIndex;

	UPROPERTY()
	uint16 IdIndex;

	UPROPERTY()
	uint16 ClassIndex;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectTransfer.h
 */
USTRUCT(noexport, BlueprintType)
struct FRemoteObjectData
{
	GENERATED_BODY()

	UPROPERTY()
	FRemoteObjectId MigrationId;

	UPROPERTY()
	FRemoteObjectTables Tables;

	UPROPERTY()
	TArray<FPackedRemoteObjectPathName> PathNames;

	UPROPERTY()
	TArray<FPackedRemoteObjectHeader> SerializedObjectHeaders;

	UPROPERTY()
	TArray<FRemoteObjectBytes> Bytes;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectTypes.h
 */
USTRUCT(noexport, BlueprintType)
struct FRemoteServerId
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 Id;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteExecutor.h
 */
USTRUCT(noexport, BlueprintType)
struct FRemoteWorkPriority
{
	GENERATED_BODY()

	UPROPERTY()
	uint64 PackedData;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteObjectTransfer.h
 */
USTRUCT(noexport, BlueprintType)
struct FRemoteObjectReference
{
	GENERATED_BODY()

	UPROPERTY()
	FRemoteObjectId ObjectId;

	UPROPERTY()
	FRemoteServerId ServerId;
};

/** 
 * @note The full C++ class is located here: Engine\Source\Runtime\CoreUObject\Public\UObject\RemoteExecutor.h
 */
USTRUCT(noexport, BlueprintType)
struct FRemoteTransactionId
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 Id;
};

/// @endcond

#endif
