// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*=============================================================================
	Object.h: Direct base class for all UE objects
=============================================================================*/

// IWYU pragma: begin_exports

#include "UObject/Script.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectBaseUtility.h"
#include "UObject/ObjectCompileContext.h"
#include "ProfilingDebugging/ResourceSize.h"
#include "UObject/PrimaryAssetId.h"
#include "Containers/VersePathFwd.h"

#include "Object.generated.h"

struct FAssetData;
class FAssetRegistryTagsContext;
namespace UE::AssetRegistry { class FAssetRegistryImpl; }
namespace UE::ConfigAccessTracking { struct FConfigAccessData; }
namespace UE::Cook { class FCookEventContext; }
namespace UE::Cook { enum class ECookEvent : uint32; }
class FConfigCacheIni;
class FCustomPropertyConditionState;
class FEditPropertyChain;
class FObjectCollectSaveOverridesContext;
class FObjectPostSaveContext;
class FObjectPostSaveRootContext;
class FObjectPreSaveContext;
class FObjectPreSaveRootContext;
class ITargetPlatform;
class ITransactionObjectAnnotation;
class FTransactionObjectEvent;
struct FArchiveCookContext;
struct FAppendToClassSchemaContext;
struct FFrame;
struct FObjectInstancingGraph;
struct FPropertyChangedChainEvent;
struct FTopLevelAssetPath;
class UClass;

namespace UE::Net
{
	class FFragmentRegistrationContext;
	enum class EFragmentRegistrationFlags : uint32;
	namespace Private
	{
		struct FNetHandleLegacyPushModelHelper;
	}
}

// IWYU pragma: end_exports

DECLARE_LOG_CATEGORY_EXTERN(LogObj, Log, All);

/** Parameter enum for CastChecked() function, defines when it will check/assert */
namespace ECastCheckedType
{
	enum Type
	{
		/** Null is okay, only assert on incorrect type */
		NullAllowed,
		/** Null is not allowed, assert on incorrect type or null */
		NullChecked
	};
};

#if WITH_EDITOR
/** Flags to control how editor change events get propagated */
enum class EEditChangePropagationFlags : uint8
{
	/**
	 * No flags; use default change event propagation behavior.
	 */
	None = 0,
	/**
	 * If set, only the subset of affected instances that realign with
	 * their archetype will be marked dirty. If not set, affected instances
	 * will always be marked dirty, even if they already match the archetype.
	 */
	 OnlyMarkRealignedInstancesAsDirty = (1 << 0),
};

ENUM_CLASS_FLAGS(EEditChangePropagationFlags);
#endif // WITH_EDITOR

/** 
 * The base class of all UE objects. The type of an object is defined by its UClass.
 * This provides support functions for creating and using objects, and virtual functions that should be overridden in child classes.
 * 
 * @see https://docs.unrealengine.com/ProgrammingAndScripting/ProgrammingWithCPP/UnrealArchitecture/Objects
 */
UCLASS(Abstract, MinimalAPI, Config = Engine)
class UObject : public UObjectBaseUtility
{
	GENERATED_BODY()

public:
	typedef UObject WithinClass;

	/** Default constructor */
	COREUOBJECT_API UObject();

	/** 
	 *  Constructor that takes an ObjectInitializer. 
	 *  Typically not needed, but can be useful for class hierarchies that support
	 *  optional subobjects or subobject class overriding
	 */
	COREUOBJECT_API UObject(const FObjectInitializer& ObjectInitializer);

	/** DO NOT USE. This constructor is for internal usage only for statically-created objects. */
	COREUOBJECT_API UObject(EStaticConstructor, EObjectFlags InFlags);

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	COREUOBJECT_API UObject(FVTableHelper& Helper);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval UObject(UE::CodeGen::ConstInit::FObjectParams InParams)
		: UObjectBaseUtility(InParams)
	{
	}
#endif

	/** Utility function for templates below */
	COREUOBJECT_API UObject* CreateDefaultSubobject(FName SubobjectFName, UClass* ReturnType, UClass* ClassToCreateByDefault, bool bIsRequired, bool bIsTransient);

	/**
	 * Create a component or subobject only to be used with the editor. They will be stripped out in packaged builds.
	 * @param	TReturnType					Class of return type, all overrides must be of this type
	 * @param	SubobjectName				Name of the new component
	 * @param	bTransient					True if the component is being assigned to a transient property. This does not make the component itself transient, but does stop it from inheriting parent defaults
	 */
	template<class TReturnType>
	TReturnType* CreateEditorOnlyDefaultSubobject(FName SubobjectName, bool bTransient = false)
	{
		UClass* ReturnType = TReturnType::StaticClass();
		return static_cast<TReturnType*>(CreateEditorOnlyDefaultSubobjectImpl(SubobjectName, ReturnType, bTransient));
	}

	/**
	 * Create a component or subobject that will be instanced inside all instances of this class.
	 * @param	TReturnType					Class of return type, all overrides must be of this type
	 * @param	SubobjectName				Name of the new component
	 * @param	bTransient					True if the component is being assigned to a transient property. This does not make the component itself transient, but does stop it from inheriting parent defaults
	 */
	template<class TReturnType>
	TReturnType* CreateDefaultSubobject(FName SubobjectName, bool bTransient = false)
	{
		UClass* ReturnType = TReturnType::StaticClass();
		return static_cast<TReturnType*>(CreateDefaultSubobject(SubobjectName, ReturnType, ReturnType, /*bIsRequired =*/ true, bTransient));
	}
	
	/**
	 * Create a component or subobject, allows creating a child class and returning the parent class.
	 * @param	TReturnType					Class of return type, all overrides must be of this type
	 * @param	TClassToConstructByDefault	Class of object to actually construct, must be a subclass of TReturnType
	 * @param	SubobjectName				Name of the new component
	 * @param	bTransient					True if the component is being assigned to a transient property. This does not make the component itself transient, but does stop it from inheriting parent defaults
	 */
	template<class TReturnType, class TClassToConstructByDefault>
	TReturnType* CreateDefaultSubobject(FName SubobjectName, bool bTransient = false)
	{
		return static_cast<TReturnType*>(CreateDefaultSubobject(SubobjectName, TReturnType::StaticClass(), TClassToConstructByDefault::StaticClass(), /*bIsRequired =*/ true, bTransient));
	}
	
	/**
	 * Create an optional component or subobject.Optional subobjects will not get created.
	 * if a derived class specifies DoNotCreateDefaultSubobject with the subobject name.
	 * @param	TReturnType					Class of return type, all overrides must be of this type
	 * @param	SubobjectName				Name of the new component
	 * @param	bTransient					True if the component is being assigned to a transient property. This does not make the component itself transient, but does stop it from inheriting parent defaults
	 */
	template<class TReturnType>
	TReturnType* CreateOptionalDefaultSubobject(FName SubobjectName, bool bTransient = false)
	{
		UClass* ReturnType = TReturnType::StaticClass();
		return static_cast<TReturnType*>(CreateDefaultSubobject(SubobjectName, ReturnType, ReturnType, /*bIsRequired =*/ false, bTransient));
	}
	
	/**
	 * Create an optional component or subobject. Optional subobjects will not get created.
	 * if a derived class specifies DoNotCreateDefaultSubobject with the subobject name.
	 * @param	TReturnType					Class of return type, all overrides must be of this type
	 * @param	TClassToConstructByDefault	Class of object to actually construct, must be a subclass of TReturnType
	 * @param	SubobjectName				Name of the new component
	 * @param	bTransient					True if the component is being assigned to a transient property. This does not make the component itself transient, but does stop it from inheriting parent defaults
	 */
	template<class TReturnType, class TClassToConstructByDefault>
	TReturnType* CreateOptionalDefaultSubobject(FName SubobjectName, bool bTransient = false)
	{
		return static_cast<TReturnType*>(CreateDefaultSubobject(SubobjectName, TReturnType::StaticClass(), TClassToConstructByDefault::StaticClass(), /*bIsRequired =*/ false, bTransient));
	}

	/**
	 * Gets all directly nested default subobjects that are associated with this object instance.
	 * @param	OutDefaultSubobjects	Array containing the directly nested default subobjects of this object.
	 */
	COREUOBJECT_API void GetDefaultSubobjects(TArray<UObject*>& OutDefaultSubobjects);

	/**
	 * Finds a default subobject associated with this object instance by its name.
	 * @param	Name	Object name to look for matching the SubobjectName used to create the default subobject
	 */
	COREUOBJECT_API UObject* GetDefaultSubobjectByName(FName ToFind);

	/*----------------------------------
			UObject interface
	----------------------------------*/

protected:
    /** 
     * This function actually does the work for the GetDetailedInfo() and is virtual.  
     * It should only be called from GetDetailedInfo as GetDetailedInfo is safe to call on NULL object pointers
     */
	virtual FString GetDetailedInfoInternal() const { return TEXT("No_Detailed_Info_Specified"); }

public:
	/**
	 * Called after the C++ constructor and after the properties have been initialized, including those loaded from config.
	 * This is called before any serialization, Verse construction, or other setup has occurred.
	 */
	COREUOBJECT_API virtual void PostInitProperties();
	
	/**
	 * Called after properties are overwritten, including after subobjects initialization from a CDO.
	 * This could be called multiple times during an object lifetime, which is not the case for PostInitProperties which is expected to be called only once.
	 */
	COREUOBJECT_API virtual void PostReinitProperties();

	/**
	* Called after the C++ constructor has run on the Class Default Object (CDO) for a class. This is an obscure routine used to deal with the recursion 
	* in the construction of the default materials
	*/
	virtual void PostCDOContruct()
	{
	}

#if WITH_EDITOR
	using FPostCDOCompiledContext = FObjectPostCDOCompiledContext;

	/**
	 * Called after the Blueprint compiler has finished generating the Class Default Object (CDO) for a class. This can only happen in the editor.
	 * This is called when the CDO and its associated class structure have been fully generated and populated, and allows the assignment of cached/derived data, 
	 * eg) caching the name/count of properties of a certain type, or inspecting the properties on the class and using their meta-data and CDO default values to derive game data.
	 */
	virtual void PostCDOCompiled(const FPostCDOCompiledContext& Context)
	{
	}
#endif

	/**
	 * Called from within SavePackage on the passed in base/root object. The return value of this function will be passed to PostSaveRoot. 
	 * This is used to allow objects used as a base to perform required actions before saving and cleanup afterwards.
	 *
	 * @param	ObjectSaveContext Context providing access to parameters of the save,
	*							  Also allows storage of variables like bCleanupIsRequired for use in PostSaveRoot
	 */
	COREUOBJECT_API virtual void PreSaveRoot(FObjectPreSaveRootContext ObjectSaveContext);

	/**
	 * Called from within SavePackage on the passed in base/root object. 
	 * This function is called after the package has been saved and can perform cleanup.
	 *
	 * @param	ObjectSaveContext Context providing access to parameters of the save and to values from PreSaveRoot
	 */
	COREUOBJECT_API virtual void PostSaveRoot(FObjectPostSaveRootContext ObjectSaveContext);

	/**
	 * Presave function. Gets called once before an object gets serialized for saving. This function is necessary
	 * for save time computation as Serialize gets called three times per object from within SavePackage.
	 *
	 * @warning: Objects created from within PreSave will NOT have PreSave called on them!!!
	 */
	COREUOBJECT_API virtual void PreSave(FObjectPreSaveContext SaveContext);

	/**
	 * Collect any save override state.
	 */
	COREUOBJECT_API virtual void CollectSaveOverrides(FObjectCollectSaveOverridesContext SaveContext);

#if WITH_EDITOR
	/**
	 * Called during cooking for various events that happen on each UObject in a package during load/transform/save.
	 * @see UE::Cook::ECookEvent for the possible events.
	 */
	virtual void OnCookEvent(UE::Cook::ECookEvent CookEvent, UE::Cook::FCookEventContext& Context)
	{
	}

	/**
	 * Note that the object will be modified.  If we are currently recording into the 
	 * transaction buffer (undo/redo), save a copy of this object into the buffer and 
	 * marks the package as needing to be saved.
	 *
	 * @param	bAlwaysMarkDirty	if true, marks the package dirty even if we aren't
	 *								currently recording an active undo/redo transaction
	 * @return true if the object was saved to the transaction buffer
	 */
	COREUOBJECT_API virtual bool Modify( bool bAlwaysMarkDirty=true );

	/** Method to disable the sub object special handling in the transaction buffer for this object*/
	COREUOBJECT_API virtual bool IsCapturingAsRootObjectForTransaction() const;

	/** Utility to allow overrides of Modify to avoid doing work if this object cannot be safely modified */
	COREUOBJECT_API bool CanModify() const;

#else
	UE_FORCEINLINE_HINT bool Modify(bool bAlwaysMarkDirty = true) { return false; }
	UE_FORCEINLINE_HINT bool IsCapturingAsRootObjectForTransaction() const { return false; };
#endif

#if WITH_EDITOR
	/** 
	 * Called when the object was loaded from another class via active class redirects.
	 */
	virtual void LoadedFromAnotherClass(const FName& OldClassName) {}
#endif

	/**
	 * Called to defer loading a subobject to its top-level container object. Useful with the package override
	 * feature, where an object might not be saved in the same package as its outer, especially if the the top-level
	 * object does lazy loading of these objects.
	 * 
 	 * @param	SubObjectPath		object path to handle, relative to the current implementer
	 * @param	OutObject			receives the loaded object, or existing loaded object if bOnlyTestExistence is true and object is already loaded
	 * @param	bLoadIfExists		if true, load the object if it exists
	 * 
	 * @return true if the object loaded or exists, depending on bLoadIfExists
* 
	 */
	virtual bool ResolveSubobject(const TCHAR* SubObjectPath, UObject*& OutObject, bool bLoadIfExists) { return false; }

	/**
	 * Called before calling PostLoad() in FAsyncPackage::PostLoadObjects(). This is the safeguard to prevent PostLoad() from stalling the main thread.
	 */
	virtual bool IsReadyForAsyncPostLoad() const { return true; }

	/** 
	 * Do any object-specific cleanup required immediately after loading an object.
	 * This is not called for newly-created objects, and by default will always execute on the game thread.
	 */
	COREUOBJECT_API virtual void PostLoad();

	/**
	 * Instances components for objects being loaded from disk, if necessary.  Ensures that component references
	 * between nested components are fixed up correctly.
	 *
	 * @param	OuterInstanceGraph	when calling this method on subobjects, specifies the instancing graph which contains all instanced
	 *								subobjects and components for a subobject root.
	 */
	COREUOBJECT_API virtual void PostLoadSubobjects(FObjectInstancingGraph* OuterInstanceGraph);
	
	/**
	 * Called before destroying the object.  This is called immediately upon deciding to destroy the object, to allow the object to begin an
	 * asynchronous cleanup process.
	 */
	COREUOBJECT_API virtual void BeginDestroy();

	/**
	 * Called to check if the object is ready for FinishDestroy.  This is called after BeginDestroy to check the completion of the
	 * potentially asynchronous object cleanup.
	 * @return True if the object's asynchronous cleanup has completed and it is ready for FinishDestroy to be called.
	 */
	virtual bool IsReadyForFinishDestroy() { return true; }

#if WITH_EDITOR
	/**
	 * Called in response to the linker changing, this can only happen in the editor
	 */
	virtual void PostLinkerChange() {}
#endif

	/**
	 * Called to finish destroying the object.  After UObject::FinishDestroy is called, the object's memory should no longer be accessed.
	 *
	 * @warning Because properties are destroyed here, Super::FinishDestroy() should always be called at the end of your child class's FinishDestroy() method, rather than at the beginning.
	 */
	COREUOBJECT_API virtual void FinishDestroy();

	/** 
	 * Handles reading, writing, and reference collecting using FArchive.
	 * This implementation handles all FProperty serialization, but can be overridden for native variables.
	 */
	COREUOBJECT_API virtual void Serialize(FArchive& Ar);
	COREUOBJECT_API virtual void Serialize(FStructuredArchive::FRecord Record);
#if WITH_EDITORONLY_DATA
	/**
	 * Call Ar.UsingCustomVersion for every CustomVersion that might be serialized by this class when saving.
	 * This duplicates CustomVersions declared in Serialize; Serialize still needs to declare them.
	 * Used to track which customversions will be used by a package when it is resaved.
	 * Not yet exhaustive; add CustomVersions as necessary to remove EditorDomain warnings about missing versions. */
	static COREUOBJECT_API void DeclareCustomVersions(FArchive& Ar, const UClass* SpecificSubclass);
	/**
	 * Append config values or settings that can change how instances of the class are cooked, including especially
	 * values that determine how version upgraded are conducted. Can also append a unique guid when necessary to
	 * invalidate previous results because serialization changed and no custom version was udpated.
	 */
	static COREUOBJECT_API void AppendToClassSchema(FAppendToClassSchemaContext& Context);
	/**
	 * Declare classes that can be constructed by this class during loading. This declaration is implicitly transitive;
	 * the caller is responsible for following the graph of ConstructClasses to find all transitive ConstructClasses.
	 * 
	 * @param OutConstructClasses Output list of classes that this class can construct.
	 * @param SpecificSubclass The class on which DeclaredConstructClasses was called. This can differ from the current
	 *        class because each class calls Super::DeclareConstructClasses.
	 */
	static COREUOBJECT_API void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
	/**
	 * Allows class to inject items into the linker tables of the package that is loading an export of this class, for example
	 * adding dynamically determined imports. Only objects with RF_HasDynamicImports currently invoke this.
	 * @param Linker  A reference to the linker used for loading the export
	 * @param Export  The export being processed
	 * @param OutDynamicImports  The dynamic imports to inject
	 */
	static COREUOBJECT_API void InjectDynamicImportsFor(const class FLinkerLoad& Linker, const struct FObjectExport& Export, TArray<struct FDynamicImport>& OutDynamicImports);
#endif

	/** After a critical error, perform any mission-critical cleanup, such as restoring the video mode or releasing hardware resources. */
	virtual void ShutdownAfterError() {}

#if WITH_EDITOR
	/** 
	 * This is called when a property is about to be modified externally
	 *
	 * @param PropertyThatWillChange	Property that will be changed
	 */
	COREUOBJECT_API virtual void PreEditChange(FProperty* PropertyAboutToChange);

	/**
	 * This alternate version of PreEditChange is called when properties inside structs are modified.  The property that was actually modified
	 * is located at the tail of the list.  The head of the list of the FStructProperty member variable that contains the property that was modified.
	 *
	 * @param PropertyAboutToChange the property that is about to be modified
	 */
	COREUOBJECT_API virtual void PreEditChange( class FEditPropertyChain& PropertyAboutToChange );

	/**
	 * Called by the editor to query whether a property of this object is allowed to be modified.
	 * The property editor uses this to disable controls for properties that should not be changed.
	 * When overriding this function you should always call the parent implementation first.
	 *
	 * @param	InProperty	The property to query
	 *
	 * @return	true if the property can be modified in the editor, otherwise false
	 */
	COREUOBJECT_API virtual bool CanEditChange( const FProperty* InProperty ) const;

	/**
	 * Alternate version of CanEditChange that includes the full property chain leading to the property in question.
	 * The head of the chain is the FStructProperty member variable that contains the property that was modified.
	 * The active property in the chain is the specific FProperty in question (the one given to the other signature of CanEditChange)
	 * 
	 * @param PropertyChain The chain to the property in question. The ActiveNode of the chain corresponds to the specific property in question.
	 *
	 * @return	true if the property can be modified in the editor, otherwise false
	 */
	COREUOBJECT_API virtual bool CanEditChange(const FEditPropertyChain& PropertyChain) const;

	/** 
	 * Intentionally non-virtual as it calls the FPropertyChangedEvent version
	 */
	COREUOBJECT_API void PostEditChange();

	/**
	 * Called when a property on this object has been modified externally
	 *
	 * @param PropertyThatChanged the property that was modified
	 */
	COREUOBJECT_API virtual void PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent);

	/**
	 * This alternate version of PostEditChange is called when properties inside structs are modified.  The property that was actually modified
	 * is located at the tail of the list.  The head of the list of the FStructProperty member variable that contains the property that was modified.
	 */
	COREUOBJECT_API virtual void PostEditChangeChainProperty( struct FPropertyChangedChainEvent& PropertyChangedEvent );

	/** Called to set the flags that control how editor change events get propagated to archetype instances. */
	COREUOBJECT_API void SetEditChangePropagationFlags(EEditChangePropagationFlags InFlags);

	/** Called before applying a transaction to the object.  Default implementation simply calls PreEditChange. */
	COREUOBJECT_API virtual void PreEditUndo();

	/** Called after applying a transaction to the object.  Default implementation simply calls PostEditChange. */
	COREUOBJECT_API virtual void PostEditUndo();

	/** Called after applying a transaction to the object in cases where transaction annotation was provided. Default implementation simply calls PostEditChange. */
	COREUOBJECT_API virtual void PostEditUndo(TSharedPtr<ITransactionObjectAnnotation> TransactionAnnotation);

	/**
	 * Called after the object has been transacted in some way.
	 * TransactionEvent describes what actually happened.
	 * @note Unlike PostEditUndo (which is called for any object in the transaction), this is only called on objects that are actually changed by the transaction.
	 */
	COREUOBJECT_API virtual void PostTransacted(const FTransactionObjectEvent& TransactionEvent);

	/** Find or create and populate an annotation object with any external data required for applying a transaction */
	COREUOBJECT_API TSharedPtr<ITransactionObjectAnnotation> FindOrCreateTransactionAnnotation() const;

	/** Create and restore a previously serialized annotation object with any external data required for applying a transaction */
	COREUOBJECT_API TSharedPtr<ITransactionObjectAnnotation> CreateAndRestoreTransactionAnnotation(FArchive& Ar) const;

protected:
	/** Factory a new annotation object and optionally populate it with data */
	enum class ETransactionAnnotationCreationMode : uint8 { DefaultInstance, FindOrCreate };
	virtual TSharedPtr<ITransactionObjectAnnotation> FactoryTransactionAnnotation(const ETransactionAnnotationCreationMode InCreationMode) const { return nullptr; }

	/**
	 * Test the selection state of a UObject
	 *
	 * @return		true if the object is selected, false otherwise.
	 * @todo UE	 this doesn't belong here, but it doesn't belong anywhere else any better
	 */
	COREUOBJECT_API virtual bool IsSelectedInEditor() const;

public:
#endif // WITH_EDITOR

	/** Called at the end of Rename(), but only if the rename was actually carried out */
	COREUOBJECT_API virtual void PostRename(UObject* OldOuter, const FName OldName);

	/**
	 * Called before duplication.
	 *
	 * @param DupParams the full parameters the object will be duplicated with.
	 *        Allows access to modify params such as the duplication seed for example for pre-filling the dup-source => dup-target map used by StaticDuplicateObject. 
	 * @see FObjectDuplicationParameters
	 */
	virtual void PreDuplicate(FObjectDuplicationParameters& DupParams) {}

	/**
	 * Called after duplication & serialization and before PostLoad. Used to e.g. make sure UStaticMesh's UModel gets copied as well.
	 * Note: NOT called on components on actor duplication (alt-drag or copy-paste).  Use PostEditImport as well to cover that case.
	 */
	virtual void PostDuplicate(bool bDuplicateForPIE) {}
	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) 
	{
		PostDuplicate(DuplicateMode == EDuplicateMode::PIE);
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/**
	 * Called when an object is about to be migrated and after it's been migrated to another process.
	 */
	COREUOBJECT_API virtual void PostMigrate(const struct FUObjectMigrationContext& MigrationContext);

	virtual bool IsMigrationRoot() const { return false; }
#endif

	/**
	 * Called during saving to determine the load flags to save with the object.
	 * If false, this object will be discarded on clients
	 *
	 * @return	true if this object should be loaded on clients
	 */
	COREUOBJECT_API virtual bool NeedsLoadForClient() const;

	/**
	 * Called during saving to determine the load flags to save with the object.
	 * If false, this object will be discarded on servers
	 *
	 * @return	true if this object should be loaded on servers
	 */
	COREUOBJECT_API virtual bool NeedsLoadForServer() const;

	/**
	 * Called during saving to determine the load flags to save with the object.
	 * If false, this object will be discarded on the target platform
	 *
	 * @return	true if this object should be loaded on the target platform
	 */
	COREUOBJECT_API virtual bool NeedsLoadForTargetPlatform(const class ITargetPlatform* TargetPlatform) const;

	/**
	 * Called during saving to include this object in client/servers running in editor builds, even if they wouldn't normally be.
	 * If false, this object will still get loaded if NeedsLoadForServer/Client are true
	 * 
	 * @return	true if this object should always be loaded for games running in editor builds
	 */
	virtual bool NeedsLoadForEditorGame() const
	{
		return false;
	}

	/**
	* Called during saving to determine if the object is forced to be editor only or not
	*
	* @return	true if this object should never be loaded outside the editor
	*/
	virtual bool IsEditorOnly() const
	{
		return false;
	}

	/**
	* Called during saving to determine if the object's references are used in game even when the object itself
	* is never loaded outside the editor (because e.g. its references are followed during cooking)
	*
	* @return	true if this object's references should be marked as used in game even when the object is editoronly
	*/
	virtual bool HasNonEditorOnlyReferences() const
	{
		return false;
	}

	/**
	* Called during async load to determine if PostLoad can be called on the loading thread.
	*
	* @return	true if this object's PostLoad is thread safe
	*/
	virtual bool IsPostLoadThreadSafe() const
	{
		return false;
	}

	/**
	* Called during cooking. Must return all objects that will be Preload()ed when this is serialized at load time. Only used by the EDL.
	*
	* @param	OutDeps				all objects that will be preloaded when this is serialized at load time
	*/
	COREUOBJECT_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps);

	/**
	* Called during cooking. Returns a list of objects. The packages containing those objects will be prestreamed, when the package containing this is loaded. Only used by the EDL.
	*
	* @param	OutPrestream				all objects that will be prestreamed when this packages is streamed
	*/
	virtual void GetPrestreamPackages(TArray<UObject*>& OutPrestream)
	{
	}

	/**
	*	Update the list of classes that we should exclude from dedicated server builds
	*/
	static COREUOBJECT_API void UpdateClassesExcludedFromDedicatedServer(const TArray<FString>& InClassNames, const TArray<FString>& InModulesNames);

	/**
	*	Update the list of classes that we should exclude from dedicated client builds
	*/
	static COREUOBJECT_API void UpdateClassesExcludedFromDedicatedClient(const TArray<FString>& InClassNames, const TArray<FString>& InModulesNames);

	/** 
	 *	Determines if you can create an object from the supplied template in the current context (editor, client only, dedicated server, game/listen) 
	 *	This calls NeedsLoadForClient & NeedsLoadForServer
	 */
	static COREUOBJECT_API bool CanCreateInCurrentContext(UObject* Template);

	/**
	 * Exports the property values for the specified object as text to the output device.
	 * Override this if you need custom support for copy/paste.
	 *
	 * @param	Out				The output device to send the exported text to
	 * @param	Indent			Number of spaces to prepend to each line of output
	 *
	 * @see ImportCustomProperties()
	 */
	virtual void ExportCustomProperties(FOutputDevice& Out, uint32 Indent)	{}

	/**
	 * Exports the property values for the specified object as text to the output device. Required for Copy&Paste
	 * Override this if you need custom support for copy/paste.
	 *
	 * @param	SourceText		The input data (zero terminated), will never be null
	 * @param	Warn			For error reporting, will never be null
	 *
	 * @see ExportCustomProperties()
	 */
	virtual void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) {}

	/**
	 * Called after importing property values for this object (paste, duplicate or .t3d import)
	 * Allow the object to perform any cleanup for properties which shouldn't be duplicated or
	 * are unsupported by the script serialization
	 */
	virtual void PostEditImport() {}

	/**
	 * Called from ReloadConfig after the object has reloaded its configuration data.
	 */
	virtual void PostReloadConfig( class FProperty* PropertyThatWasLoaded ) {}

	/** 
	 * Rename this object to a unique name, or change its outer.
     * @remark A unique name will be generated if NewName is null and if either is true: NewOuter is null or NewOuter equals the current outer.
	 * 
	 * @param	NewName		The new name of the object, if null then NewOuter should be set
	 * @param	NewOuter	New Outer this object will be placed within, if null it will use the current outer
	 * @param	Flags		Flags to specify what happens during the rename
	 */
	COREUOBJECT_API virtual bool Rename(const TCHAR* NewName=nullptr, UObject* NewOuter=nullptr, ERenameFlags Flags=REN_None);

	/** Return a one line description of an object for viewing in the thumbnail view of the generic browser */
	virtual FString GetDesc() { return TEXT( "" ); }

	/** Return the UStruct corresponding to the sidecar data structure that stores data that is constant for all instances of this class. */
	COREUOBJECT_API UScriptStruct* GetSparseClassDataStruct() const;

#if WITH_EDITOR
	virtual void MoveDataToSparseClassDataStruct() const {}
	virtual bool ConformSparseClassDataStruct(UScriptStruct* SourceSparseClassDataStruct, void* SourceSparseClassData) const { return false; }
#endif

#if WITH_ENGINE
	/** 
	 * Returns what UWorld this object is contained within. 
	 * By default this will follow its Outer chain, but it should be overridden if that will not work.
	 */
	COREUOBJECT_API virtual class UWorld* GetWorld() const;

	/** Internal function used by UEngine::GetWorldFromContextObject() */
	COREUOBJECT_API class UWorld* GetWorldChecked(bool& bSupported) const;

#if WITH_EDITOR
	/** Checks to see if GetWorld() is implemented on a specific class. Optionally overridable in derived classes. */
	COREUOBJECT_API virtual bool ImplementsGetWorld() const;
#endif // #if WITH_EDITOR

#endif // #if WITH_ENGINE

	/**
	 * Callback for retrieving a textual representation of natively serialized properties.  Child classes should implement this method if they wish
	 * to have natively serialized property values included in things like diffcommandlet output.
	 *
	 * @param	out_PropertyValues	receives the property names and values which should be reported for this object.  The map's key should be the name of
	 *								the property and the map's value should be the textual representation of the property's value.  The property value should
	 *								be formatted the same way that FProperty::ExportText formats property values (i.e. for arrays, wrap in quotes and use a comma
	 *								as the delimiter between elements, etc.)
	 * @param	ExportFlags			bitmask of EPropertyPortFlags used for modifying the format of the property values
	 *
	 * @return	true if property values were added to the map.
	 */
	virtual bool GetNativePropertyValues( TMap<FString,FString>& out_PropertyValues, uint32 ExportFlags=0 ) const
	{
		return false;
	}

	/**
	 * Get the size of the object/resource for use in memory tools or to display to artists/LDs in the Editor
	 * This is the extended version which separates up the used memory into different memory regions (the actual definition of which may be platform specific).
	 *
	 * @param	CumulativeResourceSize	Struct used to count up the cumulative size of the resource as to be displayed to artists/LDs in the Editor.
	 */
	COREUOBJECT_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize);

	/**
	 * Get the size of the object/resource for use in memory tools or to display to artists/LDs in the Editor
	 * This is the simple version which just returns the total number of bytes used by this object.
	 *
	 * @param	Mode					Indicates which resource size should be returned.
	 * @return The cumulative size of this object in memory
	 */
	SIZE_T GetResourceSizeBytes(EResourceSizeMode::Type Mode)
	{
		FResourceSizeEx ResSize = FResourceSizeEx(Mode);
		GetResourceSizeEx(ResSize);
		return ResSize.GetTotalMemoryBytes();
	}

	/** 
	 * Returns the name of the exporter factory used to export this object
	 * Used when multiple factories have the same extension
	 */
	virtual FName GetExporterName( void )
	{
		return( FName() );
	}

	/**
	 * Callback used to allow object register its direct object references that are not already covered by
	 * the token stream.
	 *
	 * @param InThis Object to collect references from.
	 * @param Collector	FReferenceCollector objects to be used to collect references.
	 */
	static COREUOBJECT_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

	/**
	 * Helper function to call AddReferencedObjects for this object's class.
	 *
	 * @param Collector	FReferenceCollector objects to be used to collect references.
	 */
	COREUOBJECT_API void CallAddReferencedObjects(FReferenceCollector& Collector);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/**
	* Coming from verse, marks the object as Reachable if it's currently marked as MaybeUnreachable by incremental GC.
	*/
	COREUOBJECT_API void VerseMarkAsReachable() const;
#endif

	/**
	 * Save information for StaticAllocateObject in the case of overwriting an existing object.
	 * StaticAllocateObject will call delete on the result after calling Restore()
	 *
	 * @return An FRestoreForUObjectOverwrite that can restore the object or NULL if this is not necessary.
	 */
	virtual FRestoreForUObjectOverwrite* GetRestoreForUObjectOverwrite()
	{
		return NULL;
	}

	/**
	 * Returns whether native properties are identical to the one of the passed in component.
	 *
	 * @param	Other	Other component to compare against
	 *
	 * @return true if native properties are identical, false otherwise
	 */
	virtual bool AreNativePropertiesIdenticalTo( UObject* Other ) const
	{
		return true;
	}

	/** Struct used by GetAssetRegistryTags() to return tag info */
	struct FAssetRegistryTag
	{
		/** Enum specifying the type of this tag */
		enum ETagType
		{
			/** This tag should not be shown in the UI */
			TT_Hidden,
			/** This tag should be shown, and sorted alphabetically in the UI */
			TT_Alphabetical,
			/** This tag should be shown, and is a number */
			TT_Numerical,
			/** This tag should be shown, and is an "x" delimited list of dimensions */ 
			TT_Dimensional,
			/** This tag should be shown, and is a timestamp formatted via FDateTime::ToString */
			TT_Chronological,
		};

		/** Flags controlling how this tag should be shown in the UI */
		enum ETagDisplay
		{
			/** No special display */
			TD_None = 0,
			/** For TT_Chronological, include the date */
			TD_Date = 1<<0,
			/** For TT_Chronological, include the time */
			TD_Time = 1<<1,
			/** For TT_Chronological, specifies that the timestamp should be displayed using the invariant timezone (typically for timestamps that are already in local time) */
			TD_InvariantTz = 1<<2,
			/** For TT_Numerical, specifies that the number is a value in bytes that should be displayed using FText::AsMemory */
			TD_Memory = 1<<3,
		};
		
		/** Logical name of this tag */
		FName Name;

		/** Value string for this tag, may represent any data type */
		FString Value;

		/** Broad description of kind of data represented in Value */
		ETagType Type = TT_Alphabetical;

		/** Flags describing more detail for displaying in the UI */
		uint32 DisplayFlags = TD_None;

		FAssetRegistryTag() = default;
		FAssetRegistryTag(FName InName, const FString& InValue, ETagType InType, uint32 InDisplayFlags = TD_None)
			: Name(InName), Value(InValue), Type(InType), DisplayFlags(InDisplayFlags) {}
		FAssetRegistryTag(FName InName, FString&& InValue, ETagType InType, uint32 InDisplayFlags = TD_None)
			: Name(InName), Value(MoveTemp(InValue)), Type(InType), DisplayFlags(InDisplayFlags) {}

#if WITH_EDITOR
		/** Event for listeners who want to add tags to some UObjects' GetAssetRegistryTags. 
		  * Listeners should be threadsafe as this event may be broadcast on a background thread during concurrent save
		  **/
		DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnGetObjectAssetRegistryTagsWithContext, FAssetRegistryTagsContext);
		COREUOBJECT_API static FOnGetObjectAssetRegistryTagsWithContext OnGetExtraObjectTagsWithContext;

		DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnGetObjectAssetRegistryTags, const UObject* /*Object*/, TArray<FAssetRegistryTag>& /*InOutTags*/);
		UE_DEPRECATED(5.4, "Subscribe to OnGetExtraObjectTagsWithContext instead")
		COREUOBJECT_API static FOnGetObjectAssetRegistryTags OnGetExtraObjectTags;
		DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnGetExtendedAssetRegistryTagsForSave, const UObject* /*Object*/, const ITargetPlatform* TargetPlatform, TArray<FAssetRegistryTag>& /*InOutTags*/);
		UE_DEPRECATED(5.4, "Subscribe to OnGetExtraObjectTagsWithContext instead, and early exit if !Context.IsSaving")
		COREUOBJECT_API static FOnGetExtendedAssetRegistryTagsForSave OnGetExtendedAssetRegistryTagsForSave;
#endif // WITH_EDITOR
	};

	/**
	 * Gathers a list of asset registry searchable tags which are name/value pairs with some type information.
	 * This only needs to be implemented for asset objects.
	 * For editor localization please also implement GetAssetRegistryTagMetadata.
	 * @param	OutTags		A list of key-value pairs associated with this object and their types
	 * @see GetAssetRegistryTagMetadata
	 */
	COREUOBJECT_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	COREUOBJECT_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const;

#if WITH_EDITOR
	/**
	 * Get an additional object that should be added to the AssetDatas stored for a cooked package, to handle
	 * objects stripped from cooked packages. e.g. UBlueprintGeneratedClass returns its UBlueprint.
	 */
	COREUOBJECT_API virtual void GetAdditionalAssetDataObjectsForCook(FArchiveCookContext& CookContext,
		TArray<UObject*>& OutObjects) const;
	/**
	 * Temporary interim solution to gather asset registry data at save time only.  Can depend on the target platform being saved for.
	 *
	 * @param	TargetPlatform	The platform that this object is being saved for.
	 * @param	OutTags			A list of key-value pairs associated with this object and their types
	 */
	COREUOBJECT_API virtual void GetExtendedAssetRegistryTagsForSave(const ITargetPlatform* TargetPlatform, TArray<FAssetRegistryTag>& OutTags) const;
#endif // WITH_EDITOR

	COREUOBJECT_API void GetAssetRegistryTags(FAssetData& Out) const;
	/**
	 * Gathers a list of asset registry tags for an FAssetData. Output data will be removed from the Context and
	 * moved onto the Out FAssetData.
	 */
	COREUOBJECT_API void GetAssetRegistryTags(FAssetRegistryTagsContext Context, FAssetData& Out) const;

	/** Get the common tag name used for all asset source file import paths */
	static COREUOBJECT_API const FName& SourceFileTagName();

#if WITH_EDITOR

	/** Contains an AssetData referencing the asset to be processed and an array of tags and values to be updated on that asset */
	struct FPostLoadAssetRegistryTagsContext
	{
		FPostLoadAssetRegistryTagsContext(const FAssetData& InAssetData, TArray<FAssetRegistryTag>& InTagsAndValuesToUpdate,
			bool bInCanCreateObjects)
		: AssetData(InAssetData), TagsAndValuesToUpdate(InTagsAndValuesToUpdate), bCanCreateObjects(bInCanCreateObjects) { }

		const FAssetData& GetAssetData() const { return AssetData; }
		void AddTagToUpdate(const FAssetRegistryTag& Tag) { TagsAndValuesToUpdate.Add(Tag); }

		bool CanCreateObjects() const { return bCanCreateObjects; }
		bool PostLoadCouldComplete() const { return bPostLoadCouldComplete; }
		void SetPostLoadCouldComplete(bool bValue) { bPostLoadCouldComplete = bValue; }
	private:
		/** The AssetData being considered */
		const FAssetData& AssetData;
		/** The asset registry tags to be updated for this asset data */
		TArray<FAssetRegistryTag>& TagsAndValuesToUpdate;
		bool bCanCreateObjects = true;
		bool bPostLoadCouldComplete = true;
	};

protected:
	/**
	 * Performs fixup on loaded asset registry data.
	 *
	 * DO NOT CALL ASSETREGISTRY FUNCTIONS FROM THIS FUNCTION, IT WILL DEADLOCK because this function is called from inside the AssetRegistry CriticalSection. 
	 * IT MUST BE SAFE TO CALL IN MULTITHREADED BACKGROUND THREADS because the AssetRegistry can call this function from a background thread.
	 * 
	 * Note that this function is only called on Class Default Objects where the actual object instance data used to generate 
	 * the asset data is not available.
	 * @param Context Contains input and receives output from the function. @see FPostloadAssetRegistryTagsContext
	 */
	virtual void ThreadedPostLoadAssetRegistryTagsOverride(FPostLoadAssetRegistryTagsContext& Context) const {}

public:

	/** Internal use function for the AssetRegistry. Performs general fixup required by UObject and invokes ThreadedPostLoadAssetRegistryTagsOverride */
	COREUOBJECT_API void ThreadedPostLoadAssetRegistryTags(FPostLoadAssetRegistryTagsContext& Context) const;

	/**
	 * Additional data pertaining to asset registry tags used by the editor
	 */
	struct FAssetRegistryTagMetadata
	{
		FText DisplayName;
		FText TooltipText;
		FText Suffix;
		FString ImportantValue;

		/** Set override display name */
		FAssetRegistryTagMetadata& SetDisplayName(const FText& InDisplayName)
		{
			DisplayName = InDisplayName;
			return *this;
		}

		/** Set tooltip text pertaining to the asset registry tag in the column view header */
		FAssetRegistryTagMetadata& SetTooltip(const FText& InTooltipText)
		{
			TooltipText = InTooltipText;
			return *this;
		}

		/** Set suffix appended to the tag value */
		FAssetRegistryTagMetadata& SetSuffix(const FText& InSuffix)
		{
			Suffix = InSuffix;
			return *this;
		}

		/** Set value deemed to be 'important' for this registry tag */
		FAssetRegistryTagMetadata& SetImportantValue(const FString& InImportantValue)
		{
			ImportantValue = InImportantValue;
			return *this;
		}
	};

	/** Gathers a collection of asset registry tag metadata */
	COREUOBJECT_API virtual void GetAssetRegistryTagMetadata(TMap<FName, FAssetRegistryTagMetadata>& OutMetadata) const;

	/** The metadata tags to be transferred from the FMetaData to the Asset Registry */
	static COREUOBJECT_API TSet<FName>& GetMetaDataTagsForAssetRegistry();

#endif

	/** Returns true if this object is considered an asset. */
	COREUOBJECT_API virtual bool IsAsset() const;

	/**
	 * Returns an Type:Name pair representing the PrimaryAssetId for this object.
	 * Assets that need to be globally referenced at runtime should return a valid Identifier.
	 * If this is valid, the object can be referenced by identifier using the AssetManager 
	 */
	COREUOBJECT_API virtual FPrimaryAssetId GetPrimaryAssetId() const;

	/** Returns true if this object is considered a localized resource. */

#if !UE_STRIP_DEPRECATED_PROPERTIES 
	UE_DEPRECATED(5.8, "Use FPackageName::IsLocalizedPackage instead")
	COREUOBJECT_API virtual bool IsLocalizedResource() const;
#endif

	/** Returns true if this object is safe to add to the root set. */
	COREUOBJECT_API virtual bool IsSafeForRootSet() const;

	/** 
	 * Tags objects that are part of the same asset with the specified object flag, used for GC checking
	 *
	 * @param	ObjectFlags	Object Flags to enable on the related objects
	 */
	COREUOBJECT_API virtual void TagSubobjects(EObjectFlags NewFlags);

	/** 
	 * Returns the list of replicated properties for the class and their replication configuration.
	 * This function used to require every replicated property of the class to be registered in the array. 
	 * But now you can leave it empty (or call the Super) and the class's replicated properties will still be replicated.
	 * The NetDriver will automatically add all properties that are CPF_Net and not found in the array.
	 * Note that these auto-added properties will use default configs: No conditions, PushModel = off, etc.
	 */
	COREUOBJECT_API virtual void GetLifetimeReplicatedProps( TArray< class FLifetimeProperty > & OutLifetimeProps ) const;

	/** Called when this object begins replicating to initialize the state of custom property conditions */
	COREUOBJECT_API virtual void GetReplicatedCustomConditionState(FCustomPropertyConditionState& OutActiveState) const;
	/**
	 * RegisterReplicationFragments is called when we an object is added to the ReplicationSystem, it allows an object to register new or existing ReplicationFragments describing data to be replicated and how it should be accessed
	 * For more information about ReplicationFragments see ReplicationFragment.h
	 * 
	 * @param Context Context FFragmentRegistrationContext in which FReplicationFragments could be registered
	 * @param RegistrationFlags Flags specifying what should be registered in the call
	 */
	COREUOBJECT_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags);

	/** IsNameStableForNetworking means an object can be referred to its path name (relative to outer) over the network */
	COREUOBJECT_API virtual bool IsNameStableForNetworking() const;

	/** IsFullNameStableForNetworking means an object can be referred to its full path name over the network */
	COREUOBJECT_API virtual bool IsFullNameStableForNetworking() const;

	/** IsSupportedForNetworking means an object can be referenced over the network */
	COREUOBJECT_API virtual bool IsSupportedForNetworking() const;

	/** Returns a list of sub-objects that have stable names for networking */
#if !UE_STRIP_DEPRECATED_PROPERTIES 
	UE_DEPRECATED(5.7, "This function is useless now and will be deleted")
	virtual void GetSubobjectsWithStableNamesForNetworking(TArray<UObject*> &ObjList) {}
#endif

	/** Called right before receiving a bunch */
	COREUOBJECT_API virtual void PreNetReceive();

	/** Called right after receiving a bunch */
	COREUOBJECT_API virtual void PostNetReceive();

	/** Called right after calling all OnRep notifies (called even when there are no notifies) */
	virtual void PostRepNotifies() {}

	/** Called right before being marked for destruction due to network replication */
	COREUOBJECT_API virtual void PreDestroyFromReplication();

#if WITH_EDITOR
	/**
	 * Generic function to validate objects during changelist validations, etc.
	 *
	 * @param	Context	the context holding validation warnings/errors.
	 * @return Valid if this object has data validation rules set up for it and the data for this object is valid. Returns Invalid if it does not pass 
	 *         the rules. Returns NotValidated if no rules are set for this object.
	 */
	COREUOBJECT_API virtual EDataValidationResult IsDataValid(class FDataValidationContext& Context) const;
#endif // WITH_EDITOR

	/*----------------------------------------------------------
		Non virtual functions, not intended to be overridden
	----------------------------------------------------------*/

	/**
	 * Test the selection state of a UObject
	 *
	 * @return		true if the object is selected, false otherwise.
	 * @todo UE this doesn't belong here, but it doesn't belong anywhere else any better
	 */
	COREUOBJECT_API bool IsSelected() const;

#if WITH_EDITOR
	/**
	 * Calls PreEditChange on all instances based on an archetype in AffectedObjects. Recurses on any instances.
	 *
	 * @param	AffectedObjects		the array of objects which have this object in their ObjectArchetype chain and will be affected by the change.
	 *								Objects which have this object as their direct ObjectArchetype are removed from the list once they're processed.
	 */
	COREUOBJECT_API void PropagatePreEditChange( TArray<UObject*>& AffectedObjects, FEditPropertyChain& PropertyAboutToChange );

	/**
	 * Calls PostEditChange on all instances based on an archetype in AffectedObjects. Recurses on any instances.
	 *
	 * @param	AffectedObjects		the array of objects which have this object in their ObjectArchetype chain and will be affected by the change.
	 *								Objects which have this object as their direct ObjectArchetype are removed from the list once they're processed.
	 */
	COREUOBJECT_API void PropagatePostEditChange( TArray<UObject*>& AffectedObjects, FPropertyChangedChainEvent& PropertyChangedEvent );
#endif // WITH_EDITOR

	/**
	 * Serializes the script property data located at Data.  When saving, only saves those properties which differ from the corresponding
	 * value in the specified 'DiffObject' (usually the object's archetype).
	 *
	 * @param	Ar				the archive to use for serialization
	 */
	COREUOBJECT_API void SerializeScriptProperties( FArchive& Ar ) const;

	/**
	 * Serializes the script property data located at Data.  When saving, only saves those properties which differ from the corresponding
	 * value in the specified 'DiffObject' (usually the object's archetype).
	 *
	 * @param	Slot				the archive slot to serialize to
	 */
	COREUOBJECT_API void SerializeScriptProperties( FStructuredArchive::FSlot Slot ) const;

	/**
	 * Wrapper function for InitProperties() which handles safely tearing down this object before re-initializing it
	 * from the specified source object.
	 *
	 * @param	SourceObject	the object to use for initializing property values in this object.  If not specified, uses this object's archetype.
	 * @param	InstanceGraph	contains the mappings of instanced objects and components to their templates
	 */
	COREUOBJECT_API void ReinitializeProperties( UObject* SourceObject=NULL, struct FObjectInstancingGraph* InstanceGraph=NULL );

	/**
	 * This will return detail info about this specific object. (e.g. AudioComponent will return the name of the cue,
	 * ParticleSystemComponent will return the name of the ParticleSystem)  The idea here is that in many places
	 * you have a component of interest but what you really want is some characteristic that you can use to track
	 * down where it came from.  
	 *
	 * @note	safe to call on NULL object pointers!
	 */
	COREUOBJECT_API FString GetDetailedInfo() const;

	/**
	 * Called before destroying the object.  This is called immediately upon deciding to destroy the object, to allow the object to begin an
	 * asynchronous cleanup process.
	 */
	COREUOBJECT_API bool ConditionalBeginDestroy();

	/** Called when an object is actually destroyed, memory should never be accessed again */
	COREUOBJECT_API bool ConditionalFinishDestroy();
	
	/** PostLoad if needed. */
	COREUOBJECT_API void ConditionalPostLoad();

	/** Preload if needed. Mostly used in the loader or loader threads. */
	COREUOBJECT_API void ConditionalPreload();

	/**
	 * Instances subobjects and components for objects being loaded from disk, if necessary.  Ensures that references
	 * between nested components are fixed up correctly.
	 *
	 * @param	OuterInstanceGraph	when calling this method on subobjects, specifies the instancing graph which contains all instanced
	 *								subobjects and components for a subobject root.
	 */
	COREUOBJECT_API void ConditionalPostLoadSubobjects( struct FObjectInstancingGraph* OuterInstanceGraph=NULL );

#if WITH_EDITOR
	/**
	 * Starts caching of platform specific data for the target platform
	 * Called when cooking before serialization so that object can prepare platform specific data
	 * Not called during normal loading of objects
	 * 
	 * @param	TargetPlatform	target platform to cache platform specific data for
	 */
	virtual void BeginCacheForCookedPlatformData( const ITargetPlatform* TargetPlatform ) {  }
	
	/**
	 * Have we finished loading all the cooked platform data for the target platforms requested in BeginCacheForCookedPlatformData
	 * 
	 * @param	TargetPlatform target platform to check for cooked platform data
	 */
	virtual bool IsCachedCookedPlatformDataLoaded( const ITargetPlatform* TargetPlatform ) { return true; }

	/**
	 * All caching has finished for this object (all IsCachedCookedPlatformDataLoaded functions have finished for all platforms)
	 */
	virtual void WillNeverCacheCookedPlatformDataAgain() { }

	/**
	 * Clears cached cooked platform data for specific platform
	 * 
	 * @param	TargetPlatform	target platform to cache platform specific data for
	 */
	virtual void ClearCachedCookedPlatformData( const ITargetPlatform* TargetPlatform ) {  }

	/**
	 * Clear all cached cooked platform data
	 * 
	 * @param	TargetPlatform	target platform to cache platform specific data for
	 */
	virtual void ClearAllCachedCookedPlatformData() { }

	/**
	 * Called during cook to allow objects to generate additional cooked files alongside their cooked package.
	 * @note Implement CookAdditionalFilesOverride to define sub class behavior.
	 *
	 * @param	PackageFilename full path to the package that this object is being saved to on disk
	 * @param	TargetPlatform	target platform to cook additional files for
	 * @param	WriteAdditionalFile function for writing the additional files
	 */
	void CookAdditionalFiles(const TCHAR* PackageFilename, const ITargetPlatform* TargetPlatform,
		TFunctionRef<void(const TCHAR* Filename, void* Data, int64 Size)> WriteAdditionalFile)
	{
		CookAdditionalFilesOverride(PackageFilename, TargetPlatform, WriteAdditionalFile);
	}

private:
	/**
	 * Called during cook to allow objects to generate additional cooked files alongside their cooked package.
	 * Files written using the provided function will be handled as part of the saved cooked package
	 * and contribute to package total file size, and package hash when enabled.
	 * @note These should typically match the name of the package, but with a different extension.
	 *
	 * @param	PackageFilename full path to the package that this object is being saved to on disk
	 * @param	TargetPlatform	target platform to cook additional files for
	 * @param	WriteAdditionalFile function for writing the additional files
	 */
	virtual void CookAdditionalFilesOverride(const TCHAR* PackageFilename, const ITargetPlatform* TargetPlatform,
		TFunctionRef<void(const TCHAR* Filename, void* Data, int64 Size)> WriteAdditionalFile)
	{
	}

public:
#endif
	/**
	 * Determine if this object has SomeObject in its archetype chain.
	 */
	inline bool IsBasedOnArchetype( const UObject* const SomeObject ) const;

	/** Returns a UFunction with the specified name, wrapper for UClass::FindFunctionByName() */
	COREUOBJECT_API UFunction* FindFunction( FName InName ) const;
	
	/** Version of FindFunction() that will assert if the function was not found */
	COREUOBJECT_API UFunction* FindFunctionChecked( FName InName ) const;

	/**
	 * Given OtherObject (which will be the same type as 'this'), recursively find any matching sub-objects from 'this' that also exist within OtherObject, and add the mappings to ObjectMapping.
	 *
	 * @param	OtherObject		The to find matching sub-objects within.
	 * @param	ObjectMapping	The complete mapping between this object hierarchy and the other object hierarchy.
	 */
	COREUOBJECT_API virtual void BuildSubobjectMapping(UObject* OtherObject, TMap<UObject*, UObject*>& ObjectMapping) const;

	/**
	 * Checks default sub-object assumptions.
	 *
	 * @param bForceCheck Force checks even if not enabled globally.
	 * @return true if the assumptions are met, false otherwise.
	 */
	COREUOBJECT_API bool CheckDefaultSubobjects(bool bForceCheck = false) const;

	/**
	 * Save configuration out to ini files
	 * @warning Must be safe to call on class-default object
	 */
	COREUOBJECT_API void SaveConfig(uint64 RequiredPropertyFlags = CPF_Config, const TCHAR* Filename = nullptr,
		FConfigCacheIni* Config=GConfig, bool bAllowCopyToDefaultObject = true);

	struct FSaveConfigContext
	{
		EPropertyFlags RequiredPropertyFlags = CPF_Config;
		FString Filename;
		FConfigCacheIni* ConfigSystem = GConfig;
		bool bAllowCopyToDefaultObject = true;

		bool bSaveToLayer = false;
		TSet<FName> PropertyNames = {};
		FString OverrideConfigSectionName;
	};
	COREUOBJECT_API void SaveConfig(const FSaveConfigContext& Context);

	/**
	 * Try to Saves just the section(s) for this class into the default ini file for the class (with just the changes from base)
	 *
	 *  @param SpecificFileLocation The Ini file or if Empty the default ini file for the call
	 *  @param bWarnIfFail If true and unable to update the Ini file due to being read only log a warning
	 *  @return true if it was able to write false otherwise
	 */
	COREUOBJECT_API bool TryUpdateDefaultConfigFile(const FString& SpecificFileLocation = "", bool bWarnIfFail = true);

	/**
	 * Saves just the section(s) for this class into the global user ini file for the class (with just the changes from base)
	 */
	COREUOBJECT_API void UpdateGlobalUserConfigFile();

	/**
	 * Saves just the section(s) for this class into the project user ini file for the class (with just the changes from base)
	 */
	COREUOBJECT_API void UpdateProjectUserConfigFile();

	/**
	 * Saves just the property into the global user ini file for the class (with just the changes from base)
	 */
	COREUOBJECT_API void UpdateSinglePropertyInConfigFile(const FProperty* InProperty, const FString& InConfigIniName);

private:
	/**
	 * Saves just the section(s) for this class into the given ini file for the class (with just the changes from base)
	 */
	COREUOBJECT_API void UpdateSingleSectionOfConfigFile(const FString& ConfigIniName);

	/**
	 * Ensures that current thread is NOT during vtable ptr retrieval process
	 * of some UClass.
	 */
	COREUOBJECT_API void EnsureNotRetrievingVTablePtr() const;

public:
	/**
	 * Get the default config filename for the specified UObject
	 */
	COREUOBJECT_API FString GetDefaultConfigFilename() const;

	/**
	 * Get the global user override config filename for the specified UObject
	 */
	COREUOBJECT_API FString GetGlobalUserConfigFilename() const;

	/**
	 * Get the project user override config filename for the specified UObject
	 */
	COREUOBJECT_API FString GetProjectUserConfigFilename() const;

#if WITH_EDITOR
	/** 
	 * Delegate to return the current preview platform name, or NAME_None if no preview platform is currently active.
	 * Used by LoadConfig for CLASS_PerPlatformConfig objects.
	 */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnGetPreviewPlatform, FName&);
	COREUOBJECT_API static FOnGetPreviewPlatform OnGetPreviewPlatform;
#endif

	/** 
	 * Returns the override config hierarchy platform (if NDAd platforms need defaults to not be in Base*.ini but still want editor to load them) 
	 */
	virtual const TCHAR* GetConfigOverridePlatform() const { return nullptr; }

	/**
	 * Allows Non-PerObjectConfig classes, to override the ini section name used for loading config settings
	 *
	 * @param SectionName	Reference to the unmodified config section name, that can be altered/modified
	 */
	virtual void OverrideConfigSection(FString& SectionName) {}

	/**
	 * Allows PerObjectConfig classes, to override the ini section name used for the PerObjectConfig object.
	 *
	 * @param SectionName	Reference to the unmodified config section name, that can be altered/modified
	 */
	virtual void OverridePerObjectConfigSection(FString& SectionName) {}

	/**
	 * Imports property values from an .ini file.
	 *
	 * @param	Class				the class to use for determining which section of the ini to retrieve text values from
	 * @param	Filename			indicates the filename to load values from; if not specified, uses ConfigClass's ClassConfigName
	 * @param	PropagationFlags	indicates how this call to LoadConfig should be propagated; expects a bitmask of UE::ELoadConfigPropagationFlags values.
	 * @param	PropertyToLoad		if specified, only the ini value for the specified property will be imported.
	 * @param 	OutAccessedKeys     if specified the object is not modified and (EditorOnly) all the config keys it
	 *                              would read are added to OutAccessedKeys. In non-editor, function returns with no action.
	 */
	COREUOBJECT_API void LoadConfig(UClass* ConfigClass = nullptr, const TCHAR* Filename = nullptr,
		uint32 PropagationFlags = UE::LCPF_None, class FProperty* PropertyToLoad = nullptr,
		TArray<UE::ConfigAccessTracking::FConfigAccessData>* OutAccessedKeys = nullptr);

	struct FLoadConfigParams 
	{
		FLoadConfigParams() = default;
				
		// the class to look in the config system for (defaults to Obj's Class if null)
		UClass* ConfigClass = nullptr;
		// the config system to read from (defaults to GConfig if null)
		FConfigCacheIni* ConfigSystem = nullptr;
		
		// an optional filename to use to look up in the ConfigSystem
		FString Filename;
		// flags to control if we want to propagate LoadConfig to instances of the class, child classes, etc. see ELoadConfigPropagationFlags
		uint32 PropagationFlags = UE::LCPF_None;
		// if this is set, only a single property of the class will be loaded from config
		class FProperty* PropertyToLoad = nullptr;
		// if this is set, we only track what sections/keys are read from the config system, without modifying the Object at all
		TArray<UE::ConfigAccessTracking::FConfigAccessData>* AccessedValues = nullptr;
		// if set, this values will be read from this COnfigFile instead of looking up in ConfigSystem
		FConfigFile* OverrideFile = nullptr;
	};

	COREUOBJECT_API void LoadConfig(const FLoadConfigParams& Context);
	
	/**
	 * Wrapper method for LoadConfig that is used when reloading the config data for objects at runtime which have already loaded their config data at least once.
	 * Allows the objects the receive a callback that its configuration data has been reloaded.
	 *
	 * @param	Class				the class to use for determining which section of the ini to retrieve text values from
	 * @param	Filename			indicates the filename to load values from; if not specified, uses ConfigClass's ClassConfigName
	 * @param	PropagationFlags	indicates how this call to LoadConfig should be propagated; expects a bitmask of UE::ELoadConfigPropagationFlags values.
	 * @param	PropertyToLoad		if specified, only the ini value for the specified property will be imported
	 */
	COREUOBJECT_API void ReloadConfig( UClass* ConfigClass=NULL, const TCHAR* Filename=NULL, uint32 PropagationFlags=UE::LCPF_None, class FProperty* PropertyToLoad=NULL );

	/** Import an object from a file. */
	COREUOBJECT_API void ParseParms( const TCHAR* Parms );

	/**
	 * Outputs a string to an arbitrary output device, describing the list of objects which are holding references to this one.
	 *
	 * @param	Ar						the output device to send output to
	 * @param	Referencers				optionally allows the caller to specify the list of references to output.
	 */
	COREUOBJECT_API void OutputReferencers( FOutputDevice& Ar, FReferencerInformationList* Referencers=NULL );
	
	/** Called by OutputReferencers() to get the internal list of referencers to write */
	COREUOBJECT_API void RetrieveReferencers( TArray<FReferencerInformation>* OutInternalReferencers, TArray<FReferencerInformation>* OutExternalReferencers);

	/**
	 * Changes the linker and linker index to the passed in one. A linker of NULL and linker index of INDEX_NONE
	 * indicates that the object is without a linker.
	 *
	 * @param LinkerLoad				New LinkerLoad object to set
	 * @param LinkerIndex				New LinkerIndex to set
	 * @param bShouldDetachExisting		If true, detach existing linker and call PostLinkerChange
	 * @param bSilenceLogs				If true, this function will not output non-fatal log messages
	 */
	COREUOBJECT_API void SetLinker( FLinkerLoad* LinkerLoad, int32 LinkerIndex, bool bShouldDetachExisting = true, bool bSilenceLogs = false);

	/**
	 * Return the template that an object with this class, outer and name would be
	 * 
	 * @return the archetype for this object
	 */
	static COREUOBJECT_API UObject* GetArchetypeFromRequiredInfo(const UClass* Class, const UObject* Outer, FName Name, EObjectFlags ObjectFlags);

	/**
	 * Return the template this object is based on. 
	 * 
	 * @return the archetype for this object
	 */
	COREUOBJECT_API UObject* GetArchetype() const;

	/**
	 * Builds a list of objects which have this object in their archetype chain.
	 *
	 * @param	Instances	receives the list of objects which have this one in their archetype chain
	 */
	COREUOBJECT_API void GetArchetypeInstances( TArray<UObject*>& Instances );

	/**
	 * Wrapper for calling UClass::InstanceSubobjectTemplates() for this object.
	 */
	COREUOBJECT_API void InstanceSubobjectTemplates( struct FObjectInstancingGraph* InstanceGraph = NULL );

	/**
	 * Returns true if this object implements the interface T, false otherwise.
	 */
	template<class T>
	bool Implements() const;


	/*-----------------------------
			Virtual Machine
	-----------------------------*/

	/** Called by VM to execute a UFunction with a filled in UStruct of parameters */
	COREUOBJECT_API virtual void ProcessEvent( UFunction* Function, void* Parms );

	/**
	 * Return the space this function should be called.   Checks to see if this function should
	 * be called locally, remotely, or simply absorbed under the given conditions
	 *
	 * @param Function function to call
	 * @param Stack stack frame for the function call
	 * @return bitmask representing all callspaces that apply to this UFunction in the given context
	 */
	virtual int32 GetFunctionCallspace( UFunction* Function, FFrame* Stack )
	{
		return FunctionCallspace::Local;
	}

	/**
	 * Call the actor's function remotely
	 *
	 * @param Function function to call
	 * @param Parms arguments to the function call
	 * @param OutParms out parameter information (irrelevant for RPC's)
	 * @param Stack stack frame for the function call
	 * @return returns true if the function was processed by a NetDriver
	 */
	virtual bool CallRemoteFunction( UFunction* Function, void* Parms, struct FOutParmRec* OutParms, FFrame* Stack )
	{
		return false;
	}

	/** Handle calling a function by name when executed from the console or a command line */
	COREUOBJECT_API bool CallFunctionByNameWithArguments( const TCHAR* Cmd, FOutputDevice& Ar, UObject* Executor, bool bForceCallWithNonExec = false );

	/** Internal VM method for executing a function */
	COREUOBJECT_API void CallFunction( FFrame& Stack, RESULT_DECL, UFunction* Function );

	/**
	 * Internal function call processing.
	 * @warning: might not write anything to Result if proper type isn't returned.
	 */
	DECLARE_FUNCTION(ProcessInternal);

	/**
	 * This function handles a console exec sent to the object; it is virtual so 'nexus' objects like
	 * a player controller can reroute the command to several different objects.
	 */
	virtual bool ProcessConsoleExec(const TCHAR* Cmd, FOutputDevice& Ar, UObject* Executor)
	{
		return CallFunctionByNameWithArguments(Cmd, Ar, Executor);
	}

	/** advances Stack's code past the parameters to the given Function and if the function has a return value, copies the zero value for that property to the memory for the return value
	 * @param Stack the script stack frame
	 * @param Result pointer to where the return value should be written
	 * @param Function the function being called
	 */
	COREUOBJECT_API void SkipFunction(FFrame& Stack, RESULT_DECL, UFunction* Function);

	/**
	 * Called on the target when a class is loaded with ClassGeneratedBy is loaded.  Should regenerate the class if needed, and return the updated class
	 * @return	Updated instance of the class, if needed, or NULL if no regeneration was necessary
	 */
#if WITH_EDITOR
	virtual UClass* RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO) { return nullptr; };
#else
#if !UE_STRIP_DEPRECATED_PROPERTIES 
	UE_DEPRECATED(5.8, "This function is now editor only")
	virtual UClass* RegenerateClass(UClass* ClassToRegenerate, UObject* PreviousCDO) { return nullptr; };
#endif
#endif
	/** 
	 * Returns whether this object is contained in or part of a blueprint object
	 */
	COREUOBJECT_API bool IsInBlueprint() const;

	/** 
	 *  Destroy properties that won't be destroyed by the native destructor
	 */
	COREUOBJECT_API void DestroyNonNativeProperties();

	/** Called during subobject creation to mark this component as editor only, which causes it to get stripped in packaged builds */
#if WITH_EDITORONLY_DATA
	virtual void MarkAsEditorOnlySubobject() { }
#else
#if !UE_STRIP_DEPRECATED_PROPERTIES 
	UE_DEPRECATED(5.8, "This function is now editor only")
	virtual void MarkAsEditorOnlySubobject() { }
#endif
#endif

	/**
	 * Abort with a member function call at the top of the callstack, helping to ensure that most platforms will stuff this object's memory into the resulting minidump.
	 */
	COREUOBJECT_API void AbortInsideMemberFunction() const;

	// UnrealScript intrinsics, do not call directly

	// Undefined native handler
	DECLARE_FUNCTION(execUndefined);

	// Variables
	DECLARE_FUNCTION(execLocalVariable);
	DECLARE_FUNCTION(execInstanceVariable);
	DECLARE_FUNCTION(execDefaultVariable);
	DECLARE_FUNCTION(execLocalOutVariable);
	DECLARE_FUNCTION(execInterfaceVariable);
	DECLARE_FUNCTION(execClassSparseDataVariable);
	DECLARE_FUNCTION(execInterfaceContext);
	DECLARE_FUNCTION(execArrayElement);
	DECLARE_FUNCTION(execBoolVariable);
	DECLARE_FUNCTION(execClassDefaultVariable);
	DECLARE_FUNCTION(execEndFunctionParms);

	// Do Nothing 
	DECLARE_FUNCTION(execNothing);
	DECLARE_FUNCTION(execNothingInt32);
	DECLARE_FUNCTION(execNothingOp4a);

	/** Breakpoint; only observed in the editor; executing it at any other time is a NOP */
	DECLARE_FUNCTION(execBreakpoint);

	/** Tracepoint; only observed in the editor; executing it at any other time is a NOP */
	DECLARE_FUNCTION(execTracepoint);
	DECLARE_FUNCTION(execWireTracepoint);

	/** Instrumentation event for profiling; only observed in the builds with blueprint instrumentation */
	DECLARE_FUNCTION(execInstrumentation);

	DECLARE_FUNCTION(execEndOfScript);

	/** failsafe for functions that return a value - returns the zero value for a property and logs that control reached the end of a non-void function */
	DECLARE_FUNCTION(execReturnNothing);
	DECLARE_FUNCTION(execEmptyParmValue);

	// Commands
	DECLARE_FUNCTION(execJump);
	DECLARE_FUNCTION(execJumpIfNot);
	DECLARE_FUNCTION(execAssert);

	/** 
	 * Push a code offset onto the execution flow stack for future execution.  
	 * Current execution continues to the next instruction after the push one.
	 */
	DECLARE_FUNCTION(execPushExecutionFlow);

	/**
	 * Pops a code offset from the execution flow stack and starts execution there.
	 * If there are no stack entries left, it is treated as an execution error.
	 */
	DECLARE_FUNCTION(execPopExecutionFlow);
	DECLARE_FUNCTION(execComputedJump);

	/** 
	 * Pops a code offset from the execution flow stack and starts execution there, if a condition is not true.
	 * If there are no stack entries left, it is treated as an execution error.
	 */
	DECLARE_FUNCTION(execPopExecutionFlowIfNot);


	// Assignment
	DECLARE_FUNCTION(execLet);
	DECLARE_FUNCTION(execLetObj);
	DECLARE_FUNCTION(execLetWeakObjPtr);
	DECLARE_FUNCTION(execLetBool);
	DECLARE_FUNCTION(execLetDelegate);
	DECLARE_FUNCTION(execLetMulticastDelegate);

	// Delegates 
	DECLARE_FUNCTION(execAddMulticastDelegate);
	DECLARE_FUNCTION(execClearMulticastDelegate);
	DECLARE_FUNCTION(execEatReturnValue);
	DECLARE_FUNCTION(execRemoveMulticastDelegate);

	// Context expressions
	DECLARE_FUNCTION(execSelf);
	DECLARE_FUNCTION(execContext);
	DECLARE_FUNCTION(execContext_FailSilent);
	DECLARE_FUNCTION(execStructMemberContext);

	// Function calls
	DECLARE_FUNCTION(execVirtualFunction);
	DECLARE_FUNCTION(execFinalFunction);
	DECLARE_FUNCTION(execLocalVirtualFunction);
	DECLARE_FUNCTION(execLocalFinalFunction);

	// Struct comparison
	DECLARE_FUNCTION(execStructCmpEq);
	DECLARE_FUNCTION(execStructCmpNe);
	DECLARE_FUNCTION(execStructMember);

	DECLARE_FUNCTION(execEqualEqual_DelegateDelegate);
	DECLARE_FUNCTION(execNotEqual_DelegateDelegate);
	DECLARE_FUNCTION(execEqualEqual_DelegateFunction);
	DECLARE_FUNCTION(execNotEqual_DelegateFunction);

	// Constants
	DECLARE_FUNCTION(execIntConst);
	DECLARE_FUNCTION(execInt64Const);
	DECLARE_FUNCTION(execUInt64Const);
	DECLARE_FUNCTION(execSkipOffsetConst);
	DECLARE_FUNCTION(execFloatConst);
	DECLARE_FUNCTION(execDoubleConst);
	DECLARE_FUNCTION(execStringConst);
	DECLARE_FUNCTION(execUnicodeStringConst);
	DECLARE_FUNCTION(execTextConst);
	DECLARE_FUNCTION(execPropertyConst);
	DECLARE_FUNCTION(execObjectConst);
	DECLARE_FUNCTION(execSoftObjectConst);
	DECLARE_FUNCTION(execFieldPathConst);

	DECLARE_FUNCTION(execInstanceDelegate);
	DECLARE_FUNCTION(execNameConst);
	DECLARE_FUNCTION(execByteConst);
	DECLARE_FUNCTION(execIntZero);
	DECLARE_FUNCTION(execIntOne);
	DECLARE_FUNCTION(execTrue);
	DECLARE_FUNCTION(execFalse);
	DECLARE_FUNCTION(execNoObject);
	DECLARE_FUNCTION(execNullInterface);
	DECLARE_FUNCTION(execIntConstByte);
	DECLARE_FUNCTION(execRotationConst);
	DECLARE_FUNCTION(execVectorConst);
	DECLARE_FUNCTION(execVector3fConst);
	DECLARE_FUNCTION(execTransformConst);
	DECLARE_FUNCTION(execStructConst);
	DECLARE_FUNCTION(execSetArray);
	DECLARE_FUNCTION(execSetSet);
	DECLARE_FUNCTION(execSetMap);
	DECLARE_FUNCTION(execArrayConst);
	DECLARE_FUNCTION(execSetConst);
	DECLARE_FUNCTION(execMapConst);
	DECLARE_FUNCTION(execBitFieldConst);

	// Object construction
	DECLARE_FUNCTION(execNew);
	DECLARE_FUNCTION(execClassContext);
	DECLARE_FUNCTION(execNativeParm);

	// Conversions 
	DECLARE_FUNCTION(execCast);
	DECLARE_FUNCTION(execDynamicCast);
	DECLARE_FUNCTION(execMetaCast);
	DECLARE_FUNCTION(execInterfaceCast);
	DECLARE_FUNCTION(execDoubleToFloatCast);
	DECLARE_FUNCTION(execFloatToDoubleCast);
	DECLARE_FUNCTION(execObjectToBool);
	DECLARE_FUNCTION(execInterfaceToBool);
	DECLARE_FUNCTION(execObjectToInterface);
	DECLARE_FUNCTION(execInterfaceToInterface);
	DECLARE_FUNCTION(execInterfaceToObject);

	// Dynamic array functions
	// Array support
	DECLARE_FUNCTION(execGetDynArrayElement);
	DECLARE_FUNCTION(execSetDynArrayElement);
	DECLARE_FUNCTION(execGetDynArrayLength);
	DECLARE_FUNCTION(execSetDynArrayLength);
	DECLARE_FUNCTION(execDynArrayInsert);
	DECLARE_FUNCTION(execDynArrayRemove);
	DECLARE_FUNCTION(execDynArrayFind);
	DECLARE_FUNCTION(execDynArrayFindStruct);
	DECLARE_FUNCTION(execDynArrayAdd);
	DECLARE_FUNCTION(execDynArrayAddItem);
	DECLARE_FUNCTION(execDynArrayInsertItem);
	DECLARE_FUNCTION(execDynArrayRemoveItem);
	DECLARE_FUNCTION(execDynArraySort);

	DECLARE_FUNCTION(execBindDelegate);
	DECLARE_FUNCTION(execCallMulticastDelegate);

	DECLARE_FUNCTION(execLetValueOnPersistentFrame);

	DECLARE_FUNCTION(execCallMathFunction);

	DECLARE_FUNCTION(execSwitchValue);

	DECLARE_FUNCTION(execArrayGetByRef);

	DECLARE_FUNCTION(execAutoRtfmTransact);
	DECLARE_FUNCTION(execAutoRtfmStopTransact);
	DECLARE_FUNCTION(execAutoRtfmAbortIfNot);
	DECLARE_FUNCTION(execAutoRtfmAbort);


	/**
	 * Executes some portion of the ubergraph.
	 *
	 * @param	EntryPoint	The entry point to start code execution at.
	 */
	UFUNCTION(BlueprintImplementableEvent, meta=(BlueprintInternalUseOnly = "true"))
	void ExecuteUbergraph(int32 EntryPoint);

protected: 
	/** Checks it's ok to perform subobjects check at this time. */
	COREUOBJECT_API bool CanCheckDefaultSubObjects(bool bForceCheck, bool& bResult) const;

	/**
	* Checks default sub-object assumptions.
	*
	* @return true if the assumptions are met, false otherwise.
	*/
	COREUOBJECT_API virtual bool CheckDefaultSubobjectsInternal() const;

private:
	COREUOBJECT_API void ProcessContextOpcode(FFrame& Stack, RESULT_DECL, bool bCanFailSilent);

	/**
	* Create a component or subobject only to be used with the editor.
	* @param	Outer						outer to construct the subobject in
	* @param	ReturnType					type of the new component
	* @param	SubobjectName				name of the new component
	* @param	bTransient					true if the component is being assigned to a transient property
	*/
	COREUOBJECT_API UObject* CreateEditorOnlyDefaultSubobjectImpl(FName SubobjectName, UClass* ReturnType, bool bTransient = false);

public:

	enum class ENetFields_Private
	{
		NETFIELD_REP_START = 0,
		NETFIELD_REP_END = -1
	};

#if VALIDATE_CLASS_REPS
	virtual void ValidateGeneratedRepEnums(const TArray<struct FRepRecord>& ClassReps) const {}
#elif !UE_STRIP_DEPRECATED_PROPERTIES 
	UE_DEPRECATED(5.8, "VALIDATE_CLASS_REPS is not defined")
	virtual void ValidateGeneratedRepEnums(const TArray<struct FRepRecord>& ClassReps) const {}
#endif

private:
	
	friend struct FObjectNetPushIdHelper;
	virtual void SetNetPushIdDynamic(const uint64 NewNetPushId)
	{
		// This method should only be called on Objects that are networked, and those should
		// always have this implemented (by UHT).
		checkf(false, TEXT("REPLICATED_BASE_CLASS macro is not implemented in %s. Something went very wrong."), *GetFullName());
	}

	// Here for UHT to be happy
	friend COREUOBJECT_API UClass* Z_Construct_UClass_UObject(ETypeConstructPhase);

public:

	/** Should only ever be used by internal systems. */
	virtual uint64 GetNetPushIdDynamic() const
	{
		return uint64(int64(INDEX_NONE));
	}
};

struct FObjectNetPushIdHelper
{
private:
	friend struct FNetPrivatePushIdHelper;
	friend struct FNetObjectManagerPushIdHelper;
	friend struct UE::Net::Private::FNetHandleLegacyPushModelHelper;

	static void SetNetPushIdDynamic(UObject* Object, const uint64 NewNetPushId)
	{
		Object->SetNetPushIdDynamic(NewNetPushId);
	}
};

struct FInternalUObjectBaseUtilityIsValidFlagsChecker
{
	inline static bool CheckObjectValidBasedOnItsFlags(const UObject* Test)
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(Test->InternalIndex);
		// Here we don't really check if the flags match but if the end result is the same
		checkSlow(ObjectItem->HasAnyFlags(EInternalObjectFlags::Garbage) == Test->HasAnyFlags(RF_MirroredGarbage));
		return !ObjectItem->HasAnyFlags(EInternalObjectFlags::Garbage | EInternalObjectFlags::Unreachable);
	}
};

#if WITH_EDITORONLY_DATA
struct FAppendToClassSchemaContext
{
public:
	explicit FAppendToClassSchemaContext(void* InHasher) // Type is void* to mask the implementation detail
		:Hasher(InHasher)
	{
	}
	COREUOBJECT_API void Update(const void* Data, uint64 Size);

private:
	void* Hasher; // Type is void* to mask the implementation detail
};
#endif

/**
* Test validity of object
*
* @param	Test			The object to test
* @return	Return true if the object is usable: non-null, reachable and not pending kill or garbage
*/
UE_FORCEINLINE_HINT bool IsValid(const UObject *Test)
{
	return Test && FInternalUObjectBaseUtilityIsValidFlagsChecker::CheckObjectValidBasedOnItsFlags(Test);
}

/**
* Test validity of object similar to IsValid(Test) however the null pointer test is skipped
*
* @param	Test			The object to test
* @return	Return true if the object is usable: reachable and not pending kill or garbage
*/
inline bool IsValidChecked(const UObject* Test)
{
	check(Test);
	return FInternalUObjectBaseUtilityIsValidFlagsChecker::CheckObjectValidBasedOnItsFlags(Test);
}

/**
* Returns a pointer to a valid object if the Test object passes IsValid() tests, otherwise null
*
* @param	Test			The object to test
* @return	Pointer to a valid object if the Test object passes IsValid() tests, otherwise null
*/
template <typename T>
T* GetValid(T* Test)
{
	static_assert(std::is_base_of<UObject, T>::value, "GetValid can only work with UObject-derived classes");
	return IsValid(Test) ? Test : nullptr;
}

/**
* Returns a pointer to a valid object if the Test object passes IsValid() tests, otherwise null
*
* @param	Test			The object to test
* @return	Pointer to a valid object if the Test object passes IsValid() tests, otherwise null
*/
template <typename T>
const T* GetValid(const T* Test)
{
	static_assert(std::is_base_of<UObject, T>::value, "GetValid can only work with UObject-derived classes");
	return IsValid(Test) ? Test : nullptr;
}

#if WITH_EDITOR
/** Callback for editor object selection. This must be in core instead of editor for UObject::IsSelectedInEditor to work */
extern COREUOBJECT_API TFunction<bool(const UObject*)> GIsObjectSelectedInEditor;
#endif
