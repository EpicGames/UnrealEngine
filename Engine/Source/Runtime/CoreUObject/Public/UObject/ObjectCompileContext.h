// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
struct FObjectPostCDOCompiledContext
{
	/** True if the Blueprint is currently being regenerated for the first time after being loaded (aka, compile-on-load) */
	bool bIsRegeneratingOnLoad = false;
	/** True if this notification was from a 'skeleton-only' compile */
	bool bIsSkeletonOnly = false;
};
#endif
