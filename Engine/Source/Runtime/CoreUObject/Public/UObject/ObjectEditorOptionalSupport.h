// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/NotNull.h"

#if WITH_EDITORONLY_DATA

class FArchive;
class UObject;
class UClass;
struct FGuid;

namespace UE::EditorOptional
{

	COREUOBJECT_API void ConditionalUpgradeObject(FArchive& Ar, TNotNull<UObject*> SecondaryObject, const FGuid& VersionGuid, int Version);

	COREUOBJECT_API void UpgradeObject(FArchive& Ar, TNotNull<UObject*> SecondaryObject);

	COREUOBJECT_API UObject* CreateEditorOptionalObject(TNotNull<UObject*> MainObject, TNotNull<const UClass*> EditorOptionalClass, const TCHAR* OverrideName = nullptr);

	template<typename EditorOptionalClass>
	EditorOptionalClass* CreateEditorOptionalObject(TNotNull<UObject*> MainObject)
	{
		// use the non-templated version of this function, and typecast the result (we don't need CastChecked since we know it's the right class)
		return Cast<EditorOptionalClass>(CreateEditorOptionalObject(MainObject, EditorOptionalClass::StaticClass()));
	}
}

#endif
