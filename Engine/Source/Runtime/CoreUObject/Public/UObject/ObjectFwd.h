// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Misc/EnumClassFlags.h"

/**
* Enum which specifies the mode in which full object names are constructed
*/
enum class EObjectFullNameFlags
{
	// Standard object full name (i.e. "Type PackageName.ObjectName:SubobjectName")
	None = 0,

	// Adds package to the type portion (i.e. "TypePackage.TypeName PackageName.ObjectName:SubobjectName")
	IncludeClassPackage = 1,
};

ENUM_CLASS_FLAGS(EObjectFullNameFlags);

namespace UE::GC
{
/**
* Marks the object as Reachable if it's currently marked as MaybeUnreachable by incremental GC.
*/
COREUOBJECT_API void MarkAsReachable(const UObject* Obj);
}

namespace UE::CoreUObject::Private
{
COREUOBJECT_API UPackage* GetPackage(const UObject* Obj);

COREUOBJECT_API FString GetFullName(const UObject* Obj,
																		const UObject* StopOuter=nullptr,
																		EObjectFullNameFlags Flags = EObjectFullNameFlags::None);

COREUOBJECT_API FString GetPathName( const UObject* Obj, const UObject* StopOuter=nullptr );

COREUOBJECT_API UClass* GetClass(UObject* Obj);
COREUOBJECT_API FName GetFName(const UObject* Obj);
COREUOBJECT_API UObject* GetOuter(const UObject* Obj);
COREUOBJECT_API bool IsIn(const UObject* Obj, const UObject* SomeOuter);

COREUOBJECT_API bool HasAnyFlags(const UObject* Obj, int32 FlagsToCheck);
}
