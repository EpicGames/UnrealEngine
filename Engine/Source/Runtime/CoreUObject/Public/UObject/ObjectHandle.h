// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/ScriptArray.h"
#include "CoreTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "Misc/UEOps.h"
#include "Templates/TypeHash.h"
#include "Templates/TypeCompatibleBytes.h"
#include "UObject/CompiledInObjectPtr.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectHandleTracking.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectRef.h"
#include "UObject/PackedObjectRef.h"
#include "UObject/RemoteObject.h"

class UClass;
class UPackage;

/**
 * FObjectHandle is either a packed object ref or the resolved pointer to an object.  Depending on configuration
 * when you create a handle, it may immediately be resolved to a pointer.
 */
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || (UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC)

namespace UE::CoreUObject::Private
{
	struct FObjectHandlePrivate;
}

using FObjectHandle = UE::CoreUObject::Private::FObjectHandlePrivate;
#define UE_OBJECT_HANDLE_IS_OBJECT_PTR 0

#elif UE_WITH_REMOTE_OBJECT_HANDLE

namespace UE::CoreUObject::Private
{
	struct FRemoteObjectHandlePrivate;
}

using FObjectHandle = UE::CoreUObject::Private::FRemoteObjectHandlePrivate;
#define UE_OBJECT_HANDLE_IS_OBJECT_PTR 0

#else

using FObjectHandle = UObject*;
//NOTE: operator==, operator!=, GetTypeHash fall back to the default on UObject* or void* through coercion.
#define UE_OBJECT_HANDLE_IS_OBJECT_PTR 1

#endif

inline bool IsObjectHandleNull(FObjectHandle Handle);
inline bool IsObjectHandleResolved(FObjectHandle Handle);
inline bool IsObjectHandleTypeSafe(FObjectHandle Handle);

//Private functions that forced public due to inlining.
namespace UE::CoreUObject::Private
{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || (UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC)

	struct FObjectHandlePrivate
	{
		//Stores either FPackedObjectRef or a UObject*
		// Pointer union member is for constinit initialization where we want the linker to be able to do relocations for us within a binary.
		union 
		{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
			TSAN_ATOMIC(UPTRINT) PointerOrRef;
#endif
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
			UE::CodeGen::ConstInit::FCompiledInObjectPtr CompiledInPtr;
#endif
			const void* Pointer;
		};

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		UE_FORCEINLINE_HINT UPTRINT GetPointerOrRef() const
		{
			return PointerOrRef;
		}

		explicit inline operator bool() const
		{
			return PointerOrRef != 0;
		}
#else 
		explicit inline operator bool() const
		{
			return Pointer != nullptr;
		}
#endif

		// When using TSAN, PointerOrRef has non-trivial copy construction so we need to explicitly implement constructors to handle the union.
		// This means this type can't use aggregate initialization even when TSAN is disabled
		FObjectHandlePrivate()
		: Pointer(nullptr)
		{
		}
		~FObjectHandlePrivate() = default;
		[[nodiscard]] constexpr explicit FObjectHandlePrivate(const void* InPointer)
			: Pointer(InPointer)
		{
		}
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		[[nodiscard]] explicit FObjectHandlePrivate(UPTRINT Packed)
			: PointerOrRef(Packed)
		{
		}
#endif
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
		[[nodiscard]] explicit consteval FObjectHandlePrivate(UE::CodeGen::ConstInit::FCompiledInObjectPtr InCompiledInPtr)
			: CompiledInPtr(InCompiledInPtr)
		{
		}
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE && (USING_THREAD_SANITISER || USING_INSTRUMENTATION)
		[[nodiscard]] constexpr FObjectHandlePrivate(const FObjectHandlePrivate& Other)
		{
			UE_IF_CONSTEVAL
			{
				Pointer = Other.Pointer;
			}
			else
			{
				PointerOrRef = Other.PointerOrRef;
			}
		}
		constexpr FObjectHandlePrivate& operator=(const FObjectHandlePrivate& Other)
		{
			UE_IF_CONSTEVAL
			{
				Pointer = Other.Pointer;
			}
			else
			{
				PointerOrRef = Other.PointerOrRef;
			}
			return *this;
		}
#else // USING_THREAD_SANITISER || USING_INSTRUMENTATION
		constexpr FObjectHandlePrivate(const FObjectHandlePrivate&) = default;
		constexpr FObjectHandlePrivate& operator=(const FObjectHandlePrivate&) = default;
#endif // USING_THREAD_SANITISER || USING_INSTRUMENTATION

		bool UEOpEquals(FObjectHandlePrivate Rhs) const;
	};

	// Allow generated operators to be accessible from comparisons in other namespaces.
	UE_OPS_NAMESPACE_VISIBLE(FObjectHandlePrivate)

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	/* Returns the packed object ref for this object IF one exists otherwise returns a null PackedObjectRef */
	COREUOBJECT_API FPackedObjectRef FindExistingPackedObjectRef(const UObject* Object);

	/* Creates and ObjectRef from a packed object ref*/
	COREUOBJECT_API FObjectRef MakeObjectRef(FPackedObjectRef Handle);
#endif // UE_WITH_OBJECT_HANDLE_LATE_RESOLVE


#elif UE_WITH_REMOTE_OBJECT_HANDLE
	enum struct ERemoteObjectHandleType : uint8
	{
		ObjectPointer, // must be 0 so that the handle can be reinterpreted as a pointer when resolved
		StubPointer,
		RemoteId,
	};

	struct FRemoteObjectHandlePrivate
	{
		// Stores either UObject* or  FRemoteObjectStub* or FRemoteObjectId
		// Pointer union member is for constinit initialization where we want the linker to be able to do relocations for us within a binary.
		union 
		{
			TSAN_ATOMIC(UPTRINT) Payload; // least-significant 2 bits are the HandleType
			const void* Pointer;
		};

		static constexpr UPTRINT HandleTypeMask = (0b11ull);
		static constexpr UPTRINT PayloadMask = ~HandleTypeMask;

		ERemoteObjectHandleType ExtractHandleType() const
		{
			return (ERemoteObjectHandleType)(Payload & HandleTypeMask);
		}

		UE::RemoteObject::Handle::FRemoteObjectStub* ExtractStubPointer() const
		{
			ensure(ExtractHandleType() == ERemoteObjectHandleType::StubPointer);
			return BitCast<UE::RemoteObject::Handle::FRemoteObjectStub*>(Payload & PayloadMask);
		}

		const UObject* ExtractObjectPointer() const
		{
			ensure(ExtractHandleType() == ERemoteObjectHandleType::ObjectPointer);
			return BitCast<const UObject*>(Payload & PayloadMask);
		}

		FRemoteObjectId ExtractRemoteId() const
		{
			ensure(ExtractHandleType() == ERemoteObjectHandleType::RemoteId);
			return BitCast<FRemoteObjectId>(Payload & PayloadMask);
		}

		constexpr FRemoteObjectHandlePrivate()
		{
			Payload = 0;
		}

		constexpr FRemoteObjectHandlePrivate(const FRemoteObjectHandlePrivate& Rhs)
		{
			*this = Rhs;
		}

		constexpr FRemoteObjectHandlePrivate(FRemoteObjectHandlePrivate&& Rhs)
		{
			*this = Rhs;
		}

		explicit constexpr FRemoteObjectHandlePrivate(const UObject* Object)
		{
			UE_IF_CONSTEVAL
			{
				// ensure that the handle type is zero so that we can just
				// copy the pointer in and not need to worry about the
				// bit representation which isn't knowable at compile time
				static_assert((uint8)ERemoteObjectHandleType::ObjectPointer == 0);
				Pointer = Object;
			}
			else
			{
				AssignFromObjectPointer(Object);
			}
		}

		explicit FRemoteObjectHandlePrivate(const UE::RemoteObject::Handle::FRemoteObjectStub* RemoteInfo)
		{
			AssignFromStubPointer(RemoteInfo);
		}

		constexpr FRemoteObjectHandlePrivate& operator=(const FRemoteObjectHandlePrivate& Rhs)
		{
			UE_IF_CONSTEVAL
			{
				// the remote heap doesn't exist at compile time so we can always
				// just copy the payload
				Payload = Rhs.Payload;
			}
			else
			{
				// special rules for assignment:
				// if this handle is on the RemoteHeap then we are only
				// allowed to store it as a RemoteId
			
				if (Rhs.ExtractHandleType() == ERemoteObjectHandleType::RemoteId)
				{
					// if Rhs is already encoded as a RemoteId, just copy the payload
					Payload = Rhs.Payload;
				}
				else if (!RemoteObject::IsPointerOnRemoteHeap(this))
				{
					// if this handle is not on the remote heap, just copy the payload
					Payload = Rhs.Payload;
				}
				else
				{
					// otherwise get the RemoteId from the Rhs and assign from that
					AssignFromRemoteId(Rhs.GetRemoteId());
				}
			}

			return *this;
		}

		COREUOBJECT_API void AssignFromRawPayload(UPTRINT Payload);
		COREUOBJECT_API void AssignFromStubPointer(const UE::RemoteObject::Handle::FRemoteObjectStub* Value);
		COREUOBJECT_API void AssignFromObjectPointer(const UObject* Value);
		COREUOBJECT_API void AssignFromRemoteId(FRemoteObjectId Value);

		COREUOBJECT_API UE::RemoteObject::Handle::FRemoteObjectStub* ToStub() const;

		COREUOBJECT_API FRemoteObjectId GetRemoteId() const;

		COREUOBJECT_API static FRemoteObjectHandlePrivate ConvertToRemoteHandle(UObject* Object);
		COREUOBJECT_API static FRemoteObjectHandlePrivate FromIdNoResolve(FRemoteObjectId ObjectId);

		COREUOBJECT_API bool UEOpEquals(FRemoteObjectHandlePrivate Rhs) const;
	};

	// Allow generated operators to be accessible from comparisons in other namespaces.
	UE_OPS_NAMESPACE_VISIBLE(FRemoteObjectHandlePrivate)

#endif

	///these functions are always defined regardless of UE_WITH_OBJECT_HANDLE_LATE_RESOLVE value

	/* Makes a resolved FObjectHandle from an UObject. */
	inline constexpr FObjectHandle MakeObjectHandle(UObject* Object);

	/* Returns the UObject from Handle and the handle is updated cache the resolved UObject */
	inline UObject* ResolveObjectHandle(FObjectHandle& Handle);

	/* Returns the UClass for UObject store in Handle. Handle is not resolved */
	inline UClass* ResolveObjectHandleClass(FObjectHandle Handle);

	/* Returns the UObject from Handle and the handle is updated cache the resolved UObject.
	 * Does not cause ObjectHandleTracking to fire a read event
	 */
	inline UObject* ResolveObjectHandleNoRead(FObjectHandle& Handle);

	/** Resolves an ObjectHandle without checking if already resolved. Invalid to call for resolved handles */
	inline UObject* ResolveObjectHandleNoReadNoCheck(FObjectHandle& Handle);

	/** Read the handle as a pointer without checking if it is resolved. Invalid to call for unresolved handles. */
	inline UObject* ReadObjectHandlePointerNoCheck(FObjectHandle Handle);

#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	/** 
	 * Makes an object handle from generated parameters from UHT.
	 * If Ptr is non-null, makes a direct object ptr that can be linked by the linker.
	 * Otherwise if CompiledInPtr is an encoded reference, makes an encoded pointer that can be linked at runtime.
	 * Otherwise, makes a null handle.
	 */
	inline consteval FObjectHandle MakeCompiledInObjectHandle(void* Ptr, UE::CodeGen::ConstInit::FCompiledInObjectPtr CompiledInPtr);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

/* return true if handle is null */
inline bool IsObjectHandleNull(FObjectHandle Handle)
{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	return !Handle.PointerOrRef;
#elif UE_WITH_REMOTE_OBJECT_HANDLE
	return !Handle.Payload;
#else
	return !Handle;
#endif
}

/* checks if a handle is resolved. 
 * nullptr counts as resolved
 * all handles are resolved when late resolved is off
 */
inline bool IsObjectHandleResolved(FObjectHandle Handle)
{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	return !(Handle.PointerOrRef & 1);
#elif UE_WITH_REMOTE_OBJECT_HANDLE
	if (Handle.ExtractHandleType() == UE::CoreUObject::Private::ERemoteObjectHandleType::ObjectPointer)
	{
		if (IsObjectHandleNull(Handle))
		{
			return true;
		}

		return UE::RemoteObject::Handle::GetResidence(ReadObjectHandlePointerNoCheck(Handle)) == EResidence::Local;
	}

	return false;
#else
	return true;
#endif
}

/* checks if a handle is resolved.
 * nullptr counts as resolved
 * all handles are resolved when late resolved is off
 */
inline bool IsObjectHandleResolved_ForGC(FObjectHandle Handle)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	// Unlike the non-GC version we don't check if the object (if still exists locally) is marked as Remote
	return (Handle.ExtractHandleType() == UE::CoreUObject::Private::ERemoteObjectHandleType::ObjectPointer);
#else
	return IsObjectHandleResolved(Handle);
#endif
}

/* checks if a handle represents a remote object.
 * nullptr counts as not remote
 * all handles are not remote when remote handles are not enabled
 */
UE_DEPRECATED(5.8, "Use GetObjectHandleResidence() instead.")
inline bool IsObjectHandleRemote(FObjectHandle Handle)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Handle.ExtractHandleType() == UE::CoreUObject::Private::ERemoteObjectHandleType::ObjectPointer)
	{
		if (!IsObjectHandleNull(Handle))
		{
			return UE::RemoteObject::Handle::IsRemote(ReadObjectHandlePointerNoCheck(Handle));
		}
	}
	else if (Handle.ExtractHandleType() == UE::CoreUObject::Private::ERemoteObjectHandleType::StubPointer)
	{
		return UE::RemoteObject::Handle::IsRemote(Handle.ExtractStubPointer());
	}
	else if (Handle.ExtractHandleType() == UE::CoreUObject::Private::ERemoteObjectHandleType::RemoteId)
	{
		return UE::RemoteObject::Handle::IsRemote(Handle.ExtractRemoteId());
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	return false;
}

/* checks the residence of the object represented by the specified handle.
 * nullptr counts as local
 * all handles are local when remote handles are not enabled
 */
inline EResidence GetObjectHandleResidence(FObjectHandle Handle)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	using namespace UE::CoreUObject::Private;

	switch (Handle.ExtractHandleType())
	{
	case ERemoteObjectHandleType::ObjectPointer:
		return UE::RemoteObject::Handle::GetResidence(Handle.ExtractObjectPointer());
	case ERemoteObjectHandleType::StubPointer:
		return UE::RemoteObject::Handle::GetResidence(Handle.ExtractStubPointer());
	case ERemoteObjectHandleType::RemoteId:
		return UE::RemoteObject::Handle::GetResidence(Handle.ToStub());
	default:
		checkf(false, TEXT("Unexpected remote object handle type"));
		break;
	}
#endif
	return EResidence::Local;
}

/* return true if a handle is type safe.
 * null and resolved handles are considered type safe
 */ 
inline bool IsObjectHandleTypeSafe(FObjectHandle Handle)
{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE && UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
	return !((Handle.PointerOrRef & 3) == 3);
#elif UE_WITH_REMOTE_OBJECT_HANDLE
	return true;
#else
	return true;
#endif
}

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
inline bool UE::CoreUObject::Private::FObjectHandlePrivate::UEOpEquals(FObjectHandlePrivate RHS) const
{
	using namespace UE::CoreUObject::Private;

	bool LhsResolved = IsObjectHandleResolved(*this);
	bool RhsResolved = IsObjectHandleResolved(RHS);

	//if both resolved or both unresolved compare the uintptr
	if (LhsResolved == RhsResolved)
	{
		return PointerOrRef == RHS.PointerOrRef;
	}

	//only one side can be resolved
	if (LhsResolved)
	{
		//both sides can't be null as resolved status would have be true for both
		const UObject* Obj = ReadObjectHandlePointerNoCheck(*this);
		if (!Obj)
		{
			return false;
		}

		//if packed ref empty then can't be equal as RHS is an unresolved pointer
		FPackedObjectRef PackedLhs = FindExistingPackedObjectRef(Obj);
		if (PackedLhs.EncodedRef == 0)
		{
			return false;
		}
		return PackedLhs.EncodedRef == RHS.PointerOrRef;

	}
	else
	{
		//both sides can't be null as resolved status would have be true for both
		const UObject* Obj = ReadObjectHandlePointerNoCheck(RHS);
		if (!Obj)
		{
			return false;
		}

		//if packed ref empty then can't be equal as RHS is an unresolved pointer
		FPackedObjectRef PackedRhs = FindExistingPackedObjectRef(Obj);
		if (PackedRhs.EncodedRef == 0)
		{
			return false;
		}
		return PackedRhs.EncodedRef == PointerOrRef;
	}

}

inline uint32 GetTypeHash(UE::CoreUObject::Private::FObjectHandlePrivate Handle)
{
	using namespace UE::CoreUObject::Private;

	if (Handle.PointerOrRef == 0)
	{
		return 0;
	}

	if (IsObjectHandleResolved(Handle))
	{
		const UObject* Obj = ReadObjectHandlePointerNoCheck(Handle);

		FPackedObjectRef PackedObjectRef = FindExistingPackedObjectRef(Obj);
		if (PackedObjectRef.EncodedRef == 0)
		{
			return GetTypeHash(Obj);
		}
		return GetTypeHash(PackedObjectRef.EncodedRef);
	}
	return GetTypeHash(Handle.GetPointerOrRef());
}
#elif UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
inline bool UE::CoreUObject::Private::FObjectHandlePrivate::UEOpEquals(FObjectHandlePrivate RHS) const
{
	return Pointer == RHS.Pointer;
}

inline uint32 GetTypeHash(UE::CoreUObject::Private::FObjectHandlePrivate Handle)
{
	return GetTypeHash(Handle.Pointer);
}
#elif UE_WITH_REMOTE_OBJECT_HANDLE

inline FRemoteObjectId GetRemoteObjectId(UE::CoreUObject::Private::FRemoteObjectHandlePrivate Handle)
{
	using namespace UE::CoreUObject::Private;
	using namespace UE::RemoteObject::Handle;
	return Handle.GetRemoteId();
}

/**
* GetTypeHash for FRemoteObjectHandlePrivate is guaranteed to return the same value whether a non-null handle is resolved or not (GetTypeHash of FRemoteObjectId)
*/
inline uint32 GetTypeHash(UE::CoreUObject::Private::FRemoteObjectHandlePrivate Handle)
{
	return GetTypeHash(GetRemoteObjectId(Handle));
}
#endif

/////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace UE::CoreUObject::Private
{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE 
	inline constexpr FObjectHandle MakeObjectHandle(UObject* Object)
	{
		UE_IF_CONSTEVAL
		{
			return FObjectHandle(Object);
		}
		else
		{ 
			return FObjectHandle(UPTRINT(Object));
		}
	}
#elif UE_WITH_REMOTE_OBJECT_HANDLE
	inline constexpr FObjectHandle MakeObjectHandle(UObject* Object)
	{
		return FObjectHandle(Object);
	}
#else
	inline constexpr FObjectHandle MakeObjectHandle(UObject* Object)
	{
		return FObjectHandle(Object);
	}
#endif

#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	inline consteval FObjectHandle MakeCompiledInObjectHandle(void* Ptr, UE::CodeGen::ConstInit::FCompiledInObjectPtr CompiledInPtr)
	{
		if (Ptr)
		{
			return FObjectHandle(Ptr);
		}
		else if (!CompiledInPtr.IsUnset())
		{
			return FObjectHandle(CompiledInPtr);
		}
		else 
		{
			return FObjectHandle(nullptr);
		}
	}
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC

	inline UObject* ResolveObjectHandle(FObjectHandle& Handle)
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
		UObject* ResolvedObject = ResolveObjectHandleNoRead(Handle);
		UE::CoreUObject::Private::OnHandleRead(ResolvedObject);
		return ResolvedObject;
#elif UE_WITH_REMOTE_OBJECT_HANDLE
		UObject* ResolvedObject = ResolveObjectHandleNoRead(Handle);
		return ResolvedObject;
#else
		return ReadObjectHandlePointerNoCheck(Handle);
#endif
	}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	inline FPackedObjectRef ReadObjectHandlePackedObjectRefNoCheck(FObjectHandle Handle)
	{
		return FPackedObjectRef{ Handle.PointerOrRef };
	}
#endif

	inline UClass* ResolveObjectHandleClass(FObjectHandle Handle)
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		if (IsObjectHandleResolved(Handle))
		{
			UObject* Obj = ReadObjectHandlePointerNoCheck(Handle);
			return Obj != nullptr ? UE::CoreUObject::Private::GetClass(Obj) : nullptr;
		}
		else
		{
			// @TODO: OBJPTR: This should be cached somewhere instead of resolving on every call
			FPackedObjectRef PackedObjectRef = ReadObjectHandlePackedObjectRefNoCheck(Handle);
			FObjectRef ObjectRef = MakeObjectRef(PackedObjectRef);
			return ObjectRef.ResolveObjectRefClass();
		}
#elif UE_WITH_REMOTE_OBJECT_HANDLE
		UObject* Obj = nullptr;
		if (IsObjectHandleResolved_ForGC(Handle))
		{
			Obj = ReadObjectHandlePointerNoCheck(Handle);
		}
		else
		{
			const UE::RemoteObject::Handle::FRemoteObjectStub* Stub = Handle.ToStub();
			if (UClass* Class = Stub->Class.GetClass())
			{
				return Class;
			}
			else
			{
				Obj = ResolveObjectHandle(Handle);
			}
		}
		return Obj != nullptr ? UE::CoreUObject::Private::GetClass(Obj) : nullptr;
#else
		UObject* Obj = ReadObjectHandlePointerNoCheck(Handle);
		return Obj != nullptr ? UE::CoreUObject::Private::GetClass(Obj) : nullptr;
#endif
	}

	inline UObject* ResolveObjectHandleNoRead(FObjectHandle& Handle)
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		FObjectHandle LocalHandle = Handle;
		if (IsObjectHandleResolved(LocalHandle))
		{
			UObject* ResolvedObject = ReadObjectHandlePointerNoCheck(LocalHandle);
			return ResolvedObject;
		}
		else
		{
			FPackedObjectRef PackedObjectRef = ReadObjectHandlePackedObjectRefNoCheck(LocalHandle);
			FObjectRef ObjectRef = MakeObjectRef(PackedObjectRef);
			UObject* ResolvedObject = ObjectRef.Resolve();
#if UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
			if (IsObjectHandleTypeSafe(LocalHandle))
#endif
			{
				Handle = MakeObjectHandle(ResolvedObject);
			}
			return ResolvedObject;
		}
#elif UE_WITH_REMOTE_OBJECT_HANDLE
		FObjectHandle LocalHandle = Handle;
		UObject* ResolvedObject = nullptr;
		if (LocalHandle.ExtractHandleType() == ERemoteObjectHandleType::RemoteId)
		{
			ResolvedObject = UE::RemoteObject::Handle::ResolveObject(LocalHandle.ToStub());
		}
		else if (LocalHandle.ExtractHandleType() == ERemoteObjectHandleType::StubPointer)
		{
			ResolvedObject = UE::RemoteObject::Handle::ResolveObject(LocalHandle.ExtractStubPointer());
		}
		else if (LocalHandle.ExtractHandleType() == ERemoteObjectHandleType::ObjectPointer)
		{
			ResolvedObject = const_cast<UObject*>(LocalHandle.ExtractObjectPointer());
			if (ResolvedObject != nullptr && UE::RemoteObject::Handle::GetResidence(ResolvedObject) != EResidence::Local)
			{
				ResolvedObject = UE::RemoteObject::Handle::ResolveObject(ResolvedObject);
			}
		}

		// do not overwrite the internal handle if it is on the remote heap, it must
		// always be stored as a RemoteId
		if (!RemoteObject::IsPointerOnRemoteHeap(&Handle))
		{
			Handle = MakeObjectHandle(ResolvedObject);
		}
		UE::RemoteObject::Handle::TouchResidentObject(ResolvedObject);
		return ResolvedObject;
#else
		return ReadObjectHandlePointerNoCheck(Handle);
#endif
	}


	inline UObject* NoResolveObjectHandleNoRead(const FObjectHandle& Handle)
	{
		FObjectHandle LocalHandle = Handle;
		if (IsObjectHandleResolved_ForGC(LocalHandle))
		{
			UObject* ResolvedObject = ReadObjectHandlePointerNoCheck(LocalHandle);
			return ResolvedObject;
		}
		else
		{
			return nullptr;
		}
	}

	inline UObject* ResolveObjectHandleNoReadNoCheck(FObjectHandle& Handle)
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		FObjectHandle LocalHandle = Handle;
		FPackedObjectRef PackedObjectRef = ReadObjectHandlePackedObjectRefNoCheck(LocalHandle);
		FObjectRef ObjectRef = MakeObjectRef(PackedObjectRef);
		UObject* ResolvedObject = ObjectRef.Resolve();
#if UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
		if (IsObjectHandleTypeSafe(LocalHandle))
#endif
		{
			LocalHandle = MakeObjectHandle(ResolvedObject);
			Handle = LocalHandle;
		}
		return ResolvedObject;
#elif UE_WITH_REMOTE_OBJECT_HANDLE
		FObjectHandle LocalHandle = Handle;
		// Unresolved handle may mean two things: we still have the remote object memory (it hasn't been GC'd yet) or we only have a stub
		UObject* ResolvedObject = nullptr;
		if (IsObjectHandleResolved_ForGC(LocalHandle))
		{
			ResolvedObject = UE::RemoteObject::Handle::ResolveObject(ReadObjectHandlePointerNoCheck(LocalHandle));
		}
		else
		{
			ResolvedObject = UE::RemoteObject::Handle::ResolveObject(LocalHandle.ToStub());
		}
		LocalHandle = MakeObjectHandle(ResolvedObject);
		Handle = LocalHandle;
		return ResolvedObject;	
#else
		return ReadObjectHandlePointerNoCheck(Handle);
#endif
	}

	inline UObject* ReadObjectHandlePointerNoCheck(FObjectHandle Handle)
	{
#if UE_OBJECT_HANDLE_IS_OBJECT_PTR
		return Handle;
#elif UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
		return reinterpret_cast<UObject*>(Handle.GetPointerOrRef());
#elif UE_WITH_REMOTE_OBJECT_HANDLE
		return const_cast<UObject*>(Handle.ExtractObjectPointer());
#else
		return const_cast<UObject*>(reinterpret_cast<const UObject*>(Handle.Pointer));
#endif
	}

	//Natvis structs
	struct FObjectHandlePackageDebugData
	{
		FMinimalName PackageName;
		FScriptArray ObjectDescriptors;
		uint8 _Padding[sizeof(FRWLock)];
	};

	struct FObjectHandleDataClassDescriptor
	{
		FMinimalName PackageName;
		FMinimalName ClassName;
	};

	struct FObjectPathIdDebug
	{
		uint32 Index = 0;
		uint32 Number = 0;

		static constexpr uint32 WeakObjectMask = ~((~0u) >> 1);       //most significant bit
		static constexpr uint32 SimpleNameMask = WeakObjectMask >> 1; //second most significant bits
	};

	struct FObjectDescriptorDebug
	{
		FObjectPathIdDebug ObjectPath;
		FObjectHandleDataClassDescriptor ClassDescriptor;
	};

	struct FStoredObjectPathDebug
	{
		static constexpr const int32 NumInlineElements = 3;
		int32 NumElements;

		union
		{
			FMinimalName Short[NumInlineElements];
			FMinimalName* Long;
		};
	};

	inline constexpr uint32 TypeIdShift = 1;
	inline constexpr uint32 ObjectIdShift = 2;
	inline constexpr uint32 PackageIdShift = 34;
	inline constexpr uint32 PackageIdMask = 0x3FFF'FFFF;

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	//forward declarations
	void InitObjectHandles(int32 Size);
	void FreeObjectHandle(const UObjectBase* Object);
	void UpdateRenamedObject(const UObject* Obj, FName NewName, UObject* NewOuter);
	UE::CoreUObject::Private::FPackedObjectRef MakePackedObjectRef(const UObject* Object);
#endif
}
