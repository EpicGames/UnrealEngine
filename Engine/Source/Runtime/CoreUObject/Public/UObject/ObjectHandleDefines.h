// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Included through other header

#include "Misc/CoreMiscDefines.h"


#ifndef UE_WITH_OBJECT_HANDLE_LATE_RESOLVE
	#define UE_WITH_OBJECT_HANDLE_LATE_RESOLVE (!UE_WITH_REMOTE_OBJECT_HANDLE && WITH_EDITORONLY_DATA)
#endif


#ifndef UE_WITH_OBJECT_HANDLE_TRACKING
	#define UE_WITH_OBJECT_HANDLE_TRACKING (!UE_WITH_REMOTE_OBJECT_HANDLE && WITH_EDITORONLY_DATA)
#endif


#ifndef UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
	#define UE_WITH_OBJECT_HANDLE_TYPE_SAFETY (!UE_WITH_REMOTE_OBJECT_HANDLE && WITH_EDITORONLY_DATA)
#endif