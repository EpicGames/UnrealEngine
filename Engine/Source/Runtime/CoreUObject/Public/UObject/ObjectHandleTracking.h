// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "UObject/ObjectHandleDefines.h"
#include "Templates/Function.h"
#include <atomic>

struct FObjectRef;

namespace UE::CoreUObject
{
	/**
	 * FObjectHandles can optionally support tracking.  Because of the low level nature of object handles, anything that
	 * registers itself for these callbacks should ensure that it is:
	 * 1) error free (ie: should not cause exceptions even in unusual circumstances)
	 * 2) fault tolerant (ie: could be called at a time when an exception has happened)
	 * 3) thread-safe (ie: could be called from any thread)
	 * 4) high performance (ie: will be called many times)
	 */
#if UE_WITH_OBJECT_HANDLE_TRACKING

	 /**
	  * Callback notifying when an object value is read from a handle.  Fired regardless of whether the handle
	  * was resolved as part of the read operation or not and whether the object being read is null or not.
	  *
	  * @param ReadObject	The object that was read from a handle.
	  */
	using FObjectHandleReadFunc = TFunction<void(const TArrayView<const UObject* const>& Objects)>;

	/**
	 * Callback notifying when a class is resolved from an object handle or object reference.
	 * Classes are resolved either independently for a given handle/reference or as part of each object resolve.
	 *
	 * @param SourceRef		The object reference (either user provided or internally computed from a handle) from which the class was resolved.
	 * @param ClassPackage	The package containing the resolved class. May be null.
	 * @param Class			The resolved class. May be null.
	 */
	using FObjectHandleClassResolvedFunc = TFunction<void(const FObjectRef& SourceRef, UPackage* ClassPackage, UClass* Class)>;
	/**
	 * Callback notifying when a object handle is resolved.
	 *
	 * @param SourceRef		The object reference (either user provided or internally computed from a handle) from which the object was resolved.
	 * @param ObjectPackage	The package containing the resolved object. May be null.
	 * @param Object		The resolved Object. May be null.
	 */
	using FObjectHandleReferenceResolvedFunc = TFunction<void(const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)>;
	/**
	 * Callback notifying when an object was loaded through an object handle.  Will not notify you about global object loads, just ones that occur
	 * as the byproduct of resolving an ObjectHandle.
	 *
	 * @param SourceRef		The object reference (either user provided or internally computed from a handle) from which the object was resolved.
	 * @param ObjectPackage	The package containing the resolved object. May be null.
	 * @param Object		The resolved object. May be null.
	 */
	using FObjectHandleReferenceLoadedFunc = TFunction<void(const FObjectRef& SourceRef, UPackage* ObjectPackage, UObject* Object)>;

	struct FObjectHandleTrackingCallbackId
	{
		int32 Id = -1;
		bool IsValid() { return Id != -1; }
	};

	/**
	 * Installs a new callback for notifications that an object value has been read from a handle.
	 *
	 * @param Function		The new handle read callback to install.
	 * @return				The handle so that you can remove the callback at a later date.
	 */
	COREUOBJECT_API FObjectHandleTrackingCallbackId AddObjectHandleReadCallback(FObjectHandleReadFunc Delegate);
	COREUOBJECT_API void RemoveObjectHandleReadCallback(FObjectHandleTrackingCallbackId DelegateHandle);

	/**
	 * Installs a new callback for notifications that a class has been resolved from an object handle or object reference.
	 *
	 * @param Function		The new class resolved callback to install.
	 * @return				The handle so that you can remove the callback at a later date.
	 */
	COREUOBJECT_API FObjectHandleTrackingCallbackId AddObjectHandleClassResolvedCallback(FObjectHandleClassResolvedFunc Callback);
	COREUOBJECT_API void RemoveObjectHandleClassResolvedCallback(FObjectHandleTrackingCallbackId DelegateHandle);

	/**
	 * Installs a new callback for notifications that an object has been resolved from an object handle or object reference.
	 *
	 * @param Function		The new object resolved callback to install.
	 * @return				The handle so that you can remove the callback at a later date.
	 */
	COREUOBJECT_API FObjectHandleTrackingCallbackId AddObjectHandleReferenceResolvedCallback(FObjectHandleReferenceResolvedFunc Callback);
	COREUOBJECT_API void RemoveObjectHandleReferenceResolvedCallback(FObjectHandleTrackingCallbackId DelegateHandle);

	/**
	 * Installs a new callback for notifications that an object has been loaded from an object handle or object reference.
	 *
	 * @param Function		The new object resolved callback to install.
	 * @return				The handle so that you can remove the callback at a later date.
	 */
	COREUOBJECT_API FObjectHandleTrackingCallbackId AddObjectHandleReferenceLoadedCallback(FObjectHandleReferenceLoadedFunc Callback);
	COREUOBJECT_API void RemoveObjectHandleReferenceLoadedCallback(FObjectHandleTrackingCallbackId DelegateHandle);

#endif
}

#if UE_WITH_OBJECT_HANDLE_TRACKING
namespace UE::CoreUObject::Private
{
	extern COREUOBJECT_API std::atomic<int32> HandleReadCallbackQuantity;
	COREUOBJECT_API void OnHandleReadInternal(TArrayView<const UObject* const> Objects);
	COREUOBJECT_API void OnHandleReadInternal(const UObject* Object);
	inline void OnHandleRead(TArrayView<const UObject* const> Objects)
	{
		if (HandleReadCallbackQuantity.load(std::memory_order_relaxed) > 0)
		{
			OnHandleReadInternal(Objects);
		}
	}
	inline void OnHandleRead(const UObject* Object)
	{
		if (HandleReadCallbackQuantity.load(std::memory_order_relaxed) > 0)
		{
			OnHandleReadInternal(Object);
		}
	}
	COREUOBJECT_API void OnClassReferenceResolved(const FObjectRef& ObjectRef, UPackage* ClassPackage, UClass* Class);
	COREUOBJECT_API void OnReferenceResolved(const FObjectRef& ObjectRef, UPackage* ObjectPackage, UObject* Object);
	COREUOBJECT_API void OnReferenceLoaded(const FObjectRef& ObjectRef, UPackage* ObjectPackage, UObject* Object);
}
#else

namespace UE::CoreUObject::Private
{
	inline void OnHandleRead(const UObject* Object) { }
	inline void OnHandleRead(TArrayView<const UObject* const> Objects) { }
	inline void OnClassReferenceResolved(const FObjectRef& ObjectRef, UPackage* Package, UClass* Class) { }
	inline void OnReferenceResolved(const FObjectRef& ObjectRef, UPackage* Package, UClass* Class) { }
	inline void OnReferenceLoaded(const FObjectRef& ObjectRef, UPackage* Package, UObject* Object) { }
}


#endif