// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Requires.h"
#include "UObject/WeakObjectPtr.h"

#include <type_traits>

struct FObjectKey;

namespace UE::CoreUObject::Private
{
	FObjectKey MakeObjectKey(int32 ObjectIndex, int32 ObjectSerialNumber);
}

/** FObjectKey is an immutable, copyable key which can be used to uniquely identify an object for the lifetime of the application */
struct FObjectKey
{
public:
	/** Default constructor */
	inline FObjectKey()
		: ObjectIndex(UE::Core::Private::InvalidWeakObjectIndex)
		, ObjectSerialNumber(0)
	{
	}

	/** Construct from an object pointer */
	inline FObjectKey(const UObject* Object)
		: ObjectIndex(UE::Core::Private::InvalidWeakObjectIndex)
		, ObjectSerialNumber(0)
	{
		if (Object)
		{
			FWeakObjectPtr Weak(Object);
			ObjectIndex = Weak.ObjectIndex;
			ObjectSerialNumber = Weak.ObjectSerialNumber;
#if UE_WITH_REMOTE_OBJECT_HANDLE
			RemoteId = Weak.ObjectRemoteId;
#endif
		}
	}
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U, const UObject*>)
	>
	UE_FORCEINLINE_HINT FObjectKey(U Object)
		: FObjectKey(ImplicitConv<const UObject*>(Object))
	{
	}

	/** Compare this key with another */
	inline bool operator==(const FObjectKey& Other) const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return RemoteId == Other.RemoteId;
#else
		return ObjectIndex == Other.ObjectIndex && ObjectSerialNumber == Other.ObjectSerialNumber;
#endif
	}

	/** Compare this key with another */
	inline bool operator!=(const FObjectKey& Other) const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return RemoteId != Other.RemoteId;
#else
		return ObjectIndex != Other.ObjectIndex || ObjectSerialNumber != Other.ObjectSerialNumber;
#endif
	}

	/** Compare this key with another */
	inline bool operator<(const FObjectKey& Other) const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return RemoteId < Other.RemoteId;
#else
		return ObjectIndex < Other.ObjectIndex || (ObjectIndex == Other.ObjectIndex && ObjectSerialNumber < Other.ObjectSerialNumber);
#endif
	}

	/** Compare this key with another */
	inline bool operator<=(const FObjectKey& Other) const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return RemoteId <= Other.RemoteId;
#else
		return ObjectIndex <= Other.ObjectIndex || (ObjectIndex == Other.ObjectIndex && ObjectSerialNumber <= Other.ObjectSerialNumber);
#endif
	}

	/** Compare this key with another */
	inline bool operator>(const FObjectKey& Other) const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return RemoteId > Other.RemoteId;
#else
		return ObjectIndex > Other.ObjectIndex || (ObjectIndex == Other.ObjectIndex && ObjectSerialNumber > Other.ObjectSerialNumber);
#endif
	}

	/** Compare this key with another */
	inline bool operator>=(const FObjectKey& Other) const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return RemoteId >= Other.RemoteId;
#else
		return ObjectIndex > Other.ObjectIndex || (ObjectIndex == Other.ObjectIndex && ObjectSerialNumber >= Other.ObjectSerialNumber);
#endif
	}

	inline friend FArchive& operator<<(FArchive& Ar, FObjectKey& Key)
	{
		check(!Ar.IsPersistent());
		return Ar << Key.ObjectIndex << Key.ObjectSerialNumber
#if UE_WITH_REMOTE_OBJECT_HANDLE
			<< Key.RemoteId
#endif
			;
	}

	/**
	 * Attempt to access the object from which this key was constructed.
	 * @return The object used to construct this key, or nullptr if it is no longer valid
	 */
	UObject* ResolveObjectPtr() const
	{
		FWeakObjectPtr WeakPtr;
		WeakPtr.ObjectIndex = ObjectIndex;
		WeakPtr.ObjectSerialNumber = ObjectSerialNumber;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		WeakPtr.ObjectRemoteId = RemoteId;
#endif
		return WeakPtr.Get();
	}

	/**
	 * Attempt to access the object from which this key was constructed, even if it is marked as Garbage.
	 * @return The object used to construct this key, or nullptr if it is no longer valid
	 */
	UObject* ResolveObjectPtrEvenIfGarbage() const
	{
		FWeakObjectPtr WeakPtr;
		WeakPtr.ObjectIndex = ObjectIndex;
		WeakPtr.ObjectSerialNumber = ObjectSerialNumber;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		WeakPtr.ObjectRemoteId = RemoteId;
#endif
		constexpr bool bEvenIfGarbage = true;
		return WeakPtr.Get(bEvenIfGarbage);
	}

	/**
	 * Attempt to access the object from which this key was constructed, even if it is RF_PendingKill or RF_Unreachable
	 * @return The object used to construct this key, or nullptr if it is no longer valid
	 */
	UObject* ResolveObjectPtrEvenIfUnreachable() const
	{
		FWeakObjectPtr WeakPtr;
		WeakPtr.ObjectIndex = ObjectIndex;
		WeakPtr.ObjectSerialNumber = ObjectSerialNumber;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		WeakPtr.ObjectRemoteId = RemoteId;
#endif
		return WeakPtr.GetEvenIfUnreachable();
	}

	/**
	 * Create an FWeakObjectPtr that points to the same object as this key.
	 * @return The new FWeakObjectPtr.
	 */
	FWeakObjectPtr GetWeakObjectPtr() const
	{
		FWeakObjectPtr WeakPtr;
		WeakPtr.ObjectIndex = ObjectIndex;
		WeakPtr.ObjectSerialNumber = ObjectSerialNumber;
#if UE_WITH_REMOTE_OBJECT_HANDLE
		WeakPtr.ObjectRemoteId = RemoteId;
#endif
		return WeakPtr;
	}

	/** Hash function */
	[[nodiscard]] friend uint32 GetTypeHash(const FObjectKey& Key)
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return GetTypeHash(Key.RemoteId);
#else
		return HashCombine(Key.ObjectIndex, Key.ObjectSerialNumber);
#endif
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/** Get globally unique id of the object */
	FRemoteObjectId GetRemoteId() const
	{
		return RemoteId;
	}
#endif

private:
#if !UE_WITH_REMOTE_OBJECT_HANDLE
	FObjectKey(int32 Index, int32 Serial)
		: ObjectIndex(Index)
		, ObjectSerialNumber(Serial)
	{
	}

	friend FObjectKey UE::CoreUObject::Private::MakeObjectKey(int32 ObjectIndex, int32 ObjectSerialNumber);
#endif

	int32		ObjectIndex;
	int32		ObjectSerialNumber;
#if UE_WITH_REMOTE_OBJECT_HANDLE
	FRemoteObjectId RemoteId;
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
};

/** TObjectKey is a strongly typed, immutable, copyable key which can be used to uniquely identify an object for the lifetime of the application */
template<typename InElementType>
class TObjectKey
{
public:
	typedef InElementType ElementType;

	/** Default constructor */
	UE_FORCEINLINE_HINT TObjectKey() = default;

	/** Construct from an object pointer */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U, const InElementType*>)
	>
	UE_FORCEINLINE_HINT TObjectKey(U Object)
		: ObjectKey(ImplicitConv<const InElementType*>(Object))
	{
	}

	/** Compare this key with another */
	UE_FORCEINLINE_HINT bool operator==(const TObjectKey& Other) const
	{
		return ObjectKey == Other.ObjectKey;
	}

	/** Compare this key with another */
	UE_FORCEINLINE_HINT bool operator!=(const TObjectKey& Other) const
	{
		return ObjectKey != Other.ObjectKey;
	}

	/** Compare this key with another */
	UE_FORCEINLINE_HINT bool operator<(const TObjectKey& Other) const
	{
		return ObjectKey < Other.ObjectKey;
	}

	/** Compare this key with another */
	UE_FORCEINLINE_HINT bool operator<=(const TObjectKey& Other) const
	{
		return ObjectKey <= Other.ObjectKey;
	}

	/** Compare this key with another */
	UE_FORCEINLINE_HINT bool operator>(const TObjectKey& Other) const
	{
		return ObjectKey > Other.ObjectKey;
	}

	/** Compare this key with another */
	UE_FORCEINLINE_HINT bool operator>=(const TObjectKey& Other) const
	{
		return ObjectKey >= Other.ObjectKey;
	}

	/** @return the object key */
	UE_REWRITE operator FObjectKey() const
	{
		return ObjectKey;
	}

	//** Hash function */
	[[nodiscard]] friend uint32 GetTypeHash(const TObjectKey& Key)
	{
		return GetTypeHash(Key.ObjectKey);
	}

	/**
	 * Attempt to access the object from which this key was constructed.
	 * @return The object used to construct this key, or nullptr if it is no longer valid
	 */
	InElementType* ResolveObjectPtr() const
	{
		return (InElementType*)ObjectKey.ResolveObjectPtr();
	}

	/**
	 * Attempt to access the object from which this key was constructed, even if it is marked as Garbage.
	 * @return The object used to construct this key, or nullptr if it is no longer valid
	 */
	InElementType* ResolveObjectPtrEvenIfGarbage() const
	{
		return static_cast<InElementType*>(ObjectKey.ResolveObjectPtrEvenIfGarbage());
	}

private:
	FObjectKey ObjectKey;
};
