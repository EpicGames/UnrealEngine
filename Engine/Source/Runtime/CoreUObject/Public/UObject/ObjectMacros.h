// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ObjectMacros.h: Helper macros and defines for UObject system
=============================================================================*/

#pragma once

// IWYU pragma: begin_exports

#include "Misc/EnumClassFlags.h"
#include "UObject/Script.h"

class FObjectInitializer;
class FReferenceCollector;
struct FAppendToClassSchemaContext;
struct FFrame;
struct FClassReloadVersionInfo;
struct FTopLevelAssetPath;

// IWYU pragma: end_exports

/** Represents a serializable object pointer in blueprint bytecode. This is always 64-bits, even on 32-bit platforms. */
typedef	uint64 ScriptPointerType;

#if PLATFORM_VTABLE_AT_END_OF_CLASS
#error "not supported in UE"
#endif

#ifndef USE_COMPILED_IN_NATIVES
#define USE_COMPILED_IN_NATIVES	1
#endif

/** Set this to 0 to disable UObject thread safety features */
#define THREADSAFE_UOBJECTS 1

// Enumeration of different methods of determining ustruct relationships.
#define USTRUCT_ISCHILDOF_OUTERWALK  1 // walks the super struct chain                                     - original IsA behavior
#define USTRUCT_ISCHILDOF_STRUCTARRAY 2 // stores an array of parents per struct and uses this to compare - faster than 1 and thread-safe but can have issues with BP reinstancing and hot reload

// USTRUCT_FAST_ISCHILDOF_IMPL sets which implementation of IsChildOf to use.
#if WITH_EDITOR
	// On editor, we use the outerwalk implementation because BP reinstancing and hot reload
	// mess up the struct array
	#define USTRUCT_FAST_ISCHILDOF_IMPL USTRUCT_ISCHILDOF_OUTERWALK
#else
	#define USTRUCT_FAST_ISCHILDOF_IMPL USTRUCT_ISCHILDOF_STRUCTARRAY
#endif

// USTRUCT_FAST_ISCHILDOF_COMPARE_WITH_OUTERWALK, if set, does a checked comparison of the current implementation against the outer walk - used for testing.
#define USTRUCT_FAST_ISCHILDOF_COMPARE_WITH_OUTERWALK 0

// Allow UObject reflection info to be initialized in the binary at compile time instead of allocated on the heap at runtime.
#ifndef UE_WITH_CONSTINIT_UOBJECT
	#define UE_WITH_CONSTINIT_UOBJECT 0
#endif

// Allows relaxing uobject constinit declarations to be ordinary globals with dynamic initialization allowed
#ifndef UE_CONSTINIT_UOBJECT_DECL
	#define UE_CONSTINIT_UOBJECT_DECL constinit
#endif


/*-----------------------------------------------------------------------------
	Core enumerations.
-----------------------------------------------------------------------------*/

/** Flags for loading objects, used by LoadObject() and related functions and passed as a uint32 */
enum ELoadFlags
{
	LOAD_None						= 0x00000000,	///< No flags.
	LOAD_Async						= 0x00000001,	///< Loads the package using async loading path/ reader
	LOAD_NoWarn						= 0x00000002,	///< Don't display warning if load fails.
	LOAD_EditorOnly					= 0x00000004,	///< Load for editor-only purposes and by editor-only code
	LOAD_ResolvingDeferredExports	= 0x00000008,	///< Denotes that we should not defer export loading (as we're resolving them)
	LOAD_Verify						= 0x00000010,	///< Only verify existance; don't actually load.
//	LOAD_Unused						= 0x00000020,	///< Allow plain DLLs.
//	LOAD_Unused						= 0x00000040
	LOAD_NoVerify					= 0x00000080,	///< Don't verify imports yet.
	LOAD_IsVerifying				= 0x00000100,	///< Is verifying imports
	LOAD_SkipLoadImportedPackages	= 0x00000200,	///< Assume that all import packages are already loaded and don't call LoadPackage when creating imports
	LOAD_RegenerateBulkDataGuids UE_DEPRECATED(5.5, "This flag is now obsolete as the functionality is unnecessary.") = 0x00000400, ///< BulkData identifiers should be regenerated as they are loaded
//	LOAD_Unused						= 0x00000800,
	LOAD_DisableDependencyPreloading = 0x00001000,	///< Bypass dependency preloading system
	LOAD_Quiet						= 0x00002000,	///< No log warnings.
	LOAD_FindIfFail					= 0x00004000,	///< Tries FindObject if a linker cannot be obtained (e.g. package is currently being compiled)
	LOAD_MemoryReader				= 0x00008000,	///< Loads the file into memory and serializes from there.
	LOAD_NoRedirects				= 0x00010000,	///< Never follow redirects when loading objects; redirected loads will fail
	LOAD_ForDiff					= 0x00020000,	///< Loading for diffing in the editor
	LOAD_PackageForPIE				= 0x00080000,	///< This package is being loaded for PIE, it must be flagged as such immediately
	LOAD_DeferDependencyLoads       = 0x00100000,	///< Do not load external (blueprint) dependencies (instead, track them for deferred loading)
	LOAD_ForFileDiff				= 0x00200000,	///< Load the package (not for diffing in the editor), instead verify at the two packages serialized output are the same, if they are not then debug break so that you can get the callstack and object information
	LOAD_DisableCompileOnLoad		= 0x00400000,	///< Prevent this load call from running compile on load for the loaded blueprint (intentionally not recursive, dependencies will still compile on load)
	LOAD_DisableEngineVersionChecks = 0x00800000,	///< Prevent this load call from running engine version checks
};

/** Flags for saving objects/packages, passed into UPackage::SavePackage() as a uint32 */
enum ESaveFlags
{
	SAVE_None						= 0x00000000,	///< No flags
	SAVE_NoError					= 0x00000001,	///< Don't generate errors on save
	SAVE_FromAutosave				= 0x00000002,	///< Used to indicate this save was initiated automatically
	SAVE_KeepDirty					= 0x00000004,	///< Do not clear the dirty flag when saving
	SAVE_KeepPersistentGUID			= 0x00000008,	///< Do not replace the persistent GUID when saving a package to a different path. This is potentially dangerous as it could result in duplicate GUIDs and so should only be used in cases
													///< where we know that the resulting package is temporary or will eventually replace the original package file.
	SAVE_Async						= 0x00000010,	///< Save to a memory writer, then actually write to disk async
	SAVE_Unversioned_Native			= 0x00000020,	///< Save all versions as zero. Upon load this is changed to the current version. This is only reasonable to use with full cooked builds for distribution.
	SAVE_CutdownPackage				= 0x00000040,	///< Saving cutdown packages in a temp location WITHOUT renaming the package.
	SAVE_KeepEditorOnlyCookedPackages = 0x00000080,  ///< Keep packages which are marked as editor only even though we are cooking
	SAVE_Concurrent					= 0x00000100,	///< We are save packages in multiple threads at once and should not call non-threadsafe functions or rely on globals. UE::IsSavingPackage() should be set and PreSave/Postsave functions should be called before/after the entire concurrent save.
//	SAVE_Unused						= 0x00000200,
//	SAVE_Unused						= 0x00000400,
//	SAVE_Unused						= 0x00000800,
	SAVE_CompareLinker UE_DEPRECATED(5.6, "Comparing Linkers is no longer used. Contact Epic if you need this functionality.") = 0x00001000,
	SAVE_BulkDataByReference		= 0x00002000,	///< When saving to a different file than the package's LoadedPath, point bulkdata in the new file to be loaded from the original file.
	SAVE_Unversioned_Properties		= 0x00004000,	///< Properties are saved without property name information, and are saved/loaded in the order of the current binary.
													///< This is only reasonable to use with full cooked builds for distribution, or with a domain that selects the payload matching a given version externally.
	SAVE_Optional					= 0x00008000,	///< Indicate that we to save optional exports. This flag is only valid while cooking. Optional exports are filtered if not specified during cooking. 
	SAVE_AllowTimeout				= 0x00010000,	///< If present, if SavePackage is blocked by an asynchronous operation, it will quickly fail with ESavePackageResult::Timeout, otherwise it will wait for a while and then fail with ESavePackageResult::Error.
	SAVE_RehydratePayloads			= 0x00020000,	///< Any virtualized payloads in the package should be pulled and stored locally in the package file during the save
	SAVE_CookSoftPackageReferences	= 0x00040000,	///< Indicate that we want to save soft package references for cooked packages
	SAVE_Unversioned = SAVE_Unversioned_Native | SAVE_Unversioned_Properties,
};

/** 
 * Package flags, passed into UPackage::SetPackageFlags and related functions
 *
 * This MUST be kept in sync with EPackageFlags defined in
 * Engine\Source\Programs\Shared\EpicGames.Core\UnrealEngineTypes.cs
 */
enum EPackageFlags
{
	PKG_None						= 0x00000000,	///< No flags
	PKG_NewlyCreated				= 0x00000001,	///< Newly created package, not saved yet. In editor only.
	PKG_ClientOptional				= 0x00000002,	///< Purely optional for clients.
	PKG_ServerSideOnly				= 0x00000004,   ///< Only needed on the server side.
	PKG_CompiledIn					= 0x00000010,   ///< This package is from "compiled in" classes.
	PKG_ForDiffing					= 0x00000020,	///< This package was loaded just for the purposes of diffing
	PKG_EditorOnly					= 0x00000040,	///< This is editor-only package (for example: editor module script package)
	PKG_Developer					= 0x00000080,	///< Developer module
	PKG_UncookedOnly				= 0x00000100,	///< Loaded only in uncooked builds (i.e. runtime in editor)
	PKG_Cooked						= 0x00000200,	///< Package is cooked
	PKG_ContainsNoAsset				= 0x00000400,	///< Package doesn't contain any asset object (although asset tags can be present)
	PKG_NotExternallyReferenceable  = 0x00000800,   ///< Objects in this package cannot be referenced in a different plugin or mount point (i.e /Game -> /Engine)
	PKG_AccessSpecifierEpicInternal = 0x00001000,	///< Objects in this package can only be referenced in a different plugin or mount point by Epic
	PKG_UnversionedProperties		= 0x00002000,   ///< Uses unversioned property serialization instead of versioned tagged property serialization
	PKG_ContainsMapData				= 0x00004000,   ///< Contains map data (UObjects only referenced by a single ULevel) but is stored in a different package
	PKG_IsSaving					= 0x00008000,	///< Temporarily set on a package while it is being saved.
	PKG_Compiling					= 0x00010000,	///< package is currently being compiled
	PKG_ContainsMap					= 0x00020000,	///< Set if the package contains a ULevel/ UWorld object
	PKG_RequiresLocalizationGather	= 0x00040000,	///< Set if the package contains any data to be gathered by localization
	PKG_LoadUncooked				= 0x00080000,	///< This package must be loaded uncooked from IoStore/ZenStore (set only when cooking, to communicate to zenstore via the PackageStoreOptimizer)
	PKG_PlayInEditor				= 0x00100000,	///< Set if the package was created for the purpose of PIE
	PKG_ContainsScript				= 0x00200000,	///< Package is allowed to contain UClass objects
	PKG_DisallowExport				= 0x00400000,	///< Editor should not export asset in this package
//	PKG_Unused						= 0x00800000,
//	PKG_Unused						= 0x01000000,	
//	PKG_Unused						= 0x02000000,	
//	PKG_Unused						= 0x04000000,
	PKG_CookGenerated				= 0x08000000,	///< This package was generated by the cooker and does not exist in the WorkspaceDomain
	PKG_DynamicImports UE_DEPRECATED(5.8, "This flag is now obsolete.") = 0x10000000,	///< Obsolete
	PKG_RuntimeGenerated			= 0x20000000,	///< This package contains elements that are runtime generated, and may not follow standard loading order rules
	PKG_ReloadingForCooker			= 0x40000000,   ///< This package is reloading in the cooker, try to avoid getting data we will never need. We won't save this package.
	PKG_FilterEditorOnly			= 0x80000000,	///< Package has editor-only data filtered out

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EPackageFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

// Transient Flags are cleared when serializing to or from PackageFileSummary
#define PKG_TransientFlags	(PKG_NewlyCreated | PKG_IsSaving | PKG_ReloadingForCooker)

/** Flag mask that indicates if this package is a package that exists in memory only. */
#define PKG_InMemoryOnly	(EPackageFlags)(PKG_CompiledIn | PKG_NewlyCreated)

ENUM_CLASS_FLAGS(EPackageFlags);

// Internal enums.

enum EStaticConstructor				{EC_StaticConstructor};
enum EInternal						{EC_InternalUseOnlyConstructor};
enum ECppProperty					{EC_CppProperty};

/** DO NOT USE. Helper class to invoke specialized hot-reload constructor. */
class FVTableHelper
{
public:
	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	FVTableHelper()
	{
		EnsureRetrievingVTablePtrDuringCtor(TEXT("FVTableHelper()"));
	}
};

/** Empty API definition.  Used as a placeholder parameter when no DLL export/import API is needed for a UObject class */
#define NO_API

/**
 * Flags describing a class.
 *
 * This MUST be kept in sync with EClassFlags defined in
 * Engine\Source\Programs\Shared\EpicGames.Core\UnrealEngineTypes.cs
 */
enum EClassFlags
{
	/** No Flags */
	CLASS_None				  = 0x00000000u,
	/** Class is abstract and can't be instantiated directly. */
	CLASS_Abstract            = 0x00000001u,
	/** Save object configuration only to Default INIs, never to local INIs. Must be combined with CLASS_Config */
	CLASS_DefaultConfig		  = 0x00000002u,
	/** Load object configuration at construction time. */
	CLASS_Config			  = 0x00000004u,
	/** This object type can't be saved; null it out at save time. */
	CLASS_Transient			  = 0x00000008u,
	/** This object type may not be available in certain context. (i.e. game runtime or in certain configuration). Optional class data is saved separately to other object types. (i.e. might use sidecar files) */
	CLASS_Optional            = 0x00000010u,
	/** */
	CLASS_MatchedSerializers  = 0x00000020u,
	/** Indicates that the config settings for this class will be saved to Project/User*.ini (similar to CLASS_GlobalUserConfig) */
	CLASS_ProjectUserConfig	  = 0x00000040u,
	/** Class is a native class - native interfaces will have CLASS_Native set, but not RF_MarkAsNative */
	CLASS_Native			  = 0x00000080u,
	/** Class can contain partials */
	CLASS_Partial			  = 0x00000100u,

	/** Do not allow users to create in the editor. */
	CLASS_NotPlaceable        = 0x00000200u,
	/** Handle object configuration on a per-object basis, rather than per-class. */
	CLASS_PerObjectConfig     = 0x00000400u,
	
	/** Whether SetUpRuntimeReplicationData still needs to be called for this class */
	CLASS_ReplicationDataIsSetUp = 0x00000800u,
	
	/** Class can be constructed from editinline New button. */
	CLASS_EditInlineNew		  = 0x00001000u,
	/** Display properties in the editor without using categories. */
	CLASS_CollapseCategories  = 0x00002000u,
	/** Class is an interface **/
	CLASS_Interface           = 0x00004000u,
	/**  Config for this class is overridden in platform inis, reload when previewing platforms **/
	CLASS_PerPlatformConfig   = 0x00008000u,
	/** all properties and functions in this class are const and should be exported as const */
	CLASS_Const			      = 0x00010000u,

	/** Class flag indicating objects of this class need deferred dependency loading */
	CLASS_NeedsDeferredDependencyLoading = 0x00020000u,
	
	/** Indicates that the class was created from blueprint source material */
	CLASS_CompiledFromBlueprint  = 0x00040000u,

	/** Indicates that only the bare minimum bits of this class should be DLL exported/imported */
	CLASS_MinimalAPI	      = 0x00080000u,
	
	/** Indicates this class must be DLL exported/imported (along with all of it's members) */
	CLASS_RequiredAPI	      = 0x00100000u,

	/** Indicates that references to this class default to instanced. Used to be subclasses of UComponent, but now can be any UObject */
	CLASS_DefaultToInstanced  = 0x00200000u,

	/** Indicates that the parent token stream has been merged with ours. */
	CLASS_TokenStreamAssembled  = 0x00400000u,
	/** Class has component properties. */
	CLASS_HasInstancedReference= 0x00800000u,
	/** Don't show this class in the editor class browser or edit inline new menus. */
	CLASS_Hidden			  = 0x01000000u,
	/** Don't save objects of this class when serializing */
	CLASS_Deprecated		  = 0x02000000u,
	/** Class not shown in editor drop down for class selection */
	CLASS_HideDropDown		  = 0x04000000u,
	/** Class settings are saved to <AppData>/..../Blah.ini (as opposed to CLASS_DefaultConfig) */
	CLASS_GlobalUserConfig	  = 0x08000000u,
	/** Class was declared directly in C++ and has no boilerplate generated by UnrealHeaderTool */
	CLASS_Intrinsic			  = 0x10000000u,
	/** Class has already been constructed (maybe in a previous DLL version before hot-reload). */
	CLASS_Constructed		  = 0x20000000u,
	/** Indicates that object configuration will not check against ini base/defaults when serialized */
	CLASS_ConfigDoNotCheckDefaults = 0x40000000u,
	/** Class has been consigned to oblivion as part of a blueprint recompile, and a newer version currently exists. */
	CLASS_NewerVersionExists  = 0x80000000u,

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EClassFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

// Declare bitwise operators to allow EClassFlags to be combined but still retain type safety
ENUM_CLASS_FLAGS(EClassFlags);

/** Flags to inherit from base class */
#define CLASS_Inherit ((EClassFlags)(CLASS_Transient | CLASS_Optional | CLASS_DefaultConfig | CLASS_Config | CLASS_PerObjectConfig | CLASS_ConfigDoNotCheckDefaults | CLASS_NotPlaceable \
						| CLASS_Const | CLASS_HasInstancedReference | CLASS_Deprecated | CLASS_DefaultToInstanced | CLASS_GlobalUserConfig | CLASS_ProjectUserConfig | CLASS_PerPlatformConfig | CLASS_NeedsDeferredDependencyLoading))

/** These flags will be cleared by the compiler when the class is parsed during script compilation */
#define CLASS_RecompilerClear ((EClassFlags)(CLASS_Inherit | CLASS_Abstract | CLASS_Native | CLASS_Intrinsic | CLASS_TokenStreamAssembled))

/** These flags will never be loaded or saved */
#define CLASS_ShouldNeverBeLoaded ((EClassFlags)(CLASS_Native | CLASS_Optional | CLASS_Intrinsic | CLASS_TokenStreamAssembled | CLASS_ReplicationDataIsSetUp))

/** These flags will be inherited from the base class only for non-intrinsic classes */
#define CLASS_ScriptInherit ((EClassFlags)(CLASS_Inherit | CLASS_EditInlineNew | CLASS_CollapseCategories))

/** This is used as a mask for the flags put into generated code for "compiled in" classes. */
#define CLASS_SaveInCompiledInClasses ((EClassFlags)(\
	CLASS_Abstract | \
	CLASS_DefaultConfig | \
	CLASS_GlobalUserConfig | \
	CLASS_ProjectUserConfig | \
	CLASS_PerPlatformConfig | \
	CLASS_Config | \
	CLASS_Transient | \
	CLASS_Optional | \
	CLASS_Native | \
	CLASS_NotPlaceable | \
	CLASS_PerObjectConfig | \
	CLASS_PerPlatformConfig | \
	CLASS_ConfigDoNotCheckDefaults | \
	CLASS_EditInlineNew | \
	CLASS_CollapseCategories | \
	CLASS_Interface | \
	CLASS_DefaultToInstanced | \
	CLASS_HasInstancedReference | \
	CLASS_Hidden | \
	CLASS_Deprecated | \
	CLASS_HideDropDown | \
	CLASS_Intrinsic | \
	CLASS_Const | \
	CLASS_MinimalAPI | \
	CLASS_RequiredAPI | \
	CLASS_MatchedSerializers | \
	CLASS_NeedsDeferredDependencyLoading))

#define CLASS_AllFlags ((EClassFlags)0xFFFFFFFFu)


/**
 * Flags used for quickly casting classes of certain types; all class cast flags are inherited
 *
 * This MUST be kept in sync with EClassCastFlags defined in
 * Engine\Source\Programs\Shared\EpicGames.Core\UnrealEngineTypes.cs
 */
enum EClassCastFlags : uint64
{
	CASTCLASS_None = 0x0000000000000000,

	CASTCLASS_UField						= 0x0000000000000001,
	CASTCLASS_FInt8Property					= 0x0000000000000002,
	CASTCLASS_UEnum							= 0x0000000000000004,
	CASTCLASS_UStruct						= 0x0000000000000008,
	CASTCLASS_UScriptStruct					= 0x0000000000000010,
	CASTCLASS_UClass						= 0x0000000000000020,
	CASTCLASS_FByteProperty					= 0x0000000000000040,
	CASTCLASS_FIntProperty					= 0x0000000000000080,
	CASTCLASS_FFloatProperty				= 0x0000000000000100,
	CASTCLASS_FUInt64Property				= 0x0000000000000200,
	CASTCLASS_FClassProperty				= 0x0000000000000400,
	CASTCLASS_FUInt32Property				= 0x0000000000000800,
	CASTCLASS_FInterfaceProperty			= 0x0000000000001000,
	CASTCLASS_FNameProperty					= 0x0000000000002000,
	CASTCLASS_FStrProperty					= 0x0000000000004000,
	CASTCLASS_FProperty						= 0x0000000000008000,
	CASTCLASS_FObjectProperty				= 0x0000000000010000,
	CASTCLASS_FBoolProperty					= 0x0000000000020000,
	CASTCLASS_FUInt16Property				= 0x0000000000040000,
	CASTCLASS_UFunction						= 0x0000000000080000,
	CASTCLASS_FStructProperty				= 0x0000000000100000,
	CASTCLASS_FArrayProperty				= 0x0000000000200000,
	CASTCLASS_FInt64Property				= 0x0000000000400000,
	CASTCLASS_FDelegateProperty				= 0x0000000000800000,
	CASTCLASS_FNumericProperty				= 0x0000000001000000,
	CASTCLASS_FMulticastDelegateProperty	= 0x0000000002000000,
	CASTCLASS_FObjectPropertyBase			= 0x0000000004000000,
	CASTCLASS_FWeakObjectProperty			= 0x0000000008000000,
	CASTCLASS_FLazyObjectProperty			= 0x0000000010000000,
	CASTCLASS_FSoftObjectProperty			= 0x0000000020000000,
	CASTCLASS_FTextProperty					= 0x0000000040000000,
	CASTCLASS_FInt16Property				= 0x0000000080000000,
	CASTCLASS_FDoubleProperty				= 0x0000000100000000,
	CASTCLASS_FSoftClassProperty			= 0x0000000200000000,
	CASTCLASS_UPackage						= 0x0000000400000000,
	CASTCLASS_ULevel						= 0x0000000800000000,
	CASTCLASS_AActor						= 0x0000001000000000,
	CASTCLASS_APlayerController				= 0x0000002000000000,
	CASTCLASS_APawn							= 0x0000004000000000,
	CASTCLASS_USceneComponent				= 0x0000008000000000,
	CASTCLASS_UPrimitiveComponent			= 0x0000010000000000,
	CASTCLASS_USkinnedMeshComponent			= 0x0000020000000000,
	CASTCLASS_USkeletalMeshComponent		= 0x0000040000000000,
	CASTCLASS_UBlueprint					= 0x0000080000000000,
	CASTCLASS_UDelegateFunction				= 0x0000100000000000,
	CASTCLASS_UStaticMeshComponent			= 0x0000200000000000,
	CASTCLASS_FMapProperty					= 0x0000400000000000,
	CASTCLASS_FSetProperty					= 0x0000800000000000,
	CASTCLASS_FEnumProperty					= 0x0001000000000000,
	CASTCLASS_USparseDelegateFunction		= 0x0002000000000000,
	CASTCLASS_FMulticastInlineDelegateProperty	= 0x0004000000000000,
	CASTCLASS_FMulticastSparseDelegateProperty	= 0x0008000000000000,
	CASTCLASS_FFieldPathProperty			= 0x0010000000000000,
	//										= 0x0020000000000000,
	//										= 0x0040000000000000,
	CASTCLASS_FLargeWorldCoordinatesRealProperty = 0x0080000000000000,
	CASTCLASS_FOptionalProperty				= 0x0100000000000000,
	CASTCLASS_FVValueProperty				= 0x0200000000000000,
	CASTCLASS_FVRestValueProperty			= 0x0400000000000000,
	CASTCLASS_FVerseStringProperty			= 0x0800000000000000,
	CASTCLASS_FUtf8StrProperty				= 0x1000000000000000,
	CASTCLASS_FAnsiStrProperty				= 0x2000000000000000,
	CASTCLASS_FVCellProperty				= 0x4000000000000000,
	//										= 0x8000000000000000,

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EClassCastFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

#define CASTCLASS_AllFlags ((EClassCastFlags)0xFFFFFFFFFFFFFFFF)

ENUM_CLASS_FLAGS(EClassCastFlags)


/**
 * Flags associated with each property in a class, overriding the
 * property's default behavior.
 * @warning When adding one here, please update ParsePropertyFlags()
 * 
 * This MUST be kept in sync with EPackageFlags defined in
 * Engine\Source\Programs\Shared\EpicGames.Core\UnrealEngineTypes.cs
 */
enum EPropertyFlags : uint64
{
	CPF_None = 0,

	CPF_Edit							= 0x0000000000000001,	///< Property is user-settable in the editor.
	CPF_ConstParm						= 0x0000000000000002,	///< This is a constant function parameter
	CPF_BlueprintVisible				= 0x0000000000000004,	///< This property can be read by blueprint code
	CPF_ExportObject					= 0x0000000000000008,	///< Object can be exported with actor.
	CPF_BlueprintReadOnly				= 0x0000000000000010,	///< This property cannot be modified by blueprint code
	CPF_Net								= 0x0000000000000020,	///< Property is relevant to network replication.
	CPF_EditFixedSize					= 0x0000000000000040,	///< Indicates that elements of an array can be modified, but its size cannot be changed.
	CPF_Parm							= 0x0000000000000080,	///< Function/When call parameter.
	CPF_OutParm							= 0x0000000000000100,	///< Value is copied out after function call.
	CPF_ZeroConstructor					= 0x0000000000000200,	///< memset is fine for construction
	CPF_ReturnParm						= 0x0000000000000400,	///< Return value.
	CPF_DisableEditOnTemplate			= 0x0000000000000800,	///< Disable editing of this property on an archetype/sub-blueprint
	CPF_NonNullable						= 0x0000000000001000,	///< Object property can never be null
	CPF_Transient   					= 0x0000000000002000,	///< Property is transient: shouldn't be saved or loaded, except for Blueprint CDOs.
	CPF_Config      					= 0x0000000000004000,	///< Property should be loaded/saved as permanent profile.
	CPF_RequiredParm					= 0x0000000000008000,	///< Parameter must be linked explicitly in blueprint. Leaving the parameter out results in a compile error. 
	CPF_DisableEditOnInstance			= 0x0000000000010000,	///< Disable editing on an instance of this class
	CPF_EditConst   					= 0x0000000000020000,	///< Property is uneditable in the editor.
	CPF_GlobalConfig					= 0x0000000000040000,	///< Load config from base class, not subclass.
	CPF_InstancedReference				= 0x0000000000080000,	///< Property is a component references.
	CPF_ExperimentalExternalObjects		= 0x0000000000100000,	///< Property saves objects in separate files, breaks hard links and reload them based on discovery.
	CPF_DuplicateTransient				= 0x0000000000200000,	///< Property should always be reset to the default value during any type of duplication (copy/paste, binary duplication, etc.)
	//CPF_								= 0x0000000000400000,	///< 
	//CPF_    							= 0x0000000000800000,	///< 
	CPF_SaveGame						= 0x0000000001000000,	///< Property should be serialized for save games, this is only checked for game-specific archives with ArIsSaveGame
	CPF_NoClear							= 0x0000000002000000,	///< Hide clear button.
	CPF_Virtual							= 0x0000000004000000,	///< Property is defined on an interface and does not include a useful Offset_Internal.
	CPF_ReferenceParm					= 0x0000000008000000,	///< Value is passed by reference; CPF_OutParam and CPF_Param should also be set.
	CPF_BlueprintAssignable				= 0x0000000010000000,	///< MC Delegates only.  Property should be exposed for assigning in blueprint code
	CPF_Deprecated  					= 0x0000000020000000,	///< Property is deprecated.  Read it from an archive, but don't save it.
	CPF_IsPlainOldData					= 0x0000000040000000,	///< If this is set, then the property can be memcopied instead of CopyCompleteValue / CopySingleValue
	CPF_RepSkip							= 0x0000000080000000,	///< Not replicated. For non replicated properties in replicated structs 
	CPF_RepNotify						= 0x0000000100000000,	///< Notify actors when a property is replicated
	CPF_Interp							= 0x0000000200000000,	///< interpolatable property for use with cinematics
	CPF_NonTransactional				= 0x0000000400000000,	///< Property isn't transacted
	CPF_EditorOnly						= 0x0000000800000000,	///< Property should only be loaded in the editor
	CPF_NoDestructor					= 0x0000001000000000,	///< No destructor
	//CPF_								= 0x0000002000000000,	///<
	CPF_AutoWeak						= 0x0000004000000000,	///< Only used for weak pointers, means the export type is autoweak
	CPF_ContainsInstancedReference		= 0x0000008000000000,	///< Property contains component references.
	CPF_AssetRegistrySearchable			= 0x0000010000000000,	///< asset instances will add properties with this flag to the asset registry automatically
	CPF_SimpleDisplay					= 0x0000020000000000,	///< The property is visible by default in the editor details view
	CPF_AdvancedDisplay					= 0x0000040000000000,	///< The property is advanced and not visible by default in the editor details view
	CPF_Protected						= 0x0000080000000000,	///< property is protected from the perspective of script
	CPF_BlueprintCallable				= 0x0000100000000000,	///< MC Delegates only.  Property should be exposed for calling in blueprint code
	CPF_BlueprintAuthorityOnly			= 0x0000200000000000,	///< MC Delegates only.  This delegate accepts (only in blueprint) only events with BlueprintAuthorityOnly.
	CPF_TextExportTransient				= 0x0000400000000000,	///< Property shouldn't be exported to text format (e.g. copy/paste)
	CPF_NonPIEDuplicateTransient		= 0x0000800000000000,	///< Property should only be copied in PIE
	CPF_ExposeOnSpawn					= 0x0001000000000000,	///< Property is exposed on spawn
	CPF_PersistentInstance				= 0x0002000000000000,	///< A object referenced by the property is duplicated like a component. (Each actor should have an own instance.)
	CPF_UObjectWrapper					= 0x0004000000000000,	///< Property was parsed as a wrapper class like TSubclassOf<T>, FScriptInterface etc., rather than a USomething*
	CPF_HasGetValueTypeHash				= 0x0008000000000000,	///< This property can generate a meaningful hash value.
	CPF_NativeAccessSpecifierPublic		= 0x0010000000000000,	///< Public native access specifier
	CPF_NativeAccessSpecifierProtected	= 0x0020000000000000,	///< Protected native access specifier
	CPF_NativeAccessSpecifierPrivate	= 0x0040000000000000,	///< Private native access specifier
	CPF_SkipSerialization				= 0x0080000000000000,	///< Property shouldn't be serialized, can still be exported to text
	CPF_TObjectPtr						= 0x0100000000000000,	///< Property is a TObjectPtr<T> instead of a USomething*. Need to differentiate between TObjectclassOf and TObjectPtr
	CPF_ExperimentalOverridableLogic	= 0x0200000000000000,	///< ****Experimental*** Property will use different logic to serialize knowing what changes are done against its default use the overridable information provided by the overridable manager on the object
	CPF_ExperimentalAlwaysOverriden		= 0x0400000000000000,	///< ****Experimental*** Property should never inherit from the parent when using overridable serialization
	CPF_ExperimentalNeverOverriden		= 0x0800000000000000,	///< ****Experimental*** Property should never be overridden when using overridable serialization
	CPF_AllowSelfReference				= 0x1000000000000000,	///< Enables the instancing graph self referencing logic, delegates and verse function are already using this
	CPF_ForcePostConstructLink			= 0x2000000000000000,	///< Use to force native UCLASS properties to be included in the PostConstructLink chain in a derived Blueprint class, regardless of whether it differs from the native class constructor-initialized value

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EPropertyFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

/** All Native Access Specifier flags */
#define CPF_NativeAccessSpecifiers	(CPF_NativeAccessSpecifierPublic | CPF_NativeAccessSpecifierProtected | CPF_NativeAccessSpecifierPrivate)

/** All parameter flags */
#define CPF_ParmFlags				(CPF_Parm | CPF_OutParm | CPF_ReturnParm | CPF_RequiredParm | CPF_ReferenceParm | CPF_ConstParm )

/** Flags that are propagated to properties inside containers */
#define CPF_PropagateToArrayInner    (CPF_ExportObject | CPF_PersistentInstance | CPF_InstancedReference | CPF_ContainsInstancedReference | CPF_Config | CPF_EditConst | CPF_Deprecated | CPF_EditorOnly | CPF_AutoWeak | CPF_UObjectWrapper )
#define CPF_PropagateToOptionalInner (CPF_ExportObject | CPF_PersistentInstance | CPF_InstancedReference | CPF_ContainsInstancedReference | CPF_Config | CPF_EditConst | CPF_Deprecated | CPF_EditorOnly | CPF_AutoWeak | CPF_UObjectWrapper | CPF_Edit )
#define CPF_PropagateToMapValue      (CPF_ExportObject | CPF_PersistentInstance | CPF_InstancedReference | CPF_ContainsInstancedReference | CPF_Config | CPF_EditConst | CPF_Deprecated | CPF_EditorOnly | CPF_AutoWeak | CPF_UObjectWrapper | CPF_Edit )
#define CPF_PropagateToMapKey        (CPF_ExportObject | CPF_PersistentInstance | CPF_InstancedReference | CPF_ContainsInstancedReference | CPF_Config | CPF_EditConst | CPF_Deprecated | CPF_EditorOnly | CPF_AutoWeak | CPF_UObjectWrapper | CPF_Edit )
#define CPF_PropagateToSetElement    (CPF_ExportObject | CPF_PersistentInstance | CPF_InstancedReference | CPF_ContainsInstancedReference | CPF_Config | CPF_EditConst | CPF_Deprecated | CPF_EditorOnly | CPF_AutoWeak | CPF_UObjectWrapper | CPF_Edit )

/** The flags that should never be set on interface properties */
#define CPF_InterfaceClearMask		(CPF_ExportObject|CPF_InstancedReference|CPF_ContainsInstancedReference)

/** All the properties that can be stripped for final release console builds */
#define CPF_DevelopmentAssets		(CPF_EditorOnly)

/** All the properties that should never be loaded or saved */
#define CPF_ComputedFlags			(CPF_IsPlainOldData | CPF_NoDestructor | CPF_ZeroConstructor | CPF_HasGetValueTypeHash)

#define CPF_TObjectPtrWrapper 		(CPF_UObjectWrapper | CPF_TObjectPtr)
/** Mask of all property flags */
#define CPF_AllFlags				((EPropertyFlags)0xFFFFFFFFFFFFFFFF)

ENUM_CLASS_FLAGS(EPropertyFlags)

/**
 * Extra flags for array properties.
 */
enum class EArrayPropertyFlags : uint8
{
	None,
	UsesMemoryImageAllocator

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EArrayPropertyFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

ENUM_CLASS_FLAGS(EArrayPropertyFlags)

/**
 * Extra flags for map properties.
 */
enum class EMapPropertyFlags : uint8
{
	None,
	UsesMemoryImageAllocator

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EMapPropertyFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

ENUM_CLASS_FLAGS(EMapPropertyFlags)

enum class EPropertyObjectReferenceType : uint32
{
	None = 0,
	Strong = 1 << 0, // Hard reference to a UObject, keeps the object from being garbage collected
	Weak = 1 << 1,   // Weak reference to a UObject, does not keep the object from being garbage collected, does not become valid again after object is unloaded and reloaded.
	Soft = 1 << 2,	 // Soft path/identity-based reference to a UObject, does not keep the object from being garbage collected, does become valid again after object is unloaded and reloaded.
	Conservative = 1 << 3,	 // Not a real reference type, used to mark native struct serializers which may serialize unknown reference types and to conservatively populate RefLink with such struct properties.

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EPropertyObjectReferenceType. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};
ENUM_CLASS_FLAGS(EPropertyObjectReferenceType);

inline constexpr EPropertyObjectReferenceType EPropertyObjectReferenceType_Any = EPropertyObjectReferenceType::Strong | EPropertyObjectReferenceType::Weak | EPropertyObjectReferenceType::Soft | EPropertyObjectReferenceType::Conservative;

COREUOBJECT_API const TCHAR* LexToString(EPropertyObjectReferenceType Type);

/**
 * Flags describing an object instance
 * When modifying this enum, update the LexToString implementation! 
 */
enum EObjectFlags : uint32
{
	// Do not add new flags unless they truly belong here. There are alternatives.
	// if you change any the bit of any of the RF_Load flags, then you will need legacy serialization
	RF_NoFlags					= 0x00000000,	///< No flags, used to avoid a cast

	// This first group of flags mostly has to do with what kind of object it is. Other than transient, these are the persistent object flags.
	// The garbage collector also tends to look at these.
	RF_Public					=0x00000001,	///< Object is visible outside its package.
	RF_Standalone				=0x00000002,	///< Keep object around for editing even if unreferenced.
	RF_MarkAsNative				=0x00000004,	///< Object (UField) will be marked as native on construction (DO NOT USE THIS FLAG in HasAnyFlags() etc)
	RF_Transactional			=0x00000008,	///< Object is transactional.
	RF_ClassDefaultObject		=0x00000010,	///< This object is used as the default template for all instances of a class. One object is created for each class
	RF_ArchetypeObject			=0x00000020,	///< This object can be used as a template for instancing objects. This is set on all types of object templates
	RF_Transient				=0x00000040,	///< Don't save object.

	// This group of flags is primarily concerned with garbage collection.
	RF_MarkAsRootSet			=0x00000080,	///< Object will be marked as root set on construction and not be garbage collected, even if unreferenced (DO NOT USE THIS FLAG in HasAnyFlags() etc)
	RF_TagGarbageTemp			=0x00000100,	///< This is a temp user flag for various utilities that need to use the garbage collector. The garbage collector itself does not interpret it.

	// The group of flags tracks the stages of the lifetime of a uobject
	RF_NeedInitialization		=0x00000200,	///< This object has not completed its initialization process. Cleared when ~FObjectInitializer completes
	RF_NeedLoad					=0x00000400,	///< During load, indicates object needs loading.
	RF_KeepForCooker UE_DEPRECATED(5.6, "This flag is now obsolete as the functionality is unnecessary.")			=0x00000800,	///< Keep this object during garbage collection because it's still being used by the cooker
	RF_GeneratePublicHash		=0x00000800,	///< During cooking, generate a public hash for this object even is not marked RF_Public.
	RF_NeedPostLoad				=0x00001000,	///< Object needs to be postloaded.
	RF_NeedPostLoadSubobjects	=0x00002000,	///< During load, indicates that the object still needs to instance subobjects and fixup serialized component references
	RF_NewerVersionExists		=0x00004000,	///< Object has been consigned to oblivion due to its owner package being reloaded, and a newer version currently exists
	RF_BeginDestroyed			=0x00008000,	///< BeginDestroy has been called on the object.
	RF_FinishDestroyed			=0x00010000,	///< FinishDestroy has been called on the object.

	// Misc. Flags
	RF_BeingRegenerated			=0x00020000,	///< Flagged on UObjects that are used to create UClasses (e.g. Blueprints) while they are regenerating their UClass on load (See FLinkerLoad::CreateExport()), as well as UClass objects in the midst of being created
	RF_DefaultSubObject			=0x00040000,	///< Flagged on subobject templates that were created in a class constructor, and all instances created from those templates
	RF_WasLoaded				=0x00080000,	///< Flagged on UObjects that were loaded
	RF_TextExportTransient		=0x00100000,	///< Do not export object to text form (e.g. copy/paste). Generally used for sub-objects that can be regenerated from data in their parent object.
	RF_LoadCompleted			=0x00200000,	///< Object has been completely serialized by linkerload at least once. DO NOT USE THIS FLAG, It should be replaced with RF_WasLoaded.
	RF_InheritableComponentTemplate = 0x00400000, ///< Flagged on subobject templates stored inside a class instead of the class default object, they are instanced after default subobjects
	RF_DuplicateTransient		=0x00800000,	///< Object should not be included in any type of duplication (copy/paste, binary duplication, etc.)
	RF_StrongRefOnFrame			=0x01000000,	///< References to this object from persistent function frame are handled as strong ones.
	RF_NonPIEDuplicateTransient	=0x02000000,	///< Object should not be included for duplication unless it's being duplicated for a PIE session
	RF_ImmutableDefaultObject	=0x04000000,	///< Hidden version of the class default object that cannot and will never be mutated
	RF_WillBeLoaded				=0x08000000,	///< This object was constructed during load and will be loaded shortly
	RF_HasExternalPackage		=0x10000000,	///< This object has an external package assigned and should look it up when getting the outermost package
	RF_MigratingAsset			=0x20000000,	///< Objects marked as MigratingAsset can be migrated to another server even if they've been loaded from disk

	// RF_MirroredGarbage is mirrored in EInternalObjectFlags::Garbage because checking the internal flags is much faster for the Garbage Collector
	// while checking the object flags is much faster outside of it where the Object pointer is already available and most likely cached.
	RF_MirroredGarbage			=0x40000000,	///< Garbage from logical point of view and should not be referenced. This flag is mirrored in EInternalObjectFlags as Garbage for performance
	RF_HasDynamicImports		=0x80000000,	///< This object has dynamic imports
	RF_AllocatedInSharedPage UE_DEPRECATED(5.8, "This flag is now obsolete as the functionality is unnecessary.") =0x80000000,	///< Allocated from a ref-counted page shared with other UObjects

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EObjectFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

/** Mask for all object flags */
#define RF_AllFlags				(EObjectFlags)0xffffffff	///< All flags, used mainly for error checking

/** Flags to load from unreal asset files */
#define RF_Load						((EObjectFlags)(RF_Public | RF_Standalone | RF_Transactional | RF_ClassDefaultObject | RF_ArchetypeObject | RF_GeneratePublicHash | RF_DefaultSubObject | RF_TextExportTransient | RF_InheritableComponentTemplate | RF_DuplicateTransient | RF_NonPIEDuplicateTransient | RF_MigratingAsset | RF_HasDynamicImports)) 

/** Sub-objects will inherit these flags from their SuperObject */
#define RF_PropagateToSubObjects	((EObjectFlags)(RF_Public | RF_ArchetypeObject | RF_Transactional | RF_Transient))

ENUM_CLASS_FLAGS(EObjectFlags);

COREUOBJECT_API FString LexToString(EObjectFlags Flags);

/** 
 * Objects flags for internal use (GC, low level UObject code)
 *
 * This MUST be kept in sync with EInternalObjectFlags defined in
 * Engine\Source\Programs\Shared\EpicGames.Core\UnrealEngineTypes.cs
 */
enum class EInternalObjectFlags : uint32
{
	None = 0,

#if UE_WITH_REMOTE_OBJECT_HANDLE
	Remote = 1 << 0,			//< Object is no longer owned by this process
	RemoteReference = 1 << 1,	//< Object referenced by remote process
	Borrowed = 1 << 2,			//< Object that is not owned by this process 
	LocalNotReady = 1 << 3,		//< Object's memory is local but requires additional work before it can be accessed
#else
	Remote = 0,
	RemoteReference = 0,
	Borrowed = 0,
	LocalNotReady = 0,
#endif


	// NOTE that the minimum flag bit index is currently 14 (unless compiling with UE_WITH_REMOTE_OBJECT_HANDLE). 
	// See EInternalObjectFlags_MinFlagBitIndex and UE_ENABLE_FUOBJECT_ITEM_PACKING in UObjectArray.h

	ReachabilityFlag0 = 1 << 14, ///< One of the flags used by Garbage Collector to determine UObject's reachability state
	ReachabilityFlag1 = 1 << 15, ///< One of the flags used by Garbage Collector to determine UObject's reachability state
	ReachabilityFlag2 = 1 << 16, ///< One of the flags used by Garbage Collector to determine UObject's reachability state

	AutoRTFMConstructionAborted = 1 << 17, //< Object was constructed in an AutoRTFM transaction that has been aborted
	// Unused = 1 << 18,
	// Unused = 1 << 19,
	LoaderImport = 1 << 20, ///< Object is ready to be imported by another package during loading
	Garbage = 1 << 21, ///< Garbage from logical point of view and should not be referenced. This flag is mirrored in EObjectFlags as RF_Garbage for performance
	AsyncLoadingPhase1 = 1 << 22, ///< Object is being asynchronously loaded.
	ReachableInCluster = 1 << 23, ///< External reference to object in cluster exists
	ClusterRoot = 1 << 24, ///< Root of a cluster
	Native = 1 << 25, ///< Native (UClass only). 
	Async = 1 << 26, ///< Object exists only on a different thread than the game thread.
	AsyncLoadingPhase2 = 1 << 27, ///< Object is being asynchronously loaded.
	Unreachable = 1 << 28, ///< Object is not reachable on the object graph.
	RefCounted UE_DEPRECATED(5.7, "Use GetRefCount() to determine if a refcount exists instead.") = 1 << 29, ///< Object currently has ref-counts associated with it.
	RootSet = 1 << 30, ///< Object will not be garbage collected, even if unreferenced.
	PendingConstruction = 1u << 31, ///< Object didn't have its class constructor called yet (only the UObjectBase one to initialize its most basic members)

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EInternalObjectFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};
ENUM_CLASS_FLAGS(EInternalObjectFlags);

//~ Make sure these macros are up to date!
#define EInternalObjectFlags_MinFlagBitIndex 14
#define EInternalObjectFlags_RefCountMask 0xFFFFFFFF
#define EInternalObjectFlags_GarbageCollectionKeepFlags (EInternalObjectFlags::Borrowed | EInternalObjectFlags::Native | EInternalObjectFlags::Async | EInternalObjectFlags::AsyncLoadingPhase1 | EInternalObjectFlags::AsyncLoadingPhase2 | EInternalObjectFlags::LoaderImport)
#define EInternalObjectFlags_AllFlags (EInternalObjectFlags::Remote | EInternalObjectFlags::RemoteReference | EInternalObjectFlags::Borrowed | EInternalObjectFlags::LocalNotReady | EInternalObjectFlags::ReachabilityFlag0 | EInternalObjectFlags::ReachabilityFlag1 | EInternalObjectFlags::ReachabilityFlag2 | EInternalObjectFlags::AutoRTFMConstructionAborted | EInternalObjectFlags::LoaderImport | EInternalObjectFlags::Garbage | EInternalObjectFlags::ReachableInCluster | EInternalObjectFlags::ClusterRoot | EInternalObjectFlags::Native | EInternalObjectFlags::Async | EInternalObjectFlags::AsyncLoadingPhase1 | EInternalObjectFlags::AsyncLoadingPhase2 | EInternalObjectFlags::RootSet | EInternalObjectFlags::PendingConstruction | EInternalObjectFlags::Unreachable)
#define EInternalObjectFlags_RootFlags (EInternalObjectFlags::RootSet | EInternalObjectFlags_GarbageCollectionKeepFlags)
#define EInternalObjectFlags_ReachabilityFlags (EInternalObjectFlags::ReachabilityFlag0 | EInternalObjectFlags::ReachabilityFlag1 | EInternalObjectFlags::ReachabilityFlag2 | EInternalObjectFlags::Unreachable)
#define EInternalObjectFlags_AsyncLoading (EInternalObjectFlags::AsyncLoadingPhase1 | EInternalObjectFlags::AsyncLoadingPhase2)

/**
 * Flags describing a UEnum 
 * 
 * This MUST be kept in sync with EEnumFlags defined in
 * Engine\Source\Programs\Shared\EpicGames.Core\UnrealEngineTypes.cs
 */
enum class EEnumFlags : uint8
{
	None,

	Flags = 0x00000001, // Whether the UEnum represents a set of flags
	NewerVersionExists = 0x00000002, // If set, this UEnum has been replaced by a newer version

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EEnumFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

ENUM_CLASS_FLAGS(EEnumFlags)

/*----------------------------------------------------------------------------
	Core types.
----------------------------------------------------------------------------*/

class UObject;
class FProperty;
class FObjectInitializer; 

struct FReferencerInformation 
{
	/** the object that is referencing the target */
	UObject*				Referencer;

	/** the total number of references from Referencer to the target */
	int32						TotalReferences;

	/** the array of UProperties in Referencer which hold references to target */
	TArray<const FProperty*>		ReferencingProperties;

	COREUOBJECT_API FReferencerInformation( UObject* inReferencer );
	COREUOBJECT_API FReferencerInformation( UObject* inReferencer, int32 InReferences, const TArray<const FProperty*>& InProperties );
};

struct FReferencerInformationList
{
	TArray<FReferencerInformation>		InternalReferences;
	TArray<FReferencerInformation>		ExternalReferences;

	COREUOBJECT_API FReferencerInformationList();
	COREUOBJECT_API FReferencerInformationList( const TArray<FReferencerInformation>& InternalRefs, const TArray<FReferencerInformation>& ExternalRefs );
};

/*----------------------------------------------------------------------------
	Core macros.
----------------------------------------------------------------------------*/

// Special prefix for default objects (the UObject in a UClass containing the default values, etc)
#define DEFAULT_OBJECT_PREFIX TEXT("Default__")

///////////////////////////////
/// UObject definition macros
///////////////////////////////

// These macros wrap metadata parsed by the Unreal Header Tool, and are otherwise
// ignored when code containing them is compiled by the C++ compiler
#define UPROPERTY(...)
#define UFUNCTION(...)
#define USTRUCT(...)
#define UPARTIAL(...)
#define UMETA(...)
#define UPARAM(...)
#define UENUM(...)
#define UDELEGATE(...)
#define RIGVM_METHOD(...)
#define VMODULE(...)

// Used to inline generated cpp files from UObject headers
#define UE_INLINE_STRINGIFY(name) #name
#define UE_INLINE_GENERATED_CPP_BY_NAME(name) UE_INLINE_STRINGIFY(name.gen.cpp)

// This pair of macros is used to help implement GENERATED_BODY() and GENERATED_USTRUCT_BODY()
#define BODY_MACRO_COMBINE_INNER(A,B,C,D) A##B##C##D
#define BODY_MACRO_COMBINE(A,B,C,D) BODY_MACRO_COMBINE_INNER(A,B,C,D)

// Include a redundant semicolon at the end of the generated code block, so that intellisense parsers can start parsing
// a new declaration if the line number/generated code is out of date.
#define GENERATED_BODY_LEGACY(...) BODY_MACRO_COMBINE(CURRENT_FILE_ID,_,__LINE__,_GENERATED_BODY_LEGACY);
#define GENERATED_BODY(...) BODY_MACRO_COMBINE(CURRENT_FILE_ID,_,__LINE__,_GENERATED_BODY);

#define GENERATED_USTRUCT_BODY(...) GENERATED_BODY()
#define GENERATED_UCLASS_BODY(...) GENERATED_BODY_LEGACY()
#define GENERATED_UINTERFACE_BODY(...) GENERATED_BODY_LEGACY()
#define GENERATED_IINTERFACE_BODY(...) GENERATED_BODY_LEGACY()

#if UE_BUILD_DOCS || defined(__INTELLISENSE__ )
#define UCLASS(...)
#define VINTERFACES(...)
#else
#define UCLASS(...) BODY_MACRO_COMBINE(CURRENT_FILE_ID,_,__LINE__,_PROLOG)
#define VINTERFACES(...) BODY_MACRO_COMBINE(CURRENT_FILE_ID,_,__LINE__,_VINTERFACES)
#endif

#define UINTERFACE(...) UCLASS()

// This macro is used to declare a thunk function in autogenerated boilerplate code
#define DECLARE_FUNCTION(func) static void func( UObject* Context, FFrame& Stack, RESULT_DECL )

// This macro is used to define a thunk function in autogenerated boilerplate code
#define DEFINE_FUNCTION(func) void func( UObject* Context, FFrame& Stack, RESULT_DECL )

// Macro for defining verse only properties that are the given type with BPVM and another with the new VM
#if WITH_VERSE_VM
#define UE_VERSE_FIELD(T, Name, Initializer) Verse::VRestValue Name {0}
#else
#define UE_VERSE_FIELD(T, Name, Initializer) UE_REMOVE_OPTIONAL_PARENS(T) Name UE_REMOVE_OPTIONAL_PARENS(Initializer)
#endif

// These are used for syntax highlighting and to allow autocomplete hints

namespace UC
{
	// valid keywords for the UCLASS macro
	enum 
	{
		/// This keyword is used to set the actor group that the class is show in, in the editor.
		classGroup,

		/// Declares that instances of this class should always have an outer of the specified class.  This is inherited by subclasses unless overridden.
		Within, /* =OuterClassName */

		/// Exposes this class as a type that can be used for variables in blueprints. This is inherited by subclasses unless overridden.
		BlueprintType,

		/// Prevents this class from being used for variables in blueprints. This is inherited by subclasses unless overridden.
		NotBlueprintType,

		/// Exposes this class as an acceptable base class for creating blueprints. The default is NotBlueprintable, unless inherited otherwise. This is inherited by subclasses.
		Blueprintable,

		/// Specifies that this class is *NOT* an acceptable base class for creating blueprints. The default is NotBlueprintable, unless inherited otherwise. This is inherited by subclasses.
		NotBlueprintable,

		/// This keyword indicates that the class should be accessible outside of it's module, but does not need all methods exported.
		/// It exports only the autogenerated methods required for Cast<>, etc... to work.
		MinimalAPI,

		/// Prevents automatic generation of the constructor declaration.
		customConstructor,

		/// Prevents automatic generation of the FieldNotify declaration.
		CustomFieldNotify,

		/// Class was declared directly in C++ and has no boilerplate generated by UnrealHeaderTool.
		/// DO NOT USE THIS FLAG ON NEW CLASSES.
		Intrinsic,

		/// No autogenerated code will be created for this class; the header is only provided to parse metadata from.
		/// DO NOT USE THIS FLAG ON NEW CLASSES.
		noexport,

		/// Allow users to create and place this class in the editor.  This flag is inherited by subclasses.
		placeable,

		/// This class cannot be placed in the editor (it cancels out an inherited placeable flag).
		notplaceable,

		/// All instances of this class are considered "instanced". Instanced classes (components) are duplicated upon construction. This flag is inherited by subclasses. 
		DefaultToInstanced,

		/// All properties and functions in this class are const and should be exported as const.  This flag is inherited by subclasses.
		Const,

		/// Class is abstract and can't be instantiated directly.
		Abstract,

		/// This class is deprecated and objects of this class won't be saved when serializing.  This flag is inherited by subclasses.
		deprecated,

		/// This class can't be saved; null it out at save time.  This flag is inherited by subclasses.
		Transient,

		/// This class should be saved normally (it cancels out an inherited transient flag).
		nonTransient,

		/// This class is optional and might not be available in certain context. reference from non optional data type is not allowed.
		Optional,

		/// Load object configuration at construction time.  These flags are inherited by subclasses.
		/// Class containing config properties. Usage config=ConfigName or config=inherit (inherits config name from base class).
		config,
		/// Handle object configuration on a per-object basis, rather than per-class. 
		perObjectConfig,
		/// Determine whether on serialize to configs a check should be done on the base/defaults ini's
		configdonotcheckdefaults,

		/// Save object config only to Default INIs, never to local INIs.
		defaultconfig,

		/// Mark the editor config file to load from if loading into this object.
		EditorConfig,

		/// These affect the behavior of the property editor.
		/// Class can be constructed from editinline New button.
		editinlinenew,
		/// Class can't be constructed from editinline New button.
		noteditinlinenew,
		/// Class not shown in editor drop down for class selection.
		hidedropdown,

		/// Shows the specified categories in a property viewer. Usage: showCategories=CategoryName or showCategories=(category0, category1, ...)
		showCategories,
		/// Hides the specified categories in a property viewer. Usage: hideCategories=CategoryName or hideCategories=(category0, category1, ...)
		hideCategories,
		/// Indicates that this class is a wrapper class for a component with little intrinsic functionality (this causes things like hideCategories and showCategories to be ignored if the class is subclassed in a Blueprint)
		ComponentWrapperClass,
		/// Shows the specified function in a property viewer. Usage: showFunctions=FunctionName or showFunctions=(category0, category1, ...)
		showFunctions,
		/// Hides the specified function in a property viewer. Usage: hideFunctions=FunctionName or hideFunctions=(category0, category1, ...)
		hideFunctions,
		/// Specifies which categories should be automatically expanded in a property viewer.
		autoExpandCategories,
		/// Specifies which categories should be automatically collapsed in a property viewer.
		autoCollapseCategories,
		/// Clears the list of auto collapse categories.
		dontAutoCollapseCategories,
		/// Display properties in the editor without using categories.
		collapseCategories,
		/// Display properties in the editor using categories (default behaviour).
		dontCollapseCategories,
		/// Specifies category display order, unspecified will follow default display order.
		prioritizeCategories,

		/// All the properties of the class are hidden in the main display by default, and are only shown in the advanced details section.
		AdvancedClassDisplay,

		/// A root convert limits a sub-class to only be able to convert to child classes of the first root class going up the hierarchy.
		ConversionRoot,

		/// Marks this class as 'experimental' (a totally unsupported and undocumented prototype)
		Experimental,

		/// Marks this class as an 'early access' preview (while not considered production-ready, it's a step beyond 'experimental' and is being provided as a preview of things to come)
		EarlyAccessPreview,

		/// Some properties are stored once per class in a sidecar structure and not on instances of the class
		SparseClassDataType,

		/// Specifies the struct that contains the CustomThunk implementations
		CustomThunkTemplates
	};
}

namespace UI
{
	// valid keywords for the UINTERFACE macro, see the UCLASS versions, above
	enum 
	{
		/// This keyword indicates that the interface should be accessible outside of it's module, but does not need all methods exported.
		/// It exports only the autogenerated methods required for Cast<>, etc... to work.
		MinimalAPI,

		/// Specifies that this interface can be directly implemented by blueprints, this is implied if the interface has any blueprint events.
		Blueprintable,

		/// Specifies that this interface cannot be implemented by blueprints, equivalent to CannotImplementInterfaceInBlueprint metadata.
		NotBlueprintable,

		/// Sets IsConversionRoot metadata flag for this interface.
		ConversionRoot,
	};
}

namespace UF
{
	// valid keywords for the UFUNCTION and UDELEGATE macros
	enum 
	{
		/// This function is designed to be overridden by a blueprint.  Do not provide a body for this function;
		/// the autogenerated code will include a thunk that calls ProcessEvent to execute the overridden body.
		BlueprintImplementableEvent,

		/// This function is designed to be overridden by a blueprint, but also has a native implementation.
		/// Provide a body named [FunctionName]_Implementation instead of [FunctionName]; the autogenerated
		/// code will include a thunk that calls the implementation method when necessary.
		BlueprintNativeEvent,

		/// This function is sealed and cannot be overridden in subclasses.
		/// It is only a valid keyword for events; declare other methods as static or final to indicate that they are sealed.
		SealedEvent,

		/// This function is executable from the command line.
		Exec,

		/// This function is replicated, and executed on servers.  Provide a body named [FunctionName]_Implementation instead of [FunctionName];
		/// the autogenerated code will include a thunk that calls the implementation method when necessary.
		Server,

		/// This function is replicated, and executed on clients.  Provide a body named [FunctionName]_Implementation instead of [FunctionName];
		/// the autogenerated code will include a thunk that calls the implementation method when necessary.
		Client,

		/// This function is both executed locally on the server and replicated to all clients, regardless of the Actor's NetOwner
		NetMulticast,

		/// Replication of calls to this function should be done on a reliable channel.
		/// Only valid when used in conjunction with Client, Server, or NetMulticast
		Reliable,

		/// Replication of calls to this function can be done on an unreliable channel.
		/// Only valid when used in conjunction with Client, Server, or NetMulticast
		Unreliable,

		/// This function will not have visible exec pins by default. Implies BlueprintCallable.
		BlueprintPure,

		/// This function can be called from blueprint code and should be exposed to the user of blueprint editing tools.
		BlueprintCallable,

		/// This function is used as the get accessor for a blueprint exposed property. Implies BlueprintPure and BlueprintCallable.
		BlueprintGetter,

		/// This function is used as the set accessor for a blueprint exposed property. Implies BlueprintCallable.
		BlueprintSetter,

		/// This function will not execute from blueprint code if running on something without network authority
		BlueprintAuthorityOnly,

		/// This function is cosmetic and will not run on dedicated servers
		BlueprintCosmetic,

		/// Indicates that a Blueprint exposed function should not be exposed to the end user
		BlueprintInternalUseOnly,
	
		/// This function can be called in the editor on selected instances via a button in the details panel.
		CallInEditor,

		/// The UnrealHeaderTool code generator will not produce a execFoo thunk for this function; it is up to the user to provide one.
		CustomThunk,

		/// Specifies the category of the function when displayed in blueprint editing tools.
		/// Usage: Category=CategoryName or Category="MajorCategory,SubCategory"
		Category,

		/// Generate a field entry for the NotifyFieldValueChanged interface.
		FieldNotify,

		/// This function must supply a _Validate implementation
		WithValidation,

		/// This function is RPC service request
		ServiceRequest,

		/// This function is RPC service response
		ServiceResponse,
		
		/// [FunctionMetadata]	Marks a UFUNCTION as accepting variadic arguments. Variadic functions may have extra terms they need to emit after the main set of function arguments
		///						These are all considered wildcards so no type checking will be performed on them
		Variadic,

		/// [FunctionMetadata] Indicates the display name of the return value pin
		ReturnDisplayName, 

		/// [FunctionMetadata] Indicates that a particular function parameter is for internal use only, which means it will be both hidden and not connectible.
		InternalUseParam, 

		// [FunctionMetadata] Indicates that the function should appear as blueprint function even if it doesn't return a value.
		ForceAsFunction, 

		/// [FunctionMetadata] Indicates that the function should be ignored when considered for blueprint type promotion
		IgnoreTypePromotion,
	};
}

namespace UP
{
	// valid keywords for the UPROPERTY macro
	enum 
	{
		/// This property is const and should be exported as const.
		Const,

		/// Property should be loaded/saved to ini file as permanent profile.
		Config,

		/// Same as above but load config from base class, not subclass.
		GlobalConfig,

		/// Property should be loaded as localizable text. Implies ReadOnly.
		Localized,

		/// Property is transient: shouldn't be saved, zero-filled at load time.
		Transient,

		/// Property should always be reset to the default value during any type of duplication (copy/paste, binary duplication, etc.)
		DuplicateTransient,

		/// Property should always be reset to the default value unless it's being duplicated for a PIE session - deprecated, use NonPIEDuplicateTransient instead
		NonPIETransient,

		/// Property should always be reset to the default value unless it's being duplicated for a PIE session
		NonPIEDuplicateTransient,

		/// Value is copied out after function call. Only valid on function param declaration.
		Ref,

		/// Object property can be exported with it's owner.
		Export,

		/// Hide clear button in the editor.
		NoClear,

		/// Indicates that elements of an array can be modified, but its size cannot be changed.
		EditFixedSize,

		/// Property is relevant to network replication.
		Replicated,

		/// Property is relevant to network replication. Notify actors when a property is replicated (usage: ReplicatedUsing=FunctionName).
		ReplicatedUsing,

		/// Skip replication (only for struct members and parameters in service request functions).
		NotReplicated,

		/// Interpolatable property for use with cinematics. Always user-settable in the editor.
		Interp,

		/// Property isn't transacted.
		NonTransactional,

		/// Property is a component reference. Implies EditInline and Export.
		Instanced,

		/// MC Delegates only.  Property should be exposed for assigning in blueprints.
		BlueprintAssignable,

		/// Specifies the category of the property. Usage: Category=CategoryName.
		Category,

		/// Properties appear visible by default in a details panel
		SimpleDisplay,

		/// Properties are in the advanced dropdown in a details panel
		AdvancedDisplay,

		/// Indicates that this property can be edited by property windows in the editor
		EditAnywhere,

		/// Indicates that this property can be edited by property windows, but only on instances, not on archetypes
		EditInstanceOnly,

		/// Indicates that this property can be edited by property windows, but only on archetypes
		EditDefaultsOnly,

		/// Indicates that this property is visible in property windows, but cannot be edited at all
		VisibleAnywhere,
		
		/// Indicates that this property is only visible in property windows for instances, not for archetypes, and cannot be edited
		VisibleInstanceOnly,

		/// Indicates that this property is only visible in property windows for archetypes, and cannot be edited
		VisibleDefaultsOnly,

		/// This property can be read by blueprints, but not modified.
		BlueprintReadOnly,

		/// This property has an accessor to return the value. Implies BlueprintReadOnly if BlueprintSetter or BlueprintReadWrite is not specified. (usage: BlueprintGetter=FunctionName).
		BlueprintGetter,

		/// This property can be read or written from a blueprint.
		BlueprintReadWrite,

		/// This property has an accessor to set the value. Implies BlueprintReadWrite. (usage: BlueprintSetter=FunctionName).
		BlueprintSetter,

		/// The AssetRegistrySearchable keyword indicates that this property and it's value will be automatically added
		/// to the asset registry for any asset class instances containing this as a member variable.  It is not legal
		/// to use on struct properties or parameters.
		AssetRegistrySearchable,

		/// Property should be serialized for save games.
		/// This is only checked for game-specific archives with ArIsSaveGame set
		SaveGame,

		/// MC Delegates only.  Property should be exposed for calling in blueprint code
		BlueprintCallable,

		/// MC Delegates only. This delegate accepts (only in blueprint) only events with BlueprintAuthorityOnly.
		BlueprintAuthorityOnly,

		/// Property shouldn't be exported to text format (e.g. copy/paste)
		TextExportTransient,

		/// Property shouldn't be serialized, can still be exported to text
		SkipSerialization,

		/// If true, the self pin should not be shown or connectable regardless of purity, const, etc. similar to InternalUseParam
		HideSelfPin, 

		/// Generate a field entry for the NotifyFieldValueChanged interface.
		FieldNotify,
	};
}

namespace US
{
	// valid keywords for the USTRUCT macro
	enum 
	{
		/// No autogenerated code will be created for this class; the header is only provided to parse metadata from.
		NoExport,

		/// Indicates that this struct should always be serialized as a single unit
		Atomic,

		/// Immutable is only legal in Object.h and is being phased out, do not use on new structs!
		Immutable,

		/// Exposes this struct as a type that can be used for variables in blueprints
		BlueprintType,

		/// Indicates that a BlueprintType struct should not be exposed to the end user
		BlueprintInternalUseOnly,

		/// Indicates that a BlueprintType struct and its derived structs should not be exposed to the end user
		BlueprintInternalUseOnlyHierarchical,
	};
}

// Metadata specifiers
namespace UM
{
	// Metadata usable in any UField (UCLASS(), USTRUCT(), UPROPERTY(), UFUNCTION(), etc...)
	enum
	{
		/// Overrides the automatically generated tooltip from the class comment
		ToolTip,

		/// A short tooltip that is used in some contexts where the full tooltip might be overwhelming (such as the parent class picker dialog)
		ShortTooltip,

		/// A setting to determine validation of tooltips and comments. Needs to be set to "Strict"
		DocumentationPolicy,
	};

	// Metadata usable in UCLASS
	enum
	{
		/// [ClassMetadata] Used for Actor Component classes. If present indicates that it can be spawned by a Blueprint.
		BlueprintSpawnableComponent,

		/// [ClassMetadata] Used for Actor and Component classes. If the native class cannot tick, Blueprint generated classes based this Actor or Component can have bCanEverTick flag overridden even if bCanBlueprintsTickByDefault is false.
		ChildCanTick,

		/// [ClassMetadata] Used for Actor and Component classes. If the native class cannot tick, Blueprint generated classes based this Actor or Component can never tick even if bCanBlueprintsTickByDefault is true.
		ChildCannotTick,
		
		/// [ClassMetadata] Used for objects with hundreds of nested properties like animation data to stop the editor from hanging to build debug data.
		DebugTreeLeaf,

		/// [ClassMetadata] Used to make the first subclass of a class ignore all inherited showCategories and hideCategories commands
		IgnoreCategoryKeywordsInSubclasses,

		/// [ClassMetadata] For BehaviorTree nodes indicates that the class is deprecated and will display a warning when compiled.
		DeprecatedNode,

		/// [ClassMetadata] [PropertyMetadata] [FunctionMetadata] Used in conjunction with DeprecatedNode, DeprecatedProperty, or DeprecatedFunction to customize the warning message displayed to the user.
		DeprecationMessage,

		/// [ClassMetadata] [PropertyMetadata] [FunctionMetadata] The name to display for this class, property, or function instead of auto-generating it from the name.
		DisplayName,

		/// [ClassMetadata] [PropertyMetadata] [FunctionMetadata] The name to use for this class, property, or function when exporting it to a scripting language. May include deprecated names as additional semi-colon separated entries.
		ScriptName,

		/// [ClassMetadata] Specifies that this class is an acceptable base class for creating blueprints.
		IsBlueprintBase,

		/// [ClassMetadata] Comma delimited list of blueprint events that are not be allowed to be overridden in classes of this type
		KismetHideOverrides,

		/// [ClassMetadata] [PropertyMetadata] Used by TObjectPtr properties.  Specifies that references to objects of this class should be eager/lazy/background loaded in editor by default.  Can be overridden by LoadBehavior specified on the referencing property field.
		LoadBehavior,

		/// [ClassMetadata] Specifies interfaces that are not compatible with the class.
		ProhibitedInterfaces,

		/// [ClassMetadata] Used by BlueprintFunctionLibrary classes to restrict the graphs the functions in the library can be used in to the classes specified.
		RestrictedToClasses,

		/// [ClassMetadata] Indicates that when placing blueprint nodes in graphs owned by this class that the hidden world context pin should be visible because the self context of the class cannot
		///                 provide the world context and it must be wired in manually
		ShowWorldContextPin,

		//[ClassMetadata] Do not spawn an object of the class using Generic Create Object node in Blueprint. It makes sense only for a BluprintType class, that is neither Actor, nor ActorComponent.
		DontUseGenericSpawnObject,

		//[ClassMetadata] Expose a proxy object of this class in Async Task node.
		ExposedAsyncProxy,

		//[ClassMetadata] Only valid on Blueprint Function Libraries. Mark the functions in this class as callable on non-game threads in an Animation Blueprint.
		BlueprintThreadSafe,

		/// [ClassMetadata] Indicates the class uses hierarchical data. Used to instantiate hierarchical editing features in details panels
		UsesHierarchy,
	};

	// Metadata usable in USTRUCT
	enum
	{
		/// [StructMetadata] Indicates that the struct has a custom break node (and what the path to the BlueprintCallable UFunction is) that should be used instead of the default BreakStruct node.  
		HasNativeBreak,

		/// [StructMetadata] Indicates that the struct has a custom make node (and what the path to the BlueprintCallable UFunction is) that should be used instead of the default MakeStruct node.  
		HasNativeMake,

		/// [StructMetadata] Pins in Make and Break nodes are hidden by default.
		HiddenByDefault,

		/// [StructMetadata] Indicates that node pins of this struct type cannot be split
		DisableSplitPin,
	};

	// Metadata usable in UPROPERTY
	enum
	{
		/// [PropertyMetadata] Used for Subclass and SoftClass properties.  Indicates whether abstract class types should be shown in the class picker.
		AllowAbstract,

		/// [PropertyMetadata] Used for ComponentReference properties.  Indicates whether other actor that are not in the property outer hierarchy should be shown in the component picker.
		AllowAnyActor,

		/// [PropertyMetadata] Used for FSoftObjectPath, ComponentReference and UClass properties.  Comma delimited list that indicates the class type(s) of assets to be displayed in the asset picker(FSoftObjectPath) or component picker or class viewer (UClass).
		AllowedClasses,

		/// [PropertyMetadata] Used for FVector properties.  It causes a ratio lock to be added when displaying this property in details panels.
		AllowPreserveRatio,

		/// [PropertyMetadata] Indicates that a private member marked as BluperintReadOnly or BlueprintReadWrite should be accessible from blueprints
		AllowPrivateAccess,

		/// [PropertyMetadata] Used for integer properties.  Clamps the valid values that can be entered in the UI to be between 0 and the length of the array specified.
		ArrayClamp,

		/// [PropertyMetadata] Used for SoftObjectPtr/SoftObjectPath properties. Comma separated list of Bundle names used inside PrimaryDataAssets to specify which bundles this reference is part of
		AssetBundles,

		/// [PropertyMetadata] Used for Subclass and SoftClass properties.  Indicates whether the class picker should only show classes that can be used as a base for blueprints.
		BlueprintBaseOnly,

		/// [PropertyMetadata] Property defaults are generated by the Blueprint compiler and will not be copied when CopyPropertiesForUnrelatedObjects is called post-compile.
		BlueprintCompilerGeneratedDefaults,

		/// [PropertyMetadata] Used for float and integer properties.  Specifies the minimum value that may be entered for the property.
		ClampMin,

		/// [PropertyMetadata] Used for float and integer properties.  Specifies the maximum value that may be entered for the property.
		ClampMax,

		/// [PropertyMetadata] Property is serialized to config and we should be able to set it anywhere along the config hierarchy.
		ConfigHierarchyEditable,

		/// [PropertyMetadata] Use by "Config" properties. Indicates that changing the value of this property will not take effect until a restart of the program.
		/// If in the editor, then the user will be prompted to reboot now or later.
		///
		/// This is particularly useful on project settings and for state that is read very early on the engine's boot and cannot be changed once it is running.
		/// 
		/// Example:
		///		UPROPERTY(EditAnywhere, Config, Category = "YourCategory", meta=(ConfigRestartRequired=true))
		///		bool bEnableSomeFeature = false;
		ConfigRestartRequired,

		/// [PropertyMetadata] Used by FDirectoryPath properties. Indicates that the path will be picked using the Slate-style directory picker inside the game Content dir.
		ContentDir,

		/// [PropertyMetadata] Used for float and integer properties.  Specifies the step size when the slider is used (e.g. for Delta = 10, the value will scroll among steps of 10).
		Delta,

		/// [PropertyMetadata] This property is deprecated, any blueprint references to it cause a compilation warning.
		DeprecatedProperty,

		/// [ClassMetadata] [PropertyMetadata] [FunctionMetadata] Used in conjunction with DeprecatedNode, DeprecatedProperty, or DeprecatedFunction to customize the warning message displayed to the user.
		// DeprecationMessage, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the property section)

		/// [ClassMetadata] [PropertyMetadata] [FunctionMetadata] The name to display for this class, property, or function instead of auto-generating it from the name.
		// DisplayName, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the property section)

		/// [ClassMetadata] [PropertyMetadata] [FunctionMetadata] The name to use for this class, property, or function when exporting it to a scripting language. May include deprecated names as additional semi-colon separated entries.
		//ScriptName, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the property section)

		/// [PropertyMetadata] Used by asset properties. Filters out assets containing the specified asset registry tags.
		DisallowedAssetDataTags,

		/// [PropertyMetadata] Used for FSoftObjectPath, ActorComponentReference and UClass properties.  Comma delimited list that indicates the class type(s) of assets that will NOT be displayed in the asset picker (FSoftObjectPath) or component picker or class viewer (UClass).
		DisallowedClasses,

		/// [PropertyMetadata] Indicates that the property should be displayed immediately after the property named in the metadata.
		DisplayAfter,

		/// [PropertyMetadata] The relative order within its category that the property should be displayed in where lower values are sorted first..
		/// If used in conjunction with DisplayAfter, specifies the priority relative to other properties with same DisplayAfter specifier.
		DisplayPriority,

		/// [PropertyMetadata] Indicates that the property is an asset type and it should display the thumbnail of the selected asset.
		DisplayThumbnail,	
	
		/// [PropertyMetadata] Specifies a boolean property that is used to indicate whether editing of this property is disabled.
		EditCondition,

		/// [PropertyMetadata] Specifies a boolean property that is used to indicate whether this property should be hidden when the EditCondition evaluates to false. Requires an EditCondition to be set.
		EditConditionHides,

		/// [PropertyMetadata] Keeps the elements of an array from being reordered by dragging 
		EditFixedOrder,
		
		/// [PropertyMetadata] Used for FSoftObjectPath properties in conjunction with AllowedClasses. Indicates whether only the exact classes specified in AllowedClasses can be used or whether subclasses are valid.
		ExactClass,

		/// [PropertyMetadata] Specifies a list of categories whose functions should be exposed when building a function list in the Blueprint Editor.
		ExposeFunctionCategories,

		/// [PropertyMetadata] Specifies whether the property should be exposed on a Spawn Actor for the class type.
		ExposeOnSpawn,

		/// [PropertyMetadata] Used by FFilePath properties. Indicates the path filter to display in the file picker.
		FilePathFilter,

		/// [PropertyMetadata] Used by FFilePath properties. Indicates that the FilePicker dialog will output a path relative to the game directory when setting the property. An absolute path will be used when outside the game directory.
		RelativeToGameDir,

		/// [PropertyMetadata] Deprecated.
		FixedIncrement,

		/// [PropertyMetaData] Force the specified sibling Editor PropertyNode to refresh itself when this property changes. Useful to trigger an Editor refresh of an unrelated Property when this one is modified in PostEditChangeProperty or similar. If no sibling is specified, all of them will refresh.
		ForceRebuildProperty,

		/// [PropertyMetadata] Used by asset properties. Indicates that the asset pickers should always show engine content
		ForceShowEngineContent,

		/// [PropertyMetadata] Used by asset properties. Indicates that the asset pickers should always show plugin content
		ForceShowPluginContent,

		/// [PropertyMetadata] Used for FColor and FLinearColor properties. Indicates that the Alpha property should be hidden when displaying the property widget in the details.
		HideAlphaChannel,

		/// [PropertyMetadata] Indicates that the property should be hidden in the details panel. Currently only used by events.
		HideInDetailPanel,

		/// [PropertyMetadata] Used for Subclass and SoftClass properties. Specifies to hide the ability to change view options in the class picker
		HideViewOptions,

		/// [PropertyMetadata] Used for bypassing property initialization tests when the property cannot be safely tested in a deterministic fashion. Example: random numbers, guids, etc.
		IgnoreForMemberInitializationTest,

		/// [PropertyMetadata] Signifies that the bool property is only displayed inline as an edit condition toggle in other properties, and should not be shown on its own row.
		InlineEditConditionToggle,

		/// [PropertyMetadata] Somewhat misleading name (because a higher value is less sensitive), but maps directly to the LinearDeltaSensitivity 
		/// parameter in sliders created for float and integer properties. When the property does not have both a UIMin and UIMax to bound it, this
		/// parameter prevents the slider sensitivity from always behaving exponentially (ie slower around 0 and faster elsewhere). Instead, the mouse 
		/// displacement is simply divided by this value before being multiplied by Delta (which defaults to 1 when LinearDeltaSensitivity is specified,
		/// and is still raised to SliderExponent). The value should be a positive int (0 does nothing, negative sign ignored if present).
		LinearDeltaSensitivity,

		/// [ClassMetadata] [PropertyMetadata] Used by TObjectPtr properties.  Allows the referenced object to be eager/lazy/background loaded in editor.
		// LoadBehavior, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the property section)

		/// [PropertyMetadata] Used by FDirectoryPath properties.  Converts the path to a long package name
		LongPackageName,

		/// [PropertyMetadata] Used for Transform/Rotator properties (also works on arrays of them). Indicates that the property should be exposed in the viewport as a movable widget.
		MakeEditWidget,

		/// [PropertyMetadata] For properties in a structure indicates the default value of the property in a blueprint make structure node.
		MakeStructureDefaultValue,

		/// [PropertyMetadata] Used FSoftClassPath properties. Indicates the parent class that the class picker will use when filtering which classes to display.
		MetaClass,

		/// [PropertyMetadata] Used for Subclass and SoftClass properties. Indicates the selected class must implement a specific interface
		MustImplement,

		/// [PropertyMetadata] Used for object properties. Indicates the selected object must be of a class which implements a specific interface
		ObjectMustImplement,

		/// [PropertyMetadata] Used for numeric properties. Stipulates that the value must be a multiple of the metadata value.
		Multiple,

		/// [PropertyMetadata] Used for FString and FText properties.  Indicates the maximum length of the value that can be edited, or <=0 for unlimited.
		MaxLength,

		/// [PropertyMetadata] Used for FString and FText properties.  Indicates that the edit field should be multi-line, allowing entry of newlines.
		MultiLine,

		/// [PropertyMetadata] Used for FString and FText properties.  Indicates that the edit field is a secret field (e.g a password) and entered text will be replaced with dots. Do not use this as your only security measure.  The property data is still stored as plain text. 
		PasswordField,

		/// [PropertyMetadata] Used for array properties. Indicates that the duplicate icon should not be shown for entries of this array in the property panel.
		NoElementDuplicate,

		/// [PropertyMetadata] Property wont have a 'reset to default' button when displayed in property windows
		NoResetToDefault,

		/// [PropertyMetadata] Object properties pointing to an UObject instance whos class is marked editinline will not show their properties inline in property windows. Useful for getting actor components to appear in the component tree but not inline in the root actor details panel.
		NoEditInline,

		/// [PropertyMetadata] Used for integer and float properties. Indicates that the spin box element of the number editing widget should not be displayed.
		NoSpinbox,

		/// [PropertyMetadata] Used for Subclass properties. Indicates whether only placeable classes should be shown in the class picker.
		OnlyPlaceable,

		/// [PropertyMetadata] Used by FDirectoryPath properties. Indicates that the directory dialog will output a relative path when setting the property.
		RelativePath,

		/// [PropertyMetadata] Used by FDirectoryPath properties. Indicates that the directory dialog will output a path relative to the game content directory when setting the property.
		RelativeToGameContentDir,

		/// [PropertyMetadata] Used by asset properties. Filters out asset that do not have matching data registry tags, RowStructure=NameOfDataTableRow could be used to restrict data tables.
		RequiredAssetDataTags,

		/// [PropertyMetadata] [FunctionMetadata] Flag set on a property or function to prevent it being exported to a scripting language.
		ScriptNoExport,

		/// [PropertyMetadata] Used by struct properties. Indicates that the inner properties will not be shown inside an expandable struct, but promoted up a level.
		ShowOnlyInnerProperties,

		/// [PropertyMetadata] Used for Subclass and SoftClass properties. Shows the picker as a tree view instead of as a list
		ShowTreeView,

		/// [PropertyMetadata] Used by numeric properties. Indicates how rapidly the value will grow when moving an unbounded slider.
		SliderExponent,

		/// [PropertyMetadata] Used by arrays of structs. Indicates a single property inside of the struct that should be used as a title summary when the array entry is collapsed.
		TitleProperty,

		/// [PropertyMetadata] Used for float and integer properties.  Specifies the lowest that the value slider should represent.
		UIMin,

		/// [PropertyMetadata] Used for float and integer properties.  Specifies the highest that the value slider should represent.
		UIMax,

		/// [PropertyMetadata] Used on any numeric property to declare the internal units for a property. See UnitConversion.h/cpp for details.
		///                    If editor preferances allow, properties marked up with Units= will be displayed in the user's locale preference,
		///                    and/or in the most appropriate unit for the value's magnitude (ie, showing cm as km, or bytes as megabytes etc).
		Units,

		/// [PropertyMetadata] Used instead of 'Units' metadata to forcibly show a property in a fixed unit, without locale or other conversion
		ForceUnits,

		/// [PropertyMetadata] Used for SoftObjectPtr/SoftObjectPath properties to specify a reference should not be tracked. This reference will not be automatically cooked or saved into the asset registry for redirector/delete fixup.
		Untracked,

		/// [PropertyMetadata] For functions that should be compiled in development mode only.
		DevelopmentOnly, 

		/// [PropertyMetadata] (Internal use only) Used for the latent action manager to fix up a latent action with the VM
		NeedsLatentFixup,

		/// [PropertyMetadata] (Internal use only) Used for the latent action manager to track where it's re-entry should be
		LatentCallbackTarget,

		/// [PropertyMetadata] Causes FString and FName properties to have a limited set of options generated dynamically, e.g. meta=(GetOptions="FuncName"). Supports external static function references via "Module.Class.Function" syntax.
		///
		/// UFUNCTION()
		/// static TArray<FName> FuncName() const; // If the field wants an FName value
		/// static TArray<FString> FuncName() const; // If the field wants an FString value
		GetOptions,

		/// [PropertyMetadata] The property can be exposed as a data pin, but is hidden by default.
		PinHiddenByDefault,

		/// [PropertyMetadata] Used for enum properties to define the subset of valid values as a comma-separated string; values outside of this subset are not made available.
		ValidEnumValues,

		/// [PropertyMetadata] Used for enum properties to define the subset of invalid values as a comma-separated string; values within this subset are not made available.
		InvalidEnumValues,

		/// [PropertyMetadata] Used for enum properties to define the subset of restricted values through a UFunction, e.g. meta=(GetRestrictedEnumValues="FuncName"). 
		///
		/// UFUNCTION()
		/// TArray<FString> FuncName() const;
		GetRestrictedEnumValues,
		
		/// [PropertyMetadata] Causes assets to be filtered through a UFunction, e.g. meta=(GetAssetFilter="FuncName"). The UFunction should return true to exclude the asset.
		///
		/// UFUNCTION()
		/// bool FuncName(const FAssetData& AssetData) const;
		GetAssetFilter,
		
		/// [PropertyMetadata] Causes classes to be filtered through a static UFunction, e.g. meta=(GetClassFilter="FuncName"). The UFunction should return true to exclude the class.
		///
		/// UFUNCTION()
		/// bool FuncName(const UClass* Class) const;
		GetClassFilter,
		
		/// [PropertyMetadata] Get allowed classes through a UFunction, e.g. meta=(GetAllowedClasses="FuncName").
		///
		/// UFUNCTION()
		/// TArray<UClass*> FuncName() const;
		GetAllowedClasses,
		
		/// [PropertyMetadata] Get disallowed classes through a UFunction, e.g. meta=(GetDisallowedClasses="FuncName").
		///
		/// UFUNCTION()
		/// TArray<UClass*> FuncName() const;
		GetDisallowedClasses,

		/// [PropertyMetadata] Used for edit inline UObject* properties, enables detail class customization for instanced UObject properties.
		AllowEditInlineCustomization,

		/// [PropertyMetadata] Forcibly hide the asset picker for the pin corresponding to the property.
		// HidePinAssetPicker, (Commented out so as to avoid duplicate name with version in the Function section, but still show in the property section)
	};

	// Metadata usable in UPROPERTY for customizing the behavior of Persona and UMG
	// TODO: Move this to be contained in those modules specifically?
	enum 
	{
		/// [PropertyMetadata] The property is not exposed as a data pin and is only be editable in the details panel. Applicable only to properties that will be displayed in Persona and UMG.
		NeverAsPin, 

		/// [PropertyMetadata] The property can be exposed as a data pin and is visible by default. Applicable only to properties that will be displayed in Persona and UMG.
		PinShownByDefault, 

		/// [PropertyMetadata] The property is always exposed as a data pin. Applicable only to properties that will be displayed in Persona and UMG.
		AlwaysAsPin, 

		/// [PropertyMetadata] Indicates that the property has custom code to display and should not generate a standard property widget in the details panel. Applicable only to properties that will be displayed in Persona.
		CustomizeProperty,
	};

	// Metadata usable in UPROPERTY for customizing the behavior of Material Expressions
	// TODO: Move this to be contained in that module?
	enum
	{
		/// [PropertyMetadata] Used for float properties in MaterialExpression classes. If the specified FMaterialExpression pin is not connected, this value is used instead.
		OverridingInputProperty,

		/// [PropertyMetadata] Used for FMaterialExpression properties in MaterialExpression classes. If specified the pin need not be connected and the value of the property marked as OverridingInputProperty will be used instead.
		RequiredInput,
	};


	// Metadata usable in UFUNCTION
	enum
	{
		/// [FunctionMetadata] Used with a comma-separated list of parameter names that should show up as advanced pins (requiring UI expansion).
		/// Alternatively you can set a number, which is the number of paramaters from the start that should *not* be marked as advanced (eg 'AdvancedDisplay="2"' will mark all but the first two advanced).
		AdvancedDisplay,

		/// [FunctionMetadata] Indicates that a BlueprintCallable function should use a Call Array Function node and that the parameters specified in the comma delimited list should be treated as wild card array properties.
		ArrayParm,

		/// [FunctionMetadata] Used when ArrayParm has been specified to indicate other function parameters that should be treated as wild card properties linked to the type of the array parameter.
		ArrayTypeDependentParams,

		/// [FunctionMetadata] For reference parameters, indicates that a value should be created to be used for the input if none is linked via BP.
		/// This also allows for inline editing of the default value on some types (take FRotator for instance). Only valid for inputs.
		AutoCreateRefTerm,

		HideAssetPicker UE_DEPRECATED(5.6, "Use HidePinAssetPicker instead."),

		/// [FunctionMetadata] Forcibly hide the asset picker for pins matching any parameter names in this piece of metadata.
		HidePinAssetPicker,

		/// [FunctionMetadata] This function is an internal implementation detail, used to implement another function or node.  It is never directly exposed in a graph.
		BlueprintInternalUseOnly,

		/// [FunctionMetadata] This function is only accessible from within its class and derived classes.
		BlueprintProtected,

		/// [FunctionMetadata] Used for BlueprintCallable functions that have a WorldContext pin to indicate that the function can be called even if the class does not implement the virtual function GetWorld().
		CallableWithoutWorldContext,

		/// [FunctionMetadata] Indicates that a BlueprintCallable function should use the Commutative Associative Binary node.
		CommutativeAssociativeBinaryOperator,

		/// [FunctionMetadata] Indicates that a BlueprintCallable function should display in the compact display mode and the name to use in that mode.
		CompactNodeTitle,

		/// [FunctionMetadata] Used with CustomThunk to declare that a parameter is actually polymorphic
		CustomStructureParam,

		/// [FunctionMetadata] For BlueprintCallable functions indicates that the object property named's default value should be the self context of the node
		DefaultToSelf,

		/// [FunctionMetadata] This function is deprecated, any blueprint references to it cause a compilation warning.
		DeprecatedFunction,

		/// [ClassMetadata] [FunctionMetadata] Used in conjunction with DeprecatedNode or DeprecatedFunction to customize the warning message displayed to the user.
		// DeprecationMessage, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the function section)

		/// [FunctionMetadata] For BlueprintCallable functions indicates that an input/output (determined by whether it is an input/output enum) exec pin should be created for each entry in the enum specified.
		/// Use ReturnValue to refer to the return value of the function. Also works for bools.
		ExpandEnumAsExecs,

		// Synonym for ExpandEnumAsExecs
		ExpandBoolAsExecs,

		/// [ClassMetadata] [PropertyMetadata] [FunctionMetadata] The name to display for this class, property, or function instead of auto-generating it from the name.
		// DisplayName, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the function section)

		/// [ClassMetadata] [PropertyMetadata] [FunctionMetadata] The name to use for this class, property, or function when exporting it to a scripting language. May include deprecated names as additional semi-colon separated entries.
		//ScriptName, (Commented out so as to avoid duplicate name with version in the Class section, but still show in the function section)

		/// [PropertyMetadata] [FunctionMetadata] Flag set on a property or function to prevent it being exported to a scripting language.
		//ScriptNoExport, (Commented out so as to avoid duplicate name with version in the Property section, but still show in the function section)

		/// [FunctionMetadata] Flags a static function taking a struct or or object as its first argument so that it "hoists" the function to be a method of the struct or class when exporting it to a scripting language.
		/// The value is optional, and may specify a name override for the method. May include deprecated names as additional semi-colon separated entries.
		ScriptMethod,

		/// [FunctionMetadata] Used with ScriptMethod to denote that the return value of the function should overwrite the value of the instance that made the call (structs only, equivalent to using UPARAM(ref) on the struct argument).
		ScriptMethodSelfReturn,

		/// [FunctionMetadata] Used with ScriptMethod to denote that the first argument of the function should be considered mutable even if passed as a const reference (structs only, equivalent to using UPARAM(ref) on the struct argument).
		ScriptMethodMutable,

		/// [FunctionMetadata] Flags a static function taking a struct as its first argument so that it "hoists" the function to be an operator of the struct when exporting it to a scripting language.
		/// The value describes the kind of operator using C++ operator syntax (see below), and may contain multiple semi-colon separated values.
		/// The signature of the function depends on the operator type, and additional parameters may be passed as long as they're defaulted and the basic signature requirements are met.
		/// - For the bool conversion operator (bool) the signature must be:
		///		bool FuncName(const FMyStruct&); // FMyStruct may be passed by value rather than const-ref
		/// - For the unary conversion operators (neg(-obj)) the signature must be:
		///		FMyStruct FuncName(const FMyStruct&); // FMyStruct may be passed by value rather than const-ref
		/// - For comparison operators (==, !=, <, <=, >, >=) the signature must be:
		///		bool FuncName(const FMyStruct, OtherType); // OtherType can be any type, FMyStruct may be passed by value rather than const-ref
		///	- For mathematical operators (+, -, *, /, %, &, |, ^, >>, <<) the signature must be:
		///		ReturnType FuncName(const FMyStruct&, OtherType); // ReturnType and OtherType can be any type, FMyStruct may be passed by value rather than const-ref
		///	- For mathematical assignment operators (+=, -=, *=, /=, %=, &=, |=, ^=, >>=, <<=) the signature must be:
		///		FMyStruct FuncName(const FMyStruct&, OtherType); // OtherType can be any type, FMyStruct may be passed by value rather than const-ref
		ScriptOperator,

		/// [FunctionMetadata] Flags a static function returning a value so that it "hoists" the function to be a constant of its host type when exporting it to a scripting language.
		/// The constant will be hosted on the class that owns the function, but ScriptConstantHost can be used to host it on a different type (struct or class).
		/// The value is optional, and may specify a name override for the constant. May include deprecated names as additional semi-colon separated entries.
		ScriptConstant,

		/// [FunctionMetadata] Used with ScriptConstant to override the host type for a constant. Should be the name of a struct or class with no prefix, eg) Vector2D or Actor
		ScriptConstantHost,

		/// [FunctionMetadata] For BlueprintCallable functions indicates that the parameter pin should be hidden from the user's view.
		HidePin,

		/// [FunctionMetadata] For some functions used by async task nodes, specify this parameter should be skipped when exposing pins
		HideSpawnParms,

		/// [FunctionMetadata] For BlueprintCallable functions provides additional keywords to be associated with the function for search purposes.
		Keywords,

		/// [FunctionMetadata] Indicates that a BlueprintCallable function is Latent
		Latent,

		/// [FunctionMetadata] For Latent BlueprintCallable functions indicates which parameter is the LatentInfo parameter
		LatentInfo,

		/// [FunctionMetadata] For BlueprintCallable functions indicates that the material override node should be used
		MaterialParameterCollectionFunction,

		/// [FunctionMetadata] For BlueprintCallable functions indicates that the function should be displayed the same as the implicit Break Struct nodes
		NativeBreakFunc,

		/// [FunctionMetadata] For BlueprintCallable functions indicates that the function should be displayed the same as the implicit Make Struct nodes
		NativeMakeFunc,

		/// [FunctionMetadata] Used by BlueprintCallable functions to indicate that this function is not to be allowed in the Construction Script.
		UnsafeDuringActorConstruction,

		/// [FunctionMetadata] Used by BlueprintCallable functions to indicate which parameter is used to determine the World that the operation is occurring within.
		WorldContext,

		/// [FunctionMetadata] Used only by static BlueprintPure functions from BlueprintLibrary. A cast node will be automatically added for the return type and the type of the first parameter of the function.
		BlueprintAutocast,

		// [FunctionMetadata] Only valid in Blueprint Function Libraries. Mark this function as an exception to the class's general BlueprintThreadSafe metadata.
		NotBlueprintThreadSafe,

		/// [FunctionMetadata] [InterfaceMetadata] Metadata that flags function params that govern what type of object the function returns
		DeterminesOutputType,

		/// [FunctionMetadata] [InterfaceMetadata] Metadata that flags the function output param that will be controlled by the "MD_DynamicOutputType" pin
		DynamicOutputParam,

		/// [FunctionMetadata][InterfaceMetadata] Metadata to identify an DataTable Pin. Depending on which DataTable is selected, we display different RowName options
		DataTablePin,

		/// [FunctionMetadata][InterfaceMetadata] Metadata that flags TSet parameters that will have their type determined at blueprint compile time
		SetParam,

		/// [FunctionMetadata] [InterfaceMetadata] Metadata that flags TMap function parameters that will have their type determined at blueprint compile time
		MapParam,

		/// [FunctionMetadata] [InterfaceMetadata]  Metadata that flags TMap function parameters that will have their key type determined at blueprint compile time
		MapKeyParam,

		/// [FunctionMetadata][InterfaceMetadata] Metadata that flags TMap function parameter that will have their value type determined at blueprint compile time
		MapValueParam,

		/// [FunctionMetadata] [InterfaceMetadata] Metadata that identifies an integral property as a bitmask.
		Bitmask,

		/// [FunctionMetadata] [InterfaceMetadata] Metadata that associates a bitmask property with a bitflag enum.
		BitmaskEnum,

		/// [InterfaceMetadata] Metadata that identifies an enum as a set of explicitly-named bitflags.
		Bitflags,

		/// [InterfaceMetadata] Metadata that signals to the editor that enum values correspond to mask values instead of bitshift (index) values.
		UseEnumValuesAsMaskValuesInEditor,

		/// [InterfaceMetadata] Stub function used internally by animation blueprints
		AnimBlueprintFunction,

		/// [FunctionMetadata] [InterfaceMetadata] Metadata that flags TArray function parameters that will have their type determined at blueprint compile time
		ArrayParam,
	};

	// Metadata usable in UINTERFACE
	enum
	{
		/// [InterfaceMetadata] This interface cannot be implemented by a blueprint (e.g., it has only non-exposed C++ member methods), equivalent to NotBlueprintable
		CannotImplementInterfaceInBlueprint,

		/// [InterfaceMetadata] If this metadata is set to true, then the interface cannot generate generic "Message"
		/// nodes which are used for type-agnostic message passing.
		/// 
		/// This is useful if you want to clean up the blueprint action context menu and force users to use a different method of calling
		/// into your interface.
		///
		/// Example:
		///		UINTERFACE(MinimalAPI, meta=(CannotGenerateMessageNodes=true))
		///		class UFooBarSubsystemInterface : public UInterface { GENERATED_BODY() };
		///
		///	This metadata is read in the BlueprintActionDatabaseImpl::AddClassFunctionActions function
		CannotGenerateMessageNodes
	};
}

#define RELAY_CONSTRUCTOR(TClass, TSuperClass) TClass(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get()) : TSuperClass(ObjectInitializer) {}
#define UE_DELETE_COPY_MOVE_CONSTRUCTORS(TClass) TClass(const TClass&) = delete; TClass(TClass&&) = delete;

#if !USE_COMPILED_IN_NATIVES
#define COMPILED_IN_FLAGS(TStaticFlags) (TStaticFlags& ~(CLASS_Intrinsic))
#else
#define COMPILED_IN_FLAGS(TStaticFlags) (TStaticFlags | CLASS_Intrinsic)
#endif

#define DECLARE_SERIALIZER( TClass ) \
	friend FArchive &operator<<( FArchive& Ar, TClass*& Res ) \
	{ \
		return Ar << (UObject*&)Res; \
	} \
	friend void operator<<(FStructuredArchive::FSlot InSlot, TClass*& Res) \
	{ \
		InSlot << (UObject*&)Res; \
	}

#define IMPLEMENT_FARCHIVE_SERIALIZER( TClass ) void TClass::Serialize(FArchive& Ar) { TClass::Serialize(FStructuredArchiveFromArchive(Ar).GetSlot().EnterRecord()); }
#define IMPLEMENT_FSTRUCTUREDARCHIVE_SERIALIZER( TClass ) void TClass::Serialize(FStructuredArchive::FRecord Record) { FArchiveUObjectFromStructuredArchive Ar(Record.EnterField(TEXT("BaseClassAutoGen"))); TClass::Serialize(Ar.GetArchive()); Ar.Close(); }
#define DECLARE_FARCHIVE_SERIALIZER( TClass, API ) virtual API void Serialize(FArchive& Ar) override;
#define DECLARE_FSTRUCTUREDARCHIVE_SERIALIZER( TClass, API ) virtual API void Serialize(FStructuredArchive::FRecord Record) override;

#if WITH_RELOAD
#define CONSTRUCT_RELOAD_VERSION_INFO(VersionInfo, ...) VersionInfo { __VA_ARGS__ }
#else
#define CONSTRUCT_RELOAD_VERSION_INFO(VersionInfo, ...) VersionInfo()
#endif

/**
* Two-phase construction for types. This is used to handle circular dependencies between types.
*/
enum class ETypeConstructPhase : uint8
{
	Inner,
	Outer,
};
using FTypeConstructFunc = UObject*(ETypeConstructPhase);

/*-----------------------------------------------------------------------------
	Class declaration macros.
-----------------------------------------------------------------------------*/

#define DECLARE_CLASS( TClass, TSuperClass, TStaticFlags, TStaticCastFlags, TPackage, TRequiredAPI ) \
	TRequiredAPI static UClass* GetPrivateStaticClass(ETypeConstructPhase); \
	DECLARE_CLASS2(TClass, TSuperClass, TStaticFlags, TStaticCastFlags, TPackage, GetPrivateStaticClass )

#define DECLARE_CLASS2( TClass, TSuperClass, TStaticFlags, TStaticCastFlags, TPackage, TPrivateAccessor ) \
private: \
	friend struct FUObjectCppClassStaticFunctions; \
	friend struct TUObjectCppClassStaticThunks<TClass>; \
public: \
	/** Bitwise union of #EClassFlags pertaining to this class.*/ \
	static constexpr EClassFlags StaticClassFlags=EClassFlags(TStaticFlags); \
	/** Typedef for the base class ({{ typedef-type }}) */ \
	typedef TSuperClass Super;\
	/** Typedef for {{ typedef-type }}. */ \
	typedef TClass ThisClass;\
	/** Returns a UClass object representing this class at runtime */ \
	UE_REWRITE static UClass* StaticClass() \
	{ \
		return TPrivateAccessor(ETypeConstructPhase::Inner); \
	} \
	/** Returns the package this class belongs in */ \
	UE_REWRITE static const TCHAR* StaticPackage() \
	{ \
		return TPackage; \
	} \
	/** Returns the unique cast flag for this class if any */ \
	UE_REWRITE constexpr static EClassCastFlags StaticClassCastFlags() \
	{ \
		return TStaticCastFlags; \
	} \
	/** Returns all cast flags for this class, including those inherited from parent classes */ \
	UE_REWRITE consteval static EClassCastFlags StaticAllClassCastFlags() \
	{ \
		if constexpr (std::is_same_v<ThisClass, Super>) { return StaticClassCastFlags(); } \
		return StaticClassCastFlags() | Super::StaticAllClassCastFlags(); \
	} \
	/** For internal use only; use StaticConstructObject() to create new objects. */ \
	inline void* operator new(const size_t InSize, EInternal InInternalOnly, UObject* InOuter = GetTransientPackageAsObject(), FName InName = NAME_None, EObjectFlags InSetFlags = RF_NoFlags) \
	{ \
		return StaticAllocateObject(StaticClass(), InOuter, InName, InSetFlags); \
	} \
	/** For internal use only; use StaticConstructObject() to create new objects. */ \
	inline void* operator new( const size_t InSize, EInternal* InMem ) \
	{ \
		return (void*)InMem; \
	} \
	/* Eliminate V1062 warning from PVS-Studio while keeping MSVC and Clang happy. */ \
	inline void operator delete(void* InMem) \
	{ \
		::operator delete(InMem); \
	}

#define DEFINE_FORBIDDEN_DEFAULT_CONSTRUCTOR_CALL(TClass) \
	static_assert(false, "You have to define " #TClass "::" #TClass "() or " #TClass "::" #TClass "(const FObjectInitializer&). This is required by UObject system to work correctly.");

#define DEFINE_DEFAULT_CONSTRUCTOR_CALL(TClass) \
	static void __DefaultConstructor(const FObjectInitializer& X) { new((EInternal*)X.GetObj())TClass; }

#define DEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass) \
	static void __DefaultConstructor(const FObjectInitializer& X) { new((EInternal*)X.GetObj())TClass(X); }

#if CHECK_PUREVIRTUALS
#define DEFINE_ABSTRACT_DEFAULT_CONSTRUCTOR_CALL(TClass) \
	static void __DefaultConstructor(const FObjectInitializer& X) { }

#define DEFINE_ABSTRACT_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass) \
	static void __DefaultConstructor(const FObjectInitializer& X) { }
#else
#define DEFINE_ABSTRACT_DEFAULT_CONSTRUCTOR_CALL(TClass) \
	DEFINE_DEFAULT_CONSTRUCTOR_CALL(TClass)

#define DEFINE_ABSTRACT_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass) \
	DEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass)
#endif

#define DECLARE_VTABLE_PTR_HELPER_CTOR(API, TClass) \
	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */ \
	API TClass(FVTableHelper& Helper);

// Namespace should be in the form of "Whatever::" or blank for no namespace
#define DEFINE_VTABLE_PTR_HELPER_CTOR_NS(Namespace, TClass) \
	Namespace TClass::TClass(FVTableHelper& Helper) : Super(Helper) {};

#define DEFINE_VTABLE_PTR_HELPER_CTOR(TClass) DEFINE_VTABLE_PTR_HELPER_CTOR_NS(, TClass)

#define DEFINE_VTABLE_PTR_HELPER_CTOR_CALLER_DUMMY() \
	static UObject* __VTableCtorCaller(FVTableHelper& Helper) \
	{ \
		return nullptr; \
	}

#if WITH_RELOAD && !CHECK_PUREVIRTUALS
	#define DEFINE_VTABLE_PTR_HELPER_CTOR_CALLER(TClass) \
		static UObject* __VTableCtorCaller(FVTableHelper& Helper) \
		{ \
			return new (EC_InternalUseOnlyConstructor, GetTransientPackageAsObject(), NAME_None, RF_NeedLoad | RF_ClassDefaultObject | RF_TagGarbageTemp) TClass(Helper); \
		}
#else // WITH_RELOAD && !CHECK_PUREVIRTUALS
	#define DEFINE_VTABLE_PTR_HELPER_CTOR_CALLER(TClass) \
		DEFINE_VTABLE_PTR_HELPER_CTOR_CALLER_DUMMY()
#endif // WITH_RELOAD && !CHECK_PUREVIRTUALS

#define DECLARE_CLASS_INTRINSIC_NO_CTOR(TClass,TSuperClass,TStaticFlags,TPackage) \
	DECLARE_CLASS(TClass, TSuperClass, TStaticFlags | CLASS_Intrinsic, CASTCLASS_None, TPackage, NO_API) \
	static void StaticRegisterNatives##TClass() {} \
	DECLARE_SERIALIZER(TClass) \
	UE_DELETE_COPY_MOVE_CONSTRUCTORS(TClass) \
	DEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass) \
	static void IntrinsicClassInit(UClass* Class); \
	static UObject* __VTableCtorCaller(FVTableHelper& Helper) \
	{ \
		return new (EC_InternalUseOnlyConstructor, GetTransientPackageAsObject(), NAME_None, RF_NeedLoad | RF_ClassDefaultObject | RF_TagGarbageTemp) TClass(Helper); \
	}

#define DECLARE_CLASS_INTRINSIC(TClass,TSuperClass,TStaticFlags,TPackage) \
	DECLARE_CLASS_INTRINSIC_API(TClass,TSuperClass,TStaticFlags,TPackage, NO_API)

#define DECLARE_CLASS_INTRINSIC_API(TClass,TSuperClass,TStaticFlags,TPackage, TRequiredAPI) \
	DECLARE_CLASS(TClass,TSuperClass,TStaticFlags|CLASS_Intrinsic,CASTCLASS_None,TPackage,TRequiredAPI) \
	RELAY_CONSTRUCTOR(TClass, TSuperClass) \
	UE_DELETE_COPY_MOVE_CONSTRUCTORS(TClass) \
	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */ \
	TClass(FVTableHelper& Helper) : Super(Helper) {}; \
	static void StaticRegisterNatives##TClass() {} \
	DECLARE_SERIALIZER(TClass) \
	DEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass) \
	static void IntrinsicClassInit(UClass* Class); \
	static UObject* __VTableCtorCaller(FVTableHelper& Helper) \
	{ \
		return new (EC_InternalUseOnlyConstructor, GetTransientPackageAsObject(), NAME_None, RF_NeedLoad | RF_ClassDefaultObject | RF_TagGarbageTemp) TClass(Helper); \
	}

#define DECLARE_CASTED_CLASS_INTRINSIC_WITH_API_NO_CTOR( TClass, TSuperClass, TStaticFlags, TPackage, TStaticCastFlags, TRequiredAPI ) \
	DECLARE_CLASS(TClass, TSuperClass, TStaticFlags | CLASS_Intrinsic, TStaticCastFlags, TPackage, TRequiredAPI) \
	UE_DELETE_COPY_MOVE_CONSTRUCTORS(TClass) \
	static void StaticRegisterNatives##TClass() {} \
	DECLARE_SERIALIZER(TClass) \
	DEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass) \
	static void IntrinsicClassInit(UClass* Class); \
	static UObject* __VTableCtorCaller(FVTableHelper& Helper) \
	{ \
		return new (EC_InternalUseOnlyConstructor, GetTransientPackageAsObject(), NAME_None, RF_NeedLoad | RF_ClassDefaultObject | RF_TagGarbageTemp) TClass(Helper); \
	}

#define DECLARE_CASTED_CLASS_INTRINSIC_WITH_API( TClass, TSuperClass, TStaticFlags, TPackage, TStaticCastFlags, TRequiredAPI ) \
	DECLARE_CLASS(TClass,TSuperClass,TStaticFlags|CLASS_Intrinsic,TStaticCastFlags,TPackage,TRequiredAPI ) \
	RELAY_CONSTRUCTOR(TClass, TSuperClass) \
	UE_DELETE_COPY_MOVE_CONSTRUCTORS(TClass) \
	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */ \
	TClass(FVTableHelper& Helper) : Super(Helper) {}; \
	static void StaticRegisterNatives##TClass() {} \
	DECLARE_SERIALIZER(TClass) \
	DEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass) \
	static void IntrinsicClassInit(UClass* Class); \
	static UObject* __VTableCtorCaller(FVTableHelper& Helper) \
	{ \
		return new (EC_InternalUseOnlyConstructor, GetTransientPackageAsObject(), NAME_None, RF_NeedLoad | RF_ClassDefaultObject | RF_TagGarbageTemp) TClass(Helper); \
	}

#define DECLARE_CASTED_CLASS_INTRINSIC_NO_CTOR_NO_VTABLE_CTOR( TClass, TSuperClass, TStaticFlags, TPackage, TStaticCastFlags, TRequiredAPI ) \
	DECLARE_CLASS(TClass,TSuperClass,TStaticFlags|CLASS_Intrinsic,TStaticCastFlags,TPackage, TRequiredAPI ) \
	UE_DELETE_COPY_MOVE_CONSTRUCTORS(TClass) \
	static void StaticRegisterNatives##TClass() {} \
	DECLARE_SERIALIZER(TClass) \
	DEFINE_DEFAULT_OBJECT_INITIALIZER_CONSTRUCTOR_CALL(TClass) \
	static void IntrinsicClassInit(UClass* Class); \
	static UObject* __VTableCtorCaller(FVTableHelper& Helper) \
	{ \
		return new (EC_InternalUseOnlyConstructor, GetTransientPackageAsObject(), NAME_None, RF_NeedLoad | RF_ClassDefaultObject | RF_TagGarbageTemp) TClass(Helper); \
	}

#define DECLARE_CASTED_CLASS_INTRINSIC_NO_CTOR( TClass, TSuperClass, TStaticFlags, TPackage, TStaticCastFlags, TRequiredAPI ) \
	DECLARE_CASTED_CLASS_INTRINSIC_NO_CTOR_NO_VTABLE_CTOR( TClass, TSuperClass, TStaticFlags, TPackage, TStaticCastFlags, TRequiredAPI ) \
	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */ \
	TClass(FVTableHelper& Helper) : Super(Helper) {}; \


#define DECLARE_CASTED_CLASS_INTRINSIC( TClass, TSuperClass, TStaticFlags, TPackage, TStaticCastFlags ) \
	DECLARE_CASTED_CLASS_INTRINSIC_WITH_API( TClass, TSuperClass, TStaticFlags, TPackage, TStaticCastFlags, NO_API) \

// Declare that objects of class being defined reside within objects of the specified class.
#define DECLARE_WITHIN_INTERNAL( TWithinClass, bCanUseOnCDO ) \
	/** The required type of this object's outer ({{ typedef-type }}) */ \
	typedef class TWithinClass WithinClass;  \
	TWithinClass* GetOuter##TWithinClass() const { return (ensure(bCanUseOnCDO || !HasAnyFlags(RF_ClassDefaultObject)) ? (TWithinClass*)GetOuter() : nullptr); }

#define DECLARE_WITHIN( TWithinClass ) \
	DECLARE_WITHIN_INTERNAL( TWithinClass, false )

#define DECLARE_WITHIN_UPACKAGE() \
	DECLARE_WITHIN_INTERNAL( UPackage, true )

#define UOBJECT_CPPCLASS_STATICFUNCTIONS_ALLCONFIGS(TClass) \
	FUObjectCppClassStaticFunctions::AddReferencedObjectsType(&TClass::AddReferencedObjects)
	/* UObjectCppClassStaticFunctions: Extend this macro with the address of your new static function, if it applies to all configs. */
	/* Order must match the order in the FUObjectCppClassStaticFunctions constructor. */

#if WITH_EDITORONLY_DATA
	#define UOBJECT_CPPCLASS_STATICFUNCTIONS_WITHEDITORONLYDATA(TClass) \
		, FUObjectCppClassStaticFunctions::DeclareCustomVersionsType(&TClass::DeclareCustomVersions) \
		, FUObjectCppClassStaticFunctions::AppendToClassSchemaType(&TClass::AppendToClassSchema) \
		, FUObjectCppClassStaticFunctions::DeclareConstructClassesType(&TClass::DeclareConstructClasses) \
		, FUObjectCppClassStaticFunctions::InjectDynamicImportsForType(&TClass::InjectDynamicImportsFor)
		/* UObjectCppClassStaticFunctions: Extend this macro with the address of your new static function, if it is editor-only. */
		/* Order must match the order in the FUObjectCppClassStaticFunctions constructor. */

#else
	#define UOBJECT_CPPCLASS_STATICFUNCTIONS_WITHEDITORONLYDATA(TClass)
#endif

/**
 * A macro called from the IMPLEMENT_CLASS macro that allows the compiler to report to the UClass constructor
 * the class-specific overrides of UnrealEngine's list of reflected UObject static functions.
 */
#define UOBJECT_CPPCLASS_STATICFUNCTIONS_FORCLASS(TClass) \
	FUObjectCppClassStaticFunctions \
	( \
		UOBJECT_CPPCLASS_STATICFUNCTIONS_ALLCONFIGS(TClass) \
		UOBJECT_CPPCLASS_STATICFUNCTIONS_WITHEDITORONLYDATA(TClass) \
	)

// Used to allow constinit initialization UClass variables outside of CoreUObject in modular 
// builds to take pointers to these methods rather than the methods of UObject, which can't 
// be used in constant expressions in a different dll.
template<typename T>
struct TUObjectCppClassStaticThunks
{
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	static void AddReferencedObjects(UObject* ThisObject, FReferenceCollector& Ar)
	{
		T::AddReferencedObjects(ThisObject, Ar);
	}

#if WITH_EDITORONLY_DATA
	static void DeclareCustomVersions(FArchive& Ar, const UClass* SpecificSubclass)
	{
		T::DeclareCustomVersions(Ar, SpecificSubclass);
	}
	static void AppendToClassSchema(FAppendToClassSchemaContext& Context)
	{
		T::AppendToClassSchema(Context);
	}
	static void DeclareConstructClasses(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass)
	{
		T::DeclareConstructClasses(OutConstructClasses, SpecificSubclass);
	}
	static void InjectDynamicImportsFor(const class FLinkerLoad& Linker, const struct FObjectExport& Export, TArray<struct FDynamicImport>& OutDynamicImports)
	{
		T::InjectDynamicImportsFor(Linker, Export, OutDynamicImports);
	}
#endif // WITH_EDITORONLY_DATA
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
};

/**
 * Collection of the pointers to our specified list of static functions that are defined on a specific c++ class,
 * for reference by the corresponding UClass. The pointers in this structure point to the class's version of that
 * function, if it exists, or to the version on the nearest parent class where it does exist, similar to a virtual
 * function table in c++.
 */
struct FUObjectCppClassStaticFunctions
{
public:
	typedef void (*AddReferencedObjectsType)	(UObject* ThisObject, FReferenceCollector& Ar);
#if WITH_EDITORONLY_DATA
	typedef void (*DeclareCustomVersionsType)	(FArchive& Ar, const UClass* SpecificSubclass);
	typedef void (*AppendToClassSchemaType)		(FAppendToClassSchemaContext& Context);
	typedef void (*DeclareConstructClassesType)	(TArray<FTopLevelAssetPath>& OutConstructClasses, const UClass* SpecificSubclass);
	typedef void (*InjectDynamicImportsForType)	(const class FLinkerLoad& Linker, const struct FObjectExport& Export, TArray<struct FDynamicImport>& OutDynamicImports);
#endif
	// UObjectCppClassStaticFunctions: Extend this list of types with the type of your new static function.

#if UE_WITH_CONSTINIT_UOBJECT
	template<typename TClass>
	static constexpr FUObjectCppClassStaticFunctions ForClass()
	{
#if IS_MONOLITHIC
		using DispatchType = TClass;
#else
		using DispatchType = TUObjectCppClassStaticThunks<TClass>;
#endif // IS_MONOLITHIC
		return FUObjectCppClassStaticFunctions(
			&DispatchType::AddReferencedObjects
	#if WITH_EDITORONLY_DATA
			, &DispatchType::DeclareCustomVersions, &DispatchType::AppendToClassSchema, &DispatchType::DeclareConstructClasses, &DispatchType::InjectDynamicImportsFor
	#endif
		);
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

	explicit constexpr FUObjectCppClassStaticFunctions(AddReferencedObjectsType InAddReferencedObjects
#if WITH_EDITORONLY_DATA
		, DeclareCustomVersionsType InDeclareCustomVersions
		, AppendToClassSchemaType InAppendToClassSchema
		, DeclareConstructClassesType InDeclareConstructClasses
		, InjectDynamicImportsForType InInjectDynamicImportsFor
#endif
	)
		: AddReferencedObjects(InAddReferencedObjects)
#if WITH_EDITORONLY_DATA
		, DeclareCustomVersions(InDeclareCustomVersions)
		, AppendToClassSchema(InAppendToClassSchema)
		, DeclareConstructClasses(InDeclareConstructClasses)
		, InjectDynamicImportsFor(InInjectDynamicImportsFor)
#endif
	{
		// Null elements are not valid in this constructor
		check(InAddReferencedObjects);
#if WITH_EDITORONLY_DATA
		check(InDeclareCustomVersions);
		check(InAppendToClassSchema);
		check(InDeclareConstructClasses);
		check(InInjectDynamicImportsFor);
#endif
		// UObjectCppClassStaticFunctions: Extend the constructor with initializers for your new static function member.
		// Order must match the order in UOBJECT_CPPCLASS_STATICFUNCTIONS_FORCLASS.
	}
public:
	FUObjectCppClassStaticFunctions() = default;
	constexpr FUObjectCppClassStaticFunctions(const FUObjectCppClassStaticFunctions&) = default;
	FUObjectCppClassStaticFunctions& operator=(const FUObjectCppClassStaticFunctions&) = default;
	bool IsInitialized() const
	{
		// All methods that set the functions in this class guarantee that if any function is non-null, they all are.
		// So we only need to check the first one to know whether we are completely initialized.
		return AddReferencedObjects != nullptr;
	}
	void Reset()
	{
		*this = FUObjectCppClassStaticFunctions();
	}

public:
	AddReferencedObjectsType GetAddReferencedObjects() const
	{
		return AddReferencedObjects;
	}
	void SetAddReferencedObjects(AddReferencedObjectsType InAddReferencedObjects)
	{
		check(InAddReferencedObjects != nullptr); // It is not valid to clear single elements (see IsInitialized). Call Reset to clear all elements.
		AddReferencedObjects = InAddReferencedObjects;
	}
#if WITH_EDITORONLY_DATA
	DeclareCustomVersionsType GetDeclareCustomVersions() const
	{
		return DeclareCustomVersions;
	}
	void SetDeclareCustomVersions(DeclareCustomVersionsType InDeclareCustomVersions)
	{
		check(InDeclareCustomVersions != nullptr); // It is not valid to clear single elements (see IsInitialized). Call Reset to clear all elements.
		DeclareCustomVersions = InDeclareCustomVersions;
	}
	AppendToClassSchemaType GetAppendToClassSchema() const
	{
		return AppendToClassSchema;
	}
	void SetAppendToClassSchema(AppendToClassSchemaType InAppendToClassSchema)
	{
		check(InAppendToClassSchema != nullptr); // It is not valid to clear single elements (see IsInitialized). Call Reset to clear all elements.
		AppendToClassSchema = InAppendToClassSchema;
	}
	DeclareConstructClassesType GetDeclareConstructClasses() const
	{
		return DeclareConstructClasses;
	}
	void SetDeclareConstructClasses(DeclareConstructClassesType InDeclareConstructClasses)
	{
		check(InDeclareConstructClasses != nullptr); // It is not valid to clear single elements (see IsInitialized). Call Reset to clear all elements.
		DeclareConstructClasses = InDeclareConstructClasses;
	}
	InjectDynamicImportsForType GetInjectDynamicImportsFor() const
	{
		return InjectDynamicImportsFor;
	}
	void SetInjectDynamicImportsFor(InjectDynamicImportsForType InInjectDynamicImportsFor)
	{
		check(InInjectDynamicImportsFor != nullptr); // It is not valid to clear single elements (see IsInitialized). Call Reset to clear all elements.
		InjectDynamicImportsFor = InInjectDynamicImportsFor;
	}
#endif
	// UObjectCppClassStaticFunctions: Extend the list of accessors for your new static function.

private:
	AddReferencedObjectsType AddReferencedObjects = nullptr;
#if WITH_EDITORONLY_DATA
	DeclareCustomVersionsType DeclareCustomVersions = nullptr;
	AppendToClassSchemaType AppendToClassSchema = nullptr;
	DeclareConstructClassesType DeclareConstructClasses = nullptr;
	InjectDynamicImportsForType InjectDynamicImportsFor = nullptr;
#endif
	// UObjectCppClassStaticFunctions: Extend this list of members with the member for your new static function.

};

// Implement the GetPrivateStaticClass and the registration info but do not auto register the class.  
// This is no longer used by UnrealHeaderTool
#if !UE_WITH_CONSTINIT_UOBJECT
#define IMPLEMENT_CLASS_NO_AUTO_REGISTRATION(TClass) \
	FClassRegistrationInfo Z_Registration_Info_UClass_##TClass; \
	UClass* Z_Construct_UClass##TClass##_Inner() \
	{ \
		if (!Z_Registration_Info_UClass_##TClass.InnerSingleton) \
		{ \
			/* this could be handled with templates, but we want it external to avoid code bloat */ \
			GetPrivateStaticClassBody( \
				TClass::StaticPackage(), \
				(TCHAR*)TEXT(#TClass) + 1 + ((TClass::StaticClassFlags & CLASS_Deprecated) ? 11 : 0), \
				Z_Registration_Info_UClass_##TClass.InnerSingleton, \
				TClass::StaticRegisterNatives##TClass, \
				DataSizeOf<TClass>(), \
				alignof(TClass), \
				TClass::StaticClassFlags, \
				TClass::StaticClassCastFlags(), \
				TClass::StaticConfigName(), \
				(UClass::ClassConstructorType)InternalConstructor<TClass>, \
				(UClass::ClassVTableHelperCtorCallerType)InternalVTableHelperCtorCaller<TClass>, \
				UOBJECT_CPPCLASS_STATICFUNCTIONS_FORCLASS(TClass), \
				&TClass::Super::StaticClass, \
				&TClass::WithinClass::StaticClass \
			); \
		} \
		return Z_Registration_Info_UClass_##TClass.InnerSingleton; \
	} \
	UClass* TClass::GetPrivateStaticClass(ETypeConstructPhase) \
	{ \
		return Z_Construct_UClass##TClass##_Inner(); \
	}
#endif // !UE_WITH_CONSTINIT_UOBJECT

// Register a class at startup time.
#if !UE_WITH_CONSTINIT_UOBJECT
#define IMPLEMENT_CLASS(TClass, TClassCrc) \
	IMPLEMENT_CLASS_NO_AUTO_REGISTRATION(TClass) \
	/* Do not change the AutoInitialize_ without changing LC_SymbolPatterns */ \
	static FRegisterCompiledInInfo AutoInitialize_##TClass(&Z_Construct_UClass_##TClass, TClass::StaticPackage(), TEXT(#TClass), Z_Registration_Info_UClass_##TClass, CONSTRUCT_RELOAD_VERSION_INFO(FClassReloadVersionInfo, sizeof(TClass), TClassCrc));
#endif // !UE_WITH_CONSTINIT_UCLASS

#if UE_WITH_CONSTINIT_UOBJECT
#define IMPLEMENT_INTRINSIC_CLASS(TClass, TRequiredAPI, TSuperClass, TSuperRequiredAPI, TPackage, InitCode) \
	IMPLEMENT_INTRINSIC_CLASS_CONSTINIT(TClass, TRequiredAPI, TSuperClass, TSuperRequiredAPI, TPackage, InitCode)
// Used for intrinsics, this sets up the boiler plate, plus an initialization singleton, which can create properties and GC tokens
#else
#define IMPLEMENT_INTRINSIC_CLASS(TClass, TRequiredAPI, TSuperClass, TSuperRequiredAPI, TPackage, InitCode) \
	TRequiredAPI UClass* Z_Construct_UClass_##TClass(ETypeConstructPhase); \
	extern FClassRegistrationInfo Z_Registration_Info_UClass_##TClass; \
	struct Z_Construct_UClass_##TClass##_Statics \
	{ \
		static UClass* Construct() \
		{ \
			extern TSuperRequiredAPI UClass* Z_Construct_UClass_##TSuperClass(ETypeConstructPhase); \
			UClass* SuperClass = Z_Construct_UClass_##TSuperClass(ETypeConstructPhase::Outer); \
			UClass* Class = TClass::StaticClass(); \
			UObjectForceRegistration(Class); \
			check(Class->GetSuperClass() == SuperClass); \
			TClass::IntrinsicClassInit(Class); \
			Class->StaticLink(); \
			return Class; \
		} \
	}; \
	void TClass::IntrinsicClassInit(UClass* Class) \
	{ \
		InitCode \
	} \
	UClass* Z_Construct_UClass##TClass##_Inner(); \
	UClass* Z_Construct_UClass_##TClass(ETypeConstructPhase Phase) \
	{ \
		if (Phase == ETypeConstructPhase::Inner) \
		{ \
			return Z_Construct_UClass##TClass##_Inner(); \
		} \
		if (!Z_Registration_Info_UClass_##TClass.OuterSingleton) \
		{ \
			Z_Registration_Info_UClass_##TClass.OuterSingleton = Z_Construct_UClass_##TClass##_Statics::Construct();\
		} \
		check(Z_Registration_Info_UClass_##TClass.OuterSingleton->GetClass()); \
		return Z_Registration_Info_UClass_##TClass.OuterSingleton; \
	} \
	IMPLEMENT_CLASS(TClass, 0)

#endif

#if UE_WITH_CONSTINIT_UOBJECT
// CONSTINIT_UOBJECT_TODO: Should intrinsic classes inherit class flags here rather than waiting until Link?
#define IMPLEMENT_INTRINSIC_CLASS_CONSTINIT(TClass, TRequiredAPI, TSuperClass, TSuperRequiredAPI, TPackage, InitCode) \
	static constinit FCppClassTypeInfoStatic Z_StaticCppClassTypeInfo_##TClass = { \
		TCppClassTypeTraits<TClass>::IsAbstract, \
	}; \
	constinit extern UPackage Z_ConstInit_UPackage__Script_##TPackage; \
	constinit extern TNoDestroy<UClass> Z_ConstInit_UClass_UClass; \
	constinit extern TNoDestroy<UClass> Z_ConstInit_UClass_##TSuperClass; \
	constinit TNoDestroy<UClass> Z_ConstInit_UClass_##TClass{NoDestroyConstEval, \
		UE::CodeGen::ConstInit::FObjectParams{ \
			.Flags = RF_Public|RF_Transient|RF_MarkAsNative|RF_MarkAsRootSet, \
			.Class = std::is_same_v<UClass, TClass> ? nullptr : &Z_ConstInit_UClass_UClass, \
			/* Trim the U/A prefix which is still a single byte in UTF8 */ \
			.NameUTF8 = &UTF8TEXT(UE_STRINGIZE(TClass))[1], \
			.Outer = &Z_ConstInit_UPackage__Script_##TPackage, \
		}, \
		UE::CodeGen::ConstInit::FUFieldParams{}, \
		UE::CodeGen::ConstInit::FStructParams{ \
			.Super = &Z_ConstInit_UClass_##TSuperClass, \
			.BaseChain = TIntrinsicClassBaseChain<TClass>::ToArrayView(), \
			.PropertiesSize = DataSizeOf<TClass>(), \
			.MinAlignment = alignof(TClass), \
		}, \
		UE::CodeGen::ConstInit::FClassParams{ \
			.ConfigName = TClass::StaticConfigName(), \
			.ClassConstructor = InternalConstructor<TClass>, \
			.ClassVTableHelperCtorCaller = InternalVTableHelperCtorCaller<TClass>, \
			.CppClassStaticFunctions = UOBJECT_CPPCLASS_STATICFUNCTIONS_FORCLASS(TClass), \
			.ClassFlags = TClass::StaticClassFlags, \
			.ClassCastFlags = TClass::StaticAllClassCastFlags(), \
			.CppClassTypeInfo = &Z_StaticCppClassTypeInfo_##TClass, \
			.ClassWithin = UE::Private::GetConstInitClass<TClass::WithinClass>(), \
		}, \
	}; \
	struct Z_Construct_UClass_##TClass##_Statics { \
		static void Construct() \
		{ \
			UClass* SuperClass = TSuperClass::StaticClass(); \
			UClass* Class = TClass::StaticClass(); \
			InitCode; \
		} \
	}; \
	static FRegisterIntrinsicClass Z_Register_UClass_##TClass{ &Z_ConstInit_UClass_##TClass, &Z_Construct_UClass_##TClass##_Statics::Construct }; \
	UClass* TClass::GetPrivateStaticClass(ETypeConstructPhase) \
	{ \
		return &Z_ConstInit_UClass_##TClass; \
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

#if UE_WITH_CONSTINIT_UOBJECT
#define IMPLEMENT_CORE_INTRINSIC_CLASS(TClass, TSuperClass, InitCode) \
	IMPLEMENT_INTRINSIC_CLASS_CONSTINIT(TClass, COREUOBJECT_API, TSuperClass, COREUOBJECT_API, CoreUObject, InitCode)
#else
#define IMPLEMENT_CORE_INTRINSIC_CLASS(TClass, TSuperClass, InitCode) \
	IMPLEMENT_INTRINSIC_CLASS(TClass, COREUOBJECT_API, TSuperClass, COREUOBJECT_API, CoreUObject, InitCode)
#endif

/** Options to the UObject::Rename() function, bit flag */
typedef uint32 ERenameFlags;

/** Default rename behavior */
#define REN_None					    (0x0000)
/** Rename won't call ResetLoaders or flush async loading. You should pass this if you are renaming a deep subobject and do not need to reset loading for the outer package */
#define REN_ForceNoResetLoaders		    UE_DEPRECATED_MACRO(5.8, "Rename will no longer call ResetLoaders making this flag no longer needed. Prefer REN_AllowPackageLinkerMismatch if you wish to intentionally allow the linker to contain references to objects whose names no longer match what was loaded from disk.") \
										(0x0001) 
/** Just test to make sure that the rename is guaranteed to succeed if an non test rename immediately follows */
#define REN_Test					    (0x0002) 
/** Indicates that the object (and new outer) should not be dirtied */
#define REN_DoNotDirty				    (0x0004) 
/** Don't create an object redirector, even if the class is marked RF_Public */
#define REN_DontCreateRedirectors	    (0x0010) 
/** Don't call Modify() on the objects, so they won't be stored in the transaction buffer */
#define REN_NonTransactional		    (0x0020) 
/** Force unique names across all packages not just within the scope of the new outer */
#define REN_ForceGlobalUnique		    (0x0040) 
/** Prevent renaming of any child generated classes and CDO's in blueprints */
#define REN_SkipGeneratedClasses	    (0x0080) 
/** Prevents renaming from unregistering/registering all components */
#define REN_SkipComponentRegWork	    (0x0100) 
/** Prevents rename from clearing an object's linker when being renamed to a new package */
#define REN_AllowPackageLinkerMismatch	(0x0200) 

/*-----------------------------------------------------------------------------
	Misc.
-----------------------------------------------------------------------------*/

typedef void (*FAsyncCompletionCallback)( UObject* LinkerRoot, void* CallbackUserData );

namespace GameplayTagsManager
{
	enum
	{
		/// Used for filtering by tag widget
		Categories,
		
		/// Used for filtering by tag widget for any parameters of the function that end up as BP pins
		GameplayTagFilter,
	};
}

/*-----------------------------------------------------------------------------
	UObject.
-----------------------------------------------------------------------------*/

namespace UE
{
	/**
	 * Controls how calls to LoadConfig() should be propagated
	 */
	enum ELoadConfigPropagationFlags
	{
		LCPF_None					=	0x0,

		/**
		 * Indicates that the object should read ini values from each section up its class's hierarchy chain;
		 * Useful when calling LoadConfig on an object after it has already been initialized against its archetype
		 */
		LCPF_ReadParentSections		=	0x1,

		/**
		 * Indicates that LoadConfig() should be also be called on the class default objects for all children of the original class.
		 */
		LCPF_PropagateToChildDefaultObjects		=	0x2,

		/**
		 * Indicates that LoadConfig() should be called on all instances of the original class.
		 */
		LCPF_PropagateToInstances	=	0x4,

		/**
		 * Indicates that this object is reloading its config data
		 */
		LCPF_ReloadingConfigData	=	0x8,

		/** 
		 * All flags that should be persisted to propagated recursive calls 
		 */
		LCPF_PersistentFlags		=	LCPF_ReloadingConfigData,
	};
}

/**
 * Helper class used to save and restore information across a StaticAllocateObject over the top of an existing object.
 * Currently only used by UClass
 */
class FRestoreForUObjectOverwrite
{
public:
	/** virtual destructor **/
	virtual ~FRestoreForUObjectOverwrite() = default;
	/** Called once the new object has been reinitialized 
	**/
	virtual void Restore() const=0;
};

#ifndef VALIDATE_CLASS_REPS
#define VALIDATE_CLASS_REPS DO_CHECK
#endif

#if VALIDATE_CLASS_REPS
#define DECLARE_VALIDATE_GENERATED_REP_ENUMS( API ) API virtual void ValidateGeneratedRepEnums(const TArray<struct FRepRecord>& ClassReps) const override;
#else
#define DECLARE_VALIDATE_GENERATED_REP_ENUMS( API )
#endif
