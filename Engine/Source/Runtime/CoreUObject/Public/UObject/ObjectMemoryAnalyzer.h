// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "UObject/UObjectAnnotation.h"

class FOutputDevice;
class UClass;
class UObject;

/** Object mempory usage info */
struct FObjectMemoryUsage
{
	struct EObjFlags
	{
		enum Type
		{
			IsRoot					= 1<<0,
			IsReferencedByRoot		= 1<<1,
			IsProcessed				= 1<<2,
			IsReferencedByNonRoot	= 1<<3,
		};
	};

	/**
	 * default constructor
	 * Default constructor must be the default item
	 */
	FObjectMemoryUsage()
		: InclusiveMemoryUsage(0)
		, ExclusiveMemoryUsage(0)
		, Flags(0)
		, Object(NULL)
	{
	}
	/**
	 * Initialization constructor
	 * @param InMarks marks to initialize to
	 */
	FObjectMemoryUsage(SIZE_T InclusiveMemUsage, SIZE_T ExclusiveMemUsage, uint32 InFlags = 0)
		: InclusiveMemoryUsage(InclusiveMemUsage)
		, ExclusiveMemoryUsage(ExclusiveMemUsage)
		, Flags(InFlags)
		, Object(NULL)
	{
	}
	/**
	 * Determine if this annotation
	 * @return true is this is a default pair. We only check the linker because CheckInvariants rules out bogus combinations
	 */
	UE_FORCEINLINE_HINT bool IsDefault()
	{
		return false;
	}

	UE_FORCEINLINE_HINT bool IsRoot() const { return !!(Flags&EObjFlags::IsRoot); }
	UE_FORCEINLINE_HINT bool IsReferencedByRoot() const { return !!(Flags&EObjFlags::IsReferencedByRoot); }
	UE_FORCEINLINE_HINT bool IsProcessed() const { return !!(Flags&EObjFlags::IsProcessed); }
	UE_FORCEINLINE_HINT bool IsReferencedByNonRoot() const { return !!(Flags&EObjFlags::IsReferencedByNonRoot); }
	
	SIZE_T InclusiveMemoryUsage;
	SIZE_T ExclusiveMemoryUsage;
	SIZE_T InclusiveResourceSize;
	SIZE_T ExclusiveResourceSize;
	uint32 Flags;

	UObject*		 Object;
	TArray<UObject*> RootReferencer;
	TArray<UObject*> NonRootReferencer;
};


/** 
 * Analyzes memory usage of UObjects
 **/
struct FObjectMemoryAnalyzer
{
	/** Flags to modify the memory counting behavior */
	struct EAnalyzeFlags
	{
		enum Type
		{
			/** Accounts for default objects */
			IncludeDefaultObjects	= 1<<0,
		};
	};

	struct ESortKey
	{
		enum Type
		{
			InclusiveSize,
			ExclusiveSize,
			InclusiveResSize,
			ExclusiveResSize,
			InclusiveTotal,
			ExclusiveTotal,
		};
	};

	struct FCompareFSortBySize
	{		
		FCompareFSortBySize( ESortKey::Type InSortKey )
			: SortKey( InSortKey )
		{}

		inline bool operator()( const FObjectMemoryUsage& A, const FObjectMemoryUsage& B ) const
		{
			switch (SortKey)
			{
				case ESortKey::InclusiveSize:		return A.InclusiveMemoryUsage > B.InclusiveMemoryUsage;
				case ESortKey::ExclusiveSize:		return A.ExclusiveMemoryUsage > B.ExclusiveMemoryUsage;
				case ESortKey::InclusiveResSize:	return A.InclusiveResourceSize > B.InclusiveResourceSize;
				case ESortKey::ExclusiveResSize:	return A.ExclusiveResourceSize > B.ExclusiveResourceSize;
				case ESortKey::InclusiveTotal:		return (A.InclusiveMemoryUsage + A.InclusiveResourceSize*1024) > (B.InclusiveMemoryUsage + B.InclusiveResourceSize*1024);
				case ESortKey::ExclusiveTotal:		return (A.ExclusiveMemoryUsage + A.ExclusiveResourceSize*1024) > (B.ExclusiveMemoryUsage + B.ExclusiveResourceSize*1024);
			}

			check(false);
			return false;
		}

		ESortKey::Type SortKey;
	};

	COREUOBJECT_API FObjectMemoryAnalyzer(uint32 Flags = 0);
	COREUOBJECT_API FObjectMemoryAnalyzer(class UClass* BaseClass, uint32 Flags = 0);
	COREUOBJECT_API FObjectMemoryAnalyzer(class UObject* Object, uint32 Flags = 0);
	COREUOBJECT_API FObjectMemoryAnalyzer(const TArray<class UObject*>& ObjectList, uint32 Flags = 0);

	/** Analyzes the memory usage of the specified object */
	COREUOBJECT_API void AnalyzeObject(class UObject* Object);
	/** Analyzes the memory usage of the specified object list */
	COREUOBJECT_API void AnalyzeObjects(const TArray<class UObject*>& ObjectList);
	/** Analyzes the memory usage of all objects with the specified class */
	COREUOBJECT_API void AnalyzeObjects(class UClass* BaseClass);

	/** Returns the results */
	COREUOBJECT_API int32 GetResults(TArray<FObjectMemoryUsage>& Results);
	/** Returns the memory usage of an object not in the result set ( ex. referenced by a result ) */
	COREUOBJECT_API const FObjectMemoryUsage& GetObjectMemoryUsage(class UObject* Obj);

	struct EPrintFlags
	{
		enum Type
		{
			PrintReferences	= 1<<0,
			PrintReferencer = 1<<1,

			PrintAll		= PrintReferences | PrintReferencer,
		};
	};

	COREUOBJECT_API void PrintResults(FOutputDevice& Ar, uint32 PrintFlags = 0);

private:
	int32 GetReferencedObjects(UObject* Obj, TArray<UObject*>& ReferencedObjects);
	void ProcessSubObjRecursive(UObject* Root, UObject* Object);
	SIZE_T CalculateSizeRecursive(UObject* Object);
	FString GetFlagsString(const FObjectMemoryUsage& Annotation);
	void PrintSubObjects(FOutputDevice& Ar, const FString& Indent, UObject* Parent, uint32 PrintFlags);


	/** All objects of this class will be analyzed */
	UClass* BaseClass;
	/** List of objects which memory usage should be analyzed*/
	TArray<UObject*> ObjectList;
	/** Anootations containing the gathered data*/
	FUObjectAnnotationDense<FObjectMemoryUsage, true> MemUsageAnnotations;
	/** Flags to modify mem counting behavior */
	uint32 AnalyzeFlags;
};
