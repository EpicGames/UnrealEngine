// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/UEOps.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/IsTObjectPtr.h"
#include "Templates/NonNullPointer.h"
#include "UObject/GarbageCollectionGlobals.h"
#include "UObject/ObjectHandle.h"
#include "UObject/UObjectGlobals.h"

#include <type_traits>

#define UE_WITH_OBJECT_PTR_DEPRECATIONS 0
#if UE_WITH_OBJECT_PTR_DEPRECATIONS
	#define UE_OBJPTR_DEPRECATED(Version, Message) UE_DEPRECATED(Version, Message)
#else
	#define UE_OBJPTR_DEPRECATED(Version, Message) 
#endif

#ifndef UE_OBJECT_PTR_GC_BARRIER
	#define UE_OBJECT_PTR_GC_BARRIER 1
#endif


/** 
 * Wrapper macro for use in places where code needs to allow for a pointer type that could be a TObjectPtr<T> or a raw object pointer during a transitional period.
 * The coding standard disallows general use of the auto keyword, but in wrapping it in this macro, we have a record
 * of the explicit type meant to be used, and an avenue to go back and change these instances to an explicit
 * TObjectPtr<T> after the transition is complete and the type won't be toggling back and forth anymore.
 */
#define UE_TRANSITIONAL_OBJECT_PTR(Type) auto 
#define UE_TRANSITIONAL_OBJECT_PTR_TEMPLATE(Templ, Type) auto 
#define UE_TRANSITIONAL_OBJECT_PTR_TEMPLATE_SUFFIXED(Templ, Type, Suffix) auto 
#define UE_TRANSITIONAL_OBJECT_PTR_TEMPLATE2_ARG1(Templ, Type1, Type2) auto 
#define UE_TRANSITIONAL_OBJECT_PTR_TEMPLATE2_ARG1_SUFFIXED(Templ, Type1, Type2, Suffix) auto 
#define UE_TRANSITIONAL_OBJECT_PTR_TEMPLATE2_ARG2(Templ, Type1, Type2) auto 
#define UE_TRANSITIONAL_OBJECT_PTR_TEMPLATE2_ARG2_SUFFIXED(Templ, Type1, Type2, Suffix) auto 
#define UE_TRANSITIONAL_OBJECT_PTR_TEMPLATE2_ARG_BOTH(Templ, Type1, Type2) auto 
#define UE_TRANSITIONAL_OBJECT_PTR_TEMPLATE2_ARG_BOTH_SUFFIXED(Templ, Type1, Type2, Suffix) auto 

#define UE_OBJECT_PTR_NONCONFORMANCE_SUPPORT 0 UE_DEPRECATED_MACRO(5.6, "UE_OBJECT_PTR_NONCONFORMANCE_SUPPORT has been deprecated and shouldn't be used")

class UClass;

template <typename T>
struct TObjectPtr;

/**
 * FObjectPtr is the basic, minimally typed version of TObjectPtr
 */
struct FObjectPtr
{
public:
	[[nodiscard]] constexpr FObjectPtr()
		: Handle(UE::CoreUObject::Private::MakeObjectHandle(nullptr))
	{
	}

	[[nodiscard]] explicit constexpr FObjectPtr(ENoInit)
	{
	}

	[[nodiscard]] FORCEINLINE constexpr FObjectPtr(TYPE_OF_NULLPTR)
		: Handle(UE::CoreUObject::Private::MakeObjectHandle(nullptr))
	{
	}

	[[nodiscard]] explicit FORCEINLINE constexpr FObjectPtr(UObject* Object)
		: Handle(UE::CoreUObject::Private::MakeObjectHandle(Object))
	{
#if UE_OBJECT_PTR_GC_BARRIER
		ConditionallyMarkAsReachable(Object);
#endif // UE_OBJECT_PTR_GC_BARRIER
	}

	UE_OBJPTR_DEPRECATED(5.0, "Construction with incomplete type pointer is deprecated.  Please update this code to use MakeObjectPtrUnsafe.")
	[[nodiscard]] explicit FORCEINLINE FObjectPtr(void* IncompleteObject)
		: Handle(UE::CoreUObject::Private::MakeObjectHandle(reinterpret_cast<UObject*>(IncompleteObject)))
	{
	}

#if UE_WITH_CONSTINIT_UOBJECT
	template<typename U>
	[[nodiscard]] explicit consteval FObjectPtr(UE::CodeGen::ConstInit::TObjectRefParam<U> Ptr)
#if IS_MONOLITHIC
		: Handle(UE::CoreUObject::Private::MakeObjectHandle(Ptr.ObjectPtr))
#else
		: Handle(UE::CoreUObject::Private::MakeCompiledInObjectHandle(Ptr.ObjectPtr, Ptr.CompiledInPtr))
#endif // IS_MONOLITHIC
	{
	}
#endif // UE_WITH_CONSTINIT_UOBJECT

#if !UE_OBJECT_HANDLE_IS_OBJECT_PTR
	[[nodiscard]] explicit FORCEINLINE FObjectPtr(FObjectHandle Handle)
		: Handle(Handle)
	{
#if UE_OBJECT_PTR_GC_BARRIER
		ConditionallyMarkAsReachable(*this);
#endif // UE_OBJECT_PTR_GC_BARRIER
	}
#endif
	
	[[nodiscard]] explicit FORCEINLINE FObjectPtr(FRemoteObjectId ObjectId)
#if UE_WITH_REMOTE_OBJECT_HANDLE
		: FObjectPtr(UE::CoreUObject::Private::FRemoteObjectHandlePrivate::FromIdNoResolve(ObjectId))
#else
		: FObjectPtr()
#endif
	{
	}

	[[nodiscard]] FORCEINLINE UObject* Get() const
	{
		return UE::CoreUObject::Private::ResolveObjectHandle(Handle);
	}

	[[nodiscard]] FORCEINLINE UClass* GetClass() const
	{
		return UE::CoreUObject::Private::ResolveObjectHandleClass(Handle);
	}


#if UE_OBJECT_PTR_GC_BARRIER
	[[nodiscard]] constexpr FObjectPtr(FObjectPtr&& InOther)
		: Handle(MoveTemp(InOther.Handle))
	{
		ConditionallyMarkAsReachable(*this);
	}

	[[nodiscard]] constexpr FObjectPtr(const FObjectPtr& InOther)
		: Handle(InOther.Handle)
	{
		ConditionallyMarkAsReachable(*this);
	}

	FObjectPtr& operator=(FObjectPtr&& InOther)
	{
		ConditionallyMarkAsReachable(InOther);
		Handle = MoveTemp(InOther.Handle);
		return *this;
	}

	FObjectPtr& operator=(const FObjectPtr& InOther)
	{
		ConditionallyMarkAsReachable(InOther);
		Handle = InOther.Handle;
		return *this;
	}
#else
	[[nodiscard]] constexpr FObjectPtr(FObjectPtr&&) = default;
	[[nodiscard]] constexpr FObjectPtr(const FObjectPtr&) = default;
	FObjectPtr& operator=(FObjectPtr&&) = default;
	FObjectPtr& operator=(const FObjectPtr&) = default;
#endif // UE_OBJECT_PTR_GC_BARRIER

	FObjectPtr& operator=(UObject* Other)
	{
#if UE_OBJECT_PTR_GC_BARRIER
		ConditionallyMarkAsReachable(Other);
#endif // UE_OBJECT_PTR_GC_BARRIER
		Handle = UE::CoreUObject::Private::MakeObjectHandle(Other);
		return *this;
	}

	UE_OBJPTR_DEPRECATED(5.0, "Assignment with incomplete type pointer is deprecated.  Please update this code to use MakeObjectPtrUnsafe.")
	FObjectPtr& operator=(void* IncompleteOther)
	{
		Handle = UE::CoreUObject::Private::MakeObjectHandle(reinterpret_cast<UObject*>(IncompleteOther));
		return *this;
	}

	FObjectPtr& operator=(TYPE_OF_NULLPTR)
	{
		Handle = UE::CoreUObject::Private::MakeObjectHandle(nullptr);
		return *this;
	}

	[[nodiscard]] FORCEINLINE bool UEOpEquals(const FObjectPtr& Other) const
	{
		return (Handle == Other.Handle);
	}

	UE_OBJPTR_DEPRECATED(5.0, "Use of ToTObjectPtr is unsafe and is deprecated.")
	[[nodiscard]] FORCEINLINE TObjectPtr<UObject>& ToTObjectPtr();
	UE_OBJPTR_DEPRECATED(5.0, "Use of ToTObjectPtr is unsafe and is deprecated.")
	[[nodiscard]] FORCEINLINE const TObjectPtr<UObject>& ToTObjectPtr() const;

	[[nodiscard]] FORCEINLINE UObject* operator->() const { return Get(); }
	[[nodiscard]] FORCEINLINE UObject& operator*() const { return *Get(); }

	[[nodiscard]] FORCEINLINE bool operator!() const { return IsObjectHandleNull(Handle); }
	[[nodiscard]] explicit FORCEINLINE operator bool() const { return !IsObjectHandleNull(Handle); }

	[[nodiscard]] FORCEINLINE bool IsResolved() const { return IsObjectHandleResolved(Handle); }

	/**
	 * FObjectPtr::IsRemote is only used when UE_WITH_REMOTE_OBJECT_HANDLE is true, and is mutually exclusive with other uses of unresolved FObjectPtr.
	 * When using UE_WITH_REMOTE_OBJECT_HANDLE, unresolved FObjectPtrs mean the object is not locally available and is present on another server, see @RemoteObject.h
	*/	
	[[nodiscard]] FORCEINLINE EResidence GetResidence() const { return GetObjectHandleResidence(Handle); }

	UE_DEPRECATED(5.8, "Use GetResidence() instead.")
	[[nodiscard]] FORCEINLINE bool IsRemote() const { return GetResidence() == EResidence::Remote; }

	// Gets the PathName of the object without resolving the object reference.
	// @TODO OBJPTR: Deprecate this.
	[[nodiscard]] FString GetPath() const { return GetPathName(); }

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_REMOTE_OBJECT_HANDLE
	// Gets the PathName of the object without resolving the object reference.
	[[nodiscard]] COREUOBJECT_API FString GetPathName() const;

	// Gets the FName of the object without resolving the object reference.
	[[nodiscard]] COREUOBJECT_API FName GetFName() const;

	// Gets the string name of the object without resolving the object reference.
	[[nodiscard]] FORCEINLINE FString GetName() const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		if (IsObjectHandleNull(Handle))
		{
			return TEXT("None");
		}

		// We enforce GetFName() does not cause migrations when run in the Open and instead will return NAME_None if non-local
		const FName MyName = GetFName();
		if (MyName != NAME_None)
		{
			return MyName.ToString();
		}

		// We know the handle was non-null, so we are remote
		return FString::Printf(TEXT("%s (%s)"), *GetRemoteId().ToString(), EnumToString(GetResidence()));
#else
		return GetFName().ToString();
#endif
	}

	// Gets the Outer UObject for this Object without resolving the object reference.
	[[nodiscard]] COREUOBJECT_API FObjectPtr GetOuter() const;

	// Gets the package for this Object without resolving the object reference.
	[[nodiscard]] COREUOBJECT_API FObjectPtr GetPackage() const;

	// Reports whether this Object is within an Outer or Package. In some cases may need to resolve the object reference.
	[[nodiscard]] COREUOBJECT_API bool IsIn(FObjectPtr SomeOuter) const;

	/** Returns the full name for the object in the form: Class ObjectPath */
	[[nodiscard]] COREUOBJECT_API FString GetFullName(EObjectFullNameFlags Flags = EObjectFullNameFlags::None) const;
#else
	[[nodiscard]] FString GetPathName() const
	{
		const UObject* ResolvedObject = Get();
		return ResolvedObject ? UE::CoreUObject::Private::GetPathName(ResolvedObject) : TEXT("None");
	}

	[[nodiscard]] FName GetFName() const
	{
		const UObject* ResolvedObject = Get();
		return ResolvedObject ? UE::CoreUObject::Private::GetFName(ResolvedObject) : NAME_None;
	}

	[[nodiscard]] FString GetName() const
	{
		return GetFName().ToString();
	}

	[[nodiscard]] FObjectPtr GetOuter() const
	{
		const UObject* ResolvedObject = Get();
		return ResolvedObject ? FObjectPtr(UE::CoreUObject::Private::GetOuter(ResolvedObject)) : FObjectPtr(nullptr);
	}

	[[nodiscard]] FObjectPtr GetPackage() const
	{
		const UObject* ResolvedObject = Get();
		return ResolvedObject ? FObjectPtr(UE::CoreUObject::Private::GetPackage(ResolvedObject)) : FObjectPtr(nullptr);
	}

	[[nodiscard]] bool IsIn(FObjectPtr SomeOuter) const
	{
		const UObject* ResolvedObject = Get();
		const UObject* ResolvedSomeOuter = SomeOuter.Get();
		return (ResolvedObject && ResolvedSomeOuter) 
			? UE::CoreUObject::Private::IsIn(ResolvedObject, ResolvedSomeOuter) : false;
	}

	/**
	 * Returns the fully qualified pathname for this object as well as the name of the class, in the format:
	 * 'ClassName Outermost.[Outer:]Name'.
	 */
	[[nodiscard]] FString GetFullName(EObjectFullNameFlags Flags = EObjectFullNameFlags::None) const
	{
		// UObjectBaseUtility::GetFullName is safe to call on null objects.
		return UE::CoreUObject::Private::GetFullName(Get(), nullptr, Flags);
	}
#endif

	[[nodiscard]] FORCEINLINE FObjectHandle GetHandle() const { return Handle; }
	[[nodiscard]] FORCEINLINE FObjectHandle& GetHandleRef() const { return Handle; }

	[[nodiscard]] COREUOBJECT_API bool IsA(const UClass* SomeBase) const;

	template <typename T>
	[[nodiscard]] FORCEINLINE bool IsA() const
	{
		return IsA(T::StaticClass());
	}

	[[nodiscard]] FORCEINLINE FRemoteObjectId GetRemoteId() const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return GetHandle().GetRemoteId();
#else
		return FRemoteObjectId();
#endif
	}

private:
	[[nodiscard]] friend FORCEINLINE uint32 GetTypeHash(const FObjectPtr& Object)
	{
		return GetTypeHash(Object.Handle);
	}

	union
	{
		mutable FObjectHandle Handle;
		// DebugPtr allows for easier dereferencing of a resolved FObjectPtr in watch windows of debuggers.  If the address in the pointer
		// is an odd/uneven number, that means the object reference is unresolved and you will not be able to dereference it successfully.
		UObject* DebugPtr;
	};

#if UE_OBJECT_PTR_GC_BARRIER
	FORCEINLINE constexpr void ConditionallyMarkAsReachable(const FObjectPtr& InPtr) const
	{
		UE_IF_NOT_CONSTEVAL
		{
			if (UE::GC::GIsIncrementalReachabilityPending && InPtr.IsResolved())
			{
				if (UObject* Obj = UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(InPtr.GetHandleRef()))
				{
					UE::GC::MarkAsReachable(Obj);
				}
			}
		}
	}
	FORCEINLINE constexpr void ConditionallyMarkAsReachable(const UObject* InObj) const
	{
		UE_IF_NOT_CONSTEVAL
		{ 
			if (UE::GC::GIsIncrementalReachabilityPending && InObj)
			{
				UE::GC::MarkAsReachable(InObj);
			}
		}
	}
#endif // UE_OBJECT_PTR_GC_BARRIER
};

template <typename T>
struct TPrivateObjectPtr;

namespace ObjectPtr_Private
{
	/** Coerce to pointer through implicit conversion to const T* (overload through less specific "const T*" parameter to avoid ambiguity with other coercion options that may also exist. */
	template <typename T>
	FORCEINLINE const T* CoerceToPointer(const T* Other)
	{
		return Other;
	}

	/** Coerce to pointer through implicit conversion to CommonPointerType where CommonPointerType is deduced, and must be a C++ pointer, not a wrapper type. */
	template <
		typename T,
		typename U
		UE_REQUIRES(std::is_pointer_v<std::common_type_t<const T*, U>>)
	>
	FORCEINLINE std::common_type_t<const T*, U> CoerceToPointer(const U& Other)
	{
		return Other;
	}

	/** Coerce to pointer through the use of a ".Get()" member, which is the convention within Unreal smart pointer types. */
	template <
		typename T,
		typename U
		UE_REQUIRES(!TIsTObjectPtr_V<U>)
	>
	FORCEINLINE auto CoerceToPointer(const U& Other) -> decltype(Other.Get())
	{
		return Other.Get();
	}

	template <typename T, typename U>
	bool IsObjectPtrEqualToRawPtrOfRelatedType(const TObjectPtr<T>& Ptr, const U* Other)
	{
		// simple test if Ptr is null (avoids resolving either side)
		if (!Ptr)
		{
			return !Other;
		}

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_REMOTE_OBJECT_HANDLE
		if (Ptr.IsResolved())
		{
			return Ptr.GetNoResolveNoCheck() == Other;
		}
		else if (!Other) //avoids resolving if Other is null
		{
			return false;	// from above, we already know that !Ptr is false
		}
#endif
#if UE_WITH_REMOTE_OBJECT_HANDLE
		// avoids resolving since we can compare against globally unique id
		return Ptr.GetRemoteId() == FRemoteObjectId(reinterpret_cast<const UObjectBase*>(Other));
#else
		return Ptr.GetNoReadNoCheck() == Other;
#endif
	}

	/** Perform shallow equality check between a TObjectPtr and another (non TObjectPtr) type that we can coerce to a pointer. */
	template <
		typename T,
		typename U,
		decltype(CoerceToPointer<T>(std::declval<U>()) == std::declval<const T*>())* = nullptr
		UE_REQUIRES(!TIsTObjectPtr_V<U>)
	>
	bool IsObjectPtrEqual(const TObjectPtr<T>& Ptr, const U& Other)
	{
		// This function deliberately avoids the tracking code path as we are only doing
		// a shallow pointer comparison.
		return IsObjectPtrEqualToRawPtrOfRelatedType<T>(Ptr, ObjectPtr_Private::CoerceToPointer<T>(Other));
	}

	/** Check for NULL without resolving the handle. */
	template <
		typename T
#if UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
		UE_REQUIRES(std::is_same_v<std::remove_const_t<T>, UObject>)
#endif
	>
	FORCEINLINE bool IsObjectPtrNull(const FObjectPtr& ObjectPtr)
	{
		return !ObjectPtr.operator bool();
	}

	/** Resolve and return the underlying reference. */
	template <
		typename T
#if UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
		UE_REQUIRES(std::is_same_v<std::remove_const_t<T>, UObject>)
#endif
	>
	FORCEINLINE T* Get(const FObjectPtr& ObjectPtr)
	{
		return (T*)ObjectPtr.Get();
	}

#if UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
	/** Check for NULL without resolving the handle. Always returns true if the handle is not type safe (only when T != UObject). */
	template <
		typename T
		UE_REQUIRES(!std::is_same_v<std::remove_const_t<T>, UObject>)
	>
	FORCEINLINE bool IsObjectPtrNull(const FObjectPtr& ObjectPtr)
	{
		if (!IsObjectHandleTypeSafe(ObjectPtr.GetHandle()))
		{
			// Type is unsafe; this pointer will resolve to NULL.
			return true;
		}

		return !ObjectPtr.operator bool();
	}

	/** Resolve and return the underlying reference. Always returns NULL if the handle is not type safe (only when T != UObject). */
	template <
		typename T
		UE_REQUIRES(!std::is_same_v<std::remove_const_t<T>, UObject>)
	>
	FORCEINLINE T* Get(const FObjectPtr& ObjectPtr)
	{
		if (!IsObjectHandleTypeSafe(ObjectPtr.GetHandle()))
		{
			// Type is unsafe; return NULL without resolving.
			return nullptr;
		}

		return (T*)ObjectPtr.Get();
	}
#endif

	template <typename T, int = sizeof(T)>
	char (&ResolveTypeIsComplete(int))[2];

	template <typename T>
	char (&ResolveTypeIsComplete(...))[1];

	struct Friend;
};

/**
 * TObjectPtr is a type of pointer to a UObject that is meant to function as a drop-in replacement for raw pointer
 * member properties. It is size equivalent to a 64-bit pointer and supports access tracking and optional lazy load
 * behavior in editor builds. It stores either the address to the referenced object or (in editor builds) an index in
 * the object handle table that describes a referenced object that hasn't been loaded yet. It is serialized
 * identically to a raw pointer to a UObject. When resolved, its participation in garbage collection is identical to a
 * raw pointer to a UObject.
 *
 * This is useful for automatic replacement of raw pointers to support advanced cook-time dependency tracking and
 * editor-time lazy load use cases. See UnrealObjectPtrTool for tooling to automatically replace raw pointer members
 * with FObjectPtr/TObjectPtr members instead.
 */
template <typename T>
struct TObjectPtr
{
#if !defined(PLATFORM_COMPILER_IWYU) && !defined(UE_HEADER_UNITS)
	// TObjectPtr should only be used on types T that are EITHER:
	// - incomplete (ie: forward declared and we have not seen their definition yet)
	// - complete and derived from UObject
	// This means that the following are invalid and must fail to compile:
	// - TObjectPtr<int>
	// - TObjectPtr<IInterface>
	static_assert(std::disjunction<std::bool_constant<sizeof(ObjectPtr_Private::ResolveTypeIsComplete<T>(1)) != 2>, std::is_base_of<UObject, T>>::value, "TObjectPtr<T> can only be used with types derived from UObject");
#endif

public:
	using ElementType = T;

	[[nodiscard]] constexpr TObjectPtr()
		: ObjectPtr()
	{
	}

#if UE_OBJECT_PTR_GC_BARRIER
	[[nodiscard]] TObjectPtr(TObjectPtr<T>&& Other)
		: ObjectPtr(MoveTemp(Other.ObjectPtr))
	{
	}
	[[nodiscard]] TObjectPtr(const TObjectPtr<T>& Other)
		: ObjectPtr(Other.ObjectPtr)
	{
	}
#else
	[[nodiscard]] TObjectPtr(TObjectPtr<T>&& Other) = default;
	TObjectPtr(const TObjectPtr<T>& Other) = default;
#endif // UE_OBJECT_PTR_GC_BARRIER

	[[nodiscard]] explicit constexpr FORCEINLINE TObjectPtr(ENoInit)
		: ObjectPtr(NoInit)
	{
	}

	[[nodiscard]] FORCEINLINE constexpr TObjectPtr(TYPE_OF_NULLPTR)
		: ObjectPtr(nullptr)
	{
	}

	[[nodiscard]] explicit FORCEINLINE constexpr TObjectPtr(FObjectPtr ObjPtr)
		: ObjectPtr(ObjPtr)
	{
	}

	[[nodiscard]] explicit FORCEINLINE consteval TObjectPtr(EConstEval, T* Object)
		: ObjectPtr(static_cast<UObject*>(Object))
	{
	}

#if UE_WITH_CONSTINIT_UOBJECT
	[[nodiscard]] explicit consteval TObjectPtr(UE::CodeGen::ConstInit::TObjectRefParam<T> Ptr)
		: ObjectPtr(Ptr)
	{
	}
#endif

	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	[[nodiscard]] FORCEINLINE constexpr TObjectPtr(const TObjectPtr<U>& Other)
		: ObjectPtr(Other.ObjectPtr)
	{
		IWYU_MARKUP_IMPLICIT_CAST(U, T);
	}

	template <
		typename U
		UE_REQUIRES(!TIsTObjectPtr_V<std::decay_t<U>> && std::is_convertible_v<U, T*>)
	>
	[[nodiscard]] constexpr FORCEINLINE TObjectPtr(const U& Object)
		: ObjectPtr(const_cast<std::remove_const_t<T>*>(ImplicitConv<T*>(Object)))
	{
	}

	[[nodiscard]] explicit FORCEINLINE TObjectPtr(TPrivateObjectPtr<T>&& PrivatePtr)
		: ObjectPtr(const_cast<UObject*>(PrivatePtr.Pointer))
	{
	}

#if UE_OBJECT_PTR_GC_BARRIER
	TObjectPtr<T>& operator=(TObjectPtr<T>&& Other)
	{
		ObjectPtr = MoveTemp(Other.ObjectPtr);
		return *this;
	}
	TObjectPtr<T>& operator=(const TObjectPtr<T>& Other)
	{
		ObjectPtr = Other.ObjectPtr;
		return *this;
	}
#else
	TObjectPtr<T>& operator=(TObjectPtr<T>&&) = default;
	TObjectPtr<T>& operator=(const TObjectPtr<T>&) = default;
#endif // UE_OBJECT_PTR_GC_BARRIER

	FORCEINLINE TObjectPtr<T>& operator=(TYPE_OF_NULLPTR)
	{
		ObjectPtr = nullptr;
		return *this;
	}

	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	FORCEINLINE TObjectPtr<T>& operator=(const TObjectPtr<U>& Other)
	{
		IWYU_MARKUP_IMPLICIT_CAST(U, T);
		ObjectPtr = Other.ObjectPtr;
		return *this;
	}

	template <
		typename U
		UE_REQUIRES(!TIsTObjectPtr_V<std::decay_t<U>> && std::is_convertible_v<U, T*>)
	>
	FORCEINLINE TObjectPtr<T>& operator=(U&& Object)
	{
		ObjectPtr = const_cast<std::remove_const_t<T>*>(ImplicitConv<T*>(Object));
		return *this;
	}

	FORCEINLINE TObjectPtr<T>& operator=(TPrivateObjectPtr<T>&& PrivatePtr)
	{
		ObjectPtr = const_cast<UObject*>(PrivatePtr.Pointer);
		return *this;
	}

	// Equality/Inequality comparisons against other TObjectPtr
	template <
		typename U,
		typename Base = std::common_type_t<T*, U*>
	>
	[[nodiscard]] FORCEINLINE bool UEOpEquals(const TObjectPtr<U>& Other) const
	{
#if UE_WITH_OBJECT_HANDLE_TYPE_SAFETY
		// Do a NULL test first before comparing the underlying handles, in case either side is
		// a non-NULL, but unsafe type pointer (which would equate to NULL when Get() is called).
		if (ObjectPtr_Private::IsObjectPtrNull<T>(ObjectPtr))
		{
			return ObjectPtr_Private::IsObjectPtrNull<U>(Other.ObjectPtr);
		}
		else if (ObjectPtr_Private::IsObjectPtrNull<U>(Other.ObjectPtr))
		{
			return false;
		}
#endif
		return ObjectPtr == Other.ObjectPtr;
	}

	// Equality/Inequality comparisons against nullptr
	[[nodiscard]] FORCEINLINE bool UEOpEquals(TYPE_OF_NULLPTR) const
	{
		return ObjectPtr_Private::IsObjectPtrNull<T>(ObjectPtr);
	}

	// Equality/Inequality comparisons against another type that can be implicitly converted to the pointer type kept in a TObjectPtr
	template <
		typename U,
		typename = decltype(ObjectPtr_Private::IsObjectPtrEqual(std::declval<const TObjectPtr<T>&>(), std::declval<const U&>()))
	>
	[[nodiscard]] FORCEINLINE bool UEOpEquals(const U& Other) const
	{
		return ObjectPtr_Private::IsObjectPtrEqual(*this, Other);
	}

	// @TODO: OBJPTR: There is a risk that the FObjectPtr is storing a reference to the wrong type.  This could
	//			happen if data was serialized at a time when a pointer field was declared to be of type A, but then the declaration
	//			changed and the pointer field is now of type B.  Upon deserialization of pre-existing data, we'll be holding
	//			a reference to the wrong type of object which we'll just send back static_casted as the wrong type.  Doing
	//			a check or checkSlow here could catch this, but it would be better if the check could happen elsewhere that
	//			isn't called as frequently.
	[[nodiscard]] FORCEINLINE T* Get() const { return ObjectPtr_Private::Get<T>(ObjectPtr); }
	[[nodiscard]] FORCEINLINE UClass* GetClass() const { return ObjectPtr.GetClass(); }
	[[nodiscard]] FORCEINLINE TObjectPtr<UObject> GetOuter() const
	{ 
		FObjectPtr Ptr = ObjectPtr.GetOuter();
		return TObjectPtr<UObject>(Ptr);
	}

	[[nodiscard]] FORCEINLINE TObjectPtr<UPackage> GetPackage() const
	{
		FObjectPtr Ptr = ObjectPtr.GetPackage();
		return TObjectPtr<UPackage>(Ptr);
	}

	[[nodiscard]] FORCEINLINE bool IsIn(FObjectPtr SomeOuter) const
	{
		return ObjectPtr.IsIn(SomeOuter);
	}

	template <typename U>
	[[nodiscard]] FORCEINLINE bool IsIn(TObjectPtr<U> SomeOuter) const
	{
		return ObjectPtr.IsIn(SomeOuter.ObjectPtr);
	}

	[[nodiscard]] FORCEINLINE operator T* () const 
	{
		return Get();
	}
	template <typename U>
	UE_OBJPTR_DEPRECATED(5.0, "Explicit cast to other raw pointer types is deprecated.  Please use the Cast API or get the raw pointer with ToRawPtr and cast that instead.")
	[[nodiscard]] explicit FORCEINLINE operator U* () const 
	{
		return (U*)Get();
	}
	[[nodiscard]] explicit FORCEINLINE operator UPTRINT() const 
	{
		return (UPTRINT)Get(); 
	}
	[[nodiscard]] FORCEINLINE T* operator->() const 
	{
		return Get();
	}
	[[nodiscard]] FORCEINLINE T& operator*() const 
	{
		return *Get();
	}

	UE_OBJPTR_DEPRECATED(5.0, "Conversion to a mutable pointer is deprecated.  Please pass a TObjectPtr<T>& instead so that assignment can be tracked accurately.")
	[[nodiscard]] explicit FORCEINLINE operator T*& () 
	{
		return GetInternalRef();
	}

	[[nodiscard]] FORCEINLINE bool operator!() const 
	{
		return ObjectPtr_Private::IsObjectPtrNull<T>(ObjectPtr);
	}
	[[nodiscard]] explicit FORCEINLINE operator bool() const 
	{
		return !ObjectPtr_Private::IsObjectPtrNull<T>(ObjectPtr);
	}
	[[nodiscard]] FORCEINLINE bool IsResolved() const
	{
		return ObjectPtr.IsResolved();
	}
	UE_DEPRECATED(5.8, "Use GetResidence() instead.")
	[[nodiscard]] FORCEINLINE bool IsRemote() const 
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return ObjectPtr.IsRemote();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	[[nodiscard]] FORCEINLINE EResidence GetResidence() const
	{
		return ObjectPtr.GetResidence();
	}
	[[nodiscard]] FORCEINLINE FString GetPath() const 
	{
		return ObjectPtr.GetPath();
	}
	[[nodiscard]] FORCEINLINE FString GetPathName() const
	{
		return ObjectPtr.GetPathName();
	}
	[[nodiscard]] FORCEINLINE FName GetFName() const
	{
		return ObjectPtr.GetFName();
	}
	[[nodiscard]] FORCEINLINE FString GetName() const
	{
		return ObjectPtr.GetName();
	}
	[[nodiscard]] FORCEINLINE FString GetFullName(EObjectFullNameFlags Flags = EObjectFullNameFlags::None) const 
	{
		return ObjectPtr.GetFullName(Flags);
	}
	[[nodiscard]] FORCEINLINE FObjectHandle GetHandle() const 
	{
		return ObjectPtr.GetHandle();
	}
	[[nodiscard]] FORCEINLINE bool IsA(const UClass* SomeBase) const 
	{
		return ObjectPtr.IsA(SomeBase);
	}
	template <typename U> 
	[[nodiscard]] FORCEINLINE bool IsA() const 
	{
		return ObjectPtr.IsA<U>();
	}

	[[nodiscard]] FORCEINLINE uint32 GetPtrTypeHash() const
	{
		return GetTypeHash(ObjectPtr);
	}

	FORCEINLINE void SerializePtrStructured(FStructuredArchiveSlot Slot)
	{
		Slot << ObjectPtr;
	}

	TObjectPtr<T>& GetAccessTrackedObjectPtr() 
	{
		return *this;
	}

	const TObjectPtr<T>& GetAccessTrackedObjectPtr() const
	{
		return *this;
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	[[nodiscard]] FORCEINLINE FRemoteObjectId GetRemoteId() const
	{
		return ObjectPtr.GetRemoteId();
	}
#endif

	friend ObjectPtr_Private::Friend;
	friend struct FObjectPtr;
	template <typename U> friend struct TObjectPtr;
	template <typename U, typename V> friend bool ObjectPtr_Private::IsObjectPtrEqualToRawPtrOfRelatedType(const TObjectPtr<U>& Ptr, const V* Other);

private:
	FORCEINLINE T* GetNoReadNoCheck() const { return (T*)(UE::CoreUObject::Private::ResolveObjectHandleNoReadNoCheck(ObjectPtr.GetHandleRef())); }
	FORCEINLINE T* GetNoResolveNoCheck() const { return (T*)(UE::CoreUObject::Private::ReadObjectHandlePointerNoCheck(ObjectPtr.GetHandleRef())); }

	[[nodiscard]] UE_REWRITE FObjectHandle& GetHandleRef() const 
	{
		return ObjectPtr.GetHandleRef();
	}

	// @TODO: OBJPTR: There is a risk of a gap in access tracking here.  The caller may get a mutable pointer, write to it, then
	//			read from it.  That last read would happen without an access being recorded.  Not sure if there is a good way
	//			to handle this case without forcing the calling code to be modified.
	FORCEINLINE T*& GetInternalRef()
	{
		ObjectPtr_Private::Get<T>(ObjectPtr);
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
		check(ObjectPtr.IsResolved());
#endif
		return (T*&)ObjectPtr.GetHandleRef();
	}

	union
	{
		FObjectPtr ObjectPtr;
		// DebugPtr allows for easier dereferencing of a resolved TObjectPtr in watch windows of debuggers.  If the address in the pointer
		// is an odd/uneven number, that means the object reference is unresolved and you will not be able to dereference it successfully.
		T* DebugPtr;
	};
};

// Equals against nullptr to optimize comparing against nullptr as it avoids resolving

template <typename T>
struct TRemoveObjectPointer
{
	typedef T Type;
};
template <typename T>
struct TRemoveObjectPointer<TObjectPtr<T>>
{
	typedef T Type;
};

namespace ObjectPtr_Private
{
	template <typename T> struct TRawPointerType                               { using Type = T;  };
	template <typename T> struct TRawPointerType<               TObjectPtr<T>> { using Type = T*; };
	template <typename T> struct TRawPointerType<const          TObjectPtr<T>> { using Type = T*; };
	template <typename T> struct TRawPointerType<      volatile TObjectPtr<T>> { using Type = T*; };
	template <typename T> struct TRawPointerType<const volatile TObjectPtr<T>> { using Type = T*; };
	template <typename T>
	class TNonAccessTrackedObjectPtr;
	struct Friend
	{
		template <typename T>
		FORCEINLINE static uint32 GetPtrTypeHash(const TObjectPtr<T>& InObjectPtr)
		{
			return GetTypeHash(InObjectPtr.ObjectPtr);
		}

		template <typename T>
		FORCEINLINE static FArchive& Serialize(FArchive& Ar, TObjectPtr<T>& InObjectPtr)
		{
			Ar << InObjectPtr.ObjectPtr;
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			if( Ar.IsLoading() )
			{
				// Prevent incompatibly mocked objects from leaking to callers.. this can happen
				// because of CoreUObject types (e.g. UScriptStruct) being extended in modules
				// which are unavailable in UE_INTERNAL_UNIVERSAL_PLACEHOLDERS environments. In this case
				// the type that CoreUObject wants to use is erased above so the linker
				// can't create a placeholder that necessarily conforms to the native code's
				// requirements.
				const UObject* Resolved = InObjectPtr;
				if(Resolved && !(InObjectPtr.template IsA<T>()))
				{
					InObjectPtr = nullptr;
				}
			}
#endif // UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
			return Ar;
		}

		template <typename T>
		FORCEINLINE static void SerializePtrStructured(FStructuredArchiveSlot Slot, TObjectPtr<T>& InObjectPtr)
		{
			Slot << InObjectPtr.ObjectPtr;
		}

		template <typename T>
		FORCEINLINE static T* NoAccessTrackingGet(const TObjectPtr<T>& Ptr)
		{
			return reinterpret_cast<T*>(UE::CoreUObject::Private::ResolveObjectHandleNoRead(Ptr.ObjectPtr.GetHandleRef()));
		}

		template <typename T>
		FORCEINLINE static T* NoAccessTrackingGetNoResolve(const TObjectPtr<T>& Ptr)
		{
			return reinterpret_cast<T*>(UE::CoreUObject::Private::NoResolveObjectHandleNoRead(Ptr.ObjectPtr.GetHandleRef()));
		}

		template<typename T>
		UE_REWRITE static FObjectHandle& GetHandleRef(FObjectPtr& Ptr)
		{
			return Ptr.GetHandleRef();
		}

		template<typename T>
		UE_REWRITE static FObjectHandle& GetHandleRef(TObjectPtr<T>& Ptr)
		{
			return Ptr.GetHandleRef();
		}
		
		template<typename T>
		static FObjectHandle& GetHandleRef(TNonAccessTrackedObjectPtr<T>& Ptr);
	};
	
	template <typename T>
	class TNonAccessTrackedObjectPtr
	{
	public:
		constexpr TNonAccessTrackedObjectPtr() = default;

		explicit TNonAccessTrackedObjectPtr(ENoInit)
			: ObjectPtr{NoInit}
		{
		}

		explicit constexpr TNonAccessTrackedObjectPtr(T* Ptr)
			: ObjectPtr{Ptr}
		{
		}

		// Unsafe constructor that doesn't check that the argument really is a T 
		explicit consteval TNonAccessTrackedObjectPtr(EConstEval, UObject* Ptr)
			: ObjectPtr{FObjectPtr(Ptr)}
		{
		}

#if UE_WITH_CONSTINIT_UOBJECT
		template<typename U>
		explicit consteval TNonAccessTrackedObjectPtr(UE::CodeGen::ConstInit::TObjectRefParam<U> Ptr)
			: ObjectPtr{FObjectPtr(Ptr)}
		{
		}
#endif // UE_WITH_CONSTINIT_UOBJECT

		TNonAccessTrackedObjectPtr& operator=(T* Value)
		{
			ObjectPtr = Value;
			return *this;
		}
	
		T* Get() const
		{
			return ObjectPtr_Private::Friend::NoAccessTrackingGet(ObjectPtr);
		}

		UE_INTERNAL T* GetNoResolve() const
		{
			return ObjectPtr_Private::Friend::NoAccessTrackingGetNoResolve(ObjectPtr);
		}

		[[nodiscard]] UE_REWRITE FObjectHandle GetHandle() const 
		{
			return ObjectPtr.GetHandle();
		}

		explicit operator UPTRINT() const
		{
			return BitCast<UPTRINT>(Get());
		}

		TObjectPtr<T>& GetAccessTrackedObjectPtr() 
		{
			return ObjectPtr;
		}

		const TObjectPtr<T>& GetAccessTrackedObjectPtr() const
		{
			return ObjectPtr;
		}

		operator T*() const
		{
			return Get();
		}
		
		T* operator->() const
		{
			return Get();
		}		
	
	private:
		friend struct Friend;
		TObjectPtr<T> ObjectPtr;
	};

	template<typename T>
	inline FObjectHandle& Friend::GetHandleRef(TNonAccessTrackedObjectPtr<T>& Ptr)
	{
		return Ptr.ObjectPtr.GetHandleRef();
	}
}

template <typename T>
[[nodiscard]] FORCEINLINE uint32 GetTypeHash(const TObjectPtr<T>& InObjectPtr)
{
	return ObjectPtr_Private::Friend::GetPtrTypeHash(InObjectPtr);
}

template <typename T>
FORCEINLINE FArchive& operator<<(FArchive& Ar, TObjectPtr<T>& InObjectPtr)
{
	return ObjectPtr_Private::Friend::Serialize(Ar, InObjectPtr);
}

template <typename T>
FORCEINLINE void operator<<(FStructuredArchiveSlot Slot, TObjectPtr<T>& InObjectPtr)
{
	ObjectPtr_Private::Friend::SerializePtrStructured(Slot, InObjectPtr);
}

template <typename T>
TPrivateObjectPtr<T> MakeObjectPtrUnsafe(const UObject* Obj);

template <typename T>
struct TPrivateObjectPtr
{
public:
	TPrivateObjectPtr(const TPrivateObjectPtr<T>& Other) = default;

private:
	/** Only for use by MakeObjectPtrUnsafe */
	explicit TPrivateObjectPtr(const UObject* InPointer)
		: Pointer(InPointer)
	{
	}

	const UObject* Pointer;
	friend struct TObjectPtr<T>;
	friend TPrivateObjectPtr MakeObjectPtrUnsafe<T>(const UObject* Obj);
};

/** Used to allow the caller to provide a pointer to an incomplete type of T that has explicitly cast to a UObject. */
template <typename T>
[[nodiscard]] TPrivateObjectPtr<T> MakeObjectPtrUnsafe(const UObject* Obj)
{
	return TPrivateObjectPtr<T>{Obj};
}

template <typename T>
[[nodiscard]] TObjectPtr<T> ToObjectPtr(T* Obj)
{
	return TObjectPtr<T>{Obj};
}

template <typename T>
[[nodiscard]] FORCEINLINE T* ToRawPtr(const TObjectPtr<T>& Ptr)
{
	// NOTE: This is specifically not getting a reference to the internal pointer.
	return Ptr.Get();
}

template <typename T>
[[nodiscard]] FORCEINLINE T* ToRawPtr(T* Ptr)
{
	return Ptr;
}

#if !UE_DEPRECATE_MUTABLE_TOBJECTPTR
template <typename T, SIZE_T Size>
UE_DEPRECATED(5.6, "Mutable ToRawPtrArrayUnsafe() is deprecated. Use MutableView() or TArray<TObjectPtr<...>> instead.")
[[nodiscard]] FORCEINLINE T**
ToRawPtrArrayUnsafe(TObjectPtr<T>(&ArrayOfPtr)[Size])
{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
	for (TObjectPtr<T>& Item : ArrayOfPtr)
	{
		// NOTE: Relying on the fact that the TObjectPtr will cache the resolved pointer in place after calling Get.
		(void)Item.Get();
		check(Item.IsResolved());
	}
#endif

	return reinterpret_cast<T**>(ArrayOfPtr);
}
#endif

template <typename T, SIZE_T Size>
[[nodiscard]] FORCEINLINE const T* const *
ToRawPtrArray(const TObjectPtr<T>(&ArrayOfPtr)[Size])
{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
	using TypeCompat = TContainerElementTypeCompatibility<const TObjectPtr<T>>;
	TypeCompat::ReinterpretRangeContiguous(&ArrayOfPtr[0], &ArrayOfPtr[Size], Size);
#endif

	return reinterpret_cast<const T* const *>(ArrayOfPtr);
}

template <typename T>
[[nodiscard]] FORCEINLINE T** ToRawPtrArrayUnsafe(T** ArrayOfPtr)
{
	return ArrayOfPtr;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
template <
	typename ArrayType,
	typename ArrayTypeNoRef = std::remove_reference_t<ArrayType>
	UE_REQUIRES(TIsTArray_V<ArrayTypeNoRef>)
>
#if UE_DEPRECATE_MUTABLE_TOBJECTPTR
[[nodiscard]] const auto&
#else
UE_DEPRECATED(5.6, "Mutable ToRawPtrTArrayUnsafe() is deprecated. Use MutableView() or TArray<TObjectPtr<...>> instead.")
[[nodiscard]] decltype(auto)
#endif
ToRawPtrTArrayUnsafe(ArrayType&& Array)
{
	using ArrayElementType         = typename ArrayTypeNoRef::ElementType;
	using ArrayAllocatorType       = typename ArrayTypeNoRef::AllocatorType;
	using RawPointerType           = typename ObjectPtr_Private::TRawPointerType<ArrayElementType>::Type;
	using QualifiedRawPointerType  = TCopyQualifiersFromTo_T<ArrayElementType, RawPointerType>;
	using NewArrayType             = TArray<QualifiedRawPointerType, ArrayAllocatorType>;
	using RefQualifiedNewArrayType = typename TCopyQualifiersAndRefsFromTo<ArrayType, NewArrayType>::Type;

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
	using TypeCompat = TContainerElementTypeCompatibility<ArrayElementType>;
	TypeCompat::ReinterpretRange(Array.begin(), Array.end());
#endif

	return (RefQualifiedNewArrayType&)Array;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

inline constexpr auto TContainerElementTypeCompatibility_DefaultOperatorFunc = [](auto& InIt) -> decltype(auto) { return *InIt; };

template <typename T>
struct TContainerElementTypeCompatibility<TObjectPtr<T>>
{
	typedef T* ReinterpretType;
	typedef T* CopyFromOtherType;

	template <typename IterBeginType, typename IterEndType, typename OperatorType = std::remove_reference_t<decltype(*std::declval<IterBeginType>())>& (*)(IterBeginType&)>
	UE_OBJPTR_DEPRECATED(5.0, "Reinterpretation between ranges of one type to another type is deprecated.")
	static void ReinterpretRange(IterBeginType Iter, IterEndType IterEnd, OperatorType Operator = TContainerElementTypeCompatibility_DefaultOperatorFunc)
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING || UE_WITH_REMOTE_OBJECT_HANDLE
		while (Iter != IterEnd)
		{
			(void)Operator(Iter).Get();
			check(UE_WITH_REMOTE_OBJECT_HANDLE || Operator(Iter).IsResolved());
			++Iter;
		}
#endif
	}

	template <typename IterBeginType, typename IterEndType, typename SizeType, typename OperatorType = std::remove_reference_t<decltype(*std::declval<IterBeginType>())>& (*)(IterBeginType&)>
	UE_OBJPTR_DEPRECATED(5.0, "Reinterpretation between ranges of one type to another type is deprecated.")
	static void ReinterpretRangeContiguous(IterBeginType Iter, IterEndType IterEnd, SizeType Size, OperatorType Operator = TContainerElementTypeCompatibility_DefaultOperatorFunc)
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING || UE_WITH_REMOTE_OBJECT_HANDLE
		const TObjectPtr<T>* Begin = &*Iter;
		while (Iter != IterEnd)
		{
			auto& Ptr = Operator(Iter);
			const FObjectPtr& ObjPtr = reinterpret_cast<const FObjectPtr&>(Ptr);
			UE::CoreUObject::Private::ResolveObjectHandleNoRead(ObjPtr.GetHandleRef());
			check(UE_WITH_REMOTE_OBJECT_HANDLE || ObjPtr.IsResolved());
			++Iter;
		}
		const UObject* const* ObjPtr = reinterpret_cast<const UObject* const*>(Begin);
		UE::CoreUObject::Private::OnHandleRead(TArrayView<const UObject* const>(ObjPtr, Size));
#endif
	}
	
	UE_OBJPTR_DEPRECATED(5.0, "Copying ranges of one type to another type is deprecated.")
	static constexpr void CopyingFromOtherType() {}
};

template <typename T>
struct TContainerElementTypeCompatibility<const TObjectPtr<T>>
{
	typedef T* const ReinterpretType;
	typedef T* const CopyFromOtherType;

	template <typename IterBeginType, typename IterEndType, typename OperatorType = const std::remove_cv_t<std::remove_reference_t<decltype(*std::declval<IterBeginType>())>>&(*)(IterBeginType&)>
	UE_OBJPTR_DEPRECATED(5.0, "Reinterpretation between ranges of one type to another type is deprecated.")
	static void ReinterpretRange(IterBeginType Iter, IterEndType IterEnd, OperatorType Operator = TContainerElementTypeCompatibility_DefaultOperatorFunc)
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING || UE_WITH_REMOTE_OBJECT_HANDLE
		while (Iter != IterEnd)
		{
			Operator(Iter).Get();
			check(UE_WITH_REMOTE_OBJECT_HANDLE || Operator(Iter).IsResolved());
			++Iter;
		}
#endif
	}

	template <typename IterBeginType, typename IterEndType, typename SizeType, typename OperatorType = std::remove_reference_t<decltype(*std::declval<IterBeginType>())>& (*)(IterBeginType&)>
	UE_OBJPTR_DEPRECATED(5.0, "Reinterpretation between ranges of one type to another type is deprecated.")
	static void ReinterpretRangeContiguous(IterBeginType Iter, IterEndType IterEnd, SizeType Size, OperatorType Operator = TContainerElementTypeCompatibility_DefaultOperatorFunc)
	{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING || UE_WITH_REMOTE_OBJECT_HANDLE
		const TObjectPtr<T>* Begin = &*Iter;
		while (Iter != IterEnd)
		{
			auto& Ptr = Operator(Iter);
			const FObjectPtr& ObjPtr = reinterpret_cast<const FObjectPtr&>(Ptr);
			UE::CoreUObject::Private::ResolveObjectHandleNoRead(ObjPtr.GetHandleRef());
			check(UE_WITH_REMOTE_OBJECT_HANDLE || ObjPtr.IsResolved());
			++Iter;
		}
		const UObject* const* ObjPtr = reinterpret_cast<const UObject* const*>(Begin);
		UE::CoreUObject::Private::OnHandleRead(TArrayView<const UObject* const>(ObjPtr, Size));
#endif
	}

	UE_OBJPTR_DEPRECATED(5.0, "Copying ranges of one type to another type is deprecated.")
	static constexpr void CopyingFromOtherType() {}
};

// Trait which allows TObjectPtr to be default constructed by memsetting to zero.
template <typename T>
struct TIsZeroConstructType<TObjectPtr<T>>
{
	enum { Value = true };
};

// Trait which allows TObjectPtr to be memcpy'able from pointers.
template <typename T>
struct TIsBitwiseConstructible<TObjectPtr<T>, T*>
{
	enum { Value = !UE_OBJECT_PTR_GC_BARRIER };
};

template <typename T, class PREDICATE_CLASS>
struct TDereferenceWrapper<TObjectPtr<T>, PREDICATE_CLASS>
{
	const PREDICATE_CLASS& Predicate;

	TDereferenceWrapper(const PREDICATE_CLASS& InPredicate)
		: Predicate(InPredicate) {}

	/** Dereference pointers */
	FORCEINLINE bool operator()(const TObjectPtr<T>& A, const TObjectPtr<T>& B) const
	{
		return Predicate(*A, *B);
	}
};

template <typename T>
struct TCallTraits<TObjectPtr<T>> : public TCallTraitsBase<TObjectPtr<T>>
{
	using ConstPointerType = typename TCallTraitsParamTypeHelper<const TObjectPtr<const T>, true>::ConstParamType;
};


template <typename T>
[[nodiscard]] FORCEINLINE TWeakObjectPtr<T> MakeWeakObjectPtr(TObjectPtr<T> Ptr)
{
#if UE_WITH_REMOTE_OBJECT_HANDLE
	if (Ptr.GetResidence() != EResidence::Local)
	{
		return TWeakObjectPtr<T>(Ptr.GetRemoteId());
	}
	else
#endif
	{
		return TWeakObjectPtr<T>(Ptr);
	}
}

FORCEINLINE TObjectPtr<UObject>& FObjectPtr::ToTObjectPtr()
{
	return *reinterpret_cast<TObjectPtr<UObject>*>(this);
}

FORCEINLINE const TObjectPtr<UObject>& FObjectPtr::ToTObjectPtr() const
{
	return *reinterpret_cast<const TObjectPtr<UObject>*>(this);
}

/** Swap variants between TObjectPtr<T> and raw pointer to T */
template <typename T>
inline void Swap(TObjectPtr<T>& A, T*& B)
{
	Swap(static_cast<T*&>(MutableView(A)), B);
}
template <typename T>
inline void Swap(T*& A, TObjectPtr<T>& B)
{
	Swap(A, static_cast<T*&>(MutableView(B)));
}

/** Swap variants between TArray<TObjectPtr<T>> and TArray<T*> */
#if !UE_DEPRECATE_MUTABLE_TOBJECTPTR
UE_DEPRECATED(5.6, "Swap between TObjectPtr arrays and raw pointer arrays is deprecated. Swap TArray<TObjectPtr<...>> values instead.")
template <typename T>
inline void Swap(TArray<TObjectPtr<T>>& A, TArray<T*>& B)
{
	Swap(ToRawPtrTArrayUnsafe(A), B);
}
template <typename T>
inline void Swap(TArray<T*>& A, TArray<TObjectPtr<T>>& B)
{
	Swap(A, ToRawPtrTArrayUnsafe(B));
}
#endif

/** Exchange variants between TObjectPtr<T> and raw pointer to T */
template <typename T>
inline void Exchange(TObjectPtr<T>& A, T*& B)
{
	Swap(static_cast<T*&>(MutableView(A)), B);
}
template <typename T>
inline void Exchange(T*& A, TObjectPtr<T>& B)
{
	Swap(A, static_cast<T*&>(MutableView(B)));
}

#if !UE_DEPRECATE_MUTABLE_TOBJECTPTR
/** Exchange variants between TArray<TObjectPtr<T>> and TArray<T*> */
UE_DEPRECATED(5.6, "Exchange between TObjectPtr arrays and raw pointer arrays is deprecated. Exchange TArray<TObjectPtr<...>> values instead.")
template <typename T>
inline void Exchange(TArray<TObjectPtr<T>>& A, TArray<T*>& B)
{
	Swap(ToRawPtrTArrayUnsafe(A), B);
}
template <typename T>
inline void Exchange(TArray<T*>& A, TArray<TObjectPtr<T>>& B)
{
	Swap(A, ToRawPtrTArrayUnsafe(B));
}
#endif

/**
 * Returns a pointer to a valid object if the Test object passes IsValid() tests, otherwise null
 */
template <typename T>
[[nodiscard]] T* GetValid(const TObjectPtr<T>& Test)
{
	T* TestPtr = ToRawPtr(Test);
	return IsValid(TestPtr) ? TestPtr : nullptr;
}

//------------------------------------------------------------------------------
// Suppose you want to have a function that outputs an array of either T*'s or TObjectPtr<T>'s
// this template will make that possible to encode,
//  
// template<typename InstanceClass>
// void GetItemInstances(TArray<InstanceClass>& OutInstances) const
// {
//     static_assert(TIsPointerOrObjectPtrToBaseOf<InstanceClass, UBaseClass>::Value, "Output array must contain pointers or TObjectPtrs to a base of UBaseClass");
// }
//------------------------------------------------------------------------------

template <typename T, typename DerivedType>
struct TIsPointerOrObjectPtrToBaseOfImpl
{
	enum { Value = false };
};

template <typename T, typename DerivedType>
struct TIsPointerOrObjectPtrToBaseOfImpl<T*, DerivedType>
{
	enum { Value = std::is_base_of_v<DerivedType, T> };
};

template <typename T, typename DerivedType>
struct TIsPointerOrObjectPtrToBaseOfImpl<TObjectPtr<T>, DerivedType>
{
	enum { Value = std::is_base_of_v<DerivedType, T> };
};

template <typename T, typename DerivedType>
struct TIsPointerOrObjectPtrToBaseOf
{
	enum { Value = TIsPointerOrObjectPtrToBaseOfImpl<std::remove_cv_t<T>, DerivedType>::Value };
};

//------------------------------------------------------------------------------
// Suppose now that you have a templated function that takes in an array of either
// UBaseClass*'s or TObjectPtr<UBaseClass>'s, and you need to use that inner UBaseClass type
// for coding,
// 
// This is how you can refer to that inner class.  Where InstanceClass is the template
// argument for where TIsPointerOrObjectPtrToBaseOf is used,
// 
// TPointedToType<InstanceClass>* Thing = new TPointedToType<InstanceClass>();
//------------------------------------------------------------------------------

template <typename T>
struct TPointedToTypeImpl;

template <typename T>
struct TPointedToTypeImpl<T*>
{
	using Type = T;
};

template <typename T>
struct TPointedToTypeImpl<TObjectPtr<T>>
{
	using Type = T;
};

template <typename T>
using TPointedToType = typename TPointedToTypeImpl<T>::Type;

//------------------------------------------------------------------------------

namespace UE::Core::Private // private facilities; not for direct use
{
	template <typename T>
	struct TObjectPtrDecayTypeOf
	{
		using Type = T;
		static FORCEINLINE void PerformDecayActions(const T&) { /* nb: intentionally empty */ }
	};

	template <typename T>
	struct TObjectPtrDecayTypeOf<TObjectPtr<T>>
	{
		using Type = T*;

		static FORCEINLINE void PerformDecayActions(const TObjectPtr<T>& Value)
		{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
			(void)Value.Get();
			check(Value.IsResolved());
#endif
		}
	};
	
	template <typename T>
	struct TObjectPtrDecayTypeOf<TSet<T>>
	{
		using Type = TSet<typename TObjectPtrDecayTypeOf<T>::Type>;

		static FORCEINLINE void PerformDecayActions(const TSet<T>& Value)
		{
			for (const auto& V : Value)
			{
				TObjectPtrDecayTypeOf<T>::PerformDecayActions(V);
			}
		}
	};

	template <typename K, typename V>
	struct TObjectPtrDecayTypeOf<TMap<K, V>>
	{
		using Type = TMap<typename TObjectPtrDecayTypeOf<K>::Type, typename TObjectPtrDecayTypeOf<V>::Type>;
			
		static FORCEINLINE void PerformDecayActions(const TMap<K, V>& Value)
		{
			for (const auto& KV : Value)
			{
				TObjectPtrDecayTypeOf<K>::PerformDecayActions(KV.Key);
				TObjectPtrDecayTypeOf<V>::PerformDecayActions(KV.Value);
			}
		}
	};

	template <typename T>
	struct TObjectPtrDecayTypeOf<TArray<T>>
	{
		using Type = TArray<typename TObjectPtrDecayTypeOf<T>::Type>;
			
		static FORCEINLINE void PerformDecayActions(const TArray<T>& Value)
		{
			for (const auto& V : Value)
			{
				TObjectPtrDecayTypeOf<T>::PerformDecayActions(V);
			}
		}
	};

	template <typename T>
	struct TObjectPtrDecayTypeOf<TNonNullPtr<TObjectPtr<T>>>
	{
		using Type = TNonNullPtr<T>;

		static FORCEINLINE void PerformDecayActions(const TNonNullPtr<TObjectPtr<T>>& Value)
		{
#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING
			(void)Value.GetRef().Get();
#endif
		}
	};

	template <typename T>
	struct TObjectPtrWrapTypeOf
	{
		using Type = T;
	};
	
	template <typename T>
	struct TObjectPtrWrapTypeOf<T*>
	{
		using Type = TObjectPtr<T>;
	};

	template <typename T>
	struct TObjectPtrWrapTypeOf<TArrayView<T>>
	{
		using Type = TArrayView<typename TObjectPtrWrapTypeOf<T>::Type>;
	};
	
	template <typename T>
	struct TObjectPtrWrapTypeOf<TArray<T>>
	{
		using Type = TArray<typename TObjectPtrWrapTypeOf<T>::Type>;
	};

	template <typename T>
	struct TObjectPtrWrapTypeOf<TSet<T>>
	{
		using Type = TSet<typename TObjectPtrWrapTypeOf<T>::Type>;
	};

	template <typename K, typename V>
	struct TObjectPtrWrapTypeOf<TMap<K, V>>
	{
		using Type = TMap<typename TObjectPtrWrapTypeOf<K>::Type, typename TObjectPtrWrapTypeOf<V>::Type>;
	};	

	namespace Unsafe
	{
		template <typename T,
							typename DecayTraits = UE::Core::Private::TObjectPtrDecayTypeOf<T>,
							typename U = typename DecayTraits::Type>
		[[nodiscard]] U& Decay(T& A)
		{
			DecayTraits::PerformDecayActions(A);
			return reinterpret_cast<U&>(A);
		}
	}

	template <typename T, typename ViewType>
	struct TMutableViewTraits
	{
		static_assert(sizeof(T) == 0, "TMutableView not supported for this type. (Should it be?)");
	};

	template <typename T, typename ViewType>
	struct TMutableViewTraits<TObjectPtr<T>, ViewType>
	{
		static void Close(ViewType& View)
		{
#if UE_OBJECT_PTR_GC_BARRIER
			if (UE::GC::GIsIncrementalReachabilityPending && View)
			{
				UE::GC::MarkAsReachable(View);
			}
#endif // UE_OBJECT_PTR_GC_BARRIER
		}
	};
	
	template <typename T, typename ViewType>
	struct TMutableViewTraits<TArray<TObjectPtr<T>>, ViewType>
	{
		static void Close(ViewType& View)
		{
#if UE_OBJECT_PTR_GC_BARRIER
			if (UE::GC::GIsIncrementalReachabilityPending)
			{
				const UObject* const* Data = reinterpret_cast<const UObject* const*>(View.GetData());
				for (int32 Index = 0; Index < View.Num(); ++Index)
				{
					if (Data[Index])
					{
						UE::GC::MarkAsReachable(Data[Index]);
					}
				}
			}
#endif // UE_OBJECT_PTR_GC_BARRIER
		}
	};

	template <typename V, typename ViewType>
	struct TMutableViewTraits<TSet<TObjectPtr<V>>, ViewType>
	{
		static void Close(ViewType& View)
		{
#if UE_OBJECT_PTR_GC_BARRIER
			if (UE::GC::GIsIncrementalReachabilityPending)
			{
				for (const typename ViewType::ElementType& Element : View)
				{
					if (Element)
					{
						UE::GC::MarkAsReachable(Element);
					}
				}
			}
#endif // UE_OBJECT_PTR_GC_BARRIER
		}
	};

	template <typename K, typename V, typename ViewType>
	struct TMutableViewTraits<TMap<K, V>, ViewType>
	{
		static void Close(ViewType& View)
		{
#if UE_OBJECT_PTR_GC_BARRIER
			static constexpr bool bKeyReference = TIsTObjectPtr_V<K>;
			static constexpr bool bValueReference = TIsTObjectPtr_V<V>;
			static_assert(bKeyReference || bValueReference);
			if (UE::GC::GIsIncrementalReachabilityPending)
			{
				for (const typename ViewType::ElementType& Pair : View)
				{
					if constexpr (bKeyReference)
					{
						if (Pair.Key)
						{
							UE::GC::MarkAsReachable(Pair.Key);
						}
					}
					if constexpr (bValueReference)
					{
						if (Pair.Value)
						{
							UE::GC::MarkAsReachable(Pair.Value);
						}
					}
				}
			}
#endif // UE_OBJECT_PTR_GC_BARRIER
		}
	};
	
	template <typename T>
	class TMutableView
	{
	public:
		using ViewType = typename TObjectPtrDecayTypeOf<T>::Type;
		using TraitType = UE::Core::Private::TMutableViewTraits<T, ViewType>;

		explicit TMutableView(T& Value)
			: View{&Unsafe::Decay(Value)}
		{
		}

		~TMutableView()
		{
			TraitType::Close(*View);
		}

		TMutableView(const TMutableView&) = delete;
		TMutableView(TMutableView&&) = delete;
		TMutableView& operator=(const TMutableView&) = delete;
		TMutableView& operator=(TMutableView&&) = delete;
		
		operator ViewType&() const
		{
			return *View;
		}

	private:
		ViewType* View;
	};

	// nb: TMaybeObjectPtr class exists as a temporary compatibility shim with existing code;
	//     do not use in new code. it allows code that holds pointers to abstract classes that
	//     might point to instances that also subclass UObject to properly interact with
	//     garbage collection.
	template <typename T>
	class TMaybeObjectPtr final
	{
		static_assert(!std::is_convertible_v<T, const UObjectBase*>, "TMaybeObjectPtr's type argument shouldn't be a subclass of UObjectBase");

	public:
		TMaybeObjectPtr() = default;

		explicit TMaybeObjectPtr(T* P)
			: Ptr{P}
		{
			ConditionallyMarkAsReachable();
		}

		TMaybeObjectPtr(const TMaybeObjectPtr& Other)
			: TMaybeObjectPtr{Other.Ptr}
		{
		}

		TMaybeObjectPtr(TMaybeObjectPtr&& Other)
			: TMaybeObjectPtr{Other.Ptr}
		{
		}

		TMaybeObjectPtr& operator=(const TMaybeObjectPtr& Other)
		{
			return *this = Other.Ptr;
		}

		TMaybeObjectPtr& operator=(TMaybeObjectPtr&& Other)
		{
			return *this = Other;
		}

		TMaybeObjectPtr& operator=(T* P)
		{
			Ptr = P;
			ConditionallyMarkAsReachable();
			return *this;
		}
		
		operator T*() const
		{
			return Ptr;
		}
		
		void AddReferencedObject(FReferenceCollector& Collector, UObject* ReferencingObject)
		{
			if (const UObject* Obj = Cast<UObject>(Ptr))
			{
				TObjectPtr<UObject> ObjectPtr{const_cast<UObject*>(Obj)};
				Collector.AddReferencedObject(ObjectPtr, ReferencingObject);
				if (!ObjectPtr)
				{
					Ptr = nullptr;
				}
			}
		}

	private:
		FORCEINLINE void ConditionallyMarkAsReachable() const
		{
			if (const UObject* Obj = Cast<UObject>(Ptr); Obj && UE::GC::GIsIncrementalReachabilityPending)
			{
				UE::GC::MarkAsReachable(Obj);
			}
		}

		T* Ptr{};
	};	
}

/*
	MutableView: safely obtain temporary mutable access to a TObjectPtr's
							 underlying storage.

	// Basic Usage:

		void MutatingFunc(TArray<UObject*>& MutableArray);

		TArray<TObjectPtr<UObject>> Array;
		MutatingFunc(Array);							 // unsafe; compile error

		MutatingFunc(MutableView(Array));			 // ok; Array will safely "catch up" on TObjectPtr
																					 // semantics when MutatingFunc returns.

		// it's generally preferable to pass references around (to avoid nullptr),
		// but for compat with existing functions that take a pointer:

		void NeedsAPointer(TArray<UObject*>* MutableArrayPtr);
		NeedsAPointer(ToRawPtr(MutableView(Array)));

	// Scoped Usage:

		TObjectPtr<UObject> MyPtr;
		{
			auto MyScopedView = MutableView(MyPtr);
			UObject*& MyRawPtrRef = MyScopedView;									 // ok

			UObject*& MyHardToFindBug = MutableView(MyPtr);				 // not ok
		}
*/

template <typename T>
[[nodiscard]] decltype(auto) MutableView(T& A)
{
	return UE::Core::Private::TMutableView<T>{A};
}

template <typename T>
[[nodiscard]] decltype(auto) ToRawPtr(const UE::Core::Private::TMutableView<T>& X)
{
	typename std::remove_reference_t<decltype(X)>::ViewType& Ref = X;
	return &Ref;
}

/*
	whenever const access to the underlying storage of a TObjectPtr<...> (or a container of them)
	is needed, use Decay, eg:

		TMap<int, TArray<TObjectPtr<UObject>>> MyContainer;

		void MyFunc(const TMap<int, TArray<UObject*>>&);

		MyFunc(MyContainer);							// compile error
		MyFunc(ObjectPtrDecay(MyContainer));				// ok
*/
template <typename T,
					typename DecayTraits = typename UE::Core::Private::TObjectPtrDecayTypeOf<T>,
					typename U = typename DecayTraits::Type>
[[nodiscard]] const U& ObjectPtrDecay(const T& Value)
{
	DecayTraits::PerformDecayActions(Value);
	return reinterpret_cast<const U&>(Value);
}

/*
	"Wrap" is the opposite of "Decay"
*/
template <typename T,
					typename U = typename UE::Core::Private::TObjectPtrWrapTypeOf<T>::Type>
[[nodiscard]] U& ObjectPtrWrap(T& Value)
{
	return reinterpret_cast<U&>(Value);
}

template <typename T>
using TObjectPtrWrapTypeOf = typename UE::Core::Private::TObjectPtrWrapTypeOf<T>::Type;

template <typename T,
					typename U = typename UE::Core::Private::TObjectPtrWrapTypeOf<T>::Type>
[[nodiscard]] const U& ObjectPtrWrap(const T& Value)
{
	return reinterpret_cast<const U&>(Value);
}

/* cast utilities; best avoided, use mostly for compatibility with existing code */
template <typename To, typename From>
[[nodiscard]] FORCEINLINE To* StaticCastPtr(const TObjectPtr<From>& P)
{
	return static_cast<To*>(P.Get());
}

template <typename T>
[[nodiscard]] FORCEINLINE decltype(auto) ConstCast(const TObjectPtr<T>& P)
{
	return reinterpret_cast<TObjectPtr<std::remove_cv_t<T>>&>(const_cast<TObjectPtr<T>&>(P));
}


template<typename ObjectType>
class TNonNullPtr<TObjectPtr<ObjectType>>
{
public:
	
	[[nodiscard]] FORCEINLINE TNonNullPtr(EDefaultConstructNonNullPtr)
		: Object(nullptr)
	{	
	}	

	/**
	 * nullptr constructor - not allowed.
	 */
	[[nodiscard]] FORCEINLINE TNonNullPtr(TYPE_OF_NULLPTR)
	{
		// Essentially static_assert(false), but this way prevents GCC/Clang from crying wolf by merely inspecting the function body
		static_assert(sizeof(ObjectType) == 0, "Tried to initialize TNonNullPtr with a null pointer!");
	}

	/**
	 * Constructs a non-null pointer from the provided pointer. Must not be nullptr.
	 */
	[[nodiscard]] FORCEINLINE TNonNullPtr(TObjectPtr<ObjectType> InObject)
		: Object(InObject)
	{
		ensureAlwaysMsgf(InObject, TEXT("Tried to initialize TNonNullPtr with a null pointer!"));
	}

	/**
	 * Constructs a non-null pointer from another non-null pointer
	 */
	template <
		typename OtherObjectType
		UE_REQUIRES(std::is_convertible_v<OtherObjectType*, ObjectType*>)
	>
	[[nodiscard]] FORCEINLINE TNonNullPtr(const TNonNullPtr<OtherObjectType>& Other)
		: Object(Other.Object)
	{
	}

	/**
	 * Assignment operator taking a nullptr - not allowed.
	 */
	FORCEINLINE TNonNullPtr& operator=(TYPE_OF_NULLPTR)
	{
		// Essentially static_assert(false), but this way prevents GCC/Clang from crying wolf by merely inspecting the function body
		static_assert(sizeof(ObjectType) == 0, "Tried to assign a null pointer to a TNonNullPtr!");
		return *this;
	}

	/**
	 * Assignment operator taking a pointer
	 */
	FORCEINLINE TNonNullPtr& operator=(ObjectType* InObject)
	{
		ensureMsgf(InObject, TEXT("Tried to assign a null pointer to a TNonNullPtr!"));
		Object = InObject;
		return *this;
	}

	/**
	 * Assignment operator taking another TNonNullPtr
	 */
	template <
		typename OtherObjectType
		UE_REQUIRES(std::is_convertible_v<OtherObjectType*, ObjectType*>)
	>
	FORCEINLINE TNonNullPtr& operator=(const TNonNullPtr<OtherObjectType>& Other)
	{
		Object = Other.Object;
		return *this;
	}

	/**
	 * Returns the internal pointer
	 */
	[[nodiscard]] FORCEINLINE operator ObjectType*() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));
		return Object;
	}

	/**
	 * Returns the internal pointer
	 */
	[[nodiscard]] FORCEINLINE ObjectType* Get() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));
		return Object;
	}

	/**
	 * Dereference operator returns a reference to the object this pointer points to
	 */
	[[nodiscard]] FORCEINLINE ObjectType& operator*() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));
		return *Object;
	}

	/**
	 * Arrow operator returns a pointer to this pointer's object
	 */
	[[nodiscard]] FORCEINLINE ObjectType* operator->() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));
		return Object;
	}

	[[nodiscard]] FORCEINLINE const TObjectPtr<ObjectType>& GetRef() const
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));		
		return Object;
	}
	
	[[nodiscard]] FORCEINLINE TObjectPtr<ObjectType>& GetRef() 
	{
		ensureMsgf(Object, TEXT("Tried to access null pointer!"));		
		return Object;
	}	

	[[nodiscard]] FORCEINLINE bool IsInitialized() const
	{
		return Object != nullptr;
	}

	/**
	 * Use IsInitialized if needed
	 */
	explicit operator bool() const = delete;

private:
	friend FORCEINLINE uint32 GetTypeHash(const TNonNullPtr& InPtr)
	{
		return GetTypeHash(InPtr.Object);
	}

	/** The object we're holding a reference to. */
	TObjectPtr<ObjectType> Object;
};
