// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/LinkedListBuilder.h"
#include "UObject/ObjectPtr.h"

template<typename InElementType>
struct TObjectPtrLinkedListBuilderNextLink
{
	using ElementType = InElementType;

	[[nodiscard]] UE_FORCEINLINE_HINT static auto GetNextPtr(ElementType& Element)
	{
		return &(Element.Next);
	}
};

/**
 * Single linked list builder for some cases where we want to migrate from raw ptrs to TObjectPtr
 * In the existing cases, the start ptr is TObjectPtr and the links are raw pointers. 
 * We wish to migrate the links to TObjectPtr.
 */
template<typename InElementType, typename InLinkAccessor = TObjectPtrLinkedListBuilderNextLink<InElementType>>
struct TObjectPtrLinkedListBuilder : public TLinkedListBuilderBase<InElementType, TObjectPtr<InElementType>, InLinkAccessor>
{
	using Super = TLinkedListBuilderBase<InElementType, TObjectPtr<InElementType>, InLinkAccessor>;
	using ElementType = Super::ElementType;
	using PointerType = Super::PointerType;
	using Super::Super;

	UE_NONCOPYABLE(TObjectPtrLinkedListBuilder);

	// Allow construction from raw ptr because call sites were handling the type mismatch between
	// the head and next ptrs in lists by converting the TObjectPtr to a raw ptr with MutableView.
	[[nodiscard]] explicit TObjectPtrLinkedListBuilder(ElementType** ListStartPtr)
		: Super(reinterpret_cast<PointerType*>(ListStartPtr))
	{
	}
};