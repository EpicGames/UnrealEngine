// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"

/** Utility type to iterate a linked list of TObjectPtr, optionally avoiding access tracking for performance reasons. */
template<typename InElementType, TObjectPtr<InElementType> InElementType::*InNextMember, bool bConst, bool bNoAccessTracking>
struct TObjectPtrLinkedListIteratorBase
{
    using ElementType = std::conditional_t<bConst, const InElementType, InElementType>;

    TObjectPtrLinkedListIteratorBase() = default;
    TObjectPtrLinkedListIteratorBase(ElementType* InStart)
        : Current(InStart)
    {
    }
    TObjectPtrLinkedListIteratorBase(const TObjectPtrLinkedListIteratorBase&) = default;
    TObjectPtrLinkedListIteratorBase(TObjectPtrLinkedListIteratorBase&&) = default;
    TObjectPtrLinkedListIteratorBase& operator=(const TObjectPtrLinkedListIteratorBase&) = default;
    TObjectPtrLinkedListIteratorBase& operator=(TObjectPtrLinkedListIteratorBase&&) = default;

    TObjectPtrLinkedListIteratorBase& operator++()
    {
        return MoveNext();
    }

    TObjectPtrLinkedListIteratorBase operator++(int)
    {
        return MoveNext();
    }

    explicit operator bool()
    {
        return !!Current;
    }

    ElementType* operator*() const
    {
        return Current;
    }

    ElementType* operator->() const
    {
        return Current;
    }

    bool operator==(const TObjectPtrLinkedListIteratorBase& Other) const
    {
        return Current == Other.Current;
    }

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
    bool operator!=(const TObjectPtrLinkedListIterator& Other) const
    {
        return Current != Other.Current;
    }
#endif

    TObjectPtrLinkedListIteratorBase begin()
    {
        return *this;
    }
    TObjectPtrLinkedListIteratorBase end()
    {
        return TObjectPtrLinkedListIteratorBase{};
    }
private:
    TObjectPtrLinkedListIteratorBase& MoveNext()
    {
        if constexpr (bNoAccessTracking)
        {
            Current = ObjectPtr_Private::Friend::NoAccessTrackingGet(Current->*InNextMember);
        }
        else 
        {
            Current = Current->*InNextMember.Get();
        }
        return *this;
    }

    ElementType* Current = nullptr;
};

template<typename InElementType, TObjectPtr<InElementType> InElementType::*InNextMember, bool bNoAccessTracking>
using TObjectPtrLinkedListIterator = TObjectPtrLinkedListIteratorBase<InElementType, InNextMember, /* bConst*/ false, bNoAccessTracking>;

template<typename InElementType, TObjectPtr<InElementType> InElementType::*InNextMember, bool bNoAccessTracking>
using TObjectPtrLinkedListIteratorConst = TObjectPtrLinkedListIteratorBase<InElementType, InNextMember, /* bConst*/ true, bNoAccessTracking>;