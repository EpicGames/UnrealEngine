// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnRedirector.h: Object redirector definition.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/IntrinsicClassFwd.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

class FArchive;
class FObjectPreSaveContext;
class FString;

UE_DEFINE_INTRINSIC_CLASS_CONSTINIT_ACCESSOR(UObjectRedirector, COREUOBJECT_API);

/**
 * This class will redirect an object load to another object, so if an object is renamed
 * to a different package or group, external references to the object can be found
 */
class UObjectRedirector : public UObject
{
	DECLARE_CASTED_CLASS_INTRINSIC_WITH_API(UObjectRedirector, UObject, CLASS_MatchedSerializers, TEXT("/Script/CoreUObject"), CASTCLASS_None, COREUOBJECT_API)

	// Variables.
	UObject*		DestinationObject;
	// UObject interface.
	virtual void PreSave(FObjectPreSaveContext ObjectSaveContext) override;
	void Serialize(FArchive& Ar) override;
	void Serialize(FStructuredArchive::FRecord Record) override;
	virtual bool NeedsLoadForEditorGame() const override;
	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	virtual bool HasNonEditorOnlyReferences() const override
	{
		return true;
	}

	/**
	 * Callback for retrieving a textual representation of natively serialized properties.  Child classes should implement this method if they wish
	 * to have natively serialized property values included in things like diffcommandlet output.
	 *
	 * @param	out_PropertyValues	receives the property names and values which should be reported for this object.  The map's key should be the name of
	 *								the property and the map's value should be the textual representation of the property's value.  The property value should
	 *								be formatted the same way that FProperty::ExportText formats property values (i.e. for arrays, wrap in quotes and use a comma
	 *								as the delimiter between elements, etc.)
	 * @param	ExportFlags			bitmask of EPropertyPortFlags used for modifying the format of the property values
	 *
	 * @return	return true if property values were added to the map.
	 */
	virtual bool GetNativePropertyValues( TMap<FString,FString>& out_PropertyValues, uint32 ExportFlags=0 ) const override;

#if WITH_EDITOR
	/** @return TRUE when serializing the reference to the destination object. */
	bool IsSerializingDestinationObject() const
	{
		return bIsSerializingDestinationObject;
	}

private:
	bool bIsSerializingDestinationObject = false;
#endif
};
