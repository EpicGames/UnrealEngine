// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/StringBuilder.h"
#include "UObject/ObjectHandleDefines.h"
#include "UObject/ObjectFwd.h"
#include "UObject/ObjectMacros.h"

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_OBJECT_HANDLE_TRACKING

DECLARE_LOG_CATEGORY_EXTERN(LogObjectRef, Log, All);


struct FObjectRef;

namespace UE::CoreUObject::Private
{
	struct FPackedObjectRef;
	class FObjectPathId;
	FPackedObjectRef MakePackedObjectRef(const FObjectRef& ObjectRef);
}

/**
 * FObjectRef represents a heavyweight reference that contains the specific pieces of information needed to reference an object
 * (or null) that may or may not be loaded yet.  The expectation is that given the imports of a package we have loaded, we should
 * be able to create an FObjectRef to objects in a package we haven't yet loaded.  For this reason, FObjectRef has to be derivable
 * from the serialized (not transient) contents of an FObjectImport.
 */
struct FObjectRef
{
	FObjectRef() = default;
	FObjectRef(FName PackageName, FName ClassPackageName, FName ClassName, UE::CoreUObject::Private::FObjectPathId ObjectPath);
	explicit FObjectRef(const UObject* Object);

	bool operator==(const FObjectRef&& Other) const
	{
		return (PackageName == Other.PackageName) && (ObjectPathId == Other.ObjectPathId) && (ClassPackageName == Other.ClassPackageName) && (ClassName == Other.ClassName);
	}

	/** Returns the path to the class for this object. */
	FString GetClassPathName(EObjectFullNameFlags Flags = EObjectFullNameFlags::IncludeClassPackage) const
	{
		TStringBuilder<FName::StringBufferSize> Result;
		AppendClassPathName(Result, Flags);
		return FString(Result);
	}

	/** Appends the path to the class for this object to the builder, does not reset builder. */
	COREUOBJECT_API void AppendClassPathName(FStringBuilderBase& OutClassPathNameBuilder, EObjectFullNameFlags Flags = EObjectFullNameFlags::IncludeClassPackage) const;

	/** Returns the name of the object in the form: ObjectName */
	COREUOBJECT_API FName GetFName() const;

	/** Returns the full path for the object in the form: ObjectPath */
	FString GetPathName() const
	{
		TStringBuilder<FName::StringBufferSize> Result;
		AppendPathName(Result);
		return FString(Result);
	}

	/** Appends the path to the object to the builder, does not reset builder. */
	COREUOBJECT_API void AppendPathName(FStringBuilderBase& OutPathNameBuilder) const;

	/** Returns the full name for the object in the form: Class ObjectPath */
	FString GetFullName(EObjectFullNameFlags Flags = EObjectFullNameFlags::None) const
	{
		TStringBuilder<256> FullName;
		GetFullName(FullName, Flags);
		return FString(FullName);
	}

	/** Populates OutFullNameBuilder with the full name for the object in the form: Class ObjectPath */
	void GetFullName(FStringBuilderBase& OutFullNameBuilder, EObjectFullNameFlags Flags = EObjectFullNameFlags::None) const
	{
		OutFullNameBuilder.Reset();
		AppendClassPathName(OutFullNameBuilder, Flags);
		OutFullNameBuilder.AppendChar(TEXT(' '));
		AppendPathName(OutFullNameBuilder);
	}

	/** Returns the name for the asset in the form: Class'ObjectPath' */
	FString GetExportTextName() const
	{
		TStringBuilder<256> ExportTextName;
		GetExportTextName(ExportTextName);
		return FString(ExportTextName);
	}

	/** Populates OutExportTextNameBuilder with the name for the object in the form: Class'ObjectPath' */
	void GetExportTextName(FStringBuilderBase& OutExportTextNameBuilder) const
	{
		OutExportTextNameBuilder.Reset();
		AppendClassPathName(OutExportTextNameBuilder);
		OutExportTextNameBuilder.AppendChar(TEXT('\''));
		AppendPathName(OutExportTextNameBuilder);
		OutExportTextNameBuilder.AppendChar(TEXT('\''));
	}

	inline bool IsNull() const 
	{ 
		return PackageName.IsNone() && ObjectPathId == 0;
	}

	COREUOBJECT_API UClass* ResolveObjectRefClass(uint32 LoadFlags = LOAD_None) const;
	COREUOBJECT_API UObject* Resolve(uint32 LoadFlags = LOAD_None) const;

public:
	FName PackageName;
	FName ClassPackageName;
	FName ClassName;

private:
	uint64 ObjectPathId = 0;

	friend UE::CoreUObject::Private::FPackedObjectRef UE::CoreUObject::Private::MakePackedObjectRef(const FObjectRef& ObjectRef);
	UE::CoreUObject::Private::FObjectPathId GetObjectPath() const;

};

#endif
