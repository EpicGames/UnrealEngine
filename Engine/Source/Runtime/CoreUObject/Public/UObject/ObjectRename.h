// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "HAL/Platform.h"

class UPackage;

namespace UE::Object
{
	// Helper function to rename a package which ought to have been freed but which can't be GC'd
	// because of outstanding references.
	// Allows a new copy of that package to be loaded. 
	// Flushes any internal caches (e.g. async loading) that would be broken by simply renaming the object.
	COREUOBJECT_API void RenameLeakedPackage(UPackage* Package);
}
