// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CString.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Guid.h"
#include "Serialization/BulkDataCookedIndex.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#if !WITH_VERSE_VM
#include "Containers/Utf8String.h"
#endif

class FArchive;
class FLinkerLoad;
class FPackageIndex;
class UClass;
class UObject;

namespace Verse
{
	struct VCppClassInfo;
	struct VCell;
	struct VUniqueString;
}

/**
 * Wrapper for index into a ULnker's ImportMap or ExportMap.
 * Values greater than zero indicate that this is an index into the ExportMap.  The
 * actual array index will be (FPackageIndex - 1).
 *
 * Values less than zero indicate that this is an index into the ImportMap. The actual
 * array index will be (-FPackageIndex - 1)
 */
class FPackageIndex
{
	/**
	 * Values greater than zero indicate that this is an index into the ExportMap.  The
	 * actual array index will be (FPackageIndex - 1).
	 *
	 * Values less than zero indicate that this is an index into the ImportMap. The actual
	 * array index will be (-FPackageIndex - 1)
	 */
	int32 Index;

	/** Internal constructor, sets the index directly **/
	UE_FORCEINLINE_HINT explicit FPackageIndex(int32 InIndex)
		: Index(InIndex)
	{

	}
public:
	/** Constructor, sets the value to null **/
	UE_FORCEINLINE_HINT FPackageIndex()
		: Index(0)
	{

	}
	/** return true if this is an index into the import map **/
	UE_FORCEINLINE_HINT bool IsImport() const
	{
		return Index < 0;
	}
	/** return true if this is an index into the export map **/
	UE_FORCEINLINE_HINT bool IsExport() const
	{
		return Index > 0;
	}
	/** return true if this null (i.e. neither an import nor an export) **/
	UE_FORCEINLINE_HINT bool IsNull() const
	{
		return Index == 0;
	}
	/** Check that this is an import and return the index into the import map **/
	inline int32 ToImport() const
	{
		check(IsImport());
		return -Index - 1;
	}
	/** Check that this is an export and return the index into the export map **/
	inline int32 ToExport() const
	{
		check(IsExport());
		return Index - 1;
	}
	/** Return the raw value, for debugging purposes**/
	UE_FORCEINLINE_HINT int32 ForDebugging() const
	{
		return Index;
	}

	/** Create a FPackageIndex from an import index **/
	inline static FPackageIndex FromImport(int32 ImportIndex)
	{
		check(ImportIndex >= 0);
		return FPackageIndex(-ImportIndex - 1);
	}
	/** Create a FPackageIndex from an export index **/
	inline static FPackageIndex FromExport(int32 ExportIndex)
	{
		check(ExportIndex >= 0);
		return FPackageIndex(ExportIndex + 1);
	}

	/** Compare package indecies for equality **/
	UE_FORCEINLINE_HINT bool operator==(const FPackageIndex& Other) const
	{
		return Index == Other.Index;
	}
	/** Compare package indecies for inequality **/
	UE_FORCEINLINE_HINT bool operator!=(const FPackageIndex& Other) const
	{
		return Index != Other.Index;
	}

	/** Compare package indecies **/
	UE_FORCEINLINE_HINT bool operator<(const FPackageIndex& Other) const
	{
		return Index < Other.Index;
	}
	UE_FORCEINLINE_HINT bool operator>(const FPackageIndex& Other) const
	{
		return Index > Other.Index;
	}
	UE_FORCEINLINE_HINT bool operator<=(const FPackageIndex& Other) const
	{
		return Index <= Other.Index;
	}
	UE_FORCEINLINE_HINT bool operator>=(const FPackageIndex& Other) const
	{
		return Index >= Other.Index;
	}
	/**
	 * Serializes a package index value from or into an archive.
	 *
	 * @param Ar - The archive to serialize from or to.
	 * @param Value - The value to serialize.
	 */
	inline friend FArchive& operator<<(FArchive& Ar, FPackageIndex& Value)
	{
		FStructuredArchiveFromArchive(Ar).GetSlot() << Value;
		return Ar;
	}

	/**
	 * Serializes a package index value from or into a structured archive slot.
	 *
	 * @param Slot - The structured archive slot to serialize from or to.
	 * @param Value - The value to serialize.
	 */
	UE_FORCEINLINE_HINT friend void operator<<(FStructuredArchive::FSlot Slot, FPackageIndex& Value)
	{
		Slot << Value.Index;
	}

	[[nodiscard]] UE_FORCEINLINE_HINT friend uint32 GetTypeHash(const FPackageIndex& In)
	{
		return uint32(In.Index);
	}

	/**
		Lex functions
	*/
	/**
	Lex functions
	*/
	friend FString LexToString(const FPackageIndex& Value)
	{
		return FString::FromInt(Value.Index);
	}

	friend void LexFromString(FPackageIndex& Value, const TCHAR* String)
	{
		Value.Index = FCString::Atoi(String);
	}
};

/**
 * Base class for UObject resource types.  FObjectResources are used to store UObjects on disk
 * via FLinker's ImportMap (for resources contained in other packages) and ExportMap (for resources
 * contained within the same package)
 */
struct FObjectResource
{
	/**
	 * The name of the UObject represented by this resource.
	 * Serialized
	 */
	FName			ObjectName;

	/**
	 * Location of the resource for this resource's Outer.  Values of 0 indicate that this resource
	 * represents a top-level UPackage object (the linker's LinkerRoot).
	 * Serialized
	 */
	FPackageIndex	OuterIndex;

#if WITH_EDITORONLY_DATA
	/**
	 * Name of the class this object was serialized with (in case active class redirects have changed it)
	 * If this is a class and was directly redirected, this is what it was redirected from
	 */
	FName			OldClassName;
#endif

	FObjectResource();
	FObjectResource( UObject* InObject );
};

/*-----------------------------------------------------------------------------
	FObjectExport.
-----------------------------------------------------------------------------*/

/**
 * UObject resource type for objects that are contained within this package and can
 * be referenced by other packages.
 */
struct FObjectExport : public FObjectResource
{
	/**
	 * Location of the resource for this export's class (if non-zero).  A value of zero
	 * indicates that this export represents a UClass object; there is no resource for
	 * this export's class object
	 * Serialized
	 */
	FPackageIndex  	ClassIndex;

	/**
	* Location of this resource in export map. Used for export fixups while loading packages.
	* Value of zero indicates resource is invalid and shouldn't be loaded.
	* Not serialized.
	*/
	FPackageIndex ThisIndex;

	/**
	 * Location of the resource for this export's SuperField (parent).  Only valid if
	 * this export represents a UStruct object. A value of zero indicates that the object
	 * represented by this export isn't a UStruct-derived object.
	 * Serialized
	 */
	FPackageIndex 	SuperIndex;

	/**
	* Location of the resource for this export's template/archetypes.  Only used
	* in the new cooked loader. A value of zero indicates that the value of GetArchetype
	* was zero at cook time, which is more or less impossible and checked.
	* Serialized
	*/
	FPackageIndex 	TemplateIndex;

	/**
	 * The object flags for the UObject represented by this resource.  Only flags that
	 * match the RF_Load combination mask will be loaded from disk and applied to the UObject.
	 * Serialized
	 */
	EObjectFlags	ObjectFlags;

	/**
	 * The number of bytes to serialize when saving/loading this export's UObject.
	 * Serialized
	 */
	int64         	SerialSize;

	/**
	 * The location (into the FLinker's underlying file reader archive) of the beginning of the
	 * data for this export's UObject.  Used for verification only.
	 * Serialized
	 */
	int64         	SerialOffset;

	/**
	 * The location (relative to SerialOffset) of the beginning of the portion of this export's data that is
	 * serialized using tagged property serialization.
	 * Serialized into versioned packages as of EUnrealEngineObjectUE5Version::SCRIPT_SERIALIZATION_OFFSET
	 * Otherwise transient
	 */
	int64				ScriptSerializationStartOffset;

	/**
	 * The location (relative to SerialOffset) of the end of the portion of this export's data that is 
	 * serialized using tagged property serialization.
	 * Serialized into versioned packages as of EUnrealEngineObjectUE5Version::SCRIPT_SERIALIZATION_OFFSET
	 * Otherwise transient
	 */
	int64				ScriptSerializationEndOffset;

	/**
	 * The UObject represented by this export.  Assigned the first time CreateExport is called for this export.
	 * Transient
	 */
	UObject*		Object;

	/**
	 * The index into the FLinker's ExportMap for the next export in the linker's export hash table.
	 * Transient
	 */
	int32			HashNext;

	/**
	 * Whether the export was forced into the export table via OBJECTMARK_ForceTagExp.
	 * Serialized
	 */
	bool			bForcedExport:1;   

	/**
	 * Whether the export should be loaded on clients
	 * Serialized
	 */
	bool			bNotForClient:1;   

	/**
	 * Whether the export should be loaded on servers
	 * Serialized
	 */
	bool			bNotForServer:1;

	/**
	 * Whether the export should be always loaded in editor game
	 * False means that the object is 
	 * True doesn't means, that the object won't be loaded.
	 * Serialized
	 */
	bool			bNotAlwaysLoadedForEditorGame:1;

	/**
	 * True if this export is an asset object.
	 */
	bool			bIsAsset:1;

	/**
	 * If this export is an instanced object inherited from a template and should
	 * only be created if the template has the object
	 */
	bool			bIsInheritedInstance:1;

	/**
	 * True if this export should have its iostore public hash generated even if not RF_Public.
	 */
	bool			bGeneratePublicHash:1;

	/**
	 * Force this export to not load, it failed because the outer didn't exist.
	 */
	bool			bExportLoadFailed:1;

	/**
	 * Export was filtered out on load
	 */
	bool			bWasFiltered:1;

	/** If this object is a top level package (which must have been forced into the export table via OBJECTMARK_ForceTagExp)
	 * this is the package flags for the original package file
	 * Serialized
	 */
	uint32			PackageFlags;

	/**
	 * The export table must serialize as a fixed size, this is use to index into a long list, which is later loaded into the array. -1 means dependencies are not present
	 * These are contiguous blocks, so CreateBeforeSerializationDependencies starts at FirstExportDependency + SerializationBeforeSerializationDependencies
	 */
	int32 FirstExportDependency;
	int32 SerializationBeforeSerializationDependencies;
	int32 CreateBeforeSerializationDependencies;
	int32 SerializationBeforeCreateDependencies;
	int32 CreateBeforeCreateDependencies;

	/**
	 * Constructors
	 */
	COREUOBJECT_API FObjectExport();
	FObjectExport(UObject* InObject, bool bInNotAlwaysLoadedForEditorGame = true);

	// Workaround for clang deprecation warnings for deprecated PackageGuid member in implicit constructors
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FObjectExport(FObjectExport&&) = default;
	FObjectExport(const FObjectExport&) = default;
	FObjectExport& operator=(FObjectExport&&) = default;
	FObjectExport& operator=(const FObjectExport&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Resets the stored Object and any transient flags */
	COREUOBJECT_API void ResetObject();
	
	/** I/O functions */
	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FObjectExport& E);
	friend COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, FObjectExport& E);
};

/*-----------------------------------------------------------------------------
	FObjectTextExport
-----------------------------------------------------------------------------*/

/**
 * Simple wrapper around a FObjectExport which does the text asset specific serialization of export data
 */
// TODO SavePackageDeprecation: remove once SavePackage2 replaces SavePackage
struct FObjectTextExport
{
	/**
	 * Constructor
	 */
	FObjectTextExport(FObjectExport& InExport, UObject* InOuter)
		: Export(InExport)
		, Outer(InOuter)
	{
	}

	/** The export object that we are wrapping */
	FObjectExport& Export;

	/** The outer that this export lives inside */
	UObject* Outer;

	/** String full object path for this export's class */
	FString ClassName;

	/** String full object path for this export's superstruct, if applicable */
	FString SuperStructName;

	/** String full object path for this export's outer, if applicable (i.e. if it's not the package itself) */
	FString OuterName;

	/** Serializer */
	friend COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, FObjectTextExport& E);
};

/*-----------------------------------------------------------------------------
	FObjectImport.
-----------------------------------------------------------------------------*/

/**
 * UObject resource type for objects that are referenced by this package, but contained
 * within another package.
 */
struct FObjectImport : public FObjectResource
{
	/**
	 * The name of the package that contains the class of the UObject represented by this resource.
	 * Serialized
	 */
	FName			ClassPackage;

	/**
	 * The name of the class for the UObject represented by this resource.
	 * Serialized
	 */
	FName			ClassName;

	/**
	 * Package Name this import belongs to. Can be none, in that case follow the outer chain
	 * until a set PackageName is found or until OuterIndex is null
	 * Serialized
	 */
	FName			PackageName;

	/**
	 * Index into SourceLinker's ExportMap for the export associated with this import's UObject.
	 * Transient
	 */
	int32             SourceIndex;

	/** 
	 * Indicate if the import comes from an optional package, used to generate the proper chunk id in the io store
	 */
	bool			bImportOptional;

	/**
	 * The UObject represented by this resource.  Assigned the first time CreateImport is called for this import.
	 * Transient
	 */
	UObject* XObject;

	/**
	 * The linker that contains the original FObjectExport resource associated with this import.
	 * Transient
	 */
	FLinkerLoad* SourceLinker;

	/**
	 * Constructors
	 */
	COREUOBJECT_API FObjectImport();
	FObjectImport( UObject* InObject );
	FObjectImport( UObject* InObject, UClass* InClass );

	/**
	 * Accessor function to check if the import has package name set
	 * @returns true if the import has a PackageName set
	 */
	bool HasPackageName() const
	{
		return !PackageName.IsNone();
	}

	/**
	 * Accessor function to get the import package name
	 * @returns the import package name, if any
	 */
	FName GetPackageName() const
	{
		return PackageName;
	}

	/**
	 * Accessor function to set the import PackageName
	 * @param InPackageName the package name to set
	 */
	void SetPackageName(FName InPackageName)
	{
		PackageName = InPackageName;
	}
	
	/** I/O functions */
	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FObjectImport& I);
	friend COREUOBJECT_API void operator<<( FStructuredArchive::FSlot Slot, FObjectImport& I );
};

/** Data resource flags. */
enum class EObjectDataResourceFlags : uint32
{
	None					= 0,
	Inline					= (1 << 0),
	Streaming				= (1 << 1),
	Optional				= (1 << 2),
	Duplicate				= (1 << 3),
	MemoryMapped			= (1 << 4),
	DerivedDataReference	= (1 << 5),
};
ENUM_CLASS_FLAGS(EObjectDataResourceFlags);

/**
 * UObject binary/bulk data resource type.
 */
struct FObjectDataResource
{
	enum class EVersion : uint32
	{
		Invalid,
		Initial,
		AddedCookedIndex,
		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};

	/** Data resource flags. */
	EObjectDataResourceFlags Flags = EObjectDataResourceFlags::None;
	/** The cooked index to which the bulkdata payload belongs */
	FBulkDataCookedIndex CookedIndex;
	/** Location of the data in the underlying storage type. */
	int64 SerialOffset = -1;
	/** Location of the data in the underlying storage type if this data is duplicated. */
	int64 DuplicateSerialOffset = -1;
	/** Number of bytes to serialize when loading/saving the data. */
	int64 SerialSize = -1;
	/** Uncompressed size of the data. */
	int64 RawSize = -1;
	/** Location of this resource outer/owning object in the export table. */
	FPackageIndex OuterIndex;
	/** Bulk data flags. */
	uint32 LegacyBulkDataFlags = 0;
	/** I/O functions. */
	COREUOBJECT_API static FArchive& Serialize(FArchive& Ar, TArray<FObjectDataResource>& DataResources);
	COREUOBJECT_API static void Serialize(FStructuredArchive::FSlot Slot, TArray<FObjectDataResource>& DataResources);
};

/** Represents a VCell stored on disk via FLinker's ImportMap or ExportMap. */
struct FCellResource
{
	/** The full Verse path of this cell. */
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	Verse::VUniqueString* VersePath;
#else
	FUtf8String VersePath;
#endif
};

/** Represents a VCell contained within this package. */
struct FCellExport : FCellResource
{
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	const Verse::VCppClassInfo* CppClassInfo;
#else
	FName CppClassInfo;
#endif

	/** Offset into the FLinker's underlying archive of the VCell's data. */
	int64 SerialOffset;
	/** Size of the first part of the VCell's data, used to allocate it on load. */
	int64 SerialLayoutSize;
	/** Full size of the VCell's data, including both the layout and content. */
	int64 SerialSize;

	Verse::VCell* Cell;

	int32 FirstExportDependency;
	int32 SerializationBeforeSerializationDependencies;
	int32 CreateBeforeSerializationDependencies;

	COREUOBJECT_API FCellExport();
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	FCellExport(Verse::VCell* InCell);
#endif

	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FCellExport& E);
	friend COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, FCellExport& E);
};

/** Represents a VCell referenced by this package, but stored in another package. */
struct FCellImport : FCellResource
{
	FPackageIndex PackageIndex;

	Verse::VCell* Cell;

	COREUOBJECT_API FCellImport();
	FCellImport(Verse::VCell* InCell);

	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FCellImport& I);
	friend COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, FCellImport& I);
};

/** Represents a dynamically injected import. Eventually turned into a FObjectImport. */
struct FDynamicImport
{
	/**
	 * Package Name to import.
	 */
	FName PackageName;

	/**
	 * Optional instance name to rename the package.
	 */
	FName InstanceName;

	/**
	 * Object name within the package (optional).
	 * If NAME_None, imports the package itself.
	 * If set, imports a specific object within the package.
	 */
	FName ObjectName;

	/**
	 * Class name for the object being imported (optional).
	 * Required when ObjectName is set.
	 */
	FName ClassName;

	/**
	 * Class package for the object being imported (optional).
	 * Required when ObjectName is set.
	 */
	FName ClassPackage;
};
