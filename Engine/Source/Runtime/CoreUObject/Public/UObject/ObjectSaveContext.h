// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Serialization/ArchiveSavePackageData.h"
#include "Templates/RefCounting.h"
#include "UObject/CookEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectSaveOverride.h"
#if WITH_EDITOR
#include "Cooker/BuildResultDependenciesMap.h"
#include "Cooker/CookDependency.h"
#endif

class FPackagePath;
class IPackageWriter;
class ITargetPlatform;
class UPackage;
struct FSoftObjectPath;
namespace UE::Cook { class ICookInfo; }
#if WITH_EDITOR
namespace UE::Cook { class IDeterminismHelper; }
#endif

/** SavePackage calls PreSave and Serialize hooks on each object, and Serialize is called more than once. */
enum class EObjectSaveContextPhase : uint8
{
	/**
	 * Phase has not been set. This can be returned from GetPhase functions when the serialize calls need to report
	 * a cook targetplatform but are not called from SavePackage.
	 */
	Invalid,
	/**
	 * SavePackage is calling PreSave on objects in the package.
	 * Build dependencies are valid for writing during this phase.
	 */
	PreSave,
	/**
	 * SavePackage is calling Serialize(FArchive&) on objects in the package. The FArchive is an object collector
	 * and the save is collecting imports, exports, and names.
	 * Build dependencies are valid for writing during this phase.
	 */
	Harvest,
	/**
	 * The Archive is coming from the cooker, outside of a SavePackage. It is discarding all results other than
	 * AddCook*Dependency results. No other save functions (BeginCacheForCookedPlatformData, PreSave, PostSave) have
	 * been called or will be called. Objects and structs serializing with this archive should serialize the same
	 * structures and UObject pointers that they serialize when serializing Harvest.
	 * This phase is executed during cook for packages that are build dependencies but are not cooked themselves.
	 */
	CookDependencyHarvest,
	/**
	 * SavePackage is calling Serialize(FArchive&) on objects in the package. The FArchive is recording the blobs of
	 * data from each exported UObject that will be written to disk.
	 * It is not valid and will cause an error if build dependencies are written during this phase.
	 */
	Write,
	/**
	 * SavePackage is calling PostSaveRoot on objects in the package.
	 */
	PostSave,
};

/** Data used to provide information about the save parameters during PreSave/PostSave. */
struct FObjectSaveContextData
{
	COREUOBJECT_API FObjectSaveContextData();
	COREUOBJECT_API ~FObjectSaveContextData();
	COREUOBJECT_API FObjectSaveContextData(const FObjectSaveContextData& Other);
	COREUOBJECT_API FObjectSaveContextData(FObjectSaveContextData&& Other);

	/** Standard constructor; calculates derived fields from the given externally-specified fields. */
	COREUOBJECT_API FObjectSaveContextData(UPackage* Package, const ITargetPlatform* InTargetPlatform, const TCHAR* InTargetFilename, uint32 InSaveFlags);
	COREUOBJECT_API FObjectSaveContextData(UPackage* Package, const ITargetPlatform* InTargetPlatform, const FPackagePath& TargetPath, uint32 InSaveFlags);

	/** Set the fields set by the standard constructor. */
	COREUOBJECT_API void Set(UPackage* Package, const ITargetPlatform* InTargetPlatform, const TCHAR* InTargetFilename, uint32 InSaveFlags);
	COREUOBJECT_API void Set(UPackage* Package, const ITargetPlatform* InTargetPlatform, const FPackagePath& TargetPath, uint32 InSaveFlags);

	/**
	 * Add a save override to specific object. (i.e. mark certain objects or properties transient for this save)
	 */
	void AddSaveOverride(UObject* Target, FObjectSaveOverride InOverride)
	{
		if (FObjectSaveOverride* MatchingSaveOverride = SaveOverrides.Find(Target))
		{
			MatchingSaveOverride->Merge(InOverride);
		}
		else
		{
			SaveOverrides.Add(Target, MoveTemp(InOverride));
		}
	}

	// Global Parameters that are read-only by the interfaces

	/**
	 * The target Filename being saved into (not the temporary file for saving).
	 * The path is in the standard UnrealEngine form - it is as a relative path from the process binary directory.
	 * Set to the empty string if the saved bytes are not being saved to a file.
	 */
	FString TargetFilename;

	/** The target platform of the save, if cooking. Null if not cooking. */
	const ITargetPlatform* TargetPlatform = nullptr;

	/** The CookInfo providing extended information about the current cook. Null if not cooking. */
	UE::Cook::ICookInfo* CookInfo = nullptr;

	/** The PackageWriter passed to SavePackage, may be null. */
	IPackageWriter* PackageWriter = nullptr;

	/** The object the Save event is being called on, if known. */
	UObject* Object = nullptr;

	/** The save flags (ESaveFlags) of the save. */
	uint32 SaveFlags = 0;

	/** Package->GetPackageFlags before the save, or 0 if no package. */
	uint32 OriginalPackageFlags = 0;

	UE::Cook::ECookType CookType = UE::Cook::ECookType::Unknown;
	UE::Cook::ECookingDLC CookingDLC = UE::Cook::ECookingDLC::Unknown;

	/** Set to the appropriate phase when calling Serialize during SavePackage. */
	EObjectSaveContextPhase ObjectSaveContextPhase = EObjectSaveContextPhase::Invalid;

	/**
	 * Set to true when the package is being saved due to a procedural save.
	 * Any save without the possibility of user-generated edits to the package is a procedural save (Cooking, EditorDomain).
	 * This allows us to execute transforms that only need to be executed in response to new user data.
	 */
	bool bProceduralSave = false;

	/**
	 * Set to true when the LoadedPath of the package being saved is being updated.
	 * This allows us to update the in-memory package when it is saved in editor to match its new save file.
	 */
	bool bUpdatingLoadedPath = false;

	/**
	 * Always true normally. When a system is executing multiple PreSaves/PostSaves concurrently before a single save,
	 * all but the first PreSaves have this set to false. If there are PostSaves they are executed in reverse order,
	 * and all but the last PostSave have this set to false.
	 */
	bool bOuterConcurrentSave = true;

	/** Set to false if the save failed, before calling any PostSaves. */
	bool bSaveSucceeded = true;

	/**
	 * Applicable only to cook saves: True if the SavePackage call should write extra debug data for debugging
	 * cook determinism or incremental cook issues.
	 */
	bool bDeterminismDebug = false;

	// Collection variables that are written but not read during the PreSave/PostSave functions
#if WITH_EDITOR
	UE_DEPRECATED(5.6, "Use BuildResultDependencies instead.")
	TArray<UE::Cook::FCookDependency> CookDependencies;
	/** Output variable for BuildResult names and their dependencies, reported by AddBuildDependency functions. */
	UE::Cook::FBuildResultDependenciesMap BuildResultDependencies;
	/** Output variable for runtime dependencies reported by AddRuntimeDependency functions. */
	TArray<FSoftObjectPath> CookRuntimeDependencies;
#endif

	// Per-object Output variables; writable from PreSave functions, readable from PostSave functions

	/** List of property overrides per object to apply to during save */
	TMap<UObject*, FObjectSaveOverride> SaveOverrides;

	/** A bool that can be set from PreSave to indicate PostSave needs to take some extra cleanup steps. */
	bool bCleanupRequired = false;

	// Variables set/read per call to PreSave/PostSave functions
	/** PreSave contract enforcement; records whether PreSave is overridden. */
	int32 NumRefPasses = 0;

	/** Call-site enforcement; records whether the base PreSave was called. */
	bool bBaseClassCalled = false;

	/** Set to true when the current object being serialized needs to call serialize again in PostSave phase. */
	bool bRequestPostSaveSerialization = false;

};

/** Interface used by CollectSaveOverrides to access the save parameters. */
class FObjectCollectSaveOverridesContext
{
public:
	explicit FObjectCollectSaveOverridesContext(FObjectSaveContextData& InData)
		: Data(InData)
	{
		// Note: Doesn't increment NumRefPasses as CollectSaveOverrides is called from PreSave
	}

	FObjectCollectSaveOverridesContext(const FObjectCollectSaveOverridesContext& Other)
		: Data(Other.Data)
	{
		// Note: Doesn't increment NumRefPasses as CollectSaveOverrides is called from PreSave
	}

	/** Report whether this is a save into a target-specific cooked format. */
	bool IsCooking() const { return Data.TargetPlatform != nullptr; }

	/** Return the targetplatform of the save, if cooking. Null if not cooking. */
	const ITargetPlatform* GetTargetPlatform() const { return Data.TargetPlatform; }

	bool IsCookByTheBook() const { return GetCookType()  == UE::Cook::ECookType::ByTheBook; }
	bool IsCookOnTheFly() const { return GetCookType() == UE::Cook::ECookType::OnTheFly; }
	bool IsCookTypeUnknown() const { return GetCookType() == UE::Cook::ECookType::Unknown; }
	UE::Cook::ECookType GetCookType() const { return Data.CookType; }
	UE::Cook::ECookingDLC GetCookingDLC() const { return Data.CookingDLC; }

	/**
	 * Return whether the package is being saved due to a procedural save.
	 * Any save without the possibility of user-generated edits to the package is a procedural save (Cooking, EditorDomain).
	 * This allows us to execute transforms that only need to be executed in response to new user data.
	 */
	bool IsProceduralSave() const { return Data.bProceduralSave; }

	/** Return the save flags (ESaveFlags) of the save. */
	uint32 GetSaveFlags() const { return Data.SaveFlags; }

	/**
	 * Add a save override to specific object. (i.e. mark certain objects or properties transient for this save)
	 */
	void AddSaveOverride(UObject* Target, FObjectSaveOverride InOverride)
	{
		Data.AddSaveOverride(Target, MoveTemp(InOverride));
	}

protected:
	FObjectSaveContextData& Data;
	friend class UObject;
};

/** Interface used by PreSave to access the save parameters. */
class FObjectPreSaveContext
{
public:
	explicit FObjectPreSaveContext(FObjectSaveContextData& InData)
		: Data(InData)
	{
		++Data.NumRefPasses; // Record the number of copies; used to check whether PreSave is overridden
	}

	FObjectPreSaveContext(const FObjectPreSaveContext& Other)
		: Data(Other.Data)
	{
		++Data.NumRefPasses; // Record the number of copies; used to check whether PreSave is overridden
	}

	/**
	 * The target Filename being saved into (not the temporary file for saving).
	 * The path is in the standard UnrealEngine form - it is as a relative path from the process binary directory.
	 * Empty string if the saved bytes are not being saved to a file. Never null.
	 */
	const TCHAR* GetTargetFilename() const { return *Data.TargetFilename; }

	/** Report whether this is a save into a target-specific cooked format. */
	bool IsCooking() const { return Data.TargetPlatform != nullptr; }

	/** Return the targetplatform of the save, if cooking. Null if not cooking. */
	const ITargetPlatform* GetTargetPlatform() const { return Data.TargetPlatform; }

	bool IsCookByTheBook() const { return GetCookType()  == UE::Cook::ECookType::ByTheBook; }
	bool IsCookOnTheFly() const { return GetCookType() == UE::Cook::ECookType::OnTheFly; }
	bool IsCookTypeUnknown() const { return GetCookType() == UE::Cook::ECookType::Unknown; }
	UE::Cook::ECookType GetCookType() const { return Data.CookType; }
	UE::Cook::ECookingDLC GetCookingDLC() const { return Data.CookingDLC; }

	/** Return which phase of SavePackage callbacks are active. @see EObjectSaveContextPhase. */
	EObjectSaveContextPhase GetPhase() const { return Data.ObjectSaveContextPhase; }

#if WITH_EDITOR
	/**
	 * Applicable only to cook saves: True if the SavePackage call should write extra debug data for debugging
	 * cook determinism or incremental cook issues.
	 */
	COREUOBJECT_API bool IsDeterminismDebug() const;
	/**
	 * Ignored unless IsDeterminismDebug()=true. An object should call this function to register
	 * their callback class for adding determinism diagnostics the package save.
	 */
	COREUOBJECT_API void RegisterDeterminismHelper(const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper);
#endif

	/**
	 * Return whether the package is being saved due to a procedural save.
	 * Any save without the possibility of user-generated edits to the package is a procedural save (Cooking, EditorDomain).
	 * This allows us to execute transforms that only need to be executed in response to new user data.
	 */
	bool IsProceduralSave() const { return Data.bProceduralSave; }

	/**
	 * Return whether the package is being saved due to the editor auto save feature.
	 */
	bool IsFromAutoSave() const { return (Data.SaveFlags & SAVE_FromAutosave) != 0; }

	/**
	 * Return whether LoadedPath of the package being saved is being updated.
	 * This allows us to update the in-memory package when it is saved in editor to match its new save file.
	 */
	bool IsUpdatingLoadedPath() const { return Data.bUpdatingLoadedPath; }

	/** Return the save flags (ESaveFlags) of the save. */
	uint32 GetSaveFlags() const { return Data.SaveFlags; }

	/**
	 * Always true normally. When a system is executing multiple PreSaves concurrently before a single save,
	 * will return false for all but the first PreSave.
	 */
	bool IsFirstConcurrentSave() const { return Data.bOuterConcurrentSave; }

protected:
	FObjectSaveContextData& Data;
	friend class UObject;
};

/** Interface used by FArchiveSavePackageData during Serialize(FArchive& Ar) to access the save parameters. */
class FObjectSavePackageSerializeContext
{
public:
	explicit FObjectSavePackageSerializeContext(FObjectSaveContextData& InData)
		: Data(InData)
	{
	}

	FObjectSavePackageSerializeContext(const FObjectSavePackageSerializeContext& Other)
		: Data(Other.Data)
	{
	}

	/**
	 * The target Filename being saved into (not the temporary file for saving).
	 * The path is in the standard UnrealEngine form - it is as a relative path from the process binary directory.
	 * Empty string if the saved bytes are not being saved to a file. Never null.
	 */
	const TCHAR* GetTargetFilename() const { return *Data.TargetFilename; }

	/** Report whether this is a save into a target-specific cooked format. */
	bool IsCooking() const { return Data.TargetPlatform != nullptr; }

	/** Return the targetplatform of the save, if cooking. Null if not cooking. */
	const ITargetPlatform* GetTargetPlatform() const { return Data.TargetPlatform; }

	bool IsCookByTheBook() const { return GetCookType() == UE::Cook::ECookType::ByTheBook; }
	bool IsCookOnTheFly() const { return GetCookType() == UE::Cook::ECookType::OnTheFly; }
	bool IsCookTypeUnknown() const { return GetCookType() == UE::Cook::ECookType::Unknown; }
	UE::Cook::ECookType GetCookType() const { return Data.CookType; }
	UE::Cook::ECookingDLC GetCookingDLC() const { return Data.CookingDLC; }

	/** Return which phase of SavePackage callbacks are active. @see EObjectSaveContextPhase. */
	EObjectSaveContextPhase GetPhase() const { return Data.ObjectSaveContextPhase; }

#if WITH_EDITOR
	/**
	 * Return true if the archive is listening to AddCookLoadDependency and AddCookRuntimeDependency
	 * calls. If it returns true, objects must add their dependencies, failing to do so will cause
	 * false positive incremental skips.
	 */
	COREUOBJECT_API bool IsHarvestingCookDependencies() const;

	/**
	 * Add the given FCookDependency to the build dependencies for the package being cook-saved. Incremental cooks will
	 * invalidate the package and recook it if the CookDependency changes. Other packages that incorporate the loaded
	 * data of the package into their own cooked results will also be recooked if this build dependency changes.
	 * Calling this function during editor save (rather than cook save) has another meaning. It is ignored for
	 * FCookDependency of type other than FCookDependency::Package, but for FCookDependency::Package it identifies
	 * a reference that propagates chunk management in the AssetRegistry, but does not cause its target to be cooked.
	 */
	COREUOBJECT_API void AddCookLoadDependency(UE::Cook::FCookDependency BuildDependency);

	/**
	 * Add the given FCookDependency to the build dependencies for the package being cook-saved. Incremental cooks will
	 * invalidate the package and recook it if the CookDependency changes.
	 */
	COREUOBJECT_API void AddCookSaveDependency(UE::Cook::FCookDependency BuildDependency);
	/**
	 * Add the given UObject's package as a runtime dependency for the package being cook-saved. It will
	 * be cooked for the current platform even if it is not otherwise referenced, and even if the object
	 * being PreSaved does not end up saving into the package's exports for the current platform.
	 */
	COREUOBJECT_API void AddCookRuntimeDependency(FSoftObjectPath Dependency);
	/** Serialize an object to find all packages that it references, and AddCookRuntimeDependency for each one. */
	COREUOBJECT_API void HarvestCookRuntimeDependencies(UObject* HarvestReferencesFrom);

	/**
	 * Applicable only to cook saves: True if the SavePackage call should write extra debug data for debugging
	 * cook determinism or incremental cook issues.
	 */
	COREUOBJECT_API bool IsDeterminismDebug() const;
	/**
	 * Ignored unless IsDeterminismDebug()=true. An object should call this function to register
	 * their callback class for adding determinism diagnostics the package save.
	 */
	COREUOBJECT_API void RegisterDeterminismHelper(const TRefCountPtr<UE::Cook::IDeterminismHelper>& DeterminismHelper);
	/** Called during the Harvest phase when the current object need to call Serialize again during the PostSave phase. */
	COREUOBJECT_API void RequestPostSaveSerialization();
#endif

	/**
	 * Return whether the package is being saved due to a procedural save.
	 * Any save without the possibility of user-generated edits to the package is a procedural save (Cooking, EditorDomain).
	 * This allows us to execute transforms that only need to be executed in response to new user data.
	 */
	bool IsProceduralSave() const { return Data.bProceduralSave; }

	/**
	 * Return whether the package is being saved due to the editor auto save feature.
	 */
	bool IsFromAutoSave() const { return (Data.SaveFlags & SAVE_FromAutosave) != 0; }

	/**
	 * Return whether LoadedPath of the package being saved is being updated.
	 * This allows us to update the in-memory package when it is saved in editor to match its new save file.
	 */
	bool IsUpdatingLoadedPath() const { return Data.bUpdatingLoadedPath; }

	/** Return the save flags (ESaveFlags) of the save. */
	uint32 GetSaveFlags() const { return Data.SaveFlags; }

	/**
	 * Always true normally. When a system is executing multiple PreSaves concurrently before a single save,
	 * will return false for all but the first PreSave.
	 */
	bool IsFirstConcurrentSave() const { return Data.bOuterConcurrentSave; }

protected:
	FObjectSaveContextData& Data;
};

/** Interface used by PostSave to access the save parameters. */
class FObjectPostSaveContext
{
public:
	explicit FObjectPostSaveContext(FObjectSaveContextData& InData)
		: Data(InData)
	{
		++Data.NumRefPasses; // Record the number of copies; used to check whether PreSave is overridden
	}

	FObjectPostSaveContext(const FObjectPostSaveContext& Other)
		: Data(Other.Data)
	{
		++Data.NumRefPasses; // Record the number of copies; used to check whether PreSave is overridden
	}

	/**
	 * The target Filename being saved into (not the temporary file for saving).
	 * The path is in the standard UnrealEngine form - it is as a relative path from the process binary directory.
	 * Empty string if the saved bytes are not being saved to a file. Never null.
	 */
	const TCHAR* GetTargetFilename() const { return *Data.TargetFilename; }

	/** Report whether this is a save into a target-specific cooked format. */
	bool IsCooking() const { return Data.TargetPlatform != nullptr; }

	/** Return the targetplatform of the save, if cooking. Null if not cooking. */
	const ITargetPlatform* GetTargetPlatform() const { return Data.TargetPlatform; }

	bool IsCookByTheBook() const { return GetCookType() == UE::Cook::ECookType::ByTheBook; }
	bool IsCookOnTheFly() const { return GetCookType() == UE::Cook::ECookType::OnTheFly; }
	bool IsCookTypeUnknown() const { return GetCookType() == UE::Cook::ECookType::Unknown; }
	UE::Cook::ECookType GetCookType() const { return Data.CookType; }
	UE::Cook::ECookingDLC GetCookingDLC() const { return Data.CookingDLC; }

	/**
	 * Return whether the package is being saved due to a procedural save.
	 * Any save without the possibility of user-generated edits to the package is a procedural save (Cooking, EditorDomain).
	 * This allows us to execute transforms that only need to be executed in response to new user data.
	 */
	bool IsProceduralSave() const { return Data.bProceduralSave; }

	/**
	 * Return whether the package is being saved due to the editor auto save feature.
	 */
	bool IsFromAutoSave() const { return (Data.SaveFlags & SAVE_FromAutosave) != 0;	}

	/**
	 * Return whether LoadedPath of the package being saved is being updated.
	 * This allows us to update the in-memory package when it is saved in editor to match its new save file.
	 */
	bool IsUpdatingLoadedPath() const { return Data.bUpdatingLoadedPath; }

	/** Return the save flags (ESaveFlags) of the save. */
	uint32 GetSaveFlags() const { return Data.SaveFlags; }

	/** Package->GetPackageFlags before the save, or 0 if no package. */
	uint32 GetOriginalPackageFlags() const { return Data.OriginalPackageFlags; }

	/** Return whether the Save was successful. Note that some PostSave operations are only called when this is true. */
	bool SaveSucceeded() const { return Data.bSaveSucceeded; }

	/**
	 * Always true normally. When a system is executing multiple PreSaves and PostSaves concurrently before a single save,
	 * PostSaves are executed in reverse order of the PreSaves, and this function returns false for all but the last one.
	 */
	bool IsLastConcurrentSave() const { return Data.bOuterConcurrentSave; }

	/** Return which phase of SavePackage callbacks are active. @see EObjectSaveContextPhase. */
	EObjectSaveContextPhase GetPhase() const { return Data.ObjectSaveContextPhase; }

protected:
	FObjectSaveContextData& Data;
	friend class UObject;
};

/** Interface used by PreSaveRoot to access the save parameters. */
class FObjectPreSaveRootContext : public FObjectPreSaveContext
{
public:
	explicit FObjectPreSaveRootContext(FObjectSaveContextData& InData)
		: FObjectPreSaveContext(InData)
	{
	}

	FObjectPreSaveRootContext(const FObjectPreSaveRootContext& Other)
		: FObjectPreSaveContext(Other.Data)
	{
	}

	/** Set whether PostSaveRoot needs to take extra cleanup steps (false by default). */
	void SetCleanupRequired(bool bCleanupRequired) { Data.bCleanupRequired = bCleanupRequired; }

};

/** Interface used by PostSaveRoot to access the save parameters. */
class FObjectPostSaveRootContext : public FObjectPostSaveContext
{
public:
	explicit FObjectPostSaveRootContext(FObjectSaveContextData& InData)
		: FObjectPostSaveContext(InData)
	{
	}

	FObjectPostSaveRootContext(const FObjectPostSaveRootContext& Other)
		: FObjectPostSaveContext(Other.Data)
	{
	}

	/** Return whether PreSaveRoot indicated PostSaveRoot needs to take extra cleanup steps. */
	bool IsCleanupRequired() const { return Data.bCleanupRequired; }
};

