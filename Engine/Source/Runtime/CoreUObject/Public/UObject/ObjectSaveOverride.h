// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/FieldPath.h"
#include "UObject/ObjectMacros.h"

/** 
 * Data collected during SavePackage that modifies the EPropertyFlags for a single FProperty on a single object instance when that object is serialized by SavePackage.
 * The specified changes apply during both the harvesting phase (discovery of referenced imports and exports) and the serialization to disk phase.
 * @note currently only support marking a property transient
 */
struct FPropertySaveOverride
{
	void Merge(const FPropertySaveOverride& Other)
	{
		checkf(PropertyPath == Other.PropertyPath, TEXT("Merge called with an unrelated FPropertySaveOverride!"));
		bMarkTransient |= Other.bMarkTransient;
	}

	FFieldPath PropertyPath;
	bool bMarkTransient;
};

/** Data to specify an override to apply to an object during save without mutating the object itself. */
struct FObjectSaveOverride
{
	void Merge(const FObjectSaveOverride& Other)
	{
		bForceTransient |= Other.bForceTransient;

		if (PropOverrides.IsEmpty())
		{
			PropOverrides = Other.PropOverrides;
		}
		else if (!Other.PropOverrides.IsEmpty())
		{
			for (const FPropertySaveOverride& OtherPropOverride : Other.PropOverrides)
			{
				if (FPropertySaveOverride* MatchingPropOverride = PropOverrides.FindByPredicate([&OtherPropOverride](const FPropertySaveOverride& PropOverride) { return PropOverride.PropertyPath == OtherPropOverride.PropertyPath; }))
				{
					MatchingPropOverride->Merge(OtherPropOverride);
				}
				else
				{
					PropOverrides.Add(OtherPropOverride);
				}
			}
		}
	}

	TArray<FPropertySaveOverride> PropOverrides;

	// Treats the object as RF_Transient for the duration of the save
	bool bForceTransient = false;
};
