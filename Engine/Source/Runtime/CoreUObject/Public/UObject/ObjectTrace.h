// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Trace/Config.h"

#ifndef UE_UOBJECT_TRACE_ENABLED
#define UE_UOBJECT_TRACE_ENABLED UE_TRACE_MINIMAL_ENABLED
#endif

namespace UE::Trace
{
enum class EUObjectTraceFlags : uint32
{
	None = 0,
	IncludeTotalResourceSizes = 1 << 0,
	IncludePackageTotalResourceSizes = 1 << 1,
};
} // namespace UE::Trace

#if UE_UOBJECT_TRACE_ENABLED

#include "Containers/UnrealString.h"
#include "Trace/Trace.h"

#include <atomic>

UE_TRACE_CHANNEL_EXTERN(CoreUObjectChannel, COREUOBJECT_API)

class UObject;

namespace UE::Trace
{

class FUObjectTrace
{
public:
	static bool IsCapturingSnapshot() { return bIsCapturingSnapshot; }
	COREUOBJECT_API static void StartCapturingSnapshot(EUObjectTraceFlags Options = EUObjectTraceFlags::None);
	COREUOBJECT_API static void OutputBeginSnapshot(EUObjectTraceFlags Options = EUObjectTraceFlags::None);
	COREUOBJECT_API static void OutputEndSnapshot(EUObjectTraceFlags Options = EUObjectTraceFlags::None);
	COREUOBJECT_API static bool OutputObjectSpec(UObjectBase* Object, EUObjectTraceFlags Options = EUObjectTraceFlags::None);
	COREUOBJECT_API static void OutputObjectRef(UObjectBase* Referencer, UObjectBase* Object);

private:
	static std::atomic<bool> bIsCapturingSnapshot;
};

} // namespace UE::Trace

#endif // UE_UOBJECT_TRACE_ENABLED
