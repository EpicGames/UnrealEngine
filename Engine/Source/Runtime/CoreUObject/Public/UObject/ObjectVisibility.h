// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

namespace UE
{

	COREUOBJECT_API EInternalObjectFlags GetAsyncLoadingInternalFlagsExclusion();

} // namespace UE