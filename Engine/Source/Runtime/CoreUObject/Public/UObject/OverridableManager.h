// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/OverriddenPropertySet.h"
#include "UObject/ScriptMacros.h"
#include "UObject/UObjectAnnotation.h"

// TODO: Internal headers should not be included
#include "Runtime/CoreUObject/Internal/UObject/PropertyBagRepository.h"


#include "OverridableManager.generated.h"

/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 */

#if WITH_EDITORONLY_DATA

/**
 * Struct holding the shared ptr of the overridden properties
 */
struct FOverriddenPropertyAnnotation
{
	bool IsDefault() const
	{
		return !OverriddenProperties.IsValid();
	}

	TSharedPtr<FOverriddenPropertySet> OverriddenProperties;
};

#endif // WITH_EDITORONLY_DATA

UENUM()
enum class EOverriddenState : uint8
{
	NoOverrides, // no on this object and any of its instanced subobjects
	HasOverrides, // has overrides in the object properties
	AllOverridden, // all properties are overridden for this object and its subobjects
	SubObjectsHasOverrides, // at least one of its subobjects has overrides
	Added, // This object was added
};

struct FOverrideManagerCapabilityInterface;

/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 */
class FOverridableManager 
#if WITH_EDITORONLY_DATA
	: protected FUObjectAnnotationSparse<FOverriddenPropertyAnnotation, true/*bAutoRemove*/>
#endif // WITH_EDITORONLY_DATA
{
public:
	/**
	 * @return the instance managing the overridability */
	static FOverridableManager& Get()
	{
		checkf(OverridableManager, TEXT("Expected the overridable manager to be created."))
		return *OverridableManager;
	}

	/**
	 * @return the instance managing the overridability if available
	 * @note it is not available only during initialization time */
	static FOverridableManager* TryGet()
	{
		return OverridableManager;
	}

	/**
	 * @return the static instance managing the overridability */
	COREUOBJECT_API static void Create();

	/**
	 * Lookup if the for the specified object has overridable serialization enabled
	 * @param Object to check against
	 * @return true if it uses the overridable serialization */
	 COREUOBJECT_API bool IsEnabled(TNotNull<const UObject*> Object);

	/**
	 * Sets on the specified object to use overridable serialization
	 * @param Object to enable */
	COREUOBJECT_API void Enable(TNotNull<UObject*> Object);

	/**
	 * Sets on the specified object to not use overridable serialization
	 * @param Object to disable
	 * @param bPropagateToSubObjects true if wants to propagate the disable to all subobjects */
	COREUOBJECT_API void Disable(TNotNull<UObject*> Object, bool bPropagateToSubObjects = false);

	/**
	 * Sets on the specified object to not use overridable serialization
	 * Sets the specified instanced subobject to not use overridable serialization if it is really owned by the referencer
	 * @param Object referencing this subobject
	 * @param InstancedSubObject the sub object to override all its properties */
	COREUOBJECT_API void DisableInstancedSubObject(TNotNull<const UObject*> Object, TNotNull<UObject*> InstancedSubObject);

	/**
	 * Inherit if the specified object should enable overridable serialization. It inherits it from either its default object or its outer.
	 * @param Object to be inherited on
	 * @param DefaultData its default object */
	COREUOBJECT_API void InheritEnabledFrom(TNotNull<UObject*> Object, const UObject* DefaultData);

	/**
	 * Return true if this object needs subobject template instantiation
	 * @param Object to be querying about */
	COREUOBJECT_API bool NeedSubObjectTemplateInstantiation(TNotNull<const UObject*> Object);

	/**
	 * Retrieve the overridden properties for the specified object
	 * @param Object to fetch the overridden properties
	 * @return the overridden properties if the object have overridable serialization enabled */
	template<
		typename InputType
		UE_REQUIRES(std::is_convertible_v<InputType, TNotNull<const UObject*>> || std::is_convertible_v<InputType, TNotNull<UObject*>>)
	>
	auto GetOverriddenProperties(InputType&& Object)
	{
		// Use a template because the compiler cannot discern which method to use if UObject* is passed (both TNotNulls accept UObject*).
		if constexpr (std::is_convertible_v<InputType, TNotNull<UObject*>>)
		{
			return GetOverriddenPropertiesInternal(TNotNull<UObject*>(Object));
		}
		else
		{
			return GetOverriddenPropertiesInternal(TNotNull<const UObject*>(Object));
		}
	}

	/**
	 * Restore the override operation from a saved state on this object, and it will enable it if it wasn't already enabled
	 * @note: It will not restore a Modified state has this will be done automatically when sub properties overrides are restored
	 * @param Object to set the override operation on
	 * @param Operation the override operation to set on the object
	 * @param bNeedsSubobjectTemplateInstantiation set to true if it will need a sub object template instantiation during postload, false will not do it.
	 * @param bDontClearOverrides set to true if you don't want to stomp any overrides that currently exist
	 * @return the overridden properties of the object */
	COREUOBJECT_API FOverriddenPropertySet* RestoreOverrideOperation(TNotNull<UObject*> Object, EOverriddenPropertyOperation Operation, const bool bNeedsSubobjectTemplateInstantiation, const bool bDontClearOverrides = false);

	/**
	 * Restore some of the overridden state that is not necessarily restored by the CPFUO
	 * (ex: bWasAdded come from the owner of the object and reinstantiating the object does not preserve it)
	 * @param OldObject to old object to restore from the overridden state
	 * @param NewObject the new object to restore to the overridden state */
	COREUOBJECT_API void RestoreOverrideState(TNotNull<const UObject*> OldObject, TNotNull<UObject*> NewObject);

	/**
	 * Retrieve the overridden state for the specified property of the specified object
	 * @param Object to fetch the overridden properties
	 * @param ConsiderPropertyForOverriddenState optional parameter to consider or not properties in the overridden state evaluation
	 * @param Path to the property to know its overridden state, empty will return the object overridden state
	 * @param bOutInheritedState optional parameter to know if the operation returned was inherited from a parent property
	 * @param bSkipAddedState Set to True if the actual state should be determined instead of just returning "Added", if this object is added.
	 * @param ConsiderSubObjectForOverriddenState Predicate to decide whether a given SubObject should be included in the state determination.
	 * @return the overridden state if the object has overridable serialization enabled */
	COREUOBJECT_API EOverriddenState GetOverriddenState(TNotNull<UObject*> Object, 
		const TFunction<bool(TNotNull<const FProperty*>)>& ConsiderPropertyForOverriddenState = TFunction<bool(TNotNull<const FProperty*>)>(),
		const FPropertyVisitorPath& Path = FPropertyVisitorPath(), bool* bOutInheritedState = nullptr,
		bool bSkipAddedState = false, const TFunction<bool(TNotNull<const UObject*>)>& ConsiderSubObjectForOverriddenState = {});

	/**
	 * Override the entire object properties and all its instanced subobjects
	 * @param Object to override all it properties */
	COREUOBJECT_API void OverrideAllObjectProperties(TNotNull<UObject*> Object);

	/**
	 * Clears all the overrides on the specified object
	 * @param Object to clear them on */
	COREUOBJECT_API void ClearOverrides(TNotNull<UObject*> Object);

	/**
	 * Clear all overrides of the specified instanced subobject if it is really owned by the referencer
	 * @param Object referencing the subobject
	 * @param InstancedSubObject to clear overrides on */
	COREUOBJECT_API void ClearInstancedSubObjectOverrides(TNotNull<const UObject*> Object, TNotNull<UObject*> InstancedSubObject);

	/**
	 * Propagate the clear overrides to all instanced suboject of the specified object
	 * @param Object*/
	COREUOBJECT_API void PropagateClearOverridesToInstancedSubObjects(TNotNull<UObject*> Object);

	/**
	 * Override a specific property of an object (Helper methods to call Pre/PostOverride)
	 * Note: Supports object that does not have overridable serialization enabled
	 * @param Object owning the property
	 * @param PropertyPath leading to the property that is about to be overridden */
	COREUOBJECT_API void OverrideProperty(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath);

	/**
	 * Clears an overridden properties specified by the property chain
	 * @param Object owning the property to clear
	 * @param PropertyPath to the property to clear from the root of the specified object
	 * @return true if the property was successfully cleared. */
	bool ClearOverriddenProperty(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath)
	{
		return ClearOverriddenProperty(Object, PropertyPath.GetRootIterator());
	}

	/**
	 * To be called prior to override a property of the specified object
	 * Note: Supports object that does not have overridable serialization enabled
	 * @param Object owning the property
	 * @param PropertyPath leading to the property about to be overridden
 	 * @param ChangeType optional current operation */
	COREUOBJECT_API void PreOverrideProperty(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, const EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified);

	/**
	 * To be called after the property of the specified object was overridden
	 * Note: Supports object that does not have overridable serialization enabled
	 * @param Object owning the property
	 * @param PropertyPath leading to the property that was overridden
 	 * @param ChangeType of the current operation */
	COREUOBJECT_API void PostOverrideProperty(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, const EPropertyChangeType::Type ChangeType);

	/**
	 * Retrieve the overridable operation from the specified property path
	 * @param Object owning the property
	 * @param PropertyPath leading to the property the caller is interested in
	 * @param bOutInheritedOperation optional parameter to know if the operation returned was inherited from a parent property
	 * @return the current type of override operation on the property */
	EOverriddenPropertyOperation GetOverriddenPropertyOperation(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, bool* bOutInheritedOperation = nullptr)
	{
		return GetOverriddenPropertyOperation(Object, PropertyPath.GetRootIterator(), bOutInheritedOperation);
	}

	/**
	 * Serializes the overridden properties of the specified object into the record
	 * @param Object to serialize the overridden property
	 * @param ObjectRecord the record to use for serialization */
	COREUOBJECT_API void SerializeOverriddenProperties(TNotNull<UObject*> Object, FStructuredArchive::FRecord ObjectRecord);

	/**
	 * Override a specific property of an object (Helper methods to call Pre/PostOverride)
	 * Note: Supports object that does not have overridable serialization enabled
	 * @param Object owning the property
	 * @param PropertyEvent information about the type of change including any container item index
	 * @param PropertyChain leading to the property that is about to be overridden */
	void OverrideProperty(TNotNull<UObject*> Object, const FPropertyChangedChainEvent& PropertyEvent)
	{
		OverrideProperty(Object, FPropertyVisitorPath(PropertyEvent, PropertyEvent.PropertyChain));
	}

	/**
	 * Clears an overridden properties specified by the property chain
	 * @param Object owning the property to clear
	 * @param PropertyEvent only needed to know about the container item index in any
	 * @param PropertyChain to the property to clear from the root of the specified object
	 * @return true if the property was successfully cleared. */
	bool ClearOverriddenProperty(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain)
	{
		return ClearOverriddenProperty(Object, FPropertyVisitorPath(PropertyEvent, PropertyChain));
	}

	/**
	 * To be called prior to override a property of the specified object
	 * Note: Supports object that does not have overridable serialization enabled
	 * @param Object owning the property
	 * @param PropertyChain leading to the property about to be overridden */
	void PreOverridePropertyChain(UObject* Object, const FEditPropertyChain& PropertyChain)
	{
		PreOverrideProperty(Object, FPropertyVisitorPath(FPropertyChangedEvent(nullptr), PropertyChain));
	}

	/**
	 * To be called prior to override a property of the specified object
	 * Note: Supports object that does not have overridable serialization enabled
	 * @param Object owning the property
	 * @param PropertyChainEvent information about the type of change including any container item index */
	void PreOverrideProperty(UObject* Object, const FPropertyChangedChainEvent& PropertyChainEvent)
	{
		PreOverrideProperty(Object, FPropertyVisitorPath(PropertyChainEvent, PropertyChainEvent.PropertyChain), PropertyChainEvent.ChangeType);
	}

	/**
	 * To be called after the property was overridden of the specified object
	 * Note: Supports object that does not have overridable serialization enabled
	 * @param Object owning the property
	 * @param PropertyChainEvent information about the type of change including any container item index */
	void PostOverrideProperty(UObject* Object, const FPropertyChangedChainEvent& PropertyChainEvent)
	{
		PostOverrideProperty(Object, FPropertyVisitorPath(PropertyChainEvent, PropertyChainEvent.PropertyChain), PropertyChainEvent.ChangeType);
	}

	/**
	 * Retrieve the overridable operation from the specified the edit property chain
	 * @param Object owning the property
	 * @param PropertyEvent only needed to know about the container item index in any
	 * @param PropertyChain leading to the property the caller is interested in
	 * @param bOutInheritedOperation optional parameter to know if the operation returned was inherited from a parent property
	 * @return the current type of override operation on the property */
	UE_FORCEINLINE_HINT EOverriddenPropertyOperation GetOverriddenPropertyOperation(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain, bool* bOutInheritedOperation = nullptr)
	{
		return GetOverriddenPropertyOperation(Object, FPropertyVisitorPath(PropertyEvent, PropertyChain), bOutInheritedOperation);
	}


	void HandleObjectsReInstantiated(const TMap<UObject*, UObject*>& OldToNewInstanceMap);
	void HandleDeadObjectReferences(const TSet<UClass*>& DeadClasses, const TSet<UObject*>& ActiveInstances, const TSet<UObject*>& TemplateInstances);

protected:
	FOverridableManager();

	COREUOBJECT_API static FOverridableManager* OverridableManager;

#if WITH_EDITORONLY_DATA
	FOverriddenPropertySet* Find(TNotNull<const UObject*> Object);
	FOverriddenPropertySet& FindChecked(TNotNull<const UObject*> Object);
	FOverriddenPropertySet& FindOrAdd(TNotNull<UObject*> Object);
#else
	FUObjectAnnotationSparseBool NeedsSubobjectTemplateInstantiation;
#endif // WITH_EDITORONLY_DATA

	friend FOverriddenPropertySet;
	COREUOBJECT_API void NotifyPropertyChange(const EPropertyNotificationType Notification, TNotNull<UObject*> Object, FPropertyVisitorPath::Iterator PropertyIterator, const EPropertyChangeType::Type ChangeType);
	COREUOBJECT_API EOverriddenPropertyOperation GetOverriddenPropertyOperation(TNotNull<UObject*> Object, FPropertyVisitorPath::Iterator PropertyIterator, bool* bOutInheritedOperation = nullptr);
	COREUOBJECT_API bool ClearOverriddenProperty(TNotNull<UObject*> Object, FPropertyVisitorPath::Iterator PropertyIterator);
	COREUOBJECT_API FOverriddenPropertySet* GetOverriddenPropertiesInternal(TNotNull<UObject*> Object);
	COREUOBJECT_API const FOverriddenPropertySet* GetOverriddenPropertiesInternal(TNotNull<const UObject*> Object);
};

