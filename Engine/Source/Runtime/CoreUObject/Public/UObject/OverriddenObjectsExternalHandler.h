// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "UObject/OverriddenPropertySet.h"

#if WITH_EDITOR
class IClassPropertyArrayExternalOverridesHandler
{
public: 

	virtual bool GetExternalOverrides(UObject* Object, FOverridableObjectArrayOverrides* Overrides) = 0;
	virtual bool IsSavingExternal(UObject* Object) = 0;

	virtual void NotifyAdd(UObject* InObject, UObject* AddedObject) = 0;
	virtual void NotifyRemove(UObject* InObject, UObject* RemovedObject, UObject* RemovedObjectArchetype) = 0;
};

struct FPropertyArrayExternalObjectHandler
{	
	public:

		// Native class to 
		struct FPropertyKey 
		{
			FName ClassName;
			FName PropertyName;

			FPropertyKey(UObject* Object, const class FProperty* Property);
			FPropertyKey(FName ClassName, FName PropertyName);

			bool operator== (const FPropertyKey&) const = default;

			friend uint32 GetTypeHash(const FPropertyKey& Key);
		};
		

		COREUOBJECT_API static FPropertyArrayExternalObjectHandler& Get();

		COREUOBJECT_API bool NotifyRemove(UObject* Object, const class FProperty* Property, UObject* RemovedObject, UObject* RemovedArchetype);
		COREUOBJECT_API bool NotifyAdd(UObject* Object, const class FProperty* Property, UObject* AddedObject);		

		COREUOBJECT_API bool GetExternalOverrides(UObject* Object, const class FProperty* Property, FOverridableObjectArrayOverrides* Overrides);
		COREUOBJECT_API bool IsSavingExternal(UObject* Object, const class FProperty* Property);
		
		COREUOBJECT_API void RegisterHandler(UClass* Class, FName PropertyName, IClassPropertyArrayExternalOverridesHandler* Handler);
		COREUOBJECT_API void UnregisterHandler(IClassPropertyArrayExternalOverridesHandler* Handler);

	private:

		TMap<FPropertyKey, IClassPropertyArrayExternalOverridesHandler*>	PerClassPropertyHandlers;
		static FPropertyArrayExternalObjectHandler HandlerObj;
};

#endif