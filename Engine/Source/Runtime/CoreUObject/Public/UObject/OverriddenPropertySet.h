// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "OverriddenPropertySet.generated.h"

#define UE_API COREUOBJECT_API

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogOverridableObject, Warning, All);

struct FOverridableObjectArrayOverrides
{	
	COREUOBJECT_API FOverridableObjectArrayOverrides();

	// Returned during loading
	TArray<UObject*> Added;
	TArray<UObject*> RemovedArchetypes; // Ptr to the Archetype of the removed object  
	
	// Objects that are present in the array (inherited from the archetype or wholly replaced) 
	// and that are not the result of an Add/Remove  (see next enum for a specific interpretation)
	TArray<UObject*> Objects; 
	
	// Indicates the override operation for the Objects in the previous array 
	EOverriddenPropertyOperation ObjectsOperation;
};

struct FArchiveSerializedPropertyChain;

struct FOverridableSerializationLogicInternalAdapter;
/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 * Scope responsible to control overridable serialization logic.
 */
struct FOverridableSerializationLogic
{
public:
	
	enum class ECapabilities : uint32
	{
		None = 0,
		// Enables serialization of override state into and from T3D
		T3DSerialization = 1 << 0,
		// Enables shadow serialization of subobject, any saved data can be skipped at load time unless loading in loose property or in a placeholder.
		// This will keep reference to subobjects that might contain overrides.
		SubObjectsShadowSerialization = 1 << 1,
		All = T3DSerialization | SubObjectsShadowSerialization
	};
	
	FRIEND_ENUM_CLASS_FLAGS(FOverridableSerializationLogic::ECapabilities)

	/**
	 * Call to enable overridable serialization and to set the overridden properties of the current serialized object
	 * Note this is not re-entrant and it stores information in a thread local storage
	 * @param InOverriddenProperties of the current serializing object */
	inline static void Enable(FOverriddenPropertySet* InOverriddenProperties)
	{
		checkf(!bUseOverridableSerialization, TEXT("Nobody should use this method if overridable serialization is already enabled"));
		bUseOverridableSerialization = true;
		OverriddenProperties = InOverriddenProperties;
	}

	/**
	 * Call to disable overridable serialization
	 * Note this is not re-entrant and it stores information in a thread local storage */
	inline static void Disable()
	{
		checkf(bUseOverridableSerialization, TEXT("Expecting overridable serialization to be already enabled"));
		OverriddenProperties = nullptr;
		bUseOverridableSerialization = false;
	}

	/**
	 * Called during the serialization of an object to know to know if it should do overridden serialization logic
	 * @return true if the overridable serialization is enabled on the current serializing object */
	inline static bool IsEnabled()
	{
		return bUseOverridableSerialization;
	}

	
	/**
	 * Used to enable override features selectively during development
	 * Capabilities driven by CVars
	 */
	COREUOBJECT_API static bool HasCapabilities(ECapabilities InCapabilities);

	/**
	 * Call during the serialization of an object to get its overriden properties
	 * Note: Expects the current serialized object to use overridable serialization
	 * Note this is not re-entrant and it stores information in a thread local storage
	 * @return the overridden properties of the current object being serialized */
	inline static FOverriddenPropertySet* GetOverriddenProperties()
	{
		return OverriddenProperties;
	}

	/**
	 * Call during the serialization of an object to get its overriden properties
	 * Note: Expects the current serialized object to use overridable serialization
	 * Note this is not re-entrant and it stores information in a thread local storage
	 * @return the overridden properties of the current object being serialized */
	COREUOBJECT_API static FOverriddenPropertySet* GetOverriddenPropertiesSlow();

	/**
	 * Retrieve from the Archive and the current property the overridden property operation to know if it has to be serialized or not
	 * @param Ar currently being used to serialize the current object (will be used to retrieve the current property serialized path)
	 * @param Property the property about to be serialized, can be null
	 * @param DataPtr to the memory of that property
	 * @param DefaultValue memory pointer of that property
	 * @return the overridden property operation */
	COREUOBJECT_API static EOverriddenPropertyOperation GetOverriddenPropertyOperation(const FArchive& Ar, FProperty* Property = nullptr, uint8* DataPtr = nullptr, const uint8* DefaultValue = nullptr);

	/**
	 * Use the port text path to retrieve the current overridden property operation to know if it has to be serialized or not
	 * @param DataPtr to the memory of that property
	 * @param DefaultValue memory pointer of that property
	 * @param PortFlags for the import/export text operation
	 * @return the overridden property operation */
	COREUOBJECT_API static EOverriddenPropertyOperation GetOverriddenPropertyOperationForPortText(const void* DataPtr, const void* DefaultValue, int32 PortFlags);

	/**
	 * Retrieve the current overridden property operation to know if it has to be serialized or not from a FArchiveSerializedPropertyChain
	 * @param PortFlags for the import/export text operation
	 * @param CurrentPropertyChain the current chain of property leading to the property if any
	 * @param Property the property about to be serialized, can be null
	 * @param DataPtr to the memory of that property
	 * @param DefaultValue memory pointer of that property
	 * @return the overridden property operation */
	COREUOBJECT_API static EOverriddenPropertyOperation GetOverriddenPropertyOperation(const int32 PortFlags, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property, const void* DataPtr, const void* DefaultValue);

	/**
	 * Call during the import text
	 * @return the current text import property path */
	COREUOBJECT_API static FPropertyVisitorPath* GetOverriddenPortTextPropertyPath();

	/**
	 * Call during the import text to set the property path
	 * @param Path to start tracking the property path*/
	COREUOBJECT_API static void SetOverriddenPortTextPropertyPath(FPropertyVisitorPath& Path);

	/** Call during the import text to reset property path */
	COREUOBJECT_API static void ResetOverriddenPortTextPropertyPath();

	/** To know if the specified property should shadow serialize its values for OS */
	COREUOBJECT_API static bool ShouldPropertyShadowSerializeSubObject(TNotNull<const FProperty*> Property);
	
	/** Method to determine whether the object should use its own instancing object graph e.g. during import.
	* When importing hierarchies of objects with instanced sub-objects we don't want to use the
	* instancing graph of our outer objects since we need our dedicated instances.
	* For example:
	*
	*	A_CDO { SubObj }
	*	PastedObj { A_1{ SubObj_1 } A_2{ SubObj_1 } }
	*
	*	by sharing the instancing graph, we end up with both SubObj_1 being the same pointer
	*	by using a dedicated instancing graph, both SubObj_1 are newly created object pointer 
	* 
	* @param Object The object that should be tested.
	* @param UsesOverrides Whether the given object has or will have overrides enabled. If unset, this will be determined from the object itself.
	* @returns True if this object should use its own instancing object graph e.g. during import. Otherwise, false.
	*/
	COREUOBJECT_API static bool ShouldUseDedicatedInstancingGraph(TNotNull<const UObject*> Object, TOptional<bool> UsesOverrides = {});
	/** Method to determine whether the object should use its own instancing object graph e.g. during import.
	 * @param Class The class that will be used to create the new object.
	 * @param Flags The flags that will be used to create the new object.
	 * @param Template The template that will be used to create the new object.
	 * @param bUsesOverrides Whether the given object has or will have overrides enabled.
	 * @see The other ShouldUseDedicatedInstancingGraph overload provides more information.
	 * @returns True if a dedicated instancing object graph should be used when creating and loading the object. Otherwise, false.
	 */
	COREUOBJECT_API static bool ShouldUseDedicatedInstancingGraphForNewObject(TNotNull<const UClass*> Class, EObjectFlags Flags,
		const UObject* Template, bool bUsesOverrides);

private:

	friend struct FOverridableSerializationLogicInternalAdapter;
	
	static ECapabilities Capabilities;
	static thread_local bool bUseOverridableSerialization;
	static thread_local FOverriddenPropertySet* OverriddenProperties;
	static thread_local FPropertyVisitorPath* OverriddenPortTextPropertyPath;
};

/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 * Scope responsible for enabling/disabling the overridable serialization from the parameters
*/
struct FEnableOverridableSerializationScope
{
	COREUOBJECT_API FEnableOverridableSerializationScope(bool bEnableOverridableSerialization, FOverriddenPropertySet* OverriddenProperties);
	COREUOBJECT_API ~FEnableOverridableSerializationScope();

protected:
	bool bOverridableSerializationEnabled = false;
	bool bWasOverridableSerializationEnabled = false;
	FOverriddenPropertySet* SavedOverriddenProperties = nullptr;
};


/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 * Scope responsible for tracking current property path for text importing
*/
struct FOverridableTextPortPropertyPathScope
{
	COREUOBJECT_API FOverridableTextPortPropertyPathScope(const FProperty* InProperty, int32 InIndex = INDEX_NONE, EPropertyVisitorInfoType InPropertyInfo = EPropertyVisitorInfoType::None);
	COREUOBJECT_API ~FOverridableTextPortPropertyPathScope();

protected:

	const FProperty* Property = nullptr;
	FPropertyVisitorPath DefaultPath;
};

/*
 * Override operation type for each property node
 */
UENUM()
enum class EOverriddenPropertyOperation : uint8
{
	None =	0,	/* no overridden operation was recorded on this property  */
	Modified,	/* some sub property has recorded overridden operation */
	Replace,	/* everything has been overridden from this property down to every sub property/sub object*/
	Add,		/* this element was added in the container */
	Remove,		/* this element was removed from the container */
	SubObjectsShadowing, /* only used to serialize/collect the subobjects, this saved data can be skipped at loading unless loading a loose property or in a placeholder which will keep reference to subobjects */
	Externalized, /* this indicates the elements are stored externally */
};

inline TOptional<EOverriddenPropertyOperation> GetOverriddenOperationFromString(const FString& OverriddenOperationString)
{
	int64 EnumValue = StaticEnum<EOverriddenPropertyOperation>()->GetValueByNameString(StaticEnum<EOverriddenPropertyOperation>()->GetName() + FString("::") + OverriddenOperationString);
	if (EnumValue == INDEX_NONE)
	{
		return TOptional<EOverriddenPropertyOperation>();
	}

	return TOptional((EOverriddenPropertyOperation)EnumValue);
}

inline TOptional<EOverriddenPropertyOperation> GetOverriddenOperationFromName(const FName OverriddenOperationName)
{
	return GetOverriddenOperationFromString(OverriddenOperationName.ToString());
}

inline FString GetOverriddenOperationString(EOverriddenPropertyOperation Operation)
{
	return UEnum::GetValueAsString(Operation).RightChop(StaticEnum<EOverriddenPropertyOperation>()->GetName().Len() + /*::*/2);
}

USTRUCT()
struct FOverriddenPropertyPath
{
	GENERATED_BODY()

	FOverriddenPropertyPath(const FName Name);

	FOverriddenPropertyPath() = default;
	FOverriddenPropertyPath(const FOverriddenPropertyPath&) = default;
	FOverriddenPropertyPath(FOverriddenPropertyPath&&) = default;
	FOverriddenPropertyPath& operator=(const FOverriddenPropertyPath&) = default;
	FOverriddenPropertyPath& operator=(FOverriddenPropertyPath&&) = default;

	FOverriddenPropertyPath& operator=(const FName Name);
	FOverriddenPropertyPath operator+(const FName Name) const;
	FOverriddenPropertyPath operator+(const FOverriddenPropertyPath& Other) const;
	void operator+=(const FName Name);
	void operator+=(const TArray<FName>& Names);
	void operator+=(const FOverriddenPropertyPath& Other)
	{
		this->operator+=(Other.Path);
	}

	bool operator==(const FOverriddenPropertyPath& Other) const
	{
		return CachedHash == Other.CachedHash;
	}

	bool IsEmpty() const
	{
		return Path.IsEmpty();
	}

	COREUOBJECT_API FString ToString() const;

private:
	UPROPERTY()
	TArray<FName> Path;

	UPROPERTY()
	uint32 CachedHash = 0;

	friend uint32 GetTypeHash(const FOverriddenPropertyPath& MappableString)
	{
		return MappableString.CachedHash;
	}
};

// helper struct that allocates space to temporarily store/own the value of a property and free the memory when it destructs
struct FScopedPropertyValue
{
	explicit FScopedPropertyValue() = default;
	explicit FScopedPropertyValue(FProperty* Property);
	~FScopedPropertyValue();
	FScopedPropertyValue(FScopedPropertyValue&& Other);
	FScopedPropertyValue& operator=(FScopedPropertyValue&& Other);
	FScopedPropertyValue(const FScopedPropertyValue& Other) = delete;
	FScopedPropertyValue& operator=(const FScopedPropertyValue& Other) = delete;
	operator bool() const;
	void* GetData() const;

private:
	FProperty* Property = nullptr;
	void* Data = nullptr;
};

USTRUCT()
struct FOverriddenPropertyNodeID
{
	GENERATED_BODY()

	COREUOBJECT_API FOverriddenPropertyNodeID(TNotNull<const FProperty*> Property);
	COREUOBJECT_API FOverriddenPropertyNodeID(TNotNull<const UObject*> InObject);
	COREUOBJECT_API FOverriddenPropertyNodeID(const FOverriddenPropertyNodeID& ParentNodeID, const FOverriddenPropertyNodeID& SubNodeID);

	FOverriddenPropertyNodeID() = default;
	FOverriddenPropertyNodeID(const FOverriddenPropertyNodeID&) = default;
	FOverriddenPropertyNodeID(FOverriddenPropertyNodeID&&) = default;
	FOverriddenPropertyNodeID& operator=(const FOverriddenPropertyNodeID&) = default;
	FOverriddenPropertyNodeID& operator=(FOverriddenPropertyNodeID&&) = default;

	static COREUOBJECT_API FOverriddenPropertyNodeID RootNodeId();

	// Overridden property node map helpers
	static COREUOBJECT_API FOverriddenPropertyNodeID FromMapKey(const FProperty* KeyProperty, const void* KeyData);
	COREUOBJECT_API int32 ToMapInternalIndex(FScriptMapHelper& MapHelper) const;
	COREUOBJECT_API FScopedPropertyValue ToMapKey(FScriptMapHelper& MapHelper) const;

	COREUOBJECT_API bool operator==(const FOverriddenPropertyNodeID& Other) const;

	FString ToString() const
	{
		return Path.ToString();
	}

	FString ToDebugString() const
	{
		return Path.ToString() + FString::Printf(TEXT(" (%s 0x%p)"), Object.IsValid() ? *Object->GetName() : TEXT("NULL"), Object.Get());
	}

	bool IsValid() const
	{
		return !Path.IsEmpty();
	}

	void HandleObjectsReInstantiated(const TMap<UObject*, UObject*>& Map);
	void HandleDeadObjectReferences(const TSet<UObject*>& ActiveInstances, const TSet<UObject*>& TemplateInstances);

	friend uint32 GetTypeHash(const FOverriddenPropertyNodeID& NodeID)
	{
		return GetTypeHash(NodeID.Path);
	}

private:
	UPROPERTY()
	FOverriddenPropertyPath Path;

	UPROPERTY()
	TWeakObjectPtr<const UObject> Object;
};

/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 * Overridden property information node, there will be one per overriden property/subojects
 *
 */
USTRUCT()
struct FOverriddenPropertyNode
{
	GENERATED_BODY()

	FOverriddenPropertyNode() = default;
	FOverriddenPropertyNode(const FOverriddenPropertyNode&) = default;
	FOverriddenPropertyNode(FOverriddenPropertyNode&&) = default;
	FOverriddenPropertyNode& operator=(const FOverriddenPropertyNode&) = default;
	FOverriddenPropertyNode& operator=(FOverriddenPropertyNode&&) = default;

	FOverriddenPropertyNode(const FOverriddenPropertyNodeID& InNodeID)
		: NodeID(InNodeID)
	{}
	FOverriddenPropertyNode(FOverriddenPropertyNodeID&& InNodeID)
		: NodeID(MoveTemp(InNodeID))
	{}

private:
	UPROPERTY()
	FOverriddenPropertyNodeID NodeID;

	UPROPERTY()
	EOverriddenPropertyOperation Operation = EOverriddenPropertyOperation::None;

	// Unable to make this a UPROPERTY() because of this UHT error:
	// 'Struct' recursion via arrays is unsupported for properties.
	// Instead will be serializing by hand
	TArray<FOverriddenPropertyNode> SubPropertyNodes;

public:

	const FOverriddenPropertyNodeID& GetNodeID() const
	{
		return NodeID;
	}

	EOverriddenPropertyOperation GetOperation() const
	{
		return Operation;
	}

	void SetOperation(EOverriddenPropertyOperation InOperation);

	TArrayView<FOverriddenPropertyNode> GetSubPropertyNodes()
	{
		return SubPropertyNodes;
	}

	const TArray<FOverriddenPropertyNode>& GetSubPropertyNodes() const
	{
		return SubPropertyNodes;
	}

	FOverriddenPropertyNode& FindOrAddNode(const FOverriddenPropertyNodeID& NodeID);

	bool RemoveSubPropertyNode(const FOverriddenPropertyNodeID& NodeID);

	bool operator==(const FOverriddenPropertyNode& Other) const
	{
		return NodeID == Other.NodeID;
	}

	void Reset();

	void HandleObjectsReInstantiated(const TMap<UObject*, UObject*>& Map);
	void HandleDeadObjectReferences(const TSet<UObject*>& ActiveInstances, const TSet<UObject*>& TemplateInstances);

	UE_API bool Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FOverriddenPropertyNode& P)
	{
		P.Serialize(Ar);
		return Ar;
	}
};

template<>
struct TStructOpsTypeTraits<FOverriddenPropertyNode> : public TStructOpsTypeTraitsBase2<FOverriddenPropertyNode>
{
	enum 
	{
		WithSerializer = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/*
 * Property change notification type mapping the Pre/PostEditChange callbacks
 */
enum class EPropertyNotificationType : uint8
{
	PreEdit,
	PostEdit
};

/*
 *************************************************************************************
 * Overridable serialization is experimental, not supported and use at your own risk *
 *************************************************************************************
 * Structure holding and tracking overridden properties of an UObject
 */
USTRUCT()
struct FOverriddenPropertySet
{
	GENERATED_BODY()

public:

	FOverriddenPropertySet() = default;
	FOverriddenPropertySet(TNotNull<UObject*> InOwner)
		: Owner(InOwner)
	{}

	/**
	 * Retrieve the overridable operation from the specified the edit property chain node
	 * @param PropertyIterator leading to the property interested in, invalid iterator will return the operation of the object itself
	 * @param bOutInheritedOperation optional parameter to know if the state returned was inherited from a parent property
	 * @return the current type of override operation on the property */
	UE_API EOverriddenPropertyOperation GetOverriddenPropertyOperation(FPropertyVisitorPath::Iterator PropertyIterator, bool* bOutInheritedOperation = nullptr) const;

	/**
	 * Clear any properties from the serialized property chain node
	 * @param PropertyIterator leading to the property to clear, invalid iterator will clear the overrides on the object itself
	 * @return if the operation was successful */
	UE_API bool ClearOverriddenProperty(FPropertyVisitorPath::Iterator PropertyIterator);

	/**
	 * Utility methods that call NotifyPropertyChange(Pre/PostEdit)
	 * @param PropertyIterator leading to the property that is changing, invalid iterator means it is the object itself that is changing
	 * @param Data memory of the current property */
	UE_API void OverrideProperty(FPropertyVisitorPath::Iterator PropertyIterator, const void* Data);

	/**
	 * Utility method that allows to indicate if an array property has its content externalized or not
	 * @param Property The property in the set that is being externalized/internalized 
	 * @param bExternal Indicate if the content will be stored externally */
	UE_API void SetExternalizedOverriddenProperty(const FOverriddenPropertyNodeID& NodeID, bool bExternal);

	/**
	 * Handling and storing modification on a property of an object
	 * @param Notification type either pre/post property overridden
	 * @param PropertyIterator leading to the property that is changing, null means it is the object itself that is changing
	 * @param ChangeType of the current operation
	 * @param Data memory of the current property */
	UE_API void NotifyPropertyChange(const EPropertyNotificationType Notification, FPropertyVisitorPath::Iterator PropertyIterator, const EPropertyChangeType::Type ChangeType, const void* Data);

	/**
	 * Retrieve the overridable operation from the specified the serialized property chain and the specified property
	 * @param CurrentPropertyChain leading to the property being serialized if any, null or empty it will return the node of the object itself
	 * @param Property being serialized if any, if null it will fallback on the last property of the chain
	 * @return the current type of override operation on the property */
	UE_API EOverriddenPropertyOperation GetOverriddenPropertyOperation(const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property) const;

	/**
	 * Restores the overridable operation of the current property from the serialized property chain and the specified property
	 * @param Operation to set for this property
	 * @param CurrentPropertyChain leading to the property being serialized if any, null or empty it will return the node of the object itself
	 * @param Property being serialized if any, if null it will fallback on the last property of the chain
	 * @return the node containing the information of the overridden property */
	UE_API FOverriddenPropertyNode* RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation Operation, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property);

	/**
	 * Conditionally restores the overridable operation of the current property from the serialized property chain and the specified property
	 * @note: This will not restore modified state, has restoring sub properties will do it anyway. 
	 * @param Operation to set for this property
	 * @param CurrentPropertyChain leading to the property being serialized if any, null or empty it will return the node of the object itself
	 * @param Property being serialized if any, if null it will fallback on the last property of the chain
	 * @return the node containing the information of the overridden property */
	UE_API FOverriddenPropertyNode* ConditionallyRestoreOverriddenPropertyOperation(EOverriddenPropertyOperation Operation, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property);

	/**
	 * Retrieve the overridden property node from the serialized property chain
	 * @param CurrentPropertyChain leading to the property being serialized if any, null or empty it will return the node of the object itself
	 * @return the node containing the information of the overridden property */
	UE_API const FOverriddenPropertyNode* GetOverriddenPropertyNode(const FArchiveSerializedPropertyChain* CurrentPropertyChain) const;
	UE_API const FOverriddenPropertyNode* GetOverriddenPropertyNode(const FOverriddenPropertyNodeID& NodeID) const;

	/**
	 * Retrieve the root overridden property node (read-only variant).
	 * @return the root node of the property set */
	const FOverriddenPropertyNode& GetRootOverriddenPropertyNode() const 
	{ 
		return RootNode; 
	}

	/**
	 * Retrieve the root overridden property node (mutable variant).
	 * @return the root node of the property set */
	FOverriddenPropertyNode& GetRootOverriddenPropertyNode() 
	{ 
		return RootNode; 
	}

	/**
	 * Retrieve the overridable operation given the property key
	 * @param NodeID that uniquely identify the property within the object 
	 * @return the current type of override operation on the property */
	UE_API EOverriddenPropertyOperation GetSubPropertyOperation(const FOverriddenPropertyNodeID& NodeID) const;

	/**
	 * Restores the overridable operation of a sub property of the specified node.
	 * @param Operation to set for this property
	 * @param Node from where the sub property is owned by
	 * @param NodeID the ID of the sub property 
	 * @return the node to the sub property */
	UE_API FOverriddenPropertyNode* RestoreSubPropertyOperation(EOverriddenPropertyOperation Operation, FOverriddenPropertyNode& Node, const FOverriddenPropertyNodeID& NodeID);

	/**
	 * Restores the overridable operation of a sub object of the specified node.
	 * @param Operation to set for this property
	 * @param Node from where the sub property is owned by
	 * @param SubObject of the operation
	 * @return the node to the sub property */
	UE_API FOverriddenPropertyNode* RestoreSubObjectOperation(EOverriddenPropertyOperation Operation, FOverriddenPropertyNode& Node, TNotNull<UObject*> SubObject);
	/**
	 * Check if this is an overridden property set of a CDO and that this property is owned by the class of this CDO
	 * NOTE: this is used to know if a property should be serialized to keep its default CDO value.
	 * @param Property 
	 * @return 
	 */
	UE_API bool IsCDOOwningProperty(const FProperty& Property) const;

	/**
	 * Resets all overrides of the object */
	UE_API void Reset(bool bShouldDirtyObject = true);
	UE_API void HandleObjectsReInstantiated(const TMap<UObject*, UObject*>& Map);
	UE_API void HandleDeadObjectReferences(const TSet<UObject*>& ActiveInstances, const TSet<UObject*>& TemplateInstances);

	/* return whether this object is considered added or not */
	bool WasAdded() const
	{
		return bWasAdded;
	}

	TObjectPtr<UObject> GetOwner() const
	{
		return Owner;
	}

	/**
	 * Restore some of the overridden state that is not necessarily restored by the CPFUO
	 * (ex: bWasAdded come from the owner of the object and reinstantiating the object does not preserve it)
	 * @param FromOverriddenProperties overridable properties to restore from*/
	UE_API void RestoreOverriddenState(const FOverriddenPropertySet& FromOverriddenProperties);

	/**
	 * Converts the state of all nodes in the property set into a string (for debug purposes).
	 * @return a string representing all the nodes in the set */
	UE_API FString ToDebugString() const;

protected:

	UE_API EOverriddenPropertyOperation GetOverriddenPropertyOperation(const FOverriddenPropertyNode* ParentPropertyNode, FPropertyVisitorPath::Iterator PropertyIterator, bool* bOutInheritedOperation, const void* Data) const;
	UE_API bool ClearOverriddenProperty(FOverriddenPropertyNode& ParentPropertyNode, FPropertyVisitorPath::Iterator PropertyIterator, const void* Data);
	UE_API void NotifyPropertyChange(FOverriddenPropertyNode* ParentPropertyNode, const EPropertyNotificationType Notification, FPropertyVisitorPath::Iterator PropertyIterator, const EPropertyChangeType::Type ChangeType, const void* Data, bool& bNeedsCleanup);
	UE_API void NotifyAllPropertyChange(FOverriddenPropertyNode* ParentPropertyNode, const EPropertyNotificationType Notification, const EPropertyChangeType::Type ChangeType, TNotNull<const UStruct*> Struct, const void* Data);

	UE_API EOverriddenPropertyOperation GetOverriddenPropertyOperation(const FOverriddenPropertyNode* ParentPropertyNode, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property) const;
	UE_API FOverriddenPropertyNode* RestoreOverriddenPropertyOperation(EOverriddenPropertyOperation Operation, FOverriddenPropertyNode& ParentPropertyNode, const FArchiveSerializedPropertyChain* CurrentPropertyChain, FProperty* Property);
	UE_API const FOverriddenPropertyNode* GetOverriddenPropertyNode(const FOverriddenPropertyNode& ParentPropertyNode, const FArchiveSerializedPropertyChain* CurrentPropertyChain) const;

	UE_API UObject* TryGetInstancedSubObjectValue(const FObjectPropertyBase* FromProperty, const void* ValuePtr) const;

private:
	UPROPERTY()
	TObjectPtr<UObject> Owner = nullptr;

	UPROPERTY()
	bool bWasAdded = false;

	UPROPERTY()
	FOverriddenPropertyNode RootNode { RootNodeID };

	static inline FOverriddenPropertyNodeID RootNodeID = FOverriddenPropertyNodeID::RootNodeId();

public:
	bool bNeedsSubobjectTemplateInstantiation = false;
};

#undef UE_API
