// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PropertyVisitor.h"
#include "UnrealType.h"
#include "Object.h"

namespace UE
{
	/**
	 * Helper method that calls the pre and post change notification responsible for overriding an object.
	 * This is the preferred way to override because it will notify listeners
	 * @param Object to override all it's properties */
	COREUOBJECT_API void SendOverrideAllObjectPropertiesEvent(TNotNull<UObject*> Object);

	/**
	 * Helper method that calls the pre and post change notification responsible for clearing the overrides of an object.
	 * This is the preferred way to clear overrides because it will notify listeners
	 * @param Object to clear overrides on */
	COREUOBJECT_API void SendClearOverridesEvent(TNotNull<UObject*> Object);

	/**
	 * Helper method that calls the pre and post change notification responsible for overriding an object property.
	 * This is the preferred way to override properties because it will notify listeners
	 * Note: If you're also changing property values (especially if it's in a container) use SendPreOverridePropertyEvent and SendPostOverridePropertyEvent instead!
	 * @param Object owning the property
	 * @param PropertyPath leading to the property that is about to be overridden
	 * @param ChangeType of the current operation */
	COREUOBJECT_API void SendOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified);
	COREUOBJECT_API void SendOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain);

	/**
	 * Helper method that calls the pre and post change notification responsible for clearing a property override.
	 * This is the preferred way to clear property overrides because it will notify listeners
	 * @param Object owning the property to clear
	 * @param PropertyPath to the property to clear from the root of the specified object
	 * @return true if the property was successfully cleared. */
	COREUOBJECT_API void SendClearOverriddenPropertyEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath);
	COREUOBJECT_API void SendClearOverriddenPropertyEvent(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain);

	/**
	 * Helper method that calls the pre-change notification responsible for overriding an object property.
	 * This is the preferred way to override properties because it will notify listeners
	 * @param Object owning the property
	 * @param PropertyPath leading to the property about to be overridden */
	COREUOBJECT_API void SendPreOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath);
	COREUOBJECT_API void SendPreOverridePropertyEvent(TNotNull<UObject*> Object, const FEditPropertyChain& PropertyChain);

	/**
	 * Helper method that calls the post-change notification responsible for overriding an object property.
	 * This is the preferred way to override properties because it will notify listeners
	 * @param Object owning the property
	 * @param PropertyPath leading to the property that was overridden
	 * @param ChangeType of the current operation */
	COREUOBJECT_API void SendPostOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyVisitorPath& PropertyPath, EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified);
	COREUOBJECT_API void SendPostOverridePropertyEvent(TNotNull<UObject*> Object, const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain);
}
