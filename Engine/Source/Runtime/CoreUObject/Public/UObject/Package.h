// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformMath.h"
#include "IO/IoHash.h"
#include "IO/PackageId.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AssetReadLoggerConfig.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/ObjectThumbnail.h"
#include "Misc/OutputDeviceError.h"
#include "Misc/PackagePath.h"
#include "Misc/SecureHash.h"
#include "Misc/WorldCompositionUtility.h"
#include "Serialization/CustomVersion.h"
#include "Tasks/Task.h"
#include "Templates/PimplPtr.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectVersion.h"
#include "UObject/UObjectGlobals.h"

#if WITH_METADATA
#include "UObject/MetaData.h"
#endif

#if WITH_EDITORONLY_DATA
#include "IO/IoHash.h"
#endif

#if WITH_EDITOR
#include "Cooker/BuildResultDependenciesMap.h"
#endif

#include "Package.generated.h"

class Error;
class FArchive;
class FLinkerLoad;
// This is a dummy type which is not implemented anywhere. It's only
// used to flag a deprecated Conform argument to package save functions.
class FLinkerNull;
class FLinkerSave;
class FObjectPostSaveContext;
class FObjectPreSaveContext;
class FOutputDevice;
class FSavePackageContext;
class FString;
class ITargetPlatform;
class UFunction;
class UDEPRECATED_MetaData;
struct FAssetData;
struct FMD5Hash;
struct FPackageSaveInfo;
struct FSavePackageArgs;

#if WITH_EDITOR
namespace UE::Cook { class FCookDependency; }
#endif

/**
* Represents the result of saving a package
*/
enum class ESavePackageResult
{
	/** Package was saved successfully */
	Success,
	/** Unknown error occured when saving package */
	Error,
	/** Canceled by user */
	Canceled,
	/** [When cooking] Package was not saved because it contained editor-only data */
	ContainsEditorOnlyData,
	ReferencedOnlyByEditorOnlyData UE_DEPRECATED(5.6, "The cooker now uses SkipOnlyEditorOnly to detect ReferencedOnlyByEditorOnlyData instead of detecting it during SavePackage"),
	/** [When cooking] Package was not saved because it contains assets that were converted into native code */
	ReplaceCompletely,
	/** [When cooking] Package was saved, but we should generate a stub so that other converted packages can interface with it*/
	GenerateStub,
	DifferentContent	UE_DEPRECATED(5.0, "Diffing is now done using FDiffPackageWriter."),
	/** [When cooking] The file requested (when cooking on the fly) did not exist on disk */
	MissingFile,
	/** Result from ISavePackageValidator that indicates an error. */
	ValidatorError,
	/** Result from ISavePackageValidator that suppresses the save but is not an error. */
	ValidatorSuppress,
	/** Internal save result used to identify a valid empty internal save realm to skip over. @see ESaveRealm */
	EmptyRealm,
	/** SavePackage is blocked by an asynchronous operation, so it quickly aborted. Can only be returned if SAVE_AllowTimeout is present in SaveFlags */
	Timeout,
};

inline bool IsSuccessful(ESavePackageResult Result)
{
	return
		Result == ESavePackageResult::Success ||
		Result == ESavePackageResult::GenerateStub ||
		Result == ESavePackageResult::ReplaceCompletely;
}

namespace UE::SavePackageUtilities
{

/**
 * A dependency in a runtime cook package from the load phase (create or serialize) of one object to the load phase of
 * another object.
 * Experimental, may be changed without deprecation.
 */
struct FPreloadDependency
{
	UObject* SourceObject;
	UObject* TargetObject;
	bool bSourceIsSerialize;
	bool bTargetIsSerialize;
};

}
/**
* Struct returned from save package, contains the enum as well as extra data about what was written
*/
struct FSavePackageResultStruct
{
	/** Success/failure of the save operation */
	ESavePackageResult Result;

	/** Total size of all files written out, including bulk data */
	int64 TotalFileSize;

	/** Serialized package flags */
	uint32 SerializedPackageFlags;

	UE_DEPRECATED(5.6, "Returning the LinkerSave for comparison is no longer used. Contact Epic if you need this functionality.")
	TPimplPtr<FLinkerSave> LinkerSave;

	TArray<FAssetData> SavedAssets;

	TArray<FName> ImportPackages;
	TArray<FName> SoftPackageReferences;
	TArray<FName> UntrackedSoftPackageReferences;

#if WITH_EDITOR
	UE_DEPRECATED(5.6, "Use BuildResultDependencies instead..")
	TArray<UE::Cook::FCookDependency> CookDependencies;
	UE::Cook::FBuildResultDependenciesMap BuildResultDependencies;

	/** Only populated during cook saves. */
	TArray<UObject*> Imports;
	/** Only populated during cook saves. */
	TArray<UObject*> Exports;
	/**
	 * Only populated during cook saves. The list of object dependencies from exports in this package to
	 * other objects, either imports or exports, and also specifying which stage of the source object and which stage
	 * of the target object have the dependency. These dependencies are also recorded in the save package and are used
	 * for runtime loading of cooked packages. This output variable allows the cooker to run validation on that graph
	 * of dependencies.
	 * Experimental, may be changed without deprecation.
	 */
	UE_INTERNAL TArray<UE::SavePackageUtilities::FPreloadDependency> PreloadDependencies;
#endif

	/** Constructors, it will implicitly construct from the result enum */
	COREUOBJECT_API FSavePackageResultStruct();
	COREUOBJECT_API FSavePackageResultStruct(ESavePackageResult InResult);
	COREUOBJECT_API FSavePackageResultStruct(ESavePackageResult InResult, int64 InTotalFileSize);
	COREUOBJECT_API FSavePackageResultStruct(ESavePackageResult InResult, int64 InTotalFileSize,
		uint32 InSerializedPackageFlags);
	UE_DEPRECATED(5.6, "Returning the LinkerSave for comparison is no longer used. Contact Epic if you need this functionality.")
	COREUOBJECT_API FSavePackageResultStruct(ESavePackageResult InResult, int64 InTotalFileSize,
		uint32 InSerializedPackageFlags, TPimplPtr<FLinkerSave> Linker);
	COREUOBJECT_API FSavePackageResultStruct(FSavePackageResultStruct&& Other);
	COREUOBJECT_API FSavePackageResultStruct& operator=(FSavePackageResultStruct&& Other);
	COREUOBJECT_API ~FSavePackageResultStruct();

	bool operator==(const FSavePackageResultStruct& Other) const
	{
		return Result == Other.Result;
	}

	bool operator!=(const FSavePackageResultStruct& Other) const
	{
		return Result != Other.Result;
	}

	/** Returns whether the package save was successful */
	bool IsSuccessful() const
	{
		return ::IsSuccessful(Result);
	}
};

/** Controls how package is externally referenced by other plugins and mount points */
enum class EAssetAccessSpecifier : uint8
{
	Private,
	Public,
	EpicInternal,
};

COREUOBJECT_API FText AssetAccessSpecifierAsText(EAssetAccessSpecifier);

/**
* A package.
*/
PRAGMA_DISABLE_DEPRECATION_WARNINGS // Required for auto-generated functions referencing PackageFlags
UCLASS(MinimalAPI, Config=Engine)
class UPackage : public UObject
{
	GENERATED_BODY()

	/** DO NOT USE. This constructor is for internal usage only for hot-reload purposes. */
	UPackage(FVTableHelper& Helper)
		: Super(Helper)
	{
	};

	friend UDEPRECATED_MetaData;

public:

	UPackage(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get())
		: Super(ObjectInitializer)
	{
	}

	~UPackage() = default;

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval UPackage(UE::CodeGen::ConstInit::FPackageParams InParams)
		: UObject(InParams.Object)
		, bDirty(false)
#if WITH_EDITORONLY_DATA
		, bIsDynamicPIEPackagePending(false)
#endif
		, bHasBeenFullyLoaded(false)
		, bCanBeImported(false)
#if WITH_EDITORONLY_DATA
		, PersistentGuid(FGuid(InParams.BodiesHash, InParams.DeclarationsHash, 0, 0))
		, ChunkIDs(ConstEval)
#endif
		, PackageFlagsPrivate(InParams.Flags)
		, PackageId()
		, LoadedPath()
#if WITH_METADATA
		, MetaData(ConstEval)
#endif
#if WITH_EDITORONLY_DATA
		, PIEInstanceID(INDEX_NONE)
#endif
	{

	}
#endif

	/** delegate type for package dirty state events.  ( Params: UPackage* ModifiedPackage ) */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPackageDirtyStateChanged, class UPackage*);
	/** delegate type for package saved events ( Params: const FString& PackageFileName, UObject* Outer ) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPackageSaved, const FString&, UObject*);					
	/** delegate type for package saved events ( Params: const FString& PackageFileName, UObject* Outer, FObjectPostSaveContext ObjectSaveContext ) */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPackageSavedWithContext, const FString&, UPackage*, FObjectPostSaveContext);
	/** delegate type for when a package is marked as dirty via UObjectBaseUtility::MarkPackageDirty ( Params: UPackage* ModifiedPackage, bool bWasDirty ) */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPackageMarkedDirty, class UPackage*, bool);
	/** delegate type for when a package is about to be saved */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPreSavePackage, class UPackage*);
	/** delegate type for when a package is about to be saved */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FPreSavePackageWithContext, class UPackage*, FObjectPreSaveContext);

#if WITH_METADATA
	/** A delegate that is called when old metadata is supposed to move to new metadata, such as when renaming an asset.
	 *  Since metadata is not handled by the UPROPERTY-system, data transfer has to be done manually.
	 *  By default, data is thrown out.  */
	DECLARE_MULTICAST_DELEGATE_FourParams(FOnMetaDataTransferRequestedDelegate, const FMetaData& OldMetaData, FMetaData& NewMetaData, const UPackage* OldPackage, UPackage* NewPackage);
#endif
	
	/** Delegate to notify subscribers when a package is about to be saved. */
	COREUOBJECT_API static FPreSavePackageWithContext PreSavePackageWithContextEvent;
	/** Delegate to notify subscribers when a package has been saved. This is triggered when the package saving
	*  has completed and was successful. */
	COREUOBJECT_API static FOnPackageSavedWithContext PackageSavedWithContextEvent;
	/** Delegate to notify subscribers when the dirty state of a package is changed.
	*  Allows the editor to register the modified package as one that should be prompted for source control checkout. 
	*  Use Package->IsDirty() to get the updated dirty state of the package */
	COREUOBJECT_API static FOnPackageDirtyStateChanged PackageDirtyStateChangedEvent;
	/** 
	* Delegate to notify subscribers when a package is marked as dirty via UObjectBaseUtility::MarkPackageDirty 
	* Note: Unlike FOnPackageDirtyStateChanged, this is always called, even when the package is already dirty
	* Use bWasDirty to check the previous dirty state of the package
	* Use Package->IsDirty() to get the updated dirty state of the package
	*/
	COREUOBJECT_API static FOnPackageMarkedDirty PackageMarkedDirtyEvent;

#if WITH_METADATA
	COREUOBJECT_API static FOnMetaDataTransferRequestedDelegate OnMetaDataTransferRequestedDelegate;
#endif
private:
	/** Used by the editor to determine if a package has been changed. */
	uint8	bDirty:1;

#if WITH_EDITORONLY_DATA
	/** True if this package is a dynamic PIE package with external objects still loading */
	uint8 bIsDynamicPIEPackagePending:1;
#endif

public:
	/** Whether this package has been fully loaded (aka had all it's exports created) at some point. */
	mutable uint8 bHasBeenFullyLoaded:1;

	/**
	 * Whether this package can be imported, i.e. its package name is a package that exists on disk.
	 * Note: This includes all normal packages where the Name matches the FileName
	 * and localized packages shadowing an existing source package,
	 * but excludes level streaming packages with /Temp/ names.
	 */
	uint8 bCanBeImported:1;

#if WITH_EDITORONLY_DATA
	/** True if this packages has been cooked for the editor / opened cooked by the editor
        *   Note: This flag is accessed on different threads, do not mix with other bitfields above as the value returned might get corrupted.
        */
	bool bIsCookedForEditor{ false };
#endif

#if WITH_EDITORONLY_DATA || UE_ENABLE_ASSET_READ_LOGGER
private:
	/** This flag is manipulated on different threads, do not mix with other bitfields above as they might get corrupted. */
	std::atomic<bool> bHasBeenEndLoaded{ false };

public:
	/** This flag becomes true for loaded packages after serialization and postload and before returning from LoadPackage or calling load completion delegate.
	  *  For newly created packages in editor, it becomes true once the package is saved.
	*/
	bool GetHasBeenEndLoaded() const
	{
		return bHasBeenEndLoaded.load(std::memory_order_acquire) != 0;
	}
	void SetHasBeenEndLoaded(bool bValue)
	{
		bHasBeenEndLoaded.store(bValue, std::memory_order_release);
	}
#endif // WITH_EDITORONLY_DATA || UE_ENABLE_ASSET_READ_LOGGER

private:
#if WITH_EDITORONLY_DATA
	/** Persistent GUID of package if it was loaded from disk. Persistent across saves. */
	FGuid PersistentGuid;

	/** Chunk IDs for the streaming install chunks this package will be placed in.  Empty for no chunk. Used during cooking. */
	TArray<int32> ChunkIDs;

	FIoHash SavedHash;
#endif // WITH_EDITORONLY_DATA

	/** Package Flags */
	std::atomic<uint32> PackageFlagsPrivate;
	
	/** Globally unique id */
	FPackageId PackageId;

	/** The PackagePath this package was loaded from */
	FPackagePath LoadedPath;

	// @note this should probably be entirely deprecated and removed but certain stat dump function are still using it,
	// just compile it out in shipping for now
#if !UE_BUILD_SHIPPING
	/**
	 * Time in seconds it took to fully load this package.
	 * 0 if package is either in process of being loaded or has never been fully loaded.
	 */
	float LoadTime = 0.0f;
#endif

	struct FAdditionalInfo
	{
	/** Linker package version this package has been serialized with. This is mostly used by PostLoad **/
		FPackageFileVersion LinkerPackageVersion = GPackageFileUEVersion;

	/** Linker licensee version this package has been serialized with. This is mostly used by PostLoad **/
		int32 LinkerLicenseeVersion = GPackageFileLicenseeUEVersion;

	/** Linker custom version container this package has been serialized with. This is mostly used by PostLoad **/
	FCustomVersionContainer LinkerCustomVersion;

	/** Linker load associated with this package */
		FLinkerLoad* LinkerLoad = nullptr;

	/** size of the file for this package; if the package was not loaded from a file or was a forced export in another package, this will be zero */
		uint64 FileSize = 0;

		// World browser information
		TUniquePtr< FWorldTileInfo > WorldTileInfo;
	};
	
	/** Contains additional information if they differ from the defaults. */
	TUniquePtr<FAdditionalInfo> AdditionalInfo;

#if WITH_METADATA
	// MetaData for the editor
	FMetaData MetaData;

	// Deprecated UMetaData member when loading older packages
	UE_DEPRECATED(5.6, "UMetaData was replaced by FMetaData, this member is only used for migrating t he existing data, do not use.")
	UDEPRECATED_MetaData* DeprecatedMetaData = nullptr;
#endif // WITH_METADATA

#if WITH_EDITORONLY_DATA
	/** Editor only: Thumbnails stored in this package */
	TUniquePtr< FThumbnailMap > ThumbnailMap;

	/** Editor only: PIE instance ID this package belongs to, INDEX_NONE otherwise */
	int32 PIEInstanceID;
#endif

#if WITH_RELOAD
	/** Link list of delegates registered to the package.  The next pointer chain can't be used for this. */
	TArray<UFunction*> Delegates;
#endif

public:

	// For now, assume all packages have stable net names
	virtual bool IsNameStableForNetworking() const override { return true; }
	// We override NeedsLoadForClient to avoid calling the expensive generic version, which only makes sure that the
	// UPackage static class isn't excluded
	virtual bool NeedsLoadForClient() const override { return true; }
	virtual bool NeedsLoadForServer() const override { return true; }
	COREUOBJECT_API virtual bool IsPostLoadThreadSafe() const override;
	COREUOBJECT_API virtual bool Rename(const TCHAR* NewName = nullptr, UObject* NewOuter = nullptr, ERenameFlags Flags = REN_None) override;

#if WITH_EDITORONLY_DATA
	/** Sets the bIsDynamicPIEPackagePending flag */
	void SetDynamicPIEPackagePending(bool bInIsDynamicPIEPackagePending)
	{
		bIsDynamicPIEPackagePending = bInIsDynamicPIEPackagePending;
	}
	/** returns the bIsDynamicPIEPackagePending flag */
	bool IsDynamicPIEPackagePending() const { return bIsDynamicPIEPackagePending; }
#endif

	/**
	* Called after the C++ constructor and after the properties have been initialized, but before the config has been
	* loaded, etc. Mainly this is to emulate some behavior of when the constructor was called after the properties were
	* initialized.
	*/
	COREUOBJECT_API virtual void PostInitProperties() override;

	COREUOBJECT_API virtual void FinishDestroy() override;

	/** Serializer */
	COREUOBJECT_API virtual void Serialize( FArchive& Ar ) override;

	/** Packages are never assets */
	virtual bool IsAsset() const override { return false; }

	bool IsInstanced() const
	{
		const FName LoadedPathName = LoadedPath.GetPackageFName();
		return !LoadedPathName.IsNone() && (LoadedPathName != GetFName());
	}

#if WITH_EDITOR
	/**
	 *  Functionality to support SoftGC in the cooker. When activated by the cooker, objects in
	 * each package are Objects in each package are stored in SoftGCPackageToObjectList and are
	 * marked as referenced if the package is referenced.
	 */
	COREUOBJECT_API static bool bSupportCookerSoftGC;
	COREUOBJECT_API static TMap<UPackage*, TArrayView<TObjectPtr<UObject>>> SoftGCPackageToObjectList;
	/** Static class override of UObject::AddReferencedObjects */
	COREUOBJECT_API static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
#endif

	// UPackage interface.

private:
	friend class FLinkerLoad;
	friend class FSaveContext;
	friend struct FAsyncPackage2;

	void SetLinker(FLinkerLoad* InLinker)
	{
		LLM_SCOPE_BYNAME(TEXT("Package/AdditionalInfo"));
		if (AdditionalInfo.IsValid())
		{
			AdditionalInfo->LinkerLoad = InLinker;
		}
		else if (InLinker != nullptr)
		{
			AdditionalInfo = MakeUnique<FAdditionalInfo>();
			AdditionalInfo->LinkerLoad = InLinker;
		}
	}

	void SetLinkerPackageVersion(FPackageFileVersion InVersion)
	{
		LLM_SCOPE_BYNAME(TEXT("Package/AdditionalInfo"));
		if (AdditionalInfo.IsValid())
		{
			AdditionalInfo->LinkerPackageVersion = MoveTemp(InVersion);
		}
		else if (InVersion != GPackageFileUEVersion)
		{
			AdditionalInfo = MakeUnique<FAdditionalInfo>();
			AdditionalInfo->LinkerPackageVersion = MoveTemp(InVersion);
		}
	}

	void SetLinkerLicenseeVersion(int32 InVersion)
	{
		LLM_SCOPE_BYNAME(TEXT("Package/AdditionalInfo"));
		if (AdditionalInfo.IsValid())
		{
			AdditionalInfo->LinkerLicenseeVersion = InVersion;
		}
		else if (InVersion != GPackageFileLicenseeUEVersion)
		{
			AdditionalInfo = MakeUnique<FAdditionalInfo>();
			AdditionalInfo->LinkerLicenseeVersion = InVersion;
		}
	}

	void SetLinkerCustomVersions(FCustomVersionContainer InVersions)
	{
		LLM_SCOPE_BYNAME(TEXT("Package/AdditionalInfo"));
		if (AdditionalInfo.IsValid())
		{
			AdditionalInfo->LinkerCustomVersion = MoveTemp(InVersions);
		}
		else if (!InVersions.GetAllVersions().IsEmpty())
		{
			AdditionalInfo = MakeUnique<FAdditionalInfo>();
			AdditionalInfo->LinkerCustomVersion = MoveTemp(InVersions);
		}
	}

	void SetFileSize(int64 InFileSize)
	{
		LLM_SCOPE_BYNAME(TEXT("Package/AdditionalInfo"));
		if (AdditionalInfo.IsValid())
		{
			AdditionalInfo->FileSize = InFileSize;
		}
		else if (InFileSize != 0)
		{
			AdditionalInfo = MakeUnique<FAdditionalInfo>();
			AdditionalInfo->FileSize = MoveTemp(InFileSize);
		}
	}

public:

	/**
	 * Returns the PIE instance id used by the package if any, or INDEX_NONE otherwise 
	 */
	int32 GetPIEInstanceID() const
	{
#if WITH_EDITORONLY_DATA
		return PIEInstanceID;
#else
		return INDEX_NONE;
#endif
	}

	/**
	 * Set the PIE instance id for this package
	 * @param InPIEInstanceID The PIE instance id to use or INDEX_NONE to remove any id set
	 */
	void SetPIEInstanceID(int32 InPIEInstanceID)
	{
#if WITH_EDITORONLY_DATA
		PIEInstanceID = InPIEInstanceID;
#endif
	}

	FLinkerLoad* GetLinker() const
	{
		return AdditionalInfo.IsValid() ? AdditionalInfo->LinkerLoad : nullptr;
	}

	const FPackageFileVersion& GetLinkerPackageVersion() const
	{
		return AdditionalInfo.IsValid() ? AdditionalInfo->LinkerPackageVersion : GPackageFileUEVersion;
	}

	int32 GetLinkerLicenseeVersion() const
	{
		return AdditionalInfo.IsValid() ? AdditionalInfo->LinkerLicenseeVersion : GPackageFileLicenseeUEVersion;
	}

	const FCustomVersionContainer& GetLinkerCustomVersions() const
	{
		static FCustomVersionContainer EmptyVersions;
		return AdditionalInfo.IsValid() ? AdditionalInfo->LinkerCustomVersion : EmptyVersions;
	}

private:
	void EmptyLinkerCustomVersion()
	{
		if (AdditionalInfo.IsValid())
		{
			AdditionalInfo->LinkerCustomVersion.Empty();
		}
	}

public:
	/**
	 * Sets the time it took to load this package.
	 */
	void SetLoadTime( float InLoadTime ) 
	{
#if !UE_BUILD_SHIPPING
		LoadTime = InLoadTime;
#endif
	}

	/**
	* Returns the time it took the last time this package was fully loaded, 0 otherwise.
	*
	* @return Time it took to load.
	*/
	float GetLoadTime() const
	{
#if !UE_BUILD_SHIPPING
		return LoadTime;
#else
		return 0.0f;
#endif
	}

	/**
	 * Clear the package dirty flag without any transaction tracking
	 */
	void ClearDirtyFlag()
	{
		bDirty = false;
	}

	/**
	* Marks/Unmarks the package's bDirty flag, save the package to the transaction buffer if a transaction is ongoing
	*/
	COREUOBJECT_API void SetDirtyFlag( bool bIsDirty );

	/**
	* Returns whether the package needs to be saved.
	*
	* @return		true if the package is dirty and needs to be saved, false otherwise.
	*/
	bool IsDirty() const
	{
		return bDirty;
	}

#if WITH_EDITOR
	/**
	* Marks this package as newly created (has no corresponding file on disk).
	*/
	void MarkAsNewlyCreated()
	{
		MarkAsUnloaded();
		SetPackageFlags(PKG_NewlyCreated);
		SetFileSize(0);
	}

	/**
	* Marks this package as unloaded.
	*/
	void MarkAsUnloaded()
	{
		bHasBeenFullyLoaded = false;
		ClearFlags(RF_WasLoaded);
	}
#endif

	/**
	* Marks this package as being fully loaded.
	*/
	void MarkAsFullyLoaded()
	{
		bHasBeenFullyLoaded = true;
	}

	/**
	* Returns whether the package is fully loaded.
	*
	* @return true if fully loaded or no file associated on disk, false otherwise
	*/
	COREUOBJECT_API bool IsFullyLoaded() const;

	/**
	* Fully loads this package. Safe to call multiple times and won't clobber already loaded assets.
	*/
	COREUOBJECT_API void FullyLoad();

	/**
	 * Get the path this package was loaded from; may be different than packagename, and may not be set if the package
	 * was not loaded from disk
	 */
	COREUOBJECT_API const FPackagePath& GetLoadedPath() const;

	/**
	 * Set the path this package was loaded from; typically called only by the linker during load
	 */
	COREUOBJECT_API void SetLoadedPath(const FPackagePath& PackagePath);

	/**
	* Marks/Unmarks the package's bCanBeImported flag.
	*/
	void SetCanBeImportedFlag(bool bInCanBeImported)
	{
		bCanBeImported = bInCanBeImported;
	}

	/**
	* Returns whether the package can be imported.
	*
	* @return		true if the package can be imported.
	*/
	bool CanBeImported() const
	{
		return bCanBeImported;
	}

	/**
	* Called to indicate that this package contains a ULevel or UWorld object.
	*/
	void ThisContainsMap() 
	{
		SetPackageFlags(PKG_ContainsMap);
	}

	/**
	* Returns whether this package contains a ULevel or UWorld object.
	*
	* @return		true if package contains ULevel/ UWorld object, false otherwise.
	*/
	bool ContainsMap() const
	{
		return HasAnyPackageFlags(PKG_ContainsMap);
	}

	/**
	* Called to indicate that this package contains data required to be gathered for localization.
	*/
	void ThisRequiresLocalizationGather(bool Value)
	{
		if(Value)
		{
			SetPackageFlags(PKG_RequiresLocalizationGather);
		}
		else
		{
			ClearPackageFlags(PKG_RequiresLocalizationGather);
		}
	}

	/**
	* Returns whether this package contains data required to be gathered for localization.
	*
	* @return		true if package contains contains data required to be gathered for localization, false otherwise.
	*/
	bool RequiresLocalizationGather() const
	{
		return HasAnyPackageFlags(PKG_RequiresLocalizationGather);
	}

	/**
	* Call this to indicate that this package should load uncooked when possible (ie hybrid cooked editor).
	* It requires an FArchive param to validate that we only set this flag on cooked packages. Eventually,
	* this flag shouldn't be needed anymore, but we don't want to save this flag in source/checked in packages, 
	* in case the particular PKG flag bit is reused, as that could confuse whatever code uses that bit in the future
	*/
	void ThisShouldLoadUncooked(const FArchive& Ar)
	{
		if (Ar.IsSaving() && Ar.IsCooking())
		{
			SetPackageFlags(PKG_LoadUncooked);
		}
	}


	/**
	* Sets all package flags to the specified values.
	*
	* @param	NewFlags		New value for package flags
	*/
	inline void SetPackageFlagsTo( uint32 NewFlags )
	{
		uint32 OldFlags = GetPackageFlags();

		// Fast path without atomics if already set
		if (NewFlags == OldFlags)
		{
			return;
		}

		if (AutoRTFM::IsClosed())
		{
			UE_AUTORTFM_OPEN
			{
				OldFlags = PackageFlagsPrivate.exchange(NewFlags, std::memory_order_relaxed);
			};

			// If we abort we undo setting the flags we just set.
			AutoRTFM::OnAbort([this, OldFlags]
				{
					PackageFlagsPrivate.store(OldFlags, std::memory_order_relaxed);
				});
		}
		else
		{
			PackageFlagsPrivate.store(NewFlags, std::memory_order_relaxed);
		}
	}

	/**
	* Set the specified flags to true. Does not affect any other flags.
	*
	* @param	FlagsToSet		Package flags to enable
	*/
	inline void SetPackageFlags( uint32 FlagsToSet )
	{
		uint32 OldFlags = GetPackageFlags();
		uint32 NewFlags = OldFlags | FlagsToSet;

		// Fast path without atomics if already set
		if (NewFlags == OldFlags)
		{
			return;
		}

		if (AutoRTFM::IsClosed())
		{
			UE_AUTORTFM_OPEN
			{
				OldFlags = PackageFlagsPrivate.fetch_or(FlagsToSet, std::memory_order_relaxed);
			};

			// If we abort we undo setting the flags we just set.
			AutoRTFM::OnAbort([this, OldFlags, FlagsToSet]
				{
					uint32 MaskFlags = OldFlags;

					// Now just extract out the old flags that mattered (the ones we were setting).
					MaskFlags &= FlagsToSet;
					// And unmask the flags we didn't mention.
					MaskFlags |= ~FlagsToSet;

					PackageFlagsPrivate.fetch_and(MaskFlags, std::memory_order_relaxed);
				});
		}
		else
		{
			PackageFlagsPrivate.fetch_or(FlagsToSet, std::memory_order_relaxed);
		}
	}

	/**
	* Set the specified flags to false. Does not affect any other flags.
	*
	* @param	FlagsToClear		Package flags to disable
	*/
	inline void ClearPackageFlags( uint32 FlagsToClear )
	{
		uint32 OldFlags = GetPackageFlags();
		uint32 NewFlags = OldFlags & ~FlagsToClear;

		// Fast path without atomics if already cleared
		if (NewFlags == OldFlags)
		{
			return;
		}

		if (AutoRTFM::IsClosed())
		{
			UE_AUTORTFM_OPEN 
			{
				OldFlags = PackageFlagsPrivate.fetch_and(~FlagsToClear, std::memory_order_relaxed);
			};

			// If we abort we undo clearing the flags we just unset.
			AutoRTFM::OnAbort([this, OldFlags, FlagsToClear]
			{
				int32 MaskFlags = OldFlags;

				// Now just extract out the old flags that mattered (the ones we were setting).
				MaskFlags &= FlagsToClear;

				PackageFlagsPrivate.fetch_or(MaskFlags, std::memory_order_relaxed);
			});
		}
		else
		{
			PackageFlagsPrivate.fetch_and(~FlagsToClear, std::memory_order_relaxed);
		}
	}

	/**
	* Used to safely check whether the passed in flag is set.
	*
	* @param	FlagsToCheck		Package flags to check for
	*
	* @return	true if the passed in flag is set, false otherwise
	*			(including no flag passed in, unless the FlagsToCheck is CLASS_AllFlags)
	*/
	UE_FORCEINLINE_HINT bool HasAnyPackageFlags( uint32 FlagsToCheck ) const
	{
		return (GetPackageFlags() & FlagsToCheck) != 0;
	}

	/**
	* Used to safely check whether all of the passed in flags are set.
	*
	* @param FlagsToCheck	Package flags to check for
	* @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	*/
	UE_FORCEINLINE_HINT bool HasAllPackagesFlags( uint32 FlagsToCheck ) const
	{
		return ((GetPackageFlags() & FlagsToCheck) == FlagsToCheck);
	}

	/**
	* Gets the package flags.
	*
	* @return	The package flags.
	*/
	inline uint32 GetPackageFlags() const
	{
		uint32 Result = 0;

		UE_AUTORTFM_OPEN
		{
			Result = PackageFlagsPrivate.load(std::memory_order_relaxed);
		};

		return Result;
	}

	/**
	* @return true if the package is marked as ExternallyReferenceable by all plugins and mount points
	*/
	UE_FORCEINLINE_HINT bool IsExternallyReferenceable() const
	{
		return GetAssetAccessSpecifier() == EAssetAccessSpecifier::Public;
	}

	/**
	* Sets whether or not the package is ExternallyReferenceable by all plugins and mount points
	* 
	* @param bValue Sets the package to be ExternallyReferenceable if true
	*/
	UE_FORCEINLINE_HINT void SetIsExternallyReferenceable(bool bValue)
	{
		SetAssetAccessSpecifier(bValue ? EAssetAccessSpecifier::Public : EAssetAccessSpecifier::Private);
	}

	/** Gets how package can be referenced from other plugins and mount points */
	inline EAssetAccessSpecifier GetAssetAccessSpecifier() const
	{
		if (HasAnyPackageFlags(PKG_NotExternallyReferenceable))
		{
			return EAssetAccessSpecifier::Private;
		}
		else if (HasAnyPackageFlags(PKG_AccessSpecifierEpicInternal))
		{
			return EAssetAccessSpecifier::EpicInternal;
		}

		return EAssetAccessSpecifier::Public;
	}

	/**
	* Sets how the package can be referenced from other plugins and mount points
	* 
	* @param InAccessSpecifier	How package can be referenced
	* @return true if changed
	*/
	inline bool SetAssetAccessSpecifier(const EAssetAccessSpecifier InAccessSpecifier)
	{
		if (GetAssetAccessSpecifier() == InAccessSpecifier)
		{
			return false;
		}

#if WITH_EDITOR
		Modify();
#endif

		ClearPackageFlags(PKG_NotExternallyReferenceable | PKG_AccessSpecifierEpicInternal);

		if (EAssetAccessSpecifier::Private == InAccessSpecifier)
		{
			SetPackageFlags(PKG_NotExternallyReferenceable);
		}
		else if (EAssetAccessSpecifier::EpicInternal == InAccessSpecifier)
		{
			SetPackageFlags(PKG_AccessSpecifierEpicInternal);
		}

		return true;
	}

#if WITH_EDITORONLY_DATA
	/** Returns true if this package has a thumbnail map */
	bool HasThumbnailMap() const
	{
		return ThumbnailMap.IsValid();
	}

	/** Returns the thumbnail map for this package (const).  Only call this if HasThumbnailMap returns true! */
	const FThumbnailMap& GetThumbnailMap() const
	{
		check( HasThumbnailMap() );
		return *ThumbnailMap;
	}

	/** Access the thumbnail map for this package.  Only call this if HasThumbnailMap returns true! */
	FThumbnailMap& AccessThumbnailMap()
	{
		check( HasThumbnailMap() );
		return *ThumbnailMap;
	}

	/** Set the internal thumbnail map for this package. */
	void SetThumbnailMap(TUniquePtr<FThumbnailMap> InThumbnailMap)
	{
		ThumbnailMap = MoveTemp(InThumbnailMap);
	}

	/** returns our persistent Guid */
	UE_FORCEINLINE_HINT FGuid GetPersistentGuid() const
	{
		return PersistentGuid;
	}
	/** sets a specific persistent Guid */
	UE_FORCEINLINE_HINT void SetPersistentGuid(FGuid NewPersistentGuid)
	{
		PersistentGuid = NewPersistentGuid;
	}

	/** Hash of the package's .uasset/.umap file when it was last saved by the editor. */
	COREUOBJECT_API FIoHash GetSavedHash() const;
	COREUOBJECT_API void SetSavedHash(const FIoHash& InSavedHash);
#endif

#if WITH_RELOAD
	const TArray<UFunction*>& GetReloadDelegates() const
	{
		return Delegates;
	}

	void SetReloadDelegates(TArray<UFunction*> InDelegates) 
	{
		Delegates = MoveTemp(InDelegates);
	}
#endif

	/** Get the world tile info if any*/
	FWorldTileInfo* GetWorldTileInfo() const
	{
		return AdditionalInfo.IsValid() ? AdditionalInfo->WorldTileInfo.Get() : nullptr;
	}

	/** Set the world tile info */
	void SetWorldTileInfo(TUniquePtr<FWorldTileInfo> InWorldTileInfo)
	{
		LLM_SCOPE_BYNAME(TEXT("Package/AdditionalInfo"));
		if (AdditionalInfo.IsValid())
		{
			AdditionalInfo->WorldTileInfo = MoveTemp(InWorldTileInfo);
		}
		else if (InWorldTileInfo.IsValid())
		{
			AdditionalInfo = MakeUnique<FAdditionalInfo>();
			AdditionalInfo->WorldTileInfo = MoveTemp(InWorldTileInfo);
		}
	}

	/** returns our FileSize */
	UE_FORCEINLINE_HINT int64 GetFileSize() const
	{
		return AdditionalInfo.IsValid() ? AdditionalInfo->FileSize : 0;
	}

	/** returns our ChunkIDs */
	const TArray<int32>& GetChunkIDs() const
	{
#if WITH_EDITORONLY_DATA
		return ChunkIDs;
#else
		static TArray<int32> Dummy;
		return Dummy;
#endif
	}

	/** sets our ChunkIDs */
	UE_FORCEINLINE_HINT void SetChunkIDs(const TArray<int32>& InChunkIDs)
	{
#if WITH_EDITORONLY_DATA
		ChunkIDs = InChunkIDs;
#endif
	}

	/** returns the unique package id */
	UE_FORCEINLINE_HINT FPackageId GetPackageId() const
	{
		return PackageId;
	}

	/** sets the unique package id */
	UE_FORCEINLINE_HINT void SetPackageId(FPackageId InPackageId)
	{
		PackageId = InPackageId;
	}

	/** returns the unique package id to load */
	UE_FORCEINLINE_HINT FPackageId GetPackageIdToLoad() const
	{
		return FPackageId::FromName(LoadedPath.GetPackageFName());
	}

	/**
	 * Utility function to find Asset in this package, if any
	 * @return the asset in the package, if any
	 */
	COREUOBJECT_API UObject* FindAssetInPackage(EObjectFlags RequiredTopLevelFlags = RF_NoFlags) const;

	/**
	 * Return the list of packages found assigned to object outer-ed to the top level objects of this package
	 * @return the array of external packages
	 */
	COREUOBJECT_API TArray<UPackage*> GetExternalPackages() const;

	////////////////////////////////////////////////////////
	// MetaData 

#if WITH_METADATA
	/**
	* Gets (after possibly creating) a metadata object for this package
	*
	* @param	bAllowLoadObject				Can load an object to find it's FMetaData if not currently loaded.
	*
	* @return A valid FMetaData pointer for all objects in this package
	*/
	COREUOBJECT_API FMetaData& GetMetaData();
#endif

	/**
	* Save one specific object asynchronously (along with any objects it references contained within the same Outer) into an Unreal package.
	*
	* @param	InPackage		the package to save the asset into
	* @param	InAsset			the asset that should be saved into the package
	* @param	Filename		the name to use for the new package file
	* @param	SaveArgs		Extended arguments to control the save
	* @see		FSavePackageContext
	*
	* @return	A task containing a FSavePackageResultStruct enum value with the result of saving a package as well as extra data
	*
	* @note         Some tasks might need to execute on the gamethread. If the engine isn't ticking, it might be required
        *               to pump the gamethread task queue manually. (i.e. FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread) or similar).
	*/
	COREUOBJECT_API static UE::Tasks::TTask<FSavePackageResultStruct> SaveAsync(UPackage* InPackage, UObject* InAsset, const TCHAR* InFilename,
		const FSavePackageArgs& SaveArgs);

	/**
	* Save one specific object (along with any objects it references contained within the same Outer) into an Unreal package.
	* 
	* @param	InOuter			the outer to use for the new package
	* @param	InAsset			the object that should be saved into the package
	* @param	Filename		the name to use for the new package file
	* @param	SaveArgs		Extended arguments to control the save
	* @see		FSavePackageContext
	*
	* @return	FSavePackageResultStruct enum value with the result of saving a package as well as extra data
	*/
	COREUOBJECT_API static FSavePackageResultStruct Save(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename,
		const FSavePackageArgs& SaveArgs);

	/**
	 * Save a list of packages concurrently.
	 * SaveConcurrent is currently experimental and shouldn't be used until it can safely replace Save.
	 */
	COREUOBJECT_API static ESavePackageResult SaveConcurrent(TArrayView<FPackageSaveInfo> InPackages,
		const FSavePackageArgs& SaveArgs, TArray<FSavePackageResultStruct>& OutResults);

	/**
	* Save one specific object (along with any objects it references contained within the same Outer) into an Unreal package.
	*
	* @param	InOuter			the outer to use for the new package
	* @param	InAsset			the object that should be saved into the package
	* @param	Filename		the name to use for the new package file
	* @param	SaveArgs		Extended arguments to control the save
	* @see		FSavePackageContext
	*
	* @return	true if the package was saved successfully.
	*/
	COREUOBJECT_API static bool SavePackage(UPackage* InOuter, UObject* InAsset, const TCHAR* Filename,
		const FSavePackageArgs& SaveArgs);

	/** Wait for any SAVE_Async file writes to complete **/
	COREUOBJECT_API static void WaitForAsyncFileWrites();

	/** Return true if SAVE_Async file writes are pending **/
	COREUOBJECT_API static bool HasAsyncFileWrites();

	/**
	* Determines if a package contains no more assets.
	*
	* @param Package			the package to test
	* @param LastReferencer	the optional last UObject referencer to this package. This object will be excluded when determining if the package is empty
	* @return true if Package contains no more assets.
	*/
	COREUOBJECT_API static bool IsEmptyPackage(UPackage* Package, const UObject* LastReferencer = NULL);
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
