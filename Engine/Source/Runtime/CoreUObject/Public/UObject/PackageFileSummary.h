// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "IO/IoHash.h"
#include "Misc/EngineVersion.h"
#include "Misc/Guid.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/ObjectVersion.h"

#if WITH_EDITORONLY_DATA
#include "IO/IoHash.h"
#endif

class FArchive;
struct FCompressedChunk;

/*----------------------------------------------------------------------------
	Items stored in Unrealfiles.
----------------------------------------------------------------------------*/

/**
 * Revision data for an Unreal package file.
 */
//@todo: shouldn't need ExportCount/NameCount with the linker free package map; if so, clean this stuff up
struct FGenerationInfo
{
	/**
	 * Number of exports in the linker's ExportMap for this generation.
	 */
	int32 ExportCount;

	/**
	 * Number of names in the linker's NameMap for this generation.
	 */
	int32 NameCount;

	/** Constructor */
	FGenerationInfo( int32 InExportCount, int32 InNameCount );

	/** I/O function
	 * we use a function instead of operator<< so we can pass in the package file summary for version tests, since archive version hasn't been set yet
	 */
	void Serialize(FArchive& Ar, const struct FPackageFileSummary& Summary);
	void Serialize(FStructuredArchive::FSlot Slot, const struct FPackageFileSummary& Summary);
};

/**
 * A "table of contents" for an Unreal package file.  Stored at the top of the file.
 */
struct FPackageFileSummary
{
	/**
	* Magic tag compared against PACKAGE_FILE_TAG to ensure that package is an Unreal package.
	*/
	int32		Tag;

private:
	/* UE file version */
	FPackageFileVersion FileVersionUE;
	/* Licensee file version */
	int32		FileVersionLicenseeUE;
	/* Custom version numbers. Keyed off a unique tag for each custom component. */
	FCustomVersionContainer CustomVersionContainer;

	/**
	 * The flags for the package
	 */
	uint32	PackageFlags;

public:
	/**
	* Total size of all information that needs to be read in to create a FLinkerLoad. This includes
	* the package file summary, name table and import & export maps.
	*/
	int32		TotalHeaderSize;

	/**
	* The package name the file was last saved with.
	*/
	FString	PackageName;

	/**
	* Number of names used in this package
	*/
	int32		NameCount;

	/**
	* Location into the file on disk for the name data
	*/
	int32 	NameOffset;

	/**
	* Number of soft object paths references contained in this package
	* @note: the soft object path lists slightly duplicate information found in the SoftPackageReferences list
	*/
	int32		SoftObjectPathsCount;

	/**
	* Location into the file on disk for the soft object paths reference list
	*/
	int32		SoftObjectPathsOffset;

	/**
	* Localization ID of this package
	* @note This is copy of the version stored in the package meta-data. It exists here so we can query it without having to load the whole package
	*/
	FString		LocalizationId;

	/**
	* Number of gatherable text data items in this package
	*/
	int32		GatherableTextDataCount;

	/**
	* Location into the file on disk for the gatherable text data items
	*/
	int32 		GatherableTextDataOffset;

	/**
	* Location into the file on disk for the MetaData data
	*/
	int32		MetaDataOffset;

	/**
	* Number of exports contained in this package
	*/
	int32		ExportCount;

	/**
	* Location into the file on disk for the ExportMap data
	*/
	int32		ExportOffset;

	/**
	* Number of imports contained in this package
	*/
	int32		ImportCount;

	/**
	* Location into the file on disk for the ImportMap data
	*/
	int32		ImportOffset;

	/**
	* Number of cells contained in this package
	*/
	int32		CellExportCount;

	/**
	* Location into the file on disk for the CellExportMap data
	*/
	int32		CellExportOffset;

	/**
	* Number of cell imports contained in this package
	*/
	int32		CellImportCount;

	/**
	* Location into the file on disk for the CellImportMap data
	*/
	int32		CellImportOffset;

	/**
	* Location into the file on disk for the DependsMap data
	*/
	int32		DependsOffset;

	/**
	* Number of soft package references contained in this package
	*/
	int32		SoftPackageReferencesCount;

	/**
	* Location into the file on disk for the soft package reference list
	*/
	int32		SoftPackageReferencesOffset;

	/**
	* Location into the file on disk for the SearchableNamesMap data
	*/
	int32		SearchableNamesOffset;

	/**
	* Thumbnail table offset
	*/
	int32		ThumbnailTableOffset;

	/**
	* Number of import type hierarchy entries
	*/
	int32		ImportTypeHierarchiesCount;

	/**
	* Location into the file on disk for the import type hierarchy map
	*/
	int32		ImportTypeHierarchiesOffset;

private:
	/**
	 * Hash of the Package's bytes when it was saved to disk. Accessed through GetSavedHash/SetSavedHash.
	 */
	FIoHash		SavedHash;

public:
#if WITH_EDITORONLY_DATA
	/**
	* Current persistent id for this package
	*/
	FGuid	PersistentGuid;
#endif

	/**
	* Data about previous versions of this package
	*/
	TArray<FGenerationInfo> Generations;

	/**
	* Engine version this package was saved with. This may differ from CompatibleWithEngineVersion for assets saved with a hotfix release.
	*/
	FEngineVersion SavedByEngineVersion;

	/**
	* Engine version this package is compatible with. Assets saved by Hotfix releases and engine versions that maintain binary compatibility will have
	* a CompatibleWithEngineVersion.Patch that matches the original release (as opposed to SavedByEngineVersion which will have a patch version of the new release).
	*/
	FEngineVersion CompatibleWithEngineVersion;

	/**
	* Flags used to compress the file on save and uncompress on load.
	*/
	uint32	CompressionFlags;

	/**
	* Value that is used to determine if the package was saved by Epic (or licensee) or by a modder, etc
	*/
	uint32	PackageSource;

	/**
	* If true, this file will not be saved with version numbers or was saved without version numbers. In this case they are assumed to be the current version.
	* This is only used for full cooks for distribution because it is hard to guarantee correctness
	**/
	bool bUnversioned;

	/**
	* Location into the file on disk for the asset registry tag data
	*/
	int32 	AssetRegistryDataOffset;

	/** Offset to the location in the file where the bulkdata starts */
	int64	BulkDataStartOffset;
	/**
	* Offset to the location in the file where the FWorldTileInfo data starts
	*/
	int32 	WorldTileInfoDataOffset;

	/**
	* Streaming install ChunkIDs
	*/
	TArray<int32>	ChunkIDs;

	int32		PreloadDependencyCount;

	/**
	* Location into the file on disk for the preload dependency data
	*/
	int32		PreloadDependencyOffset;

	/**
	* Number of names that are referenced from serialized export data (sorted first in the name map)
	*/
	int32		NamesReferencedFromExportDataCount;
	
	/**
	* Location into the file on disk for the payload table of contents data
	*/
	int64		PayloadTocOffset;
	
	/**
	* Location into the file of the data resource(s)
	*/
	int32		DataResourceOffset;

	/** Constructor */
	COREUOBJECT_API FPackageFileSummary();

	// Workaround for clang deprecation warnings for deprecated members in implicit constructors
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FPackageFileSummary(FPackageFileSummary&&) = default;
	FPackageFileSummary(const FPackageFileSummary&) = default;
	FPackageFileSummary& operator=(FPackageFileSummary&&) = default;
	FPackageFileSummary& operator=(const FPackageFileSummary&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	FPackageFileVersion GetFileVersionUE() const
	{
		return FileVersionUE;
	}

	int32 GetFileVersionLicenseeUE() const
	{
		return FileVersionLicenseeUE;
	}

	const FCustomVersionContainer& GetCustomVersionContainer() const
	{
		return CustomVersionContainer;
	}

	void SetCustomVersionContainer(const FCustomVersionContainer& InContainer);

	/** Used to manually set the package file and licensee versions */
	COREUOBJECT_API void SetFileVersions(const int32 EpicUE4, const int32 EpicUE5, const int32 LicenseeUE, const bool bInSaveUnversioned = false);
	
	/** Set both the package file versions and the licensee version to the most recent version numbers supported */
	void SetToLatestFileVersions(const bool bInSaveUnversioned)
	{
		FileVersionUE = GPackageFileUEVersion;
		FileVersionLicenseeUE = GPackageFileLicenseeUEVersion;

		bUnversioned = bInSaveUnversioned;
	}

	/** Returns true if any of the package file versions are older than the minimum supported versions */
	bool IsFileVersionTooOld() const
	{
		return FileVersionUE < VER_UE4_OLDEST_LOADABLE_PACKAGE;
	}

	/** Returns true if any of the package file versions are newer than currently supported by the running process */
	bool IsFileVersionTooNew() const
	{
		return FileVersionUE.FileVersionUE4 > GPackageFileUEVersion.FileVersionUE4 || FileVersionUE.FileVersionUE5 > GPackageFileUEVersion.FileVersionUE5;
	}

	/** 
	 * Returns false if the summary is unversioned and the current process does not support that. 
	 * If this returns false and the summary was loaded from disk then the serialization of the summary was aborted early!
	 */
	COREUOBJECT_API bool IsFileVersionValid() const;

	/** Get the summary package flags. */
	uint32 GetPackageFlags() const
	{
		return PackageFlags;
	}

	/** Set the summary package flags while stripping out temporary flags (i.e. NewlyCreated, IsSaving) */
	void SetPackageFlags(uint32 InPackageFlags);

#if WITH_EDITORONLY_DATA
	/** Hash of the package's .uasset/.umap file when it was last saved by the editor. */
	COREUOBJECT_API FIoHash GetSavedHash() const;
	COREUOBJECT_API void SetSavedHash(const FIoHash& InSavedHash);
	COREUOBJECT_API int64 GetSavedHashRelativeOffset() const;
#endif

	/** I/O functions */
	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FPackageFileSummary& Sum);
	friend COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, FPackageFileSummary& Sum);

private:

	/** 
	 * Set the UE4 version to below the oldest loadable version. This is used when an error is encountered during serialization 
	 * and we need to prevent the use of the package. When calling this make sure you log a warning with the 'LogLinker' category
	 * to inform the user of the problem.
	 */
	void InvalidateFileVersion()
	{
		FileVersionUE = FPackageFileVersion::CreateUE4Version((EUnrealEngineObjectUE4Version)(VER_UE4_OLDEST_LOADABLE_PACKAGE - 1));
	}
};
