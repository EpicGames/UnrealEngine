// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IAssetRegistryInterface;
class UPackage;

/** Data needed by ReloadPackages */
struct FReloadPackageData
{
	FReloadPackageData()
		: PackageToReload(nullptr)
		, LoadFlags(LOAD_None)
	{
	}

	FReloadPackageData(UPackage* InPackageToReload, const uint32 InLoadFlags)
		: PackageToReload(InPackageToReload)
		, LoadFlags(InLoadFlags)
	{
	}

	/** Pointer to the package to reload (pointer will become invalid if the package is reloaded successfully) */
	UPackage* PackageToReload;

	/** Flags controlling loading behavior for the replacement package */
	uint32 LoadFlags;
};

/** Enum describing the phase of the package reload */
enum class EPackageReloadPhase : uint8
{
	/** Called once for each each package in a batch prior to loading its new version. This phase can be used to discard anything that may be required before the new package is loaded (note: the new package isn't guaranteed to load successfully!). */
	PrePackageLoad,
	/** Called once for each each package in a batch prior to any object fix-up happening. This phase can be used to make sure that any UI referencing old objects is removed or updated as needed. */
	PrePackageFixup,
	/** Called once for each each package in a batch prior to automatic fix-up. This phase can be used to fix-up any references that can't automatically be fixed-up (ie, non-uproperty or ARO references). */
	OnPackageFixup,
	/** Called once for each each package in a batch after all reference fix-up has happened, and before the old package is purged. This phase can be used to finalize the changes made to any objects that were referencing the objects from the old package. */
	PostPackageFixup,
	/** Called once before a batch starts */
	PreBatch,
	/** Called once after a batch has ended, but before GC has run */
	PostBatchPreGC,
	/** Called once after a batch has ended, and after GC has run */
	PostBatchPostGC,
};

/** Delegate payload for FOnPackageReloaded */
class FPackageReloadedEvent
{
public:
	FPackageReloadedEvent(const UPackage* InOldPackage, const UPackage* InNewPackage, TMap<UObject*, UObject*> InRepointedObjects)
		: OldPackage(InOldPackage)
		, NewPackage(InNewPackage)
		, RepointedObjects(ObjectPtrWrap(MoveTemp(InRepointedObjects)))
		, ObjectReferencers()
	{
	}

	/**
	 * Get the old package pointer (the original package that was replaced).
	 */
	UE_FORCEINLINE_HINT const UPackage* GetOldPackage() const
	{
		return OldPackage;
	}

	/**
	 * Get the new package pointer (the package that replaced the original one). 
	 */
	UE_FORCEINLINE_HINT const UPackage* GetNewPackage() const
	{
		return NewPackage;
	}

	/**
	 * Get the raw map of repointed objects.
	 */
	UE_FORCEINLINE_HINT const TMap<UObject*, UObject*>& GetRepointedObjects() const
	{
		return ObjectPtrDecay(RepointedObjects);
	}

	/**
	 * Given an object pointer, check to see if it needs to be repointed to an object in the new package, and populate OutRepointedObject if needed.
	 * @return true if the object can be repointed, false otherwise.
	 */
	template <typename InObjectType, typename OutObjectType>
	inline bool GetRepointedObject(InObjectType* InObject, OutObjectType*& OutRepointedObject) const
	{
		static_assert(TPointerIsConvertibleFromTo<InObjectType, const volatile UObject>::Value && TPointerIsConvertibleFromTo<OutObjectType, const volatile UObject>::Value, "FPackageReloadedEvent::GetRepointedObject can only be used with UObject types");
		return GetRepointedObjectInternal((const UObject*)InObject, (const UObject*&)OutRepointedObject);
	}

	/**
	 * Given an object pointer, check to see if it needs to be repointed to an object in the new package, and update it if needed.
	 * @return true if the object was repointed, false otherwise.
	 */
	template <typename ObjectType>
	inline bool RepointObject(ObjectType*& Object) const
	{
		static_assert(TPointerIsConvertibleFromTo<ObjectType, const volatile UObject>::Value, "FPackageReloadedEvent::RepointObject can only be used with UObject types");
		return GetRepointedObjectInternal((const UObject*)Object, (const UObject*&)Object);
	}

	/**
	 * Get the set of objects that were referencing any of the objects that were replaced by the package reload.
	 * @note Beware that object referencers may become null as the result of other operations that happen during reloading (such as recompiling Blueprints).
	 */
	UE_FORCEINLINE_HINT const TArray<TWeakObjectPtr<UObject>>& GetObjectReferencers() const
	{
		return ObjectReferencers;
	}

	/**
	 * Add a referencing object for any of the objects that were replaced by the package reload.
	 */
	UE_FORCEINLINE_HINT void AddObjectReferencer(UObject* InObject)
	{
		ObjectReferencers.AddUnique(InObject);
	}

	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObjects(RepointedObjects);
	}

private:
	bool GetRepointedObjectInternal(const UObject* InObject, const UObject*& OutRepointedObject) const
	{
		if (InObject && RepointedObjects.Contains(InObject))
		{
			OutRepointedObject = RepointedObjects[InObject];
			return true;
		}
		return false;
	}

	const UPackage* OldPackage;
	const UPackage* NewPackage;
	TMap<TObjectPtr<UObject>, TObjectPtr<UObject>> RepointedObjects;
	TArray<TWeakObjectPtr<UObject>> ObjectReferencers;
};

/**
 * Given an array of packages, sort them so that dependencies will be processed before the packages that depend on them.
 *
 * @param	PackagesToReload		Array of packages to sort. Will be replaced with the sorted data.
 */
COREUOBJECT_API void SortPackagesForReload(TArray<UPackage*>& PackagesToReload);

/**
 * Checks to see if a package has been loaded, and if so, unloads it before loading it again. Does nothing if the package isn't currently loaded.
 *
 * @param	InPackageToReload		Pointer to the package to reload (pointer will become invalid if the package is reloaded successfully).
 * @param	InLoadFlags				Flags controlling loading behavior for the replacement package.
 *
 * @return	Reloaded package if successful, null otherwise.
 */
COREUOBJECT_API UPackage* ReloadPackage(UPackage* InPackageToReload, const uint32 InLoadFlags);

/**
 * Given an array of packages, checks to see if each package has been loaded, and if so, unloads it before loading it again. Does nothing if the package isn't currently loaded.
 * @note This doesn't re-order the loads to make sure that dependencies are re-loaded first, you must handle that in your calling code (@see SortPackagesForReload).
 *
 * @param	InPackagesToReload		Array of packages to reload.
 * @param	OutReloadedPackages		Array of new package pointers. An entry will be present for every item from InPackagesToReload, however they may be null.
 * @param	InNumPackagesPerBatch	The number of packages to process before running a pointer fix-up and GC. More packages per-batch will process faster, but consume more memory.
 */
COREUOBJECT_API void ReloadPackages(const TArrayView<FReloadPackageData>& InPackagesToReload, TArray<UPackage*>& OutReloadedPackages, const int32 InNumPackagesPerBatch = 1);
