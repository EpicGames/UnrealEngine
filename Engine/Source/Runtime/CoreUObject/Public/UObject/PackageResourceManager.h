// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Delegates/Delegate.h"
#include "Features/IModularFeature.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/PlatformFile.h"
#include "Logging/LogMacros.h"
#include "Misc/PackageName.h"
#include "Misc/PackagePath.h"
#include "Misc/PackageSegment.h"
#include "Modules/ModuleInterface.h"
#include "Serialization/Archive.h"
#include "Serialization/BulkDataCookedIndex.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FArchive;
class FName;
class FPreloadableArchive;
class IAsyncReadFileHandle;
class IAsyncReadRequest;
class IMappedFileHandle;
class IPackageResourceManager;
class UPackage;
struct FFileStatData;

DECLARE_LOG_CATEGORY_EXTERN(LogPackageResourceManager, Log, All);
DECLARE_DELEGATE_RetVal(IPackageResourceManager*, FSetPackageResourceManager);
DECLARE_MULTICAST_DELEGATE(FOnClearPackageResourceManager);

/**
 * Format for a package payload reported by the PackageResourceManager.
 * The appropriate ArchiveInputFormatter must be used for each format.
 */
enum class EPackageFormat : uint8
{
	Binary, // Standard UnrealFormat, every field marshalled to bytes with Serialize(void*,int)
	Text, // Text format, use FJsonArchiveInputFormatter
};
COREUOBJECT_API EPackageFormat ExtensionToPackageFormat(EPackageExtension Extension);

struct FOpenPackageResult
{
	/** Archive containing the bytes for the requested PackagePath segment */
	TUniquePtr<FArchive> Archive;
	/**
	 * Format of the archive, binary or text.
	 * Currently only header segments can have EPackageFormat::Text, all other segments have EPackageFormat::Binary
	 */
	EPackageFormat Format = EPackageFormat::Binary;
	/**
	 * True if the package is of unknown version and needs to check for version and corruption.
	 * False if the package was loaded from a repository specifically for the current binary's versions
	 * and has already been checked for corruption.
	 */
	bool bNeedsEngineVersionChecks = true;

	void CopyMetaData(const FOpenPackageResult& Other)
	{
		Format = Other.Format;
		bNeedsEngineVersionChecks = Other.bNeedsEngineVersionChecks;
	}
};

struct FOpenAsyncPackageResult
{
	/** AsyncReadFileHandle for the requested Segment bytes, in canceled state if it does not exist. */
	TUniquePtr<IAsyncReadFileHandle> Handle;
	/**
	 * Format of the archive, binary or text.
	 * Currently only header segments can have EPackageFormat::Text, all other segments have EPackageFormat::Binary
	 */
	EPackageFormat Format = EPackageFormat::Binary;
	/**
	 * True if the package is of unknown version and needs to check for version and corruption.
	 * False if the package was loaded from a repository specifically for the current binary's versions
	 * and has already been checked for corruption.
	 */
	bool bNeedsEngineVersionChecks = true;

	FOpenAsyncPackageResult() = default;
	FOpenAsyncPackageResult(const FOpenAsyncPackageResult&) = delete;
	FOpenAsyncPackageResult(FOpenAsyncPackageResult&&) = default;
	FOpenAsyncPackageResult& operator=(const FOpenAsyncPackageResult&) = delete;
	FOpenAsyncPackageResult& operator=(FOpenAsyncPackageResult&&) = default;

	COREUOBJECT_API FOpenAsyncPackageResult(TUniquePtr<IAsyncReadFileHandle>&& InHandle, EPackageFormat InFormat, bool bInNeedsEngineVersionChecks = true);
	COREUOBJECT_API ~FOpenAsyncPackageResult();

	void CopyMetaData(const FOpenPackageResult& Other)
	{
		Format = Other.Format;
		bNeedsEngineVersionChecks = Other.bNeedsEngineVersionChecks;
	}
};

enum class EPackageExternalResource
{
	/**
	 * Open a Package file from the Workspace Domain, even if the Manager is providing packages from a different domain.
	 * The identifier is the PackageName to open.
	 */
	WorkspaceDomainFile,
};

/**
 * Provides directory queries and Archive payloads for PackagePaths and their PackageSegments from a repository that
 * might be the local content directories, a database running on the current machine, or a remote database.
 */
class IPackageResourceManager
{
public:
	// Public API

	/**
	 * Returns the package resource manager. It is illegal to call Get before calling initialize.
	 */
	COREUOBJECT_API static IPackageResourceManager& Get();
	virtual ~IPackageResourceManager() = default;

	/**
	 * Report whether the PackageResourceManager supports PackagePaths that are unmounted LocalPaths
	 *
	 * If unsupported, functions without LocalOnly in their name that take a PackagePath will behave as if the package
	 * does not exist in the repository for unmounted LocalPaths
	 * If unsupported, functions with LocalOnly in their name will indicate the lack of support when called with
	 * unmounted LocalPaths; see each function's description for its precise behavior
	 */
	virtual bool SupportsLocalOnlyPaths() = 0;

	/**
	 * Report whether the PackageResourceManager supports PackagePaths that are unmounted PackageNames
	 *
	 * If unsupported, functions without PackageNameOnly in their name that take a PackagePath will behave as if the
	 * package does not exist in the repository for unmounted PackageNames
	 * If unsupported, functions with PackageNameOnly in their name will indicate the lack of support when called with
	 * unmounted PackageNames; see each function's description for its precise behavior
	 */
	virtual bool SupportsPackageOnlyPaths() = 0;

	/**
	 * Report whether the package exists
	 *
	 * @param PackagePath The package to look for
	 * @param OutUpdatedPath If non-null and the package is found, PackagePath is copied into this path (noop if
	 *        OutUpdatedPath == &PackagePath), and if the package exists, the specific extension found is set
	 * @return true if the package exists, else false
	 */
	COREUOBJECT_API bool DoesPackageExist(const FPackagePath& PackagePath, FPackagePath* OutUpdatedPath = nullptr);

	/**
	 * Report the payload size of the package
	 *
	 * @param PackagePath The package to look for
	 * @param OutUpdatedPath If non-null and the package is found, PackagePath is copied into this path (noop if
	 *        OutUpdatedPath == &PackagePath), and if the package exists, the specific extension found is set
	 * @return The size in bytes of the package, or INDEX_NONE if it isn't found
	 */
	COREUOBJECT_API int64 FileSize(const FPackagePath& PackagePath,	FPackagePath* OutUpdatedPath = nullptr);

	/**
	 * Open an FArchive to read the bytes of the package
	 *
	 * @param PackagePath The package to look for
	 * @param OutUpdatedPath If non-null and the package is found, PackagePath is copied into this path (noop if
	 *        OutUpdatedPath == &PackagePath), and if the package exists, the specific extension found is set
	 * @return An FOpenPackageResult, with Result.Archive == archive for the bytes of the package or nullptr
	 *         if it isn't found, and with other data describing the returned archive (see FOpenPackageResult)
	 */
	COREUOBJECT_API FOpenPackageResult OpenReadPackage(const FPackagePath& PackagePath, FPackagePath* OutUpdatedPath = nullptr);

	/**
	 * Open an IAsyncReadFileHandle to asynchronously read the bytes of the package
	 *
	 * If the PackagePath specifies the extension, this call does not hit the disk/network or block.
	 * Otherwise, this call will read from network/disk to find the extension
	 * This call will always return a non-null handle, even if the package does not exist
	 *
	 * @param PackagePath The package to look for
	 * @return An FOpenAsyncPackageResult, with Handle that will read from the package if it exists,
	 *         or will be in the canceled state if the package does not exist, and with other data
	 *         describing the returned archive (see FOpenAsyncPackageResult)
	 */
	COREUOBJECT_API FOpenAsyncPackageResult OpenAsyncReadPackage(const FPackagePath& PackagePath);

	/**
	 * Open an IMappedFileHandle to the package, if the PackageResourceManager supports it
	 *
	 * Will return nullptr if the package does not exist or if the PackageResourceManager
	 * does not support MappedFilesHandles
	 *
	 * @param PackagePath The package to look for
	 * @param OutUpdatedPath If non-null and the package is found, PackagePath is copied into this path (noop if
	 *        OutUpdatedPath == &PackagePath), and if the package exists, the specific extension found is set
	 * @return IMappedFileHandle* or nullptr
	 */
	COREUOBJECT_API IMappedFileHandle* OpenMappedHandleToPackage(const FPackagePath& PackagePath, FPackagePath* OutUpdatedPath = nullptr);

	/**
	 * Find the package in the package repository and set OutNormalizedPath equal to PackagePath,
	 * but with capitalization of the PackageName and LocalPath matching the capitalization present in the package
	 * repository's internal path string (e.g. the path on disk)
	 *
	 * @param PackagePath The package to look for
	 * @param OutNormalizedPath If non-null and the package is found, PackagePath is copied into this path (noop if
	 *        OutUpdatedPath == &PackagePath), with matching capitalization and with the specific extension found
	 * @return true if the package exists, else false
	 */
	virtual bool TryMatchCaseOnDisk(const FPackagePath& PackagePath, FPackagePath* OutNormalizedPath = nullptr) = 0;

	/**
	 * Open a seekable binary FArchive to read the bytes of the given External Resource.
	 * An ExternalResource is in a separate domain from the one out of which this ResourceManager serves PackagePaths.
	 *
	 * @param ResourceType Id for the method used to map the identifier to an archive.
	 * @param Identifier Id for which resource to return within the ResourceType's domain.
	 * @return The opened FArchive, or nullptr if it wasn't found.
	 */
	virtual TUniquePtr<FArchive> OpenReadExternalResource(EPackageExternalResource ResourceType, FStringView Identifier) = 0;

	/**
	 * Open an IAsyncReadFileHandle to asynchronously read the bytes of the given ExternalResource.
	 * An ExternalResource is in a separate domain from the one out of which this ResourceManager serves PackagePaths.
	 *
	 * TODO: This call should not hit the disk/network or block, but it currently does, to find the extension.
	 * This call will always return a non-null handle, even if the package does not exist
	 *
	 * @param ResourceType Id for the method used to map the identifier to an archive.
	 * @param Identifier Id for which resource to return within the ResourceType's domain.
	 * @return An FOpenAsyncPackageResult, with Handle that will read from the package if it exists,
	 *         or will be in the canceled state if the package does not exist, and with other data
	 *         describing the returned archive (see FOpenAsyncPackageResult)
	 */
	virtual FOpenAsyncPackageResult OpenAsyncReadExternalResource(
		EPackageExternalResource ResourceType, FStringView Identifier) = 0;

	/**
	 * Report whether a given ExternalResource exists. The same behavior as OpenReadExternalResource != nullptr,
	 * but more performant.
	 * 
	 * @param ResourceType Id for the method used to map the identifier to an archive.
	 * @param Identifier Id for which resource to return within the ResourceType's domain.
	 * @return true if the ExternalResource exists, else false
	 */
	virtual bool DoesExternalResourceExist(EPackageExternalResource ResourceType, FStringView Identifier) = 0;

	/**
	 * Search the given subdirectory of a package mount for all packages with the given package basename in the
	 * package repository
	 *
	 * @param OutPackages The list of packages found
	 * @param PackageMount The package name of the mount point to look under
	 * @param FileMount The local file path of the mount point to look under, must be the local file path that
	 *        corresponds to PackageMount
	 * @param RootRelPath A relative path from PackageMount that specifies the subdirectory of the mount point to use
	 *        as the root of the search
	 * @param BasenameWildcard The basenamewithoutpath to look for
	 * BasenameWildcard can not include path or extension; / and . are invalid characters
	 * BasenameWildcard can include *?-type wildcards (FString::MatchesWildcard)
	 */
	COREUOBJECT_API void FindPackagesRecursive(TArray<FPackagePath>& OutPackages, FStringView PackageMount,
		FStringView FileMount, FStringView RootRelPath, FStringView BasenameWildcard);

	/** Return value specifies whether iteration should continue */
	typedef TFunctionRef<bool(const FPackagePath& PackagePath)> FPackagePathVisitor;
	/** Return value specifies whether iteration should continue */
	typedef TFunctionRef<bool(const FPackagePath& PackagePath, const FFileStatData& StatData)> FPackagePathStatVisitor;

	/**
	 * Call the callback on all packages in the package repository that are in the given subdirectory of a package mount
	 *
	 * @param PackageMount The package name of the mount point's to look under
	 * @param FileMount The local file path of the mount point to look under,
	 *        must be the local file path that corresponds to PackageMount
	 * @param RootRelPath A relative path from PackageMount that specifies the subdirectory of the mount point to use
	 *        as the root of the search
	 * @param Callback The callback called on each package
	 */
	COREUOBJECT_API void IteratePackagesInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
		FPackagePathVisitor Callback);
	
	/**
	 * Call the callback on all packages in the given local path
	 *
	 * PackageResourceManagers that do not support LocalOnly paths will return without calling the Callback
	 *
	 * @param RootDir The local path on disk to search
	 * @param Callback The callback called on each package
	 */
	COREUOBJECT_API void IteratePackagesInLocalOnlyDirectory(FStringView RootDir, FPackagePathVisitor Callback);

	/**
	 * Call the callback - with stat data - on all packages in the package repository that are in the given
	 * subdirectory of a package mount
	 *
	 * @param PackageMount The package name of the mount point's to look under
	 * @param FileMount The local file path of the mount point to look under, must be the local file path that
	 *        corresponds to PackageMount
	 * @param RootRelPath A relative path from PackageMount that specifies the subdirectory of the mount point to use
	 *        as the root of the search
	 * @param Callback The callback called on each package
	 */
	COREUOBJECT_API void IteratePackagesStatInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
		FPackagePathStatVisitor Callback);

	/**
	 * Call the callback - with stat data - on all packages in the given local path
	 *
	 * PackageResourceManagers that do not support LocalOnly paths will return without calling the Callback
	 *
	 * @param RootDir The local path on disk to search
	 * @param Callback The callback called on each package
	 * @param bOutSupported If nonnull, receives a true or false value for whether this PackageResourceManager supports
	 *        searching LocalOnlyDirectories
	 */
	COREUOBJECT_API void IteratePackagesStatInLocalOnlyDirectory(FStringView RootDir, FPackagePathStatVisitor Callback);

	/**
	 * Call during engine startup to choose the appropriate PackageResourceManager for the current configuration and
	 * construct it and bind it to be returned from Get
	 */
	COREUOBJECT_API static void Initialize();

	/**
	 * Call during engine shutdown to free the PackageResourceManager that was created by Initialize; Get will 
	 * and return nullptr from this point on
	 */
	COREUOBJECT_API static void Shutdown();

	COREUOBJECT_API static FSetPackageResourceManager& GetSetPackageResourceManagerDelegate();
	COREUOBJECT_API static FOnClearPackageResourceManager& GetOnClearPackageResourceManagerDelegate();

public:
	/** DoesPackageExist that takes a PackageSegment */
	virtual bool DoesPackageExist(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) = 0;

	/** FileSize that takes a PackageSegment */
	virtual int64 FileSize(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) = 0;

	/** OpenReadPackage that takes a PackageSegment */
	virtual FOpenPackageResult OpenReadPackage(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex,
		EPackageSegment PackageSegment, FPackagePath* OutUpdatedPath = nullptr) = 0;

	/** OpenAsyncReadPackage that takes a PackageSegment */
	virtual FOpenAsyncPackageResult OpenAsyncReadPackage(const FPackagePath& PackagePath, FBulkDataCookedIndex CookedIndex, EPackageSegment PackageSegment) = 0;

	/* OpenMappedHandleToPackage that takes a PackageSegment */
	virtual IMappedFileHandle* OpenMappedHandleToPackage(const FPackagePath& PackagePath, EPackageSegment PackageSegment,
		FPackagePath* OutUpdatedPath = nullptr) = 0;

	/** FindPackagesRecursive that returns PackageSegments */
	virtual void FindPackagesRecursive(TArray<TPair<FPackagePath, EPackageSegment>>& OutPackages, FStringView PackageMount,
		FStringView FileMount, FStringView RootRelPath, FStringView BasenameWildcard) = 0;

	/** Return value specifies whether iteration should continue */
	typedef TFunctionRef<bool(const FPackagePath& PackagePath, EPackageSegment PackageSegment)> FPackageSegmentVisitor;
	/** Return value specifies whether iteration should continue */
	typedef TFunctionRef<bool(const FPackagePath& PackagePath, EPackageSegment PackageSegment, const FFileStatData& StatData)> FPackageSegmentStatVisitor;

	/** IteratePackagesInPath that takes a FPackageSegmentVisitor */
	virtual void IteratePackagesInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
		FPackageSegmentVisitor Callback) = 0;

	/** IteratePackagesInLocalOnlyDirectory that takes a FPackageSegmentStatVisitor */
	virtual void IteratePackagesInLocalOnlyDirectory(FStringView RootDir, FPackageSegmentVisitor Callback) = 0;

	/** IteratePackagesStatInPath that takes a FPackageSegmentVisitor */
	virtual void IteratePackagesStatInPath(FStringView PackageMount, FStringView FileMount, FStringView RootRelPath,
		FPackageSegmentStatVisitor Callback) = 0;

	/** IteratePackagesStatInLocalOnlyDirectory that takes a FPackageSegmentStatVisitor */
	virtual void IteratePackagesStatInLocalOnlyDirectory(FStringView RootDir, FPackageSegmentStatVisitor Callback) = 0;


	// Preloading Package Archives
#if WITH_EDITOR
	/**
	 * Try to register the given FPreloadableFile instance to handle the next call to TryTakePreloadableArchive for its PackagePath
	 *
	 * Only works on PackagePaths with PackageName, will fail on unmounted LocalOnly paths.
	 * Will fail if the instance has not been initialized or if another instance has already registered for the PackagePath.
	 * Return whether the instance is currently registered. Returns true if the instance was already registered.
	 * Registered files are referenced-counted, and the reference will not be dropped until
	 * TryTakePreloadableArchive or UnRegister is called
	 *
	 * @param PackagePath The path to registry to return the Archive
	 * @param PreloadableArchive The archive to return from TryTakePreloadableArchive
	 * @param PackageFormat The format of the archive, as returned from OpenReadPackage
	 * @return True if and only if the archive is now registered for the packagepath, False if the packagepath could
	 *         not be registered or a different archive was already registered
	 */
	COREUOBJECT_API static bool TryRegisterPreloadableArchive(const FPackagePath& PackagePath,
		const TSharedPtr<FPreloadableArchive>& PreloadableArchive, const FOpenPackageResult& PackageFormat);

	/**
	 * Look up an FPreloadableFile instance registered for the given PackagePath, and return an FOpenPackageResult from it
	 *
	 * If found, removes the registration so no future call to TryTakePreloadableArchive can use the same FArchive.
	 * If the instance is in PreloadHandle mode, the Lower-Level FArchive will be detached from the FPreloadableFile
	 * and returned using DetachLowerLevel.
	 * If the instance is in PreloadBytes mode, a ProxyArchive will be returned that forwards call to the FPreloadableFile
	 * instance.
	 */
	COREUOBJECT_API static bool TryTakePreloadableArchive(const FPackagePath& PackagePath, FOpenPackageResult& OutResult);
	
	/**
	 * Remove any FPreloadableArchive instance that is registered for the given PackagePath.
	 *
	 * @return True if there was an archive registered for the PackagePath
	 */
	COREUOBJECT_API static bool UnRegisterPreloadableArchive(const FPackagePath& PackagePath);

private:
	static TMap<FName, TPair<TSharedPtr<FPreloadableArchive>, FOpenPackageResult>> PreloadedPaths;
	static FCriticalSection PreloadedPathsLock;
#endif
};

#if WITH_EDITOR
enum class EEditorDomainEnabled : uint8
{
	Disabled,
	Utilities,
	PackageResourceManager,
};
/**
 * Report whether the EditorDomain is enabled by config, for systems that need to behave differently if editordomain
 * will be enabled and need to know before the PackageResourceManager is constructed.
 */
COREUOBJECT_API EEditorDomainEnabled IsEditorDomainEnabled();
#endif
