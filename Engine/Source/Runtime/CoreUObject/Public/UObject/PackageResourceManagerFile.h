// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IPackageResourceManager;

/**
 * Create an FPackageResourceManagerFile, which implements the IPackageResourceManager interface by reading from the local files on disk.
 * This is the most natural implementation, but does not support e.g. caching of upgrade steps or more efficient formats.
 * FPackageResourceManagerFile is kept private for implentation hiding, and this function is the only public way to access it.
 */
COREUOBJECT_API IPackageResourceManager* MakePackageResourceManagerFile();
