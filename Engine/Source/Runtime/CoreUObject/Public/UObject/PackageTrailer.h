// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compression/CompressedBuffer.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "IO/IoHash.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "Virtualization/VirtualizationTypes.h"

class FArchive;
class FLinkerSave;
class FPackagePath;

namespace UE
{

/** Trailer Format
 * The FPackageTrailer is a container that will commonly be appended to the end of a package file. The main purpose of the trailer is to store
 * the bulkdata payloads contained by the package until they are virtualized or moved to an additional storage location.
 * 
 * By storing the payloads in a data format adjacent to the rest of the package we can perform the virtualization process without needing to 
 * re-save the package itself which in turn should allow for external tools to be able to perform the virtualization process themselves
 * rather than needing to force it through engine code.
 * 
 * The package trailer is intended to an easy format for external code/script to be able to manipulate. To make things clearer we do not 
 * serialize containers directly but write out each data structure one at a time so that it should be easy to see how to manipulate the file.
 * 
 * The file is split into three parts:
 * 
 * [Header]
 * The header contains the useful info about the trailer and the payloads in general. @See UE::FLookupTableEntry for details about 
 * the look up table's data.
 * 
 * [Payload Data]
 * If the trailer is in the workspace domain package then we will store all non-virtualized payloads here. If the trailer is in the editor 
 * domain then there will be no payload data section and the header will be referencing the trailer in the workspace domain instead.
 * 
 * [Footer]
 * The footer allows for us to load the trailer in reverse and replicates the end of package file tag (PACKAGE_FILE_TAG), it should only be
 * used for finding the start of the trailer or validation.
 * 
 * CurrentVersion UE::EPackageTrailerVersion::INITIAL
 * ______________________________________________________________________________________________________________________________________________
 * | [Header]																																	|
 * | Tag				| uint64			| Should match FHeader::HeaderTag, used to identify that the data being read is an FPackageTrailer	|
 * | Version			| uint32			| Version number of the format@see UE::EPackageTrailerVersion										|
 * | HeaderLength		| uint32			| The total size of the header on disk in bytes.													|
 * | PayloadsDataLength	| uint64			| The total size of the payload data on disk in bytes												|
 * | NumPayloads		| int32				| The number of payloads in LookupTableArray														|
 * | LookupTableArray	| FLookupTableEntry | An array of FLookupTableEntry @see UE::Private::FLookupTableEntry									|
 * |____________________________________________________________________________________________________________________________________________|
 * | [Payload Data]																																|
 * | Array				| FCompressedBuffer | A binary blob containing all of the payloads. Individual payloads can be found via				|
 * |										 the LookupTableArray found in the header.															|
 * |____________________________________________________________________________________________________________________________________________|
 * | [Footer]																																	|
 * | Tag				| uint64			| Should match FFooter::FooterTag, used to identify that the data being read is an FPackageTrailer	|
 * | TrailerLength		| uint64			| The total size of the trailer on disk in bytes. Can be used to find the start of the trailer when	|
 * |										  reading backwards.																				|
 * | PackageTag			| uint32			| The end of package tag, PACKAGE_FILE_TAG. This is used to validate that a package file on disk is	|
 * |										  not corrupt. By ending the trailer with this tag we allow that validation code to work.			|
 * |____________________________________________________________________________________________________________________________________________|
 */

 /** Used to filter requests based on how a payload is stored*/
enum class EPayloadStorageType : uint8
{
	/** All payload regardless of type. */
	Any,
	/** All payloads stored locally in the package trailer. */
	Local,
	/** All payloads that are a reference to payloads stored in the workspace domain trailer*/
	Referenced,
	/** All payloads stored in a virtualized backend. */
	Virtualized
};

/** Used to filter requests based on how a payload can function */
enum class EPayloadFilter
{
	/** All payloads that are stored locally and do not have virtualization disabled */
	CanVirtualize
};

// TODO: Could consider merging EPayloadStorageType and EPayloadStatus, only difference is Any vs NotFound

/** Used to show the status of a payload */
enum class EPayloadStatus
{
	/** The payload is not registered in the package trailer */
	NotFound = 0,
	/** The payload is stored locally inside the current package trailer where ever that is written to disk */
	StoredLocally,
	/** The payload is stored in the workspace domain trailer */
	StoredAsReference,
	/** The payload is virtualized and needs to be accessed via the IVirtualizationSystem */
	StoredVirtualized,
};

/** Lists the various methods of payload access that the trailer supports */
enum class EPayloadAccessMode : uint8
{
	/** The payload is stored in the Payload Data segment of the trailer and the offsets in FLookupTableEntry will be relative to the start of this segment */
	Local = 0,
	/** The payload is stored in another package trailer (most likely the workspace domain package file) and the offsets in FLookupTableEntry are absolute offsets in that external file */
	Referenced,
	/** The payload is virtualized and needs to be accessed via IVirtualizationSystem */
	Virtualized
};

/** Flags that can be set on payloads in a payload trailer */
enum class EPayloadFlags : uint16
{
	/** No flags are set */
	None = 0,
};

ENUM_CLASS_FLAGS(EPayloadFlags);

enum class EPackageTrailerVersion : uint32;

namespace Private
{

struct FLookupTableEntry
{
	/** Size of the entry when serialized to disk in bytes */
	static constexpr uint32 SizeOnDisk = 49;	// Identifier		| 20 bytes
												// OffsetInFile		| 8 bytes
												// CompressedSize	| 8 bytes
												// RawSize			| 8 bytes
												// Flags			| 4 bytes
												// AccessMode		| 1 byte

	FLookupTableEntry() = default;
	FLookupTableEntry(const FIoHash& InIdentifier, uint64 InRawSize);

	void Serialize(FArchive& Ar, EPackageTrailerVersion PackageTrailerVersion);
	
	[[nodiscard]] bool IsLocal() const
	{
		return AccessMode == EPayloadAccessMode::Local;
	}

	[[nodiscard]] bool IsReferenced() const
	{
		return AccessMode == EPayloadAccessMode::Referenced;
	}

	[[nodiscard]] bool IsVirtualized() const
	{
		return AccessMode == EPayloadAccessMode::Virtualized;
	}

	/** Identifier for the payload */
	FIoHash Identifier;
	/** The offset into the file where we can find the payload, note that a virtualized payload will have an offset of INDEX_NONE */
	int64 OffsetInFile = INDEX_NONE;
	/** The size of the payload when compressed. This will be the same value as RawSize if the payload is not compressed */
	uint64 CompressedSize = INDEX_NONE;
	/** The size of the payload when uncompressed. */
	uint64 RawSize = INDEX_NONE;
	/** Bitfield of flags, see @UE::EPayloadFlags */
	EPayloadFlags Flags = EPayloadFlags::None;
	/** Bitfield of flags showing if the payload allowed to be virtualized or the reason why it cannot be virtualized, see @UE::EPayloadFilterReason */
	Virtualization::EPayloadFilterReason FilterFlags = Virtualization::EPayloadFilterReason::None;

	EPayloadAccessMode AccessMode = EPayloadAccessMode::Local;
};

} // namespace Private

//** Info about a payload stored in the trailer
struct FPayloadInfo
{
	int64 OffsetInFile = INDEX_NONE;

	uint64 CompressedSize = INDEX_NONE;
	uint64 RawSize = INDEX_NONE;

	EPayloadAccessMode AccessMode = EPayloadAccessMode::Local;
	EPayloadFlags Flags = EPayloadFlags::None;
	Virtualization::EPayloadFilterReason FilterFlags = Virtualization::EPayloadFilterReason::None;
};

/** 
 * This class is used to build a FPackageTrailer and write it disk.
 * 
 * While saving a package, payloads should be added to a FPackageTrailer via ::AddPayload then once
 * the package has been saved to disk ::BuildAndAppendTrailer should be called. 
 */
class FPackageTrailerBuilder
{
public:
	using AdditionalDataCallback = TFunction<void(FLinkerSave& LinkerSave, const class FPackageTrailer& Trailer)>;

	/**
	 * Creates a builder from a pre-existing FPackageTrailer.
	 * Payloads stored locally in the source trailer will be loaded from disk via the provided archive so that the
	 * builder can write them to any future trailer that it creates.
	 * 
	 * @param Trailer		The trailer to create the builder from
	 * @param Ar			An archive that the trailer can use to load payloads from 
	 * @param DebugContext	The name or path of the of the file that owns the trailer. Used for error messages.
	 */
	[[nodiscard]] static COREUOBJECT_API FPackageTrailerBuilder CreateFromTrailer(const class FPackageTrailer& Trailer, FArchive& Ar, FString DebugContext);

	/**
	 * Creates a builder from a pre-existing FPackageTrailer that will will reference the local payloads of the
	 * source trailer. 
	 * This means that there is no need to load the payloads.
	 *
	 * @param Trailer		The trailer to create the reference from.
	 * @param DebugContext	The name or path of the of the file that owns the trailer. Used for error messages.
	 */
	[[nodiscard]] static COREUOBJECT_API TUniquePtr<UE::FPackageTrailerBuilder> CreateReferenceToTrailer(const class FPackageTrailer& Trailer, FString DebugContext);
	
	FPackageTrailerBuilder() = default;
	COREUOBJECT_API FPackageTrailerBuilder(FString&& DebugContext);
	~FPackageTrailerBuilder() = default;

	// Methods that can be called while building the trailer

	/**
	 * Adds a payload to the builder to be written to the trailer. Duplicate payloads will be discarded and only a 
	 * single instance stored in the trailer.
	 * 
	 * @param Identifier	The identifier of the payload
	 * @param Payload		The payload data
	 * @param Flags			The custom flags to be applied to the payload
	 * @param Callback		This callback will be invoked once the FPackageTrailer has been built and appended to disk.
	 */
	COREUOBJECT_API void AddPayload(const FIoHash& Identifier, FCompressedBuffer Payload, UE::Virtualization::EPayloadFilterReason Filter, AdditionalDataCallback&& Callback);
	COREUOBJECT_API void AddPayload(const FIoHash& Identifier, FCompressedBuffer Payload, UE::Virtualization::EPayloadFilterReason Filter);
	/**
	 * Adds an already virtualized payload to the builder to be written to the trailer. When the trailer is written
	 * the payload will have EPayloadAccessMode::Virtualized set as it's access mode. It is assumed that the payload
	 * is already stored in the virtualization backends and it is up to the calling code to confirm this.
	 * Duplicate payloads will be discarded and only a single instance stored in the trailer.
	 * 
	 * @param Identifier	The identifier of the payload
	 * @param RawSize		The size of the payload (in bytes) when uncompressed
	 */
	COREUOBJECT_API void AddVirtualizedPayload(const FIoHash& Identifier, int64 RawSize);

	/** 
	 * Allows the caller to replace a payload in the builder that is already marked as virtualized and replace it
	 * with one that will be stored locally.
	 * 
	 * @param Identifier	The identifier of the payload
	 * @param Payload		The content of the payload
	 * 
	 * @return true if a virtualized payload was replaced, false if the payload was not in the builder at all
	 */
	COREUOBJECT_API bool UpdatePayloadAsLocal(const FIoHash& Identifier, FCompressedBuffer Payload);
	
	/**
	 * @param ExportsArchive	The linker associated with the package being written to disk.
	 * @param DataArchive		The archive where the package data has been written to. This is where the FPackageTrailer will be written to
	 * @return True if the builder was created and appended successfully and false if any error was encountered
	 */
	[[nodiscard]] COREUOBJECT_API bool BuildAndAppendTrailer(FLinkerSave* Linker, FArchive& DataArchive);

	/**
	 * @param ExportsArchive		The linker associated with the package being written to disk.
	 * @param DataArchive			The archive where the package data has been written to. This is where the FPackageTrailer will be written to
	 * @param InOutArchiveOffset	The offset at which the trailer is written in the archive. In this function version, DataArchive might
	 *								be an archive for the entire archive, or it might be a separate archive solely for the package trailer.
	 *								The caller must provide the initial value with the accumulated offset, and the function will increment this
	 *								value by how many bytes are written to DataArchive.
	 * @return True if the builder was created and appended successfully and false if any error was encountered
	 */
	[[nodiscard]] COREUOBJECT_API bool BuildAndAppendTrailer(FLinkerSave* Linker, FArchive& DataArchive, int64& InOutArchiveOffset);

	/** Returns if the builder has any payload entries or not */
	[[nodiscard]] COREUOBJECT_API bool IsEmpty() const;

	[[nodiscard]] COREUOBJECT_API bool IsLocalPayloadEntry(const FIoHash& Identifier) const;
	[[nodiscard]] COREUOBJECT_API bool IsReferencedPayloadEntry(const FIoHash& Identifier) const;
	[[nodiscard]] COREUOBJECT_API bool IsVirtualizedPayloadEntry(const FIoHash& Identifier) const;

	/** 
	 * Returns the length of the trailer (in bytes) that the builder would currently create.
	 * 
	 * NOTE: At the moment this is not const as we need to check for and remove duplicate
	 * payload entries as we do this before building the trailer not when gathering the entry info.
	 */
	[[nodiscard]] COREUOBJECT_API uint64 CalculateTrailerLength();

	/** Returns the total number of payload entries in the builder */
	[[nodiscard]] COREUOBJECT_API int32 GetNumPayloads() const;
	
	/** Returns the number of payload entries in the builder with the access mode EPayloadAccessMode::Local */
	[[nodiscard]] COREUOBJECT_API int32 GetNumLocalPayloads() const;
	/** Returns the number of payload entries in the builder with the access mode EPayloadAccessMode::Referenced */
	[[nodiscard]] COREUOBJECT_API int32 GetNumReferencedPayloads() const;
	/** Returns the number of payload entries in the builder with the access mode EPayloadAccessMode::Virtualized */
	[[nodiscard]] COREUOBJECT_API int32 GetNumVirtualizedPayloads() const;

	/** Returns the debug context associated with the builder, used for adding further description to error messages */
	[[nodiscard]] const FString& GetDebugContext() const
	{
		return DebugContext;
	}

private:
	
	/** All of the data required to add a payload that is stored locally within the trailer */
	struct LocalEntry
	{
		LocalEntry() = default;
		LocalEntry(FCompressedBuffer&& InPayload, Virtualization::EPayloadFilterReason InFilterFlags)
			: Payload(InPayload)
			, FilterFlags(InFilterFlags)
		{

		}
		~LocalEntry() = default;

		FCompressedBuffer Payload;
		Virtualization::EPayloadFilterReason FilterFlags = Virtualization::EPayloadFilterReason::None;
	};

	/** All of the data required to add a reference to a payload stored in another trailer */
	struct ReferencedEntry
	{
		ReferencedEntry() = default;
		ReferencedEntry(int64 InOffset, int64 InCompressedSize, int64 InRawSize)
			: Offset(InOffset)
			, CompressedSize(InCompressedSize)
			, RawSize(InRawSize)
		{

		}
		~ReferencedEntry() = default;

		int64 Offset = INDEX_NONE;
		int64 CompressedSize = INDEX_NONE;
		int64 RawSize = INDEX_NONE;
	};


	/** All of the data required to add a payload that is virtualized */
	struct VirtualizedEntry
	{
		VirtualizedEntry() = default;
		VirtualizedEntry(int64 InRawSize)
			: RawSize(InRawSize)
		{

		}
		~VirtualizedEntry() = default;

		int64 RawSize = INDEX_NONE;
	};

	/** Returns the total length of the header if we were to build a trailer right now */
	COREUOBJECT_API uint32 CalculatePotentialHeaderSize() const;
	/** Returns the total length of all payloads combined if we were to build a trailer right now */
	COREUOBJECT_API uint64 CalculatePotentialPayloadSize() const;

	COREUOBJECT_API void RemoveDuplicateEntries();

	// Members used when building the trailer

	/** Context used when giving error messages so that the user can identify the cause of problems */
	FString DebugContext;

	/** Payloads that will be stored locally when the trailer is written to disk */
	TMap<FIoHash, LocalEntry> LocalEntries;
	/** Payloads that reference entries in another trailer */
	TMap<FIoHash, ReferencedEntry> ReferencedEntries;
	/** Payloads that are already virtualized and so will not be written to disk */
	TMap<FIoHash, VirtualizedEntry> VirtualizedEntries;
	/** Callbacks to invoke once the trailer has been written to the end of a package */
	TArray<AdditionalDataCallback> Callbacks;
};

/** 
 *
 * The package trailer should only ever stored the payloads in the workspace domain. If the package trailer is in the editor
 * domain then it's values should be valid, but when loading non-virtualized payloads they need to come from the workspace 
 * domain package.
 */
class FPackageTrailer
{
public:
	/** Try to load a trailer from a given package path. Note that it will always try to load the trailer from the workspace domain */
	[[nodiscard]] static COREUOBJECT_API bool TryLoadFromPackage(const FPackagePath& PackagePath, FPackageTrailer& OutTrailer);

	/** Try to load a trailer from a given file path. */
	[[nodiscard]] static COREUOBJECT_API bool TryLoadFromFile(const FString& Path, FPackageTrailer& OutTrailer);

	/** Try to load a trailer from a given archive. Assumes that the trailer is at the end of the archive */
	[[nodiscard]] static COREUOBJECT_API bool TryLoadFromArchive(FArchive& Ar, FPackageTrailer& OutTrailer);

	FPackageTrailer() = default;
	~FPackageTrailer() = default;

	FPackageTrailer(const FPackageTrailer& Other) = default;
	FPackageTrailer& operator=(const FPackageTrailer & Other) = default;

	FPackageTrailer(FPackageTrailer&& Other) = default;
	FPackageTrailer& operator=(FPackageTrailer&& Other) = default;

	/** 
	 * Returns true if the trailer contains actual data from a package file and false if it just contains the defaults of an unloaded
	 * trailer.
	 */
	[[nodiscard]] bool IsValid() const
	{
		return Header.Tag == FHeader::HeaderTag;
	}

	/** 
	 * Serializes the trailer from the given archive assuming that the seek position of the archive is already at the correct position
	 * for the trailer.
	 * 
	 * @param Ar	The archive to load the trailer from
	 * @return		True if a valid trailer was found and was able to be loaded, otherwise false. If the trailer was found but failed
	 *				to load then the archive will be set to the error state.
	 */
	[[nodiscard]] COREUOBJECT_API bool TryLoad(FArchive& Ar);

	/** 
	 * Serializes the trailer from the given archive BUT assumes that the seek position of the archive is at the end of the trailer
	 * and so will attempt to read the footer first and use that to find the start of the trailer in order to read the header.
	 * 
	 * @param Ar	The archive to load the trailer from
	 * @return		True if a valid trailer was found and was able to be loaded, otherwise false. If the trailer was found but failed
	 *				to load then the archive will be set to the error state.
	 */
	[[nodiscard]] COREUOBJECT_API bool TryLoadBackwards(FArchive& Ar);

	/** 
	 * Loads a payload that is stored locally within the package trailer. Payloads stored externally (either referenced
	 * or virtualized) will not load.
	 * 
	 * @param Id The payload to load
	 * @param Ar The archive from which the payload trailer was also loaded from
	 * 
	 * @return	The payload in the form of a FCompressedBuffer. If the payload does not exist in the trailer or is not
	 *			stored locally in the trailer then the FCompressedBuffer will be null.
	 */
	[[nodiscard]] COREUOBJECT_API FCompressedBuffer LoadLocalPayload(const FIoHash& Id, FArchive& Ar) const;

	/** 
	 * Calling this indicates that the payload has been virtualized and will no longer be stored on disk. 
	 * 
	 * @param Identifier The payload that has been virtualized
	 * @return True if the payload was in the trailer, otherwise false
	 */
	[[nodiscard]] COREUOBJECT_API bool UpdatePayloadAsVirtualized(const FIoHash& Identifier);

	/**
	 * Iterates over all payloads in the trailer and invokes the provoided callback on them
	 */
	COREUOBJECT_API void ForEachPayload(TFunctionRef<void(const FIoHash&, uint64, uint64, EPayloadAccessMode, UE::Virtualization::EPayloadFilterReason)> Callback) const;

	/** Attempt to find the status of the given payload. @See EPayloadStatus */
	[[nodiscard]] COREUOBJECT_API EPayloadStatus FindPayloadStatus(const FIoHash& Id) const;

	/** Returns the absolute offset of the payload in the package file, invalid and virtualized payloads will return INDEX_NONE */
	[[nodiscard]] COREUOBJECT_API int64 FindPayloadOffsetInFile(const FIoHash& Id) const;

	/** Returns the size of the payload on as stored on disk, invalid and virtualized payloads will return INDEX_NONE */
	[[nodiscard]] COREUOBJECT_API int64 FindPayloadSizeOnDisk(const FIoHash& Id) const;

	/** Returns the total size of the of the trailer on disk in bytes */
	[[nodiscard]] COREUOBJECT_API int64 GetTrailerLength() const;

	[[nodiscard]] COREUOBJECT_API FPayloadInfo GetPayloadInfo(const FIoHash& Id) const;

	/** Returns an array of the payloads with the given storage type. @See EPayloadStoragetype */
	[[nodiscard]] COREUOBJECT_API TArray<FIoHash> GetPayloads(EPayloadStorageType StorageType) const;

	/** Returns the number of payloads that the trailer owns with the given storage type. @See EPayloadStoragetype */
	[[nodiscard]] COREUOBJECT_API int32 GetNumPayloads(EPayloadStorageType Type) const;

	/** Returns an array of the payloads that match the given filter type. @See EPayloadFilter */
	[[nodiscard]] COREUOBJECT_API TArray<FIoHash> GetPayloads(EPayloadFilter Filter) const;

	/** Returns the number of payloads that the trailer owns that match the given filter type. @See EPayloadFilter */
	[[nodiscard]] COREUOBJECT_API int32 GetNumPayloads(EPayloadFilter Filter) const;

	/** Returns the offset (in bytes) of the package trailer in the archive that it was loaded from or written too. */
	[[nodiscard]] int64 GetPackageTrailerOffset() const
	{
		return PackageTrailerOffset;
	}

	struct FHeader
	{
		/** Unique value used to identify the header */
		static constexpr uint64 HeaderTag = 0xD1C43B2E80A5F697;

		/** 
		 * Size of the static header data when serialized to disk in bytes. Note that we still need to 
		 * add the size of the data in PayloadLookupTable to get the final header size on disk 
		 */
		static constexpr uint32 StaticHeaderSizeOnDisk = 28;	// HeaderTag			| 8 bytes
																// Version				| 4 bytes
																// HeaderLength			| 4 bytes
																// PayloadsDataLength	| 8 bytes
																// NumPayloads			| 4 bytes

		/** Expected tag at the start of the header */
		uint64 Tag = 0;
		/** Version of the header */
		int32 Version = INDEX_NONE;
		/** Total length of the header on disk in bytes */
		uint32 HeaderLength = 0;
		/** Total length of the payloads on disk in bytes */
		uint64 PayloadsDataLength = 0;
		/** Lookup table for the payloads on disk */
		TArray<Private::FLookupTableEntry> PayloadLookupTable;

		/** Serialization operator */
		friend FArchive& operator<<(FArchive& Ar, FHeader& Header);
	};

	struct FFooter
	{
		/** Unique value used to identify the footer */
		static constexpr uint64 FooterTag = 0x29BFCA045138DE76;

		/** Size of the footer when serialized to disk in bytes */
		static constexpr uint32 SizeOnDisk = 20;	// Tag				| 8 bytes
													// TrailerLength	| 8 bytes
													// PackageTag		| 4 bytes

		/** Expected tag at the start of the footer */
		uint64 Tag = 0;
		/** Total length of the trailer on disk in bytes */
		uint64 TrailerLength = 0;	
		/** End the trailer with PACKAGE_FILE_TAG, which we expect all package files to end with */
		uint32 PackageTag = 0;

		/** Serialization operator */
		friend FArchive& operator<<(FArchive& Ar, FFooter& Footer);
	};

private:
	friend class FPackageTrailerBuilder;

	/** Create a valid footer for the current trailer */
	COREUOBJECT_API FFooter CreateFooter() const;

	/** Offset into the archive where the FPackageTrailer is located */
	int64 PackageTrailerOffset = INDEX_NONE;

	/** 
	 * The header of the trailer. Since this contains the lookup table for payloads we keep this in memory once the trailer
	 * has been loaded. There is no need to keep the footer in memory
	 */
	FHeader Header;
};

/**
 * Used to find the identifiers of the payload in a given package. Note that this will return the payloads included in the
 * package on disk and will not take into account any edits to the package if they are in memory and unsaved.
 *
 * @param PackagePath	The package to look in.
 * @param Filter		What sort of payloads should be returned. @see EPayloadType
 * @param OutPayloadIds	This array will be filled with the FPayloadId values found in the package that passed the filter.
 *						Note that existing content in the array will be preserved. It is up to the caller to empty it.
 *
 * @return 				True if the package was parsed successfully (although it might not have contained any payloads) and false if opening or parsing the package file failed.
 */
[[nodiscard]] COREUOBJECT_API bool FindPayloadsInPackageFile(const FPackagePath& PackagePath, EPayloadStorageType Filter, TArray<FIoHash>& OutPayloadIds);

/** Allow EPayloadAccessMode to work with string builder */
template <typename CharType>
inline TStringBuilderBase<CharType>& operator<<(TStringBuilderBase<CharType>& Builder, EPayloadAccessMode Mode)
{
	switch (Mode)
	{
		case UE::EPayloadAccessMode::Local:
			Builder << TEXT("Local");
			break;
		case UE::EPayloadAccessMode::Referenced:
			Builder << TEXT("Referenced");
			break;
		case UE::EPayloadAccessMode::Virtualized:
			Builder << TEXT("Virtualized");
			break;
		default:
			Builder << TEXT("Invalid");
			break;
	}

	return Builder;
}


} //namespace UE

[[nodiscard]] COREUOBJECT_API FString LexToString(UE::Virtualization::EPayloadFilterReason FilterFlags);
