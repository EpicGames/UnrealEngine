// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/ObjectHandleDefines.h"

// HEADER_UNIT_SKIP - Included through other header

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE

namespace UE::CoreUObject::Private
{
	/**
	 * FPackedObjectRef represents a lightweight reference that can fit in the space of a pointer and be able to refer to an object
	 * (or null) that may or may not be loaded without pointing to its location in memory (even if it is currently loaded).
	 */
	struct FPackedObjectRef
	{
		[[nodiscard]] FPackedObjectRef() = default;

		[[nodiscard]] explicit FPackedObjectRef(UPTRINT InEncodedRef)
			: EncodedRef(InEncodedRef)
		{
		}

		// Must be 0 for a reference to null.
		// The least significant bit must always be 1 in a non-null reference.
		UPTRINT EncodedRef;

		inline bool operator==(FPackedObjectRef RHS) const
		{
			return EncodedRef == RHS.EncodedRef;
		}

		inline bool operator!=(FPackedObjectRef RHS) const
		{
			return EncodedRef != RHS.EncodedRef;
		}

		friend inline uint32 GetTypeHash(FPackedObjectRef ObjectRef)
		{
			return GetTypeHashHelper(ObjectRef.EncodedRef);
		}

		inline bool IsNull() { return EncodedRef == 0; }
	};
}

#endif
