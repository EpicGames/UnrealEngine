// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
PerPlatformProperties.h: Property types that can be overridden on a per-platform basis at cook time
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Serialization/Archive.h"
#include "Containers/Map.h"
#include "Algo/Find.h"
#include "Serialization/MemoryLayout.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/FrameRate.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"
#include "PerPlatformProperties.generated.h"

// This is an unsupported feature.
#ifndef WITH_PREVIEW_PPX_DATA
#define WITH_PREVIEW_PPX_DATA 0
#endif

namespace PerPlatformProperty::Private
{
	template<typename MapType>
	struct TGetKeyType
	{
		using Type = typename TTupleElement<0, typename MapType::ElementType>::Type;
	};

	template<typename NameType>
	struct FNameFuncs;

	template<>
	struct FNameFuncs<FName>
	{
		static FName NameToKey(FName Name) { return Name; }

		template<typename ValueType>
		static void SerializePerPlatformMap(FArchive& Ar, TMap<FName, ValueType>& Map)
		{
			Ar << Map;
		}

		template<typename ValueType>
		static void SerializePerPlatformMap(FArchive& UnderlyingArchive, FStructuredArchive::FRecord& Record, TMap<FName, ValueType>& Map)
		{
			Record << SA_VALUE(TEXT("PerPlatform"), Map);
		}
	};

	template<typename MapType>
	using KeyFuncs = FNameFuncs<typename TGetKeyType<MapType>::Type>;
}

#if WITH_PREVIEW_PPX_DATA
struct PPXPreviewOverrides
{
	static COREUOBJECT_API FName PerPlatformOverride;
	static COREUOBJECT_API FName PerPlatformGroupOverride;
	static COREUOBJECT_API void SetPPXPreviewOverrides(FName& InPerPlatformOverride, FName& InPerPlatformGroupOverride);
};
#endif

/** TPerPlatformProperty - template parent class for per-platform properties 
 *  Implements Serialize function to replace value at cook time, and 
 *  backwards-compatible loading code for properties converted from simple types.
 */
template<typename _StructType, typename _ValueType, EName _BasePropertyName>
struct TPerPlatformProperty
{
	typedef _ValueType ValueType;
	typedef _StructType StructType;

#if WITH_EDITOR || WITH_PREVIEW_PPX_DATA
	/** Get the value for the given platform (using standard "ini" name, so Windows, not Win64 or WindowsClient), which can be used to lookup the group */
	_ValueType GetValueForPlatform(FName PlatformName) const
	{
		const _StructType* This = StaticCast<const _StructType*>(this);
		using MapType = decltype(This->PerPlatform);
		using KeyFuncs = typename PerPlatformProperty::Private::KeyFuncs<MapType>;

		const _ValueType* Ptr = PlatformName != NAME_None ? This->PerPlatform.Find(PlatformName) : nullptr;
		if (Ptr == nullptr)
		{
#if WITH_EDITOR
			const FDataDrivenPlatformInfo& Info = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName);
			if (Info.PlatformGroupName != NAME_None)
			{
				Ptr = This->PerPlatform.Find(Info.PlatformGroupName);
			}
#else 
			#if WITH_PREVIEW_PPX_DATA
			if(PPXPreviewOverrides::PerPlatformGroupOverride != NAME_None)
			{
				Ptr = This->PerPlatform.Find(PPXPreviewOverrides::PerPlatformGroupOverride);
			}
			#endif
#endif
		}

		return Ptr ? *Ptr : This->Default;
	}
#endif

	_ValueType GetDefault() const
	{
		const _StructType* This = StaticCast<const _StructType*>(this);
		return This->Default;
	}
	
	_ValueType GetValue() const
	{
#if WITH_EDITORONLY_DATA && WITH_EDITOR
		FName PlatformName;
		// Lookup the override preview platform info, if any
		// @todo this doesn't set PlatformName, just a group, but GetValueForPlatform() will technically work being given a Group name instead of a platform name, so we just use it
		if (UObject::OnGetPreviewPlatform.IsBound() && UObject::OnGetPreviewPlatform.Execute(PlatformName))
		{
			return GetValueForPlatform(PlatformName);
		}
		else		
#endif
		{
#if WITH_PREVIEW_PPX_DATA
			if (PPXPreviewOverrides::PerPlatformOverride != NAME_None || PPXPreviewOverrides::PerPlatformGroupOverride != NAME_None)
			{
				return GetValueForPlatform(PPXPreviewOverrides::PerPlatformOverride);
			}
			else
#endif
			{
				const _StructType* This = StaticCast<const _StructType*>(this);
				return This->Default;
			}
		}
	}

	/* Load old properties that have been converted to FPerPlatformX */
	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FArchive& Ar)
	{
		if (Tag.Type == _BasePropertyName)
		{
			_StructType* This = StaticCast<_StructType*>(this);
			_ValueType OldValue;
			Ar << OldValue;
			*This = _StructType(OldValue);
			return true;
		}
		return false;
	}

	/* Serialization */
	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	/* Serialization */
	bool Serialize(FStructuredArchive::FSlot Slot)
	{
		Slot << *this;
		return true;
	}
};

template<typename _StructType, typename _ValueType, EName _BasePropertyName>
FArchive& operator<<(FArchive& Ar, TPerPlatformProperty<_StructType, _ValueType, _BasePropertyName>& P);
template<typename _StructType, typename _ValueType, EName _BasePropertyName>
void operator<<(FStructuredArchive::FSlot Slot, TPerPlatformProperty<_StructType, _ValueType, _BasePropertyName>& P);

/** FPerPlatformInt - int32 property with per-platform overrides */
USTRUCT(BlueprintType)
struct FPerPlatformInt
#if CPP
:	public TPerPlatformProperty<FPerPlatformInt, int32, NAME_IntProperty>
#endif
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = PerPlatform)
	int32 Default;

#if WITH_EDITORONLY_DATA || WITH_PREVIEW_PPX_DATA
	UPROPERTY(EditAnywhere, Category = PerPlatform, meta = (AllowedInOptional))
	TMap<FName, int32> PerPlatform;
#endif

	FPerPlatformInt()
	:	Default(0)
	{
	}

	FPerPlatformInt(int32 InDefaultValue)
	:	Default(InDefaultValue)
	{
	}

	COREUOBJECT_API FString ToString() const;

};

extern template COREUOBJECT_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformInt, int32, NAME_IntProperty>&);
extern template COREUOBJECT_API void operator<<(FStructuredArchive::FSlot Slot, TPerPlatformProperty<FPerPlatformInt, int32, NAME_IntProperty>&);

template<>
struct TStructOpsTypeTraits<FPerPlatformInt>
	: public TStructOpsTypeTraitsBase2<FPerPlatformInt>
{
	enum
	{
		WithSerializeFromMismatchedTag = true,
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/** FPerPlatformFloat - float property with per-platform overrides */
USTRUCT(BlueprintType, meta = (CanFlattenStruct))
struct FPerPlatformFloat
#if CPP
:	public TPerPlatformProperty<FPerPlatformFloat, float, NAME_FloatProperty>
#endif
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = PerPlatform)
	float Default;

#if WITH_EDITORONLY_DATA || WITH_PREVIEW_PPX_DATA
	UPROPERTY(EditAnywhere, Category = PerPlatform, meta = (AllowedInOptional))
	TMap<FName, float> PerPlatform;
#endif

	FPerPlatformFloat()
	:	Default(0.f)
	{
	}

	FPerPlatformFloat(float InDefaultValue)
	:	Default(InDefaultValue)
	{
	}

};
extern template COREUOBJECT_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformFloat, float, NAME_FloatProperty>&);

template<>
struct TStructOpsTypeTraits<FPerPlatformFloat>
:	public TStructOpsTypeTraitsBase2<FPerPlatformFloat>
{
	enum
	{
		WithSerializeFromMismatchedTag = true,
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/** FPerPlatformBool - bool property with per-platform overrides */
USTRUCT(BlueprintType)
struct FPerPlatformBool
#if CPP
:	public TPerPlatformProperty<FPerPlatformBool, bool, NAME_BoolProperty>
#endif
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = PerPlatform)
	bool Default;

#if WITH_EDITORONLY_DATA || WITH_PREVIEW_PPX_DATA
	UPROPERTY(EditAnywhere, Category = PerPlatform, meta = (AllowedInOptional))
	TMap<FName, bool> PerPlatform;
#endif

	FPerPlatformBool()
	:	Default(false)
	{
	}

	FPerPlatformBool(bool InDefaultValue)
	:	Default(InDefaultValue)
	{
	}
};
extern template COREUOBJECT_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformBool, bool, NAME_BoolProperty>&);

template<>
struct TStructOpsTypeTraits<FPerPlatformBool>
	: public TStructOpsTypeTraitsBase2<FPerPlatformBool>
{
	enum
	{
		WithSerializeFromMismatchedTag = true,
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/** FPerPlatformFrameRate - FFrameRate property with per-platform overrides */
USTRUCT(BlueprintType)
struct FPerPlatformFrameRate
#if CPP
:	public TPerPlatformProperty<FPerPlatformFrameRate, FFrameRate, NAME_FrameRate>
#endif
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = PerPlatform)
	FFrameRate Default;

#if WITH_EDITORONLY_DATA || WITH_PREVIEW_PPX_DATA
	UPROPERTY(EditAnywhere, Category = PerPlatform, meta = (AllowedInOptional))
	TMap<FName, FFrameRate> PerPlatform;
#endif

	FPerPlatformFrameRate()
	:	Default(30, 1)
	{
	}

	FPerPlatformFrameRate(FFrameRate InDefaultValue)
	:	Default(InDefaultValue)
	{
	}
	
	bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FArchive& Ar)
	{
		if(const UStruct* FrameRateStruct = FindObject<UStruct>(FTopLevelAssetPath("/Script/CoreUObject.FrameRate")))
		{
			FFrameRate Value;
			Ar << Value.Denominator;
			Ar << Value.Numerator;
			Default = Value;

			return true;
		}
		
		return false;
	}
};

extern template COREUOBJECT_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformFrameRate, FFrameRate, NAME_FrameRate>&);

template<>
struct TStructOpsTypeTraits<FPerPlatformFrameRate>
	: public TStructOpsTypeTraitsBase2<FPerPlatformFrameRate>
{
	enum
	{
		WithSerializeFromMismatchedTag = false,
		WithSerializer = true
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};
