// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
 * PerPlatformPropertiesImpl.inl: Serializer implementation
 *
 * This file needs to be included by a cpp once for each module that declares a PerPlatformProperty
 * And following the include, template instantiations of the serializaers for each property class, for example:
 * 
 * template SLATECORE_API FArchive& operator<<(FArchive&, TPerPlatformProperty<FPerPlatformMyEnumType, EMyEnumType, NAME_EnumProperty>&);
 * template SLATECORE_API void operator<<(FStructuredArchive::FSlot Slot, TPerPlatformProperty<FPerPlatformMyEnumType, EMyEnumType, NAME_EnumProperty>&);
 * =============================================================================*/

#pragma once

#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Misc/ConfigCacheIni.h"
#endif

/** Serializer to cook out the most appropriate platform override */
template<typename StructType, typename ValueType, EName BasePropertyName>
FArchive& operator<<(FArchive& Ar, TPerPlatformProperty<StructType, ValueType, BasePropertyName>& Property)
{
	bool bStripSerializeAllProperties = true;
#if WITH_EDITOR
	bool bClientUsePreviewPPXData = false;
	if(Ar.IsCooking())
	{
		bool bIsClientOnly = Ar.CookingTarget()->IsClientOnly();
		bool bIsServerOnly = Ar.CookingTarget()->IsServerOnly();
		bool bHasEditorOnlyData = Ar.CookingTarget()->HasEditorOnlyData();
		if(bIsClientOnly || !(bIsClientOnly || bIsServerOnly || bHasEditorOnlyData))
		{
			bClientUsePreviewPPXData = Ar.CookingTarget()->GetConfigSystem()->GetBoolOrDefault(TEXT("CookedPreviewData"), TEXT("bClientUsePreviewPPXData"), false, GEngineIni);		
		}
	}
	bStripSerializeAllProperties = !bClientUsePreviewPPXData && Ar.IsCooking() && !Ar.IsSavingOptionalObject();
#elif WITH_PREVIEW_PPX_DATA
	bStripSerializeAllProperties = false;
#endif
	
	StructType* This = StaticCast<StructType*>(&Property);
	Ar << bStripSerializeAllProperties;
#if WITH_EDITOR
	if (Ar.IsCooking() && !Ar.IsSavingOptionalObject())
	{
		// Save out platform override if it exists and Default otherwise
		ValueType Value = Property.GetValueForPlatform(*Ar.CookingTarget()->IniPlatformName());
		Ar << Value;
	}
	else
#endif
	{
		Ar << This->Default;
	}

	if (!bStripSerializeAllProperties)
	{	
#if WITH_EDITORONLY_DATA
		if (Ar.IsCooking() && !Ar.IsSavingOptionalObject())
		{
			TMap<FName, ValueType> ToBeCookedPerPlatform(This->PerPlatform);
			for (auto It = ToBeCookedPerPlatform.CreateIterator(); It; ++It)
			{
				if (FDataDrivenPlatformInfoRegistry::GetPlatformInfo(It.Key()).bIsConfidential)
				{
					It.RemoveCurrent();
				}
			}
			Ar << ToBeCookedPerPlatform;
		}
		else
		{
			Ar << This->PerPlatform;
		}
#elif WITH_PREVIEW_PPX_DATA
		{
			Ar << This->PerPlatform;
		}
#else
		// If we have old assets that were serialized with a PerPlatform map, but we no longer need it.
		TMap<FName, ValueType> TempPerPlatform;
		Ar << TempPerPlatform;
#endif
	}

	return Ar;
}

/** Serializer to cook out the most appropriate platform override */
template<typename StructType, typename ValueType, EName BasePropertyName>
void operator<<(FStructuredArchive::FSlot Slot, TPerPlatformProperty<StructType, ValueType, BasePropertyName>& Property)
{
	FArchive& Ar = Slot.GetUnderlyingArchive();
	FStructuredArchive::FRecord Record = Slot.EnterRecord();
	bool bStripSerializeAllProperties = true;
#if WITH_EDITOR
	bool bClientUsePreviewPPXData = false;
	if (Ar.IsCooking())
	{
		bool bIsClientOnly = Ar.CookingTarget()->IsClientOnly();
		bool bIsServerOnly = Ar.CookingTarget()->IsServerOnly();
		bool bHasEditorOnlyData = Ar.CookingTarget()->HasEditorOnlyData();
		if (bIsClientOnly || !(bIsClientOnly || bIsServerOnly || bHasEditorOnlyData))
		{
			bClientUsePreviewPPXData = Ar.CookingTarget()->GetConfigSystem()->GetBoolOrDefault(TEXT("CookedPreviewData"), TEXT("bClientUsePreviewPPXData"), false, GEngineIni);
		}
	}
	bStripSerializeAllProperties = !bClientUsePreviewPPXData && Ar.IsCooking() && !Ar.IsSavingOptionalObject();
#elif WITH_PREVIEW_PPX_DATA
	bStripSerializeAllProperties = false;
#endif
	StructType* This = StaticCast<StructType*>(&Property);
	Record << SA_VALUE(TEXT("bCooked"), bStripSerializeAllProperties);
#if WITH_EDITOR
	if (Ar.IsCooking() && !Ar.IsSavingOptionalObject())
	{
		// Save out platform override if it exists and Default otherwise
		ValueType Value = Property.GetValueForPlatform(*Ar.CookingTarget()->IniPlatformName());
		Record << SA_VALUE(TEXT("Value"), Value);
	}
	else
#endif
	{
		Record << SA_VALUE(TEXT("Value"), This->Default);
	}

	if (!bStripSerializeAllProperties)
	{
#if WITH_EDITORONLY_DATA
		if (Ar.IsCooking() && !Ar.IsSavingOptionalObject())
		{
			TMap<FName, ValueType> ToBeCookedPerPlatform(This->PerPlatform);
			for (auto It = ToBeCookedPerPlatform.CreateIterator(); It; ++It)
			{
				if (FDataDrivenPlatformInfoRegistry::GetPlatformInfo(It.Key()).bIsConfidential)
				{
					It.RemoveCurrent();
				}
			}
			Record << SA_VALUE(TEXT("PerPlatform"), ToBeCookedPerPlatform);
		}
		else
		{
			Record << SA_VALUE(TEXT("PerPlatform"), This->PerPlatform);
		}
#elif WITH_PREVIEW_PPX_DATA
		Record << SA_VALUE(TEXT("PerPlatform"), This->PerPlatform);
#else
		// If we have old assets that were serialized with a PerPlatform map, but we no longer need it.
		TMap<FName, ValueType> TempPerPlatform;
		Record << SA_VALUE(TEXT("PerPlatform"), TempPerPlatform);
#endif
	}
}

