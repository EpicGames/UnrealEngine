// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	PersistentObjectPtr.h: Template that is a base class for Lazy and Asset pointers
=============================================================================*/

#pragma once

#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"

class UObject;

/**
 * TPersistentObjectPtr is a template base class for FLazyObjectPtr and FSoftObjectPtr
 */
template<class TObjectID>
struct TPersistentObjectPtr
{
	using ElementType = TObjectID;

	/** Default constructor, will be null */
	UE_FORCEINLINE_HINT TPersistentObjectPtr()
	{
		Reset();
	}

	/** Reset the lazy pointer back to the null state */
	inline void Reset()
	{
		WeakPtr.Reset();
		ObjectID.Reset();
	}

	/** Resets the weak ptr only, call this when ObjectId may change */
	UE_FORCEINLINE_HINT void ResetWeakPtr()
	{
		WeakPtr.Reset();
	}

	/** Construct from a unique object identifier */
	explicit inline TPersistentObjectPtr(const TObjectID& InObjectID)
		: WeakPtr()
		, ObjectID(InObjectID)
	{
	}

	/** Copy from a unique object identifier */
	inline void operator=(const TObjectID& InObjectID)
	{
		WeakPtr.Reset();
		ObjectID = InObjectID;
	}

	/** Copy from an object pointer */
	void operator=(FObjectPtr Object)
	{
		if (Object)
		{
			ObjectID = TObjectID::GetOrCreateIDForObject(Object);
			if (CanCacheObjectPointer(Object))
			{
				WeakPtr = Object;
			}
			else
			{
				WeakPtr.Reset();
			}
		}
		else
		{
			Reset();
		}
	}
	UE_FORCEINLINE_HINT void operator=(const UObject* Object)
	{
		*this = FObjectPtr(const_cast<UObject*>(Object));
	}
	template <typename T>
	UE_FORCEINLINE_HINT void operator=(TObjectPtr<T> Object)
	{
		// This needs to be a template instead of TObjectPtr<const UObject> because C++ does derived-to-base
		// pointer conversions ('standard conversion sequences') in more cases than TSmartPtr<Derived>-to-TSmartPtr<Base>
		// conversions ('user-defined conversions'), meaning it doesn't auto-convert in many real use cases.
		//
		// https://en.cppreference.com/w/cpp/language/implicit_conversion

		*this = FObjectPtr(Object);
	}

	/** Copy from an existing weak pointer, reserve IDs if required */
	inline void operator=(const FWeakObjectPtr& Other)
	{
		// If object exists need to make sure it gets registered properly in above function, if it doesn't exist just empty it
		const UObject* Object = Other.Get();
		*this = Object;
	}

	/**
	 * Gets the unique object identifier associated with this lazy pointer. Valid even if pointer is not currently valid
	 *
	 * @return Unique ID for this object, or an invalid FUniqueObjectGuid if this pointer isn't set to anything
	 */
	UE_FORCEINLINE_HINT const TObjectID& GetUniqueID() const
	{
		return ObjectID;
	}

	/** Non-const version of the above */
	UE_FORCEINLINE_HINT TObjectID& GetUniqueID()
	{
		return ObjectID;
	}

	/**
	 * Dereference the pointer, which may cause it to become valid again. Will not try to load pending outside of game thread
	 *
	 * @return nullptr if this object is gone or the pointer was null, otherwise a valid UObject pointer
	 */
	inline UObject* Get() const
	{
		UObject* Object = WeakPtr.Get();
		
		// Do a full resolve if the cached object is null but we have a valid object ID that might resolve
		// This used to check TObjectID::GetCurrentTag() before resolving but that was unreliable and did not improve performance in actual use
		if (!Object && ObjectID.IsValid())
		{
			Object = ObjectID.ResolveObject();
			if (CanCacheObjectPointer(FObjectPtr(Object)))
			{
				WeakPtr = Object;
			}

			// Make sure it isn't garbage to match the default behavior of WeakPtr.Get() without looking it up again
			return ::GetValid(Object);
		}
		return Object;
	}

	/**
	 * Dereference the lazy pointer, which may cause it to become valid again. Will not try to load pending outside of game thread
	 *
	 * @param bEvenIfPendingKill, if this is true, pendingkill objects are considered valid
	 * @return nullptr if this object is gone or the lazy pointer was null, otherwise a valid UObject pointer
	 */
	inline UObject* Get(bool bEvenIfPendingKill) const
	{
		UObject* Object = WeakPtr.Get(bEvenIfPendingKill);

		// Do a full resolve if the cached object is null but we have a valid object ID that might resolve
		// This used to check TObjectID::GetCurrentTag() before resolving but that was unreliable and did not improve performance in actual use
		if (!Object && ObjectID.IsValid())
		{
			Object = ObjectID.ResolveObject();
			FWeakObjectPtr LocalWeakPtr = Object;

			if (CanCacheObjectPointer(FObjectPtr(Object)))
			{
				WeakPtr = Object;
			}
			// Get the object again using the correct flag
			Object = LocalWeakPtr.Get(bEvenIfPendingKill);
		}
		return Object;
	}

	/** Dereference the pointer */
	UE_FORCEINLINE_HINT UObject& operator*() const
	{
		return *Get();
	}

	/** Dereference the pointer */
	UE_FORCEINLINE_HINT UObject* operator->() const
	{
		return Get();
	}

	/** Compare pointers for equality. Only Serial Number matters for the base implementation */
	UE_FORCEINLINE_HINT bool operator==(const TPersistentObjectPtr& Rhs) const
	{
		return ObjectID == Rhs.ObjectID;
	}

	UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR) const
	{
		return !IsValid();
	}

	/** Compare pointers for inequality. Only Serial Number matters for the base implementation */
	UE_FORCEINLINE_HINT bool operator!=(const TPersistentObjectPtr& Rhs) const
	{
		return ObjectID != Rhs.ObjectID;
	}

	UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR) const
	{
		return IsValid();
	}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	UE_FORCEINLINE_HINT friend bool operator==(TYPE_OF_NULLPTR, const TPersistentObjectPtr& Rhs)
	{
		return !Rhs.IsValid();
	}

	UE_FORCEINLINE_HINT friend bool operator!=(TYPE_OF_NULLPTR, const TPersistentObjectPtr& Rhs)
	{
		return Rhs.IsValid();
	}
#endif

	/**  
	 * Test if this does not point to a live UObject, but may in the future
	 * 
	 * @return true if this does not point to a real object, but could possibly
	 */
	UE_FORCEINLINE_HINT bool IsPending() const
	{
		return Get() == nullptr && ObjectID.IsValid();
	}

	/**  
	 * Test if this points to a live UObject
	 *
	 * @return true if Get() would return a valid non-null pointer
	 */
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return !!Get();
	}

	/**  
	 * Slightly different than !IsValid(), returns true if this used to point to a UObject, but doesn't any more and has not been assigned or reset in the mean time.
	 *
	 * @return true if this used to point at a real object but no longer does.
	 */
	UE_FORCEINLINE_HINT bool IsStale() const
	{
		return WeakPtr.IsStale();
	}
	/**  
	 * Test if this can never point to a live UObject
	 *
	 * @return true if this is explicitly pointing to no object
	 */
	UE_FORCEINLINE_HINT bool IsNull() const
	{
		return !ObjectID.IsValid();
	}

	/** Hash function */
	UE_FORCEINLINE_HINT friend uint32 GetTypeHash(const TPersistentObjectPtr& Ptr)
	{
		return GetTypeHash(Ptr.ObjectID);
	}

private:

	// Returns whether the object pointer can be stored in WeakPtr for later retrieval
	// For example, objects that are in the process of being async loaded may not be cached
	inline bool CanCacheObjectPointer(FObjectPtr Ptr) const
	{
		if (IsInAsyncLoadingThread() && Ptr && Ptr->HasAnyInternalFlags(EInternalObjectFlags_AsyncLoading))
		{
			return false;
		}
		return true;
	}

	/** Once the object has been noticed to be loaded, this is set to the object weak pointer **/
	mutable FWeakObjectPtr	WeakPtr;
	/** Guid for the object this pointer points to or will point to. **/
	TObjectID				ObjectID;
};

template <class TObjectID> struct TIsPODType<TPersistentObjectPtr<TObjectID> > { enum { Value = TIsPODType<TObjectID>::Value }; };
template <class TObjectID> struct TIsWeakPointerType<TPersistentObjectPtr<TObjectID> > { enum { Value = TIsWeakPointerType<FWeakObjectPtr>::Value }; };
