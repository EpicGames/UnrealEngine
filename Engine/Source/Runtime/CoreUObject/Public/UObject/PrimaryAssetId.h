// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/TypeHash.h"
#include "UObject/NameTypes.h"
#include "UObject/StructOpsTypeTraits.h"
#include "UObject/UnrealNames.h"

#include "PrimaryAssetId.generated.h"

class FCbWriter;
class FOutputDevice;
class UObject;

/**
 * A primary asset type, represented as an FName internally and implicitly convertible back and forth
 * This exists so the blueprint API can understand it's not a normal FName
 */
USTRUCT(BlueprintType)
struct FPrimaryAssetType
{
	GENERATED_BODY()

	/** Convert from FName */
	FPrimaryAssetType() = default;
	FPrimaryAssetType(FName InName) : Name(InName) {}
	FPrimaryAssetType(EName InName) : Name(FName(InName)) {}
	FPrimaryAssetType(const WIDECHAR* InName) : Name(FName(InName)) {}
	FPrimaryAssetType(const ANSICHAR* InName) : Name(FName(InName)) {}

	FPrimaryAssetType(const FPrimaryAssetType&) = default;
	FPrimaryAssetType(FPrimaryAssetType&&) = default;
	FPrimaryAssetType& operator=(const FPrimaryAssetType&) = default;
	FPrimaryAssetType& operator=(FPrimaryAssetType&&) = default;

	/** Convert to FName */
	operator FName&() { return Name; }
	operator const FName&() const { return Name; }

	/** Returns internal Name explicitly, not normally needed */
	[[nodiscard]] FName GetName() const
	{
		return Name;
	}

	[[nodiscard]] bool operator==(const FPrimaryAssetType& Other) const
	{
		return Name == Other.Name;
	}

	[[nodiscard]] bool operator!=(const FPrimaryAssetType& Other) const
	{
		return Name != Other.Name;
	}

	/** Returns true if this is a valid Type */
	[[nodiscard]] bool IsValid() const
	{
		return Name != NAME_None;
	}

	/** Returns string version of this Type */
	[[nodiscard]] FString ToString() const
	{
		return Name.ToString();
	}

	/** Appends to the given builder the string version of this Type */
	void AppendString(FStringBuilderBase& Builder) const
	{
		Name.AppendString(Builder);
	}

	/** UStruct Overrides */
	bool ExportTextItem(FString& ValueStr, FPrimaryAssetType const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
	bool SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);

	[[nodiscard]] friend inline uint32 GetTypeHash(const FPrimaryAssetType& Key)
	{
		return GetTypeHash(Key.Name);
	}

	/** Compares FPrimaryAssetTypes lexically based on internal FName. Sort is alphabetical ascending, ignores case. */
	[[nodiscard]] int32 CompareLexical(const FPrimaryAssetType& Other) const
	{
		return Name.Compare(Other.Name);
	}

	/** Slow alphabetical order that is stable / deterministic over process runs, ignores case. */
	[[nodiscard]] bool LexicalLess(const FPrimaryAssetType& Other) const
	{
		return Name.LexicalLess(Other.Name);
	}

	/** Fast non-alphabetical order that is only stable during this process' lifetime. Uses FName comparison index. */
	[[nodiscard]] bool FastLess(const FPrimaryAssetType& Other) const
	{
		return Name.FastLess(Other.Name);
	}

private:
	/** The FName representing this type */
	UPROPERTY(EditAnywhere, SaveGame, BlueprintReadWrite, Category = PrimaryAssetType, meta=(AllowPrivateAccess))
	FName Name;
};

template<>
struct TStructOpsTypeTraits<FPrimaryAssetType> : public TStructOpsTypeTraitsBase2<FPrimaryAssetType>
{
	enum
	{
		WithZeroConstructor = true,
		WithCopy = true,
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

/**
 * This identifies an object as a "primary" asset that can be searched for by the AssetManager and used in various tools
 */
USTRUCT(BlueprintType)
struct FPrimaryAssetId
{
	GENERATED_BODY()

	/** An FName describing the logical type of this object, usually the name of a base UClass. For example, any Blueprint derived from APawn will have a Primary Asset Type of "Pawn".
	"PrimaryAssetType:PrimaryAssetName" should form a unique name across your project. */
	UPROPERTY(EditAnywhere, SaveGame, BlueprintReadWrite, Category = PrimaryAssetId)
	FPrimaryAssetType PrimaryAssetType;
	/** An FName describing this asset. This is usually the short name of the object, but could be a full asset path for things like maps, or objects with GetPrimaryId() overridden.
	"PrimaryAssetType:PrimaryAssetName" should form a unique name across your project. */
	UPROPERTY(EditAnywhere, SaveGame, BlueprintReadWrite, Category = PrimaryAssetId)
	FName PrimaryAssetName;

	/** Static names to represent the AssetRegistry tags for the above data */
	static COREUOBJECT_API const FName PrimaryAssetTypeTag;
	static COREUOBJECT_API const FName PrimaryAssetNameTag;
	/** AssetRegistry tag used to store primary asset display name (optional) */
	static COREUOBJECT_API const FName PrimaryAssetDisplayNameTag;

	FPrimaryAssetId() = default;
	FPrimaryAssetId(FPrimaryAssetType InAssetType, FName InAssetName)
		: PrimaryAssetType(InAssetType), PrimaryAssetName(InAssetName)
	{}

	[[nodiscard]] static COREUOBJECT_API FPrimaryAssetId ParseTypeAndName(const TCHAR* TypeAndName, uint32 Len);
	[[nodiscard]] static COREUOBJECT_API FPrimaryAssetId ParseTypeAndName(FName TypeAndName);
	[[nodiscard]] static FPrimaryAssetId ParseTypeAndName(const FString& TypeAndName)
	{
		return ParseTypeAndName(*TypeAndName, static_cast<uint32>(TypeAndName.Len()));
	}

	explicit FPrimaryAssetId(const FString& TypeAndName)
		: FPrimaryAssetId(ParseTypeAndName(TypeAndName))
	{}

	FPrimaryAssetId(const FPrimaryAssetId&) = default;
	FPrimaryAssetId(FPrimaryAssetId&&) = default;
	FPrimaryAssetId& operator=(const FPrimaryAssetId&) = default;
	FPrimaryAssetId& operator=(FPrimaryAssetId&&) = default;

	/** Returns true if this is a valid identifier */
	[[nodiscard]] bool IsValid() const
	{
		return PrimaryAssetType != NAME_None && PrimaryAssetName != NAME_None;
	}

	/** Returns string version of this identifier in Type:Name format */
	[[nodiscard]] FString ToString() const
	{
		TStringBuilder<256> Builder;
		AppendString(Builder);
		return FString::ConstructFromPtrSize(Builder.GetData(), Builder.Len());
	}

	/** Appends to the given builder the string version of this identifier in Type:Name format */
	void AppendString(FStringBuilderBase& Builder) const
	{
		if (IsValid())
		{
			PrimaryAssetType.AppendString(Builder);
			Builder << TEXT(":");
			PrimaryAssetName.AppendString(Builder);
		}
	}

	/** Converts from Type:Name format */
	[[nodiscard]] static FPrimaryAssetId FromString(const FString& String)
	{
		return FPrimaryAssetId(String);
	}

	[[nodiscard]] bool operator==(const FPrimaryAssetId& Other) const
	{
		return PrimaryAssetType == Other.PrimaryAssetType && PrimaryAssetName == Other.PrimaryAssetName;
	}

	[[nodiscard]] bool operator!=(const FPrimaryAssetId& Other) const
	{
		return PrimaryAssetType != Other.PrimaryAssetType || PrimaryAssetName != Other.PrimaryAssetName;
	}

	inline friend FArchive& operator<<(FArchive& Ar, FPrimaryAssetId& Other)
	{
        Ar << Other.PrimaryAssetType;
		Ar << Other.PrimaryAssetName;
		return Ar;
	}

	/** UStruct Overrides */
	bool ExportTextItem(FString& ValueStr, FPrimaryAssetId const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);
	bool SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);

	[[nodiscard]] friend inline uint32 GetTypeHash(const FPrimaryAssetId& Key)
	{
		uint32 Hash = 0;

		Hash = HashCombine(Hash, GetTypeHash(Key.PrimaryAssetType));
		Hash = HashCombine(Hash, GetTypeHash(Key.PrimaryAssetName));
		return Hash;
	}

	/** Compare lexical ordering by PrimaryAssetType first followed by PrimaryAssetName, ignores case. */
	[[nodiscard]] int32 CompareLexical(const FPrimaryAssetId& Other) const
	{
		const int32 AssetTypeCompare = PrimaryAssetType.CompareLexical(Other.PrimaryAssetType);
		if (AssetTypeCompare != 0)
		{
			return AssetTypeCompare;
		}

		return PrimaryAssetName.Compare(Other.PrimaryAssetName);
	}

	/** Slow alphabetical order that is stable / deterministic over process runs, ignores case. */
	[[nodiscard]] bool LexicalLess(const FPrimaryAssetId& Other) const
	{
		return CompareLexical(Other) < 0;
	}

private:
	 COREUOBJECT_API void SerializeForLog(FCbWriter& Writer, const FPrimaryAssetId& Value);
};

template<>
struct TStructOpsTypeTraits<FPrimaryAssetId> : public TStructOpsTypeTraitsBase2<FPrimaryAssetId>
{
	enum
	{
		WithZeroConstructor = true,
		WithCopy = true,
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
};

COREUOBJECT_API FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FPrimaryAssetId& Id);
