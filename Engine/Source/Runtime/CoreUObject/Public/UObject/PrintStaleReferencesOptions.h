// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

/* Options for FReferenceChainSearch::FindAndPrintStaleReferencesToObject function */
enum class EPrintStaleReferencesOptions
{
	None = 0,
	Log = 1,
	Display = 2,
	Warning = 3,
	Error = 4,
	Fatal = 5,
	Ensure = 0x00000100,

	// Only search for direct references to the object or one of its inners, not a full reference chain 
	Minimal = 0x00000200,

	VerbosityMask = 0x000000ff
};
ENUM_CLASS_FLAGS(EPrintStaleReferencesOptions);