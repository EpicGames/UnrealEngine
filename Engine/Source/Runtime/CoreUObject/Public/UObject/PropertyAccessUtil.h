// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

#include "PropertyAccessUtil.generated.h"

class FOutputDevice;
class FProperty;
class UObject;
class UStruct;

/**
 * Result flags from property access.
 */
enum class EPropertyAccessResultFlags : uint8
{
	/** The property was accessed successfully */
	Success = 0,

	/** The property could not be accessed due to a permission error (the permission flags can give more detail of the error) */
	PermissionDenied = 1<<0,
	/** The property could not be read or written due to a failure converting from the source or to the destination */
	ConversionFailed = 1<<1,

	/** Permission flag added when the property cannot be accessed due to being protected (is not marked for editor or Blueprint access) */
	AccessProtected = 1<<4,
	/** Permission flag added when attempting to edit a property on a template that cannot be edited on templates */
	CannotEditTemplate = 1<<5,
	/** Permission flag added when attempting to edit a property on an instance that cannot be edited on instances */
	CannotEditInstance = 1<<6,
	/** Permission flag added when attempting to edit a property that is read-only (based on the given read-only flags) */
	ReadOnly = 1<<7,
};
ENUM_CLASS_FLAGS(EPropertyAccessResultFlags);

/**
 * Enum controlling when to emit property change notifications when setting a property value.
 */
UENUM(BlueprintType)
enum class EPropertyAccessChangeNotifyMode : uint8
{
	/** Notify only when a value change has actually occurred */
	Default,
	/** Never notify that a value change has occurred */
	Never,
	/** Always notify that a value change has occurred, even if the value is unchanged */
	Always,
};

/**
 * Information needed to emit property change notifications when setting a property value.
 */
struct FPropertyAccessChangeNotify
{
	/** The kind of change that occurred */
	EPropertyChangeType::Type ChangeType = EPropertyChangeType::Unspecified;
	/** The object that is being changed */
	UObject* ChangedObject = nullptr;
	/** The chain of properties that are being changed */
	FEditPropertyChain ChangedPropertyChain;
	/** When to emit property change notifications */
	EPropertyAccessChangeNotifyMode NotifyMode = EPropertyAccessChangeNotifyMode::Default;

	/**
	 * Maps each container property's name to the element index affected by this change.
	 * Empty for non-container changes. For nested container changes, each container level
	 * in the chain contributes one entry, e.g. { "OuterArray" -> 2, "InnerArray" -> 5 }.
	 */
	TMap<FString, int32> ElementIndicesMap;
};

/**
 * Callback used to get the value of a property.
 */
using FPropertyAccessGetFunc = TFunctionRef<bool()>;

/**
 * Callback used to set the value of a property.
 */
using FPropertyAccessSetFunc = TFunctionRef<bool(const FPropertyAccessChangeNotify*)>;

/**
 * Callback used to build the information needed to emit property change notifications when setting a property value.
 */
using FPropertyAccessBuildChangeNotifyFunc = TFunctionRef<TUniquePtr<FPropertyAccessChangeNotify>()>;

namespace PropertyAccessUtil
{
	/** Flags that make a property read-only when settings its value at runtime */
	static const uint64 RuntimeReadOnlyFlags = CPF_EditConst | CPF_BlueprintReadOnly;

	/** Flags that make a property read-only when settings its value in the editor */
	static const uint64 EditorReadOnlyFlags = CPF_EditConst;

	/**
	 * High-level function for getting the value of a property from an object.
	 * @note This function calls CanGetPropertyValue internally.
	 *
	 * @param InObjectProp Property to get the value of.
	 * @param InObject Object containing the property.
	 * @param InDestProp Property of the value to set (must be compatible with the source property).
	 * @param InDestValue Instance to fill with the property value (must be a valid and constructed block of memory that is compatible with the property).
	 * @param InArrayIndex For fixed-size array properties denotes which index of the array to get, or INDEX_NONE to get the entire property.
	 *
	 * @return Flags describing whether the get was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags GetPropertyValue_Object(const FProperty* InObjectProp, const UObject* InObject, const FProperty* InDestProp, void* InDestValue, const int32 InArrayIndex);

	/**
	 * High-level function for getting the value of a property from a property container (object or struct).
	 * @note This function calls CanGetPropertyValue internally.
	 *
	 * @param InContainerProp Property to get the value of.
	 * @param InContainerData The instance data containing the property.
	 * @param InDestProp Property of the value to set (must be compatible with the source property).
	 * @param InDestValue Instance to fill with the property value (must be a valid and constructed block of memory that is compatible with the property).
	 * @param InArrayIndex For fixed-size array properties denotes which index of the array to get, or INDEX_NONE to get the entire property.
	 *
	 * @return Flags describing whether the get was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags GetPropertyValue_InContainer(const FProperty* InContainerProp, const void* InContainerData, const FProperty* InDestProp, void* InDestValue, const int32 InArrayIndex);

	/**
	 * High-level function for getting the single-element value of a property from memory.
	 * @note This function calls CanGetPropertyValue internally.
	 *
	 * @param InSrcProp Property to get the value of.
	 * @param InSrcValue The property value to copy.
	 * @param InDestProp Property of the value to set (must be compatible with the source property).
	 * @param InDestValue Instance to fill with the property value (must be a valid and constructed block of memory that is compatible with the property).
	 *
	 * @return Flags describing whether the get was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags GetPropertyValue_DirectSingle(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue);

	/**
	 * High-level function for getting the multi-element value of a property from memory.
	 * @note This function calls CanGetPropertyValue internally.
	 *
	 * @param InSrcProp Property to get the value of.
	 * @param InSrcValue The property value to copy.
	 * @param InDestProp Property of the value to set (must be compatible with the source property).
	 * @param InDestValue Instance to fill with the property value (must be a valid and constructed block of memory that is compatible with the property).
	 *
	 * @return Flags describing whether the get was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags GetPropertyValue_DirectComplete(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue);

	/**
	 * Low-level function for getting the value of a property.
	 * @note This function does *not* CanGetPropertyValue internally, you must call it yourself to validate the get.
	 *
	 * @param InGetFunc Logic for getting the value of the property.
	 *
	 * @return Flags describing whether the get was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags GetPropertyValue(const FPropertyAccessGetFunc& InGetFunc);

	/**
	 * Low-level function for checking whether it's valid to get the value of a property.
	 *
	 * @param InProp Property to query.
	 *
	 * @return Flags describing whether it's valid to get the value of the property.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags CanGetPropertyValue(const FProperty* InProp);

	/**
	 * Finds all instances of an archetype object which are inheriting the given property's value from the archetype, then returns those instance UObjects.
	 *
	 * @param InObjectProp Property to compare the value of.
	 * @param InObject Archetype object containing the property.
	 * @param OutArchetypeInstances Array to store results in.
	 *
	 * @return True if archetype instances were searched and regardless of whether any instances were returned.
	 */
	COREUOBJECT_API bool GetArchetypeInstancesInheritingPropertyValue(const FProperty* InObjectProp, UObject* InObject, TArray<UObject*>& OutArchetypeInstances);

	/**
	 * Finds all instances of an archetype object which are inheriting the given property's value from the archetype, then returns the containers for the given property which is either the UObject's address or the sparse data struct.
	 *
	 * @param InObjectProp Property to compare the value of.
	 * @param InObject Archetype object containing the property.
	 * @param OutArchetypeInstContainers Array to store results in. These are pointers to the container containing the given property value. If the property is sparse class data, these will point to the sparse data struct.
	 *
	 * @return True if archetype instances were searched and regardless of whether any instances were returned.
	 */
	COREUOBJECT_API bool GetArchetypeInstancesInheritingPropertyValue_AsContainerData(const FProperty* InObjectProp, UObject* InObject, TArray<void*>& OutArchetypeInstContainers);

	/**
	 * High-level function for setting the value of a property on an object. If the object is an archetype, value will be propagated to instances that haven't modified the value.
	 * @note This function calls CanSetPropertyValue internally, and will emit property change notifications for the object.
	 *
	 * @param InObjectProp Property to set the value of.
	 * @param InObject Object containing the property.
	 * @param InSrcProp Property of the value to set (must be compatible with the dest property).
	 * @param InSrcValue The value to set on the property.
	 * @param InArrayIndex For fixed-size array properties denotes which index of the array to set, or INDEX_NONE to set the entire property.
	 * @param InReadOnlyFlags Flags controlling which properties are considered read-only.
	 * @param InNotifyMode When to emit property change notifications.
	 *
	 * @return Flags describing whether the set was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags SetPropertyValue_Object(const FProperty* InObjectProp, UObject* InObject, const FProperty* InSrcProp, const void* InSrcValue, const int32 InArrayIndex, const uint64 InReadOnlyFlags, const EPropertyAccessChangeNotifyMode InNotifyMode);

	/**
	 * High-level function for setting the value of a property on a property container (object or struct).
	 * @note This function calls CanSetPropertyValue internally.
	 *
	 * @param InContainerProp Property to set the value of.
	 * @param InContainerData The instance data containing the property.
	 * @param InArchetypeInstContainerData When the object is an archetype: container addresses of instances of the archetype to evaluate. Those with currently the same value as archetype will also have their property updates.
	 * @param InSrcProp Property of the value to set (must be compatible with the dest property).
	 * @param InSrcValue The value to set on the property.
	 * @param InArrayIndex For fixed-size array properties denotes which index of the array to set, or INDEX_NONE to set the entire property.
	 * @param InReadOnlyFlags Flags controlling which properties are considered read-only.
	 * @param InOwnerIsTemplate True if the owner object is considered a template (see IsObjectTemplate).
	 * @param InBuildChangeNotifyFunc Logic for building the information needed to emit property change notifications when setting a property value (can return nullptr if no notifications are needed or possible).
	 *
	 * @return Flags describing whether the set was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags SetPropertyValue_InContainer(const FProperty* InContainerProp, void* InContainerData, const TArray<void*>& InArchetypeInstContainerData, const FProperty* InSrcProp, const void* InSrcValue, const int32 InArrayIndex, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const FPropertyAccessBuildChangeNotifyFunc& InBuildChangeNotifyFunc);
	
	/**
	 * High-level function for setting the single-element value of a property in memory.
	 * @note This function calls CanSetPropertyValue internally.
	 *
	 * @param InSrcProp Property to set the value of.
	 * @param InSrcValue The value to set on the property.
	 * @param InDestProp Property to get the value from (must be compatible with the source property).
	 * @param InDestValue Instance to fill with the property value (must be a valid and constructed block of memory that is compatible with the property).
	 * @param InArchetypeInstDestValues When the object is an archetype: direct value addresses of instances of the archetype to evaluate. Those with currently the same value as archetype will also have their property updates.
	 * @param InReadOnlyFlags Flags controlling which properties are considered read-only.
	 * @param InOwnerIsTemplate True if the owner object is considered a template (see IsObjectTemplate).
	 * @param InBuildChangeNotifyFunc Logic for building the information needed to emit property change notifications when setting a property value (can return nullptr if no notifications are needed or possible).
	 *
	 * @return Flags describing whether the set was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags SetPropertyValue_DirectSingle(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue, const TArray<void*>& InArchetypeInstDestValues, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const FPropertyAccessBuildChangeNotifyFunc& InBuildChangeNotifyFunc);

	/**
	 * High-level function for setting the multi-element value of a property in memory.
	 * @note This function calls CanSetPropertyValue internally.
	 *
	 * @param InSrcProp Property to set the value of.
	 * @param InSrcValue The value to set on the property.
	 * @param InDestProp Property to get the value from (must be compatible with the source property).
	 * @param InDestValue Instance to fill with the property value (must be a valid and constructed block of memory that is compatible with the property).
	 * @param InArchetypeInstDestValues When the object is an archetype: direct value addresses of instances of the archetype to evaluate. Those with currently the same value as archetype will also have their property updates.
	 * @param InReadOnlyFlags Flags controlling which properties are considered read-only.
	 * @param InOwnerIsTemplate True if the owner object is considered a template (see IsObjectTemplate).
	 * @param InBuildChangeNotifyFunc Logic for building the information needed to emit property change notifications when setting a property value (can return nullptr if no notifications are needed or possible).
	 *
	 * @return Flags describing whether the set was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags SetPropertyValue_DirectComplete(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue, const TArray<void*>& InArchetypeInstDestValues, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate, const FPropertyAccessBuildChangeNotifyFunc& InBuildChangeNotifyFunc);

	/**
	 * Low-level function for setting the value of a property.
	 * @note This function does *not* CanSetPropertyValue internally, you must call it yourself to validate the set.
	 *
	 * @param InSetFunc Logic for setting the value of the property (should call EmitPreChangeNotify and EmitPostChangeNotify using the given FPropertyAccessChangeNotify instance).
	 * @param InBuildChangeNotifyFunc Logic for building the information needed to emit property change notifications when setting a property value (can return nullptr if no notifications are needed or possible).
	 *
	 * @return Flags describing whether the get was successful.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags SetPropertyValue(const FPropertyAccessSetFunc& InSetFunc, const FPropertyAccessBuildChangeNotifyFunc& InBuildChangeNotifyFunc);
	
	/**
	 * Low-level function for checking whether it's valid to set the value of a property.
	 *
	 * @param InProp Property to query.
	 *
	 * @return Flags describing whether it's valid to set the value of the property.
	 */
	COREUOBJECT_API EPropertyAccessResultFlags CanSetPropertyValue(const FProperty* InProp, const uint64 InReadOnlyFlags, const bool InOwnerIsTemplate);

	/**
	 * Low-level function called before modifying an object to notify that its value is about to change.
	 *
	 * @param InChangeNotify Information needed to emit property change notifications, or nullptr if no notifications are needed or possible.
	 * @param InIdenticalValue True if the value being set was identical to the current value, false otherwise.
	 */
	COREUOBJECT_API void EmitPreChangeNotify(const FPropertyAccessChangeNotify* InChangeNotify, const bool InIdenticalValue);

	/**
	 * Low-level function called after modifying an object to notify that its value has changed.
	 *
	 * @param InChangeNotify Information needed to emit property change notifications, or nullptr if no notifications are needed or possible.
	 * @param InIdenticalValue True if the value being set was identical to the current value, false otherwise.
	 */
	COREUOBJECT_API void EmitPostChangeNotify(const FPropertyAccessChangeNotify* InChangeNotify, const bool InIdenticalValue);

	/**
	 * Low-level function to build the basic information needed to emit property change notifications.
	 * @note This function can only build the notification for a property directly on the object instance, as more complex cases require external management as the reflection data doesn't let you backtrack over different types.
	 *
	 * @param InProp Property being modified.
	 * @param InObject Object being modified.
	 * @param InNotifyMode When to emit property change notifications.
	 *
	 * @return The information needed to emit property change notifications.
	 */
	COREUOBJECT_API TUniquePtr<FPropertyAccessChangeNotify> BuildBasicChangeNotify(const FProperty* InProp, const UObject* InObject, const EPropertyAccessChangeNotifyMode InNotifyMode, const EPropertyChangeType::Type ChangeType);

	/**
	 * Low-level function for checking whether the given object instance is considered a template for property access.
	 *
	 * @param InObject Object to query.
	 *
	 * @return True if the object instance is considered a template.
	 */
	COREUOBJECT_API bool IsObjectTemplate(const UObject* InObject);

	/**
	 * Low-level function to find a property by its name, following redirectors if it cannot be found.
	 *
	 * @param InPropName The name of the property to find.
	 * @param InStruct The struct that should contain the property.
	 *
	 * @return The found property, or null if the property cannot be found.
	 */
	COREUOBJECT_API FProperty* FindPropertyByName(const FName InPropName, const UStruct* InStruct);

	/**
	 * Low-level function to import a default value to the given property.
	 * @note This is useful for applying defaults to function properties (eg, parameter data from UEdGraphSchema_K2::FindFunctionParameterDefaultValue, or local data from 
	 *       FBlueprintEditorUtils::FindLocalVariable), as these may have come from UHT which can sometimes have a slightly different form than ImportText can handle natively.
	 *
	 * @param InProp The property to import the default value for.
	 * @param InPropValue The destination location to receive the imported value.
	 * @param InDefaultValue The default value to import.
	 *
	 * @return True if the default value could be imported, false otherwise.
	 */
	COREUOBJECT_API bool ImportDefaultPropertyValue(const FProperty* InProp, void* InPropValue, const FString& InDefaultValue, FOutputDevice* ErrorText = (FOutputDevice*)GWarn);

	/**
	 * Low-level function to determine if two properties are compatible with each other (for use by CopySinglePropertyValue/CopyCompletePropertyValue).
	 * 
	 * @param InSrcProp Source property being compared.
	 * @param InDestProp Destination property being compared.
	 * 
	 * @return True if the properties are compatible with each other, false otherwise.
	 */
	COREUOBJECT_API bool ArePropertiesCompatible(const FProperty* InSrcProp, const FProperty* InDestProp);

	/**
	 * Low-level function to determine if two properties have identical values (single property value, ignoring ArrayDim).
	 * @note This function calls ArePropertiesCompatible internally.
	 * 
	 * @param InSrcProp Source property being compared.
	 * @param InSrcValue Source value being compared.
	 * @param InDestProp Destination property being compared.
	 * @param InDestValue Destination value being compared.
	 * 
	 * @return True if the properties have identical values, false otherwise.
	 */
	COREUOBJECT_API bool IsSinglePropertyIdentical(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, const void* InDestValue);

	/**
	 * Low-level function to determine if two properties have identical values (complete property value, for each ArrayDim).
	 * @note This function calls ArePropertiesCompatible internally.
	 *
	 * @param InSrcProp Source property being compared.
	 * @param InSrcValue Source value being compared.
	 * @param InDestProp Destination property being compared.
	 * @param InDestValue Destination value being compared.
	 *
	 * @return True if the properties have identical values, false otherwise.
	 */
	COREUOBJECT_API bool IsCompletePropertyIdentical(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, const void* InDestValue);

	/**
	 * Low-level function to copy the value from one property to another (single property value, ignoring ArrayDim).
	 * @note This function calls ArePropertiesCompatible internally.
	 *
	 * @param InSrcProp Source property being copied from.
	 * @param InSrcValue Source value being copied from.
	 * @param InDestProp Destination property being copied to.
	 * @param InDestValue Destination value being copied to.
	 *
	 * @return True if the properties value was copied, false otherwise.
	 */
	COREUOBJECT_API bool CopySinglePropertyValue(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue);
	
	/**
	 * Low-level function to copy the value from one property to another (complete property value, for each ArrayDim).
	 * @note This function calls ArePropertiesCompatible internally.
	 *
	 * @param InSrcProp Source property being copied from.
	 * @param InSrcValue Source value being copied from.
	 * @param InDestProp Destination property being copied to.
	 * @param InDestValue Destination value being copied to.
	 *
	 * @return True if the properties value was copied, false otherwise.
	 */
	COREUOBJECT_API bool CopyCompletePropertyValue(const FProperty* InSrcProp, const void* InSrcValue, const FProperty* InDestProp, void* InDestValue);
}
