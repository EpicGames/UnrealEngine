// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
PropertyIterrator.h: TPropertyIterator implementation.
=============================================================================*/
#pragma once

#include "UObject/Field.h"
#include "UObject/UObjectIterator.h"

/**
 * Iterates over all structs and their properties
 */
template <class T>
class TPropertyIterator
{
private:
	/** The object being searched for the specified field */
	const UStruct* Struct;
	/** The current location in the list of fields being iterated */
	FField* Field;
	TObjectIterator<UStruct> ClassIterator;

public:
	TPropertyIterator()
		: Struct(nullptr)
		, Field(nullptr)
	{
		if (ClassIterator)
		{
			Struct = *ClassIterator;
			Field = Struct->ChildProperties;

			if (!Field || !Field->IsA<T>())
			{
				IterateToNext();
			}
		}
	}

	/** conversion to "bool" returning true if the iterator is valid. */
	UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return Field != nullptr;
	}
	/** inverse of the "bool" operator */
	UE_FORCEINLINE_HINT bool operator !() const
	{
		return !(bool)*this;
	}

	inline friend bool operator==(const TPropertyIterator<T>& Lhs, const TPropertyIterator<T>& Rhs) { return Lhs.Field == Rhs.Field; }
	inline friend bool operator!=(const TPropertyIterator<T>& Lhs, const TPropertyIterator<T>& Rhs) { return Lhs.Field != Rhs.Field; }

	inline void operator++()
	{
		checkSlow(Field);
		IterateToNext();
	}
	inline T* operator*()
	{
		checkSlow(Field);
		return (T*)Field;
	}
	inline T* operator->()
	{
		checkSlow(Field);
		return (T*)Field;
	}
	inline const UStruct* GetStruct()
	{
		return Struct;
	}
protected:
	inline void IterateToNext()
	{
		FField* NewField = Field;
		do
		{
			if (NewField)
			{
				NewField = NewField->Next;
			}
			if (!NewField && ClassIterator)
			{
				++ClassIterator;
				if(ClassIterator)
				{
					Struct = *ClassIterator;
					NewField = Struct->ChildProperties;
				}
				else
				{
					NewField = nullptr;
				}
			}
		} while (ClassIterator && (!NewField || !NewField->IsA<T>()));
		Field = NewField;
	}
};