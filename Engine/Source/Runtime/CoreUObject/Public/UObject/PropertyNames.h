// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/NameTypes.h"

namespace PropertyNames
{
	COREUOBJECT_API extern const FName PropertyVisibilityOverrideName;
}
