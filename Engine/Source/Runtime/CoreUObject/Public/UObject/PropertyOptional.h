// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/UnrealType.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/Optional.h"

#define UE_API COREUOBJECT_API

// 
// Encapsulates the memory layout logic for an optional without implementing the full FProperty API.
//
struct FOptionalPropertyLayout
{
	explicit constexpr FOptionalPropertyLayout(FProperty* InValueProperty)
		: ValueProperty(InValueProperty)
	{
		check(ValueProperty);
	}

	FProperty* GetValueProperty() const 
	{
		checkf(ValueProperty, TEXT("Expected ValueProperty to be initialized"));
		return ValueProperty; 
	}

	inline bool IsSet(const void* Data) const
	{
		checkSlow(Data);
		return ValueProperty->HasIntrusiveUnsetOptionalState()
			? ValueProperty->IsIntrusiveOptionalValueSet(Data)
			: *GetIsSetPointer(Data);
	}
	inline void* MarkSetAndGetInitializedValuePointerToReplace(void* Data) const
	{
		checkSlow(Data);
		if (ValueProperty->HasIntrusiveUnsetOptionalState())
		{
			if (!IsSet(Data))
			{
				// Need to destroy the value in its optional unset state first 
				ValueProperty->DestroyValue(Data);
				ValueProperty->InitializeValue(Data);
			}
		}
		else
		{
			bool* IsSetPointer = GetIsSetPointer(Data);
			if (!*IsSetPointer)
			{
				ValueProperty->InitializeValue(Data);
				*IsSetPointer = true;
			}
		}
		return Data;
	}
	inline void MarkUnset(void* Data) const
	{
		checkSlow(Data);
		if (ValueProperty->HasIntrusiveUnsetOptionalState())
		{
			ValueProperty->ClearIntrusiveOptionalValue(Data);
		}
		else
		{
			bool* IsSetPointer = GetIsSetPointer(Data);
			if (*IsSetPointer)
			{
				ValueProperty->DestroyValue(Data);
				*IsSetPointer = false;
			}
		}
	}

	// For reading the value of a set optional.
	// Must be called on a non-null pointer to a set optional.
	inline const void* GetValuePointerForRead(const void* Data) const 
	{
		checkSlow(Data && IsSet(Data));
		return Data; 
	}
	
	// For replacing the value of a set optional.
	// Must be called on a non-null pointer to a set optional.
	inline void* GetValuePointerForReplace(void* Data) const
	{
		checkSlow(Data && IsSet(Data));
		return Data;
	}

	// For reading the value of a set optional.
	// Must be called on a non-null pointer to an optional.
	// If called on an unset optional, will return null.
	inline const void* GetValuePointerForReadIfSet(const void* Data) const
	{
		checkSlow(Data);
		return IsSet(Data) ? Data : nullptr;
	}
	
	// For replacing the value of a set optional.
	// Must be called on a non-null pointer to an optional.
	// If called on an unset optional, will return null.
	inline void* GetValuePointerForReplaceIfSet(void* Data) const
	{
		checkSlow(Data);
		return IsSet(Data) ? Data : nullptr;
	}
	
	// For calling from polymorphic code that doesn't know whether it needs the value pointer for
	// read or replace, or whether it has a const pointer or not.
	// Must be called on a non-null pointer to a set optional.
	inline const void* GetValuePointerForReadOrReplace(const void* Data) const
	{
		checkSlow(Data && IsSet(Data));
		return Data;
	}
	inline void* GetValuePointerForReadOrReplace(void* Data) const
	{
		checkSlow(Data && IsSet(Data));
		return Data;
	}
	
	// For calling from polymorphic code that doesn't know whether it needs the value pointer for
	// read or replace, or whether it has a const pointer or not.
	// Must be called on a non-null pointer to an optional.
	// If called on an unset optional, will return null.
	inline const void* GetValuePointerForReadOrReplaceIfSet(const void* Data) const
	{
		checkSlow(Data);
		return IsSet(Data) ? Data : nullptr;
	}
	inline void* GetValuePointerForReadOrReplaceIfSet(void* Data) const
	{
		checkSlow(Data);
		return IsSet(Data) ? Data : nullptr;
	}

	inline int32 CalcSize() const
	{
		if (ValueProperty->HasIntrusiveUnsetOptionalState())
		{
			return ValueProperty->GetSize();
		}
		else
		{
			return Align(CalcIsSetOffset() + 1, ValueProperty->GetMinAlignment());
		}
	}
	
protected:

	constexpr FOptionalPropertyLayout() : ValueProperty(nullptr) {}

	// Variables
	FProperty* ValueProperty; // The type of the inner value

	inline int32 CalcIsSetOffset() const
	{
		check(!ValueProperty->HasIntrusiveUnsetOptionalState());
		checkfSlow(
			ValueProperty->GetSize() == Align(ValueProperty->GetSize(), ValueProperty->GetMinAlignment()),
			TEXT("Expected optional value property to have aligned size, but got misaligned size %i for %s that has minimum alignment %i"),
			ValueProperty->GetSize(),
			*ValueProperty->GetFullName(),
			ValueProperty->GetMinAlignment());
		return ValueProperty->GetSize();
	}

	UE_FORCEINLINE_HINT bool* GetIsSetPointer(void* Data) const
	{
		return reinterpret_cast<bool*>(reinterpret_cast<uint8*>(Data) + CalcIsSetOffset());
	}
	UE_FORCEINLINE_HINT const bool* GetIsSetPointer(const void* Data) const
	{
		return reinterpret_cast<const bool*>(reinterpret_cast<const uint8*>(Data) + CalcIsSetOffset());
	}
};

//
// A property corresponding to UE's optional type, TOptional<T>.
// NOTE: this property is not yet handled by all UE subsystems that produce or consume properties.
//
class FOptionalProperty : public FProperty, public FOptionalPropertyLayout
{
	DECLARE_FIELD_API(FOptionalProperty, FProperty, CASTCLASS_FOptionalProperty, UE_API)

public:

	UE_API FOptionalProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FOptionalProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FOptionalProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FOptionalProperty(InOwner, InName) {}
	UE_API FOptionalProperty(FFieldVariant InOwner, const UECodeGen_Private::FGenericPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	consteval FOptionalProperty(
		UE::CodeGen::ConstInit::FPropertyParams InBaseParams,
		FProperty* InValueProperty
	)
		: Super(InBaseParams)
		, FOptionalPropertyLayout(InValueProperty)
	{
		// CONSTINIT_UOBJECT_TODO: Inner property element size at compile time (currently populated during LinkInternal)
	}
#endif

	UE_API virtual ~FOptionalProperty();

	// Sets the optional property's value property.
	UE_API void SetValueProperty(FProperty* InValueProperty);

	// UObject interface
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void GetPreloadDependencies(TArray<UObject*>& OutDeps) override;
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	COREUOBJECT_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// End of UObject interface

	// Field interface
	UE_API virtual void PostDuplicate(const FField& InField) override;
	UE_API virtual FField* GetInnerFieldByName(const FName& InName) override;
	UE_API virtual void GetInnerFields(TArray<FField*>& OutFields) override;
	UE_API virtual void AddCppProperty(FProperty* Property) override;
	// End of Field interface

	// UHT interface
	UE_API virtual FString GetCPPType(FString* ExtendedTypeText = NULL, uint32 CPPExportFlags = 0) const override;
	UE_API virtual FString GetCPPMacroType(FString& ExtendedTypeText) const override;
	// End of UHT interface

	// FProperty interface
	UE_API virtual void LinkInternal(FArchive& Ar) override;
	UE_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags) const override;
	UE_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
	UE_API virtual bool NetSerializeItem(FArchive& Ar, UPackageMap* Map, TNotNull<void*> Data, TArray<uint8>* MetaData = nullptr) const override;
	UE_API virtual bool SupportsNetSharedSerialization() const override;
	UE_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	UE_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* Parent, int32 PortFlags, FOutputDevice* ErrorText) const override;
	UE_API virtual void CopyValuesInternal(void* Dest, void const* Src, int32 Count) const override;
	UE_API virtual void ClearValueInternal(void* Data) const override;
	UE_API virtual void InitializeValueInternal(void* Data) const override;
	UE_API virtual void DestroyValueInternal(void* Data) const override;
	UE_API virtual bool ContainsClearOnFinishDestroyInternal(TArray<const FStructProperty*>& EncounteredStructProps) const override;
	UE_API virtual void FinishDestroyInternal(void* Data) const override;
	UE_API virtual void InstanceSubobjects(void* Data, void const* DefaultData, TNotNull<UObject*> Owner, struct FObjectInstancingGraph* InstanceGraph) override;
	UE_API virtual int32 GetMinAlignment() const override;
	UE_API virtual bool ContainsObjectReference(TArray<const FStructProperty*>& EncounteredStructProps, EPropertyObjectReferenceType InReferenceType = EPropertyObjectReferenceType::Strong) const override;
	UE_API virtual void EmitReferenceInfo(UE::GC::FSchemaBuilder& Schema, int32 BaseOffset, TArray<const FStructProperty*>& EncounteredStructProps, UE::GC::FPropertyStack& DebugPath) override;
	UE_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	UE_API virtual uint32 GetValueTypeHashInternal(const void* Src) const override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual bool UseBinaryOrNativeSerialization(const FArchive& Ar) const override;
	UE_API virtual bool LoadTypeName(UE::FPropertyTypeName Type, const FPropertyTag* Tag = nullptr) override;
	UE_API virtual void SaveTypeName(UE::FPropertyTypeNameBuilder& Type) const override;
	UE_API virtual bool CanSerializeFromTypeName(UE::FPropertyTypeName Type) const override;
	UE_API virtual bool HasIntrusiveUnsetOptionalState() const override;
	UE_API virtual bool SameType(const FProperty* Other) const override;

	UE_API virtual EPropertyVisitorControlFlow Visit(FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc) const override;
	UE_API virtual void* ResolveVisitedPathInfo(void* Data, const FPropertyVisitorInfo& Info) const override;
	// End of FProperty interface
};

#undef UE_API
