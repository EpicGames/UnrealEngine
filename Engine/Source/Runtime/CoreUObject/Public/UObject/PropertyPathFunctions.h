// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointerFwd.h"
#include "Containers/SharedString.h"
#include "Containers/Map.h"
#include "UnrealType.h"

#define UE_API COREUOBJECT_API

class FName;
class FProperty;
class UObject;
class UStruct;
struct FPropertyChangedChainEvent;
class FEditPropertyChain;

namespace UE { class FPropertyPathName; }
namespace UE { class FPropertyTypeName; }

namespace UE
{

UE_API extern const FName NAME_Key;
UE_API extern const FName NAME_Value;

/**
 * Find a property in Struct that matches both the name and the type.
 *
 * Type matching uses FProperty::CanSerializeFromTypeName.
 *
 * @return A matching property if found, otherwise null.
 */
UE_API FProperty* FindPropertyByNameAndTypeName(const UStruct* Struct, FName Name, FPropertyTypeName TypeName);

/**
 * A reference to a single property value in a container.
 *
 * An example of accessing the value from a valid reference:
 * void* Data = Value.Property->ContainerPtrToValuePtr<void>(Value.Container, Value.ArrayIndex);
 *
 * An example of querying initialized property value state from a valid reference:
 * bool bInitialized = !Value.Struct ||
 *     FInitializedPropertyValueState(Value.Struct, Value.Container).IsSet(Value.Property, Value.ArrayIndex);
 */
struct FPropertyValueInContainer
{
	/** The referenced property. If Struct is set, Property is one of its properties. */
	const FProperty* Property = nullptr;
	/** The type of the container that contains the referenced property value. Null for a property in a container. */
	const UStruct* Struct = nullptr;
	/** The container that contains the referenced property value. If Struct is set, Container is an instance of it. */
	void* Container = nullptr;
	/** The static array index within the referenced property. Always non-negative in a valid reference. */
	int32 ArrayIndex = INDEX_NONE;

	inline explicit operator bool() const
	{
		return !!Property;
	}

	/** Returns a pointer to the property value that this references. */
	template <typename ValueType>
	inline ValueType* GetValuePtr() const
	{
		// A lambda template allows this to be defined with only a forward declaration of FProperty.
		return [this]<typename PropertyType>(PropertyType* P)
		{
			return P->template ContainerPtrToValuePtr<ValueType>(Container, ArrayIndex);
		}(Property);
	}

	/** If this value is for an element of a dynamic conatiner (for example an FArrayProperty element) then return true. */
	inline bool IsDynamicContainerElement() const
	{
		return Struct == nullptr;
	}
};

// FPropertyPathNames can have wildcarded container properties (signified by no index provided). 
// In that case, Value will be set to the first element and Next() will iterate to the next
class FPropertyPathNameResolver
{
public:
	UE_API explicit FPropertyPathNameResolver(const FPropertyPathName& Path, UObject* Object);

	FPropertyValueInContainer Value;
	const FPropertyPathName& Path;
	UObject* const Object;
	FEditPropertyChain EventChain;
	TArray<FPropertyValueInContainer> ContainerValues; // all the values of containers

	// returns true if the path was resolvable. In the case of a wildcarded path, returns false when all indices have been visited
	inline explicit operator bool() const
	{
		return !!Value;
	}

	// in the case of wildcarded paths, Next() will iterate to the next available index in the conatainer(s).
	UE_API FPropertyPathNameResolver& Next();
	UE_API bool BuildChangeEvent(TSharedPtr<FPropertyChangedChainEvent>& OutEvent, int32 ChangeType);
	UE_API void Reset();

private:
	FPropertyPathNameResolver& Resolve();
	TArray<TMap<FString, int32>> EventArrayIndices;

	struct FContainerIterator
	{
		int32 PathIndex;
		const FProperty* Property;
		void* NextContainer;
		const UStruct* NextStruct;
		int32 ChainLength;
		int32 ContainerValuesNum;
		int32 CurIndex;
	};

	// stack that holds the current iteration info of wildcard iterators
	TArray<FContainerIterator> ActiveContainerIterators;
};

/**
 * Try to resolve the property path to a property value within the object.
 *
 * @param Path The non-empty property path to resolve.
 * @param Object The object representing the root of the property path to resolve.
 * @return A valid property value reference if resolved, otherwise an invalid reference.
 */
UE_API FPropertyValueInContainer TryResolvePropertyPath(const FPropertyPathName& Path, UObject* Object);

} // UE

#undef UE_API
