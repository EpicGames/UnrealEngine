// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/PropertyTypeName.h"
#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/NameTypes.h"

#define UE_API COREUOBJECT_API

class FJsonValue;

namespace UE
{

/**
 * Represents one segment of a property path including the property type and the container index.
 */
struct FPropertyPathNameSegment
{
	/** Name of the property. */
	FName Name;
	/** Type of the property, if available. */
	FPropertyTypeName Type;
	/** Index within a container, or INDEX_NONE if not in a container. 
		Associative containers (Sets and Maps) use virtual indices that skip gaps instead of logical indices. */
	int32 Index = INDEX_NONE;

	[[nodiscard]] inline FName PackNameWithIndex() const
	{
		return FName(Name, NAME_EXTERNAL_TO_INTERNAL(Index));
	}

	[[nodiscard]] inline FPropertyPathNameSegment SetNameWithIndex(FName NameWithIndex) const
	{
		return {FName(NameWithIndex, NAME_NO_NUMBER_INTERNAL), Type, NAME_INTERNAL_TO_EXTERNAL(NameWithIndex.GetNumber())};
	}

	[[nodiscard]] inline FPropertyPathNameSegment SetType(FPropertyTypeName NewType) const
	{
		return {Name, NewType, Index};
	}

	UE_API TSharedRef<FJsonValue> ToJsonValue() const;
	UE_API static bool FromJsonValue(const TSharedPtr<FJsonValue>& JsonValue, FPropertyPathNameSegment& OutSegment);
};

/**
 * Represents the path to a property, by name, including the property type and the container index.
 *
 * Sequenced containers use the index directly and associative containers (Sets and Maps) use virtual indices that skip gaps instead of logical indices.
 */
class FPropertyPathName
{
	static_assert(NAME_NO_NUMBER == INDEX_NONE);

	struct FSegment
	{
		FName NameWithIndex;
		FPropertyTypeName Type;

		inline FPropertyPathNameSegment Unpack() const
		{
			return FPropertyPathNameSegment().SetNameWithIndex(NameWithIndex).SetType(Type);
		}

		inline static FSegment Pack(const FPropertyPathNameSegment& Segment)
		{
			return {Segment.PackNameWithIndex(), Segment.Type};
		}

		bool operator==(const FSegment& Segment) const;
		bool operator<(const FSegment& Segment) const;
		int32 Compare(const FSegment& Segment) const;
	};

public:
	/** Returns the number of segments in the path. */
	inline int32 GetSegmentCount() const
	{
		return Segments.Num();
	}

	/** Returns the segment at a valid index. */
	inline FPropertyPathNameSegment GetSegment(int32 Index) const
	{
		return Segments[Index].Unpack();
	}

	/** Sets the segment at a valid index. */
	inline void SetSegment(int32 Index, const FPropertyPathNameSegment& Segment)
	{
		Segments[Index] = FSegment::Pack(Segment);
	}

	/** Pushes a new segment on the end of the path. */
	inline void Push(const FPropertyPathNameSegment& Segment)
	{
		Segments.Emplace(FSegment::Pack(Segment));
	}

	/** Pops the last segment off the end of the path. */
	inline void Pop()
	{
		Segments.Pop(EAllowShrinking::No);
	}

	/** Sets the index of the last segment of the path. Ignored if the path is empty. */
	inline void SetIndex(int32 Index)
	{
		if (!Segments.IsEmpty())
		{
			Segments.Last().NameWithIndex.SetNumber(NAME_EXTERNAL_TO_INTERNAL(Index));
		}
	}

	inline bool IsEmpty() const { return Segments.IsEmpty(); }
	inline void Empty() { Segments.Empty(); }
	inline void Reset() { Segments.Reset(); }

	UE_API bool operator==(const FPropertyPathName& Path) const;
	UE_API bool operator<(const FPropertyPathName& Path) const;

	inline bool operator!=(const FPropertyPathName& Path) const { return !(*this == Path); }
	inline bool operator<=(const FPropertyPathName& Path) const { return !(Path < *this); }
	inline bool operator>=(const FPropertyPathName& Path) const { return !(*this < Path); }
	inline bool operator>(const FPropertyPathName& Path) const { return (Path < *this); }

	friend inline FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FPropertyPathName& Path)
	{
		Path.ToString(Builder);
		return Builder;
	}

	UE_API void ToString(FStringBuilderBase& Out, FStringView Separator = TEXTVIEW(" -> ")) const;

	UE_API TSharedRef<FJsonValue> ToJsonValue() const;
	UE_API static bool FromJsonValue(const TSharedPtr<FJsonValue>& JsonValue, FPropertyPathName& OutPath);

	UE_API friend uint32 GetTypeHash(const FPropertyPathName& Path);

private:
	TArray<FSegment> Segments;
};

} // UE

#undef UE_API
