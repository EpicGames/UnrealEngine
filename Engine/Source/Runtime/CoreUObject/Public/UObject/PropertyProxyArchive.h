// Copyright Epic Games, Inc. All Rights Reserved.

/*==================================================================================================
	PropertyProxyArchive.h: Simple proxy archive for serializing references to FFields from Bytecode
====================================================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "UObject/FieldPath.h"
#include "Serialization/ArchiveProxy.h"

/**
  * Simple proxy archive for serializing references to FFields from Bytecode
  */
class FPropertyProxyArchive : public FArchiveProxy
{
public:

	/** List of fields that could not be resolved at serialize time possibly due to their classes not being fully loaded yet */
	UStruct::FUnresolvedScriptPropertiesArray UnresolvedProperties;
	/** Current bytecode offset */
	int32& BytecodeIndex;
	/** Script container object */
	UStruct* Container;

	FPropertyProxyArchive(FArchive& InInnerArchive, int32& InBytecodeIndex, UStruct* InContainer)
		: FArchiveProxy(InInnerArchive)
		, BytecodeIndex(InBytecodeIndex)
		, Container(InContainer)
	{
		ArIsFilterEditorOnly = InnerArchive.ArIsFilterEditorOnly;
	}
	virtual FArchive& operator<<(FField*& Value) override
	{
		if (!IsPersistent() || IsObjectReferenceCollector())
		{
			// For reference collectors (like FArchiveReplaceFieldReferences): fully serialize the entire field to find all of its UObject references
			InnerArchive << Value;
		}

		// Serialize the field as FFieldPath
		TFieldPath<FField> PropertyPath(Value);
		*this << PropertyPath;
		if (IsLoading())
		{
			Value = PropertyPath.Get(Container);
			if (!Value && !PropertyPath.IsPathToFieldEmpty())
			{
				// Store the field path for deferred resolving
				UnresolvedProperties.Add(TPair<TFieldPath<FField>, int32>(PropertyPath, BytecodeIndex));
			}
		}
		return *this;
	}
};