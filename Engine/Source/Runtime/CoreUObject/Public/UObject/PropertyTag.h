// Copyright Epic Games, Inc. All Rights Reserved.

/*-----------------------------------------------------------------------------
	FPropertyTag.
-----------------------------------------------------------------------------*/

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/NameTypes.h"
#include "UObject/PropertyTypeName.h"

enum class EOverriddenPropertyOperation : uint8;
class FArchive;
class FProperty;

/**
 * Used by the tag to describe how the property was serialized.
 */
enum class EPropertyTagSerializeType : uint8
{
	/** Tag was loaded from an older version or has not yet been saved. */
	Unknown,
	/** Serialization of the property value was skipped. Tag has no value. */
	Skipped,
	/** Serialized with tagged property serialization. */
	Property,
	/** Serialized with binary or native serialization. */
	BinaryOrNative,
};

/**
 *  A tag describing a class property, to aid in serialization.
 */
struct FPropertyTag
{
private:
	FProperty* Prop = nullptr; // Transient
	UE::FPropertyTypeName TypeName;

public:
	FName	Type;		// Type of property
	FName	Name;		// Name of property.
	int32	Size = 0;   // Property size.
	int32	ArrayIndex = INDEX_NONE; // Index if an array; else 0.
	int64	SizeOffset = INDEX_NONE; // location in stream of tag size member
	FGuid	PropertyGuid;
	uint8	HasPropertyGuid = 0;
	uint8	BoolVal = 0;// a boolean property's value (never need to serialize data for bool properties except here)
	EPropertyTagSerializeType SerializeType = EPropertyTagSerializeType::Unknown;
	EOverriddenPropertyOperation OverrideOperation; // Overridable serialization state reconstruction 
	bool	bExperimentalOverridableLogic = false; // Remember if property had CPF_ExperimentalOverridableLogic when saved
	TOptional<bool>	bExperimentalExternalObjects; // Remember if property had CPF_ExperimentalOverridableLogic|CPF_ExperimentalExternalObjects when saved and the object enabled serialization of external objects for that property

	// Constructors.
	FPropertyTag();
	UE_INTERNAL COREUOBJECT_API FPropertyTag(FProperty* Property, int32 InIndex, uint8* Value);

	inline FProperty* GetProperty() const
	{
		return Prop;
	}

	void SetProperty(FProperty* Property);
	void SetPropertyGuid(const FGuid& InPropertyGuid);

	inline UE::FPropertyTypeName GetType() const { return TypeName; }
	void SetType(UE::FPropertyTypeName TypeName);

	// Serializer.
	friend FArchive& operator<<(FArchive& Ar, FPropertyTag& Tag);
	friend void operator<<(FStructuredArchive::FSlot Slot, FPropertyTag& Tag);

	// Property serializer.
	void SerializeTaggedProperty(FArchive& Ar, FProperty* Property, uint8* Value, const uint8* Defaults) const;
	UE_INTERNAL COREUOBJECT_API void SerializeTaggedProperty(FStructuredArchive::FSlot Slot, FProperty* Property, uint8* Value, const uint8* Defaults) const;

private:
	friend void LoadPropertyTagNoFullType(FStructuredArchive::FSlot Slot, FPropertyTag& Tag);
	friend void SerializePropertyTagAsText(FStructuredArchive::FSlot Slot, FPropertyTag& Tag);
};

struct UE_INTERNAL FPropertyTagScope
{
	FPropertyTagScope(const FPropertyTag* InCurrentPropertyTag)
	: PropertyTagToRestore(CurrentPropertyTag)
	{
		CurrentPropertyTag = InCurrentPropertyTag;
	}

	~FPropertyTagScope()
	{
		CurrentPropertyTag = PropertyTagToRestore;
	}

	static UE_FORCEINLINE_HINT const FPropertyTag* GetCurrentPropertyTag()
	{
		return CurrentPropertyTag;
	}
private:
	const FPropertyTag* PropertyTagToRestore;

	static thread_local const FPropertyTag* CurrentPropertyTag;
};
