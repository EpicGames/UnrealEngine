// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/UnrealType.h"

class FArchive;
class FProperty;
class FString;
class UObject;

// A helper struct which owns a single instance of the type pointed to by a property.
// The instance is properly constructed, destructed and can be serialized and have other
// functions called on it.
struct FPropertyTempVal
{
	COREUOBJECT_API explicit FPropertyTempVal(FProperty* InProp);
	COREUOBJECT_API ~FPropertyTempVal();

	// Uncopyable
	FPropertyTempVal(const FPropertyTempVal&) = delete;
	FPropertyTempVal& operator=(const FPropertyTempVal&) = delete;

	// Serializes the instance
	COREUOBJECT_API void Serialize(FArchive& Ar, const void* Defaults = nullptr);

	// Exports the text of the instance
	COREUOBJECT_API void ExportText(FString& ValueStr, const void* Defaults = nullptr, UObject* Parent = nullptr, int32 PortFlags = 0, UObject* ExportRootScope = nullptr);

	// Returns a pointer to the internal instance
	UE_FORCEINLINE_HINT void* Get()
	{
		return Value;
	}

	// Returns a pointer to the internal instance
	UE_FORCEINLINE_HINT const void* Get() const
	{
		return Value;
	}

private:
	// The property which is used to manage to underlying instance.
	FProperty* Prop;

	// The memory of 
	void* Value;
};
