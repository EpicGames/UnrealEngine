// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/NameTypes.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"

#include "PropertyText.generated.h"

// Structs that support UPROPERTY(..., meta = (GetOptions = <function>)) on FName and FString
// properties which allows the display name to be set separately from the value for a given FName or FString
//
// Example usage:
//
// UCLASS()
// class UMyClass : public UObject
// {
// 	GENERATED_BODY()
// public:
//     UPROPERTY(EditAnywhere, meta = (GetOptions = GetFooOptions))
// 	FName Foo;
//
// 	UFUNCTION()
// 	static TArray<FPropertyTextFName> GetFooOptions();
// };


/**
 * Tuple returned by <function> in a UPROPERTY(..., meta = (GetOptions = <function>))
 * allowing value to be specified as a FName
 */
USTRUCT()
struct FPropertyTextFName
{
	GENERATED_BODY()
	
	FName ValueString;
	FText DisplayName;
};

/**
 * Tuple returned by <function> in a UPROPERTY(..., meta = (GetOptions = <function>))
 * allowing value to be specified as a FString
 */
USTRUCT()
struct FPropertyTextString
{
	GENERATED_BODY()
	FString ValueString;
	FText DisplayName;
};
