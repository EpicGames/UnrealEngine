// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "UObject/NameTypes.h"

#define UE_API COREUOBJECT_API

class FArchive;
class FProperty;
class FStructuredArchiveSlot;
class UField;
struct FGuid;

namespace UE { class FPropertyTypeNameBuilder; }

namespace UE
{

struct FPropertyTypeNameNode
{
	FName Name;
	int32 InnerCount = 0;
};

/**
 * Represents the type name of a property, including any containers and underlying types.
 *
 * The path name for a type is represented as the object name with the outer chain as type parameters.
 * - /Script/CoreUObject.Outer0:Outer1.Outer2.StructName -> StructName(/Script/CoreUObject,Outer0,Outer1,Outer2)
 *
 * Examples:
 * - int32 -> IntProperty
 * - TArray<int32> -> ArrayProperty(IntProperty)
 * - TArray<FStructType> -> ArrayProperty(StructProperty(StructType(/Script/Module),e21f566f-7153-433a-959d-bfb3abed17e2))
 * - TMap<FKeyStruct, EByteEnum> -> MapProperty(StructProperty(KeyStruct(/Script/Module)),EnumProperty(ByteEnum(/Script/Module),ByteProperty))
 *
 * Incomplete property types created from sources with incomplete type information. Consumers must support this.
 * - ArrayProperty(StructProperty)
 * - MapProperty(EnumProperty,StructProperty)
 */
class FPropertyTypeName
{
public:
	FPropertyTypeName() = default;

	[[nodiscard]] UE_API explicit FPropertyTypeName(const FProperty* Property);

	[[nodiscard]] inline bool IsEmpty() const
	{
		return Index == 0;
	}

	/**
	 * Returns the type at the root of this property type name.
	 *
	 * Example: MapProperty(StructProperty(KeyStruct),EnumProperty(ByteEnum,ByteProperty))
	 * - GetName() -> MapProperty
	 */
	[[nodiscard]] UE_API FName GetName() const;

	/**
	 * Returns the number of type parameters under the root of this property type name.
	 *
	 * Example: MapProperty(StructProperty(KeyStruct),EnumProperty(ByteEnum,ByteProperty))
	 * - GetParameterCount() -> 2
	 */
	[[nodiscard]] UE_API int32 GetParameterCount() const;

	/**
	 * Returns the indexed parameter under the root of this property type name.
	 *
	 * An out-of-bounds index will return an empty type name.
	 *
	 * Example: MapProperty(StructProperty(KeyStruct),EnumProperty(ByteEnum,ByteProperty))
	 * - GetParameter(0) -> StructProperty(KeyStruct)
	 * - GetParameter(1) -> EnumProperty(ByteEnum,ByteProperty)
	 */
	[[nodiscard]] UE_API FPropertyTypeName GetParameter(int32 ParamIndex) const;

	/**
	 * Returns the indexed parameter type name under the root of this property type name.
	 *
	 * An out-of-bounds index will return a name of None.
	 *
	 * Example: MapProperty(StructProperty(KeyStruct),EnumProperty(ByteEnum,ByteProperty))
	 * - GetParameterName(0) -> StructProperty
	 * - GetParameterName(1) -> EnumProperty
	 */
	[[nodiscard]] inline FName GetParameterName(int32 ParamIndex) const
	{
		return GetParameter(ParamIndex).GetName();
	}

	/**
	 * Resets this to an empty type name.
	 */
	inline void Reset()
	{
		Index = 0;
	}

	/** Returns true if this is StructProperty with a first parameter of StructName. */
	[[nodiscard]] UE_API bool IsStruct(FName StructName) const;
	/** Returns true if this is EnumProperty or ByteProperty with a first parameter of EnumName. */
	[[nodiscard]] UE_API bool IsEnum(FName EnumName) const;

private:
	UE_API friend uint32 GetTypeHash(const FPropertyTypeName& TypeName);
	UE_API friend void AppendHash(FBlake3& Builder, const FPropertyTypeName& TypeName);

	UE_API friend bool operator==(const FPropertyTypeName& Lhs, const FPropertyTypeName& Rhs);
	UE_API friend bool operator<(const FPropertyTypeName& Lhs, const FPropertyTypeName& Rhs);

	UE_API friend FArchive& operator<<(FArchive& Ar, FPropertyTypeName& TypeName);
	UE_API friend void operator<<(FStructuredArchiveSlot Slot, FPropertyTypeName& TypeName);

	UE_API friend FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FPropertyTypeName& TypeName);

	int32 Index = 0;

	friend FPropertyTypeNameBuilder;
};

/**
 * Builder for FPropertyTypeName.
 *
 * Example: MapProperty(StructProperty(KeyStruct(/Script/CoreUObject),StructGuid),EnumProperty(ByteEnum(/Script/CoreUObject),ByteProperty))
 *
 * FPropertyTypeNameBuilder Builder;
 * Builder.AddName(NAME_MapProperty));
 * Builder.BeginParameters();
 *   Builder.AddName(NAME_StructProperty);
 *   Builder.BeginParameters();
 *     Builder.AddPath(KeyStruct); // const UStruct*
 *     if (const FGuid StructGuid = KeyStruct->GetCustomGuid(); StructGuid.IsValid())
 *         Builder.AddGuid(StructGuid);
 *   Builder.EndParameters();
 *   Builder.AddName(NAME_EnumProperty);
 *   Builder.BeginParameters();
 *     Builder.AddPath(ByteEnum); // const UEnum*
 *     Builder.AddName(NAME_ByteProperty);
 *   Builder.EndParameters();
 * Builder.EndParameters();
 * FPropertyTypeName Name = Builder.Build();
 */
class FPropertyTypeNameBuilder
{
public:
	/** Add a name without any type parameters. */
	UE_API void AddName(FName Name);

	/** Add a guid in the format DigitsWithHyphensLower. */
	UE_API void AddGuid(const FGuid& Guid);

	/** Add a path in the format ObjectName(/Path/Name,Outer0,Outer1,Outer2) */
	UE_API void AddPath(const UField* Field);

	/** Add a type name with its parameters. */
	UE_API void AddType(FPropertyTypeName Name);

	/** Mark the beginning of the type parameters for the last added type. */
	UE_API void BeginParameters();

	/** Mark the end of the type parameters for the matching begin. */
	UE_API void EndParameters();

	/** Build from the types that have been added. */
	[[nodiscard]] UE_API FPropertyTypeName Build() const;

	/** Resets the builder to allow it to be reused. */
	UE_API void Reset();

	/**
	 * Try to parse and add a complete type name with its type parameters.
	 *
	 * Builder is restored to its previous state when parsing fails.
	 *
	 * @param Name A name in the format returned by operator<<(FStringBuilderBase&, const FPropertyTypeName&).
	 * @return true if a name was parsed into the builder, false on failure.
	 */
	[[nodiscard]] UE_API bool TryParse(FStringView Name);

private:
	[[nodiscard]] UE_REWRITE friend auto begin(const FPropertyTypeNameBuilder& Builder) { return Builder.Nodes.begin(); }
	[[nodiscard]] UE_REWRITE friend auto end(const FPropertyTypeNameBuilder& Builder) { return Builder.Nodes.end(); }

	TArray<FPropertyTypeNameNode, TInlineAllocator<10>> Nodes;
	TArray<int32, TInlineAllocator<10>> OuterNodeIndex;
	int32 ActiveIndex = INDEX_NONE;
};

} // UE

#undef UE_API
