// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Templates/Function.h"

class FProperty;
class FEditPropertyChain;
struct FPropertyChangedEvent;
struct FArchiveSerializedPropertyChain;

enum class EPropertyVisitorControlFlow : uint8
{
	Stop, // Stop the visit
	StepOver, // Skip over to the next property or item
	StepOut, // Stop iteration at this level and continue on the outer on the next property or item
	StepInto, // Introspect the inner properties if any
};

enum class EPropertyVisitorInfoType : uint8
{
	None, // Property is not inside a container
	StaticArrayIndex, // Property is a static array and has a valid index
	ContainerIndex, // Property is inside a container and has a valid index
	MapKey, // Property represents a key of the map container and has a valid index
	MapValue, // Property represents a value of the map container and has a valid index
};

struct FPropertyVisitorInfo
{
	explicit FPropertyVisitorInfo(const FProperty* InProperty
		, int32 InIndex = INDEX_NONE
		, EPropertyVisitorInfoType InPropertyInfo = EPropertyVisitorInfoType::None
		, const UStruct* InParentStructType = nullptr
		)
		: Property(InProperty)
		, ParentStructType(InParentStructType)
		, Index(InIndex)
		, PropertyInfo(InPropertyInfo)
	{
	}

	explicit FPropertyVisitorInfo(const FProperty* InProperty, const UStruct* InParentStructType)
	: Property(InProperty)
	, ParentStructType(InParentStructType)
	{
	}

	FPropertyVisitorInfo(const FPropertyVisitorInfo&) = default;
	FPropertyVisitorInfo(FPropertyVisitorInfo&&) = default;

	FPropertyVisitorInfo& operator=(const FPropertyVisitorInfo&) = default;
	FPropertyVisitorInfo& operator=(FPropertyVisitorInfo&&) = default;

	COREUOBJECT_API bool Identical(const FPropertyVisitorInfo& Other) const;

	/** @note The default comparison only compares the key data to match GetTypeHash and is for use with hashed containers and ResolveVisitedPathInfo_Generic; use Identical for an exact comparison */
	friend bool operator==(const FPropertyVisitorInfo& A, const FPropertyVisitorInfo& B)
	{
		return A.Property == B.Property && A.PropertyInfo == B.PropertyInfo && A.Index == B.Index;
	}
	friend bool operator!=(const FPropertyVisitorInfo& A, const FPropertyVisitorInfo& B)
	{
		return !(A == B);
	}

	friend uint32 GetTypeHash(const FPropertyVisitorInfo& A)
	{
		return HashCombine(HashCombine(GetTypeHash(A.Property), GetTypeHash(A.PropertyInfo)), GetTypeHash(A.Index));
	}

	/** The property currently being visited */
	const FProperty* Property = nullptr;

	/**
	 * The parent struct that provided the property being iterated, if iterating a sub-property within a struct.
	 * @note This is slightly different from Property->GetOwnerStruct() as you might be iterating a FDerived instance 
	 *       but processing a FBase struct property. In this case this will be set to FDerived rather than FBase.
	 */
	const UStruct* ParentStructType = nullptr;

	/**
     * Index of the element being visited in the container, otherwise INDEX_NONE.
     * For maps and sets it indicates the logical index. */
	int32 Index = INDEX_NONE;

	/** Whether this property is inside a container and if it is key or a value of a map */
	EPropertyVisitorInfoType PropertyInfo = EPropertyVisitorInfoType::None;

	/* Indicate that this property contains inner properties */
	bool bContainsInnerProperties = false;
};

struct FPropertyVisitorPath
{
public:
	FPropertyVisitorPath() = default;
	FPropertyVisitorPath(TFunction<void(const FPropertyVisitorInfo&)> InOnPushFunc, TFunction<void(const FPropertyVisitorInfo&)> InOnPopFunc)
		: OnPushFunc(InOnPushFunc)
		, OnPopFunc(InOnPopFunc)
	{
	}

	FPropertyVisitorPath(const FPropertyVisitorPath&) = default;
	FPropertyVisitorPath(FPropertyVisitorPath&&) = default;

	FPropertyVisitorPath& operator=(const FPropertyVisitorPath&) = default;
	FPropertyVisitorPath& operator=(FPropertyVisitorPath&&) = default;
	
	explicit FPropertyVisitorPath(const FPropertyVisitorInfo& Info)
	{
		Push(Info);
	}

	explicit FPropertyVisitorPath(TArrayView<const FPropertyVisitorInfo> InPath)
	{
		Path = InPath;
	}

	explicit COREUOBJECT_API FPropertyVisitorPath(const FPropertyChangedEvent& PropertyEvent, const FEditPropertyChain& PropertyChain);
	explicit COREUOBJECT_API FPropertyVisitorPath(const FArchiveSerializedPropertyChain& PropertyChain);

	void Push(const FPropertyVisitorInfo& Info)
	{
		Path.Push(Info);
		if (OnPushFunc)
		{
			OnPushFunc(Info);
		}
	}

	FPropertyVisitorInfo Pop()
	{
		FPropertyVisitorInfo Info = Path.Pop(EAllowShrinking::No);
		if (OnPopFunc)
		{
			OnPopFunc(Info);
		}
		return MoveTemp(Info);
	}

	int32 Num() const
	{
		return Path.Num();
	}

	FPropertyVisitorInfo& Top()
	{
		return Path.Top();
	}

	const FPropertyVisitorInfo& Top() const
	{
		return Path.Top();
	}

	const TArray<FPropertyVisitorInfo>& GetPath() const
	{
		return Path;
	}

	friend bool operator==(const FPropertyVisitorPath& A, const FPropertyVisitorPath& B)
	{
		return A.Path == B.Path;
	}
	friend bool operator!=(const FPropertyVisitorPath& A, const FPropertyVisitorPath& B)
	{
		return !(A == B);
	}

	friend uint32 GetTypeHash(const FPropertyVisitorPath& A)
	{
		return GetTypeHash(A.Path);
	}

	COREUOBJECT_API FString ToString(const TCHAR* Separator = TEXT(".")) const;
	COREUOBJECT_API void ToString(FStringBuilderBase& Out, const TCHAR* Separator = TEXT(".")) const;
	COREUOBJECT_API void AppendString(FStringBuilderBase& Out, const TCHAR* Separator = TEXT(".")) const;

	/**
	 * Is this property path contained in the specified one
	 * @param OtherPath property path to check if it is contained in
	 * @param bIsEqual optional parameter to know if it fully match
	 * @return true if it is contained in the specified path
	 */
	COREUOBJECT_API bool Contained(const FPropertyVisitorPath& OtherPath, bool* bIsEqual = nullptr) const;

	/**
	 * Retrieves the data using the specified root object
	 * @param Object to search into
	 * @return 
	 */
	COREUOBJECT_API void* GetPropertyDataPtr(UObject* Object) const;

	/** Iterator for a property visitor path */
	using Iterator = TArray<FPropertyVisitorInfo>::TConstIterator;

	/**
	 * Returns an iterator on the root path node, useful when calling methods that are recursive
	 * @return an iterator on the root path node
	 */
	Iterator GetRootIterator() const
	{
		return Iterator(Path);
	}

	/**
	 * Invalid iterator pointing to an empty path
	 * @return an iterator to an empty path
	 */
	COREUOBJECT_API static Iterator InvalidIterator();

	/**
	 * Converts path to an archive serialized property chain
	 * This method useful when APIs like in the FOverridableManager are taking in a
	 * FArchiveSerializedPropertyChain parameter and would like to use them.
	 * It is used in many places in the FProperty::ImportText
	 * @return the archive serialized property chain built from the property path
	 */
	COREUOBJECT_API FArchiveSerializedPropertyChain ToSerializedPropertyChain() const;

protected:
	TArray<FPropertyVisitorInfo> Path;
	TFunction<void(const FPropertyVisitorInfo&)> OnPushFunc;
	TFunction<void(const FPropertyVisitorInfo&)> OnPopFunc;
};

struct FPropertyVisitorScope
{
public:
	FPropertyVisitorScope(FPropertyVisitorPath& InPath, const FPropertyVisitorInfo& Info)
		: Path(InPath)
	{
		Path.Push(Info);
	}

	~FPropertyVisitorScope()
	{
		Path.Pop();
	}

protected:
	FPropertyVisitorPath& Path;
};

struct FPropertyVisitorData
{
	explicit FPropertyVisitorData(void* InPropertyData, void* InParentStructData)
		: PropertyData(InPropertyData)
		, ParentStructData(InParentStructData)
	{}

	/** Utility that constructs a new visitor data object with new property data but the same parent struct data */
	FPropertyVisitorData VisitPropertyData(void* InPropertyData) const
	{
		return FPropertyVisitorData(InPropertyData, ParentStructData);
	}

	/** Data associated with the property being iterated */
	void* PropertyData = nullptr;
	/** Data associated with the parent struct that provided the property being iterated */
	void* ParentStructData = nullptr;
};

struct FPropertyVisitorContext
{
	enum class EScope : uint8
	{
		// Visits all the properties (default)
		All,
		// Visits only the object reference properties
		ObjectRefs
	};

	explicit FPropertyVisitorContext(FPropertyVisitorPath& InPath, const FPropertyVisitorData& InData, const EScope InScope = EScope::All)
		: Path(InPath)
		, Data(InData)
		, Scope(InScope)
	{}

	/** Utility that constructs a new visitor context object with new property data but the same path and scope*/
	FPropertyVisitorContext VisitPropertyData(void* InPropertyData) const
	{
		FPropertyVisitorData PropertyData = Data.VisitPropertyData(InPropertyData);
		return FPropertyVisitorContext(Path, PropertyData, Scope);
	}

	FPropertyVisitorPath& Path;
	const FPropertyVisitorData Data;
	const EScope Scope;
};

namespace PropertyVisitorHelpers
{

namespace Private
{

template <typename Type>
void* ResolveVisitedPathInfo(const Type* This, void* Data, const FPropertyVisitorInfo& Info)
{
	return This->ResolveVisitedPathInfo(Data, Info);
}

} // namespace Private

/**
 * Given a FPropertyVisitorPath, attempt to resolve that to a valid data pointer.
 * RootObject is required to implement ResolveVisitedPathInfo to provide the resolver logic.
 */
template <typename Type>
void* ResolveVisitedPath(const Type* RootObject, void* RootData, const FPropertyVisitorPath& Path)
{
	void* FoundPropertyData = nullptr;
	if (const TArray<FPropertyVisitorInfo>& PathArray = Path.GetPath();
		PathArray.Num() > 0)
	{
		FoundPropertyData = Private::ResolveVisitedPathInfo(RootObject, RootData, PathArray[0]);
		for (int32 PathIndex = 1; FoundPropertyData && PathIndex < PathArray.Num(); ++PathIndex)
		{
			const FPropertyVisitorInfo& PreviousInfo = PathArray[PathIndex - 1];
			FoundPropertyData = Private::ResolveVisitedPathInfo(PreviousInfo.Property, FoundPropertyData, PathArray[PathIndex]);
		}
	}
	return FoundPropertyData;
}

/**
 * A generic implementation of ResolveVisitedPathInfo that uses Visit to find the property data pointer.
 * This may be used as the ResolveVisitedPathInfo implementation for your type if it doesn't have a more optimized version.
 */
template <typename Type>
void* ResolveVisitedPathInfo_Generic(Type* This, FPropertyVisitorPath& Path, void* Data, const FPropertyVisitorInfo& Info)
{
	void* FoundInnerData = nullptr;

	FPropertyVisitorData VisitorData(Data, /*ParentStructData*/nullptr);

	FPropertyVisitorContext Context(Path, VisitorData);
	This->Visit(Context, [&FoundInnerData, &Info, InnerPathDepth = Path.Num() + 1](const FPropertyVisitorContext& Context)
	{
		const FPropertyVisitorPath& InnerPath = Context.Path;
		const FPropertyVisitorData& InnerVisitorData = Context.Data;

		if (InnerPath.Num() < InnerPathDepth)
		{
			return EPropertyVisitorControlFlow::StepInto;
		}

		void* InnerData = InnerVisitorData.PropertyData;

		if (Info == InnerPath.Top())
		{
			FoundInnerData = InnerData;
			return EPropertyVisitorControlFlow::Stop;
		}
		return EPropertyVisitorControlFlow::StepOver;
	});
	return FoundInnerData;
}
template <typename Type>
void* ResolveVisitedPathInfo_Generic(Type* This, void* Data, const FPropertyVisitorInfo& Info)
{
	FPropertyVisitorPath Path;
	return ResolveVisitedPathInfo_Generic(This, Path, Data, Info);
}

UE_DEPRECATED(5.7, "Visit is deprecated, please use Visit with context instead.")
COREUOBJECT_API EPropertyVisitorControlFlow VisitProperty(const UStruct* PropertyOwner, const FProperty* Property, FPropertyVisitorPath& Path, const FPropertyVisitorData& InData, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorPath& /*Path*/, const FPropertyVisitorData& /*Data*/)> InFunc);

/** Visit the property from an instance. */
COREUOBJECT_API EPropertyVisitorControlFlow VisitProperty(const UStruct* PropertyOwner, const FProperty* Property, FPropertyVisitorContext& Context, const TFunctionRef<EPropertyVisitorControlFlow(const FPropertyVisitorContext& /*Context*/)> InFunc);

/** Convert the given path to a string */
COREUOBJECT_API FString PathToString(TArrayView<const FPropertyVisitorInfo> Path, const TCHAR* Separator = TEXT("."));
COREUOBJECT_API void PathToString(TArrayView<const FPropertyVisitorInfo> Path, FStringBuilderBase& Out, const TCHAR* Separator = TEXT("."));
COREUOBJECT_API void PathAppendString(TArrayView<const FPropertyVisitorInfo> Path, FStringBuilderBase& Out, const TCHAR* Separator = TEXT("."));

/**
 * Is this property path contained in the specified one
 * @param OtherPath property path to check if it is contained in
 * @param bIsEqual optional parameter to know if it fully match
 * @return true if it is contained in the specified path
 */
COREUOBJECT_API bool PathIsContainedWithin(TArrayView<const FPropertyVisitorInfo> Path, TArrayView<const FPropertyVisitorInfo> OtherPath, bool* bIsEqual = nullptr);

/** Convert the given path to a serialized property chain */
COREUOBJECT_API FArchiveSerializedPropertyChain PathToSerializedPropertyChain(TArrayView<const FPropertyVisitorInfo> Path);

} // namespace PropertyVisitorHelpers
