// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"


template <typename PropertyBaseClass>
class TPropertyWithSetterAndGetter : public PropertyBaseClass
{
public:
	template <typename PropertyCodegenParams>
	TPropertyWithSetterAndGetter(FFieldVariant InOwner, const PropertyCodegenParams& Prop)
		: PropertyBaseClass(InOwner, Prop)
		, SetterFunc(Prop.SetterFunc)
		, GetterFunc(Prop.GetterFunc)
	{
	}

#if UE_WITH_CONSTINIT_UOBJECT
	template<typename... ArgTypes>
	consteval TPropertyWithSetterAndGetter(SetterFuncPtr InSetterFunc, GetterFuncPtr InGetterFunc, UE::CodeGen::ConstInit::FPropertyParams PropertyParams, ArgTypes&&... InArgs)
		: PropertyBaseClass(PropertyParams, Forward<ArgTypes>(InArgs)...)
		, SetterFunc(InSetterFunc)
		, GetterFunc(InGetterFunc)
	{
	}
#endif

	virtual bool HasSetter() const override
	{
		return !!SetterFunc;
	}

	virtual bool HasGetter() const override
	{
		return !!GetterFunc;
	}

	virtual bool HasSetterOrGetter() const override
	{
		return !!SetterFunc || !!GetterFunc;
	}

	virtual void CallSetter(void* Container, const void* InValue) const override
	{
		checkf(SetterFunc, TEXT("Calling a setter on %s but the property has no setter defined."), *PropertyBaseClass::GetFullName());
		SetterFunc(Container, InValue);
	}

	virtual void CallGetter(const void* Container, void* OutValue) const override
	{
		checkf(GetterFunc, TEXT("Calling a getter on %s but the property has no getter defined."), *PropertyBaseClass::GetFullName());
		GetterFunc(Container, OutValue);
	}

protected:

	SetterFuncPtr SetterFunc = nullptr;
	GetterFuncPtr GetterFunc = nullptr;
};
