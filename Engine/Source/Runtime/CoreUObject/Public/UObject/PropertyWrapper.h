// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "PropertyWrapper.generated.h"

#define UE_API COREUOBJECT_API

/**
 * FProperty wrapper object.
 * The purpose of this object is to provide a UObject wrapper for native FProperties that can
 * be used by property editors (grids).
 * Specialized wrappers can be used to allow specialized editors for specific property types.
 * Property wrappers are owned by UStruct that owns the property they wrap and are tied to its lifetime
 * so that weak object pointer functionality works as expected.
 */
UCLASS(Transient, MinimalAPI)
class UPropertyWrapper : public UObject
{
	GENERATED_BODY()

public:
	UPropertyWrapper(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	{
	}
protected:
	/** Cached property object */
	FProperty* DestProperty;
public:
	/** Sets the property this object wraps */
	void SetProperty(FProperty* InProperty)
	{
		DestProperty = InProperty;
	}
	/* Gets property wrapped by this object */
	FProperty* GetProperty()
	{
		return DestProperty;
	}
	/* Gets property wrapped by this object */
	const FProperty* GetProperty() const
	{
		return DestProperty;
	}
};

UCLASS(Transient, MinimalAPI)
class UMulticastDelegatePropertyWrapper : public UPropertyWrapper
{
	GENERATED_BODY()

public:
	UMulticastDelegatePropertyWrapper(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	{
	}
};

UCLASS(Transient, MinimalAPI)
class UMulticastInlineDelegatePropertyWrapper : public UMulticastDelegatePropertyWrapper
{
	GENERATED_BODY()

public:
	UMulticastInlineDelegatePropertyWrapper(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	{
	}
};

#undef UE_API 