// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/UnrealType.h"

/** Reference collector that will purge (null) any references to the given set of objects (as if they'd been marked PendingKill) */
class FPurgingReferenceCollector : public FReferenceCollector
{
public:
	bool HasObjectToPurge() const
	{
		return ObjectsToPurge.Num() > 0;
	}

	void AddObjectToPurge(const UObject* Object)
	{
		ObjectsToPurge.Add(Object);
	}

	virtual bool IsIgnoringArchetypeRef() const override
	{
		return false;
	}

	virtual bool IsIgnoringTransient() const override
	{
		return false;
	}

protected:
	virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* ReferencingProperty) override
	{
		ConditionalPurgeObject(Object);
	}

	virtual void HandleObjectReferences(UObject** InObjects, const int32 ObjectNum, const UObject* InReferencingObject, const FProperty* InReferencingProperty) override
	{
		for (int32 ObjectIndex = 0; ObjectIndex < ObjectNum; ++ObjectIndex)
		{
			UObject*& Object = InObjects[ObjectIndex];
			ConditionalPurgeObject(Object);
		}
	}

	void ConditionalPurgeObject(UObject*& Object)
	{
		if (ObjectsToPurge.Contains(Object))
		{
			Object = nullptr;
		}
	}

	TSet<const UObject*> ObjectsToPurge;
};
