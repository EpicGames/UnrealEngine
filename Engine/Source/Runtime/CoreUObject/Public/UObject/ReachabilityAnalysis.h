// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ReachabilityAnalysis.h: Utility functions for configuring and running
	Incremental reachability analysis
=============================================================================*/

#pragma once

#include "CoreMinimal.h"

struct FUObjectItem;

/**
 * Enables or disables incremental reachability analysis.
 *
 * @param	bEnabled True if incremental reachability analysis is to be enabled
 */
COREUOBJECT_API void SetIncrementalReachabilityAnalysisEnabled(bool bEnabled);

/**
 * Returns true if incremental reachability analysis is enabled
 */
COREUOBJECT_API bool GetIncrementalReachabilityAnalysisEnabled();

/**
 * Sets time limit for incremental rachability analysis (if enabled).
 *
 * @param	TimeLimitSeconds Time limit (in seconds) for incremental raachability analysis
 */
COREUOBJECT_API void SetReachabilityAnalysisTimeLimit(float TimeLimitSeconds);

/**
 * Returns the time limit for incremental rachability analysis.
 *
 * @return	time limit (in seconds) for incremental rachability analysis.
 */
COREUOBJECT_API float GetReachabilityAnalysisTimeLimit();


namespace UE::GC
{

/**
 * Entry in GUnreachableObjects array that can either be an FUObjectItem (initially) or an UObject after unreachable objects had their InternalIndex freed.
 */
union FUnreachableObject
{
	FUObjectItem* ObjectItem;
	UObject* Object;
};

}

namespace UE::GC::Private
{

struct FIterationTimerStat
{
	double Total = 0.0;
	double Max = 0.0;
	int32 NumIterations = 0;
	int32 SlowestIteration = 0;

	double operator += (double ElapsedTime)
	{
		int32 IterationIndex = NumIterations++;
		Total += ElapsedTime;
		if (ElapsedTime > Max)
		{
			Max = ElapsedTime;
			SlowestIteration = IterationIndex;
		}
		return Total;
	}

	FString ToString() const;
};

struct FStats
{
	double TotalTime = 0.0;

	double ReachabilityTimeLimit = 0.0;
	double UnhashingTimeLimit = 0.0;
	double DestroyGarbageTimeLimit = 0.0;

	double VerifyTime = 0.0;
	double VerifyNoUnreachableTime = 0.0;
	double GarbageTrackingTime = 0.0;

	double InitialReferenceCollectionTime = 0.0;
	double MarkObjectsAsUnreachableTime = 0.0;
	double TraceExternalRootsTime = 0.0;
	double NotifyUnreachableTime = 0.0;
	double DissolveUnreachableClustersTime = 0.0;

	FIterationTimerStat ReachabilityTime;
	FIterationTimerStat ReferenceCollectionTime;
	FIterationTimerStat GatherUnreachableTime;
	FIterationTimerStat UnhashingTime;
	FIterationTimerStat DestroyGarbageTime;

	int32 NumObjects = 0;
	int32 NumRoots = 0;
	int32 NumClusters = 0;
	int32 NumClusteredObjects = 0;
	int32 NumUnreachableObjects = 0;
	int32 NumDissolvedClusters = 0;
	int32 NumUnreachableClusteredObjects = 0;
	int32 NumBarrierObjects = 0;
	int32 NumWeakReferencesForClearing = 0;
	int32 NumObjectsThatNeedWeakReferenceClearing = 0;

	bool bStartedAsFullPurge = false;
	bool bFinishedAsFullPurge = false;
	bool bFlushedAsyncLoading = false;
	bool bPurgedPreviousGCObjects = false;
	bool bInProgress = false;

	void DumpToLog();
};

} // namespace UE::GC::Private


