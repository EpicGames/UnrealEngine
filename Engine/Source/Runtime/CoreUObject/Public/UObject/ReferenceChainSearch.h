// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "HAL/ThreadHeartBeat.h"
#include "HAL/UnrealMemory.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/TypeHash.h"
#include "UObject/GarbageCollection.h"
#include "UObject/GarbageCollectionHistory.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/PrintStaleReferencesOptions.h"

class FGCObjectInfo;
class FOutputDevice;
class UObject;
struct FGCSnapshot;
template <typename FuncType> class TFunctionRef;

/** Search mode flags */
enum class EReferenceChainSearchMode
{
	// Returns all reference chains found
	Default = 0,
	// Returns only reference chains from external objects
	ExternalOnly = 1 << 0,
	// Returns only the shortest reference chain for each rooted object
	Shortest = 1 << 1,
	// Returns only the longest reference chain for each rooted object
	Longest = 1 << 2,
	// Returns only the direct referencers
	Direct = 1 << 3,
	// Returns complete chains. (Ignoring non GC objects)
	FullChain = 1 << 4,
	// Returns the shortest path to a garbage object from which the target object is reachable
	ShortestToGarbage = 1 << 5,
	// Attempts to find a plausible path to the target object with minimal memory usage
	// E.g. returns a direct external reference to the target object if one is found 
	//	otherwise returns an external reference to an inner of the target object
	Minimal = 1 << 6,
	// Skips the disregard-for-GC set that will never be GCd and whose outgoing references are not checked during GC
	GCOnly = 1 << 7,

	// Print results
	PrintResults = 1 << 16,
	// Print ALL results (in some cases there may be thousands of reference chains)
	PrintAllResults = 1 << 17,
};

ENUM_CLASS_FLAGS(EReferenceChainSearchMode);

class FReferenceChainSearch
{
	// Reference chain searching is a very slow operation.
	// Suspend the hang and hitch detectors for the lifetime of this instance.
	FSlowHeartBeatScope SuspendHeartBeat;
	FDisableHitchDetectorScope SuspendGameThreadHitch;

public:

	/** Type of reference */
	enum class EReferenceType
	{
		Unknown = 0,
		Property = 1,
		AddReferencedObjects = 2,
		OuterChain = 3,
	};

	/** Extended information about a reference */
	template<typename T> 
	struct TReferenceInfo
	{
		// Maximum number of stack frames to keep for AddReferencedObjects function calls
		static constexpr uint32 MaxStackFrames = 30;

		/** Object that is being referenced */
		T* Object = nullptr;
		TArray<uint64> StackFrames;
		/** Type of reference to the object being referenced */
		EReferenceType Type;
		/** Name of the object or property that is referencing this object */
		FString ReferencerName;

		/** Default ctor */
		TReferenceInfo()
			: Type(EReferenceType::Unknown)
		{
		}

		/** Simple reference constructor. Probably will be filled with more info later */
		TReferenceInfo(T* InObject)
			: Object(InObject)
			, Type(EReferenceType::Unknown)
		{
		}

		/** Full reference info constructor */
		TReferenceInfo(
			T* InObject, 
			EReferenceType InType, 
			FString InReferencerName,
			TConstArrayView<uint64> InStackFrames = {}
		)
			: Object(InObject)
			, StackFrames(InStackFrames)
			, Type(InType)
			, ReferencerName(MoveTemp(InReferencerName))
		{
		}

		TReferenceInfo(TReferenceInfo&&) = default;
		TReferenceInfo& operator=(TReferenceInfo&&) = default;

		bool operator ==(const TReferenceInfo& Other) const
		{
			return Object == Other.Object;
		}

		friend uint32 GetTypeHash(const TReferenceInfo& Info)
		{
			return GetTypeHash(Info.Object);
		}

		/** Dumps this reference info to string. Does not include the object being referenced */
		FString ToString() const
		{
			if (Type == EReferenceType::Property)
			{
				return FString::Printf(TEXT("->%s"), *ReferencerName);
			}
			else if (Type == EReferenceType::AddReferencedObjects)
			{
				if (!ReferencerName.IsEmpty())
				{
					return FString::Printf(TEXT("::AddReferencedObjects(): %s"), *ReferencerName);
				}
				else
				{
					return TEXT("::AddReferencedObjects()");
				}
			}
			return FString();
		}
	};
	
	struct FGraphNode;

	/** Convenience type definitions to avoid template hell */
	typedef TReferenceInfo<FGCObjectInfo> FObjectReferenceInfo;
	typedef TReferenceInfo<FGraphNode> FNodeReferenceInfo;

	/** Single node in the reference graph */
	struct FGraphNode
	{
		/** Object pointer */
		FGCObjectInfo* ObjectInfo = nullptr;
		/** Objects referenced by this object with reference info */
		TSet<FNodeReferenceInfo> ReferencedObjects;
		/** Objects that have references to this object */
		TSet<FGraphNode*> ReferencedByObjects;
		/** Non-zero if this node has been already visited during reference search */
		int32 Visited = 0;

		int64 GetAllocatedSize() const
		{
			return ReferencedObjects.GetAllocatedSize() + ReferencedByObjects.GetAllocatedSize();
		}
	};


	/** Reference chain. The first object in the list is the target object and the last object is a root object */
	class FReferenceChain
	{
		friend class FReferenceChainSearch;

		/** The target nodes that caused this chain to be created. Usually this will be Nodes[0] unless the chain was truncated by request. */
		FGraphNode* TargetNode = nullptr;

		/** Nodes in this reference chain. The first node is the target object and the last one is a root object */
		TArray<FGraphNode*> Nodes;
		/** Reference information for Nodes */
		TArray<const FNodeReferenceInfo*> ReferenceInfos;

		/** Fills this chain with extended reference info for each node */
		void FillReferenceInfo();

	public:
		FReferenceChain() {}
		FReferenceChain(int32 ReserveDepth)
		{
			Nodes.Reserve(ReserveDepth);
		}

		FReferenceChain(FGraphNode* InTargetNode, TArray<FGraphNode*> InNodes, TArray<const FNodeReferenceInfo*> InReferenceInfos)
			: TargetNode(InTargetNode)
			, Nodes(MoveTemp(InNodes))
			, ReferenceInfos(MoveTemp(InReferenceInfos))
		{
		}

		int64 GetAllocatedSize() const
		{
			return Nodes.GetAllocatedSize() + ReferenceInfos.GetAllocatedSize();
		}

		/** Adds a new node to the chain */
		void AddNode(FGraphNode* InNode)
		{
			Nodes.Add(InNode);
		}
		void InsertNode(FGraphNode* InNode)
		{
			Nodes.Insert(InNode, 0);
		}
		/** Gets a node from the chain */
		FGraphNode* GetNode(int32 NodeIndex) const
		{
			return Nodes[NodeIndex];
		}
		FGraphNode* GetRootNode(FGraphNode* Exclude) const
		{
			if (Nodes.Last() == Exclude && Nodes.Num() > 2)
			{
				return Nodes[Nodes.Num()-2];
			}

			return Nodes.Last();
		}
		FGraphNode* GetRootNode() const
		{
			return Nodes.Last();
		}
		/** Returns the number of nodes in the chain */
		int32 Num() const
		{
			return Nodes.Num();
		}
		/** Returns a duplicate of this chain */
		FReferenceChain* Split()
		{
			FReferenceChain* NewChain = new FReferenceChain(*this);
			return NewChain;
		}
		/** Checks if this chain contains the specified node */
		bool Contains(const FGraphNode* InNode) const
		{
			return Nodes.Contains(InNode);
		}
		/** Gets extended reference info for the specified node index */
		const FNodeReferenceInfo* GetReferenceInfo(int32 NodeIndex) const
		{
			return ReferenceInfos[NodeIndex];
		}
		/** Check if this reference chain represents an external reference (the root is not in target object) */
		bool IsExternal() const;
	};

	/** Parameters passed to callback function when printing results */
	struct FCallbackParams
	{
		/** Referenced object */
		FGCObjectInfo* Object = nullptr;
		/** Object that is referencing the current object */
		FGCObjectInfo* Referencer = nullptr;
		/** Information about the type of reference (Referencer -> Object) */
		const FNodeReferenceInfo* ReferenceInfo = nullptr;
		/** For use when outputting custom information: current indent */
		int32 Indent = 0;
		/** Output device used for printing */
		FOutputDevice* Out = nullptr;
	};

	/** Constructs a new search engine and finds references to the specified object */
	COREUOBJECT_API explicit FReferenceChainSearch(UObject* InObjectToFindReferencesTo,
		EReferenceChainSearchMode Mode = EReferenceChainSearchMode::PrintResults,
		ELogVerbosity::Type InVerbosityForPrint = ELogVerbosity::Log);
	COREUOBJECT_API explicit FReferenceChainSearch(TConstArrayView<UObject*> InObjectsToFindReferencesTo,
		EReferenceChainSearchMode Mode = EReferenceChainSearchMode::PrintResults,
		ELogVerbosity::Type InVerbosityForPrint = ELogVerbosity::Log);

	/** Constructs a new search engine but does not find references to any objects until one of the PerformSearch*() functions is called */
	COREUOBJECT_API explicit FReferenceChainSearch(EReferenceChainSearchMode Mode);

	/** Destructor */
	COREUOBJECT_API ~FReferenceChainSearch();

#if ENABLE_GC_HISTORY
	/** Searches for references in a previous GC run snapshot temporarily acquiring its object info */
	COREUOBJECT_API void PerformSearchFromGCSnapshot(UObject* InObjectToFindReferencesTo, FGCSnapshot& InSnapshot);
	COREUOBJECT_API void PerformSearchFromGCSnapshot(TConstArrayView<UObject*> InObjectsToFindReferencesTo, FGCSnapshot& InSnapshot);
#endif //ENABLE_GC_HISTORY

	/**
	 * Dumps results to log
	 * @param bDumpAllChains - if set to false, the output will be trimmed to the first 100 reference chains
	 * @returns The number of results printed.
	 */
	COREUOBJECT_API int32 PrintResults(bool bDumpAllChains = false, UObject* TargetObject = nullptr, ELogVerbosity::Type VerbosityForPrint = ELogVerbosity::Log) const;
	COREUOBJECT_API void SetVerbosityForPrint(ELogVerbosity::Type Verbosity);

	/**
	 * Dumps results to log
	 * @param ReferenceCallback - function called when processing each reference, if true is returned the next reference will be processed otherwise printing will be aborted
	 * @param bDumpAllChains - if set to false, the output will be trimmed to the first 100 reference chains
	 * @returns The number of results printed.
	 */
	COREUOBJECT_API int32 PrintResults(TFunctionRef<bool(FCallbackParams& Params)> ReferenceCallback, bool bDumpAllChains = false, UObject* TargetObject = nullptr, ELogVerbosity::Type VerbosityForPrint = ELogVerbosity::Log) const;

	/** Returns a string with a short report explaining the root path, will contain newlines */
	COREUOBJECT_API FString GetRootPath(UObject* TargetObject = nullptr) const;

	/** 
	 * Returns a string with a short report explaining the root path, will contain newlines 
	 * @param ReferenceCallback - function called when processing each reference, if true is returned the next reference will be processed otherwise printing will be aborted
	 */
	COREUOBJECT_API FString GetRootPath(TFunctionRef<bool(FCallbackParams& Params)> ReferenceCallback, UObject* TargetObject = nullptr) const;

	/** Returns all reference chains */
	const TArray<FReferenceChain*>& GetReferenceChains() const
	{
		return ReferenceChains;
	}

	int64 GetAllocatedSize() const;

	/**
	* Attempts to find a reference chain leading to a world that should have been garbage collected
	* @param ObjectToFindReferencesTo World or its package (or any object from the world package that should've been destroyed)
	* @param Options Determines how the stale references messages should be logged
	*/
	COREUOBJECT_API	static FString FindAndPrintStaleReferencesToObject(UObject* ObjectToFindReferencesTo, EPrintStaleReferencesOptions Options);
	COREUOBJECT_API static TArray<FString> FindAndPrintStaleReferencesToObjects(TConstArrayView<UObject*> ObjectsToFindReferencesTo, EPrintStaleReferencesOptions Options);
private:

	/** The objects we're going to look for references to */
	TArray<FGCObjectInfo*> ObjectInfosToFindReferencesTo;

	/** Search mode and options */
	EReferenceChainSearchMode SearchMode = EReferenceChainSearchMode::Default;

	/** All reference chain found during the search, one per entry in ObjectsToFindReferencesTo */
	TArray<FReferenceChain*> ReferenceChains;

	/** All nodes created during the search */
	TMap<FGCObjectInfo*, FGraphNode*> AllNodes;
	/** Maps UObject pointers to object info structs */
	TMap<const UObject*, FGCObjectInfo*> ObjectToInfoMap;

	ELogVerbosity::Type VerbosityForPrint = ELogVerbosity::Log;

	/** Frees memory */
	void Cleanup();

	/** Link two nodes together with the given reference info */
	void LinkNodes(
		UObject* From,
		UObject* To,
		EReferenceType ReferenceType,
		FName PropertyName,
		TConstArrayView<uint64> StackFrames = {});
	void LinkNodes(
		FGCObjectInfo* From,
		FGCObjectInfo* To,
		EReferenceType ReferenceType,
		FName PropertyName,
		TConstArrayView<uint64> StackFrames = {});

public:
	/** Returns a string with all flags (we care about) set on an object */
	COREUOBJECT_API static FString GetObjectFlags(const FGCObjectInfo& InObject);

	/** Dumps a reference chain to log */
	COREUOBJECT_API static void DumpChain(FReferenceChainSearch::FReferenceChain* Chain, TFunctionRef<bool(FCallbackParams& Params)> ReferenceCallback,
		TMap<uint64, FString>& CallstackCache, FOutputDevice& Out, ELogVerbosity::Type InVerbosityForPrint);
};
