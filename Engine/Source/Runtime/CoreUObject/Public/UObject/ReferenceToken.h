// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"

class FGCObject;
class FGCObjectInfo;
class FGCVerseCellInfo;
class UObject;

namespace Verse
{
	struct VCell;
}

namespace UE
{
	namespace GC
	{
		class FMemberId;
	}
}

/** Represents the different types that the reference token can represent */
enum class EReferenceTokenType
{
	// The following types have associated pointers and must have values in the range of [0, 7)
	None,
	Object,
	GCObject,
	VerseCell,
	GCObjectInfo,
	GCVerseCellInfo,

	// The following types must have the three lower bits clear
	Barrier = 1 << 3,
};

/** The reference token represents different types that might appear in reference relationships */
struct FReferenceToken
{
	FReferenceToken()
		: EncodedBits(0)
	{
	}

	explicit FReferenceToken(const UObject* Object)
		: EncodedBits(BitCast<uint64>(Object) | static_cast<uint64>(EReferenceTokenType::Object))
	{
	}

	explicit FReferenceToken(const FGCObject* GCObject)
		: EncodedBits(BitCast<uint64>(GCObject) | static_cast<uint64>(EReferenceTokenType::GCObject))
	{
	}

	explicit FReferenceToken(const Verse::VCell* Cell)
		: EncodedBits(BitCast<uint64>(Cell) | static_cast<uint64>(EReferenceTokenType::VerseCell))
	{
	}

	explicit FReferenceToken(const FGCObjectInfo* GCObjectInfo)
		: EncodedBits(BitCast<uint64>(GCObjectInfo) | static_cast<uint64>(EReferenceTokenType::GCObjectInfo))
	{
	}

	explicit FReferenceToken(const FGCVerseCellInfo* GCVerseCellInfo)
		: EncodedBits(BitCast<uint64>(GCVerseCellInfo) | static_cast<uint64>(EReferenceTokenType::GCVerseCellInfo))
	{
	}

	explicit FReferenceToken(EReferenceTokenType TokenType)
		: EncodedBits(static_cast<uint64>(TokenType))
	{
		checkf((EncodedBits & EncodingBits) == 0, TEXT("Reference token type constructor can only be used with the pointer-less types"));
	}

	EReferenceTokenType GetType() const
	{
		// None is treated as a special value.  When found, then all bits are converted to the enum.
		EReferenceTokenType Type = static_cast<EReferenceTokenType>(EncodedBits & EncodingBits);
		return Type == EReferenceTokenType::None ? static_cast<EReferenceTokenType>(EncodedBits) : Type;
	}

	bool IsObject() const
	{
		return GetType() == EReferenceTokenType::Object;
	}

	UObject* AsObject() const
	{
		checkSlow(IsObject());
		return BitCast<UObject*>(EncodedBits & ~EncodingBits);
	}

	bool IsGCObject() const
	{
		return GetType() == EReferenceTokenType::GCObject;
	}

	FGCObject* AsGCObject() const
	{
		checkSlow(IsGCObject());
		return BitCast<FGCObject*>(EncodedBits & ~EncodingBits);
	}

	bool IsGCObjectInfo() const
	{
		return GetType() == EReferenceTokenType::GCObjectInfo;
	}

	FGCObjectInfo* AsGCObjectInfo() const
	{
		checkSlow(IsGCObjectInfo());
		return BitCast<FGCObjectInfo*>(EncodedBits & ~EncodingBits);
	}

	bool IsVerseCell() const
	{
		return GetType() == EReferenceTokenType::VerseCell;
	}

	Verse::VCell* AsVerseCell() const
	{
		checkSlow(IsVerseCell());
		return BitCast<Verse::VCell*>(EncodedBits & ~EncodingBits);
	}

	bool IsGCVerseCellInfo() const
	{
		return GetType() == EReferenceTokenType::GCVerseCellInfo;
	}

	FGCVerseCellInfo* AsGCVerseCellInfo() const
	{
		checkSlow(IsGCVerseCellInfo());
		return BitCast<FGCVerseCellInfo*>(EncodedBits & ~EncodingBits);
	}

	/** Returns a formatted string with reference's info */
	FString GetDescription() const;
	FString GetDescription(FName PropertyName, const TCHAR* DefaultPropertyName) const;

	/** Convert the member id to a member name */
	FString GetMemberName(UE::GC::FMemberId& MemberId) const;

	bool operator ==(const FReferenceToken& Other) const
	{
		return EncodedBits == Other.EncodedBits;
	}

	friend uint32 GetTypeHash(const FReferenceToken& Info)
	{
		return GetTypeHash(Info.EncodedBits);
	}

private:
	static constexpr uint64 EncodingBits = 0b111;
	uint64 EncodedBits{ 0 };
};
