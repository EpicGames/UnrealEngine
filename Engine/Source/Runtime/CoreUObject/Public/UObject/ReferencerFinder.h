// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"
#include "Serialization/ArchiveUObject.h"

class UObject;

enum class EReferencerFinderFlags : uint8
{
	None = 0,
	SkipInnerReferences = 1 << 0, // Do not add inner objects to the referencers of an outer.
	SkipWeakReferences  = 1 << 1, // Do not add weak references to the referencers
};

ENUM_CLASS_FLAGS(EReferencerFinderFlags);

/**
 * Helper class for finding all objects referencing any of the objects in Referencees list
 */
class FReferencerFinder
{
public:
	static COREUOBJECT_API TArray<UObject*> GetAllReferencers(const TArray<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore, EReferencerFinderFlags Flags = EReferencerFinderFlags::None);
	static COREUOBJECT_API TArray<UObject*> GetAllReferencers(const TSet<UObject*>& Referencees, const TSet<UObject*>* ObjectsToIgnore, EReferencerFinderFlags Flags = EReferencerFinderFlags::None);

	/** Called when the initial load phase is complete and we're done registering native UObject classes */
	static COREUOBJECT_API void NotifyRegistrationComplete();
};
