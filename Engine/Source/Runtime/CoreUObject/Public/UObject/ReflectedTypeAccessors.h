// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/CoreReflectedTypeAccessors.h"

class UClass;
class UScriptStruct;

/*-----------------------------------------------------------------------------
C++ templated Static(Class/Struct) retrieval function prototypes.
-----------------------------------------------------------------------------*/

template<typename ClassType> inline UClass* StaticClass()
{
	return ClassType::StaticClass();
}

template<typename StructType> inline UScriptStruct* StaticStruct()
{
	return StructType::StaticStruct();
}
