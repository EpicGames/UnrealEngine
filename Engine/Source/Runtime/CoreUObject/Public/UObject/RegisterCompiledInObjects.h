// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#if UE_WITH_CONSTINIT_UOBJECT 

#include "Containers/ArrayView.h"
#include "Misc/TVariant.h"
#include "UObject/CompiledInObjectPtr.h"
#include "UObject/UObjectGlobals.h"

class UClass;
class UDelegateFunction;
class UObject;
class UPackage;
namespace UE::CoreUObject::Private { struct FPartialClass; }

// Utility struct for passing many arrays with designated initializer for clarity 
struct FCompiledInObjects
{
#if !IS_MONOLITHIC
	// Tables of objects in other modules for resolving encoding pointers in these objects
	TConstArrayView<TConstArrayView<UObject*>> LinkedModules;

	// Objects which can be linked by other objects using the package's link id and the index of the
	// object in this array.
	// The following arrays may bes slices of this array produced by UHT to reduce duplication.
	// This array and the following arrays may contain null entries if objects are declared inside #if scopes in headers
	// This means that link indices are stable regardless of the value of those preprocessor definitions
	TConstArrayView<UObject*> LinkableObjects;
#endif // !IS_MONOLITHIC

	// Packages that are part of this module - usually 1 for UObject modules or 2 for modules with Verse
	TConstArrayView<UPackage*> Packages;

	// Arrays of top level objects to initialize segmented by type to avoid casting.
	// Typed as UObject* because they are slices of a single array
	TConstArrayView<UObject*> Classes;
	TConstArrayView<UObject*> Structs;
	TConstArrayView<UObject*> Enums;
	// Other types of objects (e.g. UDelegateFunction) which can be initialized by casting after
	// classes are propery initialized.
	TConstArrayView<UObject*> Others;
	
	TConstArrayView<UE::CoreUObject::Private::FPartialClass*> Partials;

	inline TConstArrayView<UClass*> GetClasses() const
	{
		return MakeArrayView(reinterpret_cast<UClass* const*>(Classes.GetData()), Classes.Num());
	}
	inline TConstArrayView<UScriptStruct*> GetStructs() const
	{
		return MakeArrayView(reinterpret_cast<UScriptStruct* const*>(Structs.GetData()), Structs.Num());
	}
	inline TConstArrayView<UEnum*> GetEnums() const
	{
		return MakeArrayView(reinterpret_cast<UEnum* const*>(Enums.GetData()), Enums.Num());
	}
};

// Register partially initialized objects such as packages, classes, structs and enums for runtime construction
// This structure is stored as a linked list where possible to avoid dynamic allocation during startup 
// UHT emits one of these per module
struct FRegisterCompiledInObjects
{
	// Register a package and all the objects which need to be registered directly.
	// Other objects may be registered as subobjects, e.g. Functions
	[[nodiscard]] explicit FRegisterCompiledInObjects(FCompiledInObjects InObjects)
		: Objects(InObjects)
	{
		Register();
	}

	FCompiledInObjects Objects;

	FRegisterCompiledInObjects* ListNext = nullptr;

	// Register all our objects for deferred construction
	COREUOBJECT_API void Register();
};

struct FRegisterIntrinsicClass
{
	using FIntrinsicClassConstructor = void();

	[[nodiscard]] explicit FRegisterIntrinsicClass(UClass* InClass, FIntrinsicClassConstructor* InConstructor)
		: Class(InClass)
		, Constructor(InConstructor)
	{
		Register();
	}

	UClass* Class = nullptr;
	FIntrinsicClassConstructor* Constructor = nullptr;

	FRegisterIntrinsicClass* ListNext = nullptr;

	// Register all our objects for deferred construction
	COREUOBJECT_API void Register();
};

#endif