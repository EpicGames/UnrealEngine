// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Reload.h: Unreal reload support.
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/CoreNative.h"
#include "Modules/ModuleManager.h"

/**
 * Systems that implement a reload capability implement this interface and register it with 
 * the module manager while a reload is in progress.
 */

class IReload
{
public:

	virtual ~IReload() = default;

	/**
	 * Returns the type of reload currently in progress.
	 */
	virtual EActiveReloadType GetType() const = 0;

	/**
	 * When classes, structures, and enumerations are renamed, the given prefix is applied.
	 */
	virtual const TCHAR* GetPrefix() const = 0;

	/**
	 * Return if re-instancing is to be allowed
	 */
	virtual bool GetEnableReinstancing(bool bHasChanged) const = 0;

	/**
	 * Invoke when a duplicate function has been detected.
	 */
	virtual void NotifyFunctionRemap(FNativeFuncPtr NewFunctionPointer, FNativeFuncPtr OldFunctionPointer) = 0;

	/**
	 * Invoke to register a new or changed class, enumeration, or structure for re-instancing.
	 */
	virtual void NotifyChange(UClass* New, UClass* Old) = 0;
	virtual void NotifyChange(UEnum* New, UEnum* Old) = 0;
	virtual void NotifyChange(UScriptStruct* New, UScriptStruct* Old) = 0;
	virtual void NotifyChange(UPackage* New, UPackage* Old) = 0;

	/**
	 * Perform the re-instancing 
	 */
	virtual void Reinstance() = 0;

	/**
	 * For a given CDO, return the new CDO if it has been re-instanced.  If it hasn't been re-instanced,
	 * then the supplied CDO will be returned. If a non-CDO object is passed in, it will be returned as is.
	 */
	virtual UObject* GetReinstancedCDO(UObject* CDO) = 0;
	virtual const UObject* GetReinstancedCDO(const UObject* CDO) = 0;

	/**
	 * Return a map of the reinstanced classes where the old class is the key and the new class is the value.
	 */
	 virtual const TMap<UClass*, UClass*>& GetReinstancedClasses() = 0;

	 /**
	  * Return an array of the new classes
	  */
	 virtual const TArray<UClass*>& GetNewClasses() = 0;
};

/**
 * Invoke when a duplicate function has been detected.  If reloading is in progress, it will be notified
 * of the change.
 *
 * @return  Returns true if a reload operation is in progress.  Returns false if the duplication is an error.
 */

inline bool ReloadNotifyFunctionRemap(FNativeFuncPtr NewFunctionPointer, FNativeFuncPtr OldFunctionPointer)
{
#if WITH_RELOAD
	if (IReload* Reload = GetActiveReloadInterface())
	{
		Reload->NotifyFunctionRemap(NewFunctionPointer, OldFunctionPointer);
		return true;
	}
#endif
	return false;
}
