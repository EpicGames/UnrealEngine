// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/RemoteObjectTypes.h"
#include "UObject/Class.h"
#include "Async/Async.h"

DECLARE_LOG_CATEGORY_EXTERN(LogRemoteExec, Display, All);

struct FRemoteTransactionId
{
	FRemoteTransactionId() = default;
	explicit constexpr FRemoteTransactionId(uint32 InId)
		: Id(InId)
	{
	}

	static FRemoteTransactionId Invalid()
	{
		return FRemoteTransactionId();
	}

	bool operator==(const FRemoteTransactionId& Rhs) const { return Id == Rhs.Id; }
	bool operator!=(const FRemoteTransactionId& Rhs) const { return Id != Rhs.Id; }

	uint32 GetIdNumber() const
	{
		return Id;
	}
	bool IsValid() const
	{
		return Id != 0;
	}
	FString ToString() const
	{
		return FString::FromInt(Id);
	}
	void operator = (FRemoteTransactionId Other)
	{
		Id = Other.Id;
	}

	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FRemoteTransactionId& Id);

private:
	uint32 Id = 0;
};

enum class ERemoteWorkType
{
	Migration = 0,
	Normal = 1
};

COREUOBJECT_API const TCHAR* EnumToString(ERemoteWorkType WorkType);

struct FRemoteWorkPriority
{
	uint64 PackedData = 0;

	// the data is packed in a uint64 to allow passing by value in a register
	// this allows comparison of priority to be a single uint64 compare-less-than
	// 
	// the WorkDepth is stored subtracted from 0xFF so that less-than means higher priority
	//   0xFF is the root depth
	//   0xFE is one deeper
	//
	using FPriorityPacking = TIntegerSequence<int32,
		1, // 0 if this priority represents (higher priority) transactional migration work, 1 otherwise
		REMOTE_OBJECT_SERVER_ID_BIT_SIZE, // FRemoteServerId RootServerId
		8, // uint8 WorkDepth
		32>; // unique id bits to ensure this priority is unique

	ERemoteWorkType GetWorkType() const { return !(UnpackPriorityElement(PackedData, 0)) ? ERemoteWorkType::Migration : ERemoteWorkType::Normal; }
	FRemoteServerId GetRootServerId() const { return FRemoteServerId::FromIdNumber((uint32)UnpackPriorityElement(PackedData, 1)); }
	uint32 GetWorkDepth() const { return (uint32)UnpackPriorityElement(PackedData, 2); }
	uint32 GetUniqueId() const { return (uint32)UnpackPriorityElement(PackedData, 3); }
	bool IsValid() const { return GetRootServerId().IsValid(); }

	FRemoteWorkPriority()
	{
		PackedData = PackPriority(ERemoteWorkType::Normal, 0, 0, 0);
	}
	
	COREUOBJECT_API static FRemoteWorkPriority CreateRootWorkPriority(FRemoteServerId ServerId, ERemoteWorkType WorkType = ERemoteWorkType::Normal);

	// creates a copy of this priority with the WorkDepth decreased by 1
	COREUOBJECT_API FRemoteWorkPriority CreateDependentWorkPriority() const;

	COREUOBJECT_API FString ToString() const;

private:
	template<int32... Bits, typename... ArgTypes>
	static uint64 PackPriorityImpl(TIntegerSequence<int32, Bits...>, ArgTypes... InValues)
	{
		static_assert(sizeof...(Bits) == sizeof...(ArgTypes));
		static_assert((Bits + ...) <= 64, "sum of the bit sizes must be <= 64");

		int32 BitWidths[] = { Bits... };
		uint64 Values[] = { static_cast<uint64>(InValues)... };

		uint64 Result = 0;
		int32 Offset = 64; // pack the values in decreasing significance

		for(int32 Index = 0; Index < sizeof...(Bits); Index++)
		{
			Offset -= BitWidths[Index];
			uint64 Mask = (1ULL << BitWidths[Index]) - 1;
			Result |= (Values[Index] & Mask) << Offset;
		}

		return Result;
	}

	template<int32... Bits>
	static uint64 UnpackPriorityElementImpl(TIntegerSequence<int32, Bits...>, uint64 PackedPriority, int32 OutIndex)
	{
		static_assert((Bits + ...) <= 64, "sum of the bit sizes must be <= 64");

		int32 BitWidths[] = { Bits... };

		int32 Offset = 64;
		uint64 Result = 0;
		for(int32 Index = 0; Index < sizeof...(Bits); Index++)
		{
			Offset -= BitWidths[Index];

			uint64 Mask = (1ULL << BitWidths[Index]) - 1;

			if (Index == OutIndex)
			{
				Result = (PackedPriority >> Offset) & Mask;
				break;
			}
		}

		return Result;
	}

	template<typename... ArgTypes>
	static uint64 PackPriority(ArgTypes... Values)
	{
		return PackPriorityImpl(FPriorityPacking{}, Values...);
	}

	static uint64 UnpackPriorityElement(uint64 PackedPriority, int32 OutIndex)
	{
		return UnpackPriorityElementImpl(FPriorityPacking{}, PackedPriority, OutIndex);
	}
};

COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FRemoteWorkPriority& Priority);
COREUOBJECT_API bool IsHigherPriority(FRemoteWorkPriority Lhs, FRemoteWorkPriority Rhs);
COREUOBJECT_API bool IsEqualPriority(FRemoteWorkPriority Lhs, FRemoteWorkPriority Rhs);
COREUOBJECT_API bool IsHigherOrEqualPriority(FRemoteWorkPriority Lhs, FRemoteWorkPriority Rhs);
COREUOBJECT_API bool operator==(FRemoteWorkPriority Lhs, FRemoteWorkPriority Rhs);

class FRemoteSubsystemBase
{
protected:
	TArray<TTuple<FRemoteTransactionId, void*>> Requests;
	FRemoteServerId LocalServerId;

public:

	FRemoteSubsystemBase() = default;
	explicit FRemoteSubsystemBase(FRemoteServerId InLocalServerId) : LocalServerId(InLocalServerId) {}
	virtual ~FRemoteSubsystemBase() = default;

	FRemoteServerId GetLocalServerId() const { return LocalServerId; }

	int32 GetRequestCount() const { return Requests.Num(); }

	// implemented in FRemoteSubsystem<>
	virtual void* CreateRequest(FRemoteTransactionId RequestId, FRemoteWorkPriority Priority) = 0;
	virtual void DestroyRequest(FRemoteTransactionId RequestId) = 0;
	virtual void SetActiveRequest(FRemoteTransactionId RequestId) = 0;
	virtual void ClearActiveRequest() = 0;

	virtual void ActivateForTesting() = 0;
	virtual void DeactivateForTesting() = 0;

	// for subsystem implementor to override:
	virtual const TCHAR* NameForDebug() = 0;
	virtual void BeginRequest() = 0;
	virtual void TickSubsystem() = 0;
	virtual void TickRequest() = 0;
	virtual void TickAbortedRequest() = 0;
	virtual bool AreDependenciesSatisfied() const = 0;
	virtual void BeginMultiServerCommit(TArray<FRemoteServerId>& OutMultiServerCommitRemoteServers) = 0;
	virtual void ExecuteMultiServerCommit() = 0;
	virtual void AbortMultiServerCommit() = 0;
	virtual void CommitMultiServerCommit() = 0;

	virtual void EndRequest(bool bTransactionCommitted) = 0;
};

template<typename SubsystemType, typename RequestType>
class FRemoteSubsystem : public FRemoteSubsystemBase
{
public:
	using FRemoteSubsystemBase::FRemoteSubsystemBase;

	static inline SubsystemType* Instance = nullptr;

	~FRemoteSubsystem()
	{
		if (Instance == this)
		{
			Instance = nullptr;
		}
	}

	void ActivateForTesting() final
	{
		Instance = static_cast<SubsystemType*>(this);
	}

	void DeactivateForTesting() final
	{
		Instance = nullptr;
	}

	RequestType* ActiveRequest = nullptr;

	RequestType* GetRequestByIndex(int32 RequestIndex) const
	{
		const TTuple<FRemoteTransactionId, void*> RequestData = Requests[RequestIndex];
		return static_cast<RequestType*>(RequestData.Get<1>());
	}

	RequestType* TryGetRequest(FRemoteTransactionId RequestId) const
	{
		for (int32 RequestIndex = 0; RequestIndex < Requests.Num(); RequestIndex++)
		{
			const TTuple<FRemoteTransactionId, void*>& RequestData = Requests[RequestIndex];
			if (RequestData.Get<0>() == RequestId)
				return static_cast<RequestType*>(RequestData.Get<1>());
		}

		return nullptr;
	}

	int32 TryGetRequestIndex(FRemoteTransactionId RequestId) const
	{
		int32 Result = INDEX_NONE;

		for (int32 RequestIndex = 0; RequestIndex < Requests.Num(); RequestIndex++)
		{
			const TTuple<FRemoteTransactionId, void*>& RequestData = Requests[RequestIndex];
			if (RequestData.Get<0>() == RequestId)
			{
				Result = RequestIndex;
				break;
			}
		}

		return Result;
	}

	void* CreateRequest(FRemoteTransactionId RequestId, FRemoteWorkPriority Priority) final
	{
		RequestType* Request = new RequestType{};
		Request->RequestId = RequestId;
		Request->Priority = Priority;
		Requests.Add(TTuple<FRemoteTransactionId, void*>(RequestId, Request));
		return Request;
	}

	void DestroyRequest(FRemoteTransactionId RequestId) final
	{
		int32 RequestIndex = TryGetRequestIndex(RequestId);
		check(RequestIndex != INDEX_NONE);
		RequestType* Request = GetRequestByIndex(RequestIndex);
		Requests.RemoveAt(RequestIndex, EAllowShrinking::No);
		delete Request;
	}

	void SetActiveRequest(FRemoteTransactionId RequestId) final
	{
		ActiveRequest = TryGetRequest(RequestId);
		check(ActiveRequest);
	}

	void ClearActiveRequest() final
	{
		ActiveRequest = nullptr;
	}
};

class FRemoteExecutor;

struct FRemoteExecutorDelegates
{
	// Protocol delegates
	TDelegate<void()> TickNetworkDelegate;
	TDelegate<void(FRemoteTransactionId, FRemoteWorkPriority, const TArray<FRemoteServerId>&)> BeginMultiServerCommitDelegate;
	TDelegate<void(FRemoteTransactionId, const TArray<FRemoteServerId>&)> ReadyMultiServerCommitDelegate;
	TDelegate<void(FRemoteTransactionId, const TArray<FRemoteServerId>&)> AbandonMultiServerCommitDelegate;
	TDelegate<void(FRemoteTransactionId, const TArray<FRemoteServerId>&)> EndMultiServerCommitDelegate;
	TDelegate<void(FRemoteTransactionId, FRemoteServerId)> ReadyRemoteMultiServerCommitDelegate;
	TDelegate<void(FRemoteTransactionId, FRemoteServerId)> AbortRemoteMultiServerCommitDelegate;
};

namespace UE::RemoteExecutor
{
	enum class EAbortTransactionBehavior : uint8
	{
		// We abort the transaction immediately (we do not finish the current callstack)
		Immediately,

		// We abort the transaction instead of committing it (we continue to execute speculatively)
		BeforeCommit
	};

	enum class ERemoteExecutorAbortReason : uint8
	{
		Unspecified,
		RequiresDependencies,
		AbandonWork
	};

	/**
	 * Denote that this scope runs in speculation mode.
	 * Speculation mode allows us to return stale objects, or null when in a transaction & dereferencing a remote object.
	 * Execution will continue (rather than aborting), but cannot commit in this mode (an attempt to commit will instead abort).
	 * This allows us to continue to speculate about what a transaction might do, request those results, and abort only once.
	 * 
	 * Note: Due to the possibility of returning null, a scope must protect against null even if the object
	 * actually exists on another server.
	 */
	struct FSpeculationExecutionScope
	{
		COREUOBJECT_API FSpeculationExecutionScope();
		COREUOBJECT_API ~FSpeculationExecutionScope();
	};

	/** Delegate executed when a transaction is queued in the executor for execution */
	extern COREUOBJECT_API TMulticastDelegate<void(FRemoteTransactionId, FName)> OnTransactionQueuedDelegate;

	/** Delegate executed when a ExecuteTransactional starts an AutoRTFM transaction */
	extern COREUOBJECT_API TMulticastDelegate<void(FRemoteTransactionId, FName)> OnTransactionStartingDelegate;

	/** Delegate executed when a ExecuteTransactional observes a completed AutoRTFM transaction */
	extern COREUOBJECT_API TMulticastDelegate<void(FRemoteTransactionId, uint32)> OnTransactionCompletedDelegate;

	/** Delegate executed when a ExecuteTransactional observes an aborted AutoRTFM transaction */
	extern COREUOBJECT_API TMulticastDelegate<void(FRemoteTransactionId, uint32, const FString&)> OnTransactionAbortedDelegate;

	/** Delegate executed when a transaction is released (execution is finished and the work is removed from the executor) */
	extern COREUOBJECT_API TMulticastDelegate<void(FRemoteTransactionId)> OnTransactionReleasedDelegate;

	extern COREUOBJECT_API TDelegate<void(FName /*Text*/)> OnRegionBeginDelegate;
	extern COREUOBJECT_API TDelegate<void(const FString& /*Text*/)> OnRegionEndDelegate;

	COREUOBJECT_API FRemoteExecutor* CreateRemoteExecutor(
		FRemoteServerId LocalServerId,
		FRemoteExecutorDelegates* Delegates);

	COREUOBJECT_API void DestroyRemoteExecutor(FRemoteExecutor* Executor);

	COREUOBJECT_API void SetRemoteExecutorForTesting(FRemoteExecutor* Executor);
	COREUOBJECT_API void SetSingleStepModeForTesting(bool bEnable);
	
	COREUOBJECT_API bool IsRemoteExecutorActive();

	COREUOBJECT_API bool IsSpeculationMode();
	COREUOBJECT_API void RegisterRemoteSubsystem(FRemoteExecutor* Executor, FRemoteSubsystemBase* Subsystem);

	/** Notify the executor that the network is waiting (we should tick it next chance we get) */
	COREUOBJECT_API void NotifyNetworkIsWaiting();

	COREUOBJECT_API void ExecuteTransactional(FName WorkName, const TFunctionRef<void(void)>& Work);
	COREUOBJECT_API void ExecuteTransactionalWithExplicitPriority(FName WorkName, FRemoteWorkPriority WorkPriority, const TFunctionRef<void(void)>& Work);
	COREUOBJECT_API void EnqueueWorkWithExplicitPriority(FName WorkName, FRemoteWorkPriority WorkPriority, bool bIsTransactional, TUniqueFunction<void(void)>&& Work);
	COREUOBJECT_API void EnqueueWork(FName WorkName, bool bIsTransactional, TUniqueFunction<void(void)>&& Work);
	COREUOBJECT_API void EnqueueMigrationWork(FName WorkName, TUniqueFunction<void(void)>&& InWork);
	COREUOBJECT_API void EnqueueWorkListWithExplicitPriority(FName WorkName, FRemoteWorkPriority WorkPriority, bool bIsTransactional, TArray<TUniqueFunction<void(void)>>&& WorkList);
	COREUOBJECT_API void EnqueueWorkList(FName WorkName, bool bIsTransactional, TArray<TUniqueFunction<void(void)>>&& WorkList);
	COREUOBJECT_API void ExecutePendingWork();
	COREUOBJECT_API bool HasPendingWork();

	COREUOBJECT_API void AbortCurrentTransaction(EAbortTransactionBehavior AbortBehavior, ERemoteExecutorAbortReason AbortReason, FStringView AbortDescription);

	COREUOBJECT_API void AbortTransactionRequiresDependencies(FStringView Description);
	COREUOBJECT_API void AbortTransactionAndAbandonWork(FStringView Description);

	COREUOBJECT_API void TransactionRequiresMultiServerCommit(FStringView Description);
	
	COREUOBJECT_API void BeginRemoteMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId, FRemoteWorkPriority RequestPriority);
	COREUOBJECT_API void ReadyRemoteMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId);
	COREUOBJECT_API void AbandonRemoteMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId);
	COREUOBJECT_API void EndRemoteMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId);
	COREUOBJECT_API void EnqueueRemoteMultiServerCommitAction(FRemoteServerId ServerId, FRemoteTransactionId RequestId, TUniqueFunction<void()>&& Action);

	COREUOBJECT_API void ReadyMultiServerCommitResponse(FRemoteServerId ServerId, FRemoteTransactionId RequestId);
	COREUOBJECT_API void AbortMultiServerCommit(FRemoteServerId ServerId, FRemoteTransactionId RequestId);
	
	COREUOBJECT_API FRemoteWorkPriority CreateRootWorkPriority();
}
