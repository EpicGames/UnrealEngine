// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreTypes.h"
#include "AutoRTFM.h"
#include "UObject/RemoteExecutor.h"
#include "RemoteHeap.generated.h"

namespace UE
{
// Sends a cache line to another server (for read or exclusive response)
// Includes LastMutatorServerId so the receiver knows the sharing tree root
extern TDelegate<void(uint64 /*Address*/, FRemoteServerId /*Destination*/,
	const TArray<uint8>& /*Data*/, FRemoteServerId /*LastMutatorServerId*/,
	uint64 /*LineVersion*/, uint64 /*AllocationBase*/, uint64 /*AllocationSize*/)>
	RemoteHeapSendLineDelegate;

// Requests a cache line for read (Shared) access
extern TDelegate<void(uint64 /*Address*/, FRemoteServerId /*ExpectedOwner*/,
	FRemoteWorkPriority /*Priority*/, FRemoteServerId /*Destination*/)>
	RemoteHeapRequestLineReadDelegate;

// Requests a cache line for exclusive (write) access
// bNeedsData: true if the requester needs the line data (Remote -> Exclusive),
// false if the requester already has a Shared copy (Shared -> Exclusive)
extern TDelegate<void(uint64 /*Address*/, FRemoteServerId /*ExpectedOwner*/,
	FRemoteWorkPriority /*Priority*/, FRemoteServerId /*Destination*/,
	bool /*bNeedsData*/)>
	RemoteHeapRequestLineExclusiveDelegate;

// Acknowledge eviction of a cache line for exclusive request
// Includes the list of servers the evicting server forwarded the invalidation to
extern TDelegate<void(FRemoteServerId /*Destination*/,
	uint64 /*Address*/, uint64 /*LineVersion*/,
	const TArray<FRemoteServerId>& /*ForwardedServers*/)>
	RemoteHeapExclusiveAckDelegate;

extern TDelegate<void(uint64 /*Address*/, FRemoteServerId /*Owner*/)> RemoteHeapDeallocateDelegate;

// Notifies a server that a line it requested has been deallocated by the allocating server
extern TDelegate<void(uint64 /*Address*/, FRemoteServerId /*Destination*/)>
	RemoteHeapNotifyLineDeallocatedDelegate;

const uint64 RemoteHeapSizePerServer = 0x0000000400000000ull; // 16GiB

struct FRemoteHeapAllocatorOverride
{
	void* (*Allocate)(void* Context, size_t Size) = nullptr;
	void  (*Deallocate)(void* Context, void* Ptr) = nullptr;
	void* Context = nullptr;
};

enum class ERemoteHeapLineState : uint8
{
	Exclusive,
	Shared,
	Remote,
};

struct SRemoteHeapLineData
{
	uint64 Address = 0;
	TArray<uint8> Data;
};

struct SRemoteHeapLineEntry : SRemoteHeapLineData
{
	ERemoteHeapLineState State = ERemoteHeapLineState::Exclusive;
	FRemoteServerId LastMutatorServerId;
	TArray<FRemoteServerId> RemoteServers;
	uint64 LineVersion = 0;
	uint64 AllocationBase = 0;
	uint64 AllocationSize = 0;
};

class FRemoteHeapRequest
{
public:
	FRemoteTransactionId RequestId;
	FRemoteWorkPriority Priority;

	TMap<uint64, ERemoteHeapLineState> RequestedLines;

	// can only copy Exclusive heap lines here
	TArray<SRemoteHeapLineData> LocalMutatedLines;
};

class FRemoteHeapOutgoingLineRequest
{
public:
	uint64 Address = 0;
	FRemoteServerId DestinationServerId;
	FRemoteWorkPriority RequestPriority;
	ERemoteHeapLineState RequestedAccess;
	bool bNeedsData = false;

	FString ToString() const
	{
		return FString::Format(TEXT("[address %llx dest %s %s %s%s]"),
			{Address,
			*DestinationServerId.ToString(),
			*RequestPriority.ToString(),
			RequestedAccess == ERemoteHeapLineState::Exclusive ? TEXT("exclusive") : TEXT("read"),
			bNeedsData ? TEXT(" needs-data") : TEXT("")});
	}
};

class FRemoteHeapOutgoingLineRequests
{
public:
	uint64 Address = 0;

	// IndividualRequests is sorted by priority (highest first)
	TArray<FRemoteHeapOutgoingLineRequest> IndividualRequests;
};

class FRemoteHeapLineRequest
{
public:
	uint64 Address = 0;
	FRemoteWorkPriority RequestPriority;
	ERemoteHeapLineState RequestedAccess;
	uint64 AckVersion = 0;
	TSet<FRemoteServerId> NeededAcks;
	TSet<FRemoteServerId> ReceivedAcks;

	bool AreAcksSatisfied() const
	{
		if (ReceivedAcks.Num() < NeededAcks.Num())
		{
			return false;
		}
		for (const FRemoteServerId& Needed : NeededAcks)
		{
			if (!ReceivedAcks.Contains(Needed))
			{
				return false;
			}
		}
		return true;
	}
};

class FRemoteHeap : public FRemoteSubsystem<FRemoteHeap, FRemoteHeapRequest>
{
	using FRemoteSubsystem::FRemoteSubsystem;

	FRemoteHeapAllocatorOverride* AllocatorOverride = nullptr; // non-owning, must outlive this heap

	TArray<SRemoteHeapLineEntry> HeapLines;

	// this is a list of heap lines that we have locally that 
	// other servers are asking for
	TArray<FRemoteHeapOutgoingLineRequests> OutgoingLineRequests;

	// this is a list of heap lines that we have sent requests
	// for and are awaiting responses
	TArray<FRemoteHeapLineRequest> PendingLineRequests;

	void NotifyIndividualRequestsDeallocated(const TArray<FRemoteHeapOutgoingLineRequest>& IndividualRequests);
	void ForwardIndividualRequests(FRemoteServerId DestinationServer, const TArray<FRemoteHeapOutgoingLineRequest>& IndividualRequests, bool bForceNeedsData);

	enum class ELineRequestAction { Consumed, Advance, Retry };
	SRemoteHeapLineEntry* FindHeapLine(uint64 Address);
	const SRemoteHeapLineEntry* FindHeapLine(uint64 Address) const;
	ELineRequestAction ProcessExclusiveLineRequest(FRemoteHeapOutgoingLineRequests& LineRequests, SRemoteHeapLineEntry& LineEntry, const FRemoteHeapOutgoingLineRequest& HighestPriorityRequest);
	ELineRequestAction ProcessSharedLineRequest(FRemoteHeapOutgoingLineRequests& LineRequests, SRemoteHeapLineEntry& LineEntry, const FRemoteHeapOutgoingLineRequest& HighestPriorityRequest);


public:
	static void CreateInstance(class FRemoteExecutor* Executor, FRemoteServerId LocalServerId, FRemoteHeapAllocatorOverride* AllocatorOverride = nullptr);
	static uint64 GetRemoteHeapBaseAddressForServerId(FRemoteServerId ServerId);

	const TArray<SRemoteHeapLineEntry>& GetHeapLinesForTesting() const { return HeapLines; }

	const TCHAR* NameForDebug() final { return TEXT("RemoteHeap"); }
	void BeginRequest() final;
	void TickSubsystem() final;
	void TickRequest() final;
	void TickAbortedRequest() final;
	bool AreDependenciesSatisfied() const final;
	void EndRequest(bool bTransactionCommitted) final;

	void BeginMultiServerCommit(TArray<FRemoteServerId>& OutMultiServerCommitRemoteServers) final;
	void ExecuteMultiServerCommit() final;
	void AbortMultiServerCommit() final;
	void CommitMultiServerCommit() final;
	
	void OnLineReceived(uint64 Address, const TArray<uint8>& Data,
		FRemoteServerId LastMutatorServerId, uint64 LineVersion,
		FRemoteServerId SenderServerId, uint64 AllocationBase, uint64 AllocationSize);

	void OnReadLineRequested(uint64 Address, FRemoteWorkPriority Priority,
		FRemoteServerId Destination);

	void OnExclusiveRequested(uint64 Address, FRemoteWorkPriority Priority,
		FRemoteServerId Destination, bool bNeedsData);

	void OnExclusiveAck(uint64 Address, uint64 LineVersion,
		FRemoteServerId AckingServer,
		const TArray<FRemoteServerId>& ForwardedServers);

	void OnLineDeallocated(uint64 Address);

	void ExtractHeapRangeReadonly(uint64 StartAddress, uint64 Size, void* DstData, bool bWillWriteHint);
	void InsertHeapRangeMutable(uint64 StartAddress, uint64 Size, const void* SrcData);

	uint8 RemoteHeapRead1(uint64 Addr, bool bWillWriteHint);
	void RemoteHeapWrite1(uint64 Addr, uint8 Value);

	uint16 RemoteHeapRead2(uint64 Addr, bool bWillWriteHint);
	void RemoteHeapWrite2(uint64 Addr, uint16 Value);

	uint32 RemoteHeapRead4(uint64 Addr, bool bWillWriteHint);
	void RemoteHeapWrite4(uint64 Addr, uint32 Value);

	uint64 RemoteHeapRead8(uint64 Addr, bool bWillWriteHint);
	void RemoteHeapWrite8(uint64 Addr, uint64 Value);

	void RemoteHeapRead(void* DestPointer, uint64 Size, uint64 SourceAddress, bool bWillWriteHint);
	void RemoteHeapWrite(uint64 DestAddress, uint64 Size, const void* SourcePointer);

	void* RemoteHeapAllocate(uint64 Size);
	void RemoteHeapDeallocate(void* Ptr);
};

void RemoteHeapOnLineReceived(uint64 Address, const TArray<uint8>& Data,
	FRemoteServerId LastMutatorServerId, uint64 LineVersion,
	FRemoteServerId SenderServerId, uint64 AllocationBase, uint64 AllocationSize);
void RemoteHeapOnLineRequested(uint64 Address, FRemoteWorkPriority Priority,
	FRemoteServerId Destination);
void RemoteHeapOnExclusiveRequested(uint64 Address, FRemoteWorkPriority Priority,
	FRemoteServerId Destination, bool bNeedsData);
void RemoteHeapOnExclusiveAck(uint64 Address, uint64 LineVersion,
	FRemoteServerId AckingServer,
	const TArray<FRemoteServerId>& ForwardedServers);
void RemoteHeapOnLineDeallocated(uint64 Address);
void RemoteHeapDeallocate(uint64 Address);

} // namespace UE

// ERemoteHeapCoverageFlow is declared at global scope (outside namespace UE) so that
// UHT can generate reflection data for it via UENUM(). The enum is always present;
// only the mutable FRemoteHeapCoverage tracking struct is compiled out in non-test builds.
UENUM()
enum class ERemoteHeapCoverageFlow : uint32
{
	// TickSubsystem flows
	TickSubsystem_LineNotFound_AllocatingServer         UMETA(DisplayName="TickSubsystem: line not found, we are allocator -> notify deallocated"),
	TickSubsystem_LineRemote_Forward                    UMETA(DisplayName="TickSubsystem: line is Remote -> forward to LastMutatorServerId"),
	TickSubsystem_ExclusiveRequest_LineLocked           UMETA(DisplayName="TickSubsystem: lower-priority remote exclusive -> deferred"),
	TickSubsystem_ExclusiveRequest_FromExclusive        UMETA(DisplayName="TickSubsystem: evict Exclusive line, empty ForwardedServers"),
	TickSubsystem_ExclusiveRequest_FromShared_NoPendingWrite UMETA(DisplayName="TickSubsystem: evict Shared line, build ForwardedServers"),
	TickSubsystem_ExclusiveRequest_FromShared_PendingWrite   UMETA(DisplayName="TickSubsystem: yield pending write to higher-priority requester"),
	TickSubsystem_SharedRequest_LineLocked              UMETA(DisplayName="TickSubsystem: lower-priority remote read -> deferred"),
	TickSubsystem_SharedRequest_Served                  UMETA(DisplayName="TickSubsystem: send copy, downgrade Exclusive->Shared if needed"),
	TickSubsystem_SharedRequest_HasPendingExclusiveUpgrade   UMETA(DisplayName="TickSubsystem: add new child to NeededAcks, request eviction"),

	// TickRequest flows
	TickRequest_AlreadySatisfied                        UMETA(DisplayName="TickRequest: line present at required access level, skip"),
	TickRequest_NewExclusiveRequest                     UMETA(DisplayName="TickRequest: no pending write request -> send new exclusive RPC"),
	TickRequest_BoostExclusiveRequest                   UMETA(DisplayName="TickRequest: pending write exists at lower priority -> re-send at higher"),
	TickRequest_NewReadRequest                          UMETA(DisplayName="TickRequest: no pending read request -> send new read RPC"),
	TickRequest_BoostReadRequest                        UMETA(DisplayName="TickRequest: pending read exists at lower priority -> re-send at higher"),
	TickRequest_ReadSkipped_HigherPriorityWrite         UMETA(DisplayName="TickRequest: higher-priority write pending -> skip read request"),

	// OnLineReceived flows
	OnLineReceived_SharedRequestSatisfied               UMETA(DisplayName="OnLineReceived: pending Shared request satisfied, removed"),
	OnLineReceived_ExclusiveRequest_WaitingForAcks      UMETA(DisplayName="OnLineReceived: pending Exclusive request, data arrived, still waiting on acks"),

	// OnExclusiveAck flows
	OnExclusiveAck_Abandoned                            UMETA(DisplayName="OnExclusiveAck: no matching write request -> forward ack onward"),
	OnExclusiveAck_NewerVersion                         UMETA(DisplayName="OnExclusiveAck: ack version newer than tracked -> reset ack tracking"),
	OnExclusiveAck_SameVersion_Promoted                 UMETA(DisplayName="OnExclusiveAck: all acks collected and data present -> promote to Exclusive"),

	// OnLineDeallocated flow
	OnLineDeallocated                                   UMETA(DisplayName="OnLineDeallocated: remove line and pending requests from local cache"),

	Count                                               UMETA(Hidden)
};

namespace UE
{

#if WITH_DEV_AUTOMATION_TESTS

struct FRemoteHeapCoverage
{
	// Clears coverage. Call at the start of each test.
	COREUOBJECT_API static void Reset();

	// Records that a flow was reached.
	COREUOBJECT_API static void RecordFlow(ERemoteHeapCoverageFlow Flow);

	// Returns the coverage bitfield (since last Reset).
	COREUOBJECT_API static uint64 GetCoveredFlows();

	// Returns true if the given flow was hit since the last Reset.
	COREUOBJECT_API static bool IsFlowCovered(ERemoteHeapCoverageFlow Flow);

	// Returns a mask with a bit set for every defined flow (i.e. full coverage).
	COREUOBJECT_API static uint64 GetFullCoverageMask();
};

#endif // WITH_DEV_AUTOMATION_TESTS

struct FRemoteHeapAllocator
{
	using SizeType = int32;

	enum { NeedsElementType = false };
	enum { RequireRangeCheck = true };

	struct ForAnyElementType
	{
		~ForAnyElementType();

		void* GetAllocation() const
		{
			return Allocation;
		}

		SizeType GetInitialCapacity() const
		{
			return 0;
		}

		SizeType CalculateSlackReserve(SizeType NewMax, SizeType NumBytesPerElement) const
		{
			return DefaultCalculateSlackReserve(NewMax, NumBytesPerElement, false);
		}

		SizeType CalculateSlackShrink(SizeType NewMax, SizeType CurrentMax, SizeType NumBytesPerElement) const
		{
			return DefaultCalculateSlackShrink(NewMax, CurrentMax, NumBytesPerElement, false);
		}

		SizeType CalculateSlackGrow(SizeType NewMax, SizeType CurrentMax, SizeType NumBytesPerElement) const
		{
			return DefaultCalculateSlackGrow(NewMax, CurrentMax, NumBytesPerElement, false);
		}

		SizeType GetAllocatedSize(SizeType CurrentMax, SizeType NumBytesPerElement) const
		{
			return CurrentMax * NumBytesPerElement;
		}

		bool HasAllocation() const
		{
			return Allocation != nullptr;
		}

		COREUOBJECT_API void ResizeAllocation(SizeType CurrentNum, SizeType NewMax, SizeType NumBytesPerElement);

	private:
		void* Allocation = nullptr;
	};

	template<typename ElementType>
	struct ForElementType : public ForAnyElementType
	{
		ElementType* GetAllocation() const
		{
			return (ElementType*)ForAnyElementType::GetAllocation();
		}
	};
};

} // namespace UE
