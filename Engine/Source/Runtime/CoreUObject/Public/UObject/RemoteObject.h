// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/RemoteObjectCore.h"
#include "UObject/RemoteObjectTypes.h"
#include "UObject/ObjectMacros.h"
#include "Templates/RefCounting.h"

class UObject;
class UClass;
struct FTransactionalMigrationData;

/**
 * Remote objects are unique UObjects that are referenced by a local server instance but their memory is actually owned by (exists on) another server.
 * 
 * It's possible that an object is remote but its memory hasn't been freed yet (UObject with EInternalObjectFlags::Remote flag that hasn't been GC'd yet). 
 * In such case any attempt to access that object through TObjectPtr will result in its memory being migrated from a remote server to a local server. 
 * Remote object memory is freed in the next GC pass after the object has been migrated and any existing references to that object (must be referenced by TObjectPtr) 
 * will be updated by GC to point to the remote object's FRemoteObjectStub. 
 */
namespace UE::RemoteObject
{
	COREUOBJECT_API bool IsPointerOnRemoteHeap(const void* Pointer);
}

namespace UE::RemoteObject::Handle
{
	/**
	* Structure that holds a pointer to a native class or to a pathname of a class on disk (Blueprint etc)
	*/
	struct FRemoteObjectClass
	{
		UPTRINT PathNameOrClass = 0;

		FRemoteObjectClass() = default;
		COREUOBJECT_API explicit FRemoteObjectClass(UClass* InClass);

		bool IsNative() const
		{
			return !(PathNameOrClass & 1);
		}

		bool IsValid() const
		{
			return !!PathNameOrClass;
		}
		
		COREUOBJECT_API UClass* GetClass() const;
	};

	// Structure that holds basic information about an object that has been migrated (from or to the local server)
	// This is what FObjectPtr that references a remote object actually points to after the remote object's memory has been claimed by GC or before the remote object has been resolved.
	struct FRemoteObjectStub
	{
		FRemoteObjectId Id;
		FRemoteObjectId OuterId;
		FName Name;
		FRemoteObjectClass Class;
		TRefCountPtr<FTransactionalMigrationData> MigratedData;
		bool bWasGarbage = false;

		/** SerialNumber this object had on this server */
		int32 SerialNumber = 0;

		/** Server id that where the object currently resides */
		FRemoteServerId ResidentServerId;

		/** Server id of the server that has ownership of the object
			(note: only valid if the object is Local) */
		FRemoteServerId OwningServerId;

		FRemoteObjectStub() = default;
		UE_DEPRECATED(5.8, "Stubs should only be initialized with UObject properties when they are being added to the internal stub map.")
		COREUOBJECT_API explicit FRemoteObjectStub(UObject* Object);
	};

	enum class ERemoteReferenceType
	{
		Strong = 0,
		Weak = 1
	};

	enum class ERemoteObjectGetClassBehavior
	{
		/** If the object was never local, then simply return a null class */
		ReturnNullIfNeverLocal,

		/** If the object was never local, force a migration to enforce a correct return value */
		MigrateIfNeverLocal,
	};

	/**
	 * Advanced use only.
	 * If you are in a situation where you are using raw pointer API's that potentially cause migrations, but you need to
	 * grab the FRemoteObjectId of that UObject, you can use this function.
	 * For the duration of the passed-in CodeThatReturnsRawUObjectPtr, we will disable attempts to resolve remote object pointers.
	 * The UObject that is returned from CodeThatReturnsRawUObjectPtr will be converted to a FRemoteObjectId.
	 * @param CodeThatReturnsRawUObjectPtr a short lambda or function that accesses nothing but the single UObject it is returning.
	 * @return the FRemoteObjectId for the returned value. It can be invalid (in the case of nullptr).
	 */
	COREUOBJECT_API FRemoteObjectId GetRemoteObjectId(TFunctionRef<const UObject*()> CodeThatReturnsRawUObjectPtr);

	/**
	* Resolves a remote object given its stub, aborting the active transaction if the object is unavailable
	* @param Stub Basic data required to migrate a remote object
	* @param RefType Reference type that wants to resolve an object
	* @return Resolved object
	*/
	COREUOBJECT_API UObject* ResolveObject(FRemoteObjectStub* Stub, ERemoteReferenceType RefType = ERemoteReferenceType::Strong);

	/**
	* Resolves a remote object, aborting the active transaction if the object is unavailable
	* @param Object Object to resolve (remote object memory that has not yet been GC'd)
	* @param RefType Reference type that wants to resolve an object
	* @return Resolved object
	*/
	COREUOBJECT_API UObject* ResolveObject(UObject* Object, ERemoteReferenceType RefType = ERemoteReferenceType::Strong);
	
	COREUOBJECT_API void TouchResidentObject(UObject* Object);

	/**
	 * Attempts to Get the Class of a given ObjectId without performing a migration.  This will succeed if the ObjectId has ever been local.
	 * If the RemoteObjectId has never been local, you can decide if a Migration will occur or Null should be returned.
	 */
	COREUOBJECT_API UClass* GetClass(FRemoteObjectId ObjectId, ERemoteObjectGetClassBehavior GetClassBehavior);

	/**
	 * Determine if an Object can be resolved (exists in memory, or could be migrated)
	 * @param ObjectId The ObjectId to consider for resolution
	 * @param CanResolveObjectBehavior which behavior semantics should we consider when determining if an object is resolvable?
	 * @return True if a remote object can be resolved using the given behavior semantics
	*/
	COREUOBJECT_API bool CanResolveObject(FRemoteObjectId ObjectId, ECanResolveObjectBehavior CanResolveObjectBehavior);

	/**
	* Gets the current residency of an object
	* @param ObjectId Id of the object to check
	* @return Residency of the object
	*/
	COREUOBJECT_API EResidence GetResidence(FRemoteObjectId ObjectId);

	/**
	* Gets the current residency of an object
	* @param Stub Remote object stub representing the object to check
	* @return Residency of the object
	*/
	COREUOBJECT_API EResidence GetResidence(const FRemoteObjectStub* Stub);

	/**
	* Gets the current residency of an object
	* @param Object Object whose residency is to be checked
	* @return Residency of the object
	*/
	COREUOBJECT_API EResidence GetResidence(const UObject* Object);

	/**
	* Checks if an object associated with the specified unique id is remote
	* @return True if an object associated with the specified unique id is remote
	*/
	UE_DEPRECATED(5.8, "Use GetResidence(ObjectId) instead.")
	COREUOBJECT_API bool IsRemote(FRemoteObjectId ObjectId);

	/**
	* Checks if an object associated with the specified stub is remote
	* @return True if an object associated with the specified stub is remote
	*/
	UE_DEPRECATED(5.8, "Use GetResidence(Stub) instead.")
	COREUOBJECT_API bool IsRemote(const FRemoteObjectStub* Stub);

	/**
	* Checks if an object (memory that has not yet been GC'd) is remote
	* @return True if the object is remote
	*/
	UE_DEPRECATED(5.8, "Use GetResidence(Object) instead.")
	COREUOBJECT_API bool IsRemote(const UObject* Object);

	/**
	* Checks if a locally resident object is owned by this server
	*/
	COREUOBJECT_API bool IsOwned(const UObject* Object);

	/**
	* Checks if an object id is owned by this server
	* We are only able to check if we own the object. If we don't
	* own the object then we don't have a reliable way of knowing
	* who the owner is, which is why the below GetOwnerServerId
	* function requires the object be locally resident
	*/
	COREUOBJECT_API bool IsOwned(FRemoteObjectId ObjectId);

	/**
	* Checks if an object id represents a valid object in the same sense as the global IsValid(Object)
	* This information may not be accurate if an object has been marked as garbage on a remote server and hasn't been migrated back yet
	* @params ObjectId remote object id
	* @return true if an object id represents an object that is in memory and is not marked as garbage or a remote object which was not marked as garbage when it was being migrated, otherwise false (or when the provided id is not internally known)
	*/
	COREUOBJECT_API bool IsValid(FRemoteObjectId ObjectId);

	/**
	* Get the owner server id for a locally resident object
	*/
	COREUOBJECT_API FRemoteServerId GetOwnerServerId(const UObject* Object);

	/**
	* Get the owner server id for the specified object id (may not report the correct owner if the object is not local)
	*/
	COREUOBJECT_API FRemoteServerId GetOwnerServerId(FRemoteObjectId ObjectId);

	/**
	* Sets the owner server id for a locally resident object
	*/
	COREUOBJECT_API void ChangeOwnerServerId(const UObject* Object, FRemoteServerId NewOwnerServerId);

	/**
	* Checks if the current callstack is inside of a transactional PostMigrate step of migrating objects to this server
	*/
	COREUOBJECT_API bool IsTransactionallyPostMigratingObjects();
}
