// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteObjectTypes.h"

/**
* Structure that holds unique Names and RemoteIds associated with remote object data or pathnames (noexport type)
*/
struct FRemoteObjectTables
{
	using FNameIndexType = uint16;

	TArray<FName> Names;
	TArray<FRemoteObjectId> RemoteIds;

	/**
     * Compare function instead of operator== so that it's not used accidentally for performance reasons
	 */
	bool Compare(const FRemoteObjectTables& InOtherTables) const
	{
		return Names == InOtherTables.Names && RemoteIds == InOtherTables.RemoteIds;
	}

	UE_FORCEINLINE_HINT int32 Num() const
	{
		return Names.Num();
	}

	FNameIndexType AddUniqueName(const FName& Name)
	{
		int32 NameIndex = Names.IndexOfByKey(Name);
		if (NameIndex == INDEX_NONE)
		{
			NameIndex = Names.Add(Name);
		}
		return IntCastChecked<FNameIndexType>(NameIndex);
	}

	COREUOBJECT_API void Serialize(FArchive& Ar);
};

inline FArchive& operator<<(FArchive& Ar, FRemoteObjectTables& Tables)
{
	Tables.Serialize(Ar);
	return Ar;
}


/**
* Structure that represents remote object pathname (noexport type)
* Stores pathnames as an array of indices (into a sidecar FRemoteObjectTables object) of FNames and associated FRemoteObjectIds
* Names and Ids are stored from the innermost object first to the outermost object last
*/
struct FPackedRemoteObjectPathName
{
	using FNameIndexType = FRemoteObjectTables::FNameIndexType;
	TArray<FNameIndexType> RemoteIds;
	TArray<FNameIndexType> Names;

	UE_FORCEINLINE_HINT int32 Num() const
	{
		return Names.Num();
	}

	/**
     * Compare function instead of operator== so that it's not used accidentally for performance reasons
	 */
	bool Compare(const FPackedRemoteObjectPathName& InOtherPathName) const
	{
		return RemoteIds == InOtherPathName.RemoteIds && Names == InOtherPathName.Names;
	}

	UE_FORCEINLINE_HINT FName GetSegmentName(int32 InSegmentIndex, const FRemoteObjectTables& InTables) const
	{
		return InTables.Names[Names[InSegmentIndex]];
	}

	UE_FORCEINLINE_HINT FRemoteObjectId GetSegmentId(int32 InSegmentIndex, const FRemoteObjectTables& InTables) const
	{
		return InTables.RemoteIds[RemoteIds[InSegmentIndex]];
	}

	COREUOBJECT_API UObject* Resolve(const FRemoteObjectTables& InTables) const;
	COREUOBJECT_API FString ToString(const FRemoteObjectTables& InTables, int32 InMinPathSegmentIndex = 0) const;
	COREUOBJECT_API void Serialize(FArchive& Ar);
};

inline FArchive& operator<<(FArchive& Ar, FPackedRemoteObjectPathName& PathName)
{
	PathName.Serialize(Ar);
	return Ar;
}

/**
* Structure that represents remote object pathname (noexport type)
* Stores pathnames as an array of FNames and associated FRemoteObjectIds
* Names and Ids are stored from the innermost object first to the outermost object last
*/
struct FRemoteObjectPathName : public FRemoteObjectTables
{
	FRemoteObjectPathName() = default;
	FRemoteObjectPathName(FRemoteObjectPathName&&) = default;
	COREUOBJECT_API explicit FRemoteObjectPathName(UObject* InObject);
	COREUOBJECT_API explicit FRemoteObjectPathName(FRemoteObjectId RemoteId);
	FRemoteObjectPathName& operator=(FRemoteObjectPathName&&) = default;

	FRemoteObjectPathName(const FRemoteObjectPathName&) = default;
	FRemoteObjectPathName& operator=(const FRemoteObjectPathName&) = default;

	UE_FORCEINLINE_HINT FName GetSegmentName(int32 InSegmentIndex, const FRemoteObjectTables&) const
	{
		return Names[InSegmentIndex];
	}

	UE_FORCEINLINE_HINT FRemoteObjectId GetSegmentId(int32 InSegmentIndex, const FRemoteObjectTables&) const
	{
		return RemoteIds[InSegmentIndex];
	}

	UE_FORCEINLINE_HINT FName GetObjectName() const
	{
		if (Names.Num())
		{
			return Names[0];
		}
		return FName();
	}

	COREUOBJECT_API UObject* Resolve() const;
	COREUOBJECT_API FString ToString(int32 InMinPathSegmentIndex = 0) const;
};