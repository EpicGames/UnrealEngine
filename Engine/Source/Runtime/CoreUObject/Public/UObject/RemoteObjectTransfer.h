// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/AssertionMacros.h"
#include "UObject/RemoteObjectPathName.h"
#include "Async/Async.h"
#include "UObject/RemoteExecutor.h"
#include "Misc/UEOps.h"

namespace UE::Net
{
	struct FRemoteObjectReferenceNetSerializer;
}

struct FUObjectMigrationContext;
struct FRemoteObjectData;

/**
* Structure that holds serialized remote object data chunk (< 64KB of data) (noexport type)
*/
struct FRemoteObjectBytes
{
	TArray<uint8> Bytes;

	bool Compare(const FRemoteObjectBytes& InOtherData) const
	{
		return Bytes == InOtherData.Bytes;
	}
};

/**
* Structure that holds packed serialized remote object header (noexport type)
*/
struct FPackedRemoteObjectHeader
{
	FRemoteObjectTables::FNameIndexType NameIndex = 0;
	FRemoteObjectTables::FNameIndexType IdIndex = 0;
	FRemoteObjectTables::FNameIndexType ClassIndex = 0;

	bool Compare(const FPackedRemoteObjectHeader& InOtherHeader) const
	{
		return NameIndex == InOtherHeader.NameIndex && IdIndex == InOtherHeader.IdIndex && ClassIndex == InOtherHeader.ClassIndex;
	}
};

/**
* Iterator that maps packed serialized remote object id indices to the actual unique remote object ids
*/
class FSerializedRemoteObjectIterator
{
	const FRemoteObjectData& Data;
	int32 Index = 0;

public:

	FSerializedRemoteObjectIterator(const FRemoteObjectData& InData, int32 InIndex = 0)
		: Data(InData)
		, Index(InIndex)
	{
	}
	inline FSerializedRemoteObjectIterator& operator++()
	{
		++Index;
		return *this;
	}
	// UEOpEquals is required by UEOps.h to support range-for iteration
	[[nodiscard]] FORCEINLINE bool UEOpEquals(const FSerializedRemoteObjectIterator& Other) const
	{
		return (Index == Other.Index);
	}
	inline int32 GetIndex() const
	{
		return Index;
	}
	// operator bool is for the old UE-style iterator iteration support
	inline operator bool() const;
	inline FRemoteObjectId GetId() const;
	inline FName GetName() const;
	inline UClass* GetClass() const;
	inline FRemoteObjectId operator*() const
	{
		return GetId();
	}
};

/**
* Structure that holds remote object memory (noexport type)
*/
struct FRemoteObjectData
{
	/** Contains a unique id of the migration this data was created for. ServerId represents the sending side */
	FRemoteObjectId MigrationId;
	/** Tables containing lists of unique remote ids and FNames */
	FRemoteObjectTables Tables;
	/** List of unique pathnames (stored as a list of indices of unique FNames) */
	TArray<FPackedRemoteObjectPathName> PathNames;
	/** List of packed remote object headers of the objects serialized in this ObjectData struct (in the same order they were serialized so it matches the order of object headers after deserialization) */
	TArray<FPackedRemoteObjectHeader> SerializedObjectHeaders;
	/** Serialized object headers and data (properties) */
	TArray<FRemoteObjectBytes> Bytes;

	/**
	* Returns true if both structures contain identical data
	*/
	COREUOBJECT_API bool Compare(const FRemoteObjectData& InOtherData) const;

	/**
	* Returns the size of serialized data
	*/
	inline int32 GetNumBytes() const
	{
		int32 Num = 0;
		for (const FRemoteObjectBytes& Chunk : Bytes)
		{
			Num += Chunk.Bytes.Num();
		}
		return Num;
	}

	/**
	* Returns a unique remote object id based on its index the id tables
	*/
	inline FRemoteObjectId GetRemoteObjectId(FRemoteObjectTables::FNameIndexType IdIndex) const
	{
		return Tables.RemoteIds[IdIndex];
	}

	/**
	* Returns a remote object id for the serialized object
	* @param ObjectIndex Index of the serialized object
	* @return Remote Object Id of the serialized object
	*/
	inline FRemoteObjectId GetSerializedObjectId(int32 ObjectIndex) const
	{
		return GetRemoteObjectId(SerializedObjectHeaders[ObjectIndex].IdIndex);
	}

	/**
	* Returns the root serialized object's remote id (the first serialized object's id)
	* @return Remote Object Id of the root serialized object
	*/
	inline FRemoteObjectId GetRootSerializedObjectId() const
	{
		return SerializedObjectHeaders.Num() ? GetSerializedObjectId(0) : FRemoteObjectId();
	}

	/**
	* Returns a unique FName based on its index the name tables
	*/
	inline FName GetName(FRemoteObjectTables::FNameIndexType NameIndex) const
	{
		return Tables.Names[NameIndex];
	}

	/**
	* Attempts to return an object name associated with a remote id. 'None' if the id does not represent an object serialized in this object data.
	*/
	inline FName GetName(FRemoteObjectId RemoteId) const
	{
		for (int32 ObjectIndex = 0; ObjectIndex < SerializedObjectHeaders.Num(); ++ObjectIndex)
		{
			if (GetSerializedObjectId(ObjectIndex) == RemoteId)
			{
				return GetName(SerializedObjectHeaders[ObjectIndex].NameIndex);
			}
		}
		return FName();
	}

	// Support for iterating over serialized object ids:
	// 
	// FRemoteObjectData Data;
	// for (FRemoteObjectId SerializedObjectId : Data) { }
	
	FSerializedRemoteObjectIterator begin() const
	{
		return FSerializedRemoteObjectIterator(*this, 0);
	}
	FSerializedRemoteObjectIterator end() const
	{
		return FSerializedRemoteObjectIterator(*this, SerializedObjectHeaders.Num());
	}
};

inline FSerializedRemoteObjectIterator::operator bool() const
{
	return Index < Data.SerializedObjectHeaders.Num();
}
inline FRemoteObjectId FSerializedRemoteObjectIterator::GetId() const
{
	return Data.Tables.RemoteIds[Data.SerializedObjectHeaders[Index].IdIndex];
}
inline FName FSerializedRemoteObjectIterator::GetName() const
{
	return Data.Tables.Names[Data.SerializedObjectHeaders[Index].NameIndex];
}
inline UClass* FSerializedRemoteObjectIterator::GetClass() const
{
	return Cast<UClass>(Data.PathNames[Data.SerializedObjectHeaders[Index].ClassIndex].Resolve(Data.Tables));
}

namespace UE::RemoteObject::Serialization::Network
{
	/** 
	 * Add a RPC, and its arguments, for an object to a queue to be processed later.
	 *
	 * There is a queue per root object that contains all RPCs for that root object or any of its sub-objects.
	 *
	 * If SubObject is null then its assumed the RPC is for the actor.
	 */
	COREUOBJECT_API void EnqueueRPC(UObject* RootObject, UObject* SubObject, UFunction* Function, void* Parameters);

	/**
	 * Serialize and deserialize and RPC queue for a root object.
	 *
	 * Each RPC that is serialized will be removed from the root object's queue, while each RPC that is deserialized
	 * will be added to the root object's queue.
	 */
	COREUOBJECT_API void SerializeRPCQueue(UObject* RootObject, FArchive& Ar);

	/** 
	 * Process all RPCs queued up for a root object and its components and sub-objects.
	 */
	COREUOBJECT_API void ProcessRPCQueue(UObject* RootObject);
}

namespace UE::RemoteObject::Transfer
{
	/** Information for performing a migration (send) an object to a remote server */
	struct FMigrateSendParams
	{
		/** The migration context (meta data) of the send request */
		FUObjectMigrationContext& MigrationContext;

		/** The serialized data of the object being sent */
		FRemoteObjectData ObjectData;
	};

	/**
	* Called when remote object data has been received from a remote server
	* @param ObjectId Remote object id
	* @param RemoteServerId Server that sent object data
	* @param Data Remote object data. Data ownership is transferred to the remote object queue
	*/
	COREUOBJECT_API void OnObjectDataReceived(FRemoteServerId OwnerServerId, FRemoteObjectId ObjectId, FRemoteServerId RemoteServerId, FRemoteObjectData& Data);

	/**
	* Called when a remote object request was denied by a remote server
	* @param ObjectId Remote object id
	* @param RemoteServerId Server that owns the object data
	*/
	COREUOBJECT_API void OnObjectDataDenied(FRemoteObjectId ObjectId, FRemoteServerId RemoteServerId);

	/**
	* Migrates and transfers ownership of an object to a remote server
	* @param Object Local object to migrate
	* @param DestinationServerId Id of a remote server to receive the object and assume ownership
	*/
	COREUOBJECT_API void TransferObjectOwnershipToRemoteServer(UObject* Object, FRemoteServerId DestinationServerId);

	/**
	* Migrates an object to a remote server without changing ownership
	* @param ObjectId Remote object id
	* @param DestinationServerId Id of a remote server to receive the object
	*/
	COREUOBJECT_API void MigrateObjectToRemoteServer(FRemoteObjectId ObjectId, FRemoteServerId DestinationServerId);
	COREUOBJECT_API void MigrateObjectToRemoteServerWithExplicitPriority(FRemoteWorkPriority RequestPriority, FRemoteObjectId Id, FRemoteServerId DestinationServerId);
	
	/**
	* Migrates an object from a remote server (temp function)
	* @param ObjectId Remote object id
	* @param CurrentOwnerServerId Id of a remote server that currently owns the object
	* @return Migrated object
	*/
	COREUOBJECT_API void MigrateObjectFromRemoteServer(FRemoteObjectId ObjectId, FRemoteServerId CurrentOwnerServerId);

	/**
	* Deserializes objects (possibly creating them in the process) using data from a remote server and a migration context inside of a transaction
	* @param MigratedData Data received from a remote server coupled with the migration context struct
	*/
	void TransactionallyMigrateObjects(TRefCountPtr<FTransactionalMigrationData> MigratedData);

	/**
	* Reports code that touches a resident object
	*/
	COREUOBJECT_API void TouchResidentObject(UObject* Object);

	/**
	* Registers object ID as known to be owned by another server, without migrating it
	* @param ObjectId Remote object id
	* @param ResidentServerId (optional) Id of a remote server where the object currently resides. Note that the resident server id will only be updated for objects that have not yet been registered on this server.
	*/
	COREUOBJECT_API void RegisterRemoteObjectId(FRemoteObjectId Id, FRemoteServerId ResidentServerId = FRemoteServerId());

	/**
	* Returns the list of all object IDs currently borrowed from another server.
	*/
	COREUOBJECT_API void GetAllBorrowedObjects(TArray<FRemoteObjectId>& OutBorrowedObjectIds);

	/**
	* Registers object for sharing, marking it as owned by the current server
	* @param Object to make shareable
	*/
	COREUOBJECT_API void RegisterSharedObject(UObject* Object);

	extern COREUOBJECT_API const FRemoteServerId DatabaseId;

	COREUOBJECT_API void InitRemoteObjectTransfer(class FRemoteExecutor* Executor);

	/**
	* Returns object data that has not yet been deserialized on this server to their respective owners
	*/
	COREUOBJECT_API int32 ReturnObjectDataToOwnedServers();

	/** Delegate that transfers object data to another server */
	extern COREUOBJECT_API TDelegate<void(const FMigrateSendParams& /*Params*/)> RemoteObjectTransferDelegate;

	/** Delegate that handles an object request being denied */
	extern COREUOBJECT_API TDelegate<void(FRemoteObjectId /*ObjectId*/, FRemoteServerId /*DestinationServerId*/)> RemoteObjectDeniedTransferDelegate;

	/** Delegate that requests remote object data from LastKnownResidentServerId to be transferred to DestinationServerId. Allows requests to be forwarded if ObjectId does not reside on LastKnownResidentServerId. */
	extern COREUOBJECT_API TDelegate<void(FRemoteWorkPriority /*RequestPriority*/, FRemoteObjectId /*ObjectId*/, FRemoteServerId /*LastKnownResidentServerId*/, FRemoteServerId /*DestinationServerId*/)> RequestRemoteObjectDelegate;

	/***************************************************************************************
	 * Notification delegates (not required to be bound to); useful for stats reporting
	 */

	/** Delegate executed when objects have been migrated from another server. */
	extern COREUOBJECT_API TMulticastDelegate<void(const TArray<UObject*>& /*ReceivedObjects*/, const FRemoteObjectData& /*ObjectData*/, const FUObjectMigrationContext& /*MigrationContext*/)> OnObjectsReceivedDelegate;

	/** Delegate executed when object data has been migrated to another server */
	extern COREUOBJECT_API TMulticastDelegate<void(const TSet<TObjectPtr<UObject>>& /*SentObjects*/,  const FRemoteObjectData& /*ObjectData*/, const FUObjectMigrationContext& /*MigrationContext*/)> OnObjectsSentDelegate;

	/** Delegate executed when object data has been loaded from disk */
	extern COREUOBJECT_API TMulticastDelegate<void(const TArray<UObject*>& /*LoadedObjects*/, const FRemoteObjectData& /*ObjectData*/, const FUObjectMigrationContext& /*MigrationContext*/)> OnObjectsLoadedFromDiskDelegate;

	/** Delegate executed when object data has been saved to disk */
	extern COREUOBJECT_API TMulticastDelegate<void(const TSet<UObject*>& /*SavedObjects*/, const FRemoteObjectData& /*ObjectData*/, const FUObjectMigrationContext& /*MigrationContext*/)> OnObjectsSavedToDiskDelegate;

	/** Delegate executed when an object has been accessed by a transaction */
	extern COREUOBJECT_API TMulticastDelegate<void(FRemoteTransactionId /*RequestId*/, FRemoteObjectId /*ObjectId*/)> OnObjectTouchedDelegate;

	/***************************************************************************************/

	/** Delegate that stores locally unrachable object data into a database. */
	extern COREUOBJECT_API TDelegate<void(const FMigrateSendParams&)> StoreRemoteObjectDataDelegate;

	/** Delegate that restores object data from a database. */
	extern COREUOBJECT_API TDelegate<void(const FUObjectMigrationContext&)> RestoreRemoteObjectDataDelegate;

} // namespace UE::RemoteObject::Transfer

struct FRemoteObjectReference
{
private:
	/** Object id being shared with another server */
	FRemoteObjectId ObjectId;
	/** Id of a server that shared the object (last owner of the object) */
	FRemoteServerId ServerId;

public:
	FRemoteObjectReference() = default;

	FRemoteObjectReference(const FRemoteObjectReference&) = default;
	FRemoteObjectReference& operator=(const FRemoteObjectReference&) = default;

	COREUOBJECT_API explicit FRemoteObjectReference(const FObjectPtr& Ptr);

	template <typename T>
	explicit FRemoteObjectReference(const TObjectPtr<T>& Ptr)
		: FRemoteObjectReference(FObjectPtr(Ptr.GetHandle()))
	{
	}

	COREUOBJECT_API explicit FRemoteObjectReference(const FWeakObjectPtr& WeakPtr);

	template <typename T>
	explicit FRemoteObjectReference(const TWeakObjectPtr<T>& WeakPtr)
		: FRemoteObjectReference((FWeakObjectPtr)WeakPtr)
	{
	}

	UE_FORCEINLINE_HINT bool operator==(const FRemoteObjectReference& Other) const
	{
		return ObjectId == Other.ObjectId;
	}

	COREUOBJECT_API FObjectPtr ToObjectPtr() const;
	COREUOBJECT_API FWeakObjectPtr ToWeakPtr() const;
	COREUOBJECT_API UObject* Resolve() const;

	UE_FORCEINLINE_HINT FRemoteObjectId GetRemoteId() const
	{
		return ObjectId;
	}
	UE_FORCEINLINE_HINT FRemoteServerId GetSharingServerId() const
	{
		return ServerId;
	}

	UE_FORCEINLINE_HINT EResidence GetResidence() const
	{
		return UE::RemoteObject::Handle::GetResidence(ObjectId);
	}

	UE_DEPRECATED(5.8, "Use GetResidence(ObjectId) instead.")
	UE_FORCEINLINE_HINT bool IsRemote() const
	{
		return GetResidence() == EResidence::Remote;
	}

	COREUOBJECT_API bool Serialize(FArchive& Ar);
	COREUOBJECT_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	friend FArchive& operator<<(FArchive& Ar, FRemoteObjectReference& Ref);

private:
	friend struct UE::Net::FRemoteObjectReferenceNetSerializer;
	COREUOBJECT_API void NetDequantize(FRemoteObjectId InObjectId, FRemoteServerId InServerId, const FRemoteObjectPathName& InPath);
};

template<>
struct TStructOpsTypeTraits<FRemoteObjectReference> : public TStructOpsTypeTraitsBase2<FRemoteObjectReference>
{
	enum
	{
		WithNetSerializer = true,
		WithSerializer = true,
	};
};

inline FArchive& operator<<(FArchive& Ar, FRemoteObjectReference& Ref)
{
	Ref.Serialize(Ar);
	return Ar;
}
