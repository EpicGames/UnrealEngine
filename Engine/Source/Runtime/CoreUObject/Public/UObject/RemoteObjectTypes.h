// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/TypeHash.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "UObject/ObjectMacros.h"

class UObjectBase;
class UObject;
class FArchive;

namespace UE::Net
{
	struct FRemoteObjectReferenceNetSerializer;
	struct FRemoteServerIdNetSerializer;
	struct FRemoteObjectIdNetSerializer;
}
namespace UE::RemoteObject::Private
{
	struct FRemoteIdLocalizationHelper;
}
namespace UE::RemoteObject::Handle
{
	enum class ECanResolveObjectBehavior
	{
		/** Do not consider any state when considering if an Object is resolvable */
		Default,

		/** If an Object is garbage, say it is not resolvable (optimize away nullptr resolves) */
		CannotResolveGarbageObjects
	};
}

constexpr inline int32 REMOTE_OBJECT_RESERVED_BIT_SIZE = 2;
constexpr inline int32 REMOTE_OBJECT_SERVER_ID_BIT_SIZE = 10;
constexpr inline int32 REMOTE_OBJECT_SERIAL_NUMBER_BIT_SIZE = 52;

constexpr inline uint64 MAX_REMOTE_OBJECT_SERVER_ID = ((uint64_t)1 << REMOTE_OBJECT_SERVER_ID_BIT_SIZE) - 1;
constexpr inline uint64 MAX_REMOTE_OBJECT_SERIAL_NUMBER = ((uint64_t)1 << REMOTE_OBJECT_SERIAL_NUMBER_BIT_SIZE) - 1;

enum class ERemoteServerIdConstants : uint32
{
	Invalid = 0,
	FirstValid,
	Max = MAX_REMOTE_OBJECT_SERVER_ID,

	Database = Max,
	Asset = Max - 1,
	Local = Max - 2,

	// Add new reserved server IDs above this line in a descending order (so the next would be Something = Max - 2)

	FirstReservedPlusOne,
	FirstReserved = FirstReservedPlusOne - 1
};

struct FRemoteServerId
{
	friend struct FRemoteObjectId;
	friend UE::RemoteObject::Private::FRemoteIdLocalizationHelper;
	friend struct UE::Net::FRemoteObjectReferenceNetSerializer;
	friend struct UE::Net::FRemoteServerIdNetSerializer;
	friend struct UE::Net::FRemoteObjectIdNetSerializer;

	FRemoteServerId() = default;
	FRemoteServerId(const FRemoteServerId&) = default;
	FRemoteServerId& operator=(const FRemoteServerId&) = default;

	explicit FRemoteServerId(ERemoteServerIdConstants InId)
		: Id((uint32)InId)
	{
	}

	static COREUOBJECT_API FRemoteServerId FromString(const FStringView& InText);
	COREUOBJECT_API FString ToString() const;

	uint32 GetIdNumber() const
	{
		return GetGlobalized().Id;
	}

	bool IsValid() const
	{
		return Id != (uint32)ERemoteServerIdConstants::Invalid;
	}

	inline int32 Compare(FRemoteServerId Other) const
	{
		VerifyComparableServerIds(Id, Other.Id);
		return (int32)GlobalizeId(Id) - (int32)GlobalizeId(Other.Id);
	}

	bool operator == (FRemoteServerId Other) const
	{
		return Compare(Other) == 0;
	}
	bool operator != (FRemoteServerId Other) const
	{
		return Compare(Other) != 0;
	}
	bool operator < (FRemoteServerId Other) const
	{
		return Compare(Other) < 0;
	}
	bool operator <= (FRemoteServerId Other) const
	{
		return Compare(Other) <= 0;
	}
	bool IsAsset() const
	{
		return (Id == (uint32)ERemoteServerIdConstants::Asset);
	}
	bool IsDatabase() const
	{
		return (Id == (uint32)ERemoteServerIdConstants::Database);
	}
	bool IsLocal() const
	{
		return (Id == (uint32)ERemoteServerIdConstants::Local) || (*this == GlobalServerId);
	}

	static UE_FORCEINLINE_HINT FRemoteServerId GetLocalServerId()
	{
		return FRemoteServerId(ERemoteServerIdConstants::Local);
	}
	
	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FRemoteServerId& Id);

	static COREUOBJECT_API void InitGlobalServerId(FRemoteServerId Id);

	static COREUOBJECT_API bool IsGlobalServerIdInitialized();

	static COREUOBJECT_API FRemoteServerId FromIdNumber(uint32 InNumber);

	COREUOBJECT_API bool Serialize(FArchive& Ar);
	COREUOBJECT_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	UE_FORCEINLINE_HINT uint32 GetTypeHash() const
	{
		return ::GetTypeHash(GetLocalized().Id);
	}

	friend void AssignGlobalServerIdDebuggingState();

private:

	inline FRemoteServerId GetLocalized() const
	{
		return (Id == GlobalServerId.Id) ? FRemoteServerId(ERemoteServerIdConstants::Local) : *this;
	}

	inline FRemoteServerId GetGlobalized() const
	{
		if (Id == (uint32)ERemoteServerIdConstants::Local)
		{
			checkf(GlobalServerId.Id != (uint32)ERemoteServerIdConstants::Local, TEXT("Unable to convert Local server id to global. Global server id hasn't been initialized yet."));
			return GlobalServerId;
		}
		return *this;
	}

	inline static void VerifyComparableServerIds(uint32 Id0, uint32 Id1)
	{
#if DO_CHECK
		if (!IsGlobalServerIdInitialized())
		{
			checkf(Id0 != (uint32)ERemoteServerIdConstants::Local || Id1 == (uint32)ERemoteServerIdConstants::Invalid || Id1 >= (uint32)ERemoteServerIdConstants::FirstReserved,
				TEXT("When the global server id is not initialized it's only possible to compare predefined server IDs (lhs: %u, rhs: %u)"), Id0, Id1);
			checkf(Id1 != (uint32)ERemoteServerIdConstants::Local || Id0 == (uint32)ERemoteServerIdConstants::Invalid || Id0 >= (uint32)ERemoteServerIdConstants::FirstReserved,
				TEXT("When the global server id is not initialized it's only possible to compare predefined server IDs (lhs: %u, rhs: Local)"), Id0);
		}
#endif
	}

	inline static uint32 GlobalizeId(uint32 InId)
	{
		if (InId == (uint32)ERemoteServerIdConstants::Local)
		{
			return GlobalServerId.Id;
		}
		return InId;
	}

	/** 
	 * Constructor used exclusively by FRemoteObjectId::GetServerId() to bypass range changes in the other constructors 
	 */
	FRemoteServerId(uint32 InId, ERemoteServerIdConstants /*Unused*/)
		: Id(InId)
	{
	}

	uint32 Id = (uint32)ERemoteServerIdConstants::Invalid;

	COREUOBJECT_API static FRemoteServerId GlobalServerId;
};

enum class ERemoteIdToStringVerbosity
{
	Default = 0,
	Id = 1,
	Name = 2,
	PathName = 3,
	FullName = 4,
	FullNameAttributes = 5,

	LastPlusOne,
	Max = LastPlusOne - 1
};

struct FRemoteObjectId
{
private:
	union
	{
		struct
		{
			uint64 Reserved : REMOTE_OBJECT_RESERVED_BIT_SIZE;  /* Reserved for static flag in NetGUID and FRemoteObjectHandle */
			uint64 SerialNumber : REMOTE_OBJECT_SERIAL_NUMBER_BIT_SIZE;
			uint64 ServerId : REMOTE_OBJECT_SERVER_ID_BIT_SIZE;
		};
		uint64 Id = 0;
	};

	static_assert((REMOTE_OBJECT_RESERVED_BIT_SIZE + REMOTE_OBJECT_SERIAL_NUMBER_BIT_SIZE + REMOTE_OBJECT_SERVER_ID_BIT_SIZE) <= (sizeof(Id) * 8), "FRemoteObjectId component bit sizes must fit within the Id member");

	inline int64 GetComparableValue() const
	{
		return int64(SerialNumber) | (int64(FRemoteServerId::GlobalizeId(ServerId)) << REMOTE_OBJECT_SERIAL_NUMBER_BIT_SIZE);
	}

	inline int64 Compare(FRemoteObjectId Other) const
	{
		FRemoteServerId::VerifyComparableServerIds(ServerId, Other.ServerId);
		return GetComparableValue() - Other.GetComparableValue();
	}

	inline FRemoteObjectId GetLocalized() const
	{
		return FRemoteObjectId(GetServerId().GetLocalized(), SerialNumber);
	}

	inline FRemoteObjectId GetGlobalized() const
	{
		return FRemoteObjectId(GetServerId().GetGlobalized(), SerialNumber);
	}

public:

	enum
	{
		RemoteObjectSupportCompiledIn = UE_WITH_REMOTE_OBJECT_HANDLE
	};

	FRemoteObjectId() = default;

	static FRemoteObjectId CreateFromInt(uint64 InRawId)
	{
		FRemoteObjectId RemoteObjectId;
		RemoteObjectId.Id = InRawId;
		return RemoteObjectId;
	}

	FRemoteObjectId(FRemoteServerId InServerId, uint64 InSerialNumber)
		: Reserved(0)
		, SerialNumber(InSerialNumber)
		, ServerId(InServerId.Id)
	{
		// Make sure the arguments will fit into the different bitfields.
		ensure((uint64_t)SerialNumber <= MAX_REMOTE_OBJECT_SERIAL_NUMBER);
		ensure((uint64_t)InServerId.Id <= MAX_REMOTE_OBJECT_SERVER_ID);
	}

	explicit COREUOBJECT_API FRemoteObjectId(const UObjectBase* InObject);

	UE_FORCEINLINE_HINT uint32 GetTypeHash() const
	{
		return ::GetTypeHash(GetLocalized().Id);
	}

	UE_FORCEINLINE_HINT bool operator==(const FRemoteObjectId& Other) const
	{
		// Don't compare against the Reserved bit
		return Compare(Other) == 0;
	}
	UE_FORCEINLINE_HINT bool operator!=(const FRemoteObjectId& Other) const
	{
		// Don't compare against the Reserved bit
		return Compare(Other) != 0;
	}
	UE_FORCEINLINE_HINT bool operator<(const FRemoteObjectId& Other) const
	{
		return Compare(Other) < 0;
	}
	UE_FORCEINLINE_HINT bool operator<=(const FRemoteObjectId& Other) const
	{
		return Compare(Other) <= 0;
	}
	UE_FORCEINLINE_HINT bool operator>(const FRemoteObjectId& Other) const
	{
		return Compare(Other) > 0;
	}
	UE_FORCEINLINE_HINT bool operator>=(const FRemoteObjectId& Other) const
	{
		return Compare(Other) >= 0;
	}
	UE_FORCEINLINE_HINT bool IsValid() const
	{
		return *this != FRemoteObjectId();
	}

	UE_FORCEINLINE_HINT uint64 GetIdNumber() const
	{
		return GetGlobalized().Id;
	}
	
	UE_FORCEINLINE_HINT FRemoteServerId GetServerId() const
	{
		return FRemoteServerId(ServerId, ERemoteServerIdConstants::Invalid);
	}

	UE_FORCEINLINE_HINT bool IsAsset() const
	{
		return GetServerId().IsAsset();
	}

	UE_FORCEINLINE_HINT bool IsLocal() const
	{
		return GetServerId().IsLocal();
	}

	COREUOBJECT_API FString ToString(ERemoteIdToStringVerbosity InVerbosityOverride = ERemoteIdToStringVerbosity::Default) const;
	static COREUOBJECT_API FRemoteObjectId FromString(const FStringView& InText);
	
	UE_DEPRECATED(5.8, "Use FRemoteObjectId::Generate(InObject, InName, InOuterPathName, InInitialFlags)")
	COREUOBJECT_API static FRemoteObjectId Generate(UObjectBase* InObject, const TCHAR* InName, EInternalObjectFlags InInitialFlags = EInternalObjectFlags::None);

	/**
	* Generates remote id for the specified object (assigns the server id of the remote object id based on the fact if the object was loaded or not (Asset / Local).
	* Note that this happens BEFORE the object is added the to global GUObjectArray
	* @param InObject Object to generated the id for
	* @param InName Name of the object (because it might not have one set yet)
	* @param InOuterPathName Pathname of the Outer object (when the Outer is not set yet, can be null if the Outer is set)
	* @param InInitialFlags Initial internal flags the specified object is being constructed with
	* @return Remote id
	*/
	COREUOBJECT_API static FRemoteObjectId Generate(UObjectBase* InObject, const TCHAR* InName, const TCHAR* InOuterPathName, EInternalObjectFlags InInitialFlags = EInternalObjectFlags::None);
	COREUOBJECT_API static FRemoteObjectId Generate(ERemoteServerIdConstants ServerId);
	COREUOBJECT_API static bool IsMigratingAsset(UObjectBase* InObject);

	COREUOBJECT_API bool Serialize(FArchive& Ar);
	COREUOBJECT_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	friend COREUOBJECT_API FArchive& operator<<(FArchive& Ar, FRemoteObjectId& Id);
	friend struct UE::Net::FRemoteObjectReferenceNetSerializer;
	friend struct UE::Net::FRemoteObjectIdNetSerializer;
	friend UE::RemoteObject::Private::FRemoteIdLocalizationHelper;
	friend struct FUObjectItem;
	friend class FNetGUIDCache;
	friend class URemoteObjectNetRefHandleGenerator;
private:
	
	static bool IsAssetInternal(UObjectBase* InObject, EInternalObjectFlags InInitialFlags);
};

UE_FORCEINLINE_HINT uint32 GetTypeHash(const FRemoteObjectId& ObjectId)
{
	return ObjectId.GetTypeHash();
}

UE_FORCEINLINE_HINT uint32 GetTypeHash(const FRemoteServerId& ServerId)
{
	return ServerId.GetTypeHash();
}
