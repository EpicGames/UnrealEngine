// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Logging/LogMacros.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Optional.h"
#include "Misc/OutputDeviceError.h"
#include "ObjectMacros.h"
#include "Serialization/ArchiveCookData.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/FileRegions.h"
#include "Serialization/PackageWriter.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectSaveOverride.h"
#include "UObject/Package.h"

class ITargetPlatform;
class UObject;

#if !defined(UE_WITH_SAVEPACKAGE)
#	define UE_WITH_SAVEPACKAGE 1
#endif

class FArchive;
class FCbFieldView;
class FCbWriter;
class FIoBuffer;
class FOutputDevice;
class FPackagePath;
class FSavePackageContext;
class IPackageWriter;
namespace UE { class FLogRecord; }
struct FArchiveSavePackageData;
struct FObjectSaveContextData;

/**
 * Struct to encapsulate arguments specific to saving one package
 */
struct FPackageSaveInfo
{
	class UPackage* Package = nullptr;
	class UObject* Asset = nullptr;
	FString Filename;
};

/**
 * Struct to encapsulate UPackage::Save arguments. 
 * These arguments are shared between packages when saving multiple packages concurrently.
 */
struct FSavePackageArgs
{
	/* nullptr if not cooking, passed to the FArchive */
	FArchiveCookData* ArchiveCookData = nullptr;

	bool IsCooking() const { return ArchiveCookData != nullptr; }
	const ITargetPlatform* GetTargetPlatform() const { return ArchiveCookData ? &ArchiveCookData->TargetPlatform : nullptr; }
	
	/**
	 * For all objects which are not referenced[either directly, or indirectly] through the InAsset provided
	 * to the Save call (See UPackage::Save), only objects that contain any of these flags will be saved.
	 * If RF_NoFlags is specified, only objects which are referenced by InAsset will be saved into the package.
	 */
	EObjectFlags TopLevelFlags = RF_NoFlags;
	/* Flags to control saving, a bitwise-or'd combination of values from ESaveFlags */
	uint32 SaveFlags = SAVE_None;
	/* Whether we should forcefully byte swap before writing header and exports to disk. Passed into FLinkerSave. */
	bool bForceByteSwapping = false;
	/* If true (the default), warn when saving to a long filename. */
	bool bWarnOfLongFilename = true;
	/** If true, the Save will send progress events that are displayed in the editor. */
	bool bSlowTask = true;
	/*
	 * If not FDateTime::MinValue() (the default), the timestamp the saved file should be set to.
	 * (Intended for cooking only...)
	 */
	FDateTime FinalTimeStamp;
	/** Receives error/warning messages sent by the Save, to log and respond to their severity level. */
	FOutputDevice* Error = GError;
	/** Structure to hold longer-lifetime parameters that apply to multiple saves */
	FSavePackageContext* SavePackageContext = nullptr;

	/**
	 * In/Out list of property overrides per object to apply to during save. This list can be extended by PreSave functions
	 * during the save.
	 */
	TMap<UObject*, FObjectSaveOverride>* InOutSaveOverrides = nullptr;

	FSavePackageArgs() = default;
	FSavePackageArgs(const FSavePackageArgs&) = default;
	FSavePackageArgs(FSavePackageArgs&&) = default;
	FSavePackageArgs& operator=(const FSavePackageArgs&) = default;
	FSavePackageArgs& operator=(FSavePackageArgs&&) = default;
};

/** Param struct for external import validation functions */
struct FImportsValidationContext
{
	FImportsValidationContext(const UPackage* InPackage, const TSet<TObjectPtr<UObject>>& InImports, FOutputDevice* InOutputDevice)
		: Package(InPackage)
		, Imports(InImports)
		, OutputDevice(InOutputDevice)
	{}

	const UPackage* Package;
	const TSet<TObjectPtr<UObject>>& Imports;
	FOutputDevice* OutputDevice;
};

/** Param struct for external export validation functions */
struct FExportsValidationContext
{
	enum class EFlags
	{
		None		= 0,
		IsCooking	= 1 << 0,
	};

	FExportsValidationContext(const UPackage* InPackage, const TSet<UObject*>& InExports, const TMap<UObject*, FObjectSaveOverride>& InSaveOverrides, 
		EFlags InFlags, const ITargetPlatform* InTargetPlatform, FOutputDevice* InOutputDevice)
		: Package(InPackage)
		, Exports(InExports)
		, SaveOverrides(InSaveOverrides)
		, Flags(InFlags)
		, TargetPlatform(InTargetPlatform)
		, OutputDevice(InOutputDevice)
	{}

	const UPackage* Package;
	const TSet<UObject*>& Exports;
	const TMap<UObject*, FObjectSaveOverride>& SaveOverrides;
	const EFlags Flags;
	const ITargetPlatform* TargetPlatform; // Null if not cooking
	FOutputDevice* OutputDevice;
};
ENUM_CLASS_FLAGS(FExportsValidationContext::EFlags)

/** struct persistent settings used by all save unless overridden. @see FSavePackageContext */
class FSavePackageSettings
{
public:
	typedef ESavePackageResult ExternalImportValidationFunc(const FImportsValidationContext& InValidationContext);
	typedef ESavePackageResult ExternalExportValidationFunc(const FExportsValidationContext& InValidationContext);
	
	FSavePackageSettings() = default;

	/** Get the default settings by save when none are specified. */
	COREUOBJECT_API static FSavePackageSettings& GetDefaultSettings();

	bool IsDefault() const
	{
		return ExternalImportValidations.Num() == 0 && ExternalExportValidations.Num() == 0;
	}

	const TArray<TFunction<ExternalImportValidationFunc>>& GetExternalImportValidations() const
	{
		return ExternalImportValidations;
	}
	const TArray<TFunction<ExternalExportValidationFunc>>& GetExternalExportValidations() const
	{
		return ExternalExportValidations;
	}

	void AddExternalImportValidation(TFunction<ExternalImportValidationFunc> InValidation)
	{
		ExternalImportValidations.Add(MoveTemp(InValidation));
	}
	void AddExternalExportValidation(TFunction<ExternalExportValidationFunc> InValidation)
	{
		ExternalExportValidations.Add(MoveTemp(InValidation));
	}

private:
	TArray<TFunction<ExternalImportValidationFunc>> ExternalImportValidations;
	TArray<TFunction<ExternalExportValidationFunc>> ExternalExportValidations;
};

class FSavePackageContext
{
public:

	FSavePackageContext(const ITargetPlatform* InTargetPlatform, IPackageWriter* InPackageWriter, FSavePackageSettings InSettings = FSavePackageSettings())
	: TargetPlatform(InTargetPlatform)
	, PackageWriter(InPackageWriter)
	, SavePackageSettings(MoveTemp(InSettings))
	{
		if (PackageWriter)
		{
			PackageWriterCapabilities = PackageWriter->GetCapabilities();
		}
	}

	COREUOBJECT_API ~FSavePackageContext();

	const TArray<TFunction<FSavePackageSettings::ExternalImportValidationFunc>>& GetExternalImportValidations() const
	{
		return SavePackageSettings.GetExternalImportValidations();
	}

	const TArray<TFunction<FSavePackageSettings::ExternalExportValidationFunc>>& GetExternalExportValidations() const
	{
		return SavePackageSettings.GetExternalExportValidations();
	}

	const ITargetPlatform* const TargetPlatform;
	IPackageWriter* const PackageWriter;
	IPackageWriter::FCapabilities PackageWriterCapabilities;

private:
	FSavePackageSettings SavePackageSettings;
};

namespace UE::SavePackageUtilities
{
	/**
	 * Return whether the given save parameters indicate the LoadedPath of the package being saved should be updated.
	 * This allows us to update the in-memory package when it is saved in editor to match its new save file.
	 */
	COREUOBJECT_API bool IsUpdatingLoadedPath(bool bIsCooking, const FPackagePath& TargetPackagePath, uint32 SaveFlags);

	/**
	 * Return whether the given save parameters indicate the package is a procedural save.
	 * Any save without the the possibility of user-generated edits to the package is a procedural save (Cooking, EditorDomain).
	 * This allows us to execute transforms that only need to be executed in response to new user data.
	 */
	COREUOBJECT_API bool IsProceduralSave(bool bIsCooking, const FPackagePath& TargetPackagePath, uint32 SaveFlags);

	/** Call the PreSave function on the given object and log a warning if there is an incorrect override. */
	COREUOBJECT_API void CallPreSave(UObject* Object, FObjectSaveContextData& ObjectSaveContext);

#if WITH_EDITOR
	/** Call the CookEvent UE::Cook::ECookEvent::PlatformCookDependencies on the given object. */
	COREUOBJECT_API void CallCookEventPlatformCookDependencies(UObject* Object, FObjectSaveContextData& ObjectSaveContext);
#endif

	/** Call the PreSaveRoot function on the given object. */
	COREUOBJECT_API void CallPreSaveRoot(UObject* Object, FObjectSaveContextData& ObjectSaveContext);

	/** Call the PostSaveRoot function on the given object. */
	COREUOBJECT_API void CallPostSaveRoot(UObject* Object, FObjectSaveContextData& ObjectSaveContext, bool bCleanupRequired);

	/** Add any required TopLevelFlags based on the save parameters. */
	COREUOBJECT_API EObjectFlags NormalizeTopLevelFlags(EObjectFlags TopLevelFlags, bool bIsCooking);

	COREUOBJECT_API void IncrementOutstandingAsyncWrites();
	COREUOBJECT_API void DecrementOutstandingAsyncWrites();

	COREUOBJECT_API void ResetCookStats();
	COREUOBJECT_API int32 GetNumPackagesSaved();

#if WITH_EDITOR
	/** void FAddResaveOnDemandPackage(FName SystemName, FName PackageName); */
	DECLARE_DELEGATE_TwoParams(FAddResaveOnDemandPackage, FName, FName);
	/**
	 * Low-level systems execute this delegate during automated resave-on-demand to request that a package be resaved.
	 * Automated resave managers like the ResavePackagesCommandlet subscribe to it to know which packages to save.
	 */
	extern COREUOBJECT_API FAddResaveOnDemandPackage OnAddResaveOnDemandPackage;
#endif

}

namespace UE::SavePackageUtilities::Private
{

/**
 * Base archive class for Archives used during SavePackage to implement the flags used for SavePackage Serialization
 * used to setup all the common flags. This type is an implementation detail of SavePackage (placed in this public
 * header so it can be shared with some SavePackage implementation present in the cooker) and might change without deprecation.
 */
class FArchiveSavePackageCollector : public FArchiveUObject
{
public:
	/** The default constructor sets no flags and relies on subclass calling SetArchiveFlags later instead. */
	FArchiveSavePackageCollector() = default;
	/** Convenience constructor that calls SetArchiveFlags. */
	COREUOBJECT_API FArchiveSavePackageCollector(FArchiveSavePackageData& InSavePackageData,
		bool bFilterEditorOnly, bool bSaveUnversioned, bool bCooking);

	/** Set up the Archive with flags and properties needed to act as a collector of references during SavePackage. */
	COREUOBJECT_API void SetArchiveFlags(FArchiveSavePackageData& InSavePackageData,
		bool bFilterEditorOnly, bool bSaveUnversioned, bool bCooking);
};

/** The portflags used for SavePackage archives. */
COREUOBJECT_API uint32 GetSavePackagePortFlags();

}

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogSavePackage, Log, All);
