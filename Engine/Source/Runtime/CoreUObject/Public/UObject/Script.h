// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Script.h: Blueprint bytecode execution engine.
=============================================================================*/

#pragma once

// IWYU pragma: begin_exports

#include "Delegates/Delegate.h"
#include "HAL/ThreadSingleton.h"
#include "Stats/Stats.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/CoreMisc.h"
#include "Memory/VirtualStackAllocator.h"

struct FFrame;
struct FBlueprintExceptionInfo;
namespace verse { class task; }

// IWYU pragma: end_exports

// It's best to set only one of these, but strictly speaking you could set both.
// The results will be confusing. Native time would be included only in a coarse 
// 'native time' timer, and all overhead would be broken up per script function
#define TOTAL_OVERHEAD_SCRIPT_STATS (STATS && 0)
#define PER_FUNCTION_SCRIPT_STATS ((STATS || ENABLE_STATNAMEDEVENTS) && 1)

DECLARE_STATS_GROUP(TEXT("Scripting"), STATGROUP_Script, STATCAT_Advanced);

#if TOTAL_OVERHEAD_SCRIPT_STATS
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Blueprint - (All) VM Time (ms)"),     STAT_ScriptVmTime_Total,     STATGROUP_Script, COREUOBJECT_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Blueprint - (All) Native Time (ms)"), STAT_ScriptNativeTime_Total, STATGROUP_Script, COREUOBJECT_API);
#endif // TOTAL_OVERHEAD_SCRIPT_STATS

/*-----------------------------------------------------------------------------
	Constants & types.
-----------------------------------------------------------------------------*/

// Sizes.
enum { MAX_STRING_CONST_SIZE = 1024 }; 

/**
 * this is the size of the buffer used by the VM for unused simple (not constructed) return values.
 */
enum { MAX_SIMPLE_RETURN_VALUE_SIZE = 64 };

/**
 * a typedef for the size (in bytes) of a property; typedef'd because this value must be synchronized between the
 * blueprint compiler and the VM
 */
typedef uint16 VariableSizeType;


/**
 * a typedef for the number of bytes to skip-over when certain expressions are evaluated by the VM
 * (e.g. context expressions that resolve to NULL, etc.)
 * typedef'd because this type must be synchronized between the blueprint compiler and the VM
 */

// If you change this, make sure to bump either VER_MIN_SCRIPTVM_UE4 or VER_MIN_SCRIPTVM_LICENSEEUE4
#define SCRIPT_LIMIT_BYTECODE_TO_64KB 0

#if SCRIPT_LIMIT_BYTECODE_TO_64KB
typedef uint16 CodeSkipSizeType;
#else
typedef uint32 CodeSkipSizeType;
#endif

// Context object for data and utilities that may be needed throughout BP execution
// In the future, it would be preferable for this not to be a thread singleton but to have
// clearer initialization/termination semantics and per-thread tuning for the stack allocator
class FBlueprintContext
{
public:

	COREUOBJECT_API static FBlueprintContext* GetThreadSingleton();

	FBlueprintContext();

	FVirtualStackAllocator* GetVirtualStackAllocator() { return &VirtualStackAllocator; }

private:

	FVirtualStackAllocator VirtualStackAllocator;
};

//
// Blueprint VM intrinsic return value declaration.
//
#define RESULT_PARAM Z_Param__Result
#define RESULT_DECL void*const RESULT_PARAM

/** Space where UFunctions are asking to be called */
namespace FunctionCallspace
{
	enum Type
	{
		/** This function call should be absorbed (ie client side with no authority) */
		Absorbed = 0x0,
		/** This function call should be called remotely via its net driver */
		Remote = 0x1,
		/** This function call should be called locally */
		Local = 0x2
	};

	/** @return the stringified version of the enum passed in */
	inline const TCHAR* ToString(FunctionCallspace::Type Callspace)
	{
		switch (Callspace)
		{
		case Absorbed:
			return TEXT("Absorbed");
		case Remote:
			return TEXT("Remote");
		case Local:
			return TEXT("Local");
		}

		return TEXT("");
	}
}

//
// Function flags.
//
// Note: Please keep ParseFunctionFlags in sync when this enum is modified.
//
// This MUST be kept in sync with EEnumFlags defined in
// Engine\Source\Programs\Shared\EpicGames.Core\UnrealEngineTypes.cs
enum EFunctionFlags : uint32
{
	// Function flags.
	FUNC_None				= 0x00000000,

	FUNC_Final				= 0x00000001,	// Function is final (prebindable, non-overridable function).
	FUNC_RequiredAPI		= 0x00000002,	// Indicates this function is DLL exported/imported.
	FUNC_BlueprintAuthorityOnly= 0x00000004,   // Function will only run if the object has network authority
	FUNC_BlueprintCosmetic	= 0x00000008,   // Function is cosmetic in nature and should not be invoked on dedicated servers
	// FUNC_				= 0x00000010,   // unused.
	// FUNC_				= 0x00000020,   // unused.
	FUNC_Net				= 0x00000040,   // Function is network-replicated.
	FUNC_NetReliable		= 0x00000080,   // Function should be sent reliably on the network.
	FUNC_NetRequest			= 0x00000100,	// Function is sent to a net service
	FUNC_Exec				= 0x00000200,	// Executable from command line.
	FUNC_Native				= 0x00000400,	// Native function.
	FUNC_Event				= 0x00000800,   // Event function.
	FUNC_NetResponse		= 0x00001000,   // Function response from a net service
	FUNC_Static				= 0x00002000,   // Static function.
	FUNC_NetMulticast		= 0x00004000,	// Function is networked multicast Server -> All Clients
	FUNC_UbergraphFunction	= 0x00008000,   // Function is used as the merge 'ubergraph' for a blueprint, only assigned when using the persistent 'ubergraph' frame
	FUNC_MulticastDelegate	= 0x00010000,	// Function is a multi-cast delegate signature (also requires FUNC_Delegate to be set!)
	FUNC_Public				= 0x00020000,	// Function is accessible in all classes (if overridden, parameters must remain unchanged).
	FUNC_Private			= 0x00040000,	// Function is accessible only in the class it is defined in (cannot be overridden, but function name may be reused in subclasses.  IOW: if overridden, parameters don't need to match, and Super.Func() cannot be accessed since it's private.)
	FUNC_Protected			= 0x00080000,	// Function is accessible only in the class it is defined in and subclasses (if overridden, parameters much remain unchanged).
	FUNC_Delegate			= 0x00100000,	// Function is delegate signature (either single-cast or multi-cast, depending on whether FUNC_MulticastDelegate is set.)
	FUNC_NetServer			= 0x00200000,	// Function is executed on servers (set by replication code if passes check)
	FUNC_HasOutParms		= 0x00400000,	// function has out (pass by reference) parameters
	FUNC_HasDefaults		= 0x00800000,	// function has structs that contain defaults
	FUNC_NetClient			= 0x01000000,	// function is executed on clients
	FUNC_DLLImport			= 0x02000000,	// function is imported from a DLL
	FUNC_BlueprintCallable	= 0x04000000,	// function can be called from blueprint code
	FUNC_BlueprintEvent		= 0x08000000,	// function can be overridden/implemented from a blueprint
	FUNC_BlueprintPure		= 0x10000000,	// function can be called from blueprint code, and is also pure (produces no side effects). If you set this, you should set FUNC_BlueprintCallable as well.
	FUNC_EditorOnly			= 0x20000000,	// function can only be called from an editor scrippt.
	FUNC_Const				= 0x40000000,	// function can be called from blueprint code, and only reads state (never writes state)
	FUNC_NetValidate		= 0x80000000,	// function must supply a _Validate implementation

	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
	// DO NOT ADD composite flags to EFunctionFlags. Debugger visualizations have trouble displaying composite flag values as text.
	// !!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!!
};

UE_FORCEINLINE_HINT FArchive& operator<<(FArchive& Ar, EFunctionFlags& Flags)
{
	Ar << (uint32&)Flags;
	return Ar;
}

ENUM_CLASS_FLAGS(EFunctionFlags)

// Combinations of flags.
#define FUNC_FuncInherit       ((EFunctionFlags)(FUNC_Exec | FUNC_Event | FUNC_BlueprintCallable | FUNC_BlueprintEvent | FUNC_BlueprintAuthorityOnly | FUNC_BlueprintCosmetic | FUNC_Const))
#define FUNC_FuncOverrideMatch ((EFunctionFlags)(FUNC_Exec | FUNC_Final | FUNC_Static | FUNC_Public | FUNC_Protected | FUNC_Private))
#define FUNC_NetFuncFlags      ((EFunctionFlags)(FUNC_Net | FUNC_NetReliable | FUNC_NetServer | FUNC_NetClient | FUNC_NetMulticast))
#define FUNC_AccessSpecifiers  ((EFunctionFlags)(FUNC_Public | FUNC_Private | FUNC_Protected))
#define FUNC_AllFlags          ((EFunctionFlags)(0xFFFFFFFF))

//
// Evaluatable expression item types.
//
enum EExprToken : uint8
{
	// Variable references.
	EX_LocalVariable		= 0x00,	// A local variable.
	EX_InstanceVariable		= 0x01,	// An object variable.
	EX_DefaultVariable		= 0x02, // Default variable for a class context.
	//						= 0x03,
	EX_Return				= 0x04,	// Return from function.
	//						= 0x05,
	EX_Jump					= 0x06,	// Goto a local address in code.
	EX_JumpIfNot			= 0x07,	// Goto if not expression.
	//						= 0x08,
	EX_Assert				= 0x09,	// Assertion.
	//						= 0x0A,
	EX_Nothing				= 0x0B,	// No operation.
	EX_NothingInt32			= 0x0C, // No operation with an int32 argument (useful for debugging script disassembly)
	//						= 0x0D,
	//						= 0x0E,
	EX_Let					= 0x0F,	// Assign an arbitrary size value to a variable.
	//						= 0x10,
	EX_BitFieldConst		= 0x11, // assign to a single bit, defined by an FProperty
	EX_ClassContext			= 0x12,	// Class default object context.
	EX_MetaCast             = 0x13, // Metaclass cast.
	EX_LetBool				= 0x14, // Let boolean variable.
	EX_EndParmValue			= 0x15,	// end of default value for optional function parameter
	EX_EndFunctionParms		= 0x16,	// End of function call parameters.
	EX_Self					= 0x17,	// Self object.
	EX_Skip					= 0x18,	// Skippable expression.
	EX_Context				= 0x19,	// Call a function through an object context.
	EX_Context_FailSilent	= 0x1A, // Call a function through an object context (can fail silently if the context is NULL; only generated for functions that don't have output or return values).
	EX_VirtualFunction		= 0x1B,	// A function call with parameters.
	EX_FinalFunction		= 0x1C,	// A prebound function call with parameters.
	EX_IntConst				= 0x1D,	// Int constant.
	EX_FloatConst			= 0x1E,	// Floating point constant.
	EX_StringConst			= 0x1F,	// String constant.
	EX_ObjectConst		    = 0x20,	// An object constant.
	EX_NameConst			= 0x21,	// A name constant.
	EX_RotationConst		= 0x22,	// A rotation constant.
	EX_VectorConst			= 0x23,	// A vector constant.
	EX_ByteConst			= 0x24,	// A byte constant.
	EX_IntZero				= 0x25,	// Zero.
	EX_IntOne				= 0x26,	// One.
	EX_True					= 0x27,	// Bool True.
	EX_False				= 0x28,	// Bool False.
	EX_TextConst			= 0x29, // FText constant
	EX_NoObject				= 0x2A,	// NoObject.
	EX_TransformConst		= 0x2B, // A transform constant
	EX_IntConstByte			= 0x2C,	// Int constant that requires 1 byte.
	EX_NoInterface			= 0x2D, // A null interface (similar to EX_NoObject, but for interfaces)
	EX_DynamicCast			= 0x2E,	// Safe dynamic class casting.
	EX_StructConst			= 0x2F, // An arbitrary UStruct constant
	EX_EndStructConst		= 0x30, // End of UStruct constant
	EX_SetArray				= 0x31, // Set the value of arbitrary array
	EX_EndArray				= 0x32,
	EX_PropertyConst		= 0x33, // FProperty constant.
	EX_UnicodeStringConst   = 0x34, // Unicode string constant.
	EX_Int64Const			= 0x35,	// 64-bit integer constant.
	EX_UInt64Const			= 0x36,	// 64-bit unsigned integer constant.
	EX_DoubleConst			= 0x37, // Double constant.
	EX_Cast					= 0x38,	// A casting operator which reads the type as the subsequent byte
	EX_SetSet				= 0x39,
	EX_EndSet				= 0x3A,
	EX_SetMap				= 0x3B,
	EX_EndMap				= 0x3C,
	EX_SetConst				= 0x3D,
	EX_EndSetConst			= 0x3E,
	EX_MapConst				= 0x3F,
	EX_EndMapConst			= 0x40,
	EX_Vector3fConst		= 0x41,	// A float vector constant.
	EX_StructMemberContext	= 0x42, // Context expression to address a property within a struct
	EX_LetMulticastDelegate	= 0x43, // Assignment to a multi-cast delegate
	EX_LetDelegate			= 0x44, // Assignment to a delegate
	EX_LocalVirtualFunction	= 0x45, // Special instructions to quickly call a virtual function that we know is going to run only locally
	EX_LocalFinalFunction	= 0x46, // Special instructions to quickly call a final function that we know is going to run only locally
	//						= 0x47, // CST_ObjectToBool
	EX_LocalOutVariable		= 0x48, // local out (pass by reference) function parameter
	//						= 0x49, // CST_InterfaceToBool
	EX_DeprecatedOp4A		= 0x4A,
	EX_InstanceDelegate		= 0x4B,	// const reference to a delegate or normal function object
	EX_PushExecutionFlow	= 0x4C, // push an address on to the execution flow stack for future execution when a EX_PopExecutionFlow is executed.   Execution continues on normally and doesn't change to the pushed address.
	EX_PopExecutionFlow		= 0x4D, // continue execution at the last address previously pushed onto the execution flow stack.
	EX_ComputedJump			= 0x4E,	// Goto a local address in code, specified by an integer value.
	EX_PopExecutionFlowIfNot = 0x4F, // continue execution at the last address previously pushed onto the execution flow stack, if the condition is not true.
	EX_Breakpoint			= 0x50, // Breakpoint.  Only observed in the editor, otherwise it behaves like EX_Nothing.
	EX_InterfaceContext		= 0x51,	// Call a function through a native interface variable
	EX_ObjToInterfaceCast   = 0x52,	// Converting an object reference to native interface variable
	EX_EndOfScript			= 0x53, // Last byte in script code
	EX_CrossInterfaceCast	= 0x54, // Converting an interface variable reference to native interface variable
	EX_InterfaceToObjCast   = 0x55, // Converting an interface variable reference to an object
	//						= 0x56,
	//						= 0x57,
	//						= 0x58,
	//						= 0x59,
	EX_WireTracepoint		= 0x5A, // Trace point.  Only observed in the editor, otherwise it behaves like EX_Nothing.
	EX_SkipOffsetConst		= 0x5B, // A CodeSizeSkipOffset constant
	EX_AddMulticastDelegate = 0x5C, // Adds a delegate to a multicast delegate's targets
	EX_ClearMulticastDelegate = 0x5D, // Clears all delegates in a multicast target
	EX_Tracepoint			= 0x5E, // Trace point.  Only observed in the editor, otherwise it behaves like EX_Nothing.
	EX_LetObj				= 0x5F,	// assign to any object ref pointer
	EX_LetWeakObjPtr		= 0x60, // assign to a weak object pointer
	EX_BindDelegate			= 0x61, // bind object and name to delegate
	EX_RemoveMulticastDelegate = 0x62, // Remove a delegate from a multicast delegate's targets
	EX_CallMulticastDelegate = 0x63, // Call multicast delegate
	EX_LetValueOnPersistentFrame = 0x64,
	EX_ArrayConst			= 0x65,
	EX_EndArrayConst		= 0x66,
	EX_SoftObjectConst		= 0x67,
	EX_CallMath				= 0x68, // static pure function from on local call space
	EX_SwitchValue			= 0x69,
	EX_InstrumentationEvent	= 0x6A, // Instrumentation event
	EX_ArrayGetByRef		= 0x6B,
	EX_ClassSparseDataVariable = 0x6C, // Sparse data variable
	EX_FieldPathConst		= 0x6D,
	//						= 0x6E,
	//						= 0x6F,
	EX_AutoRtfmTransact     = 0x70, // AutoRTFM: run following code in a transaction
	EX_AutoRtfmStopTransact = 0x71, // AutoRTFM: if in a transaction, abort or break, otherwise no operation
	EX_AutoRtfmAbortIfNot   = 0x72, // AutoRTFM: evaluate bool condition, abort transaction on false
	EX_AutoRtfmAbort        = 0x73, // AutoRTFM: abort transaction
	EX_Max					= 0xFF,
};

enum EAutoRtfmStopTransactMode : uint8
{
	GracefulExit,
	AbortingExit,
	AbortingExitAndAbortParent,
};

enum ECastToken : uint8
{
	CST_ObjectToInterface		= 0x00,
	CST_ObjectToBool			= 0x01,
	CST_InterfaceToBool			= 0x02,
	CST_DoubleToFloat			= 0x03,
	CST_FloatToDouble			= 0x04,

	CST_Max						= 0xFF,
};

// Kinds of text literals
enum class EBlueprintTextLiteralType : uint8
{
	/* Text is an empty string. The bytecode contains no strings, and you should use FText::GetEmpty() to initialize the FText instance. */
	Empty,
	/** Text is localized. The bytecode will contain three strings - source, key, and namespace - and should be loaded via FInternationalization */
	LocalizedText,
	/** Text is localized. The bytecode will contain four strings - source, key, namespace, and devnotes - and should be loaded via FInternationalization */
	LocalizedTextWithNotes,
	/** Text is culture invariant. The bytecode will contain one string, and you should use FText::AsCultureInvariant to initialize the FText instance. */
	InvariantText,
	/** Text is a literal FString. The bytecode will contain one string, and you should use FText::FromString to initialize the FText instance. */
	LiteralString,
	/** Text is from a string table. The bytecode will contain an object pointer (not used) and two strings - the table ID, and key - and should be found via FText::FromStringTable */
	StringTableEntry,
};

// Script instrumentation event types
namespace EScriptInstrumentation
{
	enum Type
	{
		Class = 0,
		ClassScope,
		Instance,
		Event,
		InlineEvent,
		ResumeEvent,
		PureNodeEntry,
		NodeDebugSite,
		NodeEntry,
		NodeExit,
		PushState,
		RestoreState,
		ResetState,
		SuspendState,
		PopState,
		TunnelEndOfThread,
		Stop
	};
}

// Information about a blueprint instrumentation signal
struct FScriptInstrumentationSignal
{
public:

	COREUOBJECT_API FScriptInstrumentationSignal(EScriptInstrumentation::Type InEventType, const UObject* InContextObject, const struct FFrame& InStackFrame, const FName EventNameIn = NAME_None);

	FScriptInstrumentationSignal(EScriptInstrumentation::Type InEventType, const UObject* InContextObject, UFunction* InFunction, const int32 LinkId = INDEX_NONE)
		: EventType(InEventType)
		, ContextObject(InContextObject)
		, Function(InFunction)
		, EventName(NAME_None)
		, StackFramePtr(nullptr)
		, LatentLinkId(LinkId)
	{
	}

	/** Access to the event type */
	EScriptInstrumentation::Type GetType() const { return EventType; }

	/** Designates the event type */
	void SetType(EScriptInstrumentation::Type InType) { EventType = InType;	}

	/** Returns true if the context object is valid */
	bool IsContextObjectValid() const { return ContextObject != nullptr; }

	/** Returns the context object */
	const UObject* GetContextObject() const { return ContextObject; }

	/** Returns true if the stackframe is valid */
	bool IsStackFrameValid() const { return StackFramePtr != nullptr; }

	/** Returns the stackframe */
	const FFrame& GetStackFrame() const { return *StackFramePtr; }

	/** Returns the owner class name of the active instance */
	COREUOBJECT_API const UClass* GetClass() const;

	/** Returns the function scope class */
	COREUOBJECT_API const UClass* GetFunctionClassScope() const;

	/** Returns the name of the active function */
	COREUOBJECT_API FName GetFunctionName() const;

	/** Returns the script code offset */
	COREUOBJECT_API int32 GetScriptCodeOffset() const;

	/** Returns the latent link id for latent events */
	int32 GetLatentLinkId() const { return LatentLinkId; }

protected:

	/** The event signal type */
	EScriptInstrumentation::Type EventType;
	/** The context object the event is from */
	const UObject* ContextObject;
	/** The function that emitted this event */
	const UFunction* Function;
	/** The event override name */
	const FName EventName;
	/** The stack frame for the  */
	const struct FFrame* StackFramePtr;
	const int32 LatentLinkId;
};

// Blueprint core runtime delegates
class FBlueprintCoreDelegates
{
public:
	// Callback for debugging events such as a breakpoint (Object that triggered event, active stack frame, Info)
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnScriptDebuggingEvent, const UObject*, const struct FFrame&, const FBlueprintExceptionInfo&);
	// Callback for blueprint profiling signals
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnScriptInstrumentEvent, const FScriptInstrumentationSignal& );
	// Callback for blueprint instrumentation enable/disable events
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnToggleScriptProfiler, bool );

public:
	// Called when a script exception occurs
	static COREUOBJECT_API FOnScriptDebuggingEvent OnScriptException;
	// Called when a script profiling event is fired
	static COREUOBJECT_API FOnScriptInstrumentEvent OnScriptProfilingEvent;
	// Called when a script profiler is enabled/disabled
	static COREUOBJECT_API FOnToggleScriptProfiler OnToggleScriptProfiler;

public:
	static COREUOBJECT_API void ThrowScriptException(const UObject* ActiveObject, struct FFrame& StackFrame, const FBlueprintExceptionInfo& Info);
	static COREUOBJECT_API void InstrumentScriptEvent(const FScriptInstrumentationSignal& Info);
	static COREUOBJECT_API void SetScriptMaximumLoopIterations( const int32 MaximumLoopIterations );
	static COREUOBJECT_API bool IsDebuggingEnabled();
};

#if DO_BLUEPRINT_GUARD

/**
 * Helper struct for dealing with tracking blueprint context and exceptions
 */
struct COREUOBJECT_API FBlueprintContextTracker : TThreadSingleton<FBlueprintContextTracker>
{
	FBlueprintContextTracker() = default;

	/** @return Reference to the FBlueprintContextTracker for the current thread, creating the FBlueprintContextTracker if none exists */
	static FBlueprintContextTracker& Get();

	/** @return Pointer to the FBlueprintContextTracker for the current thread, if any */
	static const FBlueprintContextTracker* TryGet();

	/** Resets runaway tracking, will unset flag */
	void ResetRunaway();

	/** Increments and returns the Runaway counter */
	UE_FORCEINLINE_HINT int32 AddRunaway()
	{
		return ++Runaway;
	}

	/** Called at the start of a script function execution */
	void EnterScriptContext(const class UObject* ContextObject, const class UFunction* ContextFunction);

	/** Called at the end of a script function execution */
	void ExitScriptContext();

	/** Called periodically when branching occurs. Sets bScriptTimedOut and maxes out Runaway if the script time limit is exceeded. */
	void EnforceScriptTimeLimit();

	/** Record an access violation warning for a specific object, returns true if warning should be logged */
	bool RecordAccessViolation(const UObject* Object);

	/** Returns how many function executions deep we are, may be higher than ScriptStack size */
	UE_FORCEINLINE_HINT int32 GetScriptEntryTag() const
	{
		return ScriptEntryTag;
	}

	/** Returns current script stack frame */
	UE_FORCEINLINE_HINT TArrayView<const FFrame* const> GetCurrentScriptStack() const
	{
		return MakeArrayView<const FFrame* const>(ScriptStack.GetData(), ScriptStack.Num());
	}

	UE_FORCEINLINE_HINT TArrayView<FFrame* const> GetCurrentScriptStackWritable() const
	{
		return MakeArrayView<FFrame* const>(ScriptStack.GetData(), ScriptStack.Num());
	}

	/** Delegate called from EnterScriptContext, could be called on any thread! This can be used to detect entries into script from native code */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnEnterScriptContext, const struct FBlueprintContextTracker&, const UObject*, const UFunction*);
	static FOnEnterScriptContext OnEnterScriptContext;

	/** Delegate called from ExitScriptContext, could be called on any thread! This can be used to clean up debugging context */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnExitScriptContext, const struct FBlueprintContextTracker&);
	static FOnExitScriptContext OnExitScriptContext;

private:

	// Runaway tracking
	int32 Runaway = 0;
	int32 Recurse = 0;
	bool bRanaway = false;
	bool bScriptTimedOut = false;

	// Script entry point tracking, enter/exit context
	int32 ScriptEntryTag = 0;

	// Stack pointers from the VM to be unrolled when we assert
	TArray<FFrame*> ScriptStack;

	// Map of reported access warnings in exception handler
	TMap<FName, int32> DisplayedWarningsMap;

	// Only FFrame can modify the stack
	friend FFrame;
	friend void ProcessLocalScriptFunction(UObject* Context, FFrame& Stack, RESULT_DECL);
	friend verse::task;
};

#endif // DO_BLUEPRINT_GUARD


// Scoped struct to allow execution of script in editor, while resetting the runaway loop counts
struct FEditorScriptExecutionGuard
{
public:
	COREUOBJECT_API FEditorScriptExecutionGuard();
	COREUOBJECT_API ~FEditorScriptExecutionGuard();

private:
	bool bOldGAllowScriptExecutionInEditor;
};

#if TOTAL_OVERHEAD_SCRIPT_STATS
	// Low overhead timer used to instrument the VM (ProcessEvent and ProcessInternal):
	struct FBlueprintEventTimer
	{
		struct FPausableScopeTimer;
		struct FScopedVMTimer;

		class FThreadedTimerManager : public TThreadSingleton<FThreadedTimerManager>
		{
		public:
			FThreadedTimerManager()
				: ActiveTimer(nullptr)
				, ActiveVMScope(nullptr)
			{}

			FPausableScopeTimer* ActiveTimer;

			// We need to keep track of the current VM timer because we only want to
			// track time while 'in' the VM. We use this to detect whether we're running
			// script or just doing RPC:
			FScopedVMTimer* ActiveVMScope;
		};

		struct FPausableScopeTimer
		{
			FPausableScopeTimer()
				: PreviouslyActiveTimer(nullptr)
				, TotalTime(0.0)
				, StartTime(0.0)
			{
			}

			void Start();
			void Pause(double CurrentTime) { TotalTime += CurrentTime - StartTime; }
			void Resume() { StartTime = FPlatformTime::Seconds();  }
			double Stop();

		private:
			FPausableScopeTimer* PreviouslyActiveTimer;
			double TotalTime;
			double StartTime;
		};

		struct FScopedVMTimer
		{
			COREUOBJECT_API FScopedVMTimer();
			COREUOBJECT_API ~FScopedVMTimer();

			FPausableScopeTimer Timer;
			FScopedVMTimer* VMParent;
		};

		struct FScopedNativeTimer
		{
			COREUOBJECT_API FScopedNativeTimer();
			COREUOBJECT_API ~FScopedNativeTimer();

			FPausableScopeTimer Timer;
		};
	};

#define SCOPED_SCRIPT_NATIVE_TIMER(VarName) \
	FBlueprintEventTimer::FScopedNativeTimer VarName

#else  // TOTAL_OVERHEAD_SCRIPT_STATS
	#define SCOPED_SCRIPT_NATIVE_TIMER(VarName) 
#endif // TOTAL_OVERHEAD_SCRIPT_STATS
/** @return True if the char can be used in an identifier in c++ */
COREUOBJECT_API bool IsValidCPPIdentifierChar(TCHAR Char);

/** @return A string that contains only Char if Char IsValidCPPIdentifierChar, otherwise returns a corresponding sequence of valid c++ chars */
COREUOBJECT_API FString ToValidCPPIdentifierChars(TCHAR Char);

/** 
	@param InName The string to transform
	@param bDeprecated whether the name has been deprecated
	@Param Prefix The prefix to be prepended to the return value, accepts nullptr or empty string
	@return A corresponding string that contains only valid c++ characters and is prefixed with Prefix
*/
COREUOBJECT_API FString UnicodeToCPPIdentifier(const FString& InName, bool bDeprecated, const TCHAR* Prefix);
