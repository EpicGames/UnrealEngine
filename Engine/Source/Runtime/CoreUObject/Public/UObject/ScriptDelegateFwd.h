// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScriptDelegateFwd.h: Delegate forward declarations
=============================================================================*/

#pragma once

// IWYU pragma: begin_exports

#include "UObject/WeakObjectPtrFwd.h"

struct FNotThreadSafeDelegateMode;
struct FThreadSafeDelegateMode;

template <typename ThreadSafetyMode = FNotThreadSafeDelegateMode> class TScriptDelegate;
template <typename ThreadSafetyMode = FNotThreadSafeDelegateMode> class TMulticastScriptDelegate;

// Typedef script delegates for convenience.
typedef TScriptDelegate<> FScriptDelegate;
typedef TMulticastScriptDelegate<> FMulticastScriptDelegate;

typedef TScriptDelegate<FThreadSafeDelegateMode> FTSScriptDelegate;
typedef TMulticastScriptDelegate<FThreadSafeDelegateMode> FTSMulticastScriptDelegate;

// IWYU pragma: end_exports
