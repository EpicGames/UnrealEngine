// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScriptInterface.h: Script interface definitions.
=============================================================================*/

#pragma once

#include "UObject/UObjectGlobals.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"

#include <type_traits>

/**
 * This utility class stores the FProperty data for an interface property used in both blueprint and native code.
 * For objects natively implementing an interface, ObjectPointer and InterfacePointer point to different locations in the same UObject.
 * For objects that only implement an interface in blueprint, only ObjectPointer will be set because there is no native representation.
 * UClass::ImplementsInterface can be used along with Execute_ event wrappers to properly handle BP-implemented interfaces.
 */
class FScriptInterface
{
private:
	/**
	 * A pointer to a UObject that implements an interface.
	 */
	TObjectPtr<UObject>	ObjectPointer = nullptr;

	/**
	 * For native interfaces, pointer to the location of the interface object within the UObject referenced by ObjectPointer.
	 */
	void*		InterfacePointer = nullptr;

protected:
	/**
	 * Serialize ScriptInterface
	 */
	COREUOBJECT_API FArchive& Serialize(FArchive& Ar, class UClass* InterfaceType);

public:
	/**
	 * Default constructor
	 */
	FScriptInterface() = default;

	/**
	 * Construction from object and interface
	 */
	FScriptInterface( UObject* InObjectPointer, void* InInterfacePointer )
	: ObjectPointer(InObjectPointer), InterfacePointer(InInterfacePointer)
	{}

	/**
	 * Copyable
	 */
	FScriptInterface(const FScriptInterface&) = default;
	FScriptInterface& operator=(const FScriptInterface&) = default;

	/**
	 * Returns the ObjectPointer contained by this FScriptInterface
	 */
	UE_FORCEINLINE_HINT UObject* GetObject() const
	{
		return ObjectPointer;
	}

	/**
	 * Returns the ObjectPointer contained by this FScriptInterface
	 */
	UE_FORCEINLINE_HINT TObjectPtr<UObject>& GetObjectRef()
	{
		return ObjectPointer;
	}

	/**
	 * Returns the pointer to the native interface if it is valid
	 */
	UE_FORCEINLINE_HINT void* GetInterface() const
	{
		// Only access the InterfacePointer if we have a valid ObjectPointer. This is necessary because garbage collection may only clear the ObjectPointer.
		// This will also return null for objects that only implement the interface in a blueprint class because there is no native representation.
		return ObjectPointer ? InterfacePointer : nullptr;
	}

	/**
	 * Sets the value of the ObjectPointer for this FScriptInterface
	 */
	inline void SetObject( UObject* InObjectPointer )
	{
		ObjectPointer = InObjectPointer;
		if ( ObjectPointer == nullptr )
		{
			SetInterface(nullptr);
		}
	}

	/**
	 * Sets the value of the InterfacePointer for this FScriptInterface
	 */
	UE_FORCEINLINE_HINT void SetInterface( void* InInterfacePointer )
	{
		InterfacePointer = InInterfacePointer;
	}

	/**
	 * Comparison operator, taking a reference to another FScriptInterface
	 */
	UE_FORCEINLINE_HINT bool operator==( const FScriptInterface& Other ) const
	{
		return GetInterface() == Other.GetInterface() && ObjectPointer == Other.GetObject();
	}
	UE_FORCEINLINE_HINT bool operator!=( const FScriptInterface& Other ) const
	{
		return GetInterface() != Other.GetInterface() || ObjectPointer != Other.GetObject();
	}

	void AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(ObjectPointer);
	}

	friend inline uint32 GetTypeHash(const FScriptInterface& Instance)
	{
		return GetTypeHash(Instance.InterfacePointer);
	}
};



template<> struct TIsPODType<class FScriptInterface> { enum { Value = true }; };
template<> struct TIsZeroConstructType<class FScriptInterface> { enum { Value = true }; };

/**
 * Templated version of FScriptInterface, which provides accessors and operators for referencing the interface portion of an object implementing an interface.
 * This type is only useful with native interfaces, UClass::ImplementsInterface should be used to check for blueprint interfaces.
 */
template <typename InInterfaceType>
class TScriptInterface : public FScriptInterface
{
public:
	using InterfaceType = InInterfaceType;
	using UObjectType = TCopyQualifiersFromTo_T<InterfaceType, UObject>;

	/**
	 * Default constructor
	 */
	TScriptInterface() = default;

	/**
	 * Construction from nullptr
	 */
	TScriptInterface(TYPE_OF_NULLPTR) {}

	/**
	 * Construction from an object type that may natively implement InterfaceType
	 */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U, UObjectType*>)
	>
	inline TScriptInterface(U&& Source)
	{
		// Always set the object
		UObjectType* SourceObject = ImplicitConv<UObjectType*>(Source);
		SetObject(SourceObject);

		if constexpr (std::is_base_of<InInterfaceType, std::remove_pointer_t<std::remove_reference_t<U>>>::value)
		{
			// If we know at compile time that we got passed some subclass of InInterfaceType, set it
			// without a cast (avoiding the cast also allows us to not require linking to its module)
			SetInterface(Source);
		}
		else
		{
			// Tries to set the native interface instance, this will set it to null for BP-implemented interfaces
			InInterfaceType* SourceInterface = Cast<InInterfaceType>(SourceObject);
			SetInterface(SourceInterface);
			if( SourceObject && 
				!SourceInterface && 
				!SourceObject->GetClass()->ImplementsInterface(InterfaceType::UClassType::StaticClass()))
			{
				SetObject(nullptr);
			}
		}
	}

	/**
	 * Construction from another script interface of a compatible interface type
	 */
	template <
		typename OtherInterfaceType
		UE_REQUIRES(std::is_convertible_v<OtherInterfaceType*, InInterfaceType*>)
	>
	inline TScriptInterface(const TScriptInterface<OtherInterfaceType>& Other)
	{
		SetObject(Other.GetObject());

		InInterfaceType* SourceInterface = Other.GetInterface();
		SetInterface(SourceInterface);
	}

	/**
	 * Assignment from an object type that may natively implement InterfaceType
	 */
	template <typename ObjectType>
	TScriptInterface(TObjectPtr<ObjectType> SourceObject)
		: TScriptInterface(SourceObject.Get())
	{
	}

	/**
	 * Copyable
	 */
	TScriptInterface(const TScriptInterface&) = default;
	TScriptInterface& operator=(const TScriptInterface&) = default;

	/**
	 * Assignment from nullptr
	 */
	TScriptInterface& operator=(TYPE_OF_NULLPTR)
	{
		*this = TScriptInterface();
		return *this;
	}

	/**
	 * Assignment from an object type that may natively implement InterfaceType
	 */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U, UObjectType*>)
	>
	TScriptInterface& operator=(U&& Source)
	{
		*this = TScriptInterface(Source);
		return *this;
	}

	/**
	 * Assignment from another script interface of a compatible interface type
	 */
	template <
		typename OtherInterfaceType
		UE_REQUIRES(std::is_convertible_v<OtherInterfaceType*, InInterfaceType*>)
	>
	TScriptInterface& operator=(const TScriptInterface<OtherInterfaceType>& Other)
	{
		*this = TScriptInterface(Other);
		return *this;
	}

	/**
	 * Assignment from an object type that may natively implement InterfaceType
	 */
	template <typename ObjectType>
	TScriptInterface& operator=(TObjectPtr<ObjectType> SourceObject)
	{
		*this = TScriptInterface(SourceObject);
		return *this;
	}

	/**
	 * Comparison operator, taking a pointer to InterfaceType
	 */
	template <
		typename OtherInterface
		UE_REQUIRES(std::is_convertible_v<OtherInterface*, InInterfaceType*>)
	>
	UE_FORCEINLINE_HINT bool operator==( const OtherInterface* Other ) const
	{
		return GetInterface() == Other;
	}
	template <
		typename OtherInterface
		UE_REQUIRES(std::is_convertible_v<OtherInterface*, InInterfaceType*>)
	>
	UE_FORCEINLINE_HINT bool operator!=( const OtherInterface* Other ) const
	{
		return GetInterface() != Other;
	}

	/**
	 * Comparison operator, taking a reference to another TScriptInterface
	 */
	UE_FORCEINLINE_HINT bool operator==( const TScriptInterface& Other ) const
	{
		return GetInterface() == Other.GetInterface() && GetObject() == Other.GetObject();
	}
	UE_FORCEINLINE_HINT bool operator!=( const TScriptInterface& Other ) const
	{
		return GetInterface() != Other.GetInterface() || GetObject() != Other.GetObject();
	}

	/**
	 * Comparison operator, taking a nullptr
	 */
	UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR) const
	{
		return GetInterface() == nullptr;
	}
	UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR) const
	{
		return GetInterface() != nullptr;
	}

	/**
	 * Member access operator.  Provides transparent access to the native interface pointer contained by this TScriptInterface
	 */
	UE_FORCEINLINE_HINT InInterfaceType* operator->() const
	{
		return GetInterface();
	}

	/**
	 * Dereference operator.  Provides transparent access to the native interface pointer contained by this TScriptInterface
	 *
	 * @return	a reference (of type InterfaceType) to the object pointed to by InterfacePointer
	 */
	UE_FORCEINLINE_HINT InInterfaceType& operator*() const
	{
		return *GetInterface();
	}

	/**
	 * Returns the pointer to the interface
	 */
	UE_FORCEINLINE_HINT InInterfaceType* GetInterface() const
	{
		return (InInterfaceType*)FScriptInterface::GetInterface();
	}

	/**
	 * Sets the value of the InterfacePointer for this TScriptInterface
	 */
	UE_FORCEINLINE_HINT void SetInterface(InInterfaceType* InInterfacePointer)
	{
		FScriptInterface::SetInterface((void*)InInterfacePointer);
	}

	/**
	 * Returns the ObjectPointer contained by this TScriptInterface
	 */
	UE_FORCEINLINE_HINT UObjectType* GetObject() const
	{
		return FScriptInterface::GetObject();
	}

	/**
	 * Returns the ObjectPointer contained by this TScriptInterface
	 */
	UE_FORCEINLINE_HINT TObjectPtr<UObjectType>& GetObjectRef()
	{
		return *(TObjectPtr<UObjectType>*)&FScriptInterface::GetObjectRef();
	}

	/**
	 * Sets the value of the ObjectPointer for this TScriptInterface
	 */
	UE_FORCEINLINE_HINT void SetObject( UObjectType* InObjectPointer )
	{
		FScriptInterface::SetObject(const_cast<UObject*>(InObjectPointer));
	}

	/**
	 * Boolean operator, returns true if this object natively implements InterfaceType.
	 * This will return false for objects that only implement the interface in blueprint classes.
	 */
	UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return GetInterface() != nullptr;
	}

	friend FArchive& operator<<( FArchive& Ar, TScriptInterface& Interface )
	{
		return Interface.Serialize(Ar, InInterfaceType::UClassType::StaticClass());
	}
};

template <typename InterfaceType>
struct TCallTraits<TScriptInterface<InterfaceType>> : public TCallTraitsBase<TScriptInterface<InterfaceType>>
{
	using ConstPointerType = TScriptInterface<const InterfaceType>;
};
