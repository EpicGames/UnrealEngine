// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ScriptMacros.h: Kismet VM execution engine.
=============================================================================*/

#pragma once

// IWYU pragma: begin_keep
#include "UObject/Script.h"
#include "UObject/ScriptInterface.h"
#include "UObject/StrProperty.h"
#include "UObject/UnrealType.h"
#include "UObject/Stack.h"
#include "UObject/FieldPathProperty.h"
#include "UObject/PropertyOptional.h"
// IWYU pragma: end_keep

/*-----------------------------------------------------------------------------
	Macros.
-----------------------------------------------------------------------------*/

/**
 * This is the largest possible size that a single variable can be; a variables size is determined by multiplying the
 * size of the type by the variables ArrayDim (always 1 unless it's a static array).
 */
enum {MAX_VARIABLE_SIZE = 0x0FFF };

#define ZERO_INIT(Type,ParamName) FMemory::Memzero(&ParamName,sizeof(Type));

#define PARAM_PASSED_BY_VAL(ParamName, PropertyType, ParamType)																							\
	UE_REMOVE_OPTIONAL_PARENS(ParamType) ParamName;																										\
	Stack.StepCompiledIn<PropertyType>(&ParamName);

#define PARAM_PASSED_BY_VAL_ZEROED(ParamName, PropertyType, ParamType)																					\
	UE_REMOVE_OPTIONAL_PARENS(ParamType) ParamName = (UE_REMOVE_OPTIONAL_PARENS(ParamType))0;															\
	Stack.StepCompiledIn<PropertyType>(&ParamName);

#define PARAM_PASSED_BY_VAL_INITED(ParamName, PropertyType, ParamType, ...)																				\
	UE_REMOVE_OPTIONAL_PARENS(ParamType) ParamName{__VA_ARGS__};																						\
	Stack.StepCompiledIn<PropertyType>(&ParamName);

#define PARAM_PASSED_BY_REF_PASSTHRU(ParamName, PropertyType, ParamType, PassThruType)																	\
	UE_REMOVE_OPTIONAL_PARENS(PassThruType) ParamName##Temp;																							\
	UE_REMOVE_OPTIONAL_PARENS(ParamType)& ParamName = Stack.StepCompiledInRef<PropertyType, UE_REMOVE_OPTIONAL_PARENS(ParamType)>(&ParamName##Temp);

#define PARAM_PASSED_BY_REF(ParamName, PropertyType, ParamType)																							\
	UE_REMOVE_OPTIONAL_PARENS(ParamType) ParamName##Temp;																								\
	UE_REMOVE_OPTIONAL_PARENS(ParamType)& ParamName = Stack.StepCompiledInRef<PropertyType, UE_REMOVE_OPTIONAL_PARENS(ParamType)>(&ParamName##Temp);

#define PARAM_PASSED_BY_REF_ZEROED(ParamName, PropertyType, ParamType)																					\
	UE_REMOVE_OPTIONAL_PARENS(ParamType) ParamName##Temp = (UE_REMOVE_OPTIONAL_PARENS(ParamType))0;														\
	UE_REMOVE_OPTIONAL_PARENS(ParamType)& ParamName = Stack.StepCompiledInRef<PropertyType, UE_REMOVE_OPTIONAL_PARENS(ParamType)>(&ParamName##Temp);

#define P_GET_PROPERTY(PropertyType, ParamName)																											\
	PropertyType::TCppType ParamName = PropertyType::GetDefaultPropertyValue();																			\
	Stack.StepCompiledIn<PropertyType>(&ParamName);

#define P_GET_PROPERTY_REF(PropertyType, ParamName)																										\
	PropertyType::TCppType ParamName##Temp = PropertyType::GetDefaultPropertyValue();																	\
	PropertyType::TCppType& ParamName = Stack.StepCompiledInRef<PropertyType, PropertyType::TCppType>(&ParamName##Temp);



#define P_GET_UBOOL(ParamName)						uint32 ParamName##32 = 0; bool ParamName=false;	Stack.StepCompiledIn<FBoolProperty>(&ParamName##32); ParamName = !!ParamName##32; // translate the bitfield into a bool type for non-intel platforms
#define P_GET_UBOOL8(ParamName)						uint32 ParamName##32 = 0; uint8 ParamName=0;    Stack.StepCompiledIn<FBoolProperty>(&ParamName##32); ParamName = ParamName##32 ? 1 : 0; // translate the bitfield into a bool type for non-intel platforms
#define P_GET_UBOOL16(ParamName)					uint32 ParamName##32 = 0; uint16 ParamName=0;   Stack.StepCompiledIn<FBoolProperty>(&ParamName##32); ParamName = ParamName##32 ? 1 : 0; // translate the bitfield into a bool type for non-intel platforms
#define P_GET_UBOOL32(ParamName)					uint32 ParamName=0;                             Stack.StepCompiledIn<FBoolProperty>(&ParamName); ParamName = ParamName ? 1 : 0; // translate the bitfield into a bool type for non-intel platforms
#define P_GET_UBOOL64(ParamName)					uint64 ParamName=0;                             Stack.StepCompiledIn<FBoolProperty>(&ParamName); ParamName = ParamName ? 1 : 0; // translate the bitfield into a bool type for non-intel platforms
#define P_GET_UBOOL_REF(ParamName)					PARAM_PASSED_BY_REF_ZEROED(ParamName, FBoolProperty, bool)

#define P_GET_STRUCT(StructType,ParamName)			PARAM_PASSED_BY_VAL(ParamName, FStructProperty, StructType)
#define P_GET_STRUCT_REF(StructType,ParamName)		PARAM_PASSED_BY_REF(ParamName, FStructProperty, StructType)

#define P_GET_OBJECT(ObjectType,ParamName)			PARAM_PASSED_BY_VAL_ZEROED(ParamName, FObjectPropertyBase, ObjectType*)
#define P_GET_OBJECT_REF(ObjectType,ParamName)		PARAM_PASSED_BY_REF_ZEROED(ParamName, FObjectPropertyBase, ObjectType*)

#define P_GET_OBJECT_NO_PTR(ObjectType,ParamName)			PARAM_PASSED_BY_VAL_ZEROED(ParamName, FObjectPropertyBase, ObjectType)
#define P_GET_OBJECT_REF_NO_PTR(ObjectType,ParamName)		PARAM_PASSED_BY_REF_ZEROED(ParamName, FObjectPropertyBase, ObjectType)

#define P_GET_TARRAY(ElementType,ParamName)			PARAM_PASSED_BY_VAL(ParamName, FArrayProperty, TArray<ElementType>)
#define P_GET_TARRAY_REF(ElementType,ParamName)		PARAM_PASSED_BY_REF(ParamName, FArrayProperty, TArray<ElementType>)

#define P_GET_TMAP(KeyType,ValueType,ParamName)			PARAM_PASSED_BY_VAL(ParamName, FMapProperty, (TMap<KeyType, ValueType>))
#define P_GET_TMAP_REF(KeyType,ValueType,ParamName)		PARAM_PASSED_BY_REF(ParamName, FMapProperty, (TMap<KeyType, ValueType>))

#define P_GET_TSET(ElementType,ParamName)			PARAM_PASSED_BY_VAL(ParamName, FSetProperty, TSet<ElementType>)
#define P_GET_TSET_REF(ElementType,ParamName)		PARAM_PASSED_BY_REF(ParamName, FSetProperty, TSet<ElementType>)

#define P_GET_TINTERFACE(ObjectType,ParamName)		PARAM_PASSED_BY_VAL(ParamName, FInterfaceProperty, TScriptInterface<ObjectType>)
#define P_GET_TINTERFACE_REF(ObjectType,ParamName)	PARAM_PASSED_BY_REF(ParamName, FInterfaceProperty, TScriptInterface<ObjectType>)

#define P_GET_WEAKOBJECT(ObjectType,ParamName)		PARAM_PASSED_BY_VAL(ParamName, FWeakObjectProperty, typename UE_REMOVE_OPTIONAL_PARENS(ObjectType)::ElementType*)
#define P_GET_WEAKOBJECT_REF(ObjectType,ParamName)	PARAM_PASSED_BY_REF_PASSTHRU(ParamName, FWeakObjectProperty, ObjectType, typename UE_REMOVE_OPTIONAL_PARENS(ObjectType)::ElementType*)

#define P_GET_WEAKOBJECT_NO_PTR(ObjectType,ParamName)		PARAM_PASSED_BY_VAL(ParamName, FWeakObjectProperty, typename UE_REMOVE_OPTIONAL_PARENS(ObjectType)::ElementType*)
#define P_GET_WEAKOBJECT_REF_NO_PTR(ObjectType,ParamName)	PARAM_PASSED_BY_REF_PASSTHRU(ParamName, FWeakObjectProperty, ObjectType, typename UE_REMOVE_OPTIONAL_PARENS(ObjectType)::ElementType*)

#define P_GET_SOFTOBJECT(ObjectType,ParamName)		PARAM_PASSED_BY_VAL(ParamName, FSoftObjectProperty, ObjectType)
#define P_GET_SOFTOBJECT_REF(ObjectType,ParamName)	PARAM_PASSED_BY_REF(ParamName, FSoftObjectProperty, ObjectType)

#define P_GET_SOFTCLASS(ObjectType,ParamName)		PARAM_PASSED_BY_VAL(ParamName, FSoftClassProperty, ObjectType)
#define P_GET_SOFTCLASS_REF(ObjectType,ParamName)	PARAM_PASSED_BY_REF(ParamName, FSoftClassProperty, ObjectType)

#define P_GET_TFIELDPATH(FieldType,ParamName)		PARAM_PASSED_BY_VAL(ParamName, FFieldPathProperty, FieldType)
#define P_GET_TFIELDPATH_REF(FieldType,ParamName)	PARAM_PASSED_BY_REF(ParamName, FFieldPathProperty, FieldType)

#define P_GET_ARRAY(ElementType,ParamName)			ElementType ParamName[(MAX_VARIABLE_SIZE/sizeof(ElementType))+1];		Stack.StepCompiledIn<FProperty>(ParamName);
#define P_GET_ARRAY_REF(ElementType,ParamName)		ElementType ParamName##Temp[(MAX_VARIABLE_SIZE/sizeof(ElementType))+1]; ElementType* ParamName = Stack.StepCompiledInRef<FProperty, ElementType*>(ParamName##Temp);

#define P_GET_ENUM(EnumType,ParamName)				EnumType ParamName = (EnumType)0; Stack.StepCompiledIn<FEnumProperty>(&ParamName);
#define P_GET_ENUM_REF(EnumType,ParamName)			PARAM_PASSED_BY_REF_ZEROED(ParamName, FEnumProperty, EnumType)

// The following macros are not supported by blueprints but exist for Verse generated functions

#define P_GET_UTF8CHAR(PropertyType, ParamName)		UTF8CHAR ParamName = (UTF8CHAR)0; Stack.StepCompiledIn<FByteProperty>(&ParamName);
#define P_GET_UTF8CHAR_REF(PropertyType, ParamName)	UTF8CHAR ParamName##Temp = (UTF8CHAR)0; UTF8CHAR& ParamName = Stack.StepCompiledInRef<FByteProperty, UTF8CHAR>(&ParamName##Temp);

#define P_GET_OBJECTPTR(ElementType,ParamName)		PARAM_PASSED_BY_VAL(ParamName, FObjectPropertyBase, TObjectPtr<ElementType>)
#define P_GET_OBJECTPTR_REF(ElementType,ParamName)	PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, TObjectPtr<ElementType>)

#define P_GET_VERSETYPE(ParamName)					PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, verse::type, Verse::EDefaultConstructNativeType::UnsafeDoNotUse)
#define P_GET_VERSETYPE_REF(ParamName)				PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, verse::type)

#define P_GET_VERSESUBTYPE(ElementType, ParamName)		PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, verse::subtype<ElementType>, Verse::EDefaultConstructNativeType::UnsafeDoNotUse)
#define P_GET_VERSESUBTYPE_REF(ElementType, ParamName)	PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, verse::subtype<ElementType>)

#define P_GET_VERSECASTABLETYPE(ParamName)			PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, verse::castable_type, Verse::EDefaultConstructNativeType::UnsafeDoNotUse)
#define P_GET_VERSECASTABLETYPE_REF(ParamName)		PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, verse::castable_type)

#define P_GET_VERSECASTABLESUBTYPE(ElementType, ParamName)		PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, verse::castable_subtype<ElementType>, Verse::EDefaultConstructNativeType::UnsafeDoNotUse)
#define P_GET_VERSECASTABLESUBTYPE_REF(ElementType, ParamName)	PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, verse::castable_subtype<ElementType>)

#define P_GET_VERSECONCRETETYPE(ParamName)			PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, verse::concrete_type, Verse::EDefaultConstructNativeType::UnsafeDoNotUse)
#define P_GET_VERSECONCRETETYPE_REF(ParamName)		PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, verse::concrete_type)

#define P_GET_VERSECONCRETESUBTYPE(ElementType, ParamName)		PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, verse::concrete_subtype<ElementType>, Verse::EDefaultConstructNativeType::UnsafeDoNotUse)
#define P_GET_VERSECONCRETESUBTYPE_REF(ElementType, ParamName)	PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, verse::concrete_subtype<ElementType>)

#define P_GET_VERSECASTABLECONCRETETYPE(ParamName)			PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, verse::castable_concrete_type, Verse::EDefaultConstructNativeType::UnsafeDoNotUse)
#define P_GET_VERSECASTABLECONCRETETYPE_REF(ParamName)		PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, verse::castable_concrete_type)

#define P_GET_VERSECASTABLECONCRETESUBTYPE(ElementType, ParamName)		PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, verse::castable_concrete_subtype<ElementType>, Verse::EDefaultConstructNativeType::UnsafeDoNotUse)
#define P_GET_VERSECASTABLECONCRETESUBTYPE_REF(ElementType, ParamName)	PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, verse::castable_concrete_subtype<ElementType>)


#define P_GET_INTERFACEINSTANCE(ElementType, ParamName)		PARAM_PASSED_BY_VAL_INITED(ParamName, FObjectPropertyBase, TInterfaceInstance<ElementType>, EDefaultConstructNonNullPtr::Uninitialized)
#define P_GET_INTERFACEINSTANCE_REF(ElementType, ParamName)	PARAM_PASSED_BY_REF(ParamName, FObjectPropertyBase, TInterfaceInstance<ElementType>)

#define P_GET_TOPTIONAL(ElementType,ParamName)		PARAM_PASSED_BY_VAL(ParamName, FOptionalProperty, TOptional<ElementType>)
#define P_GET_TOPTIONAL_REF(ElementType,ParamName)	PARAM_PASSED_BY_REF(ParamName, FOptionalProperty, TOptional<ElementType>)
#define P_FINISH									Stack.Code += !!Stack.Code; /* increment the code ptr unless it is null */

#define P_THIS_OBJECT								(Context)
#define P_THIS_CAST(ClassType)						((ClassType*)P_THIS_OBJECT)
#define P_THIS										P_THIS_CAST(ThisClass)

#define P_NATIVE_BEGIN { SCOPED_SCRIPT_NATIVE_TIMER(ScopedNativeCallTimer);
#define P_NATIVE_END   }

#ifndef UE_SCRIPT_ARGS_GC_BARRIER
#define UE_SCRIPT_ARGS_GC_BARRIER 1
#endif

#if UE_SCRIPT_ARGS_GC_BARRIER
#define P_ARG_GC_BARRIER(X) MutableView(ObjectPtrWrap(X))
#else
#define P_ARG_GC_BARRIER(X) X
#endif
