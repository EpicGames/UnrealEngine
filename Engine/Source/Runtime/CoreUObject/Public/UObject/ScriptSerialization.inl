// Copyright Epic Games, Inc. All Rights Reserved.

// [[ IncludeTool: Inline ]] // Markup to tell IncludeTool that this file is state changing and cannot be optimized out.

/**
 * This header contains the code for serialization of script bytecode and [eventually] tagged property values.
 * Extracted to header file to allow custom definitions of the macros used by these methods
 */
#ifdef VISUAL_ASSIST_HACK
	EExprToken SerializeExpr( int32&, FArchive& );
	int32 iCode=0;
	FNullArchive Ar;
	TArray<uint8> Script;
	EExprToken Expr = (EExprToken)0;
#endif

#ifndef XFER
#define XFER(T) \
			{ \
				T Temp; \
				if (!Ar.IsLoading()) \
				{ \
					Temp =  FPlatformMemory::ReadUnaligned<T>(&Script[iCode]); \
				} \
				Ar << Temp; \
				if (!Ar.IsSaving()) \
				{ \
					FPlatformMemory::WriteUnaligned<T>(&Script[iCode], Temp); \
				} \
				iCode += sizeof(T); \
			}
#endif

//FScriptName
#ifndef XFERNAME
	#define XFERNAME() \
	{ \
   	    FName Name; \
		FScriptName ScriptName; \
        if (!Ar.IsLoading()) \
		{ \
			FMemory::Memcpy( &ScriptName, &Script[iCode], sizeof(FScriptName) ); \
			Name = ScriptNameToName(ScriptName); \
		} \
		Ar << Name; \
		if (!Ar.IsSaving()) \
		{ \
			ScriptName = NameToScriptName(Name); \
			FMemory::Memcpy( &Script[iCode], &ScriptName, sizeof(FScriptName) ); \
		} \
		iCode += sizeof(FScriptName); \
	}
#endif	//XFERNAME

// ASCII string
#ifndef XFERSTRING
	#define XFERSTRING() \
	{ \
		do XFER(uint8) while( Script[iCode-1] ); \
	}
#endif	//XFERSTRING

// UTF-16 string
#ifndef XFERUNICODESTRING
	#define XFERUNICODESTRING() \
	{ \
		do XFER(uint16) while( Script[iCode-1] || Script[iCode-2] ); \
	}
#endif	//XFERUNICODESTRING

//FText
#ifndef XFERTEXT
	#define XFERTEXT() \
	{ \
		XFER(uint8); \
		const EBlueprintTextLiteralType TextLiteralType = (EBlueprintTextLiteralType)Script[iCode - 1]; \
		switch (TextLiteralType) \
		{ \
		case EBlueprintTextLiteralType::Empty: \
			break; \
		case EBlueprintTextLiteralType::LocalizedTextWithNotes: \
			SerializeExpr( iCode, Ar ); \
		case EBlueprintTextLiteralType::LocalizedText: \
			SerializeExpr( iCode, Ar );	\
			SerializeExpr( iCode, Ar ); \
			SerializeExpr( iCode, Ar ); \
			break; \
		case EBlueprintTextLiteralType::InvariantText: \
			SerializeExpr( iCode, Ar );	\
			break; \
		case EBlueprintTextLiteralType::LiteralString: \
			SerializeExpr( iCode, Ar );	\
			break; \
		case EBlueprintTextLiteralType::StringTableEntry: \
			XFER_OBJECT_POINTER( UObject* ); \
			FIXUP_EXPR_OBJECT_POINTER( UObject* ); \
			SerializeExpr( iCode, Ar );	\
			SerializeExpr( iCode, Ar ); \
			break; \
		default: \
			checkf(false, TEXT("Unknown EBlueprintTextLiteralType! Please update XFERTEXT to handle this type of text.")); \
			break; \
		} \
	}
#endif	//XFERTEXT

#ifndef XFERPTR 
	#define XFERPTR(T) \
	{ \
   	    T AlignedPtr = NULL; \
		ScriptPointerType TempCode; \
        if (!Ar.IsLoading()) \
		{ \
			FMemory::Memcpy( &TempCode, &Script[iCode], sizeof(ScriptPointerType) ); \
			AlignedPtr = (T)(TempCode); \
		} \
		Ar << AlignedPtr; \
		if (!Ar.IsSaving()) \
		{ \
			TempCode = (ScriptPointerType)(AlignedPtr); \
			FMemory::Memcpy( &Script[iCode], &TempCode, sizeof(ScriptPointerType) ); \
		} \
		iCode += sizeof(ScriptPointerType); \
	}
#endif	//	XFERPTR

#ifndef XFERTOBJPTR
	#define XFERTOBJPTR() \
	{ \
		TObjectPtr<UObject> AlignedPtr; \
		if (!Ar.IsLoading()) \
		{ \
			FMemory::Memcpy(&AlignedPtr, &Script[iCode], sizeof(ScriptPointerType)); \
		} \
			Ar << AlignedPtr; \
		if (!Ar.IsSaving()) \
		{ \
			FMemory::Memcpy(&Script[iCode], &AlignedPtr, sizeof(ScriptPointerType)); \
		} \
		iCode += sizeof(ScriptPointerType); \
	}
#endif	//	XFERTOBJPTR

#ifndef XFER_FUNC_POINTER
	#define XFER_FUNC_POINTER	XFERPTR(UStruct*)
#endif	// XFER_FUNC_POINTER

#ifndef XFER_FUNC_NAME
	#define XFER_FUNC_NAME		XFERNAME()
#endif	// XFER_FUNC_NAME

#ifndef XFER_PROP_POINTER
	#define XFER_PROP_POINTER	XFERPTR(FProperty*)
#endif

#ifndef XFER_OBJECT_POINTER
	#define XFER_OBJECT_POINTER(Type)	XFERPTR(Type)
#endif

#ifndef XFER_TOBJECT_PTR
	#define XFER_TOBJECT_PTR	XFERTOBJPTR
#endif

#ifndef FIXUP_EXPR_OBJECT_POINTER
	// sometimes after a UOBject* expression is loaded it may require some post-
	// processing (see: the overridden FIXUP_EXPR_OBJECT_POINTER(), defined in Class.cpp)
	#define FIXUP_EXPR_OBJECT_POINTER(Type) 
#endif

/** UStruct::SerializeExpr() */
#ifdef SERIALIZEEXPR_INC
	EExprToken Expr=(EExprToken)0;

	// Get expr token.
	XFER(uint8);
	Expr = (EExprToken)Script[iCode-1];

	switch( Expr )
	{
		case EX_Cast:
		{
			// A type conversion.
			XFER(uint8); //which kind of conversion
			SerializeExpr( iCode, Ar );
			break;
		}
		case EX_ObjToInterfaceCast:
		case EX_CrossInterfaceCast:
		case EX_InterfaceToObjCast:
		{
			// A conversion from an object or interface variable to a native interface variable.  
			// We use a different bytecode to avoid the branching each time we process a cast token.
			
			XFER_OBJECT_POINTER(UClass*); // the interface class to convert to
			FIXUP_EXPR_OBJECT_POINTER(UClass*);

			SerializeExpr( iCode, Ar );
			break;
		}
		case EX_Let:
		{
			XFER_PROP_POINTER;
		}
		case EX_LetObj:
		case EX_LetWeakObjPtr:
		case EX_LetBool:
		case EX_LetDelegate:
		case EX_LetMulticastDelegate:
		{
			SerializeExpr( iCode, Ar ); // Variable expr.
			SerializeExpr( iCode, Ar ); // Assignment expr.
			break;
		}
		case EX_LetValueOnPersistentFrame:
		{
			XFER_PROP_POINTER;			// Destination property.
			SerializeExpr(iCode, Ar);	// Assignment expr.
			break;
		}
		case EX_StructMemberContext:
		{
			XFERPTR(FProperty*);        // struct member expr.
			SerializeExpr( iCode, Ar ); // struct expr.
			break;
		}
		case EX_Jump:
		{
			XFER(CodeSkipSizeType); // Code offset.
			break;
		}
		case EX_ComputedJump:
		{
			SerializeExpr( iCode, Ar ); // Integer expression, specifying code offset.
			break;
		}
		case EX_LocalVariable:
		case EX_InstanceVariable:
		case EX_DefaultVariable:
		case EX_LocalOutVariable:
		case EX_ClassSparseDataVariable:
		case EX_PropertyConst:
		{
			XFER_PROP_POINTER;
			break;
		}
		case EX_InterfaceContext:
		{
			SerializeExpr(iCode,Ar);
			break;
		}
		case EX_PushExecutionFlow:
		{
			XFER(CodeSkipSizeType);		// location to push
			break;
		}
		case EX_NothingInt32:
		{
			XFER(int32);
			break;
		}
		case EX_Nothing:
		case EX_EndOfScript:
		case EX_EndFunctionParms:
		case EX_EndStructConst:
		case EX_EndArray:
		case EX_EndArrayConst:
		case EX_EndSet:
		case EX_EndMap:
		case EX_EndSetConst:
		case EX_EndMapConst:
		case EX_IntZero:
		case EX_IntOne:
		case EX_True:
		case EX_False:
		case EX_NoObject:
		case EX_NoInterface:
		case EX_Self:
		case EX_EndParmValue:
		case EX_PopExecutionFlow:
		case EX_DeprecatedOp4A:
		{
			break;
		}
		case EX_WireTracepoint:
		case EX_Tracepoint:
		{
			break;
		}
		case EX_Breakpoint:
		{
			if (Ar.IsLoading())
			{
				// Turn breakpoints into tracepoints on load
				Script[iCode-1] = EX_Tracepoint;
			}
			break;
		}
		case EX_InstrumentationEvent:
		{
			if (Script[iCode] == EScriptInstrumentation::InlineEvent)
			{
				iCode += sizeof(FScriptName);
			}
			iCode += sizeof(uint8);
			break;
		}
		case EX_Return:
		{
			SerializeExpr( iCode, Ar ); // Return expression.
			break;
		}
		case EX_CallMath:
		case EX_LocalFinalFunction:
		case EX_FinalFunction:
		{
			XFER_FUNC_POINTER;											// Stack node.
			FIXUP_EXPR_OBJECT_POINTER(UStruct*);
			while( SerializeExpr( iCode, Ar ) != EX_EndFunctionParms ); // Parms.
			break;
		}
		case EX_LocalVirtualFunction:
		case EX_VirtualFunction:
		{
			XFER_FUNC_NAME;												// Virtual function name.
			while( SerializeExpr( iCode, Ar ) != EX_EndFunctionParms );	// Parms.
			break;
		}
		case EX_CallMulticastDelegate:
		{
			XFER_FUNC_POINTER;											// Stack node.
			FIXUP_EXPR_OBJECT_POINTER(UStruct*);
			while( SerializeExpr( iCode, Ar ) != EX_EndFunctionParms ); // Parms.
			break;
		}
		case EX_ClassContext:
		case EX_Context:
		case EX_Context_FailSilent:
		{
			SerializeExpr( iCode, Ar ); // Object expression.
			XFER(CodeSkipSizeType);		// Code offset for NULL expressions.
			XFERPTR(FField*);			// Property corresponding to the r-value data, in case the l-value needs to be mem-zero'd
			SerializeExpr( iCode, Ar ); // Context expression.
			break;
		}
		case EX_AddMulticastDelegate:
		case EX_RemoveMulticastDelegate:
		{
			SerializeExpr( iCode, Ar );	// Delegate property to assign to
			SerializeExpr( iCode, Ar ); // Delegate to add to the MC delegate for broadcast
			break;
		}
		case EX_ClearMulticastDelegate:
		{
			SerializeExpr( iCode, Ar );	// Delegate property to clear
			break;
		}
		case EX_IntConst:
		{
			XFER(int32);
			break;
		}
		case EX_Int64Const:
		{
			XFER(int64);
			break;
		}
		case EX_UInt64Const:
		{
			XFER(uint64);
			break;
		}
		case EX_SkipOffsetConst:
		{
			XFER(CodeSkipSizeType);
			break;
		}
		case EX_FloatConst:
		{
			XFER(float);
			break;
		}
		case EX_DoubleConst:
		{
			XFER(double);
			break;
		}
		case EX_StringConst:
		{
			XFERSTRING();
			break;
		}
		case EX_UnicodeStringConst:
		{
			XFERUNICODESTRING();
			break;
		}
		case EX_TextConst:
		{
			XFERTEXT();
			break;
		}
		case EX_ObjectConst:
		{
			XFER_TOBJECT_PTR();
			FIXUP_EXPR_OBJECT_POINTER(TObjectPtr<UObject>);

			break;
		}
		case EX_SoftObjectConst:
		{
			// if collecting references inform the archive of the reference:
			if (Ar.IsSaving() && Ar.IsObjectReferenceCollector())
			{
				XFER(uint8);
				Expr = (EExprToken)Script[iCode - 1];
				check(Expr == EX_StringConst || Expr == EX_UnicodeStringConst);
				FString LongPath;
				if (Expr == EX_StringConst)
				{
					LongPath = (ANSICHAR*)&Script[iCode];
					XFERSTRING();
				}
				else
				{
					LongPath = FString((UCS2CHAR*)&Script[iCode]);
					XFERUNICODESTRING();
				}
				FSoftObjectPath Path(LongPath);
				Ar << Path;
				// we can't patch the path, but we could log an attempt to do so
				// or change the implementation to support patching (allocating 
				// these strings in a special region or distinct object)
			}
			else
			{
				// else just write the string literal instructions:
				SerializeExpr(iCode, Ar);
			}
			break;
		}
		case EX_FieldPathConst:
		{
			SerializeExpr(iCode, Ar);
			break;
		}
		case EX_NameConst:
		{
			XFERNAME();
			break;
		}
		case EX_RotationConst:
		{
			if(Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
			{
				XFER(int64); XFER(int64); XFER(int64);
			}
			else
			{
				XFER(int32); XFER(int32); XFER(int32);
			}
			break;
		}
		case EX_VectorConst:
		{
			if(Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
			{
				XFER(double); XFER(double); XFER(double);
			}
			else
			{
				XFER(float); XFER(float); XFER(float);
			}
			break;
		}
		case EX_Vector3fConst:
		{
			XFER(float); XFER(float); XFER(float);
			break;
		}
		case EX_TransformConst:
		{
			if(Ar.UEVer() >= EUnrealEngineObjectUE5Version::LARGE_WORLD_COORDINATES)
			{
				// Rotation
				XFER(double); XFER(double); XFER(double); XFER(double);
				// Translation
				XFER(double); XFER(double); XFER(double);
				// Scale
				XFER(double); XFER(double); XFER(double);
			}
			else
			{
				// Rotation
				XFER(float); XFER(float); XFER(float); XFER(float);
				// Translation
				XFER(float); XFER(float); XFER(float);
				// Scale
				XFER(float); XFER(float); XFER(float);
			}
			break;
		}
		case EX_StructConst:
		{
			XFERPTR(UScriptStruct*);	// Struct.
			XFER(int32);					// Serialized struct size
			while( SerializeExpr( iCode, Ar ) != EX_EndStructConst );
			break;
		}
		case EX_SetArray:
		{
			// If not loading, or its a newer version
			if((!GetLinker()) || !Ar.IsLoading() || (Ar.UEVer() >= VER_UE4_CHANGE_SETARRAY_BYTECODE))
			{
				// Array property to assign to
				EExprToken TargetToken = SerializeExpr( iCode, Ar );
			}
			else
			{
				// Array Inner Prop
				XFERPTR(FProperty*);
			}
		
			while( SerializeExpr( iCode, Ar) != EX_EndArray );
			break;
		}
		case EX_SetSet:
			SerializeExpr( iCode, Ar ); // set property
			XFER(int32);			// Number of elements
			while( SerializeExpr( iCode, Ar) != EX_EndSet );
			break;
		case EX_SetMap:
			SerializeExpr( iCode, Ar ); // map property
			XFER(int32);			// Number of elements
			while( SerializeExpr( iCode, Ar) != EX_EndMap );
			break;
		case EX_ArrayConst:
		{
			XFERPTR(FProperty*);	// Inner property
			XFER(int32);			// Number of elements
			while (SerializeExpr(iCode, Ar) != EX_EndArrayConst);
			break;
		}
		case EX_SetConst:
		{
			XFERPTR(FProperty*);	// Inner property
			XFER(int32);			// Number of elements
			while (SerializeExpr(iCode, Ar) != EX_EndSetConst);
			break;
		}
		case EX_MapConst:
		{
			XFERPTR(FProperty*);	// Key property
			XFERPTR(FProperty*);	// Val property
			XFER(int32);			// Number of elements
			while (SerializeExpr(iCode, Ar) != EX_EndMapConst);
			break;
		}
		case EX_BitFieldConst:
		{
			XFERPTR(FProperty*);	// Bit property
			XFER(uint8);			// bit value
			break;
		}
		case EX_ByteConst:
		case EX_IntConstByte:
		{
			XFER(uint8);
			break;
		}
		case EX_MetaCast:
		{
			XFER_OBJECT_POINTER(UClass*);
			FIXUP_EXPR_OBJECT_POINTER(UClass*);

			SerializeExpr( iCode, Ar );
			break;
		}
		case EX_DynamicCast:
		{
			XFER_OBJECT_POINTER(UClass*);
			FIXUP_EXPR_OBJECT_POINTER(UClass*);

			SerializeExpr( iCode, Ar );
			break;
		}
		case EX_JumpIfNot:
		{
			XFER(CodeSkipSizeType);		// Code offset.
			SerializeExpr( iCode, Ar ); // Boolean expr.
			break;
		}
		case EX_PopExecutionFlowIfNot:
		{
			SerializeExpr( iCode, Ar ); // Boolean expr.
			break;
		}
		case EX_Assert:
		{
			XFER(uint16); // Line number.
			XFER(uint8); // debug mode or not
			SerializeExpr( iCode, Ar ); // Assert expr.
			break;
		}
		case EX_Skip:
		{
			XFER(CodeSkipSizeType);		// Skip size.
			SerializeExpr( iCode, Ar ); // Expression to possibly skip.
			break;
		}
		case EX_InstanceDelegate:
		{
			XFER_FUNC_NAME;				// the name of the function assigned to the delegate.
			break;
		}
		case EX_BindDelegate:
		{
			XFER_FUNC_NAME;
			SerializeExpr( iCode, Ar );	// Delegate property to assign to
			SerializeExpr( iCode, Ar ); 
			break;
		}
		case EX_SwitchValue:
		{
			XFER(uint16); // number of cases, without default one
			const uint16 NumCases = FPlatformMemory::ReadUnaligned<uint16>(&Script[iCode - sizeof(uint16)]);
			XFER(CodeSkipSizeType); // Code offset, go to it, when done.
			SerializeExpr(iCode, Ar);	//index term

			for (uint16 CaseIndex = 0; CaseIndex < NumCases; ++CaseIndex)
			{
				SerializeExpr(iCode, Ar);	// case index value term
				XFER(CodeSkipSizeType);		// offset to the next case
				SerializeExpr(iCode, Ar);	// case term
			}

			SerializeExpr(iCode, Ar);	//default term
			break;
		}
		case EX_ArrayGetByRef:
		{
			SerializeExpr( iCode, Ar );
			SerializeExpr( iCode, Ar );
			break;
		}
		case EX_AutoRtfmTransact:
		{
			XFER(int32); // Transaction id
			XFER(CodeSkipSizeType); // Code offset.
			while( SerializeExpr( iCode, Ar ) != EX_AutoRtfmStopTransact ); // Parms.
			break;
		}
		case EX_AutoRtfmStopTransact:
		{
			XFER(int32); // transaction id
			XFER(int8); // stop mode
			break;
		}
		case EX_AutoRtfmAbortIfNot:
		{
			SerializeExpr(iCode,Ar);
			break;
		}
		case EX_AutoRtfmAbort:
		{
			break;
		}
		default:
		{
			// This should never occur.
			UE_LOGF(LogScriptSerialization, Warning, "Error: Unknown bytecode 0x%02X; ignoring it", (uint8)Expr );
			break;
		}
	}

#endif	//SERIALIZEEXPR_INC


#ifdef SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
	#undef XFER
	#undef XFERPTR
	#undef XFERNAME
	#undef XFERSTRING
	#undef XFERUNICODESTRING
	#undef XFERTEXT
	#undef XFER_FUNC_POINTER
	#undef XFER_FUNC_NAME
	#undef XFER_PROP_POINTER
	#undef FIXUP_EXPR_OBJECT_POINTER
#endif	//SERIALIZEEXPR_AUTO_UNDEF_XFER_MACROS
