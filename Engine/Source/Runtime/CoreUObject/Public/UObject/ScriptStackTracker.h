// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
ScriptStackTracker.h: Script Stack Tracking within Unreal Engine.
=============================================================================*/
#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

class FOutputDevice;
struct FFrame;

struct FScriptStackTracker
{
private:
	/** Maximum number of backtrace depth. */
	static const int32 MAX_BACKTRACE_DEPTH = 50;

	/** Helper structure to capture callstack addresses and stack count. */
	struct FCallStack
	{
		/** Stack count, aka the number of calls to CalculateStack */
		int64 StackCount;
		/** String representation of script callstack. */
		FString StackTrace;
	};

	/** Array of unique callstacks. */
	TArray<FCallStack> CallStacks;
	/** Mapping from callstack CRC to index in callstack array. */
	TMap<uint32,int32> CRCToCallStackIndexMap;
	/** Whether we are currently capturing or not, used to avoid re-entrancy. */
	bool bAvoidCapturing;
	/** Whether stack tracking is enabled. */
	bool bIsEnabled;
	/** Frame counter at the time tracking was enabled. */
	uint64 StartFrameCounter;
	/** Frame counter at the time tracking was disabled. */
	uint64 StopFrameCounter;

public:
	/** Constructor, initializing all member variables */
	FScriptStackTracker( bool bInIsEnabled = false )
		:	bAvoidCapturing(false)
		,	bIsEnabled(bInIsEnabled)
		,	StartFrameCounter(0)
		,	StopFrameCounter(0)
	{}

	/**
	 * Captures the current stack and updates stack tracking information.
	 */
	COREUOBJECT_API void CaptureStackTrace(const FFrame* StackFrame, int32 EntriesToIgnore = 0);

	/**
	 * Dumps capture stack trace summary to the passed in log.
	 */
	COREUOBJECT_API void DumpStackTraces( int32 StackThreshold, FOutputDevice& Ar );

	/** Resets stack tracking. */
	COREUOBJECT_API void ResetTracking();

	/** Toggles tracking. */
	COREUOBJECT_API void ToggleTracking();
};


