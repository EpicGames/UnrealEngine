// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Containers/StringFwd.h"
#include "Containers/StringOverload.h"
#include "Containers/StringView.h"
#include "Containers/Utf8String.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSingleton.h"
#include "Misc/CString.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/Function.h"
#include "Templates/TypeHash.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/StructOpsTypeTraits.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UObjectHash.h"

#include "SoftObjectPath.generated.h"

class FArchive;
class FCbWriter;
class FOutputDevice;
struct FPropertyTag;
struct FUObjectSerializeContext;

// Wrapper for UE_DEPRECATED that can be selectively enabled.
#define UE_SOFTOBJECTPATH_FSTRING_DEPRECATED(Version, Message) /*UE_DEPRECATED(Version, Message)*/

/** Delegate called on completion of async loading a soft object. The UObject will be null if the load failed */
DECLARE_DELEGATE_TwoParams(FLoadSoftObjectPathAsyncDelegate, const FSoftObjectPath&, UObject*);

constexpr inline auto FSoftObjectPath_DefaultPreFixupForPIEFunction = [](int32, FSoftObjectPath&) {};

/**
 * A struct that contains a string reference to an object, either a package, a top level asset or a subobject.
 * This can be used to make soft references to assets that are loaded on demand.
 * This is stored internally as an FTopLevelAssetPath pointing to the top level asset (/package/path.assetname) and an optional string subobject path.
 * If the MetaClass metadata is applied to a FProperty with this the UI will restrict to that type of asset.
 */
USTRUCT(BlueprintType, meta=(HasNativeMake="/Script/Engine.KismetSystemLibrary.MakeSoftObjectPath", HasNativeBreak="/Script/Engine.KismetSystemLibrary.BreakSoftObjectPath"))
struct FSoftObjectPath
{
	GENERATED_BODY()
public:

	[[nodiscard]] FSoftObjectPath() = default;
	[[nodiscard]] FSoftObjectPath(const FSoftObjectPath& Other) = default;
	[[nodiscard]] FSoftObjectPath(FSoftObjectPath&& Other) = default;
	~FSoftObjectPath() = default;
	FSoftObjectPath& operator=(const FSoftObjectPath& Path) = default;
	FSoftObjectPath& operator=(FSoftObjectPath&& Path) = default;

	/** Construct from a path string. Non-explicit for backwards compatibility. */
	[[nodiscard]] FSoftObjectPath(const FString& Path)
	{
		SetPath(FStringView(Path));
	}
	UE_SOFTOBJECTPATH_FSTRING_DEPRECATED(5.6, "FSoftObjectPath with a wide string subpath has been deprecated - please use a UTF8 subpath instead.")
	[[nodiscard]] explicit FSoftObjectPath(FTopLevelAssetPath InAssetPath, TStringOverload<FWideString> InSubPathString)
	{
		SetPath(InAssetPath, FUtf8String(InSubPathString.MoveTemp()));
	}
	[[nodiscard]] explicit FSoftObjectPath(FTopLevelAssetPath InAssetPath, TStringOverload<FUtf8String> InSubPathString)
	{
		SetPath(InAssetPath, InSubPathString.MoveTemp());
	}
	/** Explicitly extend a top-level object path with an empty subobject path. */
	[[nodiscard]] explicit FSoftObjectPath(FTopLevelAssetPath InAssetPath)
	{
		SetPath(InAssetPath);
	}
	[[nodiscard]] explicit FSoftObjectPath(FWideStringView Path)
	{
		SetPath(Path);
	}
	[[nodiscard]] explicit FSoftObjectPath(FAnsiStringView Path)
	{
		SetPath(Path);
	}
	[[nodiscard]] explicit FSoftObjectPath(const WIDECHAR* Path)
	{
		SetPath(FWideStringView(Path));
	}
	[[nodiscard]] explicit FSoftObjectPath(const ANSICHAR* Path)
	{
		SetPath(FAnsiStringView(Path));
	}
	[[nodiscard]] explicit FSoftObjectPath(TYPE_OF_NULLPTR)
	{
	}

	template <typename T>
	[[nodiscard]] FSoftObjectPath(const TObjectPtr<T>& InObject)
	{
		// Avoid conversion to string if the object is resolved
		if (InObject.IsResolved())
		{
			// reinterpret_cast here is used because T may be an incomplete type, but we only support using TObjectPtr for UObject-derived types 
			SetPath(reinterpret_cast<const UObject*>(InObject.Get()));
		}
		else
		{
			SetPath(InObject.GetPathName());
		}
	}

	[[nodiscard]] FSoftObjectPath(const FObjectPtr& InObject)
	{
		// Avoid conversion to string if the object is resolved
		if (InObject.IsResolved())
		{
			SetPath(InObject.Get());
		}
		else
		{
			SetPath(InObject.GetPathName());
		}
	}

	/** Construct from an existing object in memory */
	[[nodiscard]] FSoftObjectPath(const UObject* InObject)
	{
		// Avoid conversion to string 
		SetPath(InObject);
	}

	/** Static methods for more meaningful construction sites. */
	UE_DEPRECATED(5.6, "FSoftObjectPath::ConstructFromPackageAsset has been deprecated - please use ConstructFromAssetPath instead.")
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromPackageAsset(FName InPackageName, FName InAssetName);
	UE_SOFTOBJECTPATH_FSTRING_DEPRECATED(5.6, "FSoftObjectPath::ConstructFromAssetPathAndSubpath with a wide string subpath has been deprecated - please use a UTF8 subpath instead.")
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromAssetPathAndSubpath(FTopLevelAssetPath InAssetPath, TStringOverload<FWideString> InSubPathString);
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromAssetPathAndSubpath(FTopLevelAssetPath InAssetPath, TStringOverload<FUtf8String> InSubPathString);
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromAssetPath(FTopLevelAssetPath InAssetPath);
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromStringPath(FString&& InPath);
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromStringPath(FStringView InPath);
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromStringPath(FUtf8StringView InPath);
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromObject(const UObject* InObject);
	[[nodiscard]] COREUOBJECT_API static FSoftObjectPath ConstructFromObject(const FObjectPtr& InObject);
	template <typename T>
	[[nodiscard]] static FSoftObjectPath ConstructFromObject(const TObjectPtr<T>& InObject)
	{
		return ConstructFromObject(InObject.Get());
	}

	FSoftObjectPath& operator=(const FTopLevelAssetPath Path)
	{
		SetPath(Path);
		return *this;
	}
	FSoftObjectPath& operator=(const FString& Path)
	{
		SetPath(FStringView(Path));
		return *this;
	}
	FSoftObjectPath& operator=(FWideStringView Path)
	{
		SetPath(Path);
		return *this;
	}
	FSoftObjectPath& operator=(FAnsiStringView Path)
	{
		SetPath(Path);
		return *this;
	}
	FSoftObjectPath& operator=(const WIDECHAR* Path)
	{
		SetPath(FWideStringView(Path));
		return *this;
	}
	FSoftObjectPath& operator=(const ANSICHAR* Path)
	{
		SetPath(FAnsiStringView(Path));
		return *this;
	}
	FSoftObjectPath& operator=(TYPE_OF_NULLPTR)
	{
		Reset();
		return *this;
	}

	/** Returns string representation of reference, in form /package/path.assetname[:subpath] */
	[[nodiscard]] COREUOBJECT_API FString ToString() const;

	/** Append string representation of reference, in form /package/path.assetname[:subpath] */
	COREUOBJECT_API void ToString(FStringBuilderBase& Builder) const;
	COREUOBJECT_API void ToString(FUtf8StringBuilderBase& Builder) const;

	/** Append string representation of reference, in form /package/path.assetname[:subpath] */
	COREUOBJECT_API void AppendString(FString& Builder) const;
	COREUOBJECT_API void AppendString(FStringBuilderBase& Builder) const;
	COREUOBJECT_API void AppendString(FUtf8StringBuilderBase& Builder) const;

	/** Returns the top-level asset part of this path, without the subobject path. */
	[[nodiscard]] FTopLevelAssetPath GetAssetPath() const
	{
		return AssetPath;
	}

	/** Returns this path without the SubPath component, restricting the result to a top level asset but keeping the type as FSoftObjectPath in contrast to GetAssetPath. */
	[[nodiscard]] FSoftObjectPath GetWithoutSubPath() const
	{
		return FSoftObjectPath(AssetPath);
	}

	/** Returns string version of asset path, including both package and asset but not sub object */
	[[nodiscard]] inline FString GetAssetPathString() const
	{
		if (AssetPath.IsNull())
		{
			return FString();
		}

		return AssetPath.ToString();
	}

	/** Returns the sub path, which is often empty */
	UE_SOFTOBJECTPATH_FSTRING_DEPRECATED(5.6, "FSoftObjectPath::GetSubPathString has been deprecated - please use GetSubPathUtf8String() instead.")
	[[nodiscard]] UE_FORCEINLINE_HINT FString GetSubPathString() const
	{
		return FString(SubPathString);
	}
	[[nodiscard]] UE_FORCEINLINE_HINT const FUtf8String& GetSubPathUtf8String() const
	{
		return SubPathString;
	}

	UE_SOFTOBJECTPATH_FSTRING_DEPRECATED(5.6, "FSoftObjectPath::ConstructFromAssetPathAndSubpath with a wide string subpath has been deprecated - please use a UTF8 subpath instead.")
	UE_FORCEINLINE_HINT void SetSubPathString(TStringOverload<FWideString> InSubPathString)
	{
		SubPathString = FUtf8String(InSubPathString.MoveTemp());
	}
	UE_FORCEINLINE_HINT void SetSubPathString(TStringOverload<FUtf8String> InSubPathString)
	{
		SubPathString = InSubPathString.MoveTemp();
	}

	/** Returns /package/path, leaving off the asset name and sub object */
	[[nodiscard]] FString GetLongPackageName() const
	{
		FName PackageName = GetAssetPath().GetPackageName();
		return PackageName.IsNone() ? FString() : PackageName.ToString();
	}

	/** Returns /package/path, leaving off the asset name and sub object */
	[[nodiscard]] FName GetLongPackageFName() const
	{
		return GetAssetPath().GetPackageName();
	}

	/** Returns assetname string, leaving off the /package/path part and sub object */
	[[nodiscard]] FString GetAssetName() const
	{
		FName AssetName = GetAssetPath().GetAssetName();
		return AssetName.IsNone() ? FString() : AssetName.ToString();
	}

	/** Returns assetname string, leaving off the /package/path part and sub object */
	[[nodiscard]] FName GetAssetFName() const
	{
		return GetAssetPath().GetAssetName();
	}

	/** Sets asset path of this reference based on a string path */
	COREUOBJECT_API void SetPath(const FTopLevelAssetPath& InAssetPath);
	UE_SOFTOBJECTPATH_FSTRING_DEPRECATED(5.6, "FSoftObjectPath::SetPath with a wide string subpath has been deprecated - please use a UTF8 subpath instead.")
	COREUOBJECT_API void SetPath(const FTopLevelAssetPath& InAssetPath, TStringOverload<FWideString> InSubPathString);
	COREUOBJECT_API void SetPath(const FTopLevelAssetPath& InAssetPath, TStringOverload<FUtf8String> InSubPathString);
	COREUOBJECT_API void SetPath(FWideStringView Path);
	COREUOBJECT_API void SetPath(FAnsiStringView Path);
	COREUOBJECT_API void SetPath(FUtf8StringView Path);
	COREUOBJECT_API void SetPath(const UObject* InObject);
	void SetPath(const WIDECHAR* Path)
	{
		SetPath(FWideStringView(Path));
	}
	void SetPath(const ANSICHAR* Path)
	{
		SetPath(FAnsiStringView(Path));
	}
	void SetPath(const FString& Path)
	{
		SetPath(FStringView(Path));
	}

	/** Remap the package to handle renames. */
	COREUOBJECT_API bool RemapPackage(FName OldPackageName, FName NewPackageName);

	/**
	 * Attempts to load the asset, this will call LoadObject which can be very slow
	 * @param InLoadContext Optional load context when called from nested load callstack
	 * @params LoadFlags Flags to forward to LoadObject e.g. LOAD_Quiet
	 * @return Loaded UObject, or nullptr if the reference is null or the asset fails to load
	 */
	COREUOBJECT_API UObject* TryLoad(FUObjectSerializeContext* InLoadContext = nullptr, ELoadFlags LoadFlags = LOAD_None) const;

	/**
	 * Attempts to asynchronously load the object referenced by this path.
	 * This will call LoadAssetAsync to load the top level asset and resolve the sub paths before calling the delegate.
	 * FStreamableManager can be used for more control over callback timing and garbage collection.
	 * 
	 * @param	InCompletionDelegate	Delegate to be invoked when the async load finishes, this will execute on the game thread as soon as the load succeeds or fails
	 * @param	InOptionalParams		Optional parameters for async loading the asset
	 * @return Unique ID associated with this load request (the same object or package can be associated with multiple IDs).
	 */
	COREUOBJECT_API int32 LoadAsync(FLoadSoftObjectPathAsyncDelegate InCompletionDelegate, FLoadAssetAsyncOptionalParams InOptionalParams = FLoadAssetAsyncOptionalParams()) const;

	/**
	 * Attempts to find a currently loaded object that matches this path
	 *
	 * @return Found UObject, or nullptr if not currently in memory
	 */
	[[nodiscard]] COREUOBJECT_API UObject* ResolveObject() const;

	/** Resets reference to point to null */
	void Reset()
	{
		AssetPath.Reset();
		SubPathString.Reset();
	}
	
	/** Check if this could possibly refer to a real object, or was initialized to null */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid() const
	{
		return AssetPath.IsValid();
	}

	/** Checks to see if this is initialized to null */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsNull() const
	{
		return AssetPath.IsNull();
	}

	/** Check if this represents an asset, meaning it is not null but does not have a sub path */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsAsset() const
	{
		return !AssetPath.IsNull() && SubPathString.IsEmpty();
	}

	/** Check if this represents a sub object, meaning it has a sub path */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsSubobject() const
	{
		return !AssetPath.IsNull() && !SubPathString.IsEmpty();
	}

	/** Return true if this path appears before Other in lexical order */
	[[nodiscard]] bool LexicalLess(const FSoftObjectPath& Other) const
	{
		int32 PathCompare = AssetPath.Compare(Other.AssetPath);
		if (PathCompare != 0)
		{
			return PathCompare < 0;
		}
		return SubPathString.Compare(Other.SubPathString) < 0; 
	}

	/** Return true if this path appears before Other using fast index-based fname order */
	[[nodiscard]] bool FastLess(const FSoftObjectPath& Other) const
	{
		int32 PathCompare = AssetPath.CompareFast(Other.AssetPath);
		if (PathCompare != 0)
		{
			return PathCompare < 0;
		}
		return SubPathString.Compare(Other.SubPathString) < 0; 
	}

	/** Struct overrides */
	COREUOBJECT_API bool Serialize(FArchive& Ar);
	COREUOBJECT_API bool Serialize(FStructuredArchive::FSlot Slot);
	[[nodiscard]] COREUOBJECT_API bool operator==(FSoftObjectPath const& Other) const;

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	[[nodiscard]] bool operator!=(FSoftObjectPath const& Other) const
	{
		return !(*this == Other);
	}
#endif

	COREUOBJECT_API bool ExportTextItem(FString& ValueStr, FSoftObjectPath const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	COREUOBJECT_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive = nullptr);
	COREUOBJECT_API bool SerializeFromMismatchedTag(struct FPropertyTag const& Tag, FStructuredArchive::FSlot Slot);

	/** Serializes the internal path and also handles save/PIE fixups. Call this from the archiver overrides */
	COREUOBJECT_API void SerializePath(FArchive& Ar);

	/** Serializes the internal path without any save/PIE fixups. Only call this directly if you know what you are doing */
	COREUOBJECT_API void SerializePathWithoutFixup(FArchive& Ar);

	/** Fixes up path for saving, call if saving with a method that skips SerializePath. This can modify the path, it will return true if it was modified */
	COREUOBJECT_API bool PreSavePath(bool* bReportSoftObjectPathRedirects = nullptr);

	/** 
	 * Handles when a path has been loaded, call if loading with a method that skips SerializePath. This does not modify path but might call callbacks
	 * @param InArchive The archive that loaded this path
	 */
	COREUOBJECT_API void PostLoadPath(FArchive* InArchive) const;

	/** Fixes up this SoftObjectPath to add the PIE prefix depending on what is currently active, returns true if it was modified. The overload that takes an explicit PIE instance is preferred, if it's available. */
	COREUOBJECT_API bool FixupForPIE(TFunctionRef<void(int32, FSoftObjectPath&)> InPreFixupForPIECustomFunction = FSoftObjectPath_DefaultPreFixupForPIEFunction);

	/** Fixes up this SoftObjectPath to add the PIE prefix for the given PIEInstance index, returns true if it was modified */
	COREUOBJECT_API bool FixupForPIE(int32 PIEInstance, TFunctionRef<void(int32, FSoftObjectPath&)> InPreFixupForPIECustomFunction = FSoftObjectPath_DefaultPreFixupForPIEFunction);

	/** Fixes soft object path for CoreRedirects to handle renamed native objects, returns true if it was modified */
	COREUOBJECT_API bool FixupCoreRedirects();

	[[nodiscard]] inline friend uint32 GetTypeHash(FSoftObjectPath const& This)
	{
		uint32 Hash = GetTypeHash(This.AssetPath);
			
		if (!This.SubPathString.IsEmpty())
		{
			Hash = HashCombineFast(Hash, GetTypeHash(This.SubPathString));
		}

		return Hash;
	}

	[[nodiscard]] static COREUOBJECT_API FSoftObjectPath GetOrCreateIDForObject(FObjectPtr Object);
	[[nodiscard]] static UE_FORCEINLINE_HINT FSoftObjectPath GetOrCreateIDForObject(const UObject* Object)
	{
		return GetOrCreateIDForObject(FObjectPtr(const_cast<UObject*>(Object)));
	}
	template <typename T>
	[[nodiscard]] static UE_FORCEINLINE_HINT FSoftObjectPath GetOrCreateIDForObject(TObjectPtr<T> Object)
	{
		// This needs to be a template instead of TObjectPtr<const UObject> because C++ does derived-to-base
		// pointer conversions ('standard conversion sequences') in more cases than TSmartPtr<Derived>-to-TSmartPtr<Base>
		// conversions ('user-defined conversions'), meaning it doesn't auto-convert in many real use cases.
		//
		// https://en.cppreference.com/w/cpp/language/implicit_conversion

		return GetOrCreateIDForObject(FObjectPtr(Object));
	}

	/** Adds list of packages names that have been created specifically for PIE, this is used for editor fixup */
	static COREUOBJECT_API void AddPIEPackageName(FName NewPIEPackageName);
	
	/** Disables special PIE path handling, call when PIE finishes to clear list */
	static COREUOBJECT_API void ClearPIEPackageNames();

#if WITH_EDITOR
	/**
	 * Text of the Untracked parameter in a UPROPERTY's meta field, if this parameter is present it marks that
	 * the SoftObjectPath reference should not cause the target package to be added to the cook.
	 */
	static COREUOBJECT_API FName NAME_Untracked;
#endif
private:
	/** Asset path, patch to a top level object in a package. This is /package/path.assetname */
	UPROPERTY()
	FTopLevelAssetPath AssetPath;

	/** Optional FString for subobject within an asset. This is the sub path after the : */
	UPROPERTY()
	FUtf8String SubPathString;

	/** Package names currently being duplicated, needed by FixupForPIE */
	static COREUOBJECT_API TSet<FName> PIEPackageNames;

	[[nodiscard]] COREUOBJECT_API UObject* ResolveObjectInternal() const;

	COREUOBJECT_API friend void SerializeForLog(FCbWriter& Writer, const FSoftObjectPath& Value);
	friend struct Z_Construct_UScriptStruct_FSoftObjectPath_Statics;
};

template<>
struct TStructOpsTypeTraits<FSoftObjectPath> : public TStructOpsTypeTraitsBase2<FSoftObjectPath>
{
	enum
	{
		WithZeroConstructor = true,
		WithStructuredSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::Soft;
};


/** Fast non-alphabetical order that is only stable during this process' lifetime. */
struct FSoftObjectPathFastLess
{
	[[nodiscard]] bool operator()(const FSoftObjectPath& Lhs, const FSoftObjectPath& Rhs) const
	{
		return Lhs.FastLess(Rhs);
	}
};

/** Slow alphabetical order that is stable / deterministic over process runs. */
struct FSoftObjectPathLexicalLess
{
	[[nodiscard]] bool operator()(const FSoftObjectPath& Lhs, const FSoftObjectPath& Rhs) const
	{
		return Lhs.LexicalLess(Rhs);
	}
};

inline FStringBuilderBase& operator<<(FStringBuilderBase& Builder, const FSoftObjectPath& Path)
{
	Path.ToString(Builder);
	return Builder;
}
inline FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, const FSoftObjectPath& Path)
{
	Path.ToString(Builder);
	return Builder;
}

/**
 * A struct that contains a string reference to a class, can be used to make soft references to classes
 */
USTRUCT(BlueprintType, meta=(HasNativeMake="/Script/Engine.KismetSystemLibrary.MakeSoftClassPath", HasNativeBreak="/Script/Engine.KismetSystemLibrary.BreakSoftClassPath"))
struct FSoftClassPath : public FSoftObjectPath
{
	GENERATED_BODY()
public:
	[[nodiscard]] FSoftClassPath() = default;
	[[nodiscard]] FSoftClassPath(const FSoftClassPath& Other) = default;
	[[nodiscard]] FSoftClassPath(FSoftClassPath&& Other) = default;
	~FSoftClassPath() = default;
	FSoftClassPath& operator=(const FSoftClassPath& Path) = default;
	FSoftClassPath& operator=(FSoftClassPath&& Path) = default;

	/**
	 * Construct from a path string
	 */
	[[nodiscard]] FSoftClassPath(const FString& PathString)
		: FSoftObjectPath(PathString)
	{
	}

	/**
	 * Construct from an existing class, will do some string processing
	 */
	[[nodiscard]] FSoftClassPath(const UClass* InClass)
		: FSoftObjectPath(InClass)
	{
	}

	/**
	* Attempts to load the class.
	* @return Loaded UObject, or null if the class fails to load, or if the reference is not valid.
	*/
	template<typename T>
	UClass* TryLoadClass() const
	{
		if ( IsValid() )
		{
			return LoadClass<T>(nullptr, *ToString(), nullptr, LOAD_None, nullptr);
		}

		return nullptr;
	}

	/**
	 * Attempts to find a currently loaded object that matches this object ID
	 * @return Found UClass, or NULL if not currently loaded
	 */
	[[nodiscard]] COREUOBJECT_API UClass* ResolveClass() const;

	COREUOBJECT_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	[[nodiscard]] static COREUOBJECT_API FSoftClassPath GetOrCreateIDForClass(const UClass *InClass);

private:
	/** Forbid creation for UObject. This class is for UClass only. Use FSoftObjectPath instead. */
	[[nodiscard]] explicit FSoftClassPath(const UObject* InObject)
	{
	}

	/** Forbidden. This class is for UClass only. Use FSoftObjectPath instead. */
	static COREUOBJECT_API FSoftObjectPath GetOrCreateIDForObject(const UObject *Object);
};

template<>
struct TStructOpsTypeTraits<FSoftClassPath> : public TStructOpsTypeTraitsBase2<FSoftClassPath>
{
	enum
	{
		WithZeroConstructor = true,
		WithSerializer = true,
		WithCopy = true,
		WithIdenticalViaEquality = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
		WithStructuredSerializeFromMismatchedTag = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::Soft;
};

/** Options for how to set soft object path collection */
enum class ESoftObjectPathCollectType : uint8
{
	/**
	 * The SoftObjectPath being loaded is not in a package, so we do not need to record it in inclusion or exclusion lists,
	 * or translate redirects for it.
	 */
	NonPackage,
	/**
	 * The SoftObjectPath being loaded is in a package header, not in an Object, so we do not need to record it in
	 * inclusion or exclusion lists, but we do need to translate redirects for it.
	 */
	PackageHeader,
	/** References is not tracked in any situation, transient reference */
	NeverCollect,
	/** Editor only reference, this is tracked for redirector fixup but not for cooking */
	EditorOnlyCollect,
	/** Game reference, this is gathered for both redirector fixup and cooking */
	AlwaysCollect,
};

namespace UE::SoftObjectPath
{

/**
 * Whether the CollectType indicates that this SoftObjectPath is from a package, either in its header or in an object
 * within the package.
 */
inline bool IsPackageType(ESoftObjectPathCollectType CollectType)
{
	return CollectType != ESoftObjectPathCollectType::NonPackage;
}

/** Whether the CollectType indicates that this SoftObjectPath is in a UObject in a package. */
inline bool IsObjectType(ESoftObjectPathCollectType CollectType)
{
	return (CollectType != ESoftObjectPathCollectType::NonPackage)
		& (CollectType != ESoftObjectPathCollectType::PackageHeader);
}

/**
 * Whether the CollectType is one of the ones that should always or sometimes be collected. This means only ObjectType
 * CollectTypes that are not NeverCollect.
 */
inline bool IsCollectable(ESoftObjectPathCollectType CollectType)
{
	return (CollectType != ESoftObjectPathCollectType::NonPackage)
		& (CollectType != ESoftObjectPathCollectType::PackageHeader)
		& (CollectType != ESoftObjectPathCollectType::NeverCollect);
}

}

/** Rules for actually serializing the internals of soft object paths */
enum class ESoftObjectPathSerializeType : uint8
{
	/** Never serialize the raw names */
	NeverSerialize,
	/** Only serialize if the archive has no size */
	SkipSerializeIfArchiveHasSize,
	/** Always serialize the soft object path internals */
	AlwaysSerialize,
};

UE_DECLARE_THREAD_SINGLETON_TLS(class FSoftObjectPathThreadContext, COREUOBJECT_API)

class FSoftObjectPathThreadContext : public TThreadSingleton<FSoftObjectPathThreadContext>
{
	friend TThreadSingleton<FSoftObjectPathThreadContext>;
	friend struct FSoftObjectPathSerializationScope;

	[[nodiscard]] FSoftObjectPathThreadContext()
	{
	}

	struct FSerializationOptions
	{
		FName PackageName;
		FName PropertyName;
		ESoftObjectPathCollectType CollectType;
		ESoftObjectPathSerializeType SerializeType;

		[[nodiscard]] FSerializationOptions()
			: CollectType(ESoftObjectPathCollectType::AlwaysCollect)
		{
		}

		[[nodiscard]] explicit FSerializationOptions(FName InPackageName, FName InPropertyName, ESoftObjectPathCollectType InCollectType, ESoftObjectPathSerializeType InSerializeType)
			: PackageName(InPackageName)
			, PropertyName(InPropertyName)
			, CollectType(InCollectType)
			, SerializeType(InSerializeType)
		{
		}
	};

	TArray<FSerializationOptions> OptionStack;
public:
	/** 
	 * Returns the current serialization options that were added using SerializationScope or LinkerLoad
	 *
	 * @param OutPackageName Package that this string asset belongs to
	 * @param OutPropertyName Property that this path belongs to
	 * @param OutCollectType Type of collecting that should be done
	 * @param Archive The FArchive that is serializing this path if known. If null it will check FUObjectThreadContext
	 */
	COREUOBJECT_API bool GetSerializationOptions(FName& OutPackageName, FName& OutPropertyName, ESoftObjectPathCollectType& OutCollectType, ESoftObjectPathSerializeType& OutSerializeType, FArchive* Archive = nullptr) const;
};

/** Helper class to set and restore serialization options for soft object paths */
struct FSoftObjectPathSerializationScope
{
	/** 
	 * Create a new serialization scope, which affects the way that soft object paths are saved
	 *
	 * @param SerializingPackageName Package that this string asset belongs to
	 * @param SerializingPropertyName Property that this path belongs to
	 * @param CollectType Set type of collecting that should be done, can be used to disable tracking entirely
	 */
	[[nodiscard]] explicit FSoftObjectPathSerializationScope(FName SerializingPackageName, FName SerializingPropertyName, ESoftObjectPathCollectType CollectType, ESoftObjectPathSerializeType SerializeType)
	{
		FSoftObjectPathThreadContext::Get().OptionStack.Emplace(SerializingPackageName, SerializingPropertyName, CollectType, SerializeType);
	}

	[[nodiscard]] explicit FSoftObjectPathSerializationScope(ESoftObjectPathCollectType CollectType)
	{
		FSoftObjectPathThreadContext::Get().OptionStack.Emplace(NAME_None, NAME_None, CollectType, ESoftObjectPathSerializeType::AlwaysSerialize);
	}

	~FSoftObjectPathSerializationScope()
	{
		FSoftObjectPathThreadContext::Get().OptionStack.Pop();
	}
};

/** Structure for file paths that are displayed in the editor with a picker UI. */
USTRUCT(BlueprintType)
struct FFilePath
{
	GENERATED_BODY()

	/**
	 * The path to the file.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = FilePath)
	FString FilePath;
};

/** Structure for directory paths that are displayed in the editor with a picker UI. */
USTRUCT(BlueprintType)
struct FDirectoryPath
{
	GENERATED_BODY()

	/**
	 * The path to the directory.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Path)
	FString Path;
};

// Fixup archive
struct FSoftObjectPathFixupArchive : public FArchiveUObject
{
	[[nodiscard]] explicit FSoftObjectPathFixupArchive(TFunction<void(FSoftObjectPath&)> InFixupFunction)
		: FixupFunction(InFixupFunction)
	{
		this->SetIsSaving(true);
		this->ArShouldSkipBulkData = true;
		this->SetShouldSkipCompilingAssets(true);
	}

	[[nodiscard]] explicit FSoftObjectPathFixupArchive(const FString& InOldAssetPathString, const FString& InNewAssetPathString)
		: FSoftObjectPathFixupArchive([OldAssetPathString = InOldAssetPathString, NewAssetPath = FTopLevelAssetPath(InNewAssetPathString)](FSoftObjectPath& Value)
		{
			if (!Value.IsNull() && Value.GetAssetPathString().Equals(OldAssetPathString, ESearchCase::IgnoreCase))
			{
				Value = FSoftObjectPath(NewAssetPath, Value.GetSubPathUtf8String());
			}
		})
	{
	}

	FArchive& operator<<(FSoftObjectPath& Value) override
	{
		FixupFunction(Value);
		return *this;
	}

	virtual FArchive& operator<<(FObjectPtr& Value) override
	{
		//do nothing to avoid resolving
		return *this;
	}

	void Fixup(UObject* Root)
	{
		Root->Serialize(*this);
		TArray<UObject*> SubObjects;
		GetObjectsWithOuter(Root, SubObjects);
		for (UObject* Obj : SubObjects)
		{
			Obj->Serialize(*this);
		}
	}

	TFunction<void(FSoftObjectPath&)> FixupFunction;
};

namespace UE::SoftObjectPath::Private
{
	UE_DEPRECATED(5.1, "This function is only for use in fixing up deprecated APIs.")
	[[nodiscard]] inline TArray<FName> ConvertSoftObjectPaths(TConstArrayView<FSoftObjectPath> InPaths)
	{
		TArray<FName> Out;
		Out.Reserve(InPaths.Num());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Algo::Transform(InPaths, Out, [](const FSoftObjectPath& Path)
		{
			return FName(*Path.ToString());
		});
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return Out;
	}

	UE_DEPRECATED(5.1, "This function is only for use in fixing up deprecated APIs.")
	[[nodiscard]] inline TArray<FSoftObjectPath> ConvertObjectPathNames(TConstArrayView<FName> InPaths)
	{
		TArray<FSoftObjectPath> Out;
		Out.Reserve(InPaths.Num());
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Algo::Transform(InPaths, Out, [](FName Name)
		{
				FSoftObjectPath Result;
				Result.SetPath(Name.ToString());
				return Result;
		});
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return Out;
	}
}

#if WITH_LOW_LEVEL_TESTS
#include <ostream>
COREUOBJECT_API std::ostream& operator<<(std::ostream& Stream, const FSoftObjectPath& Value);
#endif
