// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	SoftObjectPtr.h: Pointer to UObject asset, keeps extra information so that it is works even if the asset is not in memory
=============================================================================*/

#pragma once

#include "UObject/Object.h"
#include "Templates/Casts.h"
#include "UObject/PersistentObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "Templates/Models.h"
#include "Concepts/EqualityComparable.h"

#include <type_traits>

#ifndef UE_DEPRECATE_SOFTOBJECTPTR_CONVERSIONS
	#define UE_DEPRECATE_SOFTOBJECTPTR_CONVERSIONS 1
#endif
#if UE_DEPRECATE_SOFTOBJECTPTR_CONVERSIONS
	#define UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(Version, Message) UE_DEPRECATED(Version, Message)
#else
	#define UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(Version, Message)
#endif

/**
 * TIsSoftObjectPointerType
 * Trait for recognizing 'soft' (path-based) object pointer types
 */
template<typename T> 
struct TIsSoftObjectPointerType
{ 
	enum { Value = false };
};

/**
 * FSoftObjectPtr is a type of weak pointer to a UObject, that also keeps track of the path to the object on disk.
 * It will change back and forth between being Valid and Pending as the referenced object loads or unloads.
 * It has no impact on if the object is garbage collected or not.
 *
 * This is useful to specify assets that you may want to asynchronously load on demand.
 */
struct FSoftObjectPtr : public TPersistentObjectPtr<FSoftObjectPath>
{
private:
	using Super = TPersistentObjectPtr<FSoftObjectPath>;

public:	
	[[nodiscard]] UE_FORCEINLINE_HINT FSoftObjectPtr() = default;
	[[nodiscard]] UE_FORCEINLINE_HINT FSoftObjectPtr(const FSoftObjectPtr& Other) = default;
	[[nodiscard]] UE_FORCEINLINE_HINT FSoftObjectPtr(FSoftObjectPtr&& Other) = default;
	UE_FORCEINLINE_HINT ~FSoftObjectPtr() = default;
	UE_FORCEINLINE_HINT FSoftObjectPtr& operator=(const FSoftObjectPtr& Other) = default;
	UE_FORCEINLINE_HINT FSoftObjectPtr& operator=(FSoftObjectPtr&& Other) = default;

	[[nodiscard]] explicit UE_FORCEINLINE_HINT FSoftObjectPtr(const FSoftObjectPath& ObjectPath)
		: Super(ObjectPath)
	{
	}

	[[nodiscard]] explicit UE_FORCEINLINE_HINT FSoftObjectPtr(FObjectPtr Object)
	{
		(*this)=Object;
	}
	[[nodiscard]] explicit UE_FORCEINLINE_HINT FSoftObjectPtr(const UObject* Object)
		: FSoftObjectPtr(FObjectPtr(const_cast<UObject*>(Object)))
	{
	}
	template <typename T>
	[[nodiscard]] explicit UE_FORCEINLINE_HINT FSoftObjectPtr(TObjectPtr<T> Object)
		: FSoftObjectPtr(FObjectPtr(Object))
	{
		// This needs to be a template instead of TObjectPtr<const UObject> because C++ does derived-to-base
		// pointer conversions ('standard conversion sequences') in situations that TSmartPtr<Derived>-to-TSmartPtr<Base>
		// conversions ('user-defined conversions') doesn't, meaning it won't auto-convert in many real use cases.
		//
		// https://en.cppreference.com/w/cpp/language/implicit_conversion
	}

	/** Synchronously load (if necessary) and return the asset object represented by this asset ptr */
	UObject* LoadSynchronous(ELoadFlags LoadFlags = LOAD_None) const
	{
		UObject* Asset = Get();
		if (Asset == nullptr && !IsNull())
		{
			ToSoftObjectPath().TryLoad(nullptr, LoadFlags);
			
			// TryLoad will have loaded this pointer if it is valid
			Asset = Get();
		}
		return Asset;
	}

	/** Returns the StringObjectPath that is wrapped by this SoftObjectPtr */
	[[nodiscard]] UE_FORCEINLINE_HINT const FSoftObjectPath& ToSoftObjectPath() const
	{
		return GetUniqueID();
	}

	/** Returns string representation of reference, in form /package/path.assetname */
	[[nodiscard]] UE_FORCEINLINE_HINT FString ToString() const
	{
		return ToSoftObjectPath().ToString();
	}

	/** Returns /package/path string, leaving off the asset name */
	[[nodiscard]] UE_FORCEINLINE_HINT FString GetLongPackageName() const
	{
		return ToSoftObjectPath().GetLongPackageName();
	}
	
	/** Returns /package/path name, leaving off the asset name */
	[[nodiscard]] UE_FORCEINLINE_HINT FName GetLongPackageFName() const
	{
		return ToSoftObjectPath().GetLongPackageFName();
	}

	/** Returns assetname string, leaving off the /package/path. part */
	[[nodiscard]] UE_FORCEINLINE_HINT FString GetAssetName() const
	{
		return ToSoftObjectPath().GetAssetName();
	}

#if WITH_EDITOR
	/** Overridden to deal with PIE lookups */
	[[nodiscard]] inline UObject* Get() const
	{
		if (UE::GetPlayInEditorID() != INDEX_NONE)
		{
			// Cannot use or set the cached value in PIE as it may affect other PIE instances or the editor
			TWeakObjectPtr<UObject> Result = GetUniqueID().ResolveObject();
			// If this object is pending kill or otherwise invalid, this will return nullptr just like TPersistentObjectPtr<FSoftObjectPath>::Get()
			return Result.Get();
		}
		return Super::Get();
	}
#endif

	// Implicit conversion from UObject* via assignment shouldn't really be allowed if the constructor is explicit
	using TPersistentObjectPtr<FSoftObjectPath>::operator=;
	FSoftObjectPtr& operator=(FObjectPtr Ptr)
	{
		Super* SuperThis = this;
		(*SuperThis) = Ptr;
		return *this;
	}
	UE_FORCEINLINE_HINT FSoftObjectPtr& operator=(const UObject* Ptr)
	{
		return *this = FObjectPtr(const_cast<UObject*>(Ptr));
	}
	template <typename T>
	UE_FORCEINLINE_HINT FSoftObjectPtr& operator=(TObjectPtr<T> Ptr)
	{
		// This needs to be a template instead of TObjectPtr<const UObject> because C++ does derived-to-base
		// pointer conversions ('standard conversion sequences') in situations that TSmartPtr<Derived>-to-TSmartPtr<Base>
		// conversions ('user-defined conversions') doesn't, meaning it won't auto-convert in many real use cases.
		//
		// https://en.cppreference.com/w/cpp/language/implicit_conversion

		return *this = FObjectPtr(Ptr);
	}
};

template <> struct TIsPODType<FSoftObjectPtr> { enum { Value = TIsPODType<TPersistentObjectPtr<FSoftObjectPath> >::Value }; };
template <> struct TIsWeakPointerType<FSoftObjectPtr> { enum { Value = TIsWeakPointerType<TPersistentObjectPtr<FSoftObjectPath> >::Value }; };
template <> struct TIsSoftObjectPointerType<FSoftObjectPtr> { enum { Value = true }; };

/**
 * TSoftObjectPtr is templatized wrapper of the generic FSoftObjectPtr, it can be used in UProperties
 */
template<class T=UObject>
struct TSoftObjectPtr
{
	template <class U>
	friend struct TSoftObjectPtr;

public:
	using ElementType = T;
	
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr() = default;
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(const TSoftObjectPtr& Other) = default;
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(TSoftObjectPtr&& Other) = default;
	UE_FORCEINLINE_HINT ~TSoftObjectPtr() = default;
	UE_FORCEINLINE_HINT TSoftObjectPtr& operator=(const TSoftObjectPtr& Other) = default;
	UE_FORCEINLINE_HINT TSoftObjectPtr& operator=(TSoftObjectPtr&& Other) = default;
	
	/** Construct from another soft pointer */
	template <
		class U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(const TSoftObjectPtr<U>& Other)
		: SoftObjectPtr(Other.SoftObjectPtr)
	{
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		class U
		UE_REQUIRES(!std::is_convertible_v<U*, T*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Constructing TSoftObjectPtr from an incompatible pointer type has been deprecated.")
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(const TSoftObjectPtr<U>& Other)
		: SoftObjectPtr(Other.SoftObjectPtr)
	{
	}
#endif

	/** Construct from a moveable soft pointer */
	template <
		class U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(TSoftObjectPtr<U>&& Other)
		: SoftObjectPtr(MoveTemp(Other.SoftObjectPtr))
	{
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		class U
		UE_REQUIRES(!std::is_convertible_v<U*, T*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Constructing TSoftObjectPtr from an incompatible pointer type has been deprecated.")
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(TSoftObjectPtr<U>&& Other)
		: SoftObjectPtr(MoveTemp(Other.SoftObjectPtr))
	{
	}
#endif

	/** Construct from an object already in memory */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(U* Object)
		: SoftObjectPtr(Object)
	{
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		typename U
		UE_REQUIRES(!std::is_convertible_v<U*, T*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Constructing TSoftObjectPtr from an incompatible pointer type has been deprecated.")
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(U* Object)
		: SoftObjectPtr(Object)
	{
	}
#endif
#if UE_ENABLE_NOTNULL_WRAPPER
	/** Construct from an object already in memory. Convenience function for implicit initialization from TNotNull pointers. */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(TNotNull<U*> Object)
		: SoftObjectPtr(Object)
	{
	}
	/** Construct from an object already in memory. Convenience function for implicit initialization from TNotNull pointers. */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(TNotNull<TObjectPtr<U>> Object)
		: SoftObjectPtr(static_cast<TObjectPtr<U>>(Object).Get())
	{
	}
#endif

	/** Construct from a TObjectPtr<U> which may or may not be in memory. */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(const TObjectPtr<U> Object)
		: SoftObjectPtr(Object.Get())
	{
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		typename U
		UE_REQUIRES(!std::is_convertible_v<U*, T*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Constructing TSoftObjectPtr from an incompatible pointer type has been deprecated.")
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(const TObjectPtr<U> Object)
		: SoftObjectPtr(Object.Get())
	{
	}
#endif

	/** Construct from a nullptr */
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftObjectPtr(TYPE_OF_NULLPTR)
		: SoftObjectPtr(nullptr)
	{
	}

	/** Construct from a soft object path */
	template <
		typename SoftObjectPathType
		UE_REQUIRES(std::is_same_v<SoftObjectPathType, FSoftObjectPath>)
	>
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TSoftObjectPtr(SoftObjectPathType ObjectPath)
		: SoftObjectPtr(MoveTemp(ObjectPath))
	{
		// The reason for the strange templated/constrained signature is to prevent this type hole:
		//
		// const UThing* ConstPtr = ...;
		// TSoftObjectPtr<UThing*> NonConstSoftPtr(ConstPtr); // implicitly constructs an FSoftObjectPath from ConstPtr, then explicitly constructs the TSoftObjectPtr from that
		//
		// The real fix is to make the FSoftObjectPath(const UObject*) constructor explicit, but that will involve substantially more changes.
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Constructing TSoftObjectPtr from an FString has been deprecated - instead, explicitly construct a FSoftObjectPath.")
	[[nodiscard]] explicit TSoftObjectPtr(const FString& Path)
		: SoftObjectPtr(FSoftObjectPath(Path))
	{
	}
#endif

	/** Reset the soft pointer back to the null state */
	UE_FORCEINLINE_HINT void Reset()
	{
		SoftObjectPtr.Reset();
	}

	/** Resets the weak ptr only, call this when ObjectId may change */
	UE_FORCEINLINE_HINT void ResetWeakPtr()
	{
		SoftObjectPtr.ResetWeakPtr();
	}

	/** Copy from an object already in memory */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline TSoftObjectPtr& operator=(U* Object)
	{
		SoftObjectPtr = Object;
		return *this;
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		typename U
		UE_REQUIRES(!std::is_convertible_v<U*, T*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Assigning TSoftObjectPtr from an incompatible pointer type has been deprecated.")
	inline TSoftObjectPtr& operator=(U* Object)
	{
		SoftObjectPtr = Object;
		return *this;
	}
#endif

#if UE_ENABLE_NOTNULL_WRAPPER
	/** Copy from an object already in memory. Convenience function for implicit assignment from TNotNull pointers. */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline TSoftObjectPtr& operator=(TNotNull<U*> Object)
	{
		SoftObjectPtr = Object;
		return *this;
	}
	/** Copy from an object already in memory. Convenience function for implicit assignment from TNotNull pointers. */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline TSoftObjectPtr& operator=(TNotNull<TObjectPtr<U>> Object)
	{
		SoftObjectPtr = static_cast<TObjectPtr<U>>(Object).Get();
		return *this;
	}
#endif

	/** Copy from a TObjectPtr<U> which may or may not be in memory. */
	template <
		typename U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline TSoftObjectPtr& operator=(const TObjectPtr<U> Object)
	{
		SoftObjectPtr = Object.Get();
		return *this;
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		typename U
		UE_REQUIRES(!std::is_convertible_v<U*, T*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Assigning TSoftObjectPtr from an incompatible pointer type has been deprecated.")
	inline TSoftObjectPtr& operator=(const TObjectPtr<U> Object)
	{
		SoftObjectPtr = Object.Get();
		return *this;
	}
#endif

	/** Assign from a nullptr */
	inline TSoftObjectPtr& operator=(TYPE_OF_NULLPTR)
	{
		SoftObjectPtr = nullptr;
		return *this;
	}

	/** Copy from a soft object path */
	inline TSoftObjectPtr& operator=(FSoftObjectPath ObjectPath)
	{
		SoftObjectPtr = MoveTemp(ObjectPath);
		return *this;
	}

	/** Copy from a weak pointer to an object already in memory */
	template <
		class U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline TSoftObjectPtr& operator=(const TWeakObjectPtr<U>& Other)
	{
		SoftObjectPtr = Other;
		return *this;
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		class U
		UE_REQUIRES(!std::is_convertible_v<U*, T*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Assigning TSoftObjectPtr from an incompatible pointer type has been deprecated.")
	inline TSoftObjectPtr& operator=(const TWeakObjectPtr<U>& Other)
	{
		SoftObjectPtr = Other;
		return *this;
	}
#endif

	/** Copy from another soft pointer */
	template <
		class U
		UE_REQUIRES(std::is_convertible_v<U*, T*>)
	>
	inline TSoftObjectPtr& operator=(TSoftObjectPtr<U> Other)
	{
		SoftObjectPtr = MoveTemp(Other.SoftObjectPtr);
		return *this;
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		class U
		UE_REQUIRES(!std::is_convertible_v<U*, T*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Assigning TSoftObjectPtr from an incompatible pointer type has been deprecated.")
	inline TSoftObjectPtr& operator=(TSoftObjectPtr<U> Other)
	{
		SoftObjectPtr = MoveTemp(Other.SoftObjectPtr);
		return *this;
	}
#endif
	/**
	 * Compare soft pointers for equality
	 * Caution: Two soft pointers might not be equal to each other, but they both might return nullptr
	 *
	 * @param Other soft pointer to compare to
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TSoftObjectPtr& Rhs) const
	{
		return SoftObjectPtr == Rhs.SoftObjectPtr;
	}
	template <
		typename U
		UE_REQUIRES(TModels_V<CEqualityComparableWith, T*, U*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TSoftObjectPtr<U>& Rhs) const
	{
		return SoftObjectPtr == Rhs.SoftObjectPtr;
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		typename U
		UE_REQUIRES(!TModels_V<CEqualityComparableWith, T*, U*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Comparing TSoftObjectPtrs of incompatible pointer types has been deprecated.")
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TSoftObjectPtr<U>& Rhs) const
	{
		return SoftObjectPtr == Rhs.SoftObjectPtr;
	}
#endif

	template <
		typename U
		UE_REQUIRES(TModels_V<CEqualityComparableWith, T*, U*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(U* Rhs) const
	{
		return SoftObjectPtr == TSoftObjectPtr<U>(Rhs).SoftObjectPtr;
	}

#if !defined(PLATFORM_COMPILER_IWYU)
	template <
		typename U
		UE_REQUIRES(!TModels_V<CEqualityComparableWith, T*, U*>)
	>
	UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Comparing TSoftObjectPtrs of incompatible pointer types has been deprecated.")
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(U* Rhs) const
	{
		return SoftObjectPtr == TSoftObjectPtr<U>(Rhs).SoftObjectPtr;
	}
#endif

	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR) const
	{
		return SoftObjectPtr == nullptr;
	}

	/**
	 * Dereference the soft pointer.
	 *
	 * @return nullptr if this object is gone or the lazy pointer was null, otherwise a valid UObject pointer
	 */
	[[nodiscard]] T* Get() const;

	/** Dereference the soft pointer */
	[[nodiscard]] UE_FORCEINLINE_HINT T& operator*() const
	{
		return *Get();
	}

	/** Dereference the soft pointer */
	[[nodiscard]] UE_FORCEINLINE_HINT T* operator->() const
	{
		return Get();
	}

	/** Synchronously load (if necessary) and return the asset object represented by this asset ptr */
	T* LoadSynchronous() const
	{
		UObject* Asset = SoftObjectPtr.LoadSynchronous();
		return Cast<T>(Asset);
	}

	/**
	 * Attempts to asynchronously load the object referenced by this soft pointer.
	 * This is a wrapper around the LoadAsync function in SoftObjectPath, and the delegate is responsible for validating it loaded the correct type
	 *
	 * @param	InCompletionDelegate	Delegate to be invoked when the async load finishes, this will execute on the game thread as soon as the load succeeds or fails
	 * @param	InOptionalParams		Optional parameters for async loading the asset
	 * @return Unique ID associated with this load request (the same object or package can be associated with multiple IDs).
	 */
	int32 LoadAsync(FLoadSoftObjectPathAsyncDelegate InCompletionDelegate, FLoadAssetAsyncOptionalParams InOptionalParams = FLoadAssetAsyncOptionalParams()) const
	{
		return SoftObjectPtr.ToSoftObjectPath().LoadAsync(MoveTemp(InCompletionDelegate), MoveTemp(InOptionalParams));
	}

	/**  
	 * Test if this points to a live UObject
	 *
	 * @return true if Get() would return a valid non-null pointer
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid() const
	{
		// This does the runtime type check
		return Get() != nullptr;
	}

	/**  
	 * Test if this does not point to a live UObject, but may in the future
	 *
	 * @return true if this does not point to a real object, but could possibly
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsPending() const
	{
		return SoftObjectPtr.IsPending();
	}

	/**  
	 * Test if this can never point to a live UObject
	 *
	 * @return true if this is explicitly pointing to no object
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsNull() const
	{
		return SoftObjectPtr.IsNull();
	}

	/** Returns the StringObjectPath that is wrapped by this SoftObjectPtr */
	[[nodiscard]] UE_FORCEINLINE_HINT const FSoftObjectPath& GetUniqueID() const
	{
		return SoftObjectPtr.GetUniqueID();
	}

	/** Returns the StringObjectPath that is wrapped by this SoftObjectPtr */
	[[nodiscard]] UE_FORCEINLINE_HINT const FSoftObjectPath& ToSoftObjectPath() const
	{
		return SoftObjectPtr.GetUniqueID();
	}

	/** Returns string representation of reference, in form /package/path.assetname */
	[[nodiscard]] UE_FORCEINLINE_HINT FString ToString() const
	{
		return ToSoftObjectPath().ToString();
	}

	/** Returns /package/path string, leaving off the asset name */
	[[nodiscard]] UE_FORCEINLINE_HINT FString GetLongPackageName() const
	{
		return ToSoftObjectPath().GetLongPackageName();
	}
	
	/** Returns /package/path name, leaving off the asset name */
	[[nodiscard]] UE_FORCEINLINE_HINT FName GetLongPackageFName() const
	{
		return ToSoftObjectPath().GetLongPackageFName();
	}

	/** Returns assetname string, leaving off the /package/path part */
	[[nodiscard]] UE_FORCEINLINE_HINT FString GetAssetName() const
	{
		return ToSoftObjectPath().GetAssetName();
	}

	/** Dereference soft pointer to see if it points somewhere valid */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return IsValid();
	}

	/** Hash function */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetPtrTypeHash() const
	{
		return GetTypeHash(static_cast<const TPersistentObjectPtr<FSoftObjectPath>&>(SoftObjectPtr));
	}

	UE_FORCEINLINE_HINT void Serialize(FArchive& Ar)
	{
		Ar << SoftObjectPtr;
	}

private:
	FSoftObjectPtr SoftObjectPtr;
};

template <typename T>
TSoftObjectPtr(T*) -> TSoftObjectPtr<T>;

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(TModels_V<CEqualityComparableWith, LhsType*, RhsType*>)
>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(LhsType* Lhs, const TSoftObjectPtr<RhsType>& Rhs)
{
	return Rhs == Lhs;
}
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(!TModels_V<CEqualityComparableWith, LhsType*, RhsType*>)
>
UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Comparing TSoftObjectPtrs of incompatible pointer types has been deprecated.")
[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(LhsType* Lhs, const TSoftObjectPtr<RhsType>& Rhs)
{
	return Rhs == Lhs;
}
template <typename RhsType>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(TYPE_OF_NULLPTR Lhs, const TSoftObjectPtr<RhsType>& Rhs)
{
	return Rhs == Lhs;
}
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(TModels_V<CEqualityComparableWith, LhsType*, RhsType*>)
>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TSoftObjectPtr<LhsType>& Lhs, const TSoftObjectPtr<RhsType>& Rhs)
{
	return !(Lhs == Rhs);
}
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(!TModels_V<CEqualityComparableWith, LhsType*, RhsType*>)
>
UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Comparing TSoftObjectPtrs of incompatible pointer types has been deprecated.")
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TSoftObjectPtr<LhsType>& Lhs, const TSoftObjectPtr<RhsType>& Rhs)
{
	return !(Lhs == Rhs);
}
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(TModels_V<CEqualityComparableWith, LhsType*, RhsType*>)
>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TSoftObjectPtr<LhsType>& Lhs, RhsType* Rhs)
{
	return !(Lhs == Rhs);
}
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(!TModels_V<CEqualityComparableWith, LhsType*, RhsType*>)
>
UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Comparing TSoftObjectPtrs of incompatible pointer types has been deprecated.")
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TSoftObjectPtr<LhsType>& Lhs, RhsType* Rhs)
{
	return !(Lhs == Rhs);
}
template <typename LhsType>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TSoftObjectPtr<LhsType>& Lhs, TYPE_OF_NULLPTR Rhs)
{
	return !(Lhs == Rhs);
}
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(TModels_V<CEqualityComparableWith, LhsType*, RhsType*>)
>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(LhsType* Lhs, const TSoftObjectPtr<RhsType>& Rhs)
{
	return !(Lhs == Rhs);
}
template <
	typename LhsType,
	typename RhsType
	UE_REQUIRES(!TModels_V<CEqualityComparableWith, LhsType*, RhsType*>)
>
UE_SOFTOBJECTPTR_CONVERSION_DEPRECATED(5.5, "Comparing TSoftObjectPtrs of incompatible pointer types has been deprecated.")
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(LhsType* Lhs, const TSoftObjectPtr<RhsType>& Rhs)
{
	return !(Lhs == Rhs);
}
template <typename RhsType>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(TYPE_OF_NULLPTR Lhs, const TSoftObjectPtr<RhsType>& Rhs)
{
	return !(Lhs == Rhs);
}
#endif

/** Hash function */
template<class T>
[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetTypeHash(const TSoftObjectPtr<T>& Ptr)
{
	return Ptr.GetPtrTypeHash();
}

template<class T>
FArchive& operator<<(FArchive& Ar, TSoftObjectPtr<T>& Ptr)
{
	Ptr.Serialize(Ar);
	return Ar;
}


template<class T> struct TIsPODType<TSoftObjectPtr<T> > { enum { Value = TIsPODType<FSoftObjectPtr>::Value }; };
template<class T> struct TIsWeakPointerType<TSoftObjectPtr<T> > { enum { Value = TIsWeakPointerType<FSoftObjectPtr>::Value }; };
template<class T> struct TIsSoftObjectPointerType<TSoftObjectPtr<T>> { enum { Value = TIsSoftObjectPointerType<FSoftObjectPtr>::Value }; };

template <typename T>
struct TCallTraits<TSoftObjectPtr<T>> : public TCallTraitsBase<TSoftObjectPtr<T>>
{
	using ConstPointerType = TSoftObjectPtr<const T>;
};

/** Utility to create a TSoftObjectPtr without specifying the type */
template <class T>
[[nodiscard]] TSoftObjectPtr<T> MakeSoftObjectPtr(T* Object)
{
	static_assert(std::is_base_of_v<UObject, T>, "Type must derive from UObject");
	return TSoftObjectPtr<T>(Object);
}

template <class T>
[[nodiscard]] TSoftObjectPtr<T> MakeSoftObjectPtr(TObjectPtr<T> Object)
{
	static_assert(std::is_base_of_v<UObject, T>, "Type must derive from UObject");
	return TSoftObjectPtr<T>(ToRawPtr(Object));
}

/**
 * TSoftClassPtr is a templatized wrapper around FSoftObjectPtr that works like a TSubclassOf, it can be used in UProperties for blueprint subclasses
 */
template<class TClass=UObject>
class TSoftClassPtr
{
	template <class TClassA>
	friend class TSoftClassPtr;

public:
	using ElementType = TClass;
	
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftClassPtr() = default;
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftClassPtr(const TSoftClassPtr& Other) = default;
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftClassPtr(TSoftClassPtr&& Other) = default;
	UE_FORCEINLINE_HINT ~TSoftClassPtr() = default;
	UE_FORCEINLINE_HINT TSoftClassPtr& operator=(const TSoftClassPtr& Other) = default;
	UE_FORCEINLINE_HINT TSoftClassPtr& operator=(TSoftClassPtr&& Other) = default;
		
	/** Construct from another soft pointer */
	template <
		class TClassA
		UE_REQUIRES(std::is_convertible_v<TClassA*, TClass*>)
	>
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftClassPtr(const TSoftClassPtr<TClassA>& Other)
		: SoftObjectPtr(Other.SoftObjectPtr)
	{
	}

	/** Construct from a class already in memory */
	[[nodiscard]] UE_FORCEINLINE_HINT TSoftClassPtr(const UClass* From)
		: SoftObjectPtr(From)
	{
	}

	/** Construct from a soft object path */
	[[nodiscard]] explicit UE_FORCEINLINE_HINT TSoftClassPtr(const FSoftObjectPath& ObjectPath)
		: SoftObjectPtr(ObjectPath)
	{
	}

	/** Reset the soft pointer back to the null state */
	UE_FORCEINLINE_HINT void Reset()
	{
		SoftObjectPtr.Reset();
	}

	/** Resets the weak ptr only, call this when ObjectId may change */
	UE_FORCEINLINE_HINT void ResetWeakPtr()
	{
		SoftObjectPtr.ResetWeakPtr();
	}

	/** Copy from a class already in memory */
	UE_FORCEINLINE_HINT void operator=(const UClass* From)
	{
		SoftObjectPtr = From;
	}

	/** Copy from a soft object path */
	UE_FORCEINLINE_HINT void operator=(const FSoftObjectPath& ObjectPath)
	{
		SoftObjectPtr = ObjectPath;
	}

	/** Copy from a weak pointer already in memory */
	template <
		class TClassA
		UE_REQUIRES(std::is_convertible_v<TClassA*, TClass*>)
	>
	inline TSoftClassPtr& operator=(const TWeakObjectPtr<TClassA>& Other)
	{
		SoftObjectPtr = Other;
		return *this;
	}

	/** Copy from another soft pointer */
	template <
		class TClassA
		UE_REQUIRES(std::is_convertible_v<TClassA*, TClass*>)
	>
	inline TSoftClassPtr& operator=(const TSoftObjectPtr<TClassA>& Other)
	{
		SoftObjectPtr = Other.SoftObjectPtr;
		return *this;
	}

	/**  
	 * Compare soft pointers for equality
	 * Caution: Two soft pointers might not be equal to each other, but they both might return nullptr
	 *
	 * @param Other soft pointer to compare to 
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const TSoftClassPtr& Other) const
	{
		return SoftObjectPtr == Other.SoftObjectPtr;
	}
#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
	/**  
	 * Compare soft pointers for inequality
	 * Caution: Two soft pointers might not be equal to each other, but they both might return nullptr
	 *
	 * @param Other soft pointer to compare to
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const TSoftClassPtr& Other) const
	{
		return SoftObjectPtr != Other.SoftObjectPtr;
	}
#endif

	/**  
	 * Dereference the soft pointer
	 *
	 * @return nullptr if this object is gone or the soft pointer was null, otherwise a valid UClass pointer
	 */
	[[nodiscard]] inline UClass* Get() const
	{
		UClass* Class = dynamic_cast<UClass*>(SoftObjectPtr.Get());
		if (!Class || !Class->IsChildOf(TClass::StaticClass()))
		{
			return nullptr;
		}
		return Class;
	}

	/** Dereference the soft pointer */
	[[nodiscard]] UE_FORCEINLINE_HINT UClass& operator*() const
	{
		return *Get();
	}

	/** Dereference the soft pointer */
	[[nodiscard]] UE_FORCEINLINE_HINT UClass* operator->() const
	{
		return Get();
	}

	/**  
	 * Test if this points to a live UObject
	 *
	 * @return true if Get() would return a valid non-null pointer
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsValid() const
	{
		// This also does the UClass type check
		return Get() != nullptr;
	}

	/**  
	 * Test if this does not point to a live UObject, but may in the future
	 *
	 * @return true if this does not point to a real object, but could possibly
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsPending() const
	{
		return SoftObjectPtr.IsPending();
	}

	/**  
	 * Test if this can never point to a live UObject
	 *
	 * @return true if this is explicitly pointing to no object
	 */
	[[nodiscard]] UE_FORCEINLINE_HINT bool IsNull() const
	{
		return SoftObjectPtr.IsNull();
	}

	/** Returns the StringObjectPath that is wrapped by this SoftObjectPtr */
	[[nodiscard]] UE_FORCEINLINE_HINT const FSoftObjectPath& GetUniqueID() const
	{
		return SoftObjectPtr.GetUniqueID();
	}

	/** Returns the StringObjectPath that is wrapped by this SoftObjectPtr */
	[[nodiscard]] UE_FORCEINLINE_HINT const FSoftObjectPath& ToSoftObjectPath() const
	{
		return SoftObjectPtr.GetUniqueID();
	}

	/** Returns string representation of reference, in form /package/path.assetname  */
	[[nodiscard]] UE_FORCEINLINE_HINT FString ToString() const
	{
		return ToSoftObjectPath().ToString();
	}

	/** Returns /package/path string, leaving off the asset name */
	[[nodiscard]] UE_FORCEINLINE_HINT FString GetLongPackageName() const
	{
		return ToSoftObjectPath().GetLongPackageName();
	}
	
	/** Returns /package/path name, leaving off the asset name */
	[[nodiscard]] UE_FORCEINLINE_HINT FName GetLongPackageFName() const
	{
		return ToSoftObjectPath().GetLongPackageFName();
	}

	/** Returns assetname string, leaving off the /package/path part */
	[[nodiscard]] UE_FORCEINLINE_HINT FString GetAssetName() const
	{
		return ToSoftObjectPath().GetAssetName();
	}

	/** Dereference soft pointer to see if it points somewhere valid */
	[[nodiscard]] UE_FORCEINLINE_HINT explicit operator bool() const
	{
		return IsValid();
	}

	/** Hash function */
	[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetPtrTypeHash() const
	{
		return GetTypeHash(static_cast<const TPersistentObjectPtr<FSoftObjectPath>&>(SoftObjectPtr));
	}

	/** Synchronously load (if necessary) and return the asset object represented by this asset ptr */
	UClass* LoadSynchronous(ELoadFlags LoadFlags = LOAD_None) const
	{
		UObject* Asset = SoftObjectPtr.LoadSynchronous(LoadFlags);
		UClass* Class = dynamic_cast<UClass*>(Asset);
		if (!Class || !Class->IsChildOf(TClass::StaticClass()))
		{
			return nullptr;
		}
		return Class;
	}

	/**
	 * Attempts to asynchronously load the object referenced by this soft pointer.
	 * This is a wrapper around the LoadAsync function in SoftObjectPath, and the delegate is responsible for validating it loaded the correct type
	 *
	 * @param	InCompletionDelegate	Delegate to be invoked when the async load finishes, this will execute on the game thread as soon as the load succeeds or fails
	 * @param	InOptionalParams		Optional parameters for async loading the asset
	 * @return Unique ID associated with this load request (the same object or package can be associated with multiple IDs).
	 */
	int32 LoadAsync(FLoadSoftObjectPathAsyncDelegate InCompletionDelegate, FLoadAssetAsyncOptionalParams InOptionalParams = FLoadAssetAsyncOptionalParams()) const
	{
		return SoftObjectPtr.ToSoftObjectPath().LoadAsync(MoveTemp(InCompletionDelegate), MoveTemp(InOptionalParams));
	}

	inline void Serialize(FArchive& Ar)
	{
		Ar << static_cast<FSoftObjectPtr&>(SoftObjectPtr);
	}

private:
	FSoftObjectPtr SoftObjectPtr;
};

template <class T> struct TIsPODType<TSoftClassPtr<T> > { enum { Value = TIsPODType<FSoftObjectPtr>::Value }; };
template <class T> struct TIsWeakPointerType<TSoftClassPtr<T> > { enum { Value = TIsWeakPointerType<FSoftObjectPtr>::Value }; };

template <typename T>
struct TCallTraits<TSoftClassPtr<T>> : public TCallTraitsBase<TSoftClassPtr<T>>
{
	using ConstPointerType = TSoftClassPtr<const T>;
};

/** Utility to create a TSoftObjectPtr without specifying the type */
template <class T>
[[nodiscard]] TSoftClassPtr<std::remove_cv_t<T>> MakeSoftClassPtr(T* Object)
{
	static_assert(std::is_base_of_v<UClass, T>, "Type must derive from UClass");
	return TSoftClassPtr<std::remove_cv_t<T>>(Object);
}

template <class T>
[[nodiscard]] TSoftClassPtr<std::remove_cv_t<T>> MakeSoftClassPtr(TObjectPtr<T> Object)
{
	static_assert(std::is_base_of_v<UClass, T>, "Type must derive from UClass");
	return TSoftClassPtr<std::remove_cv_t<T>>(ToRawPtr(Object));
}

/** Fast non-alphabetical order that is only stable during this process' lifetime. */
struct FSoftObjectPtrFastLess : private FSoftObjectPathFastLess
{
	template <typename SoftObjectPtrType>
	bool operator()(const SoftObjectPtrType& Lhs, const SoftObjectPtrType& Rhs) const
	{
		return FSoftObjectPathFastLess::operator()(Lhs.ToSoftObjectPath(), Rhs.ToSoftObjectPath());
	}
};

/** Slow alphabetical order that is stable / deterministic over process runs. */
struct FSoftObjectPtrLexicalLess : private FSoftObjectPathLexicalLess
{
	template <typename SoftObjectPtrType>
	bool operator()(const SoftObjectPtrType& Lhs, const SoftObjectPtrType& Rhs) const
	{
		return FSoftObjectPathLexicalLess::operator()(Lhs.ToSoftObjectPath(), Rhs.ToSoftObjectPath());
	}
};

/** Not directly inlined on purpose so compiler have the option of not inlining it. (and it also works with extern template) */
template<class T>
T* TSoftObjectPtr<T>::Get() const
{
	return dynamic_cast<T*>(SoftObjectPtr.Get());
}

#if !PLATFORM_COMPILER_HAS_GENERATED_COMPARISON_OPERATORS
template<class TClass>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator==(const UClass* Lhs, const TSoftClassPtr<TClass>& Rhs)
{
	return Rhs == Lhs;
}

template<class TClass>
[[nodiscard]] UE_FORCEINLINE_HINT bool operator!=(const UClass* Lhs, const TSoftClassPtr<TClass>& Rhs)
{
	return Rhs != Lhs;
}
#endif

/** Hash function */
template<class TClass>
[[nodiscard]] UE_FORCEINLINE_HINT uint32 GetTypeHash(const TSoftClassPtr<TClass>& Ptr)
{
	return Ptr.GetPtrTypeHash();
}

template<class TClass>
FArchive& operator<<(FArchive& Ar, TSoftClassPtr<TClass>& Ptr)
{
	Ptr.Serialize(Ar);
	return Ar;
}
