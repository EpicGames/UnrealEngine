// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/FunctionFwd.h"
#include "HAL/Platform.h" // for 'should be preceded by' error related to API decorator

class UClass;
class FProperty;

namespace UE::Reflection
{
	/**
	 * Returns true if the provided class has sparse class data and that sparse class data differs from its 
	 * super class's sparse class data.
	 *
	 * @param	Class	The class to inspect - its SparseClassData will be compared against its SuperClass's SparseClassData.
	 * @param	Filter	A filter that returns true if the given property should be compared. 
						Typically Filter is used to skip transient properties. If an FArchive is available consider 
						honoring FProperty::ShouldSerializeValue
	 */
	COREUOBJECT_API bool DoesSparseClassDataOverrideArchetype(const UClass* Class, const TFunctionRef<bool(FProperty*)>& Filter);
}
