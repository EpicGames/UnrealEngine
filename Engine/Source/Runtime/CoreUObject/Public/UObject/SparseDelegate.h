// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/SparseArray.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptDelegateFwd.h"
#include "UObject/UObjectArray.h"
#include "UObject/WeakObjectPtr.h"

class FOutputDevice;
class FString;
class UObjectBase;
class UWorld;
struct FSparseDelegate;

/**
*  Sparse delegates can be used for infrequently bound dynamic delegates so that the object uses only 
*  1 byte of storage instead of having the full overhead of the delegate invocation list.
*  The cost to invoke, add, remove, etc. from the delegate is higher than using the delegate
*  directly and thus the memory savings benefit should be traded off against the frequency
*  with which you would expect the delegate to be bound.
*/


/** Helper class for handling sparse delegate bindings */
struct FSparseDelegateStorage
{
public:

	/** Registers the sparse delegate so that the offset can be determined. */
	static COREUOBJECT_API void RegisterDelegateOffset(const UObject* OwningObject, FName DelegateName, size_t OffsetToOwner);

	/** Binds a sparse delegate to the owner. Returns if the delegate was successfully bound. */
	static COREUOBJECT_API bool Add(const UObject* DelegateOwner, const FName DelegateName, FScriptDelegate Delegate);

	/** Binds a sparse delegate to the owner, verifying first that the delegate is not already bound. Returns if the delegate was successfully bound. */
	static COREUOBJECT_API bool AddUnique(const UObject* DelegateOwner, const FName DelegateName, FScriptDelegate Delegate);

	/** Returns whether a sparse delegate is bound to the owner. */
	static COREUOBJECT_API bool Contains(const UObject* DelegateOwner, const FName DelegateName, const FScriptDelegate& Delegate);
	static COREUOBJECT_API bool Contains(const UObject* DelegateOwner, const FName DelegateName, const UObject* InObject, FName InFunctionName);

	/** Removes a delegate binding from the owner's sparse delegate storage. Returns true if there are still bindings to the delegate. */
	static COREUOBJECT_API bool Remove(const UObject* DelegateOwner, const FName DelegateName, const FScriptDelegate& Delegate);
	static COREUOBJECT_API bool Remove(const UObject* DelegateOwner, const FName DelegateName, const UObject* InObject, FName InFunctionName);

	/** Removes all sparse delegate binding from the owner for a given object. Returns true if there are still bindings to the delegate. */
	static COREUOBJECT_API bool RemoveAll(const UObject* DelegateOwner, const FName DelegateName, const UObject* UserObject);

	/** Clear all of the named sparse delegate bindings from the owner. */
	static COREUOBJECT_API void Clear(const UObject* DelegateOwner, const FName DelegateName);

	/** Acquires the actual Multicast Delegate from the annotation if any delegates are bound to it. Will be null if no entry exists in the annotation for this object/delegatename. */
	static COREUOBJECT_API FMulticastScriptDelegate* GetMulticastDelegate(const UObject* DelegateOwner, const FName DelegateName);

	/** Acquires the actual Multicast Delegate from the annotation if any delegates are bound to it as a shared pointer. Will be null if no entry exists in the annotation for this object/delegatename. */
	static COREUOBJECT_API TSharedPtr<FMulticastScriptDelegate> GetSharedMulticastDelegate(const UObject* DelegateOwner, const FName DelegateName);

	/** Directly sets the Multicast Delegate for this object/delegatename pair. If the delegate is unbound it will be assigned/inserted anyways. */
	static COREUOBJECT_API void SetMulticastDelegate(const UObject* DelegateOwner, const FName DelegateName, FMulticastScriptDelegate Delegate);

	/** Using the registry of sparse delegates recover the FSparseDelegate address from the UObject and name. */
	static COREUOBJECT_API FSparseDelegate* ResolveSparseDelegate(const UObject* OwningObject, FName DelegateName);

	/** Using the registry of sparse delegates recover the UObject owner from the FSparseDelegate address owning class and delegate names. */
	static COREUOBJECT_API UObject* ResolveSparseOwner(const FSparseDelegate& SparseDelegate, const FName OwningClassName, const FName DelegateName);

	/** Outputs a report about which delegates are bound. */
	static COREUOBJECT_API void SparseDelegateReport(const TArray<FString>&, UWorld*, FOutputDevice&);

private:

	struct FObjectListener : public FUObjectArray::FUObjectDeleteListener
	{
		virtual ~FObjectListener();
		virtual void NotifyUObjectDeleted(const UObjectBase* Object, int32 Index) override;
		virtual void OnUObjectArrayShutdown();
		void EnableListener();
		void DisableListener();

		virtual SIZE_T GetAllocatedSize() const override
		{
			return 0;
		}
	};

	/** Allow the object listener to use the critical section and remove objects from the map */
	friend struct FObjectListener;

	/** A listener to get notified when objects have been deleted and remove them from the map */
	static COREUOBJECT_API FObjectListener SparseDelegateObjectListener;

	/** Critical Section for locking access to the sparse delegate map */
	static COREUOBJECT_API FTransactionallySafeCriticalSection SparseDelegateMapCritical;

	/** Delegate map is a map of Delegate names to a shared pointer of the multicast script delegate */
	typedef TMap<FName, TSharedPtr<FMulticastScriptDelegate>> FSparseDelegateMap;

	/** Map of objects to the map of delegates that are bound to that object */
	static COREUOBJECT_API TMap<const UObjectBase*, FSparseDelegateMap> SparseDelegates;
	
	/** Sparse delegate offsets are indexed by ActorClass/DelegateName pair */
	static COREUOBJECT_API TMap<TPair<FName, FName>, size_t> SparseDelegateObjectOffsets;
};

/** Base implementation for all sparse delegate types */
struct FSparseDelegate
{
public:
	FSparseDelegate()
		: bIsBound(false)
	{
	}

	/**
	* Checks to see if any functions are bound to this multi-cast delegate
	*
	* @return	True if any functions are bound
	*/
	bool IsBound() const
	{
		return bIsBound;
	}

	/**
	* Adds a function delegate to this multi-cast delegate's invocation list if a delegate with the same signature
	* doesn't already exist in the invocation list
	*
	* @param	DelegateOwner	UObject that owns the resolved sparse delegate
	* @param	DelegateName	Name of the resolved sparse delegate
	* @param	InDelegate		Delegate to bind to the sparse delegate
	* 
	* NOTE:  Only call this function from blueprint sparse delegate infrastructure on a resolved generic FScriptDelegate pointer.
	*        Generally from C++ you should call AddUnique() directly.
	*/
	void __Internal_AddUnique(const UObject* DelegateOwner, FName DelegateName, FScriptDelegate InDelegate)
	{
		bIsBound |= FSparseDelegateStorage::AddUnique(DelegateOwner, DelegateName, MoveTemp(InDelegate));
	}

	/**
	* Removes a function from this multi-cast delegate's invocation list (performance is O(N)).  Note that the
	* order of the delegates may not be preserved!
	*
	* @param	DelegateOwner	UObject that owns the resolved sparse delegate
	* @param	DelegateName	Name of the resolved sparse delegate
	* @param	InDelegate		Delegate to remove from the sparse delegate
	*
	* NOTE:  Only call this function from blueprint sparse delegate infrastructure on a resolved generic FScriptDelegate pointer.
	*        Generally from C++ you should call Remove() directly.
	*/
	void __Internal_Remove(const UObject* DelegateOwner, FName DelegateName, const FScriptDelegate& InDelegate)
	{
		if (bIsBound)
		{
			bIsBound = FSparseDelegateStorage::Remove(DelegateOwner, DelegateName, InDelegate);
		}
	}

	/**
	* Removes all functions from this delegate's invocation list
	*
	* @param	DelegateOwner	UObject that owns the resolved sparse delegate
	* @param	DelegateName	Name of the resolved sparse delegate
	*
	* NOTE:  Only call this function from blueprint sparse delegate infrastructure on a resolved generic FScriptDelegate pointer.
	*        Generally from C++ you should call Clear() directly.
	*/
	void __Internal_Clear(const UObject* DelegateOwner, FName DelegateName)
	{
		if (bIsBound)
		{
			FSparseDelegateStorage::Clear(DelegateOwner, DelegateName);
			bIsBound = false;
		}
	}

protected:

	friend class FMulticastSparseDelegateProperty;
	bool bIsBound;
};

/** Sparse version of TDynamicDelegate */
template <typename MulticastDelegate, typename OwningClass, typename DelegateInfoClass>
struct TSparseDynamicDelegate : public FSparseDelegate
{
public:
	typedef typename MulticastDelegate::FDelegate FDelegate; 

protected:
	FName GetDelegateName() const
	{
		static const FName DelegateFName(DelegateInfoClass::GetDelegateName());
		return DelegateFName;
	}

private:
	UObject* GetDelegateOwner() const
	{
		const size_t OffsetToOwner = DelegateInfoClass::template GetDelegateOffset<OwningClass>();
		check(OffsetToOwner);
		UObject* DelegateOwner = reinterpret_cast<UObject*>((uint8*)this - OffsetToOwner);
		check(DelegateOwner->IsValidLowLevelFast(false)); // Most likely the delegate is trying to be used on the stack, in an object it wasn't defined for, or for a class member with a different name than it was defined for. It is only valid for a sparse delegate to be used for the exact class/property name it is defined with.
		return DelegateOwner;
	}

public:
	/** Returns the multicast delegate if any delegates are bound to the sparse delegate */
	TSharedPtr<MulticastDelegate> GetShared() const
	{
		TSharedPtr<MulticastDelegate> Result;
		if (bIsBound)
		{
			Result = StaticCastSharedPtr<MulticastDelegate>(FSparseDelegateStorage::GetSharedMulticastDelegate(GetDelegateOwner(), GetDelegateName()));
		}
		return Result;
	}

	/**
	* Checks whether a function delegate is already a member of this multi-cast delegate's invocation list
	*
	* @param	InDelegate	Delegate to check
	* @return	True if the delegate is already in the list.
	*/
	bool Contains(const FScriptDelegate& InDelegate) const
	{
		return (bIsBound ? FSparseDelegateStorage::Contains(GetDelegateOwner(), GetDelegateName(), InDelegate) : false);
	}

	/**
	* Checks whether a function delegate is already a member of this multi-cast delegate's invocation list
	*
	* @param	InObject		Object of the delegate to check
	* @param	InFunctionName	Function name of the delegate to check
	* @return	True if the delegate is already in the list.
	*/
	bool Contains(const UObject* InObject, FName InFunctionName) const
	{
		return (bIsBound ? FSparseDelegateStorage::Contains(GetDelegateOwner(), GetDelegateName(), InObject, InFunctionName) : false);
	}

	/**
	* Adds a function delegate to this multi-cast delegate's invocation list
	*
	* @param	InDelegate	Delegate to add
	*/
	void Add(FScriptDelegate InDelegate)
	{
		bIsBound |= FSparseDelegateStorage::Add(GetDelegateOwner(), GetDelegateName(), MoveTemp(InDelegate));
	}

	/**
	* Adds a function delegate to this multi-cast delegate's invocation list if a delegate with the same signature
	* doesn't already exist in the invocation list
	*
	* @param	InDelegate	Delegate to add
	*/
	void AddUnique(FScriptDelegate InDelegate)
	{
		FSparseDelegate::__Internal_AddUnique(GetDelegateOwner(), GetDelegateName(), MoveTemp(InDelegate));
	}

	/**
	* Removes a function from this multi-cast delegate's invocation list (performance is O(N)).  Note that the
	* order of the delegates may not be preserved!
	*
	* @param	InDelegate	Delegate to remove
	*/
	void Remove(const FScriptDelegate& InDelegate)
	{
		FSparseDelegate::__Internal_Remove(GetDelegateOwner(), GetDelegateName(), InDelegate);
	}

	/**
	* Removes a function from this multi-cast delegate's invocation list (performance is O(N)).  Note that the
	* order of the delegates may not be preserved!
	*
	* @param	InObject		Object of the delegate to remove
	* @param	InFunctionName	Function name of the delegate to remove
	*/
	void Remove(const UObject* InObject, FName InFunctionName)
	{
		if (bIsBound)
		{
			bIsBound = FSparseDelegateStorage::Remove(GetDelegateOwner(), GetDelegateName(), InObject, InFunctionName);
		}
	}

	/**
	* Removes all delegate bindings from this multicast delegate's
	* invocation list that are bound to the specified object.
	*
	* This method also compacts the invocation list.
	*
	* @param InObject The object to remove bindings for.
	*/
	void RemoveAll(const UObject* Object)
	{
		if (bIsBound)
		{
			bIsBound = FSparseDelegateStorage::RemoveAll(GetDelegateOwner(), GetDelegateName(), Object);
		}
	}

	/**
	* Removes all functions from this delegate's invocation list
	*/
	void Clear()
	{
		FSparseDelegate::__Internal_Clear(GetDelegateOwner(), GetDelegateName());
	}
	
	/**
	* Broadcasts this delegate to all bound objects, except to those that may have expired.
	*/
	template<typename... ParamTypes>
	void Broadcast(ParamTypes... Params)
	{
		if (TSharedPtr<MulticastDelegate> MCDelegate = GetShared())
		{
			MCDelegate->Broadcast(Params...);
		}
	}

private:
	template <typename UserClass>
	bool IsAlreadyBoundImpl(const UserClass* InUserObject, FName InFunctionName) const
	{
		check(InUserObject);

		return this->Contains(InUserObject, InFunctionName);
	}

public:
	/**
	* Tests if a UObject instance and a UObject method address pair are already bound to this multi-cast delegate.
	*
	* @param	InUserObject		UObject instance
	* @param	InMethodPtr			Member function address pointer
	* @param	InFunctionName		Name of member function, without class name
	* @return	True if the instance/method is already bound.
	*
	* NOTE:  Do not call this function directly.  Instead, call IsAlreadyBound() which is a macro proxy function that
	*        automatically sets the function name string for the caller.
	*
	* NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	*/
	template <typename UserClass>
	UE_REWRITE bool __Internal_IsAlreadyBound(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName) const
	{
		return this->IsAlreadyBoundImpl(InUserObject, InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE bool __Internal_IsAlreadyBound(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName) const
	{
		return this->IsAlreadyBoundImpl(ToRawPtr(InUserObject), InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE bool __Internal_IsAlreadyBound(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName) const
	{
		return this->IsAlreadyBoundImpl(InUserObject, InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE bool __Internal_IsAlreadyBound(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName) const
	{
		return this->IsAlreadyBoundImpl(ToRawPtr(InUserObject), InFunctionName);
	}

private:
	template <typename UserClass>
	void AddDynamicImpl(const UserClass* InUserObject, FName InFunctionName)
	{
		check(InUserObject);

		FDelegate NewDelegate;
		NewDelegate.BindDynamicImpl(InUserObject, InFunctionName);

		this->Add(NewDelegate);
	}

public:
	/**
	* Binds a UObject instance and a UObject method address to this multi-cast delegate.
	*
	* @param	InUserObject		UObject instance
	* @param	InMethodPtr			Member function address pointer
	* @param	InFunctionName		Name of member function, without class name
	*
	* NOTE:  Do not call this function directly.  Instead, call AddDynamic() which is a macro proxy function that
	*        automatically sets the function name string for the caller.
	*
	* NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	*/
	template <typename UserClass>
	UE_REWRITE void __Internal_AddDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->AddDynamicImpl(InUserObject, InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_AddDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->AddDynamicImpl(ToRawPtr(InUserObject), InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_AddDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		this->AddDynamicImpl(InUserObject, InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_AddDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		this->AddDynamicImpl(ToRawPtr(InUserObject), InFunctionName);
	}

private:
	template <typename UserClass>
	void AddUniqueDynamicImpl(const UserClass* InUserObject, FName InFunctionName)
	{
		check(InUserObject);

		FDelegate NewDelegate;
		NewDelegate.BindDynamicImpl(InUserObject, InFunctionName);

		this->AddUnique(NewDelegate);
	}

public:
	/**
	* Binds a UObject instance and a UObject method address to this multi-cast delegate, but only if it hasn't been bound before.
	*
	* @param	InUserObject		UObject instance
	* @param	InMethodPtr			Member function address pointer
	* @param	InFunctionName		Name of member function, without class name
	*
	* NOTE:  Do not call this function directly.  Instead, call AddUniqueDynamic() which is a macro proxy function that
	*        automatically sets the function name string for the caller.
	*
	* NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	*/
	template <typename UserClass>
	UE_REWRITE void __Internal_AddUniqueDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->AddUniqueDynamicImpl(InUserObject, InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_AddUniqueDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		UE_STATIC_DEPRECATE(5.8, std::is_const_v<UserClass>, "Binding a delegate with a const object pointer and non-const function is deprecated.");
		this->AddUniqueDynamicImpl(ToRawPtr(InUserObject), InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_AddUniqueDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		this->AddUniqueDynamicImpl(InUserObject, InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_AddUniqueDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		this->AddUniqueDynamicImpl(ToRawPtr(InUserObject), InFunctionName);
	}

private:
	template <typename UserClass>
	void RemoveDynamicImpl(const UserClass* InUserObject, FName InFunctionName)
	{
		check(InUserObject);

		this->Remove(InUserObject, InFunctionName);
	}

public:
	/**
	* Unbinds a UObject instance and a UObject method address from this multi-cast delegate.
	*
	* @param	InUserObject		UObject instance
	* @param	InMethodPtr			Member function address pointer
	* @param	InFunctionName		Name of member function, without class name
	*
	* NOTE:  Do not call this function directly.  Instead, call RemoveDynamic() which is a macro proxy function that
	*        automatically sets the function name string for the caller.
	*
	* NOTE:  We're not actually using the incoming method pointer.  We simply require it for type-safety reasons.
	*/
	template <typename UserClass>
	UE_REWRITE void __Internal_RemoveDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		this->RemoveDynamicImpl(InUserObject, InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_RemoveDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<false, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		this->RemoveDynamicImpl(ToRawPtr(InUserObject), InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_RemoveDynamic(UserClass* InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		this->RemoveDynamicImpl(InUserObject, InFunctionName);
	}
	template <typename UserClass>
	UE_REWRITE void __Internal_RemoveDynamic(TObjectPtr<UserClass> InUserObject, typename FDelegate::template TMethodPtrResolver<true, UserClass>::FMethodPtr InMethodPtr, FName InFunctionName)
	{
		this->RemoveDynamicImpl(ToRawPtr(InUserObject), InFunctionName);
	}

private:
	// Hide internal functions that never need to be called on the derived classes
	void __Internal_AddUnique(const UObject*, FName, FScriptDelegate);
	void __Internal_Remove(const UObject*, FName, const FScriptDelegate&);
	void __Internal_Clear(const UObject*, FName);
};

// This macro was an implementation detail and should not be used by user code.  The DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE*() macros below should be used instead.
#define FUNC_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClassName, OwningClass, DelegateName, FuncType) UE_DEPRECATED_MACRO(5.8, "FUNC_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE has been deprecated.") UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClassName, OwningClass, DelegateName, FuncType)

#define UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClassName, OwningClass, DelegateName, FuncType) \
	UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_DELEGATE(SparseDelegateClassName##_MCSignature, FuncType) \
	struct SparseDelegateClassName##InfoGetter \
	{ \
		static const char* GetDelegateName() { return #DelegateName; } \
		template<typename T> \
		static size_t GetDelegateOffset() { return offsetof(T, DelegateName); } \
	}; \
	struct SparseDelegateClassName : public TSparseDynamicDelegate<SparseDelegateClassName##_MCSignature, OwningClass, SparseDelegateClassName##InfoGetter> \
	{ \
	};

/** Declares a sparse blueprint-accessible broadcast delegate that can bind to multiple native UFUNCTIONs simultaneously */
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void())

#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_OneParam(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name))
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_TwoParams(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name, Param2Type Param2Name))
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_ThreeParams(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name))
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FourParams(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name))
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_FiveParams(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name))
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_SixParams(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name))
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_SevenParams(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name))
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_EightParams(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name, Param8Type, Param8Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name, Param8Type Param8Name))
#define DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE_NineParams(SparseDelegateClass, OwningClass, DelegateName, Param1Type, Param1Name, Param2Type, Param2Name, Param3Type, Param3Name, Param4Type, Param4Name, Param5Type, Param5Name, Param6Type, Param6Name, Param7Type, Param7Name, Param8Type, Param8Name, Param9Type, Param9Name) UE_PRIVATE_DECLARE_DYNAMIC_MULTICAST_SPARSE_DELEGATE(SparseDelegateClass, OwningClass, DelegateName, void(Param1Type Param1Name, Param2Type Param2Name, Param3Type Param3Name, Param4Type Param4Name, Param5Type Param5Name, Param6Type Param6Name, Param7Type Param7Name, Param8Type Param8Name, Param9Type Param9Name))
