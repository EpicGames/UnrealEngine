// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	Stack.h: Kismet VM execution stack definition.
=============================================================================*/

#pragma once

#include "UObject/Script.h"
#include "Misc/CoreMisc.h"
#include "Templates/Casts.h"
#include "UObject/UnrealType.h"
#include "UObject/EnumProperty.h"
#include "Memory/VirtualStackAllocator.h"

DECLARE_LOG_CATEGORY_EXTERN(LogScriptFrame, Warning, All);

#ifndef UE_USE_VIRTUAL_STACK_ALLOCATOR_FOR_SCRIPT_VM
#define UE_USE_VIRTUAL_STACK_ALLOCATOR_FOR_SCRIPT_VM 0
#endif 

#if UE_USE_VIRTUAL_STACK_ALLOCATOR_FOR_SCRIPT_VM
FORCEINLINE void* UeVstackAllocHelper(FVirtualStackAllocator* Allocator, size_t Size, size_t Align)
{
	if (Size == 0)
	{
		return nullptr;	
	}
	if (Align < 16)
	{
		Align = 16;
	}
	return Allocator->Allocate(Size, Align);
}

#define UE_VSTACK_MAKE_FRAME(Name, VirtualStackAllocatorPtr) FScopedStackAllocatorBookmark Name = (VirtualStackAllocatorPtr)->CreateScopedBookmark()
#define UE_VSTACK_ALLOC(VirtualStackAllocatorPtr, Size) UeVstackAllocHelper((VirtualStackAllocatorPtr), (Size), 0) 
#define UE_VSTACK_ALLOC_ALIGNED(VirtualStackAllocatorPtr, Size, Align) UeVstackAllocHelper((VirtualStackAllocatorPtr), (Size), (Align))
#else
#define UE_VSTACK_MAKE_FRAME(Name, VirtualStackAllocatorPtr)
#define UE_VSTACK_ALLOC(VirtualStackAllocatorPtr, Size) FMemory_Alloca(Size)
#define UE_VSTACK_ALLOC_ALIGNED(VirtualStackAllocatorPtr, Size, Align) FMemory_Alloca_Aligned(Size, Align)
#endif

/**
 * Property data type enums.
 * @warning: if values in this enum are modified, you must update:
 * - FPropertyBase::GetSize() hardcodes the sizes for each property type
 */
enum EPropertyType
{
	CPT_None,
	CPT_Byte,
	CPT_UInt16,
	CPT_UInt32,
	CPT_UInt64,
	CPT_Int8,
	CPT_Int16,
	CPT_Int,
	CPT_Int64,
	CPT_Bool,
	CPT_Bool8,
	CPT_Bool16,
	CPT_Bool32,
	CPT_Bool64,
	CPT_Float,
	CPT_ObjectReference,
	CPT_Name,
	CPT_Delegate,
	CPT_Interface,
	CPT_Unused_Index_19,
	CPT_Struct,
	CPT_Unused_Index_21,
	CPT_Unused_Index_22,
	CPT_String,
	CPT_Text,
	CPT_MulticastDelegate,
	CPT_WeakObjectReference,
	CPT_LazyObjectReference,
	CPT_ObjectPtrReference,
	CPT_SoftObjectReference,
	CPT_Double,
	CPT_Map,
	CPT_Set,
	CPT_FieldPath,
	CPT_FLargeWorldCoordinatesReal,

	CPT_MAX
};



/*-----------------------------------------------------------------------------
	Execution stack helpers.
-----------------------------------------------------------------------------*/

typedef TArray< CodeSkipSizeType, TInlineAllocator<8> > FlowStackType;

//
// Information remembered about an Out parameter.
//
struct FOutParmRec
{
	FProperty* Property;
	uint8*      PropAddr;
	FOutParmRec* NextOutParm;
};

//
// Information about script execution at one stack level.
//

struct FFrame : public FOutputDevice
{	
public:
	// Variables.
	UFunction* Node;
	UObject* Object;
	uint8* Code;
	uint8* Locals;

	FProperty* MostRecentProperty;
	uint8* MostRecentPropertyAddress;
	uint8* MostRecentPropertyContainer;

	/** The execution flow stack for compiled Kismet code */
	FlowStackType FlowStack;

	/** Previous frame on the stack */
	FFrame* PreviousFrame;

	/** contains information on any out parameters */
	FOutParmRec* OutParms;

	/** If a class is compiled in then this is set to the property chain for compiled-in functions. In that case, we follow the links to setup the args instead of executing by code. */
	FField* PropertyChainForCompiledIn;

	/** Currently executed native function */
	UFunction* CurrentNativeFunction;

#if UE_USE_VIRTUAL_STACK_ALLOCATOR_FOR_SCRIPT_VM
	FVirtualStackAllocator* CachedThreadVirtualStackAllocator;
#endif

	/** Previous tracking frame */
	FFrame* PreviousTrackingFrame;

	bool bArrayContextFailed;

	/** If this flag gets set (usually from throwing a EBlueprintExceptionType::AbortExecution exception), execution shall immediately stop and return */
	bool bAbortingExecution;

#if PER_FUNCTION_SCRIPT_STATS
	/** Increment for each PreviousFrame on the stack (Max 255) */
	uint8 DepthCounter;
#endif
public:

	// Constructors.
	FFrame( UObject* InObject, UFunction* InNode, void* InLocals, FFrame* InPreviousFrame = NULL, FField* InPropertyChainForCompiledIn = NULL );

	virtual ~FFrame()
	{
#if DO_BLUEPRINT_GUARD
		FBlueprintContextTracker& BlueprintExceptionTracker = FBlueprintContextTracker::Get();
		if (BlueprintExceptionTracker.ScriptStack.Num())
		{
			BlueprintExceptionTracker.ScriptStack.Pop(EAllowShrinking::No);
		}

		// ensure that GTopTrackingStackFrame is accurate
		if (BlueprintExceptionTracker.ScriptStack.Num() == 0)
		{
			ensure(PreviousTrackingFrame == nullptr);
		}
		else
		{
			ensure(BlueprintExceptionTracker.ScriptStack.Last() == PreviousTrackingFrame);
		}
#endif
		PopThreadLocalTopStackFrame(PreviousTrackingFrame);
		
		if (PreviousTrackingFrame)
		{
			// we propagate bAbortingExecution to frames below to avoid losing abort state
			// across heterogeneous frames (eg. bpvm -> c++ -> bpvm)
			PreviousTrackingFrame->bAbortingExecution |= bAbortingExecution;
		}
	}

	// Functions.
	COREUOBJECT_API void Step(UObject* Context, RESULT_DECL);

	/** Convenience function that calls Step, but also returns true if both MostRecentProperty and MostRecentPropertyAddress are non-null. */
	FORCEINLINE_DEBUGGABLE bool StepAndCheckMostRecentProperty(UObject* Context, RESULT_DECL);

	/** Replacement for Step that uses an explicitly specified property to unpack arguments **/
	COREUOBJECT_API void StepExplicitProperty(void*const Result, FProperty* Property);

	/** Replacement for Step that checks the for byte code, and if none exists, then PropertyChainForCompiledIn is used. Also, makes an effort to verify that the params are in the correct order and the types are compatible. **/
	template<class TProperty>
	FORCEINLINE_DEBUGGABLE void StepCompiledIn(void* Result);
	FORCEINLINE_DEBUGGABLE void StepCompiledIn(void* Result, const FFieldClass* ExpectedPropertyType);

	/** Replacement for Step that checks the for byte code, and if none exists, then PropertyChainForCompiledIn is used. Also, makes an effort to verify that the params are in the correct order and the types are compatible. **/
	template<class TProperty, typename TNativeType>
	FORCEINLINE_DEBUGGABLE TNativeType& StepCompiledInRef(void*const TemporaryBuffer);

	COREUOBJECT_API virtual void Serialize( const TCHAR* V, ELogVerbosity::Type Verbosity, const class FName& Category ) override;
	
	COREUOBJECT_API static void KismetExecutionMessage(const TCHAR* Message, ELogVerbosity::Type Verbosity, FName WarningId = FName());

	/** Returns the current script op code */
	const uint8 PeekCode() const { return *Code; }

	/** Skips over the number of op codes specified by NumOps */
	void SkipCode(const int32 NumOps) { Code += NumOps; }

	template<typename T>
	T Read();
	template<typename TNumericType>
	TNumericType ReadInt();
	float ReadFloat();
	double ReadDouble();
	ScriptPointerType ReadPointer();
	FName ReadName();
	UObject* ReadObject();
	int32 ReadWord();
	FProperty* ReadProperty();

	/** May return null */
	FProperty* ReadPropertyUnchecked();

	/**
	 * Reads a value from the bytestream, which represents the number of bytes to advance
	 * the code pointer for certain expressions.
	 *
	 * @param	ExpressionField		receives a pointer to the field representing the expression; used by various execs
	 *								to drive VM logic
	 */
	CodeSkipSizeType ReadCodeSkipCount();

	/**
	 * Reads a value from the bytestream which represents the number of bytes that should be zero'd out if a NULL context
	 * is encountered
	 *
	 * @param	ExpressionField		receives a pointer to the field representing the expression; used by various execs
	 *								to drive VM logic
	 */
	VariableSizeType ReadVariableSize(FProperty** ExpressionField);

	/**
 	 * This will return the StackTrace of the current callstack from the last native entry point
	 **/
	COREUOBJECT_API FString GetStackTrace() const;

	/**
	 * This will return the StackTrace of the current callstack from the last native entry point
	 * 
	 * @param StringBuilder to populate
	 **/
	COREUOBJECT_API void GetStackTrace(FStringBuilderBase& StringBuilder) const;

	/**
	* This will return the StackTrace of the all script frames currently active
	* 
	* @param	bReturnEmpty if true, returns empty string when no script callstack found
	* @param	bTopOfStackOnly if true only returns the top of the callstack
	**/
	COREUOBJECT_API static FString GetScriptCallstack(bool bReturnEmpty = false, bool bTopOfStackOnly = false);

	/**
	* This will return the StackTrace of the all script frames currently active
	*
	* @param	StringBuilder to populate
	* @param	bReturnEmpty if true, returns empty string when no script callstack found
	* @param	bTopOfStackOnly if true only returns the top of the callstack
	**/
	COREUOBJECT_API static void GetScriptCallstack(FStringBuilderBase& StringBuilder, bool bReturnEmpty = false, bool bTopOfStackOnly = false);
		
	/**
	* This will append a string of the form "ScopeName.FunctionName" associated with this stack frame
	*
	* @param	StringBuilder to populate
	**/
	COREUOBJECT_API void GetStackDescription(FStringBuilderBase& StringBuilder) const;

#if DO_BLUEPRINT_GUARD
	static void InitPrintScriptCallstack();
#endif

	COREUOBJECT_API static FFrame* PushThreadLocalTopStackFrame(FFrame* NewTopStackFrame);
	COREUOBJECT_API static void PopThreadLocalTopStackFrame(FFrame* NewTopStackFrame);
	COREUOBJECT_API static FFrame* GetThreadLocalTopStackFrame();
};


/*-----------------------------------------------------------------------------
	FFrame implementation.
-----------------------------------------------------------------------------*/

inline FFrame::FFrame( UObject* InObject, UFunction* InNode, void* InLocals, FFrame* InPreviousFrame, FField* InPropertyChainForCompiledIn )
	: Node(InNode)
	, Object(InObject)
	, Code(InNode->Script.GetData())
	, Locals((uint8*)InLocals)
	, MostRecentProperty(nullptr)
	, MostRecentPropertyAddress(nullptr)
	, MostRecentPropertyContainer(nullptr)
	, PreviousFrame(InPreviousFrame)
	, OutParms(NULL)
	, PropertyChainForCompiledIn(InPropertyChainForCompiledIn)
	, CurrentNativeFunction(nullptr)
	, bArrayContextFailed(false)
	, bAbortingExecution(false)
#if PER_FUNCTION_SCRIPT_STATS
	, DepthCounter(0)
#endif
{
#if DO_BLUEPRINT_GUARD
	FBlueprintContextTracker::Get().ScriptStack.Push(this);
#endif
	PreviousTrackingFrame = PushThreadLocalTopStackFrame(this);
	
	{
		// we propagate bAbortingExecution to *upper* frames to avoid invoking code
		// on top of already-aborted frames
		if (PreviousTrackingFrame)
		{
			bAbortingExecution |= PreviousTrackingFrame->bAbortingExecution;
		}
		if (InPreviousFrame)
		{
			bAbortingExecution |= InPreviousFrame->bAbortingExecution;
		}
	}

#if PER_FUNCTION_SCRIPT_STATS
	if (InPreviousFrame)
	{
		DepthCounter = (InPreviousFrame->DepthCounter < MAX_uint8) ? InPreviousFrame->DepthCounter + 1 : MAX_uint8;
	}
#endif
#if UE_USE_VIRTUAL_STACK_ALLOCATOR_FOR_SCRIPT_VM
	if (InPreviousFrame == nullptr)
	{
		CachedThreadVirtualStackAllocator = FBlueprintContext::GetThreadSingleton()->GetVirtualStackAllocator();
	}
	else
	{
		CachedThreadVirtualStackAllocator = InPreviousFrame->CachedThreadVirtualStackAllocator;
	}
#endif
}

template<typename T>
inline T FFrame::Read()
{
	T Result = FPlatformMemory::ReadUnaligned<T>(Code);
	Code += sizeof(T);
	return Result;
}

template<typename TNumericType>
inline TNumericType FFrame::ReadInt()
{
	return Read<TNumericType>();
}

inline UObject* FFrame::ReadObject()
{
	UObject* Result = (UObject*) ReadPointer();

#if UE_WITH_OBJECT_HANDLE_LATE_RESOLVE || UE_WITH_REMOTE_OBJECT_HANDLE
	TObjectPtr<UObject> ObjPtr(Result);
	return ObjPtr.Get();
#else
	return Result;
#endif
}

inline FProperty* FFrame::ReadProperty()
{
	FProperty* Result = ReadPropertyUnchecked();

	// Callers don't check for NULL; this method is expected to succeed.
	check(Result);

	return Result;
}

inline FProperty* FFrame::ReadPropertyUnchecked()
{
	FProperty* Result = (FProperty*)ReadPointer();
	MostRecentProperty = Result;
	return Result;
}

inline float FFrame::ReadFloat()
{
	return Read<float>();
}

inline double FFrame::ReadDouble()
{
	return Read<double>();
}

inline ScriptPointerType FFrame::ReadPointer()
{
	// Serialized pointers are always the size of ScriptPointerType 
	ScriptPointerType Pointer = FPlatformMemory::ReadUnaligned<ScriptPointerType>(Code);

	Code += sizeof(ScriptPointerType);
	return Pointer;
}

inline int32 FFrame::ReadWord()
{
	return Read<uint16>();
}

/**
 * Reads a value from the bytestream, which represents the number of bytes to advance
 * the code pointer for certain expressions.
 */
inline CodeSkipSizeType FFrame::ReadCodeSkipCount()
{
	CodeSkipSizeType Result = FPlatformMemory::ReadUnaligned<CodeSkipSizeType>(Code);
	Code += sizeof(CodeSkipSizeType);
	return Result;
}

inline VariableSizeType FFrame::ReadVariableSize( FProperty** ExpressionField )
{
	VariableSizeType Result=0;

	FField* Field = (FField*)ReadPropertyUnchecked(); // Is it safe to assume it's an FField?
	FProperty* Property = CastField<FProperty>(Field);
	if (Property)
	{
		Result = (VariableSizeType)Property->GetSize();
	}

	if (ExpressionField != nullptr)
	{
		*ExpressionField = Property;
	}

	return Result;
}

inline FName FFrame::ReadName()
{
	FScriptName Result = FPlatformMemory::ReadUnaligned<FScriptName>(Code);
	Code += sizeof(FScriptName);
	return ScriptNameToName(Result);
}

COREUOBJECT_API void GInitRunaway();

FORCEINLINE_DEBUGGABLE bool FFrame::StepAndCheckMostRecentProperty(UObject* Context, RESULT_DECL)
{
	Step(Context, RESULT_PARAM);

	return (MostRecentProperty && MostRecentPropertyAddress);
}

/**
 * Replacement for Step that checks the for byte code, and if none exists, then PropertyChainForCompiledIn is used.
 * Also makes an effort to verify that the params are in the correct order and the types are compatible.
 **/
template<class TProperty>
FORCEINLINE_DEBUGGABLE void FFrame::StepCompiledIn(void* Result)
{
	StepCompiledIn(Result, TProperty::StaticClass());
}

FORCEINLINE_DEBUGGABLE void FFrame::StepCompiledIn(void* Result, const FFieldClass* ExpectedPropertyType)
{
	if (Code)
	{
		Step(Object, Result);
	}
	else
	{
		checkSlow(ExpectedPropertyType && ExpectedPropertyType->IsChildOf(FProperty::StaticClass()));
		checkSlow(PropertyChainForCompiledIn && PropertyChainForCompiledIn->IsA(ExpectedPropertyType));
		FProperty* Property = (FProperty*)PropertyChainForCompiledIn;
		PropertyChainForCompiledIn = Property->Next;
		StepExplicitProperty(Result, Property);
	}
}

template<class TProperty, typename TNativeType>
FORCEINLINE_DEBUGGABLE TNativeType& FFrame::StepCompiledInRef(void*const TemporaryBuffer)
{
	MostRecentPropertyAddress = nullptr;
	MostRecentPropertyContainer = nullptr;

	if (Code)
	{
		Step(Object, TemporaryBuffer);
	}
	else
	{
		checkSlow(CastField<TProperty>(PropertyChainForCompiledIn) && CastField<FProperty>(PropertyChainForCompiledIn));
		TProperty* Property = (TProperty*)PropertyChainForCompiledIn;
		PropertyChainForCompiledIn = Property->Next;
		StepExplicitProperty(TemporaryBuffer, Property);
	}

	return (MostRecentPropertyAddress != NULL) ? *(TNativeType*)(MostRecentPropertyAddress) : *(TNativeType*)TemporaryBuffer;
}
