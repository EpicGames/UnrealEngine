// Copyright Epic Games, Inc. All Rights Reserved.

/*******************************************************************************************************************
 * NOTICE                                                                                                          *
 *                                                                                                                 *
 * This file is not intended to be included directly - it will be included by other .h files which have predefined *
 * some macros to be expanded within this file.  As such, it does not have a #pragma once as it is intended to be  *
 * included multiple times with different macro definitions.                                                       *
 *                                                                                                                 *
 * #includes needed to compile this file need to be specified in StrPropertyIncludes.h.inl file rather than here.  *
 *******************************************************************************************************************/
#define UE_INCLUDETOOL_IGNORE_INCONSISTENT_STATE

#ifndef UE_STRPROPERTY_CLASS
	#error "StrProperty.h.inl should only be included after defining UE_STRPROPERTY_CLASS"
#endif
#ifndef UE_STRPROPERTY_STRINGTYPE
	#error "StrProperty.h.inl should only be included after defining UE_STRPROPERTY_STRINGTYPE"
#endif
#ifndef UE_STRPROPERTY_CASTCLASSFLAG
	#error "StrProperty.h.inl should only be included after defining UE_STRPROPERTY_CASTCLASSFLAG"
#endif
#ifndef UE_STRPROPERTY_PROPERTYPARAMSSTRUCT
	#error "StrProperty.h.inl should only be included after defining UE_STRPROPERTY_PROPERTYPARAMSSTRUCT"
#endif

class UE_STRPROPERTY_CLASS : public TProperty_WithEqualityAndSerializer<UE_STRPROPERTY_STRINGTYPE, FProperty>
{
	DECLARE_FIELD_API(UE_STRPROPERTY_CLASS, (TProperty_WithEqualityAndSerializer<UE_STRPROPERTY_STRINGTYPE, FProperty>), UE_STRPROPERTY_CASTCLASSFLAG, COREUOBJECT_API)
public:
	using TTypeFundamentals = Super::TTypeFundamentals;
	using TCppType = TTypeFundamentals::TCppType;

	COREUOBJECT_API UE_STRPROPERTY_CLASS(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FStrProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	COREUOBJECT_API UE_STRPROPERTY_CLASS(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : UE_STRPROPERTY_CLASS(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	COREUOBJECT_API UE_STRPROPERTY_CLASS(FFieldVariant InOwner, const UECodeGen_Private::UE_STRPROPERTY_PROPERTYPARAMSSTRUCT& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval UE_STRPROPERTY_CLASS(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	COREUOBJECT_API explicit UE_STRPROPERTY_CLASS(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// FProperty interface
protected:
	COREUOBJECT_API virtual void ExportText_Internal( FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope ) const override;
	COREUOBJECT_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	COREUOBJECT_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	COREUOBJECT_API uint32 GetValueTypeHashInternal(const void* Src) const override;
	// End of FProperty interface
};
