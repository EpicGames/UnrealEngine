// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnrealType.h: Unreal engine base type definitions.
=============================================================================*/

#pragma once

#include "CoreGlobals.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/PropertyPortFlags.h"
#include "UObject/PropertyTag.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
