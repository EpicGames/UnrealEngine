// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UniquePtr.h"

#include "UObject/StrongObjectPtrTemplates.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "UObject/GCObject.h"
#endif