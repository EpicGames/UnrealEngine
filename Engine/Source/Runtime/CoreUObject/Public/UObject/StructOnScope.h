// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Class.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/Casts.h"
#include "Templates/RemoveReference.h"

class FReferenceCollector;

class FStructOnScope
{
protected:
	TWeakObjectPtr<const UStruct> ScriptStruct;
	uint8* SampleStructMemory;
	TWeakObjectPtr<UPackage> Package;
	/** Whether the struct memory is owned by this instance. */
	bool OwnsMemory;

	virtual void Initialize()
	{
		if (const UStruct* ScriptStructPtr = ScriptStruct.Get())
		{
			SampleStructMemory = (uint8*)FMemory::Malloc(ScriptStructPtr->GetStructureSize() ? ScriptStructPtr->GetStructureSize() : 1, ScriptStructPtr->GetMinAlignment());
			ScriptStructPtr->InitializeStruct(SampleStructMemory);
			OwnsMemory = true;
		}
	}

public:

	COREUOBJECT_API FStructOnScope();
	COREUOBJECT_API FStructOnScope(const UStruct* InScriptStruct);
	COREUOBJECT_API FStructOnScope(const UStruct* InScriptStruct, uint8* InData);
	COREUOBJECT_API FStructOnScope(FStructOnScope&& InOther);

	FStructOnScope& operator=(FStructOnScope&& InOther)
	{
		if (this != &InOther)
		{
			Reset();

			ScriptStruct = InOther.ScriptStruct;
			SampleStructMemory = InOther.SampleStructMemory;
			OwnsMemory = InOther.OwnsMemory;

			InOther.OwnsMemory = false;
			InOther.Reset();
		}
		return *this;
	}

	FStructOnScope(const FStructOnScope&) = delete;
	FStructOnScope& operator=(const FStructOnScope&) = delete;

	virtual bool OwnsStructMemory() const
	{
		return OwnsMemory;
	}

	virtual uint8* GetStructMemory()
	{
		return SampleStructMemory;
	}

	virtual const uint8* GetStructMemory() const
	{
		return SampleStructMemory;
	}

	virtual const UStruct* GetStruct() const
	{
		return ScriptStruct.Get();
	}

	TWeakObjectPtr<const UStruct>& GetStructPtr() 
	{
		return ScriptStruct;
	}
	
	COREUOBJECT_API virtual UPackage* GetPackage() const;
	COREUOBJECT_API virtual void SetPackage(UPackage* InPackage);

	virtual bool IsValid() const
	{
		return ScriptStruct.IsValid() && SampleStructMemory;
	}

	virtual void Destroy()
	{
		if (!OwnsMemory)
		{
			return;
		}

		if (const UStruct* ScriptStructPtr = ScriptStruct.Get())
		{
			if (SampleStructMemory)
			{
				ScriptStructPtr->DestroyStruct(SampleStructMemory);
			}
			ScriptStruct = nullptr;
		}

		if (SampleStructMemory)
		{
			FMemory::Free(SampleStructMemory);
			SampleStructMemory = nullptr;
		}
	}

	virtual void Reset()
	{
		Destroy();

		ScriptStruct = nullptr;
		SampleStructMemory = nullptr;
		OwnsMemory = false;
	}

	COREUOBJECT_API virtual ~FStructOnScope();

	/** Re-initializes the scope with a specified UStruct */
	void Initialize(TWeakObjectPtr<const UStruct> InScriptStruct)
	{
		Destroy();
		ScriptStruct = InScriptStruct;
		Initialize();
	}
	
	// If the struct definition is still available and there is struct memory to read from,
	// add any object references in the struct data to the collector
	COREUOBJECT_API void AddReferencedObjects(FReferenceCollector& Collector);
};

/**
 * Typed FStructOnScope that exposes type-safe access to the wrapped struct
 */
template<typename T>
class TStructOnScope final : public FStructOnScope
{
public:
	TStructOnScope() = default;
	virtual ~TStructOnScope() = default;

	TStructOnScope(TStructOnScope&& InOther) = default;
	TStructOnScope& operator=(TStructOnScope&& InOther) = default;

	template<typename U, typename = typename TEnableIf<TIsDerivedFrom<typename TRemoveReference<U>::Type, T>::IsDerived, void>::Type>
	explicit TStructOnScope(U&& InStruct)
		: FStructOnScope(TBaseStructure<typename TRemoveReference<U>::Type>::Get())
	{
		if (const UScriptStruct* ScriptStructPtr = ::Cast<UScriptStruct>(ScriptStruct.Get()))
		{
			ScriptStructPtr->CopyScriptStruct(SampleStructMemory, &InStruct);
		}
	}

	template<typename U, typename = typename TEnableIf<TIsDerivedFrom<typename TRemoveReference<U>::Type, T>::IsDerived, void>::Type>
	TStructOnScope& operator=(U&& InStruct)
	{
		Initialize(TBaseStructure<typename TRemoveReference<U>::Type>::Get());
		if (const UScriptStruct* ScriptStructPtr = ::Cast<UScriptStruct>(ScriptStruct.Get()))
		{
			ScriptStructPtr->CopyScriptStruct(SampleStructMemory, &InStruct);
		}
		return *this;
	}

	/**
	 * Initialize the TStructOnScope as a struct of type U which needs to derive from T
	 * @params InArgs The arguments to pass to the constructor of type U
	 */
	template<typename U
		, typename = typename TEnableIf<TIsDerivedFrom<typename TRemoveReference<U>::Type, T>::IsDerived, void>::Type
		, typename... TArgs>
	void InitializeAs(TArgs&&... InArgs)
	{
		Destroy();
		if (UScriptStruct* ScriptStructPtr = TBaseStructure<U>::Get())
		{
			ScriptStruct = ScriptStructPtr;
			SampleStructMemory = (uint8*)FMemory::Malloc(ScriptStructPtr->GetStructureSize() ? ScriptStructPtr->GetStructureSize() : 1);
			new (SampleStructMemory) U(Forward<TArgs>(InArgs)...);
			OwnsMemory = true;
		}
	}

	/**
	 * Initialize the TStructOnScope from a FStructOnScope containing data that derives from T
	 * @params InOther The FStructOnScope to initialize from
	 * @return True if the conversion was successful, false otherwise
	 */
	bool InitializeFrom(const FStructOnScope& InOther)
	{
		if (const UScriptStruct* ScriptStructPtr = ::Cast<const UScriptStruct>(InOther.GetStruct()))
		{
			if (ScriptStructPtr->IsChildOf(TBaseStructure<T>::Get()))
			{
				Initialize(ScriptStructPtr);
				ScriptStructPtr->CopyScriptStruct(SampleStructMemory, InOther.GetStructMemory());
				return true;
			}
		}
		else
		{
			Destroy();
			return true;
		}
		return false;
	}

	/**
	 * Initialize the TStructOnScope from a FStructOnScope containing data that derives from T
	 * @params InOther The FStructOnScope to initialize from
	 * @return True if the conversion was successful, false otherwise
	 */
	bool InitializeFrom(FStructOnScope&& InOther)
	{
		if (this == &InOther)
		{
			return true;
		}
		if (const UScriptStruct* ScriptStructPtr = ::Cast<const UScriptStruct>(InOther.GetStruct()))
		{
			if (ScriptStructPtr->IsChildOf(TBaseStructure<T>::Get()) && InOther.OwnsStructMemory())
			{
				*static_cast<FStructOnScope*>(this) = MoveTemp(InOther);
				return true;
			}
		}
		else
		{
			Destroy();
			return true;
		}
		return false;
	}

	/**
	 * Initialize the TStructOnScope from a FStructOnScope containing data that derives from T
	 * @params InOther The FStructOnScope to initialize from (will assert if it contains an invalid type to store for T)
	 */
	void InitializeFromChecked(const FStructOnScope& InOther)
	{
		if (!InitializeFrom(InOther))
		{
			UE_LOGF(LogClass, Fatal, "Initialize of %ls to %ls failed", *InOther.GetStruct()->GetName(), *TBaseStructure<T>::Get()->GetName());
		}
	}

	/**
	 * Initialize the TStructOnScope from a FStructOnScope containing data that derives from T
	 * @params InOther The FStructOnScope to initialize from (will assert if it contains an invalid type to store for T)
	 */
	void InitializeFromChecked(FStructOnScope&& InOther)
	{
		if (!InitializeFrom(MoveTemp(InOther)))
		{
			UE_LOGF(LogClass, Fatal, "Initialize of %ls failed", *TBaseStructure<T>::Get()->GetName());
		}
	}

	T* Get() const
	{
		return reinterpret_cast<T*>(SampleStructMemory);
	}

	T* operator->() const
	{
		return Get();
	}

	explicit operator bool() const
	{
		return IsValid();
	}

	template<typename U>
	U* Cast()
	{
		if (GetStruct()->IsChildOf(TBaseStructure<U>::Get()))
		{
			return reinterpret_cast<U*>(SampleStructMemory);
		}
		return nullptr;
	}

	template<typename U>
	const U* Cast() const
	{
		return const_cast<TStructOnScope*>(this)->Cast<U>();
	}

	template<typename U>
	U* CastChecked()
	{
		U* Result = nullptr;
		if (!IsValid())
		{
			UE_LOGF(LogClass, Fatal, "Cast of nullptr to %ls failed", *TBaseStructure<U>::Get()->GetName());
			return Result;
		}

		Result = Cast<U>();
		if (!Result)
		{
			UE_LOGF(LogClass, Fatal, "Cast of %ls to %ls failed", *TBaseStructure<T>::Get()->GetName(), *TBaseStructure<U>::Get()->GetName());
		}
		return Result;
	}

	template<typename U>
	const U* CastChecked() const
	{
		return const_cast<TStructOnScope*>(this)->CastChecked<U>();
	}

	void Serialize(FArchive& Ar)
	{
		if (Ar.IsLoading())
		{
			FString StructPath;
			Ar << StructPath;
			if (!StructPath.IsEmpty())
			{
				UScriptStruct* ScriptStructPtr = FindObject<UScriptStruct>(nullptr, *StructPath, EFindObjectFlags::None);
				if (ScriptStructPtr == nullptr || !ScriptStructPtr->IsChildOf(TBaseStructure<T>::Get()))
				{
					Ar.SetError();
					return;
				}
				ScriptStruct = ScriptStructPtr;
				Initialize();
				ScriptStructPtr->SerializeItem(Ar, SampleStructMemory, nullptr);
			}
		}
		// Saving
		else
		{
			FString StructPath;
			if (UScriptStruct* ScriptStructPtr = const_cast<UScriptStruct*>(::Cast<UScriptStruct>(ScriptStruct.Get())))
			{
				StructPath = ScriptStructPtr->GetPathName();
				Ar << StructPath;
				ScriptStructPtr->SerializeItem(Ar, SampleStructMemory, nullptr);
			}
			else
			{
				Ar << StructPath;
			}
		}
	}

private:
	using FStructOnScope::OwnsStructMemory;
	using FStructOnScope::GetStructMemory;	
	using FStructOnScope::GetPackage;
	using FStructOnScope::SetPackage;
	using FStructOnScope::Destroy;
	using FStructOnScope::Initialize;
};

/**
 * Allocates a new struct of type U with the given arguments and returns it as a Typed StructOnScope of type T.
 *
 * @param Args The arguments to pass to the constructor of U.
 *
 * @return A TStructOnScope<T> which holds a struct to a newly-constructed U with the specified Args.
 */
template <typename T, typename U = T, typename... TArgs>
inline TStructOnScope<T> MakeStructOnScope(TArgs&&... Args)
{
	TStructOnScope<T> Struct;
	Struct.template InitializeAs<U>(Forward<TArgs>(Args)...);
	return Struct;
}

template<typename T>
FArchive& operator<<(FArchive& Ar, TStructOnScope<T>& Struct)
{
	Struct.Serialize(Ar);
	return Ar;
}
