// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/IsPODType.h"
#include "UObject/ObjectMacros.h"

/** type traits to cover the custom aspects of a script struct **/
template <class CPPSTRUCT>
struct TStructOpsTypeTraitsBase2
{
	enum
	{
		WithZeroConstructor            = false,                         // struct can be constructed as a valid object by filling its memory footprint with zeroes.
		WithNoInitConstructor          = false,                         // struct has a constructor which takes an EForceInit parameter which will force the constructor to perform initialization, where the default constructor performs 'uninitialization'.
		WithNoDestructor               = false,                         // struct will not have its destructor called when it is destroyed.
		WithCopy                       = !TIsPODType<CPPSTRUCT>::Value, // struct can be copied via its copy assignment operator.
		WithMoveAssign                 = false,                         // struct can be move-assigned via its move assignment operator. Opt-in; enables MoveAssignScriptStruct to use operator=(T&&) instead of copy.
		WithIdenticalViaEquality       = false,                         // struct can be compared via its operator==.  This should be mutually exclusive with WithIdentical.
		WithIdentical                  = false,                         // struct can be compared via an Identical(const T* Other, uint32 PortFlags) function.  This should be mutually exclusive with WithIdenticalViaEquality.
		WithExportTextItem             = false,                         // struct has an ExportTextItem function used to serialize its state into a string.
		WithImportTextItem             = false,                         // struct has an ImportTextItem function used to deserialize a string into an object of that class.
		WithAddStructReferencedObjects = false,                         // struct has an AddStructReferencedObjects function which allows it to add references to the garbage collector.
		WithSerializer                 = false,                         // struct has a Serialize function for serializing its state to an FArchive.
		WithStructuredSerializer       = false,                         // struct has a Serialize function for serializing its state to an FStructuredArchive.
		WithPostSerialize              = false,                         // struct has a PostSerialize function which is called after it is serialized
		WithNetSerializer              = false,                         // struct has a NetSerialize function for serializing its state to an FArchive used for network replication.
		WithNetDeltaSerializer         = false,                         // struct has a NetDeltaSerialize function for serializing differences in state from a previous NetSerialize operation.
		WithSerializeFromMismatchedTag = false,                         // struct has a SerializeFromMismatchedTag function for converting from other property tags.
		WithStructuredSerializeFromMismatchedTag = false,               // struct has an FStructuredArchive-based SerializeFromMismatchedTag function for converting from other property tags.
		WithPostScriptConstruct        = false,                         // struct has a PostScriptConstruct func tion which is called after it is constructed in blueprints
		WithNetSharedSerialization     = false,                         // struct has a NetSerialize function that does not require the package map to serialize its state.
		WithGetPreloadDependencies     = false,                         // struct has a GetPreloadDependencies function to return all objects that will be Preload()ed when the struct is serialized at load time.
		WithPureVirtual                = false,                         // struct has PURE_VIRTUAL functions and cannot be constructed when CHECK_PUREVIRTUALS is true
		WithFindInnerPropertyInstance  = false,                         // struct has a FindInnerPropertyInstance function that can provide an FProperty and data pointer when given a property FName
		WithCanEditChange              = false,                         // struct has an editor-only CanEditChange function that can conditionally make child properties read-only in the details panel (same idea as UObject::CanEditChange)
		WithClearOnFinishDestroy       = false,                         // struct should be cleared during owner UObject's FinishDestroy. Clearing calls destructor and initializes again to default value. This is intended for structs which may need to access UObject pointer members during destruction. Referenced objects may already have their FinishDestroy() called. Clearing should ensure that no UObject pointer members are used during the final destruction.
		WithVisitor                    = false,                         // struct has Visit function that allows to visit additional properties
		WithIntrusiveOptionalSafeForGC = false,                         // struct with an intrusive unset state for TOptional certifies that object fields it contains are nulled in the unset state
		WithPostLoad UE_INTERNAL       = false,                         // struct has an editor-only PostLoad function which is called in editor by the UObject::PostLoad that owns the struct. The user has to request the callback via the annotation. Use when a deprecation requires access to a UObject.
	};

	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::Conservative; // struct's Serialize method(s) may serialize object references of these types - default Conservative means unknown and object reference collector archives should serialize this struct 
};

template<class CPPSTRUCT>
struct TStructOpsTypeTraits : public TStructOpsTypeTraitsBase2<CPPSTRUCT>
{
};