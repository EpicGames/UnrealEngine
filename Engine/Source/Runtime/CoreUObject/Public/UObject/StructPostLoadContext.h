// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/NotNull.h"
#include "UObject/PropertyPathName.h"
#include "UObject/UObjectAnnotation.h"

struct FUObjectSerializeContext;
class UObjectBase;

#if WITH_EDITOR
namespace UE
{
	/**
	 * Request a struct PostLoad during serialization.
	 * This is only valid in the editor.
	 * Use for structs that need to do fix up logic after all objects have been loaded.
	 */
	class UE_INTERNAL FStructPostLoadContext
	{
	public:
		/** Struct PostLoad request annotation. */
		struct FAnnotation
		{
			bool IsDefault() const
			{
				return !bHasInvalidPropertyPathName && StructPaths.Num() == 0;
			}

			/** The property path of the struct that request a PostLoad. **/
			TArray<FPropertyPathName, TInlineAllocator<1>> StructPaths;

			/** The serialize context doesn't contain a valid property path. */
			bool bHasInvalidPropertyPathName = false;
		};

	public:
		COREUOBJECT_API static FStructPostLoadContext& Get();

		/** A struct request a post load. */
		COREUOBJECT_API bool RequestPostLoad(TNotNull<const FUObjectSerializeContext*> Context);

		/** Gets the annotation for the given object. */
		FAnnotation GetAndRemoveRequest(TNotNull<const UObjectBase*> Object)
		{
			return Annotations.GetAndRemoveAnnotation(Object);
		}

		/** Run the PostLoad on any request for the object. */
		COREUOBJECT_API void OnPostLoad(TNotNull<UObjectBase*> Object);

	private:
		FUObjectAnnotationSparse<FAnnotation, true/*bAutoRemove*/> Annotations;
	};
}
#endif