// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "TemplateString.generated.h"

/** Structure for templated strings that are displayed in the editor with an optional set of valid args. */
USTRUCT(BlueprintType)
struct FTemplateString
{
	GENERATED_BODY()

	/**
	* The template string.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Template)
	FString Template;

	/**
	* The (localizable) resolved text, used to persist the result.
	* This is especially useful when resolution isn't always available, or is static (editor vs. runtime, for example)
	*/
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = Template)
	FText Resolved;

	/** Returns validity based on brace matching, and if provided, arg presence in ValidArgs. */
	bool IsValid(const TArray<FString>& InValidArgs = {}) const;
};
