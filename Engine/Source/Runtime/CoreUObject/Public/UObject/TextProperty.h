// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Serialization/StructuredArchive.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

class FOutputDevice;
class UField;
class UObject;
class UStruct;
struct FPropertyTag;

class FTextProperty : public TProperty<FText, FProperty>
{
	DECLARE_FIELD_API(FTextProperty, (TProperty<FText, FProperty>), CASTCLASS_FTextProperty, COREUOBJECT_API)

public:

	typedef Super::TTypeFundamentals TTypeFundamentals;
	typedef TTypeFundamentals::TCppType TCppType;

	COREUOBJECT_API FTextProperty(FFieldVariant InOwner, const FName& InName);
	UE_DEPRECATED(5.8, "FTextProperty constructor with InObjectFlags is deprecated, remove that parameter.")
	FTextProperty(FFieldVariant InOwner, const FName& InName, EObjectFlags InObjectFlags) : FTextProperty(InOwner, InName) {}

	/**
	 * Constructor used for constructing compiled in properties
	 * @param InOwner Owner of the property
	 * @param PropBase Pointer to the compiled in structure describing the property
	 **/
	COREUOBJECT_API FTextProperty(FFieldVariant InOwner, const UECodeGen_Private::FTextPropertyParams& Prop);

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval FTextProperty(UE::CodeGen::ConstInit::FPropertyParams InBaseParams)
		: Super(InBaseParams)
	{
	}
#endif

#if WITH_EDITORONLY_DATA
	explicit FTextProperty(UField* InField);
#endif // WITH_EDITORONLY_DATA

	// FProperty interface
	COREUOBJECT_API virtual EConvertFromTypeResult ConvertFromType(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot, uint8* Data, const UStruct* DefaultsStruct, const uint8* Defaults) override;
	COREUOBJECT_API virtual bool Identical(TNotNull<const void*> A, const void* B, uint32 PortFlags ) const override;
	COREUOBJECT_API virtual void SerializeItem(FStructuredArchive::FSlot Slot, TNotNull<void*> Value, void const* Defaults) const override;
protected:
	COREUOBJECT_API virtual void ExportText_Internal(FString& ValueStr, TNotNull<const void*> PropertyValueOrContainer, EPropertyPointerType PropertyPointerType, const void* DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const override;
	COREUOBJECT_API virtual const TCHAR* ImportText_Internal(const TCHAR* Buffer, TNotNull<void*> ContainerOrPropertyPtr, EPropertyPointerType PropertyPointerType, UObject* OwnerObject, int32 PortFlags, FOutputDevice* ErrorText) const override;
public:
	// End of FProperty interface

	enum class EIdenticalLexicalCompareMethod : uint8
	{
		None,
		SourceString,
		DisplayString
	};
	static COREUOBJECT_API bool Identical_Implementation(const FText& A, const FText& B, uint32 PortFlags);
	static COREUOBJECT_API bool Identical_Implementation(const FText& A, const FText& B, uint32 PortFlags, EIdenticalLexicalCompareMethod LexicalCompareMethod);
};
