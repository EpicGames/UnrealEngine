// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Serialization/Archive.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveNameHelpers.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Templates/TypeHash.h"
#include "Traits/IsCharType.h"
#include "UObject/NameTypes.h"
#include "UObject/StructOpsTypeTraits.h"

#include <type_traits>

#include "TopLevelAssetPath.generated.h"

class FCbWriter;
struct FPropertyTag;
class FCbWriter;
class FString;
class UObject;

/* 
 * A struct that can reference a top level asset such as '/Path/To/Package.AssetName'
 * Stores two FNames internally to avoid
 *  a) storing a concatenated FName that bloats global FName storage
 *  b) storing an empty FString for a subobject path as FSoftObjectPath allows
 * Can also be used to reference the package itself in which case the second name is NAME_None
 * and the object resolves to the string `/Path/To/Package`
*/
USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/Engine.KismetSystemLibrary.MakeTopLevelAssetPath", HasNativeBreak = "/Script/Engine.KismetSystemLibrary.BreakTopLevelAssetPath"))
struct FTopLevelAssetPath
{
	GENERATED_BODY()
public:
	FTopLevelAssetPath() { }
	FTopLevelAssetPath(TYPE_OF_NULLPTR) { }

	/** Construct directly from components */
	FTopLevelAssetPath(FName InPackageName, FName InAssetName)
	{
		TrySetPath(InPackageName, InAssetName);
	}

	/** Construct from string / string view / raw string of a supported character type. */
	explicit FTopLevelAssetPath(const FString& Path) { TrySetPath(FStringView(Path)); }
	template<typename CharType, typename = typename std::enable_if<TIsCharType<CharType>::Value>::type>
	explicit FTopLevelAssetPath(TStringView<CharType> Path) { TrySetPath(Path); }
	template<typename CharType, typename = typename std::enable_if<TIsCharType<CharType>::Value>::type>
	explicit FTopLevelAssetPath(const CharType* Path) { TrySetPath(TStringView<CharType>(Path)); }

	/** Construct from an existing object. Creates an empty path if the object is not a package or the direct inner of a package. */
	explicit FTopLevelAssetPath(const UObject* InObject) { TrySetPath(InObject); }

	/** Assign from the same types we can construct from */
	FTopLevelAssetPath& operator=(const FString& Path) { TrySetPath(FStringView(Path)); return *this; }
	template<typename CharType>
	FTopLevelAssetPath& operator=(TStringView<CharType> Path) { TrySetPath(Path); return *this; }
	template<typename CharType, typename = typename std::enable_if<TIsCharType<CharType>::Value>::type>
	FTopLevelAssetPath& operator=(const CharType* Path) { TrySetPath(TStringView<CharType>(Path)); return *this; }
	FTopLevelAssetPath& operator=(TYPE_OF_NULLPTR) { Reset(); return *this; }

	/** Sets asset path of this reference based on components. */
	COREUOBJECT_API bool TrySetPath(FName InPackageName, FName InAssetName);
	/** Sets asset path to path of existing object. Sets an empty path and returns false if the object is not a package or the direct inner of a package. */
	COREUOBJECT_API bool TrySetPath(const UObject* InObject);
	/** Sets asset path of this reference based on a string path. Resets this object and returns false if the string is empty or does not represent a top level asset path. */
	COREUOBJECT_API bool TrySetPath(FWideStringView Path);
	COREUOBJECT_API bool TrySetPath(FUtf8StringView Path);
	COREUOBJECT_API bool TrySetPath(FAnsiStringView Path);
	template<typename CharType,	typename = typename std::enable_if<TIsCharType<CharType>::Value>::type>
	bool TrySetPath(const CharType* Path) { return TrySetPath(TStringView<CharType>(Path)); }
	bool TrySetPath(const FString& Path) { return TrySetPath(FStringView(Path)); }

	/** Remap the package to handle renames. */
	COREUOBJECT_API bool RemapPackage(FName OldPackageName, FName NewPackageName);

	/** Return the package name part e.g. /Path/To/Package as an FName. */
	FName GetPackageName() const { return PackageName; }

	/** Return the asset name part e.g. AssetName as an FName. */
	FName GetAssetName() const { return AssetName; }

	/** Append the full asset path (e.g. '/Path/To/Package.AssetName') to the string builder. */
	COREUOBJECT_API void AppendString(FWideStringBuilderBase& Builder) const;
	COREUOBJECT_API void AppendString(FUtf8StringBuilderBase& Builder) const;
	/** Append the full asset path (e.g. '/Path/To/Package.AssetName') to the string. */
	COREUOBJECT_API void AppendString(FString& OutString) const;

	/** Return the full asset path (e.g. '/Path/To/Package.AssetName') as a string. */
	COREUOBJECT_API FString ToString() const;
	/** Copy the full asset path (e.g. '/Path/To/Package.AssetName') into the provided string. */
	COREUOBJECT_API void ToString(FString& OutString) const;

	/** Check if this could possibly refer to a real object */
	bool IsValid() const
	{
		return !PackageName.IsNone();
	}

	/** Checks to see if this is initialized to null */
	bool IsNull() const
	{
		return PackageName.IsNone();
	}

	/** Resets reference to point to null */
	void Reset()
	{		
		PackageName = AssetName = FName();
	}
	/** Compares two paths for non-case-sensitive equality. */
	bool operator==(FTopLevelAssetPath const& Other) const
	{
		return PackageName == Other.PackageName && AssetName == Other.AssetName;
	}

	/** Compares two paths for non-case-sensitive inequality. */
	bool operator!=(FTopLevelAssetPath const& Other) const
	{
		return !(*this == Other);
	}

	/** Serializes the internal path. Unlike FSoftObjectPath, does not handle any PIE or redirector fixups. */
	friend FArchive& operator<<(FArchive& Ar, FTopLevelAssetPath& Path)
	{
		return Ar << Path.PackageName << Path.AssetName;
	}

	/** Serializes the internal path. Unlike FSoftObjectPath, does not handle any PIE or redirector fixups. */
	friend void operator<<(FStructuredArchive::FSlot Slot, FTopLevelAssetPath& Path)
	{
		FStructuredArchive::FRecord Record = Slot.EnterRecord();
		Record << SA_VALUE(TEXT("PackageName"), Path.PackageName) << SA_VALUE(TEXT("AssetName"), Path.AssetName);
	}

	/** Lexically compares two paths. */
	int32 Compare(const FTopLevelAssetPath& Other) const
	{
		if (int32 Diff = PackageName.Compare(Other.PackageName))
		{
			return Diff;
		}
		return AssetName.Compare(Other.AssetName);
	}

	/** Compares two paths in a fast non-lexical order that is only valid for process lifetime. */
	int32 CompareFast(const FTopLevelAssetPath& Other) const
	{
		if (int32 Diff = PackageName.CompareIndexes(Other.PackageName))
		{
			return Diff;
		}
		return AssetName.CompareIndexes(Other.AssetName);
	}

	[[nodiscard]] friend uint32 GetTypeHash(FTopLevelAssetPath const& This)
	{
		return HashCombineFast(GetTypeHash(This.PackageName), GetTypeHash(This.AssetName));
	}

	COREUOBJECT_API bool ExportTextItem(FString& ValueStr, FTopLevelAssetPath const& DefaultValue, UObject* Parent, int32 PortFlags, UObject* ExportRootScope) const;
	COREUOBJECT_API bool ImportTextItem( const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText, FArchive* InSerializingArchive = nullptr );
	COREUOBJECT_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	COREUOBJECT_API void WriteCompactBinary(FCbWriter& Writer) const;
private:
	friend FCbWriter& operator<<(FCbWriter& Writer, const FTopLevelAssetPath& Path)
	{
		Path.WriteCompactBinary(Writer);
		return Writer;
	}
	COREUOBJECT_API friend bool LoadFromCompactBinary(FCbFieldView Field, FTopLevelAssetPath& OutPath);

	/** Name of the package containing the asset e.g. /Path/To/Package */
	UPROPERTY(EditAnywhere, SaveGame, BlueprintReadWrite, Category = TopLevelAssetPath, meta = (AllowPrivateAccess = "true"))
	FName PackageName;
	/** Name of the asset within the package e.g. 'AssetName' */
	UPROPERTY(EditAnywhere, SaveGame, BlueprintReadWrite, Category = TopLevelAssetPath, meta = (AllowPrivateAccess = "true"))
	FName AssetName;

	COREUOBJECT_API friend void SerializeForLog(FCbWriter& Writer, const FTopLevelAssetPath& Value);
};

inline FWideStringBuilderBase& operator<<(FWideStringBuilderBase& Builder, const FTopLevelAssetPath& Path)
{
	Path.AppendString(Builder);
	return Builder;
}

inline FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Builder, const FTopLevelAssetPath& Path)
{
	Path.AppendString(Builder);
	return Builder;
}

/** Fast non-alphabetical order that is only stable during this process' lifetime */
struct FTopLevelAssetPathFastLess
{
	UE_FORCEINLINE_HINT bool operator()(const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) const
	{
		return A.CompareFast(B) < 0;
	}
};

/** Slow alphabetical order that is stable / deterministic over process runs */
struct FTopLevelAssetPathLexicalLess
{
	UE_FORCEINLINE_HINT bool operator()(const FTopLevelAssetPath& A, const FTopLevelAssetPath& B) const
	{
		return A.Compare(B) < 0;
	}
};

template<>
struct TStructOpsTypeTraits<FTopLevelAssetPath> : public TStructOpsTypeTraitsBase2<FTopLevelAssetPath>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true,
		WithExportTextItem = true,
		WithImportTextItem = true,
	};
};
