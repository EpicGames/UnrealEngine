// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnObjAllocator.h: Unreal object allocation
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformMath.h"
#include "Memory/LinearAllocator.h"

class UObjectBase;

class FUObjectAllocator
{
public:
	/**
	 * Disables allocation of objects from the persistend allocator
	 * Needed by the Editor to be able to clean up all objects
	 */
	COREUOBJECT_API void DisablePersistentAllocator();

	/**
	 * Allocates a UObjectBase from the free store or the permanent object pool
	 *
	 * @param Size size of uobject to allocate
	 * @param Alignment alignment of uobject to allocate
	 * @param bAllowPermanent if true, allow allocation in the permanent object pool, if it fits
	 * @param OffsetToObject offset of where the UObjectBase* starts. Should be a positive value to create memory in front of returned pointer
	 * @return newly allocated UObjectBase (not really a UObjectBase yet, no constructor like thing has been called).
	 */
	COREUOBJECT_API UObjectBase* AllocateUObject(int32 Size, int32 Alignment, bool bAllowPermanent, int32 OffsetToObject = 0);

	/**
	 * Returns a UObjectBase to the free store, unless it is in the permanent object pool
	 *
	 * @param Object object to free
	 * @param OffsetToObject offset to Object in allocation. Should match the value sent into AllocateUObject
	 */
	COREUOBJECT_API void FreeUObject(UObjectBase *Object, int32 OffsetToObject = 0) const;
};

/** Global UObjectBase allocator							*/
extern COREUOBJECT_API FUObjectAllocator GUObjectAllocator;

/** Helps check if an object is part of permanent object pool */
class FPermanentObjectPoolExtents
{
public:
	FORCEINLINE FPermanentObjectPoolExtents(const FPersistentLinearAllocatorExtends& InAllocatorExtends = GPersistentLinearAllocatorExtends)
		: Address(InAllocatorExtends.Address)
		, Size(InAllocatorExtends.Size)
	{}

	FORCEINLINE bool Contains(const UObjectBase* Object) const
	{
		return reinterpret_cast<uint64>(Object) - Address < Size;
	}

private:
	const uint64 Address;
	const uint64 Size;
};
