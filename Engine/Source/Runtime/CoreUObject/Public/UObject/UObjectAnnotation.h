// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectAnnotation.h: Unreal object annotation template
=============================================================================*/

#pragma once

#include "AutoRTFM.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectBaseUtility.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Misc/TransactionallySafeRWLock.h"

/**
* FUObjectAnnotationSparse is a helper class that is used to store sparse, slow, temporary, editor only, external 
* or other low priority information about UObjects.
*
* There is a notion of a default annotation and UObjects default to this annotation and this takes no storage.
* 
* Annotations are automatically cleaned up when UObjects are destroyed.
* Annotation are not "garbage collection aware", so it isn't safe to store pointers to other UObjects in an 
* annotation unless external guarantees are made such that destruction of the other object removes the
* annotation.
* @param TAnnotation type of the annotation
* @param bAutoRemove if true, annotation will automatically be removed, otherwise in non-final builds it will verify that the annotation was removed by other means prior to destruction.
**/
template<typename TAnnotation, bool bAutoRemove>
class FUObjectAnnotationSparse : public FUObjectArray::FUObjectDeleteListener
{
public:

	/**
	 * Interface for FUObjectAllocator::FUObjectDeleteListener
	 *
	 * @param Object object that has been destroyed
	 * @param Index	index of object that is being deleted
	 */
	virtual void NotifyUObjectDeleted(const UObjectBase *Object, int32 Index) override
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (!bAutoRemove)
		{
			UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_ReadOnly);
			// in this case we are only verifying that the external assurances of removal are met
			check(!AnnotationMap.Find(Object));
		}
		else
#endif
		{
			RemoveAnnotation(Object);
		}
	}

	virtual void OnUObjectArrayShutdown() override
	{
		RemoveAllAnnotations();
		GUObjectArray.RemoveUObjectDeleteListener(this);
	}

	/**
	 * Constructor, initializes to nothing
	 */
	FUObjectAnnotationSparse()
	{
		// default constructor is required to be default annotation
		check(TAnnotation().IsDefault());
	}

	/**
	 * Destructor, removes all annotations, which removes the annotation as a uobject destruction listener
	 */
	virtual ~FUObjectAnnotationSparse()
	{
		RemoveAllAnnotations();
	}

private:
	template<typename T>
	void AddAnnotationInternal(const UObjectBase* Object, T&& Annotation)
	{
		check(Object);
		bool bProcessRegistration = false;
		TAnnotation LocalAnnotation = Forward<T>(Annotation);
		if (LocalAnnotation.IsDefault())
		{
			RemoveAnnotation(Object); // adding the default annotation is the same as removing an annotation
		}
		else
		{
			UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_Write);
			const bool bWasEmpty = (AnnotationMap.Num() == 0);

			AnnotationMap.Add(Object, LocalAnnotation);

			if (bWasEmpty)
			{
				// we are adding the first one, so if we are auto removing or verifying removal, register now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bAutoRemove)
#endif
				{
					bProcessRegistration = true;
				}
			}
		}

		if (bProcessRegistration)
		{
			ProcessRegistration();
		}
	}

	void ProcessRegistration()
	{
		// Always take the UObjectDeleteListeners mutex first to preserve lock order in all case
		// and avoid deadlock. This is important because we need to follow this lock flow :
		// ConditionalFinishDestroy -> RemoveObjectFromDeleteListeners (UObjectDeleteListenersCritical) -> NotifyUObjectDeleted -> RemoveAnnotation (AnnotationMapCritical)
		GUObjectArray.LockUObjectDeleteListeners();
		ON_SCOPE_EXIT
		{
			GUObjectArray.UnlockUObjectDeleteListeners();
		};

		UE::TWriteScopeLock AnnotationMapLock(AnnotationMapCritical);
		if (AnnotationMap.Num() == 0)
		{
			if (bRegistered)
			{
				GUObjectArray.RemoveUObjectDeleteListener(this);
				bRegistered = false;
			}
		}
		else
		{
			if (!bRegistered)
			{
				GUObjectArray.AddUObjectDeleteListener(this);
				bRegistered = true;
			}
		}
	}
public:
	/**
	 * Add an annotation to the annotation list. If the Annotation is the default, then the annotation is removed.
	 *
	 * @param Object        Object to annotate.
	 * @param Annotation    Annotation to associate with Object.
	 */
	void AddAnnotation(const UObjectBase* Object, TAnnotation&& Annotation)
	{
		AddAnnotationInternal(Object, MoveTemp(Annotation));
	}

	void AddAnnotation(const UObjectBase* Object, const TAnnotation& Annotation)
	{
		AddAnnotationInternal(Object, Annotation);
	}
	/**
	 * Removes an annotation from the annotation list and returns the annotation if it had one 
	 *
	 * @param Object		Object to de-annotate.
	 * @return				Old annotation
	 */
	TAnnotation GetAndRemoveAnnotation(const UObjectBase *Object)
	{		
		check(Object);
		bool bProcessRegistration = false;

		TAnnotation Result;
		{
			UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_Write);
			const bool bHadElements = (AnnotationMap.Num() > 0);
			AnnotationMap.RemoveAndCopyValue(Object, Result);
			const bool bIsNowEmpty = (AnnotationMap.Num() == 0);

			if (bHadElements && bIsNowEmpty)
			{
				// we are removing the last one, so if we are auto removing or verifying removal, unregister now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bAutoRemove)
#endif
				{
					bProcessRegistration = true;
				}
			}
		}

		if (bProcessRegistration)
		{
			ProcessRegistration();
		}
		return Result;
	}
	/**
	 * Removes an annotation from the annotation list. 
	 *
	 * @param Object		Object to de-annotate.
	 */
	void RemoveAnnotation(const UObjectBase *Object)
	{
		check(Object);
		bool bProcessRegistration = false;

		{
			UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_Write);
			const bool bHadElements = (AnnotationMap.Num() > 0);
			AnnotationMap.Remove(Object);
			const bool bIsNowEmpty = (AnnotationMap.Num() == 0);

			if (bHadElements && bIsNowEmpty)
			{
				// we are removing the last one, so if we are auto removing or verifying removal, unregister now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bAutoRemove)
#endif
				{
					bProcessRegistration = true;
				}
			}
		}

		if (bProcessRegistration)
		{
			ProcessRegistration();
		}
	}
	/**
	 * Removes all annotation from the annotation list. 
	 *
	 */
	void RemoveAllAnnotations()
	{
		bool bProcessRegistration = false;

		{
			const bool bHadElements = (AnnotationMap.Num() > 0);
			UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_Write);
			AnnotationMap.Empty();

			if (bHadElements)
			{
				// we are removing the last one, so if we are auto removing or verifying removal, unregister now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bAutoRemove)
#endif
				{
					bProcessRegistration = true;
				}
			}
		}

		if (bProcessRegistration)
		{
			ProcessRegistration();
		}
	}

	/**
	 * Return the annotation associated with a uobject 
	 *
	 * @param Object		Object to return the annotation for
	 */
	inline TAnnotation GetAnnotation(const UObjectBase *Object)
	{
		check(Object);

		UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_ReadOnly);
		TAnnotation* Entry = AnnotationMap.Find(Object);
		return Entry ? *Entry : TAnnotation();
	}

	/**
	 * Return the annotation map. Caution, this is for low level use 
	 * @return A mapping from UObjectBase to annotation for non-default annotations
	 */
	const TMap<const UObjectBase *,TAnnotation>& GetAnnotationMap() const
	{
		return AnnotationMap;
	}

	/** 
	 * Reserves memory for the annotation map for the specified number of elements, used to avoid reallocations. 
	 */
	void Reserve(int32 ExpectedNumElements)
	{
		UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_Write);
		AnnotationMap.Empty(ExpectedNumElements);
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		UE::TRWScopeLock AnnotationMapLock(AnnotationMapCritical, SLT_ReadOnly);
		return AnnotationMap.GetAllocatedSize();
	}

protected:
	/**
	 * Guards the annotation map
	 */
	mutable FTransactionallySafeRWLock AnnotationMapCritical;

	/**
	 * Map from live objects to an annotation. Guarded by AnnotationMapCritical.
	 */
	TMap<const UObjectBase *, TAnnotation> AnnotationMap;

	/**
	 * Monitor registration to GUObjectArray/UObjectDeleteListener.
	 */
	bool bRegistered = false;
};


/**
* FUObjectAnnotationSparseSearchable is a helper class that is used to store sparse, slow, temporary, editor only, external 
* or other low priority information about UObjects...and also provides the ability to find a object based on the unique
* annotation. 
*
* All of the restrictions mentioned for FUObjectAnnotationSparse apply
* 
* @param TAnnotation type of the annotation
* @param bAutoRemove if true, annotation will automatically be removed, otherwise in non-final builds it will verify that the annotation was removed by other means prior to destruction.
**/
template<typename TAnnotation, bool bAutoRemove>
class FUObjectAnnotationSparseSearchable : public FUObjectAnnotationSparse<TAnnotation, bAutoRemove>
{
	typedef FUObjectAnnotationSparse<TAnnotation, bAutoRemove> Super;
public:
	/**
	 * Interface for FUObjectAllocator::FUObjectDeleteListener
	 *
	 * @param Object object that has been destroyed
	 * @param Index	index of object that is being deleted
	 */
	virtual void NotifyUObjectDeleted(const UObjectBase *Object, int32 Index) override
	{
		RemoveAnnotation(Object);
	}

	virtual void OnUObjectArrayShutdown() override
	{
		RemoveAllAnnotations();
		GUObjectArray.RemoveUObjectDeleteListener(this);
	}

	/**
	 * Destructor, removes all annotations, which removes the annotation as a uobject destruction listener
	 */
	virtual ~FUObjectAnnotationSparseSearchable()
	{
		RemoveAllAnnotations();
	}

	/**
	 * Find the UObject associated with a given annotation
	 *
	 * @param Annotation	Annotation to find
	 * @return				Object associated with this annotation or NULL if none found
	 */
	UObject* Find(const TAnnotation& Annotation)
	{
		UE::TScopeLock InverseAnnotationMapLock(InverseAnnotationMapCritical);
		checkSlow(!Annotation.IsDefault()); // it is not legal to search for the default annotation
		return (UObject*)InverseAnnotationMap.FindRef(Annotation);
	}

private:
	template<typename T> 
	void AddAnnotationInternal(const UObjectBase* Object, T&& Annotation)
	{
		UE::TScopeLock InverseAnnotationMapLock(InverseAnnotationMapCritical);
		if (Annotation.IsDefault())
		{
			RemoveAnnotation(Object); // adding the default annotation is the same as removing an annotation
		}
		else
		{
			TAnnotation ExistingAnnotation = this->GetAnnotation(Object);
			int32 NumExistingRemoved = InverseAnnotationMap.Remove(ExistingAnnotation);
			checkSlow(NumExistingRemoved == 0);

			Super::AddAnnotation(Object, Annotation);
			// should not exist in the mapping; we require uniqueness
			int32 NumRemoved = InverseAnnotationMap.Remove(Annotation);
			checkSlow(NumRemoved == 0);
			InverseAnnotationMap.Add(Forward<T>(Annotation), Object);
		}
	}


public:

	/**
	 * Add an annotation to the annotation list. If the Annotation is the default, then the annotation is removed.
	 *
	 * @param Object        Object to annotate.
	 * @param Annotation    Annotation to associate with Object.
	 */
	void AddAnnotation(const UObjectBase* Object, const TAnnotation& Annotation)
	{
		AddAnnotationInternal(Object, Annotation);
	}

	void AddAnnotation(const UObjectBase* Object, TAnnotation&& Annotation)
	{
		AddAnnotationInternal(Object, MoveTemp(Annotation));
	}

	/**
	 * Removes an annotation from the annotation list. 
	 *
	 * @param Object		Object to de-annotate.
	 */
	void RemoveAnnotation(const UObjectBase *Object)
	{
		UE::TScopeLock InverseAnnotationMapLock(InverseAnnotationMapCritical);
		TAnnotation Annotation = this->GetAndRemoveAnnotation(Object);
		if (Annotation.IsDefault())
		{
			// should not exist in the mapping
			checkSlow(!InverseAnnotationMap.Find(Annotation));
		}
		else
		{
			int32 NumRemoved = InverseAnnotationMap.Remove(Annotation);
			checkSlow(NumRemoved == 1);
		}
	}
	/**
	 * Removes all annotation from the annotation list. 
	 *
	 */
	void RemoveAllAnnotations()
	{
		UE::TScopeLock InverseAnnotationMapLock(InverseAnnotationMapCritical);
		Super::RemoveAllAnnotations();
		InverseAnnotationMap.Empty();
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return InverseAnnotationMap.GetAllocatedSize() + Super::GetAllocatedSize();
	}
private:

	/**
	 * Inverse Map annotation to live object
	 */
	TMap<TAnnotation, const UObjectBase *> InverseAnnotationMap;
	FTransactionallySafeCriticalSection InverseAnnotationMapCritical;
};


struct FBoolAnnotation
{
	/**
	 * default constructor
	 * Default constructor must be the default item
	 */
	FBoolAnnotation() :
		Mark(false)
	{
	}
	/**
	 * Initialization constructor
	 * @param InMarks marks to initialize to
	 */
	FBoolAnnotation(bool InMark) :
		Mark(InMark)
	{
	}
	/**
	 * Determine if this annotation
	 * @return true is this is a default pair. We only check the linker because CheckInvariants rules out bogus combinations
	 */
	UE_FORCEINLINE_HINT bool IsDefault()
	{
		return !Mark;
	}

	/**
	 * bool associated with an object
	 */
	bool				Mark; 

};

template <> struct TIsPODType<FBoolAnnotation> { enum { Value = true }; };


/**
* FUObjectAnnotationSparseBool is a specialization of FUObjectAnnotationSparse for bools, slow, temporary, editor only, external 
* or other low priority bools about UObjects.
*
* @todo UE this should probably be reimplemented from scratch as a TSet instead of essentially a map to a value that is always true anyway.
**/
class FUObjectAnnotationSparseBool : private FUObjectAnnotationSparse<FBoolAnnotation,true>
{
public:
	/**
	 * Sets this bool annotation to true for this object
	 *
	 * @param Object		Object to annotate.
	 */
	UE_FORCEINLINE_HINT void Set(const UObjectBase *Object)
	{
		this->AddAnnotation(Object,FBoolAnnotation(true));
	}
	/**
	 * Removes an annotation from the annotation list. 
	 *
	 * @param Object		Object to de-annotate.
	 */
	UE_FORCEINLINE_HINT void Clear(const UObjectBase *Object)
	{
		this->RemoveAnnotation(Object);
	}

	/**
	 * Removes all bool annotations 
	 *
	 */
	UE_FORCEINLINE_HINT void ClearAll()
	{
		this->RemoveAllAnnotations();
	}

	/**
	 * Return the bool annotation associated with a uobject 
	 *
	 * @param Object		Object to return the bool annotation for
	 */
	UE_FORCEINLINE_HINT bool Get(const UObjectBase *Object)
	{
		return this->GetAnnotation(Object).Mark;
	}

	/** 
	 * Reserves memory for the annotation map for the specified number of elements, used to avoid reallocations. 
	 */
	UE_FORCEINLINE_HINT void Reserve(int32 ExpectedNumElements)
	{
		FUObjectAnnotationSparse<FBoolAnnotation,true>::Reserve(ExpectedNumElements);
	}

	UE_FORCEINLINE_HINT int32 Num() const
	{
		return this->GetAnnotationMap().Num();
	}
};

/**
* FUObjectAnnotationChunked is a helper class that is used to store dense, fast and temporary, editor only, external
* or other tangential information about subsets of UObjects.
*
* There is a notion of a default annotation and UObjects default to this annotation.
*
* Annotations are automatically returned to the default when UObjects are destroyed.
* Annotation are not "garbage collection aware", so it isn't safe to store pointers to other UObjects in an
* annotation unless external guarantees are made such that destruction of the other object removes the
* annotation.
* The advantage of FUObjectAnnotationChunked is that it can reclaim memory if subsets of UObjects within predefined chunks
* no longer have any annotations associated with them.
* @param TAnnotation type of the annotation
* @param bAutoRemove if true, annotation will automatically be removed, otherwise in non-final builds it will verify that the annotation was removed by other means prior to destruction.
**/
template<typename TAnnotation, bool bAutoRemove, int32 NumAnnotationsPerChunk = 64 * 1024>
class FUObjectAnnotationChunked : public FUObjectArray::FUObjectDeleteListener
{
	struct TAnnotationChunk
	{
		int32 Num;
		TAnnotation* Items;

		TAnnotationChunk()
			: Num(0)
			, Items(nullptr)
		{}
	};


	/** Primary table to chunks of pointers **/
	TArray<TAnnotationChunk> Chunks;
	/** Number of elements we currently have **/
	int32 NumAnnotations;
	/** Number of elements we can have **/
	int32 MaxAnnotations;
	/** Current allocated memory */
	uint32 CurrentAllocatedMemory;
	/** Max allocated memory */
	uint32 MaxAllocatedMemory;

	/** Mutex */
	FTransactionallySafeRWLock AnnotationArrayCritical;

	/**
	* Makes sure we have enough chunks to fit the new index
	**/
	void ExpandChunksToIndex(int32 Index) TSAN_SAFE
	{
		check(Index >= 0);
		int32 ChunkIndex = Index / NumAnnotationsPerChunk;
		if (ChunkIndex >= Chunks.Num())
		{
			Chunks.AddZeroed(ChunkIndex + 1 - Chunks.Num());
		}
		check(ChunkIndex < Chunks.Num());
		MaxAnnotations = Chunks.Num() * NumAnnotationsPerChunk;
	}

	/**
	* Initializes an annotation for the specified index, makes sure the chunk it resides in is allocated
	**/
	TAnnotation& AllocateAnnotation(int32 Index) TSAN_SAFE
	{
		ExpandChunksToIndex(Index);

		const int32 ChunkIndex = Index / NumAnnotationsPerChunk;
		const int32 WithinChunkIndex = Index % NumAnnotationsPerChunk;

		TAnnotationChunk& Chunk = Chunks[ChunkIndex];
		if (!Chunk.Items)
		{
			Chunk.Items = new TAnnotation[NumAnnotationsPerChunk];
			CurrentAllocatedMemory += NumAnnotationsPerChunk * sizeof(TAnnotation);
			MaxAllocatedMemory = FMath::Max(CurrentAllocatedMemory, MaxAllocatedMemory);
		}
		if (Chunk.Items[WithinChunkIndex].IsDefault())
		{
			Chunk.Num++;
			check(Chunk.Num <= NumAnnotationsPerChunk);
			NumAnnotations++;
		}

		return Chunk.Items[WithinChunkIndex];
	}

	/**
	* Frees the annotation for the specified index
	**/
	void FreeAnnotation(int32 Index) TSAN_SAFE
	{
		const int32 ChunkIndex = Index / NumAnnotationsPerChunk;
		const int32 WithinChunkIndex = Index % NumAnnotationsPerChunk;

		if (ChunkIndex >= Chunks.Num())
		{
			return;
		}

		TAnnotationChunk& Chunk = Chunks[ChunkIndex];
		if (!Chunk.Items)
		{
			return;
		}

		if (Chunk.Items[WithinChunkIndex].IsDefault())
		{
			return;
		}

		Chunk.Items[WithinChunkIndex] = TAnnotation();
		Chunk.Num--;
		check(Chunk.Num >= 0);
		if (Chunk.Num == 0)
		{
			delete[] Chunk.Items;
			Chunk.Items = nullptr;
			const uint32 ChunkMemory = NumAnnotationsPerChunk * sizeof(TAnnotation);
			check(CurrentAllocatedMemory >= ChunkMemory);
			CurrentAllocatedMemory -= ChunkMemory;
		}
		NumAnnotations--;
		check(NumAnnotations >= 0);
	}

	/**
	* Releases all allocated memory and resets the annotation array
	**/
	void FreeAllAnnotations() TSAN_SAFE
	{
		for (TAnnotationChunk& Chunk : Chunks)
		{
			delete[] Chunk.Items;
		}
		Chunks.Empty();
		NumAnnotations = 0;
		MaxAnnotations = 0;
		CurrentAllocatedMemory = 0;
		MaxAllocatedMemory = 0;
	}

	/**
	* Adds a new annotation for the specified index
	**/
	template<typename T>
	void AddAnnotationInternal(int32 Index, T&& Annotation)
	{
		check(Index >= 0);
		if (Annotation.IsDefault())
		{
			FreeAnnotation(Index); // adding the default annotation is the same as removing an annotation
		}
		else
		{
			if (NumAnnotations == 0 && Chunks.Num() == 0)
			{
				// we are adding the first one, so if we are auto removing or verifying removal, register now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bAutoRemove)
#endif
				{
					GUObjectArray.AddUObjectDeleteListener(this);
				}
			}

			TAnnotation& NewAnnotation = AllocateAnnotation(Index);
			NewAnnotation = Forward<T>(Annotation);
		}
	}

public:

	/** Constructor : Probably not thread safe **/
	FUObjectAnnotationChunked() TSAN_SAFE
		: NumAnnotations(0)
		, MaxAnnotations(0)
		, CurrentAllocatedMemory(0)
		, MaxAllocatedMemory(0)
	{
	}

	virtual ~FUObjectAnnotationChunked()
	{
		RemoveAllAnnotations();
	}

public:

	/**
	 * Add an annotation to the annotation list. If the Annotation is the default, then the annotation is removed.
	 *
	 * @param Object        Object to annotate.
	 * @param Annotation    Annotation to associate with Object.
	 */
	void AddAnnotation(const UObjectBase *Object, const TAnnotation& Annotation)
	{
		check(Object);
		AddAnnotation(GUObjectArray.ObjectToIndex(Object), Annotation);
	}

	void AddAnnotation(const UObjectBase* Object, TAnnotation&& Annotation)
	{
		check(Object);
		AddAnnotation(GUObjectArray.ObjectToIndex(Object), MoveTemp(Annotation));
	}

	/**
	 * Add an annotation to the annotation list. If the Annotation is the default, then the annotation is removed.
	 *
	 * @param Index         Index of object to annotate.
	 * @param Annotation    Annotation to associate with Object.
	 */
	void AddAnnotation(int32 Index, const TAnnotation& Annotation)
	{
		UE::TWriteScopeLock AnnotationArrayLock(AnnotationArrayCritical);
		AddAnnotationInternal(Index, Annotation);
	}

	void AddAnnotation(int32 Index, TAnnotation&& Annotation)
	{
		UE::TWriteScopeLock AnnotationArrayLock(AnnotationArrayCritical);
		AddAnnotationInternal(Index, MoveTemp(Annotation));
	}

	TAnnotation& AddOrGetAnnotation(const UObjectBase *Object, TFunctionRef<TAnnotation()> NewAnnotationFn)
	{
		check(Object);
		return AddOrGetAnnotation(GUObjectArray.ObjectToIndex(Object), NewAnnotationFn);
	}
	/**
	 * Add an annotation to the annotation list. If the Annotation is the default, then the annotation is removed.
	 *
	 * @param Index			Index of object to annotate.
	 * @param Annotation	Annotation to associate with Object.
	 */
	TAnnotation& AddOrGetAnnotation(int32 Index, TFunctionRef<TAnnotation()> NewAnnotationFn)
	{		
		UE::TWriteScopeLock AnnotationArrayLock(AnnotationArrayCritical);
		
		if (NumAnnotations == 0 && Chunks.Num() == 0)
		{
			// we are adding the first one, so if we are auto removing or verifying removal, register now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bAutoRemove)
#endif
			{
				GUObjectArray.AddUObjectDeleteListener(this);
			}
		}

		ExpandChunksToIndex(Index);

		const int32 ChunkIndex = Index / NumAnnotationsPerChunk;
		const int32 WithinChunkIndex = Index % NumAnnotationsPerChunk;

		TAnnotationChunk& Chunk = Chunks[ChunkIndex];
		if (!Chunk.Items)
		{
			Chunk.Items = new TAnnotation[NumAnnotationsPerChunk];
			CurrentAllocatedMemory += NumAnnotationsPerChunk * sizeof(TAnnotation);
			MaxAllocatedMemory = FMath::Max(CurrentAllocatedMemory, MaxAllocatedMemory);
		}
		if (Chunk.Items[WithinChunkIndex].IsDefault())
		{
			Chunk.Num++;
			check(Chunk.Num <= NumAnnotationsPerChunk);
			NumAnnotations++;
			Chunk.Items[WithinChunkIndex] = NewAnnotationFn();
			check(!Chunk.Items[WithinChunkIndex].IsDefault());
		}
		return Chunk.Items[WithinChunkIndex];
	}

	/**
	 * Removes an annotation from the annotation list.
	 *
	 * @param Object		Object to de-annotate.
	 */
	void RemoveAnnotation(const UObjectBase *Object)
	{
		check(Object);
		RemoveAnnotation(GUObjectArray.ObjectToIndex(Object));
	}
	/**
	 * Removes an annotation from the annotation list.
	 *
	 * @param Object		Object to de-annotate.
	 */
	void RemoveAnnotation(int32 Index)
	{
		UE::TWriteScopeLock AnnotationArrayLock(AnnotationArrayCritical);
		FreeAnnotation(Index);
	}

	/**
	 * Return the annotation associated with a uobject
	 *
	 * @param Object		Object to return the annotation for
	 */
	inline TAnnotation GetAnnotation(const UObjectBase *Object)
	{
		check(Object);
		return GetAnnotation(GUObjectArray.ObjectToIndex(Object));
	}

	/**
	 * Return the annotation associated with a uobject
	 *
	 * @param Index		Index of the annotation to return
	 */
	inline TAnnotation GetAnnotation(int32 Index)
	{
		check(Index >= 0);

		TAnnotation Result = TAnnotation();

		UE_AUTORTFM_OPEN
		{
			UE::TReadScopeLock AnnotationArrayLock(AnnotationArrayCritical);

			const int32 ChunkIndex = Index / NumAnnotationsPerChunk;
			if (ChunkIndex < Chunks.Num())
			{
				const int32 WithinChunkIndex = Index % NumAnnotationsPerChunk;

				TAnnotationChunk& Chunk = Chunks[ChunkIndex];
				if (Chunk.Items != nullptr)
				{
					Result = Chunk.Items[WithinChunkIndex];
				}
			}
		};

		return Result;
	}

	/**
	* Return the number of elements in the array
	* Thread safe, but you know, someone might have added more elements before this even returns
	* @return	the number of elements in the array
	**/
	UE_FORCEINLINE_HINT int32 GetAnnotationCount() const
	{
		return NumAnnotations;
	}

	/**
	* Return the number max capacity of the array
	* Thread safe, but you know, someone might have added more elements before this even returns
	* @return	the maximum number of elements in the array
	**/
	UE_FORCEINLINE_HINT int32 GetMaxAnnotations() const TSAN_SAFE
	{
		return MaxAnnotations;
	}

	/**
	* Return if this index is valid
	* Thread safe, if it is valid now, it is valid forever. Other threads might be adding during this call.
	* @param	Index	Index to test
	* @return	true, if this is a valid
	**/
	UE_FORCEINLINE_HINT bool IsValidIndex(int32 Index) const
	{
		return Index >= 0 && Index < MaxAnnotations;
	}

	/**
	 * Removes all annotation from the annotation list.
	 */
	void RemoveAllAnnotations()
	{
		bool bHadAnnotations = (NumAnnotations > 0);	
		UE::TWriteScopeLock AnnotationArrayLock(AnnotationArrayCritical);
		FreeAllAnnotations();
		if (bHadAnnotations)
		{
			// we are removing the last one, so if we are auto removing or verifying removal, unregister now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
			if (bAutoRemove)
#endif
			{
				GUObjectArray.RemoveUObjectDeleteListener(this);
			}
		}
	}

	/**
	 * Frees chunk memory from empty chunks.
	 */
	void TrimAnnotations()
	{
		UE::TWriteScopeLock AnnotationArrayLock(AnnotationArrayCritical);
		for (TAnnotationChunk& Chunk : Chunks)
		{
			if (Chunk.Num == 0 && Chunk.Items)
			{
				delete[] Chunk.Items;
				Chunk.Items = nullptr;
				const uint32 ChunkMemory = NumAnnotationsPerChunk * sizeof(TAnnotationChunk);
				check(CurrentAllocatedMemory >= ChunkMemory);
				CurrentAllocatedMemory -= ChunkMemory;
			}
		}
	}

	/** Returns the memory allocated by the internal array */
	virtual SIZE_T GetAllocatedSize() const override
	{
		SIZE_T AllocatedSize = Chunks.GetAllocatedSize();
		for (const TAnnotationChunk& Chunk : Chunks)
		{
			if (Chunk.Items)
			{
				AllocatedSize += NumAnnotationsPerChunk * sizeof(TAnnotation);
			}
		}
		return AllocatedSize;
	}

	/** Returns the maximum memory allocated by the internal arrays */
	uint32 GetMaxAllocatedSize() const
	{
		return Chunks.GetAllocatedSize() + MaxAllocatedMemory;
	}

	/**
	 * Interface for FUObjectAllocator::FUObjectDeleteListener
	 *
	 * @param Object object that has been destroyed
	 * @param Index	index of object that is being deleted
	 */
	virtual void NotifyUObjectDeleted(const UObjectBase *Object, int32 Index)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (!bAutoRemove)
		{
			// in this case we are only verifying that the external assurances of removal are met
			check(Index >= MaxAnnotations || GetAnnotation(Index).IsDefault());
		}
		else
#endif
		{
			RemoveAnnotation(Index);
		}
	}

	virtual void OnUObjectArrayShutdown() override
	{
		RemoveAllAnnotations();
		GUObjectArray.RemoveUObjectDeleteListener(this);
	}
};

/**
* FUObjectAnnotationDense is a helper class that is used to store dense, fast, temporary, editor only, external 
* or other tangential information about UObjects.
*
* There is a notion of a default annotation and UObjects default to this annotation.
* 
* Annotations are automatically returned to the default when UObjects are destroyed.
* Annotation are not "garbage collection aware", so it isn't safe to store pointers to other UObjects in an 
* annotation unless external guarantees are made such that destruction of the other object removes the
* annotation.
* @param TAnnotation type of the annotation
* @param bAutoRemove if true, annotation will automatically be removed, otherwise in non-final builds it will verify that the annotation was removed by other means prior to destruction.
**/
template<typename TAnnotation, bool bAutoRemove>
class FUObjectAnnotationDense : public FUObjectArray::FUObjectDeleteListener
{
public:

	/**
	 * Interface for FUObjectAllocator::FUObjectDeleteListener
	 *
	 * @param Object object that has been destroyed
	 * @param Index	index of object that is being deleted
	 */
	virtual void NotifyUObjectDeleted(const UObjectBase *Object, int32 Index)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (!bAutoRemove)
		{
			// in this case we are only verifying that the external assurances of removal are met
			check(Index >= AnnotationArray.Num() || AnnotationArray[Index].IsDefault());
		}
		else
#endif
		{
			RemoveAnnotation(Index);
		}
	}

	virtual void OnUObjectArrayShutdown() override
	{
		RemoveAllAnnotations();
		GUObjectArray.RemoveUObjectDeleteListener(this);
	}

	/**
	 * Destructor, removes all annotations, which removes the annotation as a uobject destruction listener
	 */
	virtual ~FUObjectAnnotationDense()
	{
		RemoveAllAnnotations();
	}

	/**
	 * Add an annotation to the annotation list. If the Annotation is the default, then the annotation is removed.
	 *
	 * @param Object        Object to annotate.
	 * @param Annotation    Annotation to associate with Object.
	 */
	void AddAnnotation(const UObjectBase* Object, const TAnnotation& Annotation)
	{
		check(Object);
		AddAnnotation(GUObjectArray.ObjectToIndex(Object), Annotation);
	}

	void AddAnnotation(const UObjectBase* Object, TAnnotation&& Annotation)
	{
		check(Object);
		AddAnnotation(GUObjectArray.ObjectToIndex(Object), MoveTemp(Annotation));
	}

	/**
	 * Add an annotation to the annotation list. If the Annotation is the default, then the annotation is removed.
	 *
	 * @param Index         Index of object to annotate.
	 * @param Annotation    Annotation to associate with Object.
	 */
	void AddAnnotation(int32 Index, const TAnnotation& Annotation)
	{
		bool bProcessRegistration = false;
		{
			UE::TRWScopeLock AnnotationArrayLock(AnnotationArrayCritical, SLT_Write);
			bProcessRegistration = AddAnnotationInternal(Index, Annotation);
		}

		if (bProcessRegistration)
		{
			ProcessRegistration();
		}
	}

	void AddAnnotation(int32 Index, TAnnotation&& Annotation)
	{	
		bool bProcessRegistration = false;
		{
			UE::TRWScopeLock AnnotationArrayLock(AnnotationArrayCritical, SLT_Write);
			bProcessRegistration = AddAnnotationInternal(Index, MoveTemp(Annotation));
		}

		if (bProcessRegistration)
		{
			ProcessRegistration();
		}
	}


private:
	void ProcessRegistration()
	{
		// Always take the UObjectDeleteListeners mutex first to preserve lock order in all case
		// and avoid deadlock. This is important because we need to follow this lock flow :
		// ConditionalFinishDestroy -> RemoveObjectFromDeleteListeners (UObjectDeleteListenersCritical) -> NotifyUObjectDeleted -> RemoveAnnotation (AnnotationMapCritical)
		GUObjectArray.LockUObjectDeleteListeners();
		ON_SCOPE_EXIT
		{
			GUObjectArray.UnlockUObjectDeleteListeners();
		};

		UE::TWriteScopeLock AnnotationMapLock(AnnotationArrayCritical);
		if (AnnotationArray.Num() == 0)
		{
			if (bRegistered)
			{
				GUObjectArray.RemoveUObjectDeleteListener(this);
				bRegistered = false;
			}
		}
		else
		{
			if (!bRegistered)
			{
				GUObjectArray.AddUObjectDeleteListener(this);
				bRegistered = true;
			}
		}
	}

	template<typename T>
	[[nodiscard]] bool AddAnnotationInternal(int32 Index, T&& Annotation)
	{
		bool bProcessRegistration = false;

		check(Index >= 0);
		if (Annotation.IsDefault())
		{
			RemoveAnnotationInternal(Index); // adding the default annotation is the same as removing an annotation
		}
		else
		{
			if (AnnotationArray.Num() == 0)
			{
				// we are adding the first one, so if we are auto removing or verifying removal, register now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bAutoRemove)
#endif
				{
					bProcessRegistration = true;
				}
			}
			if (Index >= AnnotationArray.Num())
			{
				int32 AddNum = 1 + Index - AnnotationArray.Num();
				int32 Start = AnnotationArray.AddUninitialized(AddNum);
				while (AddNum--) 
				{
					new (AnnotationArray.GetData() + Start++) TAnnotation();
				}
			}
			AnnotationArray[Index] = Forward<T>(Annotation);
		}

		return bProcessRegistration;
	}

public:
	/**
	 * Removes an annotation from the annotation list. 
	 *
	 * @param Object		Object to de-annotate.
	 */
	void RemoveAnnotation(const UObjectBase *Object)
	{
		check(Object);
		RemoveAnnotation(GUObjectArray.ObjectToIndex(Object));
	}
	/**
	 * Removes an annotation from the annotation list. 
	 *
	 * @param Object		Object to de-annotate.
	 */
	void RemoveAnnotation(int32 Index)
	{
		UE::TRWScopeLock AnnotationArrayLock(AnnotationArrayCritical, SLT_Write);
		RemoveAnnotationInternal(Index);
	}

private:
	void RemoveAnnotationInternal(int32 Index)
	{
		check(Index >= 0);
		if (Index <  AnnotationArray.Num())
		{
			AnnotationArray[Index] = TAnnotation();
		}
	}

public:
	/**
	 * Removes all annotation from the annotation list. 
	 *
	 */
	void RemoveAllAnnotations()
	{
		bool bProcessRegistration = false;
		{
			UE::TRWScopeLock AnnotationArrayLock(AnnotationArrayCritical, SLT_Write);
			bool bHadElements = (AnnotationArray.Num() > 0);
			AnnotationArray.Empty();
			if (bHadElements)
			{
				// we are removing the last one, so if we are auto removing or verifying removal, unregister now
#if (UE_BUILD_SHIPPING || UE_BUILD_TEST)
				if (bAutoRemove)
#endif
				{
					bProcessRegistration = true;
				}
			}
		}

		ProcessRegistration();
	}

	/**
	 * Return the annotation associated with a uobject 
	 *
	 * @param Object		Object to return the annotation for
	 */
	inline TAnnotation GetAnnotation(const UObjectBase *Object)
	{
		check(Object);
		int32 Index = GUObjectArray.ObjectToIndex(Object);
		// Since we are potentially dealing with a corrupt ptr validation is broken into two checks.
		// First to tell if something is wrong and the second to try to provide
		// more information to the user (however in a potentially unsafe way by reading the ptr).
		ensureAlwaysMsgf(Index >= 0, TEXT("Corrupt object with negative Index passed to GetAnnotation: Index == %d"), Index);
		checkf(Index >= 0, TEXT("Failed GetAnnotation for Object '%s'"), *static_cast<const UObjectBaseUtility*>(Object)->GetPathName());
		return GetAnnotation(Index);
	}

	/**
	 * Return the annotation associated with a uobject 
	 *
	 * @param Index		Index of the annotation to return
	 */
	inline TAnnotation GetAnnotation(int32 Index)
	{
		check(Index >= 0);
		UE::TRWScopeLock AnnotationArrayLock(AnnotationArrayCritical, SLT_ReadOnly);
		if (Index < AnnotationArray.Num())
		{
			return AnnotationArray[Index];
		}
		return TAnnotation();
	}

	/**
	 * Return the annotation associated with a uobject 
	 *
	 * @param Object		Object to return the annotation for
	 * @return				Reference to annotation.
	 */
	inline TAnnotation& GetAnnotationRef(const UObjectBase *Object)
	{
		check(Object);
		return GetAnnotationRef(GUObjectArray.ObjectToIndex(Object));
	}

	/**
	 * Return the annotation associated with a uobject. Adds one if the object has
	 * no annotation yet.
	 *
	 * @param Index		Index of the annotation to return
	 * @return			Reference to the annotation.
	 */
	inline TAnnotation& GetAnnotationRef(int32 Index)
	{
		bool bProcessRegistration = false;
		{
			UE::TRWScopeLock AnnotationArrayLock(AnnotationArrayCritical, SLT_Write);
			if (Index >= AnnotationArray.Num())
			{
				bProcessRegistration = AddAnnotationInternal(Index, TAnnotation());
			}
		}

		if (bProcessRegistration)
		{
			ProcessRegistration();
		}
		return AnnotationArray[Index];
	}

	/** Returns the memory allocated by the internal array */
	virtual SIZE_T GetAllocatedSize() const override
	{
		return AnnotationArray.GetAllocatedSize();
	}

private:

	/**
	 * Map from live objects to an annotation
	 */
	TArray<TAnnotation> AnnotationArray;
	FTransactionallySafeRWLock AnnotationArrayCritical;

	/**
	 * Monitor registration to GUObjectArray/UObjectDeleteListener.
	 */
	bool bRegistered = false;
};

/**
* FUObjectAnnotationDenseBool is custom annotation that tracks a bool per UObject. 
**/
class FUObjectAnnotationDenseBool : public FUObjectArray::FUObjectDeleteListener
{
	typedef uint32 TBitType;
	enum {BitsPerElement = sizeof(TBitType) * 8};
public:

	/**
	 * Interface for FUObjectAllocator::FUObjectDeleteListener
	 *
	 * @param Object object that has been destroyed
	 * @param Index	index of object that is being deleted
	 */
	virtual void NotifyUObjectDeleted(const UObjectBase *Object, int32 Index)
	{
		RemoveAnnotation(Index);
	}

	virtual void OnUObjectArrayShutdown() override
	{
		RemoveAllAnnotations();
		GUObjectArray.RemoveUObjectDeleteListener(this);
	}

	/**
	 * Destructor, removes all annotations, which removes the annotation as a uobject destruction listener
	 */
	virtual ~FUObjectAnnotationDenseBool()
	{
		RemoveAllAnnotations();
	}

	/**
	 * Sets this bool annotation to true for this object
	 *
	 * @param Object		Object to annotate.
	 */
	inline void Set(const UObjectBase *Object)
	{
		checkSlow(Object);
		int32 Index = GUObjectArray.ObjectToIndex(Object);
		checkSlow(Index >= 0);
		if (AnnotationArray.Num() == 0)
		{
			GUObjectArray.AddUObjectDeleteListener(this);
		}
		if (Index >= AnnotationArray.Num() * BitsPerElement)
		{
			int32 AddNum = 1 + Index - AnnotationArray.Num() * BitsPerElement;
			int32 AddElements = (AddNum +  BitsPerElement - 1) / BitsPerElement;
			checkSlow(AddElements);
			AnnotationArray.AddZeroed(AddElements);
			checkSlow(Index < AnnotationArray.Num() * BitsPerElement);
		}
		AnnotationArray[Index / BitsPerElement] |= TBitType(TBitType(1) << (Index % BitsPerElement));
	}
	/**
	 * Removes an annotation from the annotation list. 
	 *
	 * @param Object		Object to de-annotate.
	 */
	inline void Clear(const UObjectBase *Object)
	{
		checkSlow(Object);
		int32 Index = GUObjectArray.ObjectToIndex(Object);
		RemoveAnnotation(Index);
	}

	/**
	 * Removes all bool annotations 
	 *
	 */
	UE_FORCEINLINE_HINT void ClearAll()
	{
		RemoveAllAnnotations();
	}

	/**
	 * Return the bool annotation associated with a uobject 
	 *
	 * @param Object		Object to return the bool annotation for
	 */
	inline bool Get(const UObjectBase *Object)
	{
		checkSlow(Object);
		int32 Index = GUObjectArray.ObjectToIndex(Object);
		checkSlow(Index >= 0);
		if (Index < AnnotationArray.Num() * BitsPerElement)
		{
			return !!(AnnotationArray[Index / BitsPerElement] & TBitType(TBitType(1) << (Index % BitsPerElement)));
		}
		return false;
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return AnnotationArray.GetAllocatedSize();
	}

private:
	/**
	 * Removes an annotation from the annotation list. 
	 *
	 * @param Object		Object to de-annotate.
	 */
	void RemoveAnnotation(int32 Index)
	{
		checkSlow(Index >= 0);
		if (Index <  AnnotationArray.Num() * BitsPerElement)
		{
			AnnotationArray[Index / BitsPerElement] &= ~TBitType(TBitType(1) << (Index % BitsPerElement));
		}
	}
	/**
	 * Removes all annotation from the annotation list. 
	 *
	 */
	void RemoveAllAnnotations()
	{
		bool bHadElements = (AnnotationArray.Num() > 0);
		AnnotationArray.Empty();
		if (bHadElements)
		{
			GUObjectArray.RemoveUObjectDeleteListener(this);
		}
	}

	/**
	 * Map from live objects to an annotation
	 */
	TArray<TBitType> AnnotationArray;
};

/**
 * FUObjectAnnotationTemp is a temporary annotation with the same API as FUObjectAnnotationSparse but without the thread safety or GC checks.
 * It should only be used for single threaded functions and must be deleted before garbage collection occurs.
 */
template<typename TAnnotation>
class FUObjectAnnotationTemp
{
public:

	/**
	 * Constructor, initializes to nothing
	 */
	FUObjectAnnotationTemp()
	{
		// default constructor is required to be default annotation
		check(AnnotationCacheValue.IsDefault());
	}

	/**
	 * Destructor, removes all annotations
	 */
	virtual ~FUObjectAnnotationTemp()
	{
		RemoveAllAnnotations();
	}

private:
	template<typename T>
	void AddAnnotationInternal(const UObjectBase* Object, T&& Annotation)
	{
		TAnnotation LocalAnnotation = Forward<T>(Annotation);
		if (LocalAnnotation.IsDefault())
		{
			RemoveAnnotation(Object); // adding the default annotation is the same as removing an annotation
		}
		else
		{
			AnnotationMap.Add(Object, LocalAnnotation);
			SetAnnotationCacheKeyAndValue(Object, MoveTemp(LocalAnnotation));
		}
	}

public:
	/**
	 * Add an annotation to the annotation list. If the Annotation is the default, then the annotation is removed.
	 *
	 * @param Object        Object to annotate.
	 * @param Annotation    Annotation to associate with Object.
	 */
	void AddAnnotation(const UObjectBase* Object, TAnnotation&& Annotation)
	{
		AddAnnotationInternal(Object, MoveTemp(Annotation));
	}

	void AddAnnotation(const UObjectBase* Object, const TAnnotation& Annotation)
	{
		AddAnnotationInternal(Object, Annotation);
	}
	/**
	 * Removes an annotation from the annotation list and returns the annotation if it had one 
	 *
	 * @param Object		Object to de-annotate.
	 * @return				Old annotation
	 */
	TAnnotation GetAndRemoveAnnotation(const UObjectBase *Object)
	{
		TAnnotation Result;
		SetAnnotationCacheKeyAndValue(Object, TAnnotation());
		AnnotationMap.RemoveAndCopyValue(Object, Result);

		return Result;
	}
	/**
	 * Removes an annotation from the annotation list. 
	 *
	 * @param Object		Object to de-annotate.
	 */
	void RemoveAnnotation(const UObjectBase *Object)
	{
		SetAnnotationCacheKeyAndValue(Object, TAnnotation());
		AnnotationMap.Remove(Object);
	}
	/**
	 * Removes all annotation from the annotation list. 
	 */
	void RemoveAllAnnotations()
	{
		SetAnnotationCacheKeyAndValue(nullptr, TAnnotation());
		AnnotationMap.Empty();
	}

	/**
	 * Return the annotation associated with a uobject 
	 *
	 * @param Object		Object to return the annotation for
	 */
	inline TAnnotation GetAnnotation(const UObjectBase *Object)
	{
		// Check cache
		if (Object == AnnotationCacheKey)
		{
			return AnnotationCacheValue;
		}
		
		// Update the cache if it isn't a match.
		TAnnotation* Entry = AnnotationMap.Find(Object);
		SetAnnotationCacheKeyAndValue(Object, Entry ? *Entry : TAnnotation());

		return Entry ? *Entry : TAnnotation();
	}

	/**
	 * Return the annotation map. Caution, this is for low level use 
	 * @return A mapping from UObjectBase to annotation for non-default annotations
	 */
	const TMap<const UObjectBase *,TAnnotation>& GetAnnotationMap() const
	{
		return AnnotationMap;
	}

	/** 
	 * Reserves memory for the annotation map for the specified number of elements, used to avoid reallocations. 
	 */
	void Reserve(int32 ExpectedNumElements)
	{
		AnnotationMap.Empty(ExpectedNumElements);
	}
private:
	void SetAnnotationCacheKeyAndValue(const UObjectBase* Key, const TAnnotation& Value)
	{
		AnnotationCacheKey = Key;
		AnnotationCacheValue = Value;
	}

	void SetAnnotationCacheKeyAndValue(const UObjectBase* Key, TAnnotation&& Value)
	{
		AnnotationCacheKey = Key;
		AnnotationCacheValue = MoveTemp(Value);
	}

protected:
	/**
	 * Map from live objects to an annotation.
	 */
	TMap<const UObjectBase *, TAnnotation> AnnotationMap;

	/**
	 * Key for a one-item cache of the last lookup into AnnotationMap.
	 * Annotation are often called back-to-back so this is a performance optimization for that.
	 */
	const UObjectBase* AnnotationCacheKey = nullptr;

	/**
	 * Value for a one-item cache of the last lookup into AnnotationMap.
	 */
	TAnnotation AnnotationCacheValue;
};
