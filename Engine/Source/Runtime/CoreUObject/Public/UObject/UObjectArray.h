// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectArray.h: Unreal object array
=============================================================================*/

#pragma once

#include "AutoRTFM.h"
#include "HAL/ThreadSafeCounter.h"
#include "Containers/LockFreeList.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "UObject/GarbageCollectionGlobals.h"
#include "UObject/UObjectBase.h"

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
namespace Verse
{
	struct VCell;
}
#endif

#ifndef UE_PACK_FUOBJECT_ITEM
#	define UE_ENABLE_FUOBJECT_ITEM_PACKING 0
#elif PLATFORM_CPU_ARM_FAMILY && FORCE_ANSI_ALLOCATOR
//	disable packing on ARM64 is we use Ansi allocator as it might use MTE or HWASan which uses top byte of a pointer and this feature will discard that
#	define UE_ENABLE_FUOBJECT_ITEM_PACKING 0
#else
#	define UE_ENABLE_FUOBJECT_ITEM_PACKING UE_PACK_FUOBJECT_ITEM && !UE_WITH_REMOTE_OBJECT_HANDLE
#endif

namespace UE::GC::Private
{
	class FGCFlags;
}

/**
* Single item in the UObject array.
*/
struct FUObjectItem
{
	friend class FUObjectArray;
	friend class UE::GC::Private::FGCFlags;

private:
	// Stores EInternalObjectFlags (and higher 13 bits of the UObject pointer packed together if UE_PACK_FUOBJECT_ITEM is set to 1)
	// These can only be changed via Set* and Clear* functions
	// The Flags are now stored on high 32-bit so that we can use InterlockedInc/InterlockedDec directly for RefCount which is stored on the low 32-bit
	// while preserving atomicity of the whole thing so RootFlags+RefCount can be evaluated in a lock-less way.
	// If we want to add more Flags, we can reduce the size of RefCount to 24 bits to give us more bits for Flags but will require that we convert
	// EInternalObjectFlags to a 64-bit.
	union
	{
		int64 FlagsAndRefCount;
#if !UE_WITH_REMOTE_OBJECT_HANDLE
		// Dummy variable for natvis
		uint8 RemoteId;
#endif
	};

#if !UE_ENABLE_FUOBJECT_ITEM_PACKING
public:
	union
	{
		// Pointer to the allocated object
		UE_DEPRECATED(5.6, "Use GetObject() and SetObject() to access Object.")
		class UObjectBase* Object = nullptr;
		uint32 ObjectPtrLow;	// this one is used as a dummy for natvis only an will be removed once packing will be enabled by default
	};
#else
	union
	{
		// Stores lower 32 bits of UObject pointer shifted by 3 to the left as all our allocations are at least 8 bytes aligned and lower 3 bits will always be 0
		uint32 ObjectPtrLow = 0;
		uint32 Object;	// this one is used as a dummy for natvis only an will be removed once packing will be enabled by default
	};
#endif
private:
	// Currently we assume UObjects are aligned by 8 bytes, that gives us 3 lower bits as zeros that we can discard.
	// This will give us total 45 bits in a pointer that we pack into a int32 and the remaining 13 bits we pack with Flags
	// EInternalObjectFlags_MinFlagBitIndex at the time of writing this is 14 and we have only 1 bit left in the EInternalObjectFlags for future use
	// We can increase UObject alignment to 16 bytes to get one more bit and reduce the overall addressable virtual memory range to get more bits if necessary
	constexpr static int32 UObjectAlignment = 8;
	constexpr static int32 UObjectPtrTrailingZeroes = FMath::CountTrailingZeros(UObjectAlignment);
	static_assert(int(EInternalObjectFlags_MinFlagBitIndex) >= 48 - 32 - UObjectPtrTrailingZeroes, "We need at least 13 bits to pack higher bits of a UObject pointer into Flags");
	constexpr static int32 FlagsMask = 0xFFFFFFFF << int(EInternalObjectFlags_MinFlagBitIndex);
	constexpr static int32 PtrMask = ~FlagsMask;

public:
	// Weak Object Pointer Serial number associated with the object
	int32 SerialNumber;
	// UObject Owner Cluster Index
	int32 ClusterRootIndex;

#if UE_WITH_REMOTE_OBJECT_HANDLE
private:
	// Globally unique id of this object
	FRemoteObjectId RemoteId;
public:
#endif

#if STATS || ENABLE_STATNAMEDEVENTS_UOBJECT
	/** Stat id of this object, 0 if nobody asked for it yet */
	mutable TStatId StatID;

#if ENABLE_STATNAMEDEVENTS_UOBJECT
	mutable PROFILER_CHAR* StatIDStringStorage;
#endif
#endif // STATS || ENABLE_STATNAMEDEVENTS

	FUObjectItem()
		: FlagsAndRefCount(0)
		, SerialNumber(0)
		, ClusterRootIndex(0)
#if ENABLE_STATNAMEDEVENTS_UOBJECT
		, StatIDStringStorage(nullptr)
#endif
	{
	}
	~FUObjectItem()
	{
#if ENABLE_STATNAMEDEVENTS_UOBJECT
		if (PROFILER_CHAR* Storage = StatIDStringStorage)
		{
			AutoRTFM::PopOnAbortHandler(Storage);
			delete[] Storage;
		}
#endif
	}

	// Non-copyable
	FUObjectItem(FUObjectItem&&) = delete;
	FUObjectItem(const FUObjectItem&) = delete;
	FUObjectItem& operator=(FUObjectItem&&) = delete;
	FUObjectItem& operator=(const FUObjectItem&) = delete;

	inline class UObjectBase* GetObject() const
	{
#if UE_ENABLE_FUOBJECT_ITEM_PACKING
		const uintptr_t Obj = (uintptr_t((FPlatformAtomics::AtomicRead_Relaxed((int64*)&FlagsAndRefCount) >> 32) & PtrMask) << (32 + UObjectPtrTrailingZeroes)) | (uintptr_t(ObjectPtrLow) << UObjectPtrTrailingZeroes);
		return (class UObjectBase*)Obj;
#else
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Object;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

	inline void SetObject(class UObjectBase* InObject)
	{
#if UE_ENABLE_FUOBJECT_ITEM_PACKING
		const uint32 ObjectPtrHi = ((uintptr_t)InObject & 0xFFFF00000000ULL) >> (32 + UObjectPtrTrailingZeroes);
		// This is not thread-safe, so SetObject should ever just be allowed at UObject creation time.
		FlagsAndRefCount |= ((uint64)ObjectPtrHi) << 32;
		ObjectPtrLow = ((uintptr_t)InObject >> UObjectPtrTrailingZeroes) & 0xFFFFFFFF;
#else
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Object = InObject;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

	UE_FORCEINLINE_HINT void SetOwnerIndex(int32 OwnerIndex)
	{
		ClusterRootIndex = OwnerIndex;
	}

	UE_FORCEINLINE_HINT int32 GetOwnerIndex() const
	{
		return ClusterRootIndex;
	}

	/** Encodes the cluster index in the ClusterRootIndex variable */
	UE_FORCEINLINE_HINT void SetClusterIndex(int32 ClusterIndex)
	{
		ClusterRootIndex = -ClusterIndex - 1;
	}

	/** Decodes the cluster index from the ClusterRootIndex variable */
	inline int32 GetClusterIndex() const
	{
		checkSlow(ClusterRootIndex < 0);
		return -ClusterRootIndex - 1;
	}

	inline int32 GetSerialNumber() const
	{
		// AutoRTFM doesn't like atomics even when relaxed, so we need to differentiate the code here
#if USING_INSTRUMENTATION || USING_THREAD_SANITISER
		// Resolving an old WeakPtr can race with new object construction.
		// Use a relaxed atomic here so that TSAN knows this is considered safe.
		return FPlatformAtomics::AtomicRead_Relaxed(&SerialNumber);
#else
		return SerialNumber;
#endif
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE
	UE_FORCEINLINE_HINT void SetRemoteId(FRemoteObjectId Id)
	{
		RemoteId = Id.GetLocalized();
	}
	UE_FORCEINLINE_HINT FRemoteObjectId GetRemoteId() const
	{
		return RemoteId;
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

	UE_FORCEINLINE_HINT void SetFlags(EInternalObjectFlags FlagsToSet)
	{
		ThisThreadAtomicallySetFlag(FlagsToSet);
	}

	inline EInternalObjectFlags GetFlags() const
	{
#if UE_ENABLE_FUOBJECT_ITEM_PACKING
		return EInternalObjectFlags(GetFlagsInternal() & FlagsMask);
#else
		return EInternalObjectFlags(GetFlagsInternal());
#endif
	}

	UE_FORCEINLINE_HINT void ClearFlags(EInternalObjectFlags FlagsToClear)
	{
		ThisThreadAtomicallyClearedFlag(FlagsToClear);
	}

	/**
	 * Uses atomics to clear the specified flag(s).
	 * @param FlagsToClear
	 * @return True if this call cleared the flag, false if it has been cleared by another thread.
	 */
	inline bool ThisThreadAtomicallyClearedFlag(EInternalObjectFlags FlagToClear)
	{
		checkf((int32(FlagToClear) & ~int32(EInternalObjectFlags_AllFlags)) == 0, TEXT("%d is not a valid internal flag value"), int32(FlagToClear));
		bool bIChangedIt = false;

		UE_AUTORTFM_OPEN
		{
			FlagToClear &= ~EInternalObjectFlags_ReachabilityFlags; // reachability flags can only be cleared by GC through *_ForGC functions
			if (!!(FlagToClear & EInternalObjectFlags_RootFlags))
			{
				bIChangedIt = ClearRootFlags(FlagToClear);
			}
			else
			{
				bIChangedIt = AtomicallyClearFlag_ForGC(FlagToClear);
			}
		};

		if (bIChangedIt)
		{
			// If we abort we undo clearing the flags we just set because `bIChangedIt` being true tells us that we were the ones who cleared the flag.
			// #jira SOL-8462: this could potentially undo flag changes made on a separate thread if an abort occurs.
			UE_AUTORTFM_ONABORT(this, FlagToClear)
			{
				ThisThreadAtomicallySetFlag(FlagToClear);
			};
		}

		return bIChangedIt;
	}

	/**
	 * Uses atomics to set the specified flag(s)
	 * @param FlagToSet
	 * @return True if this call set the flag, false if it has been set by another thread.
	 */
	inline bool ThisThreadAtomicallySetFlag(EInternalObjectFlags FlagToSet)
	{
		checkf((int32(FlagToSet) & ~int32(EInternalObjectFlags_AllFlags)) == 0, TEXT("%d is not a valid internal flag value"), int32(FlagToSet));
		bool bIChangedIt = false;

		UE_AUTORTFM_OPEN
		{
			FlagToSet &= ~EInternalObjectFlags_ReachabilityFlags; // reachability flags can only be cleared by GC through *_ForGC functions
			if (!!(FlagToSet & EInternalObjectFlags_RootFlags))
			{
				bIChangedIt = SetRootFlags(FlagToSet);
			}
			else
			{
				bIChangedIt = AtomicallySetFlag_ForGC(FlagToSet);
			}
		};

		if (bIChangedIt)
		{
			// If we abort we undo setting the flags we just set because `bIChangedIt` being true tells us that we were the ones who set the flag.
			// #jira SOL-8462: this could potentially undo flag changes made on a separate thread if an abort occurs.
			UE_AUTORTFM_ONABORT(this, FlagToSet)
			{
				ThisThreadAtomicallyClearedFlag(FlagToSet);
			};
		}
		
		return bIChangedIt;
	}

	UE_FORCEINLINE_HINT bool HasAnyFlags(EInternalObjectFlags InFlags) const
	{
		return !!(GetFlagsInternal() & int32(InFlags));
	}

	UE_FORCEINLINE_HINT bool HasAllFlags(EInternalObjectFlags InFlags) const
	{
		return (GetFlagsInternal() & int32(InFlags)) == int32(InFlags);
	}

	UE_FORCEINLINE_HINT bool IsUnreachable() const
	{
		return !!(GetFlagsInternal() & int32(EInternalObjectFlags::Unreachable));
	}

	UE_FORCEINLINE_HINT void SetGarbage()
	{
		ThisThreadAtomicallySetFlag(EInternalObjectFlags::Garbage);
	}
	UE_FORCEINLINE_HINT void ClearGarbage()
	{
		ThisThreadAtomicallyClearedFlag(EInternalObjectFlags::Garbage);
	}
	UE_FORCEINLINE_HINT bool IsGarbage() const
	{
		return !!(GetFlagsInternal() & int32(EInternalObjectFlags::Garbage));
	}

	UE_FORCEINLINE_HINT void SetRootSet()
	{
		ThisThreadAtomicallySetFlag(EInternalObjectFlags::RootSet);
	}
	UE_FORCEINLINE_HINT void ClearRootSet()
	{
		ThisThreadAtomicallyClearedFlag(EInternalObjectFlags::RootSet);
	}
	UE_FORCEINLINE_HINT bool IsRootSet() const
	{
		return !!(GetFlagsInternal() & int32(EInternalObjectFlags::RootSet));
	}

	UE_FORCEINLINE_HINT int32 GetRefCount() const
	{
		return GetRefCountInternal();
	}

	void AddRef()
	{
		UE_AUTORTFM_OPEN
		{
			const int64 NewRefCount = FPlatformAtomics::InterlockedIncrement(&FlagsAndRefCount);

			// Only dirty root when doing a transition from no root flags to a refcount.
			if ((NewRefCount & EInternalObjectFlags_RefCountMask) == 1 && (GetFlagsInternal(NewRefCount) & (int32)EInternalObjectFlags_RootFlags) == 0)
			{
				if (UE::GC::GIsIncrementalReachabilityPending)
				{
					// Setting any of the root flags on an object during incremental reachability requires a GC barrier
					// to make sure an object with root flags does not get Garbage Collected
					checkf(GetObject(), TEXT("Setting an internal object flag on a null object entry"));
					GetObject()->MarkAsReachable();
				}

				MarkRootAsDirty();
			}
		};

		// If the transaction is aborted we need to remember to release the reference we added in the open!
		UE_AUTORTFM_ONABORT(this)
		{
			this->ReleaseRef();
		};
	}

	void ReleaseRef()
	{
		UE_AUTORTFM_OPEN
		{
			// Only dirty root when doing a transition from a refcount to no refcounts and no other root flags present.
			const int64 NewRefCount = FPlatformAtomics::InterlockedDecrement(&FlagsAndRefCount);
			check((NewRefCount & EInternalObjectFlags_RefCountMask) >= 0);
			if ((NewRefCount & EInternalObjectFlags_RefCountMask) == 0 && (GetFlagsInternal(NewRefCount) & (int32)EInternalObjectFlags_RootFlags) == 0)
			{
				MarkRootAsDirty();
			}
		};

		// If the transaction is aborted we need to remember to re-add the reference we released in the open!
		// Note: this is different to how we handle ref counts in general in UE. Elsewhere we will eagerly
		// increment ref counts, but delay decrement. This is because `0` is normally the 'destroy the object'
		// trigger. But for UObject's `0` just clears the `RefCounted` flag, and since GC cannot run between
		// when we clear the flag and potentially re-add the flag below, the GC couldn't observe the state
		// that the object wasn't being actively ref-counted and thus could be collected. We do it differently
		// here because a bunch of the UObject systems depend on the ref count being correct after calls to
		// add/release.
		UE_AUTORTFM_ONABORT(this)
		{
			this->AddRef();
		};
	}

#if STATS || ENABLE_STATNAMEDEVENTS_UOBJECT
	COREUOBJECT_API void CreateStatID() const;
#endif

private:
	UE_FORCEINLINE_HINT static int32 GetFlagsInternal(int64 FlagsAndRefCount)
	{
		return (int32)(FlagsAndRefCount >> 32);
	}

	UE_FORCEINLINE_HINT int32 GetFlagsInternal() const
	{
		return GetFlagsInternal(FPlatformAtomics::AtomicRead_Relaxed((int64*)&FlagsAndRefCount));
	}
	
	UE_FORCEINLINE_HINT static int32 GetRefCountInternal(int64 FlagsAndRefCount)
	{
		return (int32)(FlagsAndRefCount & EInternalObjectFlags_RefCountMask);
	}

	UE_FORCEINLINE_HINT int32 GetRefCountInternal() const
	{
		return GetRefCountInternal(FPlatformAtomics::AtomicRead_Relaxed((int64*)&FlagsAndRefCount));
	}

	COREUOBJECT_API void MarkRootAsDirty();
	COREUOBJECT_API bool SetRootFlags(EInternalObjectFlags FlagsToSet);
	COREUOBJECT_API bool ClearRootFlags(EInternalObjectFlags FlagsToClear);

	/**
	 * Uses atomics to set the specified flag(s). GC internal version.
	 * @param FlagToSet
	 * @return True if this call set the flag, false if it has been set by another thread.
	 */
	inline bool AtomicallySetFlag_ForGC(EInternalObjectFlags FlagToSet)
	{
		static_assert(sizeof(int64) == sizeof(FlagsAndRefCount), "Flags must be 64-bit for atomics.");
		bool bIChangedIt = false;
		while (true)
		{
			const int64 StartValue = FPlatformAtomics::AtomicRead_Relaxed((int64*)&FlagsAndRefCount);
			const int32 StartFlagsValue = GetFlagsInternal(StartValue);
			if ((StartFlagsValue & int32(FlagToSet)) == int32(FlagToSet))
			{
				break;
			}
			const int32 NewFlagsValue = StartFlagsValue | int32(FlagToSet);
			const int64 NewValue = ((int64)NewFlagsValue << 32) | GetRefCountInternal(StartValue);
			if (FPlatformAtomics::InterlockedCompareExchange((int64*)&FlagsAndRefCount, NewValue, StartValue) == StartValue)
			{
				bIChangedIt = true;
				break;
			}
		}
		return bIChangedIt;
	}

	/**
	 * Uses atomics to set the specified flag(s). GC internal version.
	 * @param FlagToSet
	 * @param OldFlags    Set to the previous value of the flags before the change was made
	 * @param OldRefCount Set to the previous value of the refcount before the change was made
	 * @return True if this call set the flag, false if it has been set by another thread.
	 */
	inline bool AtomicallySetFlag_ForGC(EInternalObjectFlags FlagToSet, int32& OldFlags, int32& OldRefCount)
	{
		static_assert(sizeof(int64) == sizeof(FlagsAndRefCount), "Flags must be 64-bit for atomics.");
		bool bIChangedIt = false;
		while (true)
		{
			const int64 StartValue = FPlatformAtomics::AtomicRead_Relaxed((int64*)&FlagsAndRefCount);
			const int32 StartFlagsValue = GetFlagsInternal(StartValue);
			if ((StartFlagsValue & int32(FlagToSet)) == int32(FlagToSet))
			{
				OldFlags = StartFlagsValue;
				OldRefCount = GetRefCountInternal(StartValue);
				break;
			}
			const int32 NewFlagsValue = StartFlagsValue | int32(FlagToSet);
			const int64 NewValue = ((int64)NewFlagsValue << 32) | GetRefCountInternal(StartValue);
			if (FPlatformAtomics::InterlockedCompareExchange((int64*)&FlagsAndRefCount, NewValue, StartValue) == StartValue)
			{
				OldFlags = StartFlagsValue;
				OldRefCount = GetRefCountInternal(StartValue);
				bIChangedIt = true;
				break;
			}
		}
		return bIChangedIt;
	}

	/**
	 * Uses atomics to clear the specified flag(s). GC internal version
	 * @param FlagsToClear
	 * @return True if this call cleared the flag, false if it has been cleared by another thread.
	 */
	inline bool AtomicallyClearFlag_ForGC(EInternalObjectFlags FlagToClear)
	{
		static_assert(sizeof(int64) == sizeof(FlagsAndRefCount), "Flags must be 64-bit for atomics.");
		bool bIChangedIt = false;
		while (true)
		{
			const int64 StartValue = FPlatformAtomics::AtomicRead_Relaxed((int64*)&FlagsAndRefCount);
			const int32 StartFlagsValue = GetFlagsInternal(StartValue);
			if (!(StartFlagsValue & int32(FlagToClear)))
			{
				break;
			}
			const int32 NewFlagsValue = StartFlagsValue & ~int32(FlagToClear);
			const int64 NewValue = ((int64)NewFlagsValue << 32) | GetRefCountInternal(StartValue);
			if (FPlatformAtomics::InterlockedCompareExchange((int64*)&FlagsAndRefCount, NewValue, StartValue) == StartValue)
			{
				bIChangedIt = true;
				break;
			}
		}
		return bIChangedIt;
	}

	/**
	 * Uses atomics to clear the specified flag(s). GC internal version
	 * @param FlagsToClear
	 * @param OldFlags    Set to the previous value of the flags before the change was made
	 * @param OldRefCount Set to the previous value of the refcount before the change was made
	 * @return True if this call cleared the flag, false if it has been cleared by another thread.
	 */
	inline bool AtomicallyClearFlag_ForGC(EInternalObjectFlags FlagToClear, int32& OldFlags, int32& OldRefCount)
	{
		static_assert(sizeof(int64) == sizeof(FlagsAndRefCount), "Flags must be 64-bit for atomics.");
		bool bIChangedIt = false;
		while (true)
		{
			const int64 StartValue = FPlatformAtomics::AtomicRead_Relaxed((int64*)&FlagsAndRefCount);
			const int32 StartFlagsValue = GetFlagsInternal(StartValue);
			if (!(StartFlagsValue & int32(FlagToClear)))
			{
				OldFlags = StartFlagsValue;
				OldRefCount = GetRefCountInternal(StartValue);
				break;
			}
			const int32 NewFlagsValue = StartFlagsValue & ~int32(FlagToClear);
			const int64 NewValue = ((int64)NewFlagsValue << 32) | GetRefCountInternal(StartValue);
			if (FPlatformAtomics::InterlockedCompareExchange((int64*)&FlagsAndRefCount, NewValue, StartValue) == StartValue)
			{
				OldFlags = StartFlagsValue;
				OldRefCount = GetRefCountInternal(StartValue);
				bIChangedIt = true;
				break;
			}
		}
		return bIChangedIt;
	}
};

namespace UE::UObjectArrayPrivate
{
	COREUOBJECT_API void FailMaxUObjectCountExceeded(const int32 MaxUObjects, const int32 NewUObjectCount);

	inline void CheckUObjectLimitReached(const int32 NumUObjects, const int32 MaxUObjects, const int32 NewUObjectCount)
	{
		if ((NumUObjects + NewUObjectCount) > MaxUObjects)
		{
			FailMaxUObjectCountExceeded(MaxUObjects, NewUObjectCount);
		}
	}
};


/**
* Fixed size UObject array.
*/
class FFixedUObjectArray
{
	/** Static primary table to chunks of pointers **/
	TSAN_ATOMIC(FUObjectItem*) Objects;
	/** Number of elements we currently have **/
	TSAN_ATOMIC(int32) MaxElements;
	/** Current number of UObject slots */
	TSAN_ATOMIC(int32) NumElements;

public:

	FFixedUObjectArray()
		: Objects(nullptr)
		, MaxElements(0)
		, NumElements(0)
	{
	}

	~FFixedUObjectArray()
	{
		delete [] Objects;
	}

	/**
	* Expands the array so that Element[Index] is allocated. New pointers are all zero.
	* @param Index The Index of an element we want to be sure is allocated
	**/
	void PreAllocate(int32 InMaxElements)
	{
		check(!Objects);
		Objects = new FUObjectItem[InMaxElements];
		MaxElements = InMaxElements;
	}

	int32 AddSingle()
	{
		int32 Result = NumElements;
		UE::UObjectArrayPrivate::CheckUObjectLimitReached(NumElements, MaxElements, 1);
		check(Result == NumElements);
		++NumElements;
		FPlatformMisc::MemoryBarrier();
		check(Objects[Result].GetObject() == nullptr);
		return Result;
	}

	int32 AddRange(int32 Count)
	{
		int32 Result = NumElements + Count - 1;
		UE::UObjectArrayPrivate::CheckUObjectLimitReached(NumElements, MaxElements, Count);
		check(Result == (NumElements + Count - 1));
		NumElements += Count;
		FPlatformMisc::MemoryBarrier();
		check(Objects[Result].GetObject() == nullptr);
		return Result;
	}

	inline FUObjectItem const* GetObjectPtr(int32 Index) const
	{
		check(Index >= 0 && Index < NumElements);
		return &Objects[Index];
	}

	inline FUObjectItem* GetObjectPtr(int32 Index)
	{
		check(Index >= 0 && Index < NumElements);
		return &Objects[Index];
	}

	/**
	* Return the number of elements in the array
	* Thread safe, but you know, someone might have added more elements before this even returns
	* @return	the number of elements in the array
	**/
	UE_FORCEINLINE_HINT int32 Num() const
	{
		return NumElements;
	}

	/**
	* Return the number max capacity of the array
	* Thread safe, but you know, someone might have added more elements before this even returns
	* @return	the maximum number of elements in the array
	**/
	UE_FORCEINLINE_HINT int32 Capacity() const
	{
		return MaxElements;
	}

	/**
	* Return if this index is valid
	* Thread safe, if it is valid now, it is valid forever. Other threads might be adding during this call.
	* @param	Index	Index to test
	* @return	true, if this is a valid
	**/
	UE_FORCEINLINE_HINT bool IsValidIndex(int32 Index) const
	{
		return Index < Num() && Index >= 0;
	}
	/**
	* Return a reference to an element
	* @param	Index	Index to return
	* @return	a reference to the pointer to the element
	* Thread safe, if it is valid now, it is valid forever. This might return nullptr, but by then, some other thread might have made it non-nullptr.
	**/
	inline FUObjectItem const& operator[](int32 Index) const
	{
		FUObjectItem const* ItemPtr = GetObjectPtr(Index);
		check(ItemPtr);
		return *ItemPtr;
	}

	inline FUObjectItem& operator[](int32 Index)
	{
		FUObjectItem* ItemPtr = GetObjectPtr(Index);
		check(ItemPtr);
		return *ItemPtr;
	}

	/**
	* Return a naked pointer to the fundamental data structure for debug visualizers.
	**/
	UObjectBase*** GetRootBlockForDebuggerVisualizers()
	{
		return nullptr;
	}
};

/**
* Simple array type that can be expanded without invalidating existing entries.
* This is critical to thread safe FNames.
* @param ElementType Type of the pointer we are storing in the array
* @param MaxTotalElements absolute maximum number of elements this array can ever hold
* @param ElementsPerChunk how many elements to allocate in a chunk
**/
class FChunkedFixedUObjectArray
{
	enum
	{
		NumElementsPerChunk = 64 * 1024,
	};

	/** Primary table to chunks of pointers **/
	FUObjectItem** Objects;
	/** Number of elements we currently have **/
	TSAN_ATOMIC(int32) NumElements;
	/** Maximum number of elements **/
	TSAN_ATOMIC(int32) MaxElements;
	/** Number of chunks we currently have **/
	TSAN_ATOMIC(int32) NumChunks;
	/** Maximum number of chunks **/
	int32 MaxChunks;
	/** If requested, a contiguous memory where all objects are allocated **/
	FUObjectItem* PreAllocatedObjects;

	static constexpr bool bFUObjectItemIsPacked = UE_ENABLE_FUOBJECT_ITEM_PACKING;


	/**
	* Allocates new chunk for the array
	**/
	void ExpandChunksToIndex(int32 Index)
	{
		check(Index >= 0 && Index < MaxElements);
		int32 ChunkIndex = Index / NumElementsPerChunk;
		while (ChunkIndex >= NumChunks)
		{
			// add a chunk, and make sure nobody else tries
			FUObjectItem** Chunk = &Objects[NumChunks];
			FUObjectItem* NewChunk = new FUObjectItem[NumElementsPerChunk];
			if (FPlatformAtomics::InterlockedCompareExchangePointer((void**)Chunk, NewChunk, nullptr))
			{
				// someone else beat us to the add, we don't support multiple concurrent adds
				check(0);
			}
			else
			{
				NumChunks++;
				check(NumChunks <= MaxChunks);
			}
		}
		check(ChunkIndex < NumChunks && Objects[ChunkIndex]); // should have a valid pointer now
	}
    
public:

	/** Constructor : Probably not thread safe **/
	FChunkedFixedUObjectArray()
		: Objects(nullptr)
		, NumElements(0)
		, MaxElements(0)
		, NumChunks(0)
		, MaxChunks(0)
		, PreAllocatedObjects(nullptr)
	{
	}

	~FChunkedFixedUObjectArray()
	{
		if (!PreAllocatedObjects)
		{
			for (int32 ChunkIndex = 0; ChunkIndex < MaxChunks; ++ChunkIndex)
			{
				delete[] Objects[ChunkIndex];
			}
		}
		else
		{
			delete[] PreAllocatedObjects;
		}
		delete[] Objects;
	}

	/**
	* Expands the array so that Element[Index] is allocated. New pointers are all zero.
	* @param Index The Index of an element we want to be sure is allocated
	**/
	void PreAllocate(int32 InMaxElements, bool bPreAllocateChunks)
	{
		check(!Objects);
		MaxChunks = InMaxElements / NumElementsPerChunk + 1;
		MaxElements = MaxChunks * NumElementsPerChunk;
		Objects = new FUObjectItem*[MaxChunks];
		FMemory::Memzero(Objects, sizeof(FUObjectItem*) * MaxChunks);
		if (bPreAllocateChunks)
		{
			// Fully allocate all chunks as contiguous memory
			PreAllocatedObjects = new FUObjectItem[MaxElements];
			for (int32 ChunkIndex = 0; ChunkIndex < MaxChunks; ++ChunkIndex)
			{
				Objects[ChunkIndex] = PreAllocatedObjects + ChunkIndex * NumElementsPerChunk;
			}
			NumChunks = MaxChunks;
		}
	}

	/**
	* Return the number of elements in the array
	* Thread safe, but you know, someone might have added more elements before this even returns
	* @return	the number of elements in the array
	**/
	UE_FORCEINLINE_HINT int32 Num() const
	{
		return NumElements;
	}

	/**
	* Return the number max capacity of the array
	* Thread safe, but you know, someone might have added more elements before this even returns
	* @return	the maximum number of elements in the array
	**/
	UE_FORCEINLINE_HINT int32 Capacity() const
	{
		return MaxElements;
	}

	/**
	* Return if this index is valid
	* Thread safe, if it is valid now, it is valid forever. Other threads might be adding during this call.
	* @param	Index	Index to test
	* @return	true, if this is a valid
	**/
	UE_FORCEINLINE_HINT bool IsValidIndex(int32 Index) const
	{
		return Index < Num() && Index >= 0;
	}

	/**
	* Return a pointer to the pointer to a given element
	* @param Index The Index of an element we want to retrieve the pointer-to-pointer for
	**/
	inline FUObjectItem const* GetObjectPtr(int32 Index) const
	{
		const uint32 ChunkIndex = (uint32)Index / NumElementsPerChunk;
		const uint32 WithinChunkIndex = (uint32)Index % NumElementsPerChunk;
		checkf(IsValidIndex(Index), TEXT("IsValidIndex(%d)"), Index);
		checkf(ChunkIndex < (uint32)NumChunks, TEXT("ChunkIndex (%d) < NumChunks (%d)"), ChunkIndex, (int32)NumChunks);
		checkf(Index < MaxElements, TEXT("Index (%d) < MaxElements (%d)"), Index, (int32)MaxElements);
		FUObjectItem* Chunk = Objects[ChunkIndex];
		check(Chunk);
		return Chunk + WithinChunkIndex;
	}
	inline FUObjectItem* GetObjectPtr(int32 Index)
	{
		const uint32 ChunkIndex = (uint32)Index / NumElementsPerChunk;
		const uint32 WithinChunkIndex = (uint32)Index % NumElementsPerChunk;
		checkf(IsValidIndex(Index), TEXT("IsValidIndex(%d)"), Index);
		checkf(ChunkIndex < (uint32)NumChunks, TEXT("ChunkIndex (%d) < NumChunks (%d)"), ChunkIndex, (int32)NumChunks);
		checkf(Index < MaxElements, TEXT("Index (%d) < MaxElements (%d)"), Index, (int32)MaxElements);
		FUObjectItem* Chunk = Objects[ChunkIndex];
		check(Chunk);
		return Chunk + WithinChunkIndex;
	}

	inline void PrefetchObjectPtr(int32 Index) const
	{
		const uint32 ChunkIndex = (uint32)Index / NumElementsPerChunk;
		const uint32 WithinChunkIndex = (uint32)Index % NumElementsPerChunk;
		const FUObjectItem* Chunk = Objects[ChunkIndex];
		FPlatformMisc::Prefetch(Chunk + WithinChunkIndex);
	}

	/**
	* Return a reference to an element
	* @param	Index	Index to return
	* @return	a reference to the pointer to the element
	* Thread safe, if it is valid now, it is valid forever. This might return nullptr, but by then, some other thread might have made it non-nullptr.
	**/
	inline FUObjectItem const& operator[](int32 Index) const
	{
		FUObjectItem const* ItemPtr = GetObjectPtr(Index);
		check(ItemPtr);
		return *ItemPtr;
	}
	inline FUObjectItem& operator[](int32 Index)
	{
		FUObjectItem* ItemPtr = GetObjectPtr(Index);
		check(ItemPtr);
		return *ItemPtr;
	}

	int32 AddRange(int32 NumToAdd)
	{
		int32 Result = NumElements;
		UE::UObjectArrayPrivate::CheckUObjectLimitReached(Result, MaxElements, NumToAdd);
		ExpandChunksToIndex(Result + NumToAdd - 1);
		NumElements += NumToAdd;
		return Result;
	}

	int32 AddSingle()
	{
		return AddRange(1);
	}

	/**
	* Return a naked pointer to the fundamental data structure for debug visualizers.
	**/
	FUObjectItem*** GetRootBlockForDebuggerVisualizers()
	{
		return nullptr;
	}
    
    int64 GetAllocatedSize() const
    {
        return MaxChunks * sizeof(FUObjectItem*) + NumChunks * NumElementsPerChunk * sizeof(FUObjectItem);
    }
};

/***
*
* FUObjectArray replaces the functionality of GObjObjects and UObject::Index
*
* Note the layout of this data structure is mostly to emulate the old behavior and minimize code rework during code restructure.
* Better data structures could be used in the future, for example maybe all that is needed is a TSet<UObject *>
* One has to be a little careful with this, especially with the GC optimization. I have seen spots that assume
* that non-GC objects come before GC ones during iteration.
*
**/
class FUObjectArray
{
	friend class UObject;
	friend COREUOBJECT_API UObject* StaticAllocateObject(const UClass*, UObject*, FName, EObjectFlags, EInternalObjectFlags, bool, bool*, UPackage*, int32, FRemoteObjectId, class FGCReconstructionGuard*);

private:
	/**
	 * Reset the serial number from the game thread to invalidate all weak object pointers to it
	 *
	 * @param Object to reset
	 */
	COREUOBJECT_API void ResetSerialNumber(UObjectBase* Object);

public:

	enum ESerialNumberConstants
	{
		START_SERIAL_NUMBER = 1000,
	};

	/**
	 * Base class for UObjectBase create class listeners
	 */
	class FUObjectCreateListener
	{
	public:
		virtual ~FUObjectCreateListener() = default;
		/**
		* Provides notification that a UObjectBase has been added to the uobject array
		 *
		 * @param Object object that has been added
		 * @param Index	index of object that is being added
		 */
		virtual void NotifyUObjectCreated(const class UObjectBase *Object, int32 Index)=0;

		/**
		 * Called when UObject Array is being shut down, this is where all listeners should be removed from it 
		 */
		virtual void OnUObjectArrayShutdown()=0;
	};

	/**
	 * Base class for UObjectBase delete class listeners
	 */
	class FUObjectDeleteListener
	{
	public:
		virtual ~FUObjectDeleteListener() = default;

		/**
		 * Provides notification that a UObjectBase has been removed from the uobject array
		 *
		 * @param Object object that has been destroyed
		 * @param Index	index of object that is being deleted
		 */
		virtual void NotifyUObjectDeleted(const class UObjectBase *Object, int32 Index)=0;

		/**
		 * Called when UObject Array is being shut down, this is where all listeners should be removed from it
		 */
		virtual void OnUObjectArrayShutdown() = 0;

		/**
		 * Returns the size of heap memory allocated internally by this listener
		 */
		virtual SIZE_T GetAllocatedSize() const
		{
			return 0;
		}
	};

	/**
	 * Constructor, initializes to no permanent object pool
	 */
	COREUOBJECT_API FUObjectArray();

	/**
	 * Allocates and initializes the permanent object pool
	 *
	 * @param MaxUObjects maximum number of UObjects that can ever exist in the array
	 * @param MaxObjectsNotConsideredByGC number of objects in the permanent object pool
	 */
	COREUOBJECT_API void AllocateObjectPool(int32 MaxUObjects, int32 MaxObjectsNotConsideredByGC, bool bPreAllocateObjectArray);

	/**
	 * Disables the disregard for GC optimization.
	 *
	 */
	COREUOBJECT_API void DisableDisregardForGC();

	/**
	* If there's enough slack in the disregard pool, we can re-open it and keep adding objects to it
	*/
	COREUOBJECT_API void OpenDisregardForGC();

	/**
	 * After the initial load, this closes the disregard pool so that new object are GC-able
	 */
	COREUOBJECT_API void CloseDisregardForGC();

	/** Returns true if the disregard for GC pool is open */
	bool IsOpenForDisregardForGC() const
	{
		return OpenForDisregardForGC;
	}

	/**
	 * indicates if the disregard for GC optimization is active
	 *
	 * @return true if MaxObjectsNotConsideredByGC is greater than zero; this indicates that the disregard for GC optimization is enabled
	 */
	bool DisregardForGCEnabled() const 
	{ 
		return MaxObjectsNotConsideredByGC > 0;
	}

	/**
	 * Adds a uobject to the global array which is used for uobject iteration
	 *
	 * @param	Object Object to allocate an index for
	 * @param	InitialFlags Flags to set in the object array before the object pointer becomes visible to other threads. 
	 * @param	AlreadyAllocatedIndex already allocated internal index to use, negative value means allocate a new index
	 * @param	SerialNumber serial number to use
	 */
	COREUOBJECT_API void AllocateUObjectIndex(class UObjectBase* Object, EInternalObjectFlags InitialFlags, int32 AlreadyAllocatedIndex = -1, int32 SerialNumber = 0, FRemoteObjectId RemoteId = FRemoteObjectId());

	/**
	 * Returns a UObject index top to the global uobject array
	 *
	 * @param Object object to free
	 */
	COREUOBJECT_API void FreeUObjectIndex(class UObjectBase* Object);

	/**
	 * Returns the index of a UObject. Be advised this is only for very low level use.
	 *
	 * @param Object object to get the index of
	 * @return index of this object
	 */
	UE_FORCEINLINE_HINT int32 ObjectToIndex(const class UObjectBase* Object) const
	{
		return Object->InternalIndex;
	}

	/**
	 * Returns the UObject corresponding to index. Be advised this is only for very low level use.
	 *
	 * @param Index index of object to return
	 * @return Object at this index
	 */
	inline FUObjectItem* IndexToObject(int32 Index)
	{
		check(Index >= 0);
		if (Index < ObjObjects.Num())
		{
			return const_cast<FUObjectItem*>(&ObjObjects[Index]);
		}
		return nullptr;
	}

	UE_FORCEINLINE_HINT FUObjectItem* IndexToObjectUnsafeForGC(int32 Index)
	{
		return const_cast<FUObjectItem*>(&ObjObjects[Index]);
	}

	inline FUObjectItem* IndexToObject(int32 Index, bool bEvenIfGarbage)
	{
		FUObjectItem* ObjectItem = IndexToObject(Index);
		if (ObjectItem && ObjectItem->GetObject())
		{
			if (!bEvenIfGarbage && ObjectItem->HasAnyFlags(EInternalObjectFlags::Garbage))
			{
				ObjectItem = nullptr;
			}
		}
		return ObjectItem;
	}

	inline FUObjectItem* ObjectToObjectItem(const UObjectBase* Object)
	{
		FUObjectItem* ObjectItem = IndexToObject(Object->InternalIndex);
		return ObjectItem;
	}

	inline bool IsValid(FUObjectItem* ObjectItem, bool bEvenIfGarbage)
	{
		if (ObjectItem)
		{
			return bEvenIfGarbage ? !ObjectItem->IsUnreachable() : !(ObjectItem->HasAnyFlags(EInternalObjectFlags::Unreachable | EInternalObjectFlags::Garbage));
		}
		return false;
	}

	inline FUObjectItem* IndexToValidObject(int32 Index, bool bEvenIfGarbage)
	{
		FUObjectItem* ObjectItem = IndexToObject(Index);
		return IsValid(ObjectItem, bEvenIfGarbage) ? ObjectItem : nullptr;
	}

	inline bool IsValid(int32 Index, bool bEvenIfGarbage)
	{
		// This method assumes Index points to a valid object.
		FUObjectItem* ObjectItem = IndexToObject(Index);
		return IsValid(ObjectItem, bEvenIfGarbage);
	}

	UE_FORCEINLINE_HINT bool IsStale(FUObjectItem* ObjectItem, bool bIncludingGarbage)
	{
		// This method assumes ObjectItem is valid.
		return bIncludingGarbage ? (ObjectItem->HasAnyFlags(EInternalObjectFlags::Unreachable | EInternalObjectFlags::Garbage)) : (ObjectItem->IsUnreachable());
	}

	inline bool IsStale(int32 Index, bool bIncludingGarbage)
	{
		// This method assumes Index points to a valid object.
		FUObjectItem* ObjectItem = IndexToObject(Index);
		if (ObjectItem)
		{
			return IsStale(ObjectItem, bIncludingGarbage);
		}
		return true;
	}

	/** Returns the index of the first object outside of the disregard for GC pool */
	UE_FORCEINLINE_HINT int32 GetFirstGCIndex() const
	{
		return ObjFirstGCIndex;
	}

	/**
	 * Adds a new listener for object creation
	 *
	 * @param Listener listener to notify when an object is deleted
	 */
	COREUOBJECT_API void AddUObjectCreateListener(FUObjectCreateListener* Listener);

	/**
	 * Removes a listener for object creation
	 *
	 * @param Listener listener to remove
	 */
	COREUOBJECT_API void RemoveUObjectCreateListener(FUObjectCreateListener* Listener);

	/**
	 * Adds a new listener for object deletion
	 *
	 * @param Listener listener to notify when an object is deleted
	 */
	COREUOBJECT_API void AddUObjectDeleteListener(FUObjectDeleteListener* Listener);

	/**
	 * Removes a listener for object deletion
	 *
	 * @param Listener listener to remove
	 */
	COREUOBJECT_API void RemoveUObjectDeleteListener(FUObjectDeleteListener* Listener);

	/**
	 * Removes an object from delete listeners
	 *
	 * @param Object to remove from delete listeners
	 */
	COREUOBJECT_API void RemoveObjectFromDeleteListeners(UObjectBase* Object);

	/**
	 * Checks if a UObject pointer is valid
	 *
	 * @param	Object object to test for validity
	 * @return	true if this index is valid
	 */
	COREUOBJECT_API bool IsValid(const UObjectBase* Object) const;

	/** Checks if the object index is valid. */
	UE_FORCEINLINE_HINT bool IsValidIndex(const UObjectBase* Object) const 
	{ 
		return ObjObjects.IsValidIndex(Object->InternalIndex);
	}

	/**
	 * Returns true if this object index is "disregard for GC"
	 *
	 * @param Index object index to get for disregard for GC
	 * @return true if this object index is disregard for GC
	 */
	UE_FORCEINLINE_HINT bool IsIndexDisregardForGC(int32 ObjectIndex) const
	{
		return ObjectIndex <= ObjLastNonGCIndex;
	}

	/**
	 * Returns true if this object is "disregard for GC"...same results as the legacy RF_DisregardForGC flag
	 *
	 * @param Object object to get for disregard for GC
	 * @return true if this object is disregard for GC
	 */
	UE_FORCEINLINE_HINT bool IsDisregardForGC(const class UObjectBase* Object) const
	{
		return IsIndexDisregardForGC(Object->InternalIndex);
	}
	/**
	 * Returns the size of the global UObject array, some of these might be unused
	 *
	 * @return	the number of UObjects in the global array
	 */
	UE_FORCEINLINE_HINT int32 GetObjectArrayNum() const 
	{ 
		return ObjObjects.Num();
	}

	/**
	 * Returns the size of the global UObject array minus the number of permanent objects
	 *
	 * @return	the number of UObjects in the global array
	 */
	UE_FORCEINLINE_HINT int32 GetObjectArrayNumMinusPermanent() const 
	{ 
		return ObjObjects.Num() - (ObjLastNonGCIndex + 1);
	}

	/**
	 * Returns the number of permanent objects
	 *
	 * @return	the number of permanent objects
	 */
	UE_FORCEINLINE_HINT int32 GetObjectArrayNumPermanent() const 
	{ 
		return ObjLastNonGCIndex + 1;
	}

	/**
	 * Returns the number of actual object indices that are claimed (the total size of the global object array minus
	 * the number of available object array elements
	 *
	 * @return	The number of objects claimed
	 */
	int32 GetObjectArrayNumMinusAvailable() const
	{
		return ObjObjects.Num() - ObjAvailableListEstimateCount;
	}

	/**
	* Returns the estimated number of object indices available for allocation
	*/
	int32 GetObjectArrayEstimatedAvailable() const
	{
		return ObjObjects.Capacity() - GetObjectArrayNumMinusAvailable();
	}

	/**
	* Returns the estimated number of object indices available for allocation
	*/
	int32 GetObjectArrayCapacity() const
	{
		return ObjObjects.Capacity();
	}

	/**
	 * Clears some internal arrays to get rid of false memory leaks
	 */
	COREUOBJECT_API void ShutdownUObjectArray();

	/**
	* Given a UObject index return the serial number. If it doesn't have a serial number, give it one. Threadsafe.
	* @param Index - UObject Index
	* @return - the serial number for this UObject
	*/
	COREUOBJECT_API int32 AllocateSerialNumber(int32 Index);

	/**
	* Given a UObject index return the serial number. If it doesn't have a serial number, return 0. Threadsafe.
	* @param Index - UObject Index
	* @return - the serial number for this UObject
	*/
	inline int32 GetSerialNumber(int32 Index)
	{
		FUObjectItem* ObjectItem = IndexToObject(Index);
		checkSlow(ObjectItem);
		return ObjectItem->GetSerialNumber();
	}

#if UE_WITH_REMOTE_OBJECT_HANDLE

	/**
	* Given a UObject index return the remote id.
	* @param Index - UObject Index
	* @return - the remote id for this UObject
	*/
	inline FRemoteObjectId GetRemoteId(int32 Index)
	{
		FUObjectItem* ObjectItem = IndexToObject(Index);
		checkSlow(ObjectItem);
		return ObjectItem->GetRemoteId();
	}
#endif

	/** Locks the internal object array mutex */
	void LockInternalArray() const
	{
#if THREADSAFE_UOBJECTS
		ObjObjectsCritical.Lock();
#else
		check(IsInGameThread());
#endif
	}

	/** Unlocks the internal object array mutex */
	void UnlockInternalArray() const
	{
#if THREADSAFE_UOBJECTS
		ObjObjectsCritical.Unlock();
#endif
	}

	/**
	 * Low level iterator.
	 */
	class TIterator
	{
	public:
		enum EEndTagType
		{
			EndTag
		};

		/**
		 * Constructor
		 *
		 * @param	InArray				the array to iterate on
		 * @param	bOnlyGCedObjects	if true, skip all of the permanent objects
		 */
		TIterator( const FUObjectArray& InArray, bool bOnlyGCedObjects = false ) :	
			Array(InArray),
			Index(-1),
			CurrentObject(nullptr)
		{
			if (bOnlyGCedObjects)
			{
				Index = Array.ObjLastNonGCIndex;
			}
			Advance();
		}

		/**
		 * Constructor
		 *
		 * @param	InArray				the array to iterate on
		 * @param	bOnlyGCedObjects	if true, skip all of the permanent objects
		 */
		TIterator( EEndTagType, const TIterator& InIter ) :	
			Array (InIter.Array),
			Index(Array.ObjObjects.Num())
		{
		}

		/**
		 * Iterator advance
		 */
		UE_FORCEINLINE_HINT void operator++()
		{
			Advance();
		}

		bool operator==(const TIterator& Rhs) const { return Index == Rhs.Index; }
		bool operator!=(const TIterator& Rhs) const { return Index != Rhs.Index; }

		/** Conversion to "bool" returning true if the iterator is valid. */
		UE_FORCEINLINE_HINT explicit operator bool() const
		{ 
			return !!CurrentObject;
		}
		/** inverse of the "bool" operator */
		UE_FORCEINLINE_HINT bool operator !() const 
		{
			return !(bool)*this;
		}

		UE_FORCEINLINE_HINT int32 GetIndex() const
		{
			return Index;
		}

	protected:

		/**
		 * Dereferences the iterator with an ordinary name for clarity in derived classes
		 *
		 * @return	the UObject at the iterator
		 */
		UE_FORCEINLINE_HINT FUObjectItem* GetObject() const
		{ 
			return CurrentObject;
		}
		/**
		 * Iterator advance with ordinary name for clarity in subclasses
		 * @return	true if the iterator points to a valid object, false if iteration is complete
		 */
		inline bool Advance()
		{
			//@todo UE check this for LHS on Index on consoles
			FUObjectItem* NextObject = nullptr;
			CurrentObject = nullptr;
			while(++Index < Array.GetObjectArrayNum())
			{
				NextObject = const_cast<FUObjectItem*>(&Array.ObjObjects[Index]);
				if (NextObject->GetObject())
				{
					CurrentObject = NextObject;
					return true;
				}
			}
			return false;
		}

		/** Gets the array this iterator iterates over */
		const FUObjectArray& GetIteratedArray() const
		{
			return Array;
		}

	private:
		/** the array that we are iterating on, probably always GUObjectArray */
		const FUObjectArray& Array;
		/** index of the current element in the object array */
		int32 Index;
		/** Current object */
		mutable FUObjectItem* CurrentObject;
	};

	/** Locks the mutex protecting the list of delete listeners */
	void LockUObjectDeleteListeners()
	{
#if THREADSAFE_UOBJECTS
		UObjectDeleteListenersCritical.Lock();
#endif
	}

	/** Unlocks the mutex protecting the list of delete listeners */
	void UnlockUObjectDeleteListeners()
	{
#if THREADSAFE_UOBJECTS
		UObjectDeleteListenersCritical.Unlock();
#endif
	}

private:

	//typedef TStaticIndirectArrayThreadSafeRead<UObjectBase, 8 * 1024 * 1024 /* Max 8M UObjects */, 16384 /* allocated in 64K/128K chunks */ > TUObjectArray;
	typedef FChunkedFixedUObjectArray TUObjectArray;

	// note these variables are left with the Obj prefix so they can be related to the historical GObj versions

	/** Array of all live objects. Placed first so hot fields (Objects, NumElements) are at offset 0. */
	TUObjectArray ObjObjects;
	/** First index into objects array taken into account for GC.							*/
	int32 ObjFirstGCIndex;
	/** Index pointing to last object created in range disregarded for GC.					*/
	int32 ObjLastNonGCIndex;
	/** Maximum number of objects in the disregard for GC Pool */
	int32 MaxObjectsNotConsideredByGC;

	/** If true this is the intial load and we should load objects int the disregarded for GC range.	*/
	bool OpenForDisregardForGC;
	/** Synchronization object for all live objects.											*/
	mutable FTransactionallySafeCriticalSection ObjObjectsCritical;
	/** Available object indices.											*/
	TArray<int32> ObjAvailableList;
	/** We need the estimate without taking a lock on the array to fetch it from ObjAvailableList. */
	TSAN_ATOMIC(int32) ObjAvailableListEstimateCount = 0;

	/**
	 * Array of things to notify when a UObjectBase is created
	 */
	TArray<FUObjectCreateListener* > UObjectCreateListeners;
#if THREADSAFE_UOBJECTS
	// Needs to support recursivicity because NotifyUObjectCreated callback might call RemoveUObjectCreateListener.
	mutable FTransactionallySafeCriticalSection UObjectCreateListenersCritical;
#endif

	/**
	 * Array of things to notify when a UObjectBase is destroyed
	 */
	TArray<FUObjectDeleteListener* > UObjectDeleteListeners;
#if THREADSAFE_UOBJECTS
	mutable FTransactionallySafeCriticalSection UObjectDeleteListenersCritical;
#endif

	/** Current primary serial number **/
	FThreadSafeCounter	PrimarySerialNumber;

	/** If set to false object indices won't be recycled to the global pool and can be explicitly reused when creating new objects */
	bool bShouldRecycleObjectIndices = true;

public:

	/** INTERNAL USE ONLY: gets the internal FUObjectItem array */
	TUObjectArray& GetObjectItemArrayUnsafe()
	{
		return ObjObjects;
	}
    
	const TUObjectArray& GetObjectItemArrayUnsafe() const
	{
		return ObjObjects;
	}

    SIZE_T GetAllocatedSize() const
    {
		UE::TScopeLock ObjListLock(ObjObjectsCritical);
#if THREADSAFE_UOBJECTS
		UE::TScopeLock CreateListenersLock(UObjectCreateListenersCritical);
		UE::TScopeLock DeleteListenersLock(UObjectDeleteListenersCritical);
#endif
        return ObjObjects.GetAllocatedSize() + ObjAvailableList.GetAllocatedSize() + UObjectCreateListeners.GetAllocatedSize() + UObjectDeleteListeners.GetAllocatedSize();
    }

	SIZE_T GetDeleteListenersAllocatedSize(int32* OutNumListeners = nullptr) const
	{
#if THREADSAFE_UOBJECTS
		UE::TScopeLock ListenersLock(UObjectDeleteListenersCritical);
#endif
		SIZE_T AllocatedSize = 0;
		for (FUObjectDeleteListener* Listener : UObjectDeleteListeners)
		{
			AllocatedSize += Listener->GetAllocatedSize();
		}
		if (OutNumListeners)
		{
			*OutNumListeners = UObjectDeleteListeners.Num();
		}
		return AllocatedSize;
	}

	COREUOBJECT_API void DumpUObjectCountsToLog() const;
};

/** UObject cluster. Groups UObjects into a single unit for GC. */
struct FUObjectCluster
{
	FUObjectCluster()
		: RootIndex(INDEX_NONE)
		, bNeedsDissolving(false)
	{}

	/** Root object index */
	int32 RootIndex;
	/** Objects that belong to this cluster */
	TArray<int32> Objects;
	/** Other clusters referenced by this cluster */
	TArray<int32> ReferencedClusters;
	/** Objects that could not be added to the cluster but still need to be referenced by it */
	TArray<int32> MutableObjects;
	/** List of clusters that direcly reference this cluster. Used when dissolving a cluster. */
	TArray<int32> ReferencedByClusters;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/** All verse cells are considered mutable.  They will just be added directly to verse gc when the cluster is marked */
	TArray<Verse::VCell*> MutableCells;
#endif

	/** Cluster needs dissolving, probably due to PendingKill reference */
	bool bNeedsDissolving;
};

class FUObjectClusterContainer
{
	/** List of all clusters */
	TArray<FUObjectCluster> Clusters;
	/** List of available cluster indices */
	TArray<int32> FreeClusterIndices;
	/** Number of allocated clusters */
	int32 NumAllocatedClusters;
	/** Clusters need dissolving, probably due to PendingKill reference */
	bool bClustersNeedDissolving;

	/** Dissolves a cluster */
	COREUOBJECT_API void DissolveCluster(FUObjectCluster& Cluster);

public:

	COREUOBJECT_API FUObjectClusterContainer();

	inline FUObjectCluster& operator[](int32 Index)
	{
		checkf(Index >= 0 && Index < Clusters.Num(), TEXT("Cluster index %d out of range [0, %d]"), Index, Clusters.Num());
		return Clusters[Index];
	}

	/** Returns an index to a new cluster */
	COREUOBJECT_API int32 AllocateCluster(int32 InRootObjectIndex);

	/** Frees the cluster at the specified index */
	COREUOBJECT_API void FreeCluster(int32 InClusterIndex);

	/**
	* Gets the cluster the specified object is a root of or belongs to.
	* @Param ClusterRootOrObjectFromCluster Root cluster object or object that belongs to a cluster
	*/
	COREUOBJECT_API FUObjectCluster* GetObjectCluster(UObjectBaseUtility* ClusterRootOrObjectFromCluster);


	/** 
	 * Dissolves a cluster and all clusters that reference it 
	 * @Param ClusterRootOrObjectFromCluster Root cluster object or object that belongs to a cluster
	 */
	COREUOBJECT_API void DissolveCluster(UObjectBaseUtility* ClusterRootOrObjectFromCluster);

	/** 
	 * Dissolve all clusters marked for dissolving 
	 * @param bForceDissolveAllClusters if true, dissolves all clusters even if they're not marked for dissolving
	 */
	COREUOBJECT_API void DissolveClusters(bool bForceDissolveAllClusters = false);

	/** Dissolve the specified cluster and all clusters that reference it */
	COREUOBJECT_API void DissolveClusterAndMarkObjectsAsUnreachable(FUObjectItem* RootObjectItem);

	/*** Returns the minimum cluster size as specified in ini settings */
	COREUOBJECT_API int32 GetMinClusterSize() const;

	/** Gets the clusters array (for internal use only!) */
	TArray<FUObjectCluster>& GetClustersUnsafe() 
	{ 
		return Clusters;  
	}

	/** Returns the number of currently allocated clusters */
	int32 GetNumAllocatedClusters() const
	{
		return NumAllocatedClusters;
	}

	/** Lets the FUObjectClusterContainer know some clusters need dissolving */
	void SetClustersNeedDissolving()
	{
		bClustersNeedDissolving = true;
	}
	
	/** Checks if any clusters need dissolving */
	bool ClustersNeedDissolving() const
	{
		return bClustersNeedDissolving;
	}
};

/** Global UObject allocator							*/
extern COREUOBJECT_API FUObjectArray GUObjectArray;
extern COREUOBJECT_API FUObjectClusterContainer GUObjectClusters;

/**
	* Static version of IndexToObject for use with TWeakObjectPtr.
	*/
struct FIndexToObject
{
	static inline class UObjectBase* IndexToObject(int32 Index, bool bEvenIfGarbage)
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(Index, bEvenIfGarbage);
		return ObjectItem ? ObjectItem->GetObject() : nullptr;
	}
};

namespace verse
{
COREUOBJECT_API bool CanAllocateUObjects();
}
