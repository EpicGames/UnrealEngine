// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectBase.h: Base class for UObject, defines low level functionality
=============================================================================*/

#pragma once

// IWYU pragma: begin_exports

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformAtomics.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Stats/Stats.h"
#include "UObject/NameTypes.h"
#include "UObject/RemoteObjectTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/ObjectPtr.h"
#include "AutoRTFM.h"

class UClass;
class UEnum;
class UObject;
class UPackage;
class UScriptStruct;

namespace UE::CoreUObject::Private
{
	struct FObjectHandleUtils;
	struct FPartialClass;
	class FCompiledInObjectLinker;
}

// IWYU pragma: end_exports

namespace UE::CoreUObject
{
#if UE_WITH_CONSTINIT_UOBJECT
	// Finish constructing compiled-in objects with operations that can only be done at runtime
	// (i.e. creating FNames) but do not create any class default objects and minimize callbacks
	// to other systems (i.e. FCoreUObjectDelegates::CompiledInUObjectsRegisteredDelegate)
	// Expected to be called from Launch before any systems that want basic access to compiled-in
	// objects such as classes and structs for reflection (i.e. reading config via reflection).
	COREUOBJECT_API void ConstructCompiledInObjects();
#endif 
}

// If FName is 4 bytes than we can use padding after it to store internal object list index and use array instead of a hash map for lookup in UObjectHash.cpp
// This might change the each UClass' object list iteration order
#if !defined(UE_STORE_OBJECT_LIST_INTERNAL_INDEX)
#	define UE_STORE_OBJECT_LIST_INTERNAL_INDEX 0
#endif

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("STAT_UObjectsStatGroupTester"), STAT_UObjectsStatGroupTester, STATGROUP_UObjects, COREUOBJECT_API);

namespace UE::Private { struct FBulkNewClassReinstancer; }

/** 
 * Low level implementation of UObject, should not be used directly in game code 
 */
class UObjectBase
{
	friend class UObjectBaseUtility;
	friend struct Z_Construct_UClass_UObject_Statics;
	friend class FUObjectArray; // for access to InternalIndex without revealing it to anyone else
	friend class FUObjectAllocator; // for access to destructor without revealing it to anyone else
	friend struct FInternalUObjectBaseUtilityIsValidFlagsChecker; // for access to InternalIndex
	friend struct UE::CoreUObject::Private::FObjectHandleUtils; // for access to OuterPrivate
	friend COREUOBJECT_API void UObjectForceRegistration(UObjectBase* Object, bool bCheckForModuleRelease);
	friend COREUOBJECT_API void InitializePrivateStaticClass(
		class UClass* (*TClass_StaticClassFn)(),
		class UClass* TClass_Super_StaticClass,
		class UClass* TClass_PrivateStaticClass,
		class UClass* TClass_WithinClass_StaticClass,
		const TCHAR* PackageName,
		const TCHAR* Name
		);
	friend COREUOBJECT_API UObject* StaticAllocateObject( // for access to ObjectListInternalIndex
		const UClass*,
		UObject*,
		FName,
		EObjectFlags,
		EInternalObjectFlags,
		bool,
		bool*,
		UPackage*,
		int32,
		FRemoteObjectId,
		class FGCReconstructionGuard*);

protected:
	UObjectBase() :
		ClassPrivate(NoInit),
		NamePrivate(NoInit),  // screwy, but the name was already set and we don't want to set it again
		OuterPrivate(NoInit)
	{
	}

#if UE_WITH_CONSTINIT_UOBJECT
public:
	explicit consteval UObjectBase(UE::CodeGen::ConstInit::FObjectParams InParams)
		: ObjectFlags(InParams.Flags | RF_MarkAsNative | RF_MarkAsRootSet | RF_NeedInitialization)
		, InternalIndex(INDEX_NONE)
		, ClassPrivate(InParams.Class)
		, UninitializedNameUTF8(InParams.NameUTF8)
		, OuterPrivate(InParams.Outer)
	{
	}
protected:
#endif

	/**
	 * Constructor used for bootstrapping
	 * @param	InFlags			RF_Flags to assign
	 */
	COREUOBJECT_API UObjectBase( EObjectFlags InFlags );
public:

	/**
	 * Constructor used by StaticAllocateObject
	 * @param	InClass				non NULL, this gives the class of the new object, if known at this time
	 * @param	InFlags				RF_Flags to assign
	 * @param	InInternalFlags EInternalObjectFlags to assign
	 * @param	InOuter				outer for this object
	 * @param	InName				name of the new object
	 * @param	InInternalIndex		internal index to use (if already allocated), negative value means allocate a new index
	 * @param	InSerialNumber		serial number to re-use (if already allocated)
	 */
	COREUOBJECT_API UObjectBase(UClass* InClass,
			EObjectFlags InFlags,
			EInternalObjectFlags InInternalFlags,
			UObject *InOuter,
			FName InName,
			int32 InInternalIndex = -1,
			int32 InSerialNumber = 0,
			FRemoteObjectId InRemoteId = FRemoteObjectId());

	/**
	 * Final destructor, removes the object from the object array, and indirectly, from any annotations
	 **/
	COREUOBJECT_API virtual ~UObjectBase();

protected:
	/**
	 * Just change the FName and Outer and rehash into name hash tables. For use by higher level rename functions.
	 *
	 * @param NewName	new name for this object
	 * @param NewOuter	new outer for this object, if NULL, outer will be unchanged
	 */
	COREUOBJECT_API void LowLevelRename(FName NewName,UObject *NewOuter = NULL);

#if !UE_WITH_CONSTINIT_UOBJECT
	/** Force any base classes to be registered first */
	virtual void RegisterDependencies() {}

	/** Enqueue the registration for this object. */
	COREUOBJECT_API void Register(class UClass* (*StaticClassFn)(), const TCHAR* PackageName, const TCHAR* Name);
#endif // !UE_WITH_CONSTINIT_UOBJECT

	/**
	 * Convert a boot-strap registered class into a real one, add to uobject array, etc
	 *
	 * @param UClassStaticClass Now that it is known, fill in UClass::StaticClass() as the class
	 */
	COREUOBJECT_API virtual void DeferredRegister(UClass* UClassStaticClass, const TCHAR* PackageName, const TCHAR* Name
#if UE_WITH_REMOTE_OBJECT_HANDLE
		, FRemoteObjectId InRemoteId
#endif
	);

private:
	/**
	 * Add a newly created object to the name hash tables and the object array
	 *
	 * @param Name name to assign to this uobject
	 * @param InSetInternalFlags Internal object flags to be set on the object once it's been added to the array
	 * @param InInternalIndex already allocated internal index to use, negative value means allocate a new index
	 * @param InSerialNumber already allocated serial number to re-use
	 */
	COREUOBJECT_API void AddObject(FName Name, EInternalObjectFlags InSetInternalFlags, int32 InInternalIndex = -1, int32 InSerialNumber = 0, FRemoteObjectId InRemoteId = FRemoteObjectId());

	/**
	 * Checks to see if the object appears to be valid when being destroyed
	 * @return true if this appears to be a valid object
	 */
	COREUOBJECT_API bool IsValidLowLevelForDestruction() const;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	FRemoteObjectId GetPendingRegistrantRemoteId() const;
#endif

public:
	/**
	 * Checks to see if the object appears to be valid
	 * @return true if this appears to be a valid object
	 */
	COREUOBJECT_API bool IsValidLowLevel() const;

	/**
	 * Faster version of IsValidLowLevel.
	 * Checks to see if the object appears to be valid by checking pointers and their alignment.
	 * Name and InternalIndex checks are less accurate than IsValidLowLevel.
	 * @param bRecursive true if the Class pointer should be checked with IsValidLowLevelFast
	 * @return true if this appears to be a valid object
	 */
	COREUOBJECT_API bool IsValidLowLevelFast(bool bRecursive = true) const;

	/** 
	 * Returns the unique ID of the object...these are reused so it is only unique while the object is alive.
	 * Useful as a tag.
	**/
	FORCEINLINE uint32 GetUniqueID() const
	{
		return (uint32)InternalIndex;
	}

	/** Returns the UClass that defines the fields of this object */
	FORCEINLINE UClass* GetClass() const
	{
		return ClassPrivate;
	}
	
	/** Returns the UObject this object resides in */
	FORCEINLINE UObject* GetOuter() const
	{
#if UE_WITH_REMOTE_OBJECT_HANDLE
		return ObjectPtr_Private::Friend::NoAccessTrackingGetNoResolve(OuterPrivate.GetAccessTrackedObjectPtr());
#else
		return OuterPrivate;
#endif
	}

	/** Returns the logical name of this object */
	FORCEINLINE FName GetFName() const
	{
#if UE_WITH_CONSTINIT_UOBJECT
		checkfSlow(!EnumHasAllFlags((EObjectFlags)GetFlagsInternal(), RF_NeedInitialization|RF_MarkAsNative), TEXT("UObjectBase::GetFName called on a native object which is not yet initialized."));
#endif // UE_WITH_CONSTINIT_UOBJECT
		return NamePrivate.Name;
	}

	/** Overridable method to return a logical name for identification in stats. */
	COREUOBJECT_API virtual FName GetFNameForStatID() const;

	/** Removes the class prefix from the given string */
	static COREUOBJECT_API FString RemoveClassPrefix(const TCHAR* ClassName);

	/** Returns the external UPackage associated with this object, if any */
	COREUOBJECT_API UPackage* GetExternalPackage() const;
	
	/** Associate an external package directly to this object. */
	COREUOBJECT_API void SetExternalPackage(UPackage* InPackage);

	/** Returns the external UPackage for this object, if any, NOT THREAD SAFE, used by internal gc reference collecting. */
	COREUOBJECT_API UPackage* GetExternalPackageInternal() const;

	/**
	 * Marks the object as Reachable if it's currently marked as MaybeUnreachable by incremental GC.
	*/
	COREUOBJECT_API void MarkAsReachable() const;

	/** Increments ref-count on the object. Should always be paired with RAII (i.e. TStrongObjectPtr). */
	COREUOBJECT_API void AddRef() const;

	/** Release ref-count on the object. */
	COREUOBJECT_API void ReleaseRef() const;

	/**
	 * Retrieve the object flags directly
	 *
	 * @return Flags for this object
	 **/
	FORCEINLINE EObjectFlags GetFlags() const
	{
		EObjectFlags Flags = (EObjectFlags)GetFlagsInternal();
		checkfSlow((Flags & ~RF_AllFlags) == 0, TEXT("%s flagged as RF_AllFlags"), *GetFName().ToString());
		return Flags;
	}

	/**
	 * Retrieve the object flags directly
	 *
	 * @param InMemoryOrder	Memory ordering to apply when reading the flags from memory
	 * @return Flags for this object
	 **/
	FORCEINLINE EObjectFlags GetFlags(std::memory_order InMemoryOrder) const
	{
		EObjectFlags Flags = (EObjectFlags)GetFlagsInternal(InMemoryOrder);
		checkfSlow((Flags & ~RF_AllFlags) == 0, TEXT("%s flagged as RF_AllFlags"), *GetFName().ToString());
		return Flags;
	}

	/**
	 *	Atomically adds the specified flags.
	 */
	FORCENOINLINE void AtomicallySetFlags( EObjectFlags FlagsToAdd )
	{
		int32 OldFlags = GetFlagsInternal();
		int32 NewFlags = OldFlags | FlagsToAdd;

		// Fast path without atomics if already set
		if (NewFlags == OldFlags)
		{
			return;
		}

		UE_AUTORTFM_OPEN
		{
			FPlatformAtomics::InterlockedOr((int32*)&ObjectFlags, FlagsToAdd);
		};

		// If we abort we undo setting the flags we just set.
		AutoRTFM::OnAbort([this, OldFlags, FlagsToAdd]
			{
				int32 MaskFlags = OldFlags;

				// Now just extract out the old flags that mattered (the ones we were setting).
				MaskFlags &= FlagsToAdd;
				// And unmask the flags we didn't mention.
				MaskFlags |= ~FlagsToAdd;

				FPlatformAtomics::InterlockedAnd((int32*)&ObjectFlags, MaskFlags);
			});
	}

	/**
	 *	Atomically clears the specified flags.
	 */
	FORCENOINLINE void AtomicallyClearFlags( EObjectFlags FlagsToClear )
	{
		int32 OldFlags = GetFlagsInternal();
		int32 NewFlags = OldFlags & ~FlagsToClear;

		// Fast path without atomics if already cleared
		if (NewFlags == OldFlags)
		{
			return;
		}

		UE_AUTORTFM_OPEN 
		{
			FPlatformAtomics::InterlockedAnd((int32*)&ObjectFlags, ~FlagsToClear);
		};

		// If we abort we undo clearing the flags we just unset.
		AutoRTFM::OnAbort([this, OldFlags, FlagsToClear]
			{
				int32 MaskFlags = OldFlags;

				// Now just extract out the old flags that mattered (the ones we were setting).
				MaskFlags &= FlagsToClear;

				FPlatformAtomics::InterlockedOr((int32*)&ObjectFlags, MaskFlags);
			});
	}

	static void PrefetchClass(UObject* Object) { FPlatformMisc::Prefetch(Object, offsetof(UObjectBase, ClassPrivate)); }
	static void PrefetchOuter(UObject* Object) { FPlatformMisc::Prefetch(Object, offsetof(UObjectBase, OuterPrivate)); }

private:
	FORCEINLINE int32 GetFlagsInternal() const
	{
		static_assert(sizeof(int32) == sizeof(ObjectFlags), "Flags must be 32-bit for atomics.");

		int32 Result = 0;

		UE_AUTORTFM_OPEN
		{
			Result = ((std::atomic<int32>*)&ObjectFlags)->load(std::memory_order_relaxed);
		};

		return Result;
	}

	FORCEINLINE int32 GetFlagsInternal(std::memory_order InMemoryOrder) const
	{
		static_assert(sizeof(int32) == sizeof(ObjectFlags), "Flags must be 32-bit for atomics.");

		int32 Result = 0;

		UE_AUTORTFM_OPEN
		{
			// Make this load use memory ordering so that we can rely on looking at flags
			// as a barrier between the thread setting a flag and a thread taking a decision
			// depending on that flag.
			Result = ((std::atomic<int32>*)&ObjectFlags)->load(InMemoryOrder);
		};

		return Result;
	}

	/** Flags used to track and report various object states. This needs to be 8 byte aligned on 32-bit
	    platforms to reduce memory waste */
	EObjectFlags					ObjectFlags;

	/** Index into GObjectArray...very private. */
	int32							InternalIndex;

	/** Class the object belongs to. */
	ObjectPtr_Private::TNonAccessTrackedObjectPtr<UClass>							ClassPrivate;

	// Merged data for constinit UObjects to hold their name before the FName system is initialized
	// The object flags RF_MarkAsNative | RF_NeedInitialization are used to assert correct use of this union.
	struct FNameAndObjectHashIndex
	{
		FName Name;
#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
		/** Internal index into an array that stores all objects of this object's class.
		It's used for registering and unregistering of UObjects in a global hash map.
		This optimization uses array instead of a hash map for reduced memory usage
		*/
		int32 ObjectListInternalIndex;
#endif
		constexpr FNameAndObjectHashIndex()
		: Name()
#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
		, ObjectListInternalIndex(INDEX_NONE)
#endif
        {
        }
		FNameAndObjectHashIndex(ENoInit)
		: Name(NoInit)
		{}
		FNameAndObjectHashIndex(FName InName)
		: Name(InName)
#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
		, ObjectListInternalIndex(INDEX_NONE)
#endif
		{
		}
	};
	union 
	{
		FNameAndObjectHashIndex NamePrivate = {};
#if UE_WITH_CONSTINIT_UOBJECT
		const UTF8CHAR* UninitializedNameUTF8;
#endif
	};
#if UE_WITH_CONSTINIT_UOBJECT
	static_assert(sizeof(UninitializedNameUTF8) <= sizeof(FNameAndObjectHashIndex), 
		"Wasted space in UObject? If FName is small in this configuration, enable UE_STORE_OBJECT_LIST_INTERNAL_INDEX to speed up the UObject Hash.");
#endif
	
#if UE_WITH_CONSTINIT_UOBJECT
	/** Return a valid FName for this object, even if this is a constinit object initialized from code generated by UHT. */
	FName GetUninitializedName()
	{
		if (EnumHasAllFlags(ObjectFlags, RF_MarkAsNative | RF_NeedInitialization))
		{
			FName NewName(UninitializedNameUTF8);
			return NewName;
		}
		checkfSlow(false, TEXT("Compiled-in object unexpectedly already had its name initialized? %s"), *NamePrivate.Name.ToString());
		return FName();
	}
#endif

	/** Object this object resides in. */
	ObjectPtr_Private::TNonAccessTrackedObjectPtr<UObject>						OuterPrivate;
	
	friend class FBlueprintCompileReinstancer;
	friend struct UE::Private::FBulkNewClassReinstancer;
	friend class FContextObjectManager;
	friend class FDeferredRegistry;
	friend class FCompiledInObjectRegistry;
	friend void AddToClassMap(class FUObjectHashTables& ThreadHash, UObjectBase* Object);
	friend void RemoveFromClassMap(class FUObjectHashTables& ThreadHash, UObjectBase* Object);
	friend class FObjectPurge;

#if UE_WITH_CONSTINIT_UOBJECT
	void AddConstInitObject();
#endif

#if UE_STORE_OBJECT_LIST_INTERNAL_INDEX
	// Internal accessor for internal index to hide gory union/struct details
	inline int32& GetObjectHashInternalIndex() { return NamePrivate.ObjectListInternalIndex; }
#endif

#if WITH_EDITOR
	/** This is used by the reinstancer to re-class and re-archetype the current instances of a class before recompiling */
	COREUOBJECT_API void SetClass(UClass* NewClass);
#endif

protected:
#if UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
	// Overridden for types generated by UHT which may contain encoded references to compiled-in objects in other modules
	COREUOBJECT_API virtual void LinkCompiledInPointerFields(UE::CoreUObject::Private::FCompiledInObjectLinker& Linker);
#endif // UE_WITH_CONSTINIT_UOBJECT && !IS_MONOLITHIC
};

/**
 * Checks to see if the UObject subsystem is fully bootstrapped and ready to go.
 * If true, then all objects are registered and auto registration of natives is over, forever.
 *
 * @return true if the UObject subsystem is initialized.
 */
COREUOBJECT_API bool UObjectInitialized();

/**
 * Force a pending registrant to register now instead of in the natural order
 */
COREUOBJECT_API void UObjectForceRegistration(UObjectBase* Object, bool bCheckForModuleRelease = true);

#if !UE_WITH_CONSTINIT_UOBJECT
/**
 * Structure that represents the registration information for a given class, structure, or enumeration
 */
template <typename T, typename V>
struct TRegistrationInfo
{
	using TType = T;
	using TVersion = V;

	TType* InnerSingleton = nullptr;
	TType* OuterSingleton = nullptr;
	TVersion ReloadVersionInfo;
};

/**
 * Reload version information for classes
 */
struct FClassReloadVersionInfo
{
#if WITH_RELOAD
	SIZE_T Size = 0;
	uint32 Hash = 0;
#endif
};

/**
 * Registration information for classes
 */
using FClassRegistrationInfo = TRegistrationInfo<UClass, FClassReloadVersionInfo>;

/**
 * Composite class register compiled in info
 */
struct FClassRegisterCompiledInInfo
{
	class UClass* (*Register)(ETypeConstructPhase);
	const TCHAR* Name;
	FClassRegistrationInfo* Info;
	FClassReloadVersionInfo VersionInfo;
};

/**
 * Adds a class registration and version information. The InInfo parameter must be static.
 */
COREUOBJECT_API void RegisterCompiledInInfo(class UClass* (*InRegister)(ETypeConstructPhase), const TCHAR* InPackageName, const TCHAR* InName, FClassRegistrationInfo& InInfo, const FClassReloadVersionInfo& InVersionInfo);

/**
 * Reload version information for structures
 */
struct FStructReloadVersionInfo
{
#if WITH_RELOAD
	SIZE_T Size = 0;
	uint32 Hash = 0;
#endif
};

/**
 * Registration information for structures
 */
using FStructRegistrationInfo = TRegistrationInfo<UScriptStruct, FStructReloadVersionInfo>;

/**
 * Composite structures register compiled in info
 */
struct FStructRegisterCompiledInInfo
{
	class UScriptStruct* (*OuterRegister)(ETypeConstructPhase);
	void* (*CreateCppStructOps)();
	const TCHAR* Name;
	FStructRegistrationInfo* Info;
	FStructReloadVersionInfo VersionInfo;
};


struct FPartialReloadVersionInfo
{
#if WITH_RELOAD
	SIZE_T Size = 0;
	uint32 Hash = 0;
#endif
};

using FPartialRegistrationInfo = TRegistrationInfo<UE::CoreUObject::Private::FPartialClass, FPartialReloadVersionInfo>;

struct FPartialRegisterCompiledInInfo
{
	UE::CoreUObject::Private::FPartialClass* (*OuterRegister)(ETypeConstructPhase);
	const TCHAR* Name;
	FPartialRegistrationInfo* Info;
	FPartialReloadVersionInfo VersionInfo;
};

/**
 * Adds a struct registration and version information. The InInfo parameter must be static.
 */
COREUOBJECT_API void RegisterCompiledInInfo(class UScriptStruct* (*InOuterRegister)(ETypeConstructPhase), const TCHAR* InPackageName, const TCHAR* InName, FStructRegistrationInfo& InInfo, const FStructReloadVersionInfo& InVersionInfo);

/**
 * Invoke the registration method wrapped in notifications.
 */
COREUOBJECT_API class UScriptStruct* GetStaticStruct(class UScriptStruct* (*InRegister)(ETypeConstructPhase), UObject* StructOuter, const TCHAR* StructName);

/**
 * Reload version information for enumerations
 */
struct FEnumReloadVersionInfo
{
#if WITH_RELOAD
	uint32 Hash = 0;
#endif
};

/**
 * Registration information for enums
 */
using FEnumRegistrationInfo = TRegistrationInfo<UEnum, FEnumReloadVersionInfo>;

/**
 * Composite enumeration register compiled in info
 */
struct FEnumRegisterCompiledInInfo
{
	class UEnum* (*OuterRegister)(ETypeConstructPhase);
	const TCHAR* Name;
	FEnumRegistrationInfo* Info;
	FEnumReloadVersionInfo VersionInfo;
};

/**
 * Adds a static enum registration and version information. The InInfo parameter must be static.
 */
COREUOBJECT_API void RegisterCompiledInInfo(class UEnum* (*InOuterRegister)(ETypeConstructPhase), const TCHAR* InPackageName, const TCHAR* InName, FEnumRegistrationInfo& InInfo, const FEnumReloadVersionInfo& InVersionInfo);

/**
 * Invoke the registration method wrapped in notifications.
 */
COREUOBJECT_API class UEnum* GetStaticEnum(class UEnum* (*InRegister)(ETypeConstructPhase), UObject* EnumOuter, const TCHAR* EnumName);

/**
 * Reload version information for packages 
 */
struct FPackageReloadVersionInfo
{
#if WITH_RELOAD
	uint32 BodyHash = 0;
	uint32 DeclarationsHash = 0;
#endif
};

/**
 * Registration information for packages
 */
using FPackageRegistrationInfo = TRegistrationInfo<UPackage, FPackageReloadVersionInfo>;

/**
 * Adds a static package registration and version information. The InInfo parameter must be static.
 */
COREUOBJECT_API void RegisterCompiledInInfo(UPackage* (*InOuterRegister)(ETypeConstructPhase), const TCHAR* InPackageName, FPackageRegistrationInfo& InInfo, const FPackageReloadVersionInfo& InVersionInfo);

/**
 * Register compiled in information for multiple classes, structures, and enumerations
 */
COREUOBJECT_API void RegisterCompiledInInfo(const TCHAR *PackageName, const FClassRegisterCompiledInInfo *ClassInfo, size_t NumClassInfo, const FStructRegisterCompiledInInfo *StructInfo, size_t NumStructInfo, const FEnumRegisterCompiledInInfo *EnumInfo, size_t NumEnumInfo, const FPartialRegisterCompiledInInfo *PartialsInfo, size_t NumPartialsInfo);

#endif // !UE_WITH_CONSTINIT_UOBJECT


/** Must be called to register UObject-related callbacks with the module manager */
COREUOBJECT_API void RegisterModularObjectsProcessing();

/** Must be called after a module has been loaded that contains UObject classes */
COREUOBJECT_API void ProcessNewlyLoadedUObjects(FName InModuleName = NAME_None, bool bCanProcessNewlyLoadedObjects = true);

#if !UE_WITH_CONSTINIT_UOBJECT
/**
 * Helper class to perform registration of object information.  It blindly forwards a call to RegisterCompiledInInfo
 */
struct FRegisterCompiledInInfo
{
	template <typename... ArgTypes>
	FRegisterCompiledInInfo(ArgTypes&&... Args)
	{
		RegisterCompiledInInfo(Forward<ArgTypes>(Args)...);
	}
};
#endif // !UE_WITH_CONSTINIT_UOBJECT

/**
 * Final phase of UObject initialization. all auto register objects are added to the main data structures.
 */
void UObjectBaseInit();

/**
 * Final phase of UObject shutdown
 */
void UObjectBaseShutdown();
