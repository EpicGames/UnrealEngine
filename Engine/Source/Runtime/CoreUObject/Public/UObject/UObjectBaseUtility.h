// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectBaseUtility.h: Unreal UObject functions that only depend on UObjectBase
=============================================================================*/

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Containers/VersePathFwd.h"
#include "HAL/PlatformMath.h"
#include "Misc/AssertionMacros.h"
#include "Misc/EnumClassFlags.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Stats/Stats.h"
#include "Stats/StatsCommon.h"
#include "Trace/Detail/Channel.h"
#include "Trace/Detail/Channel.inl"
#include "Trace/Trace.h"
#include "UObject/GarbageCollectionGlobals.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectVersion.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectBase.h"
#include "UObject/UObjectMarks.h"
#include "UObject/ObjectFwd.h"
#include "AutoRTFM.h"

class UClass;
class UObject;
class UPackage;
struct FGuid;

#if defined(_MSC_VER) && _MSC_VER == 1900
	#ifdef PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
		PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS
	#endif
#endif

/**
 * Provides utility functions for UObject, this class should not be used directly
 */
class UObjectBaseUtility : public UObjectBase
{
	/** If true references to objects marked as Garbage will be automatically eliminated by Garbage Collector*/
	static COREUOBJECT_API bool bGarbageEliminationEnabled;

	friend struct FInternalUObjectBaseUtilityIsValidFlagsChecker;

public:
	// Constructors.
	UObjectBaseUtility() {}
	UObjectBaseUtility( EObjectFlags InFlags )
		: UObjectBase(InFlags)
	{
	}

#if UE_WITH_CONSTINIT_UOBJECT
	explicit consteval UObjectBaseUtility(UE::CodeGen::ConstInit::FObjectParams InParams)
	: UObjectBase(InParams)
	{
	}
#endif

	/*-------------------
			Flags
	-------------------*/

	/** Modifies object flags for a specific object */
	inline void SetFlags( EObjectFlags NewFlags )
	{
		checkSlow(!(NewFlags & (RF_MarkAsNative | RF_MarkAsRootSet | RF_MirroredGarbage))); // These flags can't be used outside of constructors / internal code
		checkf(!(NewFlags & RF_MirroredGarbage) || (GetFlags() & (NewFlags & RF_MirroredGarbage)) == (NewFlags & RF_MirroredGarbage), TEXT("RF_MirroredGarbage can not be set through SetFlags function. Use MarkAsGarbage() instead"));
		AtomicallySetFlags(NewFlags);
	}

	/** Clears subset of flags for a specific object */
	inline void ClearFlags( EObjectFlags FlagsToClear )
	{
		checkSlow(!(FlagsToClear & (RF_MarkAsNative | RF_MarkAsRootSet | RF_MirroredGarbage)) || FlagsToClear == RF_AllFlags); // These flags can't be used outside of constructors / internal code
		checkf(!(FlagsToClear & RF_MirroredGarbage) || (GetFlags() & (FlagsToClear & RF_MirroredGarbage)) == RF_NoFlags, TEXT("RF_MirroredGarbage can not be cleared through ClearFlags function. Use ClearGarbage() instead"));
		AtomicallyClearFlags(FlagsToClear);
	}

	/**
	 * Used to safely check whether any of the passed in flags are set. 
	 *
	 * @param FlagsToCheck	Object flags to check for.
	 * @return				true if any of the passed in flags are set, false otherwise  (including no flags passed in).
	 */
	inline bool HasAnyFlags( EObjectFlags FlagsToCheck ) const
	{
		checkSlow(!(FlagsToCheck & (RF_MarkAsNative | RF_MarkAsRootSet)) || FlagsToCheck == RF_AllFlags); // These flags can't be used outside of constructors / internal code
		return (GetFlags() & FlagsToCheck) != 0;
	}

	/**
	 * Used to safely check whether any of the passed in flags are set. 
	 *
	 * @param FlagsToCheck	Object flags to check for.
	 * @param MemoryOrder	Memory ordering to apply when reading the flags from memory
	 * @return true if any of the passed in flags are set, false otherwise  (including no flags passed in).
	 */
	inline bool HasAnyFlags( EObjectFlags FlagsToCheck, std::memory_order InMemoryOrder) const
	{
		checkSlow(!(FlagsToCheck & (RF_MarkAsNative | RF_MarkAsRootSet)) || FlagsToCheck == RF_AllFlags); // These flags can't be used outside of constructors / internal code
		return (GetFlags(InMemoryOrder) & FlagsToCheck) != 0;
	}

	/**
	 * Used to safely check whether all of the passed in flags are set. 
	 *
	 * @param FlagsToCheck	Object flags to check for
	 * @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	 */
	inline bool HasAllFlags( EObjectFlags FlagsToCheck ) const
	{
		checkSlow(!(FlagsToCheck & (RF_MarkAsNative | RF_MarkAsRootSet)) || FlagsToCheck == RF_AllFlags); // These flags can't be used outside of constructors / internal code
		return ((GetFlags() & FlagsToCheck) == FlagsToCheck);
	}

	/**
	 * Used to safely check whether all of the passed in flags are set. 
	 *
	 * @param FlagsToCheck	Object flags to check for
	 * @param MemoryOrder	Memory ordering to apply when reading the flags from memory
	 * @return true if all of the passed in flags are set (including no flags passed in), false otherwise
	 */
	inline bool HasAllFlags( EObjectFlags FlagsToCheck, std::memory_order InMemoryOrder) const
	{
		checkSlow(!(FlagsToCheck & (RF_MarkAsNative | RF_MarkAsRootSet)) || FlagsToCheck == RF_AllFlags); // These flags can't be used outside of constructors / internal code
		return ((GetFlags(InMemoryOrder) & FlagsToCheck) == FlagsToCheck);
	}

	/**
	 * Returns object flags that are both in the mask and set on the object.
	 *
	 * @param Mask	Mask to mask object flags with
	 * @param Objects flags that are set in both the object and the mask
	 */
	UE_FORCEINLINE_HINT EObjectFlags GetMaskedFlags( EObjectFlags Mask = RF_AllFlags ) const
	{
		return EObjectFlags(GetFlags() & Mask);
	}

	/*----------------------------------------------------
			Marks, implemented in UObjectMarks.cpp
	----------------------------------------------------*/

	/**
	 * Adds marks to an object
	 *
	 * @param	Marks	Logical OR of OBJECTMARK_'s to apply 
	 */
	UE_FORCEINLINE_HINT void Mark(EObjectMark Marks) const
	{
		MarkObject(this,Marks);
	}

	/**
	 * Removes marks from and object
	 *
	 * @param	Marks	Logical OR of OBJECTMARK_'s to remove 
	 */
	UE_FORCEINLINE_HINT void UnMark(EObjectMark Marks) const
	{
		UnMarkObject(this,Marks);
	}

	/**
	 * Tests an object for having ANY of a set of marks
	 *
	 * @param	Marks	Logical OR of OBJECTMARK_'s to test
	 * @return	true if the object has any of the given marks.
	 */
	UE_FORCEINLINE_HINT bool HasAnyMarks(EObjectMark Marks) const
	{
		return ObjectHasAnyMarks(this,Marks);
	}

	/**
	 * Tests an object for having ALL of a set of marks
	 *
	 * @param	Marks	Logical OR of OBJECTMARK_'s to test
	 * @return	true if the object has any of the given marks.
	 */
	UE_FORCEINLINE_HINT bool HasAllMarks(EObjectMark Marks) const
	{
		return ObjectHasAllMarks(this,Marks);
	}

	/**
	 * Returns all of the object marks on a specific object
	 *
	 * @param	Object	Object to get marks for
	 * @return	all Marks for an object
	 */
	UE_FORCEINLINE_HINT EObjectMark GetAllMarks() const
	{
		return ObjectGetAllMarks(this);
	}

	/**
	 * Marks this object as Garbage.
	 */
	inline void MarkAsGarbage()
	{
		check(!IsRooted());

		AtomicallySetFlags(RF_MirroredGarbage);
		GUObjectArray.IndexToObject(InternalIndex)->SetGarbage();

		// If we explicitly marked the object as garbage, remove the async flag so it's visible to the GC
		AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
	}

	/**
	 * Unmarks this object as Garbage.
	 */
	inline void ClearGarbage()
	{
		AtomicallyClearFlags(RF_MirroredGarbage);
		GUObjectArray.IndexToObject(InternalIndex)->ClearGarbage();
	}

	/**
	 * Add an object to the root set. This prevents the object and all
	 * its descendants from being deleted during garbage collection.
	 */
	UE_FORCEINLINE_HINT void AddToRoot()
	{
		GUObjectArray.IndexToObject(InternalIndex)->SetRootSet();
	}

	/** Remove an object from the root set. */
	UE_FORCEINLINE_HINT void RemoveFromRoot()
	{
		GUObjectArray.IndexToObject(InternalIndex)->ClearRootSet();
	}

	/**
	 * Returns true if this object is explicitly rooted
	 *
	 * @return true if the object was explicitly added as part of the root set.
	 */
	UE_FORCEINLINE_HINT bool IsRooted() const
	{
		return GUObjectArray.IndexToObject(InternalIndex)->IsRootSet();
	}

	/**
	 * Returns the refcount of the object
	 *
	 * @return the refcount of the object.
	 */
	UE_FORCEINLINE_HINT int32 GetRefCount() const
	{
		return GUObjectArray.IndexToObject(InternalIndex)->GetRefCount();
	}

	/** Checks if the object is unreachable. */
	UE_FORCEINLINE_HINT bool IsUnreachable() const
	{
		return GUObjectArray.IndexToObject(InternalIndex)->IsUnreachable();
	}

	/** Checks if the object is native. */
	UE_FORCEINLINE_HINT bool IsNative() const
	{
		return GUObjectArray.IndexToObject(InternalIndex)->HasAnyFlags(EInternalObjectFlags::Native);
	}

	/**
	 * Clears passed in internal flags.
	 *
	 * @param FlagsToClear	Object flags to clear.
	 * @return				true if any of the passed in flags are set, false otherwise  (including no flags passed in).
	 */
	inline void SetInternalFlags(EInternalObjectFlags FlagsToSet) const
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(InternalIndex);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		checkf(!(FlagsToSet & EInternalObjectFlags::Garbage) || (FlagsToSet & EInternalObjectFlags::Garbage) == (ObjectItem->GetFlags() & EInternalObjectFlags::Garbage), TEXT("SetInternalFlags should not set the Garbage flag. Use MarkAsGarbage instead"));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		ObjectItem->SetFlags(FlagsToSet);
	}

	/**
	 * Gets internal flags.
	 *
	 * @param FlagsToClear	Object flags to clear.
	 * @return				true if any of the passed in flags are set, false otherwise  (including no flags passed in).
	 */
	UE_FORCEINLINE_HINT EInternalObjectFlags GetInternalFlags() const
	{
		return GUObjectArray.IndexToObject(InternalIndex)->GetFlags();
	}

	/**
	 * Used to safely check whether any of the passed in internal flags are set.
	 *
	 * @param FlagsToCheck	Object flags to check for.
	 * @return				true if any of the passed in flags are set, false otherwise  (including no flags passed in).
	 */
	UE_FORCEINLINE_HINT bool HasAnyInternalFlags(EInternalObjectFlags FlagsToCheck) const
	{
		return GUObjectArray.IndexToObject(InternalIndex)->HasAnyFlags(FlagsToCheck);
	}

	/**
	 * Clears passed in internal flags.
	 *
	 * @param FlagsToClear	Object flags to clear.
	 * @return				true if any of the passed in flags are set, false otherwise  (including no flags passed in).
	 */
	inline void ClearInternalFlags(EInternalObjectFlags FlagsToClear) const
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(InternalIndex);
		checkf(!(FlagsToClear & EInternalObjectFlags::Garbage) || (ObjectItem->GetFlags() & (FlagsToClear & EInternalObjectFlags::Garbage)) == EInternalObjectFlags::None, TEXT("ClearInternalFlags should not clear Garbage flag. Use ClearGarbage() instead"));
		ObjectItem->ClearFlags(FlagsToClear);
	}

	/**
	 * Atomically clears passed in internal flags.
	 *
	 * @param FlagsToClear	Object flags to clear.
	 * @return				true if any of the passed in flags are set, false otherwise  (including no flags passed in).
	 */
	inline bool AtomicallyClearInternalFlags(EInternalObjectFlags FlagsToClear) const
	{
		FUObjectItem* ObjectItem = GUObjectArray.IndexToObject(InternalIndex);
		checkf((ObjectItem->GetFlags() & (FlagsToClear & EInternalObjectFlags::Garbage)) == EInternalObjectFlags::None, TEXT("ClearInternalFlags should not clear Garbage flag. Use ClearGarbage() instead"));
		return ObjectItem->ThisThreadAtomicallyClearedFlag(FlagsToClear);
	}


	/*-------------------
			Names
	-------------------*/

	/**
	 * Returns the fully qualified pathname for this object as well as the name of the class, in the format:
	 * 'ClassName Outermost[.Outer].Name'.
	 *
	 * @param	StopOuter	if specified, indicates that the output string should be relative to this object.  if StopOuter
	 *						does not exist in this object's Outer chain, the result would be the same as passing NULL.
	 * @param	Flags		flags that control the behavior of full name generation
	 *
	 * @note	safe to call on NULL object pointers!
	 */
	COREUOBJECT_API FString GetFullName( const UObject* StopOuter=NULL, EObjectFullNameFlags Flags = EObjectFullNameFlags::None ) const;

	/**
	 * Version of GetFullName() that eliminates unnecessary copies.
	 */
	COREUOBJECT_API void GetFullName( const UObject* StopOuter, FString& ResultString, EObjectFullNameFlags Flags = EObjectFullNameFlags::None ) const;

	/**
	 * Returns the fully qualified pathname for this object as well as the name of the class, in the format:
	 * 'ClassName Outermost[.Outer].Name'.
	 *
	 * @param	ResultString StringBuilder to populate
	 * @param	StopOuter	if specified, indicates that the output string should be relative to this object.  if StopOuter
	 *						does not exist in this object's Outer chain, the result would be the same as passing NULL.
	 * @param	Flags		flags that control the behavior of full name generation
	 *
	 * @note	safe to call on NULL object pointers!
	 */
	COREUOBJECT_API void GetFullName(FStringBuilderBase& ResultString, const UObject* StopOuter = NULL, EObjectFullNameFlags Flags = EObjectFullNameFlags::None) const;

	/**
	 * Returns the fully qualified pathname for this object, in the format:
	 * 'Outermost[.Outer].Name'
	 *
	 * @param	StopOuter	if specified, indicates that the output string should be relative to this object.  if StopOuter
	 *						does not exist in this object's Outer chain, the result would be the same as passing NULL.
	 *
	 * @note	safe to call on NULL object pointers!
	 */
	COREUOBJECT_API FString GetPathName( const UObject* StopOuter=NULL ) const;

	/**
	 * Versions of GetPathName() that eliminates unnecessary copies and allocations.
	 */
	COREUOBJECT_API void GetPathName(const UObject* StopOuter, FString& ResultString) const;
	COREUOBJECT_API void GetPathName(const UObject* StopOuter, FStringBuilderBase& ResultString) const;

	/** Helper function to access the private bGarbageEliminationEnabled variable */
	static inline bool IsGarbageEliminationEnabled()
	{
		return bGarbageEliminationEnabled;
	}

	/** Helper function to try to set the private bGarbageEliminationEnabled variable. */
	static COREUOBJECT_API void SetGarbageEliminationEnabled(bool bEnabled);

	/** Helper function to set the private bGarbageEliminationEnabled variable for testing purposes only as garbage elimination may not be available for general use based on compile settings. */
	static inline void ForceSetGarbageEliminationEnabledForTest(bool bEnabled)
	{
		bGarbageEliminationEnabled = bEnabled;
	}

public:
	/**
	* Called after load to determine if the object can be a cluster root
	*
	* @return	true if this object can be a cluster root
	*/
	virtual bool CanBeClusterRoot() const
	{
		return false;
	}

	/**
	* Called during cluster construction if the object can be added to a cluster
	*
	* @return	true if this object can be inside of a cluster
	*/
	COREUOBJECT_API virtual bool CanBeInCluster() const;

	/**
	* Called after PostLoad to create UObject cluster
	*/
	COREUOBJECT_API virtual void CreateCluster();

	/**
	* Called during Garbage Collection to perform additional cleanup when the cluster is about to be destroyed due to PendingKill flag being set on it.
	*/
	virtual void OnClusterMarkedAsPendingKill() {}

	/**
	* Adds this objects to a GC cluster that already exists
	* @param ClusterRootOrObjectFromCluster Object that belongs to the cluster we want to add this object to.
	* @param Add this object to the target cluster as a mutable object without adding this object's references.
	*/
	COREUOBJECT_API void AddToCluster(UObjectBaseUtility* ClusterRootOrObjectFromCluster, bool bAddAsMutableObject = false);

public:
	/**
	 * Walks up the chain of packages until it reaches the top level, which it ignores.
	 *
	 * @param	bStartWithOuter		whether to include this object's name in the returned string
	 * @return	string containing the path name for this object, minus the outermost-package's name
	 */
	COREUOBJECT_API FString GetFullGroupName( bool bStartWithOuter ) const;

	/**
	 * Returns the name of this object (with no path information)
	 * 
	 * @return Name of the object.
	 */
	UE_FORCEINLINE_HINT FString GetName() const
	{
		return GetFName().ToString();
	}

	/** Optimized version of GetName that overwrites an existing string */
	UE_FORCEINLINE_HINT void GetName(FString &ResultString) const
	{
		GetFName().ToString(ResultString);
	}
	/** Optimized version of GetName that appends to an existing string */
	UE_FORCEINLINE_HINT void AppendName(FString& ResultString) const
	{
		GetFName().AppendString(ResultString);
	}


	/*-------------------
		Outer & Package
	-------------------*/

	/**
	 * Get the object packaging mode.
	 * @return true if object has a different package than its outer's package
	 */
	COREUOBJECT_API bool IsPackageExternal() const;

	/**
	 * Utility function to temporarily detach the object external package, if any
	 * GetPackage will report the outer's package once detached
	 */
	COREUOBJECT_API void DetachExternalPackage();

	/**
	 * Utility function to reattach the object external package, if any
	 * GetPackage will report the object external package if set after this call
	 */
	COREUOBJECT_API void ReattachExternalPackage();

	/**
	 * Walks up the list of outers until it finds the top-level one that isn't a package.
	 * Will return null if called on a package
	 * @return outermost non package Outer.
	 */
	COREUOBJECT_API UObject* GetOutermostObject() const;

	/**
	 * Walks up the list of outers until it finds a package directly associated with the object.
	 *
	 * @return the package the object is in.
	 */
	COREUOBJECT_API UPackage* GetPackage() const;

	/**
	 * Gets the versepath of the UObject.
	 *
	 * @return The VersePath of the object
	 */
	UE_INTERNAL COREUOBJECT_API virtual UE::Core::FVersePath GetVersePath() const;

	/** 
	 * Legacy function, has the same behavior as GetPackage
	 * use GetPackage instead.
	 * @return the package the object is in.
	 * @see GetPackage
	 */
	COREUOBJECT_API UPackage* GetOutermost() const;

	/** 
	 * Finds the outermost package and marks it dirty. 
	 * The editor suppresses this behavior during load as it is against policy to dirty packages simply by loading them.
	 *
	 * @return false if the request to mark the package dirty was suppressed by the editor and true otherwise.
	 */
	COREUOBJECT_API bool MarkPackageDirty() const;

	/**
	 * Determines whether this object is a template object by checking flags on the object and the outer chain.
	 *
	 * @param	TemplateTypes	Specific flags to look for, the default checks class default and non-class templates
	 * @return	true if this object is a template object that can be used as an archetype
	 */
	COREUOBJECT_API bool IsTemplate(EObjectFlags TemplateTypes = RF_ArchetypeObject|RF_ClassDefaultObject) const;

	/**
	 * Traverses the outer chain searching for the next object of a certain type.  (T must be derived from UObject)
	 *
	 * @param	Target class to search for
	 * @return	a pointer to the first object in this object's Outer chain which is of the correct type.
	 */
	COREUOBJECT_API UObject* GetTypedOuter(UClass* Target) const;

	/**
	 * Traverses the outer chain searching for the next object of a certain type.  (T must be derived from UObject)
	 *
	 * @return	a pointer to the first object in this object's Outer chain which is of the correct type.
	 */
	template<typename T>
	T* GetTypedOuter() const
	{
		return (T *)GetTypedOuter(T::StaticClass());
	}

	/** 
	 * Traverses the outer chain looking for the next object that implements the specified IInterface (InterfaceClass must be an IInterface)
	 * 
	 * @return	a pointer to the interface on the first object in this object's Outer chain which implements the specified interface.
	 */
	template<typename InterfaceClassType>
	InterfaceClassType* GetImplementingOuter() const
	{
		UClass* InterfaceClass = InterfaceClassType::UClassType::StaticClass();
		if(UObjectBaseUtility* ImplementingOuter = GetImplementingOuterObject(InterfaceClass))
		{
			return static_cast<InterfaceClassType*>(ImplementingOuter->GetInterfaceAddress(InterfaceClass));
		}
		return nullptr;
	}

	/** 
	 * Traverses the outer chain looking for the next object that implements the specified UInterface (InInterfaceClass must be a subclass of UInterface)
	 *
	 * @param	InInterfaceClass	Target interface to search for
	 * @return	a pointer to the first object in this object's Outer chain which implements the specified interface.
	 */
	COREUOBJECT_API UObjectBaseUtility* GetImplementingOuterObject(const UClass* InInterfaceClass) const;

	/** 
	 * Return the dispatch to `IsInOuter` or `IsInPackage` depending on SomeOuter's class. 
	 * Legacy function, preferably use IsInOuter or IsInPackage depending on use case.
	 */
	COREUOBJECT_API bool IsIn( const UObject* SomeOuter ) const;

	/** 
	 * Overload to determine if an object is in the specified package which can now be different than its outer chain.
	 * Calls IsInPackage.
	 */
	COREUOBJECT_API bool IsIn(const UPackage* SomePackage) const;

	/** Returns true if the object is contained in the specified outer. */
	COREUOBJECT_API bool IsInOuter(const UObject* SomeOuter) const;

	/** Returns true if the object is contained in the specified package. */
	COREUOBJECT_API bool IsInPackage(const UPackage* SomePackage) const;

	/**
	 * Find out if this object is inside (has an outer) that is of the specified class
	 * @param SomeBaseClass	The base class to compare against
	 * @return True if this object is in an object of the given type.
	 */
	COREUOBJECT_API bool IsInA( const UClass* SomeBaseClass ) const;

	/**
	 * Checks whether this object's top-most package has any of the specified flags
	 *
	 * @param	CheckFlagMask	a bitmask of EPackageFlags values to check for
	 *
	 * @return	true if the PackageFlags member of this object's top-package has any bits from the mask set.
	 */
	COREUOBJECT_API bool RootPackageHasAnyFlags( uint32 CheckFlagMask ) const;


	/*-------------------
			Class
	-------------------*/

private:
	template <typename ClassType>
	static UE_FORCEINLINE_HINT bool IsChildOfWorkaround(const ClassType* ObjClass, const ClassType* TestCls)
	{
		return ObjClass->IsChildOf(TestCls);
	}

public:
	/**
	 * Returns true if this object is of the specified type.
	 *
	 * @param	SomeBase	The base class to check against, UClass* assignable. Cannot be null.
	 */
	template <typename OtherClassType>
	inline bool IsA( OtherClassType SomeBase ) const
	{
		// We have a cyclic dependency between UObjectBaseUtility and UClass,
		// so we use a template to allow inlining of something we haven't yet seen, because it delays compilation until the function is called.

		// 'static_assert' that this thing is actually a UClass pointer or convertible to it.
		const UClass* SomeBaseClass = SomeBase;
		(void)SomeBaseClass;
		checkfSlow(SomeBaseClass, TEXT("IsA(NULL) cannot yield meaningful results"));

		const UClass* ThisClass = GetClass();

		// Stop the compiler doing some unnecessary branching for nullptr checks
		UE_ASSUME(SomeBaseClass);
		UE_ASSUME(ThisClass);

		return IsChildOfWorkaround(ThisClass, SomeBaseClass);
	}

	/** Returns true if this object is of the template type. */
	template<class T>
	bool IsA() const
	{
		return IsA(T::StaticClass());
	}

	/**
	 * Finds the most-derived class which is a parent of both TestClass and this object's class.
	 *
	 * @param	TestClass	the class to find the common base for
	 */
	COREUOBJECT_API const UClass* FindNearestCommonBaseClass( const UClass* TestClass ) const;

	/**
	 * Returns a pointer to this object safely converted to a pointer of the specified interface class.
	 *
	 * @param	InterfaceClass	the interface class to use for the returned type
	 *
	 * @return	a pointer that can be assigned to a variable of the interface type specified, or NULL if this object's
	 *			class doesn't implement the interface indicated.  Will be the same value as 'this' if the interface class
	 *			isn't native.
	 */
	COREUOBJECT_API void* GetInterfaceAddress( UClass* InterfaceClass );

	/** 
	 *	Returns a pointer to the I* native interface object that this object implements.
	 *	Returns NULL if this object does not implement InterfaceClass, or does not do so natively.
	 */
	COREUOBJECT_API void* GetNativeInterfaceAddress(UClass* InterfaceClass);

	/** 
	 *	Returns a pointer to the const I* native interface object that this object implements.
	 *	Returns NULL if this object does not implement InterfaceClass, or does not do so natively.
	 */
	const void* GetNativeInterfaceAddress(UClass* InterfaceClass) const
	{
		return const_cast<UObjectBaseUtility*>(this)->GetNativeInterfaceAddress(InterfaceClass);
	}

	/**
	 * Returns whether this is a template that can be used to create instanced subobjects.
	 * By default this includes subobjects of a class default object and other templates associated with a class.
	 * 
	 * @param	TemplateTypes	Specific flags to check on this object and the outer chain
	 * @return	true if this is a template for creating instanced subobjects
	 */
	COREUOBJECT_API bool IsTemplateForSubobjects(EObjectFlags TemplateTypes = RF_ClassDefaultObject|RF_DefaultSubObject|RF_InheritableComponentTemplate) const;
	
	/**
	 * Returns whether this is a subobject (template or instance) that was originally defined inside a default class object.
	 * This will return true for all instanced objects created from an archetype that is not a class default object,
	 * but will only return true for template objects nested directly inside a class default object.
	 *
	 * @warning The behavior of this function is inconsistent for historical reasons and is not equivalent to RF_DefaultSubobject.
	 * 			Call IsTemplateSubobject to handle all types of subobject templates, or call GetArchetype() first if you have an instance.
	 *
	 * @return	true if this was instanced from a subobject template, or is the direct subobject of a class default object
	 */
	COREUOBJECT_API bool IsDefaultSubobject() const;
	

	/*--------------------------------------------------
			Linker, defined in UObjectLinker.cpp
	--------------------------------------------------*/

	/**
	 * Returns the linker for this object.
	 *
	 * @return	a pointer to the linker for this object, or NULL if this object has no linker
	 */
	COREUOBJECT_API class FLinkerLoad* GetLinker() const;

	/**
	 * Returns this object's LinkerIndex.
	 *
	 * @return	the index into my linker's ExportMap for the FObjectExport
	 *			corresponding to this object.
	 */
	COREUOBJECT_API int32 GetLinkerIndex() const;

	/**
	 * Returns the UE version of the linker for this object.
	 *
	 * @return	the UE version of the engine's package file when this object
	 *			was last saved, or GPackageFileUEVersion (current version) if
	 *			this object does not have a linker, which indicates that
	 *			a) this object is a native only class, or
	 *			b) this object's linker has been detached, in which case it is already fully loaded
	 */
	COREUOBJECT_API FPackageFileVersion GetLinkerUEVersion() const;
		
	/**
	 * Returns the licensee version of the linker for this object.
	 *
	 * @return	the licensee version of the engine's package file when this object
	 *			was last saved, or GPackageFileLicenseeVersion (current version) if
	 *			this object does not have a linker, which indicates that
	 *			a) this object is a native only class, or
	 *			b) this object's linker has been detached, in which case it is already fully loaded
	 */
	COREUOBJECT_API int32 GetLinkerLicenseeUEVersion() const;

	/**
	 * Returns the custom version of the linker for this object corresponding to the given custom version key.
	 *
	 * @return	the custom version of the engine's package file when this object
	 *			was last saved, or the current version if
	 *			this object does not have a linker, which indicates that
	 *			a) this object is a native only class, or
	 *			b) this object's linker has been detached, in which case it is already fully loaded
	 */
	COREUOBJECT_API int32 GetLinkerCustomVersion(FGuid CustomVersionKey) const;

	/** 
	 * Overloaded < operator. Compares objects by name.
	 *
	 * @return true if this object's name is lexicographically smaller than the other object's name
	 */
	UE_FORCEINLINE_HINT bool operator<( const UObjectBaseUtility& Other ) const
	{
		return GetName() < Other.GetName();
	}



	/*******
	 * Stats
	 *******/
#if STATS || ENABLE_STATNAMEDEVENTS_UOBJECT
	inline void ResetStatID()
	{
		GUObjectArray.IndexToObject(InternalIndex)->StatID = TStatId();
#if ENABLE_STATNAMEDEVENTS_UOBJECT
		GUObjectArray.IndexToObject(InternalIndex)->StatIDStringStorage = nullptr;
#endif
	}
#endif
	/**
	  * Returns the stat ID of the object, used for profiling. This will create a stat ID if needed.
	  *
	  * @param bForDeferred If true, a stat ID will be created even if a group is disabled
	  */
	inline TStatId GetStatID(bool bForDeferredUse = false) const
	{
		TStatId Result{};
		UE_AUTORTFM_OPEN
		{
#if STATS
			const TStatId& StatID = GUObjectArray.IndexToObject(InternalIndex)->StatID;

			// this is done to avoid even registering stats for a disabled group (unless we plan on using it later)
			if (bForDeferredUse || FThreadStats::IsCollectingData(GET_STATID(STAT_UObjectsStatGroupTester)))
			{
				if (!StatID.IsValidStat())
				{
					CreateStatID();
				}
				Result = StatID;
			}
			else
			{
				Result = TStatId(); // not doing stats at the moment, or ever
			}
#elif ENABLE_STATNAMEDEVENTS_UOBJECT
			const TStatId& StatID = GUObjectArray.IndexToObject(InternalIndex)->StatID;
			if (!StatID.IsValidStat() && (bForDeferredUse || GCycleStatsShouldEmitNamedEvents))
			{
				CreateStatID();
			}
			Result = StatID;
#else
			Result = TStatId(); // not doing stats at the moment, or ever
#endif // STATS
		};
		return Result;
	}

private:
#if STATS || ENABLE_STATNAMEDEVENTS_UOBJECT
	/** Creates a stat ID for this object */
	void CreateStatID() const
	{
		GUObjectArray.IndexToObject(InternalIndex)->CreateStatID();
	}
#endif

		
	/*******
	 * Config
	 *******/
public:
	COREUOBJECT_API static void ReloadObjectsFromModifiedConfigSections(const class FConfigModificationTracker* ChangeTracker);
};

/** Returns false if this pointer cannot be a valid pointer to a UObject */
inline bool IsPossiblyAllocatedUObjectPointer(UObject* Ptr)
{
	auto CountByteValues = [](UPTRINT Val, UPTRINT ByteVal) -> int32
	{
		int32 Result = 0;

		// != changed to < because of false positive in MSVC's static analyzer: warning C6295: Ill-defined for-loop.  Loop executes infinitely.
		for (int32 I = 0; I < sizeof(UPTRINT); ++I)
		{
			if ((Val & 0xFF) == ByteVal)
			{
				++Result;
			}
			Val >>= 8;
		}

		return Result;
	};

	UPTRINT PtrVal = (UPTRINT)Ptr;
	return PtrVal >= 0x1000 && CountByteValues(PtrVal, 0xCD) < sizeof(UPTRINT) / 2;
}

/**
 * Returns the logical name of this object.
 * @param Object object to retrieve the name for; null gives NAME_None.
 * @return Name of the object.
*/
inline FName GetFNameSafe(const UObjectBaseUtility* Object)
{
	if (Object == nullptr)
	{
		return NAME_None;
	}
	else
	{
		return Object->GetFName();
	}
}

/**
 * Returns the name of this object (with no path information).
 * @param Object object to retrieve the name for; null gives "None".
 * @return Name of the object.
*/
inline FString GetNameSafe(const UObjectBaseUtility* Object)
{
	if (Object == nullptr)
	{
		return TEXT("None");
	}
	else
	{
		return Object->GetName();
	}
}

#if UE_WITH_REMOTE_OBJECT_HANDLE
/**
 * Returns the name of this object (with no path information). This version can avoid migrations for remote objects.
 * @param Object object to retrieve the name for; null gives "None".
 * @return Name of the object.
 */
template<class T UE_REQUIRES(TIsDerivedFrom<T, UObjectBase>::Value)>
inline FString GetNameSafe(const TObjectPtr<T>& ObjectPtr)
{
	if (ObjectPtr)
	{
		return ObjectPtr.GetName();
	}
	else
	{
		return TEXT("None");
	}
}
#endif

/**
 * Returns the path name of this object.
 * @param Object object to retrieve the path name for; null gives "None".
 * @return path name of the object.
*/
inline FString GetPathNameSafe(const UObjectBaseUtility* Object)
{
	if (Object == nullptr)
	{
		return TEXT("None");
	}
	else
	{
		return Object->GetPathName();
	}
}

/**
 * Returns the full name of this object.
 * @param Object object to retrieve the full name for; null (or a null class!) gives "None".
 * @return full name of the object.
*/
inline FString GetFullNameSafe(const UObjectBaseUtility* Object)
{
	if (!Object || !Object->GetClass())
	{
		return TEXT("None");
	}
	else
	{
		return Object->GetFullName();
	}
}

/**
 *	Returns the native (C++) parent class of the supplied class
 *	If supplied class is native, it will be returned.
 */
COREUOBJECT_API UClass* GetParentNativeClass(UClass* Class);

/** 
* Returns the TopMost non-native parent class of the supplied class.
* 
* @param Class - The class used to find its TopMost non-native parent class.
* @param bAllowReturnNativeClass - If supplied class is native, it will be returned.
*
* @return
*	- TopMost non-native parent class found in the hierarchy.
*	- Class if the supplied Class is already native and bAllowReturnNativeClass is true.
*	- nullptr if the supplied class is already native and bAllowReturnNativeClass is false.
*/
COREUOBJECT_API UClass* GetTopMostNonNativeClass(UClass* Class, bool bAllowReturnNativeClass);

#if !defined(USE_LIGHTWEIGHT_UOBJECT_STATS_FOR_HITCH_DETECTION)
#define USE_LIGHTWEIGHT_UOBJECT_STATS_FOR_HITCH_DETECTION (1)
#endif

#if CPUPROFILERTRACE_ENABLED
COREUOBJECT_API FName GetClassTraceScope(const UObjectBaseUtility* Object);
#endif

#if STATS

/** Structure used to track time spent by a UObject */
class FScopeCycleCounterUObject : public FCycleCounter
{
public:
	/**
	 * Constructor, starts timing
	 */
	inline FScopeCycleCounterUObject(const UObjectBaseUtility *Object)
	{
		UE_AUTORTFM_OPEN
		{
			if (Object)
			{
				bool bStarted = false;
				TStatId ObjectStatId = Object->GetStatID();
				if (FThreadStats::IsCollectingData(ObjectStatId))
				{
					Start(ObjectStatId);
					bStarted = true;
				}

#if CPUPROFILERTRACE_ENABLED
				if (!bStarted && UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
				{
					StartObjectTrace(Object);
				}
#endif
			}
		};

		AutoRTFM::PushOnAbortHandler(this, [this](){ this->Stop(); });
	}
	/**
	 * Constructor, starts timing with an alternate enable stat to use high performance disable for only SOME UObject stats
	 */
	inline FScopeCycleCounterUObject(const UObjectBaseUtility *Object, TStatId OtherStat)
	{
		UE_AUTORTFM_OPEN
		{
			if (Object)
			{
				bool bStarted = false;
				if (FThreadStats::IsCollectingData(OtherStat))
				{
					TStatId ObjectStatId = Object->GetStatID();
					if (!ObjectStatId.IsNone())
					{
						Start(ObjectStatId);
						bStarted = true;
					}
				}

#if CPUPROFILERTRACE_ENABLED
				if (!bStarted && UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel))
				{
					StartObjectTrace(Object);
				}
#endif
			}
		};

		AutoRTFM::PushOnAbortHandler(this, [this](){ this->Stop(); });
	}

#if CPUPROFILERTRACE_ENABLED
	COREUOBJECT_API void StartObjectTrace(const UObjectBaseUtility* Object);
#endif

	/**
	 * Updates the stat with the time spent
	 */
	inline ~FScopeCycleCounterUObject()
	{
		AutoRTFM::PopOnAbortHandler(this);
		UE_AUTORTFM_OPEN
		{
			Stop();
		};
	}
};

/** Declares a scope cycle counter for a specific object with a Name context */
#define SCOPE_CYCLE_UOBJECT(Name, Object) \
	FScopeCycleCounterUObject ObjCycleCount_##Name(Object);

#elif UE_USE_LIGHTWEIGHT_STATS

class FScopeCycleCounterUObject
{
public:
	FScopeCycleCounter ScopeCycleCounter;
#if ENABLE_STATNAMEDEVENTS_UOBJECT && CPUPROFILERTRACE_ENABLED
	bool bPop;
#endif

	inline FScopeCycleCounterUObject(const UObjectBaseUtility *Object)
		: ScopeCycleCounter(Object ? Object->GetStatID().StatString : nullptr)
#if ENABLE_STATNAMEDEVENTS_UOBJECT && CPUPROFILERTRACE_ENABLED
		, bPop(false)
#endif
	{
#if ENABLE_STATNAMEDEVENTS_UOBJECT && CPUPROFILERTRACE_ENABLED
		UE_AUTORTFM_OPEN
		{
			if (GCycleStatsShouldEmitNamedEvents && UE_TRACE_CHANNELEXPR_IS_ENABLED(CpuChannel) && Object)
			{
				const TStatId ObjectStatId = Object->GetStatID();
				if (ObjectStatId.IsValidStat())
				{
					bPop = true;
					FCpuProfilerTrace::OutputBeginDynamicEvent(ObjectStatId.StatString);
				}
			}
		};

		if(bPop)
		{
			AutoRTFM::PushOnAbortHandler(this, [](){ FCpuProfilerTrace::OutputEndEvent(); });
		}
#endif
	}

	UE_FORCEINLINE_HINT FScopeCycleCounterUObject(const UObjectBaseUtility *Object, TStatId OtherStat)
		: FScopeCycleCounterUObject(Object)
	{
	}

	inline ~FScopeCycleCounterUObject()
	{
#if ENABLE_STATNAMEDEVENTS_UOBJECT && CPUPROFILERTRACE_ENABLED
		if (bPop)
		{
			AutoRTFM::PopOnAbortHandler(this);

			UE_AUTORTFM_OPEN
			{
				FCpuProfilerTrace::OutputEndEvent();
			};
		}
#endif
	}
};

/** Declares a scope cycle counter for a specific object with a Name context */
#define SCOPE_CYCLE_UOBJECT(Name, Object) \
	FScopeCycleCounterUObject ObjCycleCount_##Name(Object);
#elif USE_LIGHTWEIGHT_STATS_FOR_HITCH_DETECTION && USE_HITCH_DETECTION && USE_LIGHTWEIGHT_UOBJECT_STATS_FOR_HITCH_DETECTION
extern CORE_API TSAN_ATOMIC(bool) GHitchDetected;

class FScopeCycleCounterUObject
{
	const UObject* StatObject;
public:
	UE_FORCEINLINE_HINT  FScopeCycleCounterUObject(const UObject* InStatObject, TStatId OtherStat = TStatId())
	{
		StatObject = GHitchDetected ? nullptr : InStatObject;
	}

	inline ~FScopeCycleCounterUObject()
	{
		if (GHitchDetected &&  StatObject)
		{
			UE_AUTORTFM_OPEN
			{
				ReportHitch();
			};
		}
	}

	COREUOBJECT_API void ReportHitch();
};

/** Declares a scope cycle counter for a specific object with a Name context */
#define SCOPE_CYCLE_UOBJECT(Name, Object) \
	FScopeCycleCounterUObject ObjCycleCount_##Name(Object);
#else
class FScopeCycleCounterUObject
{
public:
	UE_FORCEINLINE_HINT FScopeCycleCounterUObject(const UObjectBaseUtility *Object)
	{
	}
	UE_FORCEINLINE_HINT FScopeCycleCounterUObject(const UObjectBaseUtility *Object, TStatId OtherStat)
	{
	}
};

#define SCOPE_CYCLE_UOBJECT(Name, Object)

#endif

#if defined(_MSC_VER) && _MSC_VER == 1900
	#ifdef PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS
		PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS
	#endif
#endif
