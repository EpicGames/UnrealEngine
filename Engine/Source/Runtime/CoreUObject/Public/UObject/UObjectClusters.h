// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectClusters.h: Unreal UObject Cluster helper functions
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Misc/Build.h"

class FString;

#ifndef UE_GCCLUSTER_VERBOSE_LOGGING
#define UE_GCCLUSTER_VERBOSE_LOGGING (0 && !UE_BUILD_SHIPPING)
#endif

#if !UE_BUILD_SHIPPING

// Dumps a single cluster to log
COREUOBJECT_API void DumpClusterToLog(const struct FUObjectCluster& Cluster, bool bHierarchy, bool bIndexOnly);

// Dumps all clusters to log.
COREUOBJECT_API void ListClusters(const TArray<FString>& Args);

// Attempts to find clusters with no references to them
COREUOBJECT_API void FindStaleClusters(const TArray<FString>& Args);

#endif // !UE_BUILD_SHIPPING

// Whether object clusters can be created or not.
bool CanCreateObjectClusters();
