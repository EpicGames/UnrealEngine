// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectGlobals.h: Unreal object system globals.
=============================================================================*/

#pragma once

#include "Concepts/SameAs.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Logging/LogMacros.h"
#include "Logging/LogVerbosity.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AssetReadLoggerConfig.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/PackagePath.h"
#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryLayout.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "Templates/IsArrayOrRefOfTypeByPredicate.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "Templates/IsTObjectPtr.h"
#include "Traits/IsCharEncodingCompatibleWith.h"
#include "UObject/CompiledInObjectPtr.h"
#include "UObject/CoreReflectedTypeAccessors.h"
#include "UObject/FindObjectFlags.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectHandle.h"
#include "UObject/PrimaryAssetId.h"
#include "UObject/Script.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/RemoteObjectTypes.h"
#include "UObject/UnrealNames.h"
#include "VerseTypesFwd.h"

class FArchive;
class FCbWriter;
class FLinkerInstancingContext;
class FObjectPreSaveContext;
class FOutputDevice;
class FPackagePath;
class FProperty;
class ITargetPlatform;
class UClass;
class UEnum;
class UFunction;
class UObject;
class UObjectBase;
class UPackage;
class UPackageMap;
class UScriptStruct;
class UVerseClass;
class UWorld;
struct FCustomPropertyListNode;
struct FGuid;
struct FObjectInstancingGraph;
struct FObjectPostCDOCompiledContext;
struct FObjectPtr;
struct FPrimaryAssetId;
struct FStaticConstructObjectParameters;
struct FUObjectSerializeContext;
struct FVniTypeDesc;
struct FWorldContext;
template <typename T>
struct TObjectPtr;

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogUObjectGlobals, Log, All);

DECLARE_CYCLE_STAT_EXTERN(TEXT("ConstructObject"),STAT_ConstructObject,STATGROUP_Object, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("AllocateObject"),STAT_AllocateObject,STATGROUP_ObjectVerbose, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("PostConstructInitializeProperties"),STAT_PostConstructInitializeProperties,STATGROUP_ObjectVerbose, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("LoadConfig"),STAT_LoadConfig,STATGROUP_Object, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("LoadObject"),STAT_LoadObject,STATGROUP_Object, );
DECLARE_CYCLE_STAT_WITH_FLAGS_EXTERN(TEXT("InitProperties"),STAT_InitProperties,STATGROUP_Object, EStatFlags::Verbose, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("NameTable Entries"),STAT_NameTableEntries,STATGROUP_Object, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("NameTable ANSI Entries"),STAT_NameTableAnsiEntries,STATGROUP_Object, );
DECLARE_DWORD_ACCUMULATOR_STAT_EXTERN(TEXT("NameTable Wide Entries"),STAT_NameTableWideEntries,STATGROUP_Object, );
DECLARE_MEMORY_STAT_EXTERN(TEXT("NameTable Memory Size"),STAT_NameTableMemorySize,STATGROUP_Object, );
DECLARE_CYCLE_STAT_EXTERN(TEXT("~UObject"),STAT_DestroyObject,STATGROUP_Object, );

DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("FindObject"),STAT_FindObject,STATGROUP_ObjectVerbose, );
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("FindObjectFast"),STAT_FindObjectFast,STATGROUP_ObjectVerbose, );

#define	INVALID_OBJECT	(UObject*)-1
#define PERF_TRACK_DETAILED_ASYNC_STATS (0)

#ifndef UE_GC_RUN_WEAKPTR_BARRIERS
#define UE_GC_RUN_WEAKPTR_BARRIERS 0
#endif

#if UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR
#define UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED() UE_DEPRECATED(5.3, "WARNING: Your program will randomly crash if this function is called when incremental gc is enabled. Pass TObjectPtr<...> instead of UObject* to AddReferencedObject(s) API's.")
#else
#define UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED(...)
#endif

// Private system wide variables.

/** 
 * Set while in SavePackage() to advertise that a package is being saved
 * Deprecated, use `IsSavingPackage` instead
 * @see IsSavingPackage()
 */
UE_DEPRECATED(5.8, "Use the IsSavingPackage() function instead.")
extern COREUOBJECT_API bool					GIsSavingPackage;
/** This allows loading unversioned cooked content in the editor */
extern COREUOBJECT_API int32				GAllowUnversionedContentInEditor;
/** This allows loading cooked content in the editor */
extern COREUOBJECT_API int32				GAllowCookedDataInEditorBuilds;

/** Enum used in StaticDuplicateObject() and related functions to describe why something is being duplicated */
namespace EDuplicateMode
{
	enum Type
	{
		/** No specific information about the reason for duplication */
		Normal,
		/** Object is being duplicated as part of a world duplication */
		World,
		/** Object is being duplicated as part of the process for entering Play In Editor */
		PIE
	};
};

/*-----------------------------------------------------------------------------
	FObjectDuplicationParameters.
-----------------------------------------------------------------------------*/

/**
 * This struct is used for passing parameter values to the StaticDuplicateObject() method.  Only the constructor parameters are required to
 * be valid - all other members are optional.
 */
struct FObjectDuplicationParameters
{
	/**
	 * The object to be duplicated
	 */
	UObject*		SourceObject;

	/**
	 * The object to use as the Outer for the duplicate of SourceObject.
	 */
	UObject*		DestOuter;

	/**
	 * The name to use for the duplicate of SourceObject
	 */
	FName			DestName;

	/**
	 * A bitmask of EObjectFlags to propagate to the duplicate of SourceObject (and its subobjects).
	 */
	EObjectFlags	FlagMask;

	/**
	 * A bitmask of EInternalObjectFlags to propagate to the duplicate of SourceObject (and its subobjects).
	*/
	EInternalObjectFlags InternalFlagMask;

	/**
	 * A bitmask of EObjectFlags to set on each duplicate object created.  Different from FlagMask in that only the bits
	 * from FlagMask which are also set on the source object will be set on the duplicate, while the flags in this value
	 * will always be set.
	 */
	EObjectFlags	ApplyFlags;

	/**
	 * A bitmask of EInternalObjectFlags to set on each duplicate object created.  Different from FlagMask in that only the bits
	* from FlagMask which are also set on the source object will be set on the duplicate, while the flags in this value
	* will always be set.
	*/
	EInternalObjectFlags	ApplyInternalFlags;

	/**
	 * Any PortFlags to be applied when serializing.
	 */
	uint32			PortFlags;

	EDuplicateMode::Type DuplicateMode;

	/**
	 * if an object being duplicated as an assigned external package, the duplicated object will try to assign an associated package to itself.
	 * The associated package should come from the DuplicationSeed.
	 */
	bool bAssignExternalPackages = true;

	/**
	 * if this option is true, then PostLoad won't be called on the new duplicated objects. 
	 * It will be the responsability of the caller in that case to eventually call post load on those objects. a `CreatedObjects` map should be provided.
	 */
	bool bSkipPostLoad = false;
	
	/**
	 * Optional class to specify for the destination object.
	 * @warning: MUST BE SERIALIZATION COMPATIBLE WITH SOURCE OBJECT, AND DOES NOT WORK WELL FOR OBJECT WHICH HAVE COMPLEX COMPONENT HIERARCHIES!!!
	 */
	UClass*			DestClass;

	/**
	 * Objects to use for prefilling the dup-source => dup-target map used by StaticDuplicateObject.  Can be used to allow individual duplication of several objects that share
	 * a common Outer in cases where you don't want to duplicate the shared Outer but need references between the objects to be replaced anyway.
	 *
	 * Objects in this map will NOT be duplicated
	 * Key should be the source object; value should be the object which will be used as its duplicate.
	 */
	TMap<UObject*,UObject*>	DuplicationSeed;

	/**
	 * If non-null, this will be filled with the list of objects created during the call to StaticDuplicateObject.
	 *
	 * Key will be the source object; value will be the duplicated object
	 */
	TMap<UObject*,UObject*>* CreatedObjects;

	/**
	 * Constructor
	 */
	COREUOBJECT_API FObjectDuplicationParameters( UObject* InSourceObject, UObject* InDestOuter );
};

// @note: should we start putting the code in the UE namespace?
namespace UE
{
	/**
	 * Return if the current thread is in a saving scope.
	 * @returns true the current thread is inside a saving scope.
	 */
	COREUOBJECT_API bool IsSavingPackage();

	/**
	 * Return if the passed in package is currently saving,
	 * or if if the current thread is in a saving scope if InOuter == nullptr.
	 * @param InOuter The object which we want to check if its package is saving
	 * @returns true if the package is saving, or if the current thread is in a saving scope if InOuter == nullptr.
	 */
	COREUOBJECT_API bool IsSavingPackage(UObject* InOuter);

	/**
	 * Used to toggle the value which is needed is some rare circumstance.
	 *
	 * @param bIsSavingPackage The value we want to set for the current thread.
	 * 
	 * @note Should be used with TGuardValueAccessors for RAII style restore of the old flag.
	 */
	COREUOBJECT_API void SetIsSavingPackage(bool bIsSavingPackage);

} // namespace UE

namespace UE::RemoteObject::Serialization
{
	class FRemoteObjectConstructionOverrides;
} // namespace UE::RemoteObject::Serialization

/** Parses a bit mask of property flags into an array of string literals that match the flags */
COREUOBJECT_API TArray<const TCHAR*> ParsePropertyFlags(EPropertyFlags Flags);

/** Returns the transient top-level package, which is useful for temporarily storing objects that should never be saved */
COREUOBJECT_API UPackage* GetTransientPackage();
COREUOBJECT_API UObject* GetTransientPackageAsObject();

/** Returns an object in the transient package which respects the rules of Within */
COREUOBJECT_API UObject* GetTransientOuterForRename(UClass* ForClass);

/**
 * Gets INI file name from object's reference if it contains one. 
 *
 * @returns If object reference doesn't contain any INI reference the function returns nullptr. Otherwise a ptr to INI's file name.
 */
COREUOBJECT_API const FString* GetIniFilenameFromObjectsReference(const FString& ObjectsReferenceString);

/**
 * Resolves ini object path to string object path. This used to happen automatically in ResolveName but now must be called manually
 *
 * @param ObjectReference Ini reference, of the form engine-ini:/Script/Engine.Engine.DefaultMaterialName
 * @param IniFilename Ini filename. If null it will call GetIniFilenameFromObjectsReference
 * @param bThrow If true, will print an error if it can't find the file
 *
 * @returns Resolved object path.
 */
COREUOBJECT_API FString ResolveIniObjectsReference(const FString& ObjectReference, const FString* IniFilename = nullptr, bool bThrow = false);

/**
 * Internal function that takes a fully qualified or relative object path string and converts it into a path relative to a package.
 * Normally, you should call one of the FindObject or LoadObject functions instead.
 *
 * @param	Outer					The package to search within. If null, ObjectsReferenceString be a globally scoped path and this will be filled in with the actual package if found/created
 * @param	ObjectsReferenceString	The object path string to resolve. If it is successfully resolved, this will be replaced with a path relative to Outer
 * @param	Create					If true, it will try to load or create the required package if it is not in memory
 * @param	Throw					If true, it will potentially raise an error if the object cannot be found
 * @param	LoadFlags				Flags to use if Create is true and it needs to load a package, from the ELoadFlags enum
 * @param	InstancingContext		The linker instancing context used to resolve package name during instacning (i.e. when a package file is loaded into a package with a different name)
 * @return	True if the name was successfully resolved
 */
COREUOBJECT_API bool ResolveName(UObject*& Outer, FString& ObjectsReferenceString, bool Create, bool Throw, uint32 LoadFlags = LOAD_None, const FLinkerInstancingContext* InstancingContext = nullptr);

/** Internal function used to possibly output an error message, taking into account the outer and LoadFlags. Returns true if a log message was emitted. */
COREUOBJECT_API bool SafeLoadError( UObject* Outer, uint32 LoadFlags, const TCHAR* ErrorMessage);

/** Internal function used to update the suffix to be given to the next newly-created unnamed object. */
COREUOBJECT_API int32 UpdateSuffixForNextNewObject(UObject* Parent, const UClass* Class, TFunctionRef<void(int32&)> IndexMutator);

/**
 * Fast version of StaticFindObject that relies on the passed in FName being the object name without any group/package qualifiers.
 * This will only find top level packages or subobjects nested directly within a passed in outer.
 *
 * @param	Class			The to be found object's class
 * @param	InOuter			Outer object to look inside, if null this will only look for top level packages
 * @param	InName			Object name to look for relative to InOuter
 * @param	Flags			Flags which control the search
 * @param	ExclusiveFlags	Ignores objects that contain any of the specified exclusive flags
 * @param	ExclusiveInternalFlags  Ignores objects that contain any of the specified internal exclusive flags
 *
 * @return	Returns a pointer to the found object or null if none could be found
 */
COREUOBJECT_API UObject* StaticFindObjectFast(UClass* Class, UObject* InOuter, FName InName, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObjectFast")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObjectFast(UClass* Class, UObject* InOuter, FName InName, ExactClassType bExactClass, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None)
{
	return StaticFindObjectFast(Class, InOuter, InName, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExclusiveFlags, ExclusiveInternalFlags);
}

/**
 * Fast and safe version of StaticFindObject that relies on the passed in FName being the object name without any group/package qualifiers.
 * It will not assert on UE::IsSavingPackage() or IsGarbageCollectingAndLockingUObjectHashTables(). If called from within package saving code or GC, will return nullptr
 * This will only find top level packages or subobjects nested directly within a passed in outer.
 *
 * @param	Class			The to be found object's class
 * @param	InOuter			Outer object to look inside, if null this will only look for top level packages
 * @param	InName			Object name to look for relative to InOuter
 * @param	Flags			Flags which control the search
 * @param	ExclusiveFlags	Ignores objects that contain any of the specified exclusive flags
 * @param	ExclusiveInternalFlags  Ignores objects that contain any of the specified internal exclusive flags
 *
 * @return	Returns a pointer to the found object or null if none could be found
 */
COREUOBJECT_API UObject* StaticFindObjectFastSafe(UClass* Class, UObject* InOuter, FName InName, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObjectFastSafe")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObjectFastSafe(UClass* Class, UObject* InOuter, FName InName, ExactClassType bExactClass, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None)
{
	return StaticFindObjectFastSafe(Class, InOuter, InName, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExclusiveFlags, ExclusiveInternalFlags);
}

/**
 * Tries to find an object in memory. This will handle fully qualified paths of the form /path/packagename.object:subobject and resolve references for you.
 *
 * @param	Class			The to be found object's class
 * @param	InOuter			Outer object to look inside. If this is null then InName should start with a package name
 * @param	InName			The object path to search for an object, relative to InOuter
 * @param	Flags			Flags which control the search
 *
 * @return	Returns a pointer to the found object or nullptr if none could be found
 */
COREUOBJECT_API UObject* StaticFindObject( UClass* Class, UObject* InOuter, FStringView Name, EFindObjectFlags Flags = EFindObjectFlags::None );
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObject")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObject( UClass* Class, UObject* InOuter, FStringView Name, ExactClassType bExactClass )
{
	return StaticFindObject(Class, InOuter, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Tries to find an object in memory. This version uses FTopLevelAssetPath to find the object.
 *
 * @param	Class			The to be found object's class
 * @param	ObjectPath		FName pair representing the outer package object and the inner top level object (asset)
 * @param	Flags			Flags which control the search
 *
 * @return	Returns a pointer to the found object or nullptr if none could be found
 */
COREUOBJECT_API UObject* StaticFindObject(UClass* Class, FTopLevelAssetPath ObjectPath, EFindObjectFlags Flags = EFindObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObject")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObject(UClass* Class, FTopLevelAssetPath ObjectPath, ExactClassType bExactClass)
{
	return StaticFindObject(Class, ObjectPath, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/** Version of StaticFindObject() that will assert if the object is not found */
COREUOBJECT_API UObject* StaticFindObjectChecked( UClass* Class, UObject* InOuter, FStringView Name, EFindObjectFlags Flags = EFindObjectFlags::None );
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObjectChecked")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObjectChecked( UClass* Class, UObject* InOuter, FStringView Name, ExactClassType bExactClass)
{
	return StaticFindObjectChecked(Class, InOuter, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/** Internal version of StaticFindObject that will not assert on UE::IsSavingPackage() or IsGarbageCollectingAndLockingUObjectHashTables() */
COREUOBJECT_API UObject* StaticFindObjectSafe( UClass* Class, UObject* InOuter, FStringView Name, EFindObjectFlags Flags = EFindObjectFlags::None );
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObjectSafe")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObjectSafe( UClass* Class, UObject* InOuter, FStringView Name, ExactClassType bExactClass )
{
	return StaticFindObjectSafe(Class, InOuter, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Tries to find an object in memory. This version uses FTopLevelAssetPath to find the object.
 * Version of StaticFindObject that will not assert on UE::IsSavingPackage() or IsGarbageCollectingAndLockingUObjectHashTables()
 *
 * @param	Class			The to be found object's class
 * @param	ObjectPath		FName pair representing the outer package object and the inner top level object (asset)
 * @param	Flags			Flags which control the search
 *
 * @return	Returns a pointer to the found object or nullptr if none could be found
 */
COREUOBJECT_API UObject* StaticFindObjectSafe(UClass* Class, FTopLevelAssetPath ObjectPath, EFindObjectFlags Flags = EFindObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObjectSafe")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObjectSafe(UClass* Class, FTopLevelAssetPath ObjectPath, ExactClassType bExactClass)
{
	return StaticFindObjectSafe(Class, ObjectPath, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Fast version of StaticFindAllObjects that relies on the passed in FName being the object name without any group/package qualifiers.
 * This will find all objects matching the specified name and class.
 *
 * @param	OutFoundObjects	Array of objects matching the search parameters
 * @param	ObjectClass		The to be found object's class
 * @param	ObjectName		Object name to look for relative to InOuter
 * @param	Flags			Flags which control the search
 * @param	ExclusiveFlags	Ignores objects that contain any of the specified exclusive flags
 * @param	ExclusiveInternalFlags  Ignores objects that contain any of the specified internal exclusive flags
 *
 * @return	Returns true if any objects were found, false otherwise
 */
COREUOBJECT_API bool StaticFindAllObjectsFast(TArray<UObject*>& OutFoundObjects, UClass* ObjectClass, FName ObjectName, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindAllObjectsFast")
UE_NODEBUG UE_FORCEINLINE_HINT bool StaticFindAllObjectsFast(TArray<UObject*>& OutFoundObjects, UClass* ObjectClass, FName ObjectName, ExactClassType bExactClass, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None)
{
	return StaticFindAllObjectsFast(OutFoundObjects, ObjectClass, ObjectName, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExclusiveFlags, ExclusiveInternalFlags);
}

/**
 * Fast version of StaticFindAllObjects that relies on the passed in FName being the object name without any group/package qualifiers.
 * This will find all objects matching the specified name and class.
 * This version of StaticFindAllObjectsFast will not assert on UE::IsSavingPackage() or IsGarbageCollecting()
 * 
 * @param	OutFoundObjects	Array of objects matching the search parameters
 * @param	ObjectClass		The to be found object's class
 * @param	ObjectName		Object name to look for relative to InOuter
 * @param	Flags			Flags which control the search
 * @param	ExclusiveFlags	Ignores objects that contain any of the specified exclusive flags
 * @param	ExclusiveInternalFlags  Ignores objects that contain any of the specified internal exclusive flags
 *
 * @return	Returns true if any objects were found, false otherwise
 */
COREUOBJECT_API bool StaticFindAllObjectsFastSafe(TArray<UObject*>& OutFoundObjects, UClass* ObjectClass, FName ObjectName, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindAllObjectsFastSafe")
UE_NODEBUG UE_FORCEINLINE_HINT bool StaticFindAllObjectsFastSafe(TArray<UObject*>& OutFoundObjects, UClass* ObjectClass, FName ObjectName, ExactClassType bExactClass, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None)
{
	return StaticFindAllObjectsFastSafe(OutFoundObjects, ObjectClass, ObjectName, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExclusiveFlags, ExclusiveInternalFlags);
}

/**
 * Tries to find all objects matching the search parameters in memory. This will handle fully qualified paths of the form /path/packagename.object:subobject and resolve references for you.
 *
 * @param	OutFoundObjects	Array of objects matching the search parameters
 * @param	Class			The to be found object's class
 * @param	Name			The object path to search for an object, relative to InOuter
 * @param	Flags			Flags which control the search
 *
 * @return	Returns true if any objects were found, false otherwise
 */
COREUOBJECT_API bool StaticFindAllObjects(TArray<UObject*>& OutFoundObjects, UClass* Class, const TCHAR* Name, EFindObjectFlags Flags = EFindObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindAllObjects")
UE_NODEBUG UE_FORCEINLINE_HINT bool StaticFindAllObjects(TArray<UObject*>& OutFoundObjects, UClass* Class, const TCHAR* Name, ExactClassType bExactClass)
{
	return StaticFindAllObjects(OutFoundObjects, Class, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Tries to find all objects matching the search parameters in memory. This will handle fully qualified paths of the form /path/packagename.object:subobject and resolve references for you.
 * This version of StaticFindAllObjects will not assert on UE::IsSavingPackage() or IsGarbageCollecting()
 * 
 * @param	OutFoundObjects	Array of objects matching the search parameters
 * @param	Class			The to be found object's class
 * @param	Name			The object path to search for an object, relative to InOuter
 * @param	Flags			Flags which control the search
 *
 * @return	Returns true if any objects were found, false otherwise
 */
COREUOBJECT_API bool StaticFindAllObjectsSafe(TArray<UObject*>& OutFoundObjects, UClass* Class, const TCHAR* Name, EFindObjectFlags Flags = EFindObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindAllObjectsSafe")
UE_NODEBUG UE_FORCEINLINE_HINT bool StaticFindAllObjectsSafe(TArray<UObject*>& OutFoundObjects, UClass* Class, const TCHAR* Name, ExactClassType bExactClass)
{
	return StaticFindAllObjectsSafe(OutFoundObjects, Class, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}


enum class EFindFirstObjectOptions
{
	None = 0, // Unused / defaults to Quiet
	ExactClass = 1 << 1, // Whether to require an exact match with the passed in class
	NativeFirst = 1 << 2, // If multiple results are found, prioritize native classes or native class instances
	EnsureIfAmbiguous = 1 << 3 // Ensure if multiple results are found
};
ENUM_CLASS_FLAGS(EFindFirstObjectOptions);

/**
 * Tries to find the first object matching the search parameters in memory. This will handle fully qualified paths of the form /path/packagename.object:subobject and resolve references for you.
 * If multiple objects share the same name the returned object is random and not based on its time of creation unless otherwise specified in Options (see EFindFirstObjectOptions::NativeFirst)
 * This function is slow and should not be used in performance critical situations.
 * 
 * @param	Class						The to be found object's class
 * @param	Name						The object path to search for an object, relative to InOuter
 * @param	Options						Search options
 * @param	AmbiguousMessageVerbosity	Verbosity with which to print a message if the search result is ambiguous
 * @param	InCurrentOperation			Current operation to be logged with ambiguous search warning
 *
 * @return	Returns a pointer to an object if found, null otherwise
 */
COREUOBJECT_API UObject* StaticFindFirstObject(UClass* Class, FStringView Name, EFindFirstObjectOptions Options = EFindFirstObjectOptions::None, ELogVerbosity::Type AmbiguousMessageVerbosity = ELogVerbosity::NoLogging, const TCHAR* InCurrentOperation = nullptr);

/**
 * Tries to find the first objects matching the search parameters in memory. This will handle fully qualified paths of the form /path/packagename.object:subobject and resolve references for you.
 * This version of StaticFindFirstObject will not assert on UE::IsSavingPackage() or IsGarbageCollecting()
 * If multiple objects share the same name the returned object is random and not based on its time of creation unless otherwise specified in Options (see EFindFirstObjectOptions::NativeFirst)
 * This function is slow and should not be used in performance critical situations.
 * 
 * @param	Class						The to be found object's class
 * @param	Name						The object path to search for an object, relative to InOuter
 * @param	Options						Search options
 * @param	AmbiguousMessageVerbosity	Verbosity with which to print a message if the search result is ambiguous
 * @param	InCurrentOperation			Current operation to be logged with ambiguous search warning
 *
 * @return	Returns a pointer to an object if found, null otherwise
 */
COREUOBJECT_API UObject* StaticFindFirstObjectSafe(UClass* Class, FStringView Name, EFindFirstObjectOptions Options = EFindFirstObjectOptions::None, ELogVerbosity::Type AmbiguousMessageVerbosity = ELogVerbosity::NoLogging, const TCHAR* InCurrentOperation = nullptr);

/** Loading policy to use with ParseObject */
enum class EParseObjectLoadingPolicy : uint8
{
	/** Try and find the object, but do not attempt to load it */
	Find,
	/** Try and find the object, or attempt of load it if it cannot be found (note: loading may be globally disabled via the CVar "s.AllowParseObjectLoading") */
	FindOrLoad,
};

/**
 * Parse a reference to an object from a text representation
 *
 * @param Stream			String containing text to parse
 * @param Match				Tag to search for object representation within string
 * @param Class				The class of the object to be found.
 * @param DestRes			Returned object pointer
 * @param InParent			Outer to search
 * @oaran LoadingPolicy		Controls whether the parse will attempt to load a fully qualified object reference, if needed.
 * @param bInvalidObject	[opt] Optional output.  If true, Tag was matched but the specified object wasn't found.
 *
 * @return True if the object parsed successfully, even if object was not found
 */
COREUOBJECT_API bool ParseObject( const TCHAR* Stream, const TCHAR* Match, UClass* Class, UObject*& DestRes, UObject* InParent, EParseObjectLoadingPolicy LoadingPolicy, bool* bInvalidObject=nullptr );
inline bool ParseObject( const TCHAR* Stream, const TCHAR* Match, UClass* Class, UObject*& DestRes, UObject* InParent, bool* bInvalidObject=nullptr )
{
	return ParseObject( Stream, Match, Class, DestRes, InParent, EParseObjectLoadingPolicy::Find, bInvalidObject );
}

/**
 * Find or load an object by string name with optional outer and filename specifications.
 * These are optional because the InName can contain all of the necessary information.
 *
 * @param Class			The class (or a superclass) of the object to be loaded.
 * @param InOuter		An optional object to narrow where to find/load the object from
 * @param Name			String name of the object. If it's not fully qualified, InOuter and/or Filename will be needed
 * @param Filename		An optional file to load from (Deprecated parameter)
 * @param LoadFlags		Flags controlling how to handle loading from disk, from the ELoadFlags enum
 * @param Sandbox		A list of packages to restrict the search for the object (Deprecated parameter)
 * @param bAllowObjectReconciliation	Whether to allow the object to be found via FindObject before forcing a load (Deprecated parameter)
 * @param InstancingContext				InstancingContext used to remap imports when loading a package under a new name
 *
 * @return The object that was loaded or found. nullptr for a failure.
 */
COREUOBJECT_API UObject* StaticLoadObject( UClass* Class, UObject* InOuter, FStringView Name, FStringView Filename = {}, uint32 LoadFlags = LOAD_None, UPackageMap* Sandbox = nullptr, bool bAllowObjectReconciliation = true, const FLinkerInstancingContext* InstancingContext = nullptr);

/** Version of StaticLoadObject() that will load classes */
COREUOBJECT_API UClass* StaticLoadClass(UClass* BaseClass, UObject* InOuter, FStringView Name, FStringView Filename = {}, uint32 LoadFlags = LOAD_None, UPackageMap* Sandbox = nullptr);

/**
 * Find or load an object that is one of the top level assets in a package.
 *
 * @param	Class				The class (or a superclass) of the object to be loaded.
 * @param	InPath				FName pair representing the outer package object and the inner top level object (asset)
 * @param	LoadFlags			Flags controlling how to handle loading from disk, from the ELoadFlags enum
 * @param	InstancingContext	InstancingContext used to remap imports when loading a package under a new name
 *
 * @return	Returns a pointer to the found object or nullptr if none could be found
 */
COREUOBJECT_API UObject* StaticLoadAsset(UClass* Class, FTopLevelAssetPath InPath, uint32 LoadFlags = LOAD_None, const FLinkerInstancingContext* InstancingContext = nullptr);

/**
 * Create a new instance of an object.  The returned object will be fully initialized.  If InFlags contains RF_NeedsLoad (indicating that the object still needs to load its object data from disk), components
 * are not instanced (this will instead occur in PostLoad()).  The different between StaticConstructObject and StaticAllocateObject is that StaticConstructObject will also call the class constructor on the object
 * and instance any components.
 *
 * @param	Params		The parameters to use when construction the object. @see FStaticConstructObjectParameters
 *
 * @return	A pointer to a fully initialized object of the specified class.
 */
COREUOBJECT_API UObject* StaticConstructObject_Internal(const FStaticConstructObjectParameters& Params);

/**
 * Creates a copy of SourceObject using the Outer and Name specified, as well as copies of all objects contained by SourceObject.  
 * Any objects referenced by SourceOuter or RootObject and contained by SourceOuter are also copied, maintaining their name relative to SourceOuter.
 * Any references to objects that are duplicated are automatically replaced with the copy of the object.
 *
 * @param	SourceObject	The object to duplicate
 * @param	DestOuter		The object to use as the Outer for the copy of SourceObject
 * @param	DestName		The name to use for the copy of SourceObject, if none it will be autogenerated
 * @param	FlagMask		A bitmask of EObjectFlags that should be propagated to the object copies.  The resulting object copies will only have the object flags
 *							specified copied from their source object.
 * @param	DestClass		Optional class to specify for the destination object. MUST BE SERIALIZATION COMPATIBLE WITH SOURCE OBJECT!!!
 * @param	InternalFlagsMask  Bitmask of EInternalObjectFlags that should be propagated to the object copies.
 *
 * @return	The duplicate of SourceObject.
 *
 * @deprecated This version is deprecated in favor of StaticDuplicateObjectEx
 */
COREUOBJECT_API UObject* StaticDuplicateObject(UObject const* SourceObject, UObject* DestOuter, const FName DestName = NAME_None, EObjectFlags FlagMask = RF_AllFlags, UClass* DestClass = nullptr, EDuplicateMode::Type DuplicateMode = EDuplicateMode::Normal, EInternalObjectFlags InternalFlagsMask = EInternalObjectFlags_AllFlags);

/**
 * Returns FObjectDuplicationParameters initialized based of StaticDuplicateObject parameters
 */
COREUOBJECT_API FObjectDuplicationParameters InitStaticDuplicateObjectParams(UObject const* SourceObject, UObject* DestOuter, const FName DestName = NAME_None, EObjectFlags FlagMask = RF_AllFlags, UClass* DestClass = nullptr, EDuplicateMode::Type DuplicateMode = EDuplicateMode::Normal, EInternalObjectFlags InternalFlagsMask = EInternalObjectFlags_AllFlags);

/**
 * Creates a copy of SourceObject using the Outer and Name specified, as well as copies of all objects contained by SourceObject.
 * Any objects referenced by SourceOuter or RootObject and contained by SourceOuter are also copied, maintaining their name relative to SourceOuter.
 * Any references to objects that are duplicated are automatically replaced with the copy of the object.
 *
 * @param	Parameters  Specific options to use when duplicating this object
 *
 * @return	The duplicate of SourceObject.
 */
COREUOBJECT_API UObject* StaticDuplicateObjectEx( FObjectDuplicationParameters& Parameters );

/** 
 * Parses a global context system console or debug command and executes it.
 *
 * @param	InWorld		The world to use as a context, enables certain commands
 * @param	Cmd			Command string to execute
 * @param	Ar			Output device to write results of commands to
 * 
 * @return	True if the command was successfully parsed
 */
COREUOBJECT_API bool StaticExec( UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar=*GLog );

/**
 * Static UObject tick function, used to verify certain key assumptions and to tick the async loading code.
 *
 * @param DeltaTime	Time in seconds since last call
 * @param bUseFullTimeLimit	If true, use the entire time limit even if blocked on I/O
 * @param AsyncLoadingTime Time in seconds to use for async loading limit
 */
COREUOBJECT_API void StaticTick( float DeltaTime, bool bUseFullTimeLimit = true, float AsyncLoadingTime = 0.005f );

/**
 * Loads a package and all contained objects that match context flags.
 *
 * @param	InOuter				Package to load new package into (usually nullptr or ULevel->GetOuter())
 * @param	InLongPackageName	Long package name to load, if null the name is taken from InOuter
 * @param	LoadFlags			Flags controlling loading behavior, from the ELoadFlags enum
 * @param	InReaderOverride	Optional archive to use for reading package data
 * @param	InLoadContext		Additional context when called during serialization
 * @param   InstancingContext   Additional context to map object names to their instanced counterpart when loading an instanced package
 *
 * @return	Loaded package if successful, nullptr otherwise
 */
COREUOBJECT_API UPackage* LoadPackage( UPackage* InOuter, const TCHAR* InLongPackageName, uint32 LoadFlags, FArchive* InReaderOverride = nullptr, const FLinkerInstancingContext* InstancingContext = nullptr);

/**
 * Loads a package and all contained objects that match context flags.
 *
 * @param	InOuter				Package to load new package into (usually nullptr or ULevel->GetOuter())
 * @param	InPackagePath		PackagePath to load, must be non-empty
 * @param	LoadFlags			Flags controlling loading behavior, from the ELoadFlags enum
 * @param	InReaderOverride	Optional archive to use for reading package data
 * @param	InLoadContext		Additional context when called during serialization
 * @param   InstancingContext   Additional context to map object names to their instanced counterpart when loading an instanced package
 * @param   DiffPackagePath		An additional PackagePath to load and compare to the package at InPackagePath, used when diffing packages
 *
 * @return	Loaded package if successful, nullptr otherwise
 */
COREUOBJECT_API UPackage* LoadPackage(UPackage* InOuter, const FPackagePath& InPackagePath, uint32 LoadFlags, FArchive* InReaderOverride = nullptr, const FLinkerInstancingContext* InstancingContext = nullptr, const FPackagePath* DiffPackagePath = nullptr);

/** Async package and object loading result */
namespace EAsyncLoadingResult
{
	enum Type : uint8
	{
		/** Package or object failed to load */
		Failed,
		/** Package or object failed to load because it is missing */
		FailedMissing,
		/** Package or object failed to load due to a linker error */
		FailedLinker,
		/** Package or object failed to load because its data is not local */
		FailedNotInstalled,
		/** Package or object loaded successfully */
		Succeeded,
		/** Async loading was canceled */
		Canceled
	};
}

/** Async loading progress for a specific package */
enum class EAsyncLoadingProgress : uint32
{
	/** Package failed to load */
	Failed,
	/** Package has started loading. */
	Started,
	/** Package I/O has been read. */
	Read,
	/** Package has finished its serialization phase. */
	Serialized,
	/** Package has finished all loading phase successfully */
	FullyLoaded
};

/** The type that represents an async loading priority */
typedef int32 TAsyncLoadPriority;

/**
 * Delegate called on completion of async package loading
 * @param	PackageName			Package name we were trying to load
 * @param	LoadedPackage		Loaded package if successful, nullptr otherwise	
 * @param	Result		Result of async loading.
 */
DECLARE_DELEGATE_ThreeParams(FLoadPackageAsyncDelegate, const FName& /*PackageName*/, UPackage* /*LoadedPackage*/, EAsyncLoadingResult::Type /*Result*/)

/**
 * Parameters passed to the FLoadPackageAsyncProgressDelegate callback.
 */
struct FLoadPackageAsyncProgressParams
{
	/* Name of the package. */
	const FName PackageName { NAME_None };
	/* Pointer to UPackage being loaded, can be nullptr depending on async loading progress. */
	const UPackage* LoadedPackage { nullptr };
	/* Progress of async loading. */
	const EAsyncLoadingProgress ProgressType { EAsyncLoadingProgress::Failed };
};

/**
 * Thread-safe delegate called on progress of async package loading.
 */
class FLoadPackageAsyncProgressDelegate
{
public:
	using DelegateType = TTSDelegate<void(FLoadPackageAsyncProgressParams& Params)>;

	FLoadPackageAsyncProgressDelegate(DelegateType&& InDelegate, uint32 InMask = MAX_uint32)
		: Delegate(Forward<DelegateType>(InDelegate))
		, Mask(InMask)
	{}

	constexpr static inline uint32 BuildMask(EAsyncLoadingProgress Progress)
	{
		return 1 << (uint32)Progress;
	}

	constexpr static inline uint32 BuildMask(auto... Types)
	{
		return (0 | ... | BuildMask(Types));
	}

	void Invoke(FLoadPackageAsyncProgressParams& Params)
	{
		if (Mask & BuildMask(Params.ProgressType))
		{
			Delegate.ExecuteIfBound(Params);
		}
	}

private:
	TTSDelegate<void(FLoadPackageAsyncProgressParams& Params)> Delegate;
	const uint32 Mask { MAX_uint32 };
};

/**
 * Optional parameters passed to the LoadPackageAsync function.
 */
struct FLoadPackageAsyncOptionalParams
{
	/** If not none, this is the name of the package to load into (and create if not yet existing). If none, the name is take from PackagePath. **/
	FName CustomPackageName { NAME_None };
	/** Non Thread-safe delegate to be invoked from game-thread on completion. **/
	TUniquePtr<FLoadPackageAsyncDelegate> CompletionDelegate;
	/** Thread-safe delegate to be invoked at different state of progress for the given package. **/
	TSharedPtr<FLoadPackageAsyncProgressDelegate> ProgressDelegate;
	/** Package flags used to construct loaded package in memory. **/
	EPackageFlags PackageFlags { PKG_None };
	/** Play in Editor instance ID. **/
	int32 PIEInstanceID { INDEX_NONE };
	/** Loading priority. **/
	int32 PackagePriority { 0 };
	/** Additional context to map object names to their instanced counterpart when loading an instanced package. **/
	const FLinkerInstancingContext* InstancingContext { nullptr };
	/** Flags controlling loading behavior, from the ELoadFlags enum. */
	uint32 LoadFlags { LOAD_None };
};

/**
 * Asynchronously load a package and all contained objects that match context flags. Non-blocking.
 * Use this version to specify the PackagePath rather than having the other versions internally convert the InName to a PackagePath by searching the current package mount points.
 * Use this version if you need to specify a packagename that is different from the packagename on disk; this is useful when loading multiple copies of the same package.
 *
  * @param	InPackagePath           PackagePath to load. Must be a mounted path. The package is created if it does not already exist.
  * @param	InOptionalParams        Optional parameters.
 * @return Unique ID associated with this load request (the same package can be associated with multiple IDs).
 */
COREUOBJECT_API int32 LoadPackageAsync(const FPackagePath& InPackagePath, FLoadPackageAsyncOptionalParams InOptionalParams);

/**
 * Asynchronously load a package and all contained objects that match context flags. Non-blocking.
 * Use this version to specify the PackagePath rather than having the other versions internally convert the InName to a PackagePath by searching the current package mount points.
 * Use this version if you need to specify a packagename that is different from the packagename on disk; this is useful when loading multiple copies of the same package.
 *
 * @param	InPackagePath			PackagePath to load. Must be a mounted path. The package is created if it does not already exist.
 * @param	InPackageNameToCreate	If not none, this is the name of the package to load the bytes on disk into (and create if not yet existing). If none, the name is taken from PackagePath.
 * @param	InCompletionDelegate	Delegate to be invoked when the packages has finished streaming
 * @param	InPackageFlags			Package flags used to construct loaded package in memory
 * @param	InPIEInstanceID			Play in Editor instance ID
 * @param	InPackagePriority		Loading priority
 * @param   InstancingContext		Additional context to map object names to their instanced counterpart when loading an instanced package
 * @param	LoadFlags				Flags controlling loading behavior, from the ELoadFlags enum
 * @return Unique ID associated with this load request (the same package can be associated with multiple IDs).
 */
COREUOBJECT_API int32 LoadPackageAsync(const FPackagePath& InPackagePath, FName InPackageNameToCreate = NAME_None, FLoadPackageAsyncDelegate InCompletionDelegate = FLoadPackageAsyncDelegate(), EPackageFlags InPackageFlags = PKG_None, int32 InPIEInstanceID = INDEX_NONE, TAsyncLoadPriority InPackagePriority = 0, const FLinkerInstancingContext* InstancingContext = nullptr, uint32 LoadFlags = LOAD_None);

/**
 * Asynchronously load a package and all contained objects that match context flags. Non-blocking.
 * Use this version for convenience when you just need to load a package without notification and with default behavior from a packagename/filename.
 *
 * @param	InName					PackageName or LocalFilePath of package to load. Must be a mounted name/path. The package is created if it does not already exist.
 * @param	InGuid					GUID of the package to load, or nullptr for "don't care"
 * @return Unique ID associated with this load request (the same package can be associated with multiple IDs).
 */
COREUOBJECT_API int32 LoadPackageAsync(const FString& InName, const FGuid* InGuid = nullptr);

/**
 * Asynchronously load a package and all contained objects that match context flags. Non-blocking.
 * Use this version when you need to load a package with default behavior from a packagename/filename, and need to be notified when it is loaded.
 *
 * @param	InName					PackageName or LocalFilePath of package to load. Must be a mounted name/path. The package is created if it does not already exist.
 * @param	InCompletionDelegate	Delegate to be invoked when the packages has finished streaming
 * @param	InPackagePriority		Loading priority
 * @param	InPackageFlags			Package flags used to construct loaded package in memory
 * @param	InPIEInstanceID			Play in Editor instance ID
 * @return Unique ID associated with this load request (the same package can be associated with multiple IDs).
 */
COREUOBJECT_API int32 LoadPackageAsync(const FString& InName, FLoadPackageAsyncDelegate InCompletionDelegate, TAsyncLoadPriority InPackagePriority = 0, EPackageFlags InPackageFlags = PKG_None, int32 InPIEInstanceID = INDEX_NONE);

/**
 * Asynchronously load a package and all contained objects that match context flags. Non-blocking.
 * Use this version when you need to load a package with default behavior from a packagename/filename, and need to be notified when it is loaded.
 *
 * @param	InName                  PackageName or LocalFilePath of package to load. Must be a mounted name/path. The package is created if it does not already exist.
 * @param	InOptionalParams        Optional parameters.
 * @return Unique ID associated with this load request (the same package can be associated with multiple IDs).
 */
COREUOBJECT_API int32 LoadPackageAsync(const FString& InName, FLoadPackageAsyncOptionalParams InOptionalParams);


/**
 * Delegate called on completion of async asset loading
 * @param	AssetPath			Path of the asset we were trying to load
 * @param	LoadedObject		Loaded object if successful, nullptr otherwise
 * @param	Result				Result of async loading.
 */
DECLARE_DELEGATE_ThreeParams(FLoadAssetAsyncDelegate, const FTopLevelAssetPath& /*AssetPath*/, UObject* /*LoadedObject*/, EAsyncLoadingResult::Type /*Result*/)

/**
 * Optional parameters passed to the LoadAssetAsync function.
 */
struct FLoadAssetAsyncOptionalParams
{
	/** Thread-safe delegate to be invoked at different state of progress for the given package. **/
	TSharedPtr<FLoadPackageAsyncProgressDelegate> ProgressDelegate;
	/** Loading priority. **/
	int32 PackagePriority { 0 };
	/** Additional context to map object names to their instanced counterpart when loading an instanced package. **/
	const FLinkerInstancingContext* InstancingContext { nullptr };
	/** Flags controlling loading behavior, from the ELoadFlags enum. */
	uint32 LoadFlags { LOAD_None };
};

/**
 * Asynchronously load a top level asset along with other objects in the same package. This is non-blocking and will call LoadPackageAsync.
 * FSoftObjectPath::LoadAsync can be used to asynchronously load subobjects.
 *
 * @param	InAssetPath				Top level asset to load
 * @param	InCompletionDelegate	Delegate to be invoked when the async load finishes, this will execute on the game thread as soon as the load succeeds or fails
 * @param	InOptionalParams		Optional parameters 
 * @return Unique ID associated with this load request (the same object or package can be associated with multiple IDs).
 */
COREUOBJECT_API int32 LoadAssetAsync(FTopLevelAssetPath InAssetPath, FLoadAssetAsyncDelegate InCompletionDelegate, FLoadAssetAsyncOptionalParams InOptionalParams = FLoadAssetAsyncOptionalParams());


/**
* Cancels all async package loading requests.
*/
UE_DEPRECATED(5.8, "This function is deprecated and will be removed. Do not use.")
COREUOBJECT_API void CancelAsyncLoading();

/**
* Returns true if the event driven loader is enabled in the current build
*/
UE_DEPRECATED(5.8, "This function is deprecated and will be removed. Use FPlatformProperties::RequiresCookedData() directly if appropriate.")
COREUOBJECT_API bool IsEventDrivenLoaderEnabled();

/**
 * Returns the async load percentage for a package in flight with the passed in name or -1 if there isn't one.
 * @warning THIS IS SLOW. MAY BLOCK ASYNC LOADING.
 *
 * @param	PackageName			Name of package to query load percentage for
 * @return	Async load percentage if package is currently being loaded, -1 otherwise
 */
COREUOBJECT_API float GetAsyncLoadPercentage( const FName& PackageName );

/** 
 * Deletes all unreferenced objects, keeping objects that have any of the passed in KeepFlags set. Will wait for other threads to unlock GC.
 *
 * @param	KeepFlags			objects with those flags will be kept regardless of being referenced or not
 * @param	bPerformFullPurge	if true, perform a full purge after the mark pass
 */
COREUOBJECT_API void CollectGarbage(EObjectFlags KeepFlags, bool bPerformFullPurge = true);

/**
* Performs garbage collection only if no other thread holds a lock on GC
*
* @param	KeepFlags			objects with those flags will be kept regardless of being referenced or not
* @param	bPerformFullPurge	if true, perform a full purge after the mark pass
*
* @return true if an actual attempt at garbage collection occurred
*/
COREUOBJECT_API bool TryCollectGarbage(EObjectFlags KeepFlags, bool bPerformFullPurge = true);

/**
* Calls ConditionalBeginDestroy on unreachable objects
*
* @param	bUseTimeLimit	whether the time limit parameter should be used
* @param	TimeLimit		soft time limit for this function call
*
* @return true if the time limit passed and there's still objects pending to be unhashed
*/
COREUOBJECT_API bool UnhashUnreachableObjects(bool bUseTimeLimit, double TimeLimit = 0.0);

/**
* Checks if there's objects pending to be unhashed when running incremental purge
*
* @return true if the time limit passed and there's still objects pending to be unhashed
*/
COREUOBJECT_API bool IsIncrementalUnhashPending();

/**
 * Returns whether an incremental purge is still pending/ in progress.
 *
 * @return	true if incremental purge needs to be kicked off or is currently in progress, false othwerise.
 */
COREUOBJECT_API bool IsIncrementalPurgePending();

/**
 * Gathers unreachable objects for IncrementalPurgeGarbage.
 *
 * @param bForceSingleThreaded true to force the process to just one thread
 */
AUTORTFM_DISABLE COREUOBJECT_API void GatherUnreachableObjects(bool bForceSingleThreaded);

/**
 * Returns whether an incremental reachability analysis is still pending/ in progress.
 *
 * @return	true if incremental reachability analysis needs to be kicked off or is currently in progress, false othwerise.
 */
COREUOBJECT_API bool IsIncrementalReachabilityAnalysisPending();

/**
 * Incrementally perform reachability analysis
 *
 * @param	TimeLimit	Time limit (in seconds) for this function call. 0.0 results in no time limit being used.
 */
COREUOBJECT_API void PerformIncrementalReachabilityAnalysis(double TimeLimit);

/**
 * Finalizes incremental reachability analysis (if currently running) without any time limit
 */
COREUOBJECT_API void FinalizeIncrementalReachabilityAnalysis();

/**
 * Incrementally purge garbage by deleting all unreferenced objects after routing Destroy.
 *
 * Calling code needs to be EXTREMELY careful when and how to call this function as 
 * RF_Unreachable cannot change on any objects unless any pending purge has completed!
 *
 * @param	bUseTimeLimit	whether the time limit parameter should be used
 * @param	TimeLimit		soft time limit for this function call
 */
COREUOBJECT_API void IncrementalPurgeGarbage( bool bUseTimeLimit, double TimeLimit = 0.002 );


enum class EUniqueObjectNameOptions
{
	None = 0,
	GloballyUnique		= 1 << 1,	// Whether to make the object name unique globally (across all objects that currently exist in the current session)
	UniversallyUnique	= 1 << 2	// Whether to make the object name unique universally (across all objects that currently exist in all sessions)
};
ENUM_CLASS_FLAGS(EUniqueObjectNameOptions);


#if WITH_EDITOR
/**
 * Renames a uniquely named objects obtained by MakeUniqueObjectName and 
 * attempts to lower the arbitrary per-class suffix used to make names unique if possible.
 * 
 * This is intended to allow code to recreate sequences of uniquely named objects with identical  
 * names.
 *
 * @param	Object		The objects to rename 
 */


COREUOBJECT_API void ReleaseUniquelyNamedObject(UObject* Object);
#endif

/**
 * Create a unique name by combining a base name and an arbitrary number string.
 * The object name returned is guaranteed not to exist.
 *
 * @param	Outer		the outer for the object that needs to be named
 * @param	Class		the class for the object
 * @param	BaseName	optional base name to use when generating the unique object name; if not specified, the class's name is used
 * @param	Options		Additional options. See EUniqueObjectNameOptions.
 * 
 * @return	name is the form BaseName_##, where ## is the number of objects of this
 *			type that have been created since the last time the class was garbage collected.
 */
COREUOBJECT_API FName MakeUniqueObjectName( UObject* Outer, const UClass* Class, FName BaseName = NAME_None, EUniqueObjectNameOptions Options = EUniqueObjectNameOptions::None);

/**
 * Generator class that can be used as a session to generate universally unique object names.
 */
struct AUTORTFM_OPEN FUniversallyUniqueObjectNameGenerator
{
	COREUOBJECT_API FUniversallyUniqueObjectNameGenerator(const FGuid& Seed);
	COREUOBJECT_API uint32 GetNewValue() const;

	TCHAR SessionIdBase36[14];
	mutable std::atomic<uint32> Seed;
};

/**
 * Create a universally unique name by combining a base name, a GUID and a timestamp.
 * The object name returned is guaranteed not to locally and globally exist.
 *
 * @param	Outer		the outer for the object that needs to be named
 * @param	BaseName	optional base name to use when generating the globally unique object name.
 * @param	Generator	optional generator session that can be passed to not use instead of the global one.
 * 
 * @return	name is the form BaseName_##, where ## a universally unique identifier making sure that name is universally unique.
 */
COREUOBJECT_API FName MakeUniversallyUniqueObjectName(UObject* Outer, FName InBaseName = NAME_None, FUniversallyUniqueObjectNameGenerator const* Generator = nullptr);

/**
 * Checks if the given FName is marked as a Universally Unique Object Name.
 * Note: This is an early out check. This function may return true for FNames that are not Universally Unique Object Names.
 *
 * @param	Name	The name to check.
 * @return	True if the FName is marked as a Universally Unique Object Name, otherwise false.
  */
COREUOBJECT_API bool HasUniversallyUniqueObjectNameIdentifier(FName Name);

/**
 * Given a display label string, generates an FName slug that is a valid FName for that label.
 * If the object's current name is already satisfactory, then that name will be returned.
 * For example, "[MyObject]: Object Label" becomes "MyObjectObjectLabel" FName slug.
 * 
 * Note: The generated name isn't guaranteed to be unique.
 *
 * @param DisplayLabel The label string to convert to an FName
 * @param CurrentObjectName The object's current name, or NAME_None if it has no name yet
 *
 * @return	The generated object name
 */
COREUOBJECT_API FName MakeObjectNameFromDisplayLabel(const FString& DisplayLabel, const FName CurrentObjectName);

/**
 * Returns whether an object is referenced, not counting references from itself
 *
 * @param	Res			Object to check
 * @param	KeepFlags	Objects with these flags will be considered as being referenced
 * @param	InternalKeepFlags	Objects with these internal flags will be considered as being referenced
 * @param	bCheckSubObjects	Treat subobjects as if they are the same as passed in object
 * @param	FoundReferences		If non-nullptr fill in with list of objects that hold references
 * @return true if object is referenced, false otherwise
 */
COREUOBJECT_API bool IsReferenced( UObject*& Res, EObjectFlags KeepFlags, EInternalObjectFlags InternalKeepFlags, bool bCheckSubObjects = false, FReferencerInformationList* FoundReferences = nullptr );

/**
 * Blocks till all pending package/ linker requests are fulfilled.
 *
 * @param PackageID if the package associated with this request ID gets loaded, FlushAsyncLoading returns 
 *        immediately without waiting for the remaining packages to finish loading.
 */
COREUOBJECT_API void FlushAsyncLoading(int32 PackageID = INDEX_NONE);

/**
 * Blocks till a set of pending async load requests are complete.
 *
 * @param RequestIds list of return values from LoadPackageAsync to wait for. An empty list means all requests
 */
COREUOBJECT_API void FlushAsyncLoading(TConstArrayView<int32> RequestIds);

/**
 * Return number of active async load package requests
 */
COREUOBJECT_API int32 GetNumAsyncPackages();

/**
 * Returns whether we are currently loading a package (sync or async)
 *
 * @return true if we are loading a package, false otherwise
 */
COREUOBJECT_API bool IsLoading();

/**
 * Allows or disallows async loading (for example async loading is not allowed after the final flush on exit)
 *
 * @param bAllowAsyncLoading true if async loading should be allowed, false otherwise
 */
COREUOBJECT_API void SetAsyncLoadingAllowed(bool bAllowAsyncLoading);

/**
 * State of the async package after the last tick.
 */
namespace EAsyncPackageState
{
	enum Type
	{
		/** Package tick has timed out. */
		TimeOut = 0,
		/** Package has pending import packages that need to be streamed in. */
		PendingImports,
		/** Package has finished loading. */
		Complete,
	};
}

/**
 * Serializes a bit of data each frame with a soft time limit. The function is designed to be able
 * to fully load a package in a single pass given sufficient time.
 *
 * @param	bUseTimeLimit	Whether to use a time limit
 * @param	bUseFullTimeLimit	If true, use the entire time limit even if blocked on I/O
 * @param	TimeLimit		Soft limit of time this function is allowed to consume
 * @return The minimum state of any of the queued packages.
 */
COREUOBJECT_API EAsyncPackageState::Type ProcessAsyncLoading( bool bUseTimeLimit, bool bUseFullTimeLimit, double TimeLimit);

/**
 * Blocks and runs ProcessAsyncLoading until the time limit is hit, the completion predicate returns true, or all async loading is done
 * 
 * @param	CompletionPredicate	If this returns true, stop loading. This is called periodically as long as loading continues
 * @param	TimeLimit			Hard time limit. 0 means infinite length
 * @return The minimum state of any of the queued packages.
 */
COREUOBJECT_API EAsyncPackageState::Type ProcessAsyncLoadingUntilComplete(TFunctionRef<bool()> CompletionPredicate, double TimeLimit);

/** UObjects are being loaded between these calls */
COREUOBJECT_API void BeginLoad(FUObjectSerializeContext* LoadContext, const TCHAR* DebugContext = nullptr);
COREUOBJECT_API void EndLoad(FUObjectSerializeContext* LoadContext);

/**
 * Find an existing package by name
 * @param InOuter		The Outer object to search inside
 * @param PackageName	The name of the package to find
 *
 * @return The package if it exists
 */
COREUOBJECT_API UPackage* FindPackage(UObject* InOuter, const TCHAR* PackageName);

#if WITH_EDITOR
/**
* Sets default PackageFlags for new packages made in specified mount points during CreatePackage
* @param InMountPointToDefaultPackageFlags A map of MountPoints to PackageFlags. Used to provide new Packages in each mount point with the associated DefaultFlags
*/
COREUOBJECT_API void SetMountPointDefaultPackageFlags(const TMap<FString, EPackageFlags>& InMountPointToDefaultPackageFlags);

/**
* Removes the provided list of mount points from the MountPointToDefaultPackageFlags map
* @param InMountPoints A list of mount points that will be removed and no longer have DefaultPackageFlags associated with them
*/
COREUOBJECT_API void RemoveMountPointDefaultPackageFlags(const TArrayView<FString> InMountPoints);
#endif

/**
 * Find an existing package by name or create it if it doesn't exist
 * @return The existing package or a newly created one
 *
 */
COREUOBJECT_API UPackage* CreatePackage(const TCHAR* PackageName);

/** Internal function used to set a specific property value from debug/console code */
void GlobalSetProperty( const TCHAR* Value, UClass* Class, FProperty* Property, bool bNotifyObjectOfChange );

/**
 * Save a copy of this object into the transaction buffer if we are currently recording into
 * one (undo/redo). If bMarkDirty is true, will also mark the package as needing to be saved.
 *
 * @param	bMarkDirty	If true, marks the package dirty if we are currently recording into a
 *						transaction buffer
 * @param	Object		object to save.
 *
 * @return	true if a copy of the object was saved and the package potentially marked dirty; false
 *			if we are not recording into a transaction buffer, the package is a PIE/script package,
 *			or the object is not transactional (implies the package was not marked dirty)
 */
COREUOBJECT_API bool SaveToTransactionBuffer(UObject* Object, bool bMarkDirty);

/**
 * Causes the transaction system to emit a snapshot event for the given object if the following conditions are met:
 *  a) The object is currently transacting.
 *  b) The object has changed since it started transacting.
 *
 * @param	Object		object to snapshot.
 * @param	Properties	optional list of properties that have potentially changed on the object (to avoid snapshotting the entire object).
 */
COREUOBJECT_API void SnapshotTransactionBuffer(UObject* Object);
COREUOBJECT_API void SnapshotTransactionBuffer(UObject* Object, TArrayView<const FProperty*> Properties);


/**
 * Utility struct that allows abstract classes to be allocated for non-CDOs while in scope.
 * Abstract objects are generally unsafe and should only be allocated in very unusual circumstances.
 */
struct FScopedAllowAbstractClassAllocation : public FNoncopyable
{
	COREUOBJECT_API FScopedAllowAbstractClassAllocation();
	COREUOBJECT_API ~FScopedAllowAbstractClassAllocation();
	static bool IsDisallowedAbstractClass(const UClass* InClass, EObjectFlags InFlags);

private:
	static int32 AllowAbstractCount;
};

/**
 * Check for StaticAllocateObject error; only for use with the editor, make or other commandlets.
 * 
 * @param	Class		the class of the object to create
 * @param	InOuter		the object to create this object within (the Outer property for the new object will be set to the value specified here).
 * @param	Name		the name to give the new object. If no value (NAME_None) is specified, the object will be given a unique name in the form of ClassName_#.
 * @param	SetFlags	the ObjectFlags to assign to the new object. some flags can affect the behavior of constructing the object.
 * @return	true if nullptr should be returned; there was a problem reported 
 */
bool StaticAllocateObjectErrorTests( const UClass* Class, UObject* InOuter, FName Name, EObjectFlags SetFlags);

/**
 * Create a new instance of an object or replace an existing object.  If both an Outer and Name are specified, and there is an object already in memory with the same Class, Outer, and Name, the
 * existing object will be destructed, and the new object will be created in its place.
 * 
 * @param	Class		the class of the object to create
 * @param	InOuter		the object to create this object within (the Outer property for the new object will be set to the value specified here).
 * @param	Name		the name to give the new object. If no value (NAME_None) is specified, the object will be given a unique name in the form of ClassName_#.
 * @param	SetFlags	the ObjectFlags to assign to the new object. some flags can affect the behavior of constructing the object.
 * @param InternalSetFlags	the InternalObjectFlags to assign to the new object. some flags can affect the behavior of constructing the object.
 * @param bCanReuseSubobjects	if set to true, SAO will not attempt to destroy a subobject if it already exists in memory.
 * @param bOutReusedSubobject	flag indicating if the object is a subobject that has already been created (in which case further initialization is not necessary).
 * @param ExternalPackage	External Package assigned to the allocated object, if any	
 * @param RemoteId		Globally unique id for this object
 * @param GCGuard		Special synchronization object that prevents GC from running when reconstructing an object on top of existing one
 * @return	a pointer to a fully initialized object of the specified class.
 */
COREUOBJECT_API UObject* StaticAllocateObject(const UClass* Class, UObject* InOuter, FName Name, EObjectFlags SetFlags, EInternalObjectFlags InternalSetFlags = EInternalObjectFlags::None, bool bCanReuseSubobjects = false, bool* bOutReusedSubobject = nullptr, UPackage* ExternalPackage = nullptr, int32 SerialNumber = 0, FRemoteObjectId RemoteId = FRemoteObjectId(), class FGCReconstructionGuard* GCGuard = nullptr);

/** FObjectInitializer options */
enum class EObjectInitializerOptions
{
	None							= 0,
	CopyTransientsFromClassDefaults = 1 << 0, // copy transient from the class defaults instead of the pass in archetype ptr
	InitializeProperties			= 1 << 1, // initialize property values with the archetype values
};
ENUM_CLASS_FLAGS(EObjectInitializerOptions)

class FGCObject;

/**
 * Internal class to finalize UObject creation (initialize properties) after the real C++ constructor is called.
 **/
class FObjectInitializer
{
public:
	/**
	 * Default Constructor, used when you are using the C++ "new" syntax. UObject::UObject will set the object pointer
	 **/
	COREUOBJECT_API FObjectInitializer();

	/**
	 * Constructor
	 * @param	InObj object to initialize, from static allocate object, after construction
	 * @param	InObjectArchetype object to initialize properties from
	 * @param	InOptions initialization options, see EObjectInitializerOptions
	 * @param	InInstanceGraph passed instance graph
	 */
	COREUOBJECT_API FObjectInitializer(UObject* InObj, UObject* InObjectArchetype, EObjectInitializerOptions InOptions, struct FObjectInstancingGraph* InInstanceGraph = nullptr);

	/** Special constructor for static construct object internal that passes along the params block directly */
	COREUOBJECT_API FObjectInitializer(UObject* InObj, const FStaticConstructObjectParameters& StaticConstructParams);

	UE_DEPRECATED(5.7, "FObjectInitializer constructor that takes remote object overrides is deprecated. Use FRemoteObjectConstructionOverridesScope instead.")
	FObjectInitializer(UObject* InObj, const FStaticConstructObjectParameters& StaticConstructParams, UE::RemoteObject::Serialization::FRemoteObjectConstructionOverrides* RemoteObjectOverides)
		: FObjectInitializer(InObj, StaticConstructParams)
	{
	}

private:
	/** Helper for the common behaviors in the constructors */
	COREUOBJECT_API void Construct_Internal();

public:
	COREUOBJECT_API ~FObjectInitializer();

	/** 
	 * Return the archetype that this object will copy properties from later
	**/
	UE_FORCEINLINE_HINT UObject* GetArchetype() const
	{
		return ObjectArchetype;
	}

	/**
	* Return the object that is being constructed
	**/
	UE_FORCEINLINE_HINT UObject* GetObj() const
	{
		return Obj;
	}

	UE_FORCEINLINE_HINT struct FObjectInstancingGraph* GetInstancingGraph()
	{
		return InstanceGraph;
	}

	UE_FORCEINLINE_HINT void AddPropertyPostInitCallback(TFunction<void()>&& Callback)
	{
		PropertyPostInitCallbacks.Add(MoveTemp(Callback));
	}

	/**
	* Return the class of the object that is being constructed
	**/
	COREUOBJECT_API UClass* GetClass() const;

	/**
	 * Create a component or subobject that will be instanced inside all instances of this class.
	 * @param	TReturnType					class of return type, all overrides must be of this type
	 * @param	Outer						outer to construct the subobject in
	 * @param	SubobjectName				name of the new component, this will be the same for all instances of this class
	 * @param	bTransient					true if the component is being assigned to a transient property
	 */
	template<class TReturnType>
	TReturnType* CreateDefaultSubobject(UObject* Outer, FName SubobjectName, bool bTransient = false) const
	{
		UClass* ReturnType = TReturnType::StaticClass();
		return static_cast<TReturnType*>(CreateDefaultSubobject(Outer, SubobjectName, ReturnType, ReturnType, /*bIsRequired =*/ true, bTransient));
	}

	/**
	 * Create optional component or subobject. Optional subobjects will not get created.
	 * if a derived class specifies DoNotCreateDefaultSubobject with the subobject name.
	 * @param	TReturnType					class of return type, all overrides must be of this type
	 * @param	Outer						outer to construct the subobject in
	 * @param	SubobjectName				name of the new component, this will be the same for all instances of this class
	 * @param	bTransient					true if the component is being assigned to a transient property
	 */
	template<class TReturnType>
	TReturnType* CreateOptionalDefaultSubobject(UObject* Outer, FName SubobjectName, bool bTransient = false) const
	{
		UClass* ReturnType = TReturnType::StaticClass();
		return static_cast<TReturnType*>(CreateDefaultSubobject(Outer, SubobjectName, ReturnType, ReturnType, /*bIsRequired =*/ false, bTransient));
	}

	/** 
	 * Create a component or subobject, allows creating a child class and returning the parent class.
	 * @param	TReturnType					class of return type, all overrides must be of this type 
	 * @param	TClassToConstructByDefault	class to construct by default
	 * @param	Outer						outer to construct the subobject in
	 * @param	SubobjectName				name of the new component, this will be the same for all instances of this class
	 * @param	bTransient					true if the component is being assigned to a transient property
	 */ 
	template<class TReturnType, class TClassToConstructByDefault> 
	TReturnType* CreateDefaultSubobject(UObject* Outer, FName SubobjectName, bool bTransient = false) const 
	{ 
		return static_cast<TReturnType*>(CreateDefaultSubobject(Outer, SubobjectName, TReturnType::StaticClass(), TClassToConstructByDefault::StaticClass(), /*bIsRequired =*/ true, bTransient));
	}

	/**
	 * Create a component or subobject only to be used with the editor.
	 * @param	TReturnType					class of return type, all overrides must be of this type
	 * @param	Outer						outer to construct the subobject in
	 * @param	SubobjectName				name of the new component, this will be the same for all instances of this class
	 * @param	bTransient					true if the component is being assigned to a transient property
	 */
	template<class TReturnType>
	TReturnType* CreateEditorOnlyDefaultSubobject(UObject* Outer, FName SubobjectName, bool bTransient = false) const
	{
		const UClass* ReturnType = TReturnType::StaticClass();
		return static_cast<TReturnType*>(CreateEditorOnlyDefaultSubobject(Outer, SubobjectName, ReturnType, bTransient));
	}

	/**
	* Create a component or subobject only to be used with the editor.
	* @param	Outer						outer to construct the subobject in
	* @param	SubobjectName				name of the new component, this will be the same for all instances of this class
	* @param	ReturnType					type of the new component
	* @param	bTransient					true if the component is being assigned to a transient property
	*/
	COREUOBJECT_API UObject* CreateEditorOnlyDefaultSubobject(UObject* Outer, FName SubobjectName, const UClass* ReturnType, bool bTransient = false) const;

	/**
	 * Create a component or subobject that will be instanced inside all instances of this class.
	 * @param	Outer                       outer to construct the subobject in
	 * @param	SubobjectName               name of the new component
	 * @param	ReturnType                  class of return type, all overrides must be of this type
	 * @param	ClassToConstructByDefault   if the derived class has not overridden, create a component of this type
	 * @param	bIsRequired                 true if the component is required and will always be created even if DoNotCreateDefaultSubobject was specified.
	 * @param	bIsTransient                true if the component is being assigned to a transient property
	 */
	COREUOBJECT_API UObject* CreateDefaultSubobject(UObject* Outer, FName SubobjectFName, const UClass* ReturnType, const UClass* ClassToCreateByDefault, bool bIsRequired = true, bool bIsTransient = false) const;

	/**
	 * Sets the class to use for a subobject defined in a base class, the class must be a subclass of the class used by the base class.
	 * @param	SubobjectName	name of the new component or subobject
	 * @param	Class			The class to use for the specified subobject or component.
	 */
	const FObjectInitializer& SetDefaultSubobjectClass(FName SubobjectName, const UClass* Class) const
	{
		AssertIfSubobjectSetupIsNotAllowed(SubobjectName);
		SubobjectOverrides.Add(SubobjectName, Class);
		return *this;
	}

	/**
	 * Sets the class to use for a subobject defined in a base class, the class must be a subclass of the class used by the base class.
	 * @param	SubobjectName	name of the new component or subobject
	 */
	template<class T>
	const FObjectInitializer& SetDefaultSubobjectClass(FName SubobjectName) const
	{
		return SetDefaultSubobjectClass(SubobjectName, T::StaticClass());
	}

	/**
	 * Indicates that a base class should not create a component
	 * @param	SubobjectName	name of the new component or subobject to not create
	 */
	const FObjectInitializer& DoNotCreateDefaultSubobject(FName SubobjectName) const
	{
		AssertIfSubobjectSetupIsNotAllowed(SubobjectName);
		SubobjectOverrides.Add(SubobjectName, nullptr);
		return *this;
	}

	/**
	 * Sets the class to use for a subobject defined in a nested subobject, the class must be a subclass of the class used when calling CreateDefaultSubobject.
	 * @param	SubobjectName	path to the new component or subobject
	 * @param	Class			The class to use for the specified subobject or component.
	 */
	const FObjectInitializer& SetNestedDefaultSubobjectClass(FStringView SubobjectName, const UClass* Class) const
	{
		AssertIfSubobjectSetupIsNotAllowed(SubobjectName);
		SubobjectOverrides.Add(SubobjectName, Class);
		return *this;
	}

	/**
	 * Sets the class to use for a subobject defined in a nested subobject, the class must be a subclass of the class used when calling CreateDefaultSubobject.
	 * @param	SubobjectName	path to the new component or subobject
	 * @param	Class			The class to use for the specified subobject or component.
	 */
	const FObjectInitializer& SetNestedDefaultSubobjectClass(TArrayView<const FName> SubobjectNames, const UClass* Class) const
	{
		AssertIfSubobjectSetupIsNotAllowed(SubobjectNames);
		SubobjectOverrides.Add(SubobjectNames, Class);
		return *this;
	}

	/**
	 * Sets the class to use for a subobject defined in a nested subobject, the class must be a subclass of the class used when calling CreateDefaultSubobject.
	 * @param	SubobjectName	path to the new component or subobject
	 */
	template<class T>
	const FObjectInitializer& SetNestedDefaultSubobjectClass(FStringView SubobjectName) const
	{
		return SetNestedDefaultSubobjectClass(SubobjectName, T::StaticClass());
	}

	/**
	 * Sets the class to use for a subobject defined in a nested subobject, the class must be a subclass of the class used when calling CreateDefaultSubobject.
	 * @param	SubobjectName	path to the new component or subobject
	 */
	template<class T>
	const FObjectInitializer& SetNestedDefaultSubobjectClass(TArrayView<const FName> SubobjectNames) const
	{
		return SetNestedDefaultSubobjectClass(SubobjectNames, T::StaticClass());
	}

	/**
	 * Indicates that a subobject should not create a component if created using CreateOptionalDefaultSubobject
	 * @param	SubobjectName	name of the new component or subobject to not create
	 */
	const FObjectInitializer& DoNotCreateNestedDefaultSubobject(FStringView SubobjectName) const
	{
		AssertIfSubobjectSetupIsNotAllowed(SubobjectName);
		SubobjectOverrides.Add(SubobjectName, nullptr);
		return *this;
	}

	/**
	 * Indicates that a subobject should not create a component if created using CreateOptionalDefaultSubobject
	 * @param	SubobjectName	name of the new component or subobject to not create
	 */
	const FObjectInitializer& DoNotCreateNestedDefaultSubobject(TArrayView<const FName> SubobjectNames) const
	{
		AssertIfSubobjectSetupIsNotAllowed(SubobjectNames);
		SubobjectOverrides.Add(SubobjectNames, nullptr);
		return *this;
	}

	/**
	 * Adds a member variable property that should not be instanced for this object. Not intended for use outside of engine code.
	 */
	UE_INTERNAL void AddPropertyToSubobjectExclusionList(const FProperty* Property)
	{
		SubobjectInstantiationExclusionList.Add(Property);
	}

	/**
	 * Checks if a member variable property should not be instanced for this object. Not intended for use outside of engine code.
	 */
	UE_INTERNAL bool IsPropertyInSubobjectExclusionList(const FProperty* Property) const
	{
		return SubobjectInstantiationExclusionList.Contains(Property);
	}

	/**
	 * Asserts with the specified message if code is executed inside UObject constructor
	 **/
	static COREUOBJECT_API void AssertIfInConstructor(UObject* Outer, const TCHAR* ErrorMessage);

	UE_FORCEINLINE_HINT void FinalizeSubobjectClassInitialization()
	{
		bSubobjectClassInitializationAllowed = false;
	}

	/** Gets ObjectInitializer for the currently constructed object. Can only be used inside of a constructor of UObject-derived class. */
	static COREUOBJECT_API FObjectInitializer& Get();

private:

	friend class UObject; 
	friend class FScriptIntegrationObjectHelper;

	template<class T>
	friend void InternalConstructor(const class FObjectInitializer& X);

	/**
	 * Binary initialize object properties to zero or defaults.
	 *
	 * @param	Obj					object to initialize data for
	 * @param	DefaultsClass		the class to use for initializing the data
	 * @param	DefaultData			the buffer containing the source data for the initialization
	 * @param	bCopyTransientsFromClassDefaults if true, copy the transients from the DefaultsClass defaults, otherwise copy the transients from DefaultData
	 */
	static COREUOBJECT_API void InitProperties(UObject* Obj, UClass* DefaultsClass, UObject* DefaultData, bool bCopyTransientsFromClassDefaults);

	COREUOBJECT_API bool IsInstancingAllowed() const;

	/**
	 * Calls InitProperties for any default subobjects created through this ObjectInitializer.
	 * @param bAllowInstancing	Indicates whether the object's components may be copied from their templates.
	 * @return true if there are any subobjects which require instancing.
	*/
	COREUOBJECT_API bool InitSubobjectProperties(bool bAllowInstancing) const;

	/**
	 * Create copies of the object's components from their templates.
	 * @param Class						Class of the object we are initializing
	 * @param bNeedInstancing			Indicates whether the object's components need to be instanced
	 * @param bNeedSubobjectInstancing	Indicates whether subobjects of the object's components need to be instanced
	 */
	COREUOBJECT_API void InstanceSubobjects(UClass* Class, bool bNeedInstancing, bool bNeedSubobjectInstancing) const;

	/** 
	 * Initializes a non-native property, according to the initialization rules. If the property is non-native
	 * and does not have a zero contructor, it is inialized with the default value.
	 * @param	Property			Property to be initialized
	 * @param	Data				Default data
	 * @return	Returns true if that property was a non-native one, otherwise false
	 */
	static COREUOBJECT_API bool InitNonNativeProperty(FProperty* Property, UObject* Data);
	
	/**
	 * Finalizes a constructed UObject by initializing properties, 
	 * instancing/initializing sub-objects, etc.
	 */
	COREUOBJECT_API void PostConstructInit();

private:

	/**  Little helper struct to manage overrides from derived classes **/
	struct FOverrides
	{
		/**  Add an override, make sure it is legal **/
		COREUOBJECT_API void Add(FName InComponentName, const UClass* InComponentClass, const TArrayView<const FName>* FullPath = nullptr);

		/**  Add a potentially nested override, make sure it is legal **/
		COREUOBJECT_API void Add(FStringView InComponentPath, const UClass* InComponentClass);

		/**  Add a potentially nested override, make sure it is legal **/
		COREUOBJECT_API void Add(TArrayView<const FName> InComponentPath, const UClass* InComponentClass, const TArrayView<const FName>* FullPath = nullptr);

		struct FOverrideDetails
		{
			const UClass* Class = nullptr;
			FOverrides* SubOverrides = nullptr;
		};

		/**  Retrieve an override, or TClassToConstructByDefault::StaticClass or nullptr if this was removed by a derived class **/
		FOverrideDetails Get(FName InComponentName, const UClass* ReturnType, const UClass* ClassToConstructByDefault, bool bOptional) const;

	private:
		static bool IsLegalOverride(const UClass* DerivedComponentClass, const UClass* BaseComponentClass);

		/**  Search for an override **/
		int32 Find(FName InComponentName) const
		{
			for (int32 Index = 0 ; Index < Overrides.Num(); Index++)
			{
				if (Overrides[Index].ComponentName == InComponentName)
				{
					return Index;
				}
			}
			return INDEX_NONE;
		}
		/**  Element of the override array **/
		struct FOverride
		{
			FName ComponentName;
			const UClass* ComponentClass = nullptr;
			TUniquePtr<FOverrides> SubOverrides;
			bool bDoNotCreate = false;
			FOverride(FName InComponentName)
				: ComponentName(InComponentName)
			{
			}

			FOverride& operator=(const FOverride& Other)
			{
				ComponentName = Other.ComponentName;
				ComponentClass = Other.ComponentClass;
				SubOverrides = (Other.SubOverrides ? MakeUnique<FOverrides>(*Other.SubOverrides) : nullptr);
				bDoNotCreate = Other.bDoNotCreate;
				return *this;
			}

			FOverride(const FOverride& Other)
			{
				*this = Other;
			}

			FOverride(FOverride&&) = default;
			FOverride& operator=(FOverride&&) = default;
		};
		/**  The override array **/
		TArray<FOverride, TInlineAllocator<8> > Overrides;
	};
	/**  Little helper struct to manage overrides from derived classes **/
	struct FSubobjectsToInit
	{
		/**  Add a subobject **/
		void Add(UObject* Subobject, UObject* Template)
		{
			for (int32 Index = 0; Index < SubobjectInits.Num(); Index++)
			{
				check(SubobjectInits[Index].Subobject != Subobject);
			}
			SubobjectInits.Emplace(Subobject, Template);
		}
		/**  Element of the SubobjectInits array **/
		struct FSubobjectInit
		{
			UObject* Subobject;
			UObject* Template;
			FSubobjectInit(UObject* InSubobject, UObject* InTemplate)
				: Subobject(InSubobject)
				, Template(InTemplate)
			{
			}
		};
		/**  The SubobjectInits array **/
		TArray<FSubobjectInit, TInlineAllocator<8> > SubobjectInits;
	};

	/** Asserts if SetDefaultSubobjectClass or DoNotCreateOptionalDefaultSuobject are called inside of the constructor body */
	COREUOBJECT_API void AssertIfSubobjectSetupIsNotAllowed(const FName SubobjectName) const;

	/** Asserts if SetDefaultSubobjectClass or DoNotCreateOptionalDefaultSuobject are called inside of the constructor body */
	COREUOBJECT_API void AssertIfSubobjectSetupIsNotAllowed(FStringView SubobjectName) const;

	/** Asserts if SetDefaultSubobjectClass or DoNotCreateOptionalDefaultSuobject are called inside of the constructor body */
	COREUOBJECT_API void AssertIfSubobjectSetupIsNotAllowed(TArrayView<const FName> SubobjectNames) const;

	/**  object to initialize, from static allocate object, after construction **/
	UObject* Obj;
	/**  object to copy properties from **/
	UObject* ObjectArchetype;
	/**  if true, copy the transients from the DefaultsClass defaults, otherwise copy the transients from DefaultData **/
	bool bCopyTransientsFromClassDefaults;
	/**  If true, initialize the properties **/
	bool bShouldInitializePropsFromArchetype;
	/**  Only true until ObjectInitializer has not reached the base UObject class */
	bool bSubobjectClassInitializationAllowed = true;
#if USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	/**  */
	bool bIsDeferredInitializer = false;
#endif // USE_CIRCULAR_DEPENDENCY_LOAD_DEFERRING
	/**  Instance graph **/
	struct FObjectInstancingGraph* InstanceGraph;
	/**  List of component classes to override from derived classes **/
	mutable FOverrides SubobjectOverrides;
	/**  List of member variable properties that should not instantiate subobjects */
	TSet<const FProperty*> SubobjectInstantiationExclusionList;
	/**  List of component classes to initialize after the C++ constructors **/
	mutable FSubobjectsToInit ComponentInits;
#if !UE_BUILD_SHIPPING
	/** List of all subobject names constructed for this object */
	mutable TArray<FName, TInlineAllocator<8>> ConstructedSubobjects;
#endif
	/**  Previously constructed object in the callstack */
	UObject* LastConstructedObject = nullptr;

	/** Callbacks for custom property initialization before PostInitProperties gets called */
	TFunction<void()> PropertyInitCallback;
	TArray<TFunction<void()>> PropertyPostInitCallbacks;

	friend struct FStaticConstructObjectParameters;

#if WITH_EDITORONLY_DATA
	/** Detects when a new GC object was created */
	COREUOBJECT_API void OnGCObjectCreated(FGCObject* InObject);

	/** Delegate handle for OnGCObjectCreated callback */
	FDelegateHandle OnGCObjectCreatedHandle;
	/** List of FGCObjects created during UObject construction */
	TArray<FGCObject*> CreatedGCObjects;
#endif // WITH_EDITORONLY_DATA

#if UE_WITH_REMOTE_OBJECT_HANDLE
public:
	UE_DEPRECATED(5.7, "Use FRemoteObjectConstructionOverridesScope instead.")
	UE::RemoteObject::Serialization::FRemoteObjectConstructionOverrides* GetRemoteSubObjectOverrides() const
	{
		return nullptr;
	}
#endif // UE_WITH_REMOTE_OBJECT_HANDLE
};

/**
 * This struct is used for passing parameter values to the StaticConstructObject_Internal() method.  Only the constructor parameters are required to
 * be valid - all other members are optional.
 */
struct FStaticConstructObjectParameters
{
	/** The class of the object to create */
	const UClass* Class;

	/** The object to create this object within (the Outer property for the new object will be set to the value specified here). */
	UObject* Outer;

	/** The name to give the new object.If no value(NAME_None) is specified, the object will be given a unique name in the form of ClassName_#. */
	FName Name;

	/** The ObjectFlags to assign to the new object. some flags can affect the behavior of constructing the object. */
	EObjectFlags SetFlags = RF_NoFlags;

	/** The InternalObjectFlags to assign to the new object. some flags can affect the behavior of constructing the object. */
	EInternalObjectFlags InternalSetFlags = EInternalObjectFlags::None;

	/** If true, copy transient from the class defaults instead of the pass in archetype ptr(often these are the same) */
	bool bCopyTransientsFromClassDefaults = false;

	/** If true, Template is guaranteed to be an archetype */
	bool bAssumeTemplateIsArchetype = false;

	/**
	 * If specified, the property values from this object will be copied to the new object, and the new object's ObjectArchetype value will be set to this object.
	 * If nullptr, the class default object is used instead.
	 */
	UObject* Template = nullptr;

	/** Contains the mappings of instanced objects and components to their templates */
	FObjectInstancingGraph* InstanceGraph = nullptr;

	/** Assign an external Package to the created object if non-null */
	UPackage* ExternalPackage = nullptr;

	/** Callback for custom code to initialize properties before PostInitProperties runs */
	TFunction<void()> PropertyInitCallback;

	/** Existing serial number for the object */
	int32 SerialNumber = 0;

#if UE_WITH_REMOTE_OBJECT_HANDLE
	/** Existing remote id for the object */
	FRemoteObjectId RemoteId;

	UE_DEPRECATED(5.7, "Use FRemoteObjectConstructionOverridesScope instead.")
	UE::RemoteObject::Serialization::FRemoteObjectConstructionOverrides* RemoteSubObjectOverrides = nullptr;
#endif // UE_WITH_REMOTE_OBJECT_HANDLE

private:
	FObjectInitializer::FOverrides* SubobjectOverrides = nullptr;

public:

	COREUOBJECT_API FStaticConstructObjectParameters(const UClass* InClass);

	friend FObjectInitializer;
};

/**
* Helper class for script integrations to access some UObject innards. Needed for script-generated UObject classes
*/
class FScriptIntegrationObjectHelper
{
public:
	/**
	* Binary initialize object properties to zero or defaults.
	*
	* @param	ObjectInitializer	FObjectInitializer helper
	* @param	Obj					object to initialize data for
	* @param	DefaultsClass		the class to use for initializing the data
	* @param	DefaultData			the buffer containing the source data for the initialization
	*/
	inline static void InitProperties(const FObjectInitializer& ObjectInitializer, UObject* Obj, UClass* DefaultsClass, UObject* DefaultData)
	{
		FObjectInitializer::InitProperties(Obj, DefaultsClass, DefaultData, ObjectInitializer.bCopyTransientsFromClassDefaults);
	}

	/**
	* Calls InitProperties for any default subobjects created through this ObjectInitializer.
	* @param bAllowInstancing	Indicates whether the object's components may be copied from their templates.
	* @return true if there are any subobjects which require instancing.
	*/
	inline static bool InitSubobjectProperties(const FObjectInitializer& ObjectInitializer)
	{
		return ObjectInitializer.InitSubobjectProperties(ObjectInitializer.IsInstancingAllowed());
	}

	/**
	* Create copies of the object's components from their templates.
	* @param ObjectInitializer			FObjectInitializer helper
	* @param Class						Class of the object we are initializing
	* @param bNeedInstancing			Indicates whether the object's components need to be instanced
	* @param bNeedSubobjectInstancing	Indicates whether subobjects of the object's components need to be instanced
	*/
	inline static void InstanceSubobjects(const FObjectInitializer& ObjectInitializer, UClass* Class, bool bNeedInstancing, bool bNeedSubobjectInstancing)
	{
		ObjectInitializer.InstanceSubobjects(Class, bNeedInstancing, bNeedSubobjectInstancing);
	}

	/**
	 * Finalizes a constructed UObject by initializing properties, instancing &
	 * initializing sub-objects, etc.
	 * 
	 * @param  ObjectInitializer    The initializer to run PostConstructInit() on.
	 */
	inline static void PostConstructInitObject(FObjectInitializer& ObjectInitializer)
	{
		ObjectInitializer.PostConstructInit();
	}
};

#if DO_CHECK
/** Called by NewObject to make sure Child is actually a child of Parent */
COREUOBJECT_API void CheckIsClassChildOf_Internal(const UClass* Parent, const UClass* Child);
#endif

/**
 * Convenience template for constructing a gameplay object
 *
 * @param	Outer		the outer for the new object.  If not specified, object will be created in the transient package.
 * @param	Class		the class of object to construct
 * @param	Name		the name for the new object.  If not specified, the object will be given a transient name via MakeUniqueObjectName
 * @param	Flags		the object flags to apply to the new object
 * @param	Template	the object to use for initializing the new object.  If not specified, the class's default object will be used
 * @param	bCopyTransientsFromClassDefaults	if true, copy transient from the class defaults instead of the pass in archetype ptr (often these are the same)
 * @param	InInstanceGraph						contains the mappings of instanced objects and components to their templates
 * @param	InExternalPackage					Assign an external Package to the created object if non-null
 * @param 	RemoteId	Globally unique id for this object
 *
 * @return	a pointer of type T to a new object of the specified class
 */
template< class T >
FUNCTION_NON_NULL_RETURN_START
	T* NewObject(UObject* Outer, const UClass* Class, FName Name = NAME_None, EObjectFlags Flags = RF_NoFlags, UObject* Template = nullptr, bool bCopyTransientsFromClassDefaults = false, FObjectInstancingGraph* InInstanceGraph = nullptr, UPackage* InExternalPackage = nullptr)
FUNCTION_NON_NULL_RETURN_END
{
	if (Name == NAME_None)
	{
		FObjectInitializer::AssertIfInConstructor(Outer, TEXT("NewObject with empty name can't be used to create default subobjects (inside of UObject derived class constructor) as it produces inconsistent object names. Use ObjectInitializer.CreateDefaultSubobject<> instead."));
	}

#if DO_CHECK
	// Class was specified explicitly, so needs to be validated
	CheckIsClassChildOf_Internal(T::StaticClass(), Class);
#endif

	FStaticConstructObjectParameters Params(Class);
	Params.Outer = Outer;
	Params.Name = Name;
	Params.SetFlags = Flags;
	Params.Template = Template;
	Params.bCopyTransientsFromClassDefaults = bCopyTransientsFromClassDefaults;
	Params.InstanceGraph = InInstanceGraph;
	Params.ExternalPackage = InExternalPackage;

	T* Result = static_cast<T*>(StaticConstructObject_Internal(Params));
	return Result;
}

template< class T >
FUNCTION_NON_NULL_RETURN_START
	T* NewObject(UObject* Outer = GetTransientPackageAsObject())
FUNCTION_NON_NULL_RETURN_END
{
	// Name is always None for this case
	FObjectInitializer::AssertIfInConstructor(Outer, TEXT("NewObject with empty name can't be used to create default subobjects (inside of UObject derived class constructor) as it produces inconsistent object names. Use ObjectInitializer.CreateDefaultSubobject<> instead."));

	FStaticConstructObjectParameters Params(T::StaticClass());
	Params.Outer = Outer;

	T* Result = static_cast<T*>(StaticConstructObject_Internal(Params));
	return Result;
}

template< class T >
FUNCTION_NON_NULL_RETURN_START
	T* NewObject(UObject* Outer, FName Name, EObjectFlags Flags = RF_NoFlags, UObject* Template = nullptr, bool bCopyTransientsFromClassDefaults = false, FObjectInstancingGraph* InInstanceGraph = nullptr, UPackage* InExternalPackage = nullptr)
FUNCTION_NON_NULL_RETURN_END
{
	if (Name == NAME_None)
	{
		FObjectInitializer::AssertIfInConstructor(Outer, TEXT("NewObject with empty name can't be used to create default subobjects (inside of UObject derived class constructor) as it produces inconsistent object names. Use ObjectInitializer.CreateDefaultSubobject<> instead."));
	}

	FStaticConstructObjectParameters Params(T::StaticClass());
	Params.Outer = Outer;
	Params.Name = Name;
	Params.SetFlags = Flags;
	Params.Template = Template;
	Params.bCopyTransientsFromClassDefaults = bCopyTransientsFromClassDefaults;
	Params.InstanceGraph = InInstanceGraph;
	Params.ExternalPackage = InExternalPackage;

	T* Result = static_cast<T*>(StaticConstructObject_Internal(Params));
	return Result;
}

/**
 * Convenience function for duplicating an object
 *
 * @param Class the class of the object being copied
 * @param SourceObject the object being copied
 * @param Outer the outer to use for the object
 * @param Name the optional name of the object
 *
 * @return the copied object or null if it failed for some reason
 */
COREUOBJECT_API UObject* DuplicateObject_Internal(UClass* Class, const UObject* SourceObject, UObject* Outer, const FName NAME_None);

/**
 * Convenience template for duplicating an object
 *
 * @param SourceObject the object being copied
 * @param Outer the outer to use for the object
 * @param Name the optional name of the object
 *
 * @return the copied object or null if it failed for some reason
 */
template< class T >
T* DuplicateObject(T const* SourceObject,UObject* Outer, const FName Name = NAME_None)
{
	return static_cast<T*>(DuplicateObject_Internal(T::StaticClass(), SourceObject, Outer, Name));
}

template <typename T>
T* DuplicateObject(const TObjectPtr<T>& SourceObject,UObject* Outer, const FName Name = NAME_None)
{
	return static_cast<T*>(DuplicateObject_Internal(T::StaticClass(), SourceObject, Outer, Name));
}

/**
 * Determines whether the specified object should load values using PerObjectConfig rules
 */
COREUOBJECT_API bool UsesPerObjectConfig( UObject* SourceObject );

/**
 * Returns the file to load ini values from for the specified object, taking into account PerObjectConfig-ness
 */
COREUOBJECT_API FString GetConfigFilename( UObject* SourceObject );

/*----------------------------------------------------------------------------
	Core templates.
----------------------------------------------------------------------------*/

/** Parse a reference to an object from the input stream. */
template< class T > 
inline bool ParseObject( const TCHAR* Stream, const TCHAR* Match, T*& Obj, UObject* Outer, EParseObjectLoadingPolicy LoadingPolicy, bool* bInvalidObject=nullptr )
{
	return ParseObject( Stream, Match, T::StaticClass(), (UObject*&)Obj, Outer, LoadingPolicy, bInvalidObject );
}
template< class T > 
inline bool ParseObject( const TCHAR* Stream, const TCHAR* Match, T*& Obj, UObject* Outer, bool* bInvalidObject=nullptr )
{
	return ParseObject( Stream, Match, T::StaticClass(), (UObject*&)Obj, Outer, EParseObjectLoadingPolicy::Find, bInvalidObject );
}

/**
 * Find an optional object, relies on the name being unqualified
 * @see StaticFindObjectFast()
 */
template< class T >
inline T* FindObjectFast(UObject* Outer, FName Name, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExclusiveFlags = RF_NoFlags)
{
	return (T*)StaticFindObjectFast(T::StaticClass(), Outer, Name, Flags, ExclusiveFlags);
}
template <typename T, UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("FindObjectFast")
UE_NODEBUG UE_FORCEINLINE_HINT T* FindObjectFast(UObject* Outer, FName Name, ExactClassType bExactClass, EObjectFlags ExclusiveFlags = RF_NoFlags)
{
	return FindObjectFast<T>(Outer, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExclusiveFlags);
}

/**
 * Find an optional object.
 * @see StaticFindObject()
 */
template< class T > 
inline T* FindObject( UObject* Outer, FStringView Name, EFindObjectFlags Flags = EFindObjectFlags::None )
{
	return (T*)StaticFindObject( T::StaticClass(), Outer, Name, Flags );
}
template <typename T, UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("FindObject")
UE_NODEBUG UE_FORCEINLINE_HINT T* FindObject(UObject* Outer, FStringView Name, ExactClassType bExactClass)
{
	return FindObject<T>(Outer, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Find an optional object.
 * @see StaticFindObject()
 */
template< class T >
inline T* FindObject(FTopLevelAssetPath InPath, EFindObjectFlags Flags = EFindObjectFlags::None)
{
	return (T*)StaticFindObject(T::StaticClass(), InPath, Flags);
}
template <typename T, UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("FindObject")
UE_NODEBUG UE_FORCEINLINE_HINT T* FindObject(FTopLevelAssetPath InPath, ExactClassType bExactClass)
{
	return FindObject<T>(InPath, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Find an optional object, no failure allowed
 * @see StaticFindObjectChecked()
 */
template< class T > 
inline T* FindObjectChecked( UObject* Outer, FStringView Name, EFindObjectFlags Flags = EFindObjectFlags::None )
{
	return (T*)StaticFindObjectChecked( T::StaticClass(), Outer, Name, Flags );
}
template <typename T, UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("FindObjectChecked")
UE_NODEBUG UE_FORCEINLINE_HINT T* FindObjectChecked( UObject* Outer, FStringView Name, ExactClassType bExactClass )
{
	return FindObjectChecked<T>(Outer, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Find an object without asserting on UE::IsSavingPackage() or IsGarbageCollectingAndLockingUObjectHashTables()
 * @see StaticFindObjectSafe()
 */
template< class T > 
inline T* FindObjectSafe( UObject* Outer, FStringView Name, EFindObjectFlags Flags = EFindObjectFlags::None )
{
	return (T*)StaticFindObjectSafe( T::StaticClass(), Outer, Name, Flags );
}
template <typename T, UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("FindObjectSafe")
UE_NODEBUG UE_FORCEINLINE_HINT T* FindObjectSafe( UObject* Outer, FStringView Name, ExactClassType bExactClass )
{
	return FindObjectSafe<T>(Outer, Name, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Find an optional object.
 * @see StaticFindObject()
 */
template< class T >
inline T* FindObjectSafe(FTopLevelAssetPath InPath, EFindObjectFlags Flags = EFindObjectFlags::None)
{
	return (T*)StaticFindObjectSafe(T::StaticClass(), InPath, Flags);
}
template <typename T, UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("FindObjectSafe")
UE_NODEBUG UE_FORCEINLINE_HINT T* FindObjectSafe(FTopLevelAssetPath InPath, ExactClassType bExactClass)
{
	return FindObjectSafe<T>(InPath, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None);
}

/**
 * Find an optional object with proper handling of potential ambiguity.
 * @see StaticFindFirstObject()
 */
template< class T >
inline T* FindFirstObject(FStringView Name, EFindFirstObjectOptions Options = EFindFirstObjectOptions::None, ELogVerbosity::Type AmbiguousMessageVerbosity = ELogVerbosity::NoLogging, const TCHAR* CurrentOperation = nullptr)
{
	return (T*)StaticFindFirstObject(T::StaticClass(), Name, Options, AmbiguousMessageVerbosity, CurrentOperation);
}

/**
 * Find an optional object with proper handling of potential ambiguity without asserting on UE::IsSavingPackage() or IsGarbageCollecting()
 * @see StaticFindFirstObject()
 */
template< class T >
inline T* FindFirstObjectSafe(FStringView Name, EFindFirstObjectOptions Options = EFindFirstObjectOptions::None, ELogVerbosity::Type AmbiguousMessageVerbosity = ELogVerbosity::NoLogging, const TCHAR* CurrentOperation = nullptr)
{
	return (T*)StaticFindFirstObjectSafe(T::StaticClass(), Name, Options, AmbiguousMessageVerbosity, CurrentOperation);
}

/** 
 * Load an object. 
 * @see StaticLoadObject()
 */
template< class T > 
inline T* LoadObject( UObject* Outer, FStringView Name, FStringView Filename={}, uint32 LoadFlags=LOAD_None, UPackageMap* Sandbox=nullptr, const FLinkerInstancingContext* InstancingContext=nullptr )
{
	return (T*)StaticLoadObject( T::StaticClass(), Outer, Name, Filename, LoadFlags, Sandbox, true, InstancingContext );
}

/**
 * Load a class object
 * @see StaticLoadClass
 */
template< class T > 
inline UClass* LoadClass( UObject* Outer, FStringView Name, FStringView Filename={}, uint32 LoadFlags=LOAD_None, UPackageMap* Sandbox=nullptr )
{
	return StaticLoadClass( T::StaticClass(), Outer, Name, Filename, LoadFlags, Sandbox );
}

/** 
 * Get default object of a class.
 * @see UClass::GetDefaultObject()
 */
template< class T > 
inline const T* GetDefault()
{
	return (const T*)T::StaticClass()->GetDefaultObject();
}

/**
 * Get default object of a class.
 * @see Class.h
 */
template< class T > 
inline const T* GetDefault(const UClass* Class);

/** Version of GetDefault() that allows modification */
template< class T >
inline T* GetMutableDefault()
{
	return (T*)T::StaticClass()->GetDefaultObject();
}

/** Version of GetDefault() that allows modification */
template< class T > 
inline T* GetMutableDefault(UClass *Class);

/** Returns true if a class has been loaded (e.g. it has a CDO) */
template< class T >
inline bool IsClassLoaded()
{
	return T::StaticClass()->GetDefaultObject(false) != nullptr;
}

/**
 * Looks for delegate signature with given name.
 */
COREUOBJECT_API UFunction* FindDelegateSignature(FName DelegateSignatureName);

/**
 * Determines whether the specified array contains objects of the specified class.
 *
 * @param	ObjectArray		the array to search - must be an array of pointers to instances of a UObject-derived class
 * @param	ClassToCheck	the object class to search for
 * @param	Flags			Flags which control the search
 * @param	out_Objects		if specified, any objects that match the SearchClass will be added to this array
 */
template <class T>
bool ContainsObjectOfClass( const TArray<T*>& ObjectArray, UClass* ClassToCheck, EFindObjectFlags Flags = EFindObjectFlags::None, TArray<T*>* out_Objects=nullptr )
{
	bool bResult = false;
	for ( int32 ArrayIndex = 0; ArrayIndex < ObjectArray.Num(); ArrayIndex++ )
	{
		if ( ObjectArray[ArrayIndex] != nullptr )
		{
			bool bMatchesSearchCriteria = !!(Flags & EFindObjectFlags::ExactClass)
				? ObjectArray[ArrayIndex]->GetClass() == ClassToCheck
				: ObjectArray[ArrayIndex]->IsA(ClassToCheck);

			if ( bMatchesSearchCriteria )
			{
				bResult = true;
				if ( out_Objects != nullptr )
				{
					out_Objects->Add(ObjectArray[ArrayIndex]);
				}
				else
				{
					// if we don't need a list of objects that match the search criteria, we can stop as soon as we find at least one object of that class
					break;
				}
			}
		}
	}

	return bResult;
}
template <typename T, UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("ContainsObjectOfClass")
UE_NODEBUG UE_FORCEINLINE_HINT bool ContainsObjectOfClass( const TArray<T*>& ObjectArray, UClass* ClassToCheck, ExactClassType bExactClass, TArray<T*>* out_Objects=nullptr )
{
	return ContainsObjectOfClass(ObjectArray, ClassToCheck, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, out_Objects );
}

/**
 * Utility struct for restoring object flags for all objects.
 */
class FScopedObjectFlagMarker
{
	struct FStoredObjectFlags
	{
		FStoredObjectFlags()
		: Flags(RF_NoFlags)
		, InternalFlags(EInternalObjectFlags::None)
		{}
		FStoredObjectFlags(EObjectFlags InFlags, EInternalObjectFlags InInternalFlags)
			: Flags(InFlags)
			, InternalFlags(InInternalFlags)
		{}
		EObjectFlags Flags;
		EInternalObjectFlags InternalFlags;
	};

	/**
	 * Map that tracks the ObjectFlags set on all objects; we use a map rather than iterating over all objects twice because FObjectIterator
	 * won't return objects that have RF_Unreachable set, and we may want to actually unset that flag.
	 */
	TMap<UObject*, FStoredObjectFlags> StoredObjectFlags;
	
	/**
	 * Stores the object flags for all objects in the tracking array.
	 */
	void SaveObjectFlags();

	/**
	 * Restores the object flags for all objects from the tracking array.
	 */
	void RestoreObjectFlags();

public:
	/** Constructor */
	FScopedObjectFlagMarker()
	{
		SaveObjectFlags();
	}

	/** Destructor */
	~FScopedObjectFlagMarker()
	{
		RestoreObjectFlags();
	}
};


/**
  * Iterator for arrays of UObject pointers
  * @param TObjectClass		type of pointers contained in array
*/
template<class TObjectClass>
class TObjectArrayIterator
{
	/* sample code
	TArray<APawn *> TestPawns;
	...
	// test iterator, all items
	for ( TObjectArrayIterator<APawn> It(TestPawns); It; ++It )
	{
		UE_LOGF(LogUObjectGlobals, Log, "Item %ls",*It->GetFullName());
	}
	*/
public:
	/**
		* Constructor, iterates all non-null, non pending kill objects, optionally of a particular class or base class
		* @param	InArray			the array to iterate on
		* @param	InClass			if non-null, will only iterate on items IsA this class
		* @param	InbExactClass	if true, will only iterate on exact matches
		*/
	inline TObjectArrayIterator( TArray<TObjectClass*>& InArray, UClass* InClassToCheck = nullptr, EFindObjectFlags InFlags = EFindObjectFlags::None) :	
		Array(InArray),
		Index(-1),
		ClassToCheck(InClassToCheck),
		Flags(InFlags)
	{
		Advance();
	}
	template <UE::CSameAs<bool> ExactClassType>
	UE_EXACTCLASS_BOOL_DEPRECATED("TObjectArrayIterator")
	UE_NODEBUG UE_FORCEINLINE_HINT TObjectArrayIterator( TArray<TObjectClass*>& InArray, UClass* InClassToCheck, ExactClassType bInExactClass)
		: TObjectArrayIterator(InArray, InClassToCheck, bInExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None)
	{
	}
	/**
		* Iterator advance
		*/
	UE_FORCEINLINE_HINT void operator++()
	{
		Advance();
	}
	/** conversion to "bool" returning true if the iterator is valid. */
	UE_FORCEINLINE_HINT explicit operator bool() const
	{ 
		return Index < Array.Num(); 
	}
	/** inverse of the "bool" operator */
	UE_FORCEINLINE_HINT bool operator !() const 
	{
		return !(bool)*this;
	}

	/**
		* Dereferences the iterator 
		* @return	the UObject at the iterator
	*/
	inline TObjectClass& operator*() const
	{
		checkSlow(GetObject());
		return *GetObject();
	}
	/**
		* Dereferences the iterator 
		* @return	the UObject at the iterator
	*/
	inline TObjectClass* operator->() const
	{
		checkSlow(GetObject());
		return GetObject();
	}

	/** 
	  * Removes the current element from the array, slower, but preserves the order. 
	  * Iterator is decremented for you so a loop will check all items.
	*/
	UE_FORCEINLINE_HINT void RemoveCurrent()
	{
		Array.RemoveAt(Index--);
	}
	/** 
	  * Removes the current element from the array, faster, but does not preserves the array order. 
	  * Iterator is decremented for you so a loop will check all items.
	*/
	UE_FORCEINLINE_HINT void RemoveCurrentSwap()
	{
		Array.RemoveSwap(Index--);
	}

protected:
	/**
		* Dereferences the iterator with an ordinary name for clarity in derived classes
		* @return	the UObject at the iterator
	*/
	UE_FORCEINLINE_HINT TObjectClass* GetObject() const 
	{ 
		return Array(Index);
	}
	/**
		* Iterator advance with ordinary name for clarity in subclasses
		* @return	true if the iterator points to a valid object, false if iteration is complete
	*/
	inline bool Advance()
	{
		while(++Index < Array.Num())
		{
			TObjectClass* At = GetObject();
			if (
				IsValid(At) && 
				(!ClassToCheck ||
					(!!(Flags & EFindObjectFlags::ExactClass)
						? At->GetClass() == ClassToCheck
						: At->IsA(ClassToCheck)))
				)
			{
				return true;
			}
		}
		return false;
	}
private:
	/** The array that we are iterating on */
	TArray<TObjectClass*>&	Array;
	/** Index of the current element in the object array */
	int32					Index;
	/** Class using as a criteria */
	UClass*					ClassToCheck;
	/** Flag to control the iteration */
	EFindObjectFlags		Flags;
};

/** Reference collecting archive created by FReferenceCollector::GetVerySlowReferenceCollectorArchive() */
class FReferenceCollectorArchive : public FArchiveUObject
{
	/** Object which is performing the serialization. */
	const UObject* SerializingObject = nullptr;
	/** Object that owns the serialized data. */
	const UObject* SerializedDataContainer  = nullptr;
	/** Stored pointer to reference collector. */
	class FReferenceCollector& Collector;

protected:
	UE_FORCEINLINE_HINT class FReferenceCollector& GetCollector()
	{
		return Collector;
	}

public:
	// Constructor not COREUOBJECT-exported because constructing this class is for internal use only
	FReferenceCollectorArchive(const UObject* InSerializingObject, FReferenceCollector& InCollector);

	void SetSerializingObject(const UObject* InSerializingObject)
	{
		SerializingObject = InSerializingObject;
	}
	const UObject* GetSerializingObject() const
	{
		return SerializingObject;
	}
	void SetSerializedDataContainer(const UObject* InDataContainer)
	{
		SerializedDataContainer = InDataContainer;
	}
	const UObject* GetSerializedDataContainer() const
	{
		return SerializedDataContainer;
	}

	COREUOBJECT_API virtual FArchive& operator<<(UObject*& Object) override;
	COREUOBJECT_API virtual FArchive& operator<<(FObjectPtr& Object) override;
};

/** Helper class for setting and resetting attributes on the FReferenceCollectorArchive */
class FVerySlowReferenceCollectorArchiveScope
{	
	FReferenceCollectorArchive& Archive;
	const UObject* OldSerializingObject;
	FProperty* OldSerializedProperty;
	const UObject* OldSerializedDataContainer;

public:
	FVerySlowReferenceCollectorArchiveScope(FReferenceCollectorArchive& InArchive, const UObject* InSerializingObject, FProperty* InSerializedProperty = nullptr, const UObject* InSerializedDataContainer = nullptr)
		: Archive(InArchive)
		, OldSerializingObject(InArchive.GetSerializingObject())
		, OldSerializedProperty(InArchive.GetSerializedProperty())
		, OldSerializedDataContainer(InArchive.GetSerializedDataContainer())
	{
		Archive.SetSerializingObject(InSerializingObject);
		Archive.SetSerializedProperty(InSerializedProperty);
		Archive.SetSerializedDataContainer(InSerializedDataContainer);
	}
	~FVerySlowReferenceCollectorArchiveScope()
	{
		Archive.SetSerializingObject(OldSerializingObject);
		Archive.SetSerializedProperty(OldSerializedProperty);
		Archive.SetSerializedDataContainer(OldSerializedDataContainer);
	}
	FReferenceCollectorArchive& GetArchive()
	{
		return Archive;
	}
};

/** Used by garbage collector to collect references via virtual AddReferencedObjects calls */
class FReferenceCollector
{
public:
	COREUOBJECT_API virtual ~FReferenceCollector();
	
	/** Preferred way to add a reference that allows batching. Object must outlive GC tracing, can't be used for temporary/stack references. */
	COREUOBJECT_API virtual void AddStableReference(TObjectPtr<UObject>* Object);
	
	/** Preferred way to add a reference array that allows batching. Can't be used for temporary/stack array. */
	COREUOBJECT_API virtual void AddStableReferenceArray(TArray<TObjectPtr<UObject>>* Objects);

	/** Preferred way to add a reference set that allows batching. Can't be used for temporary/stack set. */
	COREUOBJECT_API virtual void AddStableReferenceSet(TSet<TObjectPtr<UObject>>* Objects);

	template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
	inline void AddStableReferenceMap(TMapBase<KeyType, ValueType, Allocator, KeyFuncs>& Map)
	{
#if UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR
		static constexpr bool bKeyReference = TIsTObjectPtr<KeyType>::Value;
		static constexpr bool bValueReference = TIsTObjectPtr<ValueType>::Value;
		static_assert(bKeyReference || bValueReference);
		static_assert(!(std::is_pointer_v<KeyType> && std::is_convertible_v<KeyType, const UObjectBase*>));
		static_assert(!(std::is_pointer_v<ValueType> && std::is_convertible_v<ValueType, const UObjectBase*>));
#else		 
		static constexpr bool bKeyReference =	std::is_convertible_v<KeyType, const UObjectBase*>;
		static constexpr bool bValueReference =	std::is_convertible_v<ValueType, const UObjectBase*>;
		static_assert(bKeyReference || bValueReference, "Key or value must be pointer to fully-defined UObject type");
#endif
		
		for (TPair<KeyType, ValueType>& Pair : Map)
		{
			if constexpr (bKeyReference)
			{
				AddStableReference(&Pair.Key);
			}
			if constexpr (bValueReference)
			{
				AddStableReference(&Pair.Value);
			}
		}
	}

	/** Preferred way to add a reference that allows batching. Object must outlive GC tracing, can't be used for temporary/stack references. */
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()
	COREUOBJECT_API virtual void AddStableReference(UObject** Object);
	
	/** Preferred way to add a reference array that allows batching. Can't be used for temporary/stack array. */
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()
	COREUOBJECT_API virtual void AddStableReferenceArray(TArray<UObject*>* Objects);

	/** Preferred way to add a reference set that allows batching. Can't be used for temporary/stack set. */
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()	
	COREUOBJECT_API virtual void AddStableReferenceSet(TSet<UObject*>* Objects);
	
	template<class UObjectType>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()	
	inline void AddStableReference(UObjectType** Object)
	{
		static_assert(sizeof(UObjectType) > 0, "Element must be a pointer to a fully-defined type");
		static_assert(std::is_convertible_v<UObjectType*, const UObjectBase*>, "Element must be a pointer to a type derived from UObject");
		AddStableReference(reinterpret_cast<UObject**>(Object));
	}

	template<class UObjectType>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()		
	inline void AddStableReferenceArray(TArray<UObjectType*>* Objects)
	{
		static_assert(sizeof(UObjectType) > 0, "Element must be a pointer to a fully-defined type");
		static_assert(std::is_convertible_v<UObjectType*, const UObjectBase*>, "Element must be a pointer to a type derived from UObject");
		AddStableReferenceArray(reinterpret_cast<TArray<UObject*>*>(Objects)); 
	}

	template<class UObjectType>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()		
	inline void AddStableReferenceSet(TSet<UObjectType*>* Objects)
	{
		static_assert(sizeof(UObjectType) > 0, "Element must be a pointer to a fully-defined type");
		static_assert(std::is_convertible_v<UObjectType*, const UObjectBase*>, "Element must be a pointer to a type derived from UObject");
		AddStableReferenceSet(reinterpret_cast<TSet<UObject*>*>(Objects)); 
	}

	template<class UObjectType>
	UE_FORCEINLINE_HINT void AddStableReference(TObjectPtr<UObjectType>* Object)
	{
		AddStableReferenceFwd(reinterpret_cast<FObjectPtr*>(Object));
	}

	template<class UObjectType>
	UE_FORCEINLINE_HINT void AddStableReferenceArray(TArray<TObjectPtr<UObjectType>>* Objects)
	{
		AddStableReferenceArrayFwd(reinterpret_cast<TArray<FObjectPtr>*>(Objects));
	}

	template<class UObjectType>
	UE_FORCEINLINE_HINT void AddStableReferenceSet(TSet<TObjectPtr<UObjectType>>* Objects)
	{
		AddStableReferenceSetFwd(reinterpret_cast<TSet<FObjectPtr>*>(Objects));
	}

	/**
	 * Adds object reference.
	 *
	 * @param Object Referenced object.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	template<class UObjectType>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()	
	void AddReferencedObject(UObjectType*& Object, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		AROPrivate::AddReferencedObject<UObjectType>(*this, Object, ReferencingObject, ReferencingProperty);
	}

	/**
	 * Adds const object reference, this reference can still be nulled out if forcefully collected.
	 *
	 * @param Object Referenced object.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	template<class UObjectType>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()		
	void AddReferencedObject(const UObjectType*& Object, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		AROPrivate::AddReferencedObject<UObjectType>(*this, Object, ReferencingObject, ReferencingProperty);
	}

	/**
	* Adds references to an array of objects.
	*
	* @param ObjectArray Referenced objects array.
	* @param ReferencingObject Referencing object (if available).
	* @param ReferencingProperty Referencing property (if available).
	*/
	template<class UObjectType>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()		
	void AddReferencedObjects(TArray<UObjectType*>& ObjectArray, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		AROPrivate::AddReferencedObjects<UObjectType>(*this, ObjectArray, ReferencingObject, ReferencingProperty);
	}

	/**
	* Adds references to an array of const objects, these objects can still be nulled out if forcefully collected.
	*
	* @param ObjectArray Referenced objects array.
	* @param ReferencingObject Referencing object (if available).
	* @param ReferencingProperty Referencing property (if available).
	*/
	template<class UObjectType>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()	
	void AddReferencedObjects(TArray<const UObjectType*>& ObjectArray, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		AROPrivate::AddReferencedObjects<UObjectType>(*this, ObjectArray, ReferencingObject, ReferencingProperty);
	}

	/**
	* Adds references to a set of objects.
	*
	* @param ObjectSet Referenced objects set.
	* @param ReferencingObject Referencing object (if available).
	* @param ReferencingProperty Referencing property (if available).
	*/
	template<class UObjectType>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()	
	void AddReferencedObjects(TSet<UObjectType*>& ObjectSet, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		AROPrivate::AddReferencedObjects<UObjectType>(*this, ObjectSet, ReferencingObject, ReferencingProperty);
	}

	/**
	 * Adds references to a map of objects.
	 *
	 * @param Map Referenced objects map.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()		
	void AddReferencedObjects(TMapBase<KeyType*, ValueType, Allocator, KeyFuncs>& Map, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		AROPrivate::AddReferencedObjects<KeyType, ValueType, Allocator, KeyFuncs>(*this, Map, ReferencingObject, ReferencingProperty);
	}

	template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()			
	void AddReferencedObjects(TMapBase<KeyType, ValueType*, Allocator, KeyFuncs>& Map, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		AROPrivate::AddReferencedObjects<KeyType, ValueType, Allocator, KeyFuncs>(*this, Map, ReferencingObject, ReferencingProperty);
	}

	template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()				
	void AddReferencedObjects(TMapBase<KeyType*, ValueType*, Allocator, KeyFuncs>& Map, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		AROPrivate::AddReferencedObjects<KeyType, ValueType, Allocator, KeyFuncs>(*this, Map, ReferencingObject, ReferencingProperty);
	}

	/**
	 * Adds object reference.
	 *
	 * @param Object Referenced object.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	template<class UObjectType>
	void AddReferencedObject(TObjectPtr<UObjectType>& Object, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		if (IsObjectHandleResolved_ForGC(Object.GetHandle()))
		{
			HandleObjectReference(*reinterpret_cast<UObject**>(&Object), ReferencingObject, ReferencingProperty);
		}
	}

	/**
	 * Adds const object reference, this reference can still be nulled out if forcefully collected.
	 *
	 * @param Object Referenced object.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	template<class UObjectType>
	void AddReferencedObject(TObjectPtr<const UObjectType>& Object, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		static_assert(sizeof(UObjectType) > 0, "AddReferencedObject: Element must be a pointer to a fully-defined type");
		static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObject: Element must be a pointer to a type derived from UObject");
		if (IsObjectHandleResolved_ForGC(Object.GetHandle()))
		{
			HandleObjectReference(*reinterpret_cast<UObject**>(&Object), ReferencingObject, ReferencingProperty);
		}
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/**
	 * Adds Verse value reference. Defined in VVMWriteBarrier.h.
	 *
	 * @param Value Referenced value.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	template<class VCellType>
	void AddReferencedVerseValue(Verse::TWriteBarrier<VCellType>& Value, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr);

	/**
	 * Adds Verse value reference. Defined in VVMRestValue.h.
	 *
	 * @param Value Referenced value.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	void AddReferencedVerseValue(Verse::VRestValue& Value, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr);
#endif

	/**
	* Adds references to an array of objects.
	*
	* @param ObjectArray Referenced objects array.
	* @param ReferencingObject Referencing object (if available).
	* @param ReferencingProperty Referencing property (if available).
	*/
	template<class UObjectType>
	void AddReferencedObjects(TArray<TObjectPtr<UObjectType>>& ObjectArray, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		static_assert(sizeof(UObjectType) > 0, "AddReferencedObjects: Elements must be pointers to a fully-defined type");
		static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObjects: Elements must be pointers to a type derived from UObject");
		// Cannot use a reinterpret_cast due to MSVC (and not Clang) emitting a warning:
		// C4946: reinterpret_cast used between related classes: ...
		HandleObjectReferences((FObjectPtr*)(ObjectArray.GetData()), ObjectArray.Num(), ReferencingObject, ReferencingProperty);
	}

	/**
	* Adds references to an array of const objects, these objects can still be nulled out if forcefully collected.
	*
	* @param ObjectArray Referenced objects array.
	* @param ReferencingObject Referencing object (if available).
	* @param ReferencingProperty Referencing property (if available).
	*/
	template<class UObjectType>
	void AddReferencedObjects(TArray<TObjectPtr<const UObjectType>>& ObjectArray, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		static_assert(sizeof(UObjectType) > 0, "AddReferencedObjects: Elements must be pointers to a fully-defined type");
		static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObjects: Elements must be pointers to a type derived from UObject");
		// Cannot use a reinterpret_cast due to MSVC (and not Clang) emitting a warning:
		// C4946: reinterpret_cast used between related classes: ...
		HandleObjectReferences((FObjectPtr*)(ObjectArray.GetData()), ObjectArray.Num(), ReferencingObject, ReferencingProperty);
	}

	/**
	* Adds references to a set of objects.
	*
	* @param ObjectSet Referenced objects set.
	* @param ReferencingObject Referencing object (if available).
	* @param ReferencingProperty Referencing property (if available).
	*/
	template<class UObjectType>
	void AddReferencedObjects(TSet<TObjectPtr<UObjectType>>& ObjectSet, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		static_assert(sizeof(UObjectType) > 0, "AddReferencedObjects: Elements must be pointers to a fully-defined type");
		static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObjects: Elements must be pointers to a type derived from UObject");
		for (auto& Object : ObjectSet)
		{
			if (IsObjectHandleResolved_ForGC(Object.GetHandle()))
			{
				HandleObjectReference(*reinterpret_cast<UObject**>(&Object), ReferencingObject, ReferencingProperty);
			}
		}
	}

	/**
	 * Adds references to a map of objects.
	 *
	 * @param ObjectArray Referenced objects map.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
	void AddReferencedObjects(TMapBase<TObjectPtr<KeyType>, ValueType, Allocator, KeyFuncs>& Map, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		static_assert(sizeof(KeyType) > 0, "AddReferencedObjects: Keys must be pointers to a fully-defined type");
		static_assert(TPointerIsConvertibleFromTo<KeyType, const UObjectBase>::Value, "AddReferencedObjects: Keys must be pointers to a type derived from UObject");
		static_assert(!UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR ||
									!(std::is_pointer_v<ValueType> && std::is_convertible_v<ValueType, const UObjectBase*>));
		for (auto& It : Map)
		{
			if (IsObjectHandleResolved_ForGC(It.Key.GetHandle()))
			{
				HandleObjectReference(*reinterpret_cast<UObject**>(&It.Key), ReferencingObject, ReferencingProperty);
			}
		}
	}
	template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
	void AddReferencedObjects(TMapBase<KeyType, TObjectPtr<ValueType>, Allocator, KeyFuncs>& Map, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		static_assert(sizeof(ValueType) > 0, "AddReferencedObjects: Values must be pointers to a fully-defined type");
		static_assert(TPointerIsConvertibleFromTo<ValueType, const UObjectBase>::Value, "AddReferencedObjects: Values must be pointers to a type derived from UObject");
		static_assert(!UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR ||
									!(std::is_pointer_v<KeyType> && std::is_convertible_v<KeyType, const UObjectBase*>));
		for (auto& It : Map)
		{
			if (IsObjectHandleResolved_ForGC(It.Value.GetHandle()))
			{
				HandleObjectReference(*reinterpret_cast<UObject**>(&It.Value), ReferencingObject, ReferencingProperty);
			}
		}
	}
	template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
	void AddReferencedObjects(TMapBase<TObjectPtr<KeyType>, TObjectPtr<ValueType>, Allocator, KeyFuncs>& Map, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
	{
		static_assert(sizeof(KeyType) > 0, "AddReferencedObjects: Keys must be pointers to a fully-defined type");
		static_assert(sizeof(ValueType) > 0, "AddReferencedObjects: Values must be pointers to a fully-defined type");
		static_assert(TPointerIsConvertibleFromTo<KeyType, const UObjectBase>::Value, "AddReferencedObjects: Keys must be pointers to a type derived from UObject");
		static_assert(TPointerIsConvertibleFromTo<ValueType, const UObjectBase>::Value, "AddReferencedObjects: Values must be pointers to a type derived from UObject");
		for (auto& It : Map)
		{
			if (IsObjectHandleResolved_ForGC(It.Key.GetHandle()))
			{
				HandleObjectReference(*reinterpret_cast<UObject**>(&It.Key), ReferencingObject, ReferencingProperty);
			}
			if (IsObjectHandleResolved_ForGC(It.Value.GetHandle()))
			{
				HandleObjectReference(*reinterpret_cast<UObject**>(&It.Value), ReferencingObject, ReferencingProperty);
			}
		}
	}
	
	template <typename T>
	void AddReferencedObject(TWeakObjectPtr<T>& P,
													 const UObject* ReferencingObject = nullptr,
													 const FProperty* ReferencingProperty = nullptr)
	{
		AddReferencedObject(reinterpret_cast<FWeakObjectPtr&>(P),
												ReferencingObject,
												ReferencingProperty);
	}

	COREUOBJECT_API void AddReferencedObject(FWeakObjectPtr& P,
													 const UObject* ReferencingObject = nullptr,
													 const FProperty* ReferencingProperty = nullptr);

	/**
	 * Adds all strong property references from a UScriptStruct instance including the struct itself
	 * 
	 * Only necessary to handle cases of an unreflected/non-UPROPERTY struct that wants to have references emitted.
	 *
	 * Calls AddStructReferencedObjects() but not recursively on nested structs. 
	 * 
	 * This and other AddPropertyReferences functions will hopefully merge into a single function in the future.
	 * They're kept separate initially to maintain exact semantics while replacing the much slower
	 * SerializeBin/TPropertyValueIterator/GetVerySlowReferenceCollectorArchive paths.
	 */
	UE_REFERENCE_COLLECTOR_REQUIRE_OBJECTPTR_DEPRECATED()				
	COREUOBJECT_API void AddReferencedObjects(const UScriptStruct*& ScriptStruct, void* Instance, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr);

	COREUOBJECT_API void AddReferencedObjects(TObjectPtr<const UScriptStruct>& ScriptStruct, void* Instance, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr);
	COREUOBJECT_API void AddReferencedObjects(TWeakObjectPtr<const UScriptStruct>& ScriptStruct, void* Instance, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr);
	
	/** Adds all strong property references from a struct instance, but not the struct itself. Skips AddStructReferencedObjects. */
	COREUOBJECT_API void AddPropertyReferences(const UStruct* Struct, void* Instance, const UObject* ReferencingObject = nullptr);
	
	/** Same as AddPropertyReferences but also calls AddStructReferencedObjects on Struct and all nested structs */
	COREUOBJECT_API void AddPropertyReferencesWithStructARO(const UScriptStruct* Struct, void* Instance, const UObject* ReferencingObject = nullptr);

	/** Same as AddPropertyReferences but also calls AddStructReferencedObjects on all nested structs */
	COREUOBJECT_API void AddPropertyReferencesWithStructARO(const UClass* Class, void* Instance, const UObject* ReferencingObject = nullptr);

	/** Internal use only. Same as AddPropertyReferences but skips field path and interface properties. Might get removed. */
	COREUOBJECT_API void AddPropertyReferencesLimitedToObjectProperties(const UStruct* Struct, void* Instance, const UObject* ReferencingObject = nullptr);

	/**
	 * Make Add[OnlyObject]PropertyReference/AddReferencedObjects(UScriptStruct) use AddReferencedObjects(UObject*&) callbacks
	 * with ReferencingObject and ReferencingProperty context supplied and check for null references before making a callback.
	 * 
	 * Return false to use context free AddStableReference callbacks without null checks that avoid sync cache misses when batch processing references.
	 */
	virtual bool NeedsPropertyReferencer() const { return true; }

	/**
	 * If true archetype references should not be added to this collector.
	 */
	virtual bool IsIgnoringArchetypeRef() const = 0;
	/**
	 * If true transient objects should not be added to this collector.
	 */
	virtual bool IsIgnoringTransient() const = 0;
	/**
	 * Allows reference elimination by this collector.
	 */
	virtual void AllowEliminatingReferences(bool bAllow) {}


	/**
	 * Sets the property that is currently being serialized
	 */
	virtual void SetSerializedProperty(class FProperty* Inproperty) {}
	/**
	 * Gets the property that is currently being serialized
	 */
	virtual class FProperty* GetSerializedProperty() const { return nullptr; }
	/** 
	 * Marks a specific object reference as a weak reference. This does not affect GC but will be freed at a later point
	 * The default behavior returns false as weak references must be explicitly supported
	 */
	virtual bool MarkWeakObjectReferenceForClearing(UObject** WeakReference, UObject* ReferenceOwner) { return false; }
	/**
	 * Sets whether this collector is currently processing native references or not.
	 */
	virtual void SetIsProcessingNativeReferences(bool bIsNative) {}
	/**
	 * If true, this collector is currently processing native references (true by default).
	 */
	virtual bool IsProcessingNativeReferences() const { return true; }

	/** Used by parallel reachability analysis to pre-collect and then exclude some initial FGCObject references */
	virtual bool NeedsInitialReferences() const { return true; }

	/**
	* Get archive to collect references via SerializeBin / Serialize.
	*
	* NOTE: Prefer using AddPropertyReferences or AddReferencedObjects(const UScriptStruct&) instead, they're much faster.
	*/
	FReferenceCollectorArchive& GetVerySlowReferenceCollectorArchive()
	{
		if (!DefaultReferenceCollectorArchive)
		{
			CreateVerySlowReferenceCollectorArchive();
		}
		return *DefaultReferenceCollectorArchive;
	}


	struct AROPrivate final // nb: internal use only
	{
		template<class UObjectType>
		static void AddReferencedObject(FReferenceCollector& Coll,
																		UObjectType*& Object, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
		{
			// @todo: should be uncommented when proper usage is fixed everywhere
			// static_assert(sizeof(UObjectType) > 0, "AddReferencedObject: Element must be a pointer to a fully-defined type");
			// static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObject: Element must be a pointer to a type derived from UObject");
			Coll.HandleObjectReference(*(UObject**)&Object, ReferencingObject, ReferencingProperty);
		}

		template<class UObjectType>
		static void AddReferencedObject(FReferenceCollector& Coll,
																		const UObjectType*& Object,
																		const UObject* ReferencingObject = nullptr,
																		const FProperty* ReferencingProperty = nullptr)
		{
			// @todo: should be uncommented when proper usage is fixed everywhere
			// static_assert(sizeof(UObjectType) > 0, "AddReferencedObject: Element must be a pointer to a fully-defined type");
			// static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObject: Element must be a pointer to a type derived from UObject");
			Coll.HandleObjectReference(*(UObject**)const_cast<UObjectType**>(&Object), ReferencingObject, ReferencingProperty);
		}

		template<class UObjectType>
		static void AddReferencedObjects(FReferenceCollector& Coll,
																		 TArray<UObjectType*>& ObjectArray,
																		 const UObject* ReferencingObject = nullptr,
																		 const FProperty* ReferencingProperty = nullptr)
		{
			static_assert(sizeof(UObjectType) > 0, "AddReferencedObjects: Elements must be pointers to a fully-defined type");
			static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObjects: Elements must be pointers to a type derived from UObject");
			Coll.HandleObjectReferences(reinterpret_cast<UObject**>(ObjectArray.GetData()), ObjectArray.Num(), ReferencingObject, ReferencingProperty);
		}
		
		/**
		 * Adds references to an array of const objects, these objects can still be nulled out if forcefully collected.
		 *
		 * @param ObjectArray Referenced objects array.
		 * @param ReferencingObject Referencing object (if available).
		 * @param ReferencingProperty Referencing property (if available).
		 */
		template<class UObjectType>
		static void AddReferencedObjects(FReferenceCollector& Coll,
																		 TArray<const UObjectType*>& ObjectArray,
																		 const UObject* ReferencingObject = nullptr,
																		 const FProperty* ReferencingProperty = nullptr)
		{
			static_assert(sizeof(UObjectType) > 0, "AddReferencedObjects: Elements must be pointers to a fully-defined type");
			static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObjects: Elements must be pointers to a type derived from UObject");
			Coll.HandleObjectReferences(reinterpret_cast<UObject**>(const_cast<UObjectType**>(ObjectArray.GetData())), ObjectArray.Num(), ReferencingObject, ReferencingProperty);
		}

		template<class UObjectType>
		static void AddReferencedObjects(FReferenceCollector& Coll,
																		 TSet<UObjectType*>& ObjectSet,
																		 const UObject* ReferencingObject = nullptr,
																		 const FProperty* ReferencingProperty = nullptr)
		{
			static_assert(sizeof(UObjectType) > 0, "AddReferencedObjects: Elements must be pointers to a fully-defined type");
			static_assert(TPointerIsConvertibleFromTo<UObjectType, const UObjectBase>::Value, "AddReferencedObjects: Elements must be pointers to a type derived from UObject");
			for (auto& Object : ObjectSet)
			{
				Coll.HandleObjectReference(*(UObject**)&Object, ReferencingObject, ReferencingProperty);
			}
		}
	
		template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
		static void AddReferencedObjects(FReferenceCollector& Coll,
																		 TMapBase<KeyType*, ValueType, Allocator, KeyFuncs>& Map,
																		 const UObject* ReferencingObject = nullptr,
																		 const FProperty* ReferencingProperty = nullptr)
		{
			static_assert(sizeof(KeyType) > 0, "AddReferencedObjects: Keys must be pointers to a fully-defined type");
			static_assert(TPointerIsConvertibleFromTo<KeyType, const UObjectBase>::Value, "AddReferencedObjects: Keys must be pointers to a type derived from UObject");
			for (auto& It : Map)
			{
				Coll.HandleObjectReference(*(UObject**)&It.Key, ReferencingObject, ReferencingProperty);
			}
		}

		template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
		static void AddReferencedObjects(FReferenceCollector& Coll,
																		 TMapBase<KeyType, ValueType*, Allocator, KeyFuncs>& Map,
																		 const UObject* ReferencingObject = nullptr,
																		 const FProperty* ReferencingProperty = nullptr)
		{
			static_assert(sizeof(ValueType) > 0, "AddReferencedObjects: Values must be pointers to a fully-defined type");
			static_assert(TPointerIsConvertibleFromTo<ValueType, const UObjectBase>::Value, "AddReferencedObjects: Values must be pointers to a type derived from UObject");
			for (auto& It : Map)
			{
				Coll.HandleObjectReference(*(UObject**)&It.Value, ReferencingObject, ReferencingProperty);
			}
		}

		template <typename KeyType, typename ValueType, typename Allocator, typename KeyFuncs>
		static void AddReferencedObjects(FReferenceCollector& Coll, TMapBase<KeyType*, ValueType*, Allocator, KeyFuncs>& Map, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr)
		{
			static_assert(sizeof(KeyType) > 0, "AddReferencedObjects: Keys must be pointers to a fully-defined type");
			static_assert(sizeof(ValueType) > 0, "AddReferencedObjects: Values must be pointers to a fully-defined type");
			static_assert(TPointerIsConvertibleFromTo<KeyType, const UObjectBase>::Value, "AddReferencedObjects: Keys must be pointers to a type derived from UObject");
			static_assert(TPointerIsConvertibleFromTo<ValueType, const UObjectBase>::Value, "AddReferencedObjects: Values must be pointers to a type derived from UObject");
			for (auto& It : Map)
			{
				Coll.HandleObjectReference(*(UObject**)&It.Key, ReferencingObject, ReferencingProperty);
				Coll.HandleObjectReference(*(UObject**)&It.Value, ReferencingObject, ReferencingProperty);
			}
		}

		static void AddReferencedObjects(FReferenceCollector& Coll,
																		 const UScriptStruct*& ScriptStruct,
																		 void* Instance, const UObject* ReferencingObject = nullptr, const FProperty* ReferencingProperty = nullptr);
	};  

protected:
	/**
	 * Handle object reference. Called by AddReferencedObject.
	 *
	 * @param Object Referenced object.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	virtual void HandleObjectReference(UObject*& InObject, const UObject* InReferencingObject, const FProperty* InReferencingProperty) = 0;

	/**
	* Handle multiple object references. Called by AddReferencedObjects.
	* DEFAULT IMPLEMENTATION IS SLOW as it calls HandleObjectReference multiple times. In order to optimize it, provide your own implementation.
	*
	* @param Object Referenced object.
	* @param ReferencingObject Referencing object (if available).
	* @param ReferencingProperty Referencing property (if available).
	*/
	virtual void HandleObjectReferences(UObject** InObjects, const int32 ObjectNum, const UObject* InReferencingObject, const FProperty* InReferencingProperty)
	{
		for (int32 ObjectIndex = 0; ObjectIndex < ObjectNum; ++ObjectIndex)
		{
			UObject*& Object = InObjects[ObjectIndex];
			HandleObjectReference(Object, InReferencingObject, InReferencingProperty);
		}
	}

	/**
	* Handle multiple object references. Called by AddReferencedObjects.
	* DEFAULT IMPLEMENTATION IS SLOW as it calls HandleObjectReference multiple times. In order to optimize it, provide your own implementation.
	*
	* @param Object Referenced object.
	* @param ReferencingObject Referencing object (if available).
	* @param ReferencingProperty Referencing property (if available).
	*/
	COREUOBJECT_API virtual void HandleObjectReferences(FObjectPtr* InObjects, const int32 ObjectNum, const UObject* InReferencingObject, const FProperty* InReferencingProperty);

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	/**
	 * Handle Verse cell. Called by AddReferencedVerseValue.
	 *
	 * @param Value Referenced value.
	 * @param ReferencingObject Referencing object (if available).
	 * @param ReferencingProperty Referencing property (if available).
	 */
	virtual void HandleVCellReference(Verse::VCell* InCell, const UObject* InReferencingObject, const FProperty* InReferencingProperty)
	{
		// Ignore VCells by default, like `FSimpleReferenceProcessorBase::HandleTokenStreamVerseCellReference`.
	}
#endif

private:
	/** Creates the proxy archive that uses serialization to add objects to this collector */
	COREUOBJECT_API void CreateVerySlowReferenceCollectorArchive();

	/** Default proxy archive that uses serialization to add objects to this collector */
	TUniquePtr<FReferenceCollectorArchive> DefaultReferenceCollectorArchive;

	COREUOBJECT_API void AddStableReferenceSetFwd(TSet<FObjectPtr>* Objects);	 
	COREUOBJECT_API void AddStableReferenceArrayFwd(TArray<FObjectPtr>* Objects);		
	COREUOBJECT_API void AddStableReferenceFwd(FObjectPtr* Object);	 
};

/**
 * FReferenceFinder.
 * Helper class used to collect object references.
 */
class FReferenceFinder : public FReferenceCollector
{
public:

	/**
	 * Constructor
	 *
	 * @param InObjectArray Array to add object references to
	 * @param	InOuter					value for LimitOuter
	 * @param	bInRequireDirectOuter	value for bRequireDirectOuter
	 * @param	bShouldIgnoreArchetype	whether to disable serialization of ObjectArchetype references
	 * @param	bInSerializeRecursively	only applicable when LimitOuter != nullptr && bRequireDirectOuter==true;
	 *									serializes each object encountered looking for subobjects of referenced
	 *									objects that have LimitOuter for their Outer (i.e. nested subobjects/components)
	 * @param	bShouldIgnoreTransient	true to skip serialization of transient properties
	 */
	COREUOBJECT_API FReferenceFinder(TArray<UObject*>& InObjectArray, UObject* InOuter = nullptr, bool bInRequireDirectOuter = true, bool bInShouldIgnoreArchetype = false, bool bInSerializeRecursively = false, bool bInShouldIgnoreTransient = false);

	COREUOBJECT_API ~FReferenceFinder();
	/**
	 * Finds all objects referenced by Object.
	 *
	 * @param Object Object which references are to be found.
	 * @param ReferencingObject object that's referencing the current object.
	 * @param ReferencingProperty property the current object is being referenced through.
	 */
	COREUOBJECT_API virtual void FindReferences(UObject* Object, UObject* ReferencingObject = nullptr, FProperty* ReferencingProperty = nullptr);

	// FReferenceCollector interface.
	COREUOBJECT_API virtual void HandleObjectReference(UObject*& Object, const UObject* ReferencingObject, const FProperty* InReferencingProperty) override;
	virtual bool IsIgnoringArchetypeRef() const override { return bShouldIgnoreArchetype; }
	virtual bool IsIgnoringTransient() const override { return bShouldIgnoreTransient; }
	virtual void SetSerializedProperty(class FProperty* Inproperty) override
	{
		SerializedProperty = Inproperty;
	}
	virtual class FProperty* GetSerializedProperty() const override
	{
		return SerializedProperty;
	}
protected:

	/** Stored reference to array of objects we add object references to. */
	TArray<UObject*>&		ObjectArray;
	/** Set that duplicates ObjectArray. Keeps ObjectArray unique and avoids duplicate recursive serializing. */
	TSet<const UObject*>	ObjectSet;
	/** Only objects within this outer will be considered, nullptr value indicates that outers are disregarded. */
	UObject*		LimitOuter;
	/** Property that is referencing the current object */
	class FProperty* SerializedProperty;
	/** Determines whether nested objects contained within LimitOuter are considered. */
	bool			bRequireDirectOuter;
	/** Determines whether archetype references are considered. */
	bool			bShouldIgnoreArchetype;
	/** Determines whether we should recursively look for references of the referenced objects. */
	bool			bSerializeRecursively;
	/** Determines whether transient references are considered. */
	bool			bShouldIgnoreTransient;
};

/** Defined in PackageReload.h */
enum class EPackageReloadPhase : uint8;
class FGarbageCollectionTracer;
class FPackageReloadedEvent;

enum class EHotReloadedClassFlags
{
	None = 0,

	// Set when the hot reloaded class has been detected as changed
	Changed = 0x01
};

ENUM_CLASS_FLAGS(EHotReloadedClassFlags)

enum class EReloadCompleteReason
{
	None,
	HotReloadAutomatic,
	HotReloadManual,
};

struct FEndLoadPackageContext
{
	TConstArrayView<UPackage*> LoadedPackages;
	int32 RecursiveDepth;
	bool bSynchronous;
};

enum class ECompiledInUObjectsRegisteredStatus
{

	// Registration for the given module has been delayed and will be registered with a layer notification where the package name is NAME_None.
	// Calls to FindObject<> for any delayed registrations will return nullptr.
	Delayed,

	// All pending registrations have been done but CDOs have not been created.  This may be invoked multiple times for the same package name.
	// Calls to FindObject<> for all pending registrations will return the UObject in question.
	PreCDO,

	// All pending registrations and CDO creation has been done.  Calls to FindObject<> for all pending registrations will return the UObject 
	// in question.  The CDO will also be available for any registered UClasses.
	PostCDO,
};

/**
 * Global CoreUObject delegates
 */
struct FCoreUObjectDelegates
{
#if WITH_EDITOR
	/** Callback for object property modifications, called by UObject::PostEditChangeProperty with a single property event */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnObjectPropertyChanged, UObject*, struct FPropertyChangedEvent&);
	static COREUOBJECT_API FOnObjectPropertyChanged OnObjectPropertyChanged;

	/** Callback for object property modifications, called by UObject::PreEditChange with a full property chain */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPreObjectPropertyChanged, UObject*, const class FEditPropertyChain&);
	static COREUOBJECT_API FOnPreObjectPropertyChanged OnPreObjectPropertyChanged;

	/** Callback for object property modifications, called by UObject::PostEditChangeChainProperty with a full property changed chain event */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPostObjectPropertyChanged, UObject*, const struct FPropertyChangedChainEvent&);
	static COREUOBJECT_API FOnPostObjectPropertyChanged OnPostObjectPropertyChanged;

	/** Called when an object is registered for change with UObject::Modify. This gets called in both the editor and standalone game editor builds, for every object modified */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnObjectModified, UObject*);
	static COREUOBJECT_API FOnObjectModified OnObjectModified;

	/** Set of objects modified this frame, to prevent multiple triggerings of the OnObjectModified delegate */
	static COREUOBJECT_API TSet<UObject*> ObjectsModifiedThisFrame;

	/** Broadcast OnObjectModified if the broadcast hasn't occurred for this object in this frame */
	static void BroadcastOnObjectModified(UObject* Object)
	{
		if (OnObjectModified.IsBound() && !ObjectsModifiedThisFrame.Contains(Object))
		{
			ObjectsModifiedThisFrame.Add(Object);
			OnObjectModified.Broadcast(Object);
		}
	}

	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnObjectRenamed, UObject*, UObject*, FName);
	/** Called when an object is renamed. This gets called in both the editor and standalone game editor builds, for every object renamed */
	static COREUOBJECT_API FOnObjectRenamed OnObjectRenamed;

	/** Called when an object that was loaded is renamed. This gets called in both the editor and standalone game editor builds, for every renamed loaded object */
	static COREUOBJECT_API FOnObjectRenamed OnLoadedObjectRenamed;

	/** Broadcast OnObjectRenamed event */
	static void BroadcastOnObjectRenamed(UObject* Object, UObject* OldOuter, FName OldName)
	{
		OnObjectRenamed.Broadcast(Object, OldOuter, OldName);
	}

	/** Callback for an object being transacted */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnObjectTransacted, UObject*, const class FTransactionObjectEvent&);
	static COREUOBJECT_API FOnObjectTransacted OnObjectTransacted;

	/**
	 * Called when UObjects have been replaced to allow others a chance to fix their references
	 * Note that this is called after properties are copied from old to new instances but before references to replacement
	 * objects are fixed up in other objects (i.e. other objects can still be pointing to old data)
	 */
	using FReplacementObjectMap = TMap<UObject*, UObject*>; // Alias for use in the macro
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnObjectsReplaced, const FReplacementObjectMap&);
	static COREUOBJECT_API FOnObjectsReplaced OnObjectsReplaced;

	/**
	 * Called when UObjects have been re-instanced to allow others a chance to fix their references
	 * Note that this is called after references to replacement objects are fixed up in other objects (i.e. all object
	 * references should be self-consistent).
	 */
	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnObjectsReinstanced, const FReplacementObjectMap&);
	static COREUOBJECT_API FOnObjectsReinstanced OnObjectsReinstanced;

	/**
	 * Called when verse found instances of object that their classes is now dead (renamed or deleted)
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnVerseDeadReferences, const TSet<UClass*>& /*DeadClasses*/, const TSet<UObject*>& /*ActiveInstances*/, const TSet<UObject*>& /*TemplateInstances*/);
	static COREUOBJECT_API FOnVerseDeadReferences OnVerseDeadObjectReferences;

	/**
	 * Called after the Blueprint compiler has finished generating the Class Default Object (CDO) for a class. This can only happen in the editor.
	 * This is called when the CDO and its associated class structure have been fully generated and populated, and allows the assignment of cached/derived data,
	 * eg) caching the name/count of properties of a certain type, or inspecting the properties on the class and using their meta-data and CDO default values to derive game data.
	 */
	DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnObjectPostCDOCompiled, UObject*, const FObjectPostCDOCompiledContext&);
	static COREUOBJECT_API FOnObjectPostCDOCompiled OnObjectPostCDOCompiled;

	/** Callback for when an asset is saved. This is called from UObject::PreSave before it is actually written to disk, for every object saved */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnObjectPreSave, UObject*, FObjectPreSaveContext);
	static COREUOBJECT_API FOnObjectPreSave OnObjectPreSave;

	/** Callback for when an asset is loaded. This gets called in both the editor and standalone game editor builds, but only for objects that return true for IsAsset() */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnAssetLoaded, UObject*);
	static COREUOBJECT_API FOnAssetLoaded OnAssetLoaded;

	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnObjectConstructed, UObject*);
	static COREUOBJECT_API FOnObjectConstructed OnObjectConstructed;

#endif // WITH_EDITOR

#if WITH_EDITOR || UE_ENABLE_ASSET_READ_LOGGER
	/** Callback when packages end loading in LoadPackage or AsyncLoadPackage. All packages loaded recursively due to imports are included in the single call of the explicitly-loaded package. It is called in all `WITH_EDITOR` cases, including -game. But it is not called in the runtime game compiled without WITH_EDITOR. This difference in behavior between the two game configurations is necessary because WITH_EDITOR uses a different loading mechanism than runtime game.
	 * In non-editor builds the callback is also available when the asset read logger is compiled in, so profiling hooks can observe package completions. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEndLoadPackage, const FEndLoadPackageContext&);
	static COREUOBJECT_API FOnEndLoadPackage OnEndLoadPackage;
#endif // WITH_EDITOR || UE_ENABLE_ASSET_READ_LOGGER

	/** Callback when a package finishes loading via the async loader. Available in all builds (including shipping).
	 *  This fires for each package that completes loading, providing the loaded UPackage.
	 *  Note: This is called from the game thread after the package is fully loaded and post-loaded. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPackageLoadCompleted, UPackage*);
	UE_DEPRECATED(5.9, "Unsafe and will be removed: this delegate can broadcast a UPackage whose outer objects are still being mutated or already freed, so reading package state from it can crash. Editor/logger code should use FCoreUObjectDelegates::OnEndLoadPackage instead.")
	static COREUOBJECT_API FOnPackageLoadCompleted OnPackageLoadCompleted;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnHotfixableAssetLoaded, UObject*);
	/** Fired after an asset of a registered hotfixable class finishes loading. */
	static COREUOBJECT_API FOnHotfixableAssetLoaded OnHotfixableAssetLoaded;
	/**
	 * Registers a class whose instances should trigger OnHotfixableAssetLoaded when loaded.
	 * Only the exact class is matched; derived and parent classes are not included.
	 * Register each concrete class separately. Reference-counted to support multiple callers.
	 */
	static COREUOBJECT_API void RegisterHotfixableAssetClass(UClass* Class);
	/**
	 * Unregisters a previously registered hotfixable class. Only affects the exact class passed.
	 */
	static COREUOBJECT_API void UnregisterHotfixableAssetClass(UClass* Class);
	/** Returns whether the exact class (not parents or subclasses) is registered as hotfixable. */
	static COREUOBJECT_API bool IsHotfixableAssetClass(UClass* Class);

	/** Called when new sparse class data has been created (and the base data initialized) for the given class */
	DECLARE_TS_MULTICAST_DELEGATE_ThreeParams(FOnPostInitSparseClassData, UClass*, UScriptStruct*, void*);
	static COREUOBJECT_API FOnPostInitSparseClassData OnPostInitSparseClassData;

	/** Called by ReloadPackage during package reloading. It will be called several times for different phases of fix-up to allow custom code to handle updating objects as needed */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPackageReloaded, EPackageReloadPhase, FPackageReloadedEvent*);
	static COREUOBJECT_API FOnPackageReloaded OnPackageReloaded;

	/** Called when a package reload request is received from a network file server */
	DECLARE_DELEGATE_OneParam(FNetworkFileRequestPackageReload, const TArray<FString>& /*PackageNames*/);
	static COREUOBJECT_API FNetworkFileRequestPackageReload NetworkFileRequestPackageReload;

	/** Delegate used by SavePackage() to check whether a package should be saved */
	DECLARE_DELEGATE_RetVal_ThreeParams(bool, FIsPackageOKToSaveDelegate, UPackage*, const FString&, FOutputDevice*);
	static COREUOBJECT_API FIsPackageOKToSaveDelegate IsPackageOKToSaveDelegate;

	/** Delegate for reloaded classes that have been added. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FReloadAddedClassesDelegate, const TArray<UClass*>&);
	static COREUOBJECT_API FReloadAddedClassesDelegate ReloadAddedClassesDelegate;

	/** Delegate for reload re-instancing complete */
	DECLARE_MULTICAST_DELEGATE(FReloadReinstancingCompleteDelegate);
	static COREUOBJECT_API FReloadReinstancingCompleteDelegate ReloadReinstancingCompleteDelegate;

	/** Delegate for reload re-instancing complete */
	DECLARE_MULTICAST_DELEGATE_OneParam(FReloadCompleteDelegate, EReloadCompleteReason);
	static COREUOBJECT_API FReloadCompleteDelegate ReloadCompleteDelegate;

	/** Delegate for registering hot-reloaded classes that changed after hot-reload for reinstancing */
	UE_DEPRECATED(all, "RegisterClassForHotReloadReinstancingDelegate is for internal use only.")
	static COREUOBJECT_API TMulticastDelegate<void(UClass* OldClass, UClass* NewClass, EHotReloadedClassFlags Flags)> RegisterClassForHotReloadReinstancingDelegate;

	/** Delegate for reinstancing hot-reloaded classes */
	UE_DEPRECATED(all, "ReinstanceHotReloadedClassesDelegate is for internal use only")
	static COREUOBJECT_API TMulticastDelegate<void()> ReinstanceHotReloadedClassesDelegate;

	/** 
	 * Delegate invoked when requests are made to process pending UObject registrations.  This will be invoked during the loading of modules
	 * and outside of module load to register any delayed registrations.  This callback will be invoked regardless of if the module contained
	 * any UObject definitions.  The callback will also be invoked during module load in monolithic builds.
	 * 
	 * See ECompiledInUObjectsRegisteredStatus for more detailed information about the notifications.
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FCompiledInUObjectsRegisteredDelegate, FName /*Package*/, ECompiledInUObjectsRegisteredStatus Status);
	static COREUOBJECT_API FCompiledInUObjectsRegisteredDelegate CompiledInUObjectsRegisteredDelegate;

	/* Multicast delegate that is executed when any UObjects in packages need destroying before modules are unloaded. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FCompiledInUObjectsRemovedDelegate, TConstArrayView<UPackage*>);
	static COREUOBJECT_API FCompiledInUObjectsRemovedDelegate CompiledInUObjectsRemovedDelegate;

	/** Sent at the very beginning of LoadMap */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPreLoadMapDelegate, const FString& /* MapName */);
	static COREUOBJECT_API FPreLoadMapDelegate PreLoadMap;

	/** Sent at the very beginning of LoadMap */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FPreLoadMapWithContextDelegate, const FWorldContext& /*WorldContext*/, const FString& /* MapName */);
	static COREUOBJECT_API FPreLoadMapWithContextDelegate PreLoadMapWithContext;
	
	/** Sent at the end of LoadMap */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPostLoadMapDelegate, UWorld* /* LoadedWorld */);
	static COREUOBJECT_API FPostLoadMapDelegate PostLoadMapWithWorld;

	/** Sent when a network replay has started */
	static COREUOBJECT_API FSimpleMulticastDelegate PostDemoPlay;

	/** Called before garbage collection, before the GC lock is acquired. */
	static COREUOBJECT_API FSimpleMulticastDelegate& GetPreGarbageCollectDelegate();

	/** Called at the very beginning of garbage collection, once the GC lock is held. */
	static COREUOBJECT_API FSimpleMulticastDelegate& GetGarbageCollectStartedDelegate();

	/** Delegate type for reachability analysis external roots callback. First parameter is FGarbageCollectionTracer to use for tracing, second is flags with which objects should be kept alive regardless, third is whether to force single threading */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FTraceExternalRootsForReachabilityAnalysisDelegate, FGarbageCollectionTracer&, EObjectFlags, bool);

	/** Called after reachability analysis, before any purging */
	static COREUOBJECT_API FSimpleMulticastDelegate PostReachabilityAnalysis;

	/** Called after garbage collection (before purge phase if incremental purge is enabled and after purge phase if incremental purge is disabled) */
	static COREUOBJECT_API FSimpleMulticastDelegate& GetPostGarbageCollect();

	/** Called after purging unreachable objects during garbage collection */
	static COREUOBJECT_API FSimpleMulticastDelegate& GetPostPurgeGarbageDelegate();

#if !UE_BUILD_SHIPPING
	/** Called when garbage collection detects references to objects that are marked for explicit destruction by MarkAsGarbage */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnReportGarbageReferencers, TConstArrayView<struct FGarbageReferenceInfo>);
	static COREUOBJECT_API FOnReportGarbageReferencers& GetGarbageCollectReportGarbageReferencers();
#endif

	/** Called before ConditionalBeginDestroy phase of garbage collection */
	static COREUOBJECT_API FSimpleMulticastDelegate PreGarbageCollectConditionalBeginDestroy;

	/** Called after ConditionalBeginDestroy phase of garbage collection */
	static COREUOBJECT_API FSimpleMulticastDelegate PostGarbageCollectConditionalBeginDestroy;

	/** Called after garbage collection is complete, all objects have been purged (regardless of whether incremental purge is enabled or not), memory has been trimmed and all other GC callbacks have been fired. */
	static COREUOBJECT_API FSimpleMulticastDelegate GarbageCollectComplete;

	/** Queries whether an object should be loaded on top ( replace ) an already existing one */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnLoadObjectsOnTop, const FString&);
	static COREUOBJECT_API FOnLoadObjectsOnTop ShouldLoadOnTop;

	/** Called when path to world root is changed */
	DECLARE_MULTICAST_DELEGATE_OneParam(FPackageCreatedForLoad, class UPackage*);
	static COREUOBJECT_API FPackageCreatedForLoad PackageCreatedForLoad;

	/** Called when trying to figure out if a UObject is a primary asset, if it doesn't implement GetPrimaryAssetId itself */
	DECLARE_DELEGATE_RetVal_OneParam(FPrimaryAssetId, FGetPrimaryAssetIdForObject, const UObject*);
	static COREUOBJECT_API FGetPrimaryAssetIdForObject GetPrimaryAssetIdForObject;

	/** Called during cooking to see if a specific package should be cooked for a given target platform */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FShouldCookPackageForPlatform, const UPackage*, const ITargetPlatform*);
	static COREUOBJECT_API FShouldCookPackageForPlatform ShouldCookPackageForPlatform;
};

/** Allows release builds to override not verifying GC assumptions. Useful for profiling as it's hitchy. */
extern COREUOBJECT_API bool GShouldVerifyGCAssumptions;

/** If non-zero, the engine will create Garbage Collector clusters to speed up Garbage Collection */
extern COREUOBJECT_API int32 GCreateGCClusters;

/** If non-zero, the engine will attempt to create clusters from asset files */
extern COREUOBJECT_API int32 GAssetClustreringEnabled;

/** A struct used as stub for deleted ones. */
COREUOBJECT_API UScriptStruct* GetFallbackStruct();

/** Utility accessor for whether we are running with component class overrides enabled */
COREUOBJECT_API bool GetAllowNativeComponentClassOverrides();

namespace UE
{

class FAssetLog
{
public:
	inline explicit FAssetLog(const TCHAR* InPath UE_LIFETIMEBOUND) : Path(InPath) {}
	inline explicit FAssetLog(const FPackagePath& InPath UE_LIFETIMEBOUND) : PackagePath(&InPath) {}
	inline explicit FAssetLog(const UObject* InObject UE_LIFETIMEBOUND) : Object(InObject) {}

	friend void SerializeForLog(FCbWriter& Writer, const FAssetLog& AssetLog) { AssetLog.SerializeForLog(Writer); }

private:
	COREUOBJECT_API void SerializeForLog(FCbWriter& Writer) const;
	const TCHAR* Path = nullptr;
	const FPackagePath* PackagePath = nullptr;
	const UObject* Object = nullptr;
};

} // UE

namespace UE::Core::Private
{
COREUOBJECT_API void RecordAssetLog(
	const FName& CategoryName,
	ELogVerbosity::Type Verbosity,
	const FAssetLog& AssetLog,
	const FString& Message,
	const ANSICHAR* File,
	int32 Line);
} // UE::Core::Private

/**
 * FAssetMsg
 * This struct contains functions for asset-related messaging
 */
struct FAssetMsg
{
	/** Formats a path for the UE_ASSET_LOG macro */
	static COREUOBJECT_API FString FormatPathForAssetLog(const TCHAR* Path);

	/** Formats a path for the UE_ASSET_LOG macro */
	static COREUOBJECT_API FString FormatPathForAssetLog(const FPackagePath& Path);

	/** If possible, finds a path to the underlying asset for the provided object and formats it for the UE_ASSET_LOG macro */
	static COREUOBJECT_API FString FormatPathForAssetLog(const UObject* Object);

	static COREUOBJECT_API FString GetAssetLogString(const TCHAR* Path, const FString& Message);
	static COREUOBJECT_API FString GetAssetLogString(const FPackagePath& Path, const FString& Message);
	static COREUOBJECT_API FString GetAssetLogString(const UObject* Object, const FString& Message);
};

#define ASSET_LOG_FORMAT_STRING_ANSI "[AssetLog] %s: "
#define ASSET_LOG_FORMAT_STRING TEXT(ASSET_LOG_FORMAT_STRING_ANSI)

#if NO_LOGGING
	#define UE_ASSET_LOG(...)
#else
	/**
	 * A macro that outputs a formatted message to log with a canonical reference to an asset if a given logging category is active at a given verbosity level
	 * @param CategoryName name of the logging category
	 * @param Verbosity, verbosity level to test against
	 * @param Asset, Object or asset path to format
	 * @param Format, format text
	 */
	#define UE_ASSET_LOG(CategoryName, Verbosity, Asset, Format, ...) \
	{ \
		static_assert(TIsArrayOrRefOfTypeByPredicate<decltype(Format), TIsCharEncodingCompatibleWithTCHAR>::Value, "Formatting string must be a TCHAR array."); \
		static_assert((ELogVerbosity::Verbosity & ELogVerbosity::VerbosityMask) < ELogVerbosity::NumVerbosity && ELogVerbosity::Verbosity > 0, "Verbosity must be constant and in range."); \
		CA_CONSTANT_IF((ELogVerbosity::Verbosity & ELogVerbosity::VerbosityMask) <= ELogVerbosity::COMPILED_IN_MINIMUM_VERBOSITY && (ELogVerbosity::Warning & ELogVerbosity::VerbosityMask) <= FLogCategory##CategoryName::CompileTimeVerbosity) \
		{ \
			UE_LOG_EXPAND_IS_FATAL(Verbosity, UE_EMPTY, if (!CategoryName.IsSuppressed(ELogVerbosity::Verbosity))) \
			{ \
				::UE::Core::Private::RecordAssetLog(CategoryName.GetCategoryName(), ELogVerbosity::Verbosity, ::UE::FAssetLog(Asset), FString::Printf(Format, ##__VA_ARGS__), __FILE__, __LINE__); \
				UE_LOG_EXPAND_IS_FATAL(Verbosity, \
					{ \
						UE_DEBUG_BREAK_AND_PROMPT_FOR_REMOTE(); \
						FDebug::AssertFailed("", __FILE__, __LINE__, TEXT("%s: %s"), *FAssetMsg::FormatPathForAssetLog(Asset), *FString::Printf(Format, ##__VA_ARGS__)); \
						CA_ASSUME(false); \
					}, \
					UE_EMPTY \
				) \
			} \
		} \
	}
#endif // NO_LOGGING

#if WITH_EDITORONLY_DATA
namespace UE::SavePackageUtilities
{
enum class EEditorOnlyObjectResult
{
	Uninitialized,
	EditorOnly,
	NonEditorOnly,
};
}// namespace UE::SavePackageUtilities
#endif // WITH_EDITORONLY_DATA

#if WITH_EDITOR
/** 
 * Returns if true if the object is editor-only:
 * - it's a package marked as PKG_EditorOnly or inside one
 * or
 * - IsEditorOnly returns true
 * or
 * - if bCheckMarks is true, if it has the EditorOnly object mark
 * or 
 * - if bCheckRecursive is true, if its class, super struct, outer, or archetypes are editor only
 */
COREUOBJECT_API bool IsEditorOnlyObject(const UObject* InObject, bool bCheckRecursive = true);
COREUOBJECT_API bool IsEditorOnlyObject(const UObject* InObject, bool bCheckRecursive,
	TFunctionRef<UE::SavePackageUtilities::EEditorOnlyObjectResult(const UObject* Object)> LookupInCache,
	TFunctionRef<void(const UObject* Object, bool bEditorOnly)> AddToCache);

#endif //WITH_EDITOR

class FFieldClass;
struct FClassFunctionLinkInfo;
struct FCppClassTypeInfoStatic;

/** Property setter and getter wrapper function pointer */
typedef void (*SetterFuncPtr)(void* InContainer, const void* InValue);
typedef void (*GetterFuncPtr)(const void* InContainer, void* OutValue);

/// @cond DOXYGEN_IGNORE
namespace UECodeGen_Private
{
	enum class EPropertyGenFlags : uint8
	{
		None              = 0x00,

		// First 6 bits are the property type
		Byte              = 0x00,
		Int8              = 0x01,
		Int16             = 0x02,
		Int               = 0x03,
		Int64             = 0x04,
		UInt16            = 0x05,
		UInt32            = 0x06,
		UInt64            = 0x07,
		//                = 0x08,
		//                = 0x09,
		Float             = 0x0A,
		Double            = 0x0B,
		Bool              = 0x0C,
		SoftClass         = 0x0D,
		WeakObject        = 0x0E,
		LazyObject        = 0x0F,
		SoftObject        = 0x10,
		Class             = 0x11,
		Object            = 0x12,
		Interface         = 0x13,
		Name              = 0x14,
		Str               = 0x15,
		Array             = 0x16,
		Map               = 0x17,
		Set               = 0x18,
		Struct            = 0x19,
		Delegate          = 0x1A,
		InlineMulticastDelegate = 0x1B,
		SparseMulticastDelegate = 0x1C,
		Text              = 0x1D,
		Enum              = 0x1E,
		FieldPath         = 0x1F,
		LargeWorldCoordinatesReal = 0x20,
		Optional          = 0x21,
		VerseValue        = 0x22,
		Utf8Str           = 0x23,
		AnsiStr           = 0x24,
		VerseString       = 0x25,
		VerseCell         = 0x26,
		VerseType         = 0x27,
		VerseLegacyValue  = 0x28, // FVRestValueProperty with a legacy property

		// Property-specific flags
		NativeBool        = 0x40,
		ObjectPtr         = 0x40,

	};

	static_assert(std::is_same_v<int32,  int         >, "CoreUObject property system expects int32 to be an int");
	static_assert(std::is_same_v<uint32, unsigned int>, "CoreUObject property system expects uint32 to be an unsigned int");

	ENUM_CLASS_FLAGS(EPropertyGenFlags)

	// Value which masks out the type of combined EPropertyGenFlags.
	inline constexpr EPropertyGenFlags PropertyTypeMask = (EPropertyGenFlags)0x3F;

#if WITH_METADATA
	struct FMetaDataPairParam
	{
		const char* NameUTF8;
		const char* ValueUTF8;
	};
#endif

	struct FEnumeratorParam
	{
		const char*               NameUTF8;
		int64                     Value;
	};

// Macros with the standard field layouts
#define UE_PROPERTY_PARAMS_BASE_FIELDS \
	const char*					NameUTF8; \
	const char*					RepNotifyFuncUTF8; \
	EPropertyFlags				PropertyFlags; \
	EPropertyGenFlags			Flags; \
	SetterFuncPtr				SetterFunc; \
	GetterFuncPtr				GetterFunc; \
	uint16						ArrayDim;

#define UE_PROPERTY_PARAMS_BASE_WITHOUT_OFFSET_FIELDS \
	UE_PROPERTY_PARAMS_BASE_FIELDS

#define UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS \
	UE_PROPERTY_PARAMS_BASE_FIELDS \
	uint16						Offset;

#if WITH_METADATA
#define UE_PROPERTY_PARAMS_METADATA_FIELDS \
	uint16						NumMetaData; \
	const FMetaDataPairParam*	MetaDataArray;
#else
#define UE_PROPERTY_PARAMS_METADATA_FIELDS
#endif

	// This is not a base class but is just a common initial sequence of all of the F*PropertyParams types below.
	// We don't want to use actual inheritance because we want to construct aggregated compile-time tables of these things.
	struct FPropertyParamsBase
	{
		UE_PROPERTY_PARAMS_BASE_FIELDS;
	};

	struct FPropertyParamsBaseWithoutOffset // : FPropertyParamsBase
	{
		UE_PROPERTY_PARAMS_BASE_WITHOUT_OFFSET_FIELDS;
	};

	struct FPropertyParamsBaseWithOffset // : FPropertyParamsBase
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
	};

	struct FGenericPropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FBytePropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UEnum*					(*EnumFunc)(ETypeConstructPhase);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FBoolPropertyParams // : FPropertyParamsBaseWithoutOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITHOUT_OFFSET_FIELDS;
		uint16					ElementSize;
		uint16					SizeOfOuter;
		void					(*SetBitFunc)(void* Obj);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FObjectPropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UClass*					(*ClassFunc)(ETypeConstructPhase);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FClassPropertyParams // : FObjectPropertyParams
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UClass*					(*ClassFunc)(ETypeConstructPhase);
		UClass*					(*MetaClassFunc)(ETypeConstructPhase);		
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FVerseClassPropertyParams // : FClassPropertyParams
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UClass*					(*ClassFunc)(ETypeConstructPhase);
		UClass*					(*MetaClassFunc)(ETypeConstructPhase);
		bool					bRequiresConcrete;
		bool					bRequiresCastable;
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FSoftClassPropertyParams // : FObjectPropertyParamsWithoutClass
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UClass*					(*MetaClassFunc)(ETypeConstructPhase);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FInterfacePropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UClass*					(*InterfaceClassFunc)(ETypeConstructPhase);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FStructPropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UScriptStruct*			(*ScriptStructFunc)(ETypeConstructPhase);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FDelegatePropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UFunction*				(*SignatureFunctionFunc)(ETypeConstructPhase);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FMulticastDelegatePropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UFunction*				(*SignatureFunctionFunc)(ETypeConstructPhase);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FEnumPropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		UEnum*					(*EnumFunc)(ETypeConstructPhase);
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FFieldPathPropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		FFieldClass*			(*PropertyClassFunc)();
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FArrayPropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		EArrayPropertyFlags		ArrayFlags;
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	struct FMapPropertyParams // : FPropertyParamsBaseWithOffset
	{
		UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS;
		EMapPropertyFlags		MapFlags;
		UE_PROPERTY_PARAMS_METADATA_FIELDS;
	};

	// These property types don't add new any construction parameters to their base property
	typedef FGenericPropertyParams FInt8PropertyParams;
	typedef FGenericPropertyParams FInt16PropertyParams;
	typedef FGenericPropertyParams FIntPropertyParams;
	typedef FGenericPropertyParams FInt64PropertyParams;
	typedef FGenericPropertyParams FUInt16PropertyParams;
	typedef FGenericPropertyParams FUInt32PropertyParams;
	typedef FGenericPropertyParams FUInt64PropertyParams;
	typedef FGenericPropertyParams FFloatPropertyParams;
	typedef FGenericPropertyParams FDoublePropertyParams;
	typedef FGenericPropertyParams FLargeWorldCoordinatesRealPropertyParams;
	typedef FGenericPropertyParams FNamePropertyParams;
	typedef FGenericPropertyParams FStrPropertyParams;
	typedef FGenericPropertyParams FSetPropertyParams;
	typedef FGenericPropertyParams FTextPropertyParams;
	typedef FGenericPropertyParams FObjectPropertyParamsWithoutClass;
	typedef FObjectPropertyParams  FWeakObjectPropertyParams;
	typedef FObjectPropertyParams  FLazyObjectPropertyParams;
	typedef FObjectPropertyParams  FObjectPtrPropertyParams;
	typedef FClassPropertyParams   FClassPtrPropertyParams;
	typedef FObjectPropertyParams  FSoftObjectPropertyParams;
	typedef FGenericPropertyParams FUtf8StrPropertyParams;
	typedef FGenericPropertyParams FAnsiStrPropertyParams;
	typedef FGenericPropertyParams FVerseValuePropertyParams;
	typedef FGenericPropertyParams FVerseCellValuePropertyParams;
	typedef FGenericPropertyParams FVerseStringPropertyParams;

#undef UE_PROPERTY_PARAMS_BASE_FIELDS
#undef UE_PROPERTY_PARAMS_BASE_WITHOUT_OFFSET_FIELDS
#undef UE_PROPERTY_PARAMS_BASE_WITH_OFFSET_FIELDS
#undef UE_PROPERTY_PARAMS_METADATA_FIELDS

	struct FFunctionBaseParams
	{
		FTypeConstructFunc*					OuterFunc;
		UFunction*                        (*SuperFunc)(ETypeConstructPhase);
		const char*                         NameUTF8;
		const FPropertyParamsBase* const*   PropertyArray;
		uint16                              NumProperties;
		uint16                              StructureSize;
		EObjectFlags                        ObjectFlags;
		EFunctionFlags                      FunctionFlags;
		uint16                              RPCId;
		uint16                              RPCResponseId;
#if WITH_METADATA
		uint16                              NumMetaData;
		const FMetaDataPairParam*           MetaDataArray;
#endif
	};

	struct FFunctionParams : FFunctionBaseParams
	{
	};

	struct FDelegateFunctionParams : FFunctionBaseParams
	{
	};

	struct FSparseDelegateFunctionParams : FFunctionBaseParams
	{
		const char*                         OwningClassName;
		const char*							DelegateName;
	};

	struct FVerseFunctionParams : FFunctionBaseParams
	{
		const char*							AlternateName;
	};

	struct FEnumParams
	{
		FTypeConstructFunc*			OuterFunc;
		FText                     (*DisplayNameFunc)(int32);
		const char*                 NameUTF8;
		const char*                 CppTypeUTF8;
		const FEnumeratorParam*     EnumeratorParams;
		EObjectFlags                ObjectFlags;
		int16                       NumEnumerators;
		EEnumFlags                  EnumFlags;
		uint8                       CppForm; // this is of type UEnum::ECppForm
		uint8                       UnderlyingType; // this is of type UEnum::EUnderlyingType
#if WITH_METADATA
		uint16                      NumMetaData;
		const FMetaDataPairParam*   MetaDataArray;
#endif
	};

	struct FVerseEnumParams : FEnumParams
	{
		const char*					QaulifiedName;
		const FVniTypeDesc*			NativeTypeDesc;
	};

	struct FStructParams
	{
		FTypeConstructFunc*					OuterFunc;
		UScriptStruct*                    (*SuperFunc)(ETypeConstructPhase);
		void*                             (*StructOpsFunc)(); // really returns UScriptStruct::ICppStructOps*
		const char*                         NameUTF8;
		const FPropertyParamsBase* const*   PropertyArray;
		uint16                              NumProperties;
		uint16                              SizeOf;
		uint8                               AlignOf;
		EObjectFlags                        ObjectFlags;
		uint32                              StructFlags; // EStructFlags
#if WITH_METADATA
		uint16                              NumMetaData;
		const FMetaDataPairParam*           MetaDataArray;
#endif
	};

	struct FVerseStructParams : FStructParams
	{
		const char*							QaulifiedName;
		const FVniTypeDesc*					NativeTypeDesc;
		uint32								VerseClassFlags;
	};

	struct FPackageParams
	{
		const char*                        NameUTF8;
		FTypeConstructFunc* const*		   SingletonFuncArray;
		int32                              NumSingletons;
		uint32                             PackageFlags; // EPackageFlags
		uint32                             BodyCRC;
		uint32                             DeclarationsCRC;
#if WITH_METADATA
		uint16                             NumMetaData;
		const FMetaDataPairParam*          MetaDataArray;
#endif
	};

	struct FImplementedInterfaceParams
	{
		UClass* (*ClassFunc)(ETypeConstructPhase);
		int32     Offset;
		bool      bImplementedByK2;
		bool      bDirectInterface = false; // Used by verse
	};

	struct FClassParams
	{
		UClass*                                   (*ClassNoRegisterFunc)(ETypeConstructPhase);
		const char*                                 ClassConfigNameUTF8;
		const FCppClassTypeInfoStatic*              CppClassInfo;
		FTypeConstructFunc* const*					DependencySingletonFuncArray;
		const FClassFunctionLinkInfo*               FunctionLinkArray;
		const FPropertyParamsBase* const*           PropertyArray;
		const FImplementedInterfaceParams*          ImplementedInterfaceArray;
		uint32                                      NumDependencySingletons : 4;
		uint32                                      NumFunctions : 11;
		uint32                                      NumProperties : 11;
		uint32                                      NumImplementedInterfaces : 6;
		uint32                                      ClassFlags; // EClassFlags
#if WITH_METADATA
		uint16                                      NumMetaData;
		const FMetaDataPairParam*                   MetaDataArray;
#endif
	};

	struct FVerseClassParams : FClassParams
	{
		const char*									PackageRelativeVersePath;
		const char*									MangledPackageVersePath;
		const FVniTypeDesc*							NativeTypeDesc;
		uint32										VerseClassFlags;
	};

	COREUOBJECT_API void ConstructUFunction(UFunction** SingletonPtr, const FFunctionParams& Params);
	COREUOBJECT_API void ConstructUDelegateFunction(UFunction** SingletonPtr, const FDelegateFunctionParams& Params);
	COREUOBJECT_API void ConstructUSparseDelegateFunction(UFunction** SingletonPtr, const FSparseDelegateFunctionParams& Params);
	COREUOBJECT_API void ConstructUEnum(UEnum*& OutEnum, const FEnumParams& Params);
	COREUOBJECT_API void ConstructUScriptStruct(UScriptStruct*& OutStruct, const FStructParams& Params);
	COREUOBJECT_API void ConstructUPackage(UPackage*& OutPackage, const FPackageParams& Params);
	COREUOBJECT_API void ConstructUClass(UClass*& OutClass, const FClassParams& Params);
}
/// @endcond

// METADATA_PARAMS(x, y) expands to x, y, if WITH_METADATA is set, otherwise expands to nothing
#if WITH_METADATA
	#define METADATA_PARAMS(x, y) x, y,
#else
	#define METADATA_PARAMS(x, y)
#endif

// IF_WITH_METADATA expands to its contents if WITH_METADATA is set, otherwise expands to nothing
#if WITH_METADATA
	#define IF_WITH_METADATA(...) __VA_ARGS__
#else
	#define IF_WITH_METADATA(...)
#endif

// IF_WITH_EDITOR(x, y) expands to x if WITH_EDITOR is set, otherwise expands to y
#if WITH_EDITOR
	#define IF_WITH_EDITOR(x, y) x
#else
	#define IF_WITH_EDITOR(x, y) y
#endif

// IF_WITH_EDITORONLY_DATA(x, y) expands to x if WITH_EDITORONLY_DATA is set, otherwise expands to y
#if WITH_EDITORONLY_DATA
	#define IF_WITH_EDITORONLY_DATA(x, y) x
#else
	#define IF_WITH_EDITORONLY_DATA(x, y) y
#endif

// IF_WITH_VERSE_VM(x, y) expands to x if WITH_VERSE_VM is set, otherwise expands to y
#if WITH_VERSE_VM
#define IF_WITH_VERSE_VM(x, y) UE_REMOVE_OPTIONAL_PARENS(x)
#else
#define IF_WITH_VERSE_VM(x, y) UE_REMOVE_OPTIONAL_PARENS(y)
#endif

/** Enum used by DataValidation plugin to see if an asset has been validated for correctness */
enum class EDataValidationResult : uint8
{
	/** Asset has failed validation */
	Invalid,
	/** Asset has passed validation */
	Valid,
	/** Asset has not yet been validated */
	NotValidated
};

template<> COREUOBJECT_API UEnum* StaticEnum<EDataValidationResult>();

/**
 * Combines two different data validation results and returns the combined result.
 *
 * @param	Result1			One of the data validation results to be combined
 * @param	Result2			One of the data validation results to be combined
 *
 * @return	Returns the combined data validation result
 */
COREUOBJECT_API EDataValidationResult CombineDataValidationResults(EDataValidationResult Result1, EDataValidationResult Result2);

// Types used by both constinit and non-constinit UHT codegen
namespace UE::CodeGen
{
	struct FClassNativeFunction
	{
		const UTF8CHAR* NameUTF8;
		void (*Pointer)(UObject* Context, FFrame& TheStack, RESULT_DECL);
	};
}

#if UE_WITH_CONSTINIT_UOBJECT
class FStructBaseChain;
struct ICppStructOps;
struct FVerseCallableThunk;

namespace UE::CodeGen::ConstInit
{
	// Special tags for constructing classes such as UClass that are self-referential
	// i.e. the class of class is class
	enum class EClass { Class };
	enum class EObject { Object };

#if WITH_METADATA
	struct FMetaData
	{
		// NOTE: These should be changed to UTF8CHAR when UHT is modified to emit \u and \U
		// for unicode characters rather than \x
		const char* NameUTF8;
		const char* ValueUTF8;
	};
#endif

	// Parameter wrapper for FField* or UObject* which can be used as a parameter in a constexpr context
	// FFieldOwner cannot be used directly because we can't use the tagged pointer/union code in constexpr
	struct FFieldOwnerParam
	{
		union FValue
		{
			UObject* Object;
			FField* Field;
		} Value;
		bool bIsObject;

		constexpr FFieldOwnerParam()
		{
			Value.Object = nullptr;
			bIsObject = true;
		}
		// Implicit construction from UObject*
		constexpr FFieldOwnerParam(UObject* InObject)
		{
			Value.Object = InObject;
			bIsObject = true;
		}
		// Implicit construction from FField*
		constexpr FFieldOwnerParam(FField* InField)
		{
			Value.Field = InField;
			bIsObject = false;
		}
	};

	// Parameter wrapper for either an object pointer or an encoded reference to an object in another module to be linked
	// TCompiledInObjectPtr cannot be used directly because we can't use the tagged pointer/union code in constexpr
	template<typename T>
	struct TObjectRefParam
	{
		T* ObjectPtr = nullptr;
#if !IS_MONOLITHIC
		FCompiledInObjectPtr CompiledInPtr;
#endif // !IS_MONOLITHIC

		constexpr TObjectRefParam() = default;
		constexpr TObjectRefParam(const TObjectRefParam&) = default;
		constexpr TObjectRefParam& operator=(const TObjectRefParam&) = default;

		// These constructors are not explicit to simplify UHT generated code 
		constexpr TObjectRefParam(T* InPtr)
			: ObjectPtr(InPtr)
		{
		}

#if IS_MONOLITHIC
		constexpr bool IsUnset() const
		{
			return ObjectPtr == nullptr;
		}
#else // IS_MONOLITHIC
		constexpr TObjectRefParam(uint32 InModuleId, uint32 InObjectId)
			: CompiledInPtr(FCompiledInObjectReference{InModuleId, InObjectId})
		{
		}

		constexpr TObjectRefParam(FCompiledInObjectReference InRef)
			: CompiledInPtr(InRef)
		{
		}
		constexpr bool IsUnset() const
		{
			return ObjectPtr == nullptr && CompiledInPtr.IsUnset();
		}
#endif // IS_MONOLITHIC

		constexpr explicit operator bool() const 
		{
			return !IsUnset();
		}

		constexpr bool operator==(TYPE_OF_NULLPTR)
		{
			return IsUnset();
		}
		
		constexpr bool operator!=(TYPE_OF_NULLPTR)
		{
			return !IsUnset();
		}
	};

	using FObjectRefParam = TObjectRefParam<UObject>;

	// Shared parameters for all properties for constant-initialization
	struct FPropertyParams
	{
		FFieldOwnerParam 	Owner;
		FProperty* 	    	NextProperty; // Next property in this exact struct, no link to inherited properties
		const UTF8CHAR* NameUTF8;
		const UTF8CHAR* RepNotifyFuncUTF8;
		EPropertyFlags 		PropertyFlags;
		uint16         		ArrayDim;
		uint16         		Offset;
		int32				ElementSize;
#if WITH_METADATA
		TConstArrayView<FMetaData> MetaData;
#endif

		// Helper to initialize element size during delegating constructor call 
		consteval FPropertyParams& SetElementSize(int32 InSize)
		{
			ElementSize = InSize;
			return *this;
		}
	};


	// Param structures for initialization of constinit UObjects in generated code
	// Intended to be passed directly to consteval constructors as as aggregate initialization 
	// expressions using designated initializers and not stored in the binary as variables themselves
	struct FObjectParams	
	{
		EObjectFlags Flags 		= RF_NoFlags;
		// UObject* rather than UClass* because consteval constructors must be inlined and TObjectPtr
		// cannot see inheritance hierarchy in definition of UObjectBase
		FObjectRefParam Class;
		const UTF8CHAR* NameUTF8 = nullptr;
		FObjectRefParam Outer;

		consteval FObjectParams& SetClass(UObject* InClass)
		{ 
			Class.ObjectPtr = InClass;
			return *this;
		}
	};

	struct FPackageParams
	{
		FObjectParams Object;
		EPackageFlags Flags = PKG_None;
#if WITH_METADATA
		TConstArrayView<FMetaData> MetaData;
#endif
		uint32 BodiesHash = 0;
		uint32 DeclarationsHash = 0;
	};

	struct FUFieldParams
	{
		TObjectRefParam<UField> Next;
	};

	struct FStructParams
	{
		TObjectRefParam<UStruct> Super;
		UField* FirstChild; // No cross-module linking necessary 
		TArrayView<TCompiledInObjectPtr<const FStructBaseChain>> BaseChain;
		FField* ChildProperties;
		int32 PropertiesSize;
		int16 MinAlignment;
#if WITH_METADATA
		TConstArrayView<FMetaData> MetaData;
#endif
	};

	struct FClassImplementedInterface
	{
		TCompiledInObjectPtr<UClass> Class;
		int32 PointerOffset;
		bool bImplementedByK2;
		bool bVerseDirectInterface;
	};

	struct FClassParams
	{
		const TCHAR* ConfigName;
		void (*ClassConstructor)(const FObjectInitializer&);
		UObject* (*ClassVTableHelperCtorCaller)(FVTableHelper& Helper);
		FUObjectCppClassStaticFunctions CppClassStaticFunctions;
		EClassFlags ClassFlags;
		EClassCastFlags ClassCastFlags;
		const FCppClassTypeInfoStatic* CppClassTypeInfo;
		TConstArrayView<FClassNativeFunction> NativeFunctions;
		TObjectRefParam<UClass> ClassWithin;
		TArrayView<FClassImplementedInterface> Interfaces;
		TObjectRefParam<UScriptStruct> SparseClassDataStruct;

		consteval FClassParams& SetWithin(UClass* WithinClass)
		{ 
			ClassWithin.ObjectPtr = WithinClass;
			return *this;
		}
	};

	struct FVerseClassParams
	{
		const UTF8CHAR*	PackageRelativeVersePath;
		const UTF8CHAR*	MangledPackageVersePath;
		const FVniTypeDesc* NativeTypeDesc;
#if WITH_VERSE_VM
		TConstArrayView<FVerseCallableThunk> VerseCallableThunks;
#endif
		uint32 VerseClassFlags;
	};

	struct FVerseStructParams
	{
		uint32 GuidA;
		uint32 GuidB;
		uint32 GuidC;
		uint32 GuidD;
		const UTF8CHAR* QualifiedName;
		const FVniTypeDesc* NativeTypeDesc;
		uint32 VerseClassFlags;
	};

	struct FEnumParams
	{
		const UTF8CHAR* CppTypeStaticUTF8;
		TConstArrayView<const UTF8CHAR*> StaticNamesUTF8;
		TArrayView<int64> EnumValues;
		uint8 CppForm;
		uint8 UnderlyingType; // this is of type UEnum::EUnderlyingType
		EEnumFlags EnumFlags;
		FText (*DisplayNameFn)(int32);
#if WITH_METADATA
		TConstArrayView<FMetaData> MetaData;
#endif
	};

	struct FVerseEnumParams
	{
		const UTF8CHAR* QualifiedName;
		const FVniTypeDesc* NativeTypeDesc;
	};

	struct FFunctionParams
	{
		void (*NativeFunction)(UObject* Context, FFrame& TheStack, RESULT_DECL);
		EFunctionFlags FunctionFlags;
		int16 RPCId;
		int16 RPCResponseId;
	};

	struct FVerseFunctionParams
	{
		const UTF8CHAR* AlternateNameUTF8;
	};

	struct FSparseDelegateParams
	{
		const UTF8CHAR* SparseOwningClassNameUTF8;
		const UTF8CHAR* SparseDelegateNameUTF8;
	};
}
#endif // UE_WITH_CONSTINIT_UOBJECT
