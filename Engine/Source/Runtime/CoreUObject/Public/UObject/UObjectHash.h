// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectHash.h: Unreal object name hashes
=============================================================================*/

#pragma once

#include "Concepts/SameAs.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Stats/Stats.h"
#include "Templates/Function.h"
#include "UObject/FindObjectFlags.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectArray.h"
#include "UObject/UnrealNames.h"

class FOutputDevice;
class UClass;
class UObject;
class UPackage;

DECLARE_STATS_GROUP_VERBOSE(TEXT("UObject Hash"), STATGROUP_UObjectHash, STATCAT_Advanced);

#if !UE_BUILD_TEST && !UE_BUILD_SHIPPING
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("NumObjects"), STAT_Hash_NumObjects, STATGROUP_UObjectHash, COREUOBJECT_API);
#endif // !UE_BUILD_TEST && !UE_BUILD_SHIPPING

/**
 * Private internal version of StaticFindObjectFast that allows using 0 exclusion flags.
 *
 * @param	Class			The to be found object's class
 * @param	InOuter			The to be found object's outer
 * @param	InName			The to be found object's class
 * @param	ExactClass		Whether to require an exact match with the passed in class
 * @param	ExclusiveFlags	Ignores objects that contain any of the specified exclusive flags
 * @param	ExclusiveInternalFlags	Ignores objects that contain any of the specified internal exclusive flags
 * @return	Returns a pointer to the found object or NULL if none could be found
 */
COREUOBJECT_API UObject* StaticFindObjectFastInternal(const UClass* Class, const UObject* InOuter, FName InName, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObjectFastInternal")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObjectFastInternal(const UClass* Class, const UObject* InOuter, FName InName, ExactClassType bExactClass, EObjectFlags ExclusiveFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None)
{
	return StaticFindObjectFastInternal(Class, InOuter, InName, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExclusiveFlags, ExclusiveInternalFlags);
}

/**
 * Private internal version of StaticFindAllObjectsFast that allows using 0 exclusion flags.
 *
 * @param	OutFoundObjects	Array of objects matching the specified search parameters
 * @param	ObjectClass		The to be found object's class
 * @param	ObjectName		The to be found object's name
 * @param	bExactClass		Whether to require an exact match with the passed in class
 * @param	ExcludeFlags	Ignores objects that contain any of the specified exclusive flags
 * @param	ExclusiveInternalFlags	Ignores objects that contain any of the specified internal exclusive flags
 * @return	Returns true if any objects were found, false otherwise
 */
COREUOBJECT_API bool StaticFindAllObjectsFastInternal(TArray<UObject*>& OutFoundObjects, const UClass* ObjectClass, FName ObjectName, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExcludeFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindAllObjectsFastInternal")
UE_NODEBUG UE_FORCEINLINE_HINT bool StaticFindAllObjectsFastInternal(TArray<UObject*>& OutFoundObjects, const UClass* ObjectClass, FName ObjectName, ExactClassType bExactClass, EObjectFlags ExcludeFlags = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None)
{
	return StaticFindAllObjectsFastInternal(OutFoundObjects, ObjectClass, ObjectName, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExcludeFlags, ExclusiveInternalFlags);
}

/**
 * Private internal version of StaticFindFirstObject that allows using 0 exclusion flags.
 *
 * @param	ObjectClass		The to be found object's class
 * @param	ObjectName		The to be found object's name
 * @param	bExactClass		Whether to require an exact match with the passed in class
 * @param	ExcludeFlag		Ignores objects that contain any of the specified exclusive flags
 * @param	ExclusiveInternalFlags	Ignores objects that contain any of the specified internal exclusive flags
 * @return	Returns first object with the specified name and class, null if not found
 */
COREUOBJECT_API UObject* StaticFindFirstObjectFastInternal(const UClass* ObjectClass, FName ObjectName, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExcludeFlag = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindFirstObjectFastInternal")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindFirstObjectFastInternal(const UClass* ObjectClass, FName ObjectName, ExactClassType bExactClass, EObjectFlags ExcludeFlag = RF_NoFlags, EInternalObjectFlags ExclusiveInternalFlags = EInternalObjectFlags::None)
{
	return StaticFindFirstObjectFastInternal(ObjectClass, ObjectName, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExcludeFlag, ExclusiveInternalFlags);
}

/**
 * Variation of StaticFindObjectFast that uses explicit path.
 *
 * @param	ObjectClass		The to be found object's class
 * @param	ObjectName		The to be found object's name
 * @param	ObjectPathName	Full path name for the object to search for
 * @param	ExactClass		Whether to require an exact match with the passed in class
 * @param	ExclusiveFlags	Ignores objects that contain any of the specified exclusive flags
 * @return	Returns a pointer to the found object or NULL if none could be found
 */
UObject* StaticFindObjectFastExplicit(const UClass* ObjectClass, FName ObjectName, const FString& ObjectPathName, EFindObjectFlags Flags = EFindObjectFlags::None, EObjectFlags ExcludeFlags = RF_NoFlags);
template <UE::CSameAs<bool> ExactClassType>
UE_EXACTCLASS_BOOL_DEPRECATED("StaticFindObjectFastExplicit")
UE_NODEBUG UE_FORCEINLINE_HINT UObject* StaticFindObjectFastExplicit(const UClass* ObjectClass, FName ObjectName, const FString& ObjectPathName, ExactClassType bExactClass, EObjectFlags ExcludeFlags = RF_NoFlags)
{
	return StaticFindObjectFastExplicit(ObjectClass, ObjectName, ObjectPathName, bExactClass ? EFindObjectFlags::ExactClass : EFindObjectFlags::None, ExcludeFlags);
}

/**
 * Variation of StaticFindObjectFastInternal that uses remote id.
 *
 * @param	RemoteId				The to be found object's remote id
 * @param	InExclusiveFlags		Ignores objects that contain any of the specified exclusive flags
 * @param	InExlusiveInternalFlags	Ignores objects that contain any of the specified internal exclusive flags
 * @return	Returns a pointer to the found object or NULL if none could be found
 */
COREUOBJECT_API UObject* StaticFindObjectFastInternal(FRemoteObjectId RemoteId, EObjectFlags InExclusiveFlags = RF_NoFlags, EInternalObjectFlags InExlusiveInternalFlags = EInternalObjectFlags::None);

/**
 * Return all objects with a given outer
 *
 * @param	Outer						Outer to search for
 * @param	Results						Returned results
 * @param	Flags						Additional flags, see EGetObjectsFlags
 * @param	ExclusionFlags				Specifies flags to use as a filter for which objects to return
 * @param	ExclusiveInternalFlags	Specifies internal flags to use as a filter for which objects to return
 */
COREUOBJECT_API void GetObjectsWithOuter(const class UObjectBase* Outer, TArray<UObject*>& Results, EGetObjectsFlags Flags = EGetObjectsFlags::IncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None);

UE_INCLUDENESTEDOBJECTS_BOOL_DEPRECATED("GetObjectsWithOuter")
UE_NODEBUG UE_FORCEINLINE_HINT void GetObjectsWithOuter(const class UObjectBase* Outer, TArray<UObject*>& Results, bool bIncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None)
{
	GetObjectsWithOuter(Outer, Results, bIncludeNestedObjects ? EGetObjectsFlags::IncludeNestedObjects : EGetObjectsFlags::None, ExclusionFlags, ExclusionInternalFlags);
}

/**
 * Performs an operation on objects with a given outer, with the possibility to break iteration
 * Note that the operation must not modify the UObject hash maps so it can not create, rename or destroy UObjects.
 *
 * @param	Outer						Outer to search for
 * @param	Operation					Function to be called for each object, returning if we want to continue iteration or not
 * @param	Flags						Additional flags, see EGetObjectsFlags
 * @param	ExclusionFlags				Specifies flags to use as a filter for which objects to return
 * @param	ExclusiveInternalFlags	Specifies internal flags to use as a filter for which objects to return
 */
COREUOBJECT_API void ForEachObjectWithOuterBreakable(const class UObjectBase* Outer, TFunctionRef<bool(UObject*)> Operation, EGetObjectsFlags Flags = EGetObjectsFlags::IncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None);

UE_INCLUDENESTEDOBJECTS_BOOL_DEPRECATED("ForEachObjectWithOuterBreakable")
UE_NODEBUG UE_FORCEINLINE_HINT void ForEachObjectWithOuterBreakable(const class UObjectBase* Outer, TFunctionRef<bool(UObject*)> Operation, bool bIncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None)
{
	ForEachObjectWithOuterBreakable(Outer, Operation, bIncludeNestedObjects ? EGetObjectsFlags::IncludeNestedObjects : EGetObjectsFlags::None, ExclusionFlags, ExclusionInternalFlags);
}

/**
 * Performs an operation on all objects with a given outer
 * Note that the operation must not modify UObject hash maps so it can not create, rename or destroy UObjects.
 *
 * @param	Outer						Outer to search for
 * @param	Operation					Function to be called for each object
 * @param	Flags						Additional flags, see EGetObjectsFlags
 * @param	ExclusionFlags				Specifies flags to use as a filter for which objects to return
 * @param	ExclusiveInternalFlags	Specifies internal flags to use as a filter for which objects to return
 */
inline void ForEachObjectWithOuter(const class UObjectBase* Outer, TFunctionRef<void(UObject*)> Operation, EGetObjectsFlags Flags = EGetObjectsFlags::IncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None)
{
	ForEachObjectWithOuterBreakable(Outer, [Operation](UObject* Object) { Operation(Object); return true; }, Flags, ExclusionFlags, ExclusionInternalFlags);
}

UE_INCLUDENESTEDOBJECTS_BOOL_DEPRECATED("ForEachObjectWithOuter")
inline void ForEachObjectWithOuter(const class UObjectBase* Outer, TFunctionRef<void(UObject*)> Operation, bool bIncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None)
{
	ForEachObjectWithOuterBreakable(Outer, [Operation](UObject* Object) { Operation(Object); return true; }, bIncludeNestedObjects ? EGetObjectsFlags::IncludeNestedObjects : EGetObjectsFlags::None, ExclusionFlags, ExclusionInternalFlags);
}

/**
 * Find an objects with a given name and or class within an outer
 *
 * @param	Outer						Outer to search for
 * @param	ClassToLookFor				if NULL, ignore this parameter, otherwise require the returned object have this class
 * @param	NameToLookFor				if NAME_None, ignore this parameter, otherwise require the returned object have this name
 */
COREUOBJECT_API class UObjectBase* FindObjectWithOuter(const class UObjectBase* Outer, const class UClass* ClassToLookFor = nullptr, FName NameToLookFor = NAME_None);

/**
 * Returns an array of all objects found within a given package
 *
 * @param	Package						Package to search into
 * @param	Results						Array to put the results
 * @param	Flags						Additional flags, see EGetObjectsFlags
 * @param	ExclusionFlags				Specifies flags to use as a filter for which objects to return
 * @param	ExclusiveInternalFlags		Specifies internal flags to use as a filter for which objects to return
 */
COREUOBJECT_API void GetObjectsWithPackage(const class UPackage* Outer, TArray<UObject*>& Results, EGetObjectsFlags Flags = EGetObjectsFlags::IncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None);

UE_INCLUDENESTEDOBJECTS_BOOL_DEPRECATED("GetObjectsWithPackage")
UE_NODEBUG UE_FORCEINLINE_HINT void GetObjectsWithPackage(const class UPackage* Outer, TArray<UObject*>& Results, bool bIncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None)
{
	GetObjectsWithPackage(Outer, Results, bIncludeNestedObjects ? EGetObjectsFlags::IncludeNestedObjects : EGetObjectsFlags::None, ExclusionFlags, ExclusionInternalFlags);
}

/**
 * Performs an operation on all objects found within a given package
 * Note that the operation must not modify UObject hash maps so it can not create, rename or destroy UObjects.
 *
 * @param	Package						Package to iterate into
 * @param	Operation					Function to be called for each object, return false to break out of the iteration
 * @param	Flags						Additional flags, see EGetObjectsFlags
 * @param	ExclusionFlags				Specifies flags to use as a filter for which objects to return
 * @param	ExclusiveInternalFlags		Specifies internal flags to use as a filter for which objects to return
 */
COREUOBJECT_API void ForEachObjectWithPackage(const class UPackage* Outer, TFunctionRef<bool(UObject*)> Operation, EGetObjectsFlags Flags = EGetObjectsFlags::IncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None);

UE_INCLUDENESTEDOBJECTS_BOOL_DEPRECATED("ForEachObjectWithPackage")
UE_NODEBUG UE_FORCEINLINE_HINT void ForEachObjectWithPackage(const class UPackage* Outer, TFunctionRef<bool(UObject*)> Operation, bool bIncludeNestedObjects, EObjectFlags ExclusionFlags = RF_NoFlags, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None)
{
	ForEachObjectWithPackage(Outer, Operation, bIncludeNestedObjects ? EGetObjectsFlags::IncludeNestedObjects : EGetObjectsFlags::None, ExclusionFlags, ExclusionInternalFlags);
}

/**
 * Returns an array of objects of a specific class. Optionally, results can include objects of derived classes as well.
 *
 * @param	ClassToLookFor				Class of the objects to return.
 * @param	Results						An output list of objects of the specified class.
 * @param	bIncludeDerivedClasses		If true, the results will include objects of child classes as well.
 * @param	AdditionalExcludeFlags		Objects with any of these flags will be excluded from the results.
 * @param	ExclusiveInternalFlags	Specifies internal flags to use as a filter for which objects to return
 */
COREUOBJECT_API void GetObjectsOfClass(const UClass* ClassToLookFor, TArray<UObject *>& Results, bool bIncludeDerivedClasses = true, EObjectFlags ExcludeFlags = RF_ClassDefaultObject, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None);

/**
 * Performs an operation on all objects of the provided class
 * Note that the operation must not modify UObject hash maps so it can not create, rename or destroy UObjects.
 *
 * @param	Outer						UObject class to loop over instances of
 * @param	Operation					Function to be called for each object
 * @param	bIncludeDerivedClasses		If true, the results will include objects of child classes as well.
 * @param	AdditionalExcludeFlags		Objects with any of these flags will be excluded from the results.
 */
COREUOBJECT_API void ForEachObjectOfClass(const UClass* ClassToLookFor, TFunctionRef<void(UObject*)> Operation, bool bIncludeDerivedClasses = true, EObjectFlags ExcludeFlags = RF_ClassDefaultObject, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None);

/**
 * Performs an operation on all objects of the provided classes
 * Note that the operation must not modify UObject hash maps so it can not create, rename or destroy UObjects.
 *
 * @param	Classes						UObject Classes to loop over instances of
 * @param	Operation					Function to be called for each object
 * @param	bIncludeDerivedClasses		If true, the results will include objects of child classes as well.
 * @param	AdditionalExcludeFlags		Objects with any of these flags will be excluded from the results.
 */
COREUOBJECT_API void ForEachObjectOfClasses(TArrayView<const UClass* const> ClassesToLookFor, TFunctionRef<void(UObject*)> Operation, EObjectFlags ExcludeFlags = RF_ClassDefaultObject, EInternalObjectFlags ExclusionInternalFlags = EInternalObjectFlags::None);

/**
 * Returns an array of classes that were derived from the specified class.
 *
 * @param	ClassToLookFor				The parent class of the classes to return.
 * @param	Results						An output list of child classes of the specified parent class.
 * @param	bRecursive					If true, the results will include children of the children classes, recursively. Otherwise, only direct decedents will be included.
 */
COREUOBJECT_API void GetDerivedClasses(const UClass* ClassToLookFor, TArray<UClass *>& Results, bool bRecursive = true);

/** Get all base classes and their direct subclasses */
COREUOBJECT_API TMap<UClass*, TSet<UClass*>> GetAllDerivedClasses();

/**
 * Returns true if any instances of the class in question are currently being async loaded.
 *
 * @param	ClassToLookFor				The class in question
 * @return	True if there are any instances of the class being async loaded - includes instances based on derived classes. Otherwise, false
 */
COREUOBJECT_API bool ClassHasInstancesAsyncLoading(const UClass* ClassToLookFor);

/**
 * Add an object to the name hash tables
 *
 * @param	Object		Object to add to the hash tables
 */
void HashObject(class UObjectBase* Object);
/**
 * Remove an object to the name hash tables
 *
 * @param	Object		Object to remove from the hash tables
 */
void UnhashObject(class UObjectBase* Object);


/**
 * Assign an external package directly to an object in the hash tables
 * @param Object	Object to assign a package to
 * @param Package	Package to assign, null will call UnhashObjectExternalPackage
 */
void HashObjectExternalPackage(class UObjectBase* Object, class UPackage* Package);

/**
 * Assign an external package directly to an object in the hash tables
 * @param Object	Object to unassign a package from
 */
void UnhashObjectExternalPackage(class UObjectBase* Object);

/**
 * Return the assigned external package of an object, if any
 * @param Object	Object to get the external package of
 * @return the assigned external package if any
 */
UPackage* GetObjectExternalPackageThreadSafe(const class UObjectBase* Object);

/**
 * Return the assigned external package of an object, if any
 * @param Object	Object to get the external package of
 * @return the assigned external package if any
 * @note DO NOT USE, only for internal GC reference collecting
 */
UPackage* GetObjectExternalPackageInternal(const class UObjectBase* Object);

/**
* Shrink the UObject hash tables
*/
COREUOBJECT_API void ShrinkUObjectHashTables();

/**
* Get a version number representing the current state of registered classes.
*
* Can be stored and then compared to invalidate external caching of classes hierarchy whenever it changes. 
*/
COREUOBJECT_API uint64 GetRegisteredClassesVersionNumber();

/**
* Get a version number representing the current state of registered native classes.
*
* Can be stored and then compared to invalidate external caching of native classes hierarchy whenever it changes.
*/
COREUOBJECT_API uint64 GetRegisteredNativeClassesVersionNumber();

/**
 * Logs out information about the object hash for debug purposes
 *
 * @param Ar the archive to write the log data to
 * @param bShowHashBucketCollisionInfo whether to log each bucket's collision count
 */
COREUOBJECT_API void LogHashStatistics(FOutputDevice& Ar, const bool bShowHashBucketCollisionInfo);

/**
 * Logs out information about the outer object hash for debug purposes
 *
 * @param Ar the archive to write the log data to
 * @param bShowHashBucketCollisionInfo whether to log each bucket's collision count
 */
COREUOBJECT_API void LogHashOuterStatistics(FOutputDevice& Ar, const bool bShowHashBucketCollisionInfo);

enum class EObjectMemoryOverheadOptions
{
	None,
	ShowIndividualStats = 1 << 0,
	IncludeReflectionData = 1 << 1
};
ENUM_CLASS_FLAGS(EObjectMemoryOverheadOptions)

/**
 * Logs out information about the total object hash memory usage for debug purposes
 *
 * @param Ar the archive to write the log data to
 * @param InOptions Logging options
 */
COREUOBJECT_API void LogHashMemoryOverheadStatistics(FOutputDevice& Ar, const EObjectMemoryOverheadOptions InOptions);

/**
 * Locks UObject hash tables so that other threads can't hash or find new UObjects 
 */
void LockUObjectHashTables();
/**
 * Unlocks UObject hash tables
 */
void UnlockUObjectHashTables();

/** Helper class for scoped hash tables lock */
class FScopedUObjectHashTablesLock
{
public:
	UE_FORCEINLINE_HINT FScopedUObjectHashTablesLock()
	{
		LockUObjectHashTables();
	}
	UE_FORCEINLINE_HINT ~FScopedUObjectHashTablesLock()
	{
		UnlockUObjectHashTables();
	}
};
