// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectIterator.h: High level iterators for uobject
=============================================================================*/

#pragma once

#include "CoreMinimal.h"
#include "UObject/GarbageCollectionGlobals.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectArray.h"
#include "UObject/ObjectVisibility.h"
#include "UObject/Object.h"
#include "UObject/Class.h"

/**
 * Class for iterating through all objects, including class default objects, unreachable objects...all UObjects
 */
class FRawObjectIterator : public FUObjectArray::TIterator
{
public:
	/**
	 * Constructor
	 * @param	bOnlyGCedObjects	if true, skip all of the permanent objects
	 */
	FRawObjectIterator(bool bOnlyGCedObjects = false) :
		FUObjectArray::TIterator( GUObjectArray, bOnlyGCedObjects )
	{
	}
	/**
	 * Iterator dereference
	 * @return	the object pointer pointed at by the iterator
	 */
	UE_FORCEINLINE_HINT FUObjectItem* operator*() const
	{
		// casting UObjectBase to UObject for clients
		return GetObject();
	}
	/**
	 * Iterator dereference
	 * @return	the object pointer pointed at by the iterator
	 */
	UE_FORCEINLINE_HINT FUObjectItem* operator->() const
	{
		return GetObject();
	}
};



enum class EObjectIteratorThreadSafetyOptions : uint8
{
	None = 0,
	ThreadSafe = 1, // Use only with function local iterators. Persistent iterators may prevent (hang) UObject creation on worker threads otherwise.
	ThreadSafeAdvance = 2 // Can be used for global iterators but locks the global UObjectArray each time the iterator advances to the next object (can be slow but does not result in deadlocks).
};

inline EInternalObjectFlags GetObjectIteratorDefaultInternalExclusionFlags(EInternalObjectFlags InternalExclusionFlags)
{
	return InternalExclusionFlags | EInternalObjectFlags::Unreachable | EInternalObjectFlags::AutoRTFMConstructionAborted | EInternalObjectFlags::PendingConstruction | UE::GetAsyncLoadingInternalFlagsExclusion()
#if UE_WITH_REMOTE_OBJECT_HANDLE
		| EInternalObjectFlags::Remote
#endif
		;
}

/**
 * Class for iterating through all objects, including class default objects.
 * Note that when Playing In Editor, this will find objects in the
 * editor as well as the PIE world, in an indeterminate order.
 * IteratorThreadSafety	template parameter manages how this class handles thread safety of UObject iteration.
 */
template <EObjectIteratorThreadSafetyOptions IteratorThreadSafety>
class TObjectIteratorBase : public FUObjectArray::TIterator
{
protected:

	/** Wrapper around FUObjectArray::TIterator::Advance that locks the FUObjectArray mutex before advancing if necessary */
	inline bool AdvanceIterator()
	{
		bool bResult = false;
		if (IteratorThreadSafety != EObjectIteratorThreadSafetyOptions::ThreadSafeAdvance)
		{
			bResult = Advance();
		}
		else
		{
			GetIteratedArray().LockInternalArray();
			bResult = Advance();
			GetIteratedArray().UnlockInternalArray();
		}
		return bResult;
	}

public:

	/**
	 * Constructor
	 *
	 * @param	InClass						return only object of the class or a subclass
	 * @param	bOnlyGCedObjects			if true, skip all of the permanent objects
	 * @param	AdditionalExclusionFlags	RF_* flags that should not be included in results
	 * @param	InInternalExclusionFlags	EInternalObjectFlags flagged objects that should not be included in results
	 */
	explicit TObjectIteratorBase(UClass* InClass = UObject::StaticClass(), bool bOnlyGCedObjects = false, EObjectFlags AdditionalExclusionFlags = RF_NoFlags, EInternalObjectFlags InInternalExclusionFlags = EInternalObjectFlags::None)
		: FUObjectArray::TIterator(GUObjectArray, bOnlyGCedObjects)
		, Class(InClass)
		, ExclusionFlags(AdditionalExclusionFlags)
		, InternalExclusionFlags(GetObjectIteratorDefaultInternalExclusionFlags(InInternalExclusionFlags))
	{
		check(Class);

		if (IteratorThreadSafety == EObjectIteratorThreadSafetyOptions::ThreadSafe)
		{
			GetIteratedArray().LockInternalArray();
		}

		do
		{
			UObject* Object = **this;
			if (!(Object->HasAnyFlags(ExclusionFlags) || Object->HasAnyInternalFlags(InternalExclusionFlags) || (Class != UObject::StaticClass() && !Object->IsA(Class))))
			{
				break;
			}
		} while (AdvanceIterator());
	}

	/**
	 * Constructor
	 *
	 * @param	Begin	The iterator to get the end iterator of.
	 */
	TObjectIteratorBase(FUObjectArray::TIterator::EEndTagType, const TObjectIteratorBase& Begin)
		: FUObjectArray::TIterator(FUObjectArray::TIterator::EndTag, Begin)
		, Class(Begin.Class)
		, ExclusionFlags(Begin.ExclusionFlags)
		, InternalExclusionFlags(Begin.InternalExclusionFlags)
	{
		if (IteratorThreadSafety == EObjectIteratorThreadSafetyOptions::ThreadSafe)
		{
			GetIteratedArray().LockInternalArray();
		}
	}

	/**
	 * Copy Constructor
	 */
	TObjectIteratorBase(const TObjectIteratorBase& Other)
		: FUObjectArray::TIterator(Other)
		, Class(Other.Class)
		, ExclusionFlags(Other.ExclusionFlags)
		, InternalExclusionFlags(Other.InternalExclusionFlags)
	{
		if (IteratorThreadSafety == EObjectIteratorThreadSafetyOptions::ThreadSafe)
		{
			GetIteratedArray().LockInternalArray();
		}
	}

	/**
	 * Destructor
	 */
	~TObjectIteratorBase()
	{
		if (IteratorThreadSafety == EObjectIteratorThreadSafetyOptions::ThreadSafe)
		{
			GetIteratedArray().UnlockInternalArray();
		}
	}

	/**
	 * Iterator advance
	 */
	void operator++()
	{
		//@warning: behavior is partially mirrored in UnObjGC.cpp. Make sure to adapt code there as well if you make changes below.
		// verify that the async loading exclusion flag still matches (i.e. we didn't start/stop async loading within the scope of the iterator)
		checkSlow(IsInAsyncLoadingThread() || int32(InternalExclusionFlags & EInternalObjectFlags_AsyncLoading));

		while (AdvanceIterator())
		{
			UObject* Object = **this;
			if (!(Object->HasAnyFlags(ExclusionFlags) || (Class != UObject::StaticClass() && !Object->IsA(Class)) || Object->HasAnyInternalFlags(InternalExclusionFlags)))
			{
				break;
			}
		}
	}
	/**
	 * Iterator dereference
	 * @return	the object pointer pointed at by the iterator
	 */
	inline UObject* operator*() const
	{
		// casting UObjectBase to UObject for clients
		FUObjectItem* ObjectItem = GetObject();
		return (UObject*)(ObjectItem ? ObjectItem->GetObject() : nullptr);
	}
	/**
	 * Iterator dereference
	 * @return	the object pointer pointed at by the iterator
	 */
	inline UObject* operator->() const
	{
		FUObjectItem* ObjectItem = GetObject();
		return (UObject*)(ObjectItem ? ObjectItem->GetObject() : nullptr);
	}
private:
	/** Class to restrict results to */
	UClass* Class;
protected:
	/** Flags that returned objects must not have */
	EObjectFlags ExclusionFlags;
	/** Internal Flags that returned objects must not have */
	EInternalObjectFlags InternalExclusionFlags;
};

/**
 * Class for iterating through all objects, including class default objects.
 * Note that when Playing In Editor, this will find objects in the
 * editor as well as the PIE world, in an indeterminate order.
 * This iterator does not lock the gloabl FUObjectArray when iterating over it which may result in race conditions.
 */
typedef TObjectIteratorBase<EObjectIteratorThreadSafetyOptions::None> FUnsafeObjectIterator;

/**
 * Class for iterating through all objects, including class default objects.
 * Note that when Playing In Editor, this will find objects in the
 * editor as well as the PIE world, in an indeterminate order.
 * Locks the global FUObjectArray when iterating over it.
 * Should only be used with function local iterator instances.
 */
typedef TObjectIteratorBase<EObjectIteratorThreadSafetyOptions::ThreadSafe> FThreadSafeObjectIterator;

/**
 * Class for iterating through all objects, including class default objects.
 * Note that when Playing In Editor, this will find objects in the
 * editor as well as the PIE world, in an indeterminate order.
 * Locks the global FUObjectArray when advancing to the next object.
 * Should only be used with persistent iterator instances that exist for more than a frame.
 */
typedef TObjectIteratorBase<EObjectIteratorThreadSafetyOptions::ThreadSafeAdvance> FPersistentThreadSafeObjectIterator;

/**
 * Class for iterating through all objects which inherit from a
 * specified base class.  Does not include any class default objects.
 * Note that when Playing In Editor, this will find objects in the
 * editor as well as the PIE world, in an indeterminate order.
 */
template< class T > class TObjectIterator
{
public:
	enum EEndTagType
	{
		EndTag
	};

	/**
	 * Constructor
	 */
	explicit TObjectIterator(EObjectFlags AdditionalExclusionFlags = RF_ClassDefaultObject, bool bIncludeDerivedClasses = true, EInternalObjectFlags InInternalExclusionFlags = EInternalObjectFlags::None)
		: Index(-1)
	{
		GetObjectsOfClass(T::StaticClass(), ObjectArray, bIncludeDerivedClasses, AdditionalExclusionFlags, GetObjectIteratorDefaultInternalExclusionFlags(InInternalExclusionFlags));
		Advance();
	}

	/**
	* Constructor
	*/
	TObjectIterator(EEndTagType, const TObjectIterator& Begin)
		: Index(Begin.ObjectArray.Num())
	{
	}

	/**
	 * Iterator advance
	 */
	UE_FORCEINLINE_HINT void operator++()
	{
		Advance();
	}

	/** Conversion to "bool" returning true if the iterator is valid. */
	UE_FORCEINLINE_HINT explicit operator bool() const
	{ 
		return ObjectArray.IsValidIndex(Index); 
	}
	/** Conversion to "bool" returning true if the iterator is valid. */
	UE_FORCEINLINE_HINT bool operator !() const 
	{
		return !(bool)*this;
	}

	/**
	 * Iterator dereference
	 * @return	the object pointer pointed at by the iterator
	 */
	FUNCTION_NON_NULL_RETURN_START
		UE_FORCEINLINE_HINT T* operator* () const
	FUNCTION_NON_NULL_RETURN_END
	{
		return (T*)GetObject();
	}
	/**
	 * Iterator dereference
	 * @return	the object pointer pointed at by the iterator
	 */
	FUNCTION_NON_NULL_RETURN_START
		UE_FORCEINLINE_HINT T* operator-> () const
	FUNCTION_NON_NULL_RETURN_END
	{
		return (T*)GetObject();
	}

	UE_FORCEINLINE_HINT bool operator==(const TObjectIterator& Rhs) const { return Index == Rhs.Index; }
	UE_FORCEINLINE_HINT bool operator!=(const TObjectIterator& Rhs) const { return Index != Rhs.Index; }

protected:
	/**
	 * Dereferences the iterator with an ordinary name for clarity in derived classes
	 *
	 * @return	the UObject at the iterator
	 */
	UE_FORCEINLINE_HINT UObject* GetObject() const 
	{ 
		return ObjectArray[Index];
	}
	
	/**
	 * Iterator advance with ordinary name for clarity in subclasses
	 * @return	true if the iterator points to a valid object, false if iteration is complete
	 */
	inline bool Advance()
	{
		//@todo UE check this for LHS on Index on consoles
		while(++Index < ObjectArray.Num())
		{
			if (GetObject())
			{
				return true;
			}
		}
		return false;
	}

protected:
	/** Results from the GetObjectsOfClass query */
	TArray<UObject*> ObjectArray;
	/** index of the current element in the object array */
	int32 Index;
};

/** specialization for T == UObject that does not call IsA() unnecessarily */
template<> class TObjectIterator<UObject> : public FThreadSafeObjectIterator
{
public:

	/**
	 * Constructor
	 *
	 * @param	AdditionalExclusionFlags	RF_* flags that should not be included in results
	 * @param	InInternalExclusionFlags	EInternalObjectFlags flagged objects that should not be included in results
	 */
	explicit TObjectIterator(EObjectFlags AdditionalExclusionFlags = RF_ClassDefaultObject, bool bIncludeDerivedClasses = true, EInternalObjectFlags InInternalExclusionFlags = EInternalObjectFlags::None)
		: FThreadSafeObjectIterator(UObject::StaticClass(), false, AdditionalExclusionFlags, GetObjectIteratorDefaultInternalExclusionFlags(InInternalExclusionFlags))
	{
	}

	/**
	 * Constructor
	 *
	 * @param	bOnlyGCedObjects			if true, skip all of the permanent objects
	 */
	explicit TObjectIterator(bool bOnlyGCedObjects) :
		FThreadSafeObjectIterator(UObject::StaticClass(), bOnlyGCedObjects, RF_ClassDefaultObject)
	{
	}

	/**
	 * Constructor
	 *
	 * @param	Begin	The iterator to get the end iterator of.
	 */
	TObjectIterator(TObjectIteratorBase::EEndTagType, const TObjectIterator& Begin) :
		FThreadSafeObjectIterator(FUObjectArray::TIterator::EndTag, Begin)
	{
	}

	/**
	 * Iterator advance
	 */
	void operator++()
	{
		// verify that the async loading exclusion flag still matches (i.e. we didn't start/stop async loading within the scope of the iterator)
		checkSlow(IsInAsyncLoadingThread() || int32(InternalExclusionFlags & EInternalObjectFlags_AsyncLoading));
		while (AdvanceIterator())
		{
			if (!(*this)->HasAnyFlags(ExclusionFlags) && !(*this)->HasAnyInternalFlags(InternalExclusionFlags))
			{
				break;
			}
		}
	}
};

template <typename T>
struct TObjectRange
{
	TObjectRange(EObjectFlags AdditionalExclusionFlags = RF_ClassDefaultObject, bool bIncludeDerivedClasses = true, EInternalObjectFlags InInternalExclusionFlags = EInternalObjectFlags::None)
	: Begin(AdditionalExclusionFlags, bIncludeDerivedClasses, InInternalExclusionFlags)
	{
	}

	friend TObjectIterator<T> begin(const TObjectRange& Range) { return Range.Begin; }
	friend TObjectIterator<T> end  (const TObjectRange& Range) { return TObjectIterator<T>(TObjectIterator<T>::EndTag, Range.Begin); }

	TObjectIterator<T> Begin;
};

template <>
struct TObjectRange<UObject>
{
	explicit TObjectRange(EObjectFlags AdditionalExclusionFlags = RF_ClassDefaultObject, bool bIncludeDerivedClasses = true, EInternalObjectFlags InInternalExclusionFlags = EInternalObjectFlags::None)
	: Begin(AdditionalExclusionFlags, bIncludeDerivedClasses, InInternalExclusionFlags)
	{
	}

	explicit TObjectRange(bool bOnlyGCedObjects)
		: Begin(bOnlyGCedObjects)
	{
	}

	friend TObjectIterator<UObject> begin(const TObjectRange& Range) { return Range.Begin; }
	friend TObjectIterator<UObject> end  (const TObjectRange& Range) { return TObjectIterator<UObject>(TObjectIterator<UObject>::EndTag, Range.Begin); }

	TObjectIterator<UObject> Begin;
};

