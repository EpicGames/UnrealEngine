// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UObjectMark.h: Unreal object marks
=============================================================================*/

#pragma once

#include "Containers/Array.h"

class UObject;

/** 
 * Object marks are bits associated with UObject. Marks are:
 * - Never saved
 * - Temporary and usually not valid outside of the scope of some function
 * - Low performance (!) Careful here, do not use marks for performance critical things.
 * - Public in the sense that the thing marking the objects is treating objects generically and is often not a uobject
 *
 * This is primarily a legacy support subsystem. Unwinding the scope, lifetime and relationship between marks was not 
 * feasible, so the legacy marks are all grouped here.
 * New mark-like things should probably just be new custom annotations.
 *
 * An alternative design would have been to incorporate marks into the UObjectArray.
 * That would allow high performance iteration, and could save memory if the marks are not sparse.
 * This design was chosen to avoid coupling.
 *
 **/

/* Here are the legacy flags that were converted to marks; listed here for find in files 
#define RF_Saved					DECLARE_UINT64(0x0000000080000000)		// Object has been saved via SavePackage. Temporary.
#define	RF_TagImp					DECLARE_UINT64(0x0000000800000000)		// Temporary import tag in load/save.
#define RF_TagExp					DECLARE_UINT64(0x0000001000000000)		// Temporary export tag in load/save.
#define RF_NotForClient				DECLARE_UINT64(0x0010000000000000)		// LT	Don't load this object for the game client.      These are always overwritten on save based on the virtual methods.
#define RF_NotForServer				DECLARE_UINT64(0x0020000000000000)		// LT	Don't load this object for the game server.		The base virtual methods always just look at the flag so leave things unchanged.
*/

/** 
 * Enumeration for the available object marks.
 * It is strongly advised that you do NOT add to this list, but rather make a new object annotation for your needs
 * The reason is that the relationship, lifetime, etc of these marks is often unrelated, but given the legacy code
 * it is really hard to tell. We don't want to replicate that confusing situation going forward.
 **/

enum EObjectMark
{
	OBJECTMARK_NOMARKS						= 0x00000000,		// Zero, nothing marked
	OBJECTMARK_Saved						= 0x00000004,		// Object has been saved via SavePackage. Temporary.
	OBJECTMARK_TagImp						= 0x00000008,		// Temporary import tag in load/save.
	OBJECTMARK_TagExp						= 0x00000010,		// Temporary export tag in load/save.
	OBJECTMARK_NotForClient					= 0x00000020,		// Temporary save tag for client load flag.
	OBJECTMARK_NotForServer					= 0x00000040,		// Temporary save tag for server load flag.
	OBJECTMARK_NotAlwaysLoadedForEditorGame	= 0x00000080,		// Temporary save tag for editorgame load flag.
	OBJECTMARK_EditorOnly					= 0x00000100,		// Temporary editor only flag
	OBJECTMARK_NotForTargetPlatform			= 0x00000200,		// Temporary save tag for stripping objets based on TargetPlatform 
	// Set of marks to inherit up the outer chain, etc
	OBJECTMARK_INHERITEDMARKS				= OBJECTMARK_NotForClient | OBJECTMARK_NotForServer | OBJECTMARK_EditorOnly | OBJECTMARK_NotForTargetPlatform,
	OBJECTMARK_ALLMARKS						= 0xffffffff,		// -1, all possible marks
};

/**
 * Adds marks to an object
 *
 * @param	Object	Object to add marks to
 * @param	Marks	Logical OR of OBJECTMARK_'s to apply 
 */
COREUOBJECT_API void MarkObject(const class UObjectBase* Object, EObjectMark Marks);

/**
 * Removes marks from and object
 *
 * @param	Object	Object to remove marks from
 * @param	Marks	Logical OR of OBJECTMARK_'s to remove 
 */
COREUOBJECT_API void UnMarkObject(const class UObjectBase* Object, EObjectMark Marks);

/**
 * Adds marks to an ALL objects
 * Note: Some objects, those not accessible via FObjectIterator, will not be marked
 *
 * @param	Marks	Logical OR of OBJECTMARK_'s to apply 
 */
COREUOBJECT_API void MarkAllObjects(EObjectMark Marks);

/**
 * Removes marks from ALL objects
 * Note: Some objects, those not accessible via FObjectIterator, will not be marked
 *
 * @param	Marks	Logical OR of OBJECTMARK_'s to remove 
 */
COREUOBJECT_API void UnMarkAllObjects(EObjectMark Marks = OBJECTMARK_ALLMARKS);

/**
 * Tests an object for having ANY of a set of marks
 *
 * @param	Object	Object to test marks on
 * @param	Marks	Logical OR of OBJECTMARK_'s to test
 * @return	true if the object has any of the given marks.
 */
COREUOBJECT_API bool ObjectHasAnyMarks(const class UObjectBase* Object, EObjectMark Marks);

/**
 * Tests an object for having ALL of a set of marks
 *
 * @param	Object	Object to test marks on
 * @param	Marks	Logical OR of OBJECTMARK_'s to test
 * @return	true if the object has any of the given marks.
 */
COREUOBJECT_API bool ObjectHasAllMarks(const class UObjectBase* Object, EObjectMark Marks);

/**
 * Returns all of the object marks on a specific object
 *
 * @param	Object	Object to get marks for
 * @return	all Marks for an object
 */
COREUOBJECT_API EObjectMark ObjectGetAllMarks(const class UObjectBase* Object);

/**
 * Build an array of objects having ALL of a set of marks
 *
 * @param	Results		array of objects which have any flag. This array is emptied before we add to it.
 * @param	Marks		Logical OR of OBJECTMARK_'s to test
 */
COREUOBJECT_API void GetObjectsWithAllMarks(TArray<UObject *>& Results, EObjectMark Marks);

/**
* Build an array of objects having ANY of a set of marks
 *
 * @param	Results		array of objects which have any flag. This array is emptied before we add to it.
 * @param	Marks		Logical OR of OBJECTMARK_'s to test
 */
COREUOBJECT_API void GetObjectsWithAnyMarks(TArray<UObject *>& Results, EObjectMark Marks);
