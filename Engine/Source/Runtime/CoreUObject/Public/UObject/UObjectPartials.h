// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

class FLifetimeProperty;

namespace UE::CoreUObject::Private
{

////////////////////////////////////////////////////////////////////////////////////////////////////

// Function pointer types for partial lifecycle
using FPartialConstructorFunc = void(void* PartialMemory);
using FPartialDestructorFunc = void(void* PartialMemory);
using FPartialBeginDestroyFunc = void(void* PartialMemory);
using FPartialGetLifetimeReplicatedPropsFunc = void(void* PartialMemory, TArray<FLifetimeProperty>& OutLifetimeProps);

////////////////////////////////////////////////////////////////////////////////////////////////////

// Partial registration node (intrusive linked list). This is on purpose not in namespace in order to keep codegen files small (less text)
struct FPartialClass
{
	const TCHAR* Name = nullptr;
	int32 Size = 0;
	int32 Alignment = 0;

#if UE_WITH_CONSTINIT_UOBJECT
	UClass* Class = nullptr;
	FProperty* FirstProperty = nullptr;
	UField* FirstChild = nullptr;
	UField* EndChild = nullptr; // This is the first field _after_ the fields owned by this partial
#else
	UClass* (*Class)(ETypeConstructPhase) = nullptr;
	const UECodeGen_Private::FPropertyParamsBase* const* Properties = nullptr;
	int32 NumProperties = 0;

	UFunction* (**FunctionConstructors)(ETypeConstructPhase) = nullptr;  // Array of function construction function pointers (for UFunction creation)
	const void* NativeFunctions = nullptr;  // Array of FClassNativeFunction (for native binding registration)
	int32 NumFunctions = 0;
#endif

	int32* PartialOffsetPtr = nullptr; // Pointer to the native partial struct's static PartialOffset member (for fast access when calling GetPartial<Type>())

	FPartialConstructorFunc* Constructor = nullptr;
	FPartialBeginDestroyFunc* BeginDestroy = nullptr;
	FPartialDestructorFunc* Destructor = nullptr;
	FPartialGetLifetimeReplicatedPropsFunc* GetLifetimeReplicatedProps = nullptr;

	// Runtime-assigned offset from start of partial region (in bytes)
	// This is the offset from NativePropertiesSize to the start of this partial
	mutable int32 PartialOffset = MAX_int32;

	// Intrusive linked list - next partial for the same class
	FPartialClass* NextPartial = nullptr;

	// This call will link in the partial into the class (sorted by alignment/name)
	void LinkToClass(UClass& PartialClass);
};

////////////////////////////////////////////////////////////////////////////////////////////////////

// Run the constructors for partial classes associated with this object/class.
// Does not do anything with properties added by the partials as those are in the list owned by the extended class
void ConstructPartials(const UClass& Class, UObject* Obj);
void BeginDestroyPartials(const UClass& Class, UObject* Obj);
// UNIMPLEMENTED: IsReadyForFinishDestroy/FinishDestroy
void DestructPartials(const UClass& Class, UObject* Obj);
void CallPartialsGetLifetimeReplicatedProps(const UClass& Class, UObject* Obj, TArray<FLifetimeProperty>& OutLifetimeProps);

////////////////////////////////////////////////////////////////////////////////////////////////////

} // UE::CoreUObject::Private