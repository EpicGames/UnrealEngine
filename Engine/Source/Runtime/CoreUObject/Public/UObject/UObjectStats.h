// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "ProfilingDebugging/CsvProfilerConfig.h"
#include <atomic>

#if !defined(CSV_TRACK_UOBJECT_COUNT)
#define CSV_TRACK_UOBJECT_COUNT 0
#endif

#if CSV_PROFILER_STATS && CSV_TRACK_UOBJECT_COUNT
namespace UObjectStats
{
	extern COREUOBJECT_API std::atomic<int32> GUObjectCount;

	UE_FORCEINLINE_HINT void IncrementUObjectCount()
	{
		GUObjectCount.fetch_add(1, std::memory_order_relaxed);
	}

	UE_FORCEINLINE_HINT void DecrementUObjectCount()
	{
		GUObjectCount.fetch_sub(1, std::memory_order_relaxed);
	}

	UE_FORCEINLINE_HINT int32 GetUObjectCount()
	{
		return GUObjectCount.load(std::memory_order_relaxed);
	}
}
#endif
