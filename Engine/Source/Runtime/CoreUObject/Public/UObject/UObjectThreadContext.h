// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UnAsyncLoading.cpp: Unreal async loading code.
=============================================================================*/

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "HAL/ThreadSafeCounter.h"
#include "HAL/ThreadSingleton.h"
#include "Logging/LogCategory.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Templates/RefCounting.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/PropertyPathName.h"
#include "UObject/ObjectMacros.h"

class FLinkerLoad;
class FName;
class FObjectInitializer;
class IAsyncPackageLoader;
class UObject;
struct FUObjectSerializeContext;

COREUOBJECT_API DECLARE_LOG_CATEGORY_EXTERN(LogUObjectThreadContext, Log, All);

UE_DECLARE_THREAD_SINGLETON_TLS(class FUObjectThreadContext, COREUOBJECT_API)

class FUObjectThreadContext : public TThreadSingleton<FUObjectThreadContext>
{
	friend TThreadSingleton<FUObjectThreadContext>;

	COREUOBJECT_API FUObjectThreadContext();
	COREUOBJECT_API virtual ~FUObjectThreadContext();

	/** Stack of currently used FObjectInitializers for this thread */
	TArray<FObjectInitializer*> InitializerStack;

public:
	
	/**
	* Remove top element from the stack.
	*/
	void PopInitializer()
	{
		InitializerStack.Pop(EAllowShrinking::No);
	}

	/**
	* Push new FObjectInitializer on stack.
	* @param	Initializer			Object initializer to push.
	*/
	void PushInitializer(FObjectInitializer* Initializer)
	{
		InitializerStack.Push(Initializer);
	}

	/**
	* Retrieve current FObjectInitializer for current thread.
	* @return Current FObjectInitializer.
	*/
	FObjectInitializer* TopInitializer()
	{
		return InitializerStack.Num() ? InitializerStack.Last() : nullptr;
	}

	/**
	* Retrieves current FObjectInitializer for current thread. Will assert of no ObjectInitializer is currently set.
	* @return Current FObjectInitializer reference.
	*/
	FObjectInitializer& TopInitializerChecked()
	{
		if (FObjectInitializer* ObjectInitializerPtr = TopInitializer())
		{
			return *ObjectInitializerPtr;
		}
		return ReportNull();
	}

	/** true when we are routing ConditionalPostLoad/PostLoad to objects										*/
	bool IsRoutingPostLoad;
	/** true when FLinkerManager deletes linkers */
	bool IsDeletingLinkers;
	/* Global int to track how many nested loads we're doing by triggering an async load and immediately flushing that request. */
	int32 SyncLoadUsingAsyncLoaderCount;
	/* Global flag so that FObjectFinders know if they are called from inside the UObject constructors or not. */
	int32 IsInConstructor;
	/* Object that is currently being constructed with ObjectInitializer */
	UObject* ConstructedObject;
	/** The object we are routing PostLoad from the Async Loading code for */
	UObject* CurrentlyPostLoadedObjectByALT;
	/** Async Package currently processing objects */
	void* AsyncPackage;
	/** Async package loader currently processing objects */
	IAsyncPackageLoader* AsyncPackageLoader;
	/** Async loading visibility filter */
	EInternalObjectFlags AsyncVisibilityFilter = EInternalObjectFlags::None;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Stack to ensure that PostInitProperties is routed through Super:: calls. **/
	TArray<UObject*, TInlineAllocator<16, FNonshrinkingAllocator>> PostInitPropertiesCheck;
	/** Used to verify that the Super::PostLoad chain is intact. */
	TArray<UObject*, TInlineAllocator<16>> DebugPostLoad;
#endif

	/** Gets the current serialization context */
	FUObjectSerializeContext* GetSerializeContext()
	{
		return SerializeContext;
	}

private:
	/** Report that the current ObjectInitializer is null. */
	COREUOBJECT_API FObjectInitializer& ReportNull();

	/** Current serialization context */
	TRefCountPtr<FUObjectSerializeContext> SerializeContext;
};

/** Structure that holds the current serialization state of UObjects */
struct FUObjectSerializeContext
{
	friend class FUObjectThreadContext;

private:

	/** Constructor */
	COREUOBJECT_API FUObjectSerializeContext();

	/** Destructor */
	COREUOBJECT_API ~FUObjectSerializeContext();

	/** Reference count of this context */
	int32 RefCount;

	/** Imports for EndLoad optimization.	*/
	int32 ImportCount;
	/** Forced exports for EndLoad optimization. */
	int32 ForcedExportCount;
	/** Count for BeginLoad multiple loads.	*/
	int32 ObjBeginLoadCount;
	/** Objects that might need preloading. */
	TArray<UObject*> ObjectsLoaded;
	/** List of linkers that we want to close the loaders for (to free file handles) - needs to be delayed until EndLoad is called with GObjBeginLoadCount of 0 */
	TArray<FLinkerLoad*> DelayedLinkerClosePackages;

public:

	/** Points to the main UObject currently being serialized */
	UObject* SerializedObject;
	/** Points to the main PackageLinker currently being serialized (Defined in Linker.cpp) */
	FLinkerLoad* SerializedPackageLinker;
	/** The main Import Index currently being used for serialization by CreateImports() (Defined in Linker.cpp) */
	int32 SerializedImportIndex;
	/** Points to the main Linker currently being used for serialization by CreateImports() (Defined in Linker.cpp) */
	FLinkerLoad* SerializedImportLinker;
	/** The most recently used export Index for serialization by CreateExport() */
	int32 SerializedExportIndex;
	/** Points to the most recently used Linker for serialization by CreateExport() */
	FLinkerLoad* SerializedExportLinker;
#if WITH_EDITORONLY_DATA
	/** The absolute offset of the start of the serialized script properties for the serialized object, or -1. */
	UE_INTERNAL int64 SerializedObjectScriptStartOffset = -1;
	/** The absolute offset of the end of the serialized script properties for the serialized object, or -1. */
	UE_INTERNAL int64 SerializedObjectScriptEndOffset = -1;
	/** Path to the property currently being serialized */
	UE_INTERNAL UE::FPropertyPathName SerializedPropertyPath;
	/** True when SerializedPropertyPath is being tracked during serialization. */
	UE_INTERNAL bool bTrackSerializedPropertyPath;
	/** True when initialized properties will be tracked for the serialized object. */
	UE_INTERNAL bool bTrackInitializedProperties;
	/** True when serialized properties will be tracked for the serialized object. */
	UE_INTERNAL bool bTrackSerializedProperties;
	/** True when unknown properties will be tracked for the serialized object. */
	UE_INTERNAL bool bTrackUnknownProperties;
	/** True when unknown enum names will be tracked for the serialized object. */
	UE_INTERNAL bool bTrackUnknownEnumNames;
	/** True when the SerializedObject properties are being impersonated. */
	UE_INTERNAL bool bImpersonateProperties;
	/** True when serializing to a property bag placeholder object while loading. */
	UE_INTERNAL bool bIsLoadingToPropertyBagObject;
	/** True when serializing to an unknown property tree for determining property type info. The loaded value doesn't need to be correct so we can skip
		loading values in certain cases for the sake of bypassing issues with incomplete info or for the sake of optimization */
	UE_INTERNAL bool bIsLoadingToUnknownPropertyTree;
#endif

	/** Adds a new loaded object */
	COREUOBJECT_API void AddLoadedObject(UObject* InObject);
	COREUOBJECT_API void AddUniqueLoadedObjects(const TArray<UObject*>& InObjects);

	/** Checks if object loading has started */
	bool HasStartedLoading() const
	{
		return ObjBeginLoadCount > 0;
	}
	int32 GetBeginLoadCount() const
	{
		return ObjBeginLoadCount;
	}

	COREUOBJECT_API int32 IncrementBeginLoadCount();
	COREUOBJECT_API int32 DecrementBeginLoadCount();

	int32 IncrementImportCount()
	{
		return ++ImportCount;
	}
	void ResetImportCount()
	{
		ImportCount = 0;
	}

	int32 IncrementForcedExportCount()
	{
		return ++ForcedExportCount;
	}
	void ResetForcedExports()
	{
		ForcedExportCount = 0;
	}

	bool HasPendingImportsOrForcedExports() const
	{
		return ImportCount || ForcedExportCount;
	}

	bool HasLoadedObjects() const
	{
		return !!ObjectsLoaded.Num();
	}

	COREUOBJECT_API bool PRIVATE_PatchNewObjectIntoExport(UObject* OldObject, UObject* NewObject);

	/** This is only meant to be used by FAsyncPackage for performance reasons. The ObjectsLoaded array should not be manipulated directly! */
	TArray<UObject*>& PRIVATE_GetObjectsLoadedInternalUseOnly()
	{
		return ObjectsLoaded;
	}

	void AppendLoadedObjectsAndEmpty(TArray<UObject*>& InLoadedObject)
	{
		InLoadedObject.Append(ObjectsLoaded);
		ObjectsLoaded.Reset();
	}

	void ReserveObjectsLoaded(int32 InReserveSize)
	{
		ObjectsLoaded.Reserve(InReserveSize);
	}

	int32 GetNumObjectsLoaded() const
	{
		return ObjectsLoaded.Num();
	}

	void AddDelayedLinkerClosePackage(class FLinkerLoad* InLinker)
	{
		DelayedLinkerClosePackages.AddUnique(InLinker);
	}

	void RemoveDelayedLinkerClosePackage(class FLinkerLoad* InLinker)
	{
		DelayedLinkerClosePackages.Remove(InLinker);
	}

	void MoveDelayedLinkerClosePackages(TArray<class FLinkerLoad*>& OutDelayedLinkerClosePackages)
	{
		OutDelayedLinkerClosePackages = MoveTemp(DelayedLinkerClosePackages);
	}


	//~ TRefCountPtr interface
	int32 AddRef()
	{
		return ++RefCount;
	}
	int32 Release()
	{
		int32 CurrentRefCount = --RefCount;
		check(CurrentRefCount >= 0);
		if (CurrentRefCount == 0)
		{
			delete this;
		}
		return CurrentRefCount;
	}
	int32 GetRefCount() const
	{
		return RefCount;
	}
	//~ TRefCountPtr interface
};
